// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Particles/ParticlePerfStats.h"
#include "PrimitiveSceneProxy.h"
#include "HeterogeneousVolumeInterface.h"
#include "PrimitiveSceneProxyDesc.h"

class FNiagaraGpuComputeDispatchInterface;
class FNiagaraSystemRenderData;
class UNiagaraComponent;
class UNiagaraSystem;
class FNiagaraSystemInstanceController;
enum class ENiagaraOcclusionQueryMode : uint8;
using FNiagaraSystemInstanceControllerConstPtr = TSharedPtr<const FNiagaraSystemInstanceController, ESPMode::ThreadSafe>;

#define NIAGARAPROXY_EVENTS_ENABLED	(!STATS && (ENABLE_STATNAMEDEVENTS || CPUPROFILERTRACE_ENABLED))

struct FNiagaraSceneProxyDesc : public FPrimitiveSceneProxyDesc
{
	NIAGARA_API FNiagaraSceneProxyDesc();
	NIAGARA_API FNiagaraSceneProxyDesc(UNiagaraComponent& InComponent);
	NIAGARA_API ~FNiagaraSceneProxyDesc();

	UNiagaraSystem* SystemAsset;
	ENiagaraOcclusionQueryMode OcclusionQueryMode;
	FNiagaraSystemInstanceControllerConstPtr SystemInstanceController;
};

/**
* Scene proxy for drawing niagara particle simulations.
*/
class FNiagaraSceneProxy : public FPrimitiveSceneProxy
{
public:
	NIAGARA_API virtual SIZE_T GetTypeHash() const override;

	NIAGARA_API FNiagaraSceneProxy(const FNiagaraSceneProxyDesc& Desc);
	NIAGARA_API ~FNiagaraSceneProxy();

	/** Retrieves the render data for a single system */
	FNiagaraSystemRenderData* GetSystemRenderData() { return RenderData; }

	/** Called to allow renderers to free render state */
	NIAGARA_API void DestroyRenderState_Concurrent();

	/** Sets whether or not this scene proxy should be rendered. */
	NIAGARA_API void SetRenderingEnabled_GT(bool bInRenderingEnabled);

	FNiagaraGpuComputeDispatchInterface* GetComputeDispatchInterface() const { return ComputeDispatchInterface; }

#if RHI_RAYTRACING
	NIAGARA_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override;
	virtual bool IsRayTracingRelevant() const override { return true; }
	virtual bool HasRayTracingRepresentation() const override { return true; }
#endif

	inline const FMatrix& GetLocalToWorldInverse() const { return LocalToWorldInverse; }

	const FVector3f& GetLWCRenderTile() const { return DynamicData.LWCRenderTile; }

	NIAGARA_API TUniformBuffer<FPrimitiveUniformShaderParameters>* GetCustomUniformBufferResource(FRHICommandListBase& RHICmdList, bool bHasVelocity, const FBox& InstanceBounds = FBox(ForceInitToZero)) const;

	NIAGARA_API FRHIUniformBuffer* GetCustomUniformBuffer(FRHICommandListBase& RHICmdList, bool bHasVelocity, const FBox& InstanceBounds = FBox(ForceInitToZero)) const;

	NIAGARA_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

	/** Some proxy wide dynamic settings passed down with the emitter dynamic data. */
	struct FDynamicData
	{
		FVector3f	LWCRenderTile = FVector3f::ZeroVector;
		bool		bUseCullProxy = false;
		float		LODDistanceOverride = -1.0f;
#if WITH_PARTICLE_PERF_STATS
		FParticlePerfStatsContext PerfStatsContext;
#endif
	};
	const FDynamicData& GetProxyDynamicData()const { return DynamicData; }
	void SetProxyDynamicData(const FDynamicData& NewData) { DynamicData = NewData; }

	UE_DEPRECATED(5.6, "Use FNiagaraSceneProxyDesc ctor instead")
	inline FNiagaraSceneProxy(UNiagaraComponent* InComponent) : FNiagaraSceneProxy(FNiagaraSceneProxyDesc(*InComponent)) {}

private:
	NIAGARA_API void ReleaseRenderThreadResources();

	NIAGARA_API void ReleaseUniformBuffers(bool bEmpty);

	//~ Begin FPrimitiveSceneProxy Interface.
	NIAGARA_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;

	//virtual void OnActorPositionChanged() override;
	NIAGARA_API virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override;

	NIAGARA_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	NIAGARA_API virtual bool CanBeOccluded() const override;
	/** Callback from the renderer to gather simple lights that this proxy wants renderered. */
	NIAGARA_API virtual void GatherSimpleLights(const FSceneViewFamily& ViewFamily, FSimpleLightArray& OutParticleLights) const override;

	NIAGARA_API virtual uint32 GetMemoryFootprint() const override;

	NIAGARA_API uint32 GetAllocatedSize() const;

private:
	/** Custom Uniform Buffers, allows us to have renderer specific data packed inside such as pre-skinned bounds. */
	mutable UE::FMutex CustomUniformBuffersGuard;
	mutable TMap<uint32, TUniformBuffer<FPrimitiveUniformShaderParameters>*> CustomUniformBuffers;

	/** The data required to render a single instance of a NiagaraSystem */
	FNiagaraSystemRenderData* RenderData = nullptr;

	FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface = nullptr;

	FMatrix LocalToWorldInverse;

	TStatId SystemStatID;
#if NIAGARAPROXY_EVENTS_ENABLED
	FString SystemStatString;
#endif

	FDynamicData DynamicData;

	ENiagaraOcclusionQueryMode OcclusionQueryMode;
};

