// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraParameterStore.h"
#include "NiagaraScriptExecutionParameterStore.generated.h"

struct FNiagaraScriptRuntimeCompiledData;
class  FNiagaraScriptRuntimeData;

USTRUCT()
struct FNiagaraScriptExecutionPaddingInfo
{
	GENERATED_USTRUCT_BODY()
public:
	FNiagaraScriptExecutionPaddingInfo() : SrcOffset(0), DestOffset(0), SrcSize(0), DestSize(0) {}
	FNiagaraScriptExecutionPaddingInfo(uint32 InSrcOffset, uint32 InDestOffset, uint32 InSrcSize, uint32 InDestSize)
		: SrcOffset(uint16(InSrcOffset)), DestOffset(uint16(InDestOffset)), SrcSize(uint16(InSrcSize)), DestSize(uint16(InDestSize))
	{
		ensure(InSrcOffset <= TNumericLimits<uint16>::Max());
		ensure(InDestOffset <= TNumericLimits<uint16>::Max());
		ensure(InSrcSize <= TNumericLimits<uint16>::Max());
		ensure(InDestSize <= TNumericLimits<uint16>::Max());
	}

	UPROPERTY()
	uint16 SrcOffset;
	UPROPERTY()
	uint16 DestOffset;
	UPROPERTY()
	uint16 SrcSize;
	UPROPERTY()
	uint16 DestSize;
};

/**
Storage class containing actual runtime buffers to be used by the VM and the GPU.
Is not the actual source for any parameter data, rather just the final place it's gathered from various other places ready for execution.
*/
USTRUCT()
struct FNiagaraScriptExecutionParameterStore : public FNiagaraParameterStore
{
	GENERATED_USTRUCT_BODY()
public:
	FNiagaraScriptExecutionParameterStore();
	FNiagaraScriptExecutionParameterStore(const FNiagaraParameterStore& Other);
	FNiagaraScriptExecutionParameterStore& operator=(const FNiagaraParameterStore& Other);

	virtual ~FNiagaraScriptExecutionParameterStore() = default;

	//TODO: These function can probably go away entirely when we replace the FNiagaraParameters and DataInterface info in the script with an FNiagaraParameterStore.
	//Special care with prev params and internal params will have to be taken there.
#if WITH_EDITORONLY_DATA
	/** Call this init function if you are using a Niagara parameter store within a UNiagaraScript.*/
	void InitFromOwningScript(UNiagaraScript* Script, const FNiagaraScriptRuntimeCompiledData* ScriptRuntimeCompiledData, bool bNotifyAsDirty);
	void AddScriptParams(UNiagaraScript* Script, const FNiagaraScriptRuntimeCompiledData* ScriptRuntimeData);
#endif

	virtual bool AddParameter(const FNiagaraVariable& Param, bool bInitInterfaces = true, bool bTriggerRebind = true, int32* OutOffset = nullptr) override
	{
#if WITH_EDITORONLY_DATA
		int32 NewParamOffset = INDEX_NONE;
		bool bAdded;
		if (FNiagaraTypeHelper::IsLWCType(Param.GetType()))
		{
			// Custom structs containing LWC type data are converted into SWC structs for the runtime 
			UScriptStruct* Struct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(Param.GetType().GetScriptStruct(), ENiagaraStructConversion::Simulation);
			FNiagaraVariable SimParam(FNiagaraTypeDefinition(Struct), Param.GetName());
			if (Param.IsDataAllocated())
			{
				SimParam.AllocateData();
				FNiagaraTypeRegistry::GetStructConverter(Param.GetType()).ConvertDataToSimulation(SimParam.GetData(), Param.GetData());
			}
			bAdded = FNiagaraParameterStore::AddParameter(SimParam, bInitInterfaces, bTriggerRebind, &NewParamOffset);
		}
		else
		{
			bAdded = FNiagaraParameterStore::AddParameter(Param, bInitInterfaces, bTriggerRebind, &NewParamOffset);
		}
		if (OutOffset)
		{
			*OutOffset = NewParamOffset;
		}
		return bAdded;
#else
		check(0);
		return false;
#endif
	}

	virtual bool RemoveParameter(const FNiagaraVariableBase& Param) override
	{
		check(0);//Not allowed to remove parameters from an execution store as it will adjust the table layout mess up the 
		return false;
	}

	virtual void RenameParameter(const FNiagaraVariableBase& Param, FName NewName) override
	{
		check(0);//Can't rename parameters for an execution store.
	}

	virtual void Empty(bool bClearBindings = true) override
	{
		FNiagaraParameterStore::Empty(bClearBindings);
		ParameterSize = 0;
		bInitialized = false;
	}

	virtual void Reset(bool bClearBindings = true) override
	{
		FNiagaraParameterStore::Reset(bClearBindings);
		ParameterSize = 0;
		bInitialized = false;
	}

	/** Size of the parameter data not including prev frame values or internal constants. Allows copying into previous parameter values for interpolated spawn scripts. */
	UPROPERTY()
	int32 ParameterSize;

	UPROPERTY()
	uint8 bInitialized : 1;

#if WITH_EDITORONLY_DATA
	TArray<uint8> CachedScriptLiterals;
#endif

	SIZE_T GetResourceSize() const
	{
		SIZE_T ResourceSize = sizeof(FNiagaraScriptExecutionParameterStore) - sizeof(FNiagaraParameterStore);

		ResourceSize += FNiagaraParameterStore::GetResourceSize();
		return ResourceSize;
	}

private:

#if WITH_EDITORONLY_DATA
	static uint32 GenerateLayoutInfoInternal(TArray<FNiagaraScriptExecutionPaddingInfo>& Members, uint32& NextMemberOffset, const UStruct* InSrcStruct, uint32 InSrcOffset);
#endif
};

//////////////////////////////////////////////////////////////////////////
/// FNiagaraScriptInstanceParameterStore
//////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FNiagaraScriptInstanceParameterStore : public FNiagaraParameterStore
{
	GENERATED_USTRUCT_BODY()

public:
	FNiagaraScriptInstanceParameterStore();
	virtual ~FNiagaraScriptInstanceParameterStore() = default;

	/** Call this init function if you are using a Niagara parameter store within an FNiagaraScriptExecutionContext.*/
	void InitFromOwningContext(UNiagaraScript* Script, const FNiagaraScriptRuntimeData* InScriptRuntimeData, bool bNotifyAsDirty);
	void CopyCurrToPrev();

	virtual bool AddParameter(const FNiagaraVariable& Param, bool bInitInterfaces = true, bool bTriggerRebind = true, int32* OutOffset = nullptr) override
	{
		check(0);
		return false;
	}

	virtual bool RemoveParameter(const FNiagaraVariableBase& Param) override
	{
		check(0);//Not allowed to remove parameters from an execution store as it will adjust the table layout mess up the 
		return false;
	}

	virtual void RenameParameter(const FNiagaraVariableBase& Param, FName NewName) override
	{
		check(0);//Can't rename parameters for an execution store.
	}

	virtual void Empty(bool bClearBindings = true) override
	{
		FNiagaraParameterStore::Empty(bClearBindings);
		ScriptParameterStore.Reset();
		bInitialized = false;
	}

	virtual void Reset(bool bClearBindings = true) override
	{
		FNiagaraParameterStore::Reset(bClearBindings);
		ScriptParameterStore.Reset();
		bInitialized = false;
	}

	// Just the external parameters, not previous or internal...
	uint32 GetExternalParameterSize() const;

#if WITH_EDITORONLY_DATA
	TArrayView<const uint8> GetScriptLiterals() const;
#endif

	virtual NIAGARA_API TArrayView<const FNiagaraVariableWithOffset> ReadParameterVariables() const override;

private:
	FNiagaraCompiledDataReference<FNiagaraScriptExecutionParameterStore> ScriptParameterStore;

	uint8 bInitialized : 1;
};
