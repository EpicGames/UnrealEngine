// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraTypes.h"
#include "UObject/ObjectPtr.h"

#include "NiagaraScriptRuntimeCompiledData.generated.h"

struct FNiagaraVMExecutableData;

/** Represents per script runtime data which is generated at compile time. */
USTRUCT()
struct FNiagaraScriptRuntimeCompiledData
{
	GENERATED_BODY();

	UPROPERTY()
	TArray<FNiagaraScriptResolvedDataInterfaceInfo> ResolvedDataInterfaces;

	UPROPERTY()
	TArray<FNiagaraResolvedUserDataInterfaceBinding> ResolvedUserDataInterfaceBindings;

	UPROPERTY()
	TArray<FNiagaraResolvedUObjectInfo> ResolvedUObjects;

	UPROPERTY()
	ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim;

#if WITH_EDITORONLY_DATA
	bool ValidateWithScript(UNiagaraScript& InScript) const;
#endif
};

/** An object which holds a mapping of script data key to script runtime compiled data. This object is for editor use only and allows for multiple instances of
	this data to exist and facilitates referencing the data by pointer and managing the lifetime of this data across compiles. */
UCLASS()
class UNiagaraScriptRuntimeCompiledDataEditorReference : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TMap<FNiagaraScriptDataKey, FNiagaraScriptRuntimeCompiledData> ScriptRuntimeCompiledDataMap;
};

/** A struct which holds a pointer to script runtime compiled data, and at editor time also tracks source information for
	lifetime tracking in the system.  This struct is used to facilitate referencing the data by pointer, but allowing the data
	to be sourced from a UObject at editor time, or a serialized struct in cooked data. */
struct FNiagaraScriptRuntimeCompiledDataHandle
{
#if WITH_EDITOR
	FNiagaraScriptRuntimeCompiledDataHandle(const FObjectKey& InOwnerEditorReferenceKey,
		const FNiagaraScriptDataKey& InOwnerScriptDataKey, const FNiagaraScriptRuntimeCompiledData& InInstance)
		: OwnerEditorReferenceKey(InOwnerEditorReferenceKey)
		, OwnerScriptDataKey(InOwnerScriptDataKey)
		, Instance(&InInstance)
	{
	}
#endif

	FNiagaraScriptRuntimeCompiledDataHandle(const FNiagaraScriptRuntimeCompiledData& InInstance)
		: Instance(&InInstance)
	{
	}

#if WITH_EDITOR
	FObjectKey OwnerEditorReferenceKey;
	FNiagaraScriptDataKey OwnerScriptDataKey;
#endif

	const FNiagaraScriptRuntimeCompiledData* Instance = nullptr;
};