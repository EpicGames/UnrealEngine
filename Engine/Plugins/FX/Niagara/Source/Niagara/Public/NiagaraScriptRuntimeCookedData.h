// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraParameterStore.h"
#include "NiagaraScriptExecutionParameterStore.h"
#include "NiagaraScriptRuntimeCompiledData.h"

#include "NiagaraScriptRuntimeCookedData.generated.h"

USTRUCT()
struct FNiagaraScriptRuntimeCookedData
{
	GENERATED_BODY();

	UPROPERTY()
	FNiagaraScriptRuntimeCompiledData CookedScriptRuntimeCompiledData;

	/** The equivalent of ScriptExecutionParamStoreCPU (or GPU) cooked for the given platform.*/
	UPROPERTY()
	FNiagaraScriptExecutionParameterStore CookedExecutionParameterStore;

	/** The cooked binding data between ScriptExecutionParamStore and RapidIterationParameters.*/
	UPROPERTY()
	TArray<FNiagaraBoundParameter> CookedRapidIterationBindings;

#if WITH_EDITORONLY_DATA
	void Init(UNiagaraScript& InScript, const FNiagaraScriptRuntimeCompiledData& InScriptRuntimeCompiledData);
#endif //WITH_EDITORONLY_DATA

	bool ValidateWithScript(UNiagaraScript& InScript) const;
};