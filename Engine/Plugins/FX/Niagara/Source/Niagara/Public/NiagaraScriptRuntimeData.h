// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorVM.h"

#include "NiagaraScriptRuntimeCompiledData.h"
#include "NiagaraScriptRuntimeTransientData.h"

class FNiagaraScriptRuntimeData
{
public:
	FNiagaraScriptRuntimeData(
		TSharedRef<const FNiagaraScriptRuntimeCompiledDataHandle> InCompiledDataHandle,
		TSharedRef<const FNiagaraScriptRuntimeTransientData> InTransientData)
		: CompiledDataHandle(InCompiledDataHandle)
		, TransientData(InTransientData)
	{
	}

	TConstArrayView<FNiagaraScriptResolvedDataInterfaceInfo> GetResolvedDataInterfaces() const { return MakeConstArrayView(CompiledDataHandle->Instance->ResolvedDataInterfaces); }

	TConstArrayView<FNiagaraResolvedUserDataInterfaceBinding> GetResolvedUserDataInterfaceBindings() const { return MakeConstArrayView(CompiledDataHandle->Instance->ResolvedUserDataInterfaceBindings); }

	TConstArrayView<FNiagaraResolvedUObjectInfo> GetResolvedUObjects() const { return MakeConstArrayView(CompiledDataHandle->Instance->ResolvedUObjects); }

	const FNiagaraScriptExecutionParameterStore* GetExecutionReadyParameterStore() const { return TransientData->GetExecutionReadyParameterStore(); } 

	TConstArrayView<FVMExternalFunction> GetCalledVMExternalFunctionBindings() const { return TransientData->GetCalledVMExternalFunctionBindings(); }

private:
	TSharedPtr<const FNiagaraScriptRuntimeCompiledDataHandle> CompiledDataHandle;
	TSharedPtr<const FNiagaraScriptRuntimeTransientData> TransientData;
};

class FNiagaraScriptComputeRuntimeData : public FNiagaraScriptRuntimeData
{

};

class FNiagaraScriptEventRuntimeData : public FNiagaraScriptRuntimeData
{

};