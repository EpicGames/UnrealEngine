// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraScriptExecutionParameterStore.h"
#include "VectorVM.h"

struct FNiagaraScriptRuntimeCompiledData;

class FNiagaraScriptRuntimeTransientData
{
public:

#if WITH_EDITOR
	FNiagaraScriptRuntimeTransientData(UNiagaraScript& Script, const FNiagaraScriptRuntimeCompiledData& ScriptRuntimeCompiledData);
#endif
		
	FNiagaraScriptRuntimeTransientData(UNiagaraScript& Script, const FNiagaraScriptRuntimeCompiledData& ScriptRuntimeCompiledData, FNiagaraScriptExecutionParameterStore& CookedExecutionReadyParameterStore);

	~FNiagaraScriptRuntimeTransientData();

	const FNiagaraScriptExecutionParameterStore* GetExecutionReadyParameterStore() const { return ExecutionReadyParameterStore; }
	TConstArrayView<FVMExternalFunction> GetCalledVMExternalFunctionBindings() const { return MakeConstArrayView(CalledVMExternalFunctionBindings); }

private:
	void GenerateDefaultFunctionBindings(const UNiagaraScript& Script, const TArray<FNiagaraScriptResolvedDataInterfaceInfo>& ResolvedDataInterfaces);

private:
	FNiagaraScriptExecutionParameterStore* ExecutionReadyParameterStore;
	bool bOwnsParameterStore;

	TArray<FVMExternalFunction> CalledVMExternalFunctionBindings;
};