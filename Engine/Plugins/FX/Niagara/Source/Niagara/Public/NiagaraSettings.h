// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "NiagaraScript.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/TextureRenderTarget2D.h"
#include "InputCoreTypes.h"
#include "NiagaraPlatformSet.h"
#include "NiagaraSettings.generated.h"

// This enum must match the order in NiagaraDataInterfaceSkeletalMesh.ush
UENUM()
namespace ENDISkelMesh_GpuMaxInfluences
{
	enum Type : int
	{
		/** Allow up to 4 bones to be sampled. */
		AllowMax4 = 0,
		/** Allow up to 8 bones to be sampled. */
		AllowMax8 = 1,
		/** Allow an unlimited amount of bones to be sampled. */
		Unlimited = 2,
	};
}

// This enum must match the order in NiagaraDataInterfaceSkeletalMesh.ush
UENUM()
namespace ENDISkelMesh_GpuUniformSamplingFormat
{
	enum Type : int
	{
		/** 64 bits per entry. Allow for the full int32 range of triangles (2 billion). */
		Full = 0,
		/** 32 bits per entry. Allow for ~16.7 million triangles and 8 bits of probability precision. */
		Limited_24_8 = 1,
		/** 32 bits per entry. Allow for ~8.4 millions triangles and 9 bits of probability precision. */
		Limited_23_9 = 2,
	};
}

// This enum must match the order in NiagaraDataInterfaceSkeletalMesh.ush
UENUM()
namespace ENDISkelMesh_AdjacencyTriangleIndexFormat
{
	enum Type : int
	{
		/** 32 bits per entry. Allow for the full int32 range of triangles (2 billion). */
		Full = 0,
		/** 16 bits per entry. Allow for half (int16) range of triangles (64k). */
		Half = 1,
	};
}

UENUM()
enum class ENiagaraDefaultRendererPixelCoverageMode : uint8
{
	/** When renderer is set to automatic mode pixel coverage is enabled. */
	Enabled,
	/** When renderer is set to automatic mode pixel coverage is disabled. */
	Disabled,
};

UENUM()
enum class ENiagaraDefaultSortPrecision : uint8
{
	/** Low precision sorting, half float (fp16) precision, faster and adequate for most cases. */
	Low,
	/** High precision sorting, float (fp32) precision, slower but may fix sorting artifacts. */
	High,
};

UENUM()
enum class ENiagaraDefaultGpuTranslucentLatency : uint8
{
	/** Gpu simulations will always read this frames data for translucent materials. */
	Immediate,
	/** Gpu simulations will read the previous frames data if the simulation has to run in PostRenderOpaque. */
	Latent,
};

UENUM()
enum class ENiagaraCompileErrorSeverity : uint8
{
	Ignore, LogOnly, Warning, Error
};

UENUM()
namespace ENDICollisionQuery_AsyncGpuTraceProvider
{
	enum Type : int
	{
		Default = 0 UMETA(DisplayName = "Project Default"),
		HWRT = 1 UMETA(DisplayName = "HW Ray Tracing"),
		GSDF = 2 UMETA(DisplayName = "Global Signed Distance Fields"),
		None = 3 UMETA(DisplayName = "Disabled"),
	};
}

UENUM()
enum class ENiagaraStripScriptByteCodeOption : uint8
{
	Default = 0 UMETA(DisplayName = "No Stripping"),
	Strip_Original = 1 UMETA(DisplayName = "Strip Original ByteCode"),
	Strip_Experimental = 2 UMETA(DisplayName = "Strip Optimized ByteCode"),
};

#if WITH_EDITORONLY_DATA
UENUM()
enum class ENiagaraCompilationMode : int32
{
	Original = 0	UMETA(DisplayName = "Serial Compilation (Deprecated)"),
	AsyncTasks = 1	UMETA(DisplayName = "Async System Compilation"),
	Verify = 2      UMETA(Hidden)
};
#endif

UCLASS(config = Niagara, defaultconfig, meta=(DisplayName="Niagara"), MinimalAPI)
class UNiagaraSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = (AllowedClasses = "/Script/CoreUObject.ScriptStruct"))
	TArray<FSoftObjectPath> AdditionalParameterTypes;

	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = (AllowedClasses = "/Script/CoreUObject.ScriptStruct"))
	TArray<FSoftObjectPath> AdditionalPayloadTypes;

	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = (AllowedClasses = "/Script/CoreUObject.Enum"))
	TArray<FSoftObjectPath> AdditionalParameterEnums;

	/** Sets the default navigation behavior for the system preview viewport. */
	UPROPERTY(config, EditAnywhere, Category = Viewport)
	bool bSystemViewportInOrbitMode = true;

	/** If true then the "link input" menu will also show variables of different types, as long as there is a conversion script for them. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	bool bShowConvertibleInputsInStack = false;

	/** The number of frames to capture when capturing a Sim Cache from the Niagara Component Details Panel. **/
	UPROPERTY(Config, EditAnywhere, Category = SimulationCaching, meta = (UIMin = 1))
	int32 QuickSimCacheCaptureFrameCount = 5;
#endif // WITH_EDITORONLY_DATA

	/** If true then active effects rebase the simulation positions to not lose precision. Can be turned off if not needed to skip unnecessary rebasing calculations. */
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = ( ConfigRestartRequired = true ))
	bool bSystemsSupportLargeWorldCoordinates = true;

	/** Controls how we respond to a system instance crossing large world coordinate tiles. Can be overriden on individual assets. */
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = (EditCondition = "bSystemsSupportLargeWorldCoordinates", EditConditionHides))
	ENiagaraLwcTileUpdateMode LargeWorldCoordinateTileUpdateMode = ENiagaraLwcTileUpdateMode::Rebase;

	/** The maximum tiles we can cross before forcing a simulation to reset, if the tile update mode is set to RebaseOrResetSimulation. */
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = (EditCondition = "bSystemsSupportLargeWorldCoordinates", EditConditionHides))
	uint32 LargeWorldCoordinateMaxTilesBeforeReset = 4;

	/**
	 If set to true, types like positions and vectors cannot be assigned to each other without an explicit conversion step.
	 If false, type checks are loosened and some types can be implicitly converted into each other.
	 It is recommended to not disable this when working with large world coordinates. */
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = ( DisplayName = "Enforce strict type checks in the graph" ))
	bool bEnforceStrictStackTypes = true;

	/** If set to true, quaternion attributes will be interpolated via slerp instead of lerp in interpolated spawn scripts. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = Niagara)
	bool bAccurateQuatInterpolation = true;

	/** If the Niagara compiler sees that a script writes to a namespace that is read only (e.g. a particle script writing to a system attribute), what should it do. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = Niagara)
	ENiagaraCompileErrorSeverity InvalidNamespaceWriteSeverity = ENiagaraCompileErrorSeverity::Warning;

	/** Whether to limit the max tick delta time or not. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Niagara", meta = (InlineEditConditionToggle))
	bool bLimitDeltaTime = true;

	/** Limits the delta time per tick for emitters to prevent simulation spikes due to frame lags. */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Niagara", meta = (EditCondition = "bLimitDeltaTime", ForceUnits=s))
	float MaxDeltaTimePerTick = 0.125;
	
	/** Default effect type to use for effects that don't define their own. Can be null. */
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = (AllowedClasses = "/Script/Niagara.NiagaraEffectType"))
	FSoftObjectPath DefaultEffectType;

	/** Should we allow placing a Niagara System in the editor into a level which has no effect type assigned? */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	bool bAllowCreateActorFromSystemWithNoEffectType = true;

	/** Position pin type color. The other pin colors are defined in the general editor settings. */
	UPROPERTY(config, EditAnywhere, Category=Niagara)
	FLinearColor PositionPinTypeColor;

	/**
	 Controls how byte code will be stripped when loading assets that have multiple sets of bytecode (i.e. optimized).
	 */
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = (DisplayName = "Option for how to strip bytecode"))
	ENiagaraStripScriptByteCodeOption ByteCodeStripOption = ENiagaraStripScriptByteCodeOption::Strip_Original;

#if WITH_EDITORONLY_DATA
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = (DisplayName = "Option for how to compile Niagara scripts"))
	ENiagaraCompilationMode CompilationMode = ENiagaraCompilationMode::AsyncTasks;

	/**
	Enable HLSL 2021 compilation flag.
	Note: Only GPU simulations on some platforms will work with HLSL 2021.
	*/
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta = (ConfigRestartRequired = true, ConsoleVariable="fx.Niagara.Shader.EnableHLSL2021"))
	bool bEnableHLSL2021 = false;
#endif

	/** The quality levels Niagara uses. */
	UPROPERTY(config, EditAnywhere, Category = Scalability)
	TArray<FText> QualityLevels;

	/** Info texts that the component renderer shows the user depending on the selected component class. */
	UPROPERTY(config, EditAnywhere, Category = Renderer)
	TMap<FString, FText> ComponentRendererWarningsPerClass;

	/** The default render target format used by all Niagara Render Target Data Interfaces unless overridden. */
	UPROPERTY(config, EditAnywhere, Category = Renderer)
	TEnumAsByte<ETextureRenderTargetFormat> DefaultRenderTargetFormat = RTF_RGBA16f;

	/** The default buffer format used by all Niagara Grid Data Interfaces unless overridden. */
	UPROPERTY(config, EditAnywhere, Category = Renderer)
	ENiagaraGpuBufferFormat DefaultGridFormat = ENiagaraGpuBufferFormat::HalfFloat;

	/** The default setting for motion vectors in Niagara renderers */
	UPROPERTY(config, EditAnywhere, Category = Renderer)
	ENiagaraDefaultRendererMotionVectorSetting DefaultRendererMotionVectorSetting = ENiagaraDefaultRendererMotionVectorSetting::Precise;

	/** The default setting for pixel coverage mode when automatic is set on the Niagara Renderer. */
	UPROPERTY(config, EditAnywhere, Category = Renderer)
	ENiagaraDefaultRendererPixelCoverageMode DefaultPixelCoverageMode = ENiagaraDefaultRendererPixelCoverageMode::Enabled;

	/** The default setting for sorting precision when automatic is set on the Niagara Renderer. */
	UPROPERTY(config, EditAnywhere, Category = Renderer)
	ENiagaraDefaultSortPrecision DefaultSortPrecision = ENiagaraDefaultSortPrecision::Low;

	/** The default setting for Gpu simulation translucent draw latency. */
	UPROPERTY(config, EditAnywhere, Category = Renderer)
	ENiagaraDefaultGpuTranslucentLatency DefaultGpuTranslucentLatency = ENiagaraDefaultGpuTranslucentLatency::Immediate;

	/** The default InverseExposureBlend used for the light renderer. */
	UPROPERTY(config, EditAnywhere, Category = LightRenderer)
	float DefaultLightInverseExposureBlend = 0.0f;

	/* 
	When enabled we will read deformed geometry if available, i.e. data from the deformed graph / skin cache
	When disable we will only read from the default vertex data which does not include morph targets, skin, etc.
	Changing this setting requires restarting the editor.
	Note: Enabling this does add additional branches to the skel mesh data reading.
	*/
	UPROPERTY(config, EditAnywhere, Category = SkeletalMeshDI, meta = (DisplayName = "Support Reading Deformed Geometry", ConfigRestartRequired = true))
	bool NDISkelMesh_SupportReadingDeformedGeometry = true;

	UPROPERTY(config, EditAnywhere, Category = SkeletalMeshDI, meta = (DisplayName = "Support 16 bit index & weights", ToolTip = "Enabled support for 16 bit bone index & bone weight, optional to reduce shader complexity.  Changing this setting requires restarting the editor.", ConfigRestartRequired = true))
	bool NDISkelMesh_Support16BitIndexWeight = true;

	UPROPERTY(config, EditAnywhere, Category=SkeletalMeshDI, meta = ( DisplayName = "Gpu Max Bone Influences", ToolTip = "Controls the maximum number of influences we allow the Skeletal Mesh Data Interface to use on the GPU.  Changing this setting requires restarting the editor.", ConfigRestartRequired = true))
	TEnumAsByte<ENDISkelMesh_GpuMaxInfluences::Type> NDISkelMesh_GpuMaxInfluences;

	UPROPERTY(config, EditAnywhere, Category = SkeletalMeshDI, meta = (DisplayName = "Gpu Uniform Sampling Format", ToolTip = "Controls the format used for uniform sampling on the GPU.  Changing this setting requires restarting the editor.", ConfigRestartRequired = true))
	TEnumAsByte<ENDISkelMesh_GpuUniformSamplingFormat::Type> NDISkelMesh_GpuUniformSamplingFormat;

	UPROPERTY(config, EditAnywhere, Category = SkeletalMeshDI, meta = (DisplayName = "Adjacency Triangle Index Format", ToolTip = "Controls the format used for specifying triangle indexes in adjacency buffers.  Changing this setting requires restarting the editor.", ConfigRestartRequired = true))
	TEnumAsByte<ENDISkelMesh_AdjacencyTriangleIndexFormat::Type> NDISkelMesh_AdjacencyTriangleIndexFormat;

	/**
	When enabled the static mesh data interface is allowed to sample from the distance field data (if present) on the GPU.
	Enabling this feature will move all systems that contain static mesh samples into PostRenderOpaque tick group regardless of the features used.
	Changing this setting requires restarting the editor.
	*/
	UPROPERTY(config, EditAnywhere, Category = StaticMeshDI, meta = (DisplayName = "Allow Distance Fields (Experimental)", ConfigRestartRequired = true))
	bool NDIStaticMesh_AllowDistanceFields = false;

	/** 
	Defines how traces tagged as 'Project Default' will be interpreted when using the AsyncGpuTrace data interface.
	The system will go through (starting at element 0) to find the first provider that is available.
	*/
	UPROPERTY(config, EditAnywhere, Category = AsyncGpuTraceDI, meta = (DisplayName = "Trace Provider Priorities (Experimental)", ConfigRestartRequired = true))
	TArray<TEnumAsByte<ENDICollisionQuery_AsyncGpuTraceProvider::Type>> NDICollisionQuery_AsyncGpuTraceProviderOrder;

	/**
	Base path for auxiliary files written out during the generation of a Niagara Sim Cache (ie: volume files).
	*/
	UPROPERTY(config, EditAnywhere, Category = SimCache, meta = (DisplayName = "Sim Cache Auxiliary File Base Path", ConfigRestartRequired = false))
	FString SimCacheAuxiliaryFileBasePath;

	/**
	Max memory in megabytes for total CPU memory for cached volumetric data
	*/
	UPROPERTY(config, EditAnywhere, Category = SimCache, meta = (DisplayName = "Sim Cache Max CPU Memory For Volumetrics", ConfigRestartRequired = false))
	int64 SimCacheMaxCPUMemoryVolumetrics;

	UPROPERTY(config, EditAnywhere, Category = Scalability)
	TArray<FNiagaraPlatformSetRedirect> PlatformSetRedirects;
	
	UPROPERTY(config, EditAnywhere, Category = Scalability)
	FNiagaraSystemPSOPrecacheSettings DefaultPSOPrecacheSettings;

#if WITH_EDITORONLY_DATA
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	bool bGenerateMetaDataOnCompile = false;
#endif // WITH_EDITORONLY_DATA

	// Begin UDeveloperSettings Interface
	NIAGARA_API virtual FName GetCategoryName() const override;
#if WITH_EDITOR
	NIAGARA_API void AddEnumParameterType(UEnum* Enum);
	NIAGARA_API virtual FText GetSectionText() const override;

	// END UDeveloperSettings Interface

	NIAGARA_API UNiagaraEffectType* GetDefaultEffectType() const;

	NIAGARA_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNiagaraSettingsChanged, const FName&, const UNiagaraSettings*);

	/** Gets a multicast delegate which is called whenever one of the parameters in this settings object changes. */
	static NIAGARA_API FOnNiagaraSettingsChanged& OnSettingsChanged();

protected:
	static NIAGARA_API FOnNiagaraSettingsChanged SettingsChangedDelegate;
#endif

	NIAGARA_API virtual void PostInitProperties() override;
};
