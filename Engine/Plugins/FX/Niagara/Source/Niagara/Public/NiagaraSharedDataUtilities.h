// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"

class UStaticMeshComponent;
class UNiagaraComponent;
class FNiagaraWorldManager;

/** 
Base class for generic data items held by data managers.
Niagara Systems, NDCs and potentially external users will hold onto these when they are being used.
Allows Data Managers to ensure lifetime of their internal data that may be being reference inside Niagara and by external users.
*/
struct FNiagaraSharedData : public TSharedFromThis<FNiagaraSharedData>
{
	virtual ~FNiagaraSharedData(){}

	//Flag to mark this ref as invalid and it should be removed.
	uint8 bInvalidated : 1 = false;

	/** A generic ID that can be passed into Niagara in cases where Niagara needs to directly reference this data via a DI. */
	FNiagaraID ID;
};

typedef TSharedPtr<FNiagaraSharedData> FNiagaraSharedDataPtr;

/** Utilities for the creation and use of "Niagara Shared Data". i.e. generic data used/referenced both inside Niagara Systems and by external users. */
namespace NiagaraSharedDataUtilities
{
	/**
	 * Finds (or creates) Shared Data Object containing data about a given static mesh. 
	 * The ID contained within can be passed into Niagara via User parameters or NDC to indirectly read data about the given static mesh.
	 * Using NiagaraDataInterfaceStaticMeshIndirect. This can be useful in cases where binding to a specific static mesh is not possible via a traditional NiagaraDataInterfaceStaticMesh.
	 *
	 * @param WorldMan				The NiagaraWorldManager for the relevant world.
	 * @param StaticMeshComponent	The Static Mesh Component to reference.
	 * @param ReferencingComponents	A list of NiagaraComponents which will be referencing this data.
	 * @return						A shared pointer to the data. 
									This remains valid until 
										* The mesh component is destroyed.
										* There are no longer any references to the returned shared data. 
									Inside Niagara, access using an invalid ID is safe but will fail and return an error in the DI function(s).
	 */
	NIAGARA_API FNiagaraSharedDataPtr GetOrCreateSharedData_StaticMesh(FNiagaraWorldManager* WorldMan, UStaticMeshComponent* StaticMeshComponent, TArrayView<UNiagaraComponent*> ReferencingComponents);
};
