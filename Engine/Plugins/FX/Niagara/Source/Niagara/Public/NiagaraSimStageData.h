// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/UnrealMath.h"
#include "Misc/CoreMiscDefines.h"

class FRDGBuffer;

struct FNiagaraSimStageDispatchArgs
{
	// Dispatch Element Count and optional Gpu Element Count to clip OOB threads
	FIntVector3	ElementCount = FIntVector3::ZeroValue;
	uint32		GpuElementCountOffset = INDEX_NONE;

	// Indirect dispatch buffer and the byte offset into the buffer
	FRDGBuffer*	IndirectBuffer = nullptr;
	uint32		IndirectOffset = 0;
};

struct FNiagaraSimStageData
{
	uint16 bFirstStage : 1 = false;
	uint16 bLastStage : 1 = false;
	uint16 bSetDataToRender : 1 = false;
	uint16 StageIndex = INDEX_NONE;

	uint16 NumIterations = 0;
	uint16 IterationIndex = 0;

	uint16 NumLoops = 0;
	uint16 LoopIndex = 0;

	FNiagaraSimStageDispatchArgs DispatchArgs;

	class FNiagaraDataBuffer* Source = nullptr;
	uint32 SourceCountOffset = INDEX_NONE;
	uint32 SourceNumInstances = 0;

	class FNiagaraDataBuffer* Destination = nullptr;
	uint32 DestinationCountOffset = INDEX_NONE;
	uint32 DestinationNumInstances = 0;

	struct FNiagaraDataInterfaceProxyRW* AlternateIterationSource = nullptr;
	const struct FSimulationStageMetaData* StageMetaData = nullptr;
};
