// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/UnrealMath.h"
#include "Misc/CoreMiscDefines.h"
#include "NiagaraScriptBase.h"

#include "NiagaraSimStageExecutionData.generated.h"

USTRUCT()
struct FNiagaraSimStageExecutionLoopEditorData
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Stage Loop")
	bool	bEnabled = true;

	UPROPERTY(EditAnywhere, Category = "Stage Loop")
	FName	NumLoopsBindingName;

	UPROPERTY(EditAnywhere, Category = "Stage Loop")
	int32	NumLoops = 1;

	UPROPERTY(EditAnywhere, Category = "Stage Loop")
	FName	StageNameStart;

	UPROPERTY(EditAnywhere, Category = "Stage Loop")
	FName	StageNameEnd;
#endif
};

USTRUCT()
struct FNiagaraSimStageExecutionLoopData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "SimStageLoop")
	FName NumLoopsBinding;

	UPROPERTY(EditAnywhere, Category = "SimStageLoop")
	int32 NumLoops = 1;

	UPROPERTY(EditAnywhere, Category = "SimStageLoop")
	int32 StartStageIndex = 0;

	UPROPERTY(EditAnywhere, Category = "SimStageLoop")
	int32 EndStageIndex = 0;
};

struct FNiagaraSimStageExecutionData
{
	TArray<FNiagaraSimStageExecutionLoopData, TInlineAllocator<1>>	ExecutionLoops;
	TArray<FSimulationStageMetaData>								SimStageMetaData;

#if WITH_EDITORONLY_DATA
	bool Build(TConstArrayView<FSimulationStageMetaData> InSimStageMetaDatas, TConstArrayView<FNiagaraSimStageExecutionLoopEditorData> InEditorExecutionLoops);
#endif
};

using FNiagaraSimStageExecutionDataPtr = TSharedPtr<FNiagaraSimStageExecutionData>;
