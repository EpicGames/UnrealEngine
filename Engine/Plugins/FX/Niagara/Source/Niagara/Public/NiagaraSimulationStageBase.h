// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraMergeable.h"
#include "NiagaraScriptBase.h"
#include "NiagaraCommon.h"
#include "NiagaraTypes.h"
#include "NiagaraConstants.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraSimulationStageCompileData.h"
#include "NiagaraSimulationStageBase.generated.h"

class UNiagaraScript;

/**
* A base class for niagara simulation stages.  This class should be derived to add stage specific information.
*/
UCLASS(abstract, MinimalAPI)
class UNiagaraSimulationStageBase : public UNiagaraMergeable
{
	GENERATED_BODY()

public:
	static NIAGARA_API const FName ParticleSpawnUpdateName;

	UNiagaraSimulationStageBase()
	{
		bEnabled = true;
	}

	UPROPERTY()
	TObjectPtr<UNiagaraScript> Script;

	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	FName SimulationStageName;

	UPROPERTY()
	uint32 bEnabled : 1;

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;

	virtual bool FillCompilationData(TArray<FNiagaraSimulationStageCompilationData>& CompilationSimStageData) const PURE_VIRTUAL(UNiagaraSimulationStageBase::FillCompileSimStageData, return false;);
	/** Return the FName to use in place of the default for the location in the stack context. If this would be the default, return NAME_None.*/
	virtual FName GetStackContextReplacementName() const { return NAME_None; }
	NIAGARA_API void SetEnabled(bool bEnabled);
	NIAGARA_API void RequestRecompile();
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	UPROPERTY()
	FGuid OuterEmitterVersion;
	
	NIAGARA_API FVersionedNiagaraEmitterData* GetEmitterData() const;
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitter GetOuterEmitter() const;
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitterBase GetOuterEmitterBase() const;
#endif
};

UCLASS(meta = (DisplayName = "Generic Simulation Stage"), MinimalAPI)
class UNiagaraSimulationStageGeneric : public UNiagaraSimulationStageBase
{
	GENERATED_BODY()

public:
	/** Optional bool binding allowing scripts to control if the simulation stage is enabled or not. */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	FNiagaraVariableAttributeBinding EnabledBinding;

	/**
	Select what we should be iterating over, particles or data interfaces.
	The source provides things such as element count (when not overriden) and stack context variables (i.e. attributes on grids)
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	ENiagaraIterationSource IterationSource;

	/**
	Number of times (or iterations) the simulation stage will execute in a row.
	For example, setting this to 10 will mean this simulation stage runs 10 times in a row before the next stage.
	Can also be bound to a attribute so the simulation can dynamically decide
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	FNiagaraParameterBindingWithValue NumIterations;

	/**
	Controls when the simulation stage should execute, only valid for data interface iteration stages
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	ENiagaraSimStageExecuteBehavior ExecuteBehavior = ENiagaraSimStageExecuteBehavior::Always;

	/**
	Disables the ability to read / write from the same particle buffer, i.e. only update position and no other attributes.
	By default this should not be changed, but can be used to debug issues.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Simulation Stage")
	uint32 bDisablePartialParticleUpdate : 1;

	/** Source data interface to use for the simulation stage. The data interface needs to be a subclass of UNiagaraDataInterfaceRWBase, for example the Grid2D and Grid3D data interfaces. */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	FNiagaraVariableDataInterfaceBinding DataInterface;

	/**
	When enabled we iterate over the subset of particles that satisfy the binding & range selected.
	For example, if the binding was set to Particles.MyStateIndex and the range was 1-3 only the particles
	with values of 1, 2 or 3 for Particles.MyStateIndex would run the script, all others would be ignored.
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	uint32 bParticleIterationStateEnabled : 1;

	/** Particle state attribute binding, when enabled we will only allow particles who pass the state range check to be processed. */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	FNiagaraVariableAttributeBinding ParticleIterationStateBinding;

	/** The inclusive range used to check particle state binding against when enabled. */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	FIntPoint ParticleIterationStateRange = FIntPoint(0, 0);

	/**
	When enabled we force the dispatch to be linear (i.e. one dimension is used).
	NOTE: Cannot be used with OverrideGpuDispatchType.
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	uint32 bGpuDispatchForceLinear : 1;

	/** When enabled we use a custom number of threads for the dispatch. */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage", meta = (DisplayName = "Override Gpu Thread Group Num Threads"))
	uint32 bOverrideGpuDispatchNumThreads : 1;

	/** Parameter binding / constant value for Num Threads X */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage", meta = (DisplayName = "Gpu Thread Group Num Threads X"))
	FNiagaraParameterBindingWithValue OverrideGpuDispatchNumThreadsX;

	/** Parameter binding / constant value for Num Threads Y */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage", meta = (DisplayName = "Gpu Thread Group Num Threads Y", EditCondition = "DirectDispatchType == ENiagaraGpuDispatchType::TwoD || DirectDispatchType == ENiagaraGpuDispatchType::ThreeD"))
	FNiagaraParameterBindingWithValue OverrideGpuDispatchNumThreadsY;

	/** Parameter binding / constant value for Num Threads Z */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage", meta = (DisplayName = "Gpu Thread Group Num Threads Z", EditCondition = "DirectDispatchType == ENiagaraGpuDispatchType::ThreeD"))
	FNiagaraParameterBindingWithValue OverrideGpuDispatchNumThreadsZ;

	/** Dimensions to use for dispatch. */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	ENiagaraGpuDispatchType DirectDispatchType = ENiagaraGpuDispatchType::OneD;

	/** Customizes what the element count means for a direct dispatch. */
	UPROPERTY(EditAnywhere, Category = "Simulation Stage")
	ENiagaraDirectDispatchElementType DirectDispatchElementType = ENiagaraDirectDispatchElementType::NumThreads;

	/**
	Integer binding to override the number of elements the stage will execute along X.
	For example, if you want to iterate over a custom source such as triangles on a mesh you can
	set an int to the triangle count in an emitter script and bind that as the element count.
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Stage", meta = (DisplayName = "ElementCount X"))
	FNiagaraParameterBindingWithValue ElementCountX;

	/**
	Integer binding to override the number of elements the stage will execute along Y.
	For example, if you want to iterate over a 2D texture you can set an int to the texture height
	in an emitter script and bind that as the element count.
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Stage", meta = (DisplayName = "ElementCount Y", EditCondition = "DirectDispatchType == ENiagaraGpuDispatchType::TwoD || DirectDispatchType == ENiagaraGpuDispatchType::ThreeD"))
	FNiagaraParameterBindingWithValue ElementCountY;

	/**
	Integer binding to override the number of elements the stage will execute along Z.
	For example, if you want to iterate over a 3D texture you can set an int to the texture depth
	in an emitter script and bind that as the element count.
	*/
	UPROPERTY(EditAnywhere, Category = "Simulation Stage", meta = (DisplayName = "ElementCount Z", EditCondition = "DirectDispatchType == ENiagaraGpuDispatchType::ThreeD"))
	FNiagaraParameterBindingWithValue ElementCountZ;

#if WITH_EDITORONLY_DATA
	// Deprecated Properties
	UPROPERTY()
	FNiagaraVariableAttributeBinding ElementCountBinding_DEPRECATED;

	UPROPERTY()
	FNiagaraVariableAttributeBinding ElementCountXBinding_DEPRECATED;

	UPROPERTY()
	FNiagaraVariableAttributeBinding ElementCountYBinding_DEPRECATED;

	UPROPERTY()
	FNiagaraVariableAttributeBinding ElementCountZBinding_DEPRECATED;

	UPROPERTY()
	uint32 bSpawnOnly_DEPRECATED : 1;
	
	UPROPERTY()
	uint32 bOverrideGpuDispatchType_DEPRECATED : 1;

	UPROPERTY()
	ENiagaraGpuDispatchType OverrideGpuDispatchType_DEPRECATED = ENiagaraGpuDispatchType::OneD;

	UPROPERTY()
	int32 Iterations_DEPRECATED = 1;

	UPROPERTY()
	FNiagaraVariableAttributeBinding NumIterationsBinding_DEPRECATED;

	UPROPERTY()
	FIntVector OverrideGpuDispatchNumThreads_DEPRECATED = FIntVector(64, 1, 1);
#endif
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const override;
	NIAGARA_API virtual bool FillCompilationData(TArray<FNiagaraSimulationStageCompilationData>& CompilationSimStageData) const override;
	NIAGARA_API virtual void RenameEmitter(const FName& InOldName, const UNiagaraEmitter* InRenamedEmitter);
	NIAGARA_API virtual void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitter& InEmitter);
	NIAGARA_API virtual void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitter& InEmitter);
#endif
#if WITH_EDITOR
	NIAGARA_API virtual FName GetStackContextReplacementName() const override; 

	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
