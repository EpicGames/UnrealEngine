// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "NiagaraScriptBase.h"

struct FNiagaraSimulationStageCompilationData
{
	FGuid                           StageGuid;
	FName                           StageName;
	FName                           EnabledBinding;
	FIntVector3						ElementCount = FIntVector::ZeroValue;
	FName                           ElementCountXBinding;
	FName                           ElementCountYBinding;
	FName                           ElementCountZBinding;
	uint32                          NumIterations = 1;
	FName                           NumIterationsBinding;
	ENiagaraIterationSource			IterationSourceType = ENiagaraIterationSource::Particles;
	FName							IterationDataInterface;
	FName							IterationDirectBinding;
	ENiagaraSimStageExecuteBehavior ExecuteBehavior = ENiagaraSimStageExecuteBehavior::Always;
	mutable bool                    PartialParticleUpdate = false;
	bool                            bParticleIterationStateEnabled = false;
	FName                           ParticleIterationStateBinding;
	FIntPoint                       ParticleIterationStateRange = FIntPoint::ZeroValue;
	bool                            bGpuDispatchForceLinear = false;
	ENiagaraGpuDispatchType         DirectDispatchType = ENiagaraGpuDispatchType::OneD;
	ENiagaraDirectDispatchElementType DirectDispatchElementType = ENiagaraDirectDispatchElementType::NumThreads;
	bool                            bOverrideGpuDispatchNumThreads = false;
	FIntVector                      OverrideGpuDispatchNumThreads = FIntVector(1, 1, 1);
	FNiagaraVariableBase			OverrideGpuDispatchNumThreadsXBinding;
	FNiagaraVariableBase			OverrideGpuDispatchNumThreadsYBinding;
	FNiagaraVariableBase			OverrideGpuDispatchNumThreadsZBinding;
};
