// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraGPUSortInfo.h"
#include "NiagaraRendererProperties.h"
#include "Particles/SubUVAnimation.h"
#include "NiagaraSpriteRendererProperties.generated.h"

class UMaterialInstanceConstant;
class FVertexFactoryType;

/** This enum decides how a sprite particle will orient its "up" axis. Must keep these in sync with NiagaraSpriteVertexFactory.ush*/
UENUM()
enum class ENiagaraSpriteAlignment : uint8
{
	/** Only Particles.SpriteRotation and FacingMode impact the alignment of the particle.*/
	Unaligned,
	/** Imagine the particle texture having an arrow pointing up, this mode makes the arrow point in the direction of the Particles.Velocity attribute. FacingMode is ignored unless CustomFacingVector is set.*/
	VelocityAligned,
	/** Imagine the particle texture having an arrow pointing up, this mode makes the arrow point towards the axis defined by the "Particles.SpriteAlignment" attribute. FacingMode is ignored unless CustomFacingVector is set. If the "Particles.SpriteAlignment" attribute is missing, this falls back to Unaligned mode.*/
	CustomAlignment,

	/** Automatically select between Unaligned & CustomAlignment depending on if SpriteAlignment Binding is valid. */
	Automatic
};


/** This enum decides how a sprite particle will orient its "facing" axis. Must keep these in sync with NiagaraSpriteVertexFactory.ush*/
UENUM()
enum class ENiagaraSpriteFacingMode : uint8
{
	/** The sprite billboard origin is always "looking at" the camera origin, trying to keep its up axis aligned to the camera's up axis. */
	FaceCamera,
	/** The sprite billboard plane is completely parallel to the camera plane. Particle always looks "flat" */
	FaceCameraPlane,
	/** The sprite billboard faces toward the "Particles.SpriteFacing" vector attribute. If the "Particles.SpriteFacing" attribute is missing, this falls back to FaceCamera mode.*/
	CustomFacingVector,
	/** Faces the camera position, but is not dependent on the camera rotation.  This method produces more stable particles under camera rotation. Uses the up axis of (0,0,1).*/
	FaceCameraPosition,
	/** Blends between FaceCamera and FaceCameraPosition.*/
	FaceCameraDistanceBlend,

	/** Automatically select between FaceCamera & CustomFacingVector depending on if SpriteFacing binding is valid. */
	Automatic
};

UENUM()
enum class ENiagaraRendererPixelCoverageMode : uint8
{
	/** Automatically determine if we want pixel coverage enabled or disabled, based on project setting and the material used on the renderer. */
	Automatic,
	/** Disable pixel coverage. */
	Disabled,
	/** Enable pixel coverage with no color adjustment based on coverage. */
	Enabled UMETA(DisplayName = "Enabled (No Color Adjustment)"),
	/** Enable pixel coverage and adjust the RGBA channels according to coverage. */
	Enabled_RGBA UMETA(DisplayName = "Enabled (RGBA)"),
	/** Enable pixel coverage and adjust the RGB channels according to coverage. */
	Enabled_RGB UMETA(DisplayName = "Enabled (RGB)"),
	/** Enable pixel coverage and adjust the Alpha channel only according to coverage. */
	Enabled_A UMETA(DisplayName = "Enabled (A)"),
};

namespace ENiagaraSpriteVFLayout
{
	enum Type
	{
		Position,
		Color,
		Velocity,
		Rotation,
		Size,
		Facing,
		Alignment,
		SubImage,
		MaterialParam0,
		MaterialParam1,
		MaterialParam2,
		MaterialParam3,
		CameraOffset,
		UVScale,
		PivotOffset,
		MaterialRandom,
		CustomSorting,
		NormalizedAge,

		Num_Default,

		// The remaining layout params aren't needed unless accurate motion vectors are required
		PrevPosition = Num_Default,
		PrevVelocity,
		PrevRotation,
		PrevSize,
		PrevFacing,
		PrevAlignment,
		PrevCameraOffset,
		PrevPivotOffset,

		Num_Max,
	};
};

class FAssetThumbnailPool;
class SWidget;

UCLASS(editinlinenew, meta = (DisplayName = "Sprite Renderer", SupportsStateless), MinimalAPI)
class UNiagaraSpriteRendererProperties : public UNiagaraRendererProperties
{
public:
	GENERATED_BODY()

	NIAGARA_API UNiagaraSpriteRendererProperties();

	//UObject Interface
	NIAGARA_API virtual void PostLoad() override;
	NIAGARA_API virtual void PostInitProperties() override;
	NIAGARA_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
	NIAGARA_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NIAGARA_API virtual void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
	NIAGARA_API virtual void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
	NIAGARA_API virtual TArray<FNiagaraVariable> GetBoundAttributes() const override;
#endif // WITH_EDITORONLY_DATA
	//UObject Interface END

	static NIAGARA_API void InitCDOPropertiesAfterModuleStartup();

	//UNiagaraRendererProperties interface
	NIAGARA_API virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	NIAGARA_API virtual class FNiagaraBoundsCalculator* CreateBoundsCalculator() override;
	NIAGARA_API virtual void GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const override;
	virtual float GetMaterialStreamingScale() const override { return static_cast<float>(FMath::Max(FMath::Max(SubImageSize.X, SubImageSize.Y), 1.0)); }
	NIAGARA_API virtual const FVertexFactoryType* GetVertexFactoryType() const override;
	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const override { return true; };
	NIAGARA_API virtual bool PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore)  override;
	NIAGARA_API virtual void CollectPSOPrecacheData(const FNiagaraEmitterInstance* InEmitter, FPSOPrecacheParamsList& OutParams) const override;
#if WITH_EDITOR
	NIAGARA_API virtual const TArray<FNiagaraVariable>& GetOptionalAttributes() override;
	NIAGARA_API virtual void GetAdditionalVariables(TArray<FNiagaraVariableBase>& OutArray) const override;
	NIAGARA_API virtual void GetRendererWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	NIAGARA_API virtual void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	NIAGARA_API virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FText>& OutErrors, TArray<FText>& OutWarnings, TArray<FText>& OutInfo) const override;
	NIAGARA_API virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const override;
	NIAGARA_API virtual void GetRendererMaterialUsageFeedback(TArray<FNiagaraRendererFeedback>& OutFeedback) const override;
#endif
	virtual ENiagaraRendererSourceDataMode GetCurrentSourceMode() const override { return SourceMode; }

	NIAGARA_API virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) override;
	//UNiagaraMaterialRendererProperties interface END

	NIAGARA_API int32 GetNumCutoutVertexPerSubimage() const;
	NIAGARA_API uint32 GetNumIndicesPerInstance() const;

	/** The material used to render the particle. Note that it must have the Use with Niagara Sprites flag checked.*/
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering")
	TObjectPtr<UMaterialInterface> Material;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceConstant> MICMaterial;
#endif

	/** Use the UMaterialInterface bound to this user variable if it is set to a valid value. If this is bound to a valid value and Material is also set, UserParamBinding wins.*/
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering")
	FNiagaraUserParameterBinding MaterialUserParamBinding;

	/** Whether or not to draw a single element for the Emitter or to draw the particles.*/
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering")
	ENiagaraRendererSourceDataMode SourceMode = ENiagaraRendererSourceDataMode::Particles;

	/**
	Imagine the particle texture having an arrow pointing up, these modes define how the particle aligns that texture to other particle attributes.
	Automatic mode will select between Unaligned & CustomAlignment depending on if SpriteAlignment binding is valid.
	*/
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering")
	ENiagaraSpriteAlignment Alignment = ENiagaraSpriteAlignment::Automatic;

	/**
	Determines how the particle billboard orients itself relative to the camera.
	Automatic mode will select FaceCamera or CustomFacingVector depending on if SpriteFacing binding is valid
	*/
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering")
	ENiagaraSpriteFacingMode FacingMode = ENiagaraSpriteFacingMode::Automatic;

	/** Determines how we sort the particles prior to rendering.*/
	UPROPERTY(EditAnywhere, Category = "Sorting")
	ENiagaraSortMode SortMode = ENiagaraSortMode::None;

	/** World space radius that UVs generated with the ParticleMacroUV material node will tile based on. */
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering")
	float MacroUVRadius = 0.0f;

	/**
	 * Determines the location of the pivot point of this particle. It follows Unreal's UV space, which has the upper left of the image at 0,0 and bottom right at 1,1. The middle is at 0.5, 0.5.
	 * NOTE: This value is ignored if "Pivot Offset Binding" is bound to a valid attribute
	 */
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering", meta = (DisplayName = "Default Pivot in UV Space"))
	FVector2D PivotInUVSpace = FVector2D(0.5f, 0.5f);

	/** When using SubImage lookups for particles, this variable contains the number of columns in X and the number of rows in Y.*/
	UPROPERTY(EditAnywhere, Category = "SubUV", meta = (DisplayAfter = bSubImageBlend))
	FVector2D SubImageSize = FVector2D(1.0f, 1.0f);

	/** If true, blends the sub-image UV lookup with its next adjacent member using the fractional part of the SubImageIndex float value as the linear interpolation factor.*/
	UPROPERTY(EditAnywhere, Category = "SubUV", meta = (DisplayName = "Sub UV Blending Enabled"))
	uint8 bSubImageBlend : 1;

	/** If true, removes the HMD view roll (e.g. in VR) */
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering", meta = (DisplayName = "Remove HMD Roll"))
	uint8 bRemoveHMDRollInVR : 1;

	/** If true, the particles are only sorted when using a translucent material. */
	UPROPERTY(EditAnywhere, Category = "Sorting", meta = (EditCondition = "SortMode != ENiagaraSortMode::None", EditConditionHides))
	uint8 bSortOnlyWhenTranslucent : 1;

	/** Enables frustum culling of individual sprites */
	UPROPERTY(EditAnywhere, Category = "Visibility")
	uint8 bEnableCameraDistanceCulling : 1;

	/**
	When disabled the renderer will not cast shadows.
	The component controls if shadows are enabled, this flag allows you to disable the renderer casting shadows.
	*/
	UPROPERTY(EditAnywhere, Category = "Rendering")
	uint8 bCastShadows : 1 = 1;

#if WITH_EDITORONLY_DATA
	/**
	When disabled the mesh batches do not contribute to the hit proxy, i.e. they are not part of editor selection.
	*/
	UPROPERTY(EditAnywhere, Category = "Rendering")
	uint8 bIncludeInHitProxy : 1 = 1;
#endif

	/** Sort precision to use when sorting is active. */
	UPROPERTY(EditAnywhere, Category = "Sorting", meta = (EditCondition = "SortMode != ENiagaraSortMode::None", EditConditionHides))
	ENiagaraRendererSortPrecision SortPrecision = ENiagaraRendererSortPrecision::Default;

	/**
	Gpu simulations run at different points in the frame depending on what features are used, i.e. depth buffer, distance fields, etc.
	Opaque materials will run latent when these features are used.
	Translucent materials can choose if they want to use this frames or the previous frames data to match opaque draws.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Sprite Rendering")
	ENiagaraRendererGpuTranslucentLatency GpuTranslucentLatency = ENiagaraRendererGpuTranslucentLatency::ProjectDefault;

	/**
	This setting controls what happens when a sprite becomes less than a pixel in size.
	Disabling will apply nothing and can result in flickering issues, especially with Temporal Super Resolution.
	Automatic will enable the appropriate settings when the material blend mode is some form of translucency, project setting must also be enabled.
	When coverage is less than a pixel, we also calculate a percentage of coverage, and then darken or reduce opacity
	to visually compensate.	The different enabled settings allow you to control how the coverage amount is applied to
	your particle color.  If particle color is not connected to your material the compensation will not be applied.
	*/
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering")
	ENiagaraRendererPixelCoverageMode PixelCoverageMode = ENiagaraRendererPixelCoverageMode::Automatic;

	/**
	When pixel coverage is enabled this allows you to control the blend of the pixel coverage color adjustment.
	i.e. 1.0 = full, 0.5 = 50/50 blend, 0.0 = none
	*/
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering", meta = (EditCondition = "PixelCoverageMode != ENiagaraRendererPixelCoverageMode::Disabled", UIMin=0.0f, UIMax=1.0f, EditConditionHides))
	float PixelCoverageBlend = 1.0f;

	/** When FacingMode is FacingCameraDistanceBlend, the distance at which the sprite is fully facing the camera plane. */
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering", meta = (UIMin = "0"))
	float MinFacingCameraBlendDistance = 0.0f;

	/** When FacingMode is FacingCameraDistanceBlend, the distance at which the sprite is fully facing the camera position */
	UPROPERTY(EditAnywhere, Category = "Sprite Rendering", meta = (UIMin = "0"))
	float MaxFacingCameraBlendDistance = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Visibility", meta = (EditCondition = "bEnableCameraDistanceCulling", UIMin = 0.0f, EditConditionHides))
	float MinCameraDistance;

	UPROPERTY(EditAnywhere, Category = "Visibility", meta = (EditCondition = "bEnableCameraDistanceCulling", UIMin = 0.0f, EditConditionHides))
	float MaxCameraDistance = 1000.0f;

	/** If a render visibility tag is present, particles whose tag matches this value will be visible in this renderer. */
	UPROPERTY(EditAnywhere, Category = "Visibility")
	uint32 RendererVisibility = 0;

	/** Which attribute should we use for position when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding PositionBinding;

	/** Which attribute should we use for color when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding ColorBinding;

	/** Which attribute should we use for velocity when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding VelocityBinding;

	/** Which attribute should we use for sprite rotation (in degrees) when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding SpriteRotationBinding;

	/** Which attribute should we use for sprite size when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding SpriteSizeBinding;

	/** Which attribute should we use for sprite facing when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding SpriteFacingBinding;

	/** Which attribute should we use for sprite alignment when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding SpriteAlignmentBinding;

	/** Which attribute should we use for sprite sub-image indexing when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding SubImageIndexBinding;

	/** Which attribute should we use for dynamic material parameters when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterialBinding;

	/** Which attribute should we use for dynamic material parameters when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterial1Binding;

	/** Which attribute should we use for dynamic material parameters when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterial2Binding;

	/** Which attribute should we use for dynamic material parameters when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding DynamicMaterial3Binding;

	/** Which attribute should we use for camera offset when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding CameraOffsetBinding;

	/** Which attribute should we use for UV scale when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding UVScaleBinding;

	/** Which attribute should we use for pivot offset? (NOTE: Values are expected to be in UV space). */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding PivotOffsetBinding;

	/** Which attribute should we use for material randoms when generating sprites?*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding MaterialRandomBinding;

	/** Which attribute should we use for custom sorting? Defaults to Particles.NormalizedAge. */
	UPROPERTY(EditAnywhere, Category = "Bindings", meta = (EditCondition = "SourceMode != ENiagaraRendererSourceDataMode::Emitter"))
	FNiagaraVariableAttributeBinding CustomSortingBinding;

	/** Which attribute should we use for Normalized Age? */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding NormalizedAgeBinding;

	/** Which attribute should we use for RendererVisibilityTag? */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RendererVisibilityTagBinding;

	/** If this array has entries, we will create a MaterialInstanceDynamic per Emitter instance from Material and set the Material parameters using the Niagara simulation variables listed.*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraRendererMaterialParameters MaterialParameters;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FNiagaraMaterialAttributeBinding> MaterialParameterBindings_DEPRECATED;
#endif

	// The following bindings are only needed for accurate motion vectors

	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevPositionBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevVelocityBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevSpriteRotationBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevSpriteSizeBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevSpriteFacingBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevSpriteAlignmentBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevCameraOffsetBinding;
	UPROPERTY(Transient)
	FNiagaraVariableAttributeBinding PrevPivotOffsetBinding;

	virtual bool NeedsMIDsForMaterials() const { return MaterialParameters.HasAnyBindings(); }


#if WITH_EDITORONLY_DATA
	/** Use the cutout texture from the material opacity mask, or if none exist, from the material opacity.	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cutout")
	bool bUseMaterialCutoutTexture;

	/** When enabled cutouts for SubUVs will generated like there is a single frame rather than atlasing the texture. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Cutout")
	bool bCutoutIgnoreSubUV = false;

	/** Texture to generate bounding geometry from.	*/
	UPROPERTY(EditAnywhere, Category="Cutout", meta = (EditCondition = "!bUseMaterialCutoutTexture", EditConditionHides))
	TObjectPtr<UTexture2D> CutoutTexture;

	/**
	* More bounding vertices results in reduced overdraw, but adds more triangle overhead.
	* The eight vertex mode is best used when the SubUV texture has a lot of space to cut out that is not captured by the four vertex version,
	* and when the particles using the texture will be few and large.
	*/
	UPROPERTY(EditAnywhere, Category= "Cutout")
	TEnumAsByte<enum ESubUVBoundingVertexCount> BoundingMode;

	UPROPERTY(EditAnywhere, Category="Cutout")
	TEnumAsByte<enum EOpacitySourceMode> OpacitySourceMode;

	/**
	* Alpha channel values larger than the threshold are considered occupied and will be contained in the bounding geometry.
	* Raising this threshold slightly can reduce overdraw in particles using this animation asset.
	*/
	UPROPERTY(EditAnywhere, Category="Cutout", meta=(UIMin = "0", UIMax = "1"))
	float AlphaThreshold;

	NIAGARA_API void UpdateCutoutTexture();
	NIAGARA_API void CacheDerivedData();
#endif

	UPROPERTY()
	uint32 MaterialParamValidMask = 0;

	const TArray<FVector2f>& GetCutoutData() const { return DerivedData.BoundingGeometry; }

	FNiagaraRendererLayout RendererLayoutWithCustomSort;
	FNiagaraRendererLayout RendererLayoutWithoutCustomSort;

protected:
	NIAGARA_API void InitBindings();
	NIAGARA_API void SetPreviousBindings(const FVersionedNiagaraEmitterBase& SrcEmitter, ENiagaraRendererSourceDataMode InSourceMode);
	NIAGARA_API virtual void UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit = false) override;

	NIAGARA_API void UpdateMICs();

#if WITH_EDITORONLY_DATA
	NIAGARA_API virtual FNiagaraVariable GetBoundAttribute(const FNiagaraVariableAttributeBinding* Binding) const override;
#endif

private:
	/** Derived data for this asset, generated off of SubUVTexture. */
	FSubUVDerivedData DerivedData;

	static NIAGARA_API TArray<TWeakObjectPtr<UNiagaraSpriteRendererProperties>> SpriteRendererPropertiesToDeferredInit;
};
