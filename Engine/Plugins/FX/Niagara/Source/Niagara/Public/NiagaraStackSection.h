// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "NiagaraStackSection.generated.h"

/* Defines data for sections visible in the stack view. */
USTRUCT()
struct FNiagaraStackSection 
{
	GENERATED_BODY()

	FNiagaraStackSection()
		: bEnabled(true)
	{
	}

	UPROPERTY(EditAnywhere, Category=Section)
	FName SectionIdentifier;

	UPROPERTY(EditAnywhere, Category=Section)
	FText SectionDisplayName;

	UPROPERTY(EditAnywhere, Category=Section)
	TArray<FText> Categories;

	UPROPERTY()
	FText Tooltip;
	
	UPROPERTY(EditAnywhere, Category=Section)
	bool bEnabled;
};