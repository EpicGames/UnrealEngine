// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "NiagaraSystemCollection.generated.h"

#define UE_API NIAGARA_API

class UNiagaraSystem;
struct FStreamableHandle;


//A collection of Niagara Systems that can be referenced and asynchronously loaded as needed.
USTRUCT(BlueprintType)
struct FNiagaraSystemCollectionData
{
	GENERATED_BODY();

	UE_API void InitFromArray(TConstArrayView<TSoftObjectPtr<UNiagaraSystem>> InSystems);

	int32 Num()const { return Systems.Num(); }

	/** Triggers async loading of all systems in this collection. */
	UE_API void LoadAsync()const;

	/** Triggers synchronous (blocking) loading of all systems in this collection. */
	UE_API void LoadSynchronous()const;

	/** Release any currently loaded Niagara Systems. Cancel any pending loads. */
	UE_API void Release();

	/** Returns the Systems as an array, Loading them if needed. */
	UE_API const TArray<TObjectPtr<UNiagaraSystem>>& GetSystems()const;

private:

	UE_API void PostLoadSystems()const;

	UPROPERTY(EditAnywhere, Category = "Systems")
	TArray<TSoftObjectPtr<UNiagaraSystem>> Systems;

	UPROPERTY(Transient)
	mutable TArray<TObjectPtr<UNiagaraSystem>> SystemsInternal;

	mutable TSharedPtr<FStreamableHandle> AsyncLoadHandle;
};

/** A collection of Niagara Systems. */
UCLASS(BlueprintType)
class UNiagaraSystemCollection : public UDataAsset
{
	GENERATED_BODY()

public:
	
	virtual void PostLoad()override;

	/** The number of systems in this collection. */
	UFUNCTION(BlueprintCallable, Category = "System Collection")
	int32 Num()const { return SystemCollection.Num(); }

	/** Triggers async loading of all systems in this collection. */
	UFUNCTION(BlueprintCallable, Category = "System Collection")
	void LoadAsync()const { SystemCollection.LoadAsync(); }

	/** Triggers synchronous (blocking) loading of all systems in this collection. */
	UFUNCTION(BlueprintCallable, Category = "System Collection")
	void LoadSynchronous()const { SystemCollection.LoadSynchronous(); }

	/** Release any currently loaded Niagara Systems. Cancel any pending loads. */
	UFUNCTION(BlueprintCallable, Category = "System Collection")
	void Release() { SystemCollection.Release(); }

	/** Returns the Systems as an array, Loading them if needed. */
	UFUNCTION(BlueprintCallable, Category = "System Collection")
	const TArray<UNiagaraSystem*>& GetSystems()const { return SystemCollection.GetSystems();  }
private:

	UPROPERTY(EditAnywhere, Category = "Systems", meta=(ShowOnlyInnerProperties))
	FNiagaraSystemCollectionData SystemCollection;
	
	UPROPERTY(EditAnywhere, Category = "Systems", meta=(ShowOnlyInnerProperties))
	bool bLoadImmediately = true;
};

#undef UE_API
