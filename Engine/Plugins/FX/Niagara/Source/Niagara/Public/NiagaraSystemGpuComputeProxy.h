// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "RHI.h"

class FNiagaraSystemInstance;
class FNiagaraGpuComputeDispatchInterface;
struct FNiagaraComputeExecutionContext;
class FNiagaraGPUInstanceCountManager;
class FNiagaraGPUSystemTick;

class FNiagaraSystemGpuComputeProxy
{
	friend class FNiagaraGpuComputeDispatch;

public:
	FNiagaraSystemGpuComputeProxy(FNiagaraSystemInstance* OwnerInstance);
	~FNiagaraSystemGpuComputeProxy();

	void AddToRenderThread(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface);
	void RemoveFromRenderThread(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface, bool bDeleteProxy);
	void ClearTicksFromRenderThread(FNiagaraGpuComputeDispatchInterface* ComputeDispatchInterface);

	FNiagaraSystemInstanceID GetSystemInstanceID() const { return SystemInstanceID; }
	ENiagaraGpuComputeTickStage::Type GetComputeTickStage() const { return ComputeTickStage; }
	void QueueTick(FNiagaraGPUSystemTick& Tick);
	void ReleaseTicks(int32 NumTicksToRelease, bool bLastViewFamily);

	bool RequiresGlobalDistanceField() const { return bRequiresGlobalDistanceField; }
	bool RequiresDepthBuffer() const { return bRequiresDepthBuffer; }
	bool RequiresEarlyViewData() const { return bRequiresEarlyViewData; }
	bool RequiresViewUniformBuffer() const { return bRequiresViewUniformBuffer; }
	bool RequiresRayTracingScene() const { return bRequiresRayTracingScene; }
	bool RequiresCurrentFrameNDC() const { return bRequiresCurrentFrameNDC; }
	FVector3f GetSystemLWCTile() const { return SystemLWCTile; }
	uint32 GetGpuCountBufferEstimate() const { return GpuCountBufferEstimate; }

private:
	FNiagaraSystemInstance*						DebugOwnerInstance = nullptr;
	FNiagaraGpuComputeDispatchInterface*		DebugOwnerComputeDispatchInterface = nullptr;
	int32										ComputeDispatchIndex = INDEX_NONE;
	FVector3f									SystemLWCTile;

	FNiagaraSystemInstanceID					SystemInstanceID = FNiagaraSystemInstanceID();
	ENiagaraGpuComputeTickStage::Type			ComputeTickStage = ENiagaraGpuComputeTickStage::PostOpaqueRender;
	uint32										bRequiresGlobalDistanceField : 1;
	uint32										bRequiresDepthBuffer : 1;
	uint32										bRequiresEarlyViewData : 1;
	uint32										bRequiresViewUniformBuffer : 1;
	uint32										bRequiresRayTracingScene : 1;
	uint32										bRequiresCurrentFrameNDC: 1;
	uint32										GpuCountBufferEstimate = 0;

	FShaderResourceViewRHIRef					StaticFloatBuffer;

	TArray<FNiagaraComputeExecutionContext*>	ComputeContexts;
	TArray<FNiagaraGPUSystemTick>				PendingTicks;
};
