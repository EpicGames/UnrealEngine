// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraSystem.h"
#include "NiagaraDataInterfaceBindingInstance.h"
#include "Templates/UniquePtr.h"
#include "NiagaraCommon.h"
#include "NiagaraDataInterface.h"

class FNiagaraWorldManager;
class FNiagaraEmitterInstanceImpl;
class INiagaraSceneComponentUtils;
class FNiagaraSystemInstance;
class FNiagaraSystemSimulation;
class FNiagaraGpuComputeDispatchInterface;
class FNiagaraGPUSystemTick;
struct FNiagaraParameterCollectionResolver;
class UNiagaraSimCache;
class FNiagaraSystemGpuComputeProxy;
struct FNiagaraParameterStore;
struct FNiagaraScriptResolvedDataInterfaceInfo;

using FNiagaraSystemInstancePtr = TSharedPtr<FNiagaraSystemInstance, ESPMode::ThreadSafe>;

struct FNiagaraSystemInstanceFinalizeRef
{
	FNiagaraSystemInstanceFinalizeRef() {}
	explicit FNiagaraSystemInstanceFinalizeRef(FNiagaraSystemInstance** InTaskEntry) : TaskEntry(InTaskEntry) {}

	bool IsPending() const
	{
		return TaskEntry != nullptr;
	}
	void Clear()
	{
		check(TaskEntry != nullptr);
		*TaskEntry = nullptr;
		TaskEntry = nullptr;
#if DO_CHECK
		--(*DebugCounter);
#endif
	}
	void ConditionalClear()
	{
		if ( TaskEntry != nullptr )
		{
			*TaskEntry = nullptr;
			TaskEntry = nullptr;
#if DO_CHECK
			--(*DebugCounter);
#endif
		}
	}

#if DO_CHECK
	void SetDebugCounter(std::atomic<int>* InDebugCounter)
	{
		DebugCounter = InDebugCounter;
		++(*DebugCounter);
	}
#endif

private:
	class FNiagaraSystemInstance** TaskEntry = nullptr;
#if DO_CHECK
	std::atomic<int>* DebugCounter = nullptr;
#endif
};

// Hacky base class to avoid 8 bytes of padding after the vtable
class FNiagaraSystemInstanceFixLayout
{
public:
	virtual ~FNiagaraSystemInstanceFixLayout() = default;
};

class FNiagaraSystemInstance : public FNiagaraSystemInstanceFixLayout
{
	friend class FNiagaraSystemSimulation;
	friend class FNiagaraGPUSystemTick;
	friend class FNiagaraDebugHud;
	friend class UNiagaraSimCache;

public:
	DECLARE_DELEGATE(FOnPostTick);
	DECLARE_DELEGATE_OneParam(FOnComplete, bool /*bExternalCompletion*/);

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE(FOnInitialized);
	
	DECLARE_MULTICAST_DELEGATE(FOnReset);
	DECLARE_MULTICAST_DELEGATE(FOnDestroyed);
#endif

public:

	/** Defines modes for resetting the System instance. */
	enum class EResetMode : uint8
	{
		/** Resets the System instance and simulations. */
		ResetAll,
		/** Resets the System instance but not the simulations */
		ResetSystem,
		/** Full reinitialization of the system and emitters.  */
		ReInit,
		/** No reset */
		None
	};

	ENiagaraSystemInstanceState SystemInstanceState = ENiagaraSystemInstanceState::None;

	ENiagaraTickBehavior TickBehavior;

private:
	/** If async work was running when we request an Activate we will store the reset mode and perform in finalize to avoid stalling the GameThread. */
	EResetMode DeferredResetMode = EResetMode::None;

public:
	inline bool GetAreDataInterfacesInitialized() const { return bDataInterfacesInitialized; }

	/** Creates a new Niagara system instance. */
	NIAGARA_API FNiagaraSystemInstance(UWorld& InWorld, UNiagaraSystem& InAsset, FNiagaraUserRedirectionParameterStore* InOverrideParameters = nullptr,
		USceneComponent* InAttachComponent = nullptr, ENiagaraTickBehavior InTickBehavior = ENiagaraTickBehavior::UsePrereqs, bool bInPooled = false);

	/** Cleanup*/
	NIAGARA_API virtual ~FNiagaraSystemInstance();

	NIAGARA_API void Cleanup();

	/** Initializes this System instance to simulate the supplied System. */
	NIAGARA_API void Init(bool bInForceSolo=false);

	NIAGARA_API void Activate(EResetMode InResetMode = EResetMode::ResetAll);
	NIAGARA_API void Deactivate(bool bImmediate = false);
	NIAGARA_API void Complete(bool bExternalCompletion);

	NIAGARA_API void OnPooledReuse(UWorld& NewWorld);

	NIAGARA_API void SetPaused(bool bInPaused);
	inline bool IsPaused() const { return (SystemInstanceState == ENiagaraSystemInstanceState::PendingSpawnPaused) || (SystemInstanceState == ENiagaraSystemInstanceState::Paused); }

	NIAGARA_API void SetSolo(bool bInSolo);

	NIAGARA_API void SetGpuComputeDebug(bool bEnableDebug);
	NIAGARA_API void SetWarmupSettings(int32 WarmupTickCount, float WarmupTickDelta);

	NIAGARA_API UActorComponent* GetPrereqComponent() const;

	NIAGARA_API void BindParameters();
	NIAGARA_API void UnbindParameters(bool bFromComplete = false);
	void RebindParameterCollection(UNiagaraParameterCollectionInstance* OldCollection, UNiagaraParameterCollectionInstance* NewCollection);

	// Bindings Override Parameters / Instance Parameters / System Simulation Parameters to the provded parameter store
	// I.e. binds all relevant parameters from us and parent, does not do children (i.e. emitters)
	void BindToParameterStore(FNiagaraParameterStore& ParameterStore);
	// Unbinds the parameters that would be bound via BindToParameterStore
	void UnbindFromParameterStore(FNiagaraParameterStore& ParameterStore);

	inline FNiagaraParameterStore& GetInstanceParameters() { return InstanceParameters; }
	NIAGARA_API FNiagaraLWCConverter GetLWCConverter(bool bLocalSpaceEmitter = false) const;
	NIAGARA_API FTransform GetLWCSimToWorld(bool bLocalSpaceEmitter = false) const;
	void UpdateLWCTile(const FVector3f& InLWCTile, FNiagaraDataSet& EmitterDataSet);

	inline uint32 GetParameterIndex(bool PreviousFrame = false) const
	{
		return (!!(PreviousFrame && ParametersValid) ^ !!CurrentFrameIndex) ? 1 : 0;
	}

	inline void FlipParameterBuffers()
	{
		CurrentFrameIndex = ~CurrentFrameIndex;
		
		// when we've hit both buffers, we'll mark the parameters as being valid
		if (CurrentFrameIndex == 1)
		{
			ParametersValid = true;
		}
	}

	inline const FNiagaraGlobalParameters& GetGlobalParameters(bool PreviousFrame = false) const { return GlobalParameters[GetParameterIndex(PreviousFrame)]; }
	inline const FNiagaraSystemParameters& GetSystemParameters(bool PreviousFrame = false) const { return SystemParameters[GetParameterIndex(PreviousFrame)]; }
	inline const FNiagaraOwnerParameters& GetOwnerParameters(bool PreviousFrame = false) const { return OwnerParameters[GetParameterIndex(PreviousFrame)]; }
	inline const FNiagaraEmitterParameters& GetEmitterParameters(int32 EmitterIdx, bool PreviousFrame = false) const { return EmitterParameters[EmitterIdx * 2 + GetParameterIndex(PreviousFrame)]; }
	inline FNiagaraEmitterParameters& EditEmitterParameters(int32 EmitterIdx) { return EmitterParameters[EmitterIdx * 2 + GetParameterIndex()]; }
	
	NIAGARA_API FNiagaraWorldManager* GetWorldManager()const;
	NIAGARA_API bool RequiresGlobalDistanceField() const;
	NIAGARA_API bool RequiresDepthBuffer() const;
	NIAGARA_API bool RequiresEarlyViewData() const;
	NIAGARA_API bool RequiresViewUniformBuffer() const;
	NIAGARA_API bool RequiresRayTracingScene() const;
	NIAGARA_API bool RequiresCurrentFrameNDC() const;

	/** Requests the the simulation be reset on the next tick. */
	NIAGARA_API void Reset(EResetMode Mode);

	/** Manually advances the simulation. Ticks on the game thread. */
	NIAGARA_API void ManualTick(float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent);

	/** Manually advances the simulation by multiple ticks. Ticks on the game thread.
	 *  If MaxSimTime >= 0, then unprocessed ticks will be discarded after the time is elapsed to prevent stalling the game thread.
	 *  Returns true if we hit MaxSimTime before processing all the requested ticks
	 *
	 *  Note that this overrides a system's fixed tick delta time (if it has one) and uses DeltaSecondsPerTick instead. 
	 */
	NIAGARA_API bool MultiTick(int32 TicksToProcess, float DeltaSecondsPerTick, float MaxSimTime, const FGraphEventRef& MyCompletionGraphEvent);

	/** Ticks the system using the a SimCache. */
	void SimCacheTick_GameThread(UNiagaraSimCache* SimCache, float DesiredAge, float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent);
	/** Final step after concurrent tick has run. */
	void SimCacheTickFinalize_GameThread();

	/** Initial phase of system instance tick. Must be executed on the game thread. */
	void Tick_GameThread(float DeltaSeconds);
	/** Secondary phase of the system instance tick that can be executed on any thread. */
	void Tick_Concurrent(bool bEnqueueGPUTickIfNeeded = true);
	/** 
		Final phase of system instance tick. Must be executed on the game thread. 
		Returns whether the Finalize was actually done. It's possible for the finalize in a task to have already been done earlier on the GT by a WaitForAsyncAndFinalize call.
	*/
	void FinalizeTick_GameThread(bool bEnqueueGPUTickIfNeeded = true);

	void GenerateAndSubmitGPUTick();
	void InitGPUTick(FNiagaraGPUSystemTick& OutTick);

	void SetPendingFinalize(FNiagaraSystemInstanceFinalizeRef InFinalizeRef) { FinalizeRef = InFinalizeRef; }
	bool HasPendingFinalize() const { return FinalizeRef.IsPending(); }

	/**
	Wait for any pending concurrent work to complete, must be called on the GameThread.
	This will NOT call finalize on the instance so can leave a dangling finalize task.
	*/
	NIAGARA_API void WaitForConcurrentTickDoNotFinalize(bool bEnsureComplete = false);

	/**
	Wait for any pending concurrent work to complete, must be called on the GameThread.
	The instance will be finalized if pending, this can complete the instance and remove it from the system simulation.
	*/
	NIAGARA_API void WaitForConcurrentTickAndFinalize(bool bEnsureComplete = false);

	/** Handles completion of the system and returns true if the system is complete. */
	NIAGARA_API bool HandleCompletion();

	NIAGARA_API void SetEmitterEnable(FName EmitterName, bool bNewEnableState);

	/** Perform per-tick updates on data interfaces that need it. This can cause systems to complete so cannot be parallelized. */
	NIAGARA_API void TickDataInterfaces(float DeltaSeconds, bool bPostSimulate);

	ENiagaraExecutionState GetRequestedExecutionState()const { return RequestedExecutionState; }
	NIAGARA_API void SetRequestedExecutionState(ENiagaraExecutionState InState);

	ENiagaraExecutionState GetActualExecutionState() { return ActualExecutionState; }
	NIAGARA_API void SetActualExecutionState(ENiagaraExecutionState InState);

//	float GetSystemTimeSinceRendered() const { return SystemTimeSinceRenderedParam.GetValue(); }

	//int32 GetNumParticles(int32 EmitterIndex) const { return ParameterNumParticleBindings[EmitterIndex].GetValue(); }
	//float GetSpawnCountScale(int32 EmitterIndex) const { return ParameterSpawnCountScaleBindings[EmitterIndex].GetValue(); }

//	FVector GetOwnerVelocity() const { return OwnerVelocityParam.GetValue(); }

	inline bool IsComplete()const { return ActualExecutionState == ENiagaraExecutionState::Complete || ActualExecutionState == ENiagaraExecutionState::Disabled; }
	inline bool IsDisabled()const { return ActualExecutionState == ENiagaraExecutionState::Disabled; }

	/** Gets the simulation for the supplied emitter handle. */
	NIAGARA_API FNiagaraEmitterInstancePtr GetSimulationForHandle(const FNiagaraEmitterHandle& EmitterHandle) const;

	inline UWorld* GetWorld() const { return World; }
	inline UNiagaraSystem* GetSystem() const { return System; }
	inline USceneComponent* GetAttachComponent() const { return WeakAttachComponent.Get(); }
	inline FNiagaraUserRedirectionParameterStore* GetOverrideParameters() { return OverrideParameters; }
	inline const FNiagaraUserRedirectionParameterStore* GetOverrideParameters() const { return OverrideParameters; }
	[[nodiscard]] TArrayView<FNiagaraEmitterInstanceRef> GetEmitters() { return Emitters; }
	[[nodiscard]] TConstArrayView<FNiagaraEmitterInstanceRef> GetEmitters() const { return Emitters; }

	UE_DEPRECATED(5.5, "This function is experimental and subject to change.")
	static const INiagaraSceneComponentUtils& GetSceneComponentUtils(FNiagaraSystemInstance* Instance);

	inline const FBox& GetLocalBounds() const { return LocalBounds;  }
	inline const FVector3f& GetLWCTile() const { return LWCTile;  }
	NIAGARA_API TConstArrayView<FNiagaraEmitterExecutionIndex> GetEmitterExecutionOrder() const;

	inline void SetSystemFixedBounds(const FBox& InLocalBounds) { FixedBounds_GT = InLocalBounds; }
	NIAGARA_API FBox GetSystemFixedBounds() const;
	NIAGARA_API void SetEmitterFixedBounds(FName EmitterName, const FBox& InLocalBounds);
	NIAGARA_API FBox GetEmitterFixedBounds(FName EmitterName) const;

	NIAGARA_API FNiagaraEmitterInstance* GetEmitterByID(FNiagaraEmitterID ID)const;

	NIAGARA_API void SetForceSolo(bool bForceSolo);
	inline bool IsSolo() const { return bSolo; }

	inline bool NeedsGPUTick() const { return ActiveGPUEmitterCount > 0 /*&& Component->IsRegistered()*/ && !IsComplete();}

	FNiagaraSystemGpuComputeProxy* GetSystemGpuComputeProxy() { return SystemGpuComputeProxy.Get(); }

	/** Gets a multicast delegate which is called after this instance has finished ticking for the frame on the game thread */
	inline void SetOnPostTick(const FOnPostTick& InPostTickDelegate) { OnPostTickDelegate = InPostTickDelegate; }
	/** Gets a multicast delegate which is called whenever this instance is complete. */
	inline void SetOnComplete(const FOnComplete& InOnCompleteDelegate) { OnCompleteDelegate = InOnCompleteDelegate; }

#if WITH_EDITOR
	/** Gets a multicast delegate which is called whenever this instance is initialized with an System asset. */
	NIAGARA_API FOnInitialized& OnInitialized();

	/** Gets a multicast delegate which is called whenever this instance is reset due to external changes in the source System asset. */
	NIAGARA_API FOnReset& OnReset();

	NIAGARA_API FOnDestroyed& OnDestroyed();
#endif

#if WITH_EDITORONLY_DATA
	NIAGARA_API bool GetIsolateEnabled() const;
#endif

	FNiagaraSystemInstanceID GetId() const { return ID; }

	/** Returns the instance data for a particular interface for this System. */
	inline void* FindDataInterfaceInstanceData(const UNiagaraDataInterface* Interface)
	{
		if (auto* InstDataOffsetPair = DataInterfaceInstanceDataOffsets.FindByPredicate([&](auto& Pair){ return Pair.Key.Get() == Interface;}))
		{
			return &DataInterfaceInstanceData[InstDataOffsetPair->Value];
		}
		return nullptr;
	}

	template<typename TDataType>
	inline TDataType* FindTypedDataInterfaceInstanceData(const UNiagaraDataInterface* Interface)
	{
		if (auto* InstDataOffsetPair = DataInterfaceInstanceDataOffsets.FindByPredicate([&](auto& Pair) { return Pair.Key.Get() == Interface; }))
		{
			return reinterpret_cast<TDataType*>(&DataInterfaceInstanceData[InstDataOffsetPair->Value]);
		}
		return nullptr;
	}

	inline int32 FindDataInterfaceInstanceDataIndex(UNiagaraDataInterface* Interface)
	{
		return DataInterfaceInstanceDataOffsets.IndexOfByPredicate([&](auto& Pair) { return Pair.Key.Get() == Interface; });
	}

	inline const void GetDataInterfaceInstanceDataInfo(int32 Index, UNiagaraDataInterface*& OutInterface, void*& OutInstanceData)
	{
		if(DataInterfaceInstanceDataOffsets.IsValidIndex(Index))
		{
			OutInterface = DataInterfaceInstanceDataOffsets[Index].Key.Get();
			OutInstanceData = &DataInterfaceInstanceData[DataInterfaceInstanceDataOffsets[Index].Value];
		}
		else
		{
			OutInterface = nullptr;
			OutInstanceData = nullptr;
		}
	}
	
	inline const void* FindDataInterfaceInstanceData(const UNiagaraDataInterface* Interface) const { return const_cast<FNiagaraSystemInstance*>(this)->FindDataInterfaceInstanceData(Interface); }
	template<typename TDataType>
	inline const TDataType* FindTypedDataInterfaceInstanceData(const UNiagaraDataInterface* Interface) const { return const_cast<FNiagaraSystemInstance*>(this)->FindTypedDataInterfaceInstanceData<TDataType>(Interface); }

	inline const FNiagaraPerInstanceDIFuncInfo& GetPerInstanceDIFunction(ENiagaraSystemSimulationScript ScriptType, int32 FuncIndex)const { return PerInstanceDIFunctions[(int32)ScriptType][FuncIndex]; }

	NIAGARA_API void EvaluateBoundFunction(FName FunctionName, bool& UsedOnCpu, bool& UsedOnGpu) const;

#if WITH_EDITORONLY_DATA
	NIAGARA_API bool UsesCollection(const UNiagaraParameterCollection* Collection) const;

	FORCEINLINE int32 GetPerInstanceDIFunctionNum(ENiagaraSystemSimulationScript ScriptType) const { return PerInstanceDIFunctions[(int32)ScriptType].Num(); }
#endif

	inline bool IsPendingSpawn() const { return (SystemInstanceState == ENiagaraSystemInstanceState::PendingSpawn) || (SystemInstanceState == ENiagaraSystemInstanceState::PendingSpawnPaused); }

	inline float GetAge() const { return Age; }
	inline int32 GetTickCount() const { return TickCount; }

	inline float GetLastRenderTime() const { return LastRenderTime; }
	inline void SetLastRenderTime(float TimeSeconds) { LastRenderTime = TimeSeconds; }
	
	inline TSharedPtr<FNiagaraSystemSimulation, ESPMode::ThreadSafe> GetSystemSimulation()const
	{
		return SystemSimulation; 
	}

	NIAGARA_API bool IsReadyToRun() const;

	const FNiagaraParameterCollectionResolver CreateParameterCollectionResolver() const;

	/** 
	Manually advances this system's simulation by the specified number of ticks and tick delta. 
	To be advanced in this way a system must be in solo mode or moved into solo mode which will add additional overhead.
	*/
	NIAGARA_API void AdvanceSimulation(int32 TickCountToSimulate, float TickDeltaSeconds);

#if NIAGARA_SYSTEM_CAPTURE
	/** Request that this simulation capture a frame. Cannot capture if disabled or already completed.*/
	NIAGARA_API bool RequestCapture(const FGuid& RequestId);

	/** Poll for previous frame capture requests. Once queried and bool is returned, the results are cleared from this system instance.*/
	NIAGARA_API bool QueryCaptureResults(const FGuid& RequestId, TArray<TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe>>& OutCaptureResults);

	/** Only call from within the script execution states. Value is null if not capturing a frame.*/
	NIAGARA_API TArray<TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe>>* GetActiveCaptureResults();

	/** Only call from within the script execution states. Does nothing if not capturing a frame.*/
	NIAGARA_API void FinishCapture();

	/** Only call from within the script execution states. Value is false if not capturing a frame.*/
	NIAGARA_API bool ShouldCaptureThisFrame() const;

	/** Only call from within the script execution states. Value is nullptr if not capturing a frame.*/
	NIAGARA_API TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe> GetActiveCaptureWrite(const FName& InHandleName, ENiagaraScriptUsage InUsage, const FGuid& InUsageId);

#endif

	/** Dumps all of this systems info to the log. */
	NIAGARA_API void Dump()const;

	/** Dumps information about the instances tick to the log */
	NIAGARA_API void DumpTickInfo(FOutputDevice& Ar);

	FNiagaraGpuComputeDispatchInterface* GetComputeDispatchInterface() const { return ComputeDispatchInterfacePtr.Get(); }

	static NIAGARA_API bool AllocateSystemInstance(FNiagaraSystemInstancePtr& OutSystemInstanceAllocation, UWorld& InWorld, UNiagaraSystem& InAsset,
		FNiagaraUserRedirectionParameterStore* InOverrideParameters = nullptr, USceneComponent* InAttachComponent = nullptr,
		ENiagaraTickBehavior InTickBehavior = ENiagaraTickBehavior::UsePrereqs, bool bInPooled = false);
	static NIAGARA_API bool DeallocateSystemInstance(FNiagaraSystemInstancePtr& SystemInstanceAllocation);
	/*void SetHasGPUEmitters(bool bInHasGPUEmitters) { bHasGPUEmitters = bInHasGPUEmitters; }*/
	bool HasGPUEmitters() { return bHasGPUEmitters;  }

	NIAGARA_API void TickInstanceParameters_GameThread(float DeltaSeconds);

	NIAGARA_API void TickInstanceParameters_Concurrent();

	NIAGARA_API FNiagaraDataSet* CreateEventDataSet(FName EmitterName, FName EventName);
	NIAGARA_API FNiagaraDataSet* GetEventDataSet(FName EmitterName, FName EventName) const;
	NIAGARA_API void ClearEventDataSets();

	inline void SetLODDistance(float InLODDistance, float InMaxLODDistance, bool bOverride);
	inline void ClearLODDistance();

	NIAGARA_API const FString& GetCrashReporterTag()const;

#if WITH_EDITOR
	NIAGARA_API void RaiseNeedsUIResync();
	NIAGARA_API bool HandleNeedsUIResync();
#endif

	/** Get the current tick behavior */
	ENiagaraTickBehavior GetTickBehavior() const { return TickBehavior; }
	/** Set a new tick behavior, this will not move the instance straight away and will wait until the next time it is evaluated */
	NIAGARA_API void SetTickBehavior(ENiagaraTickBehavior NewTickBehavior);
	
	/** Calculates which tick group the instance should be in. */
	NIAGARA_API ETickingGroup CalculateTickGroup() const;

	/** Gets the current world transform of the system */
	inline const FTransform& GetWorldTransform() const { return WorldTransform; }
	/** Sets the world transform */
	inline void SetWorldTransform(const FTransform& InTransform) { WorldTransform = InTransform; }

	int32 GetSystemInstanceIndex() const 
	{
		return SystemInstanceIndex;
	}

	/**
	The significant index for this component. i.e. this is the Nth most significant instance of it's system in the scene.
	Passed to the script to allow us to scale down internally for less significant systems instances.
*/
	inline void SetSystemSignificanceIndex(int32 InIndex) { SignificanceIndex = InIndex; }

	/** Calculates the distance to use for distance based LODing / culling. */
	NIAGARA_API float GetLODDistance();
	float GetMaxLODDistance() const { return MaxLODDistance ;}

	NIAGARA_API void OnSimulationDestroyed();

	void SetRandomSeedOffset(int32 Offset) { RandomSeedOffset = Offset; }
	int32 GetRandomSeedOffset() const { return RandomSeedOffset; }

	NIAGARA_API FNDIStageTickHandler* GetSystemDIStageTickHandler(ENiagaraScriptUsage Usage);
private:
	NIAGARA_API void DumpStalledInfo();

	NIAGARA_API void DestroyDataInterfaceInstanceData();

	/** Builds the emitter simulations. */
	NIAGARA_API void InitEmitters();

	/** Resets the System, emitter simulations, and renderers to initial conditions. */
	NIAGARA_API void ReInitInternal();
	/** Resets for restart, assumes no change in emitter setup */
	NIAGARA_API void ResetInternal(bool bResetSimulations);

	/** Resets the parameter structs */
	NIAGARA_API void ResetParameters();

	/** Call PrepareForSImulation on each data source from the simulations and determine which need per-tick updates.*/
	NIAGARA_API void InitDataInterfaces();

	NIAGARA_API void ResolveUserDataInterfaceBindings();

	/** The LWC tile of this system instance, used to offset all local simulation relative to the origin */
	FVector3f LWCTile = FVector3f::ZeroVector;

	/** Index of this instance in the system simulation. */
	int32 SystemInstanceIndex;

	/** Index of how significant this system is in the scene. 0 = Most significant instance of this systems in the scene. */
	int32 SignificanceIndex;

	TSharedPtr<class FNiagaraSystemSimulation, ESPMode::ThreadSafe> SystemSimulation;

	UWorld* World;
	UNiagaraSystem* System = nullptr;
	FNiagaraUserRedirectionParameterStore* OverrideParameters;
	TWeakObjectPtr<USceneComponent> WeakAttachComponent;
	TUniquePtr<INiagaraSceneComponentUtils> SceneComponentUtils;

	FTransform WorldTransform;
	TOptional<FVector> PreviousLocation;

	/** The age of the System instance. */
	float Age;

	/** The last time this system rendered */
	float LastRenderTime;

	/** The tick count of the System instance. */
	int32 TickCount;

	/** Random seed used for system simulation random number generation. */
	int32 RandomSeed;

	/** A system-wide offset to permute the deterministic random seed (allows for variance among multiple instances while still being deterministic) */
	int32 RandomSeedOffset;

	/** LODDistance driven by our component. */
	float LODDistance;
	float MaxLODDistance;

	int32 WarmupTickCount = -1;
	float WarmupTickDelta = 0;
	
	FName IDName;
	FNiagaraSystemInstanceID ID;

	//-TODO:Stateless:
	//TArray<TSharedRef<FNiagaraEmitterInstanceImpl, ESPMode::ThreadSafe>> Emitters;
	TArray<FNiagaraEmitterInstanceRef> Emitters;

	FOnPostTick OnPostTickDelegate;
	FOnComplete OnCompleteDelegate;

#if WITH_EDITOR
	FOnInitialized OnInitializedDelegate;

	FOnReset OnResetDelegate;
	FOnDestroyed OnDestroyedDelegate;
#endif

#if NIAGARA_SYSTEM_CAPTURE
	TSharedPtr<TArray<TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe>>, ESPMode::ThreadSafe> CurrentCapture;
	TSharedPtr<FGuid, ESPMode::ThreadSafe> CurrentCaptureGuid;
	bool bWasSoloPriorToCaptureRequest;
	TMap<FGuid, TSharedPtr<TArray<TSharedPtr<struct FNiagaraScriptDebuggerInfo, ESPMode::ThreadSafe>>, ESPMode::ThreadSafe>> CapturedFrames;
#endif


	/** Per instance data for any data interfaces requiring it. */
	TArray<uint8, TAlignedHeapAllocator<16>> DataInterfaceInstanceData;
	TArray<int32> PreTickDataInterfaces;
	TArray<int32> PostTickDataInterfaces;

	/** Map of data interfaces to their instance data. */
	TArray<TPair<TWeakObjectPtr<UNiagaraDataInterface>, int32>> DataInterfaceInstanceDataOffsets;

	FNDIStageTickHandler SystemSpawnDIStageTickHandler;
	FNDIStageTickHandler SystemUpdateDIStageTickHandler;

	/** 
	A set of function bindings for DI calls that must be made per system instance.
	*/
	TArray<FNiagaraPerInstanceDIFuncInfo> PerInstanceDIFunctions[(int32)ENiagaraSystemSimulationScript::Num];

	/** Per system instance parameters. These can be fed by the component and are placed into a dataset for execution for the system scripts. */
	FNiagaraParameterStore InstanceParameters;
	
	static constexpr int32 ParameterBufferCount = 2;
	FNiagaraGlobalParameters GlobalParameters[ParameterBufferCount];
	FNiagaraSystemParameters SystemParameters[ParameterBufferCount];
	FNiagaraOwnerParameters OwnerParameters[ParameterBufferCount];
	TArray<FNiagaraEmitterParameters> EmitterParameters;

	// registered events for each of the emitters
	typedef TPair<FName, FName> EmitterEventKey;
	typedef TMap<EmitterEventKey, FNiagaraDataSet*> EventDataSetMap;
	EventDataSetMap EmitterEventDataSetMap;

	/** Used for double buffered global/system/emitter parameters */
	uint8 CurrentFrameIndex : 1;
	uint8 ParametersValid : 1;

	/** Indicates whether this instance must update itself rather than being batched up as most instances are. */
	uint8 bSolo : 1;
	uint8 bForceSolo : 1;

	uint8 bNotifyOnCompletion : 1;

	/** If this system has emitters that will run GPU Simulations */
	uint8 bHasGPUEmitters : 1;
	/** The system contains data interfaces that can have tick group prerequisites. */
	uint8 bDataInterfacesHaveTickPrereqs : 1;
	/** The system contains data interfaces that can have tick group post requisites. */
	uint8 bDataInterfacesHaveTickPostreqs : 1;

	uint8 bDataInterfacesInitialized : 1;

	uint8 bAlreadyBound : 1;

	uint8 bLODDistanceIsValid : 1;
	uint8 bLODDistanceIsOverridden : 1;

	/** True if the system instance is pooled. Prevents unbinding of parameters on completing the system */
	uint8 bPooled : 1;

#if WITH_EDITOR
	uint8 bNeedsUIResync : 1;
#endif

public:
	uint8 GPUParamIncludeInterpolation : 1 = false;

private:

	/** Graph event to track pending concurrent work. */
	FGraphEventRef ConcurrentTickGraphEvent;
	FGraphEventRef ConcurrentTickBatchGraphEvent;

	/** When using concurrent ticking this will be valid until it's complete. */
	FNiagaraSystemInstanceFinalizeRef FinalizeRef;

	/** Cached delta time, written during Tick_GameThread and used during other phases. */
	float CachedDeltaSeconds;

	/** Time since we last forced a bounds update. */
	float TimeSinceLastForceUpdateTransform;

	/** Optional user specified bounds. */
	FBox FixedBounds_GT;
	FBox FixedBounds_CNC;

	/** Current calculated local bounds. */
	FBox LocalBounds;

	/* Execution state requested by external code/BPs calling Activate/Deactivate. */
	ENiagaraExecutionState RequestedExecutionState;

	/** Copy of simulations internal state so that it can be passed to emitters etc. */
	ENiagaraExecutionState ActualExecutionState;

	TSharedPtr<FNiagaraGpuComputeDispatchInterface> ComputeDispatchInterfacePtr;
	TUniquePtr<FNiagaraSystemGpuComputeProxy> SystemGpuComputeProxy;

	/** Tag we feed into crash reporter for this instance. */
	mutable FString CrashReporterTag;

	/** The feature level of for this component instance. */
	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num;

public:

	ERHIFeatureLevel::Type GetFeatureLevel() const { return FeatureLevel; }

	// Transient data that is accumulated during tick.
	uint32 TotalGPUParamSize = 0;
	uint32 ActiveGPUEmitterCount = 0;

	int32 GPUDataInterfaceInstanceDataSize = 0;
	TArray<TPair<TWeakObjectPtr<UNiagaraDataInterface>, int32>> GPUDataInterfaces;

	struct FInstanceParameters
	{
		FTransform ComponentTrans = FTransform::Identity;
		FVector Velocity = FVector::ZeroVector;

		float DeltaSeconds = 0.0f;
		float TimeSeconds = 0.0f;
		float RealTimeSeconds = 0.0f;

		// delta time coming from the engine (not including fixed delta time, etc)
		float WorldDeltaSeconds = 0.0f;

		int32 EmitterCount = 0;
		int32 NumAlive = 0;

		ENiagaraExecutionState RequestedExecutionState = ENiagaraExecutionState::Active;

		void Init(int32 NumEmitters)
		{
			ComponentTrans = FTransform::Identity;
			Velocity = FVector::ZeroVector;
			DeltaSeconds = 0.0f;
			TimeSeconds = 0.0f;
			RealTimeSeconds = 0.0f;

			WorldDeltaSeconds = 0.0f;

			EmitterCount = 0;
			NumAlive = 0;

			RequestedExecutionState = ENiagaraExecutionState::Active;
		}
	};

	FRandomStream	SystemState_RandomStream;

	FInstanceParameters GatheredInstanceParameters;

	void InitSystemState();
	void TickSystemState();

	int32			SystemState_LoopCount = 0;
	float			SystemState_CurrentLoopDuration = 0.0f;
	float			SystemState_CurrentLoopDelay = 0.0f;
	float			SystemState_CurrentLoopAgeStart = 0.0f;
	float			SystemState_CurrentLoopAgeEnd = 0.0f;
};

inline void FNiagaraSystemInstance::SetLODDistance(float InLODDistance, float InMaxLODDistance, bool bOverride)
{
	bLODDistanceIsOverridden = bOverride;
	bLODDistanceIsValid = true;
	LODDistance = InLODDistance; 
	MaxLODDistance = InMaxLODDistance;
}

inline void FNiagaraSystemInstance::ClearLODDistance()
{
	bLODDistanceIsOverridden = false;
	bLODDistanceIsValid = false;
	LODDistance = 0.0f;
	MaxLODDistance = 1.0f;
}
