// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemSimulation.h"
#include "NiagaraSystemRenderData.h"
#include "NiagaraWorldManager.h"

#ifndef NIAGARA_SYSTEM_INSTANCE_CONTROLLER_ASYNC
/** When true, instance handle operations are asynchronous. Otherwise, the interface is pass-through */
#define NIAGARA_SYSTEM_INSTANCE_CONTROLLER_ASYNC 0
#endif

#if NIAGARA_SYSTEM_INSTANCE_CONTROLLER_ASYNC
#define NIAGARA_SYSTEM_INSTANCE_CONTROLLER_DEFFERED_METHOD(MethodName) \
	template <typename... ArgTypes> \
	inline void MethodName ## _Deferred(ArgTypes... Args) { \
		using ThisClass = typename TRemovePointer<decltype(this)>::Type; \
		WorldManager->GetDeferredMethodQueue().Enqueue(FNiagaraDeferredMethodQueue::FMethod::CreateThreadSafeSP(this, &ThisClass::MethodName, Forward<ArgTypes>(Args)...)); \
	}
#else
#define NIAGARA_SYSTEM_INSTANCE_CONTROLLER_DEFFERED_METHOD(MethodName) \
	template <typename... ArgTypes> inline void MethodName ## _Deferred(ArgTypes... Args) { this->MethodName(Forward<ArgTypes>(Args)...); }
#endif

/** Used to expose FNiagaraSystemInstance methods without actually providing the interface */
#define NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(MethodName, FuncMod) \
	template <typename... ArgTypes> \
	inline auto MethodName(ArgTypes... Args) FuncMod { ensure(IsValid()); return SystemInstance->MethodName(Forward<ArgTypes>(Args)...); }

class UMaterialInterface;
class UTextureRenderTarget;
class FNiagaraSystemInstanceController;
class FNiagaraSceneProxy;
class FStreamingTextureLevelContext;

using FNiagaraSystemInstanceControllerPtr = TSharedPtr<FNiagaraSystemInstanceController, ESPMode::ThreadSafe>;
using FNiagaraSystemInstanceControllerConstPtr = TSharedPtr<const FNiagaraSystemInstanceController, ESPMode::ThreadSafe>;

struct FNiagaraMaterialAndScale
{
	FNiagaraMaterialAndScale() = default;
	explicit FNiagaraMaterialAndScale(UMaterialInterface* InMaterial, float InScale) : Material(InMaterial), Scale(InScale) {}

	UMaterialInterface* Material = nullptr;
	float Scale = 1.0f;
};

using FNiagaraMaterialAndScaleArray = TArray<FNiagaraMaterialAndScale, TInlineAllocator<16>>;

/**
 * This is the main asynchronous interface for controlling operation of a single instance of a Niagara System.
 */
class FNiagaraSystemInstanceController
	: public TSharedFromThis<FNiagaraSystemInstanceController, ESPMode::ThreadSafe>
	, private FGCObject
{
public:
	DECLARE_DELEGATE(FOnMaterialsUpdated);

	FNiagaraSystemInstanceController();
	FNiagaraSystemInstanceController(const FNiagaraSystemInstanceController&) = delete;
	~FNiagaraSystemInstanceController() { Release(); }

	/** Initializes the controller with an instance of the provided system */
	void Initialize(UWorld& World, UNiagaraSystem& System, FNiagaraUserRedirectionParameterStore* OverrideParameters, USceneComponent* AttachComponent,
		ENiagaraTickBehavior TickBehavior, bool bPooled, int32 RandomSeedOffset, bool bForceSolo, int32 WarmupTickCount, float WarmupTickDelta);
	/** Deactivates the underlying system and queues it up for destruction. After calling, this interface is unusable unless you call Initialize again */
	void Release();

	inline bool IsValid() const { return SystemInstance.IsValid(); }
	bool HasValidSimulation() const;

	// Render data operations
	FNiagaraSystemRenderData* CreateSystemRenderData(ERHIFeatureLevel::Type FeatureLevel) const;
	void GenerateSetDynamicDataCommands(FNiagaraSystemRenderData::FSetDynamicDataCommandList& Commands, FNiagaraSystemRenderData& RenderData, const FNiagaraSceneProxy& SceneProxy);
	void PostTickRenderers(FNiagaraSystemRenderData& RenderData);
	void NotifyRenderersComplete(FNiagaraSystemRenderData& RenderData);

	// These methods are deprecated, as they expose internals that may be concurrently accessed
	FNiagaraSystemInstance* GetSystemInstance_Unsafe() const { return SystemInstance.Get(); }
	FNiagaraSystemSimulationPtr GetSystemSimulation_Unsafe() const { return SystemInstance->GetSystemSimulation(); }

	// Solo systems should be okay to access the system instance/simulation, as they are ticked manually, not concurrently
	FNiagaraSystemInstance* GetSoloSystemInstance() const { return ensure(IsValid() && SystemInstance->IsSolo()) ? SystemInstance.Get() : nullptr; }
	FNiagaraSystemSimulationPtr GetSoloSystemSimulation() const { return ensure(IsValid() && SystemInstance->IsSolo()) ? SystemInstance->GetSystemSimulation() : nullptr; }

	FNiagaraSystemInstanceID GetSystemInstanceID() const { ensure(IsValid()); return SystemInstance->GetId(); }
	
	void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials);
	UMaterialInterface* GetMaterialOverride(const UNiagaraRendererProperties* InProps, int32 InMaterialSubIndex) const;
	void SetOnMaterialsUpdated(const FOnMaterialsUpdated& Delegate) { OnMaterialsUpdatedDelegate = Delegate; }

	void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& List) const;

	void GetMaterialStreamingInfo(FNiagaraMaterialAndScaleArray& OutMaterialAndScales) const;
	void GetStreamingMeshInfo(const FBoxSphereBounds& OwnerBounds, FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const;

	ENiagaraGpuComputeTickStage::Type GetGpuComputeTickStage() const;

	/** Dumps system instance state and info to the log (Used by fx.Niagara.DumpComponents commandlet) */
	void DebugDump(bool bFullDump);
	SIZE_T GetTotalBytesUsed() const;

	// Deferred methods - With NIAGARA_SYSTEM_INSTANCE_CONTROLLER_ASYNC, these are executed in a deferred queue
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_DEFFERED_METHOD(Reset)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_DEFFERED_METHOD(AdvanceSimulation)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_DEFFERED_METHOD(SetPaused)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_DEFFERED_METHOD(SetVariable)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_DEFFERED_METHOD(SetRequestedExecutionState)

	// These methods are currently always pass-through to the SystemInstance. Most of them are in line to become deferred or removed from this interface.
	// The goal is to get this list of methods down to only accessors that don't change throughout the lifetime of the system
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetSystem, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetRandomSeedOffset,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetOnPostTick,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetOnComplete,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetGpuComputeDebug,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetRequestedExecutionState, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetActualExecutionState, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetEmitterEnable,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetSystemSignificanceIndex,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetLODDistance, )
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(ClearLODDistance, )
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetLODDistance, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetAreDataInterfacesInitialized, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetAge, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetComputeDispatchInterface, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(IsSolo, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetForceSolo,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(IsComplete, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(IsPendingSpawn, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(Reset,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(ManualTick,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(MultiTick,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SimCacheTick_GameThread,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetTickBehavior,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetTickBehavior,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(AdvanceSimulation,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetPaused,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(IsPaused, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetLastRenderTime,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(Activate,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(Deactivate,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(CalculateTickGroup, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetLocalBounds, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetSystemFixedBounds,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetSystemFixedBounds, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetEmitterFixedBounds,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetEmitterFixedBounds, const)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(WaitForConcurrentTickAndFinalize,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(OnSimulationDestroyed,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(GetAttachComponent,)
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(CreateParameterCollectionResolver, )
	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(RebindParameterCollection,)
	#if WITH_EDITOR
		NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(HandleNeedsUIResync,)
	#endif
	void OnPooledReuse(UWorld& NewWorld) { SystemInstance->OnPooledReuse(NewWorld); }

	//Internal use only. DO NOT USE.
	void SetVariable_InternalUseOnly_Deferred(const FNiagaraVariableBase& Variable, const FNiagaraVariant& Variant);
private:
	using FMaterialOverride = FNiagaraSystemRenderData::FMaterialOverride;

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return "FNiagaraSystemInstanceController"; }

	void UpdateEmitterMaterials();
	void SetVariable(FName InVariableName, bool InValue);
	void SetVariable(FName InVariableName, int32 InValue);
	void SetVariable(FName InVariableName, float InValue);
	void SetVariable(FName InVariableName, FVector2f InValue);
	void SetVariable(FName InVariableName, FVector3f InValue);
	void SetVariable(FName InVariableName, FVector InValue);
	void SetVariable(FName InVariableName, FVector4f InValue);
	void SetVariable(FName InVariableName, FLinearColor InValue);
	void SetVariable(FName InVariableName, FQuat4f InValue);
	void SetVariable(FName InVariableName, const FMatrix44f& InValue);
	void SetVariable(FName InVariableName, TWeakObjectPtr<UObject> Object);
	void SetVariable(FName InVariableName, TWeakObjectPtr<UMaterialInterface> Object);
	void SetVariable(FName InVariableName, TWeakObjectPtr<UStaticMesh> Object);
	void SetVariable(FName InVariableName, TWeakObjectPtr<UTexture> Texture);
	void SetVariable(FName InVariableName, TWeakObjectPtr<UTextureRenderTarget> TextureRenderTarget);

	void SetVariable(FNiagaraVariableBase Variable, FNiagaraVariant Variant);

	NIAGARA_SYSTEM_INSTANCE_CONTROLLER_SHIM(SetRequestedExecutionState,)

	void OnNeedsRendererRecache() { bNeedsRendererRecache = true; }

private:

	FNiagaraWorldManager* WorldManager = nullptr;
	FNiagaraUserRedirectionParameterStore* OverrideParameters = nullptr;
	FNiagaraSystemInstancePtr SystemInstance;
	TArray<FMaterialOverride> EmitterMaterials;
	FOnMaterialsUpdated OnMaterialsUpdatedDelegate;
	uint32 bNeedsRendererRecache : 1;
	uint32 bNeedsOverrideParametersTicked : 1;
	uint32 bNeedsUpdateEmitterMaterials : 1;
};
