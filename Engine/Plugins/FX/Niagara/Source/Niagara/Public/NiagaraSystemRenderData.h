// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraRenderer.h"
#include "PrimitiveUniformShaderParameters.h"
#include "PrimitiveViewRelevance.h"

class FNiagaraSystemInstance;
class FNiagaraSystemRenderData;
class FMeshElementCollector;
struct FNiagaraDynamicDataBase;
class FNiagaraSceneProxy;
class FSceneView;

#if RHI_RAYTRACING
struct FRayTracingInstance;
#endif

/**
 * This class wraps all data and functionality needed by a scene render proxy to render a single Niagara System Instance.
 */
class FNiagaraSystemRenderData
{
public:
	struct FMaterialOverride
	{
		TObjectPtr<UMaterialInterface> Material = nullptr;
		const UNiagaraRendererProperties* EmitterRendererProperty = nullptr; // Pointer only used for lookups, don't actually dereference
		uint32 MaterialSubIndex = 0;
	};

	struct FSetDynamicDataCommand
	{
		FNiagaraRenderer* Renderer = nullptr;
		FNiagaraDynamicDataBase* DynamicData = nullptr;

		FSetDynamicDataCommand() {}
		FSetDynamicDataCommand(FNiagaraRenderer* InRenderer, FNiagaraDynamicDataBase* InDynamicData) : Renderer(InRenderer), DynamicData(InDynamicData) {}
		void Execute() const { Renderer->SetDynamicData_RenderThread(DynamicData); }
	};

	using FSetDynamicDataCommandList = TArray<FSetDynamicDataCommand, TInlineAllocator<8>>;

	static void ExecuteDynamicDataCommands_RenderThread(const FSetDynamicDataCommandList& Commands);

public:
	FNiagaraSystemRenderData(const FNiagaraSystemInstanceController& SystemInstanceController, const FNiagaraSystemInstance& SystemInstance, ERHIFeatureLevel::Type FeatureLevel);
	FNiagaraSystemRenderData(const FNiagaraSystemRenderData&) = delete;
	~FNiagaraSystemRenderData();

	/** Called from the render thread to give renderers an opportunity to create resources needed for rendering. */
	void CreateRenderThreadResources(FRHICommandListBase& RHICmdList);
	/** Called from the render thread to give renderers an opportunity to release their resources. */
	void ReleaseRenderThreadResources();
	/** Gives the system's renderers an opportunity to free resources */
	void DestroyRenderState_Concurrent();
	/** This must be called on the RenderThread before destruction to properly clean up resources without data race issues (i.e. from a scene proxy's destructor) */
	void Destroy_RenderThread();

	/** Gets the relevance of this NiagaraSystem for this view */
	FPrimitiveViewRelevance GetViewRelevance(const FSceneView& View, const FNiagaraSceneProxy& SceneProxy) const;
	/** Gets the total size of dynamic data allocated */
	uint32 GetDynamicDataSize() const;
	/** Called at the end of the frame, before rendering when necessary to provide renderers with render thread data */
	void GenerateSetDynamicDataCommands(FSetDynamicDataCommandList& Commands, const FNiagaraSceneProxy& SceneProxy, const FNiagaraSystemInstance* SystemInstance, TConstArrayView<FMaterialOverride> MaterialOverrides);
	/** Gets the dynamic mesh elements from all renderers */
	void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector, const FNiagaraSceneProxy& SceneProxy);
#if RHI_RAYTRACING
	void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FNiagaraSceneProxy& SceneProxy);
#endif
	void GatherSimpleLights(FSimpleLightArray& OutParticleLights) const;

	void PostTickRenderers(const FNiagaraSystemInstance& SystemInstance);
	void OnSystemComplete(const FNiagaraSystemInstance& SystemInstance);
	void RecacheRenderers(const FNiagaraSystemInstance& SystemInstance, const FNiagaraSystemInstanceController& Controller);

	void SetRenderingEnabled_GT(bool bInEnabled);
	inline bool IsRenderingEnabled_GT() const { return bRenderingEnabled_GT && EmitterRenderers_GT.Num() > 0; }
	inline bool IsRenderingEnabled_RT() const { return bRenderingEnabled_RT && EmitterRenderers_RT.Num() > 0; }

	inline bool HasAnyMotionBlurEnabled() const { return bAnyMotionBlurEnabled; }
	inline bool HasAnyHeterogeneousVolumesEnabled() const { return bAnyHeterogeneousVolumesEnabled; }

	inline int32 GetNumRenderers() const { return RendererDrawOrder.Num(); }

	//FVector3f LWCRenderTile = FVector3f::ZeroVector;
private:
	/** Emitter Renderers in the order they appear in the emitters. To be accessed by the GameThread */
	TArray<FNiagaraRenderer*> EmitterRenderers_GT;
	/** Emitter Renderers in the order they appear in the emitters. To be accessed by the RenderThread */
	TArray<FNiagaraRenderer*> EmitterRenderers_RT;

	/** Indices of renderers in the order they should be rendered. */
	TArray<int32> RendererDrawOrder;

	bool bRenderingEnabled_GT = true;
	bool bRenderingEnabled_RT = true;
	bool bAnyMotionBlurEnabled = false;
	bool bAnyHeterogeneousVolumesEnabled = false;

	ERHIFeatureLevel::Type FeatureLevel;
};
