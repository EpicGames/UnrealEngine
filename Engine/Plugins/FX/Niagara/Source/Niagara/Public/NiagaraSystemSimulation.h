// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraDataSet.h"
#include "NiagaraScriptExecutionContext.h"
#include "NiagaraSystem.h"

class UNiagaraEffectType;
class UWorld;
class FNiagaraSystemSimulation;
class FNiagaraGpuComputeDispatchInterface;
struct FNiagaraParameterCollectionResolver;

using FNiagaraSystemSimulationPtr = TSharedPtr<FNiagaraSystemSimulation, ESPMode::ThreadSafe>;

enum class ENiagaraGPUTickHandlingMode
{
	None, /** No GPU Ticks needed. */
	GameThread,/** Each system has to submit it's GPU tick individually on the game thread. */
	Concurrent,/** Each system has to submit it's GPU tick individually during it's concurrent tick. */
	GameThreadBatched,/** Systems can submit their GPU ticks in batches but it must be done on the game thread. */
	ConcurrentBatched,/** Systems can submit their GPU ticks in batches during concurrent tick. */
};

struct FNiagaraTickInfo
{
	bool UsesFixedTick = false;
	float EngineTick = 0.0;
	float SystemTick = 0.0;
	int32 TickCount = 0;
	int32 TickNumber = 0;
	float TimeStepFraction = 0.0;
};

//TODO: It would be good to have the batch size be variable per system to try to keep a good work/overhead ratio.
//Can possibly adjust in future based on average batch execution time.
#define NiagaraSystemTickBatchSize 4
typedef TArray<FNiagaraSystemInstance*, TInlineAllocator<NiagaraSystemTickBatchSize>> FNiagaraSystemTickBatch;

//TODO: Pull all the layout information here, in the data set and in parameter stores out into a single layout structure that's shared between all instances of it.
//Currently there's tons of extra data and work done setting these up.
struct FNiagaraParameterStoreToDataSetBinding
{
	//Array of floats offsets
	struct FDataOffsets
	{
		/** Offset of this value in the parameter store. */
		int32 ParameterOffset;
		/** Offset of this value in the data set's components. */
		int32 DataSetComponentOffset;
		FDataOffsets(int32 InParamOffset, int32 InDataSetComponentOffset) : ParameterOffset(InParamOffset), DataSetComponentOffset(InDataSetComponentOffset) {}
	};
	struct FHalfDataOffsets : public FDataOffsets
	{
		bool ApplyAsFloat;
		FHalfDataOffsets(int32 InParamOffset, int32 InDataSetComponentOffset, bool InApplyAsFloat) : FDataOffsets(InParamOffset, InDataSetComponentOffset), ApplyAsFloat(InApplyAsFloat) {}
	};

	TArray<FDataOffsets> FloatOffsets;
	TArray<FDataOffsets> Int32Offsets;
	TArray<FHalfDataOffsets> HalfOffsets;

	void Empty()
	{
		FloatOffsets.Empty();
		Int32Offsets.Empty();
		HalfOffsets.Empty();
	}

	void Init(FNiagaraDataSet& DataSet, const FNiagaraParameterStore& ParameterStore)
	{
		//For now, until I get time to refactor all the layout info into something more coherent we'll init like this and just have to assume the correct layout sets and stores are used later.
		//Can check it but it'd be v slow.
		const auto& DataSetVariables = DataSet.GetVariables();
		const auto& DataSetVariableLayouts = DataSet.GetVariableLayouts();
		for (int i=0; i < DataSetVariables.Num(); ++i)
		{
			const FNiagaraVariableBase& Var = DataSetVariables[i];
			const int32* ParameterOffsetPtr = ParameterStore.FindParameterOffset(Var, true);
			if (ParameterOffsetPtr == nullptr)
			{
				continue;
			}

			const FNiagaraVariableLayoutInfo& Layout = DataSetVariableLayouts[i];
			const int32 ParameterOffset = *ParameterOffsetPtr;
			for (uint32 CompIdx = 0; CompIdx < Layout.GetNumFloatComponents(); ++CompIdx)
			{
				const int32 ParamOffset = ParameterOffset + Layout.LayoutInfo.GetFloatComponentByteOffset(CompIdx);
				const int32 DataSetOffset = Layout.GetFloatComponentStart() + CompIdx;
				FloatOffsets.Emplace(ParamOffset, DataSetOffset);
			}
			for (uint32 CompIdx = 0; CompIdx < Layout.GetNumInt32Components(); ++CompIdx)
			{
				const int32 ParamOffset = ParameterOffset + Layout.LayoutInfo.GetInt32ComponentByteOffset(CompIdx);
				const int32 DataSetOffset = Layout.GetInt32ComponentStart() + CompIdx;
				Int32Offsets.Emplace(ParamOffset, DataSetOffset);
			}
			for (uint32 CompIdx = 0; CompIdx < Layout.GetNumHalfComponents(); ++CompIdx)
			{
				constexpr bool ParameterSetsSupportHalf = false;

				if (ParameterSetsSupportHalf)
				{
					const int32 ParamOffset = ParameterOffset + Layout.LayoutInfo.GetHalfComponentByteOffset(CompIdx);
					const int32 DataSetOffset = Layout.GetHalfComponentStart() + CompIdx;
					HalfOffsets.Emplace(ParamOffset, DataSetOffset, !ParameterSetsSupportHalf);
				}
				else
				{
					// if parameter sets don't support half, then we need to write in floats into the parameter set, and
					// for that we need to adjust the offset based on the difference in stride between float & half
					// In reality
					const int32 ParamOffset = ParameterOffset + sizeof(float) * Layout.LayoutInfo.GetHalfComponentByteOffset(CompIdx) / sizeof(FFloat16);
					const int32 DataSetOffset = Layout.GetHalfComponentStart() + CompIdx;
					HalfOffsets.Emplace(ParamOffset, DataSetOffset, !ParameterSetsSupportHalf);
				}
			}
		}
	}

	inline void DataSetToParameterStore(FNiagaraParameterStore& ParameterStore, FNiagaraDataSet& DataSet, int32 DataSetInstanceIndex)
	{
#if NIAGARA_NAN_CHECKING
		DataSet.CheckForNaNs();
#endif

		FNiagaraDataBuffer* CurrBuffer = DataSet.GetCurrentData();

		for (const FDataOffsets& DataOffsets : FloatOffsets)
		{
			float* DataSetPtr = CurrBuffer->GetInstancePtrFloat(DataOffsets.DataSetComponentOffset, DataSetInstanceIndex);
			ParameterStore.SetParameterByOffset(DataOffsets.ParameterOffset, *DataSetPtr);
		}
		for (const FDataOffsets& DataOffsets : Int32Offsets)
		{
			int32* DataSetPtr = CurrBuffer->GetInstancePtrInt32(DataOffsets.DataSetComponentOffset, DataSetInstanceIndex);
			ParameterStore.SetParameterByOffset(DataOffsets.ParameterOffset, *DataSetPtr);
		}
		for (const FHalfDataOffsets& DataOffsets : HalfOffsets)
		{
			FFloat16* DataSetPtr = CurrBuffer->GetInstancePtrHalf(DataOffsets.DataSetComponentOffset, DataSetInstanceIndex);
			if (DataOffsets.ApplyAsFloat)
			{
				ParameterStore.SetParameterByOffset(DataOffsets.ParameterOffset, DataSetPtr->GetFloat());
			}
			else
			{
				ParameterStore.SetParameterByOffset(DataOffsets.ParameterOffset, *DataSetPtr);
			}
		}

#if NIAGARA_NAN_CHECKING
		ParameterStore.CheckForNaNs();
#endif

		ParameterStore.OnParameterChange();
	}

	inline void ParameterStoreToDataSet(const FNiagaraParameterStore& ParameterStore, FNiagaraDataSet& DataSet, int32 DataSetInstanceIndex)
	{
		FNiagaraDataBuffer& CurrBuffer = DataSet.GetDestinationDataChecked();
		const uint8* ParameterData = ParameterStore.GetParameterDataArray().GetData();

#if NIAGARA_NAN_CHECKING
		ParameterStore.CheckForNaNs();
#endif

		for (const FDataOffsets& DataOffsets : FloatOffsets)
		{
			float* ParamPtr = (float*)(ParameterData + DataOffsets.ParameterOffset);
			float* DataSetPtr = CurrBuffer.GetInstancePtrFloat(DataOffsets.DataSetComponentOffset, DataSetInstanceIndex);
			*DataSetPtr = *ParamPtr;
		}
		for (const FDataOffsets& DataOffsets : Int32Offsets)
		{
			int32* ParamPtr = (int32*)(ParameterData + DataOffsets.ParameterOffset);
			int32* DataSetPtr = CurrBuffer.GetInstancePtrInt32(DataOffsets.DataSetComponentOffset, DataSetInstanceIndex);
			*DataSetPtr = *ParamPtr;
		}
		for (const FHalfDataOffsets& DataOffsets : HalfOffsets)
		{
			FFloat16* DataSetPtr = CurrBuffer.GetInstancePtrHalf(DataOffsets.DataSetComponentOffset, DataSetInstanceIndex);

			if (DataOffsets.ApplyAsFloat)
			{
				float* ParamPtr = (float*)(ParameterData + DataOffsets.ParameterOffset);
				*DataSetPtr = *ParamPtr;
			}
			else
			{
				FFloat16* ParamPtr = (FFloat16*)(ParameterData + DataOffsets.ParameterOffset);
				*DataSetPtr = *ParamPtr;
			}
		}

#if NIAGARA_NAN_CHECKING
		DataSet.CheckForNaNs();
#endif
	}
};

struct FNiagaraConstantBufferToDataSetBinding
{
	static void CopyToDataSets(
		const FNiagaraSystemCompiledData& CompiledData,
		const FNiagaraSystemInstance& SystemInstance,
		FNiagaraDataSet& SpawnDataSet,
		FNiagaraDataSet& UpdateDataSet,
		int32 DataSetInstanceIndex);

protected:
	static void ApplyOffsets(const FNiagaraParameterDataSetBindingCollection& Offsets, const uint8* SourceData, FNiagaraDataSet& DataSet, int32 DataSetInstanceIndex);
};

struct FNiagaraSystemSimulationTickContext
{
public:
	FNiagaraSystemSimulationTickContext(class FNiagaraSystemSimulation* InOwner, TArray<FNiagaraSystemInstance*>& InInstances, FNiagaraDataSet& InDataSet, float InDeltaSeconds, int32 InSpawnNum, bool bAllowAsync);

	bool IsRunningAsync() const { return bRunningAsync; }

public:
	class FNiagaraSystemSimulation*		Owner;
	UNiagaraSystem*						System;
	UWorld*								World;

	TArray<FNiagaraSystemInstance*>&	Instances;
	FNiagaraDataSet&					DataSet;

	float								DeltaSeconds;
	int32								SpawnNum;

	int									EffectsQuality;

	bool								bRunningAsync = false;
	FGraphEventArray					BeforeInstancesTickGraphEvents;
	FGraphEventArray*					CompletionEvents = nullptr;

	FNiagaraSystemTickBatch				TickBatch;
};

/** Simulation performing all system and emitter scripts for a instances of a UNiagaraSystem in a world. */
class FNiagaraSystemSimulation : public TSharedFromThis<FNiagaraSystemSimulation, ESPMode::ThreadSafe>
{
	friend FNiagaraSystemSimulationTickContext;
	friend struct FNiagaraSimCacheHelper;
	friend class FNiagaraDebugHud;

public:
	FNiagaraSystemSimulation();
	~FNiagaraSystemSimulation();
	bool Init(UNiagaraSystem* InSystem, UWorld* InWorld, bool bInIsSolo, ETickingGroup InTickGroup, const FNiagaraParameterCollectionResolver& ParameterCollectionResolver);
	void Destroy();

	bool IsValid() const { return bCanExecute && World != nullptr && ::IsValid(System); }

	/** First phase of system sim tick. Must run on GameThread. */
	void Tick_GameThread(float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent);
	/** Second phase of system sim tick that can run on any thread. */
	void Tick_Concurrent(FNiagaraSystemSimulationTickContext& Context);

	/** Processes multiple ticks at once on the game thread. */
	bool MultiTick_GameThread(int32 TicksToProcess, float DeltaSecondsPerTick, float MaxSimTime, const FGraphEventRef& MyCompletionGraphEvent);

	/** Update TickGroups for pending instances and execute tick group promotions. */
	void UpdateTickGroups_GameThread();
	/** Spawn any pending instances, assumes that you have update tick groups ahead of time. */
	void Spawn_GameThread(float DeltaSeconds, bool bPostActorTick);
	/** Spawn any pending instances */
	void Spawn_Concurrent(FNiagaraSystemSimulationTickContext& Context);

	/** Game thread portion of sim cache tick.  Must run on GameThread. */
	void SimCacheTick_GameThread(UNiagaraSimCache* SimCache, float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent);
	/** Concurrent portion of sim cache tick, can run on any thread. */
	void SimCacheTick_Concurrent(UNiagaraSimCache* SimCache, float DeltaSeconds);

	/** Wait for system simulation concurrent tick to complete.  If bEnsureComplete is true we will trigger an ensure if it is not complete. */
	void WaitForConcurrentTickComplete(bool bEnsureComplete = false);
	/** Wait for system instances concurrent tick to complete.  If bEnsureComplete is true we will trigger an ensure if it is not complete. */
	void WaitForInstancesTickComplete(bool bEnsureComplete = false);

	void RemoveInstance(FNiagaraSystemInstance* Instance);
	void AddInstance(FNiagaraSystemInstance* Instance);

	void PauseInstance(FNiagaraSystemInstance* Instance);
	void UnpauseInstance(FNiagaraSystemInstance* Instance);

	inline UNiagaraSystem* GetSystem() const { return System; }

	/** Transfers a system instance from the current simulation into this one. */
	void TransferInstance(FNiagaraSystemInstance* SystemInst);

	void DumpInstance(const FNiagaraSystemInstance* Inst)const;

	/** Dump information about all instances tick */
	void DumpTickInfo(FOutputDevice& Ar);

	bool GetIsSolo() const { return bIsSolo; }

	FNiagaraScriptExecutionContextBase* GetSpawnExecutionContext() { return SpawnExecContext.Get(); }
	FNiagaraScriptExecutionContextBase* GetUpdateExecutionContext() { return UpdateExecContext.Get(); }

	void AddTickGroupPromotion(FNiagaraSystemInstance* Instance);

	void RemoveFromInstanceList(FNiagaraSystemInstance* Instance);
	void AddToInstanceList(FNiagaraSystemInstance* Instance, ENiagaraSystemInstanceState InstanceState);
	void SetInstanceState(FNiagaraSystemInstance* Instance, ENiagaraSystemInstanceState NewState);

	const FString& GetCrashReporterTag()const;

	ETickingGroup GetTickGroup() const { return SystemTickGroup; }

	inline FNiagaraGpuComputeDispatchInterface* GetDispatchInterface() const { return DispatchInterface; }

	ENiagaraGPUTickHandlingMode GetGPUTickHandlingMode() const;

	inline UWorld* GetWorld()const{return World;}

	const FNiagaraTickInfo& GetTickInfo() { return TickInfo; }

protected:
	void DumpStalledInfo();

	/** Sets constant parameter values */
	void SetupParameters_GameThread(float DeltaSeconds);

	/** Does any prep work for system simulation such as pulling instance parameters into a dataset. */
	void PrepareForSystemSimulate(FNiagaraSystemSimulationTickContext& Context);
	/** Runs the system spawn script for new system instances. */
	void SpawnSystemInstances(FNiagaraSystemSimulationTickContext& Context);
	/** Runs the system update script. */
	void UpdateSystemInstances(FNiagaraSystemSimulationTickContext& Context);
	/** Transfers the results of the system simulation into the emitter instances. */
	void TransferSystemSimResults(FNiagaraSystemSimulationTickContext& Context);
	/** Builds the constant buffer table for a given script execution */
	void BuildConstantBufferTable(
		const FNiagaraGlobalParameters& GlobalParameters,
		TUniquePtr<FNiagaraScriptExecutionContextBase>& ExecContext,
		FScriptExecutionConstantBufferTable& ConstantBufferTable) const;

	void AddSystemToTickBatch(FNiagaraSystemSimulationTickContext& Context, FNiagaraSystemInstance* Instance, bool bIsLastInstance);
	void FlushTickBatch(FNiagaraSystemSimulationTickContext& Context, bool bIsLastInstance);

	void Tick_GameThread_Internal(float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent);

	TArray<FNiagaraSystemInstance*>& GetSystemInstances(ENiagaraSystemInstanceState State) { check(State != ENiagaraSystemInstanceState::None); return SystemInstancesPerState[int32(State)]; }

	/** System of instances being simulated.  We use a weak object ptr here because once the last referencing object goes away this system may be come invalid at runtime. */
	UNiagaraSystem* System;

	/** We cache off the effect type in the unlikely even that someone GCs the System from under us so that we can keep the effect types instance count etc accurate. */
	TWeakObjectPtr<UNiagaraEffectType> EffectType;

	/** Which tick group we are in, only valid when not in Solo mode. */
	ETickingGroup SystemTickGroup = TG_MAX;

	/** World this system simulation belongs to. */
	UWorld* World;

	uint32 bCanExecute : 1;
	uint32 bBindingsInitialized : 1;
	uint32 bInSpawnPhase : 1;
	uint32 bIsSolo : 1;

	uint32 bRunSpawnScript : 1 = true;
	uint32 bRunUpdateScript : 1 = true;

	/** System instance per state. */
	TArray<FNiagaraSystemInstance*> SystemInstancesPerState[int32(ENiagaraSystemInstanceState::Num)];

	/** Data set for the Running instance state. */
	FNiagaraDataSet MainDataSet;
	/** Data set for the Spawning instance state. */
	FNiagaraDataSet SpawningDataSet;
	/** Data set for the Paused instance state. */
	FNiagaraDataSet PausedDataSet;

	/**
	As there's a 1 to 1 relationship between system instance and their execution in this simulation we must pull all that instances parameters into a dataset for simulation.
	In some cases this might be a big waste of memory as there'll be duplicated data from a parameter store that's shared across all instances.
	Though all these parameters can be unique per instance so for now lets just do the simple thing and improve later.
	*/
	FNiagaraDataSet SpawnInstanceParameterDataSet;
	FNiagaraDataSet UpdateInstanceParameterDataSet;

	TUniquePtr<FNiagaraScriptExecutionContextBase> SpawnExecContext;
	TUniquePtr<FNiagaraScriptExecutionContextBase> UpdateExecContext;

	/** Bindings that pull per component parameters into the spawn parameter dataset. */
	FNiagaraParameterStoreToDataSetBinding SpawnInstanceParameterToDataSetBinding;
	/** Bindings that pull per component parameters into the update parameter dataset. */
	FNiagaraParameterStoreToDataSetBinding UpdateInstanceParameterToDataSetBinding;

	/** Binding to push system attributes into each emitter spawn parameters. */
	TArray<FNiagaraParameterStoreToDataSetBinding> DataSetToEmitterSpawnParameters;
	/** Binding to push system attributes into each emitter update parameters. */
	TArray<FNiagaraParameterStoreToDataSetBinding> DataSetToEmitterUpdateParameters;
	/** Binding to push system attributes into each emitter event parameters. */
	TArray<TArray<FNiagaraParameterStoreToDataSetBinding>> DataSetToEmitterEventParameters;
	/** Binding to push system attributes into each emitter gpu parameters. */
	TArray<FNiagaraParameterStoreToDataSetBinding> DataSetToEmitterGPUParameters;
	/** Binding to push system attributes into each emitter renderer parameters. */
	TArray<FNiagaraParameterStoreToDataSetBinding> DataSetToEmitterRendererParameters;


	/** Direct bindings for Engine variables in System Spawn and Update scripts. */
	FNiagaraParameterDirectBinding<int32> SpawnNumSystemInstancesParam;
	FNiagaraParameterDirectBinding<int32> UpdateNumSystemInstancesParam;

	FNiagaraParameterDirectBinding<float> SpawnGlobalSpawnCountScaleParam;
	FNiagaraParameterDirectBinding<float> UpdateGlobalSpawnCountScaleParam;

	FNiagaraParameterDirectBinding<float> SpawnGlobalSystemCountScaleParam;
	FNiagaraParameterDirectBinding<float> UpdateGlobalSystemCountScaleParam;

	/** List of instances that are pending a tick group promotion. */
	TArray<FNiagaraSystemInstance*> PendingTickGroupPromotions;

	void InitParameterDataSetBindings(FNiagaraSystemInstance* SystemInst);

	TOptional<float> MaxDeltaTime;

	/** Event to track the system simulation async tick is complete. */
	FGraphEventRef ConcurrentTickGraphEvent;
	/** Event to track all work is complete, i.e. System Concurrent, Instance Concurrent, Finalize */
	FGraphEventRef AllWorkCompleteGraphEvent;

	mutable FString CrashReporterTag;

	FNiagaraGpuComputeDispatchInterface* DispatchInterface = nullptr;

	static bool bUseLegacyExecContexts;

	float FixedDeltaTickAge = 0;

	FNiagaraTickInfo TickInfo;
};
