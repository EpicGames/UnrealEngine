// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RenderResource.h"
#include "RenderingThread.h"
#include "RHIUtilities.h"
#include "Containers/DynamicRHIResourceArray.h"

// Static buffers shared between all instances built once on load
struct FNiagaraSystemStaticBuffers
{
public:
	FNiagaraSystemStaticBuffers() {}
	~FNiagaraSystemStaticBuffers();

	FNiagaraSystemStaticBuffers(const FNiagaraSystemStaticBuffers&) = delete;
	FNiagaraSystemStaticBuffers(FNiagaraSystemStaticBuffers&&) = delete;
	FNiagaraSystemStaticBuffers& operator=(const FNiagaraSystemStaticBuffers&) = delete;
	FNiagaraSystemStaticBuffers& operator=(FNiagaraSystemStaticBuffers&&) = delete;

	// Called to finalize the data (done on load by the system)
	void Finalize();

	// Adds float data for Cpu access, returns the index into the buffer in elements
	uint32 AddCpuData(TConstArrayView<float> InFloatData);
	// Adds float data for Gpu access, returns the index into the buffer in elements
	uint32 AddGpuData(TConstArrayView<float> InFloatData);

	// Get float buffer for Cpu Access
	TConstArrayView<float> GetCpuFloatBuffer() const { return MakeArrayView(CpuFloatBuffer); }

	// Get float buffer for Gpu Access
	FRHIShaderResourceView* GetGpuFloatBuffer() const { check(IsInRenderingThread()); return GpuFloatBuffer.SRV; }

private:
	bool					bFinalized = false;
	TArray<float>			CpuFloatBuffer;

	TResourceArray<float>	GpuFloatResource;
	FReadBuffer				GpuFloatBuffer;
};
