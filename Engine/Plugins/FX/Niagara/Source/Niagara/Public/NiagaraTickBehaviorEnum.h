// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTickBehaviorEnum.generated.h"

/** Niagara ticking behaviour */
UENUM()
enum class ENiagaraTickBehavior : uint8
{
	/** Niagara will tick after all prereqs have ticked for attachements / data interfaces, this is the safest option. */
	UsePrereqs,
	/** Niagara will ignore prereqs (attachments / data interface dependencies) and use the tick group set on the component. */
	UseComponentTickGroup,
	/** Niagara will tick in the first tick group (default is TG_PrePhysics). */
	ForceTickFirst,
	/** Niagara will tick in the last tick group (default is TG_LastDemotable). */
	ForceTickLast,
};

