// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Container for Refresh/Invalidate method delegates passed to all Widgets which may selectively Refresh/Invalidate other widgets in the script toolkit. */
struct FScriptToolkitUIContext
{
public:
	FScriptToolkitUIContext() = default;

	FScriptToolkitUIContext(
		const FSimpleDelegate& InRefreshParameterPanelDelegate,
		const FSimpleDelegate& InRefreshParameterDefinitionsPanelDelegate,
		const FSimpleDelegate& InRefreshSelectionDetailsViewPanelDelegate)
		: RefreshParameterPanelDelegate(InRefreshParameterPanelDelegate)
		, RefreshParameterDefinitionsPanelDelegate(InRefreshParameterDefinitionsPanelDelegate)
		, RefreshSelectionDetailsViewPanelDelegate(InRefreshSelectionDetailsViewPanelDelegate)
	{};

	void RefreshParameterPanel() { RefreshParameterPanelDelegate.Execute(); };
	void RefreshParameterDefinitionsPanel() { RefreshParameterDefinitionsPanelDelegate.ExecuteIfBound(); }; //@todo(ng) add parameter definitions panel to scratchpad scripts
	void RefreshSelectionDetailsViewPanel() { RefreshSelectionDetailsViewPanelDelegate.Execute(); };

private:
	FSimpleDelegate RefreshParameterPanelDelegate;
	FSimpleDelegate RefreshParameterDefinitionsPanelDelegate;
	FSimpleDelegate RefreshSelectionDetailsViewPanelDelegate;
};

/** Container for Refresh/Invalidate method delegates passed to all Widgets which may selectively Refresh/Invalidate other widgets in the system toolkit. */
struct FSystemToolkitUIContext
{
public:
	FSystemToolkitUIContext() = default;

	FSystemToolkitUIContext(
		const FSimpleDelegate& InRefreshParameterPanelDelegate,
		const FSimpleDelegate& InRefreshParameterDefinitionsPanelDelegate)
		: RefreshParameterPanelDelegate(InRefreshParameterPanelDelegate)
		, RefreshParameterDefinitionsPanelDelegate(InRefreshParameterDefinitionsPanelDelegate)
	{};

	void RefreshParameterPanel() { RefreshParameterPanelDelegate.Execute(); };
	void RefreshParameterDefinitionsPanel() { RefreshParameterDefinitionsPanelDelegate.Execute(); };

private:
	FSimpleDelegate RefreshParameterPanelDelegate;
	FSimpleDelegate RefreshParameterDefinitionsPanelDelegate;
};

/** Container for Refresh/Invalidate method delegates passed to all Widgets which may selectively Refresh/Invalidate other widgets in the parameter definitions toolkit. */
struct FParameterDefinitionsToolkitUIContext
{
public:
	FParameterDefinitionsToolkitUIContext() = default;

	FParameterDefinitionsToolkitUIContext(
		const FSimpleDelegate& InRefreshParameterPanelDelegate,
		const FSimpleDelegate& InRefreshSelectionDetailsViewPanelDelegate)
		: RefreshParameterPanelDelegate(InRefreshParameterPanelDelegate)
		, RefreshSelectionDetailsViewPanelDelegate(InRefreshSelectionDetailsViewPanelDelegate)
	{};

	void RefreshParameterPanel() { RefreshParameterPanelDelegate.Execute(); };
	void RefreshSelectionDetailsViewPanel() { RefreshSelectionDetailsViewPanelDelegate.Execute(); };

private:
	FSimpleDelegate RefreshParameterPanelDelegate;
	FSimpleDelegate RefreshSelectionDetailsViewPanelDelegate;
};
