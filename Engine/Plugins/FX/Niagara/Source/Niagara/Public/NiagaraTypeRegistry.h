// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/ScopeLock.h"
#include "Misc/SecureHash.h"
#include "UObject/GCObject.h"
#include "UObject/UnrealType.h"
#include "Containers/Queue.h"
#include "NiagaraCore.h"
#include "NiagaraTypes.h"

//////////////////////////////////////////////////////////////////////////

enum class ENiagaraTypeRegistryFlags : uint32
{
	None					= 0,

	AllowUserVariable		= (1 << 0),
	AllowSystemVariable		= (1 << 1),
	AllowEmitterVariable	= (1 << 2),
	AllowParticleVariable	= (1 << 3),
	AllowAnyVariable		= (AllowUserVariable | AllowSystemVariable | AllowEmitterVariable | AllowParticleVariable),
	AllowNotUserVariable	= (AllowSystemVariable | AllowEmitterVariable | AllowParticleVariable),

	AllowParameter			= (1 << 4),
	AllowPayload			= (1 << 5),

	IsUserDefined			= (1 << 6),

};

ENUM_CLASS_FLAGS(ENiagaraTypeRegistryFlags)

/* Contains all types currently available for use in Niagara
* Used by UI to provide selection; new uniforms and variables
* may be instanced using the types provided here
*/
class FNiagaraTypeRegistry : public FGCObject
{
public:
	// In order to simplify the requirements for handling access to the type array from various threads (async loading as an example)
	// we ensure that the entries in the array will never be invalidated.  While the TUniquePtr may be moved (through reallocation)
	// the array will not shrink, and the type definition pointer within the TUniquePtr will not be invalidated.
	using FRegisteredTypesArray = TArray<TUniquePtr<FNiagaraTypeDefinition>>;

	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetRegisteredTypes();
	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetRegisteredUserVariableTypes();
	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetRegisteredSystemVariableTypes();
	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetRegisteredEmitterVariableTypes();
	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetRegisteredParticleVariableTypes();
	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetRegisteredParameterTypes();
	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetRegisteredPayloadTypes();
	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetUserDefinedTypes();
	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetNumericTypes();
	static NIAGARA_API TArray<FNiagaraTypeDefinition> GetIndexTypes();

	static NIAGARA_API TOptional<FNiagaraTypeDefinition> GetRegisteredTypeByName(FName TypeName);

	static NIAGARA_API UNiagaraDataInterfaceBase* GetDefaultDataInterfaceByName(const FString& DIClassName);

	static NIAGARA_API void ClearUserDefinedRegistry();

	UE_DEPRECATED(4.27, "This overload is deprecated, please use the Register function that takes registration flags instead.")
	static NIAGARA_API void Register(const FNiagaraTypeDefinition& NewType, bool bCanBeParameter, bool bCanBePayload, bool bIsUserDefined);

	static NIAGARA_API void ProcessRegistryQueue();

	static NIAGARA_API void Register(const FNiagaraTypeDefinition &NewType, ENiagaraTypeRegistryFlags Flags);

	static NIAGARA_API bool IsStaticPossible(const FNiagaraTypeDefinition& InSrc);

	static NIAGARA_API void RegisterStructConverter(const FNiagaraTypeDefinition& SourceType, const FNiagaraLwcStructConverter& StructConverter);

	static NIAGARA_API FNiagaraLwcStructConverter GetStructConverter(const FNiagaraTypeDefinition& SourceType);

	static NIAGARA_API FNiagaraTypeDefinition GetTypeForStruct(UScriptStruct* InStruct);
	static NIAGARA_API void InvalidateTypesByPath(const FString& AssetPath);
	static NIAGARA_API void InvalidateTypesByPaths(TConstArrayView<FString> AssetPaths);

	/** LazySingleton interface */
	static NIAGARA_API FNiagaraTypeRegistry& Get();
	static void Init();
	static void TearDown();

	/** FGCObject interface */
	NIAGARA_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	NIAGARA_API virtual FString GetReferencerName() const override;

	template<typename TAction>
	static void ForEachRegisteredType(TAction Func)
	{
		FNiagaraTypeRegistry& Registry = Get();

		auto TrivialSelect = [](const FNiagaraTypeDefinition& TypeDef) -> bool
		{
			return true;
		};

		Registry.ForAllSelectedRegisteredTypes(TrivialSelect, Func);
	}

	class FSuspendPluginUnmounting
	{
	public:
		UE_NODISCARD_CTOR FSuspendPluginUnmounting() { SuspendPluginUnmount(); }
		~FSuspendPluginUnmounting() { ResumePluginUnmount(); }
	};

private:
	struct FQueuedRegistryEntry
	{
		FNiagaraTypeDefinition NewType;
		ENiagaraTypeRegistryFlags Flags;
	};
	friend class FLazySingleton;
	friend struct FNiagaraTypeDefinitionHandle;

	NIAGARA_API FNiagaraTypeRegistry();
	NIAGARA_API virtual ~FNiagaraTypeRegistry();

	static NIAGARA_API void SuspendPluginUnmount();
	static NIAGARA_API void ResumePluginUnmount();

	int32 RegisterTypeInternal(const FNiagaraTypeDefinition& NewType);
	const FNiagaraTypeDefinition& GetRegisteredType(int32 TypeIndex) const;

	mutable FTransactionallySafeRWLock RegisteredTypesLock;
	FRegisteredTypesArray RegisteredTypes;

	// atomic index that is incremented each time types may have been invalidated
	std::atomic<uint32> RegisteredTypesGeneration = {0};

	// Covers manipulation of all of the containers populated during registration (except for RegisteredTypes which is handled
	// by it's own lock so that we can provide a fast path for the common case of reading the array)
	FRWLock RegistrationLock;

	TArray<FNiagaraTypeDefinition> RegisteredUserVariableTypes;
	TArray<FNiagaraTypeDefinition> RegisteredSystemVariableTypes;
	TArray<FNiagaraTypeDefinition> RegisteredEmitterVariableTypes;
	TArray<FNiagaraTypeDefinition> RegisteredParticleVariableTypes;
	TArray<FNiagaraTypeDefinition> RegisteredParamTypes;
	TArray<FNiagaraTypeDefinition> RegisteredPayloadTypes;
	TArray<FNiagaraTypeDefinition> RegisteredUserDefinedTypes;
	TArray<FNiagaraTypeDefinition> RegisteredNumericTypes;
	TArray<FNiagaraTypeDefinition> RegisteredIndexTypes;

	TMap<uint32, int32> RegisteredTypeIndexMap;
	TMap<uint32, FNiagaraLwcStructConverter> RegisteredStructConversionMap;

	// For each entry in RegisteredTypes we store the SoftObjectPath of the underlying
	// class/struct so that we can handle dynamically unloading plugins
	TArray<FSoftObjectPath> RegisteredSoftObjectPaths;

	bool bModuleInitialized = false;
	TQueue<FQueuedRegistryEntry, EQueueMode::Mpsc> RegistryQueue;

	// helper method to go through the various caching methods for finding the
	// index into RegisteredTypes for the provided TypeDefinition
	static NIAGARA_API bool GetCachedRegisteredTypeIndex(const FNiagaraTypeDefinition& Type, uint32& OutTypeHash, int32& OutIndex);

	// populates the cache on the TypeDefinition and TLS
	static NIAGARA_API void CacheRegisteredTypeIndex(const FNiagaraTypeDefinition& Type, uint32 TypeHash, int32 Index);

	template<typename TSelectAction, typename TAction>
	void ForAllSelectedRegisteredTypes(TSelectAction Select, TAction Func) const
	{
		TArray<FNiagaraTypeDefinition> LocalCopy;

		{
			UE::TReadScopeLock Lock(RegisteredTypesLock);
			LocalCopy.Reserve(RegisteredTypes.Num());
			Algo::TransformIf(RegisteredTypes, LocalCopy,
				[Select](const TUniquePtr<FNiagaraTypeDefinition>& DefPtr) -> bool
				{
					return DefPtr.IsValid() && DefPtr->IsValid() && Select(*DefPtr);
				},
				[](const TUniquePtr<FNiagaraTypeDefinition>& DefPtr) -> FNiagaraTypeDefinition
				{
					return *DefPtr;
				});
		}

		for (const FNiagaraTypeDefinition& Type : LocalCopy)
		{
			if (!Func(Type))
			{
				break;
			}
		}
	}
};

// helper class to trigger access to a TLS of NiagaraTypeDefinition.  While constructed, the current thread
// will get lock-free access (minus the first call) to a registered type.  Destruction clears the TLS to avoid
// any potential fallout of holding onto stale pointers or taking up too much memory bloat
class FNiagaraTypeRegistryTLSProxy final
{
public:
	NIAGARA_API FNiagaraTypeRegistryTLSProxy();
	NIAGARA_API ~FNiagaraTypeRegistryTLSProxy();

private:
	const uint32 CallingThreadId;
};