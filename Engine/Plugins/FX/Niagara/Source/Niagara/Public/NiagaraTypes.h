// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/SecureHash.h"
#include "UObject/GCObject.h"
#include "UObject/UnrealType.h"
#include "Containers/Queue.h"
#include "NiagaraCore.h"

#include "NiagaraTypes.generated.h"

struct FNiagaraTypeDefinition;
struct FNiagaraCompileHashVisitor;
class UNiagaraDataInterfaceBase;
struct FNiagaraVariableMetaData;

DECLARE_LOG_CATEGORY_EXTERN(LogNiagara, Log, Verbose);

// basic type struct definitions

USTRUCT(meta = (DisplayName = "Wildcard"))
struct FNiagaraWildcard
{
	GENERATED_BODY()
	
};

USTRUCT(meta = (DisplayName = "float"))
struct FNiagaraFloat
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Parameters)
	float Value = 0;
};

USTRUCT(meta = (DisplayName = "int32"))
struct FNiagaraInt32
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Parameters)
	int32 Value = 0;
};

USTRUCT(meta=(DisplayName="bool"))
struct FNiagaraBool
{
	GENERATED_USTRUCT_BODY()

	// The Niagara VM expects this bitmask for its compare and select operators for false.
	enum BoolValues { 
		True = INDEX_NONE,
		False = 0
	}; 

	void SetValue(bool bValue) { Value = bValue ? True : False; }
	bool GetValue() const { return Value != False; }

	/** Sets this niagara bool's raw integer value directly using the special raw integer values expected by the VM and HLSL. */
	inline void SetRawValue(int32 RawValue) { Value = RawValue; }

	/** Gets this niagara bools raw integer value expected by the VM and HLSL. */
	inline int32 GetRawValue() const { return Value; }

	bool IsValid() const { return Value == True || Value == False; }

	FNiagaraBool() : Value(False) {}
	FNiagaraBool(bool bInValue) : Value(bInValue ? True : False) {}
	inline operator bool() const { return GetValue(); }

private:
	UPROPERTY(EditAnywhere, Category = Parameters)// Must be either FNiagaraBool::True(-1) or FNiagaraBool::False(0).
	int32 Value = False;
};

USTRUCT(meta = (DisplayName = "Position"))
struct FNiagaraPosition : public FVector3f
{
	GENERATED_USTRUCT_BODY()

	FNiagaraPosition() {}
	
	explicit inline FNiagaraPosition(EForceInit Init) : Super(Init) {}

	inline FNiagaraPosition(const float& X, const float& Y, const float& Z) : Super(X, Y, Z) {}

	inline FNiagaraPosition(const FVector3f& Other) : Super(Other) {}
	inline FNiagaraPosition(const FVector& Other) : Super(Other) {}
};


USTRUCT(meta = (DisplayName = "Half", NiagaraInternalType = "true"))
struct FNiagaraHalf
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 Value = 0;
};

USTRUCT(meta = (DisplayName = "Half Vector2", NiagaraInternalType = "true"))
struct FNiagaraHalfVector2
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 x = 0;

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 y = 0;
};

USTRUCT(meta = (DisplayName = "Half Vector3", NiagaraInternalType = "true"))
struct FNiagaraHalfVector3
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 x = 0;

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 y = 0;

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 z = 0;
};

USTRUCT(meta = (DisplayName = "Half Vector4", NiagaraInternalType = "true"))
struct FNiagaraHalfVector4
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 x = 0;

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 y = 0;

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 z = 0;

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint16 w = 0;
};

USTRUCT()
struct FNiagaraNumeric
{
	GENERATED_USTRUCT_BODY()
};

USTRUCT()
struct FNiagaraParameterMap
{
	GENERATED_USTRUCT_BODY()
};

// only used for LWC conversions, not supported by Niagara yet
USTRUCT(meta = (DisplayName = "double"))
struct FNiagaraDouble
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Parameters)
	double Value = 0;
};

USTRUCT(meta = (DisplayName = "Matrix"))
struct FNiagaraMatrix
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=NiagaraMatrix)
	FVector4f Row0 = FVector4f(ForceInitToZero);

	UPROPERTY(EditAnywhere, Category = NiagaraMatrix)
	FVector4f Row1 = FVector4f(ForceInitToZero);

	UPROPERTY(EditAnywhere, Category = NiagaraMatrix)
	FVector4f Row2 = FVector4f(ForceInitToZero);

	UPROPERTY(EditAnywhere, Category = NiagaraMatrix)
	FVector4f Row3 = FVector4f(ForceInitToZero);
};

USTRUCT(meta = (DisplayName = "EmitterID"))
struct FNiagaraEmitterID
{
	GENERATED_BODY()
	
	FNiagaraEmitterID() = default;
	explicit FNiagaraEmitterID(int32 InID) : ID(InID) {}

	UPROPERTY(EditAnywhere, Category = Emitter)
	int32 ID = INDEX_NONE;

	bool operator==(const FNiagaraEmitterID OtherID) const { return ID == OtherID.ID; }
	bool operator!=(const FNiagaraEmitterID OtherID) const { return ID != OtherID.ID; }
};

USTRUCT()
struct FNiagaraAssetVersion
{
	GENERATED_USTRUCT_BODY()

	/** The major version is used to track breaking changes between asset versions */
	UPROPERTY(VisibleAnywhere, Category = "Version Control")
	int32 MajorVersion = 1;

	/** The minor version is used to track non-breaking changes between asset versions */
	UPROPERTY(VisibleAnywhere, Category = "Version Control")
	int32 MinorVersion = 0;

	/** The guid is used to keep track of specific asset version references. The minor and major versions do not provide enough uniqueness to guard against collisions when e.g. the same version was created in different branches. */
	UPROPERTY(VisibleAnywhere, Category = "Version Control", meta=(IgnoreForMemberInitializationTest))
	FGuid VersionGuid = FGuid::NewGuid();
	
	/** If false then this version is not visible in the version selector dropdown menu of the stack. */
	UPROPERTY()
	bool bIsVisibleInVersionSelector = true;

	bool operator==(const FNiagaraAssetVersion& Other) const { return VersionGuid == Other.VersionGuid; }
	bool operator!=(const FNiagaraAssetVersion& Other) const { return !(*this == Other); }
	bool operator<(const FNiagaraAssetVersion& Other) const { return MajorVersion < Other.MajorVersion || (MajorVersion == Other.MajorVersion && MinorVersion < Other.MinorVersion); }
	bool operator<=(const FNiagaraAssetVersion& Other) const { return *this < Other || (MajorVersion == Other.MajorVersion && MinorVersion == Other.MinorVersion); }

	static NIAGARA_API FGuid CreateStableVersionGuid(UObject* Object);

	friend inline uint32 GetTypeHash(const FNiagaraAssetVersion& Version)
	{
		return HashCombine(GetTypeHash(Version.MajorVersion), GetTypeHash(Version.MinorVersion));
	}
};

struct FNiagaraLWCConverter
{
	NIAGARA_API explicit FNiagaraLWCConverter(FVector InSystemWorldPos = FVector::ZeroVector);

	[[nodiscard]] NIAGARA_API FVector3f ConvertWorldToSimulationVector(FVector WorldPosition) const;
	[[nodiscard]] NIAGARA_API FNiagaraPosition ConvertWorldToSimulationPosition(FVector WorldPosition) const;
	
	[[nodiscard]] NIAGARA_API FVector ConvertSimulationPositionToWorld(FNiagaraPosition SimulationPosition) const;
	[[nodiscard]] NIAGARA_API FVector ConvertSimulationVectorToWorld(FVector3f SimulationPosition) const;

	[[nodiscard]] NIAGARA_API FMatrix ConvertWorldToSimulationMatrix(const FMatrix& Matrix) const;
	[[nodiscard]] NIAGARA_API FMatrix ConvertSimulationToWorldMatrix(const FMatrix& Matrix) const;

	[[nodiscard]] NIAGARA_API FTransform3f ConvertWorldToSimulationTransform(const FTransform& Transform) const;
	[[nodiscard]] NIAGARA_API FTransform ConvertSimulationToWorldTransform(const FTransform3f& Transform) const;

private:
	FVector SystemWorldPos;
};

UENUM()
enum class ENiagaraStructConversionType : uint8
{
	CopyOnly, // no conversion, just copy the data
	
	DoubleToFloat,
	Vector2,
	Vector3,
	Vector4,
	Quat,
};

USTRUCT()
struct FNiagaraStructConversionStep
{
	GENERATED_USTRUCT_BODY();
	
	UPROPERTY()
	int32 LWCBytes = 0;

	UPROPERTY()
	int32 LWCOffset = 0;
	
	UPROPERTY()
	int32 SimulationBytes = 0;

	UPROPERTY()
	int32 SimulationOffset = 0;
	
	UPROPERTY()
	ENiagaraStructConversionType ConversionType = ENiagaraStructConversionType::CopyOnly;

	FNiagaraStructConversionStep();
	FNiagaraStructConversionStep(int32 InLWCBytes, int32 InLWCOffset, int32 InSimulationBytes, int32 InSimulationOffset, ENiagaraStructConversionType InConversionType);

	void CopyToSim(const uint8* LWCData, uint8* SimulationData, int32 Count, int32 LWCStride, int32 SimulationStride) const;
	void CopyFromSim(uint8* LWCData, const uint8* SimulationData, int32 Count, int32 LWCStride, int32 SimulationStride) const;

	template<typename TDstType, typename TSrcType>
	void CopyInternal(uint8* DestinationData, int32 DestStride, int32 DestOffset, const uint8* SourceData, int32 SourceStride, int32 SourceOffset, int32 Count)const
	{
		for (int32 i = 0; i < Count; ++i)
		{
			TSrcType SrcValue;
			FMemory::Memcpy(&SrcValue, (SourceData + i * SourceStride) + SourceOffset, sizeof(TSrcType));

			TDstType DstValue = TDstType(SrcValue);
			FMemory::Memcpy((DestinationData + i * DestStride) + DestOffset, &DstValue, sizeof(TDstType));
		}
	}
};

/** Can convert struct data from custom structs containing LWC data such as FVector3d into struct data suitable for Niagara simulations and vice versa. */
USTRUCT()
struct FNiagaraLwcStructConverter
{
	GENERATED_USTRUCT_BODY();

	NIAGARA_API void ConvertDataToSimulation(uint8* DestinationData, const uint8* SourceData, int32 Count = 1) const;
	NIAGARA_API void ConvertDataFromSimulation(uint8* DestinationData, const uint8* SourceData, int32 Count = 1) const;
	
	NIAGARA_API void AddConversionStep(int32 InSourceBytes, int32 InSourceOffset, int32 InSimulationBytes, int32 InSimulationOffset, ENiagaraStructConversionType ConversionType);
	bool IsValid() const { return ConversionSteps.Num() > 0 && LWCSize > 0 && SWCSize > 0; }
	
private:
	UPROPERTY()
	int32 LWCSize = 0;
	UPROPERTY()
	int32 SWCSize = 0;
	UPROPERTY()
	TArray<FNiagaraStructConversionStep> ConversionSteps;
};

/** Data controlling the spawning of particles */
USTRUCT(BlueprintType, meta = (DisplayName = "Spawn Info", NiagaraClearEachFrame = "true"))
struct FNiagaraSpawnInfo
{
	GENERATED_USTRUCT_BODY();

	FNiagaraSpawnInfo() = default;
	FNiagaraSpawnInfo(int32 InCount, float InStartDt, float InIntervalDt, int32 InSpawnGroup)
		: Count(InCount), InterpStartDt(InStartDt), IntervalDt(InIntervalDt), SpawnGroup(InSpawnGroup)
	{
	}

	/** How many particles to spawn. */
	UPROPERTY(BlueprintReadWrite, Category = SpawnInfo)
	int32 Count = 0;
	/** The sub frame delta time at which to spawn the first particle. */
	UPROPERTY(BlueprintReadWrite, Category = SpawnInfo)
	float InterpStartDt = 0.0f;
	/** The sub frame delta time between each particle. */
	UPROPERTY(BlueprintReadWrite, Category = SpawnInfo)
	float IntervalDt = 1.0f;
	/**
	 * An integer used to identify this spawn info.
	 * Typically this is unused.
	 * An example usage is when using multiple spawn modules to spawn from multiple discreet locations.
	 */
	UPROPERTY(BlueprintReadWrite, Category = SpawnInfo)
	int32 SpawnGroup = 0;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Niagara ID"))
struct FNiagaraID
{
	GENERATED_USTRUCT_BODY()

	/** 
	Index in the indirection table for this particle. Allows fast access to this particles data.
	Is always unique among currently living particles but will be reused after the particle dies.
	*/
	UPROPERTY(BlueprintReadWrite, Category = ID)
	int32 Index = 0;

	/** 
	A unique tag for when this ID was acquired. 
	Allows us to differentiate between particles when one dies and another reuses it's Index.
	*/
	UPROPERTY(BlueprintReadWrite, Category = ID)
	int32 AcquireTag = 0;

	FNiagaraID() : Index(INDEX_NONE), AcquireTag(INDEX_NONE) {}
	FNiagaraID(int32 InIndex, int32 InAcquireTag): Index(InIndex), AcquireTag(InAcquireTag){}

	bool operator==(const FNiagaraID& Other)const { return Index == Other.Index && AcquireTag == Other.AcquireTag; }
	bool operator!=(const FNiagaraID& Other)const { return !(*this == Other); }
	bool operator<(const FNiagaraID& Other)const { return Index < Other.Index || (Index == Other.Index && AcquireTag < Other.AcquireTag); }

	friend inline uint32 GetTypeHash(const FNiagaraID& ID)
	{
		return HashCombine(GetTypeHash(ID.Index), GetTypeHash(ID.AcquireTag));
	}
};

USTRUCT()
struct FNiagaraRandInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Random")
	int32 Seed1 = 0;
	
	UPROPERTY(EditAnywhere, Category = "Random")
	int32 Seed2 = 0;

	UPROPERTY(EditAnywhere, Category = "Random")
	int32 Seed3 = 0;
};

#define NIAGARA_INVALID_ID (FNiagaraID({(INDEX_NONE), (INDEX_NONE)}))

enum class ENiagaraStructConversion : uint8
{
	/** Do not modify struct data members, even if they are not compatible with the Niagara VM, as the struct is user facing (or from an external api). */
	UserFacing,
	
	/** Convert struct members that are not compatible with the simulation (e.g. lwc types) into compatible types and return a simulation-friendly struct for the VM. */
	Simulation,
};

/*
*  Can convert a UStruct with fields of base types only (float, int... - will likely add native vector types here as well)
*	to an FNiagaraTypeDefinition (internal representation)
*/
class FNiagaraTypeHelper
{
public:
	static NIAGARA_API FString ToString(const uint8* ValueData, const UObject* StructOrEnum);
	static NIAGARA_API bool IsLWCStructure(const UStruct* InStruct);
	static NIAGARA_API bool IsConvertedSWCStructure(const UStruct* InStruct);
	static NIAGARA_API bool IsLWCType(const FNiagaraTypeDefinition& InType);
	static NIAGARA_API UScriptStruct* FindNiagaraFriendlyTopLevelStruct(UScriptStruct* InStruct, ENiagaraStructConversion StructConversion);
	static NIAGARA_API bool IsNiagaraFriendlyTopLevelStruct(UScriptStruct* InStruct, ENiagaraStructConversion StructConversion);
	static NIAGARA_API UScriptStruct* GetSWCStruct(UScriptStruct* LWCStruct);
	static NIAGARA_API UScriptStruct* GetLWCStruct(UScriptStruct* LWCStruct);
	static NIAGARA_API FNiagaraTypeDefinition GetSWCType(const FNiagaraTypeDefinition& InType);
	static NIAGARA_API void TickTypeRemap();

	/**
	Attempt to get the actual LWC type for a given type that would be used in the rest of the engine etc.
	Used for cases where we actually need to store types in LWC format.
	These cannot be properly serialized currently so must be constructed at runtime from their Niagara Simulation types.
	*/
	static NIAGARA_API FNiagaraTypeDefinition GetLWCType(const FNiagaraTypeDefinition& InType);

	static const FNiagaraTypeDefinition& GetVector2DDef() { return Vector2DDef; }
	static const FNiagaraTypeDefinition& GetVectorDef() { return VectorDef; }
	static const FNiagaraTypeDefinition& GetVector4Def() { return Vector4Def; }
	static const FNiagaraTypeDefinition& GetQuatDef() { return QuatDef; }
	static const FNiagaraTypeDefinition& GetDoubleDef() { return DoubleDef; }

	static NIAGARA_API void InitLWCTypes();

	//Utility functions for accessing typedefs for specific C++ types.

	template <typename T, std::enable_if_t<!std::is_enum_v<T>, int> = 0>
	static FNiagaraTypeDefinition GetTypeDef();
	template <typename T, std::enable_if_t<!std::is_enum_v<T>, int> = 0>
	static FNiagaraTypeDefinition GetSimTypeDef();

	//Special case for any enum types to access the correct TypeDefs.
	template <typename T, std::enable_if_t<std::is_enum_v<T>, int> = 0>
	static FNiagaraTypeDefinition GetTypeDef() ;
	template <typename T, std::enable_if_t<std::is_enum_v<T>, int> = 0>
	static FNiagaraTypeDefinition GetSimTypeDef() ;


private:
	struct FRemapEntry
	{
		UScriptStruct* Get(UScriptStruct* InStruct) const
		{
#if WITH_EDITORONLY_DATA
			return SerialNumber == InStruct->FieldPathSerialNumber ? Struct.Get() : nullptr;
#else
			return Struct.Get();
#endif
		}

		TWeakObjectPtr<UScriptStruct>	Struct;
#if WITH_EDITORONLY_DATA
		int32 SerialNumber = 0;
#endif
	};

	static FRWLock RemapTableLock;
	static TMap<TWeakObjectPtr<UScriptStruct>, FRemapEntry> RemapTable;
	static std::atomic<bool> RemapTableDirty;

	/** LWC Counterparts to Niagara's base simulation types. */
	NIAGARA_API static FNiagaraTypeDefinition Vector2DDef;
	NIAGARA_API static FNiagaraTypeDefinition VectorDef;
	NIAGARA_API static FNiagaraTypeDefinition Vector4Def;
	NIAGARA_API static FNiagaraTypeDefinition QuatDef;
	NIAGARA_API static FNiagaraTypeDefinition DoubleDef;

	static FString ConvertedSWCStructSuffix;
};

/** Information about how this type should be laid out in an FNiagaraDataSet */
USTRUCT()
struct FNiagaraTypeLayoutInfo
{
	GENERATED_BODY()

private:
	UPROPERTY()
	uint16 NumFloatComponents = 0;

	UPROPERTY()
	uint16 NumInt32Components = 0;

	UPROPERTY()
	uint16 NumHalfComponents = 0;

	UPROPERTY()
	TArray<uint32> ComponentsOffsets;

public:
	uint32 GetNumFloatComponents() const { return NumFloatComponents; }
	uint32 GetNumInt32Components() const { return NumInt32Components; }
	uint32 GetNumHalfComponents() const { return NumHalfComponents; }
	uint32 GetTotalComponents() const { return GetNumFloatComponents() + GetNumInt32Components() + GetNumHalfComponents(); }

	uint32 GetFloatComponentByteOffset(int32 Component) const { return ComponentsOffsets[GetFloatArrayByteOffset() + Component]; }
	uint32 GetFloatComponentRegisterOffset(int32 Component) const { return ComponentsOffsets[GetFloatArrayRegisterOffset() + Component]; }

	uint32 GetInt32ComponentByteOffset(int32 Component) const { return ComponentsOffsets[GetInt32ArrayByteOffset() + Component]; }
	uint32 GetInt32ComponentRegisterOffset(int32 Component) const { return ComponentsOffsets[GetInt32ArrayRegisterOffset() + Component]; }

	uint32 GetHalfComponentByteOffset(int32 Component) const { return ComponentsOffsets[GetHalfArrayByteOffset() + Component]; }
	uint32 GetHalfComponentRegisterOffset(int32 Component) const { return ComponentsOffsets[GetHalfArrayRegisterOffset() + Component]; }

	NIAGARA_API void GenerateLayoutInfo(const UScriptStruct* Struct);

private:
	int32 GetFloatArrayByteOffset() const { return 0; }
	int32 GetFloatArrayRegisterOffset() const { return NumFloatComponents; }

	int32 GetInt32ArrayByteOffset() const { return NumFloatComponents * 2; }
	int32 GetInt32ArrayRegisterOffset() const { return (NumFloatComponents * 2) + NumInt32Components; }

	int32 GetHalfArrayByteOffset() const { return (NumFloatComponents * 2) + (NumInt32Components * 2); }
	int32 GetHalfArrayRegisterOffset() const { return (NumFloatComponents * 2) + (NumInt32Components * 2) + NumHalfComponents; }

	void GenerateLayoutInfoInternal(const UScriptStruct* Struct, int32& FloatCount, int32& Int32Count, int32& HalfCount, bool bCountComponentsOnly, int32 BaseOffset = 0);
};


/** Defines different modes for selecting the output numeric type of a function or operation based on the types of the inputs. */
UENUM()
enum class ENiagaraNumericOutputTypeSelectionMode : uint8
{
	/** Output type selection not supported. */
	None UMETA(Hidden),
	/** Select the largest of the numeric inputs. */
	Largest,
	/** Select the smallest of the numeric inputs. */
	Smallest,
	/** Selects the base scalar type for this numeric inputs. */
	Scalar,
	/** Selects the type based on custom logic from the node. */
	Custom,
};

/** 
The source from which a script execution state was set. Used to allow scalability etc to change the state but only if the state has not been defined by something with higher precedence. 
If this changes, all scripts must be recompiled by bumping the NiagaraCustomVersion
*/
UENUM()
enum class ENiagaraExecutionStateSource : uint32
{
	/** State set by Scalability logic. Lowest precedence. */
	Scalability,
	/** Misc internal state. For example becoming inactive after we finish our set loops. */
	Internal, 
	/** State requested by the owner. Takes precedence over everything but internal completion logic. */
	Owner, 
	/** Internal completion logic. Has to take highest precedence for completion to be ensured. */
	InternalCompletion,
};

UENUM()
enum class ENiagaraExecutionState : uint32
{
	/**  Run all scripts. Allow spawning.*/
	Active,
	/** Run all scripts but suppress any new spawning.*/
	Inactive,
	/** Clear all existing particles and move to inactive.*/
	InactiveClear,
	/** Complete. When the system or all emitters are complete the effect is considered finished. */
	Complete,
	/** Emitter only. Emitter is disabled. Will not tick or render again until a full re initialization of the system. */
	Disabled UMETA(Hidden),

	// insert new states before
	Num UMETA(Hidden)
};

UENUM()
enum class ENiagaraExecutionStateManagement : uint32
{
	/** Change the execution state to active (if possible). */
	Awaken,
	/**
	Change the execution state to inactivate (if possible) and stop spawning particles.
	Age will continue to increase and can complete if the loop condition is complete.
	Can be woken again.
	*/
	SleepAndLetParticlesFinish,
	/**
	Change the execution state to inactivate clear (if possible), and kill all existing particles.
	Age will continue to increase and can complete if the loop condition is complete.
	Can be woken again.
	*/
	SleepAndClearParticles,
	/**
	Change the execution state to complete (if possible), and kill all existing particles.
	Can not wake up again.
	*/
	KillImmediately,
	/**
	Change the execution state to inactive (if possible), and stop spawning particles.
	When we have no particles activate will change the execution state to complete (if possible).
	Can not wake up again if particles are still active.
	*/
	KillAfterParticlesFinish,

	Num UMETA(Hidden)
};

UENUM()
enum class ENiagaraCoordinateSpace : uint32
{
	/** Use the coordinate space specified by the Emitter*/
	Simulation,
	/** Use the world coordinate space*/
	World,
	/** Use the local coordinate space*/
	Local,

	// insert new states before
	//NewEnumerator0 = 0 UMETA(Hidden),
	//NewEnumerator1 = 1 UMETA(Hidden),
	//NewEnumerator2 = 2 UMETA(Hidden)
};

UENUM()
enum class ENiagaraPythonUpdateScriptReference : uint8
{
	None,
    ScriptAsset,
    DirectTextEntry
};

UENUM()
enum class ENiagaraOrientationAxis : uint32
{
	XAxis UMETA(DisplayName="X Axis"),
	YAxis UMETA(DisplayName = "Y Axis"),
	ZAxis UMETA(DisplayName = "Z Axis"),
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraTypeDefinition
{
	GENERATED_USTRUCT_BODY()

	enum FUnderlyingType : uint16
	{
		UT_None = 0,
		UT_Class,
		UT_Struct,
		UT_Enum
	};

	enum FTypeFlags : uint8
	{
		TF_None			= 0x00,
		TF_Static		= 0x01,

		/// indicates that the ClassStructOrEnum property has been serialized as the LWC struct (see FNiagaraTypeHelper)
		/// instead of the Transient SWC version of the struct
		TF_SerializedAsLWC	= 0x02,

		/// DEPRECATED, DO NOT USE
		TF_AllowLWC_DEPRECATED		= 0x04,
	};

public:

	// Construct blank raw type definition 
	inline FNiagaraTypeDefinition(UObject* Object)
		: ClassStructOrEnum(Object)
#if WITH_EDITORONLY_DATA
		, Struct_DEPRECATED(nullptr), Enum_DEPRECATED(nullptr)
#endif
	{
		if(UClass* Class = Cast<UClass>(Object))
		{
			*this = FNiagaraTypeDefinition(Class);
		}
		else if (UScriptStruct* Struct = Cast<UScriptStruct>(Object))
		{
			*this = FNiagaraTypeDefinition(Struct, EAllowUnfriendlyStruct::Deny);
		}
		else if(UEnum* Enum = Cast<UEnum>(Object))
		{
			*this = FNiagaraTypeDefinition(Enum);
		}
		else
		{
			check(0);
		}

		GeneratePathNameHash();
	}

	inline FNiagaraTypeDefinition(UClass *ClassDef)
		: ClassStructOrEnum(ClassDef), UnderlyingType(UT_Class)
#if WITH_EDITORONLY_DATA
		, Struct_DEPRECATED(nullptr), Enum_DEPRECATED(nullptr)
#endif
	{
		checkSlow(ClassStructOrEnum != nullptr);

		GeneratePathNameHash();
	}

	inline FNiagaraTypeDefinition(UEnum *EnumDef)
		: ClassStructOrEnum(EnumDef), UnderlyingType(UT_Enum)
#if WITH_EDITORONLY_DATA
		, Struct_DEPRECATED(nullptr), Enum_DEPRECATED(nullptr)
#endif
	{
		checkSlow(ClassStructOrEnum != nullptr);

		GeneratePathNameHash();
	}

	enum class EAllowUnfriendlyStruct : uint8 { Deny, Allow };
	inline FNiagaraTypeDefinition(UScriptStruct* StructDef, EAllowUnfriendlyStruct AllowUnfriendlyStruct)
		: ClassStructOrEnum(StructDef), UnderlyingType(UT_Struct)
#if WITH_EDITORONLY_DATA
		, Struct_DEPRECATED(nullptr), Enum_DEPRECATED(nullptr)
#endif
	{
		checkSlow(ClassStructOrEnum != nullptr);
		ensureAlwaysMsgf(AllowUnfriendlyStruct == EAllowUnfriendlyStruct::Allow || FNiagaraTypeHelper::IsNiagaraFriendlyTopLevelStruct(StructDef, ENiagaraStructConversion::UserFacing), TEXT("Struct(%s) is not supported."), *StructDef->GetName());

		GeneratePathNameHash();
	}
	
	inline FNiagaraTypeDefinition(UScriptStruct* StructDef) : FNiagaraTypeDefinition(StructDef, EAllowUnfriendlyStruct::Deny) {}

	inline FNiagaraTypeDefinition(FProperty* Property, EAllowUnfriendlyStruct AllowUnfriendlyStruct)
	{
		if (Property->IsA(FFloatProperty::StaticClass())) { *this = FNiagaraTypeDefinition::GetFloatDef(); }
		else if (Property->IsA(FUInt16Property::StaticClass())) { *this = FNiagaraTypeDefinition::GetHalfDef(); }
		else if (Property->IsA(FIntProperty::StaticClass())) { *this = FNiagaraTypeDefinition::GetIntDef(); }
		else if (Property->IsA(FBoolProperty::StaticClass())) { *this = FNiagaraTypeDefinition::GetBoolDef(); }
		else if (FStructProperty* StructProp = CastField<FStructProperty>(Property)) { *this = FNiagaraTypeDefinition(StructProp->Struct, AllowUnfriendlyStruct); }
		else
		{
			checkf(0, TEXT("Invalid Type"))
		}
	}

	// Construct a blank raw type definition
	inline FNiagaraTypeDefinition()
		: ClassStructOrEnum(nullptr), UnderlyingType(UT_None)
#if WITH_EDITORONLY_DATA
		, Struct_DEPRECATED(nullptr), Enum_DEPRECATED(nullptr)
#endif
	{}

	inline bool operator !=(const FNiagaraTypeDefinition &Other) const
	{
		return !(*this == Other);
	}

	inline bool operator == (const FNiagaraTypeDefinition &Other) const
	{
		return ClassStructOrEnum == Other.ClassStructOrEnum && UnderlyingType == Other.UnderlyingType && Flags == Other.Flags;
	}

	inline bool IsSameBaseDefinition(const FNiagaraTypeDefinition& Other) const
	{
		return ClassStructOrEnum == Other.ClassStructOrEnum && UnderlyingType == Other.UnderlyingType;
	}

	FText GetNameText() const
	{
		if (IsValid() == false)
		{
			return NSLOCTEXT("NiagaraTypeDefinition", "InvalidNameText", "Invalid (null type)");
		}

		FText NameText;
#if WITH_EDITOR
		if ( UnderlyingType != UT_Enum )
		{
			// fix names for common types after LWC changes
			if (*this == GetVec2Def())
			{
				NameText = FText::FromString("Vector 2D");
			}
			else if (*this == GetVec3Def())
			{
				NameText = FText::FromString("Vector");
			}
			else if (*this == GetVec4Def())
			{
				NameText = FText::FromString("Vector 4");
			}
			else if (*this == GetQuatDef())
			{
				NameText = FText::FromString("Quat");
			}
			else if (*this == FNiagaraTypeHelper::GetDoubleDef())
			{
				NameText = FText::FromString("float");
			}
			else
			{
				NameText = GetStruct()->GetDisplayNameText();
			}
		}
		else
#endif
		{
			NameText = FText::FromString(ClassStructOrEnum->GetName());
		}

		if (Flags != 0)
		{
			if (IsStatic())
			{
				NameText = FText::Format(NSLOCTEXT("NiagaraTypeDefinition", "FlagFormatNameText", "Static {0}"), NameText);
			}
			else
			{
				NameText = FText::Format(NSLOCTEXT("NiagaraTypeDefinition", "UnknownNameText", "Unknown Flag Type {0}"), NameText);
			}
		}
		
		return NameText;
		
	}

	FName GetFName() const
	{
		if ( IsValid() == false )
		{
			return FName();
		}
		return ClassStructOrEnum.GetFName();
	}

	FString GetName() const
	{
		if (IsValid() == false)
		{
			return TEXT("Invalid");
		}
		return ClassStructOrEnum.GetName();
	}

	UStruct* GetStruct() const
	{
		return UnderlyingType == UT_Enum ? IntStruct : Cast<UStruct>(ClassStructOrEnum);
	}

	UScriptStruct* GetScriptStruct()const
	{
		return Cast<UScriptStruct>(GetStruct());
	}

	/** Gets the class ptr for this type if it is a class. */
	UClass* GetClass() const
	{
		return UnderlyingType == UT_Class ? CastChecked<UClass>(ClassStructOrEnum) : nullptr;
	}

	UEnum* GetEnum() const
	{
		return UnderlyingType == UT_Enum ? CastChecked<UEnum>(ClassStructOrEnum) : nullptr;
	}

	NIAGARA_API bool IsDataInterface() const;

	inline bool IsUObject() const
	{
		const UStruct* ClassStruct = GetStruct();
		return ClassStruct ? ClassStruct->IsChildOf<UObject>() : false;
	}

	bool IsEnum() const { return UnderlyingType == UT_Enum; }

	bool IsStatic() const { return (GetFlags() & TF_Static) != 0; }

	void SetFlags(FTypeFlags InFlags)
	{
		Flags = InFlags;
	}

	uint32 GetFlags()const { return Flags; }
	

	bool IsIndexWildcard() const { return ClassStructOrEnum == FNiagaraTypeDefinition::GetWildcardStruct(); }

	NIAGARA_API FNiagaraTypeDefinition ToStaticDef() const;
	NIAGARA_API FNiagaraTypeDefinition RemoveStaticDef() const;

	int32 GetSize() const
	{
		if (Size == INDEX_NONE)
		{
			checkfSlow(IsValid(), TEXT("Type definition is not valid."));
			if (ClassStructOrEnum == nullptr || GetClass())
			{
				Size = 0;//TODO: sizeof(void*);//If we're a class then we allocate space for the user to instantiate it. This and stopping it being GCd is up to the user.
			}
			else
			{
				const int32 StructSize = CastChecked<UScriptStruct>(GetStruct())->GetStructureSize();
				ensure(StructSize <= TNumericLimits<int16>::Max());
				Size = int16(StructSize);
			}
		}
		return Size;
	}

	int32 GetAlignment() const
	{
		if (Alignment == INDEX_NONE)
		{
			checkfSlow(IsValid(), TEXT("Type definition is not valid."));
			if (ClassStructOrEnum == nullptr || GetClass())
			{
				Alignment = 0;//TODO: sizeof(void*);//If we're a class then we allocate space for the user to instantiate it. This and stopping it being GCd is up to the user.
			}
			else
			{
				const int32 StructAlignment = CastChecked<UScriptStruct>(GetStruct())->GetMinAlignment();
				ensure(StructAlignment <= TNumericLimits<int16>::Max());
				Alignment = int16(StructAlignment);
			}
		}
		return Alignment;
	}

	bool IsFloatPrimitive() const
	{
		return ClassStructOrEnum == GetFloatStruct() || ClassStructOrEnum == GetVec2Struct() || ClassStructOrEnum ==
			GetVec3Struct() || ClassStructOrEnum == GetVec4Struct() ||
			ClassStructOrEnum == GetMatrix4Struct() || ClassStructOrEnum == GetColorStruct() || ClassStructOrEnum ==
			GetQuatStruct() || ClassStructOrEnum == GetPositionStruct();
 	}

	bool IsIndexType() const
	{
		return ClassStructOrEnum == GetIntStruct() || ClassStructOrEnum == GetBoolStruct() || IsEnum();
	}
	bool IsValid() const 
	{ 
		return ClassStructOrEnum != nullptr;
	}
	void Invalidate()
	{
		ClassStructOrEnum = nullptr;
		UnderlyingTypePathNameHash = 0;
	}	
	
	NIAGARA_API bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;

#if WITH_EDITORONLY_DATA
	NIAGARA_API bool IsInternalType() const;
#endif

	/*
	Underlying type for this variable, use FUnderlyingType to determine type without casting
	This can be a UClass, UStruct or UEnum.  Pointing to something like the struct for an FVector, etc.
	In occasional situations this may be a UClass when we're dealing with DataInterface etc.
	*/
	UPROPERTY(EditAnywhere, Category=Type)
	TObjectPtr<UObject> ClassStructOrEnum;

	// See enumeration FUnderlyingType for possible values
	UPROPERTY(EditAnywhere, Category=Type)
	uint16 UnderlyingType;

	NIAGARA_API bool Serialize(FArchive& Ar);
	NIAGARA_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

private:
	// for allowing access to the cached RegisteredTypeDefIndex
	friend class FNiagaraTypeRegistry;

	UPROPERTY(EditAnywhere, Category = Type)
	uint8 Flags = TF_None;

	mutable int16 Size = INDEX_NONE;
	mutable int16 Alignment = INDEX_NONE;

	mutable int32 RegisteredTypeDefIndex = INDEX_NONE;

	// the type hash of the full path name for the underlying type.  This is helpful when identifying the type
	// especially in scenarios where plugins contain the structs/classes and we want to re-use slots in the registry
	// even if the objects are recreated (with different pointers/object indices)
	uint32 UnderlyingTypePathNameHash = INDEX_NONE;

	void InvalidateCachedIndex(bool bInvalidate = true);

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UStruct> Struct_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UEnum> Enum_DEPRECATED;
#endif

public:
	static NIAGARA_API void Init();
#if WITH_EDITOR
	static NIAGARA_API void RecreateUserDefinedTypeRegistry();
#endif
	static const FNiagaraTypeDefinition& GetFloatDef() { return FloatDef; }
	static const FNiagaraTypeDefinition& GetBoolDef() { return BoolDef; }
	static const FNiagaraTypeDefinition& GetIntDef() { return IntDef; }
	static const FNiagaraTypeDefinition& GetVec2Def() { return Vec2Def; }
	static const FNiagaraTypeDefinition& GetVec3Def() { return Vec3Def; }
	static const FNiagaraTypeDefinition& GetVec4Def() { return Vec4Def; }
	static const FNiagaraTypeDefinition& GetColorDef() { return ColorDef; }
	static const FNiagaraTypeDefinition& GetPositionDef() { return PositionDef; }
	static const FNiagaraTypeDefinition& GetQuatDef() { return QuatDef; }
	static const FNiagaraTypeDefinition& GetMatrix4Def() { return Matrix4Def; }
	static const FNiagaraTypeDefinition& GetGenericNumericDef() { return NumericDef; }
	static const FNiagaraTypeDefinition& GetParameterMapDef() { return ParameterMapDef; }
	static const FNiagaraTypeDefinition& GetIDDef() { return IDDef; }
	static const FNiagaraTypeDefinition& GetRandInfoDef() { return RandInfoDef; }
	static const FNiagaraTypeDefinition& GetUObjectDef() { return UObjectDef; }
	static const FNiagaraTypeDefinition& GetUMaterialDef() { return UMaterialDef; }
	static const FNiagaraTypeDefinition& GetUStaticMeshDef() { return UStaticMeshDef; }
	static const FNiagaraTypeDefinition& GetUTextureDef() { return UTextureDef; }
	static const FNiagaraTypeDefinition& GetUTextureRenderTargetDef() { return UTextureRenderTargetDef; }
	static const FNiagaraTypeDefinition& GetSimCacheClassDef() { return USimCacheClassDef; }
	static const FNiagaraTypeDefinition& GetWildcardDef() { return WildcardDef; }
	static const FNiagaraTypeDefinition& GetHalfDef() { return HalfDef; }
	static const FNiagaraTypeDefinition& GetHalfVec2Def() { return HalfVec2Def; }
	static const FNiagaraTypeDefinition& GetHalfVec3Def() { return HalfVec3Def; }
	static const FNiagaraTypeDefinition& GetHalfVec4Def() { return HalfVec4Def; }

	template<typename T>
	static const FNiagaraTypeDefinition& Get();

	static UScriptStruct* GetFloatStruct() { return FloatStruct; }
	static UScriptStruct* GetBoolStruct() { return BoolStruct; }
	static UScriptStruct* GetIntStruct() { return IntStruct; }
	static UScriptStruct* GetVec2Struct() { return Vec2Struct; }
	static UScriptStruct* GetVec3Struct() { return Vec3Struct; }
	static UScriptStruct* GetVec4Struct() { return Vec4Struct; }
	static UScriptStruct* GetColorStruct() { return ColorStruct; }
	static UScriptStruct* GetPositionStruct() { return PositionStruct; }
	static UScriptStruct* GetQuatStruct() { return QuatStruct; }
	static UScriptStruct* GetMatrix4Struct() { return Matrix4Struct; }
	static UScriptStruct* GetGenericNumericStruct() { return NumericStruct; }
	static UScriptStruct* GetWildcardStruct() { return WildcardStruct; }
	static UScriptStruct* GetParameterMapStruct() { return ParameterMapStruct; }
	static UScriptStruct* GetIDStruct() { return IDStruct; }
	static UScriptStruct* GetRandInfoStruct() { return RandInfoStruct; }

	static UScriptStruct* GetHalfStruct() { return HalfStruct; }
	static UScriptStruct* GetHalfVec2Struct() { return HalfVec2Struct; }
	static UScriptStruct* GetHalfVec3Struct() { return HalfVec3Struct; }
	static UScriptStruct* GetHalfVec4Struct() { return HalfVec4Struct; }

	static UEnum* GetExecutionStateEnum() { return ExecutionStateEnum; }
	static UEnum* GetCoordinateSpaceEnum() { return CoordinateSpaceEnum; }
	static UEnum* GetOrientationAxisEnum() { return OrientationAxisEnum; }
	static UEnum* GetExecutionStateSouceEnum() { return ExecutionStateSourceEnum; }
	static UEnum* GetSimulationTargetEnum() { return SimulationTargetEnum; }
	static UEnum* GetScriptUsageEnum() { return ScriptUsageEnum; }
	static UEnum* GetScriptContextEnum() { return ScriptContextEnum; }
	static UEnum* GetParameterPanelCategoryEnum() { return ParameterPanelCategoryEnum; }
	static UEnum* GetFunctionDebugStateEnum() { return FunctionDebugStateEnum; }

	static UEnum* GetParameterScopeEnum() { return ParameterScopeEnum; }

	static const FNiagaraTypeDefinition& GetCollisionEventDef() { return CollisionEventDef; }

	static NIAGARA_API bool IsScalarDefinition(const FNiagaraTypeDefinition& Type);

	FString ToString(const uint8* ValueData)const
	{
		checkf(IsValid(), TEXT("Type definition is not valid."));
		if (ValueData == nullptr)
		{
			return TEXT("(null)");
		}
		return FNiagaraTypeHelper::ToString(ValueData, ClassStructOrEnum);
	}
	
	// Evaluates if two pin types are compatible for assignment.  Note that the assignment being evaluated is InputPinType = OutputPinType
	static NIAGARA_API bool TypesAreAssignable(const FNiagaraTypeDefinition& InputPinType, const FNiagaraTypeDefinition& OutputPinType, bool bAllowLossyLWCConversions = false);
	static NIAGARA_API bool IsLossyConversion(const FNiagaraTypeDefinition& FromType, const FNiagaraTypeDefinition& ToType);
	static NIAGARA_API FNiagaraTypeDefinition GetNumericOutputType(TConstArrayView<FNiagaraTypeDefinition> TypeDefinintions, ENiagaraNumericOutputTypeSelectionMode SelectionMode);

	static const TArray<FNiagaraTypeDefinition>& GetNumericTypes() { return OrderedNumericTypes; }
	static NIAGARA_API bool IsValidNumericInput(const FNiagaraTypeDefinition& TypeDef);

	

private:

	static NIAGARA_API FNiagaraTypeDefinition FloatDef;
	static NIAGARA_API FNiagaraTypeDefinition BoolDef;
	static NIAGARA_API FNiagaraTypeDefinition IntDef;
	static NIAGARA_API FNiagaraTypeDefinition Vec2Def;
	static NIAGARA_API FNiagaraTypeDefinition Vec3Def;
	static NIAGARA_API FNiagaraTypeDefinition Vec4Def;
	static NIAGARA_API FNiagaraTypeDefinition ColorDef;
	static NIAGARA_API FNiagaraTypeDefinition PositionDef;
	static NIAGARA_API FNiagaraTypeDefinition QuatDef;
	static NIAGARA_API FNiagaraTypeDefinition Matrix4Def;
	static NIAGARA_API FNiagaraTypeDefinition NumericDef;
	static NIAGARA_API FNiagaraTypeDefinition ParameterMapDef;
	static NIAGARA_API FNiagaraTypeDefinition IDDef;
	static NIAGARA_API FNiagaraTypeDefinition RandInfoDef;
	static NIAGARA_API FNiagaraTypeDefinition UObjectDef;
	static NIAGARA_API FNiagaraTypeDefinition UMaterialDef;
	static NIAGARA_API FNiagaraTypeDefinition UTextureDef;
	static NIAGARA_API FNiagaraTypeDefinition UTextureRenderTargetDef;
	static NIAGARA_API FNiagaraTypeDefinition UStaticMeshDef;
	static NIAGARA_API FNiagaraTypeDefinition USimCacheClassDef;
	static NIAGARA_API FNiagaraTypeDefinition WildcardDef;

	static NIAGARA_API FNiagaraTypeDefinition HalfDef;
	static NIAGARA_API FNiagaraTypeDefinition HalfVec2Def;
	static NIAGARA_API FNiagaraTypeDefinition HalfVec3Def;
	static NIAGARA_API FNiagaraTypeDefinition HalfVec4Def;

	static NIAGARA_API UScriptStruct* FloatStruct;
	static NIAGARA_API UScriptStruct* BoolStruct;
	static NIAGARA_API UScriptStruct* IntStruct;
	static NIAGARA_API UScriptStruct* Vec2Struct;
	static NIAGARA_API UScriptStruct* Vec3Struct;
	static NIAGARA_API UScriptStruct* Vec4Struct;
	static NIAGARA_API UScriptStruct* QuatStruct;
	static NIAGARA_API UScriptStruct* ColorStruct;
	static NIAGARA_API UScriptStruct* Matrix4Struct;
	static NIAGARA_API UScriptStruct* NumericStruct;
	static NIAGARA_API UScriptStruct* WildcardStruct;
	static NIAGARA_API UScriptStruct* PositionStruct;

	static NIAGARA_API UScriptStruct* HalfStruct;
	static NIAGARA_API UScriptStruct* HalfVec2Struct;
	static NIAGARA_API UScriptStruct* HalfVec3Struct;
	static NIAGARA_API UScriptStruct* HalfVec4Struct;

	static NIAGARA_API UClass* UObjectClass;
	static NIAGARA_API UClass* UMaterialClass;
	static NIAGARA_API UClass* UTextureClass;
	static NIAGARA_API UClass* UTextureRenderTargetClass;

	static NIAGARA_API UEnum* SimulationTargetEnum;
	static NIAGARA_API UEnum* ScriptUsageEnum;
	static NIAGARA_API UEnum* ScriptContextEnum;
	static NIAGARA_API UEnum* ExecutionStateEnum;
	static NIAGARA_API UEnum* CoordinateSpaceEnum;
	static NIAGARA_API UEnum* OrientationAxisEnum;
	static NIAGARA_API UEnum* ExecutionStateSourceEnum;

	static NIAGARA_API UEnum* ParameterScopeEnum;
	static NIAGARA_API UEnum* ParameterPanelCategoryEnum;

	static NIAGARA_API UEnum* FunctionDebugStateEnum;

	static NIAGARA_API UScriptStruct* ParameterMapStruct;
	static NIAGARA_API UScriptStruct* IDStruct;
	static NIAGARA_API UScriptStruct* RandInfoStruct;

	static NIAGARA_API TSet<UScriptStruct*> NumericStructs;
	static NIAGARA_API TArray<FNiagaraTypeDefinition> OrderedNumericTypes;

	static NIAGARA_API TSet<UScriptStruct*> ScalarStructs;

	static NIAGARA_API TSet<UStruct*> FloatStructs;
	static NIAGARA_API TSet<UStruct*> IntStructs;
	static NIAGARA_API TSet<UStruct*> BoolStructs;

	static NIAGARA_API FNiagaraTypeDefinition CollisionEventDef;

	void NIAGARA_API GeneratePathNameHash();

	friend inline uint32 GetTypeHash(const FNiagaraTypeDefinition& Type)
	{
		return HashCombine(HashCombine(Type.UnderlyingTypePathNameHash, GetTypeHash(Type.UnderlyingType)), GetTypeHash(Type.GetFlags()));
	}

	friend inline uint32 GetPersistentTypeHash(const FNiagaraTypeDefinition& Type)
	{
		const FString PathName = GetPathNameSafe(Type.ClassStructOrEnum);
		return HashCombine(HashCombine(GetTypeHash(PathName), GetTypeHash(Type.UnderlyingType)), GetTypeHash(Type.GetFlags()));
	}
};

template<>
struct TStructOpsTypeTraits<FNiagaraTypeDefinition> : public TStructOpsTypeTraitsBase2<FNiagaraTypeDefinition>
{
	enum
	{
		WithSerializer = true,
		WithImportTextItem = true,
	};
};


//Helper to get the correct typedef for templated code.
template<typename T>
const FNiagaraTypeDefinition& FNiagaraTypeDefinition::Get()
{
	     if constexpr (std::is_same_v<T, float>) { return GetFloatDef(); }
	else if constexpr (std::is_same_v<T, FVector2f>) { return GetVec2Def(); }
	else if constexpr (std::is_same_v<T, FVector3f>) { return GetVec3Def(); }
	else if constexpr (std::is_same_v<T, FNiagaraPosition>) { return GetPositionDef(); }
	else if constexpr (std::is_same_v<T, FVector4f>) { return GetVec4Def(); }
	else if constexpr (std::is_same_v<T, int32>) { return GetIntDef(); }
	else if constexpr (std::is_same_v<T, FNiagaraBool>) { return GetBoolDef(); }
	else if constexpr (std::is_same_v<T, FQuat4f>) { return GetQuatDef(); }
	else if constexpr (std::is_same_v<T, FMatrix44f>) { return GetMatrix4Def(); }
	else if constexpr (std::is_same_v<T, FLinearColor>) { return GetColorDef(); }
	else if constexpr (std::is_same_v<T, FNiagaraID>) { return GetIDDef(); }
	else if constexpr (std::is_same_v<T, FNiagaraRandInfo>) { return GetRandInfoDef(); }
	else { static_assert(sizeof(T) == 0, "Unsupported type"); }
} //-V591

//////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FNiagaraTypeDefinitionHandle
{
	GENERATED_USTRUCT_BODY()

	FNiagaraTypeDefinitionHandle() : RegisteredTypeIndex(INDEX_NONE) {}
	explicit FNiagaraTypeDefinitionHandle(const FNiagaraTypeDefinition& Type) : RegisteredTypeIndex(Register(Type)) {}

	const FNiagaraTypeDefinition& operator*() const { return Resolve(); }
	const FNiagaraTypeDefinition* operator->() const { return &Resolve(); }

	bool operator==(const FNiagaraTypeDefinitionHandle& Other) const
	{
		return RegisteredTypeIndex == Other.RegisteredTypeIndex;
	}
	bool operator!=(const FNiagaraTypeDefinitionHandle& Other) const
	{
		return RegisteredTypeIndex != Other.RegisteredTypeIndex;
	}

	bool IsSameBase(const FNiagaraTypeDefinition& Other) const
	{
		return Resolve().IsSameBaseDefinition(Other);
	}

	bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
	{
		return Resolve().AppendCompileHash(InVisitor);
	}

private:
	friend FArchive& operator<<(FArchive& Ar, FNiagaraTypeDefinitionHandle& Handle);

	NIAGARA_API const FNiagaraTypeDefinition& Resolve() const;
	NIAGARA_API int32 Register(const FNiagaraTypeDefinition& TypeDef) const;

	UPROPERTY()
	int32 RegisteredTypeIndex = INDEX_NONE;
};

//////////////////////////////////////////////////////////////////////////
USTRUCT()
struct FNiagaraVariableBase
{
	GENERATED_USTRUCT_BODY()

	inline FNiagaraVariableBase()
		: Name(NAME_None)
		, TypeDefHandle(FNiagaraTypeDefinition::GetVec4Def())
#if WITH_EDITORONLY_DATA
		, TypeDef_DEPRECATED(FNiagaraTypeDefinition::GetVec4Def())
#endif
		{}

	inline FNiagaraVariableBase(const FNiagaraTypeDefinition& InType, const FName& InName)
		: Name(InName)
		, TypeDefHandle(InType)
#if WITH_EDITORONLY_DATA
		, TypeDef_DEPRECATED(InType)
#endif
		{}

	inline FNiagaraVariableBase(const FNiagaraVariableCommonReference& VariableRef)
		: Name(VariableRef.Name)
		, TypeDefHandle(FNiagaraTypeDefinition(VariableRef.UnderlyingType))
	{}

	/** Check if Name and Type definition are the same. The actual stored value is not checked here.*/
	bool operator==(const FNiagaraVariableBase& Other)const
	{
		return Name == Other.Name && TypeDefHandle == Other.TypeDefHandle;
	}

	/** Check if Name and Type definition are not the same. The actual stored value is not checked here.*/
	bool operator!=(const FNiagaraVariableBase& Other)const
	{
		return !(*this == Other);
	}

	/** Variables are the same name but if types are auto-assignable, allow them to match. */
	bool IsEquivalent(const FNiagaraVariableBase& Other, bool bAllowAssignableTypes = true)const
	{
		return Name == Other.Name && (TypeDefHandle == Other.TypeDefHandle || (bAllowAssignableTypes && FNiagaraTypeDefinition::TypesAreAssignable(*TypeDefHandle, *Other.TypeDefHandle)));
	}
	
	inline void SetName(FName InName)
	{
		Name = InName;
	}
	inline const FName& GetName() const
	{
		return Name;
	}

	NIAGARA_API void SetNamespacedName(const FString& InNamespace, FName InVariableName);

	void SetType(const FNiagaraTypeDefinition& InTypeDef)
	{
		TypeDefHandle = FNiagaraTypeDefinitionHandle(InTypeDef);
	}
	const FNiagaraTypeDefinition& GetType()const
	{
		return *TypeDefHandle;
	}

	inline bool IsDataInterface()const { return GetType().IsDataInterface(); }
	inline bool IsUObject()const { return GetType().IsUObject(); }

	int32 GetSizeInBytes() const
	{
		return TypeDefHandle->GetSize();
	}

	int32 GetAlignment()const
	{
		return TypeDefHandle->GetAlignment();
	}

	bool IsValid() const
	{
		return Name != NAME_None && TypeDefHandle->IsValid();
	}

	/**
	Removes the root namespace from the variable if it matches the expected namespace.
	i.e. MyNamespace.VarName would become VarName but Other.MyNamespace.VarName would not be modified.
	For more complex namespace manipulation refer to FNiagaraUtilities::ResolveAliases
	*/
	NIAGARA_API bool RemoveRootNamespace(const FStringView& ExpectedNamespace);

	/**
	Replaces the root namespace from the variable if it matches the expected namespace with the NewNamespace.
	i.e. Prev.VarName would become Next.VarName but Other.Prev.VarName would not be modified.
	For more complex namespace manipulation refer to FNiagaraUtilities::ResolveAliases
	*/
	NIAGARA_API bool ReplaceRootNamespace(const FStringView& ExpectedNamespace, const FStringView& NewNamespace);

	static bool IsInNameSpace(const FStringView& Namespace, FName VariableName)
	{
		FNameBuilder NameString;
		VariableName.ToString(NameString);

		FStringView NameStringView = NameString.ToView();
		if (Namespace.Len() > 1 && Namespace[Namespace.Len() - 1] == '.') // Includes period in namespace
		{
			return (NameStringView.Len() > Namespace.Len()) && NameStringView.StartsWith(Namespace);
		}
		else // Skips period, makes sure it's in the name string to delineate
		{
			return (NameStringView.Len() > Namespace.Len() + 1) && (NameStringView[Namespace.Len()] == '.') && NameStringView.StartsWith(Namespace);
		}
	}

	inline bool IsInNameSpace(const FStringView& Namespace) const
	{
		return IsInNameSpace(Namespace, Name);
	}

#if WITH_EDITORONLY_DATA
	// This method should not be used at runtime as we have pre-defined strings in FNiagaraConstants for runtime cases
	inline bool IsInNameSpace(const FName& Namespace) const
	{
		FNameBuilder NamespaceString;
		Namespace.ToString(NamespaceString);
		return IsInNameSpace(NamespaceString.ToView());
	}
#endif
	
	/**
	Check to see if this is an external variable or not.
	An external variable is defined as one that is not contained within the Niagara system, i.e. a user or parameter collection parameter.
	*/
	bool IsInExternalNamespace() const;

	bool Serialize(FArchive& Ar);
#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif
	bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	bool NIAGARA_API AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;

protected:
	UPROPERTY(EditAnywhere, Category = "Variable")
	FName Name;

	UPROPERTY(EditAnywhere, Category = "Variable")
	FNiagaraTypeDefinitionHandle TypeDefHandle;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty))
	FNiagaraTypeDefinition TypeDef_DEPRECATED;
#endif

	// note that this implementation uses the hash of the variable's FName which is not consistent from run to run.  Use GetPersistentTypeHash()
	// if the hash is being used in serialized data
	friend inline uint32 GetTypeHash(const FNiagaraVariableBase& Var)
	{
		return HashCombine(GetTypeHash(Var.GetType()), GetTypeHash(Var.GetName()));
	}

	friend inline uint32 GetPersistentTypeHash(const FNiagaraVariableBase& Var)
	{
		return HashCombine(GetPersistentTypeHash(Var.GetType()), GetTypeHash(Var.GetName().ToString()));
	}
};

template<>
struct TStructOpsTypeTraits<FNiagaraVariableBase> : public TStructOpsTypeTraitsBase2<FNiagaraVariableBase>
{
	enum
	{
		WithSerializer = true,
#if WITH_EDITORONLY_DATA
		WithPostSerialize = true,
#endif
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

USTRUCT(BlueprintType)
struct FNiagaraVariable : public FNiagaraVariableBase
{
	GENERATED_USTRUCT_BODY()

	FNiagaraVariable()
	{
	}

	FNiagaraVariable(const FNiagaraVariableBase& Other)
		: FNiagaraVariableBase(Other)
	{
	}

	inline FNiagaraVariable(const FNiagaraTypeDefinition& InType, const FName& InName)
		: FNiagaraVariableBase(InType, InName)
	{
	}

	/** Checks if the types match and either both variables are uninitialized or both hold exactly the same data.*/
	bool HoldsSameData(const FNiagaraVariable& Other) const
	{
		if (TypeDefHandle != Other.TypeDefHandle) {
			return false;
		}
		if (!IsDataAllocated() && !Other.IsDataAllocated()) {
			return true;
		}
		return IsDataAllocated() && Other.IsDataAllocated() && VarData.Num() == Other.VarData.Num() && FMemory::Memcmp(VarData.GetData(), Other.VarData.GetData(), VarData.Num()) == 0;
	}

	// Var data operations
	void AllocateData()
	{
		if (VarData.Num() != TypeDefHandle->GetSize())
		{
			VarData.SetNumZeroed(TypeDefHandle->GetSize());
		}
	}

	bool IsDataAllocated() const
	{
		return VarData.Num() > 0 && VarData.Num() == TypeDefHandle->GetSize();
	}

	void CopyTo(uint8* Dest) const
	{
		check(TypeDefHandle->GetSize() == VarData.Num());
		check(IsDataAllocated());
		FMemory::Memcpy(Dest, VarData.GetData(), VarData.Num());
	}
		
	template<typename T>
	void SetValue(const T& Data)
	{
		check(sizeof(T) == TypeDefHandle->GetSize());
		AllocateData();
		FMemory::Memcpy(VarData.GetData(), &Data, VarData.Num());
	}

	template<typename T>
	T GetValue() const
	{
		check(sizeof(T) == TypeDefHandle->GetSize());
		check(IsDataAllocated());
		T Value;
		FMemory::Memcpy(&Value, GetData(), TypeDefHandle->GetSize());
		return Value;
	}

	void SetData(const uint8* Data)
	{
		check(Data);
		AllocateData();
		FMemory::Memcpy(VarData.GetData(), Data, VarData.Num());
	}

	const uint8* GetData() const
	{
		return VarData.GetData();
	}

	uint8* GetData()
	{
		return VarData.GetData();
	}

	void ClearData()
	{
		VarData.Empty();
	}

	int32 GetAllocatedSizeInBytes() const
	{
		return VarData.Num();
	}

	FString ToString()const
	{
		FString Ret = Name.ToString() + TEXT("(");
		Ret += TypeDefHandle->ToString(VarData.GetData());
		Ret += TEXT(")");
		return Ret;
	}

	static int32 SearchArrayForPartialNameMatch(const TArray<FNiagaraVariable>& Variables, const FName& VariableName)
	{
		FNameBuilder VarNameBuilder(VariableName);
		FStringView VarNameView(VarNameBuilder);

		int32 BestMatchLength = INDEX_NONE;
		int32 BestMatchIdx = INDEX_NONE;

		const int32 VarNameLength = VarNameView.Len();

		for (int32 i = 0; i < Variables.Num(); i++)
		{
			const FNiagaraVariable& TestVar = Variables[i];
			FNameBuilder TestVarNameBuilder(TestVar.GetName());
			FStringView TestVarNameView(TestVarNameBuilder);

			if (VarNameView.Equals(TestVarNameView))
			{
				return i;
			}
			else
			{
				const int32 TestVarNameLength = TestVarNameView.Len();

				if (VarNameLength > TestVarNameLength && (BestMatchLength == INDEX_NONE || TestVarNameLength > BestMatchLength))
				{
					if (VarNameView.StartsWith(TestVarNameView) && VarNameView[TestVarNameLength] == TCHAR('.'))
					{
						BestMatchIdx = i;
						BestMatchLength = TestVarNameLength;
					}
				}
			}
		}

		return BestMatchIdx;
	}

	bool Serialize(FArchive& Ar);
#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif

	static void ConvertFromBaseArray(TConstArrayView<FNiagaraVariableBase> FromVariables, TArray<FNiagaraVariable>& ToVariables)
	{
		ToVariables.Reserve(ToVariables.Num() + FromVariables.Num());
		for (const FNiagaraVariableBase& BaseVar : FromVariables)
		{
			ToVariables.Emplace(BaseVar);
		}
	}

	static void ConvertToBaseArray(TConstArrayView<FNiagaraVariable> FromVariables, TArray<FNiagaraVariableBase>& ToVariables)
	{
		ToVariables.Reserve(ToVariables.Num() + FromVariables.Num());
		for (const FNiagaraVariableBase& BaseVar : FromVariables)
		{
			ToVariables.Emplace(BaseVar);
		}
	}

private:
	//This gets serialized but do we need to worry about endianness doing things like this? If not, where does that get handled?
	//TODO: Remove storage here entirely and move everything to an FNiagaraParameterStore.
	UPROPERTY(meta = (SkipForCompileHash = "true"))
	TArray<uint8> VarData;
};

struct FNiagaraVariableMatch
{
	explicit FNiagaraVariableMatch(const FNiagaraTypeDefinition& InType, const FName& InName)
		: VariableBase(InType, InName)
	{
	}

	bool operator()(const FNiagaraVariableBase& Other) const
	{
		return Other == VariableBase;
	}

private:
	FNiagaraVariableBase VariableBase;
};


template<>
struct TStructOpsTypeTraits<FNiagaraVariable> : public TStructOpsTypeTraitsBase2<FNiagaraVariable>
{
	enum
	{
		WithSerializer = true,
#if WITH_EDITORONLY_DATA
		WithPostSerialize = true,
#endif
	};
};

template<>
inline bool FNiagaraVariable::GetValue<bool>() const
{
	check(TypeDefHandle.IsSameBase(FNiagaraTypeDefinition::GetBoolDef()));
	check(IsDataAllocated());
	FNiagaraBool* BoolStruct = (FNiagaraBool*)GetData();
	return BoolStruct->GetValue();
}

template<>
inline void FNiagaraVariable::SetValue<bool>(const bool& Data)
{
	check(TypeDefHandle.IsSameBase(FNiagaraTypeDefinition::GetBoolDef()));
	AllocateData();
	FNiagaraBool* BoolStruct = (FNiagaraBool*)GetData();
	BoolStruct->SetValue(Data);
}



// Any change to this structure, or it's GetVariables implementation will require a bump in the CustomNiagaraVersion so that we
// properly rebuild the scripts
// You must pad this struct and the results of GetVariables() to a 16 byte boundry.
struct alignas(16) FNiagaraGlobalParameters
{
#if WITH_EDITOR
	NIAGARA_API static const TArray<FNiagaraVariable>& GetVariables();
	NIAGARA_API static FSHAHash GetStructHash();
#endif

	float WorldDeltaTime = 0.0f;
	float EngineDeltaTime =  0.0f;
	float EngineInvDeltaTime = 0.0f;
	float EngineTime = 0.0f;
	float EngineRealTime = 0.0f;
	int32 QualityLevel = 0;

	int32 _Pad0;
	int32 _Pad1;
};

// Any change to this structure, or it's GetVariables implementation will require a bump in the CustomNiagaraVersion so that we
// properly rebuild the scripts
// You must pad this struct and the results of GetVariables() to a 16 byte boundary.
struct alignas(16) FNiagaraSystemParameters
{
#if WITH_EDITOR
	NIAGARA_API static const TArray<FNiagaraVariable>& GetVariables();
	NIAGARA_API static FSHAHash GetStructHash();
#endif

	float EngineTimeSinceRendered = 0.0f;
	float EngineLodDistance = 0.0f;
	float EngineLodDistanceFraction = 0.0f;
	float EngineSystemAge = 0.0f;
	uint32 EngineExecutionState = 0;
	int32 EngineTickCount = 0;
	int32 EngineEmitterCount = 0;
	int32 EngineAliveEmitterCount = 0;
	int32 SignificanceIndex = 0;
	int32 RandomSeed = 0;

	int32 CurrentTimeStep = 0;
	int32 NumTimeSteps = 0;
	float TimeStepFraction = 0.0f;

	uint32 NumParticles = 0;

	int32 _Pad0;
	int32 _Pad1;
};

// Any change to this structure, or it's GetVariables implementation will require a bump in the CustomNiagaraVersion so that we
// properly rebuild the scripts
// You must pad this struct and the results of GetVariables() to a 16 byte boundary.
struct alignas(16) FNiagaraOwnerParameters
{
#if WITH_EDITOR
	NIAGARA_API static const TArray<FNiagaraVariable>& GetVariables();
	NIAGARA_API static FSHAHash GetStructHash();
#endif

	FMatrix44f	EngineLocalToWorld = FMatrix44f::Identity;
	FMatrix44f	EngineWorldToLocal = FMatrix44f::Identity;
	FMatrix44f	EngineLocalToWorldTransposed = FMatrix44f::Identity;
	FMatrix44f	EngineWorldToLocalTransposed = FMatrix44f::Identity;
	FMatrix44f	EngineLocalToWorldNoScale = FMatrix44f::Identity;
	FMatrix44f	EngineWorldToLocalNoScale = FMatrix44f::Identity;
	FQuat4f		EngineRotation = FQuat4f::Identity;
	FVector3f	EnginePosition = FVector3f(ForceInitToZero);
	float		_Pad0;
	FVector3f	EngineVelocity = FVector3f(ForceInitToZero);
	float		_Pad1;
	FVector3f	EngineXAxis = FVector3f(1.0f, 0.0f, 0.0f);
	float		_Pad2;
	FVector3f	EngineYAxis = FVector3f(0.0f, 1.0f, 0.0f);
	float		_Pad3;
	FVector3f	EngineZAxis = FVector3f(0.0f, 0.0f, 1.0f);
	float		_Pad4;
	FVector3f	EngineScale = FVector3f(1.0f, 1.0f, 1.0f);
	float		_Pad5;
	FVector4f	EngineLWCTile = FVector4f(ForceInitToZero);
};

// Any change to this structure, or it's GetVariables implementation will require a bump in the CustomNiagaraVersion so that we
// properly rebuild the scripts
// You must pad this struct and the results of GetVariables() to a 16 byte boundary.
struct alignas(16) FNiagaraEmitterParameters
{
#if WITH_EDITOR
	NIAGARA_API static const TArray<FNiagaraVariable>& GetVariables();
	NIAGARA_API static FSHAHash GetStructHash();
#endif

	int32 EmitterNumParticles = 0;
	int32 EmitterTotalSpawnedParticles = 0;
	float EmitterSpawnCountScale = 1.0f;
	float EmitterAge = 0.0f;
	int32 EmitterRandomSeed = 0;
	int32 EmitterInstanceSeed = 0;

	// todo - what else should be inserted here?  we could put an array of spawninfos/interp spawn values
	int32 _Pad0;
	int32 _Pad1;
};

// Forward decl for base versioned types
struct FVersionedNiagaraEmitterBase;
struct FVersionedNiagaraEmitterBaseWeakPtr;

// Forward decl FVersionedNiagaraEmitterWeakPtr to support FVersionedNiagaraEmitter::ToWeakPtr().
struct FVersionedNiagaraEmitterWeakPtr;
struct FVersionedNiagaraEmitterData;
class UNiagaraEmitter;

/** Struct combining an emitter with a specific version.*/
USTRUCT()
struct FVersionedNiagaraEmitter
{
	GENERATED_USTRUCT_BODY()
	
	FVersionedNiagaraEmitter() {}
	FVersionedNiagaraEmitter(UNiagaraEmitter* InEmitter, const FGuid& InVersion)
		: Emitter(InEmitter), Version(InVersion)
	{};

	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitterData* GetEmitterData() const;
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitterWeakPtr ToWeakPtr() const;
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitterBase ToBase() const;

	UPROPERTY()
	TObjectPtr<UNiagaraEmitter> Emitter = nullptr;

	UPROPERTY()
	FGuid Version;

	bool operator==(const FVersionedNiagaraEmitter& Other) const
	{
		return Emitter == Other.Emitter && Version == Other.Version;
	}

	bool operator!=(const FVersionedNiagaraEmitter& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FVersionedNiagaraEmitter& Item)
	{
		return GetTypeHash(Item.Emitter) + GetTypeHash(Item.Version);
	}
};

struct FVersionedNiagaraEmitterWeakPtr
{
	FVersionedNiagaraEmitterWeakPtr() : Emitter(nullptr) {}
	NIAGARA_API FVersionedNiagaraEmitterWeakPtr(UNiagaraEmitter* InEmitter, const FGuid& InVersion);

	[[nodiscard]] NIAGARA_API bool IsValid() const;
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitter ResolveWeakPtr() const;
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitterData* GetEmitterData() const;
	[[nodiscard]] NIAGARA_API FVersionedNiagaraEmitterBaseWeakPtr ToWeakBase() const;

	bool operator==(const FVersionedNiagaraEmitterWeakPtr& Other) const
	{
		return Emitter == Other.Emitter && Version == Other.Version;
	}

	bool operator!=(const FVersionedNiagaraEmitterWeakPtr& Other) const
	{
		return !(*this == Other);
	}

	TWeakObjectPtr<UNiagaraEmitter> Emitter;
	FGuid Version;

	friend inline uint32 GetTypeHash(const FVersionedNiagaraEmitterWeakPtr& Item)
	{
		return GetTypeHash(Item.Emitter);
	}
};

//////////////////////////////////////////////////////////////////////////
//Utilities for getting the correct LWC Niagara Type Def.

template <typename T, std::enable_if_t<!std::is_enum_v<T>, int>>
FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef() { return FNiagaraTypeDefinition(StaticStruct<T>()); }

template <typename T, std::enable_if_t<!std::is_enum_v<T>, int>>
FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef() { return FNiagaraTypeDefinition(StaticStruct<T>()); }

//Special case for any enum types to access the correct TypeDefs.
template <typename T, std::enable_if_t<std::is_enum_v<T>, int>>
FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef()
{
	return FNiagaraTypeDefinition(StaticEnum<T>());
}

template <typename T, std::enable_if_t<std::is_enum_v<T>, int>>
FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef()
{
	return FNiagaraTypeDefinition(StaticEnum<T>());
}

template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<FNiagaraBool>() { return FNiagaraTypeDefinition::GetBoolDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<bool>() { return FNiagaraTypeDefinition::GetBoolDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<int32>() { return FNiagaraTypeDefinition::GetIntDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<float>() { return FNiagaraTypeHelper::GetDoubleDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<double>() { return FNiagaraTypeHelper::GetDoubleDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<FVector2D>() { return FNiagaraTypeHelper::GetVector2DDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<FVector>() { return FNiagaraTypeHelper::GetVectorDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<FVector4>() { return FNiagaraTypeHelper::GetVector4Def(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<FQuat>() { return FNiagaraTypeHelper::GetQuatDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<FNiagaraPosition>() { return FNiagaraTypeDefinition::GetPositionDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<FLinearColor>() { return FNiagaraTypeDefinition::GetColorDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetTypeDef<FNiagaraID>() { return FNiagaraTypeDefinition::GetIDDef(); }

//Utilities for getting the correct Niagara Sim TypeDef used internally to Niagara.
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<FNiagaraBool>() { return FNiagaraTypeDefinition::GetBoolDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<bool>() { return FNiagaraTypeDefinition::GetBoolDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<int32>() { return FNiagaraTypeDefinition::GetIntDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<float>() { return FNiagaraTypeDefinition::GetFloatDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<double>() { return FNiagaraTypeDefinition::GetFloatDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<FVector2D>() { return FNiagaraTypeDefinition::GetVec2Def(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<FVector>() { return FNiagaraTypeDefinition::GetVec3Def(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<FVector4>() { return FNiagaraTypeDefinition::GetVec4Def(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<FQuat>() { return FNiagaraTypeDefinition::GetQuatDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<FNiagaraPosition>() { return FNiagaraTypeDefinition::GetVec3Def(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<FLinearColor>() { return FNiagaraTypeDefinition::GetColorDef(); }
template<> inline FNiagaraTypeDefinition FNiagaraTypeHelper::GetSimTypeDef<FNiagaraID>() { return FNiagaraTypeDefinition::GetIDDef(); }

//Utils for converting to Niagara Sim or LWC Types inside template code.
template<typename T, typename Enable = void> 
struct TNiagaraTypeHelper{ typedef T TSimType; typedef T TLWCType; };

template<> struct TNiagaraTypeHelper<bool> { typedef FNiagaraBool TSimType; typedef FNiagaraBool TLWCType; };
template<> struct TNiagaraTypeHelper<float> { typedef float TSimType; typedef double TLWCType; };
template<> struct TNiagaraTypeHelper<double> { typedef float TSimType; typedef double TLWCType; };
template<> struct TNiagaraTypeHelper<FVector2D> { typedef FVector2f TSimType; typedef FVector2D TLWCType; };
template<> struct TNiagaraTypeHelper<FVector> { typedef FVector3f TSimType; typedef FVector TLWCType; };
template<> struct TNiagaraTypeHelper<FVector4> { typedef FVector4f TSimType; typedef FVector4 TLWCType; };
template<> struct TNiagaraTypeHelper<FQuat> { typedef FQuat4f TSimType; typedef FQuat TLWCType; };

template<> struct TNiagaraTypeHelper<FVector2f> { typedef FVector2f TSimType; typedef FVector2D TLWCType; };
template<> struct TNiagaraTypeHelper<FVector3f> { typedef FVector3f TSimType; typedef FVector TLWCType; };
template<> struct TNiagaraTypeHelper<FVector4f> { typedef FVector4f TSimType; typedef FVector4 TLWCType; };
template<> struct TNiagaraTypeHelper<FQuat4f> { typedef FQuat4f TSimType; typedef FQuat TLWCType; };

template<> struct TNiagaraTypeHelper<FNiagaraPosition> { typedef FVector3f TSimType; typedef FVector TLWCType; };

//Always treat enums as int32s internally to Niagara for both Sim and Game.
template<typename T> 
struct TNiagaraTypeHelper<T, std::enable_if_t<std::is_enum_v<T>>> 
{
	typedef int32 TSimType; typedef int32 TLWCType; 
};