// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraParameterStore.h"
#include "NiagaraUserRedirectionParameterStore.generated.h"

/**
* Extension of the base parameter store to allow the user in the editor to use variable names without 
* the "User." namespace prefix. The names without the prefix just redirect to the original variables, it is just done
* for better usability.
*/
USTRUCT()
struct FNiagaraUserRedirectionParameterStore : public FNiagaraParameterStore
{
	GENERATED_USTRUCT_BODY()
public:
	FNiagaraUserRedirectionParameterStore();
	FNiagaraUserRedirectionParameterStore(const FNiagaraParameterStore& Other);
	FNiagaraUserRedirectionParameterStore& operator=(const FNiagaraParameterStore& Other);

	virtual ~FNiagaraUserRedirectionParameterStore() = default;

	void RecreateRedirections();

	/** If necessary it will replace the supplied variable with the fully qualified namespace (User.) appropriate for a user variable.
		Will return false if the variable wasn't able to be converted into a valid user namespaced variable.
	*/
	NIAGARA_API bool RedirectUserVariable(FNiagaraVariableBase& UserVar) const;

	/** Get the list of FNiagaraVariables that are exposed to the user. Note that the values will be stale and are not to be trusted directly. Get the Values using the offset specified by IndexOf or GetParameterValue.*/
	inline void GetUserParameters(TArray<FNiagaraVariable>& OutParameters) const { return UserParameterRedirects.GenerateKeyArray(OutParameters); }

	// ~ Begin FNiagaraParameterStore overrides
	inline virtual const int32* FindParameterOffset(const FNiagaraVariableBase& Parameter, bool IgnoreType = false) const override
	{
		if (IgnoreType)
		{
			for (auto it = UserParameterRedirects.CreateConstIterator(); it; ++it)
			{
				if (it->Key.GetName() == Parameter.GetName())
				{
					return FNiagaraParameterStore::FindParameterOffset(it->Value);
				}
			}
			return FNiagaraParameterStore::FindParameterOffset(Parameter, IgnoreType);
		}
		else
		{
			const FNiagaraVariable* Redirection = UserParameterRedirects.Find(Parameter);
			return FNiagaraParameterStore::FindParameterOffset(Redirection ? *Redirection : Parameter);
		}
	}
	
	inline virtual const FNiagaraVariableWithOffset* FindParameterVariable(const FNiagaraVariable& Parameter, bool IgnoreType = false) const override
	{
		if (IgnoreType)
		{
			for (auto it = UserParameterRedirects.CreateConstIterator(); it; ++it)
			{
				if (it->Key.GetName() == Parameter.GetName())
				{
					return FNiagaraParameterStore::FindParameterVariable(it->Value);
				}
			}
			return FNiagaraParameterStore::FindParameterVariable(Parameter, IgnoreType);
		}
		else
		{
			const FNiagaraVariable* Redirection = UserParameterRedirects.Find(Parameter);
			return FNiagaraParameterStore::FindParameterVariable(Redirection ? *Redirection : Parameter);
		}
	}

	virtual bool AddParameter(const FNiagaraVariable& Param, bool bInitialize = true, bool bTriggerRebind = true, int32* OutOffset = nullptr) override;
	virtual bool RemoveParameter(const FNiagaraVariableBase& InVar) override;
	virtual void InitFromSource(const FNiagaraParameterStore* SrcStore, bool bNotifyAsDirty) override;
	virtual void Empty(bool bClearBindings = true) override;
	virtual void Reset(bool bClearBindings = true) override;
	virtual bool SetPositionParameterValue(const FVector& InValue, const FName& ParamName, bool bAdd=false) override;
	virtual const FVector* GetPositionParameterValue(const FName& ParamName) const override;

#if WITH_EDITORONLY_DATA
	virtual void ConvertParameterType(const FNiagaraVariable& ExistingParam, const FNiagaraTypeDefinition& NewType) override;
#endif

	virtual void SanityCheckData(bool bInitInterfaces = true) override;
	// ~ End FNiagaraParameterStore overrides

	/** Used to upgrade a serialized FNiagaraParameterStore property to our own struct */
	bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/** Turn the input NiagaraVariable into the User namespaced version if needed, independent of whether or not it is in a redirection table.*/
	NIAGARA_API static void MakeUserVariable(FNiagaraVariableBase& InVar);
	static bool IsUserParameter(const FNiagaraVariableBase& InVar);
private:

	/** Map from the variables with shortened display names to the original variables with the full namespace */
	UPROPERTY()
	TMap<FNiagaraVariable, FNiagaraVariable> UserParameterRedirects;

	FNiagaraVariable GetUserRedirection(const FNiagaraVariable& InVar) const;
};

template<>
struct TStructOpsTypeTraits<FNiagaraUserRedirectionParameterStore> : public TStructOpsTypeTraitsBase2<FNiagaraUserRedirectionParameterStore>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
