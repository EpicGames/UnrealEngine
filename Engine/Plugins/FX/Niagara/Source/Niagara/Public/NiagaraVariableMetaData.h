// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "Math/UnitConversion.h"
#include "Math/UnitConversion.inl"
#include "NiagaraVariableMetaData.generated.h"

class IPropertyHandle;

/** Defines options for conditionally editing and showing script inputs in the UI. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraInputConditionMetadata
{
	GENERATED_USTRUCT_BODY()
public:
	/** The name of the input to use for matching the target values. */
	UPROPERTY(EditAnywhere, Category="Input Condition")
	FName InputName;

	/** The list of target values which will satisfy the input condition.  If this is empty it's assumed to be a single value of "true" for matching bool inputs. */
	UPROPERTY(EditAnywhere, Category="Input Condition")
	TArray<FString> TargetValues;
};

/** Defines override data for enum parameters displayed in the UI. */
USTRUCT()
struct FNiagaraEnumParameterMetaData
{
	GENERATED_BODY()
	
	/** If specified, this name will be used for the given enum entry. Useful for shortening names. */
	UPROPERTY(EditAnywhere, Category="Enum Override")
	FName OverrideName;

	/** If specified, this icon will be used for the given enum entry. If OverrideName isn't empty, the icon takes priority. */
	UPROPERTY(EditAnywhere, Category="Enum Override")
	TObjectPtr<UTexture2D> IconOverride = nullptr;

	UPROPERTY(EditAnywhere, Category="Enum Override", meta=(InlineEditConditionToggle))
	bool bUseColorOverride = false;
	
	UPROPERTY(EditAnywhere, Category="Enum Override", meta=(EditCondition="bUseColorOverride"))
	FLinearColor ColorOverride = FLinearColor::White;
};

UENUM()
enum class ENiagaraInputWidgetType : uint8
{
	// Default input widget
	Default,

	// slider widget, for float and int type
	Slider,

	// audio volume slider with mute control, for float input only
	Volume,

	// a numeric input, but also has a dropdown with named values
	NumericDropdown,

	// (for integer inputs only) A dropdown that behaves like an enum; only allows the exact pre-defined values.
	EnumStyle,

	// (for enum inputs only) Instead of the normal dropdown, the enum values are all displayed in a button grid.
	// This shows all possible values at once, so only makes sense if there are few input values. 
	SegmentedButtons
};

USTRUCT()
struct FWidgetNamedInputValue
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Customization")
	float Value = 0;
	
	UPROPERTY(EditAnywhere, Category="Customization")
	FText DisplayName;

	UPROPERTY(EditAnywhere, Category="Customization")
	FText Tooltip;
};

USTRUCT()
struct FWidgetSegmentValueOverride
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Customization")
	int32 EnumIndexToOverride = 0;

	UPROPERTY(EditAnywhere, Category="Customization", meta=(InlineEditConditionToggle))
	bool bOverrideDisplayName = false;
	
	// This will be used as display name instead of the enum value
	UPROPERTY(EditAnywhere, Category="Customization", meta=(EditCondition="bOverrideDisplayName"))
	FText DisplayNameOverride;

	// If set, then this icon will be displayed on the button
	UPROPERTY(EditAnywhere, Category="Customization")
	TObjectPtr<UTexture2D> DisplayIcon;
};

/** A struct that serves as display metadata for integer type static switches. Is used in conjuction with the 'EnumStyle' widget customization. */
USTRUCT()
struct FNiagaraWidgetNamedIntegerInputValue
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category="Customization")
	FText DisplayName;

	UPROPERTY(EditAnywhere, Category="Customization", meta = (MultiLine = "true"))
	FText Tooltip;
};

USTRUCT()
struct FNiagaraInputParameterCustomization
{
	GENERATED_BODY()

	// Changes the widget implementation used for the input
	UPROPERTY(EditAnywhere, Category="Customization")
	ENiagaraInputWidgetType WidgetType = ENiagaraInputWidgetType::Default;
	
	UPROPERTY(EditAnywhere, Category="Customization", meta=(InlineEditConditionToggle))
	bool bHasMinValue = false;
	
	/** min ui value (float and int types only) */
	UPROPERTY(EditAnywhere, Category="Customization", meta=(EditCondition="bHasMinValue"))
	float MinValue = 0;

	UPROPERTY(EditAnywhere, Category="Customization", meta=(InlineEditConditionToggle))
	bool bHasMaxValue = false;
	
	/** max ui value (float and int types only) */
	UPROPERTY(EditAnywhere, Category="Customization", meta=(EditCondition="bHasMaxValue"))
	float MaxValue = 1;
	
	UPROPERTY(EditAnywhere, Category="Customization", meta=(InlineEditConditionToggle))
	bool bHasStepWidth = false;
	
	/** Step width used by the input when dragging */
	UPROPERTY(EditAnywhere, Category="Customization", meta=(EditCondition="bHasStepWidth"))
	float StepWidth = 1;

	UPROPERTY(EditAnywhere, Category="Customization", meta=(EditCondition="WidgetType == ENiagaraInputWidgetType::NumericDropdown", EditConditionHides))
	TArray<FWidgetNamedInputValue> InputDropdownValues;
	
	UPROPERTY(EditAnywhere, Category="Customization", meta=(EditCondition="WidgetType == ENiagaraInputWidgetType::EnumStyle", EditConditionHides))
	TArray<FNiagaraWidgetNamedIntegerInputValue> EnumStyleDropdownValues;

	// Limits the number of buttons shown per row, 0 = unlimited
	UPROPERTY(EditAnywhere, Category="Customization", meta=(EditCondition="WidgetType == ENiagaraInputWidgetType::SegmentedButtons", EditConditionHides))
	int32 MaxSegmentsPerRow = 0;

	UPROPERTY(EditAnywhere, Category="Customization", meta=(EditCondition="WidgetType == ENiagaraInputWidgetType::SegmentedButtons", EditConditionHides))
	TArray<FWidgetSegmentValueOverride> SegmentValueOverrides;

	// If set to true then values will not be updated while typing, only when committing the input field. This is useful for values that cause big downstream changes and computational overhead, like grid size or spawned particles. 
	UPROPERTY(EditAnywhere, Category="Customization")
	bool bBroadcastValueChangesOnCommitOnly = false;
	
	/** If true then the input is also displayed and editable as a 3d widget in the viewport (vector and transform types only). */
	//UPROPERTY(EditAnywhere, Category="Customization")
	//bool bCreateViewPortEditWidget = false;
#if WITH_EDITOR
	// Maps existing property tags as best it can to a customization object
	NIAGARA_API static FNiagaraInputParameterCustomization MakeFromProperty(TSharedPtr<IPropertyHandle> InPropertyHandle);
#endif
};

UENUM()
enum class ENiagaraBoolDisplayMode : uint8
{
	DisplayAlways,
	DisplayIfTrue,
	DisplayIfFalse
};

USTRUCT()
struct FNiagaraBoolParameterMetaData
{
	GENERATED_BODY()

	/** The mode used determines the cases in which a bool parameter is displayed.
	 *  If set to DisplayAlways, both True and False cases will display. 
	 *  If set to DisplayIfTrue, it will only display if the bool evaluates to True.
	 */
	UPROPERTY(EditAnywhere, Category="Bool Override")
	ENiagaraBoolDisplayMode DisplayMode = ENiagaraBoolDisplayMode::DisplayAlways;
	
	/** If specified, this name will be used for the given bool if it evaluates to True. */
	UPROPERTY(EditAnywhere, Category="Bool Override")
	FName OverrideNameTrue;

	/** If specified, this name will be used for the given bool if it evaluates to False. */
	UPROPERTY(EditAnywhere, Category="Bool Override")
	FName OverrideNameFalse;

	/** If specified, this icon will be used for the given bool if it evaluates to True. If OverrideName isn't empty, the icon takes priority. */
	UPROPERTY(EditAnywhere, Category="Bool Override")
	TObjectPtr<UTexture2D> IconOverrideTrue = nullptr;

	/** If specified, this icon will be used for the given bool if it evaluates to False. If OverrideName isn't empty, the icon takes priority. */
	UPROPERTY(EditAnywhere, Category="Bool Override")
	TObjectPtr<UTexture2D> IconOverrideFalse = nullptr;
};

USTRUCT()
struct FNiagaraVariableMetaData
{
	GENERATED_USTRUCT_BODY()

	FNiagaraVariableMetaData()
		: bAdvancedDisplay(false)
		, bDisplayInOverviewStack(false)
		, InlineParameterSortPriority(0)
		, bOverrideColor(false)
		, InlineParameterColorOverride(FLinearColor(ForceInit))
		, bEnableBoolOverride(false)
		, bInlineEditConditionToggle(false)
		, bIsStaticSwitch_DEPRECATED(false)
		, StaticSwitchDefaultValue_DEPRECATED(0)
		, EditorSortPriority_DEPRECATED(0)
	{};

	UPROPERTY(EditAnywhere, Category = "Variable", DisplayName="Tooltip", meta = (MultiLine = true, SkipForCompileHash = "true"))
	FText Description;
	
	/** The unit to display next to input fields for this parameter - note that this is only a visual indicator and does not change any of the calculations. */
	UPROPERTY(EditAnywhere, Category = "Variable", meta = (SkipForCompileHash = "true"))
	EUnit DisplayUnit = EUnit::Unspecified;

	/** Declares that this input is advanced and should only be visible if expanded inputs have been expanded. */
	UPROPERTY(EditAnywhere, Category = "Variable", meta = (SkipForCompileHash = "true"))
	bool bAdvancedDisplay;

	/** Declares that this parameter's value will be shown in the overview node if it's set to a local value. */
	UPROPERTY(EditAnywhere, Category = "Variable", meta = (SkipForCompileHash = "true"))
	bool bDisplayInOverviewStack;

	UPROPERTY(EditAnywhere, Category = "Variable", meta = (EditCondition="bDisplayInOverviewStack", ToolTip = "Affects the sort order for parameters shown inline in the overview. Use a smaller number to push it to the top. Defaults to zero.", SkipForCompileHash = "true"))
	int32 InlineParameterSortPriority;

	UPROPERTY(EditAnywhere, Category = "Variable", meta = (InlineEditConditionToggle, ToolTip = "The color used to display a parameter in the overview. If no color is specified, the type color is used.", SkipForCompileHash = "true"))
	bool bOverrideColor;
	
	UPROPERTY(EditAnywhere, Category = "Variable", meta = (EditCondition="bOverrideColor", ToolTip = "The color used to display a parameter in the overview. If no color is specified, the type color is used.", SkipForCompileHash = "true"))
	FLinearColor InlineParameterColorOverride;

	UPROPERTY(EditAnywhere, Category = "Variable", meta = (EditCondition="bDisplayInOverviewStack", ToolTip = "The index of the entry maps to the index of an enum value. Useful for overriding how an enum parameter is displayed in the overview.", SkipForCompileHash = "true"))
	TArray<FNiagaraEnumParameterMetaData> InlineParameterEnumOverrides;

	UPROPERTY(EditAnywhere, Category = "Variable", meta = (InlineEditConditionToggle, ToolTip = "Useful to override inline bool visualization in the overview.", SkipForCompileHash = "true"))
	bool bEnableBoolOverride;
	
	UPROPERTY(EditAnywhere, Category = "Variable", meta = (EditCondition="bEnableBoolOverride", ToolTip = "Useful to override inline bool visualization in the overview.", SkipForCompileHash = "true"))
	FNiagaraBoolParameterMetaData InlineParameterBoolOverride;

	/** Declares the associated input is used as an inline edit condition toggle, so it should be hidden and edited as a 
	checkbox inline with the input which was designated as its edit condition. */
	UPROPERTY(EditAnywhere, Category = "Variable", meta = (SkipForCompileHash = "true"))
	bool bInlineEditConditionToggle;

	/** Declares the associated input should be conditionally editable based on the value of another input. */
	UPROPERTY(EditAnywhere, Category = "Variable", meta = (SkipForCompileHash = "true"))
	FNiagaraInputConditionMetadata EditCondition;

	/** Declares the associated input should be conditionally visible based on the value of another input. */
	UPROPERTY(EditAnywhere, Category = "Variable", meta = (SkipForCompileHash = "true"))
	FNiagaraInputConditionMetadata VisibleCondition;

	UPROPERTY(EditAnywhere, Category = "Variable", DisplayName = "Property Metadata", meta = (ToolTip = "Property Metadata", SkipForCompileHash = "true"))
	TMap<FName, FString> PropertyMetaData;

	UPROPERTY(EditAnywhere, Category = "Variable", DisplayName = "Alternate Aliases For Variable", AdvancedDisplay, meta = (ToolTip = "List of alternate/previous names for this variable. Note that this is not normally needed if you rename through the UX. However, if you delete and then add a different variable, intending for it to match, you will likely want to add the prior name here.\n\nYou may need to restart and reload assets after making this change to have it take effect on already loaded assets."))
	TArray<FName> AlternateAliases;

	/** Changes how the input is displayed. */
	UPROPERTY(EditAnywhere, Category = "Variable", meta = (SkipForCompileHash = "true"))
	FNiagaraInputParameterCustomization WidgetCustomization;

	bool GetIsStaticSwitch_DEPRECATED() const { return bIsStaticSwitch_DEPRECATED; };

	int32 GetStaticSwitchDefaultValue_DEPRECATED() const { return StaticSwitchDefaultValue_DEPRECATED; };

	int32 GetEditorSortPriority_DEPRECATED() const { return EditorSortPriority_DEPRECATED; }
	
	FText GetCategoryName_DEPRECATED() const { return CategoryName_DEPRECATED; }

	FName GetParentAttribute_DEPRECATED() const { return ParentAttribute_DEPRECATED; }
	
	/** Copies all the properties that are marked as editable for the user (e.g. EditAnywhere). */
	NIAGARA_API void CopyUserEditableMetaData(const FNiagaraVariableMetaData& OtherMetaData);

	FGuid GetVariableGuid() const { return VariableGuid; };

	/** Note, the Variable Guid is generally expected to be immutable. This method is provided to upgrade existing variables to have the same Guid as variable definitions. */
	void SetVariableGuid(const FGuid& InVariableGuid ) { VariableGuid = InVariableGuid; };
	void CreateNewGuid() { VariableGuid = FGuid::NewGuid(); };

private:
	/** A unique identifier for the variable that can be used by function call nodes to find renamed variables. */
	UPROPERTY(meta = (SkipForCompileHash = "true"))
	FGuid VariableGuid;

	/** This is a read-only variable that designates if the metadata is tied to a static switch or not.
	 * DEPRECATED: Migrated to UNiagaraScriptVariable::bIsStaticSwitch.
	 */
	UPROPERTY()
	bool bIsStaticSwitch_DEPRECATED;

	/** The default value to use when creating new pins or stack entries for a static switch parameter
	 * DEPRECATED: Migrated to UNiagaraScriptVariable::StaticSwitchDefaultValue.
	 */
	UPROPERTY()
	int32 StaticSwitchDefaultValue_DEPRECATED;  // TODO: This should be moved to the UNiagaraScriptVariable in the future

	UPROPERTY()
	FText CategoryName_DEPRECATED;

	UPROPERTY()
	FName ParentAttribute_DEPRECATED;

	UPROPERTY()
	int32 EditorSortPriority_DEPRECATED;
};
