// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "NiagaraTypes.h"

#include "NiagaraVariant.generated.h"

struct FNiagaraCompileHashVisitor;
class UNiagaraDataInterface;

UENUM()
enum class ENiagaraVariantMode
{
	None,
	Object,
	DataInterface,
	Bytes
};

USTRUCT()
struct FNiagaraVariant
{
	GENERATED_BODY()

	NIAGARA_API FNiagaraVariant();
	// TODO: Add move support
	FNiagaraVariant(const FNiagaraVariant&) = default;
	FNiagaraVariant& operator=(const FNiagaraVariant&) = default;
	NIAGARA_API explicit FNiagaraVariant(UNiagaraDataInterface* InDataInterface);
	NIAGARA_API explicit FNiagaraVariant(TObjectPtr<UObject> InObject);
	NIAGARA_API explicit FNiagaraVariant(const TArray<uint8>& InBytes);
	NIAGARA_API FNiagaraVariant(const void* InBytes, int32 Size);

	NIAGARA_API void Reset();

	NIAGARA_API TObjectPtr<UObject> GetUObject() const;
	NIAGARA_API void SetUObject(TObjectPtr<UObject> InObject);

	NIAGARA_API UNiagaraDataInterface* GetDataInterface() const;
	NIAGARA_API void SetDataInterface(UNiagaraDataInterface* InDataInterface);

	NIAGARA_API void AllocateBytes(int32 InCount);
	NIAGARA_API void SetBytes(const uint8* InBytes, int32 InCount);
	NIAGARA_API uint8* GetBytes() const;
	NIAGARA_API int32 GetNumBytes() const;

 	template<typename T> T GetBytesValue(const FNiagaraTypeDefinition& Type)const;
	template<typename T> void SetBytesValue(const FNiagaraTypeDefinition& Type, const T& Value);

	NIAGARA_API bool IsValid(const FNiagaraTypeDefinition& Type)const;
	
	bool IsValid() const { return CurrentMode != ENiagaraVariantMode::None; }
	ENiagaraVariantMode GetMode() const { return CurrentMode; }

	NIAGARA_API bool operator==(const FNiagaraVariant& Other) const;
	NIAGARA_API bool operator!=(const FNiagaraVariant& Other) const;

	NIAGARA_API bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;

private:
	UPROPERTY(EditAnywhere, Category=Variant, Instanced)
	TObjectPtr<UObject> Object;

	UPROPERTY(EditAnywhere, Category=Variant, Instanced)
	TObjectPtr<UNiagaraDataInterface> DataInterface;

	UPROPERTY(EditAnywhere, Category=Variant)
	TArray<uint8> Bytes;

	UPROPERTY(EditAnywhere, Category=Variant)
	ENiagaraVariantMode CurrentMode;
};

template<typename T> T FNiagaraVariant::GetBytesValue(const FNiagaraTypeDefinition& Type)const
{
	if (IsValid(Type))
	{
		check(sizeof(T) == Type.GetSize());
		T Value;
		FMemory::Memcpy(&Value, Bytes.GetData(), sizeof(T));
		return Value;
	}
	return T();
}

template<typename T> void FNiagaraVariant::SetBytesValue(const FNiagaraTypeDefinition& Type, const T& Value)
{
	check(sizeof(T) == Type.GetSize());
	AllocateBytes(sizeof(T));
	FMemory::Memcpy(Bytes.GetData(), &Value, sizeof(T));
}
