// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "NiagaraTypes.h"
#include "NiagaraVersionUpgradeBase.generated.h"

class UNiagaraClipboardFunctionInput;

namespace NiagaraVersionUpgrade
{
	struct ScriptInputs
	{
		FNiagaraAssetVersion Version;
		TArray<UNiagaraClipboardFunctionInput*> FunctionInputs;
	};
}

// Base class for niagara script upgrades in the context of versioning. Can be used instead of python upgrade scripts or alongside them as fallback if python is not available in the current environment.
UCLASS(Abstract, MinimalAPI)
class UNiagaraVersionUpgradeBase : public UObject
{
	GENERATED_BODY()
public:
	virtual bool ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage) { return true; };
};
