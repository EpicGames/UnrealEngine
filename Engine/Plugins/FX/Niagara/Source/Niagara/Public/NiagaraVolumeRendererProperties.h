// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSetAccessor.h"
#include "NiagaraParameterBinding.h"
#include "NiagaraVolumeRendererProperties.generated.h"

class UMaterialInstanceConstant;
class UMaterialInterface;
class FNiagaraEmitterInstance;
class SWidget;

namespace ENiagaraVolumeVFLayout
{
	enum Type
	{
		Position,
		Rotation,
		Scale,
		RendererVisibilityTag,
		VolumeResolutionMaxAxis,
		VolumeWorldSpaceSize,

		Num
	};
};

UCLASS(editinlinenew, MinimalAPI, meta = (DisplayName = "Volume Renderer"))
class UNiagaraVolumeRendererProperties : public UNiagaraRendererProperties
{
public:
	GENERATED_BODY()

	UNiagaraVolumeRendererProperties();

	//UObject Interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITORONLY_DATA
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif// WITH_EDITORONLY_DATA
	//UObject Interface END

	static void InitCDOPropertiesAfterModuleStartup();

	//~ UNiagaraRendererProperties interface
	virtual FNiagaraRenderer* CreateEmitterRenderer(ERHIFeatureLevel::Type FeatureLevel, const FNiagaraEmitterInstance* Emitter, const FNiagaraSystemInstanceController& InController) override;
	virtual class FNiagaraBoundsCalculator* CreateBoundsCalculator() override;
	virtual void GetUsedMaterials(const FNiagaraEmitterInstance* InEmitter, TArray<UMaterialInterface*>& OutMaterials) const override;
	virtual bool IsSimTargetSupported(ENiagaraSimTarget InSimTarget) const override { return true; };
#if WITH_EDITORONLY_DATA
	virtual const TArray<FNiagaraVariable>& GetOptionalAttributes() override;
	virtual void GetRendererTooltipWidgets(const FNiagaraEmitterInstance* InEmitter, TArray<TSharedPtr<SWidget>>& OutWidgets, TSharedPtr<FAssetThumbnailPool> InThumbnailPool) const override;
	virtual void GetRendererFeedback(const FVersionedNiagaraEmitter& InEmitter, TArray<FNiagaraRendererFeedback>& OutErrors, TArray<FNiagaraRendererFeedback>& OutWarnings, TArray<FNiagaraRendererFeedback>& OutInfo) const override;
	TArray<FNiagaraVariable> GetBoundAttributes() const override;
	virtual void RenameVariable(const FNiagaraVariableBase& OldVariable, const FNiagaraVariableBase& NewVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
	virtual void RemoveVariable(const FNiagaraVariableBase& OldVariable, const FVersionedNiagaraEmitterBase& InEmitter) override;
#endif // WITH_EDITORONLY_DATA
	virtual void CacheFromCompiledData(const FNiagaraDataSetCompiledData* CompiledData) override;
	virtual void UpdateSourceModeDerivates(ENiagaraRendererSourceDataMode InSourceMode, bool bFromPropertyEdit = false) override;
	virtual ENiagaraRendererSourceDataMode GetCurrentSourceMode() const override { return SourceMode; }
	virtual bool PopulateRequiredBindings(FNiagaraParameterStore& InParameterStore) override;
	virtual bool NeedsMIDsForMaterials() const override { return MaterialParameters.HasAnyBindings(); }
	virtual bool UseHeterogeneousVolumes() const override { return true; }
	//UNiagaraRendererProperties Interface END

	void UpdateMICs();

	static FQuat4f GetDefaultVolumeRotation() { return FQuat4f::Identity; }
	static FVector3f GetDefaultVolumeScale() { return FVector3f(1.0f, 1.0f, 1.0f); }
	static int32 GetDefaultVolumeResolutionMaxAxis() { return 0; }
	static FVector3f GetDefaultVolumeWorldSpaceSize() { return FVector3f::ZeroVector; }
	static float GetDefaultStepFactor() { return 1.0f; }
	static float GetDefaultShadowStepFactor() { return 2.0f; }
	static float GetDefaultShadowBiasFactor() { return 0.5f; }
	static float GetDefaultLightingDownsampleFactor() { return 2.0f; }

	/** What material to use for the volume. */
	UPROPERTY(EditAnywhere, Category = "Volume Rendering")
	TObjectPtr<UMaterialInterface> Material;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceConstant> MICMaterial;
#endif

	/** Binding to material. */
	UPROPERTY(EditAnywhere, Category = "Volume Rendering")
	FNiagaraParameterBinding MaterialParameterBinding;

	/** Whether or not to draw a single element for the Emitter or to draw the particles.*/
	//-TODO: Unhind when particle support is added
	//UPROPERTY(EditAnywhere, Category = "Volume Rendering")
	ENiagaraRendererSourceDataMode SourceMode = ENiagaraRendererSourceDataMode::Emitter;

	/** If a render visibility tag is present, particles whose tag matches this value will be visible in this renderer. */
	UPROPERTY(EditAnywhere, Category = "Volume Rendering")
	int32 RendererVisibility = 0;

	UPROPERTY(EditAnywhere, Category = "Volume Rendering")
	float StepFactor = 1.0f;

	UPROPERTY(EditAnywhere, Category = "Volume Rendering")
	float LightingDownsampleFactor = 2.0f;

	UPROPERTY(EditAnywhere, Category = "Volume Rendering")
	float ShadowStepFactor = 2.0f;

	UPROPERTY(EditAnywhere, Category = "Volume Rendering")
	float ShadowBiasFactor = 0.5f;

	/** Position binding for the center of the volume. */
	//-TODO: Unhind when binding is supported
	//UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding PositionBinding;

	/** Rotation binding for the volume. */
	//-TODO: Unhind when binding is supported
	//UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RotationBinding;

	/** Scale binding for the volume. */
	//-TODO: Unhind when binding is supported
	//UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding ScaleBinding;

	/** Visibility tag binding, when valid the returned values is compared with RendererVisibility. */
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding RendererVisibilityTagBinding;

	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding VolumeResolutionMaxAxisBinding;

	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraVariableAttributeBinding VolumeWorldSpaceSizeBinding;

	/** If this array has entries, we will create a MaterialInstanceDynamic per Emitter instance from Material and set the Material parameters using the Niagara simulation variables listed.*/
	UPROPERTY(EditAnywhere, Category = "Bindings")
	FNiagaraRendererMaterialParameters MaterialParameters;

	FNiagaraDataSetAccessor<FNiagaraPosition>	PositionDataSetAccessor;
	FNiagaraDataSetAccessor<FQuat4f>			RotationDataSetAccessor;
	FNiagaraDataSetAccessor<FVector3f>			ScaleDataSetAccessor;
	FNiagaraDataSetAccessor<int32>				RendererVisibilityTagAccessor;
	FNiagaraDataSetAccessor<int32>				VolumeResolutionMaxAxisAccessor;
	FNiagaraDataSetAccessor<FVector3f>			VolumeWorldSpaceSizeAccessor;
};
