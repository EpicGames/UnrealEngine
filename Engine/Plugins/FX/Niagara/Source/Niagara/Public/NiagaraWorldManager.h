// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/World.h"
#include "Particles/ParticlePerfStats.h"
#include "NiagaraParameterCollection.h"
#include "UObject/GCObject.h"
#include "NiagaraCommon.h"
#include "NiagaraDataSet.h"
#include "NiagaraScriptExecutionContext.h"
#include "NiagaraSystemSimulation.h"
#include "NiagaraSystemInstance.h"
#include "GlobalDistanceFieldParameters.h"
#include "NiagaraComponentPool.h"
#include "NiagaraEffectType.h"
#include "NiagaraScalabilityManager.h"
#include "NiagaraDebuggerCommon.h"
#include "NiagaraDeferredMethodQueue.h"

#include "NiagaraWorldManager.generated.h"

class FNiagaraDataChannelManager;
class FNiagaraSimpleObjectPool;
class UWorld;
class UNiagaraParameterCollection;
class UNiagaraParameterCollectionInstance;
class UNiagaraComponentPool;
struct FNiagaraScalabilityState;
class UNiagaraDataChannelHandler;
class UNiagaraDataManager;

enum class ENiagaraWorldManagerTickPass : int
{
	NoneOverlapping,
	Overlapping,

	Num,
	First = NoneOverlapping,
	Last  = Overlapping,
};

USTRUCT()
struct FNiagaraWorldManagerTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()

	//~ FTickFunction Interface
	virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;
	virtual FName DiagnosticContext(bool bDetailed)  override;
	//~ FTickFunction Interface

	FNiagaraWorldManager*			Owner = nullptr;
	ENiagaraWorldManagerTickPass	TickPass = ENiagaraWorldManagerTickPass::Num;
};

template<>
struct TStructOpsTypeTraits<FNiagaraWorldManagerTickFunction> : public TStructOpsTypeTraitsBase2<FNiagaraWorldManagerTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

using FNiagaraSystemSimulationRef = TSharedRef<FNiagaraSystemSimulation, ESPMode::ThreadSafe>;

enum class ENiagaraScalabilityCullingMode : uint8
{
	/* Scalability culling is enabled as normal. */
	Enabled,
	/* No scalability culling is done but FX stay registered with the managers etc for when it is resumed. */
	Paused,
	/* Scalability is disabled entirely and all tracking is dropped. */
	Disabled,
};

struct FNiagaraCachedViewInfo
{
	FMatrix ViewMat;
	FMatrix ProjectionMat;
	FMatrix ViewProjMat;
	FMatrix ViewToWorld;
	TArray<FPlane, TInlineAllocator<6>> FrutumPlanes;

	void Init(const FWorldCachedViewInfo& WorldViewInfo);
};
/**
* Manager class for any data relating to a particular world.
*/
class FNiagaraWorldManager : public FGCObject
{
	friend class FNiagaraDebugHud;

public:
	FNiagaraWorldManager();
	virtual ~FNiagaraWorldManager() override;

	void Init(UWorld* InWorld);

	void OnPostWorldInit();

	static NIAGARA_API FNiagaraWorldManager* Get(const UWorld* World);
	static void OnStartup();
	static void OnShutdown();

	// Gamethread callback to cleanup references to the given ComputeDispatchInterface before it gets deleted on the renderthread.
	static void OnComputeDispatchInterfaceDestroyed(class FNiagaraGpuComputeDispatchInterface* InComputeDispatchInterface);

	static void DestroyAllSystemSimulations(class UNiagaraSystem* System);

	//~ GCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	//~ GCObject Interface

private:
	void TickParameterCollections();
public:
	UNiagaraParameterCollectionInstance* GetParameterCollection(UNiagaraParameterCollection* Collection);
	void CleanupParameterCollections();

	FNiagaraSystemSimulationRef GetSystemSimulation(ETickingGroup TickGroup, UNiagaraSystem* System);
	void DestroySystemSimulation(UNiagaraSystem* System);
	void DestroySystemInstance(FNiagaraSystemInstancePtr& InPtr);	

#if WITH_EDITOR
	void OnSystemPostChange(UNiagaraSystem* System);
#endif

	void MarkSimulationForPostActorWork(FNiagaraSystemSimulation* SystemSimulation);
	void MarkSimulationsForEndOfFrameWait(FNiagaraSystemSimulation* SystemSimulation);

	/** Called at the very beginning of world ticking, even when paused. */
	void TickStart(float DeltaSeconds);
	
	/** Called at the very end of world ticking, even when paused. */
	void TickEnd(float DeltaSeconds);

	void Tick(ENiagaraWorldManagerTickPass TickPass, ETickingGroup TickGroup, float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent);
private:
	void ExecutePreTickWork(ETickingGroup TickGroup, float DeltaSeconds);
	void ExecuteSimulations(ENiagaraWorldManagerTickPass TickPass, ETickingGroup TickGroup, float DeltaSeconds, const FGraphEventRef& MyCompletionGraphEvent);
	void ExecutePostTickWork(ETickingGroup TickGroup, float DeltaSeconds);

public:
	void PreActorTick(ELevelTick InLevelTick, float InDeltaSeconds);

	/** Called after all actor tick groups are complete. */
	void PostActorTick(float DeltaSeconds);

	/** Called before we run end of frame updates, allows us to wait on async work. */
	void PreSendAllEndOfFrameUpdates();

	void OnWorldBeginTearDown();
	void OnWorldCleanup(bool bSessionEnded, bool bCleanupResources);
	void OnPostWorldCleanup(bool bSessionEnded, bool bCleanupResources);
	
	//TODO: Remove - UE-347574
	void LegacyOnWorldBeginTearDown();
	void LegacyOnWorldCleanup(bool bSessionEnded, bool bCleanupResources);
	void LegacyOnPostWorldCleanup(bool bSessionEnded, bool bCleanupResources);

	void PreGarbageCollect();
	void PostReachabilityAnalysis();
	void PostGarbageCollect();
	void PreGarbageCollectBeginDestroy();
	void RefreshOwnerAllowsScalability();
	
	template<typename T>
	const T& ReadGeneratedData()
	{
		return EditGeneratedData<T>();
	}

	template<typename T>
	T& EditGeneratedData()
	{
		const FNDI_GeneratedData::TypeHash Hash = T::GetTypeHash();
		const auto* ExistingValue = DIGeneratedData.Find(Hash);
		if (ExistingValue == nullptr)
		{
			ExistingValue = &DIGeneratedData.Emplace(Hash, new T());
		}
		return static_cast<T&>(**ExistingValue);
	}

	TArrayView<const FNiagaraCachedViewInfo> GetCachedViewInfo() const { return MakeArrayView(CachedViewInfo); }

	//Returns the distance to the nearest viewpoint to the give location. Used for a distance on which to base LODs.
	NIAGARA_API FVector::FReal GetLODDistance(FVector Location) const;
	NIAGARA_API FVector::FReal GetLODDistance(FVector Location, TConstArrayView<FVector> ViewPoints) const;

	UNiagaraComponentPool* GetComponentPool() { return ComponentPool; }

	void UpdateScalabilityManagers(float DeltaSeconds, bool bNewSpawnsOnly);

	// Dump details about what's inside the world manager
	void DumpDetails(FOutputDevice& Ar, bool bDetailed);
	
	NIAGARA_API UWorld* GetWorld();
	inline UWorld* GetWorld()const { return World; }

	//Various helper functions for scalability culling.
	
	void RegisterWithScalabilityManager(UNiagaraComponent* Component, UNiagaraEffectType* EffectType);
	void UnregisterWithScalabilityManager(UNiagaraComponent* Component, UNiagaraEffectType* EffectType);

	/** Should we cull an instance of this system at the passed location before it's even been spawned? */
	NIAGARA_API bool ShouldPreCull(UNiagaraSystem* System, UNiagaraComponent* Component);
	NIAGARA_API bool ShouldPreCull(UNiagaraSystem* System, FVector Location);

	void CalculateScalabilityState(UNiagaraSystem* System, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, UNiagaraEffectType* EffectType, UNiagaraComponent* Component, bool bIsPreCull, float WorstGlobalBudgetUse, FNiagaraScalabilityState& OutState);
	void CalculateScalabilityState(UNiagaraSystem* System, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, UNiagaraEffectType* EffectType, FVector Location, bool bIsPreCull, float WorstGlobalBudgetUse, FNiagaraScalabilityState& OutState);

	/*inline*/ void SortedSignificanceCull(UNiagaraEffectType* EffectType, UNiagaraComponent* Component, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, float Significance, int32& EffectTypeInstCount, uint16& SystemInstCount, FNiagaraScalabilityState& OutState);

#if DEBUG_SCALABILITY_STATE
	void DumpScalabilityState();
#endif

	void DestroyCullProxy(UNiagaraSystem* System);
	class UNiagaraCullProxyComponent* GetCullProxy(UNiagaraComponent* Component);

	template<typename TAction>
	void ForAllSystemSimulations(TAction Func);

	template<typename TAction>
	static void ForAllWorldManagers(TAction Func);

	static void PrimePoolForAllWorlds(UNiagaraSystem* System);
	void PrimePoolForAllSystems();
	void PrimePool(UNiagaraSystem* System);

	static void RequestInvalidateCachedSystemScalabilityDataForAllWorlds();

	void SetTickGroupPriority(const int32 NiagaraTickGroupIndex, const bool bHighPriority);

private:
	static void InvalidateCachedSystemScalabilityDataForAllWorlds();
	void InvalidateCachedSystemScalabilityData();

	bool HasActiveWorld() const;

	void CleanupTransientObjectsAndManagers();

public:
	void SetDebugPlaybackMode(ENiagaraDebugPlaybackMode Mode) { RequestedDebugPlaybackMode = Mode; }
	ENiagaraDebugPlaybackMode GetDebugPlaybackMode() const { return DebugPlaybackMode; }

	void SetDebugPlaybackRate(float Rate) { DebugPlaybackRate = FMath::Clamp(Rate, KINDA_SMALL_NUMBER, 10.0f); }
	float GetDebugPlaybackRate() const { return DebugPlaybackRate; }

#if WITH_NIAGARA_DEBUGGER
	class FNiagaraDebugHud* GetNiagaraDebugHud() { return NiagaraDebugHud.Get(); }
	NIAGARA_API void AddDebugHUDMessage(FName Key, const FNiagaraDebugMessage& Message);
	FNiagaraSystemScalabilityKillCounts& GetSystemScalabilityKillCounts(UNiagaraSystem* System) { return SystemScalabilityKillCounts.FindOrAdd(System); }
	const TMap<TWeakObjectPtr<UNiagaraSystem>, FNiagaraSystemScalabilityKillCounts>& GetSystemScalabilityKillCounts() const{ return SystemScalabilityKillCounts; }
#endif

	class FNiagaraDeferredMethodQueue& GetDeferredMethodQueue() { return DeferredMethods; }

	/**
	Flush the compute simulation queue and any deferred actions.
	*/
	NIAGARA_API void FlushComputeAndDeferredQueues(bool bWaitForGPU);

	/**
	This is a threadsafe queue which will execute on the game thread.
	The queue is flushed at the start of each tick group & pre / post actor tick.
	*/
	NIAGARA_API static void EnqueueGlobalDeferredCallback(TFunction<void()>&& Callback);

	/**
	* Will create a new object or return one from the pool.
	* Existing objects will not be unregistered / reset / sanitized, they come out exactly as they were put into the pool.
	* You must hold a valid reference to the object or it will be GCed.
	* New objects are outered to the world managers world.
	* EXPERIMENTAL: This API is intended for internal use currently, it is subject to change
	*/
	NIAGARA_API UObject* ObjectPoolGetOrCreate(UClass* Class, bool& bIsExistingObject);
	NIAGARA_API UObject* ObjectPoolGetOrCreate(UClass* Class);
	
	/**
	* Returns an object to the pool
	* You are expected to do any unregistering of the component and ensuring no references are held.
	* EXPERIMENTAL: This API is intended for internal use currently, it is subject to change
	*/
	NIAGARA_API void ObjectPoolReturn(UObject* Obj);

	template<class T>
	T* ObjectPoolGetOrCreate() { return CastChecked<T>(ObjectPoolGetOrCreate(T::StaticClass())); }
	template<class T>
	T* ObjectPoolGetOrCreate(bool& bIsExistingObject) { return CastChecked<T>(ObjectPoolGetOrCreate(T::StaticClass(), bIsExistingObject)); }

	/**
	* Objects added here will have a strong reference held.
	* EXPERIMENTAL: This API is intended for internal use currently, it is subject to change
	*/
	void AddReferencedObject(UObject* InObject) { check(!ReferencedObjects.Contains(InObject)); ReferencedObjects.Add(InObject); }
	/**
	* Removes an object from the WM references.
	* EXPERIMENTAL: This API is intended for internal use currently, it is subject to change
	*/
	void RemoveReferencedObject(UObject* InObject) { check(ReferencedObjects.Contains(InObject)); ReferencedObjects.RemoveSwap(InObject); }

	/** Is this component in anyway linked to the local player. */
	static bool IsComponentLocalPlayerLinked(const USceneComponent* Component);

	static void OnRefreshOwnerAllowsScalability();

	NIAGARA_API static void SetScalabilityCullingMode(ENiagaraScalabilityCullingMode NewMode);
	static ENiagaraScalabilityCullingMode GetScalabilityCullingMode() { return ScalabilityCullingMode; }

	template<typename T>
	T& GetOrCreateDataManager();

	FNiagaraDataChannelManager& GetDataChannelManager(){ return *DataChannelManager; }
	NIAGARA_API void InitDataChannel(const UNiagaraDataChannel* Channel, bool bForce);
	NIAGARA_API void RemoveDataChannel(const UNiagaraDataChannel* Channel);
	NIAGARA_API UNiagaraDataChannelHandler* FindDataChannelHandler(const UNiagaraDataChannel* Channel);

	/** Waits for all currently in flight async work to be completed. */
	void WaitForAsyncWork();

	bool UseTickPasses() const { return bUseTickPasses; }

private:
	// Callback function registered with global world delegates to instantiate world manager when a game world is created
	static void OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS);
	
	// Callback function registered with global world delegates to instantiate world manager after a game world is initialized
	static void OnPostWorldInit(UWorld* World, const UWorld::InitializationValues IVS);

	// Callback function registered with global world delegates to cleanup world manager contents
	static void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	// Callback function registered with global world delegates to cleanup world manager contentx
	static void OnPostWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	// Callback function registered with global world delegates to cleanup world manager when a game world is destroyed
	static void OnPreWorldFinishDestroy(UWorld* World);

	// Called when the world begins to be torn down for example by level streaming.
	static void OnWorldBeginTearDown(UWorld* World);

	static void OnWorldPreActorTick(UWorld* InWorld, ELevelTick InLevelTick, float InDeltaSeconds);

	// Callback for when a world tick is begun.
	static void TickWorldStart(UWorld* World, ELevelTick TickType, float DeltaSeconds);

	// Callback for when a world tick is ended.
	static void TickWorldEnd(UWorld* World, ELevelTick TickType, float DeltaSeconds);

	// Callback for when a world is ticked.
	static void TickWorld(UWorld* World, ELevelTick TickType, float DeltaSeconds);

	// Callback to handle any pre GC processing needed.
	static void OnPreGarbageCollect();

	// Callback post reachability
	static void OnPostReachabilityAnalysis();

	// Callback to handle any post GC processing needed.
	static void OnPostGarbageCollect();

	// Callback to handle any pre GC processing needed.
	static void OnPreGarbageCollectBeginDestroy();
	
	// Calculates ViewInfo data for player
	static bool PrepareCachedViewInfo(const APlayerController* PlayerController, FNiagaraCachedViewInfo& OutViewInfo);

	// Gamethread callback to cleanup references to the given ComputeDispatchInterface before it gets deleted on the renderthread.
	void OnComputeDispatchInterfaceDestroyed_Internal(class FNiagaraGpuComputeDispatchInterface* InComputeDispatchInterface);

	bool CanPreCull(UNiagaraEffectType* EffectType);

	void DistanceCull(UNiagaraEffectType* EffectType, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, FVector Location, FNiagaraScalabilityState& OutState);
	void DistanceCull(UNiagaraEffectType* EffectType, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, UNiagaraComponent* Component, FNiagaraScalabilityState& OutState);
	void ViewBasedCulling(UNiagaraEffectType* EffectType, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, FSphere BoundingSphere, float ComponentTimeSinceRendered, bool bIsPrecull, FNiagaraScalabilityState& OutState);
	void InstanceCountCull(UNiagaraEffectType* EffectType, UNiagaraSystem* System, const FNiagaraSystemScalabilitySettings& ScalabilitySettings, FNiagaraScalabilityState& OutState);
	void GlobalBudgetCull(const FNiagaraSystemScalabilitySettings& ScalabilitySettings, float WorstGlobalBudgetUse, FNiagaraScalabilityState& OutState);

	// Returns scalability state if one exists, this function is not designed for runtime performance and for debugging only
	bool GetScalabilityState(UNiagaraComponent* Component, FNiagaraScalabilityState& OutState) const;

	void HandleCSVStats(float DeltaSeconds);

	static FDelegateHandle OnWorldInitHandle;
	static FDelegateHandle OnPostWorldInitHandle;
	static FDelegateHandle OnWorldCleanupHandle;
	static FDelegateHandle OnPostWorldCleanupHandle;
	static FDelegateHandle OnPreWorldFinishDestroyHandle;
	static FDelegateHandle OnWorldBeginTearDownHandle;
	static FDelegateHandle TickWorldStartHandle;
	static FDelegateHandle TickWorldEndHandle;
	static FDelegateHandle TickWorldHandle;
	static FDelegateHandle OnWorldPreActorTickHandle;
	static FDelegateHandle OnWorldPreSendAllEndOfFrameUpdatesHandle;
	static FDelegateHandle PreGCHandle;
	static FDelegateHandle PostReachabilityAnalysisHandle;
	static FDelegateHandle PostGCHandle;
	static FDelegateHandle PreGCBeginDestroyHandle;
	static FDelegateHandle ViewTargetChangedHandle;

	static std::atomic<bool> bInvalidateCachedSystemScalabilityData;

	static NIAGARA_API TMap<class UWorld*, class FNiagaraWorldManager*> WorldManagers;

	UWorld* World = nullptr;

	int ActiveNiagaraTickGroup;

	bool bUseTickPasses = true;
	FNiagaraWorldManagerTickFunction TickFunctions[NiagaraNumTickGroups][int(ENiagaraWorldManagerTickPass::Num)];

	TMap<TObjectPtr<UNiagaraParameterCollection>, TObjectPtr<UNiagaraParameterCollectionInstance>> ParameterCollections;

	TMap<UNiagaraSystem*, FNiagaraSystemSimulationRef> SystemSimulations[NiagaraNumTickGroups][int(ENiagaraWorldManagerTickPass::Num)];

	TArray<FNiagaraSystemSimulationRef> SimulationsWithPostActorWork;

	TArray<FNiagaraSystemSimulationRef> SimulationsWithEndOfFrameWait;

	TArray<FNiagaraCachedViewInfo, TInlineAllocator<8> > CachedViewInfo;

	TObjectPtr<UNiagaraComponentPool> ComponentPool;
	bool bPoolIsPrimed = false;

	TMap<FNDI_GeneratedData::TypeHash, TUniquePtr<FNDI_GeneratedData>> DIGeneratedData;

	/** Instances that have been queued for deletion this frame, serviced in PostActorTick */
	TArray<FNiagaraSystemInstancePtr> DeferredDeletionQueue;

#if DO_CHECK
	bool bScalabilityManagersLocked = false;
#endif

	TMap<TObjectPtr<UNiagaraEffectType>, FNiagaraScalabilityManager> ScalabilityManagers;

	TMap<FName, TObjectPtr<UNiagaraDataManager>> DataManagers;

	FNiagaraDeferredMethodQueue DeferredMethods;

	/** True if the app has focus. We prevent some culling if the app doesn't have focus as it can interfere. */
	bool bAppHasFocus;

	bool bIsTearingDown = false;

	/** True if OnWorldCleanup has been called, will get reset with Init() */
	bool bIsCleaningUp = false;

	float WorldLoopTime = 0.0f;
	
	ENiagaraDebugPlaybackMode RequestedDebugPlaybackMode = ENiagaraDebugPlaybackMode::Play;
	ENiagaraDebugPlaybackMode DebugPlaybackMode = ENiagaraDebugPlaybackMode::Play;
	float DebugPlaybackRate = 1.0f;

#if WITH_NIAGARA_DEBUGGER
	TUniquePtr<class FNiagaraDebugHud> NiagaraDebugHud;
	TMap<TWeakObjectPtr<UNiagaraSystem>, FNiagaraSystemScalabilityKillCounts> SystemScalabilityKillCounts;
#endif
#if WITH_NIAGARA_LEAK_DETECTOR
	TUniquePtr<class FNiagaraComponentLeakDetector> ComponentLeakDetector;
#endif

	TMap<TObjectPtr<UNiagaraSystem>, TObjectPtr<UNiagaraCullProxyComponent>> CullProxyMap;
	TArray<TObjectPtr<UObject>> ReferencedObjects;

	/** A global flag for all scalability culling */
	static ENiagaraScalabilityCullingMode ScalabilityCullingMode;

	TUniquePtr<FNiagaraDataChannelManager> DataChannelManager;

	TUniquePtr<FNiagaraSimpleObjectPool> ObjectPool;

#if WITH_PER_FXTYPE_PARTICLE_PERF_STATS
	FParticlePerfStatsListenerPtr FXTypeCSVListener;
#endif

#if WITH_EDITOR
	using FSystemDelegateMap = TMap<TWeakObjectPtr<UNiagaraSystem>, FDelegateHandle>;
	FSystemDelegateMap SystemPostChangeDelegateHandles;
#endif

	TSharedPtr<std::atomic<float>> LastRenderThreadUpdateTime;
	float RenderThreadWorldTimeLag = 0.0f;
};


template<typename TAction>
void FNiagaraWorldManager::ForAllSystemSimulations(TAction Func)
{
	for (int TG = 0; TG < NiagaraNumTickGroups; ++TG)
	{
		for (int iTickPass = 0; iTickPass < int(ENiagaraWorldManagerTickPass::Num); ++iTickPass)
		{
			for (TPair<UNiagaraSystem*, FNiagaraSystemSimulationRef>& SimPair : SystemSimulations[TG][iTickPass])
			{
				Func(SimPair.Value.Get());
			}
		}
	}
}

template<typename TAction>
void FNiagaraWorldManager::ForAllWorldManagers(TAction Func)
{
	for (auto& Pair : WorldManagers)
	{
		if (Pair.Value)
		{
			Func(*Pair.Value);
		}
	}
}

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "NiagaraDataInterfaceSkeletalMesh.h"
#endif
