// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "NiagaraStatelessRange.h"
#include "NiagaraCommon.h"
#include "StructUtils/InstancedStruct.h"

#include "NiagaraStatelessDistribution.generated.h"

UENUM()
enum class ENiagaraDistributionMode : uint8
{
	Array,
	Binding,
	Expression,
	UniformConstant,
	NonUniformConstant,
	UniformRange,
	NonUniformRange,
	UniformCurve,
	NonUniformCurve,
	ColorGradient,
};

UENUM()
enum class ENiagaraDistributionInterpolationMode : uint8
{
	// Values sampled will be linearly interpolated
	Linear,
	// No interpolation is applied, fractional indexes will be rounded to nearest
	None,
};

UENUM()
enum class ENiagaraDistributionAddressMode : uint8
{
	// Index will be clamped, i.e. Clamp(Index, 0, Num)
	Clamp,
	// Index will wrap around, i.e. Modulo(Index, Num)
	Wrap,
};

UENUM()
enum class ENiagaraDistributionTimeRangeMode : uint8
{
	Direct,			// The lookup value is used directly to look into the data
	Normalized,		// The lookup value is normalized so should expand over the data
	Custom,			// A custom time range will be applied to the lookup value
};

// Default distribution LookupValueMode the enumeration used can be overridden using metadata on the property, i.e. DistributionLookupValueEnumPath="/Script/Niagara.ENiagaraDistributionLookupValueMode"
// Random is a special value discovered when packing the distribution data, the max number of potential values is based on EFlags::LookupValueModeMask - 1
// Meta data TimeRangeMode is used to drive how the time range will be calculated when not overriden by the user, see ENiagaraDistributionTimeRangeMode
UENUM()
enum class ENiagaraDistributionLookupValueMode : uint8
{
	// Use a randomly generated value, generally evaluated at spawn
	Random = 0xff UMETA(TimeRangeMode = "Normalized"),
	// Bound to the variable Particles.NormalizedAge
	ParticlesNormalizedAge = 0 UMETA(TimeRangeMode = "Normalized", DisplayName = "Particles.NormalizedAge"),
};

// Distribution type generally used in initial modules
UENUM()
enum class ENiagaraDistributionInitialLookupValueMode : uint8
{
	// Use a randomly generated value, evaluted at birth only
	Random = 0xff UMETA(TimeRangeMode = "Normalized"),
	// Bound to the variable Particles.UniqueID
	UniqueID = 0 UMETA(TimeRangeMode = "Direct", DisplayName = "Particles.UniqueID"),
};

enum class ENiagaraDistributionCurveLUTMode
{
	Sample,		// Each sample in the LUT represents the curve evaulation
	Accumulate,	// Each sample in the LUT represents the acculumation of the curve evaluations
};

USTRUCT()
struct FNiagaraDistributionBase
{
	GENERATED_BODY()

	virtual ~FNiagaraDistributionBase() = default;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	ENiagaraDistributionMode Mode = ENiagaraDistributionMode::UniformConstant;

	// Maps to ENiagaraDistributionInterpolationMode
	UPROPERTY(EditAnywhere, Category = "Parameters")
	uint8 InterpolationMode : 1 = uint8(ENiagaraDistributionInterpolationMode::Linear);

	// Maps to ENiagaraDistributionAddressMode
	UPROPERTY(EditAnywhere, Category = "Parameters")
	uint8 AddressMode : 1 = uint8(ENiagaraDistributionAddressMode::Clamp);

	// Custom per property, the default is to use ENiagaraDistributionLookupValueMode, check the metadata for exact binding
	UPROPERTY(EditAnywhere, Category = "Parameters")
	uint8 LookupValueMode = uint8(ENiagaraDistributionLookupValueMode::ParticlesNormalizedAge);

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FNiagaraVariableBase ParameterBinding;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FInstancedStruct ParameterExpression;

	bool IsArray() const { return Mode == ENiagaraDistributionMode::Array; }
	bool IsBinding() const { return Mode == ENiagaraDistributionMode::Binding; }
	bool IsExpression() const { return Mode == ENiagaraDistributionMode::Expression; }
	bool IsConstant() const { return Mode == ENiagaraDistributionMode::UniformConstant || Mode == ENiagaraDistributionMode::NonUniformConstant; }
	bool IsUniform() const { return Mode == ENiagaraDistributionMode::UniformConstant || Mode == ENiagaraDistributionMode::UniformRange; }
	bool IsCurve() const { return Mode == ENiagaraDistributionMode::UniformCurve || Mode == ENiagaraDistributionMode::NonUniformCurve; }
	bool IsGradient() const { return Mode == ENiagaraDistributionMode::ColorGradient; }
	bool IsRange() const { return Mode == ENiagaraDistributionMode::UniformRange || Mode == ENiagaraDistributionMode::NonUniformRange; }

	ENiagaraDistributionInterpolationMode GetInterpolationMode() const { return ENiagaraDistributionInterpolationMode(InterpolationMode); }
	ENiagaraDistributionAddressMode GetAddressMode() const { return ENiagaraDistributionAddressMode(AddressMode); }
	uint8 GetLookupValueMode() const { return LookupValueMode; }

#if WITH_EDITORONLY_DATA
	void SetInterpolationMode(ENiagaraDistributionInterpolationMode InMode) { check(uint8(InMode) >= 0 && uint8(InMode) <= 1); InterpolationMode = uint8(InMode); }
	void SetAddressMode(ENiagaraDistributionAddressMode InMode) { check(uint8(InMode) >= 0 && uint8(InMode) <= 1); AddressMode = uint8(InMode); }
	void SetLookupValueMode(uint8 InMode) { LookupValueMode = InMode; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	TArray<float> ChannelConstantsAndRanges;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	TArray<FRichCurve> ChannelCurves;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	int32 MaxLutSampleCount = 128;

	NIAGARA_API bool operator==(const FNiagaraDistributionBase& Other) const;

	void ForEachParameterBinding(TFunction<void(const FNiagaraVariableBase&)> Delegate) const;
	NIAGARA_API static void ForEachParameterBinding(const UStruct* ObjectStruct, const void* Object, TFunction<void(const FNiagaraVariableBase&)> Delegate);
	NIAGARA_API static void ForEachParameterBinding(const UObject* InObject, TFunction<void(const FNiagaraVariableBase&)> Delegate);

	NIAGARA_API virtual bool AllowDistributionMode(ENiagaraDistributionMode Mode, const FProperty* Property) const;
protected:
	NIAGARA_API virtual bool InternalAllowRangeDistribution(ENiagaraDistributionMode Mode, const FProperty* Property) const;
	NIAGARA_API virtual bool InternalAllowCurveDistribution(ENiagaraDistributionMode Mode, const FProperty* Property) const;

public:
	NIAGARA_API bool AllowAddressMode(const FProperty* Property) const;
	NIAGARA_API bool AllowInterpolationMode(const FProperty* Property) const;
	NIAGARA_API bool AllowLookupValueMode(const FProperty* Property) const;

	virtual bool DisplayAsColor() const { return false; }
	virtual int32 GetBaseNumberOfChannels() const { return 0; }
	virtual void UpdateValuesFromDistribution() { }
	virtual void UpdateTimeRangeFromDistribution(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange = FVector2f::ZeroVector) {}

	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition(); }

	static void PostEditChangeProperty(UObject* OwnerObject, FPropertyChangedEvent& PropertyChangedEvent);

	NIAGARA_API static const UEnum* GetLookupValueModeEnum(const FProperty* Property);

	NIAGARA_API static void ForEachDistribution(const UStruct* ObjectStruct, const void* Object, TFunction<void(const FNiagaraDistributionBase&)> Delegate);
	NIAGARA_API static void ForEachDistribution(const UStruct* ObjectStruct, void* Object, TFunction<void(FNiagaraDistributionBase&)> Delegate);

	NIAGARA_API static void ForEachDistribution(const UObject* InObject, TFunction<void(const FNiagaraDistributionBase&)> Delegate);
	NIAGARA_API static void ForEachDistribution(UObject* InObject, TFunction<void(FNiagaraDistributionBase&)> Delegate);
#endif
};

USTRUCT()
struct FNiagaraDistributionRangeInt
{
	GENERATED_BODY()

	FNiagaraDistributionRangeInt() = default;
	explicit FNiagaraDistributionRangeInt(int32 ConstantValue) { InitConstant(ConstantValue); }

	static constexpr int32 GetDefaultValue() { return 0; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	ENiagaraDistributionMode Mode = ENiagaraDistributionMode::UniformConstant;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FNiagaraVariableBase ParameterBinding;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FInstancedStruct ParameterExpression;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	int32 Min = GetDefaultValue();

	UPROPERTY(EditAnywhere, Category = "Parameters")
	int32 Max = GetDefaultValue();

	NIAGARA_API void InitConstant(int32 Value);
	NIAGARA_API FNiagaraStatelessRangeInt CalculateRange(const int32 Default = GetDefaultValue()) const;

	bool IsArray() const { return Mode == ENiagaraDistributionMode::Array; }
	bool IsBinding() const { return Mode == ENiagaraDistributionMode::Binding; }
	bool IsExpression() const { return Mode == ENiagaraDistributionMode::Expression; }
	bool IsConstant() const { return Mode == ENiagaraDistributionMode::UniformConstant || Mode == ENiagaraDistributionMode::NonUniformConstant; }
	bool IsUniform() const { return Mode == ENiagaraDistributionMode::UniformConstant || Mode == ENiagaraDistributionMode::UniformRange; }
	bool IsCurve() const { return Mode == ENiagaraDistributionMode::UniformCurve || Mode == ENiagaraDistributionMode::NonUniformCurve; }
	bool IsGradient() const { return Mode == ENiagaraDistributionMode::ColorGradient; }
	bool IsRange() const { return Mode == ENiagaraDistributionMode::UniformRange || Mode == ENiagaraDistributionMode::NonUniformRange; }

#if WITH_EDITORONLY_DATA
	FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetIntDef(); }
#endif
};

USTRUCT()
struct FNiagaraDistributionRangeFloat : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	FNiagaraDistributionRangeFloat() = default;
	explicit FNiagaraDistributionRangeFloat(float ConstantValue) { InitConstant(ConstantValue); }
	explicit FNiagaraDistributionRangeFloat(float MinValue, float MaxValue) { InitRange(MinValue, MaxValue); }

	static constexpr float GetDefaultValue() { return 0.0f; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	float Min = GetDefaultValue();

	UPROPERTY(EditAnywhere, Category = "Parameters")
	float Max = GetDefaultValue();

	NIAGARA_API void InitConstant(float Value);
	NIAGARA_API void InitRange(float MinValue, float MaxValue);
	NIAGARA_API FNiagaraStatelessRangeFloat CalculateRange(const float Default = GetDefaultValue()) const;

#if WITH_EDITORONLY_DATA
	virtual bool AllowDistributionMode(ENiagaraDistributionMode InMode, const FProperty* Property) const override { return InternalAllowRangeDistribution(InMode, Property); }
	virtual int32 GetBaseNumberOfChannels() const override { return 1; }
	NIAGARA_API virtual void UpdateValuesFromDistribution() override;
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetFloatDef(); }
#endif
	bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

USTRUCT()
struct FNiagaraDistributionRangeVector2 : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	FNiagaraDistributionRangeVector2() = default;
	explicit FNiagaraDistributionRangeVector2(const FVector2f& ConstantValue) { InitConstant(ConstantValue); }

	static constexpr FVector2f GetDefaultValue() { return FVector2f::ZeroVector; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector2f Min = GetDefaultValue();

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector2f Max = GetDefaultValue();

	NIAGARA_API void InitConstant(const FVector2f& Value);
	NIAGARA_API FNiagaraStatelessRangeVector2 CalculateRange(const FVector2f& Default = GetDefaultValue()) const;

#if WITH_EDITORONLY_DATA
	virtual bool AllowDistributionMode(ENiagaraDistributionMode InMode, const FProperty* Property) const override { return InternalAllowRangeDistribution(InMode, Property); }
	virtual int32 GetBaseNumberOfChannels() const override { return 2; }
	virtual void UpdateValuesFromDistribution() override;
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetVec2Def(); }
#endif
	bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

USTRUCT()
struct FNiagaraDistributionRangeVector3 : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	FNiagaraDistributionRangeVector3() = default;
	explicit FNiagaraDistributionRangeVector3(const FVector3f& ConstantValue) { InitConstant(ConstantValue); }

	static constexpr FVector3f GetDefaultValue() { return FVector3f::ZeroVector; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector3f Min = GetDefaultValue();

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector3f Max = GetDefaultValue();

	NIAGARA_API void InitConstant(const FVector3f& Value);
	NIAGARA_API FNiagaraStatelessRangeVector3 CalculateRange(const FVector3f& Default = GetDefaultValue()) const;

#if WITH_EDITORONLY_DATA
	virtual bool AllowDistributionMode(ENiagaraDistributionMode InMode, const FProperty* Property) const override { return InternalAllowRangeDistribution(InMode, Property); }
	virtual int32 GetBaseNumberOfChannels() const override { return 3; }
	virtual void UpdateValuesFromDistribution() override;
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetVec3Def(); }
#endif
	bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

USTRUCT()
struct FNiagaraDistributionRangeColor : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	FNiagaraDistributionRangeColor() = default;
	explicit FNiagaraDistributionRangeColor(const FLinearColor& ConstantValue) { InitConstant(ConstantValue); }

	static constexpr FLinearColor GetDefaultValue() { return FLinearColor::White; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FLinearColor Min = GetDefaultValue();

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FLinearColor Max = GetDefaultValue();

	NIAGARA_API void InitConstant(const FLinearColor& Value);
	NIAGARA_API FNiagaraStatelessRangeColor CalculateRange(const FLinearColor& Default = GetDefaultValue()) const;

#if WITH_EDITORONLY_DATA
	virtual bool AllowDistributionMode(ENiagaraDistributionMode InMode, const FProperty* Property) const override { return InternalAllowRangeDistribution(InMode, Property); }
	virtual bool DisplayAsColor() const { return true; }
	virtual int32 GetBaseNumberOfChannels() const override { return 4; }
	virtual void UpdateValuesFromDistribution() override;
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetColorDef(); }
#endif
};

USTRUCT()
struct FNiagaraDistributionRangeRotator : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	FNiagaraDistributionRangeRotator() { InitConstant(FRotator3f::ZeroRotator); }
	explicit FNiagaraDistributionRangeRotator(const FRotator3f& ConstantValue) { InitConstant(ConstantValue); }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FRotator3f Min = FRotator3f::ZeroRotator;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FRotator3f Max = FRotator3f::ZeroRotator;

	NIAGARA_API void InitConstant(const FRotator3f& Value);
	NIAGARA_API FNiagaraStatelessRangeRotator CalculateRange(const FRotator3f& Default = FRotator3f::ZeroRotator) const;

#if WITH_EDITORONLY_DATA
	virtual bool AllowDistributionMode(ENiagaraDistributionMode InMode, const FProperty* Property) const override { return InternalAllowRangeDistribution(InMode, Property); }
	virtual int32 GetBaseNumberOfChannels() const override { return 3; }
	virtual void UpdateValuesFromDistribution() override;
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetQuatDef(); }
#endif
	bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
};

USTRUCT()
struct FNiagaraDistributionFloat : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	FNiagaraDistributionFloat() = default;
	explicit FNiagaraDistributionFloat(float ConstantValue) { InitConstant(ConstantValue); }
	explicit FNiagaraDistributionFloat(std::initializer_list<float> CurvePoints) { InitCurve(CurvePoints); }

	static constexpr float GetDefaultValue() { return 0.0f; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	TArray<float> Values;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector2f ValuesTimeRange = FVector2f(0.0f, 1.0f);

	NIAGARA_API void InitConstant(float Value);
	NIAGARA_API void InitCurve(std::initializer_list<float> CurvePoints);
#if WITH_EDITORONLY_DATA
	NIAGARA_API void InitCurve(const TArray<FRichCurveKey>& CurveKeys);
#endif
	NIAGARA_API FNiagaraStatelessRangeFloat CalculateRange(const float Default = GetDefaultValue()) const;

#if WITH_EDITORONLY_DATA
	bool operator==(const FNiagaraDistributionFloat& Other) const
	{
		return (FNiagaraDistributionBase)*this == (FNiagaraDistributionBase)Other && Values == Other.Values;
	}
	virtual int32 GetBaseNumberOfChannels() const override { return 1; }
	NIAGARA_API virtual void UpdateValuesFromDistribution() override;
	NIAGARA_API virtual void UpdateTimeRangeFromDistribution(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange) override;
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetFloatDef(); }
#endif
};

USTRUCT()
struct FNiagaraDistributionVector2 : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	FNiagaraDistributionVector2() = default;
	explicit FNiagaraDistributionVector2(const float ConstantValue) { InitConstant(ConstantValue); }
	explicit FNiagaraDistributionVector2(const FVector2f& ConstantValue) { InitConstant(ConstantValue); }

	static constexpr FVector2f GetDefaultValue() { return FVector2f::ZeroVector; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	TArray<FVector2f> Values;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector2f ValuesTimeRange = FVector2f(0.0f, 1.0f);

	NIAGARA_API void InitConstant(const float Value);
	NIAGARA_API void InitConstant(const FVector2f& Value);
	NIAGARA_API FNiagaraStatelessRangeVector2 CalculateRange(const FVector2f& Default = GetDefaultValue()) const;

#if WITH_EDITORONLY_DATA
	virtual int32 GetBaseNumberOfChannels() const override { return 2; }
	NIAGARA_API virtual void UpdateValuesFromDistribution() override;
	NIAGARA_API virtual void UpdateTimeRangeFromDistribution(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange) override;
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetVec2Def(); }
#endif
};

USTRUCT()
struct FNiagaraDistributionVector3 : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	FNiagaraDistributionVector3() = default;
	explicit FNiagaraDistributionVector3(const float ConstantValue) { InitConstant(ConstantValue); }
	explicit FNiagaraDistributionVector3(const FVector3f& ConstantValue) { InitConstant(ConstantValue); }
	explicit FNiagaraDistributionVector3(std::initializer_list<float> CurvePoints) { InitCurve(CurvePoints); }
	explicit FNiagaraDistributionVector3(std::initializer_list<FVector3f> CurvePoints) { InitCurve(CurvePoints); }

	static constexpr FVector3f GetDefaultValue() { return FVector3f::ZeroVector; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	TArray<FVector3f> Values;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector2f ValuesTimeRange = FVector2f(0.0f, 1.0f);

	NIAGARA_API void InitConstant(const float Value);
	NIAGARA_API void InitConstant(const FVector3f& Value);
	NIAGARA_API void InitCurve(std::initializer_list<float> CurvePoints);
	NIAGARA_API void InitCurve(std::initializer_list<FVector3f> CurvePoints);
	NIAGARA_API FNiagaraStatelessRangeVector3 CalculateRange(const FVector3f& Default = GetDefaultValue()) const;

#if WITH_EDITORONLY_DATA
	virtual int32 GetBaseNumberOfChannels() const override { return 3; }
	NIAGARA_API virtual void UpdateValuesFromDistribution() override;
	NIAGARA_API virtual void UpdateTimeRangeFromDistribution(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange) override;
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetVec3Def(); }
#endif
};

USTRUCT()
struct FNiagaraDistributionPosition : public FNiagaraDistributionVector3
{
	GENERATED_BODY()

	FNiagaraDistributionPosition() = default;
	explicit FNiagaraDistributionPosition(const float ConstantValue) { InitConstant(ConstantValue); }
	explicit FNiagaraDistributionPosition(const FVector3f& ConstantValue) { InitConstant(ConstantValue); }

#if WITH_EDITORONLY_DATA
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetPositionDef(); }
#endif
};

USTRUCT()
struct FNiagaraDistributionColor : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	FNiagaraDistributionColor() = default;
	explicit FNiagaraDistributionColor(const FLinearColor& ConstantValue) { InitConstant(ConstantValue); }

	static constexpr FLinearColor GetDefaultValue() { return FLinearColor::White; }

	UPROPERTY(EditAnywhere, Category = "Parameters")
	TArray<FLinearColor> Values;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector2f ValuesTimeRange = FVector2f(0.0f, 1.0f);

	NIAGARA_API void InitConstant(const FLinearColor& Value);
	NIAGARA_API FNiagaraStatelessRangeColor CalculateRange(const FLinearColor& Default = GetDefaultValue()) const;

#if WITH_EDITORONLY_DATA
	virtual bool DisplayAsColor() const override { return true; }
	virtual int32 GetBaseNumberOfChannels() const override { return 4; }
	NIAGARA_API virtual void UpdateValuesFromDistribution() override;
	NIAGARA_API virtual void UpdateTimeRangeFromDistribution(ENiagaraDistributionTimeRangeMode TimeRangeMode, const FVector2f& CustomTimeRange) override;
	virtual FNiagaraTypeDefinition GetBindingTypeDef() const { return FNiagaraTypeDefinition::GetColorDef(); }
	bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
#endif
};

USTRUCT()
struct FNiagaraDistributionCurveFloat : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	NIAGARA_API FNiagaraDistributionCurveFloat();
	NIAGARA_API explicit FNiagaraDistributionCurveFloat(ENiagaraDistributionCurveLUTMode InLUTMode);
	NIAGARA_API explicit FNiagaraDistributionCurveFloat(std::initializer_list<float> CurvePoints);

	UPROPERTY(EditAnywhere, Category = "Parameters")
	TArray<float> Values;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector2f ValuesTimeRange = FVector2f(0.0f, 1.0f);

#if WITH_EDITORONLY_DATA
	bool operator==(const FNiagaraDistributionCurveFloat& Other) const
	{
		return (FNiagaraDistributionBase)*this == (FNiagaraDistributionBase)Other && Values == Other.Values;
	}

	virtual bool AllowDistributionMode(ENiagaraDistributionMode InMode, const FProperty* Property) const override { return InternalAllowCurveDistribution(InMode, Property); }
	virtual int32 GetBaseNumberOfChannels() const override { return 1; }
	NIAGARA_API virtual void UpdateValuesFromDistribution() override;

private:
	ENiagaraDistributionCurveLUTMode LUTMode = ENiagaraDistributionCurveLUTMode::Sample;
#endif
};

USTRUCT()
struct FNiagaraDistributionCurveVector3 : public FNiagaraDistributionBase
{
	GENERATED_BODY()

	NIAGARA_API FNiagaraDistributionCurveVector3();
	NIAGARA_API explicit FNiagaraDistributionCurveVector3(ENiagaraDistributionCurveLUTMode InLUTMode);

	UPROPERTY(EditAnywhere, Category = "Parameters")
	TArray<FVector3f> Values;

	UPROPERTY(EditAnywhere, Category = "Parameters")
	FVector2f ValuesTimeRange = FVector2f(0.0f, 1.0f);

#if WITH_EDITORONLY_DATA
	bool operator==(const FNiagaraDistributionCurveVector3& Other) const
	{
		return (FNiagaraDistributionBase)*this == (FNiagaraDistributionBase)Other && Values == Other.Values;
	}

	virtual bool AllowDistributionMode(ENiagaraDistributionMode InMode, const FProperty* Property) const override { return InternalAllowCurveDistribution(InMode, Property); }
	virtual int32 GetBaseNumberOfChannels() const override { return 3; }
	NIAGARA_API virtual void UpdateValuesFromDistribution() override;

private:
	ENiagaraDistributionCurveLUTMode LUTMode = ENiagaraDistributionCurveLUTMode::Sample;
#endif
};

template<>
struct TStructOpsTypeTraits<FNiagaraDistributionRangeFloat> : public TStructOpsTypeTraitsBase2<FNiagaraDistributionRangeFloat>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template<>
struct TStructOpsTypeTraits<FNiagaraDistributionRangeVector2> : public TStructOpsTypeTraitsBase2<FNiagaraDistributionRangeVector2>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template<>
struct TStructOpsTypeTraits<FNiagaraDistributionRangeVector3> : public TStructOpsTypeTraitsBase2<FNiagaraDistributionRangeVector3>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

template<>
struct TStructOpsTypeTraits<FNiagaraDistributionRangeRotator> : public TStructOpsTypeTraitsBase2<FNiagaraDistributionRangeRotator>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FNiagaraDistributionColor> : public TStructOpsTypeTraitsBase2<FNiagaraDistributionColor>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
#endif
