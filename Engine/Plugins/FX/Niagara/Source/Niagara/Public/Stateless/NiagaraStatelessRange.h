// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

template<typename TType> struct FNiagaraStatelessRangeDefaultValue {};
template<> struct FNiagaraStatelessRangeDefaultValue<int32> { static constexpr int32 Zero() { return 0; } };
template<> struct FNiagaraStatelessRangeDefaultValue<float> { static constexpr float Zero() { return 0.0f; } };
template<> struct FNiagaraStatelessRangeDefaultValue<FVector2f> { static const FVector2f Zero() { return FVector2f(0.0f, 0.0f); } };
template<> struct FNiagaraStatelessRangeDefaultValue<FVector3f> { static const FVector3f Zero() { return FVector3f(0.0f, 0.0f, 0.0f); } };
template<> struct FNiagaraStatelessRangeDefaultValue<FVector4f> { static const FVector4f Zero() { return FVector4f(0.0f, 0.0f, 0.0f, 0.0f); } };
template<> struct FNiagaraStatelessRangeDefaultValue<FLinearColor> { static const FLinearColor Zero() { return FLinearColor(0.0f, 0.0f, 0.0f, 0.0f); } };
template<> struct FNiagaraStatelessRangeDefaultValue<FRotator3f> { static const FRotator3f Zero() { return FRotator3f::ZeroRotator; } };

template<typename TType>
struct FNiagaraStatelessRange
{
	using ValueType = TType;

	FNiagaraStatelessRange() = default;
	FNiagaraStatelessRange(const FNiagaraStatelessRange& Other) = default;
	explicit FNiagaraStatelessRange(const ValueType& InMinMax) : Min(InMinMax), Max(InMinMax) {}
	explicit FNiagaraStatelessRange(const ValueType& InMin, const ValueType& InMax) : Min(InMin), Max(InMax) {}

	ValueType GetScale() const { return Max - Min; }

	int32		ParameterOffset = INDEX_NONE;
	ValueType	Min = FNiagaraStatelessRangeDefaultValue<TType>::Zero();
	ValueType	Max = FNiagaraStatelessRangeDefaultValue<TType>::Zero();
};

using FNiagaraStatelessRangeInt = FNiagaraStatelessRange<int32>;
using FNiagaraStatelessRangeFloat = FNiagaraStatelessRange<float>;
using FNiagaraStatelessRangeVector2 = FNiagaraStatelessRange<FVector2f>;
using FNiagaraStatelessRangeVector3 = FNiagaraStatelessRange<FVector3f>;
using FNiagaraStatelessRangeVector4 = FNiagaraStatelessRange<FVector4f>;
using FNiagaraStatelessRangeColor = FNiagaraStatelessRange<FLinearColor>;
using FNiagaraStatelessRangeRotator = FNiagaraStatelessRange<FRotator3f>;
