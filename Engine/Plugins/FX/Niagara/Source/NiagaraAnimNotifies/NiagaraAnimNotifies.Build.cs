// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NiagaraAnimNotifies : ModuleRules
{
    public NiagaraAnimNotifies(ReadOnlyTargetRules Target) : base(Target)
    {
        PublicDependencyModuleNames.AddRange(new string[] { "Core", "CoreUObject", "Engine", "Niagara" });
    }
}
