// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotifyState_TimedNiagaraEffect.h"

#include "Animation/AnimInstance.h"
#include "Animation/AnimSequenceBase.h"
#include "Components/SkeletalMeshComponent.h"

#include "NiagaraComponent.h"
#include "NiagaraSystem.h"
#include "NiagaraFunctionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState_TimedNiagaraEffect)

UAnimNotifyState_TimedNiagaraEffect::UAnimNotifyState_TimedNiagaraEffect(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Template = nullptr;
	LocationOffset.Set(0.0f, 0.0f, 0.0f);
	RotationOffset = FRotator(0.0f, 0.0f, 0.0f);
}

UFXSystemComponent* UAnimNotifyState_TimedNiagaraEffect::SpawnEffect(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation) const
{
	// Only spawn if we've got valid params
	if (ValidateParameters(MeshComp))
	{
		FFXSystemSpawnParameters SpawnParams;
		SpawnParams.SystemTemplate		= Template;
		SpawnParams.AttachToComponent	= MeshComp;
		SpawnParams.AttachPointName		= SocketName;
		SpawnParams.Location			= LocationOffset;
		SpawnParams.Rotation			= RotationOffset;
		SpawnParams.Scale				= Scale;
		SpawnParams.LocationType		= EAttachLocation::KeepRelativeOffset;
		SpawnParams.bAutoDestroy		= !bDestroyAtEnd;

		if (UNiagaraComponent* NewComponent = UNiagaraFunctionLibrary::SpawnSystemAttachedWithParams(SpawnParams))
		{
			if (bApplyRateScaleAsTimeDilation)
			{
				NewComponent->SetCustomTimeDilation(Animation->RateScale);
			}
			return NewComponent;
		}
	}
	return nullptr;
}

UFXSystemComponent* UAnimNotifyState_TimedNiagaraEffect::GetSpawnedEffect(UMeshComponent* MeshComp)
{
	if (MeshComp)
	{
		TArray<USceneComponent*> Children;
		MeshComp->GetChildrenComponents(false, Children);

		if (Children.Num())
		{
			for (USceneComponent* Component : Children)
			{
				if (Component && Component->ComponentHasTag(GetSpawnedComponentTag()))
				{
					if (UFXSystemComponent* FXComponent = CastChecked<UFXSystemComponent>(Component))
					{
						return FXComponent;
					}
				}
			}
		}
	}

	return nullptr;
}

void UAnimNotifyState_TimedNiagaraEffect::NotifyBegin(USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference)
{
	if (UFXSystemComponent* Component = SpawnEffect(MeshComp, Animation))
	{
		// tag the component with the AnimNotify that is triggering the animation so that we can properly clean it up
		Component->ComponentTags.AddUnique(GetSpawnedComponentTag());
	}

	Super::NotifyBegin(MeshComp, Animation, TotalDuration, EventReference);
}

void UAnimNotifyState_TimedNiagaraEffect::NotifyEnd(USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, const FAnimNotifyEventReference& EventReference)
{
	if (UFXSystemComponent* FXComponent = GetSpawnedEffect(MeshComp))
	{
		// untag the component
		FXComponent->ComponentTags.Remove(GetSpawnedComponentTag());

		// Either destroy the component or deactivate it to have it's active FXSystems finish.
		// The component will auto destroy once all FXSystem are gone.
		if (bDestroyAtEnd)
		{
			FXComponent->DestroyComponent();
		}
		else
		{
			FXComponent->Deactivate();
		}
	}

	Super::NotifyEnd(MeshComp, Animation, EventReference);
}

bool UAnimNotifyState_TimedNiagaraEffect::ValidateParameters(USkeletalMeshComponent* MeshComp) const
{
	bool bValid = true;

	if (!Template)
	{
		bValid = false;
	}
	else if (!MeshComp->DoesSocketExist(SocketName) && MeshComp->GetBoneIndex(SocketName) == INDEX_NONE)
	{
		bValid = false;
	}

	return bValid;
}

FName UAnimNotifyState_TimedNiagaraEffect::GetSpawnedComponentTag() const
{
	// we generate a unique tag to associate with our spawned components so that we can clean things up upon completion
	FName NotifyName = GetFName();
	NotifyName.SetNumber(GetUniqueID());

	return NotifyName;
}

FString UAnimNotifyState_TimedNiagaraEffect::GetNotifyName_Implementation() const
{
	if (Template)
	{
		return Template->GetName();
	}

	return UAnimNotifyState::GetNotifyName_Implementation();
}

//////////////////////////////////////////////////////////////////////////

UAnimNotifyState_TimedNiagaraEffectAdvanced::UAnimNotifyState_TimedNiagaraEffectAdvanced(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NotifyProgressUserParameter = FName("NormalizedNotifyProgress");
}

void UAnimNotifyState_TimedNiagaraEffectAdvanced::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNotifyAddRateScale)
		{
			bApplyRateScaleToProgress = false;
		}
	}
}

#if WITH_EDITORONLY_DATA
void UAnimNotifyState_TimedNiagaraEffectAdvanced::PostLoad()
{
	Super::PostLoad();
	if (!bEnableNormalizedNotifyProgress_DEPRECATED)
	{
		NotifyProgressType = ENiagaraAnimNotifyProgressType::None;
	}
}
#endif

void UAnimNotifyState_TimedNiagaraEffectAdvanced::NotifyBegin(USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference)
{
	Super::NotifyBegin(MeshComp, Animation, TotalDuration, EventReference);

	FInstanceProgressInfo& NewInfo = ProgressInfoMap.Add(MeshComp);
	NewInfo.Duration = TotalDuration;
	NewInfo.Elapsed = 0.0f;
}

void UAnimNotifyState_TimedNiagaraEffectAdvanced::NotifyEnd(USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	Super::NotifyEnd(MeshComp, Animation, EventReference);
	ProgressInfoMap.Remove(MeshComp);
}

void UAnimNotifyState_TimedNiagaraEffectAdvanced::NotifyTick(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference)
{
	Super::NotifyTick(MeshComp, Animation, FrameDeltaTime, EventReference);

	//Advance the progress.
	//TODO: There must be some way to avoid this map and lookup. The information about the current elapsed time and a mapping onto the notify range should be available somewhere in the mesh comp and notify.
	if (FInstanceProgressInfo* ProgressInfo = ProgressInfoMap.Find(MeshComp))
	{
		const float RateScale = bApplyRateScaleToProgress ? FMath::Abs(Animation->RateScale) : 1.0f;
		ProgressInfo->Elapsed += FrameDeltaTime * RateScale;
	}

	if (UFXSystemComponent* FXComponent = GetSpawnedEffect(MeshComp))
	{
		//send the notify progress to the FX Component
		if (NotifyProgressType != ENiagaraAnimNotifyProgressType::None && !NotifyProgressUserParameter.IsNone())
		{
			float NotifyProgress = GetNotifyProgress(MeshComp);
			if (NotifyProgressType == ENiagaraAnimNotifyProgressType::Reverse)
			{
				NotifyProgress = 1.0f - NotifyProgress;
			}
			FXComponent->SetFloatParameter(NotifyProgressUserParameter, NotifyProgress);
		}

		//Send anim curve data to the FX Component
		if (AnimCurves.Num() != 0)
		{
			if (UAnimInstance* AnimInst = MeshComp->GetAnimInstance())
			{
				for (int32 Index = 0; Index != AnimCurves.Num(); ++Index)
				{
					FName CurveName = AnimCurves[Index].AnimCurveName;
					FName NiagaraUserVariableName = AnimCurves[Index].UserVariableName;
					if ((!CurveName.IsNone()) && (!NiagaraUserVariableName.IsNone()))
					{
						float CurveValue = 0.0f;
						if (AnimInst->GetCurveValue(CurveName, CurveValue))
						{
							FXComponent->SetFloatParameter(NiagaraUserVariableName, CurveValue);
						}
					}
				}
			}
			else
			{
				// No anim instance, defer to the mesh component's curves
				for (int32 Index = 0; Index != AnimCurves.Num(); ++Index)
				{
					FName CurveName = AnimCurves[Index].AnimCurveName;
					FName NiagaraUserVariableName = AnimCurves[Index].UserVariableName;
					if ((!CurveName.IsNone()) && (!NiagaraUserVariableName.IsNone()))
					{
						float CurveValue = 0.0f;
						if (MeshComp->GetCurveValue(CurveName, 0.0f, CurveValue))
						{
							FXComponent->SetFloatParameter(NiagaraUserVariableName, CurveValue);
						}
					}
				}
			}
		}
	}
}

float UAnimNotifyState_TimedNiagaraEffectAdvanced::GetNotifyProgress(UMeshComponent* MeshComp)
{
	if (FInstanceProgressInfo* ProgressInfo = ProgressInfoMap.Find(MeshComp))
	{
		return FMath::Clamp(ProgressInfo->Elapsed / FMath::Max(ProgressInfo->Duration, SMALL_NUMBER), 0.0f, 1.0f);
	}
	return 0.0f;
}
