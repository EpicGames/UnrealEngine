// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotify_PlayNiagaraEffect.h"

#include "Components/SkeletalMeshComponent.h"
#include "Engine/World.h"

#if WITH_EDITOR
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#endif

#include "NiagaraComponent.h"
#include "NiagaraSystem.h"
#include "NiagaraFunctionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotify_PlayNiagaraEffect)

/////////////////////////////////////////////////////
// UAnimNotify_PlayNiagaraEffect

UAnimNotify_PlayNiagaraEffect::UAnimNotify_PlayNiagaraEffect()
	: Super()
{
	Attached = true;
	Scale = FVector(1.f);
	bAbsoluteScale = false;

#if WITH_EDITORONLY_DATA
	NotifyColor = FColor(192, 255, 99, 255);
#endif // WITH_EDITORONLY_DATA
}

void UAnimNotify_PlayNiagaraEffect::PostLoad()
{
	Super::PostLoad();

	RotationOffsetQuat = FQuat(RotationOffset);
}

#if WITH_EDITOR
void UAnimNotify_PlayNiagaraEffect::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimNotify_PlayNiagaraEffect, RotationOffset))
	{
		RotationOffsetQuat = FQuat(RotationOffset);
	}
}

void UAnimNotify_PlayNiagaraEffect::ValidateAssociatedAssets()
{
	static const FName NAME_AssetCheck("AssetCheck");

	if ((Template != nullptr) && (Template->IsLooping()))
	{
		UObject* ContainingAsset = GetContainingAsset();

		FMessageLog AssetCheckLog(NAME_AssetCheck);

		const FText MessageLooping = FText::Format(
			NSLOCTEXT("AnimNotify", "NiagaraSystem_ShouldNotLoop", "Niagara system {0} used in anim notify for asset {1} is set to looping, but the slot is a one-shot (it won't be played to avoid leaking a component per notify)."),
			FText::AsCultureInvariant(Template->GetPathName()),
			FText::AsCultureInvariant(ContainingAsset->GetPathName()));
		AssetCheckLog.Warning()
			->AddToken(FUObjectToken::Create(ContainingAsset))
			->AddToken(FTextToken::Create(MessageLooping));

		if (GIsEditor)
		{
			AssetCheckLog.Notify(MessageLooping, EMessageSeverity::Warning, /*bForce=*/ true);
		}
	}
}
#endif

void UAnimNotify_PlayNiagaraEffect::Notify(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference)
{
	//Store the spawned effect in a protected variable
	SpawnedEffect = SpawnEffect(MeshComp, Animation);
	
	//Call to BP to allows setting of Niagara User Variables
	Super::Notify(MeshComp, Animation, EventReference);
}

FString UAnimNotify_PlayNiagaraEffect::GetNotifyName_Implementation() const
{
	if (Template)
	{
		return Template->GetName();
	}
	else
	{
		return Super::GetNotifyName_Implementation();
	}
}

UFXSystemComponent* UAnimNotify_PlayNiagaraEffect::SpawnEffect(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation)
{
	UFXSystemComponent* ReturnComp = nullptr;

	if (Template)
	{
		if (Template->IsLooping())
		{
			return ReturnComp;
		}

		if (Attached)
		{
			ReturnComp = UNiagaraFunctionLibrary::SpawnSystemAttached(Template, MeshComp, SocketName, LocationOffset, RotationOffset, EAttachLocation::KeepRelativeOffset, true);
		}
		else
		{
			const FTransform MeshTransform = MeshComp->GetSocketTransform(SocketName);
			ReturnComp = UNiagaraFunctionLibrary::SpawnSystemAtLocation(MeshComp->GetWorld(), Template, MeshTransform.TransformPosition(LocationOffset), (MeshTransform.GetRotation() * RotationOffsetQuat).Rotator(), FVector(1.0f),true);
		}

		if (ReturnComp != nullptr)
		{
			ReturnComp->SetUsingAbsoluteScale(bAbsoluteScale);
			ReturnComp->SetRelativeScale3D_Direct(Scale);
		}
	}

	return ReturnComp;
}

UFXSystemComponent* UAnimNotify_PlayNiagaraEffect::GetSpawnedEffect() 
{
	return SpawnedEffect;
}
