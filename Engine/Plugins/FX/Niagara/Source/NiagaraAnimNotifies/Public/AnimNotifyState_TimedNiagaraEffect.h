// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "AnimNotifyState_TimedNiagaraEffect.generated.h"

class UMeshComponent;
class UNiagaraSystem;
class USkeletalMeshComponent;
class UFXSystemAsset;
class UFXSystemComponent;
class UAnimInstance;

UENUM()
enum class ENiagaraAnimNotifyProgressType : uint8
{
	/** Progress information will not be send to the FX component */
	None,
	/** Send normalized progress (0.0 to 1.0) to the FX component as a float user parameter */
	Forward,
	/** Send reversed normalized progress (1.0 to 0.0) to the FX component as a float user parameter */
	Reverse,
};

// Timed Niagara Effect Notify
// Allows a looping Niagara effect to be played in an animation that will activate
// at the beginning of the notify and deactivate at the end.
UCLASS(Blueprintable, meta = (DisplayName = "Timed Niagara Effect"), MinimalAPI)
class UAnimNotifyState_TimedNiagaraEffect : public UAnimNotifyState
{
	GENERATED_UCLASS_BODY()

	// The niagara system template to use when spawning the niagara component
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NiagaraSystem, meta = (DisplayName = "Niagara System", ToolTip = "The niagara system to spawn for the notify state"))
	TObjectPtr<UNiagaraSystem> Template;

	// The socket within our mesh component to attach to when we spawn the Niagara component
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NiagaraSystem, meta = (ToolTip = "The socket or bone to attach the system to", AnimNotifyBoneName = "true"))
	FName SocketName;

	// Offset from the socket / bone location
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NiagaraSystem, meta = (ToolTip = "Offset from the socket or bone to place the Niagara system"))
	FVector LocationOffset;

	// Offset from the socket / bone rotation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NiagaraSystem, meta = (ToolTip = "Rotation offset from the socket or bone for the Niagara system"))
	FRotator RotationOffset;

	// Scale to spawn the Niagara system at
	UPROPERTY(EditAnywhere, Category = NiagaraSystem)
	FVector Scale = FVector(1.0f);

	// Should we set the animation rate scale as time dilation on the spawn effect?
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NiagaraSystem)
	bool bApplyRateScaleAsTimeDilation = false;

	// Whether or not we destroy the component at the end of the notify or instead just stop
	// the emitters.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NiagaraSystem, meta = (DisplayName = "Destroy Immediately", ToolTip = "Whether the Niagara system should be immediately destroyed at the end of the notify state or be allowed to finish"))
	bool bDestroyAtEnd;
	
	NIAGARAANIMNOTIFIES_API virtual void NotifyBegin(class USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference) override;
	NIAGARAANIMNOTIFIES_API virtual void NotifyEnd(class USkeletalMeshComponent * MeshComp, class UAnimSequenceBase * Animation, const FAnimNotifyEventReference& EventReference) override;

	// Overridden from UAnimNotifyState to provide custom notify name.
	NIAGARAANIMNOTIFIES_API FString GetNotifyName_Implementation() const override;

	// Return FXSystemComponent created from SpawnEffect
	UFUNCTION(BlueprintCallable, Category = "AnimNotify")
	NIAGARAANIMNOTIFIES_API UFXSystemComponent* GetSpawnedEffect(UMeshComponent* MeshComp);

protected:
	// Spawns the NiagaraSystemComponent. Called from Notify.
	NIAGARAANIMNOTIFIES_API virtual UFXSystemComponent* SpawnEffect(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation) const;

	NIAGARAANIMNOTIFIES_API bool ValidateParameters(USkeletalMeshComponent* MeshComp) const;

	NIAGARAANIMNOTIFIES_API FName GetSpawnedComponentTag() const;
};

USTRUCT(BlueprintType)
struct FCurveParameterPair
{
	GENERATED_BODY();
public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AnimCurves, meta = (DisplayName = "Anim Curve Name", ToolTip = "Name of the curve in this montage."))
		FName AnimCurveName;
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AnimCurves, meta = (DisplayName = "Niagara User Float", ToolTip = "Name of the Niagara user float variable."))
		FName UserVariableName;
};

/**
Same as Timed Niagara Effect but also provides some more advanced abilities at an additional cost. 
*/
UCLASS(Blueprintable, meta = (DisplayName = "Advanced Timed Niagara Effect"), MinimalAPI)
class UAnimNotifyState_TimedNiagaraEffectAdvanced : public UAnimNotifyState_TimedNiagaraEffect
{
	GENERATED_UCLASS_BODY()

public:
	NIAGARAANIMNOTIFIES_API virtual void Serialize(class FArchive& Ar) override;
#if WITH_EDITORONLY_DATA
	NIAGARAANIMNOTIFIES_API virtual void PostLoad() override;
#endif
	
	NIAGARAANIMNOTIFIES_API virtual void NotifyBegin(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, float TotalDuration, const FAnimNotifyEventReference& EventReference) override;
	NIAGARAANIMNOTIFIES_API virtual void NotifyEnd(class USkeletalMeshComponent* MeshComp, class UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override;
	NIAGARAANIMNOTIFIES_API virtual void NotifyTick(USkeletalMeshComponent * MeshComp, UAnimSequenceBase * Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference) override;

	// Returns a 0 to 1 value for the progress of this component along the notify.
	UFUNCTION(BlueprintCallable, Category = "AnimNotify")
	NIAGARAANIMNOTIFIES_API float GetNotifyProgress(UMeshComponent* MeshComp);

	// How/if we should send progress to the FX component
	UPROPERTY(EditAnywhere, Category = NotifyProgress)
	ENiagaraAnimNotifyProgressType NotifyProgressType = ENiagaraAnimNotifyProgressType::Forward;

	// Should we apply the animation rate scale when calculating the elasped time.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NotifyProgress, meta = (DisplayName = "Notify Progress Apply Rate Scale"))
	bool bApplyRateScaleToProgress = true;

	UPROPERTY(EditAnywhere, Category = NotifyProgress, meta = (DisplayName = "User Parameter", ToolTip = "The name of your niagara user variable you would like to send the normalized notify progress to."))
	FName NotifyProgressUserParameter;

	UPROPERTY(EditAnywhere, Category = AnimCurves, meta = (DisplayName = "Anim Curve Parameters", ToolTip = "Array of fnames to map Anim Curve names to Niagara Parameters."))
	TArray<FCurveParameterPair> AnimCurves;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bEnableNormalizedNotifyProgress_DEPRECATED = true;
#endif

protected:

	struct FInstanceProgressInfo
	{
		float Duration = 1.0f;
		float Elapsed = 0.0f;
	};
	TMap<UMeshComponent*, FInstanceProgressInfo> ProgressInfoMap;
};
