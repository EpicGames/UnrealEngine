// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Animation/AnimNotifies/AnimNotify.h"

#include "AnimNotify_PlayNiagaraEffect.generated.h"



class UAnimSequenceBase;
class UNiagaraSystem;
class USkeletalMeshComponent;
class UNiagaraSystem;
class UFXSystemComponent;

UCLASS(const, hidecategories = Object, collapsecategories, meta = (DisplayName = "Play Niagara Particle Effect"), MinimalAPI)
class UAnimNotify_PlayNiagaraEffect : public UAnimNotify
{
	GENERATED_BODY()

public:

	NIAGARAANIMNOTIFIES_API UAnimNotify_PlayNiagaraEffect();

	// Begin UObject interface
	NIAGARAANIMNOTIFIES_API virtual void PostLoad() override;
#if WITH_EDITOR
	NIAGARAANIMNOTIFIES_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// End UObject interface

	// Begin UAnimNotify interface
	NIAGARAANIMNOTIFIES_API virtual FString GetNotifyName_Implementation() const override;
	
	NIAGARAANIMNOTIFIES_API virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override;
#if WITH_EDITOR
	NIAGARAANIMNOTIFIES_API virtual void ValidateAssociatedAssets() override;
#endif
	// End UAnimNotify interface

	// Niagara System to Spawn
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimNotify", meta = (DisplayName = "Niagara System"))
	TObjectPtr<UNiagaraSystem> Template;

	// Location offset from the socket
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimNotify")
	FVector LocationOffset;

	// Rotation offset from socket
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimNotify")
	FRotator RotationOffset;

	// Scale to spawn the Niagara system at
	UPROPERTY(EditAnywhere, Category = "AnimNotify")
	FVector Scale;

	// Whether or not we are in absolute scale mode
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "AnimNotify")
	bool bAbsoluteScale;

	// Return FXSystemComponent created from SpawnEffect
	UFUNCTION(BlueprintCallable, Category = "AnimNotify")
	NIAGARAANIMNOTIFIES_API UFXSystemComponent* GetSpawnedEffect();

protected:

	//FXSystem Pointer to Spawned Effect called from Notify.
	UFXSystemComponent* SpawnedEffect;

	// Cached version of the Rotation Offset already in Quat form
	FQuat RotationOffsetQuat;

	// Spawns the NiagaraSystemComponent. Called from Notify.
	NIAGARAANIMNOTIFIES_API virtual UFXSystemComponent* SpawnEffect(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation);


public:

	// Should attach to the bone/socket
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimNotify")
	uint32 Attached : 1; 	//~ Does not follow coding standard due to redirection from BP

	// SocketName to attach to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AnimNotify", meta = (AnimNotifyBoneName = "true"))
	FName SocketName;
};
