// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
* Niagara AnimNotifies module interface
*/
class INiagaraAnimNotifiesModule : public IModuleInterface
{
};


