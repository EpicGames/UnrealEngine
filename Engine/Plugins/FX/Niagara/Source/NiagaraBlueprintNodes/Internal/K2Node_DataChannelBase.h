// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelFunctionLibrary.h"
#include "K2Node_CallFunction.h"
#include "K2Node_DataChannelBase.generated.h"


UCLASS(MinimalAPI, Abstract, NotBlueprintable)
class UK2Node_DataChannelBase : public UK2Node_CallFunction
{
	GENERATED_BODY()
public:

	NIAGARABLUEPRINTNODES_API UNiagaraDataChannel* GetDataChannel() const;
	NIAGARABLUEPRINTNODES_API bool HasValidDataChannel() const;

	virtual void PostLoad() override;

	// //~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	// //~ End UEdGraphNode Interface.

	//~ Begin K2Node Interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual void PreloadRequiredAssets() override;
	virtual bool ShouldShowNodeProperties() const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason)const override;
	//~ End K2Node Interface

	UPROPERTY()
	TSet<FGuid> IgnoredVariables;

protected:

	UEdGraphPin* GetChannelSelectorPin() const;
	UEdGraphPin* GetAccessContextPin(EEdGraphPinDirection Direction) const;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid DataChannelVersion;
#endif

	/** Some nodes support a dynamic NDC reference via a pin connection. By default this is false and nodes must specify an inline NDC asset reference. */
	[[nodiscard]] virtual bool SupportsDynamicDataChannel()const { return false; }

	virtual void AddNDCDerivedPins();
	virtual void RemoveNDCDerivedPins();
	TArray<UEdGraphPin*> NDCDerivedGeneratedPins;

private:

	UPROPERTY()
	TObjectPtr<UNiagaraDataChannelAsset> DataChannel;
};

