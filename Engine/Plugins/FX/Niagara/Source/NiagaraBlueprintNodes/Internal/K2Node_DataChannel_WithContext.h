// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_DataChannelBase.h"
#include "K2Node_WriteDataChannel.h"
#include "K2Node_ReadDataChannel.h"
#include "K2Node_MakeStruct.h"
#include "K2Node_BreakStruct.h"
#include "K2Node_SetFieldsInStruct.h"
#include "K2Node_DataChannel_WithContext.generated.h"


/**
A number of K2 nodes dealing with Data Channels and specifically FNDCAccessContextInst
*/



/** 
A node handling expansion of UNiagaraDataChannelLibrary::WriteToNiagaraDataChannel_WithContext.
*/
UCLASS(MinimalAPI)
class UK2Node_WriteDataChannel_WithContext : public UK2Node_DataChannelBase
{
	GENERATED_BODY()
public:
	UK2Node_WriteDataChannel_WithContext();

	[[nodiscard]] virtual bool SupportsDynamicDataChannel()const override { return true; }
};

/**
A node handling expansion of UNiagaraDataChannelLibrary::ReadFromNiagaraDataChannel_WithContext.
*/
UCLASS(MinimalAPI)
class UK2Node_ReadDataChannel_WithContext : public UK2Node_DataChannelBase
{
	GENERATED_BODY()
public:
	UK2Node_ReadDataChannel_WithContext();

	[[nodiscard]] virtual bool SupportsDynamicDataChannel()const override { return true; }
};


/**
A node handling expansion of UNiagaraDataChannelLibrary::GetDataChannelElementCount_WithContext.
*/
UCLASS(MinimalAPI)
class UK2Node_DataChannelGetNum_WithContext : public UK2Node_DataChannelBase
{
	GENERATED_BODY()

public:
	UK2Node_DataChannelGetNum_WithContext();
	
	[[nodiscard]] virtual bool SupportsDynamicDataChannel()const override { return true; }
};

/**
A node handling expansion of UNiagaraDataChannelLibrary::WriteToNiagaraDataChannelSingle_WithContext.
This expands to multiple function calls to Write the individual NDC variables.
*/
UCLASS(MinimalAPI)
class UK2Node_WriteDataChannelSingle_WithContext : public UK2Node_WriteDataChannel
{
	GENERATED_BODY()
public:
	UK2Node_WriteDataChannelSingle_WithContext();
protected:
	[[nodiscard]] virtual FName GetWriterFunctionName()const { return GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, WriteToNiagaraDataChannel_WithContext); }
};


/**
A node handling expansion of UNiagaraDataChannelLibrary::ReadFromNiagaraDataChannelSingle_WithContext.
This expands to multiple function calls to Read the individual NDC variables.
*/
UCLASS(MinimalAPI)
class UK2Node_ReadDataChannelSingle_WithContext : public UK2Node_ReadDataChannel
{
	GENERATED_BODY()
public:
	UK2Node_ReadDataChannelSingle_WithContext();
protected:
	[[nodiscard]] virtual FName GetReaderFunctionName()const { return GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, ReadFromNiagaraDataChannel_WithContext); }
};

//////////////////////////////////////////////////////////////////////////

/** 
Base class for function calls operating on FNDCAccessContextInst structs. 
These nodes allow NDC users to create, set and get the members for the access contexts being passed to Write/Read NDC functions.
*/
UCLASS()
class UK2Node_DataChannelAccessContextOperation : public UK2Node_CallFunction
{
	GENERATED_BODY()
public:

	UK2Node_DataChannelAccessContextOperation(const FObjectInitializer& ObjectInitializer);

	//Begin UObject Interface
	virtual void PreEditChange(FProperty* PropertyThatWillChange)override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)override;
	//End UObjet Interface
	
	//Begin UK2Node Interface
	virtual void AllocateDefaultPins() override;
	void PreloadRequiredAssets() override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason)const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const;
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;

	[[nodiscard]] virtual bool ShouldShowNodeProperties()const override { return true; }

	//End UK2Node Interface

	UPROPERTY(EditAnywhere, Category = "Access Context")
	TArray<FOptionalPinFromProperty> ShowPinForProperties;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Access Context", meta = (BaseStruct = "/Script/Niagara.NDCAccessContextBase"))
	TObjectPtr<UScriptStruct> ContextStruct;

protected:

	[[nodiscard]] virtual FText GetNodeTitleForContext(UScriptStruct* Struct) const{ return FText::GetEmpty(); }
	[[nodiscard]] virtual FText GetTooltipTextForContext(UScriptStruct* Struct) const{ return FText::GetEmpty(); }

	void ExpandNodeOptionalPins(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UEdGraphPin* AccessContextSourcePin, UEdGraphPin* CurrThen, FName PerPropertyFuncName);

	TArray<FName> OldShownPins;
};

UCLASS()
class UK2Node_DataChannelAccessContext_Make : public UK2Node_DataChannelAccessContextOperation
{
	GENERATED_BODY()
public:

	UK2Node_DataChannelAccessContext_Make(const FObjectInitializer& ObjectInitializer);

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	//End UEdGraphNode Interface.

	//Begin UK2Node Interface
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//END UK2Node Interface

protected:

	[[nodiscard]] virtual FText GetNodeTitleForContext(UScriptStruct* Struct) const override;
	[[nodiscard]] virtual FText GetTooltipTextForContext(UScriptStruct* Struct) const override;
};


UCLASS()
class UK2Node_DataChannelAccessContext_GetMembers : public UK2Node_DataChannelAccessContextOperation
{
	GENERATED_BODY()
public:

	UK2Node_DataChannelAccessContext_GetMembers(const FObjectInitializer& ObjectInitializer);

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	//End UEdGraphNode Interface.

	//Begin UK2Node Interface
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//END UK2Node Interface
protected:

	[[nodiscard]] virtual FText GetNodeTitleForContext(UScriptStruct* Struct) const override;
	[[nodiscard]] virtual FText GetTooltipTextForContext(UScriptStruct* Struct) const override;
};


UCLASS()
class UK2Node_DataChannelAccessContext_SetMembers : public UK2Node_DataChannelAccessContextOperation
{
	GENERATED_BODY()
public:

	UK2Node_DataChannelAccessContext_SetMembers(const FObjectInitializer& ObjectInitializer);

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	//End UEdGraphNode Interface.

	//Begin UK2Node Interface
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//END UK2Node Interface

protected:

	[[nodiscard]] virtual FText GetNodeTitleForContext(UScriptStruct* Struct) const override;
	[[nodiscard]] virtual FText GetTooltipTextForContext(UScriptStruct* Struct) const override;
};

/** Prepares a usable access context ready for accessing an NDC. */
UCLASS()
class UK2Node_DataChannelAccessContext_Prepare : public UK2Node_DataChannelAccessContextOperation
{
	GENERATED_BODY()
public:

	UK2Node_DataChannelAccessContext_Prepare(const FObjectInitializer& ObjectInitializer);

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	//End UEdGraphNode Interface.

	//Begin UK2Node Interface
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//END UK2Node Interface

protected:

	[[nodiscard]] virtual FText GetNodeTitleForContext(UScriptStruct* Struct) const override;
	[[nodiscard]] virtual FText GetTooltipTextForContext(UScriptStruct* Struct) const override;
};

//////////////////////////////////////////////////////////////////////////