// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_DataChannelBase.h"
#include "K2Node_ReadDataChannel.generated.h"


UCLASS(MinimalAPI)
class UK2Node_ReadDataChannel : public UK2Node_DataChannelBase
{
	GENERATED_BODY()

public:
	UK2Node_ReadDataChannel();

	//~ Begin K2Node Interface
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End K2Node Interface

protected:

	virtual void AddNDCDerivedPins()override;

private:
	UFunction* GetReadFunctionForType(const FNiagaraTypeDefinition& TypeDef);
	UEdGraphPin* GetVarPin(const FName& Name) const;
	virtual FName GetReaderFunctionName()const { return GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, ReadFromNiagaraDataChannel); }
};
