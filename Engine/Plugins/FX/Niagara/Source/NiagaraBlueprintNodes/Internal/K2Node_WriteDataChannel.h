// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_DataChannelBase.h"
#include "K2Node_WriteDataChannel.generated.h"


UCLASS(MinimalAPI)
class UK2Node_WriteDataChannel : public UK2Node_DataChannelBase
{
	GENERATED_BODY()

public:
	UK2Node_WriteDataChannel();

	//~ Begin K2Node Interface
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End K2Node Interface

protected:

	virtual void AddNDCDerivedPins()override;

	UFunction* GetWriteFunctionForType(const FNiagaraTypeDefinition& TypeDef);

	[[nodiscard]] virtual FName GetWriterFunctionName()const{ return GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, WriteToNiagaraDataChannel); }
};
