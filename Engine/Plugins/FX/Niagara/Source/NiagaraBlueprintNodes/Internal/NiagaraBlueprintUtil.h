// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "EdGraph/EdGraphPin.h"

class FNiagaraBlueprintUtil
{
public:
	/**
	Attempts to convert the existing Niagara type to a matching type for BP pins
	*/
	static NIAGARABLUEPRINTNODES_API FEdGraphPinType TypeDefinitionToBlueprintType(const FNiagaraTypeDefinition& TypeDef);
};
