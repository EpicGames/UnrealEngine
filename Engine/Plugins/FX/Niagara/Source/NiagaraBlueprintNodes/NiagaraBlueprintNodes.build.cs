// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NiagaraBlueprintNodes : ModuleRules
{
	public NiagaraBlueprintNodes(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BlueprintGraph",
				"CoreUObject",
				"UnrealEd",
				"Engine",
				"NiagaraCore",
				"Niagara",
				"Kismet",
				"KismetCompiler",
				"BlueprintEditorLibrary",
				"StructUtilsEditor",
				"SlateCore",
				"Slate",
				"PropertyEditor",
			}
		);
	}
}
