// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_DataChannelBase.h"
#include "EdGraphSchema_K2.h"

#include "NiagaraDataChannelPublic.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_Self.h"
#include "K2Node_MakeStruct.h"
#include "K2Node_BreakStruct.h"
#include "KismetCompiler.h"
#include "NiagaraBlueprintUtil.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelAccessor.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet/BlueprintInstancedStructLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "K2Node_DataChannel_WithContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_DataChannelBase)

#define LOCTEXT_NAMESPACE "K2Node_DataChannelBase"

UNiagaraDataChannel* UK2Node_DataChannelBase::GetDataChannel() const
{
	return DataChannel ? DataChannel->Get() : nullptr;
}

bool UK2Node_DataChannelBase::HasValidDataChannel() const
{
	return DataChannel && DataChannel->Get();
}

void UK2Node_DataChannelBase::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	if (HasValidDataChannel() && GetDataChannel()->GetVersion() != DataChannelVersion && HasValidBlueprint())
	{
		ReconstructNode();
	}
#endif
}

void UK2Node_DataChannelBase::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	if(!SupportsDynamicDataChannel())
	{
		//Prevent connections to our channel pin if it does not support a dynamic channel.
		UEdGraphPin* ChannelPin = FindPinChecked(TEXT("Channel"));
		ChannelPin->bNotConnectable = true;
		ChannelPin->bDefaultValueIsReadOnly = false;
	}
	
	PreloadObject(DataChannel);

#if WITH_EDITORONLY_DATA
	if (HasValidDataChannel())
	{
		DataChannelVersion = GetDataChannel()->GetVersion();
		AddNDCDerivedPins();
	}
#endif
}

void UK2Node_DataChannelBase::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	const UClass* ActionKey = GetClass();
	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}


void UK2Node_DataChannelBase::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	if (!HasValidDataChannel() && !SupportsDynamicDataChannel())
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoValidDataChannel", "Node does not have a valid data channel - @@").ToString(), this);
		return;
	}

	if(UNiagaraDataChannel* NDC = GetDataChannel())
	{
		if (DataChannelVersion != NDC->GetVersion())
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("StaleNode", "Node is out of sync with the data channel asset, please refresh node to fix up the pins - @@").ToString(), this);
			return;
		}
	}

	ExpandSplitPins(CompilerContext, SourceGraph);
}

bool UK2Node_DataChannelBase::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason)const
{
	//TODO: 
	// I would like to do this kind of type checking and give some useful feedback to users.
	// However currently BP context menus add a bunch of unrelated cruft by default.
	// This means that my OutputReason is never shown to the users.
	// Instead they're just prompted to add a node converting from FNDCAccessContextInst to FNDCAccessContextInst.
	// With all the options being unrelated cruft. So this would actually add to confusion etc rather than improving things.
	// 
	if(MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && MyPin->PinType.PinSubCategoryObject == FNDCAccessContextInst::StaticStruct())
	{
		if(OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && OtherPin->PinType.PinSubCategoryObject == FNDCAccessContextInst::StaticStruct())
		{
			if(UK2Node_DataChannelAccessContextOperation* ContextOp = Cast<UK2Node_DataChannelAccessContextOperation>(OtherPin->GetOwningNode()))
			{
				if(UNiagaraDataChannel* NDC = GetDataChannel())
				{
					//If we have a specific data channel and we're connecting to a context op, ensure the NDC type and the context op type are compatible.
					const UScriptStruct* NDCType = NDC->GetAccessContextType().Get();
					if(NDCType != ContextOp->ContextStruct)
					{
						OutReason = TEXT("Invalid Type For Access Context Pin.");
						return true;						
					}
				}
			}
		}
		else
		{
			OutReason = TEXT("Invalid Type For Access Context Pin.");
			return true;
		}
	}
	return false;
}

void UK2Node_DataChannelBase::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);

#if WITH_EDITORONLY_DATA
	if (Pin == GetChannelSelectorPin())
	{
		DataChannel = nullptr;

		if (UNiagaraDataChannelAsset* ChannelAsset = Cast<UNiagaraDataChannelAsset>(Pin->DefaultObject))
		{
			DataChannel = ChannelAsset;
		}
		
		if (UNiagaraDataChannel* NDC = GetDataChannel())
		{
			DataChannelVersion = NDC->GetVersion();
		}
		else
		{		
			DataChannelVersion = FGuid();
		}

		RemoveNDCDerivedPins();
		AddNDCDerivedPins();
	}
#endif
}

void UK2Node_DataChannelBase::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);

#if WITH_EDITORONLY_DATA
	if (Pin == GetChannelSelectorPin())
	{
		DataChannel = nullptr;
		if (UNiagaraDataChannelAsset* ChannelAsset = Cast<UNiagaraDataChannelAsset>(Pin->DefaultObject))
		{
			DataChannel = Pin->LinkedTo.Num() == 0 ? ChannelAsset : nullptr;
		}
		
		if (UNiagaraDataChannel* NDC = GetDataChannel())
		{
			DataChannelVersion = NDC->GetVersion();
		}
		else
		{
			DataChannelVersion = FGuid();
		}

		RemoveNDCDerivedPins();
		AddNDCDerivedPins();
	}
#endif
}

FText UK2Node_DataChannelBase::GetMenuCategory() const
{
	static FText MenuCategory = LOCTEXT("MenuCategory", "Niagara Data Channel");
	return MenuCategory;
}

UK2Node::ERedirectType UK2Node_DataChannelBase::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	
	ERedirectType Result = UK2Node::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
	if (ERedirectType_None == Result && K2Schema && K2Schema->ArePinTypesCompatible(NewPin->PinType, OldPin->PinType) && (NewPin->PersistentGuid == OldPin->PersistentGuid) && OldPin->PersistentGuid.IsValid())
	{
		Result = ERedirectType_Name;
	}

	return Result;
}

void UK2Node_DataChannelBase::AddNDCDerivedPins()
{
 	GetGraph()->NotifyNodeChanged(this);
 
 	UBlueprint* NodeBP = GetBlueprint();
 	check(NodeBP);
 	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(NodeBP);
}

void UK2Node_DataChannelBase::RemoveNDCDerivedPins()
{
	for(UEdGraphPin* Pin : NDCDerivedGeneratedPins)
	{
		Pins.Remove(Pin);
		DestroyPin(Pin);
	}
	NDCDerivedGeneratedPins.Empty();

 	GetGraph()->NotifyNodeChanged(this);
 
 	UBlueprint* NodeBP = GetBlueprint();
 	check(NodeBP);
 	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(NodeBP);
}

void UK2Node_DataChannelBase::PreloadRequiredAssets()
{
	Super::PreloadRequiredAssets();

	if (DataChannel)
	{
		PreloadObject(DataChannel);
		PreloadObject(DataChannel->Get());
	}
}

bool UK2Node_DataChannelBase::ShouldShowNodeProperties() const
{
	return true;
}

UEdGraphPin* UK2Node_DataChannelBase::GetChannelSelectorPin() const
{
	return FindPinChecked(FName("Channel"), EGPD_Input);
}

UEdGraphPin* UK2Node_DataChannelBase::GetAccessContextPin(EEdGraphPinDirection Direction)const
{
	return FindPin(FName("AccessContext"), Direction);
}

#undef LOCTEXT_NAMESPACE
