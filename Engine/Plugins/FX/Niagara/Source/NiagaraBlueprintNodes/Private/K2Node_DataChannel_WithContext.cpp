// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_DataChannel_WithContext.h"

#include "NiagaraDataChannelPublic.h"

#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_ExecutionSequence.h"
#include "KismetCompiler.h"
#include "NiagaraBlueprintUtil.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "UObject/StructOnScope.h"
#include "UObject/TextProperty.h"

#define LOCTEXT_NAMESPACE "UK2Node_WriteDataChannel_WithContext"

UK2Node_WriteDataChannel_WithContext::UK2Node_WriteDataChannel_WithContext()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, WriteToNiagaraDataChannel_WithContext), UNiagaraDataChannelLibrary::StaticClass());
}

UK2Node_ReadDataChannel_WithContext::UK2Node_ReadDataChannel_WithContext()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, ReadFromNiagaraDataChannel_WithContext), UNiagaraDataChannelLibrary::StaticClass());
}

UK2Node_DataChannelGetNum_WithContext::UK2Node_DataChannelGetNum_WithContext()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, GetDataChannelElementCount_WithContext), UNiagaraDataChannelLibrary::StaticClass());
}

UK2Node_WriteDataChannelSingle_WithContext::UK2Node_WriteDataChannelSingle_WithContext()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, WriteToNiagaraDataChannelSingle_WithContext), UNiagaraDataChannelLibrary::StaticClass());
}

UK2Node_ReadDataChannelSingle_WithContext::UK2Node_ReadDataChannelSingle_WithContext()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, ReadFromNiagaraDataChannelSingle_WithContext), UNiagaraDataChannelLibrary::StaticClass());
}

//////////////////////////////////////////////////////////////////////////

//Optional Pin Manager that will by default show only input NDC access context properties.
//Will automatically set the correct default values for pins from the given struct.
struct FK2Node_MakeNDCAccessContextInst_OptionalPinManager : public FOptionalPinManager
{
	const uint8* const SampleStructMemory;
	UBlueprint* OwningBP;
	mutable bool bHasAdvancedPins;

	FK2Node_MakeNDCAccessContextInst_OptionalPinManager(const uint8* InSampleStructMemory, UBlueprint* InOwningBP)
		: FOptionalPinManager()
		, SampleStructMemory(InSampleStructMemory)
		, OwningBP(InOwningBP)
		, bHasAdvancedPins(false)
	{
	}

	void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const
	{
		Record.bCanToggleVisibility = true;
		Record.bShowPin = TestProperty ? TestProperty->HasMetaData(TEXT("NDCAccessContextInput")) : false;
		Record.bIsMarkedForAdvancedDisplay = TestProperty ? TestProperty->HasAnyPropertyFlags(CPF_AdvancedDisplay) : false;
		bHasAdvancedPins |= Record.bIsMarkedForAdvancedDisplay;
	}

	void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const
	{
		if (Pin && Property)
		{
			const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
			check(Schema);

			// Should pin default value be filled as FText?
			const bool bIsText = Property->IsA<FTextProperty>();
			checkSlow(bIsText == ((UEdGraphSchema_K2::PC_Text == Pin->PinType.PinCategory) && !Pin->PinType.IsContainer()));

			const bool bIsObject = Property->IsA<FObjectPropertyBase>();
			checkSlow(bIsObject == ((UEdGraphSchema_K2::PC_Object == Pin->PinType.PinCategory || UEdGraphSchema_K2::PC_Class == Pin->PinType.PinCategory ||
				UEdGraphSchema_K2::PC_SoftObject == Pin->PinType.PinCategory || UEdGraphSchema_K2::PC_SoftClass == Pin->PinType.PinCategory) && !Pin->PinType.IsContainer()));

			if (Property->HasAnyPropertyFlags(CPF_AdvancedDisplay))
			{
				Pin->bAdvancedView = true;
				bHasAdvancedPins = true;
			}

			const FString& MetadataDefaultValue = Property->GetMetaData(TEXT("MakeStructureDefaultValue"));
			if (!MetadataDefaultValue.IsEmpty())
			{
				Schema->SetPinAutogeneratedDefaultValue(Pin, MetadataDefaultValue);
				return;
			}

			if (nullptr != SampleStructMemory)
			{
				FString NewDefaultValue;
				if (FBlueprintEditorUtils::PropertyValueToString(Property, SampleStructMemory, NewDefaultValue))
				{
					if (Schema->IsPinDefaultValid(Pin, NewDefaultValue, nullptr, FText::GetEmpty()).IsEmpty())
					{
						Schema->SetPinAutogeneratedDefaultValue(Pin, NewDefaultValue);
						return;
					}
				}
			}

			Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
		}
	}
};

//////////////////////////////////////////////////////////////////////////

//Optional Pin Manager that will by default only show output NDC access context properties.
struct FK2Node_GetMembersInNDCAccessContext_OptionalPinManager : public FOptionalPinManager
{
	//~ Begin FOptionalPinsUpdater Interface
	virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override
	{
		Record.bCanToggleVisibility = true;
		Record.bShowPin = TestProperty->HasMetaData(TEXT("NDCAccessContextOutput"));
	}
};

//////////////////////////////////////////////////////////////////////////

//Optional Pin Manager that will by default only show transient input properties for an NDC Access Context.
//Used for setting values on an existing access context and so transient input pins are the most likely things we'll want to set here.
struct FK2Node_SetMembersInNDCAccessContext_OptionalPinManager : public FK2Node_MakeNDCAccessContextInst_OptionalPinManager
{
	FK2Node_SetMembersInNDCAccessContext_OptionalPinManager(const uint8* InSampleStructMemory, UBlueprint* BP) : FK2Node_MakeNDCAccessContextInst_OptionalPinManager(InSampleStructMemory, BP)
	{}

	//~ Begin FOptionalPinsUpdater Interface
	virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override
	{
		Record.bCanToggleVisibility = true;
		Record.bShowPin = TestProperty->HasMetaData(TEXT("NDCAccessContextInput")) && TestProperty->HasAnyPropertyFlags(CPF_Transient);//For the set fields node we default to any transient inputs.
	}

	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const override
	{
		FOptionalPinManager::CustomizePinData(Pin, SourcePropertyName, ArrayIndex, Property);
	}
};

//////////////////////////////////////////////////////////////////////////
//UK2Node_DataChannelAccessContextOperation - Base class for all FNDCAccessContext operations.

UK2Node_DataChannelAccessContextOperation::UK2Node_DataChannelAccessContextOperation(const FObjectInitializer& ObjectInitializer)
{
	ContextStruct = FNDCAccessContextBase::StaticStruct();
}

void UK2Node_DataChannelAccessContextOperation::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (PropertyThatWillChange && (PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin) || PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(UK2Node_DataChannelAccessContextOperation, ContextStruct)))
	{
		FOptionalPinManager::CacheShownPins(ShowPinForProperties, OldShownPins);
	}
}

void UK2Node_DataChannelAccessContextOperation::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin) || PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_DataChannelAccessContextOperation, ContextStruct))
	{
		FOptionalPinManager::EvaluateOldShownPins(ShowPinForProperties, OldShownPins, this);
		GetSchema()->ReconstructNode(*this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UK2Node_DataChannelAccessContextOperation::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason)const
{
	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);//TODO Disallow more connections. This doesn't work great atm as BPs by default fill the context menu with lots of unrelated cruft anyway.
}

void UK2Node_DataChannelAccessContextOperation::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	const UClass* ActionKey = GetClass();
	if (!ActionRegistrar.IsOpenForRegistration(GetClass()))
	{
		return;
	}

	//Add a menu item for all subtypes of FNDCAccessContextBase.
	TArray<UScriptStruct*> ContextStructs;
	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		UScriptStruct* Struct = *It;
		if (Struct->IsChildOf(FNDCAccessContextBase::StaticStruct()))
		{
			auto PostNodeSpawn = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda([Struct](UEdGraphNode* NewNode, bool bIsTemplate)
			{
				if(UK2Node_DataChannelAccessContextOperation* TypedNode = Cast<UK2Node_DataChannelAccessContextOperation>(NewNode))
				{
					TypedNode->ContextStruct = Struct;
				}
			});

			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass(), nullptr, PostNodeSpawn);
			NodeSpawner->DefaultMenuSignature.MenuName = GetNodeTitleForContext(Struct);
			NodeSpawner->DefaultMenuSignature.Category = LOCTEXT("Niagara Data Channels Menu Category", "Niagara Data Channel");
			NodeSpawner->DefaultMenuSignature.Tooltip = GetTooltipTextForContext(Struct);

			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

bool UK2Node_DataChannelAccessContextOperation::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{	
	for(UEdGraphPin* Pin : Filter.Context.Pins)
	{
		if(UK2Node_DataChannelAccessContextOperation* ContextNode = Cast<UK2Node_DataChannelAccessContextOperation>(Pin->GetOwningNode()))
		{
			if(ContextNode->ContextStruct != ContextStruct)
			{
				return true;
			}
		}
	}

	return false;
}

void UK2Node_DataChannelAccessContextOperation::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	PreloadObject(ContextStruct);
}

void UK2Node_DataChannelAccessContextOperation::PreloadRequiredAssets()
{
	Super::PreloadRequiredAssets();
	PreloadObject(ContextStruct);
}

void UK2Node_DataChannelAccessContextOperation::ExpandNodeOptionalPins(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UEdGraphPin* AccessContextSourcePin, UEdGraphPin* CurrThen, FName PerPropertyFuncName)
{
	check(CurrThen);

	//For all of our optional property pins, create shunt the pin to a new function call to perform the operation on that property only.
	//TODO: Can likely consolidate this into a single custom thunk function in future.
	TArray<UEdGraphPin*> ToRemove;
	const UEdGraphSchema_K2* Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());
	for (UEdGraphPin* SrcPin : Pins)
	{
		bool bSkip = SrcPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec || SrcPin == AccessContextSourcePin || (SrcPin->Direction == EGPD_Output && SrcPin->LinkedTo.Num() == 0);
		if (bSkip)
		{
			continue;
		}
		//For each visible pin we create a function call node to set that single property on our new context.
		if (FOptionalPinFromProperty* OptionalPin = ShowPinForProperties.FindByPredicate([&](const FOptionalPinFromProperty& CheckPin) { return CheckPin.PropertyName == SrcPin->PinName && CheckPin.bShowPin == true; }))
		{
			ToRemove.Add(SrcPin);
			//If we're an optional pin we should remove from the main function body.
			FProperty* Property = ContextStruct->FindPropertyByName(OptionalPin->PropertyName);
			{
				UK2Node_CallFunction* PropertyFuncCallNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
				FName PropertyFuncName = PerPropertyFuncName;
				PropertyFuncCallNode->SetFromFunction(UNiagaraDataChannelLibrary::StaticClass()->FindFunctionByName(PropertyFuncName));
				PropertyFuncCallNode->AllocateDefaultPins();

				UEdGraphPin* PropertyAccessContextPin = PropertyFuncCallNode->FindPinChecked(TEXT("AccessContext"));
				UEdGraphPin* PropertyContextStructPin = PropertyFuncCallNode->FindPinChecked(TEXT("ContextStruct"));
				UEdGraphPin* PropertyNamePin = PropertyFuncCallNode->FindPinChecked(TEXT("PropertyName"));
				UEdGraphPin* PropertyValuePin = PropertyFuncCallNode->FindPinChecked(TEXT("Value"));

				Schema->ConvertPropertyToPinType(Property, PropertyValuePin->PinType);

				//Needs to be a reference to be consumed as a wildcard by our custom thunk.
				//AutoCreateRefTerm on the function meta will ensure unconnected pins are auto generated a reference property.
				PropertyValuePin->PinType.bIsReference = true;

				//Connect exec and then pins				
				CompilerContext.MovePinLinksToIntermediate(*CurrThen, *PropertyFuncCallNode->GetThenPin());
				Schema->TryCreateConnection(CurrThen, PropertyFuncCallNode->GetExecPin());
				CurrThen = PropertyFuncCallNode->GetThenPin();

				Schema->TryCreateConnection(PropertyAccessContextPin, AccessContextSourcePin);
				
				PropertyContextStructPin->DefaultObject = ContextStruct;
				
				PropertyNamePin->DefaultValue = OptionalPin->PropertyName.ToString();

				if (SrcPin->LinkedTo.Num() > 0)
				{
					CompilerContext.MovePinLinksToIntermediate(*SrcPin, *PropertyValuePin);
				}
				else if (SrcPin->Direction == EGPD_Input)
				{
					//If we're an input that's unconnected, setup our default value.
 					PropertyValuePin->DefaultValue = SrcPin->DefaultValue;
 					PropertyValuePin->DefaultObject = SrcPin->DefaultObject;
 					PropertyValuePin->DefaultTextValue = SrcPin->DefaultTextValue;
				}
			}
		}
	}

	//Remove our optional property pins from the main function call.
	for(UEdGraphPin* SrcPin : ToRemove)
	{
		RemovePin(SrcPin);
	}
}

FText UK2Node_DataChannelAccessContextOperation::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetNodeTitleForContext(ContextStruct);
}

FText UK2Node_DataChannelAccessContextOperation::GetTooltipText() const
{
	return GetTooltipTextForContext(ContextStruct);
}

////

UK2Node_DataChannelAccessContext_Make::UK2Node_DataChannelAccessContext_Make(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, MakeNDCAccessContextInstance), UNiagaraDataChannelLibrary::StaticClass());
}

void UK2Node_DataChannelAccessContext_Make::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	if(ContextStruct)
	{
		FStructOnScope StructOnScope(ContextStruct);
		FK2Node_MakeNDCAccessContextInst_OptionalPinManager OptionalPinManager(StructOnScope.GetStructMemory(), GetBlueprint());
		OptionalPinManager.RebuildPropertyList(ShowPinForProperties, ContextStruct);
		OptionalPinManager.CreateVisiblePins(ShowPinForProperties, ContextStruct, EGPD_Input, this);
	}
}

void UK2Node_DataChannelAccessContext_Make::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!ContextStruct)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoValidAccessContextStruct", "Access Context Struct is null! - @@").ToString(), this);
		return;
	}

	UEdGraphPin* StructPin = FindPinChecked(TEXT("ContextStruct"));
	StructPin->DefaultObject = ContextStruct;

	UEdGraphPin * AccessContextSourcePin = FindPinChecked(TEXT("ReturnValue"));

	ExpandNodeOptionalPins(CompilerContext, SourceGraph, AccessContextSourcePin, GetThenPin(), GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, SetSinglePropertyInNDCAccessContextInstance));
}

FText UK2Node_DataChannelAccessContext_Make::GetNodeTitleForContext(UScriptStruct* Struct) const
{
	return FText::Format(LOCTEXT("UK2Node_DataChannelAccessContext_Make_TitleFmt", "Make ({0}) Instance"), Struct->GetDisplayNameText());
}

FText UK2Node_DataChannelAccessContext_Make::GetTooltipTextForContext(UScriptStruct* Struct)const
{
	return FText::Format(LOCTEXT("UK2Node_DataChannelAccessContext_Make_TooltipFmt", "Creates a new Access Context Instance with an internal type of {0}. "), Struct->GetDisplayNameText());
}

///

UK2Node_DataChannelAccessContext_GetMembers::UK2Node_DataChannelAccessContext_GetMembers(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, GetMembersInNDCAccessContextInstance), UNiagaraDataChannelLibrary::StaticClass());
}

void UK2Node_DataChannelAccessContext_GetMembers::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	if (ContextStruct)
	{
		FK2Node_GetMembersInNDCAccessContext_OptionalPinManager OptionalPinManager;
		OptionalPinManager.RebuildPropertyList(ShowPinForProperties, ContextStruct);
		OptionalPinManager.CreateVisiblePins(ShowPinForProperties, ContextStruct, EGPD_Output, this);
	}
}

void UK2Node_DataChannelAccessContext_GetMembers::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!ContextStruct)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoValidAccessContextStruct", "Access Context Struct is null! - @@").ToString(), this);
		return;
	}

	UEdGraphPin* OrigAccessContextPin = FindPinChecked(TEXT("AccessContext"));
	UEdGraphPin* AccessContextSourcePin = OrigAccessContextPin->LinkedTo.Num() > 0 ? OrigAccessContextPin->LinkedTo[0] : nullptr;
	if (!AccessContextSourcePin)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoValidAccessContext", "Access Context input must be connected. - @@").ToString(), this);
		return;
	}

	//Inject a noop exec pin just so we have a single exec point to go into our other expansion pins.
	UK2Node_ExecutionSequence* NoopNode = CompilerContext.SpawnIntermediateNode<UK2Node_ExecutionSequence>(this, SourceGraph);
	NoopNode->AllocateDefaultPins();
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *NoopNode->GetExecPin());
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *NoopNode->GetThenPinGivenIndex(0));

	ExpandNodeOptionalPins(CompilerContext, SourceGraph, AccessContextSourcePin, NoopNode->GetThenPinGivenIndex(0), GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, GetSinglePropertyInNDCAccessContextInstance));

	//This node is fully expanded way so clear out any remaining links
	BreakAllNodeLinks();
}

FText UK2Node_DataChannelAccessContext_GetMembers::GetNodeTitleForContext(UScriptStruct* Struct)const
{
	return FText::Format(LOCTEXT("UK2Node_DataChannelAccessContext_GetMembers_TitleFmt", "Get Members in ({0}) Instance"), Struct->GetDisplayNameText());
}

FText UK2Node_DataChannelAccessContext_GetMembers::GetTooltipTextForContext(UScriptStruct* Struct)const
{
	return FText::Format(LOCTEXT("UK2Node_DataChannelAccessContext_GetMembers_TooltipFmt", "Gets the internal members of an Access Contecxt Instance with an internal type of {0}. "), Struct->GetDisplayNameText());
}

///

UK2Node_DataChannelAccessContext_SetMembers::UK2Node_DataChannelAccessContext_SetMembers(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, SetMembersInNDCAccessContextInstance), UNiagaraDataChannelLibrary::StaticClass());
}

void UK2Node_DataChannelAccessContext_SetMembers::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	if (ContextStruct)
	{
		FStructOnScope StructOnScope(ContextStruct);
		FK2Node_SetMembersInNDCAccessContext_OptionalPinManager OptionalPinManager(StructOnScope.GetStructMemory(), GetBlueprint());
		OptionalPinManager.RebuildPropertyList(ShowPinForProperties, ContextStruct);
		OptionalPinManager.CreateVisiblePins(ShowPinForProperties, ContextStruct, EGPD_Input, this);
	}
}

void UK2Node_DataChannelAccessContext_SetMembers::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!ContextStruct)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoValidAccessContextStruct", "Access Context Struct is null! - @@").ToString(), this);
		return;
	}

	UEdGraphPin* OrigAccessContextPin = FindPinChecked(TEXT("AccessContext"));
	UEdGraphPin* AccessContextSourcePin = OrigAccessContextPin->LinkedTo.Num() > 0 ? OrigAccessContextPin->LinkedTo[0] : nullptr;
	if (!AccessContextSourcePin)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoValidAccessContext", "Access Context input must be connected. - @@").ToString(), this);
		return;
	}

	//Inject a noop exec pin just so we have a single exec point to go into our other expansion pins.
	UK2Node_ExecutionSequence* NoopNode = CompilerContext.SpawnIntermediateNode<UK2Node_ExecutionSequence>(this, SourceGraph);
	NoopNode->AllocateDefaultPins();
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *NoopNode->GetExecPin());
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *NoopNode->GetThenPinGivenIndex(0));

	ExpandNodeOptionalPins(CompilerContext, SourceGraph, AccessContextSourcePin, NoopNode->GetThenPinGivenIndex(0), GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, SetSinglePropertyInNDCAccessContextInstance));

	//This node is fully expanded way so clear out any remaining links
	BreakAllNodeLinks();
}

FText UK2Node_DataChannelAccessContext_SetMembers::GetNodeTitleForContext(UScriptStruct* Struct)const
{
	return FText::Format(LOCTEXT("UK2Node_DataChannelAccessContext_SetMembers_TitleFmt", "Set Members in ({0}) Instance"), Struct? Struct->GetDisplayNameText() : FText::GetEmpty());
}

FText UK2Node_DataChannelAccessContext_SetMembers::GetTooltipTextForContext(UScriptStruct* Struct)const
{
	return FText::Format(LOCTEXT("UK2Node_DataChannelAccessContext_SetMembers_TooltipFmt", "Sets the internal members of an Access Contecxt Instance with an internal type of {0}. "), Struct ? Struct->GetDisplayNameText() : FText::GetEmpty());
}

//////////////////////////////////////////////////////////////////////////

UK2Node_DataChannelAccessContext_Prepare::UK2Node_DataChannelAccessContext_Prepare(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, PrepareAccessContextFromNDCRef), UNiagaraDataChannelLibrary::StaticClass());
}

void UK2Node_DataChannelAccessContext_Prepare::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	if (ContextStruct)
	{
		FStructOnScope StructOnScope(ContextStruct);
		FK2Node_SetMembersInNDCAccessContext_OptionalPinManager OptionalPinManager(StructOnScope.GetStructMemory(), GetBlueprint());
		OptionalPinManager.RebuildPropertyList(ShowPinForProperties, ContextStruct);
		OptionalPinManager.CreateVisiblePins(ShowPinForProperties, ContextStruct, EGPD_Input, this);
	}
}

void UK2Node_DataChannelAccessContext_Prepare::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!ContextStruct)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoValidAccessContextStruct", "Access Context Struct is null! - @@").ToString(), this);
		return;
	}

	UEdGraphPin* RefInputPin = FindPinChecked(TEXT("NDCRef"));
	UEdGraphPin* ResultPin = FindPinChecked(TEXT("ReturnValue"));

	//First get a usable access context from the input ref.
	UK2Node_CallFunction* GetAccessContextFuncCallNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetAccessContextFuncCallNode->SetFromFunction(UNiagaraDataChannelLibrary::StaticClass()->FindFunctionByName(TEXT("GetUsableAccessContextFromNDCRef")));
	GetAccessContextFuncCallNode->AllocateDefaultPins();
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *GetAccessContextFuncCallNode->GetExecPin());
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *GetAccessContextFuncCallNode->GetThenPin());

	UEdGraphPin* GetAccessContextFuncCallNode_RefInputPin = GetAccessContextFuncCallNode->FindPinChecked(TEXT("NDCRef"));
	UEdGraphPin* GetAccessContextFuncCallNode_ResultPin = GetAccessContextFuncCallNode->FindPinChecked(TEXT("ReturnValue"));

	CompilerContext.MovePinLinksToIntermediate(*RefInputPin, *GetAccessContextFuncCallNode_RefInputPin);
	CompilerContext.MovePinLinksToIntermediate(*ResultPin, *GetAccessContextFuncCallNode_ResultPin);

	ExpandNodeOptionalPins(CompilerContext, SourceGraph, GetAccessContextFuncCallNode_ResultPin, GetAccessContextFuncCallNode->GetThenPin(), GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, SetSinglePropertyInNDCAccessContextInstance));

	//This node is fully expanded way so clear out any remaining links
	BreakAllNodeLinks();
}

FText UK2Node_DataChannelAccessContext_Prepare::GetNodeTitleForContext(UScriptStruct* Struct)const
{
	return FText::Format(LOCTEXT("UK2Node_DataChannelAccessContext_Prepare_TitleFmt", "Prepare ({0}) Instance"), Struct ? Struct->GetDisplayNameText() : FText::GetEmpty());
}

FText UK2Node_DataChannelAccessContext_Prepare::GetTooltipTextForContext(UScriptStruct* Struct)const
{
	return FText::Format(LOCTEXT("UK2Node_DataChannelAccessContext_Prepare_TooltipFmt", "Prepares an NDC Access Contecxt Instance with an internal type of {0}. Allows setting of its members before use. By default exposes transient inputs that will typically be set before each NDC access. "), Struct ? Struct->GetDisplayNameText() : FText::GetEmpty());
}

#undef LOCTEXT_NAMESPACE
