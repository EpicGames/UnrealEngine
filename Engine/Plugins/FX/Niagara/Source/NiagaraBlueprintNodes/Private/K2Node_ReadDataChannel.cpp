// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_ReadDataChannel.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_IfThenElse.h"
#include "KismetCompiler.h"
#include "NiagaraBlueprintUtil.h"
#include "NiagaraDataChannelAccessor.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet/KismetSystemLibrary.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_CastByteToEnum.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_ReadDataChannel)

#define LOCTEXT_NAMESPACE "K2Node_ReadDataChannel"

UK2Node_ReadDataChannel::UK2Node_ReadDataChannel()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, ReadFromNiagaraDataChannelSingle), UNiagaraDataChannelLibrary::StaticClass());
}

void UK2Node_ReadDataChannel::AddNDCDerivedPins()
{
	Super::AddNDCDerivedPins();

	if (UNiagaraDataChannel* NDC = GetDataChannel())
	{
		for (const FNiagaraDataChannelVariable& InVar : NDC->GetVariables())
		{
			if (IgnoredVariables.Contains(InVar.Version))
			{
				continue;
			}
			UEdGraphPin* NewPin = CreatePin(EGPD_Output, FNiagaraBlueprintUtil::TypeDefinitionToBlueprintType(InVar.GetType()), InVar.GetName());

			NDCDerivedGeneratedPins.Add(NewPin);

#if WITH_EDITORONLY_DATA
			NewPin->PersistentGuid = InVar.Version;
#endif
		}
	}
}

void UK2Node_ReadDataChannel::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);
	
	check(!SupportsDynamicDataChannel());
	if (!HasValidDataChannel())
	{
		return;//Compile error generated in base.
	}

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	UEdGraphPin* SuccessPin = FindPinChecked(FName("Success"), EGPD_Output);
	UEdGraphPin* FailurePin = FindPinChecked(FName("Failure"), EGPD_Output);
	
	// create function call node to init the writer object 
	UK2Node_CallFunction* CreateReaderNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CreateReaderNode->SetFromFunction(UNiagaraDataChannelLibrary::StaticClass()->FindFunctionByName(GetReaderFunctionName()));
	CreateReaderNode->AllocateDefaultPins();

	// transfer the input pins over
	static TArray<TPair<FName, FName>> PinsToTransfer = { {"Channel", "Channel"}, {"SearchParams", "SearchParams"}, {"AccessContext", "AccessContext"}, {"bReadPreviousFrame", "bReadPreviousFrame"}};
	
	for (TPair<FName, FName> Pair : PinsToTransfer)
	{
		UEdGraphPin* OrgInputPin = FindPin(Pair.Key, EGPD_Input);
		UEdGraphPin* NewInputPin = CreateReaderNode->FindPin(Pair.Value, EGPD_Input);
		if(OrgInputPin && NewInputPin)
		{
			CompilerContext.MovePinLinksToIntermediate(*OrgInputPin, *NewInputPin);
		}
	}
	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *CreateReaderNode->GetExecPin());
	if (FindPin(FName("WorldContextObject"), EGPD_Input) && CreateReaderNode->FindPin(FName("WorldContextObject")))
	{
		CompilerContext.MovePinLinksToIntermediate(*FindPin(FName("WorldContextObject"), EGPD_Input), *CreateReaderNode->FindPin(FName("WorldContextObject"), EGPD_Input));
	}

	// add a validity check for the return value
	UK2Node_CallFunction* IsValidNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	IsValidNode->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, IsValid)));
	IsValidNode->AllocateDefaultPins();
	UK2Node_IfThenElse* IfElseNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	IfElseNode->AllocateDefaultPins();

	UEdGraphPin* IsValidInputPin = IsValidNode->FindPinChecked(FName("Object"), EGPD_Input);
	UEdGraphPin* ReaderResultPin = CreateReaderNode->GetReturnValuePin();
	Schema->TryCreateConnection(IsValidInputPin, ReaderResultPin);
	Schema->TryCreateConnection(CreateReaderNode->GetThenPin(), IfElseNode->GetExecPin());
	Schema->TryCreateConnection(IsValidNode->GetReturnValuePin(), IfElseNode->GetConditionPin());

	CompilerContext.CopyPinLinksToIntermediate(*FailurePin, *IfElseNode->GetElsePin());

	// create the read function nodes
	UEdGraphPin* LastIsValidPin = nullptr;
	UEdGraphPin* IndexPin = FindPinChecked(FName("Index"), EGPD_Input);
	for (const FNiagaraDataChannelVariable& InVar : GetDataChannel()->GetVariables())
	{
		if (IgnoredVariables.Contains(InVar.Version))
		{
			continue;
		}
		UEdGraphPin* VarOutputPin = GetVarPin(InVar.GetName());
		if (VarOutputPin == nullptr)
		{
			CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("NoOutPinFound", "Missing output pin for variable '{0}' - @@"), FText::FromName(InVar.GetName())).ToString(), this);
			continue;
		}
		
		UFunction* ReadFunc = GetReadFunctionForType(InVar.GetType());
		if (ReadFunc == nullptr)
		{
			CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("NoReadFuncFound", "Unable to find a read function for data channel variable '{0}' type {1}, looks like the type is not yet supported by UNiagaraDataChannelReader. (Source Pin @@)"), FText::FromName(InVar.GetName()), FText::FromString(InVar.GetType().GetName())).ToString(), VarOutputPin);
			continue;
		}
		
		UK2Node_CallFunction* ReadDataNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		ReadDataNode->SetFromFunction(ReadFunc);
		ReadDataNode->AllocateDefaultPins();

		// connect pins of the read function
		if (!Schema->TryCreateConnection(ReaderResultPin, ReadDataNode->FindPinChecked(UEdGraphSchema_K2::PN_Self)))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("NoReaderConnection", "Unable to connect reader object result (UNiagaraDataChannelReader) to read function pins. @@").ToString(), this);
			continue;
		}
		ReadDataNode->FindPinChecked(FName("VarName"), EGPD_Input)->DefaultValue = InVar.GetName().ToString();
		CompilerContext.CopyPinLinksToIntermediate(*IndexPin, *ReadDataNode->FindPinChecked(FName("Index"), EGPD_Input));
		if (InVar.GetType().IsEnum())
		{
			// the read function only returns a byte, so we need to convert it to the actual enum pin
			UK2Node_CastByteToEnum* CastEnumNode = CompilerContext.SpawnIntermediateNode<UK2Node_CastByteToEnum>(this, SourceGraph);
			CastEnumNode->bSafe = true; 
			CastEnumNode->Enum = InVar.GetType().GetEnum(); 
			CastEnumNode->AllocateDefaultPins();
			Schema->TryCreateConnection(ReadDataNode->GetReturnValuePin(), CastEnumNode->FindPinChecked(FName("Byte"), EGPD_Input));
			CompilerContext.MovePinLinksToIntermediate(*VarOutputPin, *CastEnumNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue));
		}
		else
		{
			CompilerContext.MovePinLinksToIntermediate(*VarOutputPin, *ReadDataNode->GetReturnValuePin());
		}

		// do a boolean AND of all the IsValid return values
		UEdGraphPin* ReadValidPin = ReadDataNode->FindPinChecked(FName("IsValid"), EGPD_Output);
		if (LastIsValidPin)
		{
			UK2Node_CallFunction* BoolAndNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
			BoolAndNode->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, BooleanAND)));
			BoolAndNode->AllocateDefaultPins();
			Schema->TryCreateConnection(BoolAndNode->FindPinChecked(FName("A"), EGPD_Input), ReadValidPin);
			Schema->TryCreateConnection(BoolAndNode->FindPinChecked(FName("B"), EGPD_Input), LastIsValidPin);

			LastIsValidPin = BoolAndNode->GetReturnValuePin();
		}
		else
		{
			LastIsValidPin = ReadValidPin; 
		}
	}

	// add a last branch based on all the read results
	UK2Node_IfThenElse* IsValidBranchNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	IsValidBranchNode->AllocateDefaultPins();
	Schema->TryCreateConnection(IfElseNode->GetThenPin(), IsValidBranchNode->GetExecPin());
	
	if(LastIsValidPin)
	{
		Schema->TryCreateConnection(LastIsValidPin, IsValidBranchNode->GetConditionPin());
	}

	// connect the last exec pins
	CompilerContext.MovePinLinksToIntermediate(*FailurePin, *IsValidBranchNode->GetElsePin());
	CompilerContext.MovePinLinksToIntermediate(*SuccessPin, *IsValidBranchNode->GetThenPin());
}

UFunction* UK2Node_ReadDataChannel::GetReadFunctionForType(const FNiagaraTypeDefinition& TypeDef)
{
	if (TypeDef == FNiagaraTypeHelper::GetDoubleDef() || TypeDef == FNiagaraTypeDefinition::GetFloatDef() || TypeDef == FNiagaraTypeDefinition::GetHalfDef())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadFloat));
	}
	if (TypeDef == FNiagaraTypeHelper::GetVector2DDef() || TypeDef == FNiagaraTypeDefinition::GetVec2Def())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadVector2D));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetPositionDef())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadPosition));
	}
	if (TypeDef == FNiagaraTypeHelper::GetVectorDef() || TypeDef == FNiagaraTypeDefinition::GetVec3Def())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadVector));
	}
	if (TypeDef == FNiagaraTypeHelper::GetVector4Def() || TypeDef == FNiagaraTypeDefinition::GetVec4Def())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadVector4));
	}
	if (TypeDef == FNiagaraTypeHelper::GetQuatDef() || TypeDef == FNiagaraTypeDefinition::GetQuatDef())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadQuat));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetColorDef())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadLinearColor));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetIntDef())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadInt));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetBoolDef())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadBool));
	}
	if (TypeDef.GetStruct() == FNiagaraSpawnInfo::StaticStruct())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadSpawnInfo));
	}
	if (TypeDef.GetStruct() == FNiagaraID::StaticStruct())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadID));
	}
	if (TypeDef.GetEnum())
	{
		return UNiagaraDataChannelReader::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelReader, ReadEnum));
	}
	return nullptr;
}

UEdGraphPin* UK2Node_ReadDataChannel::GetVarPin(const FName& Name) const
{
	FEdGraphPinType ExecType = GetExecPin()->PinType;
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinType == ExecType)
		{
			continue;
		}
		if ((EGPD_Output == Pin->Direction) && Pin->PinName == Name)
		{
			return Pin;
		}
	}
	checkf(false, TEXT("Unable to find pin for variable %s"), *Name.ToString());
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
