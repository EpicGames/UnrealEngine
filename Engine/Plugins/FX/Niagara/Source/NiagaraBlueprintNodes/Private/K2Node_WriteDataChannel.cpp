// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_WriteDataChannel.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_Self.h"
#include "KismetCompiler.h"
#include "NiagaraBlueprintUtil.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelAccessor.h"
#include "Kismet/KismetSystemLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_WriteDataChannel)

#define LOCTEXT_NAMESPACE "K2Node_WriteDataChannel"

UK2Node_WriteDataChannel::UK2Node_WriteDataChannel()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelLibrary, WriteToNiagaraDataChannelSingle), UNiagaraDataChannelLibrary::StaticClass());
}

void UK2Node_WriteDataChannel::AddNDCDerivedPins()
{
	Super::AddNDCDerivedPins();

	if (UNiagaraDataChannel* NDC = GetDataChannel())
	{
		for (const FNiagaraDataChannelVariable& InVar : NDC->GetVariables())
		{
			if (IgnoredVariables.Contains(InVar.Version))
			{
				continue;
			}
			UEdGraphPin* NewPin = CreatePin(EGPD_Input, FNiagaraBlueprintUtil::TypeDefinitionToBlueprintType(InVar.GetType()), InVar.GetName());

			NDCDerivedGeneratedPins.Add(NewPin);

#if WITH_EDITORONLY_DATA
			NewPin->PersistentGuid = InVar.Version;
#endif
		}
	}
}

void UK2Node_WriteDataChannel::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);
	
	check(!SupportsDynamicDataChannel());
	if (!HasValidDataChannel())
	{
		return;//Compile error generated in base.
	}

	// create function call node to init the writer object 
	UK2Node_CallFunction* CreateWriterNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	if(CreateWriterNode == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("FailedToCreateWriterNode", "Failed to create NDC write node. - @@").ToString(), this);
		return;
	}

	CreateWriterNode->SetFromFunction(UNiagaraDataChannelLibrary::StaticClass()->FindFunctionByName(GetWriterFunctionName()));
	CreateWriterNode->AllocateDefaultPins();

#if WITH_NIAGARA_DEBUGGER
	// add path info for current BP for debug purposes 
	UK2Node_CallFunction* GetPathNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetPathNode->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, GetPathName)));
	GetPathNode->AllocateDefaultPins();
	UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
	SelfNode->AllocateDefaultPins();

	CompilerContext.GetSchema()->TryCreateConnection(SelfNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), GetPathNode->FindPinChecked(FName("Object")));
	CompilerContext.GetSchema()->TryCreateConnection(CreateWriterNode->FindPinChecked(FName("DebugSource")), GetPathNode->GetReturnValuePin());
#endif

	// transfer the input pins over
	static TArray<TPair<FName, FName>> PinsToTransfer = { {"Channel", "Channel"}, {"SearchParams", "SearchParams"}, {"AccessContext", "AccessContext"}, {"bVisibleToBlueprint", "bVisibleToGame"},
		{"bVisibleToNiagaraCPU", "bVisibleToCPU"}, {"bVisibleToNiagaraGPU", "bVisibleToGPU"}};
	
	for (TPair<FName, FName> Pair : PinsToTransfer)
	{
		UEdGraphPin* OrgInputPin = FindPin(Pair.Key, EGPD_Input);
		UEdGraphPin* NewInputPin = CreateWriterNode->FindPin(Pair.Value, EGPD_Input);
		if(OrgInputPin && NewInputPin)
		{	
			CompilerContext.MovePinLinksToIntermediate(*OrgInputPin, *NewInputPin);
		}
	}
	CreateWriterNode->FindPinChecked(FName("Count"))->DefaultValue = FString::FromInt(1);
	if (FindPin(FName("WorldContextObject"), EGPD_Input) && CreateWriterNode->FindPin(FName("WorldContextObject")))
	{
		CompilerContext.MovePinLinksToIntermediate(*FindPin(FName("WorldContextObject"), EGPD_Input), *CreateWriterNode->FindPin(FName("WorldContextObject"), EGPD_Input));
	}
	
	UEdGraphPin* OldExecPin = GetExecPin();
	UEdGraphPin* NewExecPin = CreateWriterNode->GetExecPin();
	CompilerContext.MovePinLinksToIntermediate(*OldExecPin, *NewExecPin);

	// create the write function nodes
	UEdGraphPin* LastExecPin = CreateWriterNode->GetThenPin();
	UEdGraphPin* WriterResultPin = CreateWriterNode->GetReturnValuePin();
	for (const FNiagaraDataChannelVariable& InVar : GetDataChannel()->GetVariables())
	{
		if (IgnoredVariables.Contains(InVar.Version))
		{
			continue;
		}
		UEdGraphPin* VarInputPin = FindPinChecked(InVar.GetName(), EGPD_Input);
		if (VarInputPin == nullptr)
		{
			CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("NoInputPinFound", "Missing input pin for variable '{0}' - @@"), FText::FromName(InVar.GetName())).ToString(), this);
			continue;
		}
		
		UFunction* WriteFunc = GetWriteFunctionForType(InVar.GetType());
		if (WriteFunc == nullptr)
		{
			CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("NoWriteFuncFound", "Unable to find a write function for data channel variable '{0}' type {1}, looks like the type is not yet supported by UNiagaraDataChannelWriter. (Source Pin @@)"), FText::FromName(InVar.GetName()), FText::FromString(InVar.GetType().GetName())).ToString(), VarInputPin);
			continue;
		}
		
		UK2Node_CallFunction* WriteDataNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		WriteDataNode->SetFromFunction(WriteFunc);
		WriteDataNode->AllocateDefaultPins();

		// connect input pins of the write function
		if (!CompilerContext.GetSchema()->TryCreateConnection(WriterResultPin, WriteDataNode->FindPinChecked(UEdGraphSchema_K2::PN_Self)))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("NoWriterConnection", "Unable to connect writer object result (UNiagaraDataChannelWriter) to write function pins. @@").ToString(), this);
			continue;
		}
		WriteDataNode->FindPinChecked(FName("VarName"), EGPD_Input)->DefaultValue = InVar.GetName().ToString();

		if (InVar.GetType().IsEnum())
		{
			// bit of a dirty hack to just change the pin type of the function call node, but since the underlying type is the same it should be fine... 
			WriteDataNode->FindPinChecked(FName("InData"), EGPD_Input)->PinType = VarInputPin->PinType;
		}
		CompilerContext.MovePinLinksToIntermediate(*VarInputPin, *WriteDataNode->FindPinChecked(FName("InData"), EGPD_Input));

		// connect exec pins
		CompilerContext.GetSchema()->TryCreateConnection(LastExecPin, WriteDataNode->GetExecPin());
		LastExecPin = WriteDataNode->GetThenPin();
	}

	// connect the last exec pin
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *LastExecPin);
}

UFunction* UK2Node_WriteDataChannel::GetWriteFunctionForType(const FNiagaraTypeDefinition& TypeDef)
{
	if (TypeDef == FNiagaraTypeHelper::GetDoubleDef() || TypeDef == FNiagaraTypeDefinition::GetFloatDef() || TypeDef == FNiagaraTypeDefinition::GetHalfDef())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteFloat));
	}
	if (TypeDef == FNiagaraTypeHelper::GetVector2DDef() || TypeDef == FNiagaraTypeDefinition::GetVec2Def())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteVector2D));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetPositionDef())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WritePosition));
	}
	if (TypeDef == FNiagaraTypeHelper::GetVectorDef() || TypeDef == FNiagaraTypeDefinition::GetVec3Def())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteVector));
	}
	if (TypeDef == FNiagaraTypeHelper::GetVector4Def() || TypeDef == FNiagaraTypeDefinition::GetVec4Def())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteVector4));
	}
	if (TypeDef == FNiagaraTypeHelper::GetQuatDef() || TypeDef == FNiagaraTypeDefinition::GetQuatDef())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteQuat));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetColorDef())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteLinearColor));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetIntDef())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteInt));
	}
	if (TypeDef == FNiagaraTypeDefinition::GetBoolDef())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteBool));
	}
	if (TypeDef.GetStruct() == FNiagaraSpawnInfo::StaticStruct())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteSpawnInfo));
	}
	if (TypeDef.GetStruct() == FNiagaraID::StaticStruct())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteID));
	}
	if (TypeDef.GetEnum())
	{
		return UNiagaraDataChannelWriter::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UNiagaraDataChannelWriter, WriteEnum));
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
