// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBlueprintNodesDetails.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"

#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include "K2Node_DataChannel_WithContext.h"

#include "SInstancedStructPicker.h"

#define LOCTEXT_NAMESPACE "NDCAccessContextOperationNodeDetailsDetails"

/////////////////////////////////////////////////////
// FNDCAccessContextOperationNodeDetailsDetails - Based on FSkeletalControlNodeDetails

TSharedRef<IDetailCustomization> FNDCAccessContextOperationNodeDetailsDetails::MakeInstance()
{
	return MakeShareable(new FNDCAccessContextOperationNodeDetailsDetails());
}

void FNDCAccessContextOperationNodeDetailsDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> ContextStructProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UK2Node_DataChannelAccessContextOperation,ContextStruct));
	DetailBuilder.HideProperty(ContextStructProperty);
	
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Access Context");
		
	Category.AddCustomRow(ContextStructProperty->GetPropertyDisplayName())
	.NameContent()
	[
		ContextStructProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SInstancedStructPicker, ContextStructProperty, DetailBuilder.GetPropertyUtilities())
	];

	//Refresh details when the struct is changed.
	ContextStructProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(
	[&]()
	{
		DetailBuilder.ForceRefreshDetails();
	}));

	DetailCategory = &DetailBuilder.EditCategory("PinOptions");
	TSharedRef<IPropertyHandle> AvailablePins = DetailBuilder.GetProperty("ShowPinForProperties");
	ArrayProperty = AvailablePins->AsArray();

	bool bShowAvailablePins = true;
	bool bHideInputPins = false;
	HideUnconnectedPinsNode = nullptr;

	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailBuilder.GetSelectedObjects();
	if (SelectedObjects.Num() == 1)
	{
		UObject* CurObj = SelectedObjects[0].Get();
		HideUnconnectedPinsNode = Cast<UK2Node_DataChannelAccessContextOperation>(CurObj);
		bHideInputPins = CurObj && (CurObj->IsA<UK2Node_DataChannelAccessContext_Make>() || CurObj->IsA<UK2Node_DataChannelAccessContext_SetMembers>());
	}

	if (bShowAvailablePins)
	{
		TSet<FName> UniqueCategoryNames;
		{
			int32 NumElements = 0;
			{
				uint32 UnNumElements = 0;
				if (ArrayProperty.IsValid() && (FPropertyAccess::Success == ArrayProperty->GetNumElements(UnNumElements)))
				{
					NumElements = static_cast<int32>(UnNumElements);
				}
			}
			for (int32 Index = 0; Index < NumElements; ++Index)
			{
				TSharedRef<IPropertyHandle> StructPropHandle = ArrayProperty->GetElement(Index);
				TSharedPtr<IPropertyHandle> CategoryPropHandle = StructPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, CategoryName));
				check(CategoryPropHandle.IsValid());
				FName CategoryNameValue;
				const FPropertyAccess::Result Result = CategoryPropHandle->GetValue(CategoryNameValue);
				if (ensure(FPropertyAccess::Success == Result))
				{
					UniqueCategoryNames.Add(CategoryNameValue);
				}
			}
		}

		//@TODO: Shouldn't show this if the available pins array is empty!
		const bool bGenerateHeader = true;
		const bool bDisplayResetToDefault = false;
		const bool bDisplayElementNum = false;
		const bool bForAdvanced = false;
		for (const FName& CategoryName : UniqueCategoryNames)
		{
			//@TODO: Pay attention to category filtering here


			TSharedRef<FDetailArrayBuilder> AvailablePinsBuilder = MakeShareable(new FDetailArrayBuilder(AvailablePins, bGenerateHeader, bDisplayResetToDefault, bDisplayElementNum));
			AvailablePinsBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FNDCAccessContextOperationNodeDetailsDetails::OnGenerateElementForPropertyPin, CategoryName));
			AvailablePinsBuilder->SetDisplayName((CategoryName == NAME_None) ? LOCTEXT("DefaultCategory", "Default Category") : FText::FromName(CategoryName));
			DetailCategory->AddCustomBuilder(AvailablePinsBuilder, bForAdvanced);
		}

		// Add the action buttons
		if (HideUnconnectedPinsNode.IsValid())
		{
			FDetailWidgetRow& GroupActionsRow = DetailCategory->AddCustomRow(LOCTEXT("GroupActionsSearchText", "Split Sort"))
				.ValueContent()
				.HAlign(HAlign_Left)
				.MaxDesiredWidth(250.f)
				[
					SNew(SButton)
						.OnClicked(this, &FNDCAccessContextOperationNodeDetailsDetails::HideAllUnconnectedPins, bHideInputPins)
						.ToolTipText(LOCTEXT("HideAllUnconnectedPinsTooltip", "All unconnected pins get hidden (removed from the graph node)"))
						[
							SNew(STextBlock)
								.Text(LOCTEXT("HideAllUnconnectedPins", "Hide Unconnected Pins"))
								.Font(DetailBuilder.GetDetailFont())
						]
				];
		}
	}
	else
	{
		DetailBuilder.HideProperty(AvailablePins);
	}

	
}

ECheckBoxState FNDCAccessContextOperationNodeDetailsDetails::GetShowPinValueForProperty(TSharedRef<IPropertyHandle> InElementProperty) const
{
	bool Value;
	InElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))->GetValue(Value);
	return Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FNDCAccessContextOperationNodeDetailsDetails::OnShowPinChanged(ECheckBoxState InNewState, TSharedRef<IPropertyHandle> InElementProperty)
{
	InElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))->SetValue(InNewState ==  ECheckBoxState::Checked);
}

void FNDCAccessContextOperationNodeDetailsDetails::OnGenerateElementForPropertyPin(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder, FName CategoryName)
{
	{
		TSharedPtr<IPropertyHandle> CategoryPropHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, CategoryName));
		check(CategoryPropHandle.IsValid());
		FName CategoryNameValue;
		const FPropertyAccess::Result Result = CategoryPropHandle->GetValue(CategoryNameValue);
		const bool bProperCategory = ensure(FPropertyAccess::Success == Result) && (CategoryNameValue == CategoryName);

		if (!bProperCategory)
		{
			return;
		}
	}

	FString FilterString = CategoryName.ToString();

	FText PropertyFriendlyName(LOCTEXT("Invalid", "Invalid"));
	{
		TSharedPtr<IPropertyHandle> PropertyFriendlyNameHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, PropertyFriendlyName));
		if (PropertyFriendlyNameHandle.IsValid())
		{
			FString DisplayFriendlyName;
			switch (PropertyFriendlyNameHandle->GetValue(/*out*/ DisplayFriendlyName))
			{
			case FPropertyAccess::Success:
				FilterString += TEXT(" ") + DisplayFriendlyName;
				PropertyFriendlyName = FText::FromString(DisplayFriendlyName);
				break;
			case FPropertyAccess::MultipleValues:
				ChildrenBuilder.AddCustomRow(FText::GetEmpty())
					[
						SNew(STextBlock).Text(LOCTEXT("OnlyWorksInSingleSelectMode", "Multiple types selected"))
					];
				return;
			case FPropertyAccess::Fail:
			default:
				check(false);
				break;
			}
		}
	}

	{
		TSharedPtr<IPropertyHandle> PropertyNameHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, PropertyName));
		if (PropertyNameHandle.IsValid())
		{
			FString RawName;
			if (PropertyNameHandle->GetValue(/*out*/ RawName) == FPropertyAccess::Success)
			{
				FilterString += TEXT(" ") + RawName;
			}
		}
	}

	FText PinTooltip;
	{
		TSharedPtr<IPropertyHandle> PropertyTooltipHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, PropertyTooltip));
		if (PropertyTooltipHandle.IsValid())
		{
			FString PinTooltipString;
			if (PropertyTooltipHandle->GetValue(/*out*/ PinTooltip) == FPropertyAccess::Success)
			{
				FilterString += TEXT(" ") + PinTooltip.ToString();
			}
		}
	}

	TSharedPtr<IPropertyHandle> HasOverrideValueHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bHasOverridePin));
	bool bHasOverrideValue;
	HasOverrideValueHandle->GetValue(/*out*/ bHasOverrideValue);
	FText OverrideCheckBoxTooltip;

	// Setup a tooltip based on whether the property has an override value or not.
	if (bHasOverrideValue)
	{
		OverrideCheckBoxTooltip = LOCTEXT("HasOverridePin", "Enabling this pin will make it visible for setting on the node and automatically enable the value for override when using the struct. Any updates to the resulting struct will require the value be set again or the override will be automatically disabled.");
	}
	else
	{
		OverrideCheckBoxTooltip = LOCTEXT("HasNoOverridePin", "Enabling this pin will make it visible for setting on the node.");
	}

	ChildrenBuilder.AddCustomRow(PropertyFriendlyName)
		.FilterString(FText::AsCultureInvariant(FilterString))
		.NameContent()
		[
			ElementProperty->CreatePropertyNameWidget(PropertyFriendlyName, PinTooltip)
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
				.ToolTipText(OverrideCheckBoxTooltip)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
						.IsChecked(this, &FNDCAccessContextOperationNodeDetailsDetails::GetShowPinValueForProperty, ElementProperty)
						.OnCheckStateChanged(this, &FNDCAccessContextOperationNodeDetailsDetails::OnShowPinChanged, ElementProperty)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("AsPin", " (As pin)"))
						.Font(ChildrenBuilder.GetParentCategory().GetParentLayout().GetDetailFont())
				]
		];
}

FReply FNDCAccessContextOperationNodeDetailsDetails::HideAllUnconnectedPins(const bool bHideInputPins)
{
	if (ArrayProperty.IsValid() && HideUnconnectedPinsNode.IsValid())
	{
		uint32 NumChildren = 0;
		ArrayProperty->GetNumElements(NumChildren);

		FScopedTransaction Transaction(LOCTEXT("HideUnconnectedPinsTransaction", "Hide Unconnected Pins"));

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ElementHandle = ArrayProperty->GetElement(ChildIndex);

			TSharedPtr<IPropertyHandle> PropertyNameHandle = ElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, PropertyName));
			FName ActualPropertyName;
			if (PropertyNameHandle.IsValid() && PropertyNameHandle->GetValue(ActualPropertyName) == FPropertyAccess::Success)
			{
				const UEdGraphPin* Pin = HideUnconnectedPinsNode->FindPin(ActualPropertyName.ToString(), bHideInputPins ? EGPD_Input : EGPD_Output);
				if (Pin && Pin->LinkedTo.Num() <= 0)
				{
					OnShowPinChanged(ECheckBoxState::Unchecked, ElementHandle);
				}
			}
		}
	}

	return FReply::Handled();
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
