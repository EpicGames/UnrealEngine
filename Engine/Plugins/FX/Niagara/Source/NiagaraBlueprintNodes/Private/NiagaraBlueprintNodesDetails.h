// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IPropertyHandle;

enum class ECheckBoxState : uint8;

/////////////////////////////////////////////////////
// FNDCAccessContextOperationNodeDetailsDetails - Customizes instances of UK2Node_DataChannelAccessContextOperation 

class FNDCAccessContextOperationNodeDetailsDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End of IDetailCustomization interface

protected:
	void OnGenerateElementForPropertyPin(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder, FName CategoryName);

	/** Returns the current value of the checkbox being displayed for the bShowPin property */
	ECheckBoxState GetShowPinValueForProperty(TSharedRef<IPropertyHandle> InElementProperty) const;

	/** Helper function for changing the value of the bShowPin checkbox to update the property */
	void OnShowPinChanged(ECheckBoxState InNewState, TSharedRef<IPropertyHandle> InElementProperty);

	/** Handler to hide all unconnected pins on a BreakStruct node */
	FReply HideAllUnconnectedPins(const bool bHideInputPins);

private:
	IDetailCategoryBuilder* DetailCategory;
	TWeakObjectPtr<class UK2Node_DataChannelAccessContextOperation> HideUnconnectedPinsNode;
	TSharedPtr<class IPropertyHandleArray> ArrayProperty;
};
