// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "BlueprintActionDatabase.h"
#include "BlueprintEditorLibrary.h"

#include "K2Node_DataChannel_WithContext.h"

#include "NiagaraBlueprintNodesDetails.h"

#include "PropertyEditorModule.h"

class FNiagaraBlueprintNodesModule : public IModuleInterface
{
public:

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:

	/** Callback for changes to other modules. Allows us to refresh certain things to reflect new types or changes etc. */
	void OnModulesChanged(FName Module, EModuleChangeReason Reason);

	FDelegateHandle OnModulesChangedHandle;
};


void FNiagaraBlueprintNodesModule::StartupModule()
{
	OnModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FNiagaraBlueprintNodesModule::OnModulesChanged);

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

 	PropertyModule.RegisterCustomClassLayout(
 		UK2Node_DataChannelAccessContextOperation::StaticClass()->GetFName(),
 		FOnGetDetailCustomizationInstance::CreateStatic(&FNDCAccessContextOperationNodeDetailsDetails::MakeInstance));
}

void FNiagaraBlueprintNodesModule::ShutdownModule()
{
	FModuleManager::Get().OnModulesChanged().Remove(OnModulesChangedHandle);
}

void FNiagaraBlueprintNodesModule::OnModulesChanged(FName Module, EModuleChangeReason Reason)
{
}


IMPLEMENT_MODULE(FNiagaraBlueprintNodesModule, NiagaraBlueprintNodes);