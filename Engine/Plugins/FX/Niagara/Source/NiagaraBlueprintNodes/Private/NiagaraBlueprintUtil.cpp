// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBlueprintUtil.h"

#include "EdGraphSchema_K2.h"


FEdGraphPinType FNiagaraBlueprintUtil::TypeDefinitionToBlueprintType(const FNiagaraTypeDefinition& TypeDef)
{
	FName Category;
	FName SubCategory;
	UObject* SubCategoryObject = nullptr;
	
	if (TypeDef == FNiagaraTypeHelper::GetDoubleDef() || TypeDef == FNiagaraTypeDefinition::GetFloatDef() || TypeDef == FNiagaraTypeDefinition::GetHalfDef())
	{
		Category = UEdGraphSchema_K2::PC_Real;
		SubCategory = UEdGraphSchema_K2::PC_Double;
	}
	else if (TypeDef == FNiagaraTypeDefinition::GetIntDef())
	{
		Category = UEdGraphSchema_K2::PC_Int;
	}
	else if (TypeDef == FNiagaraTypeDefinition::GetBoolDef())
	{
		Category = UEdGraphSchema_K2::PC_Boolean;
	}
	else if (TypeDef == FNiagaraTypeDefinition::GetVec3Def() || TypeDef == FNiagaraTypeDefinition::GetPositionDef())
	{
		Category = UEdGraphSchema_K2::PC_Struct;
		SubCategoryObject = FNiagaraTypeHelper::GetVectorDef().GetStruct();
	}
	else if (TypeDef == FNiagaraTypeDefinition::GetVec2Def())
	{
		Category = UEdGraphSchema_K2::PC_Struct;
		SubCategoryObject = FNiagaraTypeHelper::GetVector2DDef().GetStruct();
	}
	else if (TypeDef == FNiagaraTypeDefinition::GetVec4Def())
	{
		Category = UEdGraphSchema_K2::PC_Struct;
		SubCategoryObject = FNiagaraTypeHelper::GetVector4Def().GetStruct();
	}
	else if (TypeDef == FNiagaraTypeDefinition::GetQuatDef())
	{
		Category = UEdGraphSchema_K2::PC_Struct;
		SubCategoryObject = FNiagaraTypeHelper::GetQuatDef().GetStruct();
	}
	//TODO: add matrix def when supported
	/*else if (TypeDef == FNiagaraTypeDefinition::GetMatrix4Def())
	{
		static UScriptStruct* MatrixStruct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Matrix"));
		Category = UEdGraphSchema_K2::PC_Struct;
		SubCategoryObject = MatrixStruct;
	}*/
	else if (TypeDef == FNiagaraTypeDefinition::GetColorDef())
	{
		Category = UEdGraphSchema_K2::PC_Struct;
		SubCategoryObject = FNiagaraTypeDefinition::GetColorStruct();
	}
	else if (TypeDef.GetEnum())
	{
		Category = UEdGraphSchema_K2::PC_Byte;
		SubCategoryObject = TypeDef.GetEnum();
	}
	else if (TypeDef.IsUObject())
	{
		Category = UEdGraphSchema_K2::PC_Object;
		SubCategoryObject = TypeDef.GetClass();
	}
	else if (TypeDef.GetStruct())
	{
		Category = UEdGraphSchema_K2::PC_Struct;
		SubCategoryObject = TypeDef.GetStruct();
	}
	else if (TypeDef.GetClass())
	{
		Category = UEdGraphSchema_K2::PC_Class;
		SubCategoryObject = TypeDef.GetClass();
	}
	return FEdGraphPinType(Category, SubCategory, SubCategoryObject, EPinContainerType::None, false, FEdGraphTerminalType());
}
