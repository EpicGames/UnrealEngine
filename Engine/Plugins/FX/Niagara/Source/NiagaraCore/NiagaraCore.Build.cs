// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NiagaraCore : ModuleRules
{
    public NiagaraCore(ReadOnlyTargetRules Target) : base(Target)
    {
	    CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	    
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
            }
        );

        PublicDependencyModuleNames.AddRange(
            new string[] {
				"CoreUObject",
                "VectorVM",
            }
        );

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"RenderCore",
			}
		);
	}
}
