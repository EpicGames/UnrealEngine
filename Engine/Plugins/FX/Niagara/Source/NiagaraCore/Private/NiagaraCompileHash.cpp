// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCompileHash.h"
#include "Misc/SecureHash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraCompileHash)

bool FNiagaraCompileHash::operator==(const FSHAHash& Other) const
{
	if (DataHash.Num() != HashSize)
	{
		return false;
	}
	return FMemory::Memcmp(DataHash.GetData(), Other.Hash, HashSize) == 0;
}

bool FNiagaraCompileHash::ToSHAHash(FSHAHash& OutHash) const
{
	if (DataHash.Num() == HashSize)
	{
		FMemory::Memcpy(OutHash.Hash, DataHash.GetData(), HashSize);
		return true;
	}
	return false;
}

bool FNiagaraCompileHash::IsValid() const
{
	return DataHash.Num() == HashSize;
}

uint32 FNiagaraCompileHash::GetTypeHash() const
{
	// Use the first 4 bytes of the data hash as the hash id for this object.
	return DataHash.Num() == HashSize 
		? *((uint32*)(DataHash.GetData())) 
		: 0;
}

const uint8* FNiagaraCompileHash::GetData() const
{
	return DataHash.GetData();
}

void FNiagaraCompileHash::AppendString(FString& Out) const
{
	if (DataHash.Num() == HashSize)
	{
		BytesToHexLower(DataHash.GetData(), HashSize, Out);
	}
	else
	{
		Out.Append(TEXTVIEW("Invalid"));
	}
}

FArchive& operator<<(FArchive& Ar, FNiagaraCompileHash& Id)
{
	return Ar << Id.DataHash;
}

