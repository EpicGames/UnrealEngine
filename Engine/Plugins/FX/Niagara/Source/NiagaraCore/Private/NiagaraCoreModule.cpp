// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCoreModule.h"
#include "Modules/ModuleManager.h"
#include "NiagaraDataInterfaceBase.h"

IMPLEMENT_MODULE(INiagaraCoreModule, NiagaraCore);

IMPLEMENT_TYPE_LAYOUT(FNiagaraDataInterfaceParametersCS);

IMPLEMENT_EXPORTED_INTRINSIC_TYPE_LAYOUT(TIndexedPtr<UNiagaraDataInterfaceBase>);

UNiagaraDataInterfaceBase::UNiagaraDataInterfaceBase(class FObjectInitializer const & Initializer)
{

}

//////////////////////////////////////////////////////////////////////////

//TODO: This does not need a custom serialize. It's only here as it's called by another custom serialize that itself is pointless
//TODO: Rip these out to get back to tagged property serialize.

bool FNiagaraVariableCommonReference::Serialize(FArchive& Ar)
{
	Ar << Name;
	Ar << UnderlyingType;
	return true;
}
