// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCustomVersion.h"
#include "UObject/DevObjectVersion.h"

const FGuid FNiagaraCustomVersion::GUID(0xFCF57AFA, 0x50764283, 0xB9A9E658, 0xFFA02D32);

// Register the custom version with core
FCustomVersionRegistration GRegisterNiagaraCustomVersion(FNiagaraCustomVersion::GUID, FNiagaraCustomVersion::LatestVersion, TEXT("NiagaraVer"));

FGuid FNiagaraCustomVersion::GetLatestScriptCompileVersion()
{
	return FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().Niagara_LatestScriptCompileVersion);
}
