// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMergeable.h"
#include "UObject/PropertyPortFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraMergeable)

UNiagaraMergeable::UNiagaraMergeable()
#if WITH_EDITORONLY_DATA
	: MergeId(FGuid::NewGuid())
#endif
{
}

#if WITH_EDITOR

bool UNiagaraMergeable::Equals(const UNiagaraMergeable* Other)
{
	if (Other == nullptr || this->GetClass() != Other->GetClass())
	{
		return false;
	}

	for (TFieldIterator<FProperty> PropertyIterator(this->GetClass()); PropertyIterator; ++PropertyIterator)
	{
		if (PropertyIterator->HasAnyPropertyFlags(CPF_Transient))
		{
			// Transient properties should not be used to determine merge equality.
			continue;
		}
		if (PropertyIterator->Identical(
			PropertyIterator->ContainerPtrToValuePtr<void>(this),
			PropertyIterator->ContainerPtrToValuePtr<void>(Other), PPF_DeepComparison) == false)
		{
			return false;
		}
	}

	return true;
}

FGuid UNiagaraMergeable::GetMergeId() const
{
	return MergeId;
}

UNiagaraMergeable* UNiagaraMergeable::StaticDuplicateWithNewMergeIdInternal(UObject* InOuter) const
{
	UNiagaraMergeable* Duplicate = CastChecked<UNiagaraMergeable>(StaticDuplicateObject(this, InOuter));
	Duplicate->MergeId = FGuid::NewGuid();
	return Duplicate;
}
#endif
