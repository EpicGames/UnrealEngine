// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNotifyOnChanged.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNotifyOnChanged)

#if WITH_EDITOR

void UNiagaraNotifyOnChanged::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	OnChangedDelegate.Broadcast();
}

UNiagaraNotifyOnChanged::FOnChanged& UNiagaraNotifyOnChanged::OnChanged()
{
	return OnChangedDelegate;
}

#endif // WITH_EDITOR
