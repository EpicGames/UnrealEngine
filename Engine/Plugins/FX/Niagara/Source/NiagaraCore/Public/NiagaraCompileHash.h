// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "NiagaraCompileHash.generated.h"

USTRUCT()
struct FNiagaraCompileHash
{
	GENERATED_BODY()

	FNiagaraCompileHash()
	{
	}

	explicit FNiagaraCompileHash(const TArray<uint8>& InDataHash)
	{
		checkf(InDataHash.Num() == HashSize, TEXT("Invalid hash data."));
		DataHash = InDataHash;
	}

	explicit FNiagaraCompileHash(const uint8* InDataHash, uint32 InCount) : DataHash(InDataHash, InCount)
	{
		checkf(InCount == HashSize, TEXT("Invalid hash data."));
	}

	bool operator==(const FNiagaraCompileHash& Other) const { return DataHash == Other.DataHash; }
	bool operator!=(const FNiagaraCompileHash& Other) const { return DataHash != Other.DataHash; }
	NIAGARACORE_API bool operator==(const FSHAHash& Other) const;
	inline bool operator!=(const FSHAHash& Other) const { return !operator==(Other); }

	NIAGARACORE_API bool ToSHAHash(FSHAHash& OutHash) const;

	NIAGARACORE_API bool IsValid() const;

	NIAGARACORE_API uint32 GetTypeHash() const;

	NIAGARACORE_API const uint8* GetData() const;

	NIAGARACORE_API void AppendString(FString& Out) const;
	
	FString ToString() const
	{
		FString Out;
		AppendString(Out);
		return Out;
	}

	NIAGARACORE_API friend FArchive& operator<<(FArchive& Ar, FNiagaraCompileHash& Id);

	static constexpr uint32 HashSize = 20;

	friend bool operator<(const FNiagaraCompileHash& Lhs, const FNiagaraCompileHash& Rhs)
	{
		if (int32 NumDiff = Lhs.DataHash.Num() - Rhs.DataHash.Num())
		{
			return NumDiff < 0;
		}

		return FMemory::Memcmp(Lhs.DataHash.GetData(), Rhs.DataHash.GetData(), Lhs.DataHash.Num()) < 0;
	}

private:
	UPROPERTY()
	TArray<uint8> DataHash;
};

inline uint32 GetTypeHash(const FNiagaraCompileHash& Hash)
{
	return Hash.GetTypeHash();
}

inline bool operator==(const FSHAHash& Lhs, const FNiagaraCompileHash& Rhs)
{
	return Rhs.operator==(Lhs);
}
inline bool operator!=(const FSHAHash& Lhs, const FNiagaraCompileHash& Rhs)
{
	return !operator==(Lhs, Rhs);
}
