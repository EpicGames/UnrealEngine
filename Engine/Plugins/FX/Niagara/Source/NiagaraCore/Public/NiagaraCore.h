// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "NiagaraCore.generated.h"

typedef uint64 FNiagaraSystemInstanceID;

UENUM()
enum class ENiagaraParameterAccessLevel : uint8
{
	/**
	Parameter is not be visibile to Verse.
	*/
	Private,

	/**
	Parameter has unrestricted access, and can be modify anytime.
	Note: Modifications such are renaming, deleting, changing default values, could have an impact on existing projects and result in compilation / publishing failures.
	*/
	Public,
};

UENUM()
enum class ENiagaraIterationSource : uint8
{
	/** Iterate over all active particles. */
	Particles = 0,
	/** Iterate over all elements in the data interface. */
	DataInterface,
	/** Iterate over a user provided number of elements. */
	DirectSet,
};

/** A utility class allowing for references to FNiagaraVariableBase outside of the Niagara module. */
USTRUCT()
struct FNiagaraVariableCommonReference
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName Name;

	UPROPERTY()
	TObjectPtr<UObject> UnderlyingType;

	NIAGARACORE_API bool Serialize(FArchive& Ar);
	friend bool operator<<(FArchive& Ar, FNiagaraVariableCommonReference& VariableReference);

	bool operator==(const FNiagaraVariableCommonReference& Other)const
	{
		return Name == Other.Name && UnderlyingType == Other.UnderlyingType;
	}	
};

template<> struct TStructOpsTypeTraits<FNiagaraVariableCommonReference> : public TStructOpsTypeTraitsBase2<FNiagaraVariableCommonReference>
{
	enum
	{
		WithSerializer = true,
	};
};

inline bool operator<<(FArchive& Ar, FNiagaraVariableCommonReference& VariableReference)
{
	return VariableReference.Serialize(Ar);
}
