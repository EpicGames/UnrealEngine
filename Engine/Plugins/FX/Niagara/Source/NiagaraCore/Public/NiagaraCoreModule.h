// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
* Niagara core module interface
*/
class INiagaraCoreModule : public IModuleInterface
{
};


