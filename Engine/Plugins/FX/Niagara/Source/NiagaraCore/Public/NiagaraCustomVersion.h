// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

struct FGuid;

// Custom serialization version for all packages containing Niagara asset types
struct FNiagaraCustomVersion
{
	enum Type
	{
		// Before any version changes were made in niagara
		BeforeCustomVersionWasAdded = 0,

		// Reworked vm external function binding to be more robust.
		VMExternalFunctionBindingRework,

		// Making all Niagara files reference the version number, allowing post loading recompilation if necessary.
		PostLoadCompilationEnabled,

		// Moved some runtime cost from external functions into the binding step and used variadic templates to neaten that code greatly.
		VMExternalFunctionBindingReworkPartDeux,

		// Moved per instance data needed for certain data interfaces out to it's own struct.
		DataInterfacePerInstanceRework,

		// Added shader maps and corresponding infrastructure
		NiagaraShaderMaps,

		// Combined Spawn, Update, and Event scripts into one graph.
		UpdateSpawnEventGraphCombination,

		// Reworked data layout to store float and int data separately.
		DataSetLayoutRework,

		// Reworked scripts to support emitter & system scripts
		AddedEmitterAndSystemScripts,

		// Rework of script execution contexts to allow better reuse and reduce overhead of parameter handling. 
		ScriptExecutionContextRework,

		// Removed the Niagara variable ID's making hookup impossible until next compile
		RemovalOfNiagaraVariableIDs,
		
		// System and emitter script simulations.
		SystemEmitterScriptSimulations,

		// Adding integer random to VM. TODO: The vm really needs its own versioning system that will force a recompile when changes.
		IntegerRandom,

		// Added emitter spawn attributes
		AddedEmitterSpawnAttributes,

		// cooking of shader maps and corresponding infrastructure
		NiagaraShaderMapCooking,
		NiagaraShaderMapCooking2,		// don't serialize shader maps for system scripts
		// Added script rapid iteration variables, usually top-level module parameters...
		AddedScriptRapidIterationVariables,

		// Added type to data interface infos
		AddedTypeToDataInterfaceInfos,

		// Hooked up autogenerated default values for function call nodes.
		EnabledAutogeneratedDefaultValuesForFunctionCallNodes,

		// Now curve data interfaces have look-up tables on by default.
		CurveLUTNowOnByDefault,

		// Scripts now use a guid for identification instead of an index when there are more than one with the same usage.
		ScriptsNowUseAGuidForIdentificationInsteadOfAnIndex,

		NiagaraCombinedGPUSpawnUpdate,  // don't serialize shader maps for update scripts

		DontCompileGPUWhenNotNeeded,  // don't serialize shader maps for emitters that don't run on gpu.

		LifeCycleRework,

		NowSerializingReadWriteDataSets, // We weren't serializing event data sets previously.

		TranslatorClearOutBetweenEmitters, // Forcing the internal parameter map vars to be reset between emitter calls.

		AddSamplerDataInterfaceParams,	// added sampler shader params based on DI buffer descriptors

		GPUShadersForceRecompileNeeded, // Need to force the GPU shaders to recompile

		PlaybackRangeStoredOnSystem, // The playback range for the timeline is now stored in the system editor data.
		
		MovedToDerivedDataCache, // All cached values will auto-recompile.

		DataInterfacesNotAllocated, // Data interfaces are preallocated

		EmittersHaveGenericUniqueNames, //emitter scripts are built using "Emitter." instead of the full name.

		MovingTranslatorVersionToGuid, // no longer have compiler version enum value in this list, instead moved to a guid, which works better for the DDC

		AddingParamMapToDataSetBaseNode, // adding a parameter map in/out to the data set base node

		DataInterfaceComputeShaderParamRefactor, // refactor of CS parameters allowing regular params as well as buffers.

		CurveLUTRegen, // bumping version and forcing curves to regen their LUT on version change.

		AssignmentNodeUsesBeginDefaults, // Changing the graph generation for assignment nodes so that it uses a "Begin Defaults" node where appropriate.

		AssignmentNodeHasCorrectUsageBitmask, // Updating the usage flage bitmask for assignment nodes to match the part of the stack it's used in.

		EmitterLocalSpaceLiteralConstant, //Emitter local space is compiled into the hlsl as a literal constant to expose it to emitter scripts and allow for some better optimization of particle transforms.

		TextureDataInterfaceUsesCustomSerialize, // The cpu cache of the texture is now directly serialized instead of using array property serialization.

		TextureDataInterfaceSizeSerialize, // The texture data interface now streams size info

		SkelMeshInterfaceAPIImprovements, //API to skeletal mesh interface was improved but requires a recompile and some graph fixup.

		ImproveLoadTimeFixupOfOpAddPins, // Only do op add pin fixup on existing nodes which are before this version

		MoveCommonInputMetadataToProperties, // Moved commonly used input metadata out of the strin/string property metadata map to actual properties on the metadata struct.

		UseHashesToIdentifyCompileStateOfTopLevelScripts, // Move to using the traversed graph hash and the base script id for the FNiagaraVMExecutableDataId instead of the change id guid to prevent invalidating the DDC.

		MetaDataAndParametersUpdate, // Reworked how the metadata is stored in NiagaraGraph from storing a Map of FNiagaraVariableMetaData to storing a map of UNiagaraScriptVariable* to be used with the Details panel.

		MoveInheritanceDataFromTheEmitterHandleToTheEmitter, // Moved the emitter inheritance data from the emitter handle to the emitter to allow for chained emitter inheritance.
		
		AddLibraryAssetProperty, // Add property to all Niagara scripts indicating whether or not they belong to the library

		AddAdditionalDefinesProperty, // Addding additional defines to the GPU script
		
		RemoveGraphUsageCompileIds, // Remove the random compile id guids from the cached script usage and from the compile and script ids since the hashes serve the same purpose and are deterministic.
			
		AddRIAndDetailLevel, //Adding UseRapidIterationParams and DetailLevelMask to the GPU script

		ChangeEmitterCompiledDataToSharedRefs, // Changing the system and emitter compiled data to shared pointers to deal with lifetime issues in the editor.  They now are handled directly in system serialize.

		DisableSortingByDefault, // Sorting on Renderers is disabled by default, we add a version to maintain existing systems that expected sorting to be enabled

		MemorySaving, // Convert TMap into TArray to save memory, TMap contains an inline allocator which pushes the size to 80 bytes

		AddSimulationStageUsageEnum, // Added a new value to the script usage enum, and we need a custom version to fix the existing bitfields.

		AddGeneratedFunctionsToGPUParamInfo, // Save the functions generated by a GPU data interface inside FNiagaraDataInterfaceGPUParamInfo

		PlatformScalingRefactor, //Removed DetailLevel in favor of FNiagaraPlatfomSet based selection of per platform settings.

		PrecompileNamespaceFixup, // Promote parameters used across script executions to the Dataset, and Demote unused parameters.

		FixNullScriptVariables, // Postload fixup in UNiagaraGraph to fixup VariableToScriptVariable map entries being null. 

		PrecompileNamespaceFixup2, // Move FNiagaraVariableMetaData from storing scope enum to storing registered scope name.

		SimulationStageInUsageBitmask, // Enable the simulation stage flag by default in the usage bitmask of modules and functions

		StandardizeParameterNames, // Fix graph parameter map parameters on post load so that they all have a consisten parsable format and update the UI to show and filter based on these formats.

		ComponentsOnlyHaveUserVariables, // Make sure that UNiagaraComponents only have override maps for User variables.

		RibbonRendererUVRefactor, // Refactor the options for UV settings on the ribbon renderer.

		VariablesUseTypeDefRegistry, // Replace the TypeDefinition in VariableBase with an index into the type registry

		AddLibraryVisibilityProperty, // Expand the visibility options of the scripts to be able to hide a script completely from the user 

		SignificanceHandlers,

		ModuleVersioning, // Added support for multiple versions of script data

		MoveDefaultValueFromFNiagaraVariableMetaDataToUNiagaraScriptVariable, 

		ChangeSystemDeterministicDefault,	// Changed the default mode from deterministic to non-deterministic which matches emitters

		StaticSwitchFunctionPinsUsePersistentGuids, // Update static switch pins to use the PersistentId from their script variable so that when they're renamed their values aren't lost when reallocating pins. 

		VisibilityCullingImprovements, // Extended visibility culling options and moved properties into their own struct.

		AddBakerCameraBookmarks,

		PopulateFunctionCallNodePinNameBindings, // Function call node refresh from external changes has been refactored so that they don't need to populate their name bindings every load.

		ComponentRendererSpawnProperty, // Changed the default value for the component renderer's OnlyCreateComponentsOnParticleSpawn property

		RepopulateFunctionCallNodePinNameBindings, // Previous repopulate didn't handle module attributes like Particles.Module.Name so they need to be repopulated for renaming to work correctly.

		EventSpawnsUpdateInitialAttributeValues, // Event spawns now optionally update Initial. attribute values. New default is true but old data is kept false to maintain existing behavior.

		AddVariadicParametersToGPUFunctionInfo, // Adds list of variadic parameters to the information about GPU functions.

		DynamicPinNodeFixup, // Some data fixup for NiagaraNodeWithDynamicPins.

		RibbonRendererLinkOrderDefaultIsUniqueID,	// Ribbon renderer will default to unique ID rather than normalized age to make more things 'just work'

		SubImageBlendEnabledByDefault,	// Renderer SubImage Blends are enabled by default

		RibbonPlaneUseGeometryNormals,	// Ribbon renderer will use geometry normals by default rather than screen / facing aligned normals

		InitialOwnerVelocityFromActor, // Actors velocity is used for the initial velocity before the component has any tracking, old assets use the old zero velocity

		ParameterBindingWithValueRenameFixup, // FNiagaraParameterBindingWithValue wouldn't necessarily have the appropriate ResolvedParameter namespace when it comes to emitter merging

		SimCache_BulkDataVersion1, // Sim Cache moved to bulk data by default

		InheritanceUxRefactor, // Decoupling of 'Template' and 'Inheritance'

		NDCSpawnGroupOverrideDisabledByDefault, // NDC Read DIs will not override spawn group by default when spawning particles. Old content will remain unchanged.

		CustomSortingBindingToAge, // Before it was normalized age which can introduce flickering with sorting and random lifetimes

		StatelessInitialMeshOrientationV1,	// Update Initial Mesh Orientation Module

		HierarchyEditorScriptSupport, // Hierarchy Editor was implemented

		EmitterStateAddLoopDelayEnabled,	// Added loop delay enabled to emitter state

		SerializeUsageBitMaskToGPUFunctionInfo, // Make sure that we serialize the UsageBitMask with the GPUFunctionInfo
		
		AddedNewInterpolatedSpawnMode, // Changed the interpolated spawn property into an enum

		UserAssetTagStorageInMetaData, // Adding user asset tag storage in UMetaData for all UPackages
		
		// DO NOT ADD A NEW VERSION UNLESS YOU HAVE TALKED TO THE NIAGARA LEAD. Mismanagement of these versions can lead to data loss if it is adjusted in multiple streams simultaneously.
		// -----<new versions can be added above this line>  -------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1,
	};

	/** This value represents the compiler version. It does not provide "backward" compatibility since it is a GUID. It is
	 meant to capture the state of the translator/VM compiler structure and force a flush of any files in the DDC. Since it is 
	 a GUID, we don't need to worry about multiple people editing the translator having conflicting files in the shared DDC as 
	 there should never be any collisions.*/
	NIAGARACORE_API static FGuid GetLatestScriptCompileVersion();

	// The GUID for this custom version number
	NIAGARACORE_API const static FGuid GUID;

private:
	FNiagaraCustomVersion() {}
};




