// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NiagaraCore.h"
#include "NiagaraMergeable.h"
#include "Shader.h"
#include "Serialization/MemoryImage.h"
#include "NiagaraDataInterfaceBase.generated.h"

class INiagaraCompiler;
class UCurveVector;
class UCurveLinearColor;
class UCurveFloat;
class FShaderParametersMetadataBuilder;

class UNiagaraEmitter;
class FNiagaraSystemInstance;
class FNiagaraShader;
class FNiagaraShaderMapPointerTable;
class FNiagaraShaderParametersBuilder;
class UNiagaraDataInterfaceBase;
class FNiagaraGpuComputeDispatchInterface;
struct FNiagaraDataInterfaceGPUParamInfo;
class FRHICommandList;

struct FNiagaraDataInterfaceProxy;
struct FNiagaraComputeInstanceData;
struct FNiagaraSimStageData;

DECLARE_EXPORTED_TEMPLATE_INTRINSIC_TYPE_LAYOUT(template<>, TIndexedPtr<UNiagaraDataInterfaceBase>, NIAGARACORE_API);

/**
 * An interface to the parameter bindings for the data interface used by a Niagara compute shader.
 */
struct FNiagaraDataInterfaceParametersCS
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FNiagaraDataInterfaceParametersCS, NIAGARACORE_API, NonVirtual);
};

//////////////////////////////////////////////////////////////////////////

/** Base class for all Niagara data interfaces. */
UCLASS(abstract, EditInlineNew, MinimalAPI)
class UNiagaraDataInterfaceBase : public UNiagaraNotifyOnChanged
{
	GENERATED_UCLASS_BODY()

public:
	/**
	Override this method to provide parameters to the GPU (SRV / UAV / Constants / etc)
	The most common usage will be to provide a single structure which is nested with other parameters.
	The structure would be declared like this
		BEGIN_SHADER_PARAMETER_STRUCT(FMyShaderParameters, )
			SHADER_PARAMETER(float, MyValue)
			SHADER_PARAMETER_SRV(Buffer<float>, MySRV)
		END_SHADER_PARAMETER_STRUCT()

	And inside the override BuildShaderParameters you would simple add the structure
		ShaderParametersBuilder.AddNestedStruct<FMyShaderParameters>();

	You should also ensure that AppendCompileHash includes the parameters like this
		InVisitor->UpdateShaderParameters<FMyShaderParameters>();

	When filling the data inside the set function you would ask for parameters using the same structure.
	Note: This function is only called on the CDO only not the instance
	*/
	virtual void BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const {}

	/**
	Optional storage that can be created per shader when not in legacy binding mode.
	This can be used to store information about the compilation state to avoid doing runtime checks / look-ups.
	*/
	virtual FNiagaraDataInterfaceParametersCS* CreateShaderStorage(const FNiagaraDataInterfaceGPUParamInfo& ParameterInfo, const FShaderParameterMap& ParameterMap) const { return nullptr; }
	/**
	If we create shader storage we must also supply the type of the storage so we understand how to serialize it.
	This function effectively returns &StaticGetTypeLayoutDesc<ParameterType>().
	*/
	virtual const FTypeLayoutDesc* GetShaderStorageType() const { return nullptr; }

	/** Returns true if the DI (owned by OwnerEmitter) reads any attributes from the Provider emitter */
	virtual bool HasInternalAttributeReads(const UNiagaraEmitter* OwnerEmitter, const UNiagaraEmitter* Provider) const { return false; }
};
