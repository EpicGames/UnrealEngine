// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraNotifyOnChanged.h"

#include "NiagaraMergeable.generated.h"

UCLASS(MinimalAPI)
class UNiagaraMergeable : public UNiagaraNotifyOnChanged
{
	GENERATED_BODY()

public:
	NIAGARACORE_API UNiagaraMergeable();

#if WITH_EDITOR
	NIAGARACORE_API bool Equals(const UNiagaraMergeable* Other);

	NIAGARACORE_API FGuid GetMergeId() const;

protected:
	NIAGARACORE_API UNiagaraMergeable* StaticDuplicateWithNewMergeIdInternal(UObject* InOuter) const;
#endif

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid MergeId;
#endif
};
