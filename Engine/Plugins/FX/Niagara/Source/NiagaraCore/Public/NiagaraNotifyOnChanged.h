// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraNotifyOnChanged.generated.h"

UCLASS()
class UNiagaraNotifyOnChanged : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR

public:
	DECLARE_MULTICAST_DELEGATE(FOnChanged);

public:
	NIAGARACORE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
	NIAGARACORE_API FOnChanged& OnChanged();

private:
	FOnChanged OnChangedDelegate;

#endif // WITH_EDITOR

};
