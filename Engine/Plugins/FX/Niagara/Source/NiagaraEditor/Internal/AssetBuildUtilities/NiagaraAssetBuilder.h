// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "NiagaraTypes.h"

class UNiagaraEmitter;
class UNiagaraSystem;

/** Niagara Emitter builder which creates a new Emitter asset. */
class FNiagaraEmitterAssetBuilder
{
public:
	explicit FNiagaraEmitterAssetBuilder(const FString& InPackagePath, const FString& InAssetName)
		: PackagePath(InPackagePath), AssetName(InAssetName) {}

	/**
	 * Provide an Emitter as the source.
	 *
	 * @param Emitter - NiagaraEmitter to be as the base for the new Emitter
	 *
	 * @return reference to this
	 */
	NIAGARAEDITOR_API FNiagaraEmitterAssetBuilder& WithEmitterToCopy(UNiagaraEmitter* Emitter);

	/**
	 * Build the NiagaraEmitter.
	 *
	 * @param OutError - String for any errors during creation
	 *
	 * @return the newly created NiagaraEmitter
	 * 
	 * @note Copied over from `UNiagaraEmitterFactoryNew` to mimic asset creation as close as possible
	 */
	NIAGARAEDITOR_API UNiagaraEmitter* BuildEmitter(FString& OutError);

private:
	/** Package path of the new NiagaraEmitter asset. */
	FString PackagePath;

	/** Name of the Niagara Emitter. */
	FString AssetName;

	/** Source Emitter to use for the new Emitter. */
	UNiagaraEmitter* EmitterToCopy{ nullptr };
};

/** Niagara System builder which creates a new System asset. */
class FNiagaraSystemAssetBuilder
{
public:
	explicit FNiagaraSystemAssetBuilder(const FString& InPackagePath, const FString& InAssetName)
		: PackagePath(InPackagePath), AssetName(InAssetName) {}

	/**
	 * Provide a System as the source.
	 *
	 * @param System - NiagaraSystem to be as the base for the new System
	 *
	 * @return reference to this
	 */
	NIAGARAEDITOR_API FNiagaraSystemAssetBuilder& WithSystemToCopy(UNiagaraSystem* System);

	/**
	 * Add an Emitter to the new System.
	 *
	 * @param Emitter - NiagaraEmitter to be added
	 *
	 * @return reference to this
	 */
	NIAGARAEDITOR_API FNiagaraSystemAssetBuilder& WithEmitter(FVersionedNiagaraEmitter&& Emitter);

	/**
	 * Build the NiagaraSystem.
	 *
	 * @param OutError - String for any errors during creation
	 *
	 * @return the newly created NiagaraSystem
	 *
	 * @note Copied over from `UNiagaraSystemFactoryNew` to mimic asset creation as close as possible
	 */
	NIAGARAEDITOR_API UNiagaraSystem* BuildSystem(FString& OutError);

private:
	/** Package path of the new NiagaraSystem asset. */
	FString PackagePath;

	/** Name of the Niagara System. */
	FString AssetName;

	/** Source System to use for the new System. */
	UNiagaraSystem* SystemToCopy{ nullptr };

	/** Array of Emitters to be added to the new System. */
	TArray<FVersionedNiagaraEmitter> EmittersToAddToNewSystem;
};

#endif //WITH_EDITOR