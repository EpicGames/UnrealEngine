// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterBindingAdapterCustomization.h"

#include "Customizations/NiagaraTypeCustomizations.h"
#include "Widgets/SNiagaraParameterName.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEmitter.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraParameterMapHistory.h"
#include "SNiagaraParameterEditor.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"
#include "NiagaraVariableMetaData.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SComboButton.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SGraphActionMenu.h"

#define LOCTEXT_NAMESPACE "NiagaraParameterBindingAdapterCustomization"

namespace NiagaraParameterBindingCustomizationBasePrivate
{
	struct FBindingAction : public FEdGraphSchemaAction
	{
		FBindingAction() : FEdGraphSchemaAction(FText(), FText(), FText(), 0) {}
		explicit FBindingAction(FText InName, FText InTooltip) : FEdGraphSchemaAction(FText(), InName, InTooltip, 0) {}

		static TSharedRef<FBindingAction> MakeNone()
		{
			TSharedRef<FBindingAction> Action = MakeShared<FBindingAction>(
				LOCTEXT("None", "None"),
				LOCTEXT("NoneTooltip", "Not bound to any parameter.")
			);
			Action->Grouping = 1;
			return Action;
		}

		static TSharedRef<FBindingAction> MakeConstant(const FNiagaraTypeDefinition& TypeDef, TConstArrayView<uint8> ValueData)
		{
			TSharedRef<FBindingAction> Action = MakeShared<FBindingAction>(
				FText::Format(LOCTEXT("ConstantFormat", "Constant Value \"{0}\""), FText::FromString(TypeDef.ToString(ValueData.GetData()))),
				LOCTEXT("ConstantTooltip", "Use user defined constant value, not bound to any parameter.")
			);
			Action->Grouping = 1;
			return Action;
		}

		static TSharedRef<FBindingAction> MakeBinding(FName InVariableName, FInstancedStruct InValueStruct)
		{
			FText DisplayName = FText::FromName(InVariableName);
			TSharedRef<FBindingAction> Action = MakeShared<FBindingAction>(
				DisplayName,
				FText::Format(LOCTEXT("VariableBindActionTooltip", "Bind to variable \"{0}\""), DisplayName)
			);
			Action->Grouping = 0;
			Action->VariableName = InVariableName;
			Action->ValueStruct = MoveTemp(InValueStruct);
			return Action;
		}

		FName				VariableName;
		FInstancedStruct	ValueStruct;
	};

	FText JoinTypeArray(FText Delim, TConstArrayView<UClass*> TypeArray)
	{
		TArray<FText> TypeNamesText;
		for (UClass* Class : TypeArray)
		{
			TypeNamesText.Add(Class->GetDisplayNameText());
		}
		return FText::Join(Delim, TypeNamesText);
	}

	FText JoinTypeArray(FText Delim, TConstArrayView<FNiagaraTypeDefinition> TypeArray)
	{
		TArray<FText> TypeNamesText;
		for (const FNiagaraTypeDefinition& TypeDef : TypeArray)
		{
			TypeNamesText.Add(TypeDef.GetNameText());
		}
		return FText::Join(Delim, TypeNamesText);
	}

	template<typename TArrayType>
	void BuildTypesText(FText& InOutString, FText TypesDescription, TArrayType TypeArray)
	{
		if (TypeArray.Num() == 0)
		{
			return;
		}

		InOutString = FText::Join(
			FText::GetEmpty(),
			TArray<FText>(
				{
					InOutString,
					LOCTEXT("NewLine", "\n"),
					TypesDescription,
					JoinTypeArray(LOCTEXT("CommaSpace", ", "), TypeArray)
				}
			)
		);
	}

	FText GetAllowedBindingTypesText(const FNiagaraParameterBindingAdapter* ParameterBinding)
	{
		FText BoundTypes;
		BuildTypesText(
			BoundTypes,
			LOCTEXT("BindingTooltip_DIFormat", "Data Interface Types: "),
			ParameterBinding->GetAllowedDataInterfaces()
		);
		BuildTypesText(
			BoundTypes,
			LOCTEXT("BindingTooltip_ObjectFormat", "Object Types: "),
			ParameterBinding->GetAllowedObjects()
		);
		BuildTypesText(
			BoundTypes,
			LOCTEXT("BindingTooltip_InterfacesFormat", "Interface Types: "),
			ParameterBinding->GetAllowedInterfaces()
		);
		BuildTypesText(
			BoundTypes,
			LOCTEXT("BindingTooltip_AttributesFormat", "Attribute Types: "),
			ParameterBinding->GetAllowedTypeDefinitions()
		);
		return BoundTypes;
	}

	bool CanBindToType(const FNiagaraParameterBindingAdapter* ParameterBinding, const FNiagaraTypeDefinition& TypeDefinition)
	{
		if (ParameterBinding->GetAllowedTypeDefinitions().Contains(TypeDefinition))
		{
			return true;
		}

		if (TypeDefinition.IsStatic() && ParameterBinding->AllowStaticVariables())
		{
			if (ParameterBinding->GetAllowedTypeDefinitions().Contains(TypeDefinition.RemoveStaticDef()))
			{
				return true;
			}
		}

		if (UClass* TypeClass = TypeDefinition.GetClass())
		{
			if (TypeClass->IsChildOf(UNiagaraDataInterface::StaticClass()))
			{
				for (UClass* AllowedClass : ParameterBinding->GetAllowedDataInterfaces())
				{
					if (AllowedClass && TypeClass->IsChildOf(AllowedClass))
					{
						return true;
					}
				}

				for (UClass* AllowedInterface : ParameterBinding->GetAllowedInterfaces())
				{
					if (AllowedInterface && TypeClass->ImplementsInterface(AllowedInterface))
					{
						return true;
					}
				}
			}
			else
			{
				for (UClass* AllowedClass : ParameterBinding->GetAllowedObjects())
				{
					if (AllowedClass && TypeClass->IsChildOf(AllowedClass))
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	bool CanBindToNamespace(const FNiagaraParameterBindingAdapter* ParameterBinding, const FNiagaraVariableBase& InAliasedVariable)
	{
		if (InAliasedVariable.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString))
		{
			return ParameterBinding->AllowParticleParameters();
		}
		else if (InAliasedVariable.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString))
		{
			return ParameterBinding->AllowEmitterParameters();
		}
		else if (InAliasedVariable.IsInNameSpace(FNiagaraConstants::SystemNamespaceString))
		{
			return ParameterBinding->AllowSystemParameters();
		}
		else if (InAliasedVariable.IsInNameSpace(FNiagaraConstants::UserNamespaceString))
		{
			return ParameterBinding->AllowUserParameters();
		}
		return false;
	}
}

void FNiagaraParameterBindingAdapterCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	EmitterWeakPtr.Reset();
	SystemWeakPtr.Reset();

	// We only handle single object
	UObject* OwnerObject = nullptr;
	{
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);
		OwnerObject = OuterObjects.Num() == 1 ? OuterObjects[0] : nullptr;
		if (OwnerObject == nullptr)
		{
			HeaderRow
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(200.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FName::NameToDisplayString(CastField<FStructProperty>(PropertyHandle->GetProperty())->Struct->GetName(), false)))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];

			return;
		}
	}

	ParameterBindingAdapter = AdapterCreator(PropertyHandle->GetValueBaseAddress((uint8*)OwnerObject));
	PropertyHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateLambda([]() { }));
	PropertyHandle->MarkResetToDefaultCustomized(true);

	OwnerWeakPtr	= OwnerObject;
	EmitterWeakPtr	= OwnerObject->GetTypedOuter<UNiagaraEmitter>();
	SystemWeakPtr	= OwnerObject->GetTypedOuter<UNiagaraSystem>();

	TSharedPtr<SHorizontalBox> ParameterPanel = SNew(SHorizontalBox);

	// Can this parameter be set to a constant value?
	FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter();
	if (ParameterBinding && ParameterBinding->AllowConstantValue())
	{
		// Create scope for constant value
		const FNiagaraTypeDefinition ConstantValueTypeDef = ParameterBinding->GetConstantTypeDef();
		TConstArrayView<uint8> ConstantValue = ParameterBinding->GetConstantValue();
		ConstantValueStructOnScope = MakeShared<FStructOnScope>(ConstantValueTypeDef.GetStruct());
		FMemory::Memcpy(ConstantValueStructOnScope->GetStructMemory(), ConstantValue.GetData(), ConstantValue.Num());

		// Create editor for constant value
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(ConstantValueTypeDef);
		if (TypeEditorUtilities.IsValid())
		{
			FNiagaraInputParameterCustomization CustomizationOptions;
			CustomizationOptions.bBroadcastValueChangesOnCommitOnly = true; // each broadcast usually forces a recompile, so we only want to do it on commits
			ConstantValueParameterEditor = TypeEditorUtilities->CreateParameterEditor(ConstantValueTypeDef, EUnit::Unspecified, CustomizationOptions);
			ConstantValueParameterEditor->UpdateInternalValueFromStruct(ConstantValueStructOnScope.ToSharedRef());
			ConstantValueParameterEditor->SetOnValueChanged(SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &FNiagaraParameterBindingAdapterCustomization::OnConstantValueChanged));

			ParameterPanel->AddSlot()
			[
				SNew(SBox)
				.HAlign(ConstantValueParameterEditor->GetHorizontalAlignment())
				.VAlign(ConstantValueParameterEditor->GetVerticalAlignment())
				.IsEnabled(this, &FNiagaraParameterBindingAdapterCustomization::IsConstantEnabled)
				.Visibility(this, &FNiagaraParameterBindingAdapterCustomization::IsConstantVisibile)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						ConstantValueParameterEditor.ToSharedRef()
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					[
						SNew(SComboButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ForegroundColor(FSlateColor::UseForeground())
						.OnGetMenuContent(this, &FNiagaraParameterBindingAdapterCustomization::OnGetMenuContent)
						.ContentPadding(2)
						.MenuPlacement(MenuPlacement_BelowRightAnchor)
						.ToolTipText(this, &FNiagaraParameterBindingAdapterCustomization::GetTooltipText)
					]
				]
			];
		}
	}

	ParameterPanel->AddSlot()
	.AutoWidth()
	[
		SNew(SComboButton)
		.ContentPadding(2)
		.IsEnabled(this, &FNiagaraParameterBindingAdapterCustomization::IsBindingEnabled)
		.Visibility(this, &FNiagaraParameterBindingAdapterCustomization::IsBindingVisibile)
		.OnGetMenuContent(this, &FNiagaraParameterBindingAdapterCustomization::OnGetMenuContent)
		.ButtonContent()
		[
			SNew(SNiagaraParameterName)
			.ParameterName(this, &FNiagaraParameterBindingAdapterCustomization::GetBoundParameterName)
			.IsReadOnly(true)
			.ToolTipText(this, &FNiagaraParameterBindingAdapterCustomization::GetTooltipText)
		]
	];

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			ParameterPanel.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2, 1)
		[
			SNew(SButton)
			.IsFocusable(false)
			.ToolTipText(LOCTEXT("ResetToDefaultToolTip", "Reset to Default"))
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ContentPadding(0)
			.Visibility(this, &FNiagaraParameterBindingAdapterCustomization::IsResetToDefaultsVisible)
			.OnClicked(this, &FNiagaraParameterBindingAdapterCustomization::OnResetToDefaultsClicked)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];
}

bool FNiagaraParameterBindingAdapterCustomization::IsValid() const
{
	return OwnerWeakPtr.IsValid();
}

FNiagaraParameterBindingAdapter* FNiagaraParameterBindingAdapterCustomization::GetParameterBindingAdapter() const
{
	return IsValid() ? ParameterBindingAdapter.Get() : nullptr;
}

bool FNiagaraParameterBindingAdapterCustomization::IsBindingEnabled() const
{
	const FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter();
	return ParameterBinding && ParameterBinding->IsSetToParameter();
}

bool FNiagaraParameterBindingAdapterCustomization::IsConstantEnabled() const
{
	const FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter();
	return ParameterBinding && ParameterBinding->IsSetToConstant();
}

EVisibility FNiagaraParameterBindingAdapterCustomization::IsBindingVisibile() const
{
	return IsBindingEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FNiagaraParameterBindingAdapterCustomization::IsConstantVisibile() const
{
	return IsConstantEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
}

FName FNiagaraParameterBindingAdapterCustomization::GetBoundParameterName() const
{
	const FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter();
	return ParameterBinding ? ParameterBinding->GetBoundParameter().GetName() : FName();
}

FText FNiagaraParameterBindingAdapterCustomization::GetTooltipText() const
{
	if ( const FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter() )
	{
		TArray<FText> TooltipTexts;
		if (ParameterBinding->IsSetToConstant())
		{
			const FNiagaraTypeDefinition ConstantType = ParameterBinding->GetConstantTypeDef();
			TooltipTexts.Add(
				FText::Format(
					LOCTEXT("BindingTooltip_BoundConstant", "Bound to constant value \"{0}\" type \"{1}\""),
					FText::FromString(ConstantType.ToString(ParameterBinding->GetConstantValue().GetData())),
					FText::FromString(ConstantType.GetName())
				)
			);
		}
		else
		{
			const FNiagaraVariableBase& BoundParameter = ParameterBinding->GetBoundParameter();
			if (BoundParameter.GetName().IsNone() == false)
			{
				TooltipTexts.Add(
					FText::Format(LOCTEXT("BindingTooltip_BoundParameter", "Bound to parameter \"{0}\" type \"{1}\""),
						FText::FromName(BoundParameter.GetName()),
						FText::FromString(BoundParameter.GetType().GetName())
					)
				);
			}
			else
			{
				TooltipTexts.Add(LOCTEXT("BindingTooltip_UnboundParameter", "Not bound to any parameter."));
			}
		}

		TooltipTexts.Add(NiagaraParameterBindingCustomizationBasePrivate::GetAllowedBindingTypesText(ParameterBinding));
		return FText::Join(LOCTEXT("NewLine", "\n"), TooltipTexts);
	}
	return LOCTEXT("BindingTooltip_Invalid", "Binding is invalid");
}

void FNiagaraParameterBindingAdapterCustomization::OnConstantValueChanged() const
{
	const FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter();
	if (ParameterBinding == nullptr || !PropertyHandle.IsValid())
	{
		return;
	}
	
	FScopedTransaction Transaction(LOCTEXT("ChangeParameterValue", " Change Parameter Value"));
	
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	for (UObject* Obj : Objects)
	{
		Obj->Modify();
	}
	PropertyHandle->NotifyPreChange();
	
	ConstantValueParameterEditor->UpdateStructFromInternalValue(ConstantValueStructOnScope.ToSharedRef());
	ParameterBinding->SetConstantValue(MakeArrayView(ConstantValueStructOnScope->GetStructMemory(), ConstantValueStructOnScope->GetStruct()->GetStructureSize()));
	
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

TSharedRef<SWidget> FNiagaraParameterBindingAdapterCustomization::OnGetMenuContent() const
{
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			[
				SNew(SGraphActionMenu)
				.OnActionSelected(this, &FNiagaraParameterBindingAdapterCustomization::OnActionSelected)
				.OnCreateWidgetForAction(this, &FNiagaraParameterBindingAdapterCustomization::OnCreateWidgetForAction)
				.OnCollectAllActions(this, &FNiagaraParameterBindingAdapterCustomization::CollectAllActions)
				.AutoExpandActionMenu(false)
				.ShowFilterTextBox(true)
			]
		];
}

void FNiagaraParameterBindingAdapterCustomization::CollectAllActions(FGraphActionListBuilderBase& OutAllActions) const
{
	using namespace NiagaraParameterBindingCustomizationBasePrivate;

	// Add default action(s)
	const FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter();
	if (ParameterBinding && ParameterBinding->AllowConstantValue())
	{
		OutAllActions.AddAction(FBindingAction::MakeConstant(ParameterBinding->GetConstantTypeDef(), ParameterBinding->GetConstantValue()));
	}
	else
	{
		OutAllActions.AddAction(FBindingAction::MakeNone());
	}

	// Add binding(s)
	UNiagaraSystem* NiagaraSystem = SystemWeakPtr.Get();
	UNiagaraEmitter* NiagaraEmitter = EmitterWeakPtr.Get();
	if (ParameterBinding == nullptr || NiagaraSystem == nullptr)
	{
		return;
	}

	TArray<FNiagaraParameterMapHistory, TInlineAllocator<2>> Histories;
	if (UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(NiagaraSystem->GetSystemUpdateScript()->GetLatestSource()))
	{
		Histories.Append(UNiagaraNodeParameterMapBase::GetParameterMaps(Source->NodeGraph));
	}

	if (NiagaraEmitter)
	{
		// This is lame but currently the only way to get the version data
		if (UObject* Owner = OwnerWeakPtr.Get())
		{
			if (UNiagaraSimulationStageBase* SimulationStage = Cast<UNiagaraSimulationStageBase>(Owner))
			{
				if (UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(SimulationStage->GetEmitterData()->GraphSource))
				{
					Histories.Append(UNiagaraNodeParameterMapBase::GetParameterMaps(Source->NodeGraph));
				}
			}
			else if (UNiagaraRendererProperties* RendererProperties = Cast<UNiagaraRendererProperties>(Owner))
			{
				if (UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(RendererProperties->GetEmitterData()->GraphSource))
				{
					Histories.Append(UNiagaraNodeParameterMapBase::GetParameterMaps(Source->NodeGraph));
				}
			}
		}
	}

	TArray<FNiagaraVariable> VisitedVariables;
	TArray<TPair<FName, FInstancedStruct>> AllBindings;

	const FStringView OwnerEmitterName = NiagaraEmitter ? NiagaraEmitter->GetUniqueEmitterName() : FStringView();
	for (const FNiagaraParameterMapHistory& History : Histories)
	{
		for (const FNiagaraVariable& ResolvedVariable : History.Variables)
		{
			FNiagaraVariable AliasedVariable = ResolvedVariable;
			AliasedVariable.ReplaceRootNamespace(OwnerEmitterName, FNiagaraConstants::EmitterNamespaceString);

			if (VisitedVariables.Contains(AliasedVariable))
			{
				continue;
			}
			VisitedVariables.Add(AliasedVariable);

			// Is this type of variable allowed?
			if (CanBindToType(ParameterBinding, AliasedVariable.GetType()) == false )
			{
				continue;
			}

			if (CanBindToNamespace(ParameterBinding, AliasedVariable) == false)
			{
				continue;
			}

			// Collect bindings for this variable
			ParameterBinding->CollectBindings(AliasedVariable, ResolvedVariable, AllBindings);
		}
	}

	for (const FNiagaraVariableBase& Variable : NiagaraSystem->GetExposedParameters().ReadParameterVariables())
	{
		if (VisitedVariables.Contains(Variable))
		{
			continue;
		}
		VisitedVariables.Add(Variable);

		if (CanBindToType(ParameterBinding, Variable.GetType()) == false)
		{
			continue;
		}

		ParameterBinding->CollectBindings(Variable, Variable, AllBindings);
	}


	for (const TPair<FName, FInstancedStruct>& Binding : AllBindings)
	{
		OutAllActions.AddAction(FBindingAction::MakeBinding(Binding.Key, Binding.Value));
	}
}

TSharedRef<SWidget> FNiagaraParameterBindingAdapterCustomization::OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData) const
{
	using namespace NiagaraParameterBindingCustomizationBasePrivate;

	const FBindingAction* BindingAction = static_cast<const FBindingAction*>(InCreateData->Action.Get());
	check(BindingAction);

	if (BindingAction->VariableName.IsNone() == false)
	{
		return
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SNiagaraParameterName)
				.ParameterName(BindingAction->VariableName)
				.IsReadOnly(true)
				.ToolTipText(InCreateData->Action->GetTooltipDescription())
			];
	}
	else
	{
		return
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(InCreateData->Action->GetMenuDescription())
				.ToolTipText(InCreateData->Action->GetTooltipDescription())
			];
	}
}

void FNiagaraParameterBindingAdapterCustomization::OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions, ESelectInfo::Type InSelectionType) const
{
	using namespace NiagaraParameterBindingCustomizationBasePrivate;

	FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter();
	if (ParameterBinding == nullptr || !PropertyHandle.IsValid() || (InSelectionType != ESelectInfo::OnMouseClick && InSelectionType != ESelectInfo::OnKeyPress))
	{
		return;
	}

	for (const TSharedPtr<FEdGraphSchemaAction>& EdAction : SelectedActions)
	{
		if (EdAction.IsValid() == false)
		{
			continue;
		}

		FBindingAction* BindAction = static_cast<FBindingAction*>(EdAction.Get());
		FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeParameterBinding", " Change Parameter Binding to \"{0}\" "), BindAction->GetMenuDescription()));
		FSlateApplication::Get().DismissAllMenus();

		TArray<UObject*> Objects;
		PropertyHandle->GetOuterObjects(Objects);
		for (UObject* Obj : Objects)
		{
			Obj->Modify();
		}
		PropertyHandle->NotifyPreChange();

		ParameterBinding->SetBoundParameter(BindAction->ValueStruct);

		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}
}

EVisibility FNiagaraParameterBindingAdapterCustomization::IsResetToDefaultsVisible() const
{
	const FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter();
	return ParameterBinding && ParameterBinding->IsSetToDefault() ? EVisibility::Hidden : EVisibility::Visible;
}

FReply FNiagaraParameterBindingAdapterCustomization::OnResetToDefaultsClicked()
{
	if (FNiagaraParameterBindingAdapter* ParameterBinding = GetParameterBindingAdapter())
	{
		ParameterBinding->SetToDefault();

		if (ConstantValueStructOnScope.IsValid())
		{
			TConstArrayView<uint8> ConstantValue = ParameterBinding->GetConstantValue();
			FMemory::Memcpy(ConstantValueStructOnScope->GetStructMemory(), ConstantValue.GetData(), ConstantValue.Num());
			ConstantValueParameterEditor->UpdateInternalValueFromStruct(ConstantValueStructOnScope.ToSharedRef());
		}

		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
