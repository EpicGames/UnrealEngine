// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "UObject/StructOnScope.h"
#include "StructUtils/InstancedStruct.h"

#include "NiagaraCommon.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;
class IPropertyTypeCustomizationUtils;
class SWidget;

class UNiagaraEmitter;
class UNiagaraSystem;
class SNiagaraParameterEditor;

class FNiagaraParameterBindingAdapter
{
public:
	virtual ~FNiagaraParameterBindingAdapter() = default;

	virtual bool IsSetToParameter() const = 0;	// !AllowConstantValue || ResolveParameter.GetName().IsNone() == false
	virtual bool IsSetToConstant() const { return IsSetToParameter() == false; }

	virtual bool AllowConstantValue() const = 0;
	virtual FNiagaraTypeDefinition GetConstantTypeDef() const = 0;
	virtual TConstArrayView<uint8> GetConstantValue() const = 0;
	virtual void SetConstantValue(TConstArrayView<uint8> Memory) const = 0;

	virtual const FNiagaraVariableBase& GetBoundParameter() const = 0;
	virtual void SetBoundParameter(const FInstancedStruct& Parameter) = 0;

	virtual bool IsSetToDefault() const = 0;
	virtual void SetToDefault() = 0;

	virtual void CollectBindings(const FNiagaraVariableBase& AliasedVariable, const FNiagaraVariableBase& ResolvedVariable, TArray<TPair<FName, FInstancedStruct>>& OutBindings) const = 0;

	virtual bool AllowUserParameters() const { return true; }
	virtual bool AllowSystemParameters() const { return true; }
	virtual bool AllowEmitterParameters() const { return true; }
	virtual bool AllowParticleParameters() const { return true; }
	virtual bool AllowStaticVariables() const { return true; }

	virtual TConstArrayView<UClass*> GetAllowedDataInterfaces() const { return {}; }
	virtual TConstArrayView<UClass*> GetAllowedObjects() const { return {}; }
	virtual TConstArrayView<UClass*> GetAllowedInterfaces() const { return {}; }
	virtual TConstArrayView<FNiagaraTypeDefinition> GetAllowedTypeDefinitions() const { return {}; }
};

class FNiagaraParameterBindingAdapterCustomization : public IPropertyTypeCustomization
{
public:
	using FAdapterCreationFunction = TFunction<TUniquePtr<FNiagaraParameterBindingAdapter>(uint8* ObjectAddress)>;

	FNiagaraParameterBindingAdapterCustomization(FAdapterCreationFunction InAdapterCreator)
		: AdapterCreator(InAdapterCreator)
	{
	}

	/** IPropertyTypeCustomization interface begin */
	NIAGARAEDITOR_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	/** IPropertyTypeCustomization interface end */

private:
	bool IsValid() const;

	FNiagaraParameterBindingAdapter* GetParameterBindingAdapter() const;

	bool IsBindingEnabled() const;
	bool IsConstantEnabled() const;
	EVisibility IsBindingVisibile() const;
	EVisibility IsConstantVisibile() const;

	FName GetBoundParameterName() const;
	FText GetTooltipText() const;

	void OnConstantValueChanged() const;

	TSharedRef<SWidget> OnGetMenuContent() const;
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) const;
	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData) const;
	void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions, ESelectInfo::Type InSelectionType) const;

	EVisibility IsResetToDefaultsVisible() const;
	FReply OnResetToDefaultsClicked();

protected:
	FAdapterCreationFunction					AdapterCreator;
	TUniquePtr<FNiagaraParameterBindingAdapter>	ParameterBindingAdapter;
	TSharedPtr<IPropertyHandle>					PropertyHandle;
	TWeakObjectPtr<UObject>						OwnerWeakPtr;
	TWeakObjectPtr<UNiagaraEmitter>				EmitterWeakPtr;
	TWeakObjectPtr<UNiagaraSystem>				SystemWeakPtr;

	TSharedPtr<SNiagaraParameterEditor>			ConstantValueParameterEditor;
	TSharedPtr<FStructOnScope>					ConstantValueStructOnScope;
};
