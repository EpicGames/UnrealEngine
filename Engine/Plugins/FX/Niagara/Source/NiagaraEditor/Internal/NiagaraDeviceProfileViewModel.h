// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class UDeviceProfile;

struct FNiagaraDeviceProfileViewModel
{
	UDeviceProfile* Profile;
	TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>> Children;
};
