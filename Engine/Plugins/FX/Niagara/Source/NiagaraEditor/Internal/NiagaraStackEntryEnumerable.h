// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterface.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"

template<typename TNiagaraStackEntry>
class INiagaraStackEntryIterator
{
public:
	virtual ~INiagaraStackEntryIterator() = default;

	virtual bool IsValid() const = 0;
	virtual void MoveNext() = 0;
	virtual TNiagaraStackEntry* GetCurrent() const = 0;
};

template<typename TNiagaraStackEntry>
class TNiagaraStackEntryNullIterator : public INiagaraStackEntryIterator<TNiagaraStackEntry>
{
public:
	TNiagaraStackEntryNullIterator()
	{
	}

	virtual bool IsValid() const override { return false; }
	virtual void MoveNext() override { }
	virtual TNiagaraStackEntry* GetCurrent() const override { return nullptr; }
};

template<typename TNiagaraStackEntry>
class TNiagaraStackEntryArrayIterator : public INiagaraStackEntryIterator<TNiagaraStackEntry>
{
public:
	TNiagaraStackEntryArrayIterator(const TArray<TNiagaraStackEntry*>& InArrayEntries)
		: ArrayEntries(InArrayEntries)
		, ArrayIndex(0)
	{
	}

	virtual bool IsValid() const override
	{
		return ArrayIndex >= 0 && ArrayIndex < ArrayEntries.Num();
	}

	virtual void MoveNext() override
	{
		ArrayIndex++;
	}

	virtual TNiagaraStackEntry* GetCurrent() const override
	{
		return ArrayEntries[ArrayIndex];
	}

private:
	TArray<TNiagaraStackEntry*> ArrayEntries;
	int32 ArrayIndex = INDEX_NONE;
};

template<typename TNiagaraStackEntry>
class TNiagaraStackEntryPredicateIterator : public INiagaraStackEntryIterator<TNiagaraStackEntry>
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FEntryPredicate, TNiagaraStackEntry* /* Entry */);

public:
	TNiagaraStackEntryPredicateIterator(TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntry>> InIterator, FEntryPredicate InPredicate)
		: Iterator(InIterator)
		, Predicate(InPredicate)
	{
		if (Iterator->IsValid())
		{
			if (Predicate.Execute(Iterator->GetCurrent()) == false)
			{
				MoveToNextValidEntry();
			}
		}
	}

	virtual bool IsValid() const override
	{
		return Iterator->IsValid();
	}

	virtual void MoveNext() override
	{
		MoveToNextValidEntry();
	}

	virtual TNiagaraStackEntry* GetCurrent() const override
	{
		return Iterator->GetCurrent();
	}

private:
	void MoveToNextValidEntry()
	{
		Iterator->MoveNext();
		while (Iterator->IsValid() && Predicate.Execute(Iterator->GetCurrent()) == false)
		{
			Iterator->MoveNext();
		}
	}

private:
	TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntry>> Iterator;
	FEntryPredicate Predicate;
};

template<typename TNiagaraStackEntrySource, typename TNiagaraStackEntryTarget>
class TNiagaraStackEntryOfTypeIterator : public INiagaraStackEntryIterator<TNiagaraStackEntryTarget>
{
public:
	TNiagaraStackEntryOfTypeIterator(TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntrySource>> InSourceEntryIterator)
		: PredicateIterator(MakeShared<TNiagaraStackEntryPredicateIterator<TNiagaraStackEntrySource>>(InSourceEntryIterator,
			TNiagaraStackEntryPredicateIterator<TNiagaraStackEntrySource>::FEntryPredicate::CreateStatic(&TNiagaraStackEntryOfTypeIterator<TNiagaraStackEntrySource, TNiagaraStackEntryTarget>::IsOfType)))
	{
	}

	virtual bool IsValid() const override
	{
		return PredicateIterator->IsValid();
	}

	virtual void MoveNext() override
	{
		PredicateIterator->MoveNext();
	}

	virtual TNiagaraStackEntryTarget* GetCurrent() const override
	{
		return CastChecked<TNiagaraStackEntryTarget>(PredicateIterator->GetCurrent());
	}

private:
	static bool IsOfType(TNiagaraStackEntrySource* Entry)
	{
		return Entry->template IsA<TNiagaraStackEntryTarget>();
	}

private:
	TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntrySource>> PredicateIterator;
};

template<typename TNiagaraStackEntry>
class TNiagaraStackEntryChildrenIterator : public INiagaraStackEntryIterator<UNiagaraStackEntry>
{
public:
	TNiagaraStackEntryChildrenIterator(TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntry>> InIterator)
		: Iterator(InIterator)
	{
		if (Iterator->IsValid())
		{
			MoveNext();
		}
	}

	virtual bool IsValid() const override
	{
		return Iterator->IsValid() && CurrentChildEntryIndex >= 0 && CurrentChildEntryIndex < CurrentChildEntries.Num();
	}

	virtual void MoveNext() override
	{
		if (CurrentChildEntryIndex == INDEX_NONE)
		{
			Iterator->GetCurrent()->GetFilteredChildren(CurrentChildEntries);
			CurrentChildEntryIndex = 0;
		}
		else
		{
			CurrentChildEntryIndex++;
		}

		while (Iterator->IsValid() && CurrentChildEntryIndex >= CurrentChildEntries.Num())
		{
			Iterator->MoveNext();
			if (Iterator->IsValid())
			{
				Iterator->GetCurrent()->GetFilteredChildren(CurrentChildEntries);
				CurrentChildEntryIndex = 0;
			}
		}
	}

	virtual UNiagaraStackEntry* GetCurrent() const override
	{
		return CurrentChildEntries[CurrentChildEntryIndex];
	}

private:
	TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntry>> Iterator;
	TArray<UNiagaraStackEntry*> CurrentChildEntries;
	int32 CurrentChildEntryIndex = INDEX_NONE;
};

template<typename TNiagaraStackEntry>
class TNiagaraStackEntryEnumerable
{
public:
	class FIterator
	{
	public:
		FIterator(const TNiagaraStackEntryEnumerable<TNiagaraStackEntry>& InOwner, const TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntry>>& InIterator)
			: Owner(&InOwner)
			, Iterator(InIterator)
		{
		}

		bool operator!=(const FIterator& Other)
		{
			return !(Owner == Other.Owner && Iterator->IsValid() == false && Other.Iterator->IsValid() == false);
		}

		FIterator& operator++()
		{
			Iterator->MoveNext();
			return *this;
		}

		TNiagaraStackEntry* operator*() const
		{
			return Iterator->GetCurrent();
		}

	private:
		const TNiagaraStackEntryEnumerable<TNiagaraStackEntry>* Owner;
		TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntry>> Iterator;
	};

	TNiagaraStackEntryEnumerable(TNiagaraStackEntry& StackEntry)
		: StackEntryIterator(MakeShared<TNiagaraStackEntryArrayIterator<TNiagaraStackEntry>>(TArray<TNiagaraStackEntry*>({ &StackEntry })))
	{
	}

	TNiagaraStackEntryEnumerable(TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntry>> InStackEntryIterator)
		: StackEntryIterator(InStackEntryIterator)
	{
	}

	FIterator begin() const
	{
		return FIterator(*this, StackEntryIterator);
	}

	FIterator end() const
	{
		TSharedRef<TNiagaraStackEntryNullIterator<TNiagaraStackEntry>> NullIterator = MakeShared<TNiagaraStackEntryNullIterator<TNiagaraStackEntry>>();
		return FIterator(*this, NullIterator);
	}

	TNiagaraStackEntryEnumerable<UNiagaraStackEntry> Children() const
	{
		return TNiagaraStackEntryEnumerable<UNiagaraStackEntry>(MakeShared<TNiagaraStackEntryChildrenIterator<TNiagaraStackEntry>>(StackEntryIterator));
	}

	template<typename TNiagaraStackEntryTarget>
	TNiagaraStackEntryEnumerable<TNiagaraStackEntryTarget> OfType() const
	{
		return TNiagaraStackEntryEnumerable<TNiagaraStackEntryTarget>(MakeShared<TNiagaraStackEntryOfTypeIterator<TNiagaraStackEntry, TNiagaraStackEntryTarget>>(StackEntryIterator));
	}

	template<typename TPredicate>
	TNiagaraStackEntryEnumerable<TNiagaraStackEntry> Where(TPredicate Predicate) const
	{
		return TNiagaraStackEntryEnumerable<TNiagaraStackEntry>(MakeShared<TNiagaraStackEntryPredicateIterator<TNiagaraStackEntry>>(StackEntryIterator,
			TNiagaraStackEntryPredicateIterator<TNiagaraStackEntry>::FEntryPredicate::CreateLambda([Predicate](TNiagaraStackEntry* Entry) { return Predicate(Entry); })));
	}

	TNiagaraStackEntry* First() const
	{
		for (TNiagaraStackEntry* StackEntry : *this)
		{
			return StackEntry;
		}
		return nullptr;
	}

	TArray<TNiagaraStackEntry*> ToArray() const
	{
		TArray<TNiagaraStackEntry*> Array;
		Array.Append(*this);
		return Array;
	}

private:
	TSharedRef<INiagaraStackEntryIterator<TNiagaraStackEntry>> StackEntryIterator;
};
