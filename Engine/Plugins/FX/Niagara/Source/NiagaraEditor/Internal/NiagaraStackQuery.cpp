// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStackQuery.h"

#include "IDetailTreeNode.h"
#include "NiagaraStackEntryEnumerable.h"
#include "PropertyHandle.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackPropertyRow.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"
#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"

#define LOCTEXT_NAMESPACE "NiagaraStackQuery"

FNiagaraStackPropertyRowQuery FNiagaraStackObjectQuery::FindPropertyRow(FName PropertyName) const
{
	if (GetEntry() == nullptr)
	{
		return FNiagaraStackPropertyRowQuery(GetErrorMessage());
	}

	UNiagaraStackPropertyRow* FoundPropertyRow = TNiagaraStackEntryEnumerable<UNiagaraStackObject>(*GetEntry())
		.Children().OfType<UNiagaraStackPropertyRow>()
		.Where([](UNiagaraStackPropertyRow* PropertyRow) { return PropertyRow->GetDetailTreeNode()->GetNodeType() == EDetailNodeType::Category; })
		.Children().OfType<UNiagaraStackPropertyRow>()
		.Where([PropertyName](UNiagaraStackPropertyRow* PropertyRow)
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow->GetDetailTreeNode()->CreatePropertyHandle();
			return PropertyHandle.IsValid() && PropertyHandle->GetProperty() != nullptr && PropertyHandle->GetProperty()->GetFName() == PropertyName;
		})
		.First();

	return FoundPropertyRow != nullptr
		? FNiagaraStackPropertyRowQuery(*FoundPropertyRow)
		: FNiagaraStackPropertyRowQuery(FText::Format(LOCTEXT("PropertyRowFailFormat", "Failed to find property named {0}."), FText::FromName(PropertyName)));
}

FNiagaraStackObjectQuery FNiagaraStackFunctionInputQuery::FindObjectValue() const
{
	if (GetEntry() == nullptr)
	{
		return FNiagaraStackObjectQuery(GetErrorMessage());
	}

	UNiagaraStackObject* FoundObjectValue = TNiagaraStackEntryEnumerable<UNiagaraStackFunctionInput>(*GetEntry())
		.Children().OfType<UNiagaraStackObject>()
		.First();

	return FoundObjectValue != nullptr
		? FNiagaraStackObjectQuery(*FoundObjectValue)
		: FNiagaraStackObjectQuery(LOCTEXT("ObjectValueFailFormat", "Failed to input object value."));
}

FNiagaraStackFunctionInputQuery FNiagaraStackModuleItemQuery::FindFunctionInput(FName InputName) const
{
	if (GetEntry() == nullptr)
	{
		return FNiagaraStackFunctionInputQuery(GetErrorMessage());
	}

	// Check each category for its inputs. This does not handle nested categories
	UNiagaraStackFunctionInput* FoundInput = TNiagaraStackEntryEnumerable<UNiagaraStackModuleItem>(*GetEntry())
		.Children().OfType<UNiagaraStackScriptHierarchyRoot>()
		.Children().OfType<UNiagaraStackScriptHierarchyCategory>()
		.Children().OfType<UNiagaraStackFunctionInput>()
		.Where([InputName](UNiagaraStackFunctionInput* StackFunctionInput) { return StackFunctionInput->GetInputParameterHandle().GetName() == InputName;})
		.First();

	// Check children right below the root that don't belong to a category, is the input wasn't found
	if(FoundInput == nullptr)
	{
		FoundInput = TNiagaraStackEntryEnumerable<UNiagaraStackModuleItem>(*GetEntry())
		.Children().OfType<UNiagaraStackScriptHierarchyRoot>()
		.Children().OfType<UNiagaraStackFunctionInput>()
		.Where([InputName](UNiagaraStackFunctionInput* StackFunctionInput) { return StackFunctionInput->GetInputParameterHandle().GetName() == InputName;})
		.First();
	}
	
	return FoundInput != nullptr
		? FNiagaraStackFunctionInputQuery(*FoundInput)
		: FNiagaraStackFunctionInputQuery(FText::Format(LOCTEXT("FunctionInputFailFormat", "Failed to find input named {0}."), FText::FromName(InputName)));
}

FNiagaraStackModuleItemQuery FNiagaraStackScriptItemGroupQuery::FindSetParametersItem(FName ParameterName) const
{
	if (GetEntry() == nullptr)
	{
		return FNiagaraStackModuleItemQuery(GetErrorMessage());
	}

	UNiagaraStackModuleItem* FoundModuleItem = TNiagaraStackEntryEnumerable<UNiagaraStackScriptItemGroup>(*GetEntry())
		.Children().OfType<UNiagaraStackModuleItem>()
		.Where([ParameterName](UNiagaraStackModuleItem* ModuleItem)
		{
			UNiagaraNodeAssignment* AssignmentModuleNode = Cast<UNiagaraNodeAssignment>(&ModuleItem->GetModuleNode());
			return AssignmentModuleNode != nullptr &&
				AssignmentModuleNode->GetAssignmentTargets().ContainsByPredicate(
					[ParameterName](const FNiagaraVariable& AssignmentTarget) { return AssignmentTarget.GetName() == ParameterName; });
		})
		.First();

	return FoundModuleItem != nullptr
		? FNiagaraStackModuleItemQuery(*FoundModuleItem)
		: FNiagaraStackModuleItemQuery(FText::Format(LOCTEXT("SetParametersItemFailFormat", "Failed to find a set parameters module item with parameter {0}"), FText::FromName(ParameterName)));
}

FNiagaraStackModuleItemQuery FNiagaraStackScriptItemGroupQuery::FindModuleItem(const FString& ModuleName) const
{
	if (GetEntry() == nullptr)
	{
		return FNiagaraStackModuleItemQuery(GetErrorMessage());
	}
	UNiagaraStackModuleItem* FoundModuleItem = TNiagaraStackEntryEnumerable<UNiagaraStackScriptItemGroup>(*GetEntry())
		.Children().OfType<UNiagaraStackModuleItem>()
		.Where([ModuleName](UNiagaraStackModuleItem* ModuleItem)
			{
				return ModuleItem->GetModuleNode().GetFunctionName() == ModuleName;
			})
		.First();
	return FoundModuleItem != nullptr
		? FNiagaraStackModuleItemQuery(*FoundModuleItem)
		: FNiagaraStackModuleItemQuery(FText::Format(LOCTEXT("ModuleItemFailFormat", "Failed to find a module named {0}"), FText::FromString(ModuleName)));
}

FNiagaraStackRootQuery FNiagaraStackRootQuery::SystemStackRootEntry(TSharedRef<FNiagaraSystemViewModel> SystemViewModel)
{
	UNiagaraStackViewModel* StackViewModel = SystemViewModel->GetSystemStackViewModel();
	return 
		StackViewModel != nullptr &&
		StackViewModel->GetRootEntry() != nullptr &&
		StackViewModel->GetRootEntry()->IsA<UNiagaraStackRoot>()
			? FNiagaraStackRootQuery(*CastChecked<UNiagaraStackRoot>(StackViewModel->GetRootEntry()))
			: FNiagaraStackRootQuery(LOCTEXT("SystemStackViewModelFail", "Failed to find system stack root entry."));
}

FNiagaraStackRootQuery FNiagaraStackRootQuery::EmitterStackRootEntry(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FName EmitterName)
{
	const TSharedRef<FNiagaraEmitterHandleViewModel>* EmitterHandleViewModelPtr = SystemViewModel->GetEmitterHandleViewModels().FindByPredicate(
		[EmitterName](const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel) { return EmitterHandleViewModel->GetName() == EmitterName; });
	
	UNiagaraStackViewModel* StackViewModel = EmitterHandleViewModelPtr != nullptr ? (*EmitterHandleViewModelPtr)->GetEmitterStackViewModel() : nullptr;
	return 
		StackViewModel != nullptr &&
		StackViewModel->GetRootEntry() != nullptr &&
		StackViewModel->GetRootEntry()->IsA<UNiagaraStackRoot>()
			? FNiagaraStackRootQuery(*CastChecked<UNiagaraStackRoot>(StackViewModel->GetRootEntry()))
			: FNiagaraStackRootQuery(FText::Format(LOCTEXT("EmitterStackViewModelFailFormat", "Failed to find emitter stack view model for emitter {0}."),
				FText::FromName(EmitterName)));
}

FNiagaraStackScriptItemGroupQuery FNiagaraStackRootQuery::FindScriptGroup(ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId) const
{
	if (GetEntry() == nullptr)
	{
		return FNiagaraStackScriptItemGroupQuery(GetErrorMessage());
	}

	UNiagaraStackScriptItemGroup* FoundGroup = TNiagaraStackEntryEnumerable<UNiagaraStackRoot>(*GetEntry())
		.Children().OfType<UNiagaraStackScriptItemGroup>()
		.Where([ScriptUsage, ScriptUsageId](UNiagaraStackScriptItemGroup* ScriptItemGroup)
		{ 
			return ScriptItemGroup->GetScriptUsage() == ScriptUsage && ScriptItemGroup->GetScriptUsageId() == ScriptUsageId; 
		})
		.First();

	if (FoundGroup != nullptr)
	{
		return FNiagaraStackScriptItemGroupQuery(*FoundGroup);
	}

	static const UEnum* UsageEnum = StaticEnum<ENiagaraScriptUsage>();
	return FNiagaraStackScriptItemGroupQuery(FText::Format(LOCTEXT("ScriptGroupEntryFailFormat", "Failed to find a script group with usage: {0} and id: {1}"),
		UsageEnum->GetDisplayNameTextByValue((int64)ScriptUsage), FText::FromString(ScriptUsageId.ToString(EGuidFormats::DigitsWithHyphens))));
}

#undef LOCTEXT_NAMESPACE
