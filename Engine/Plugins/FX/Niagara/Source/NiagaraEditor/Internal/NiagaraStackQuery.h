// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "NiagaraCommon.h"
#include "Templates/SharedPointer.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"

class FNiagaraSystemViewModel;
class UNiagaraStackEntry;
class UNiagaraStackFunctionInput;
class UNiagaraStackRoot;
class UNiagaraStackScriptItemGroup;
class UNiagaraStackModuleItem;
class UNiagaraStackObject;
class UNiagaraStackPropertyRow;

template<typename TNiagaraStackEntry>
struct TNiagaraStackQueryResult
{
	TNiagaraStackQueryResult(TNiagaraStackEntry* InStackEntry, FText InErrorMessage)
		: StackEntry(InStackEntry)
		, ErrorMessage(InErrorMessage)
	{
	}

	bool IsValid() const 
	{
		return StackEntry != nullptr;
	}

	TNiagaraStackEntry* StackEntry;
	FText ErrorMessage;
};

template<typename TNiagaraStackEntry>
class TNiagaraStackQueryBase
{
public:
	TNiagaraStackQueryBase(TNiagaraStackEntry& InStackEntry)
		: StackEntry(&InStackEntry)
	{
	}

	TNiagaraStackQueryBase(FText InErrorMessage)
		: StackEntry(nullptr)
		, ErrorMessage(InErrorMessage)
	{
		ensureMsgf(ErrorMessage.IsEmpty() == false, TEXT("Error message must not be empty."));
	}

	TNiagaraStackEntry* GetEntry() const { return StackEntry; }

	FText GetErrorMessage() const { return ErrorMessage; }

	TNiagaraStackQueryResult<TNiagaraStackEntry> GetResult() const
	{
		return TNiagaraStackQueryResult<TNiagaraStackEntry>(StackEntry, ErrorMessage);
	}

private:
	TNiagaraStackEntry* StackEntry;
	FText ErrorMessage;
};

class FNiagaraStackPropertyRowQuery : public TNiagaraStackQueryBase<UNiagaraStackPropertyRow>
{
public:
	FNiagaraStackPropertyRowQuery(UNiagaraStackPropertyRow& InStackEntry) : TNiagaraStackQueryBase<UNiagaraStackPropertyRow>(InStackEntry) { }
	FNiagaraStackPropertyRowQuery(FText InErrorMessage) : TNiagaraStackQueryBase<UNiagaraStackPropertyRow>(InErrorMessage) { }
};

class FNiagaraStackObjectQuery : public TNiagaraStackQueryBase<UNiagaraStackObject>
{
public:
	FNiagaraStackObjectQuery(UNiagaraStackObject& InStackEntry) : TNiagaraStackQueryBase<UNiagaraStackObject>(InStackEntry) { }
	FNiagaraStackObjectQuery(FText InErrorMessage) : TNiagaraStackQueryBase<UNiagaraStackObject>(InErrorMessage) { }

	FNiagaraStackPropertyRowQuery NIAGARAEDITOR_API FindPropertyRow(FName PropertyName) const;
};

class FNiagaraStackFunctionInputQuery : public TNiagaraStackQueryBase<UNiagaraStackFunctionInput>
{
public:
	FNiagaraStackFunctionInputQuery(UNiagaraStackFunctionInput& InStackEntry)	: TNiagaraStackQueryBase<UNiagaraStackFunctionInput>(InStackEntry) { }
	FNiagaraStackFunctionInputQuery(FText InErrorMessage) : TNiagaraStackQueryBase<UNiagaraStackFunctionInput>(InErrorMessage) { }

	FNiagaraStackObjectQuery NIAGARAEDITOR_API FindObjectValue() const;
};

class FNiagaraStackModuleItemQuery : public TNiagaraStackQueryBase<UNiagaraStackModuleItem>
{
public:
	FNiagaraStackModuleItemQuery(UNiagaraStackModuleItem& InStackEntry) : TNiagaraStackQueryBase<UNiagaraStackModuleItem>(InStackEntry) { }
	FNiagaraStackModuleItemQuery(FText InErrorMessage) : TNiagaraStackQueryBase<UNiagaraStackModuleItem>(InErrorMessage) { }

	FNiagaraStackFunctionInputQuery NIAGARAEDITOR_API FindFunctionInput(FName InputName) const;
};

class FNiagaraStackScriptItemGroupQuery : public TNiagaraStackQueryBase<UNiagaraStackScriptItemGroup>
{
public:
	FNiagaraStackScriptItemGroupQuery(UNiagaraStackScriptItemGroup& InStackEntry) : TNiagaraStackQueryBase<UNiagaraStackScriptItemGroup>(InStackEntry) { }
	FNiagaraStackScriptItemGroupQuery(FText InErrorMessage) : TNiagaraStackQueryBase<UNiagaraStackScriptItemGroup>(InErrorMessage) { }

	FNiagaraStackModuleItemQuery NIAGARAEDITOR_API FindSetParametersItem(FName ParameterName) const;
	FNiagaraStackModuleItemQuery NIAGARAEDITOR_API FindModuleItem(const FString& ModuleName) const;
};

class FNiagaraStackRootQuery : public TNiagaraStackQueryBase<UNiagaraStackRoot>
{
public:
	static FNiagaraStackRootQuery SystemStackRootEntry(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

	static FNiagaraStackRootQuery NIAGARAEDITOR_API EmitterStackRootEntry(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel, FName EmitterName);

	FNiagaraStackScriptItemGroupQuery NIAGARAEDITOR_API FindScriptGroup(ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId) const;

private:
	FNiagaraStackRootQuery(UNiagaraStackRoot& InStackEntry) : TNiagaraStackQueryBase<UNiagaraStackRoot>(InStackEntry) { }
	FNiagaraStackRootQuery(FText InErrorMessage) : TNiagaraStackQueryBase<UNiagaraStackRoot>(InErrorMessage) { }
};