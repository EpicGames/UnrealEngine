// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessCommon.h"

#include "Misc/NotifyHook.h"
#include "UObject/StrongObjectPtr.h"

class UNiagaraStatelessEmitterTemplate;

class FNiagaraStatelessEmitterTemplateViewModel : public TSharedFromThis<FNiagaraStatelessEmitterTemplateViewModel>, public FNotifyHook
{
public:
	struct FOutputVariable
	{
		FNiagaraVariableBase	Variable;
		bool					bIsImplicit = false;
		bool					bIsModule = false;
		bool					bIsShader = false;
	};

	DECLARE_MULTICAST_DELEGATE(FOnTemplateChanged)

	explicit FNiagaraStatelessEmitterTemplateViewModel(UNiagaraStatelessEmitterTemplate* EmitterTemplate);
	virtual ~FNiagaraStatelessEmitterTemplateViewModel();

	void PostUndoRedo();

	//- Begin: FNotifyHook
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	//- End: FNotifyHook

	UNiagaraStatelessEmitterTemplate* GetTemplateForEdit() { return EmitterTemplateForEdit.Get(); }
	TConstArrayView<FOutputVariable> GetOututputVariables() const;
	TPair<ENiagaraStatelessFeatureMask, ENiagaraStatelessFeatureMask> GetFeatureMaskRange() const;

	FString GenerateComputeTemplateHLSL() const;

	FOnTemplateChanged& OnTemplateChanged() { return OnTemplateChangedDelegate; }

private:
	void BuildOutputVariables();

private:
	TStrongObjectPtr<UNiagaraStatelessEmitterTemplate>	EmitterTemplate;
	TStrongObjectPtr<UNiagaraStatelessEmitterTemplate>	EmitterTemplateForEdit;
	TArray<FOutputVariable>								OutputVariables;
	FOnTemplateChanged									OnTemplateChangedDelegate;
};
