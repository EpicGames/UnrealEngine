// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class NiagaraEditor : ModuleRules
{
	public NiagaraEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "RHI",
                "Core",
				"CoreUObject",
				"CurveEditor",
				"DerivedDataCache",
				"ApplicationCore",
                "InputCore",
				"RenderCore",
				"Json",
				"Slate",
				"SlateCore",
				"Kismet",
                "EditorStyle",
				"VectorVM",
                "MovieScene",
				"SequencerCore",
				"Sequencer",
				"TimeManagement",
				"PropertyEditor",
				"GraphEditor",
                "ShaderFormatVectorVM",
                "TargetPlatform",
                "DesktopPlatform",
				"MovieSceneTools",
                "MovieSceneTracks",
                "AdvancedPreviewScene",
				"Projects",
                "MainFrame",
				"MeshDescription",
				"StaticMeshDescription",
				"ToolMenus",
				"Renderer",
				"EditorWidgets",
				"Renderer",
				"DeveloperSettings",
				"PythonScriptPlugin",
				"ImageWrapper",
				"AssetDefinition",
				"ContentBrowser",
				"ContentBrowserData",
				"AssetTools",
				"LevelSequence",
				"SparseVolumeTexture",
				"PlacementMode",
				"EngineSettings",
				"EditorConfig",
				"MaterialEditor",
				"JsonUtilities",
			}
		);

		if (Target.bBuildTargetDeveloperTools)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SessionServices",
					"SessionFrontend"
				}
			);
		}

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"MessageLog",
				"Messaging",
                "LevelEditor",
				"WorkspaceMenuStructure"
			}
        );

		PublicDependencyModuleNames.AddRange(
            new string[] {
                "NiagaraCore",
                "NiagaraShader",
                "Engine",
                "NiagaraCore",
                "Niagara",
				"EditorFramework",
                "UnrealEd",
                "AppFramework",
				"ToolWidgets",
				"Sequencer",
				"DataHierarchyEditor",
				"MediaAssets",
				"UserAssetTagsEditor",
            }
        );

		PublicIncludePathModuleNames.AddRange(
            new string[] {
				"Engine",
				"Niagara"
			}
		);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
                "WorkspaceMenuStructure",
                }
            );
	}
}
