// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetBuildUtilities/NiagaraAssetBuilder.h"

#if WITH_EDITOR

#include "AssetToolsModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraEmitterFactoryNew.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemFactoryNew.h"
#include "ObjectTools.h"

namespace {

UPackage* CreateNiagaraPackage(const FString& PackageName)
{
	UPackage* Package = CreatePackage(*(PackageName));
	if (!IsValid(Package))
	{
		return nullptr;
	}

	EObjectFlags Flags = RF_Public | RF_Standalone | RF_Transactional;
	Package->SetFlags(Flags);

	return Package;
}

} //anonymous

FNiagaraEmitterAssetBuilder& FNiagaraEmitterAssetBuilder::WithEmitterToCopy(UNiagaraEmitter* Emitter)
{
	EmitterToCopy = Emitter;
	return *this;
}

UNiagaraEmitter* FNiagaraEmitterAssetBuilder::BuildEmitter(FString& OutError)
{
	const FString PackageName = (PackagePath / AssetName);
	UPackage* Package = CreateNiagaraPackage(PackageName);
	if (!IsValid(Package))
	{
		OutError = TEXT("Failed to create new package for Niagara Emitter.");
		return nullptr;
	}

	const EObjectFlags Flags = Package->GetFlags();

	if (!IsValid(EmitterToCopy))
	{
		FSoftObjectPath DefaultEmptyEmitter = GetDefault<UNiagaraEditorSettings>()->DefaultEmptyEmitter;
		if (DefaultEmptyEmitter.IsValid() && DefaultEmptyEmitter.IsAsset())
		{
			UObject* DefaultEmitter = DefaultEmptyEmitter.TryLoad();
			EmitterToCopy = Cast<UNiagaraEmitter>(DefaultEmitter);
		}
	}

	UNiagaraEmitter* NewNiagaraEmitter;
	if (IsValid(EmitterToCopy))
	{
		const bool bUseInheritance = EmitterToCopy->bIsInheritable;
		if (bUseInheritance)
		{
			NewNiagaraEmitter = UNiagaraEmitter::CreateWithParentAndOwner(FVersionedNiagaraEmitter(EmitterToCopy, EmitterToCopy->GetExposedVersion().VersionGuid), Package, *AssetName, Flags);
		}
		else
		{
			NewNiagaraEmitter = Cast<UNiagaraEmitter>(StaticDuplicateObject(EmitterToCopy, Package, *AssetName, Flags, UNiagaraEmitter::StaticClass()));
			NewNiagaraEmitter->SetUniqueEmitterName(AssetName);
			NewNiagaraEmitter->DisableVersioning(EmitterToCopy->GetExposedVersion().VersionGuid);
		}

		NewNiagaraEmitter->bIsInheritable = true;
		NewNiagaraEmitter->TemplateAssetDescription = FText();
		NewNiagaraEmitter->Category = FText();
	}
	else
	{
		// Create an empty emitter, source, and graph.
		NewNiagaraEmitter = NewObject<UNiagaraEmitter>(Package, UNiagaraEmitter::StaticClass(), *AssetName, Flags | RF_Transactional);

		// Figure out how an empty emitter is created
		UNiagaraEmitterFactoryNew::InitializeEmitter(NewNiagaraEmitter, false);
	}

	NewNiagaraEmitter->ForEachVersionData([](FVersionedNiagaraEmitterData& EmitterVersionData) {
		UNiagaraEmitterEditorData* EmitterEditorData = CastChecked<UNiagaraEmitterEditorData>(EmitterVersionData.GetEditorData());
		EmitterEditorData->SetShowSummaryView(EmitterVersionData.AddEmitterDefaultViewState == ENiagaraEmitterDefaultSummaryState::Summary ? true : false);
	});

	Package->SetAssetAccessSpecifier(EAssetAccessSpecifier::Public);

	// Notify the asset registry
	FAssetRegistryModule::AssetCreated(NewNiagaraEmitter);

	// Mark the package dirty...
	Package->MarkPackageDirty();

	return NewNiagaraEmitter;
}

FNiagaraSystemAssetBuilder& FNiagaraSystemAssetBuilder::WithSystemToCopy(UNiagaraSystem* System)
{
	SystemToCopy = System;
	return *this;
}
	
	
FNiagaraSystemAssetBuilder& FNiagaraSystemAssetBuilder::WithEmitter(FVersionedNiagaraEmitter&& Emitter)
{
	EmittersToAddToNewSystem.Emplace(Emitter);
	return *this;
}

UNiagaraSystem* FNiagaraSystemAssetBuilder::BuildSystem(FString& OutError)
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	if (!IsValid(Settings))
	{
		OutError = TEXT("No default Niagara Editor Settings found.");
		return nullptr;
	}

	const FString PackageName = (PackagePath / AssetName);
	UPackage* Package = CreateNiagaraPackage(PackageName);
	if (!IsValid(Package))
	{
		OutError = TEXT("Failed to create new package for Niagara System.");
		return nullptr;
	}

	const EObjectFlags Flags = Package->GetFlags();
	UNiagaraSystem* NewNiagaraSystem;
	if (IsValid(SystemToCopy))
	{
		if (SystemToCopy->IsReadyToRun() == false)
		{
			SystemToCopy->WaitForCompilationComplete();
		}

		NewNiagaraSystem = Cast<UNiagaraSystem>(StaticDuplicateObject(SystemToCopy, Package, *AssetName, Flags, UNiagaraSystem::StaticClass()));
		NewNiagaraSystem->TemplateAssetDescription = FText();
		NewNiagaraSystem->Category = FText();

		// If the new system doesn't have a thumbnail image, check the thumbnail map of the original asset's UPackage
		if (NewNiagaraSystem->ThumbnailImage == nullptr)
		{
			FString ObjectFullName = SystemToCopy->GetFullName();
			FName ObjectName = FName(ObjectFullName);
			FString PackageFullName;
			ThumbnailTools::QueryPackageFileNameForObject(ObjectFullName, PackageFullName);
			FThumbnailMap ThumbnailMap;
			ThumbnailTools::ConditionallyLoadThumbnailsFromPackage(PackageFullName, { FName(ObjectFullName) }, ThumbnailMap);

			// there should always be a dummy thumbnail in here
			if (ThumbnailMap.Contains(ObjectName))
			{
				FObjectThumbnail Thumbnail = ThumbnailMap[ObjectName];
				// we only want to copy the thumbnail over if it's not a dummy
				if (Thumbnail.GetImageWidth() != 0 && Thumbnail.GetImageHeight() != 0)
				{
					ThumbnailTools::CacheThumbnail(NewNiagaraSystem->GetFullName(), &Thumbnail, NewNiagaraSystem->GetOutermost());
				}
			}
		}
	}
	else if (EmittersToAddToNewSystem.Num() > 0)
	{
		NewNiagaraSystem = NewObject<UNiagaraSystem>(Package, *AssetName, Flags | RF_Transactional);
		UNiagaraSystemFactoryNew::InitializeSystem(NewNiagaraSystem, true);

		for (const FVersionedNiagaraEmitter& EmitterToAddToNewSystem : EmittersToAddToNewSystem)
		{
			FNiagaraEditorUtilities::AddEmitterToSystem(*NewNiagaraSystem, *EmitterToAddToNewSystem.Emitter, EmitterToAddToNewSystem.Version);
		}
	}
	else
	{
		NewNiagaraSystem = NewObject<UNiagaraSystem>(Package, *AssetName, Flags | RF_Transactional);
		UNiagaraSystemFactoryNew::InitializeSystem(NewNiagaraSystem, true);
	}

	if (NewNiagaraSystem->GetEffectType() == nullptr && Settings->GetDefaultEffectType())
	{
		UNiagaraEffectType* DefaultEffectType = Settings->GetDefaultEffectType();
		NewNiagaraSystem->SetEffectType(DefaultEffectType);
	}

	NewNiagaraSystem->RequestCompile(false);

	Package->SetAssetAccessSpecifier(EAssetAccessSpecifier::Public);

	// Notify the asset registry
	FAssetRegistryModule::AssetCreated(NewNiagaraSystem);

	// Mark the package dirty...
	Package->MarkPackageDirty();

	return NewNiagaraSystem;
}

#endif