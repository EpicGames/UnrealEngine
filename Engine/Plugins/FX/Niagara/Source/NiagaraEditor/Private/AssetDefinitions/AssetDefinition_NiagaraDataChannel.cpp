// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinitions/AssetDefinition_NiagaraDataChannel.h"

#include "NiagaraEditorStyle.h"
#include "NiagaraDataChannel.h"
#include "NiagaraEditorUtilities.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraDataChannel)

#define LOCTEXT_NAMESPACE "UAssetDefinition_NiagaraDataChannel"

FLinearColor UAssetDefinition_NiagaraDataChannel::GetAssetColor() const
{
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.DataChannelDefinitions");
}

TSoftClassPtr<> UAssetDefinition_NiagaraDataChannel::GetAssetClass() const
{
	return UNiagaraDataChannelAsset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_NiagaraDataChannel::GetAssetCategories() const
{
	static FAssetCategoryPath AssetPaths[] = { FAssetCategoryPath(EAssetCategoryPaths::FX, LOCTEXT("NiagaraDataChannel_SubCategory", "Advanced"), ECategoryMenuType::Section) };
	return AssetPaths;
}



namespace MenuExtension_NiagaraDataChannel
{
	static void ExecuteMarkDependentCompilableAssetsDirty(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		FNiagaraEditorUtilities::MarkDependentCompilableAssetsDirty(CBContext->LoadSelectedObjects<UObject>());
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, [] {
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
			{
				FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UNiagaraDataChannelAsset::StaticClass());

				FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						{
							const TAttribute<FText> Label = LOCTEXT("MarkDependentCompilableAssetsDirtyLabel", "Mark Dependent Compilable Assets Dirty");
							const TAttribute<FText> ToolTip = LOCTEXT("MarkDependentCompilableAssetsDirtyToolTip", "Finds all niagara assets which depend on this asset either directly or indirectly, and marks them dirty so they can be saved with the latest version.");
							const FSlateIcon Icon = FSlateIcon();

							FToolUIAction UIAction;
							UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteMarkDependentCompilableAssetsDirty);
							InSection.AddMenuEntry("MarkDependentCompilableAssetsDirty", Label, ToolTip, Icon, UIAction);
						}
					}));
			}));
		});
}

#undef LOCTEXT_NAMESPACE


