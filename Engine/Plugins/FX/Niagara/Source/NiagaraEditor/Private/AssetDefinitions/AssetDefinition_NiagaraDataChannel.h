// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_NiagaraDataChannel.generated.h"

UCLASS()
class UAssetDefinition_NiagaraDataChannel : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "NiagaraDataChannel", "Niagara Data Channel"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
};
