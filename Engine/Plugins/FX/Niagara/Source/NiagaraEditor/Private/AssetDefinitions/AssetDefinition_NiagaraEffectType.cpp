// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraEffectType.h"
#include "NiagaraEditorStyle.h"
#include "SDetailsDiff.h"
#include "Toolkits/SimpleAssetEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraEffectType)

#define LOCTEXT_NAMESPACE "AssetTypeActions_NiagaraEffectType"

FLinearColor UAssetDefinition_NiagaraEffectType::GetAssetColor() const
{
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.EffectType");
}

EAssetCommandResult UAssetDefinition_NiagaraEffectType::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	if (DiffArgs.OldAsset == nullptr && DiffArgs.NewAsset == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}
	
	const TSharedRef<SDetailsDiff> DetailsDiff = SDetailsDiff::CreateDiffWindow(DiffArgs.OldAsset, DiffArgs.NewAsset, DiffArgs.OldRevision, DiffArgs.NewRevision, UNiagaraEffectType::StaticClass());
	// allow users to edit NewAsset if it's a local asset
	if (!FPackageName::IsTempPackage(DiffArgs.NewAsset->GetPackage()->GetName()))
	{
		DetailsDiff->SetOutputObject(DiffArgs.NewAsset);
	}
	return EAssetCommandResult::Handled;
}

EAssetCommandResult UAssetDefinition_NiagaraEffectType::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit || OpenArgs.OpenMethod == EAssetOpenMethod::View)
	{
		FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, OpenArgs.LoadObjects<UObject>());
		return EAssetCommandResult::Handled;
	}

	return EAssetCommandResult::Unhandled;
}

FAssetOpenSupport UAssetDefinition_NiagaraEffectType::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit || OpenSupportArgs.OpenMethod == EAssetOpenMethod::View); 

}

#undef LOCTEXT_NAMESPACE
