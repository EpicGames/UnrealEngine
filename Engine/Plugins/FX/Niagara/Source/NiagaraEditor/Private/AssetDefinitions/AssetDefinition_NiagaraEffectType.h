// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraEffectType.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_NiagaraEffectType.generated.h"

UCLASS()
class UAssetDefinition_NiagaraEffectType : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_NiagaraEffectType", "Niagara Effect Type"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UNiagaraEffectType::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = {
			FAssetCategoryPath(EAssetCategoryPaths::FX, NSLOCTEXT("Niagara", "NiagaraAssetSubMenu_Advanced", "Advanced"), ECategoryMenuType::Section)
		};
		
		return Categories;
	}
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	// UAssetDefinition End
};
