// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraEmitter.h"

#include "AssetToolsModule.h"
#include "Styling/SlateIconFinder.h"
#include "ContentBrowserMenuContexts.h"
#include "NiagaraEmitter.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraSystem.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraSystemFactoryNew.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "NiagaraEditorUtilities.h"
#include "Modules/ModuleManager.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraEmitter)

#define LOCTEXT_NAMESPACE "UAssetDefinition_NiagaraEmitter"

FLinearColor UAssetDefinition_NiagaraEmitter::GetAssetColor() const
{
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.Emitter");
}

EAssetCommandResult UAssetDefinition_NiagaraEmitter::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UNiagaraEmitter* Emitter : OpenArgs.LoadObjects<UNiagaraEmitter>())
	{
		if (!Emitter->VersionToOpenInEditor.IsValid())
		{
			Emitter->VersionToOpenInEditor = Emitter->GetExposedVersion().VersionGuid;
		}
		
		TSharedRef<FNiagaraSystemToolkit> SystemToolkit(new FNiagaraSystemToolkit());
		SystemToolkit->InitializeWithEmitter(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, *Emitter);
	}
	
	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_NiagaraEmitter
{
	void ExecuteNewNiagaraSystem(const FToolMenuContext& InContext)
	{
		const FString DefaultSuffix = TEXT("_System");

		FNiagaraSystemViewModelOptions SystemOptions;
		SystemOptions.bCanModifyEmittersFromTimeline = true;
		SystemOptions.EditMode = ENiagaraSystemViewModelEditMode::SystemAsset;
		
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

		TArray<UObject*> ObjectsToSync;
		
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		for (UNiagaraEmitter* Emitter : CBContext->LoadSelectedObjects<UNiagaraEmitter>())
		{
			// Determine an appropriate names
			FString Name;
			FString PackageName;
			AssetToolsModule.Get().CreateUniqueAssetName(Emitter->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

			// Create the factory used to generate the asset
			UNiagaraSystemFactoryNew* Factory = NewObject<UNiagaraSystemFactoryNew>();
			Factory->EmittersToAddToNewSystem.Add(FVersionedNiagaraEmitter(Emitter, Emitter->GetExposedVersion().VersionGuid));
			UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), UNiagaraSystem::StaticClass(), Factory);
				
			UNiagaraSystem* System = Cast<UNiagaraSystem>(NewAsset);
			if (System != nullptr)
			{
				ObjectsToSync.Add(NewAsset);
			}
		}

		if (ObjectsToSync.Num() > 0)
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
		}
	}

	void ExecuteCreateDuplicateParent(const FToolMenuContext& InContext)
	{
		const FString DefaultSuffix = TEXT("_Parent");

		TArray<UObject*> ObjectsToSync;
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		for (UNiagaraEmitter* Emitter : CBContext->LoadSelectedObjects<UNiagaraEmitter>())
		{
			FString Name;
			FString PackageName;
			AssetToolsModule.Get().CreateUniqueAssetName(Emitter->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

			// Create the factory used to generate the asset
			UNiagaraEmitter* NewEmitter = Cast<UNiagaraEmitter>(AssetToolsModule.Get().DuplicateAssetWithDialog(Name, FPackageName::GetLongPackagePath(PackageName), Emitter));

			if (NewEmitter != nullptr)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(Emitter);

				NewEmitter->Modify();
				NewEmitter->bIsInheritable = true;
				NewEmitter->GetLatestEmitterData()->GraphSource->MarkNotSynchronized(TEXT("Emitter created"));

				Emitter->Modify();
				Emitter->SetParent(FVersionedNiagaraEmitter(NewEmitter, NewEmitter->GetExposedVersion().VersionGuid));
				Emitter->GetLatestEmitterData()->GraphSource->MarkNotSynchronized(TEXT("Emitter parent changed"));

				ObjectsToSync.Add(NewEmitter);
			}
		}

		if (ObjectsToSync.Num() > 0)
		{
			AssetToolsModule.Get().OpenEditorForAssets(ObjectsToSync);

			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
		}
	}

	static void ExecuteMarkDependentCompilableAssetsDirty(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		FNiagaraEditorUtilities::MarkDependentCompilableAssetsDirty(CBContext->LoadSelectedObjects<UObject>());
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UNiagaraEmitter::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
				if (Context && Context->bCanBeModified)
				{
					{
						const TAttribute<FText> Label = LOCTEXT("Emitter_NewNiagaraSystem", "Create Niagara System");
						const TAttribute<FText> ToolTip = LOCTEXT("Emitter_NewNiagaraSystemTooltip", "Creates a niagara system using this emitter as a base.");
						const FSlateIcon Icon = FSlateIconFinder::FindIconForClass(UNiagaraSystem::StaticClass());

						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewNiagaraSystem);
						InSection.AddMenuEntry("Emitter_NewNiagaraSystem", Label, ToolTip, Icon, UIAction);
					}
					{
						const TAttribute<FText> Label = LOCTEXT("Emitter_CreateDuplicateParent", "Create Duplicate Parent");
						const TAttribute<FText> ToolTip = LOCTEXT("Emitter_CreateDuplicateParentTooltip", "Duplicate this emitter and set this emitter's parent to the new emitter.");
						const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate");

						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateDuplicateParent);
						InSection.AddMenuEntry("Emitter_CreateDuplicateParent", Label, ToolTip, Icon, UIAction);
					}
					{
						const TAttribute<FText> Label = LOCTEXT("MarkDependentCompilableAssetsDirtyLabel", "Mark Dependent Compilable Assets Dirty");
						const TAttribute<FText> ToolTip = LOCTEXT("MarkDependentCompilableAssetsDirtyToolTip", "Finds all niagara assets which depend on this asset either directly or indirectly, and marks them dirty so they can be saved with the latest version.");
						const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.DirtyBadge");
					
						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteMarkDependentCompilableAssetsDirty);
						InSection.AddMenuEntry("MarkDependentCompilableAssetsDirty", Label, ToolTip, Icon, UIAction);
					}
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
