// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraEmitter.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_NiagaraEmitter.generated.h"

UCLASS()
class UAssetDefinition_NiagaraEmitter : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_NiagaraEmitter", "Niagara Emitter"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UNiagaraEmitter::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::FX) };
		return Categories;
	}

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
