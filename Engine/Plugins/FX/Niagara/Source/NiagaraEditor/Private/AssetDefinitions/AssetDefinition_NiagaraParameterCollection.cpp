// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraParameterCollection.h"

#include "ContentBrowserMenuContexts.h"
#include "NiagaraParameterCollection.h"
#include "Toolkits/NiagaraParameterCollectionToolkit.h"
#include "NiagaraParameterCollectionFactoryNew.h"
#include "NiagaraEditorStyle.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "IAssetTools.h"

#include "ToolMenus.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraParameterCollection)

#define LOCTEXT_NAMESPACE "AssetTypeActions_NiagaraParameterCollection"

FLinearColor UAssetDefinition_NiagaraParameterCollection::GetAssetColor() const
{
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.ParameterCollection");
}

EAssetCommandResult UAssetDefinition_NiagaraParameterCollection::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UNiagaraParameterCollection* NPC : OpenArgs.LoadObjects<UNiagaraParameterCollection>())
	{
		TSharedRef< FNiagaraParameterCollectionToolkit > NewNiagaraNPCToolkit(new FNiagaraParameterCollectionToolkit());
		NewNiagaraNPCToolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, NPC);
	}

	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_NiagaraParameterCollection
{
	static void ExecuteCreateNewNPCI(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		IAssetTools::Get().CreateAssetsFrom<UNiagaraParameterCollection>(
			CBContext->LoadSelectedObjects<UNiagaraParameterCollection>(), UNiagaraParameterCollectionInstance::StaticClass(), TEXT("_Inst"), [](UNiagaraParameterCollection* SourceObject)
			{
				UNiagaraParameterCollectionInstanceFactoryNew* Factory = NewObject<UNiagaraParameterCollectionInstanceFactoryNew>();
				Factory->InitialParent = SourceObject;
				return Factory;
			}
		);
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UNiagaraParameterCollection::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("NewNPC", "Create Niagara Parameter Collection Instance");
					const TAttribute<FText> ToolTip = LOCTEXT("NewNPCTooltip", "Creates an instance of this Niagara Parameter Collection.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.MaterialInstanceActor");
					
					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteCreateNewNPCI);
					InSection.AddMenuEntry("Niagara_NewNPC", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
