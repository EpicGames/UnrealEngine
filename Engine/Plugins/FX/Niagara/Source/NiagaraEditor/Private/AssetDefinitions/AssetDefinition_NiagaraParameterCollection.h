// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraParameterCollection.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_NiagaraParameterCollection.generated.h"

UCLASS()
class UAssetDefinition_NiagaraParameterCollection : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_NiagaraParameterCollection", "Niagara Parameter Collection"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UNiagaraParameterCollection::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::FX, NSLOCTEXT("Niagara", "NiagaraAssetSubMenu_Advanced", "Advanced"), ECategoryMenuType::Section) };
		return Categories;
	}

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
