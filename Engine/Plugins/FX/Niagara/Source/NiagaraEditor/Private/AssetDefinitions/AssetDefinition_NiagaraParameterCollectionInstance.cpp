// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraParameterCollectionInstance.h"
#include "NiagaraParameterCollection.h"
#include "Toolkits/NiagaraParameterCollectionToolkit.h"
#include "NiagaraEditorStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraParameterCollectionInstance)

#define LOCTEXT_NAMESPACE "UAssetDefinition_NiagaraParameterCollectionInstance"

FLinearColor UAssetDefinition_NiagaraParameterCollectionInstance::GetAssetColor() const
{ 
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.ParameterCollectionInstance"); 
}

EAssetCommandResult UAssetDefinition_NiagaraParameterCollectionInstance::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UNiagaraParameterCollectionInstance* Instance : OpenArgs.LoadObjects<UNiagaraParameterCollectionInstance>())
	{
		TSharedRef< FNiagaraParameterCollectionToolkit > NewNiagaraNPCToolkit(new FNiagaraParameterCollectionToolkit());
		NewNiagaraNPCToolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Instance);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
