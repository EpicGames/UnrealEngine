// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraParameterDefinitions.h"
#include "NiagaraEditorStyle.h"
#include "Toolkits/NiagaraParameterDefinitionsToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraParameterDefinitions)

FLinearColor UAssetDefinition_NiagaraParameterDefinitions::GetAssetColor() const
{
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.ParameterDefinitions");
}

EAssetCommandResult UAssetDefinition_NiagaraParameterDefinitions::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UNiagaraParameterDefinitions* ParameterDefinitions : OpenArgs.LoadObjects<UNiagaraParameterDefinitions>())
   	{
		TSharedRef<FNiagaraParameterDefinitionsToolkit> NewNiagaraParameterDefinitionsToolkit(new FNiagaraParameterDefinitionsToolkit());
		NewNiagaraParameterDefinitionsToolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, ParameterDefinitions);
	}

	return EAssetCommandResult::Handled;
}
