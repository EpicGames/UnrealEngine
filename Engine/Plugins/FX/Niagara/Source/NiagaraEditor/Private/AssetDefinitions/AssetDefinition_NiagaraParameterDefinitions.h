// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraParameterDefinitions.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_NiagaraParameterDefinitions.generated.h"

UCLASS()
class UAssetDefinition_NiagaraParameterDefinitions : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_NiagaraParameterDefinitions", "Niagara Parameter Definitions"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UNiagaraParameterDefinitions::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::FX, NSLOCTEXT("Niagara", "NiagaraAssetSubMenu_Advanced", "Advanced"), ECategoryMenuType::Section) };
		return Categories;
	}

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
