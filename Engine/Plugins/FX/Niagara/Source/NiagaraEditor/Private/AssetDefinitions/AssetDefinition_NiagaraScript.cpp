// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraScript.h"
#include "Toolkits/NiagaraScriptToolkit.h"
#include "NiagaraEditorStyle.h"
#include "AssetRegistry/AssetData.h"
#include "NiagaraEditorUtilities.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraScript)

#define LOCTEXT_NAMESPACE "NiagaraScriptAssetTypeActions"

const FName UAssetDefinition_NiagaraScript::FunctionScriptName = "Niagara Function Script";
const FText UAssetDefinition_NiagaraScript::FunctionScriptNameText = LOCTEXT("NiagaraFunctionScript", "Niagara Function Script");
const FName UAssetDefinition_NiagaraScript::ModuleScriptName = "Niagara Module Script";
const FText UAssetDefinition_NiagaraScript::ModuleScriptNameText = LOCTEXT("NiagaraModuleScript", "Niagara Module Script");
const FName UAssetDefinition_NiagaraScript::DynamicInputScriptName = "Niagara Dynamic Input Script";
const FText UAssetDefinition_NiagaraScript::DynamicInputScriptNameText = LOCTEXT("NiagaraDynamicInputScript", "Niagara Dynamic Input Script");

FLinearColor UAssetDefinition_NiagaraScript::GetAssetColor() const
{ 
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.Script"); 
}

FText UAssetDefinition_NiagaraScript::GetAssetDisplayName(const FAssetData& AssetData) const
{
	static const FName NAME_Usage(TEXT("Usage"));
	const FAssetDataTagMapSharedView::FFindTagResult Usage = AssetData.TagsAndValues.FindTag(NAME_Usage);

	if (Usage.IsSet())
	{
		static const FString FunctionString(TEXT("Function"));
		static const FString ModuleString(TEXT("Module"));
		static const FString DynamicInputString(TEXT("DynamicInput"));
		
		if (Usage.GetValue() == FunctionString)
		{
			return FunctionScriptNameText;
		}
		else if (Usage.GetValue() == ModuleString)
		{
			return ModuleScriptNameText;
		}
		else if (Usage.GetValue() == DynamicInputString)
		{
			return DynamicInputScriptNameText;
		}
	}

	return Super::GetAssetDisplayName(AssetData);
}

FAssetOpenSupport UAssetDefinition_NiagaraScript::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit || OpenSupportArgs.OpenMethod == EAssetOpenMethod::View); 
}

EAssetCommandResult UAssetDefinition_NiagaraScript::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UNiagaraScript* Script : OpenArgs.LoadObjects<UNiagaraScript>())
	{
		TSharedRef<FNiagaraScriptToolkit> NewNiagaraScriptToolkit(new FNiagaraScriptToolkit());
		NewNiagaraScriptToolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Script);
	}
	
	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_NiagaraScript
{
	static void ExecuteMarkDependentCompilableAssetsDirty(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		FNiagaraEditorUtilities::MarkDependentCompilableAssetsDirty(CBContext->LoadSelectedObjects<UObject>());
	}
	
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UNiagaraScript::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("MarkDependentCompilableAssetsDirtyLabel", "Mark Dependent Compilable Assets Dirty");
					const TAttribute<FText> ToolTip = LOCTEXT("MarkDependentCompilableAssetsDirtyToolTip", "Finds all niagara assets which depend on this asset either directly or indirectly, and marks them dirty so they can be saved with the latest version.");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.DirtyBadge");
					
					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteMarkDependentCompilableAssetsDirty);
					InSection.AddMenuEntry("MarkDependentCompilableAssetsDirty", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
