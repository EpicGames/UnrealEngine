// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraScript.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_NiagaraScript.generated.h"

UCLASS()
class UAssetDefinition_NiagaraScript : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	static const FName FunctionScriptName;
	static const FText FunctionScriptNameText;

	static const FName ModuleScriptName;
	static const FText ModuleScriptNameText;
	
	static const FName DynamicInputScriptName;
	static const FText DynamicInputScriptNameText;

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_NiagaraScript", "Niagara Script"); }
	virtual FText GetAssetDisplayName(const FAssetData& AssetData) const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UNiagaraScript::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::FX, NSLOCTEXT("Niagara", "NiagaraAssetSubMenu_Script", "Script"), ECategoryMenuType::Section) };
		return Categories;
	}

	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
