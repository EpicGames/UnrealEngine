// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraSimCache.h"
#include "NiagaraEditorSimCacheUtils.h"

#include "ContentBrowserMenuContexts.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraSimCacheJson.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Toolkits/NiagaraSimCacheToolkit.h"
#include "ToolMenus.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraSimCache)

#define LOCTEXT_NAMESPACE "UAssetDefinition_NiagaraSimCache"

FLinearColor UAssetDefinition_NiagaraSimCache::GetAssetColor() const
{
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.SimCache").ToFColor(true);
}

EAssetCommandResult UAssetDefinition_NiagaraSimCache::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UNiagaraSimCache* SimCache : OpenArgs.LoadObjects<UNiagaraSimCache>())
	{
		const TSharedRef< FNiagaraSimCacheToolkit > NewNiagaraSimCacheToolkit(new FNiagaraSimCacheToolkit());
		NewNiagaraSimCacheToolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, SimCache);
	}

	return EAssetCommandResult::Handled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_NiagaraSimCache
{	
	static void ExportToDisk(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UNiagaraSimCache*> Assets = CBContext->LoadSelectedObjects<UNiagaraSimCache>();
		FNiagaraEditorSimCacheUtils::ExportToDisk(Assets);
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped("Niagara SimCache");
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UNiagaraSimCache::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("ExportToDisk", "Export To Disk");
					const TAttribute<FText> ToolTip = LOCTEXT("ExportToDiskTooltip", "Exports the raw data for each frame to disk. Note that data from data interfaces is only exported if they implement support for it.");
					const FSlateIcon Icon = FSlateIcon();

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExportToDisk);
					InSection.AddMenuEntry("ExportToDisk", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
