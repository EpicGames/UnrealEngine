// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraStatelessEmitterTemplate.h"
#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "Toolkits/NiagaraStatelessEmitterTemplateToolkit.h"

#include "ContentBrowserMenuContexts.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "NiagaraEditorStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ToolMenus.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraStatelessEmitterTemplate)

#define LOCTEXT_NAMESPACE "AssetDefinition_NiagaraStatelessEmitterTemplate"

FText UAssetDefinition_NiagaraStatelessEmitterTemplate::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_NiagaraStatelessEmitterTemplate", "Niagara Lightweight Emitter Template");
}

FLinearColor UAssetDefinition_NiagaraStatelessEmitterTemplate::GetAssetColor() const
{
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.StatelessEmitterTemplate").ToFColor(true);
}

TSoftClassPtr<UObject> UAssetDefinition_NiagaraStatelessEmitterTemplate::GetAssetClass() const
{
	return UNiagaraStatelessEmitterTemplate::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_NiagaraStatelessEmitterTemplate::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::FX / NSLOCTEXT("Niagara", "NiagaraAssetSubMenu_Advanced", "Advanced") };
	return Categories;
}

EAssetCommandResult UAssetDefinition_NiagaraStatelessEmitterTemplate::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UNiagaraStatelessEmitterTemplate* EmitterTemplate : OpenArgs.LoadObjects<UNiagaraStatelessEmitterTemplate>())
	{
		const TSharedRef<FNiagaraStatelessEmitterTemplateToolkit> NewToolkit(new FNiagaraStatelessEmitterTemplateToolkit());
		NewToolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, EmitterTemplate);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
