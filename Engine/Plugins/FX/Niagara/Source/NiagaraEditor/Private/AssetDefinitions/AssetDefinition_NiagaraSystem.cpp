// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraSystem.h"
#include "NiagaraSystem.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "NiagaraEditorStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraSystem)

FLinearColor UAssetDefinition_NiagaraSystem::GetAssetColor() const
{
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.System");
}

EAssetCommandResult UAssetDefinition_NiagaraSystem::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UNiagaraSystem* System : OpenArgs.LoadObjects<UNiagaraSystem>())
	{
		TSharedRef<FNiagaraSystemToolkit> NiagaraSystemToolkit = MakeShared<FNiagaraSystemToolkit>();
		NiagaraSystemToolkit->InitializeWithSystem(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, *System);
	}

	return EAssetCommandResult::Handled;
}
