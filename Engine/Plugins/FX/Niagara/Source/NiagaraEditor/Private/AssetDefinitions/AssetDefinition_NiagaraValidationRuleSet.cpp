// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NiagaraValidationRuleSet.h"
#include "NiagaraEditorStyle.h"
#include "SDetailsDiff.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NiagaraValidationRuleSet)

FLinearColor UAssetDefinition_NiagaraValidationRuleSet::GetAssetColor() const
{
	return FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.AssetColors.ValidationRuleSet");
}

EAssetCommandResult UAssetDefinition_NiagaraValidationRuleSet::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	if (DiffArgs.OldAsset == nullptr && DiffArgs.NewAsset == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}
	
	const TSharedRef<SDetailsDiff> DetailsDiff = SDetailsDiff::CreateDiffWindow(DiffArgs.OldAsset, DiffArgs.NewAsset, DiffArgs.OldRevision, DiffArgs.NewRevision, UNiagaraValidationRuleSet::StaticClass());
	// allow users to edit NewAsset if it's a local asset
	if (!FPackageName::IsTempPackage(DiffArgs.NewAsset->GetPackage()->GetName()))
	{
		DetailsDiff->SetOutputObject(DiffArgs.NewAsset);
	}
	return EAssetCommandResult::Handled;
}
