// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraValidationRuleSet.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_NiagaraValidationRuleSet.generated.h"

UCLASS()
class UAssetDefinition_NiagaraValidationRuleSet : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_NiagaraValidationRuleSet", "Niagara Validation Rule Set"); }
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UNiagaraValidationRuleSet::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = {
			FAssetCategoryPath(EAssetCategoryPaths::FX, NSLOCTEXT("Niagara", "NiagaraAssetSubMenu_Advanced", "Advanced"), ECategoryMenuType::Section)
		};
		
		return Categories;
	}
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	// UAssetDefinition End
};
