// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/NiagaraDumpBytecodeCommandlet.h"

#include "AssetCompilingManager.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "HAL/FileManager.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "NiagaraEmitter.h"
#include "NiagaraScript.h"
#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDumpBytecodeCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraDumpBytecodeCommandlet, Log, All);

UNiagaraDumpByteCodeCommandlet::UNiagaraDumpByteCodeCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UNiagaraDumpByteCodeCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);

	if (!FParse::Value(*Params, TEXT("AuditOutputFolder="), AuditOutputFolder))
	{
		// No output folder specified. Use the default folder.
		AuditOutputFolder = FPaths::ProjectSavedDir() / TEXT("Audit");
	}

	// Add a timestamp to the folder
	AuditOutputFolder /= FDateTime::Now().ToString();

	FParse::Value(*Params, TEXT("FilterCollection="), FilterCollection);

	// Package Paths
	FString PackagePathsString;
	if (FParse::Value(*Params, TEXT("PackagePaths="), PackagePathsString, false))
	{
		TArray<FString> PackagePathsStrings;
		PackagePathsString.ParseIntoArray(PackagePathsStrings, TEXT(","));
		for (const FString& v : PackagePathsStrings)
		{
			PackagePaths.Add(FName(v));
		}
	}

	if (PackagePaths.Num() == 0)
	{
		PackagePaths.Add(FName(TEXT("/Game")));
	}

	if (Switches.Contains("IncludeDev"))
	{
		IncludeDeveloperFolder = true;
	}

	if (Switches.Contains("COOKED"))
	{
		ForceBakedRapidIteration = true;
		ForceAttributeTrimming = true;
	}

	if (Switches.Contains("BAKED"))
	{
		ForceBakedRapidIteration = true;
	}

	if (Switches.Contains("TRIMMED"))
	{
		ForceAttributeTrimming = true;
	}

	if (Switches.Contains("HLSL"))
	{
		DumpTranslatedHlsl = true;
	}

	if (Switches.Contains("GPU"))
	{
		DumpGpuScripts = true;
	}

	ProcessNiagaraScripts();

	return 0;
}

void UNiagaraDumpByteCodeCommandlet::ProcessBatch(TArray<FAssetData>& BatchAssets)
{
	const UEnum* UsageEnum = StaticEnum<ENiagaraScriptUsage>();
	const int32 BatchSize = BatchAssets.Num();

	TArray<TObjectPtr<UNiagaraSystem>> LoadedSystems;
	TArray<TObjectPtr<UNiagaraSystem>> PendingSystems;
	TArray<TObjectPtr<UNiagaraSystem>> CompiledSystems;
	TArray<TObjectPtr<UNiagaraSystem>> CompletedSystems;

	LoadedSystems.Reserve(BatchSize);
	PendingSystems.Reserve(BatchSize);
	CompiledSystems.Reserve(BatchSize);
	CompletedSystems.Reserve(BatchSize);

	// FullyLoad all the packages
	for (const FAssetData& AssetIt : BatchAssets)
	{
		const FString SystemName = AssetIt.GetObjectPathString();
		FNameBuilder PackageNameBuilder(AssetIt.PackageName);

		if (UPackage* Package = ::LoadPackage(nullptr, *PackageNameBuilder, LOAD_None))
		{
			FNameBuilder ShortSystemNameBuilder(AssetIt.AssetName);

			Package->FullyLoad();

			if (UNiagaraSystem* NiagaraSystem = FindObject<UNiagaraSystem>(Package, *ShortSystemNameBuilder))
			{
				LoadedSystems.Emplace(NiagaraSystem);
			}
		}
		else
		{
			UE_LOGF(LogNiagaraDumpBytecodeCommandlet, Warning, "Failed to load package %ls processing %ls", *PackageNameBuilder, *SystemName);
			continue;
		}
	}

	bool bProcessing = true;

	auto AdvanceSystems = [](TArray<TObjectPtr<UNiagaraSystem>>& InSystems, TArray<TObjectPtr<UNiagaraSystem>>& OutSystems, TFunction<bool(UNiagaraSystem*)> Op) -> void
	{
		while (!InSystems.IsEmpty())
		{
			for (int32 InIt = 0; InIt < InSystems.Num(); ++InIt)
			{
				UNiagaraSystem* InSystem = InSystems[InIt];
				if (Op(InSystem))
				{
					InSystems.RemoveAt(InIt, EAllowShrinking::No);
					OutSystems.Add(InSystem);
				}
			}

			FAssetCompilingManager::Get().ProcessAsyncTasks(true);
		}
	};

	AdvanceSystems(LoadedSystems, PendingSystems, [](UNiagaraSystem* System) -> bool
	{
		return !System->HasActiveCompilations() || System->PollForCompilationComplete();
	});

	AdvanceSystems(PendingSystems, CompiledSystems, [this](UNiagaraSystem* System) -> bool
	{
		if (ForceBakedRapidIteration || ForceAttributeTrimming)
		{
			if (ForceBakedRapidIteration)
			{
				System->SetBakeOutRapidIterationOnCook(true);
			}
			if (ForceAttributeTrimming)
			{
				System->SetTrimAttributesOnCook(true);
			}

			System->RequestCompile(true);
		}

		return true;
	});

	AdvanceSystems(CompiledSystems, CompletedSystems, [](UNiagaraSystem* System) -> bool
	{
		return System->PollForCompilationComplete();
	});

	for (UNiagaraSystem* NiagaraSystem : CompletedSystems)
	{
		const FString SystemPathName = NiagaraSystem->GetPathName();
		const FString HashedPathName = FString::Printf(TEXT("%08x"), GetTypeHash(SystemPathName));

		IFileManager::Get().MakeDirectory(*(AuditOutputFolder / HashedPathName));
		DumpByteCode(NiagaraSystem->GetSystemSpawnScript(), SystemPathName, HashedPathName, TEXT("SystemSpawnScript"));
		DumpByteCode(NiagaraSystem->GetSystemUpdateScript(), SystemPathName, HashedPathName, TEXT("SystemUpdateScript"));

		for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles())
		{
			if (!EmitterHandle.GetIsEnabled())
			{
				continue;
			}

			if (FVersionedNiagaraEmitterData* Emitter = EmitterHandle.GetEmitterData())
			{
				const bool bCpuTarget = Emitter->SimTarget == ENiagaraSimTarget::CPUSim;

				if (bCpuTarget || DumpGpuScripts)
				{
					FString EmitterName = EmitterHandle.GetUniqueInstanceName();

					// trim the emitter name down to 64 char at most
					const int32 MaxEmitterNameLength = 64;
					if (EmitterName.Len() > MaxEmitterNameLength)
					{
						UE_LOGF(LogNiagaraDumpBytecodeCommandlet, Warning, "Emitter encountered with excessively long unique name: System: %ls - Unique Emitter: %ls", *NiagaraSystem->GetFullName(), *EmitterName);
						EmitterName.MidInline(0, MaxEmitterNameLength);
					}

					IFileManager::Get().MakeDirectory(*(AuditOutputFolder / HashedPathName / EmitterName));

					if (bCpuTarget)
					{
						TArray<UNiagaraScript*> EmitterScripts;
						Emitter->GetScripts(EmitterScripts);

						for (const auto* EmitterScript : EmitterScripts)
						{
							DumpByteCode(EmitterScript, SystemPathName, HashedPathName, EmitterName / UsageEnum->GetNameStringByValue(static_cast<int64>(EmitterScript->GetUsage())));
						}
					}
					else
					{
						if (const UNiagaraScript* GPUComputeScript = Emitter->GetGPUComputeScript())
						{
							const FString UsageName = UsageEnum->GetNameStringByValue(static_cast<int64>(GPUComputeScript->GetUsage()));

							DumpShaderScript(GPUComputeScript, SystemPathName, HashedPathName, EmitterName / UsageName);
						}
					}
				}
			}
		}
	}

	::CollectGarbage(RF_NoFlags);
}

void UNiagaraDumpByteCodeCommandlet::ProcessNiagaraScripts()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	FARFilter Filter;
	Filter.PackagePaths = PackagePaths;
	Filter.bRecursivePaths = true;

	Filter.ClassPaths.Add(UNiagaraSystem::StaticClass()->GetClassPathName());
	if (!FilterCollection.IsEmpty())
	{
		ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
		if (CollectionManager.TryParseCollectionPath(FilterCollection, &CollectionContainer, &CollectionName, &ShareType))
		{
			CollectionContainer->GetObjectsInCollection(CollectionName, ShareType, Filter.SoftObjectPaths, ECollectionRecursionFlags::SelfAndChildren);
		}
	}

	TArray<FAssetData> AssetList;
	AssetRegistry.GetAssets(Filter, AssetList);

	const double StartProcessNiagaraSystemsTime = FPlatformTime::Seconds();
	constexpr uint32 BatchPackageCount = 256;

	TArray<FAssetData> BatchAssets;

	//  Iterate over all scripts
	const FString DevelopersFolder = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir().LeftChop(1));
	for (const FAssetData& AssetIt : AssetList)
	{
		if (!IncludeDeveloperFolder && AssetIt.PackageName.ToString().StartsWith(DevelopersFolder))
		{
			// Skip developer folders
			continue;
		}

		if (BatchAssets.Num() >= BatchPackageCount)
		{
			ProcessBatch(BatchAssets);
			BatchAssets.Reset();
		}

		BatchAssets.Add(AssetIt);
	}

	if (!BatchAssets.IsEmpty())
	{
		ProcessBatch(BatchAssets);
		BatchAssets.Reset();
	}

	// sort the data alphabetically (based on MetaData.FullName
	ScriptMetaData.StableSort([](const FScriptMetaData& Lhs, const FScriptMetaData& Rhs)
	{
		return Lhs.FullName.Compare(Rhs.FullName, ESearchCase::IgnoreCase) < 0;
	});

	// create the xml for the meta data
	{
		const FString MetaDataFileName = AuditOutputFolder / TEXT("NiagaraScripts.xml");
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*MetaDataFileName));
		if (FileArchive)
		{
			TUniquePtr<FOutputDeviceArchiveWrapper> OutputStream(new FOutputDeviceArchiveWrapper(FileArchive.Get()));
			if(OutputStream)
			{
				OutputStream->Log(TEXT("<?xml version='1.0' ?>"));
				OutputStream->Log(TEXT("<Scripts>"));
				for (const auto& MetaData : ScriptMetaData)
				{
					OutputStream->Log(TEXT("\t<Script>"));
					OutputStream->Logf(TEXT("\t\t<Hash>%s</Hash>"), *MetaData.SystemHash);
					OutputStream->Logf(TEXT("\t\t<Name>%s</Name>"), *MetaData.FullName);
					OutputStream->Logf(TEXT("\t\t<OpCount>%d</OpCount>"), MetaData.OpCount);
					OutputStream->Logf(TEXT("\t\t<RegisterCount>%d</RegisterCount>"), MetaData.RegisterCount);
					OutputStream->Logf(TEXT("\t\t<ConstantCount>%d</ConstantCount>"), MetaData.ConstantCount);
					OutputStream->Logf(TEXT("\t\t<AttributeCount>%d</AttributeCount>"), MetaData.AttributeCount);
					OutputStream->Log(TEXT("\t</Script>"));
				}
				OutputStream->Log(TEXT("</Scripts>"));
			}
		}
	}

	// create the csv for the meta data
	{
		const FString MetaDataFileName = AuditOutputFolder / TEXT("NiagaraScripts.csv");
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*MetaDataFileName));
		if (FileArchive)
		{
			TUniquePtr<FOutputDeviceArchiveWrapper> OutputStream(new FOutputDeviceArchiveWrapper(FileArchive.Get()));
			if (OutputStream)
			{
				OutputStream->Log(TEXT("Hash, Name, OpCount, RegisteredCount, ConstantCount, AttributeCount"));
				for (const auto& MetaData : ScriptMetaData)
				{
					OutputStream->Logf(TEXT("%s, %s, %d, %d, %d, %d"),
						*MetaData.SystemHash,
						*MetaData.FullName,
						MetaData.OpCount,
						MetaData.RegisterCount,
						MetaData.ConstantCount,
						MetaData.AttributeCount);
				}
			}
		}
	}

	ShaderMetaData.StableSort([](const FShaderMetaData& Lhs, const FShaderMetaData& Rhs)
	{
		return Lhs.FullName.Compare(Rhs.FullName, ESearchCase::IgnoreCase) < 0;
	});

	// create the csv for the meta data
	{
		const FString MetaDataFileName = AuditOutputFolder / TEXT("NiagaraShaders.csv");
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*MetaDataFileName));
		if (FileArchive)
		{
			TUniquePtr<FOutputDeviceArchiveWrapper> OutputStream(new FOutputDeviceArchiveWrapper(FileArchive.Get()));
			if (OutputStream)
			{
				OutputStream->Log(TEXT("Hash, Name, PermutationIndex, ContentHash, Size(Bytes)"));
				for (const auto& MetaData : ShaderMetaData)
				{
					OutputStream->Logf(TEXT("%s, %s, %d, %s, %d"),
						*MetaData.SystemHash,
						*MetaData.FullName,
						MetaData.PermutationIndex,
						*MetaData.ShaderHash,
						MetaData.ByteCount);
				}
			}
		}
	}

	// Probably don't need to do this, but just in case we have any 'hanging' packages 
	// and more processing steps are added later, let's clean up everything...
	::CollectGarbage(RF_NoFlags);

	double ProcessNiagaraSystemsTime = FPlatformTime::Seconds() - StartProcessNiagaraSystemsTime;
	UE_LOGF(LogNiagaraDumpBytecodeCommandlet, Log, "Took %5.3f seconds to process referenced Niagara systems...", ProcessNiagaraSystemsTime);
}

void UNiagaraDumpByteCodeCommandlet::DumpByteCode(const UNiagaraScript* Script, const FString& PathName, const FString& HashName, const FString& FilePath)
{
	if (!Script)
	{
		return;
	}

	const auto& ExecData = Script->GetVMExecutableData();

	FScriptMetaData& MetaData = ScriptMetaData.AddZeroed_GetRef();
	MetaData.SystemHash = HashName;
	MetaData.FullName = PathName / FilePath;
	MetaData.RegisterCount = ExecData.NumTempRegisters;
	MetaData.OpCount = ExecData.LastOpCount;
	MetaData.ConstantCount = ExecData.InternalParameters.GetTableSize() / 4;
	MetaData.AttributeCount = 0;

	for (const FNiagaraVariableBase& Var : ExecData.Attributes)
	{
		MetaData.AttributeCount += Var.GetType().GetSize() / 4;
	}

	if (Script)
	{
		const FString FullFilePath = AuditOutputFolder / HashName / FilePath + TEXT(".vm");

		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*FullFilePath));
		if (!FileArchive)
		{
			UE_LOGF(LogNiagaraDumpBytecodeCommandlet, Warning, "Failed to create output stream %ls", *FullFilePath);
		}

		TUniquePtr<FOutputDeviceArchiveWrapper> OutputStream(new FOutputDeviceArchiveWrapper(FileArchive.Get()));

		const auto& VMData = Script->GetVMExecutableData();

		TArray<FString> AssemblyLines;
		VMData.LastAssemblyTranslation.ParseIntoArrayLines(AssemblyLines, false);

		// we want to translate all instances of OP_X into the string from the VMVector enum
		for (FString& CurrentLine : AssemblyLines)
		{
			int OpStartIndex = CurrentLine.Find(TEXT("OP_"), ESearchCase::CaseSensitive);
			if (OpStartIndex != INDEX_NONE)
			{
				const int32 OpEndStartSearch = OpStartIndex + 3;
				int OpEndIndex = CurrentLine.Find(TEXT("("), ESearchCase::IgnoreCase, ESearchDir::FromStart, OpEndStartSearch);
				if (OpEndIndex == INDEX_NONE)
				{
					OpEndIndex = CurrentLine.Find(TEXT(";"), ESearchCase::IgnoreCase, ESearchDir::FromStart, OpEndStartSearch);
				}

				if (OpEndIndex != INDEX_NONE)
				{
					const FString OpIndexString = CurrentLine.Mid(OpEndStartSearch, OpEndIndex - OpEndStartSearch);
					int OpIndexValue = FCString::Atoi(*OpIndexString);

					FString LinePrefix = CurrentLine.Left(OpStartIndex);
					FString LineSuffix = CurrentLine.RightChop(OpEndIndex);

					
					CurrentLine = LinePrefix + TEXT(":") + VectorVM::GetOpName((EVectorVMOp)OpIndexValue) + LineSuffix;
				}
			}
			OutputStream->Log(CurrentLine);
		}
	}

	if (Script && DumpTranslatedHlsl)
	{
		const FString FullFilePath = AuditOutputFolder / HashName / FilePath + TEXT(".usf");

		FFileHelper::SaveStringToFile(Script->GetVMExecutableData().LastHlslTranslation, *FullFilePath);
	}
}

void UNiagaraDumpByteCodeCommandlet::DumpShaderScript(const UNiagaraScript* GPUComputeScript, const FString& PathName, const FString& HashName, const FString& FilePath)
{
	if (!GPUComputeScript)
	{
		return;
	}

	FScriptMetaData& MetaData = ScriptMetaData.AddZeroed_GetRef();
	MetaData.SystemHash = HashName;
	MetaData.FullName = PathName / FilePath;
	MetaData.RegisterCount = 0;
	MetaData.OpCount = 0;
	MetaData.ConstantCount = 0;
	MetaData.AttributeCount = 0;

	if (GPUComputeScript && DumpTranslatedHlsl)
	{
		const FString FullFilePath = AuditOutputFolder / HashName / FilePath + TEXT(".usf");

		FFileHelper::SaveStringToFile(GPUComputeScript->GetVMExecutableData().LastHlslTranslationGPU, *FullFilePath);
	}

#if !UE_BUILD_SHIPPING
	if (const FNiagaraShaderScript* ShaderScript = GPUComputeScript->GetRenderThreadScript())
	{
		if (const FNiagaraShaderMap* ShaderMap = ShaderScript->GetGameThreadShaderMap())
		{
			if (const FNiagaraShaderMapContent* ShaderMapContent = ShaderMap->GetContent())
			{

				FShaderListReport ShaderListReport;
				ShaderListReport.BindLambda([&HashName, FullName = PathName / FilePath, this](
						const TCHAR* FriendlyName,
						const TCHAR* OwnerName,
						const TCHAR* VertexFactory,
						const TCHAR* ShaderType,
						uint32 Permutation,
						uint32 Bytes,
						const TCHAR* Hash,
						void* Ptr)
				{
					FShaderMetaData& PermutationData = ShaderMetaData.AddDefaulted_GetRef();
					PermutationData.SystemHash = HashName;
					PermutationData.FullName = FullName;
					PermutationData.PermutationIndex = Permutation;
					PermutationData.VertexFactory = VertexFactory;
					PermutationData.ShaderType = ShaderType;
					PermutationData.ByteCount = Bytes;
					PermutationData.ShaderHash = Hash;
				});

				ShaderMapContent->DumpShaderList(*ShaderMap, ShaderListReport);
			}
		}
	}
#endif // !UE_BUILD_SHIPPING
}
