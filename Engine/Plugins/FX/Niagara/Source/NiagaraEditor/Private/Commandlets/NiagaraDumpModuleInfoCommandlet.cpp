// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/NiagaraDumpModuleInfoCommandlet.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "HAL/FileManager.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "NiagaraSystem.h"
#include "NiagaraScriptSource.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraParameterDefinitions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDumpModuleInfoCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraDumpModuleInfoCommandlet, Log, All);

UNiagaraDumpModuleInfoCommandlet::UNiagaraDumpModuleInfoCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UNiagaraDumpModuleInfoCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);

	if (!FParse::Value(*Params, TEXT("AuditOutputFolder="), AuditOutputFolder))
	{
		// No output folder specified. Use the default folder.
		AuditOutputFolder = FPaths::ProjectSavedDir() / TEXT("Audit");
	}

	// Add a timestamp to the folder
	AuditOutputFolder /= FDateTime::Now().ToString();

	FParse::Value(*Params, TEXT("FilterCollection="), FilterCollection);

	// Package Paths
	FString PackagePathsString;
	if (FParse::Value(*Params, TEXT("PackagePaths="), PackagePathsString, false))
	{
		TArray<FString> PackagePathsStrings;
		PackagePathsString.ParseIntoArray(PackagePathsStrings, TEXT(","));
		for (const FString& v : PackagePathsStrings)
		{
			PackagePaths.Add(FName(v));
		}
	}

	if (PackagePaths.Num() == 0)
	{
		//PackagePaths.Add(FName(TEXT("/Game")));
		PackagePaths.Add(FName(TEXT("/Plugins")));
		PackagePaths.Add(FName(TEXT("/Niagara")));
		PackagePaths.Add(FName(TEXT("/NiagaraFluids")));
		PackagePaths.Add(FName(TEXT("/Engine")));
	}

	ProcessNiagaraScripts();
	ProcessNiagaraParameterDefinitions();

	return 0;
}

void UNiagaraDumpModuleInfoCommandlet::ProcessNiagaraParameterDefinitions()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	FARFilter Filter;
	Filter.PackagePaths = PackagePaths;
	Filter.bRecursivePaths = true;

	Filter.ClassPaths.Add(UNiagaraParameterDefinitions::StaticClass()->GetClassPathName());
	if (!FilterCollection.IsEmpty())
	{
		ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
		if (CollectionManager.TryParseCollectionPath(FilterCollection, &CollectionContainer, &CollectionName, &ShareType))
		{
			CollectionContainer->GetObjectsInCollection(CollectionName, ShareType, Filter.SoftObjectPaths, ECollectionRecursionFlags::SelfAndChildren);
		}
	}

	TArray<FAssetData> AssetList;
	AssetRegistry.GetAssets(Filter, AssetList);

	//  Iterate over all scripts
	const FString DevelopersFolder = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir().LeftChop(1));
	FString LastPackageName = TEXT("");
	UPackage* CurrentPackage = nullptr;
	for (const FAssetData& AssetIt : AssetList)
	{
		const FString SystemName = AssetIt.GetObjectPathString();
		const FString PackageName = AssetIt.PackageName.ToString();

		if (PackageName.StartsWith(DevelopersFolder))
		{
			// Skip developer folders
			continue;
		}

		if (PackageName != LastPackageName)
		{
			UPackage* Package = ::LoadPackage(nullptr, *PackageName, LOAD_None);
			if (Package != nullptr)
			{
				LastPackageName = PackageName;
				Package->FullyLoad();
				CurrentPackage = Package;
			}
			else
			{
				UE_LOGF(LogNiagaraDumpModuleInfoCommandlet, Warning, "Failed to load package %ls processing %ls", *PackageName, *SystemName);
				CurrentPackage = nullptr;
			}
		}

		const FString ShorterSystemName = AssetIt.AssetName.ToString();
		UNiagaraParameterDefinitions* NiagaraParameterDefinitions = FindObject<UNiagaraParameterDefinitions>(CurrentPackage, *ShorterSystemName);
		if (NiagaraParameterDefinitions == nullptr)
		{
			UE_LOGF(LogNiagaraDumpModuleInfoCommandlet, Warning, "Failed to load NiagaraParameterDefinitions %ls", *SystemName);
			continue;
		}

		IFileManager::Get().MakeDirectory(*(AuditOutputFolder / TEXT("_Definitions_") + ShorterSystemName));

		const FString MetaDataFileName = AuditOutputFolder / TEXT("_Definitions_") + ShorterSystemName / TEXT("Definitions.xml");
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*MetaDataFileName));
		TUniquePtr<FOutputDeviceArchiveWrapper> OutputStream(new FOutputDeviceArchiveWrapper(FileArchive.Get()));

		const TArray<UNiagaraScriptVariable*>& ScriptPtrVars =  NiagaraParameterDefinitions->GetParametersConst();
		TArray< UNiagaraScriptVariable*> ScriptVars;

		for (UNiagaraScriptVariable* Var : ScriptPtrVars)
		{
			if (!Var)
				continue;
			ScriptVars.Add(Var);
		}


		ScriptVars.StableSort([](const UNiagaraScriptVariable& Lhs, const UNiagaraScriptVariable& Rhs)
			{
				return Lhs.Variable.GetName().ToString().Compare(Rhs.Variable.GetName().ToString(), ESearchCase::IgnoreCase) < 0;
			});

		OutputStream->Log(TEXT("<?xml version='1.0' ?>"));
		OutputStream->Log(TEXT("<Info>"));
		OutputStream->Logf(TEXT("\t<Path>%s</Path>"), *GetPathNameSafe(NiagaraParameterDefinitions));

		for (UNiagaraScriptVariable* ScriptVar : ScriptVars)
		{
			OutputStream->Log(TEXT("\t<ScriptVar>"));
			OutputStream->Logf(TEXT("\t\t<Name>%s</Name>"), *ScriptVar->Variable.GetName().ToString());
			OutputStream->Logf(TEXT("\t\t<Description>%s</Description>"), *ScriptVar->Metadata.Description.ToString());			
			OutputStream->Logf(TEXT("\t<Guid>%s</Guid>"), *ScriptVar->Metadata.GetVariableGuid().ToString());
			OutputStream->Log(TEXT("\t</ScriptVar>"));
		}

		OutputStream->Log(TEXT("</Info>"));
	}

	// Probably don't need to do this, but just in case we have any 'hanging' packages 
	// and more processing steps are added later, let's clean up everything...
	::CollectGarbage(RF_NoFlags);
}

void UNiagaraDumpModuleInfoCommandlet::ProcessNiagaraScripts()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	FARFilter Filter;
	Filter.PackagePaths = PackagePaths;
	Filter.bRecursivePaths = true;

	Filter.ClassPaths.Add(UNiagaraScript::StaticClass()->GetClassPathName());
	if (!FilterCollection.IsEmpty())
	{
		ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
		if (CollectionManager.TryParseCollectionPath(FilterCollection, &CollectionContainer, &CollectionName, &ShareType))
		{
			CollectionContainer->GetObjectsInCollection(CollectionName, ShareType, Filter.SoftObjectPaths, ECollectionRecursionFlags::SelfAndChildren);
		}
	}

	TArray<FAssetData> AssetList;
	AssetRegistry.GetAssets(Filter, AssetList);

	const static UEnum* UsageEnum = StaticEnum<ENiagaraScriptUsage>();

	const double StartProcessNiagaraSystemsTime = FPlatformTime::Seconds();

	//  Iterate over all scripts
	const FString DevelopersFolder = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir().LeftChop(1));
	FString LastPackageName = TEXT("");
	UPackage* CurrentPackage = nullptr;
	for (const FAssetData& AssetIt : AssetList)
	{
		const FString SystemName = AssetIt.GetObjectPathString();
		const FString PackageName = AssetIt.PackageName.ToString();

		if (PackageName.StartsWith(DevelopersFolder))
		{
			// Skip developer folders
			continue;
		}

		if (PackageName != LastPackageName)
		{
			UPackage* Package = ::LoadPackage(nullptr, *PackageName, LOAD_None);
			if (Package != nullptr)
			{
				LastPackageName = PackageName;
				Package->FullyLoad();
				CurrentPackage = Package;
			}
			else
			{
				UE_LOGF(LogNiagaraDumpModuleInfoCommandlet, Warning, "Failed to load package %ls processing %ls", *PackageName, *SystemName);
				CurrentPackage = nullptr;
			}
		}

		const FString ShorterSystemName = AssetIt.AssetName.ToString();
		UNiagaraScript* NiagaraScript = FindObject<UNiagaraScript>(CurrentPackage, *ShorterSystemName);
		if (NiagaraScript == nullptr)
		{
			UE_LOGF(LogNiagaraDumpModuleInfoCommandlet, Warning, "Failed to load Niagara script %ls", *SystemName);
			continue;
		}

	
		IFileManager::Get().MakeDirectory(*(AuditOutputFolder / ShorterSystemName));

		TArray<FNiagaraAssetVersion> Versions = NiagaraScript->GetAllAvailableVersions();
		for (FNiagaraAssetVersion Version : Versions)
		{
			const UNiagaraScriptSource* ScriptSrc = Cast<UNiagaraScriptSource>(NiagaraScript->GetSource(Version.VersionGuid));
			TSharedPtr<FNiagaraVersionDataAccessor> VersionDataAccessor = NiagaraScript->GetVersionDataAccessor(Version.VersionGuid);
			if (ScriptSrc)
			{
				UNiagaraGraph* FunctionGraph = ScriptSrc->NodeGraph;
				FText VersionDesc;
				bool bDeprecated = false;
				if (VersionDataAccessor)
				{
					VersionDesc = VersionDataAccessor->GetVersionChangeDescription();
					bDeprecated = VersionDataAccessor->IsDeprecated();
				}

				if (FunctionGraph)
				{
					const FString MetaDataFileName = AuditOutputFolder / ShorterSystemName / Version.VersionGuid.ToString() + TEXT(".xml");
					TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*MetaDataFileName));
					TUniquePtr<FOutputDeviceArchiveWrapper> OutputStream(new FOutputDeviceArchiveWrapper(FileArchive.Get()));

					const TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>& AllVars = FunctionGraph->GetAllMetaData();
					TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptPtrVars;
					AllVars.GenerateValueArray(ScriptPtrVars);
					TArray<UNiagaraScriptVariable*> ScriptVars;

					for (const TObjectPtr<UNiagaraScriptVariable>& Var : ScriptPtrVars)
					{
						if (!Var)
							continue;
						ScriptVars.Add(Var);
					}
					

					ScriptVars.StableSort([](const UNiagaraScriptVariable& Lhs, const UNiagaraScriptVariable& Rhs)
					{
							return Lhs.Variable.GetName().ToString().Compare(Rhs.Variable.GetName().ToString(), ESearchCase::IgnoreCase) < 0;
					});

					OutputStream->Log(TEXT("<?xml version='1.0' ?>"));
					OutputStream->Log(TEXT("<Info>"));
					OutputStream->Logf(TEXT("\t<VersionNumber>%d . %d</VersionNumber>"), Version.MajorVersion, Version.MinorVersion);
					OutputStream->Logf(TEXT("\t<VersionDesc>%s</VersionDesc>"), *VersionDesc.ToString());
					OutputStream->Logf(TEXT("\t<Deprecated>%s</Deprecated>"), bDeprecated ? TEXT("true") : TEXT("false"));
					OutputStream->Logf(TEXT("\t<Path>%s</Path>"), *GetPathNameSafe(FunctionGraph));

					for (UNiagaraScriptVariable* ScriptVar : ScriptVars)
					{
						OutputStream->Log(TEXT("\t<ScriptVar>"));
						OutputStream->Logf(TEXT("\t\t<Name>%s</Name>"), *ScriptVar->Variable.GetName().ToString());
						OutputStream->Logf(TEXT("\t\t<Type>%s</Type>"), *ScriptVar->Variable.GetType().GetName());
						OutputStream->Logf(TEXT("\t<Guid>%s</Guid>"), *ScriptVar->Metadata.GetVariableGuid().ToString());
						OutputStream->Logf(TEXT("\t\t<Description>%s</Description>"), *ScriptVar->Metadata.Description.ToString());

						OutputStream->Logf(TEXT("\t\t<Advanced>%s</Advanced>"), ScriptVar->Metadata.bAdvancedDisplay ? TEXT("true") : TEXT("false"));
						OutputStream->Logf(TEXT("\t\t<DisplayInOverviewStack>%s</DisplayInOverviewStack>"), ScriptVar->Metadata.bDisplayInOverviewStack ? TEXT("true") : TEXT("false"));
						OutputStream->Logf(TEXT("\t\t<EnableBoolOverride>%s</EnableBoolOverride>"), ScriptVar->Metadata.bEnableBoolOverride ? TEXT("true") : TEXT("false"));
						OutputStream->Logf(TEXT("\t\t<InlineEditConditionToggle>%s</InlineEditConditionToggle>"), ScriptVar->Metadata.bInlineEditConditionToggle ? TEXT("true") : TEXT("false"));
						for (const FName& Alias : ScriptVar->Metadata.AlternateAliases)
							OutputStream->Logf(TEXT("\t\t<Alias>%s</Alias>"), *Alias.ToString());
						OutputStream->Log(TEXT("\t</ScriptVar>"));
					}
					
					OutputStream->Log(TEXT("</Info>"));
				}
			}
		}
		
		/*
		* // sort the data alphabetically (based on MetaData.FullName
	ScriptMetaData.StableSort([](const FScriptMetaData& Lhs, const FScriptMetaData& Rhs)
	{
		return Lhs.FullName.Compare(Rhs.FullName, ESearchCase::IgnoreCase) < 0;
	});
		*/
	}

	
	// Probably don't need to do this, but just in case we have any 'hanging' packages 
	// and more processing steps are added later, let's clean up everything...
	::CollectGarbage(RF_NoFlags);

	double ProcessNiagaraSystemsTime = FPlatformTime::Seconds() - StartProcessNiagaraSystemsTime;
	UE_LOGF(LogNiagaraDumpModuleInfoCommandlet, Log, "Took %5.3f seconds to process referenced Niagara scripts...", ProcessNiagaraSystemsTime);
}

/*void UNiagaraDumpModuleInfoCommandlet::DumpByteCode(const UNiagaraScript* Script, const FString& FilePath)
{
	if (!Script)
	{
		return;
	}

	const auto& ExecData = Script->GetVMExecutableData();

	FScriptMetaData& MetaData = ScriptMetaData.AddZeroed_GetRef();
	MetaData.FullName = FilePath;
	MetaData.RegisterCount = ExecData.NumTempRegisters;
	MetaData.OpCount = ExecData.LastOpCount;
	MetaData.ConstantCount = ExecData.InternalParameters.GetTableSize() / 4;
	MetaData.AttributeCount = 0;

	for (const FNiagaraVariableBase& Var : ExecData.Attributes)
	{
		MetaData.AttributeCount += Var.GetType().GetSize() / 4;
	}

	const static UEnum* VmOpEnum = StaticEnum<EVectorVMOp>();

	if (Script)
	{
		const FString FullFilePath = AuditOutputFolder / FilePath;

		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*FullFilePath));
		if (!FileArchive)
		{
			UE_LOGF(LogNiagaraDumpModuleInfoCommandlet, Warning, "Failed to create output stream %ls", *FullFilePath);
		}

		TUniquePtr<FOutputDeviceArchiveWrapper> OutputStream(new FOutputDeviceArchiveWrapper(FileArchive.Get()));

		const auto& VMData = Script->GetVMExecutableData();

		TArray<FString> AssemblyLines;
		VMData.LastAssemblyTranslation.ParseIntoArrayLines(AssemblyLines, false);

		// we want to translate all instances of OP_X into the string from the VMVector enum
		for (FString& CurrentLine : AssemblyLines)
		{
			int OpStartIndex = CurrentLine.Find(TEXT("OP_"), ESearchCase::CaseSensitive);
			if (OpStartIndex != INDEX_NONE)
			{
				const int32 OpEndStartSearch = OpStartIndex + 3;
				int OpEndIndex = CurrentLine.Find(TEXT("("), ESearchCase::IgnoreCase, ESearchDir::FromStart, OpEndStartSearch);
				if (OpEndIndex == INDEX_NONE)
				{
					OpEndIndex = CurrentLine.Find(TEXT(";"), ESearchCase::IgnoreCase, ESearchDir::FromStart, OpEndStartSearch);
				}

				if (OpEndIndex != INDEX_NONE)
				{
					const FString OpIndexString = CurrentLine.Mid(OpEndStartSearch, OpEndIndex - OpEndStartSearch);
					int OpIndexValue = FCString::Atoi(*OpIndexString);

					FString LinePrefix = CurrentLine.Left(OpStartIndex);
					FString LineSuffix = CurrentLine.RightChop(OpEndIndex);

					CurrentLine = LinePrefix + TEXT(":") + *VmOpEnum->GetNameStringByValue(OpIndexValue) + LineSuffix;
				}
			}
			OutputStream->Log(CurrentLine);
		}
	}
}*/
