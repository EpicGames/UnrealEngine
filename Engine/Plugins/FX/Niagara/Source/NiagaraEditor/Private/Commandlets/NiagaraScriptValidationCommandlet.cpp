// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/NiagaraScriptValidationCommandlet.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "HAL/FileManager.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "NiagaraEditorUtilities.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "NiagaraSystem.h"
#include "NiagaraScriptSource.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraParameterDefinitions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScriptValidationCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraScriptValidation, Log, All);

UNiagaraScriptValidationCommandlet::UNiagaraScriptValidationCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UNiagaraScriptValidationCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);

	if (!FParse::Value(*Params, TEXT("AuditOutputFolder="), AuditOutputFolder))
	{
		// No output folder specified. Use the default folder.
		AuditOutputFolder = FPaths::ProjectSavedDir() / TEXT("Audit");
	}
	
	// Add a timestamp to the folder
	AuditOutputFolder /= FDateTime::Now().ToString();

	FParse::Value(*Params, TEXT("FilterCollection="), FilterCollection);

	// Package Paths
	FString PackagePathsString;
	if (FParse::Value(*Params, TEXT("PackagePaths="), PackagePathsString, false))
	{
		TArray<FString> PackagePathsStrings;
		PackagePathsString.ParseIntoArray(PackagePathsStrings, TEXT(","));
		for (const FString& v : PackagePathsStrings)
		{
			PackagePaths.Add(FName(v));
		}
	}

	if (PackagePaths.Num() == 0)
	{
		//PackagePaths.Add(FName(TEXT("/Game")));
		PackagePaths.Add(FName(TEXT("/Plugins")));
		PackagePaths.Add(FName(TEXT("/Niagara")));
		PackagePaths.Add(FName(TEXT("/NiagaraFluids")));
		PackagePaths.Add(FName(TEXT("/Engine")));
	}

	ProcessNiagaraScripts();

	return 0;
}

void UNiagaraScriptValidationCommandlet::ProcessNiagaraScripts()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	FARFilter Filter;
	Filter.PackagePaths = PackagePaths;
	Filter.bRecursivePaths = true;

	Filter.ClassPaths.Add(UNiagaraScript::StaticClass()->GetClassPathName());
	if (!FilterCollection.IsEmpty())
	{
		ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
		if (CollectionManager.TryParseCollectionPath(FilterCollection, &CollectionContainer, &CollectionName, &ShareType))
		{
			CollectionContainer->GetObjectsInCollection(CollectionName, ShareType, Filter.SoftObjectPaths, ECollectionRecursionFlags::SelfAndChildren);
		}
	}

	TArray<FAssetData> AssetList;
	AssetRegistry.GetAssets(Filter, AssetList);
	
	const double StartProcessNiagaraSystemsTime = FPlatformTime::Seconds();

	//  Iterate over all scripts
	const FString DevelopersFolder = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir().LeftChop(1));
	FString LastPackageName = TEXT("");
	UPackage* CurrentPackage = nullptr;

	IFileManager::Get().MakeDirectory(*(AuditOutputFolder / "ScriptValidation"));

	for (const FAssetData& AssetIt : AssetList)
	{
		const FString SystemName = AssetIt.GetObjectPathString();
		const FString PackageName = AssetIt.PackageName.ToString();

		if (PackageName.StartsWith(DevelopersFolder))
		{
			// Skip developer folders
			continue;
		}

		if (PackageName != LastPackageName)
		{
			UPackage* Package = ::LoadPackage(nullptr, *PackageName, LOAD_None);
			if (Package != nullptr)
			{
				LastPackageName = PackageName;
				Package->FullyLoad();
				CurrentPackage = Package;
			}
			else
			{
				UE_LOGF(LogNiagaraScriptValidation, Warning, "Failed to load package %ls processing %ls", *PackageName, *SystemName);
				CurrentPackage = nullptr;
			}
		}

		const FString ShorterScriptName = AssetIt.AssetName.ToString();
		UNiagaraScript* NiagaraScript = FindObject<UNiagaraScript>(CurrentPackage, *ShorterScriptName);
		if (NiagaraScript == nullptr)
		{
			UE_LOGF(LogNiagaraScriptValidation, Warning, "Failed to load Niagara script %ls", *SystemName);
			continue;
		}
		
		TArray<FNiagaraAssetVersion> Versions = NiagaraScript->GetAllAvailableVersions();
		
		TMap<FNiagaraAssetVersion, TMap<FGuid, TArray<FNiagaraVariableBase>>> AllResultsForScript;
		bool bShouldWriteLogFile = false;
		
		for(FNiagaraAssetVersion Version : Versions)
		{
			const UNiagaraScriptSource* ScriptSrc = Cast<UNiagaraScriptSource>(NiagaraScript->GetSource(Version.VersionGuid));
			TSharedPtr<FNiagaraVersionDataAccessor> VersionDataAccessor = NiagaraScript->GetVersionDataAccessor(Version.VersionGuid);
			if(ScriptSrc)
			{
				if(UNiagaraGraph* FunctionGraph = ScriptSrc->NodeGraph)
				{
					const TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>& AllVars = FunctionGraph->GetAllMetaData();
					TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptPtrVars;
					AllVars.GenerateValueArray(ScriptPtrVars);
					TArray<UNiagaraScriptVariable*> ScriptVars;

					for(const TObjectPtr<UNiagaraScriptVariable>& Var : ScriptPtrVars)
					{
						if(Var == nullptr)
						{
							continue;								
						}
						
						ScriptVars.Add(Var);
					}
					
					ScriptVars.StableSort([](const UNiagaraScriptVariable& Lhs, const UNiagaraScriptVariable& Rhs)
					{
						return Lhs.Variable.GetName().ToString().Compare(Rhs.Variable.GetName().ToString(), ESearchCase::IgnoreCase) < 0;
					});
					
					TMap<FGuid, TArray<FNiagaraVariableBase>> Results = FNiagaraEditorUtilities::Scripts::Validation::ValidateScriptVariableIds(NiagaraScript, Version.VersionGuid);
					for(const auto& Result : Results)
					{
						if(Result.Value.Num() > 1)
						{
							bShouldWriteLogFile = true;
							AllResultsForScript.Add(Version, Results);
						}
					}
				}
			}
		}
		
		if(bShouldWriteLogFile)
		{
			const FString MetaDataFileName = AuditOutputFolder / "ScriptValidation" / ShorterScriptName + TEXT(".xml");
			TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*MetaDataFileName));
			TUniquePtr<FOutputDeviceArchiveWrapper> OutputStream(new FOutputDeviceArchiveWrapper(FileArchive.Get()));

			OutputStream->Log(TEXT("<?xml version='1.0' ?>"));
			OutputStream->Log(TEXT("<Info>"));
			OutputStream->Logf(TEXT("\t<Path>%s</Path>"), *GetPathNameSafe(NiagaraScript));
			for(const auto& ResultForScript : AllResultsForScript)
			{
				bool bSkipLogVersion = true;
				for(const auto& GuidConflict : ResultForScript.Value)
				{
					if(GuidConflict.Value.Num() > 1)
					{
						bSkipLogVersion = false;
						break;
					}
				}

				if(bSkipLogVersion)
				{
					continue;
				}
			
				OutputStream->Logf(TEXT("\t<VersionNumber>%d . %d</VersionNumber>"), ResultForScript.Key.MajorVersion, ResultForScript.Key.MinorVersion);
				for(const auto& GuidConflict : ResultForScript.Value)
				{
					if(GuidConflict.Value.Num() > 1)
					{
						OutputStream->Log(TEXT("\t\t<GuidConflict>"));
						OutputStream->Logf(TEXT("\t\t\t<Guid>%s</Guid>"), *GuidConflict.Key.ToString());
						for(const FNiagaraVariableBase& VariableBase : GuidConflict.Value)
						{
							OutputStream->Logf(TEXT("\t\t\t<Name>%s</Name>"), *VariableBase.GetName().ToString());
						}
						OutputStream->Log(TEXT("\t\t</GuidConflict>"));
					}
				}						
			}
			OutputStream->Log(TEXT("</Info>"));	
		}
	}

	
	// Probably don't need to do this, but just in case we have any 'hanging' packages 
	// and more processing steps are added later, let's clean up everything...
	::CollectGarbage(RF_NoFlags);

	double ProcessNiagaraSystemsTime = FPlatformTime::Seconds() - StartProcessNiagaraSystemsTime;
	UE_LOGF(LogNiagaraScriptValidation, Log, "Took %5.3f seconds to process referenced Niagara scripts...", ProcessNiagaraSystemsTime);
}
