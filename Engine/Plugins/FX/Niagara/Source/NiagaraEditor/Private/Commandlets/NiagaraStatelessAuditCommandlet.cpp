// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/NiagaraStatelessAuditCommandlet.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "Stateless/NiagaraStatelessModule.h"
#include "NiagaraSystem.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "HAL/FileManager.h"
#include "UObject/UObjectGlobals.h"
#include "ObjectEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessAuditCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraStatelessAuditCommandlet, Log, All);

namespace NiagaraStatelessAuditCommandletPrivate
{
	static const FName NAME_ActiveStatelessEmitters("ActiveStatelessEmitters");

	UNiagaraSystem* LoadSystem(const FAssetData& AssetData)
	{
		const FString SystemName = AssetData.GetObjectPathString();
		const FString PackageName = AssetData.PackageName.ToString();

		if ( PackageName.StartsWith(TEXT("/Game/Developers")) )
		{
			return nullptr;
		}

		UPackage* Package = ::LoadPackage(nullptr, *PackageName, LOAD_None);
		if (Package == nullptr)
		{
			UE_LOGF(LogNiagaraStatelessAuditCommandlet, Warning, "Failed to load package %ls processing %ls", *PackageName, *SystemName);
			return nullptr;
		}
		Package->FullyLoad();

		return FindObject<UNiagaraSystem>(Package, *AssetData.AssetName.ToString());
	}
}

UNiagaraStatelessAuditCommandlet::UNiagaraStatelessAuditCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UNiagaraStatelessAuditCommandlet::Main(const FString& Params)
{
	using namespace NiagaraStatelessAuditCommandletPrivate;

	ParseParameters(Params);

	// Find assets
	TArray<FAssetData> AssetsToProcess;
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		AssetRegistry.SearchAllAssets(true);
		//AssetRegistry.GetAssetsByTags({ NAME_ActiveStatelessEmitters }, AssetsToProcess);

		FARFilter Filter;
		//Filter.PackagePaths = PackagePaths;
		Filter.bRecursivePaths = true;
		Filter.ClassPaths.Add(UNiagaraSystem::StaticClass()->GetClassPathName());
		AssetRegistry.GetAssets(Filter, AssetsToProcess);
	}

	// Process each asset
	for (const FAssetData& AssetData : AssetsToProcess)
	{
		//int32 NumStatelessEmitters = 0;
		//{
		//	FString NumStatelessEmittersString;
		//	AssetsToProcess[0].GetTagValue(NAME_ActiveStatelessEmitters, NumStatelessEmittersString);
		//	NumStatelessEmitters = FCString::Atoi(*NumStatelessEmittersString);
		//	UE_LOGF(LogNiagaraStatelessAuditCommandlet, Warning, "%ls = %ls = %d", *AssetData.PackageName.ToString(), *NumStatelessEmittersString, NumStatelessEmitters);
		//}
		//if (NumStatelessEmitters <= 0)
		//{
		//	continue;
		//}

		// Load the asset
		if ( UNiagaraSystem* NiagaraSystem = LoadSystem(AssetData) )
		{
			ProcessSystem(NiagaraSystem);
		}
	}
	WriteResults();

	return 0;
}

void UNiagaraStatelessAuditCommandlet::ParseParameters(const FString& Params)
{
	// Parse output folder
	if (!FParse::Value(*Params, TEXT("AuditOutputFolder="), AuditOutputFolder))
	{
		// No output folder specified. Use the default folder.
		AuditOutputFolder = FPaths::ProjectSavedDir() / TEXT("Audit");
	}
	AuditOutputFolder /= FDateTime::Now().ToString();
}

void UNiagaraStatelessAuditCommandlet::ProcessSystem(UNiagaraSystem* NiagaraSystem)
{
	for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles())
	{
		const UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandle.GetEmitterMode() == ENiagaraEmitterMode::Stateless ? EmitterHandle.GetStatelessEmitter() : nullptr;
		const UNiagaraStatelessEmitterTemplate* StatelessTemplate = StatelessEmitter ? StatelessEmitter->GetEmitterTemplate() : nullptr;
		if (!StatelessEmitter || !StatelessTemplate || !EmitterHandle.GetIsEnabled())
		{
			continue;
		}

		for (const UNiagaraStatelessModule* StatelessModule : StatelessEmitter->GetModules())
		{
			if (!StatelessModule || !StatelessModule->IsModuleEnabled())
			{
				continue;
			}

			FModuleUsage* ModuleUsage = ModuleUsageMap.Find(StatelessModule->GetFName());
			if (ModuleUsage == nullptr)
			{
				ModuleUsage = &ModuleUsageMap.Add(StatelessModule->GetFName());

				FString MostDerivedDevelopmentClassName;
				FObjectEditorUtils::GetClassDevelopmentStatus(StatelessModule->GetClass(), ModuleUsage->bIsExperimental, ModuleUsage->bIsEarlyAccess, MostDerivedDevelopmentClassName);
				bAnyExperimentalModules |= ModuleUsage->bIsExperimental;
				bAnyEarlyAccessModules |= ModuleUsage->bIsEarlyAccess;
			}

			++ModuleUsage->UsageCount;
			ModuleUsage->SystemNames.AddUnique(NiagaraSystem->GetName());
			ModuleUsage->EmitterNames.AddUnique(StatelessEmitter->GetName());
			ModuleUsage->SystemAndEmitterNames.AddUnique(StatelessEmitter->GetPathName());
		}
	}
}

void UNiagaraStatelessAuditCommandlet::WriteResults() const
{
	if (ModuleUsageMap.Num() == 0)
	{
		return;
	}

	// Output Overall Module Usage
	{
		TUniquePtr<FArchive> OutputStream = GetOutputFile(TEXT("ModuleUsage.csv"));
		if (OutputStream.IsValid())
		{
			OutputStream->Logf(TEXT("Module Name,Usage Count"));
			for (auto It = ModuleUsageMap.CreateConstIterator(); It; ++It)
			{
				const FName ModuleName = It.Key();
				const FModuleUsage& ModuleUsage = It.Value();
				OutputStream->Logf(TEXT("%s,%d"), *ModuleName.ToString(), ModuleUsage.UsageCount);
			}
		}
	}

	// Output Per Module Usage
	for (auto It = ModuleUsageMap.CreateConstIterator(); It; ++It)
	{
		const FName ModuleName = It.Key();
		const FModuleUsage& ModuleUsage = It.Value();

		TUniquePtr<FArchive> OutputStream = GetOutputFile(*FString::Printf(TEXT("ModulUsage_%s.txt"), *ModuleName.ToString()));
		if (!OutputStream.IsValid())
		{
			continue;
		}

		OutputStream->Logf(TEXT("System Names:"));
		for (const FString& SystemName : ModuleUsage.SystemNames)
		{
			OutputStream->Logf(TEXT("%s"), *SystemName);
		}

		OutputStream->Logf(TEXT(""));
		OutputStream->Logf(TEXT("Emitter Names:"));
		for (const FString& EmitterName : ModuleUsage.EmitterNames)
		{
			OutputStream->Logf(TEXT("%s"), *EmitterName);
		}
	}

	// Output information about using experimental / early access modules
	if (bAnyExperimentalModules || bAnyEarlyAccessModules)
	{
		TUniquePtr<FArchive> OutputStream = GetOutputFile(TEXT("InvalidModuleUsage.txt"));
		if (OutputStream.IsValid())
		{
			for (auto It = ModuleUsageMap.CreateConstIterator(); It; ++It)
			{
				const FName ModuleName = It.Key();
				const FModuleUsage& ModuleUsage = It.Value();
				if (!ModuleUsage.bIsEarlyAccess && !ModuleUsage.bIsExperimental)
				{
					continue;
				}

				OutputStream->Logf(TEXT("Module(%s) is (%s) and used in the following:"), *ModuleName.ToString(), ModuleUsage.bIsExperimental ? TEXT("Experimental") : TEXT("EarlyAccess"));
				for (const FString& SystemAndEmitterName : ModuleUsage.SystemAndEmitterNames)
				{
					OutputStream->Logf(TEXT("\t- %s"), *SystemAndEmitterName);
				}
			}
		}
	}
}

TUniquePtr<FArchive> UNiagaraStatelessAuditCommandlet::GetOutputFile(const TCHAR* Filename) const
{
	const FString FullPath = FString::Printf(TEXT("%s/%s"), *AuditOutputFolder, Filename);
	FArchive* OutputStream = IFileManager::Get().CreateDebugFileWriter(*FullPath, FILEWRITE_AllowRead);
	if (OutputStream == NULL)
	{
		UE_LOGF(LogNiagaraStatelessAuditCommandlet, Warning, "Failed to create output stream %ls", *FullPath);
	}
	return TUniquePtr<FArchive>(OutputStream);
}
