// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/NiagaraSystemAuditCommandlet.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerTypes.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "CollectionManagerModule.h"
#include "FileHelpers.h"

#include "ViewModels/NiagaraSystemViewModel.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemImpl.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraLightRendererProperties.h"
#include "NiagaraRibbonRendererProperties.h"
#include "NiagaraValidationRules.h"
#include "NiagaraValidationRule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSystemAuditCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraSystemAuditCommandlet, Log, All);

namespace NiagaraSystemAuditCommandletLocal
{
	void RunValidation(FArchive* Output, UNiagaraSystem* NiagaraSystem)
	{
		NiagaraSystem->WaitForCompilationComplete();

		TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = MakeShared<FNiagaraSystemViewModel>();
		{
			FNiagaraSystemViewModelOptions SystemOptions;
			SystemOptions.bCanModifyEmittersFromTimeline = false;
			SystemOptions.bCanSimulate = false;
			SystemOptions.bCanAutoCompile = false;
			SystemOptions.bIsForDataProcessingOnly = true;
			SystemOptions.MessageLogGuid = NiagaraSystem->GetAssetGuid();
			SystemViewModel->Initialize(*NiagaraSystem, SystemOptions);
		}

		NiagaraValidation::ValidateAllRulesInSystem(
			SystemViewModel,
			[Output, NiagaraSystem](const FNiagaraValidationResult& Result)
			{
				if ( Result.Severity == ENiagaraValidationSeverity::Error || Result.Severity == ENiagaraValidationSeverity::Warning )
				{
					Output->Logf(
						TEXT("%s,%s,%s,%s"),
						*GetFullNameSafe(NiagaraSystem),
						(Result.Severity == ENiagaraValidationSeverity::Error) ? TEXT("Error") : TEXT("Warning"),
						*Result.SummaryText.ToString(),
						*Result.Description.ToString()
					);
				}
			}
		);
	}
}


UNiagaraSystemAuditCommandlet::UNiagaraSystemAuditCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UNiagaraSystemAuditCommandlet::Main(const FString& Params)
{
	if (!FParse::Value(*Params, TEXT("AuditOutputFolder="), AuditOutputFolder))
	{
		// No output folder specified. Use the default folder.
		AuditOutputFolder = FPaths::ProjectSavedDir() / TEXT("Audit");
	}

	// Add a timestamp to the folder
	AuditOutputFolder /= FDateTime::Now().ToString();

	FParse::Value(*Params, TEXT("FilterCollection="), FilterCollection);

	INiagaraModule& NiagaraModule = FModuleManager::LoadModuleChecked<INiagaraModule>("Niagara");
	// User Data Interfaces to Find
	{
		FString DataInterfacesToFindString;
		if (FParse::Value(*Params, TEXT("DataInterfacesToFind="), DataInterfacesToFindString, false))
		{
			TArray<FString> DataInterfaceNames;
			DataInterfacesToFindString.ParseIntoArray(DataInterfaceNames, TEXT(","));
			for (const FString& DIName : DataInterfaceNames)
			{
				if (UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(DIName, EFindFirstObjectOptions::ExactClass))
				{
					DataInterfacesToFind.Add(FoundClass);
				}
				else
				{
					UE_LOGF(LogNiagaraSystemAuditCommandlet, Warning, "DataInterace %ls was not found so will not be searched", *DIName);
				}
			}
		}
	}

	// Disable on specific platforms
	// Example: -run=NiagaraSystemAuditCommandlet -DeviceProfilesToDisableGpu=Mobile
	{
		FString DeviceNamesArrayString;
		if ( FParse::Value(*Params, TEXT("DeviceProfilesToDisableGpu="), DeviceNamesArrayString, false))
		{
			TArray<FString> DeviceNamesArray;
			DeviceNamesArrayString.ParseIntoArray(DeviceNamesArray, TEXT(","));

			for (FString DeviceString : DeviceNamesArray)
			{
				FName DeviceName(DeviceString);
				TObjectPtr<UDeviceProfile>* DeviceProfile = UDeviceProfileManager::Get().Profiles.FindByPredicate([&](UObject* Device) {return Device->GetFName() == DeviceName; });
				if (DeviceProfile)
				{
					DeviceProfilesToDisableGpu.Add(*DeviceProfile);
				}
			}
		}
	}

	FParse::Bool(*Params, TEXT("RendererDetailed="), bRendererDetailed);

	// Disable on specific platforms
	// Example To Capture Specific: -run=NiagaraSystemAuditCommandlet -CaptureDataInterfaceUsage=MyDataInterface
	// Example To Capture All: -run=NiagaraSystemAuditCommandlet -CaptureDataInterfaceUsage
	{
		FString DINamesArrayString;
		if (FParse::Value(*Params, TEXT("CaptureDataInterfaceUsage="), DINamesArrayString, false))
		{
			bCaptureDataInterfaceUsage = true;

			TArray<FString> DINamesArray;
			DINamesArrayString.ParseIntoArray(DINamesArray, TEXT(","));

			for (FString DIString : DINamesArray)
			{
				NiagaraDataInterfaceUsageToCapture.Add(FName(*DIString));
			}
		}
		else
		{
			bCaptureDataInterfaceUsage = FParse::Param(*Params, TEXT("CaptureDataInterfaceUsage"));
		}
	}

	// Validation enabled?
	bool bRunValidation = false;
	FParse::Bool(*Params, TEXT("RunValidation="), bRunValidation);
	if ( bRunValidation )
	{
		NiagaraValidationIssues.Reset(GetOutputFile(TEXT("NiagaraValidationIssues")));
	}

	// Package Paths
	FString PackagePathsString;
	if (FParse::Value(*Params, TEXT("PackagePaths="), PackagePathsString, false))
	{
		TArray<FString> PackagePathsStrings;
		PackagePathsString.ParseIntoArray(PackagePathsStrings, TEXT(","));
		for (const FString& v : PackagePathsStrings)
		{
			PackagePaths.Add(FName(v));
		}
	}

	if (PackagePaths.Num() == 0 && FParse::Param(*Params, TEXT("GameContentOnly")))
	{
		PackagePaths.Add(FName(TEXT("/Game")));
	}

	// Include only assets
	FString IncludeOnlyPackagesFileName;
	if (FParse::Value(*Params, TEXT("IncludeOnlyPackages="), IncludeOnlyPackagesFileName, false))
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		if (PlatformFile.FileExists(*IncludeOnlyPackagesFileName))
		{
			TArray<FString> IncludeOnlyPackagesStringArray;
			FFileHelper::LoadFileToStringArray(IncludeOnlyPackagesStringArray, *IncludeOnlyPackagesFileName);
			for (const FString& Entry : IncludeOnlyPackagesStringArray)
			{
				IncludeOnlyPackages.Add(FName(Entry));
			}
		}
	}

	ProcessNiagaraSystems();
	DumpResults();
	NiagaraValidationIssues.Reset();

	return 0;
}

bool UNiagaraSystemAuditCommandlet::ProcessNiagaraSystems()
{
	using namespace NiagaraSystemAuditCommandletLocal;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	FARFilter Filter;
	Filter.PackagePaths = PackagePaths;
	Filter.bRecursivePaths = true;

	Filter.ClassPaths.Add(UNiagaraSystem::StaticClass()->GetClassPathName());
	if (!FilterCollection.IsEmpty())
	{
		ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
		if (CollectionManager.TryParseCollectionPath(FilterCollection, &CollectionContainer, &CollectionName, &ShareType))
		{
			CollectionContainer->GetObjectsInCollection(CollectionName, ShareType, Filter.SoftObjectPaths, ECollectionRecursionFlags::SelfAndChildren);
		}
	}

	TArray<FAssetData> AssetList;
	AssetRegistry.GetAssets(Filter, AssetList);

	const double StartProcessNiagaraSystemsTime = FPlatformTime::Seconds();

	// Get Settings
	const UNiagaraSettings* NiagaraSettings = GetDefault<UNiagaraSettings>();

	//  Iterate over all systems
	const FString DevelopersFolder = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir().LeftChop(1));
	FString LastPackageName = TEXT("");
	UPackage* CurrentPackage = nullptr;

	const bool bColledGarbage = NiagaraValidationIssues.IsValid();
	const int32 CollectGarbageFrequency = 256;
	int32 CollectGarabageCounter = 0;

	for (const FAssetData& AssetIt : AssetList)
	{
		const FString SystemName = AssetIt.GetObjectPathString();
		const FString PackageName = AssetIt.PackageName.ToString();

		if (IncludeOnlyPackages.Num())
		{
			if (!IncludeOnlyPackages.Contains(AssetIt.PackagePath) && !IncludeOnlyPackages.Contains(FName(SystemName)))
			{
				continue;
			}
		}

		if (PackageName.StartsWith(DevelopersFolder))
		{
			// Skip developer folders
			continue;
		}

		if (PackageName != LastPackageName)
		{
			UPackage* Package = ::LoadPackage(nullptr, *PackageName, LOAD_None);
			if (Package != nullptr)
			{
				LastPackageName = PackageName;
				Package->FullyLoad();
				CurrentPackage = Package;
			}
			else
			{
				UE_LOGF(LogNiagaraSystemAuditCommandlet, Warning, "Failed to load package %ls processing %ls", *PackageName, *SystemName);
				CurrentPackage = nullptr;
			}
		}

		const FString ShorterSystemName = AssetIt.AssetName.ToString();
		UNiagaraSystem* NiagaraSystem = FindObject<UNiagaraSystem>(CurrentPackage, *ShorterSystemName);
		if (NiagaraSystem == nullptr)
		{
			UE_LOGF(LogNiagaraSystemAuditCommandlet, Warning, "Failed to load Niagara system %ls", *SystemName);
			continue;
		}

		NiagaraSystem->WaitForCompilationComplete(true, false);

		// Iterate over all data interfaces used by the system / emitters
		TSet<FName> SystemDataInterfacesWihPrereqs;
		TSet<FName> SystemUserDataInterfaces;
		for (UNiagaraDataInterface* DataInterface : GetDataInterfaces(NiagaraSystem))
		{
			if (bCaptureDataInterfaceUsage)
			{
				const FName DIClassName = DataInterface->GetClass()->GetFName();
				if ( NiagaraDataInterfaceUsageToCapture.IsEmpty() || NiagaraDataInterfaceUsageToCapture.Contains(DIClassName) )
				{
					FDataInterfaceUsage& DataInterfaceUsage = NiagaraDataInterfaceUsage.FindOrAdd(DataInterface->GetClass()->GetFName());
					++DataInterfaceUsage.UsageCount;
					DataInterfaceUsage.Systems.Add(NiagaraSystem->GetFName());
				}
			}

			if (DataInterface->HasTickGroupPrereqs())
			{
				SystemDataInterfacesWihPrereqs.Add(DataInterface->GetClass()->GetFName());
			}
			if (DataInterfacesToFind.Contains(DataInterface->GetClass()))
			{
				SystemUserDataInterfaces.Add(DataInterface->GetClass()->GetFName());
			}
		}

		// Iterate over all emitters
		FString EmittersWithDynamicBounds;
		FString EmittersWithSimulationStages;
		bool bHasLights = false;
		bool bHasEvents = false;
		bool bHasGPUEmitters = false;
		bool bHasCPUEmitters = false;
		bool bHasStatelessEmitters = false;

		for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles())
		{
			bHasStatelessEmitters |= (EmitterHandle.GetEmitterMode() == ENiagaraEmitterMode::Stateless) && EmitterHandle.GetStatelessEmitter();

			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
			if (EmitterData == nullptr)
			{
				continue;
			}

			if ( !EmitterHandle.GetIsEnabled() )
			{
				continue;
			}

			bHasGPUEmitters |= EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim;
			bHasCPUEmitters |= EmitterData->SimTarget != ENiagaraSimTarget::GPUComputeSim;

			if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
			{
				// Optionally disable GPU emitters
				for (UDeviceProfile* DeviceProfile : DeviceProfilesToDisableGpu)
				{
					const int32 DeviceQualityLevelMask = EmitterData->Platforms.IsEnabledForDeviceProfile(DeviceProfile);
					if (DeviceQualityLevelMask != 0)
					{
						for (int32 iQualityLevel = 0; iQualityLevel < NiagaraSettings->QualityLevels.Num(); ++iQualityLevel)
						{
							if ( (DeviceQualityLevelMask & (1 << iQualityLevel)) != 0 )
							{
								EmitterData->Platforms.SetDeviceProfileState(DeviceProfile, iQualityLevel, ENiagaraPlatformSelectionState::Disabled);
							}
						}

						PackagesToSave.AddUnique(CurrentPackage);
						CurrentPackage->SetDirtyFlag(true);

						UE_LOGF(LogNiagaraSystemAuditCommandlet, Log, "Disabling Emitter %ls for System %ls Device Quality Level Mask 0x%08x", *EmitterHandle.GetUniqueInstanceName(), *GetNameSafe(NiagaraSystem), DeviceQualityLevelMask);
					}
				}

				// Build information to write out
				TStringBuilder<512> GpuEmitterBuilder;
				GpuEmitterBuilder.Append(EmitterData->GetDebugSimName());

				GpuEmitterBuilder.Append(EmitterData->UsesInterpolatedSpawning() ? TEXT(",true") : TEXT(",false"));

				for (int32 iQualityLevel=0; iQualityLevel < NiagaraSettings->QualityLevels.Num(); ++iQualityLevel)
				{
					const bool bEnabled = EmitterData->Platforms.IsEffectQualityEnabled(iQualityLevel);

					TArray<UDeviceProfile*> EnabledProfiles;
					TArray<UDeviceProfile*> DisabledProfiles;
					EmitterData->Platforms.GetOverridenDeviceProfiles(iQualityLevel, EnabledProfiles, DisabledProfiles);

					GpuEmitterBuilder.Append(TEXT(","));
					GpuEmitterBuilder.Append(bEnabled ? TEXT("Enabled") : TEXT("Disabled"));

					for (UDeviceProfile* EnabledProfile : EnabledProfiles)
					{
						GpuEmitterBuilder.Appendf(TEXT(" +%s"), *EnabledProfile->GetFName().ToString());
					}
					for (UDeviceProfile* DisabledProfile : DisabledProfiles)
					{
						GpuEmitterBuilder.Appendf(TEXT(" -%s"), *DisabledProfile->GetFName().ToString());
					}
				}

				GpuEmitterBuilder.Append(TEXT(","));
				for (const FNiagaraPlatformSetCVarCondition& Condition : EmitterData->Platforms.CVarConditions)
				{
					GpuEmitterBuilder.Appendf(TEXT(" CVarName(%s)"), *Condition.CVarName.ToString());
				}

				GpuEmitterBuilder.Append(TEXT(","));
				GpuEmitterBuilder.Append(NiagaraSystem->GetPathName());
				NiagaraSystemsWithGPUEmitters.Add(GpuEmitterBuilder.ToString());

				// Do we have simulation stages enabled
				const TArray<UNiagaraSimulationStageBase*>& SimulationStages = EmitterData->GetSimulationStages();
				if (SimulationStages.Num() > 0 )
				{
					int32 TotalIterations = 0;
					for (UNiagaraSimulationStageBase* SimStage : SimulationStages)
					{
						UNiagaraSimulationStageGeneric* SimStageGeneric = Cast<UNiagaraSimulationStageGeneric>(SimStage);
						if (SimStage->bEnabled && SimStageGeneric)
						{
							TotalIterations += SimStageGeneric->NumIterations.GetDefaultValue<int32>();
						}
					}
					EmittersWithSimulationStages.Appendf(TEXT("%s(%d Stages %d Iterations) "), EmitterData->GetDebugSimName(), SimulationStages.Num(), TotalIterations);
				}
			}

			bHasEvents |= EmitterData->GetEventHandlers().Num() > 0;

			for (UNiagaraRendererProperties* RendererProperties : EmitterData->GetRenderers())
			{
				if (UNiagaraLightRendererProperties* LightRendererProperties = Cast<UNiagaraLightRendererProperties>(RendererProperties))
				{
					bHasLights = true;
				}

				if ( bRendererDetailed )
				{
					if (UNiagaraRibbonRendererProperties* RibbonRendererProperties = Cast<UNiagaraRibbonRendererProperties>(RendererProperties))
					{
						static UEnum* NiagaraRibbonTessellationModeEnum = StaticEnum<ENiagaraRibbonTessellationMode>();

						TStringBuilder<512> RendererBuilder;
						RendererBuilder.Append(EmitterHandle.GetInstance().Emitter->GetPathName());
						RendererBuilder.Append(TEXT(","));
						RendererBuilder.Append(NiagaraRibbonTessellationModeEnum->GetValueAsString(RibbonRendererProperties->TessellationMode));
						RendererBuilder.Append(TEXT(","));
						if (RibbonRendererProperties->bUseGPUInit)
						{
							RendererBuilder.Append(TEXT("GPUInit=true "));
						}
						if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
						{
							RendererBuilder.Append(TEXT("GPUSim=true "));
						}
						NiagaraRibbonRenderers.Add(RendererBuilder.ToString());
					}
				}
			}

			if ( (EmitterData->CalculateBoundsMode == ENiagaraEmitterCalculateBoundMode::Dynamic) && !NiagaraSystem->bFixedBounds )
			{
				const FString EmitterName = EmitterHandle.GetUniqueInstanceName();
				if (EmittersWithDynamicBounds.Find(EmitterName) == INDEX_NONE)
				{
					if (EmittersWithDynamicBounds.Len() != 0)
					{
						EmittersWithDynamicBounds.Append(TEXT(" "));
					}
					EmittersWithDynamicBounds.Append(EmitterName);
				}
			}
		}

		// Add to different charts we will write out
		if (NiagaraSystem->GetWarmupTime() > 0.0f)
		{
			NiagaraSystemsWithWarmup.Add(FString::Printf(TEXT("%s,%f"), *NiagaraSystem->GetPathName(), NiagaraSystem->GetWarmupTime()));
		}

		if (bHasLights)
		{
			NiagaraSystemsWithLights.Add(NiagaraSystem->GetPathName());
		}

		if (bHasEvents)
		{
			NiagaraSystemsWithEvents.Add(NiagaraSystem->GetPathName());
		}

		if (bHasStatelessEmitters)
		{
			NiagaraSystemsWithStatelessEmitters.Add(NiagaraSystem->GetPathName());
		}

		if (SystemDataInterfacesWihPrereqs.Num() > 0)
		{
			FString DataInterfaceNames;
			for (auto it = SystemDataInterfacesWihPrereqs.CreateConstIterator(); it; ++it)
			{
				if (!DataInterfaceNames.IsEmpty())
				{
					DataInterfaceNames.AppendChar(TEXT(' '));
				}
				DataInterfaceNames.Append(*it->ToString());
			}
			NiagaraSystemsWithPrerequisites.Add(FString::Printf(TEXT("%s,%s"), *NiagaraSystem->GetPathName(), *DataInterfaceNames));
		}

		if ( !EmittersWithSimulationStages.IsEmpty() )
		{
			NiagaraSystemsWithSimulationStages.Add(FString::Printf(TEXT("%s,%s"), *NiagaraSystem->GetPathName(), *EmittersWithSimulationStages));
		}

		if (NiagaraSystem->ShouldCompressAttributes())
		{
			TStringBuilder<256> StringBuilder;
			StringBuilder.Append(NiagaraSystem->GetPathName());
			StringBuilder.Append(",");
			if (bHasGPUEmitters)
			{
				StringBuilder.Append("GPU ");
			}
			if (bHasCPUEmitters)
			{
				StringBuilder.Append("CPU ");
			}

			NiagaraSystemsWithCompression.Add(StringBuilder.ToString());
		}

		if ( !EmittersWithDynamicBounds.IsEmpty() )
		{
			NiagaraSystemsWithDynamicBounds.Add(FString::Printf(TEXT("%s,%s"), *NiagaraSystem->GetPathName(), *EmittersWithDynamicBounds));
		}

		if (SystemUserDataInterfaces.Num() > 0)
		{
			FString DataInterfaceNames;
			for (auto it = SystemUserDataInterfaces.CreateConstIterator(); it; ++it)
			{
				if (!DataInterfaceNames.IsEmpty())
				{
					DataInterfaceNames.AppendChar(TEXT(' '));
				}
				DataInterfaceNames.Append(*it->ToString());
			}
			NiagaraSystemsWithDataInterfaceToFind.Add(FString::Printf(TEXT("%s,%s"), *NiagaraSystem->GetPathName(), *DataInterfaceNames));
		}

		// Run validation last as it will potentially modify the asset
		if (NiagaraValidationIssues.IsValid())
		{
			RunValidation(NiagaraValidationIssues.Get(), NiagaraSystem);
		}

		if (bColledGarbage && (++CollectGarabageCounter >= CollectGarbageFrequency) )
		{
			CollectGarabageCounter = 0;
			::CollectGarbage(RF_NoFlags);
		}
	}

	// Anything to save do it
	if ( PackagesToSave.Num() > 0 )
	{
		UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, true);
	}

	// Probably don't need to do this, but just in case we have any 'hanging' packages 
	// and more processing steps are added later, let's clean up everything...
	::CollectGarbage(RF_NoFlags);

	double ProcessNiagaraSystemsTime = FPlatformTime::Seconds() - StartProcessNiagaraSystemsTime;
	UE_LOGF(LogNiagaraSystemAuditCommandlet, Log, "Took %5.3f seconds to process referenced Niagara systems...", ProcessNiagaraSystemsTime);

	return true;
}

/** Dump the results of the audit */
void UNiagaraSystemAuditCommandlet::DumpResults()
{
	// Dump all the simple mappings...
	DumpSimpleSet(NiagaraSystemsWithWarmup, TEXT("NiagaraSystemsWithWarmup"), TEXT("Name,WarmupTime"));
	DumpSimpleSet(NiagaraSystemsWithLights, TEXT("NiagaraSystemsWithLights"), TEXT("Name"));
	DumpSimpleSet(NiagaraSystemsWithEvents, TEXT("NiagaraSystemsWithEvents"), TEXT("Name"));
	DumpSimpleSet(NiagaraSystemsWithPrerequisites, TEXT("NiagaraSystemsWithPrerequisites"), TEXT("Name,DataInterface"));
	DumpSimpleSet(NiagaraSystemsWithDynamicBounds, TEXT("NiagaraSystemsWithDynamicBounds"), TEXT("Name,Emitters With Dynamic Bounds"));
	if (DataInterfacesToFind.Num() > 0)
	{
		DumpSimpleSet(NiagaraSystemsWithDataInterfaceToFind, TEXT("NiagaraSystemsWithDataInterfaceToFind"), TEXT("Name,DataInterface"));
	}
	if (NiagaraSystemsWithGPUEmitters.Num() > 0)
	{
		TStringBuilder<512> HeaderString;
		HeaderString.Append(TEXT("Emitter Name,Interpolated Spawn"));
		for (const FText& QualityLevelName : GetDefault<UNiagaraSettings>()->QualityLevels)
		{
			HeaderString.Append(TEXT(","));
			HeaderString.Append(QualityLevelName.ToString());
		}
		HeaderString.Append(TEXT(",CVar Conditions,System Path"));
		DumpSimpleSet(NiagaraSystemsWithGPUEmitters, TEXT("NiagaraSystemsWithGPUEmitters"), HeaderString.ToString());
	}
	DumpSimpleSet(NiagaraSystemsWithSimulationStages, TEXT("NiagaraSystemsWithSimulationStages"), TEXT("System,Emitters"));
	DumpSimpleSet(NiagaraSystemsWithCompression, TEXT("NiagaraSystemsWithCompression"), TEXT("System,ExecutionTypes"));
	DumpSimpleSet(NiagaraSystemsWithStatelessEmitters, TEXT("NiagaraSystemsWithStatelessEmitters"), TEXT("System"));

	if (NiagaraDataInterfaceUsage.Num() > 0)
	{
		if ( FArchive* OutputStream = GetOutputFile(TEXT("NiagaraDataInterfaceUsage")) )
		{
			OutputStream->Logf(TEXT("Name,Usage Count,Systems"));
			for ( auto UsageIt=NiagaraDataInterfaceUsage.CreateConstIterator(); UsageIt; ++UsageIt)
			{
				FString SystemList;
				for ( FName SystemName : UsageIt.Value().Systems )
				{
					SystemList.Append(*SystemName.ToString());
					SystemList.AppendChar(' ');
				}

				OutputStream->Logf(TEXT("%s,%d,%s"), *UsageIt.Key().ToString(), UsageIt.Value().UsageCount, *SystemList);
			}
			OutputStream->Close();
			delete OutputStream;
		}
	}

	if (bRendererDetailed)
	{
		DumpSimpleSet(NiagaraRibbonRenderers, TEXT("NiagaraRibbonRenderers"), TEXT("Name,TessellationMode,GPU"));
	}
}

bool UNiagaraSystemAuditCommandlet::DumpSimpleSet(TSet<FString>& InSet, const TCHAR* InShortFilename, const TCHAR* OptionalHeader)
{
	if (InSet.Num() > 0)
	{
		check(InShortFilename != NULL);

		FArchive* OutputStream = GetOutputFile(InShortFilename);
		if (OutputStream != NULL)
		{
			UE_LOGF(LogNiagaraSystemAuditCommandlet, Log, "Dumping '%ls' results...", InShortFilename);
			if (OptionalHeader != nullptr)
			{
				OutputStream->Logf(TEXT("%s"), OptionalHeader);
			}

			for (TSet<FString>::TIterator DumpIt(InSet); DumpIt; ++DumpIt)
			{
				FString ObjName = *DumpIt;
				OutputStream->Logf(TEXT("%s"), *ObjName);
			}

			OutputStream->Close();
			delete OutputStream;
		}
		else
		{
			return false;
		}
	}
	return true;
}

FArchive* UNiagaraSystemAuditCommandlet::GetOutputFile(const TCHAR* InShortFilename)
{
	const FString Filename = FString::Printf(TEXT("%s/%s.csv"), *AuditOutputFolder, InShortFilename);
	FArchive* OutputStream = IFileManager::Get().CreateDebugFileWriter(*Filename, FILEWRITE_AllowRead);
	if (OutputStream == NULL)
	{
		UE_LOGF(LogNiagaraSystemAuditCommandlet, Warning, "Failed to create output stream %ls", *Filename);
	}
	return OutputStream;
}

TArray<class UNiagaraDataInterface*> UNiagaraSystemAuditCommandlet::GetDataInterfaces(class UNiagaraSystem* NiagaraSystem)
{
	TArray<UNiagaraDataInterface*> DataInterfaces;
	for (UNiagaraDataInterface* DataInterface : NiagaraSystem->GetExposedParameters().GetDataInterfaces())
	{
		if (DataInterface != nullptr)
		{
			DataInterfaces.AddUnique(DataInterface);
		}
	}

	NiagaraSystem->ForEachScriptWithRuntimeData(
		[&DataInterfaces](const UNiagaraScript* Script, const FNiagaraScriptRuntimeData* ScriptRuntimeData)
		{
			for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolveDataInterfaceInfo : ScriptRuntimeData->GetResolvedDataInterfaces())
			{
				if ( UNiagaraDataInterface* DataInterface = ResolveDataInterfaceInfo.ResolvedDataInterface )
				{
					DataInterfaces.AddUnique(DataInterface);
				}
			}
		}
	);
	return DataInterfaces;
}

