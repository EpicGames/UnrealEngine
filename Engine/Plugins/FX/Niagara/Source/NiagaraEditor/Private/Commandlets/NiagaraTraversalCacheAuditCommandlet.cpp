// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/NiagaraTraversalCacheAuditCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/ParallelFor.h"
#include "EdGraphSchema_Niagara.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Modules/ModuleManager.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraGraphDataCache.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"
#include "TraversalCache/TraversalCache.h"
#include "NiagaraEditorModule.h"
#include "HAL/FileManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraTraversalCacheAuditCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraTraversalCacheAuditCommandlet, Log, All);

namespace NiagaraTraversalCacheAuditUtility
{

	FString TopLevelScriptToScriptName(const UNiagaraScript* InScript)
	{
		switch (InScript->GetUsage())
		{
		case ENiagaraScriptUsage::SystemSpawnScript:
			return "System spawn script";
		case ENiagaraScriptUsage::SystemUpdateScript:
			return "System update script";
		case ENiagaraScriptUsage::EmitterSpawnScript:
			return "Emitter spawn script";
		case ENiagaraScriptUsage::EmitterUpdateScript:
			return "Emitter update script";
		case ENiagaraScriptUsage::ParticleSpawnScript:
		case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
			return "Particle spawn script";
		case ENiagaraScriptUsage::ParticleUpdateScript:
			return "Particle update script";
		case ENiagaraScriptUsage::ParticleEventScript:
			return "Particle event script - " + InScript->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens);
		case ENiagaraScriptUsage::ParticleGPUComputeScript:
			return "Particle combined GPU script";
		case ENiagaraScriptUsage::ParticleSimulationStageScript:
		{
			UNiagaraSimulationStageBase* SimulationStage = InScript->GetTypedOuter<UNiagaraSimulationStageBase>();
			FString SimulationStageName = SimulationStage != nullptr
				? SimulationStage->SimulationStageName.ToString()
				: InScript->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens);
			return "Particle simulation stage script - " + SimulationStageName;
		}
		default:
			return "Unknown script";
		}
	}

	template<typename TCollection, typename TItem, typename TTransformedItem, typename TItemTransformer>
	void CollectCollectionDifferences(
		const TCollection& Items, const TCollection& NewItems,
		TArray<TTransformedItem>& OutAddedItems, TArray<TTransformedItem>& OutRemovedItems,
		TItemTransformer Transformer)
	{
		for (const TItem& NewItem : NewItems)
		{
			if (Items.Contains(NewItem) == false)
			{
				OutAddedItems.Add(Transformer(NewItem));
			}
		}
		for (const TItem& Item : Items)
		{
			if (NewItems.Contains(Item) == false)
			{
				OutRemovedItems.Add(Transformer(Item));
			}
		}
	}

	template<typename TItem, typename TItemComparer, typename TTransformedItem, typename TItemTransformer>
	bool CompareCollections(
		TArray<TItem>& Items, TArray<TItem>& NewItems,
		TArray<TTransformedItem>& OutAddedItems, TArray<TTransformedItem>& OutRemovedItems,
		TSet<TItem>& HiddenItems, TSet<TItem>& NewHiddenItems,
		TArray<TTransformedItem>& OutAddedHiddenItems, TArray<TTransformedItem>& OutRemovedHiddenItems,
		TItemComparer Comparer, TItemTransformer Transformer)
	{
		auto ItemsMatchNewItems = [&](TArray<TItem>& Items, TArray<TItem>& NewItems)->bool
		{
			Items.Sort(Comparer);
			NewItems.Sort(Comparer);
			for (int32 ItemIndex = 0; ItemIndex < Items.Num(); ItemIndex++)
			{
				if (Items[ItemIndex] != NewItems[ItemIndex])
				{
					return false;
				}
			}
			return true;
		};

		if (NewItems.Num() != Items.Num() ||
			NewHiddenItems.Num() != HiddenItems.Num() ||
			ItemsMatchNewItems(Items, NewItems) == false ||
			HiddenItems.Includes(NewHiddenItems) == false)
		{
			CollectCollectionDifferences<TArray<TItem>, TItem, TTransformedItem, TItemTransformer>(Items, NewItems, OutAddedItems, OutRemovedItems, Transformer);
			CollectCollectionDifferences<TSet<TItem>, TItem, TTransformedItem, TItemTransformer>(HiddenItems, NewHiddenItems, OutAddedHiddenItems, OutRemovedHiddenItems, Transformer);
			return false;
		}

		return true;
	}
}

FCriticalSection UNiagaraTraversalCacheAuditCommandlet::ResultCS;

int32 UNiagaraTraversalCacheAuditCommandlet::FAuditResultFunctionData::CountDifferences() const
{
	return
		AddedInputVariables.Num() + RemovedInputVariables.Num() +
		AddedHiddenInputVariables.Num() + RemovedHiddenInputVariables.Num() +
		AddedStaticInputPinVariables.Num() + RemovedStaticInputPinVariables.Num() +
		AddedHiddenStaticInputPinVariables.Num() + RemovedHiddenStaticInputPinVariables.Num();
}

int32 UNiagaraTraversalCacheAuditCommandlet::FAuditResultScriptData::CountDifferences() const
{
	int32 DifferenceCount = 0;
	for (const FAuditResultFunctionData& FunctionData : FunctionDatas)
	{
		DifferenceCount += FunctionData.CountDifferences();
	}
	return DifferenceCount;
}

int32 UNiagaraTraversalCacheAuditCommandlet::FAuditResultSystemData::CountDifferences() const
{
	int32 DifferenceCount = 0;
	for (const FAuditResultScriptData& ScriptData : ScriptDatas)
	{
		DifferenceCount += ScriptData.CountDifferences();
	}
	return DifferenceCount;
}

UNiagaraTraversalCacheAuditCommandlet::UNiagaraTraversalCacheAuditCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UNiagaraTraversalCacheAuditCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);
	if (Switches.Contains(TEXT("pause")))
	{
		int32 WaitSeconds = 20;

		const double EndSeconds = FPlatformTime::Seconds() + WaitSeconds;
		UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Display, "Pausing for %i seconds.", WaitSeconds);
		while (FPlatformTime::Seconds() < EndSeconds)
		{
			FPlatformProcess::Sleep(1);

			// Tick the engine
			CommandletHelpers::TickEngine();

			// Stop if exit was requested
			if (IsEngineExitRequested())
			{
				break;
			}
		}

		UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Display, "Pause finished.");
	}

	int32 MaxErrorsParam;
	if (FParse::Value(*Params, TEXT("MaxErrors="), MaxErrorsParam))
	{
		MaxErrorCount = MaxErrorsParam;
	}

	EnableTraversalCacheCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.Niagara.EnableTraversalCache"), false);
	int32 EnableTraversalCachePreAuditValue = EnableTraversalCacheCVar->GetInt();

	TArray<FString> SystemPaths;
	GatherAllSystemPaths(SystemPaths);

	FAuditResults Results;
	AuditSystems(SystemPaths, Results);

	EnableTraversalCacheCVar->SetWithCurrentPriority(EnableTraversalCachePreAuditValue);
	return 0;
}

void UNiagaraTraversalCacheAuditCommandlet::GatherAllSystemPaths(TArray<FString>& OutSystemPaths) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().SearchAllAssets(true);
	TArray<FAssetData> SystemAssets;
	AssetRegistryModule.Get().GetAssetsByClass(UNiagaraSystem::StaticClass()->GetClassPathName(), SystemAssets);

	OutSystemPaths.Reserve(SystemAssets.Num());
	for (FAssetData& SystemAsset : SystemAssets)
	{
		OutSystemPaths.Add(SystemAsset.PackageName.ToString());
	}

	OutSystemPaths.Sort();
}

void UNiagaraTraversalCacheAuditCommandlet::CollectFunctionData(UNiagaraSystem& System, const FNiagaraEmitterHandle* EmitterHandle, UNiagaraScript* Script, TArray<FFunctionData>& OutFunctionDatas) const
{
	if (Script == nullptr)
	{
		return;
	}

	TArray<UNiagaraNode*> TraversedNodes;
	const UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(Script->GetSource(FGuid()));
	ScriptSource->NodeGraph->BuildTraversal(TraversedNodes, Script->GetUsage(), Script->GetUsageId());
	for (UNiagaraNode* TraversedNode : TraversedNodes)
	{
		UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(TraversedNode);
		if (FunctionCallNode != nullptr)
		{
			FFunctionData& FunctionData = OutFunctionDatas.AddDefaulted_GetRef();
			FunctionData.FunctionCallNode = FunctionCallNode;

			FCompileConstantResolver CompileConstantResolver = EmitterHandle == nullptr
				? FCompileConstantResolver(&System, Script->GetUsage())
				: FCompileConstantResolver(EmitterHandle->GetInstance(), Script->GetUsage());

			FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options = FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly;
			FNiagaraStackGraphUtilities::GetStackFunctionInputs(*FunctionCallNode, FunctionData.InputVariables, FunctionData.HiddenInputVariables, CompileConstantResolver, Options);
			FNiagaraStackGraphUtilities::GetStackFunctionStaticSwitchPins(*FunctionCallNode, FunctionData.InputPins, FunctionData.HiddenInputPins, CompileConstantResolver);
		}
	}
}

void UNiagaraTraversalCacheAuditCommandlet::CollectScriptData(UNiagaraSystem* System, const FNiagaraEmitterHandle* EmitterHandle, UNiagaraScript* Script, TArray<FScriptData>& OutScriptDatas) const
{
	FScriptData& ScriptData = OutScriptDatas.AddDefaulted_GetRef();
	ScriptData.Script = Script;
	ScriptData.EmitterHandle = EmitterHandle;
	CollectFunctionData(*System, EmitterHandle, Script, ScriptData.FunctionDatas);
}

void UNiagaraTraversalCacheAuditCommandlet::CollectScriptDataForSystem(FSystemData& SystemData) const
{
	CollectScriptData(SystemData.System, nullptr, SystemData.System->GetSystemSpawnScript(), SystemData.ScriptDatas);
	CollectScriptData(SystemData.System, nullptr, SystemData.System->GetSystemUpdateScript(), SystemData.ScriptDatas);
	for (const FNiagaraEmitterHandle& EmitterHandle : SystemData.System->GetEmitterHandles())
	{
		FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetInstance().GetEmitterData();
		CollectScriptData(SystemData.System, &EmitterHandle, EmitterData->EmitterSpawnScriptProps.Script, SystemData.ScriptDatas);
		CollectScriptData(SystemData.System, &EmitterHandle, EmitterData->EmitterUpdateScriptProps.Script, SystemData.ScriptDatas);
		CollectScriptData(SystemData.System, &EmitterHandle, EmitterData->SpawnScriptProps.Script, SystemData.ScriptDatas);
		CollectScriptData(SystemData.System, &EmitterHandle, EmitterData->UpdateScriptProps.Script, SystemData.ScriptDatas);
		for (const FNiagaraEventScriptProperties& EventScript : EmitterData->EventHandlerScriptProps)
		{
			CollectScriptData(SystemData.System, &EmitterHandle, EventScript.Script, SystemData.ScriptDatas);
		}
		for (UNiagaraSimulationStageBase* SimulationStage : EmitterData->GetSimulationStages())
		{
			CollectScriptData(SystemData.System, &EmitterHandle, SimulationStage->Script, SystemData.ScriptDatas);
		}
	}
}

void UNiagaraTraversalCacheAuditCommandlet::AuditScript(UNiagaraSystem& AuditSystem, FScriptData& AuditScriptData, TArray<FAuditResultScriptData>& OutScriptResults) const
{
	using namespace NiagaraTraversalCacheAuditUtility;

	TArray<FFunctionData> TraversalCacheFunctionDatas;
	CollectFunctionData(AuditSystem, AuditScriptData.EmitterHandle, AuditScriptData.Script, TraversalCacheFunctionDatas);

	TArray<FAuditResultFunctionData> FunctionResults;
	for (FFunctionData& AuditFunctionData : AuditScriptData.FunctionDatas)
	{
		FFunctionData* TraversalCacheFunctionData = TraversalCacheFunctionDatas.FindByPredicate([&AuditFunctionData](const FFunctionData& TraversalCacheFunctionData)
			{ return TraversalCacheFunctionData.FunctionCallNode == AuditFunctionData.FunctionCallNode; });
		if (TraversalCacheFunctionData == nullptr)
		{
			FString ScriptName = AuditScriptData.EmitterHandle != nullptr
				? FString::Printf(TEXT("%s - %s"), *AuditScriptData.EmitterHandle->GetUniqueInstanceName(), *TopLevelScriptToScriptName(AuditScriptData.Script))
				: TopLevelScriptToScriptName(AuditScriptData.Script);
			UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Error, "System: %ls - Script %ls - Function %ls - Not found in traversal cache data.",
				*AuditSystem.GetPathName(), *ScriptName, *AuditFunctionData.FunctionCallNode->GetFunctionName());
		}
		else
		{
			TArray<FNiagaraVariableBase> AddedInputVariables;
			TArray<FNiagaraVariableBase> RemovedInputVariables;
			TArray<FNiagaraVariableBase> AddedHiddenInputVariables;
			TArray<FNiagaraVariableBase> RemovedHiddenInputVariables;
			auto CompareVariablesForSort = [](const FNiagaraVariable& A, const FNiagaraVariable& B) { return A.GetName().ToString() < B.GetName().ToString(); };
			auto TransformVariableToVariableBase = [](const FNiagaraVariable& Item)->FNiagaraVariableBase { return Item; };
			bool bVariableCompareMatched = CompareCollections(
				AuditFunctionData.InputVariables, TraversalCacheFunctionData->InputVariables,
				AddedInputVariables, RemovedInputVariables,
				AuditFunctionData.HiddenInputVariables, TraversalCacheFunctionData->HiddenInputVariables,
				AddedHiddenInputVariables, RemovedHiddenInputVariables,
				CompareVariablesForSort, TransformVariableToVariableBase);

			TArray<FNiagaraVariableBase> AddedStaticInputPinVariables;
			TArray<FNiagaraVariableBase> RemovedStaticInputPinVariables;
			TArray<FNiagaraVariableBase> AddedHiddenStaticInputPinVariables;
			TArray<FNiagaraVariableBase> RemovedHiddenStaticInputPinVariables;
			auto ComparePinsForSort = [](const UEdGraphPin& A, const UEdGraphPin& B) { return A.PinName.ToString() < B.PinName.ToString(); };
			auto TransformPinToVariableBase = [](const UEdGraphPin* Pin)->FNiagaraVariableBase { return UEdGraphSchema_Niagara::PinToNiagaraVariable(Pin); };
			bool bPinCompareMatched = CompareCollections(
				AuditFunctionData.InputPins, TraversalCacheFunctionData->InputPins,
				AddedStaticInputPinVariables, RemovedStaticInputPinVariables,
				AuditFunctionData.HiddenInputPins, TraversalCacheFunctionData->HiddenInputPins,
				AddedHiddenStaticInputPinVariables, RemovedHiddenStaticInputPinVariables,
				ComparePinsForSort, TransformPinToVariableBase);

			if (bVariableCompareMatched == false || bPinCompareMatched == false)
			{
				FAuditResultFunctionData& FunctionResult = FunctionResults.AddDefaulted_GetRef();
				FunctionResult.FunctionName = TraversalCacheFunctionData->FunctionCallNode->GetFunctionName();
				FunctionResult.FunctionScriptPath = *TraversalCacheFunctionData->FunctionCallNode->FunctionScript->GetPathName();
				FunctionResult.bFunctionEnabled = TraversalCacheFunctionData->FunctionCallNode->GetDesiredEnabledState() != ENodeEnabledState::Disabled;

				FunctionResult.AddedInputVariables = AddedInputVariables;
				FunctionResult.RemovedInputVariables = RemovedInputVariables;
				FunctionResult.AddedHiddenInputVariables = AddedHiddenInputVariables;
				FunctionResult.RemovedHiddenInputVariables = RemovedHiddenInputVariables;

				FunctionResult.AddedStaticInputPinVariables = AddedStaticInputPinVariables;
				FunctionResult.RemovedStaticInputPinVariables = RemovedStaticInputPinVariables;
				FunctionResult.AddedHiddenStaticInputPinVariables = AddedHiddenStaticInputPinVariables;
				FunctionResult.RemovedHiddenStaticInputPinVariables = RemovedHiddenStaticInputPinVariables;
			}
		}
	}

	if (FunctionResults.Num() > 0)
	{
		FAuditResultScriptData& ScriptResult = OutScriptResults.AddDefaulted_GetRef();
		ScriptResult.ScriptName = TopLevelScriptToScriptName(AuditScriptData.Script);
		ScriptResult.EmitterName = AuditScriptData.EmitterHandle != nullptr ? AuditScriptData.EmitterHandle->GetUniqueInstanceName() : TEXT("");
		ScriptResult.FunctionDatas = FunctionResults;
	}
}

void UNiagaraTraversalCacheAuditCommandlet::LogSystemResult(const FAuditResultSystemData& SystemResult, int32 SystemResultIndex) const
{
	for (const FAuditResultScriptData& ScriptResult : SystemResult.ScriptDatas)
	{
		UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Error, "[%i] System: %ls Emitter: %ls",
			SystemResultIndex + 1, *SystemResult.SystemPath, *ScriptResult.EmitterName);
		for (const FAuditResultFunctionData& FunctionResult : ScriptResult.FunctionDatas)
		{
			TArray<FString> DifferenceStrings;
			if (FunctionResult.AddedInputVariables.Num() > 0 || FunctionResult.RemovedInputVariables.Num() > 0)
			{
				DifferenceStrings.Add(TEXT("Input Variables"));
			}
			if (FunctionResult.AddedHiddenInputVariables.Num() > 0 || FunctionResult.RemovedHiddenInputVariables.Num() > 0)
			{
				DifferenceStrings.Add(TEXT("Hidden Input Variables"));
			}
			if (FunctionResult.AddedStaticInputPinVariables.Num() > 0 || FunctionResult.RemovedStaticInputPinVariables.Num() > 0)
			{
				DifferenceStrings.Add(TEXT("Static Pins"));
			}
			if (FunctionResult.AddedHiddenStaticInputPinVariables.Num() > 0 || FunctionResult.RemovedHiddenStaticInputPinVariables.Num() > 0)
			{
				DifferenceStrings.Add(TEXT("Hidden Static Pins"));
			}
			FString DifferenceString = FString::Join(DifferenceStrings, TEXT(", "));;
			UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Error, "[%i] Script: %ls Function: %ls%ls Function Script: %ls Difference: %ls",
				SystemResultIndex + 1,
				*ScriptResult.ScriptName,
				*FunctionResult.FunctionName,
				FunctionResult.bFunctionEnabled ? TEXT("") : TEXT(" (Disabled)"),
				FunctionResult.FunctionScriptPath != NAME_None ? *FunctionResult.FunctionScriptPath.ToString() : TEXT("(none)"),
				*DifferenceString);
		}
	}
}

void UNiagaraTraversalCacheAuditCommandlet::AuditSystems(const TArray<FString>& SystemPaths, FAuditResults& Results) const
{
	int32 NumBatches = FMath::CeilToInt((float)SystemPaths.Num() / BatchSize);
	for (int32 BatchIndex = 0; BatchIndex < NumBatches && (MaxErrorCount == 0 || Results.ErrorCount < MaxErrorCount); BatchIndex++)
	{
		// Disable the traversal cache and load all of the systems in this batch.
		EnableTraversalCacheCVar->SetWithCurrentPriority(0);
		TArray<FSystemData> BatchSystems;
		int32 SystemPathStartIndex = BatchIndex * BatchSize;
		int32 SystemPathEndIndex = FMath::Min(SystemPathStartIndex + BatchSize, SystemPaths.Num());
		for (int32 SystemPathIndex = SystemPathStartIndex; SystemPathIndex < SystemPathEndIndex; SystemPathIndex++)
		{
			FSystemData& SystemData = BatchSystems.AddDefaulted_GetRef();
			SystemData.SystemPath = SystemPaths[SystemPathIndex];
			SystemData.System = LoadObject<UNiagaraSystem>(GetTransientPackage(), *SystemData.SystemPath);
			if (SystemData.System == nullptr)
			{
				UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Warning, "[%i / %i] Load failed - %ls", SystemPathIndex + 1, SystemPaths.Num(), *SystemData.SystemPath);
			}
			else
			{
				UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Display, "[%i / %i] Loaded - %ls", SystemPathIndex + 1, SystemPaths.Num(), *SystemData.SystemPath);
				SystemData.System->WaitForCompilationComplete();
			}
		}

		// Collect the legacy script and function data for the loaded systems on the main thread.
		for (int32 BatchSystemIndex = 0; BatchSystemIndex < BatchSystems.Num(); BatchSystemIndex++)
		{
			FSystemData& SystemData = BatchSystems[BatchSystemIndex];
			int32 SystemPathIndex = SystemPathStartIndex + BatchSystemIndex;
			if (SystemData.System == nullptr)
			{
				continue;
			}

			UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Display, "[%i / %i] Collecting Legacy Data - %ls", SystemPathIndex + 1, SystemPaths.Num(), *SystemPaths[SystemPathIndex]);
			CollectScriptDataForSystem(SystemData);
		}

		// Enable the traversal cache and collect the script and function data again on worker threads and compare with the legacy data.
		EnableTraversalCacheCVar->SetWithCurrentPriority(1);
		ParallelFor(BatchSystems.Num(), [&](int32 BatchSystemIndex)
		{
			if (MaxErrorCount != 0 && Results.ErrorCount > MaxErrorCount)
			{
				return;
			}
			int32 SystemPathIndex = SystemPathStartIndex + BatchSystemIndex;
			FSystemData& AuditSystemData = BatchSystems[BatchSystemIndex];
			if (AuditSystemData.System == nullptr)
			{
				UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Warning, "[%i / %i] Skipping Validation - %ls wasn't loaded", SystemPathIndex + 1, SystemPaths.Num(), *SystemPaths[SystemPathIndex]);
				return;
			}

			UE_LOGF(LogNiagaraTraversalCacheAuditCommandlet, Display, "[%i / %i] Validating - %ls", SystemPathIndex + 1, SystemPaths.Num(), *SystemPaths[SystemPathIndex]);

			TArray<FAuditResultScriptData> ScriptResults;
			for (FScriptData& AuditScriptData : AuditSystemData.ScriptDatas)
			{
				AuditScript(*AuditSystemData.System, AuditScriptData, ScriptResults);
			}

			if (ScriptResults.Num() > 0)
			{
				FScopeLock ResultLock(&ResultCS);
				FAuditResultSystemData& SystemResult = Results.SystemDatas.AddDefaulted_GetRef();
				SystemResult.SystemPath = AuditSystemData.SystemPath;
				SystemResult.ScriptDatas = ScriptResults;
				LogSystemResult(SystemResult, Results.SystemDatas.Num() - 1);
				Results.ErrorCount += SystemResult.CountDifferences();
			}
		});

		// Collect garbage.
		::CollectGarbage(RF_NoFlags);
	}

	if (Results.SystemDatas.Num() > 0 && bWriteCSV)
	{
		WriteCSVFile(Results);
	}
}

void UNiagaraTraversalCacheAuditCommandlet::WriteCSVFile(const FAuditResults& Results) const
{
	int32 Year, Month, DayOfWeek, Day, Hour, Min, Sec, MSec;
	FPlatformTime::SystemTime(Year, Month, DayOfWeek, Day, Hour, Min, Sec, MSec);
	FString FileName = FString::Printf(TEXT("TraversalCache_%04i-%02i-%02i_%02i-%02i-%02i.csv"), Year, Month, Day, Hour, Min, Sec);
	const FString AuditFilePath = FPaths::ProjectSavedDir() / TEXT("Audit") / FileName;

	TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*AuditFilePath));
	TUniquePtr<FOutputDeviceArchiveWrapper> OutputStream(new FOutputDeviceArchiveWrapper(FileArchive.Get()));

	auto AddRows = [&OutputStream](
		const FAuditResultSystemData& SystemResult,
		const FAuditResultScriptData& ScriptResult,
		const FAuditResultFunctionData& FunctionResult,
		const TArray<FNiagaraVariableBase>& ItemVariables,
		const FString& TypeString,
		const FString& OperationString)
	{
		for (const FNiagaraVariableBase& ItemVariable : ItemVariables)
		{
			OutputStream->Logf(TEXT("%s, %s, %s, %s, %s, %s, %s, %s"),
				*SystemResult.SystemPath, *ScriptResult.EmitterName, *ScriptResult.ScriptName,
				*FunctionResult.FunctionName, FunctionResult.bFunctionEnabled ? TEXT("Enabled") : TEXT("Disabled"),
				*TypeString, *OperationString, *ItemVariable.GetName().ToString());
		}
	};

	OutputStream->Log(TEXT("SystemPath, EmitterName, ScriptName, FunctionName, Enabled, Type, Operation, Name"));
	for (const FAuditResultSystemData& SystemResult : Results.SystemDatas)
	{
		for (const FAuditResultScriptData& ScriptResult : SystemResult.ScriptDatas)
		{
			for (const FAuditResultFunctionData& FunctionResult : ScriptResult.FunctionDatas)
			{
				AddRows(SystemResult, ScriptResult, FunctionResult, FunctionResult.AddedInputVariables, TEXT("InputVariable"), TEXT("Added"));
				AddRows(SystemResult, ScriptResult, FunctionResult, FunctionResult.RemovedInputVariables, TEXT("InputVariable"), TEXT("Removed"));
				AddRows(SystemResult, ScriptResult, FunctionResult, FunctionResult.AddedHiddenInputVariables, TEXT("HiddenInputVariable"), TEXT("Added"));
				AddRows(SystemResult, ScriptResult, FunctionResult, FunctionResult.RemovedHiddenInputVariables, TEXT("HiddenInputVariable"), TEXT("Removed"));

				AddRows(SystemResult, ScriptResult, FunctionResult, FunctionResult.AddedStaticInputPinVariables, TEXT("StaticInputPin"), TEXT("Added"));
				AddRows(SystemResult, ScriptResult, FunctionResult, FunctionResult.RemovedStaticInputPinVariables, TEXT("StaticInputPin"), TEXT("Removed"));
				AddRows(SystemResult, ScriptResult, FunctionResult, FunctionResult.AddedHiddenStaticInputPinVariables, TEXT("HiddenStaticInputPin"), TEXT("Added"));
				AddRows(SystemResult, ScriptResult, FunctionResult, FunctionResult.RemovedHiddenStaticInputPinVariables, TEXT("HiddenStaticInputPin"), TEXT("Removed"));
			}
		}
	}
}

