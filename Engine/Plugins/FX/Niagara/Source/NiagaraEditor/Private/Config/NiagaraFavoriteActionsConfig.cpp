// Copyright Epic Games, Inc. All Rights Reserved.

#include "Config/NiagaraFavoriteActionsConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraFavoriteActionsConfig)

TStrongObjectPtr<UNiagaraFavoriteActionsConfig> UNiagaraFavoriteActionsConfig::Instance = nullptr;

bool FNiagaraFavoriteActionsProfile::IsFavorite(FNiagaraFavoritesActionData InAction)
{
	if(InAction.ActionIdentifier.IsValid() == false)
	{
		return false;
	}
	
	if(InAction.bFavoriteByDefault)
	{
		return UnfavoriteActions.Contains(InAction.ActionIdentifier) == false;
	}
	
	return FavoriteActions.Contains(InAction.ActionIdentifier);
}

void FNiagaraFavoriteActionsProfile::ToggleFavoriteAction(FNiagaraFavoritesActionData InAction)
{
	if(InAction.bFavoriteByDefault)
	{
		if(UnfavoriteActions.Contains(InAction.ActionIdentifier))
		{
			UnfavoriteActions.Remove(InAction.ActionIdentifier);
		}
		else
		{
			UnfavoriteActions.Add(InAction.ActionIdentifier);
		}
	}
	else
	{
		if(FavoriteActions.Contains(InAction.ActionIdentifier))
		{
			FavoriteActions.Remove(InAction.ActionIdentifier);
		}
		else
		{
			FavoriteActions.Add(InAction.ActionIdentifier);
		}
	}
}

FNiagaraFavoriteActionsProfile& UNiagaraFavoriteActionsConfig::GetActionsProfile(FName ProfileName)
{
	return Profiles.FindOrAdd(ProfileName);
}

UNiagaraFavoriteActionsConfig* UNiagaraFavoriteActionsConfig::Get()
{
	if(Instance.IsValid() == false)
	{
		Instance.Reset(NewObject<UNiagaraFavoriteActionsConfig>());
		Instance->LoadEditorConfig();
	}

	return Instance.Get();
}

void UNiagaraFavoriteActionsConfig::Shutdown()
{
	if(Instance != nullptr)
	{
		Instance->SaveEditorConfig();
		Instance.Reset();
	}
}
