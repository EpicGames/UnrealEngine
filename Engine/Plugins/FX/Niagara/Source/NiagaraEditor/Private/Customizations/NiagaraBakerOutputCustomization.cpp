// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerOutputCustomization.h"
#include "NiagaraBakerOutputSimCache.h"
#include "NiagaraBakerOutputStaticMesh.h"
#include "NiagaraBakerOutputTexture2D.h"
#include "NiagaraBakerOutputVolumeTexture.h"
#include "NiagaraBakerOutputSparseVolumeTexture.h"

#include "Editor/EditorEngine.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IDetailCustomization.h"
#include "IDetailPropertyRow.h"
#include "IDetailsView.h"

#define LOCTEXT_NAMESPACE "NiagaraBakerOutputCustomization"

//////////////////////////////////////////////////////////////////////////

void FNiagaraBakerOutputDetails::FocusContentBrowserToAsset(const FString& AssetPath)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	TArray<FAssetData> AssetDatas;
	AssetRegistry.GetAssetsByPackageName(FName(*AssetPath), AssetDatas);

	TArray<UObject*> AssetObjects;
	AssetObjects.Reserve(AssetDatas.Num());
	for (const FAssetData& AssetData : AssetDatas)
	{
		if (UObject* AssetObject = AssetData.GetAsset())
		{
			AssetObjects.Add(AssetObject);
		}
	}

	if (AssetObjects.Num() > 0)
	{
		GEditor->SyncBrowserToObjects(AssetObjects);
	}
}

void FNiagaraBakerOutputDetails::ExploreFolder(const FString& Folder)
{
	if ( FPaths::DirectoryExists(Folder) )
	{
		FPlatformProcess::ExploreFolder(*Folder);
	}
}

void FNiagaraBakerOutputDetails::BuildAssetPathWidget(IDetailCategoryBuilder& DetailCategory, TSharedRef<IPropertyHandle>& PropertyHandle, TAttribute<FText>::FGetter TooltipGetter, FOnClicked OnClicked)
{
	DetailCategory.AddProperty(PropertyHandle)
		.CustomWidget()
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				PropertyHandle->CreatePropertyValueWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.ToolTipText(TAttribute<FText>::Create(TooltipGetter))
				.OnClicked(OnClicked)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Icons.Search")))
				]
			]
		];
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FNiagaraBakerOutputSimCacheDetails::MakeInstance()
{
	return MakeShared<FNiagaraBakerOutputSimCacheDetails>();
}

FText FNiagaraBakerOutputSimCacheDetails::BrowseAssetToolTipText(TWeakObjectPtr<UNiagaraBakerOutputSimCache> WeakOutput)
{
	UNiagaraBakerOutputSimCache* Output = WeakOutput.Get();
	return FText::Format(LOCTEXT("BrowseAssetToolTipFormat", "Browse to asset '{0}'"), FText::FromString(Output ? Output->GetAssetPath(Output->SimCacheAssetPathFormat, 0) : FString()));
}

FReply FNiagaraBakerOutputSimCacheDetails::BrowseToAsset(TWeakObjectPtr<UNiagaraBakerOutputSimCache> WeakOutput)
{
	if (UNiagaraBakerOutputSimCache* Output = WeakOutput.Get() )
	{
		const FString AssetPath = Output->GetAssetPath(Output->SimCacheAssetPathFormat, 0);
		FocusContentBrowserToAsset(AssetPath);
	}
	return FReply::Handled();
}

void FNiagaraBakerOutputSimCacheDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// We only support customization on 1 object
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	if (ObjectsCustomized.Num() != 1 || !ObjectsCustomized[0]->IsA<UNiagaraBakerOutputSimCache>())
	{
		return;
	}
	TWeakObjectPtr<UNiagaraBakerOutputSimCache> WeakOutput = CastChecked<UNiagaraBakerOutputSimCache>(ObjectsCustomized[0]);

	static FName NAME_Settings("Settings");
	IDetailCategoryBuilder& DetailCategory = DetailBuilder.EditCategory(NAME_Settings);

	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	DetailCategory.GetDefaultProperties(CategoryProperties);

	for (TSharedRef<IPropertyHandle>& PropertyHandle : CategoryProperties)
	{
		const FName PropertyName = PropertyHandle->GetProperty() ? PropertyHandle->GetProperty()->GetFName() : NAME_None;
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputSimCache, SimCacheAssetPathFormat))
		{
			BuildAssetPathWidget(
				DetailCategory, PropertyHandle,
				TAttribute<FText>::FGetter::CreateStatic(&FNiagaraBakerOutputSimCacheDetails::BrowseAssetToolTipText, WeakOutput),
				FOnClicked::CreateStatic(&FNiagaraBakerOutputSimCacheDetails::BrowseToAsset, WeakOutput)
			);
		}
		else
		{
			DetailCategory.AddProperty(PropertyHandle);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FNiagaraBakerOutputStaticMeshDetails::MakeInstance()
{
	return MakeShared<FNiagaraBakerOutputStaticMeshDetails>();
}

FText FNiagaraBakerOutputStaticMeshDetails::BrowseAssetToolTipText(TWeakObjectPtr<UNiagaraBakerOutputStaticMesh> WeakOutput)
{
	UNiagaraBakerOutputStaticMesh* Output = WeakOutput.Get();
	return FText::Format(LOCTEXT("BrowseAssetToolTipFormat", "Browse to asset '{0}'"), FText::FromString(Output ? Output->GetAssetPath(Output->FramesAssetPathFormat, 0) : FString()));
}

FReply FNiagaraBakerOutputStaticMeshDetails::BrowseToAsset(TWeakObjectPtr<UNiagaraBakerOutputStaticMesh> WeakOutput)
{
	if (UNiagaraBakerOutputStaticMesh* Output = WeakOutput.Get())
	{
		const FString AssetPath = Output->GetAssetPath(Output->FramesAssetPathFormat, 0);
		FocusContentBrowserToAsset(AssetPath);
	}
	return FReply::Handled();
}

void FNiagaraBakerOutputStaticMeshDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// We only support customization on 1 object
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	if (ObjectsCustomized.Num() != 1 || !ObjectsCustomized[0]->IsA<UNiagaraBakerOutputStaticMesh>())
	{
		return;
	}
	TWeakObjectPtr<UNiagaraBakerOutputStaticMesh> WeakOutput = CastChecked<UNiagaraBakerOutputStaticMesh>(ObjectsCustomized[0]);

	static FName NAME_Settings("Settings");
	IDetailCategoryBuilder& DetailCategory = DetailBuilder.EditCategory(NAME_Settings);

	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	DetailCategory.GetDefaultProperties(CategoryProperties);

	for (TSharedRef<IPropertyHandle>& PropertyHandle : CategoryProperties)
	{
		const FName PropertyName = PropertyHandle->GetProperty() ? PropertyHandle->GetProperty()->GetFName() : NAME_None;
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputStaticMesh, FramesAssetPathFormat))
		{
			BuildAssetPathWidget(
				DetailCategory, PropertyHandle,
				TAttribute<FText>::FGetter::CreateStatic(&FNiagaraBakerOutputStaticMeshDetails::BrowseAssetToolTipText, WeakOutput),
				FOnClicked::CreateStatic(&FNiagaraBakerOutputStaticMeshDetails::BrowseToAsset, WeakOutput)
			);
		}
		else
		{
			DetailCategory.AddProperty(PropertyHandle);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FNiagaraBakerOutputTexture2DDetails::MakeInstance()
{
	return MakeShared<FNiagaraBakerOutputTexture2DDetails>();
}

FText FNiagaraBakerOutputTexture2DDetails::BrowseAtlasToolTipText(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput)
{
	UNiagaraBakerOutputTexture2D* Output = WeakOutput.Get();
	return FText::Format(LOCTEXT("BrowseAtlasToolTipFormat", "Browse to atlas asset '{0}'"), FText::FromString(Output ? Output->GetAssetPath(Output->AtlasAssetPathFormat, 0) : FString()));
}

FReply FNiagaraBakerOutputTexture2DDetails::BrowseToAtlas(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput)
{
	if ( UNiagaraBakerOutputTexture2D* Output = WeakOutput.Get() )
	{
		const FString AssetPath = Output->GetAssetPath(Output->AtlasAssetPathFormat, 0);
		FocusContentBrowserToAsset(AssetPath);
	}
	return FReply::Handled();
}

FText FNiagaraBakerOutputTexture2DDetails::BrowseFrameAssetsToolTipText(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput)
{
	UNiagaraBakerOutputTexture2D* Output = WeakOutput.Get();
	return FText::Format(LOCTEXT("BrowseFrameAssetsToolTipFormat", "Browse to frames assets '{0}'"), FText::FromString(Output ? Output->GetAssetPath(Output->FramesAssetPathFormat, 0) : FString()));
}

FReply FNiagaraBakerOutputTexture2DDetails::BrowseToFrameAssets(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput)
{
	if (UNiagaraBakerOutputTexture2D* Output = WeakOutput.Get())
	{
		const FString AssetPath = Output->GetAssetPath(Output->FramesAssetPathFormat, 0);
		FocusContentBrowserToAsset(AssetPath);
	}
	return FReply::Handled();
}

FText FNiagaraBakerOutputTexture2DDetails::BrowseExportAssetsToolTipText(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput)
{
	UNiagaraBakerOutputTexture2D* Output = WeakOutput.Get();
	return FText::Format(LOCTEXT("BrowseExportAssetsToolTipFormat", "Browse to exported assets '{0}'"), FText::FromString(Output ? Output->GetExportFolder(Output->FramesExportPathFormat, 0) : FString()));
}

FReply FNiagaraBakerOutputTexture2DDetails::BrowseToExportAssets(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput)
{
	if (UNiagaraBakerOutputTexture2D* Output = WeakOutput.Get())
	{
		const FString ExportFolder = Output->GetExportFolder(Output->FramesExportPathFormat, 0);
		ExploreFolder(ExportFolder);
	}
	return FReply::Handled();
}

void FNiagaraBakerOutputTexture2DDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// We only support customization on 1 object
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	if (ObjectsCustomized.Num() != 1 || !ObjectsCustomized[0]->IsA<UNiagaraBakerOutputTexture2D>())
	{
		return;
	}
	TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput = CastChecked<UNiagaraBakerOutputTexture2D>(ObjectsCustomized[0]);

	static FName NAME_Settings("Settings");
	IDetailCategoryBuilder& DetailCategory = DetailBuilder.EditCategory(NAME_Settings);

	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	DetailCategory.GetDefaultProperties(CategoryProperties);

	for (TSharedRef<IPropertyHandle>& PropertyHandle : CategoryProperties)
	{
		const FName PropertyName = PropertyHandle->GetProperty() ? PropertyHandle->GetProperty()->GetFName() : NAME_None;
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputTexture2D, AtlasAssetPathFormat))
		{
			BuildAssetPathWidget(
				DetailCategory, PropertyHandle,
				TAttribute<FText>::FGetter::CreateStatic(&FNiagaraBakerOutputTexture2DDetails::BrowseAtlasToolTipText, WeakOutput),
				FOnClicked::CreateStatic(&FNiagaraBakerOutputTexture2DDetails::BrowseToAtlas, WeakOutput)
			);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputTexture2D, FramesAssetPathFormat))
		{
			BuildAssetPathWidget(
				DetailCategory, PropertyHandle,
				TAttribute<FText>::FGetter::CreateStatic(&FNiagaraBakerOutputTexture2DDetails::BrowseFrameAssetsToolTipText, WeakOutput),
				FOnClicked::CreateStatic(&FNiagaraBakerOutputTexture2DDetails::BrowseToFrameAssets, WeakOutput)
			);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputTexture2D, FramesExportPathFormat))
		{
			BuildAssetPathWidget(
				DetailCategory, PropertyHandle,
				TAttribute<FText>::FGetter::CreateStatic(&FNiagaraBakerOutputTexture2DDetails::BrowseExportAssetsToolTipText, WeakOutput),
				FOnClicked::CreateStatic(&FNiagaraBakerOutputTexture2DDetails::BrowseToExportAssets, WeakOutput)
			);
		}
		else
		{
			DetailCategory.AddProperty(PropertyHandle);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FNiagaraBakerOutputVolumeTextureDetails::MakeInstance()
{
	return MakeShared<FNiagaraBakerOutputVolumeTextureDetails>();
}

FText FNiagaraBakerOutputVolumeTextureDetails::BrowseAtlasToolTipText(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput)
{
	UNiagaraBakerOutputVolumeTexture* Output = WeakOutput.Get();
	return FText::Format(LOCTEXT("BrowseAtlasToolTipFormat", "Browse to atlas asset '{0}'"), FText::FromString(Output ? Output->GetAssetPath(Output->AtlasAssetPathFormat, 0) : FString()));
}

FReply FNiagaraBakerOutputVolumeTextureDetails::BrowseToAtlas(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput)
{
	if (UNiagaraBakerOutputVolumeTexture* Output = WeakOutput.Get() )
	{
		const FString AssetPath = Output->GetAssetPath(Output->AtlasAssetPathFormat, 0);
		FocusContentBrowserToAsset(AssetPath);
	}
	return FReply::Handled();
}

FText FNiagaraBakerOutputVolumeTextureDetails::BrowseFrameAssetsToolTipText(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput)
{
	UNiagaraBakerOutputVolumeTexture* Output = WeakOutput.Get();
	return FText::Format(LOCTEXT("BrowseFrameAssetsToolTipFormat", "Browse to frames assets '{0}'"), FText::FromString(Output ? Output->GetAssetPath(Output->FramesAssetPathFormat, 0) : FString()));
}

FReply FNiagaraBakerOutputVolumeTextureDetails::BrowseToFrameAssets(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput)
{
	if (UNiagaraBakerOutputVolumeTexture* Output = WeakOutput.Get())
	{
		const FString AssetPath = Output->GetAssetPath(Output->FramesAssetPathFormat, 0);
		FocusContentBrowserToAsset(AssetPath);
	}
	return FReply::Handled();
}

FText FNiagaraBakerOutputVolumeTextureDetails::BrowseExportAssetsToolTipText(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput)
{
	UNiagaraBakerOutputVolumeTexture* Output = WeakOutput.Get();
	return FText::Format(LOCTEXT("BrowseExportAssetsToolTipFormat", "Browse to exported assets '{0}'"), FText::FromString(Output ? Output->GetExportFolder(Output->FramesExportPathFormat, 0) : FString()));
}

FReply FNiagaraBakerOutputVolumeTextureDetails::BrowseToExportAssets(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput)
{
	if (UNiagaraBakerOutputVolumeTexture* Output = WeakOutput.Get())
	{
		const FString ExportFolder = Output->GetExportFolder(Output->FramesExportPathFormat, 0);
		ExploreFolder(ExportFolder);
	}
	return FReply::Handled();
}

void FNiagaraBakerOutputVolumeTextureDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// We only support customization on 1 object
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	if (ObjectsCustomized.Num() != 1 || !ObjectsCustomized[0]->IsA<UNiagaraBakerOutputVolumeTexture>())
	{
		return;
	}
	TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput = CastChecked<UNiagaraBakerOutputVolumeTexture>(ObjectsCustomized[0]);

	static FName NAME_Settings("Settings");
	IDetailCategoryBuilder& DetailCategory = DetailBuilder.EditCategory(NAME_Settings);

	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	DetailCategory.GetDefaultProperties(CategoryProperties);

	for (TSharedRef<IPropertyHandle>& PropertyHandle : CategoryProperties)
	{
		const FName PropertyName = PropertyHandle->GetProperty() ? PropertyHandle->GetProperty()->GetFName() : NAME_None;
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputVolumeTexture, AtlasAssetPathFormat))
		{
			BuildAssetPathWidget(
				DetailCategory, PropertyHandle,
				TAttribute<FText>::FGetter::CreateStatic(&FNiagaraBakerOutputVolumeTextureDetails::BrowseAtlasToolTipText, WeakOutput),
				FOnClicked::CreateStatic(&FNiagaraBakerOutputVolumeTextureDetails::BrowseToAtlas, WeakOutput)
			);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputVolumeTexture, FramesAssetPathFormat))
		{
			BuildAssetPathWidget(
				DetailCategory, PropertyHandle,
				TAttribute<FText>::FGetter::CreateStatic(&FNiagaraBakerOutputVolumeTextureDetails::BrowseFrameAssetsToolTipText, WeakOutput),
				FOnClicked::CreateStatic(&FNiagaraBakerOutputVolumeTextureDetails::BrowseToFrameAssets, WeakOutput)
			);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputVolumeTexture, FramesExportPathFormat))
		{
			BuildAssetPathWidget(
				DetailCategory, PropertyHandle,
				TAttribute<FText>::FGetter::CreateStatic(&FNiagaraBakerOutputVolumeTextureDetails::BrowseExportAssetsToolTipText, WeakOutput),
				FOnClicked::CreateStatic(&FNiagaraBakerOutputVolumeTextureDetails::BrowseToExportAssets, WeakOutput)
			);
		}
		else
		{
			DetailCategory.AddProperty(PropertyHandle);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FNiagaraBakerOutputSparseVolumeTextureDetails::MakeInstance()
{
	return MakeShared<FNiagaraBakerOutputSparseVolumeTextureDetails>();
}

FText FNiagaraBakerOutputSparseVolumeTextureDetails::BrowseAtlasToolTipText(TWeakObjectPtr<UNiagaraBakerOutputSparseVolumeTexture> WeakOutput)
{
	UNiagaraBakerOutputSparseVolumeTexture* Output = WeakOutput.Get();
	return FText::Format(LOCTEXT("BrowseAtlasToolTipFormat", "Browse to atlas asset '{0}'"), FText::FromString(Output ? Output->GetAssetPath(Output->SparseVolumeTextureAssetPathFormat, 0) : FString()));
}

FReply FNiagaraBakerOutputSparseVolumeTextureDetails::BrowseToAtlas(TWeakObjectPtr<UNiagaraBakerOutputSparseVolumeTexture> WeakOutput)
{
	if (UNiagaraBakerOutputSparseVolumeTexture* Output = WeakOutput.Get())
	{
		const FString AssetPath = Output->GetAssetPath(Output->SparseVolumeTextureAssetPathFormat, 0);
		FocusContentBrowserToAsset(AssetPath);
	}
	return FReply::Handled();
}


void FNiagaraBakerOutputSparseVolumeTextureDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// We only support customization on 1 object
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	if (ObjectsCustomized.Num() != 1 || !ObjectsCustomized[0]->IsA<UNiagaraBakerOutputSparseVolumeTexture>())
	{
		return;
	}
	TWeakObjectPtr<UNiagaraBakerOutputSparseVolumeTexture> WeakOutput = CastChecked<UNiagaraBakerOutputSparseVolumeTexture>(ObjectsCustomized[0]);

	static FName NAME_Settings("Settings");
	IDetailCategoryBuilder& DetailCategory = DetailBuilder.EditCategory(NAME_Settings);

	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	DetailCategory.GetDefaultProperties(CategoryProperties);

	for (TSharedRef<IPropertyHandle>& PropertyHandle : CategoryProperties)
	{
		const FName PropertyName = PropertyHandle->GetProperty() ? PropertyHandle->GetProperty()->GetFName() : NAME_None;
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraBakerOutputSparseVolumeTexture, SparseVolumeTextureAssetPathFormat))
		{
			BuildAssetPathWidget(
				DetailCategory, PropertyHandle,
				TAttribute<FText>::FGetter::CreateStatic(&FNiagaraBakerOutputSparseVolumeTextureDetails::BrowseAtlasToolTipText, WeakOutput),
				FOnClicked::CreateStatic(&FNiagaraBakerOutputSparseVolumeTextureDetails::BrowseToAtlas, WeakOutput)
			);
		}
		else
		{
			DetailCategory.AddProperty(PropertyHandle);
		}
	}
}
#undef LOCTEXT_NAMESPACE
