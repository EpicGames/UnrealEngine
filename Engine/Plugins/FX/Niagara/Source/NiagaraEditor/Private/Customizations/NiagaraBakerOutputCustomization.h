// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Framework/SlateDelegates.h"
#include "Misc/Attribute.h"

class IDetailCategoryBuilder;
class IPropertyHandle;

class UNiagaraBakerOutputSimCache;
class UNiagaraBakerOutputStaticMesh;
class UNiagaraBakerOutputTexture2D;
class UNiagaraBakerOutputVolumeTexture;
class UNiagaraBakerOutputSparseVolumeTexture;

struct FNiagaraBakerOutputDetails : public IDetailCustomization
{
	static void FocusContentBrowserToAsset(const FString& AssetPath);
	static void ExploreFolder(const FString& Folder);
	static void BuildAssetPathWidget(IDetailCategoryBuilder& DetailCategory, TSharedRef<IPropertyHandle>& PropertyHandle, TAttribute<FText>::FGetter TooltipGetter, FOnClicked OnClicked);
};

struct FNiagaraBakerOutputSimCacheDetails : public FNiagaraBakerOutputDetails
{
	static TSharedRef<IDetailCustomization> MakeInstance();

	static FText BrowseAssetToolTipText(TWeakObjectPtr<UNiagaraBakerOutputSimCache> WeakOutput);
	static FReply BrowseToAsset(TWeakObjectPtr<UNiagaraBakerOutputSimCache> WeakOutput);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

struct FNiagaraBakerOutputStaticMeshDetails : public FNiagaraBakerOutputDetails
{
	static TSharedRef<IDetailCustomization> MakeInstance();

	static FText BrowseAssetToolTipText(TWeakObjectPtr<UNiagaraBakerOutputStaticMesh> WeakOutput);
	static FReply BrowseToAsset(TWeakObjectPtr<UNiagaraBakerOutputStaticMesh> WeakOutput);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

struct FNiagaraBakerOutputTexture2DDetails : public FNiagaraBakerOutputDetails
{
	static TSharedRef<IDetailCustomization> MakeInstance();

	static FText BrowseAtlasToolTipText(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput);
	static FReply BrowseToAtlas(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput);

	static FText BrowseFrameAssetsToolTipText(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput);
	static FReply BrowseToFrameAssets(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput);

	static FText BrowseExportAssetsToolTipText(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput);
	static FReply BrowseToExportAssets(TWeakObjectPtr<UNiagaraBakerOutputTexture2D> WeakOutput);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

struct FNiagaraBakerOutputVolumeTextureDetails : public FNiagaraBakerOutputDetails
{
	static TSharedRef<IDetailCustomization> MakeInstance();

	static FText BrowseAtlasToolTipText(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput);
	static FReply BrowseToAtlas(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput);

	static FText BrowseFrameAssetsToolTipText(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput);
	static FReply BrowseToFrameAssets(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput);

	static FText BrowseExportAssetsToolTipText(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput);
	static FReply BrowseToExportAssets(TWeakObjectPtr<UNiagaraBakerOutputVolumeTexture> WeakOutput);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

struct FNiagaraBakerOutputSparseVolumeTextureDetails : public FNiagaraBakerOutputDetails
{
	static TSharedRef<IDetailCustomization> MakeInstance();

	static FText BrowseAtlasToolTipText(TWeakObjectPtr<UNiagaraBakerOutputSparseVolumeTexture> WeakOutput);
	static FReply BrowseToAtlas(TWeakObjectPtr<UNiagaraBakerOutputSparseVolumeTexture> WeakOutput);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};