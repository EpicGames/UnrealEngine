// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerSettingsDetails.h"
#include "NiagaraBakerRenderer.h"
#include "NiagaraBakerSettings.h"
#include "NiagaraSystem.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailGroup.h"
#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"
#include "SGraphActionMenu.h"
#include "Widgets/Input/SComboButton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerSettingsDetails)

#define LOCTEXT_NAMESPACE "NiagaraBakerSettingsDetails"

void FNiagaraBakerTextureSourceDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	[
		SNew(SComboButton)
		.OnGetMenuContent(this, &FNiagaraBakerTextureSourceDetails::OnGetMenuContent)
		.ContentPadding(1)
		.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
		.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &FNiagaraBakerTextureSourceDetails::GetText)
		]
	];
}

FText FNiagaraBakerTextureSourceDetails::GetText() const
{
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	if ( Objects.Num() == 1 )
	{
		FNiagaraBakerTextureSource* TargetVariable = (FNiagaraBakerTextureSource*)PropertyHandle->GetValueBaseAddress((uint8*)Objects[0]);
		if ( TargetVariable->DisplayString.Len() > 0 )
		{
			return FText::FromString(TargetVariable->DisplayString);
		}
		else
		{
			return FText::FromName(TargetVariable->SourceName);
		}
	}

	return LOCTEXT("Error", "Error");
}

TSharedRef<SWidget> FNiagaraBakerTextureSourceDetails::OnGetMenuContent() const
{
	FGraphActionMenuBuilder MenuBuilder;

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			[
				SNew(SGraphActionMenu)
				.OnActionSelected(this, &FNiagaraBakerTextureSourceDetails::OnActionSelected)
				.OnCreateWidgetForAction(this, &FNiagaraBakerTextureSourceDetails::OnCreateWidgetForAction)
				.OnCollectAllActions(this, &FNiagaraBakerTextureSourceDetails::CollectAllActions)
				.AutoExpandActionMenu(false)
				.ShowFilterTextBox(true)
			]
		];
}

void FNiagaraBakerTextureSourceDetails::CollectAllActions(FGraphActionListBuilderBase& OutAllActions) const
{
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	if (Objects.Num() == 1)
	{
		if ( UNiagaraBakerOutput* BakerOutput = Cast<UNiagaraBakerOutput>(Objects[0]) )
		{
			TUniquePtr<FNiagaraBakerOutputRenderer> OutputRenderer(FNiagaraBakerRenderer::GetOutputRenderer(BakerOutput->GetClass()));
			if (OutputRenderer.IsValid())
			{
				for (const FNiagaraBakerOutputBinding& RenderBinding : OutputRenderer->GetRendererBindings(BakerOutput))
				{
					TSharedPtr<FNiagaraBakerTextureSourceAction> NewNodeAction(
						new FNiagaraBakerTextureSourceAction(RenderBinding.BindingName, RenderBinding.MenuCategory, RenderBinding.MenuEntry, FText(), 0, FText())
					);
					OutAllActions.AddAction(NewNodeAction);
				}
			}
		}
	}
}

TSharedRef<SWidget> FNiagaraBakerTextureSourceDetails::OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData) const
{
	return
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(InCreateData->Action->GetMenuDescription())
		];
}

void FNiagaraBakerTextureSourceDetails::OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions, ESelectInfo::Type InSelectionType) const
{
	if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		for (auto& CurrentAction : SelectedActions)
		{
			if (!CurrentAction.IsValid())
			{
				continue;
			}

			FSlateApplication::Get().DismissAllMenus();
			FNiagaraBakerTextureSourceAction* EventSourceAction = (FNiagaraBakerTextureSourceAction*)CurrentAction.Get();

			FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeBinding", " Change binding to \"{0}\" "), FText::FromName(*EventSourceAction->BindingName.ToString())));
			TArray<UObject*> Objects;
			PropertyHandle->GetOuterObjects(Objects);
			for (UObject* Obj : Objects)
			{
				Obj->Modify();
			}

			PropertyHandle->NotifyPreChange();
			FNiagaraBakerTextureSource* TargetVariable = (FNiagaraBakerTextureSource*)PropertyHandle->GetValueBaseAddress((uint8*)Objects[0]);
			TargetVariable->DisplayString = EventSourceAction->GetCategory().ToString() + " - " + EventSourceAction->GetMenuDescription().ToString();
			TargetVariable->SourceName = EventSourceAction->BindingName;
			PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			PropertyHandle->NotifyFinishedChangingProperties();
		}
	}
}

#undef LOCTEXT_NAMESPACE

