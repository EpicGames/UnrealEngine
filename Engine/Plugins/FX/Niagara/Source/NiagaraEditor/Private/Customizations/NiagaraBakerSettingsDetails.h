// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "UObject/ObjectMacros.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

#include "NiagaraBakerSettingsDetails.generated.h"

class IDetailLayoutBuilder;

USTRUCT()
struct FNiagaraBakerTextureSourceAction : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Simple type info
	static FName StaticGetTypeId() { static FName Type("FNiagaraBakerTextureSourceAction"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	FNiagaraBakerTextureSourceAction()
		: FEdGraphSchemaAction()
	{}

	FNiagaraBakerTextureSourceAction(FName InBindingName, FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords))
		, BindingName(InBindingName)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override
	{
		return nullptr;
	}
	//~ End FEdGraphSchemaAction Interface

	FName BindingName;
};

class FNiagaraBakerTextureSourceDetails : public IPropertyTypeCustomization
{
public:
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraBakerTextureSourceDetails>();
	}

private:
	FText GetText() const;
	TSharedRef<SWidget> OnGetMenuContent() const;
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) const;
	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData) const;
	void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions, ESelectInfo::Type InSelectionType) const;

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
};

///** Details customization for Baker texture settings. */
//class FNiagaraBakerTextureSettingsDetails : public IPropertyTypeCustomization
//{
//public:
//	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
//	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
//
//	static TSharedRef<IDetailCustomization> MakeInstance();
//};
//
///** Details customization for Baker settings. */
//class FNiagaraBakerSettingsDetails : public IDetailCustomization
//{
//public:
//	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
//	static TSharedRef<IDetailCustomization> MakeInstance();
//};
