// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraComponentDetails.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "NiagaraComponent.h"
#include "DetailWidgetRow.h"
#include "NiagaraSimCacheCapture.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "ViewModels/NiagaraParameterViewModel.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraEditorModule.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SButton.h"
#include "NiagaraScriptSource.h"
#include "NiagaraNodeInput.h"
#include "GameDelegates.h"
#include "IAssetTools.h"
#include "IDetailChildrenBuilder.h"
#include "NiagaraSettings.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SNiagaraDebugger.h"
#include "Widgets/SNiagaraSystemUserParameters.h"
#include "UserParameters/NiagaraComponentUserParametersBuilder.h"
#include "UserParameters/NiagaraSystemUserParametersBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraComponentDetails)

#define LOCTEXT_NAMESPACE "NiagaraComponentDetails"

TSharedRef<IDetailCustomization> FNiagaraSystemUserParameterDetails::MakeInstance()
{
	return MakeShared<FNiagaraSystemUserParameterDetails>();
}

void FNiagaraSystemUserParameterDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);
	
	TArray<FName> Categories;
	DetailBuilder.GetCategoryNames(Categories);

	for(FName& CategoryName : Categories)
	{
		DetailBuilder.HideCategory(CategoryName);
	}
	
	static const FName ParamCategoryName = TEXT("NiagaraSystem_UserParameters");

	if(CustomizedObjects.Num() == 1 && CustomizedObjects[0]->IsA<UNiagaraSystem>())
	{
		System = Cast<UNiagaraSystem>(CustomizedObjects[0]);
		IDetailCategoryBuilder& InputParamCategory = DetailBuilder.EditCategory(ParamCategoryName, LOCTEXT("ParamCategoryName", "User Parameters"), ECategoryPriority::Important);
		TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(System.Get());
		ensure(SystemViewModel.IsValid());
		TSharedRef<FNiagaraSystemUserParameterBuilder> SystemUserParameterBuilder = MakeShared<FNiagaraSystemUserParameterBuilder>(SystemViewModel, ParamCategoryName);
		InputParamCategory.AddCustomBuilder(SystemUserParameterBuilder);
		TSharedRef<SWidget> AdditionalHeaderWidgets = SystemUserParameterBuilder->GetAdditionalHeaderWidgets();
		InputParamCategory.HeaderContent(AdditionalHeaderWidgets);
	}
}

TSharedRef<IDetailCustomization> FNiagaraComponentDetails::MakeInstance()
{
	return MakeShareable(new FNiagaraComponentDetails);
}

FNiagaraComponentDetails::~FNiagaraComponentDetails()
{
	if (GEngine)
	{
		GEngine->OnWorldDestroyed().RemoveAll(this);
	}

	for (const TWeakObjectPtr<UNiagaraSimCache>& WeakSimCache : CapturedCaches)
	{
		if (UNiagaraSimCache* SimCache = WeakSimCache.Get())
		{
			SimCache->MarkAsGarbage();
		}
	}

	FGameDelegates::Get().GetEndPlayMapDelegate().RemoveAll(this);
}

void FNiagaraComponentDetails::OnPiEEnd()
{
	UE_LOGF(LogNiagaraEditor, Log, "onPieEnd");
	if (Component.IsValid())
	{
		if (Component->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			UE_LOGF(LogNiagaraEditor, Log, "onPieEnd - has package flags");
			UWorld* TheWorld = UWorld::FindWorldInPackage(Component->GetOutermost());
			if (TheWorld)
			{
				OnWorldDestroyed(TheWorld);
			}
		}
	}
}

void FNiagaraComponentDetails::OnWorldDestroyed(class UWorld* InWorld)
{
	// We have to clear out any temp data interfaces that were bound to the component's package when the world goes away or otherwise
	// we'll report GC leaks..
	if (Component.IsValid())
	{
		if (Component->GetWorld() == InWorld)
		{
			UE_LOGF(LogNiagaraEditor, Log, "OnWorldDestroyed - matched up");
			Builder = nullptr;
		}
	}
}

void FNiagaraComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	Builder = &DetailBuilder;

	static const FName ParamCategoryName = TEXT("NiagaraComponent_Parameters");
	static const FName ParamUtilitiesName = TEXT("Niagara Utilities");
	static const FName ScriptCategoryName = TEXT("Parameters");

	static bool bFirstTime = true;
	if (bFirstTime)
	{
		const FText DisplayName = LOCTEXT("EffectsSectionName", "Effects");
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("NiagaraComponent", "Effects", DisplayName);
		Section->AddCategory(TEXT("Niagara"));
		Section->AddCategory(ParamCategoryName);
		Section->AddCategory(ParamUtilitiesName); 
		Section->AddCategory(TEXT("Activation"));
		Section->AddCategory(ScriptCategoryName);
		Section->AddCategory(TEXT("Randomness"));
		Section->AddCategory(TEXT("Warmup"));
		bFirstTime = false;
	}

	TSharedPtr<IPropertyHandle> LocalOverridesPropertyHandle = DetailBuilder.GetProperty("OverrideParameters");
	if (LocalOverridesPropertyHandle.IsValid())
	{
		LocalOverridesPropertyHandle->MarkHiddenByCustomization();
	}

	TSharedPtr<IPropertyHandle> TemplateParameterOverridesPropertyHandle = DetailBuilder.GetProperty("TemplateParameterOverrides");
	TemplateParameterOverridesPropertyHandle->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> InstanceParameterOverridesPropertyHandle = DetailBuilder.GetProperty("InstanceParameterOverrides");
	InstanceParameterOverridesPropertyHandle->MarkHiddenByCustomization();

	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles { TemplateParameterOverridesPropertyHandle, InstanceParameterOverridesPropertyHandle };

	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);

	// we override the sort order by specifying the category priority. For same-category, the order of editing decides.
	DetailBuilder.EditCategory("Niagara", FText::GetEmpty(), ECategoryPriority::Important);
	//DetailBuilder.EditCategory(ParamCategoryName, FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Activation", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Lighting", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Attachment", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Randomness", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Parameters", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Warmup", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Materials", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	
	if (ObjectsCustomized.Num() == 1 && ObjectsCustomized[0]->IsA<UNiagaraComponent>())
	{
		Component = CastChecked<UNiagaraComponent>(ObjectsCustomized[0].Get());
		if (GEngine)
		{
			GEngine->OnWorldDestroyed().AddRaw(this, &FNiagaraComponentDetails::OnWorldDestroyed);
		}

		FGameDelegates::Get().GetEndPlayMapDelegate().AddRaw(this, &FNiagaraComponentDetails::OnPiEEnd);
			
		IDetailCategoryBuilder& InputParamCategory = DetailBuilder.EditCategory(ParamCategoryName, LOCTEXT("ParamCategoryName", "User Parameters"), ECategoryPriority::Important);
		InputParamCategory.AddCustomBuilder(MakeShared<FNiagaraComponentUserParametersNodeBuilder>(Component.Get(), PropertyHandles, ParamCategoryName));
	}
	else if (ObjectsCustomized.Num() > 1)
	{
		IDetailCategoryBuilder& InputParamCategory = DetailBuilder.EditCategory(ParamCategoryName, LOCTEXT("ParamCategoryName", "User Parameters"));
		InputParamCategory.AddCustomRow(LOCTEXT("ParamCategoryName", "User Parameters"))
			.WholeRowContent()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(LOCTEXT("OverrideParameterMultiselectionUnsupported", "Multiple override parameter sets cannot be edited simultaneously."))
			];
	}
	
	IDetailCategoryBuilder& CustomCategory = DetailBuilder.EditCategory(ParamUtilitiesName, FText::GetEmpty(), ECategoryPriority::Important);

	CustomCategory.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		.HAlign(HAlign_Left)
		[
			SNew(SBox)
			.MaxDesiredWidth(300.f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.OnClicked(this, &FNiagaraComponentDetails::OnDebugSelectedSystem)
					.ToolTipText(LOCTEXT("DebugButtonTooltip", "Open Niagara Debugger and point to the first selected particle system"))
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DebugButton", "Debug"))
					]
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.OnClicked(this, &FNiagaraComponentDetails::OnResetSelectedSystem)
					.ToolTipText(LOCTEXT("ResetEmitterButtonTooltip", "Resets the selected particle systems."))
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ResetEmitterButton", "Reset"))
					]
				]
				+ SUniformGridPanel::Slot(2,0)
				[
					SNew(SButton)
					.OnClicked(this, &FNiagaraComponentDetails::OnCaptureSelectedSystem)
					.ToolTipText(LOCTEXT("CaptureSimCacheButtonToolTip", "Capture a temporary sim cache from the first selected particle system."))
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CaptureSimCacheButton", "Capture"))
					]
				]
			]
		];
}

FReply FNiagaraComponentDetails::OnResetSelectedSystem()
{
	if (!Builder)
		return FReply::Handled();

	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = Builder->GetSelectedObjects();

	for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
	{
		if (SelectedObjects[Idx].IsValid())
		{
			if (AActor* Actor = Cast<AActor>(SelectedObjects[Idx].Get()))
			{
				// Note: Must gather then reset as the array we are iterating will be modified with the ReregisterComponent call
				TArray<UNiagaraComponent*> ComponentsToReset;
				for (UActorComponent* AC : Actor->GetComponents())
				{
					if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(AC))
					{
						ComponentsToReset.Add(NiagaraComponent);
					}
				}

				for (UNiagaraComponent* NiagaraComponent : ComponentsToReset)
				{
					NiagaraComponent->Activate(true);
					NiagaraComponent->ReregisterComponent();
				}
			}
			else if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(SelectedObjects[Idx].Get()))
			{
				NiagaraComponent->Activate(true);
				NiagaraComponent->ReregisterComponent();
			}
			
		}
	}

	return FReply::Handled();
}

FReply FNiagaraComponentDetails::OnDebugSelectedSystem()
{
	if (!Builder)
		return FReply::Handled();

	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = Builder->GetSelectedObjects();

	UNiagaraComponent* NiagaraComponentToUse = nullptr;
	for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
	{
		if (SelectedObjects[Idx].IsValid())
		{
			if (AActor* Actor = Cast<AActor>(SelectedObjects[Idx].Get()))
			{
				for (UActorComponent* AC : Actor->GetComponents())
				{
					UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(AC);
					if (NiagaraComponent)
					{
						NiagaraComponentToUse = NiagaraComponent;
						break;
					}
				}
			}
			else if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(SelectedObjects[Idx].Get()))
			{
				NiagaraComponentToUse = NiagaraComponent;
				break;
			}
		}
	}

	if (NiagaraComponentToUse)
	{

#if WITH_NIAGARA_DEBUGGER
		SNiagaraDebugger::InvokeDebugger(NiagaraComponentToUse);
#endif
	}

	return FReply::Handled();
}

FReply FNiagaraComponentDetails::OnCaptureSelectedSystem()
{
	if (!Builder)
		return FReply::Handled();

	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = Builder->GetSelectedObjects();

	UNiagaraComponent* NiagaraComponentToUse = nullptr;
	for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
	{
		if (SelectedObjects[Idx].IsValid())
		{
			if (AActor* Actor = Cast<AActor>(SelectedObjects[Idx].Get()))
			{
				for (UActorComponent* AC : Actor->GetComponents())
				{
					UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(AC);
					if (NiagaraComponent)
					{
						NiagaraComponentToUse = NiagaraComponent;
						break;
					}
				}
			}
			else if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(SelectedObjects[Idx].Get()))
			{
				NiagaraComponentToUse = NiagaraComponent;
				break;
			}
		}
	}

	if (NiagaraComponentToUse)
	{
		TSharedPtr<FNiagaraSimCacheCapture> SimCacheCapture = MakeShared<FNiagaraSimCacheCapture>();
		UNiagaraSimCache* SimCache = NewObject<UNiagaraSimCache>(GetTransientPackage());
		SimCache->SetFlags(RF_Transient);
		CapturedCaches.Emplace(SimCache);

		FNiagaraSimCacheCaptureParameters CaptureParameters;
		CaptureParameters.CaptureRate = 1;
		// A value of 0 would capture until completion. Since this can apply broadly, looping systems would not finish capturing, so don't go below 1.
		CaptureParameters.NumFrames = FMath::Max(1, NiagaraComponentToUse->SimCacheDebugNumFramesToCapture.Get(GetDefault<UNiagaraSettings>()->QuickSimCacheCaptureFrameCount));
		CaptureParameters.bCaptureAllFramesImmediatly = false;
		
		if(NiagaraComponentToUse->GetWorld())
		{
			SimCacheCapture->OnCaptureComplete().AddLambda(
			[SimCacheCapture, this](UNiagaraSimCache* CapturedSimCache)
			{
				if(CapturedSimCache)
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CapturedSimCache, EToolkitMode::Standalone);
				}
			});
			SimCacheCapture->CaptureNiagaraSimCache(SimCache, FNiagaraSimCacheCreateParameters::CreateForDebugging(), NiagaraComponentToUse, CaptureParameters);
		}

		
		return FReply::Handled();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
