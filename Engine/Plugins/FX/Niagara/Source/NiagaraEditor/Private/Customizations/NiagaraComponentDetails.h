// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "NiagaraComponent.h"
#include "NiagaraSimCacheCapture.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "NiagaraComponentDetails.generated.h"

class UNiagaraSystem;

USTRUCT()
struct FNiagaraEnumToByteHelper
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Parameters)
	uint8 Value = 0;
};

UCLASS()
class UNiagaraObjectAssetHelper : public UObject
{
	GENERATED_BODY()

	/** We are customizing the instance property metadata for this to restrict the allowed classes in the UI */
	UPROPERTY(EditAnywhere, Category = Asset)
	FSoftObjectPath Path;
};

class FNiagaraComponentDetails : public IDetailCustomization
{
public:
	virtual ~FNiagaraComponentDetails() override;

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

protected:
	void OnWorldDestroyed(class UWorld* InWorld);
	void OnPiEEnd();

	FReply OnResetSelectedSystem();
	FReply OnDebugSelectedSystem();
	FReply OnCaptureSelectedSystem();
private:
	TWeakObjectPtr<UNiagaraComponent> Component;
	IDetailLayoutBuilder* Builder = nullptr;

	TArray<TWeakObjectPtr<UNiagaraSimCache>> CapturedCaches;
};

class FNiagaraSystemUserParameterDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

protected:
	TWeakObjectPtr<UNiagaraSystem> System;
};
