// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraComponentRendererPropertiesDetails.h"
#include "NiagaraComponentRendererProperties.h" 
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailGroup.h"
#include "NiagaraComponent.h"
#include "NiagaraParameterMapHistory.h"
#include "NiagaraScriptSource.h"
#include "NiagaraGraph.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "NiagaraConstants.h"
#include "Widgets/Input/SButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SComboButton.h"
#include <ObjectEditorUtils.h>

#define LOCTEXT_NAMESPACE "FNiagaraComponentRendererPropertiesDetails"

FNiagaraComponentPropertyBinding ToPropertyBinding(TSharedPtr<IPropertyHandle> PropertyHandle, UNiagaraComponentRendererProperties* ComponentProperties)
{
	FNiagaraComponentPropertyBinding Binding;
	Binding.PropertyName = PropertyHandle->GetProperty()->GetFName();
	Binding.PropertyType = UNiagaraComponentRendererProperties::ToNiagaraType(PropertyHandle->GetProperty());
	Binding.MetadataSetterName = FName(PropertyHandle->GetProperty()->GetMetaData("BlueprintSetter"));

	return Binding;
}

FNiagaraComponentRendererPropertiesDetails::~FNiagaraComponentRendererPropertiesDetails()
{
	if (RendererProperties.IsValid())
	{
		RendererProperties->OnChanged().RemoveAll(this);
	}
}

void FNiagaraComponentRendererPropertiesDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	DetailBuilderWeakPtr = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

void FNiagaraComponentRendererPropertiesDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	if(ObjectsCustomized.Num() != 1 || !ObjectsCustomized[0]->IsA<UNiagaraComponentRendererProperties>())
	{
		return;
	}
	RendererProperties = CastChecked<UNiagaraComponentRendererProperties>(ObjectsCustomized[0].Get());

	// Touch the category so it is always shown on top, otherwise the non-customized properties are shown below the customized ones
	static const FName RenderingCategoryName = TEXT("Component Rendering");
	DetailBuilder.EditCategory(RenderingCategoryName);

	// Show a message for the template component when no class was selected
	TSharedRef<IPropertyHandle> TemplateHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraComponentRendererProperties, TemplateComponent));	
	if (!RendererProperties->TemplateComponent)
	{
		IDetailPropertyRow* DefaultRow = DetailBuilder.EditDefaultProperty(TemplateHandle);
		DefaultRow->CustomWidget().WholeRowContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(LOCTEXT("NiagaraSelectComponentTypeText", "Please select a component class first"))
			]
		];
		return;
	}

	// If we have a template component we hide the default widget and build a new category with all its properties
	if (IDetailPropertyRow* TemplateHandlePropertyRow = DetailBuilder.EditDefaultProperty(TemplateHandle))
	{
		TemplateHandlePropertyRow->Visibility(EVisibility::Collapsed);
	}

	static const FName ComponentCategoryName = TEXT("Component Properties");
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(ComponentCategoryName);

	uint32 PropertyChildCount;
	TemplateHandle->GetChildHandle(0)->GetNumChildren(PropertyChildCount);

	// we need to build a mapping between categories and properties first because the CLASS_CollapseCategories class flag can
	// remove the categories from top-level properties, so we need to traverse them differently
	TMap<FName, TArray<TSharedPtr<IPropertyHandle>>> CategoryMapping;
	UClass* TemplateClass = RendererProperties->TemplateComponent->GetClass();
	bool HasCollapsedCategories = TemplateClass->HasAnyClassFlags(CLASS_CollapseCategories);

	for (uint32 i = 0; i < PropertyChildCount; i++)
	{
		TSharedPtr<IPropertyHandle> TopLevelHandle = TemplateHandle->GetChildHandle(0)->GetChildHandle(i);

		if (HasCollapsedCategories)
		{
			if (!TopLevelHandle.IsValid() || !TopLevelHandle->GetProperty() || !TopLevelHandle->GetProperty()->GetClass())
			{
				continue;
			}

			// with collapsed categories, the properties are given to us directly, without the category properties as parents
			FProperty* Property = TopLevelHandle->GetProperty();
			FName CategoryFName = FObjectEditorUtils::GetCategoryFName(Property);
			CategoryMapping.FindOrAdd(CategoryFName).Add(TopLevelHandle);
		}
		else
		{
			uint32 CategoryChildren;
			TopLevelHandle->GetNumChildren(CategoryChildren);
			for (uint32 k = 0; k < CategoryChildren; k++)
			{
				TSharedPtr<IPropertyHandle> ChildHandle = TopLevelHandle->GetChildHandle(k);
				if (!ChildHandle->GetProperty() || !ChildHandle->GetProperty()->GetClass())
				{
					continue;
				}

				FName CategoryFName = FName(TopLevelHandle->GetPropertyDisplayName().ToString());
				CategoryMapping.FindOrAdd(CategoryFName).Add(ChildHandle);
			}
		}
	}
	

	for (TPair<FName, TArray<TSharedPtr<IPropertyHandle>>> CategoryPair : CategoryMapping)
	{
		if (CategoryPair.Key == FName("Activation"))
		{
			// we don't want the user to change the component activation settings because we need to be able to control that in the component pool
			continue;
		}
		if (CategoryPair.Value.Num() == 0)
		{
			continue;
		}

		// we add the original property categories as groups, as we don't want them to be top-level entries in the ui
		IDetailGroup& CategoryGroup = CategoryBuilder.AddGroup(CategoryPair.Key, FText::FromName(CategoryPair.Key));
		for (TSharedPtr<IPropertyHandle> PropHandle : CategoryPair.Value)
		{
			IDetailPropertyRow& PropertyRow = CategoryGroup.AddPropertyRow(PropHandle.ToSharedRef());

			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;

			// if we have a binding override, we display a different value ui for the property
			if (const FNiagaraComponentPropertyBinding* Binding = FindBinding(PropHandle))
			{
				FDetailWidgetRow& WidgetRow = PropertyRow.CustomWidget(false);
				PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, false);

				WidgetRow.NameContent()
				[
					NameWidget.ToSharedRef()
				];

				bool IsConvertingValue = Binding->PropertyType.IsValid() && Binding->PropertyType != Binding->AttributeBinding.GetType();
				FName StyleName = IsConvertingValue ? FName("FlatButton.Warning") : FName("FlatButton.Success");
				FText Tooltip = IsConvertingValue ? LOCTEXT("NiagaraPropertyBindingToolTipConverting", "Bind to a particle attribute to update this parameter each tick. \nThe currently bound value is auto-converted to fit the target type, which costs some performance.")
												  : LOCTEXT("NiagaraPropertyBindingToolTip", "Bind to a particle attribute to update this parameter each tick.");

				WidgetRow.ValueContent()
				.MaxDesiredWidth(300.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						// The binding selector
						SNew(SComboButton)
						.ButtonStyle(FAppStyle::Get(), StyleName)
						.OnGetMenuContent(this, &FNiagaraComponentRendererPropertiesDetails::GetAddBindingMenuContent, PropHandle)
						.ContentPadding(1)
						.ToolTipText(Tooltip)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text(this, &FNiagaraComponentRendererPropertiesDetails::GetCurrentBindingText, PropHandle)
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(FMargin(10.0f, 0.0f))
					[
						// Delete binding button
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ForegroundColor(FSlateColor::UseForeground())
						.ContentPadding(FMargin(2))
						.ToolTipText(LOCTEXT("NiagaraRemoveBindingToolTip", "Remove the particle attribute binding"))
						.IsFocusable(false)
						.OnClicked(this, &FNiagaraComponentRendererPropertiesDetails::ResetBindingButtonPressed, PropHandle)
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Delete"))
						]
					]
				];
			}
			else
			{
				FDetailWidgetRow& WidgetRow = PropertyRow.CustomWidget(true);
				PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, false);

				WidgetRow.NameContent()
				[
					NameWidget.ToSharedRef()
				];

				WidgetRow.ValueContent()
				.MinDesiredWidth(250.f)
				.MaxDesiredWidth(600.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						// The default property editor
						ValueWidget.ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(FMargin(10.0f, 0.0f))
					[
						// Override binding button
						SNew(SComboButton)
						.IsEnabled(this, &FNiagaraComponentRendererPropertiesDetails::IsOverridableType, PropHandle)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ForegroundColor(FSlateColor::UseForeground())
						.ContentPadding(FMargin(2))
						.ToolTipText(LOCTEXT("NiagaraRendererChangePropertyBindingToolTip", "Bind to a particle attribute to update this parameter each tick"))
						.MenuPlacement(MenuPlacement_BelowRightAnchor)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnGetMenuContent(this, &FNiagaraComponentRendererPropertiesDetails::GetAddBindingMenuContent, PropHandle)
					]
				];
			}
		}
	}
}

TSharedRef<IDetailCustomization> FNiagaraComponentRendererPropertiesDetails::MakeInstance()
{
	return MakeShared<FNiagaraComponentRendererPropertiesDetails>();
}

TSharedRef<SWidget> FNiagaraComponentRendererPropertiesDetails::GetAddBindingMenuContent(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FMenuBuilder AddMenuBuilder(true, nullptr);
	const TArray<FNiagaraVariable>& PossibleBindings = GetPossibleBindings(PropertyHandle);
	if (PossibleBindings.Num() == 0)
	{
		AddMenuBuilder.AddMenuEntry(FText::FromString(TEXT("No suitable binding available")), FText(), FSlateIcon(), FUIAction());
	}

	for (const FNiagaraVariable& BindingVar : PossibleBindings)
	{
		AddMenuBuilder.AddMenuEntry(
			FText::FromName(BindingVar.GetName()),
			FText(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this, PropertyHandle, BindingVar]()	{ ChangePropertyBinding(PropertyHandle, BindingVar); }))
		);
	}
	return AddMenuBuilder.MakeWidget();
}

void FNiagaraComponentRendererPropertiesDetails::ChangePropertyBinding(TSharedPtr<IPropertyHandle> PropertyHandle, const FNiagaraVariable& BindingVar)
{
	if (UNiagaraComponentRendererProperties* ComponentProperties = RendererProperties.Get())
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangePropertyBinding", "Change component property binding to \"{0}\" "), FText::FromName(BindingVar.GetName())));
		FNiagaraComponentPropertyBinding NewBinding = ToPropertyBinding(PropertyHandle, ComponentProperties);
		NewBinding.AttributeBinding.Setup(BindingVar, BindingVar);

		ComponentProperties->Modify();
		PropertyHandle->NotifyPreChange();

		if (const FNiagaraComponentPropertyBinding* PropertyBinding = FindBinding(PropertyHandle))
		{
			// this is an update, so remove the preexisting binding
			for (int i = ComponentProperties->PropertyBindings.Num() - 1; i >= 0; i--)
			{
				const FNiagaraComponentPropertyBinding& Binding = ComponentProperties->PropertyBindings[i];
				if (Binding.PropertyName == NewBinding.PropertyName)
				{
					ComponentProperties->PropertyBindings.RemoveAt(i);
				}
			}
		}
		ComponentProperties->PropertyBindings.Add(NewBinding);

		PropertyHandle->NotifyPostChange(EPropertyChangeType::Unspecified);
		PropertyHandle->NotifyFinishedChangingProperties();

		RefreshPropertiesPanel();
	}
}

const FNiagaraComponentPropertyBinding* FNiagaraComponentRendererPropertiesDetails::FindBinding(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	UNiagaraComponentRendererProperties* ComponentProperties = RendererProperties.Get();
	if (!ComponentProperties)
	{
		return nullptr;
	}
	FNiagaraComponentPropertyBinding SearchBinding = ToPropertyBinding(PropertyHandle, ComponentProperties);
	for (FNiagaraComponentPropertyBinding& Binding : ComponentProperties->PropertyBindings)
	{
		if (Binding.PropertyName == SearchBinding.PropertyName)
		{
			return &Binding;
		}
	}
	return nullptr;
}

FReply FNiagaraComponentRendererPropertiesDetails::ResetBindingButtonPressed(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	UNiagaraComponentRendererProperties* ComponentProperties = RendererProperties.Get();
	if (!ComponentProperties)
	{
		return FReply::Handled();
	}

	FNiagaraComponentPropertyBinding SearchBinding = ToPropertyBinding(PropertyHandle, ComponentProperties);
	FScopedTransaction Transaction(FText::Format(LOCTEXT("RemovePropertyBinding", "Remove component property binding from \"{0}\" "), FText::FromName(SearchBinding.PropertyName)));
	ComponentProperties->Modify();
	PropertyHandle->NotifyPreChange();
	for (int i = ComponentProperties->PropertyBindings.Num() - 1; i >= 0; i--)
	{
		const FNiagaraComponentPropertyBinding& Binding = ComponentProperties->PropertyBindings[i];
		if (Binding.PropertyName == SearchBinding.PropertyName)
		{
			ComponentProperties->PropertyBindings.RemoveAt(i);
		}
	}
	PropertyHandle->NotifyPostChange(EPropertyChangeType::Unspecified);
	PropertyHandle->NotifyFinishedChangingProperties();

	RefreshPropertiesPanel();
	return FReply::Handled();
}

void FNiagaraComponentRendererPropertiesDetails::RefreshPropertiesPanel()
{
	// Raw because we don't want to keep alive the details builder when calling the force refresh details
	if (IDetailLayoutBuilder* DetailLayoutBuilder = DetailBuilderWeakPtr.Pin().Get())
	{
		DetailLayoutBuilder->ForceRefreshDetails();
	}
}

FText FNiagaraComponentRendererPropertiesDetails::GetCurrentBindingText(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	if (UNiagaraComponentRendererProperties* ComponentProperties = RendererProperties.Get())
	{
		if (const FNiagaraComponentPropertyBinding* PropertyBinding = FindBinding(PropertyHandle))
		{
			return FText::FromName(PropertyBinding->AttributeBinding.GetName());
		}
	}
	return FText::FromString(TEXT("Missing"));
}

FVersionedNiagaraEmitter FNiagaraComponentRendererPropertiesDetails::GetCurrentEmitter() const
{
	if (UNiagaraComponentRendererProperties* ComponentProperties = RendererProperties.Get())
	{
		return ComponentProperties->GetOuterEmitter();
	}
	return FVersionedNiagaraEmitter();
}

TArray<FNiagaraVariable> FNiagaraComponentRendererPropertiesDetails::GetPossibleBindings(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	const FNiagaraTypeDefinition& PropertyType = UNiagaraComponentRendererProperties::ToNiagaraType(PropertyHandle->GetProperty());

	FVersionedNiagaraEmitterData* EmitterData = GetCurrentEmitter().GetEmitterData();
	if (!EmitterData || !PropertyType.IsValid())
	{
		return TArray<FNiagaraVariable>();
	}
	UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(EmitterData->GraphSource);
	if (!Source)
	{
		return TArray<FNiagaraVariable>();
	}
	
	TArray<UNiagaraNodeOutput*> OutputNodes;
	Source->NodeGraph->FindOutputNodes(OutputNodes);
	TArray<FNiagaraParameterMapHistory> ParameterMapHistories;

	for (UNiagaraNodeOutput* FoundOutputNode : OutputNodes)
	{
		FNiagaraParameterMapHistoryBuilder Builder;
		Builder.BuildParameterMaps(FoundOutputNode);
		ParameterMapHistories.Append(Builder.Histories);
	}
	
	TArray<FNiagaraVariable> Options;
	for (const FNiagaraParameterMapHistory& History : ParameterMapHistories)
	{
		for (const FNiagaraVariable& Var : History.Variables)
		{
			if (FNiagaraParameterUtilities::IsAttribute(Var) && UNiagaraComponentRendererProperties::IsConvertible(Var.GetType(), PropertyType))
			{
				Options.AddUnique(Var);
			}
		}
	}
	return Options;
}

bool FNiagaraComponentRendererPropertiesDetails::IsOverridableType(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	// TODO: we should check the available binding names here as well, but it's too expensive to traverse the param maps each tick
	// TODO: check which attributes are actually safe to set or maybe require a separate setter
	return UNiagaraComponentRendererProperties::ToNiagaraType(PropertyHandle->GetProperty()).IsValid(); // && GetPossibleBindingNames(PropertyHandle).Num() > 0;
}

#undef LOCTEXT_NAMESPACE