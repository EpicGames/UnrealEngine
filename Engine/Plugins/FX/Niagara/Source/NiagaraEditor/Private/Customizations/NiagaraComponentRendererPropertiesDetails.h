// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraComponentRendererProperties.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;

/** Details customization for the component renderer. */
class FNiagaraComponentRendererPropertiesDetails : public IDetailCustomization
{
public:
	virtual ~FNiagaraComponentRendererPropertiesDetails() override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	TWeakObjectPtr<UNiagaraComponentRendererProperties> RendererProperties;
	TWeakPtr<IDetailLayoutBuilder> DetailBuilderWeakPtr;

	TSharedRef<SWidget> GetAddBindingMenuContent(TSharedPtr<IPropertyHandle> PropertyHandle);
	const FNiagaraComponentPropertyBinding* FindBinding(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	FReply ResetBindingButtonPressed(TSharedPtr<IPropertyHandle> PropertyHandle);
	void RefreshPropertiesPanel();
	FText GetCurrentBindingText(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	FVersionedNiagaraEmitter GetCurrentEmitter() const;
	TArray<FNiagaraVariable> GetPossibleBindings(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	bool IsOverridableType(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	void ChangePropertyBinding(TSharedPtr<IPropertyHandle> PropertyHandle, const FNiagaraVariable& BindingVar);
};
