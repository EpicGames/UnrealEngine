// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelDetails.h"

#include "NiagaraDataChannel.h"

#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "ObjectEditorUtils.h"

#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"

#include "Modules/ModuleManager.h"
#include "Widgets/Input/SHyperlink.h"

#define LOCTEXT_NAMESPACE "NiagaraDataChannelDetails"

TSharedRef<IDetailCustomization> FNiagaraDataChannelAssetDetails::MakeInstance()
{
	return MakeShareable(new FNiagaraDataChannelAssetDetails);
}

FNiagaraDataChannelAssetDetails::~FNiagaraDataChannelAssetDetails()
{

}

void FNiagaraDataChannelAssetDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	if(CustomizedObjects.Num() != 1)
	{
		return;
	}
	
	DataChannelAsset = Cast<UNiagaraDataChannelAsset>(CustomizedObjects[0].Get());
	if(DataChannelAsset == nullptr)
	{
		return;
	}
	
	FName DataChannelCatName("Data Channel");
	IDetailCategoryBuilder& DataChannelCat = DetailBuilder.EditCategory(DataChannelCatName);
	
	TSharedPtr<IPropertyHandle> DataChannelProperty = DetailBuilder.GetProperty("DataChannel");
	DataChannelProperty->SetPropertyDisplayName(LOCTEXT("DataChannelTypeName", "Data Channel Type"));
	
	// we need to build a mapping between categories and properties first because the CLASS_CollapseCategories class flag can
	// remove the categories from top-level properties, so we need to traverse them differently
	TMap<FName, TArray<TSharedPtr<IPropertyHandle>>> CategoryMapping;

	//Always add the NDC type property.
	DataChannelCat.AddProperty(DataChannelProperty);
	DataChannelCat.AddCustomRow(FText::FromString("ChannelTypeHelp"))
	.Visibility(TAttribute<EVisibility>::CreateLambda([this]()
	{
		return DataChannelAsset.IsValid() && DataChannelAsset->Get() ? EVisibility::Collapsed : EVisibility::Visible;
	}))
	.WholeRowWidget
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SHyperlink)
			.Text(LOCTEXT("DataChannelTypePage", "Which data channel type should I choose?"))
			.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
			.ToolTipText(LOCTEXT("DataChannelTypePageTooltip", "Opens a docs page that explains data channel types"))
			.OnNavigate_Lambda([]()
			{
				FPlatformProcess::LaunchURL(TEXT("https://dev.epicgames.com/community/learning/tutorials/RJbm/unreal-engine-niagara-data-channels-intro"), nullptr, nullptr);
			})
		]
	];

	if(DataChannelAsset->Get())
	{		
		TSharedPtr<IPropertyHandle> InstancedNDCHandle = DataChannelProperty->GetChildHandle(0);
		check(InstancedNDCHandle);

		uint32 NumChildren = 0;
		InstancedNDCHandle->GetNumChildren(NumChildren);

		for (uint32 i = 0; i < NumChildren; i++)
		{
			TSharedPtr<IPropertyHandle> TopLevelHandle = InstancedNDCHandle->GetChildHandle(i);
			uint32 CategoryChildren;
			TopLevelHandle->GetNumChildren(CategoryChildren);
			for (uint32 k = 0; k < CategoryChildren; k++)
			{
				TSharedPtr<IPropertyHandle> ChildHandle = TopLevelHandle->GetChildHandle(k);
				if (!ChildHandle->GetProperty() || !ChildHandle->GetProperty()->GetClass())
				{
					continue;
				}

				FName CategoryFName = FName(TopLevelHandle->GetPropertyDisplayName().ToString());
				CategoryMapping.FindOrAdd(CategoryFName).Add(ChildHandle);
			}
		}
	}

	auto AddCategory = [&DetailBuilder](FName CategoryName, TArray<TSharedPtr<IPropertyHandle>>& Props)
	{
		IDetailCategoryBuilder& CatBuilder = DetailBuilder.EditCategory(CategoryName);
		for(TSharedPtr<IPropertyHandle>& Prop : Props)
		{
			CatBuilder.AddProperty(Prop);
		}
	};
	
	//Add the basic data channel props first.
	if(auto* DataChannelProps = CategoryMapping.Find(DataChannelCatName))
	{
		AddCategory(DataChannelCatName, *DataChannelProps);
	}
	for (TPair<FName, TArray<TSharedPtr<IPropertyHandle>>> CategoryPair : CategoryMapping)
	{
		FName CatName = CategoryPair.Key;
		TArray<TSharedPtr<IPropertyHandle>>& ChildProps = CategoryPair.Value;
		if (ChildProps.Num() == 0 || CatName == DataChannelCatName)
		{
			continue;
		}

		AddCategory(CatName, ChildProps);
	}
}

#undef LOCTEXT_NAMESPACE
