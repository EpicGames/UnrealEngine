// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "NiagaraDataChannelPublic.h"

class FNiagaraDataChannelAssetDetails : public IDetailCustomization
{
public:
	virtual ~FNiagaraDataChannelAssetDetails() override;

	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	TWeakObjectPtr<UNiagaraDataChannelAsset> DataChannelAsset;
};

