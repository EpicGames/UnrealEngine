// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceEmitterBindingCustomization.h"
#include "NiagaraDataInterface.h"
#include "NiagaraEditorModule.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNiagaraDebugger.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSuggestionTextBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceEmitterBindingCustomization"

void FNiagaraDataInterfaceEmitterBindingCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Get owner system (if any)
	if ( TSharedPtr<IPropertyUtilities> PropertyUtilities = CustomizationUtils.GetPropertyUtilities() )
	{
		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = PropertyUtilities->GetSelectedObjects();
		if (SelectedObjects.Num() == 1)
		{
			if ( UObject* OwnerObject = SelectedObjects[0].Get() )
			{
				OwnerObjectWeak = OwnerObject;
				NiagaraSystemWeak = OwnerObject->GetTypedOuter<UNiagaraSystem>();
			}
		}
	}

	// Get properties
	StructPropertyHandle = InStructPropertyHandle;
	BindingModeHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraDataInterfaceEmitterBinding, BindingMode));
	EmitterNameHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraDataInterfaceEmitterBinding, EmitterName));
	check(BindingModeHandle.IsValid() && EmitterNameHandle.IsValid())

	StructPropertyHandle->MarkResetToDefaultCustomized(true);

	// Build Widget
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				BindingModeHandle->CreatePropertyValueWidget()
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(EmitterTextBox, SSuggestionTextBox)
				.ForegroundColor(FSlateColor::UseForeground())
				.Visibility(this, &FNiagaraDataInterfaceEmitterBindingCustomization::IsEmitterNameVisible)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(EmitterName_GetText())
				.OnTextCommitted(this, &FNiagaraDataInterfaceEmitterBindingCustomization::EmitterName_SetText)
				.HintText(LOCTEXT("EmitterNameHint", "Enter Emitter Name..."))
				.OnShowingSuggestions(this, &FNiagaraDataInterfaceEmitterBindingCustomization::EmitterName_GetSuggestions)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2, 1)
			[
				SNew(SButton)
				.IsFocusable(false)
				.ToolTipText(LOCTEXT("ResetToDefaultToolTip", "Reset to Default"))
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ContentPadding(0)
				.Visibility(this, &FNiagaraDataInterfaceEmitterBindingCustomization::IsResetToDefaultsVisible)
				.OnClicked(this, &FNiagaraDataInterfaceEmitterBindingCustomization::OnResetToDefaultsClicked)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
}

EVisibility FNiagaraDataInterfaceEmitterBindingCustomization::IsEmitterNameVisible() const
{
	uint8 BindingModeValue;
	BindingModeHandle->GetValue(BindingModeValue);
	return ENiagaraDataInterfaceEmitterBindingMode(BindingModeValue) == ENiagaraDataInterfaceEmitterBindingMode::Other ? EVisibility::Visible : EVisibility::Hidden;
}

FText FNiagaraDataInterfaceEmitterBindingCustomization::EmitterName_GetText() const
{
	FName EmitterName;
	EmitterNameHandle->GetValue(EmitterName);
	return EmitterName.IsNone() ? FText() : FText::FromName(EmitterName);
}

void FNiagaraDataInterfaceEmitterBindingCustomization::EmitterName_SetText(const FText& NewText, ETextCommit::Type CommitInfo)
{
	FName EmitterName(NewText.ToString());
	EmitterNameHandle->SetValue(EmitterName);
}

void FNiagaraDataInterfaceEmitterBindingCustomization::EmitterName_GetSuggestions(const FString& CurrText, TArray<FString>& OutSuggestions)
{
	if ( UNiagaraSystem* NiagaraSystem = NiagaraSystemWeak.Get() )
	{
		for ( const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles() )
		{
			if ( EmitterHandle.GetIsEnabled() )
			{
				OutSuggestions.Add(EmitterHandle.GetUniqueInstanceName());
			}
		}
	}
}

FNiagaraDataInterfaceEmitterBinding* FNiagaraDataInterfaceEmitterBindingCustomization::GetEmitterBinding() const
{
	UObject* OwnerObject = OwnerObjectWeak.Get();
	return OwnerObject ? reinterpret_cast<FNiagaraDataInterfaceEmitterBinding*>(StructPropertyHandle->GetValueBaseAddress(reinterpret_cast<uint8*>(OwnerObject))) : nullptr;
}

EVisibility FNiagaraDataInterfaceEmitterBindingCustomization::IsResetToDefaultsVisible() const
{
	if (FNiagaraDataInterfaceEmitterBinding* EmitterBinding = GetEmitterBinding())
	{
		FNiagaraDataInterfaceEmitterBinding DefaultBinding;
		return *EmitterBinding == DefaultBinding ? EVisibility::Hidden : EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

FReply FNiagaraDataInterfaceEmitterBindingCustomization::OnResetToDefaultsClicked()
{
	if (FNiagaraDataInterfaceEmitterBinding* EmitterBinding = GetEmitterBinding())
	{
		FScopedTransaction Transaction(LOCTEXT("ResetEmitterBindingToDefault", "Reset Emitter Binding"));
		OwnerObjectWeak->Modify();
		StructPropertyHandle->NotifyPreChange();
		*EmitterBinding = FNiagaraDataInterfaceEmitterBinding();
		StructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		StructPropertyHandle->NotifyFinishedChangingProperties();
		EmitterTextBox->SetText(FText::FromName(EmitterBinding->EmitterName));
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
