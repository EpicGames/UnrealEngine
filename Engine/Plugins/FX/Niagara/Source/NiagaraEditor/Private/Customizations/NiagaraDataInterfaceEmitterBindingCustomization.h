// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "NiagaraDataInterfaceEmitterBinding.h"

class FDetailWidgetRow;
class IPropertyHandle;
class UNiagaraSystem;

class SSuggestionTextBox;

class FNiagaraDataInterfaceEmitterBindingCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraDataInterfaceEmitterBindingCustomization>();
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils);

	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils )
	{
	}

	EVisibility IsEmitterNameVisible() const;
	FText EmitterName_GetText() const;
	void EmitterName_SetText(const FText& NewText, ETextCommit::Type CommitInfo);
	void EmitterName_GetSuggestions(const FString& CurrText, TArray<FString>& OutSuggestions);

	FNiagaraDataInterfaceEmitterBinding* GetEmitterBinding() const;
	EVisibility IsResetToDefaultsVisible() const;
	FReply OnResetToDefaultsClicked();

	TWeakObjectPtr<UObject> OwnerObjectWeak;
	TWeakObjectPtr<UNiagaraSystem> NiagaraSystemWeak;
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> BindingModeHandle;
	TSharedPtr<IPropertyHandle> EmitterNameHandle;

	TSharedPtr<SSuggestionTextBox> EmitterTextBox;
};

