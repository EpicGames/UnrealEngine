// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDebugHUDCustomization.h"

#include "Modules/ModuleManager.h"

//Customization
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
//Widgets
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNiagaraDebugger.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SMenuAnchor.h"
///Niagara
#include "NiagaraEditorModule.h"
#include "NiagaraComponent.h"
#include "NiagaraEditorStyle.h"

#if WITH_NIAGARA_DEBUGGER

#define LOCTEXT_NAMESPACE "NiagaraDebugHUDCustomization"

//////////////////////////////////////////////////////////////////////////

namespace NiagaraDebugHUDSettingsDetailsCustomizationInternal
{

class SDebuggerSuggestionTextBox : public SComboButton
{
public:
	SLATE_BEGIN_ARGS(SDebuggerSuggestionTextBox) {}
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)
		SLATE_ARGUMENT(UClass*, ObjectClass)
		SLATE_ARGUMENT(TWeakObjectPtr<UNiagaraDebugHUDSettings>, WeakHUDSettings)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		Debugger = NiagaraEditorModule.GetDebugger();
		if ( Debugger )
		{
			Debugger->GetOnSimpleClientInfoChanged().AddSP(this, &SDebuggerSuggestionTextBox::OnSimpleClientInfoChanged);
		}

		PropertyHandle = InArgs._PropertyHandle;
		ObjectClass = InArgs._ObjectClass;
		WeakHUDSettings = InArgs._WeakHUDSettings;

		if (UNiagaraDebugHUDSettings* HUDSettings = WeakHUDSettings.Get())
		{
			HUDSettings->OnChangedDelegate.AddSP(this, &SDebuggerSuggestionTextBox::OnHUDSettingsChanged);
		}

		FText CurrentValue;
		PropertyHandle->GetValueAsFormattedText(CurrentValue);
		 Textbox = SNew(SEditableTextBox)
			.Padding(0)
			.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.Debugger.SuggestionDropdownInput")
			.MinDesiredWidth(20)
			.RevertTextOnEscape(true)
			.SelectAllTextWhenFocused(true)
			.Text(CurrentValue)
			.OnTextCommitted(this, &SDebuggerSuggestionTextBox::OnDebuggerTextCommitted);
		
		SComboButton::Construct(
			SComboButton::FArguments()
			.OnGetMenuContent(this, &SDebuggerSuggestionTextBox::GetDebuggerSuggestions)
			.ButtonContent()
			[
				Textbox.ToSharedRef()
			]
		);
	}

	void OnDebuggerTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
	{
		if (PropertyHandle)
		{
			PropertyHandle->SetValueFromFormattedString(NewText.ToString());
		}
	}

	void SelectDropdownValue(FString NewValue)
	{
		if (Textbox)
		{
			Textbox->SetText(FText::FromString(NewValue));
		}
	}

	TSharedRef<SWidget> GetDebuggerSuggestions()
	{
		FMenuBuilder MenuBuilder( true, NULL );
		
		if (!Debugger.IsValid() || !ObjectClass)
		{
			return MenuBuilder.MakeWidget();
		}

		const FNiagaraSimpleClientInfo& SimpleClientInfo = Debugger->GetSimpleClientInfo();
		TArray<FString> Options;
		if (ObjectClass == UNiagaraSystem::StaticClass())
		{
			Options = SimpleClientInfo.Systems;
		}
		else if (ObjectClass == UNiagaraEmitter::StaticClass())
		{
			Options = SimpleClientInfo.Emitters;
		}
		else if (ObjectClass == AActor::StaticClass())
		{
			Options = SimpleClientInfo.Actors;
		}
		else if (ObjectClass == UNiagaraComponent::StaticClass())
		{
			Options = SimpleClientInfo.Components;
		}
		Options.Sort();

		for (const FString& Option : Options)
		{
			FUIAction MenuAction(FExecuteAction::CreateSP(this, &SDebuggerSuggestionTextBox::SelectDropdownValue, Option));
			MenuBuilder.AddMenuEntry(FText::FromString(Option), FText(), FSlateIcon(), MenuAction);
		}
		return MenuBuilder.MakeWidget();
	}

	void OnHUDSettingsChanged()
	{
		FText CurrentValue;
		PropertyHandle->GetValueAsFormattedText(CurrentValue);
		Textbox->SetText(CurrentValue);
	}

	void OnSimpleClientInfoChanged(const FNiagaraSimpleClientInfo& ClientInfo)
	{
		bWaitingUpdate = false;
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		SComboButton::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		if (Debugger)
		{
			const bool bHasFocus = FSlateApplication::Get().HasFocusedDescendants(SharedThis(this));
			if (bHasFocus && !bWaitingUpdate)
			{
				bWaitingUpdate = true;
				Debugger->RequestUpdatedClientInfo();
			}
		}
	}

	TWeakObjectPtr<UNiagaraDebugHUDSettings>	WeakHUDSettings;
	TSharedPtr<FNiagaraDebugger>	Debugger;
	TSharedPtr<IPropertyHandle>		PropertyHandle;
	TSharedPtr<SEditableTextBox>	Textbox;
	UClass*							ObjectClass = nullptr;
	bool							bWaitingUpdate = false;
};

} // namespace

//////////////////////////////////////////////////////////////////////////

void FNiagaraDebugHUDVariableCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	EnabledPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraDebugHUDVariable, bEnabled));
	NamePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraDebugHUDVariable, Name));
	check(EnabledPropertyHandle.IsValid() && NamePropertyHandle.IsValid())

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FNiagaraDebugHUDVariableCustomization::IsEnabled)
				.OnCheckStateChanged(this, &FNiagaraDebugHUDVariableCustomization::SetEnabled)
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SEditableTextBox)
				.IsEnabled(this, &FNiagaraDebugHUDVariableCustomization::IsTextEditable)
				.Text(this, &FNiagaraDebugHUDVariableCustomization::GetText)
				.OnTextCommitted(this, &FNiagaraDebugHUDVariableCustomization::SetText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}

ECheckBoxState FNiagaraDebugHUDVariableCustomization::IsEnabled() const
{
	bool bEnabled = false;
	EnabledPropertyHandle->GetValue(bEnabled);
	return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FNiagaraDebugHUDVariableCustomization::SetEnabled(ECheckBoxState NewState)
{
	bool bEnabled = NewState == ECheckBoxState::Checked;
	EnabledPropertyHandle->SetValue(bEnabled);
}

FText FNiagaraDebugHUDVariableCustomization::GetText() const
{
	FString Text;
	NamePropertyHandle->GetValue(Text);
	return FText::FromString(Text);
}

void FNiagaraDebugHUDVariableCustomization::SetText(const FText& NewText, ETextCommit::Type CommitInfo)
{
	NamePropertyHandle->SetValue(NewText.ToString());
}

bool FNiagaraDebugHUDVariableCustomization::IsTextEditable() const
{
	bool bEnabled = false;
	EnabledPropertyHandle->GetValue(bEnabled);
	return bEnabled;
}

FNiagaraDebugHUDSettingsDetailsCustomization::FNiagaraDebugHUDSettingsDetailsCustomization(UNiagaraDebugHUDSettings* InSettings)
	: WeakSettings(InSettings)
{
}

void FNiagaraDebugHUDSettingsDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	{
		IDetailCategoryBuilder& OverviewCategory = DetailBuilder.EditCategory("Debug Overview");

		TArray<TSharedRef<IPropertyHandle>> PropertyHandles;
		OverviewCategory.GetDefaultProperties(PropertyHandles);
		for (const TSharedRef<IPropertyHandle>& PropertyHandle : PropertyHandles)
		{
			if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraDebugHUDSettingsData, bShowRegisteredComponents))
			{
				OverviewCategory.AddProperty(PropertyHandle).IsEnabled(
					TAttribute<bool>::CreateLambda([&]() -> bool { const UNiagaraDebugHUDSettings* Settings = WeakSettings.Get(); return Settings->Data.bOverviewEnabled && Settings->Data.OverviewMode == ENiagaraDebugHUDOverviewMode::Overview; })
				);
			}
			else
			{
				OverviewCategory.AddProperty(PropertyHandle).IsEnabled(
					TAttribute<bool>::CreateLambda([&]() -> bool { return WeakSettings.Get()->Data.bOverviewEnabled; })
				);
			}
		}
	}

	// Customize Filters
	{
		IDetailCategoryBuilder& FilterCategory = DetailBuilder.EditCategory("Debug Filter");
		
		FilterCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FNiagaraDebugHUDSettingsData, bFiltersEnabled), FNiagaraDebugHUDSettingsData::StaticStruct()));
		MakeCustomAssetSearch(DetailBuilder, FilterCategory, DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FNiagaraDebugHUDSettingsData, SystemFilter), FNiagaraDebugHUDSettingsData::StaticStruct()), UNiagaraSystem::StaticClass(), [&]() -> bool& { return WeakSettings.Get()->Data.bSystemFilterEnabled; }, [&]() -> bool { return WeakSettings.Get()->Data.bFiltersEnabled; });
		MakeCustomAssetSearch(DetailBuilder, FilterCategory, DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FNiagaraDebugHUDSettingsData, EmitterFilter), FNiagaraDebugHUDSettingsData::StaticStruct()), UNiagaraEmitter::StaticClass(), [&]() -> bool& { return WeakSettings.Get()->Data.bEmitterFilterEnabled; }, [&]() -> bool { return WeakSettings.Get()->Data.bFiltersEnabled; });
		MakeCustomAssetSearch(DetailBuilder, FilterCategory, DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FNiagaraDebugHUDSettingsData, ActorFilter), FNiagaraDebugHUDSettingsData::StaticStruct()), AActor::StaticClass(), [&]() -> bool& { return WeakSettings.Get()->Data.bActorFilterEnabled; }, [&]() -> bool { return WeakSettings.Get()->Data.bFiltersEnabled; });
		MakeCustomAssetSearch(DetailBuilder, FilterCategory, DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FNiagaraDebugHUDSettingsData, ComponentFilter), FNiagaraDebugHUDSettingsData::StaticStruct()), UNiagaraComponent::StaticClass(), [&]() -> bool& { return WeakSettings.Get()->Data.bComponentFilterEnabled; }, [&]() -> bool { return WeakSettings.Get()->Data.bFiltersEnabled; });
	}
}

void FNiagaraDebugHUDSettingsDetailsCustomization::MakeCustomAssetSearch(IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& DetailCategory, TSharedRef<IPropertyHandle> PropertyHandle, UClass* ObjRefClass, TFunction<bool&()> GetEditBool, TFunction<bool ()> GetVisibleBool, bool bShowInlineCheckbox)
{
	if ( !PropertyHandle->IsValidHandle() || (ObjRefClass == nullptr) )
	{
		return;
	}
	
	DetailBuilder.HideProperty(PropertyHandle);
	
	DetailCategory.AddCustomRow(PropertyHandle->GetPropertyDisplayName())
	.Visibility(TAttribute<EVisibility>::CreateLambda([GetVisibleBool]() -> EVisibility { return GetVisibleBool() ? EVisibility::Visible : EVisibility::Collapsed; }))
	.NameContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Visibility(bShowInlineCheckbox ? EVisibility::Visible : EVisibility::Collapsed)
			.IsChecked_Lambda([GetEditBool]() -> ECheckBoxState { return GetEditBool() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
			.OnCheckStateChanged_Lambda([this, GetEditBool](ECheckBoxState NewState) { GetEditBool() = NewState == ECheckBoxState::Checked; WeakSettings.Get()->NotifyPropertyChanged(); })
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			.IsEnabled_Lambda([=]() { return GetEditBool(); })
			+ SHorizontalBox::Slot()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
		]
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled_Lambda([=]() { return GetEditBool(); })
		+ SHorizontalBox::Slot()
		[
			SNew(NiagaraDebugHUDSettingsDetailsCustomizationInternal::SDebuggerSuggestionTextBox)
			.PropertyHandle(PropertyHandle)
			.ObjectClass(ObjRefClass)
			.WeakHUDSettings(WeakSettings)
		]
	];
}

#undef LOCTEXT_NAMESPACE

#endif
