// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "NiagaraCommon.h"
#include "EdGraph/EdGraphSchema.h"
#include "Layout/Visibility.h"
#include "NiagaraDebuggerCommon.h"

class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class FDetailWidgetRow;
class IPropertyHandle;
class IPropertyHandleArray;
enum class ECheckBoxState : uint8;
class FNiagaraDebugger;
class UNiagaraDebugHUDSettings;

#if WITH_NIAGARA_DEBUGGER

class FNiagaraDebugHUDVariableCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraDebugHUDVariableCustomization>();
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils);

	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils )
	{
	}

	ECheckBoxState IsEnabled() const;
	void SetEnabled(ECheckBoxState NewState);

	FText GetText() const;
	void SetText(const FText& NewText, ETextCommit::Type CommitInfo);
	bool IsTextEditable() const;

	TSharedPtr<IPropertyHandle> EnabledPropertyHandle;
	TSharedPtr<IPropertyHandle> NamePropertyHandle;
};

//////////////////////////////////////////////////////////////////////////

class FNiagaraDebugHUDSettingsDetailsCustomization : public IDetailCustomization
{
public:
	FNiagaraDebugHUDSettingsDetailsCustomization(UNiagaraDebugHUDSettings* InSettings);

	static TSharedRef<IDetailCustomization> MakeInstance(UNiagaraDebugHUDSettings* Settings)
	{
		return MakeShared<FNiagaraDebugHUDSettingsDetailsCustomization>(Settings);
	}

	// Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

private:
	void MakeCustomAssetSearch(IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& DetailCategory, TSharedRef<IPropertyHandle> PropertyHandle, UClass* ObjRefClass, TFunction<bool& ()> GetEditBool, TFunction<bool ()> GetVisibleBool, bool bShowInlineCheckbox = true);

private:
	TWeakObjectPtr<UNiagaraDebugHUDSettings> WeakSettings = nullptr;
};

//////////////////////////////////////////////////////////////////////////
#endif
