// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorPreviewActorCustomization.h"
#include "NiagaraEditorPreviewActor.h"

#include "DetailLayoutBuilder.h"

TSharedRef<IDetailCustomization> FNiagaraEditorPreviewActorCustomization::MakeInstance()
{
	return MakeShared<FNiagaraEditorPreviewActorCustomization>();
}

FName FNiagaraEditorPreviewActorCustomization::GetCustomizationTypeName()
{
	return ANiagaraEditorPreviewActor::StaticClass()->GetFName();
}

void FNiagaraEditorPreviewActorCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const TArray<FName> CategoryOrder =
	{
		FName("ActorMotion"),
		FName("Niagara"),
		FName("NiagaraComponent_Parameters"),
		FName("Niagara Utilities"),
	};

	TArray<FName> CategoryNames;
	DetailLayout.GetCategoryNames(CategoryNames);

	for (FName CategoryName : CategoryNames)
	{
		if (!CategoryOrder.Contains(CategoryName))
		{
			DetailLayout.HideCategory(CategoryName);
		}
	}
}
