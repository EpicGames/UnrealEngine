// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;

class FNiagaraEditorPreviewActorCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	static FName GetCustomizationTypeName();

	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout) override;
};

