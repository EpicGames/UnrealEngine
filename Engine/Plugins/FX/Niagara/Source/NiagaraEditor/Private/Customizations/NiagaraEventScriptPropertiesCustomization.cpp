// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/NiagaraEventScriptPropertiesCustomization.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/STextComboBox.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailLayoutBuilder.h"
#include "NiagaraEmitter.h"
#include "Widgets/Layout/SBorder.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitterHandle.h"
#include "ScopedTransaction.h"
#include "SGraphActionMenu.h"
#include "NiagaraEditorUtilities.h"
#include "Editor.h"
#include "IDetailChildrenBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEventScriptPropertiesCustomization)

#define LOCTEXT_NAMESPACE "FNiagaraEventScriptPropertiesCustomization"

TSharedRef<IPropertyTypeCustomization> FNiagaraEventScriptPropertiesCustomization::MakeInstance(TWeakObjectPtr<UNiagaraSystem> InSystem, FVersionedNiagaraEmitterWeakPtr InEmitter)
{
	return MakeShareable(new FNiagaraEventScriptPropertiesCustomization(InSystem, InEmitter));
}

FNiagaraEventScriptPropertiesCustomization::FNiagaraEventScriptPropertiesCustomization(TWeakObjectPtr<UNiagaraSystem> InSystem,
	FVersionedNiagaraEmitterWeakPtr InEmitter) : System(InSystem), Emitter(InEmitter)
{
	GEditor->RegisterForUndo(this);
}

FNiagaraEventScriptPropertiesCustomization::~FNiagaraEventScriptPropertiesCustomization()
{
	GEditor->UnregisterForUndo(this);
}

void FNiagaraEventScriptPropertiesCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	HeaderRow.ValueContent()
		[
			StructPropertyHandle->CreatePropertyValueWidget()
		];
}

void FNiagaraEventScriptPropertiesCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HandleSrcID = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraEventScriptProperties, SourceEmitterID));
	HandleEventName = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraEventScriptProperties, SourceEventName));
	HandleSpawnNumber = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraEventScriptProperties, SpawnNumber));
	HandleExecutionMode = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraEventScriptProperties, ExecutionMode));
	HandleMaxEvents = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraEventScriptProperties, MaxEventsPerFrame));
	HandleUseRandomSpawnNumber = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraEventScriptProperties, bRandomSpawnNumber));
	HandleMinSpawnNumber = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraEventScriptProperties, MinSpawnNumber));
	HandleUpdateInitialValues = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraEventScriptProperties, UpdateAttributeInitialValues));

	ResolveEmitterName();

	// Add Source property
	{
		FText EventSrcTxt = LOCTEXT("EventSource", "Source");
		FDetailWidgetRow& Row = ChildBuilder.AddCustomRow(EventSrcTxt);
		FText TooltipText = LOCTEXT("ChooseProvider", "Choose the source emitter and event.");

		TAttribute<FText> ErrorTextAttribute(this, &FNiagaraEventScriptPropertiesCustomization::GetErrorText);
		TAttribute<FText> ErrorTextTooltipAttribute(this, &FNiagaraEventScriptPropertiesCustomization::GetErrorTextTooltip);
		TSharedRef<SWidget> ErrorWidget = FNiagaraEditorUtilities::CreateInlineErrorText(ErrorTextAttribute, ErrorTextTooltipAttribute).ToSharedRef();
		TAttribute<EVisibility> VisAttribute(this, &FNiagaraEventScriptPropertiesCustomization::GetErrorVisibility);
		ComputeErrorVisibility();
		ErrorWidget->SetVisibility(VisAttribute);

		Row.NameWidget
			[
				SNew(STextBlock)
				.Text(EventSrcTxt)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ToolTipText(TooltipText)
			];
		Row.ValueWidget
			.MaxDesiredWidth(0.0f)
			.MinDesiredWidth(250.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					ErrorWidget
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &FNiagaraEventScriptPropertiesCustomization::OnGetMenuContent)
					.ContentPadding(1)
					.ToolTipText(TooltipText)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &FNiagaraEventScriptPropertiesCustomization::OnGetButtonText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.ToolTipText(TooltipText)
					]
				]
			];
	}

	// Add Execution Mode property
	{
		FText EventSrcTxt = LOCTEXT("ExecutionMode", "ExecutionMode");
		FDetailWidgetRow& ExecRow = ChildBuilder.AddCustomRow(EventSrcTxt);
		ExecRow.NameWidget
			[
				HandleExecutionMode->CreatePropertyNameWidget()
			];
		ExecRow.ValueWidget
			[
				HandleExecutionMode->CreatePropertyValueWidget()
			];
	}

	// Add Max Events property
	{
		IDetailPropertyRow& MaxRow = ChildBuilder.AddProperty(HandleMaxEvents.ToSharedRef());
	}
	
	// Add Spawn Number property
	{
		FText EventSrcTxt = LOCTEXT("SpawnNumber", "SpawnNumber");
		FDetailWidgetRow& SpawnRow = ChildBuilder.AddCustomRow(EventSrcTxt);
		TSharedRef<SWidget> SpawnNumberValueWidget = HandleSpawnNumber->CreatePropertyValueWidget();
		TAttribute<bool> EnabledAttribute(this, &FNiagaraEventScriptPropertiesCustomization::GetSpawnNumberEnabled);
		SpawnRow.IsEnabled(EnabledAttribute);

		SpawnRow.NameWidget
			[
				HandleSpawnNumber->CreatePropertyNameWidget()
			];
		SpawnRow.ValueWidget
			[
				SpawnRow.ValueWidget.Widget = SpawnNumberValueWidget
			];
	}

	// Add Spawn Number property
	{
		FText EventSrcTxt = LOCTEXT("MinimumSpawnNumber", "Minimum SpawnNumber");
		FDetailWidgetRow& SpawnRow = ChildBuilder.AddCustomRow(EventSrcTxt);
		TSharedRef<SWidget> MinSpawnNumberValueWidget = HandleMinSpawnNumber->CreatePropertyValueWidget();
		TAttribute<bool> EnabledAttribute(this, &FNiagaraEventScriptPropertiesCustomization::GetUseRandomSpawnNumber);
		SpawnRow.IsEnabled(EnabledAttribute);
		TAttribute<EVisibility> VisibilityAttribute(this, &FNiagaraEventScriptPropertiesCustomization::GetMinSpawnNumberVisible);
		SpawnRow.Visibility(VisibilityAttribute);

		SpawnRow.NameWidget
			[
				HandleMinSpawnNumber->CreatePropertyNameWidget()
			];
		SpawnRow.ValueWidget
			[
				SpawnRow.ValueWidget.Widget = MinSpawnNumberValueWidget
			];
	}

	// Add Random Spawn Number boolean property
	{
		FText SrcTxt = LOCTEXT("RandomSpawnNumber", "Random SpawnNumber");
		FDetailWidgetRow& SpawnRow = ChildBuilder.AddCustomRow(SrcTxt);
		TSharedRef<SWidget> SpawnValueWidget = HandleUseRandomSpawnNumber->CreatePropertyValueWidget();
		TAttribute<bool> EnabledAttribute(this, &FNiagaraEventScriptPropertiesCustomization::GetSpawnNumberEnabled);
		SpawnRow.IsEnabled(EnabledAttribute);
		
		SpawnRow.NameWidget
			[
				HandleUseRandomSpawnNumber->CreatePropertyNameWidget()
			];
		SpawnRow.ValueWidget
			[
				SpawnRow.ValueWidget.Widget = SpawnValueWidget
			];
	}

	// Add bool saying if event spawn scripts should update the initial value of particle attributes.
	{
		FText SrcTxt = LOCTEXT("UpdateAttributeInitialValues", "Update Initial Values");
		FDetailWidgetRow& InitialValuseRow = ChildBuilder.AddCustomRow(SrcTxt);
		TSharedRef<SWidget> InitialValuseWidget = HandleUpdateInitialValues->CreatePropertyValueWidget();
		TAttribute<bool> EnabledAttribute(this, &FNiagaraEventScriptPropertiesCustomization::GetUpdateInitialValuesEnabled);
		InitialValuseRow.IsEnabled(EnabledAttribute);

		FSimpleDelegate OnUpdateInitialValuesChangedDelegate = FSimpleDelegate::CreateSP(this, &FNiagaraEventScriptPropertiesCustomization::OnUpdateInitialValuesChanged);
		HandleUpdateInitialValues->SetOnPropertyValueChanged(OnUpdateInitialValuesChangedDelegate);

		InitialValuseRow.NameWidget
			[
				HandleUpdateInitialValues->CreatePropertyNameWidget()
			];
		InitialValuseRow.ValueWidget
			[
				InitialValuseRow.ValueWidget.Widget = InitialValuseWidget
			];
	}
}

void FNiagaraEventScriptPropertiesCustomization::PostUndo(bool bSuccess)
{
	ResolveEmitterName();
	ComputeErrorVisibility();
}

EVisibility FNiagaraEventScriptPropertiesCustomization::GetErrorVisibility() const
{
	return CachedVisibility;
}

void FNiagaraEventScriptPropertiesCustomization::ComputeErrorVisibility() 
{
	FString EventSrcIdStr;
	HandleSrcID->GetValueAsFormattedString(EventSrcIdStr);
	FString EventValueStr;
	HandleEventName->GetValue(EventValueStr);

	CachedVisibility = EVisibility::Collapsed;

	FGuid Id;
	if (FGuid::Parse(EventSrcIdStr, Id))
	{
		if (Id == FGuid() || EventValueStr.IsEmpty())
		{
			CachedVisibility = EVisibility::Collapsed;
			return;
		}

		for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
		{
			FGuid HandleGuid = Handle.GetId();
			bool bIsMatch = false;
			if (HandleGuid == Id)
			{
				bIsMatch = true;
			}

			if (Emitter.ResolveWeakPtr() == Handle.GetInstance())
			{
				HandleGuid.Invalidate();
			}

			if (HandleGuid.IsValid() == false && HandleGuid == Id)
			{
				bIsMatch = true;
			}

			if (bIsMatch)
			{
				TArray<FName> EventNames = GetEventNames(Handle.GetInstance());
				if (EventNames.Contains(*EventValueStr))
				{
					return;
				}
			}
		}
	}

	CachedVisibility = EVisibility::Visible;

}

FText FNiagaraEventScriptPropertiesCustomization::GetErrorText() const
{
	return FText();
}

FText FNiagaraEventScriptPropertiesCustomization::GetErrorTextTooltip() const
{
	FString EventValueStr;
	HandleEventName->GetValue(EventValueStr);
	return FText::Format(LOCTEXT("ErrorTextTooltip", "Either Emitter \"{0}\" does not exist or it doesn't generate the named event \"{1}\"!"), FText::FromName(CachedEmitterName), FText::FromString(EventValueStr));
}

bool FNiagaraEventScriptPropertiesCustomization::GetSpawnNumberEnabled() const
{
	uint8 Value = 255;
	if (HandleExecutionMode->GetValue(Value) && Value == (uint8)EScriptExecutionMode::SpawnedParticles)
	{
		return true;
	}
	return false;
}

bool FNiagaraEventScriptPropertiesCustomization::GetUseRandomSpawnNumber() const
{
	bool bRandom = false;
	if (GetSpawnNumberEnabled() && HandleUseRandomSpawnNumber->GetValue(bRandom))
	{
		return bRandom;
	}

	return false;
}

EVisibility FNiagaraEventScriptPropertiesCustomization::GetMinSpawnNumberVisible() const
{
	return GetUseRandomSpawnNumber() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FNiagaraEventScriptPropertiesCustomization::GetUpdateInitialValuesEnabled() const
{
	uint8 Value = 255;
	if (HandleExecutionMode->GetValue(Value) && Value == (uint8)EScriptExecutionMode::SpawnedParticles)
	{
		return true;
	}
	return false;
}

void FNiagaraEventScriptPropertiesCustomization::OnUpdateInitialValuesChanged()
{
	UNiagaraSystem::RequestCompileForEmitter(Emitter.ResolveWeakPtr());
}

void FNiagaraEventScriptPropertiesCustomization::ResolveEmitterName()
{
	FString EventSrcIdStr;
	HandleSrcID->GetValueAsFormattedString(EventSrcIdStr);
	FGuid Id;
	CachedEmitterName = NAME_None;
	if (FGuid::Parse(EventSrcIdStr, Id))
	{
		for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
		{
			FGuid HandleGuid = Handle.GetId();
			FName HandleName = Handle.GetName();

			if (HandleGuid.IsValid() && HandleGuid == Id)
			{
				CachedEmitterName = HandleName;
				return;
			}

			// Set our handle to zeroes if we are looking at this emitter.
			if (Emitter.ResolveWeakPtr() == Handle.GetInstance())
			{
				HandleGuid.Invalidate();
			}

			if (HandleGuid.IsValid() == false && HandleGuid == Id)
			{
				CachedEmitterName = HandleName;
				return;
			}
		}
	}
}

TSharedRef<SWidget> FNiagaraEventScriptPropertiesCustomization::OnGetMenuContent() const
{
	FGraphActionMenuBuilder MenuBuilder;

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			[
				SNew(SGraphActionMenu)
				.OnActionSelected(const_cast<FNiagaraEventScriptPropertiesCustomization*>(this), &FNiagaraEventScriptPropertiesCustomization::OnActionSelected)
				.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(const_cast<FNiagaraEventScriptPropertiesCustomization*>(this), &FNiagaraEventScriptPropertiesCustomization::OnCreateWidgetForAction))
				.OnCollectAllActions(const_cast<FNiagaraEventScriptPropertiesCustomization*>(this), &FNiagaraEventScriptPropertiesCustomization::CollectAllActions)
				.AutoExpandActionMenu(false)
				.ShowFilterTextBox(true)
			]
		];
}

TArray<FName> FNiagaraEventScriptPropertiesCustomization::GetEventNames(const FVersionedNiagaraEmitter& InEmitter) const
{
	TArray<FName> EventNames;
	TArray<UNiagaraScript*> Scripts;
	InEmitter.GetEmitterData()->GetScripts(Scripts);

	for (UNiagaraScript* Script : Scripts)
	{
		if (!Script->IsReadyToRun(ENiagaraSimTarget::CPUSim))
		{
			continue;
		}

		for (FNiagaraDataSetProperties& Props : Script->GetVMExecutableData().WriteDataSets)
		{
			if (Props.ID.Name.IsValid())
			{
				EventNames.AddUnique(Props.ID.Name);
			}
		}
	}
	return EventNames;
}

void FNiagaraEventScriptPropertiesCustomization::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{	
	for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
	{
		FGuid HandleGuid = Handle.GetId();
		FName EmitterName = Handle.GetName();

		// Set our handle to zeroes if we are looking at this emitter.
		if (Emitter.ResolveWeakPtr() == Handle.GetInstance())
		{
			HandleGuid.Invalidate();
		}

		TArray<FName> EventNames = GetEventNames(Handle.GetInstance());

		for (FName EventName : EventNames)
		{
			FText CategoryName = FText::FromString(FName::NameToDisplayString(EmitterName.ToString(), false));
			FString DisplayNameString = FName::NameToDisplayString(EventName.ToString(), false);
			const FText NameText = FText::FromString(DisplayNameString);
			const FText TooltipDesc = FText::Format(LOCTEXT("SetFunctionPopupTooltip", "Handle the event named \"{0}\" from Emitter \"{1}\""), FText::FromString(DisplayNameString), CategoryName);
			TSharedPtr<FNiagaraStackAssetAction_EventSource> NewNodeAction(new FNiagaraStackAssetAction_EventSource(EmitterName, EventName, NAME_None, HandleGuid, CategoryName, NameText,
				TooltipDesc, 0, FText()));
			OutAllActions.AddAction(NewNodeAction);
		}
	}

	const FText DoNothingText = LOCTEXT("RevertFunctionPopupTooltip", "Do not handle incoming events");
	TSharedPtr<FNiagaraStackAssetAction_EventSource> DoNothingAction(new FNiagaraStackAssetAction_EventSource(NAME_None, NAME_None, NAME_None, FGuid(), FText(), DoNothingText,
		DoNothingText, 0, FText()));
	OutAllActions.AddAction(DoNothingAction);
}

TSharedRef<SWidget> FNiagaraEventScriptPropertiesCustomization::OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(InCreateData->Action->GetMenuDescription())
			.ToolTipText(InCreateData->Action->GetTooltipDescription())
		];
}


void FNiagaraEventScriptPropertiesCustomization::OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType)
{
	if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		for (int32 ActionIndex = 0; ActionIndex < SelectedActions.Num(); ActionIndex++)
		{
			TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedActions[ActionIndex];

			if (CurrentAction.IsValid())
			{
				FSlateApplication::Get().DismissAllMenus();
				FNiagaraStackAssetAction_EventSource* EventSourceAction = (FNiagaraStackAssetAction_EventSource*)CurrentAction.Get();
				ChangeEventSource(EventSourceAction->EmitterGuid, EventSourceAction->EmitterName, EventSourceAction->EventName);
			}
		}
	}
}

FText FNiagaraEventScriptPropertiesCustomization::OnGetButtonText() const
{
	FString EventValueStr;
	HandleEventName->GetValue(EventValueStr);
	return GetProviderText(CachedEmitterName, *EventValueStr);
}

void FNiagaraEventScriptPropertiesCustomization::ChangeEventSource(FGuid InEmitterGuid, FName InEmitterName, FName InEventName) 
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeEventSource", " Change Event Source to \"{0}\" Event: \"{1}\""), FText::FromName(InEmitterName), FText::FromName(InEventName)));
	HandleSrcID->SetValueFromFormattedString(InEmitterGuid.ToString());
	HandleEventName->SetValueFromFormattedString(InEventName.ToString());
	CachedEmitterName = InEmitterName;
	ComputeErrorVisibility();
}

FText FNiagaraEventScriptPropertiesCustomization::GetProviderText(const FName& InEmitterName, const FName& InEventName) const
{
	if (InEmitterName == NAME_None || InEventName == NAME_None)
	{
		return LOCTEXT("NotRespondingToEvents", "Event source unassigned");
	}
	else
	{
		return FText::Format(LOCTEXT("Provider_Text", "Emitter: \"{0}\" Event: \"{1}\""), FText::FromName(InEmitterName), FText::FromName(InEventName));
	}
}

#undef LOCTEXT_NAMESPACE

