// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraFunctionCallNodeDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraNodeFunctionCall.h"
#include "ScopedTransaction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraAnalytics.h"
#include "NiagaraNodeCustomHlsl.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "NiagaraFunctionCallNodeDetails"

TSharedRef<IDetailCustomization> FNiagaraFunctionCallNodeDetails::MakeInstance()
{
	return MakeShareable(new FNiagaraFunctionCallNodeDetails);
}

void FNiagaraFunctionCallNodeDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	static const FName SwitchCategoryName = TEXT("Propagated Static Switch Values");
	static const FName VersionCategoryName = TEXT("Version Details");
	static const FName FunctionCategoryName = TEXT("Function");
	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
    

	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	if (ObjectsCustomized.Num() != 1 || !ObjectsCustomized[0]->IsA<UNiagaraNodeFunctionCall>())
	{
		return;
	}
	
	Node = CastChecked<UNiagaraNodeFunctionCall>(ObjectsCustomized[0].Get());
	FPinCollectorArray InputPins;
	Node->GetInputPins(InputPins);
	bool IsDIFunction = Node->FunctionScript == nullptr && InputPins.Num() > 0 && UNiagaraDataInterface::IsDataInterfaceType(NiagaraSchema->PinToTypeDefinition(InputPins[0]));
	bool bIsCustomHlslNode = Node->IsA(UNiagaraNodeCustomHlsl::StaticClass());
	if (IsDIFunction || bIsCustomHlslNode)
	{
		DetailBuilder.EditCategory(FunctionCategoryName).SetCategoryVisibility(false);
		DetailBuilder.EditCategory(VersionCategoryName).SetCategoryVisibility(false);
		DetailBuilder.EditCategory(SwitchCategoryName).SetCategoryVisibility(false);
	}

	UNiagaraGraph* CalledGraph = Node->GetCalledGraph();
	if (!CalledGraph)
	{
		return;
	}

	if (Node->FunctionScript->IsVersioningEnabled())
	{
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(VersionCategoryName);
		FDetailWidgetRow& Row = CategoryBuilder.AddCustomRow(LOCTEXT("NiagaraFunctionVersionCategoryFilterText", "Version Details"));
		Row.NameContent()
		[
			SNew(SBox)
	        .Padding(FMargin(0.0f, 2.0f))
	        [
	            SNew(STextBlock)
	            .TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
	            .Text(LOCTEXT("NiagaraFunctionVersionPropertyName", "Script Version"))
	        ]
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FNiagaraFunctionCallNodeDetails::OnGetVersionMenuContent)
			.ContentPadding(2)
			.ButtonContent()
			[
				SNew(STextBlock)
	            .Font(IDetailLayoutBuilder::GetDetailFont())
	            .Text_Lambda([this]()
	            {
		            FVersionedNiagaraScriptData* ScriptData = Node->GetScriptData();
		            return FText::Format(FText::FromString("{0}.{1}"), ScriptData->Version.MajorVersion, ScriptData->Version.MinorVersion);
	            })
			]
		];
	}

	// For each static switch value inside the function graph we add a checkbox row to allow the user to propagate it up in the call hierachy.
	// Doing so then disables the option to set the value directly on the function call node.
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(SwitchCategoryName);
	for (const FNiagaraVariable& SwitchInput : CalledGraph->FindStaticSwitchInputs())
	{
		FDetailWidgetRow& Row = CategoryBuilder.AddCustomRow(LOCTEXT("NiagaraFunctionPropagatedValuesFilterText", "Propagated Static Switch Values"));

		Row
		.NameContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(FText::FromName(SwitchInput.GetName()))
			]
		]
		.ValueContent()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([SwitchInput, this]()
				{
					return Node->FindPropagatedVariable(SwitchInput) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([SwitchInput, this](const ECheckBoxState NewState)
				{
					if (NewState == ECheckBoxState::Unchecked)
					{
						Node->RemovePropagatedVariable(SwitchInput);
					}
					else if (NewState == ECheckBoxState::Checked)
					{
						if (!Node->FindPropagatedVariable(SwitchInput))
						{
							Node->PropagatedStaticSwitchParameters.Emplace(SwitchInput);
						}
						CopyMetadataFromCalledGraph(SwitchInput);
					}
					Node->RefreshFromExternalChanges();
					Node->GetGraph()->NotifyGraphChanged();
				})
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				SNew(SEditableTextBox)
				.ToolTipText(LOCTEXT("NiagaraOverridePropagatedValueName_TooltipText", "When set, the name of the propagated switch value is changed to a different value."))
				.HintText(LOCTEXT("NiagaraOverridePropagatedValueName_HintText", "Override name"))
				.IsEnabled_Lambda([SwitchInput, this]() { return Node->FindPropagatedVariable(SwitchInput) != nullptr; })
				.OnTextCommitted_Lambda([SwitchInput, this](const FText& NewText, ETextCommit::Type CommitType)
				{
					FNiagaraPropagatedVariable* Propagated = Node->FindPropagatedVariable(SwitchInput);
					if (Propagated)
					{
						FNiagaraVariable OldVar = Propagated->ToVariable();
						Propagated->PropagatedName = NewText.ToString();
						CopyMetadataForNameOverride(OldVar, Propagated->ToVariable());
					}
				})
				.Text_Lambda([SwitchInput, this]()
				{
					FNiagaraPropagatedVariable* Propagated = Node->FindPropagatedVariable(SwitchInput);
					if (Propagated)
					{
						return FText::FromString(Propagated->PropagatedName);
					}
					return FText();
				})
			]
		];
	}
}

void FNiagaraFunctionCallNodeDetails::CopyMetadataFromCalledGraph(FNiagaraVariable FromVariable)
{
	UNiagaraGraph* NodeGraph = GetNodeGraph();
	UNiagaraGraph* CalledGraph = GetCalledGraph();
	if (!NodeGraph || !CalledGraph)
	{
		return;
	}
	if (NodeGraph->GetMetaData(FromVariable).IsSet())
	{
		return;
	}
	TOptional<FNiagaraVariableMetaData> OriginalData = CalledGraph->GetMetaData(FromVariable);
	if (OriginalData.IsSet())
	{
		NodeGraph->SetMetaData(FromVariable, OriginalData.GetValue());
		NodeGraph->NotifyGraphChanged();
	}
}

void FNiagaraFunctionCallNodeDetails::CopyMetadataForNameOverride(FNiagaraVariable FromVariable, FNiagaraVariable ToVariable)
{
	UNiagaraGraph* NodeGraph = GetNodeGraph();
	if (!NodeGraph)
	{
		return;
	}
	if (NodeGraph->GetMetaData(ToVariable).IsSet())
	{
		return;
	}
	TOptional<FNiagaraVariableMetaData> OriginalData = NodeGraph->GetMetaData(FromVariable);
	if (OriginalData.IsSet())
	{
		NodeGraph->SetMetaData(ToVariable, OriginalData.GetValue());
		NodeGraph->NotifyGraphChanged();
	}
}

TSharedRef<SWidget> FNiagaraFunctionCallNodeDetails::OnGetVersionMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	TArray<FNiagaraAssetVersion> AssetVersions = Node->FunctionScript->GetAllAvailableVersions();
	for (FNiagaraAssetVersion& Version : AssetVersions)
	{
		if (!Version.bIsVisibleInVersionSelector)
		{
			continue;
		}
		FVersionedNiagaraScriptData* ScriptData = Node->FunctionScript->GetScriptData(Version.VersionGuid);
		
		FText Tooltip = LOCTEXT("NiagaraFunctionCallNodeSelectVersion", "Select this script version to use by the function call node");
		if (!ScriptData->VersionChangeDescription.IsEmpty())
		{
			Tooltip = FText::Format(LOCTEXT("NiagaraSelectVersionChangelist_Tooltip", "Select this script version to use by the function call node. Change description for this version:\n{0}"), ScriptData->VersionChangeDescription);
		}
		FUIAction UIAction(FExecuteAction::CreateSP(this, &FNiagaraFunctionCallNodeDetails::SwitchToVersion, Version),
        FCanExecuteAction(),
        FIsActionChecked::CreateSP(this, &FNiagaraFunctionCallNodeDetails::IsVersionSelected, Version));
		TAttribute<FText> Label = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraFunctionCallNodeDetails::GetVersionMenuLabel, Version));
		MenuBuilder.AddMenuEntry(Label, Tooltip, FSlateIcon(), UIAction, NAME_None, EUserInterfaceActionType::RadioButton);	
	}

	return MenuBuilder.MakeWidget();
}

void FNiagaraFunctionCallNodeDetails::SwitchToVersion(FNiagaraAssetVersion Version)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("NiagaraChangeVersion_Transaction", "Changing script version"));
	FNiagaraScriptVersionUpgradeContext UpgradeContext = FNiagaraScriptVersionUpgradeContext();
	// we skip the python script here because this version change is done directly in the graph and not in the stack, so we don't need to remap any inputs 
	UpgradeContext.bSkipPythonScript = true;
	Node->ChangeScriptVersion(Version.VersionGuid, UpgradeContext);
	Node->RefreshFromExternalChanges();

	// analytics
	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Type"), TEXT("FunctionNode")));
	if (NiagaraAnalytics::IsPluginAsset(Node->FunctionScript))
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AssetName"), Node->FunctionScript->GetPackage()->GetName()));
	}
	NiagaraAnalytics::RecordEvent(TEXT("Versioning.ScriptVersionChanged"), Attributes);
}

bool FNiagaraFunctionCallNodeDetails::IsVersionSelected(FNiagaraAssetVersion Version) const
{
	return Node->SelectedScriptVersion == Version.VersionGuid || (!Node->SelectedScriptVersion.IsValid() && Version == Node->FunctionScript->GetExposedVersion());
}

FText FNiagaraFunctionCallNodeDetails::GetVersionMenuLabel(FNiagaraAssetVersion Version) const
{
	bool bIsExposed = Version == Node->FunctionScript->GetExposedVersion();
	return FText::Format(FText::FromString("{0}.{1} {2}"), Version.MajorVersion, Version.MinorVersion, bIsExposed ? LOCTEXT("NiagaraExposedVersionHint", "(exposed)") : FText());
}

UNiagaraGraph* FNiagaraFunctionCallNodeDetails::GetNodeGraph()
{
	if (!Node.IsValid())
	{
		return nullptr;
	}
	return Node->GetNiagaraGraph();
}

UNiagaraGraph* FNiagaraFunctionCallNodeDetails::GetCalledGraph()
{
	if (!Node.IsValid())
	{
		return nullptr;
	}
	return Node->GetCalledGraph();
}

#undef LOCTEXT_NAMESPACE
