// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "NiagaraTypes.h"
#include "NiagaraGraph.h"

/** This customization sets up a custom details panel for the function call node in the niagara module graph. */
class FNiagaraFunctionCallNodeDetails : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	
private:
	TWeakObjectPtr<class UNiagaraNodeFunctionCall> Node;

	void CopyMetadataFromCalledGraph(FNiagaraVariable FromVariable);
	void CopyMetadataForNameOverride(FNiagaraVariable FromVariable, FNiagaraVariable ToVariable);

	TSharedRef<SWidget> OnGetVersionMenuContent();
	void SwitchToVersion(FNiagaraAssetVersion Version);
	bool IsVersionSelected(FNiagaraAssetVersion Version) const;
	FText GetVersionMenuLabel(FNiagaraAssetVersion Version) const;

	UNiagaraGraph* GetNodeGraph();
	UNiagaraGraph* GetCalledGraph();
};
