// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraOutlinerCustomization.h"

#include "Modules/ModuleManager.h"

//Customization
#include "IStructureDetailsView.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomization.h"
#include "DetailCategoryBuilder.h"
//Widgets
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNiagaraDebugger.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "SResizeBox.h"
///Niagara
#include "NiagaraEditorModule.h"
#include "NiagaraComponent.h"
#include "NiagaraEditorStyle.h"

#include "NiagaraSimCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraOutlinerCustomization)


#if WITH_NIAGARA_DEBUGGER

#define LOCTEXT_NAMESPACE "NiagaraOutlinerCustomization"

//////////////////////////////////////////////////////////////////////////
// Outliner Row Widgets

class SNiagaraOutlinerTreeItem : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SNiagaraOutlinerTreeItem) { }
	SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_ARGUMENT(TSharedPtr<FNiagaraOutlinerTreeItem>, Item)
	SLATE_ARGUMENT(TSharedPtr<SNiagaraOutlinerTree>, Owner)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	//BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct(const FArguments& InArgs)
	{
		Item = InArgs._Item;
		HighlightText = InArgs._HighlightText;
		Owner = InArgs._Owner;

		Item->Widget = SharedThis(this);

		RefreshContent();
	}
	//END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	void RefreshContent()
	{
		if (!Item->bVisible && Item->GetType() != ENiagaraOutlinerTreeItemType::World)
		{
			ChildSlot
			.Padding(0.0f)
			[
				SNullWidget::NullWidget 
			];
			return;
		}

		TSharedRef<SWidget> ItemHeaderWidget = Item->GetHeaderWidget();
		ChildSlot
			.Padding(2, 2.0f, 2.0f, 2.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.Padding(FMargin(6.0, 3.0f, 6.0f, 3.0f))
				.ToolTipText(this, &SNiagaraOutlinerTreeItem::HandleBorderToolTipText)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &SNiagaraOutlinerTreeItem::HandleNameText)
							.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
							.HighlightText(Owner->GetSearchText())
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Right)
						[
							ItemHeaderWidget
						]
					]				
				]
			];
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			Owner->ToggleItemExpansion(Item);

			return FReply::Unhandled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	/** Callback for getting the name text */
	FText HandleNameText() const
	{
		return Item->GetShortNameText();
	}

	/** Callback for getting the text of the row border's tool tip. */
	FText HandleBorderToolTipText() const
	{
		return Item->GetFullNameText();
	}

	/** A reference to the tree item that is displayed in this row. */
	TSharedPtr<FNiagaraOutlinerTreeItem> Item;

	TAttribute<FText> HighlightText;
	
	TSharedPtr<SNiagaraOutlinerTree> Owner;
};

//////////////////////////////////////////////////////////////////////////

FString SNiagaraOutlinerTree::OutlinerItemToStringDebug(TSharedRef<FNiagaraOutlinerTreeItem> Item)
{
	FString Ret;
	
	switch(Item->GetType())
	{
	case ENiagaraOutlinerTreeItemType::World: Ret += TEXT("World: "); break;
	case ENiagaraOutlinerTreeItemType::System: Ret += TEXT("System: "); break;
	case ENiagaraOutlinerTreeItemType::Component: Ret += TEXT("Component: "); break;
	case ENiagaraOutlinerTreeItemType::Emitter: Ret += TEXT("Emitter: "); break;
	}

	Ret += Item->GetFullName();
	return Ret;
}


void SNiagaraOutlinerTree::Construct(const FArguments& InArgs, TSharedPtr<FNiagaraDebugger> InDebugger)
{
	Debugger = InDebugger;

	if (UNiagaraOutliner* Outliner = Debugger->GetOutliner())
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs ViewArgs;
		ViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		ViewArgs.bAllowSearch = false;
		ViewArgs.bHideSelectionTip = true;
		SelectedItemDetails = PropertyEditorModule.CreateStructureDetailView(
			ViewArgs,
			FStructureDetailsViewArgs(),
			nullptr);

		Outliner->OnChangedDelegate.AddSP(this, &SNiagaraOutlinerTree::RequestRefresh);

		TreeView = SNew(STreeView<TSharedRef<FNiagaraOutlinerTreeItem>>)
			.SelectionMode(ESelectionMode::Single)
			.TreeItemsSource(&RootEntries)
			.OnGenerateRow(this, &SNiagaraOutlinerTree::OnGenerateRow)
			.OnGetChildren(this, &SNiagaraOutlinerTree::OnGetChildren)
			.OnExpansionChanged(this, &SNiagaraOutlinerTree::HandleExpansionChanged)
			.OnSelectionChanged(this, &SNiagaraOutlinerTree::HandleSelectionChanged)
			.OnItemToString_Debug(this, &SNiagaraOutlinerTree::OutlinerItemToStringDebug);

		ChildSlot
		[
			SNew(SSplitter)
			+ SSplitter::Slot()
			.SizeRule(SSplitter::ESizeRule::FractionOfParent)
			.Value(0.65)
			[
				SNew(SBox)
				.Padding(2.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							// Search box allows for filtering
							SAssignNew(SearchBox, SSearchBox)
							.OnTextChanged_Lambda([this](const FText& InText) { SearchText = InText; RefreshTree(); })
						]
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew(SScrollBorder, TreeView.ToSharedRef())
							[
								TreeView.ToSharedRef()
							]
						]
					]
				]
			]
			+ SSplitter::Slot()
			.SizeRule(SSplitter::ESizeRule::FractionOfParent)
			.Value(0.35)
			[
				SNew(SBox)
				.Padding(2.0f)
				[				
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SelectedItemDetails->GetWidget().ToSharedRef()//TODO: Maybe shunt this out into the main outliner details somehow?
					]	
				]			
			]
		];

		// Set focus to the search box on creation
		FSlateApplication::Get().SetKeyboardFocus(SearchBox);
		FSlateApplication::Get().SetUserFocus(0, SearchBox);

		RefreshTree();
	}
}

void SNiagaraOutlinerTree::ToggleItemExpansion(TSharedPtr<FNiagaraOutlinerTreeItem>& Item)
{
	TreeView->SetItemExpansion(Item.ToSharedRef(), Item->Expansion != ENiagaraOutlinerSystemExpansionState::Expanded);
}

void SNiagaraOutlinerTree::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if(bNeedsRefresh)
	{
		bNeedsRefresh = false;
		RefreshTree();
	}
}

TSharedRef<ITableRow> SNiagaraOutlinerTree::OnGenerateRow(TSharedRef<FNiagaraOutlinerTreeItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	static const char* ItemStyles[] =
	{
		"NiagaraEditor.Outliner.WorldItem",
		"NiagaraEditor.Outliner.SystemItem",
		"NiagaraEditor.Outliner.ComponentItem",
		"NiagaraEditor.Outliner.EmitterItem",
	};

	ENiagaraOutlinerTreeItemType StyleType = InItem->GetType();

	return SNew(STableRow<TSharedRef<FNiagaraOutlinerTreeItem>>, InOwnerTable)
		.Style(FNiagaraEditorStyle::Get(), ItemStyles[(int32)StyleType])
		[
			SNew(SNiagaraOutlinerTreeItem)
			.Item(InItem)
			.HighlightText(SearchText)
			.Owner(SharedThis(this))
		];
}

void SNiagaraOutlinerTree::OnGetChildren(TSharedRef<FNiagaraOutlinerTreeItem> InItem, TArray<TSharedRef<FNiagaraOutlinerTreeItem>>& OutChildren)
{
	for(const TSharedRef<FNiagaraOutlinerTreeItem>& ChildItem : InItem->Children)
	{
		if(ChildItem->bVisible)
		{
			OutChildren.Add(ChildItem);
		}
	}
}

void SNiagaraOutlinerTree::HandleExpansionChanged(TSharedRef<FNiagaraOutlinerTreeItem> InItem, bool bExpanded)
{
	InItem->Expansion = bExpanded ? ENiagaraOutlinerSystemExpansionState::Expanded : ENiagaraOutlinerSystemExpansionState::Collapsed;
}

void SNiagaraOutlinerTree::HandleSelectionChanged(TSharedPtr<FNiagaraOutlinerTreeItem> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectedItemDetails.IsValid())
	{
		if (SelectedItem.IsValid())
		{
			SelectedItemDetails->SetStructureData(SelectedItem->GetDetailsViewContent());
		}
		else
		{
			SelectedItemDetails->SetStructureData(nullptr);
		}
	}
}

template<typename ChildItemType>
TSharedPtr<FNiagaraOutlinerTreeItem> SNiagaraOutlinerTree::AddChildItemToEntry(TArray<TSharedRef<FNiagaraOutlinerTreeItem>>& ExistingEntries, const TSharedPtr<FNiagaraOutlinerTreeItem>& InItem, FString ChildName, bool DefaultVisibility, ENiagaraOutlinerSystemExpansionState DefaultExpansion)
{
	TSharedPtr<FNiagaraOutlinerTreeItem> NewTreeEntry;
	if (TSharedRef<FNiagaraOutlinerTreeItem>* ExistingTreeEntry = ExistingEntries.FindByPredicate([&](const auto& Existing) { return Existing->Name == ChildName; }))
	{
		NewTreeEntry = *ExistingTreeEntry;
	}
	else
	{
		NewTreeEntry = MakeShared<ChildItemType>();
		NewTreeEntry->Name = ChildName;
		NewTreeEntry->Expansion = DefaultExpansion;
		NewTreeEntry->bVisible = DefaultVisibility;
	}

	NewTreeEntry->OwnerTree = SharedThis(this);
	NewTreeEntry->Parent = InItem;

	NewTreeEntry->bMatchesSearch = InItem.IsValid() && InItem->bMatchesSearch;
	if (SearchText.IsEmpty() == false)
	{
		NewTreeEntry->bMatchesSearch |= NewTreeEntry->GetFullName().Contains(*SearchText.ToString());
	}

	RefreshTree_Helper(NewTreeEntry);
	
	NewTreeEntry->RefreshWidget();

	if (InItem.IsValid())
	{
		InItem->Children.Add(NewTreeEntry.ToSharedRef());
		InItem->bAnyChildrenVisible |= NewTreeEntry->bVisible;
		//InItem->Expansion = FMath::Max(NewTreeEntry->Expansion, InItem->Expansion);
	}

	return NewTreeEntry;
};

void SNiagaraOutlinerTree::RefreshTree_Helper(const TSharedPtr<FNiagaraOutlinerTreeItem>& InTreeEntry)
{
	if (UNiagaraOutliner* Outliner = Debugger->GetOutliner())
	{
		if (InTreeEntry.IsValid() == false)
		{
			TArray<TSharedRef<FNiagaraOutlinerTreeItem>> ExistingItems = MoveTemp(RootEntries);
			
			//Create the root nodes.
			for (TPair<FString, FNiagaraOutlinerWorldData>& WorldData : Outliner->Data.WorldData)
			{
				FString WorldName = WorldData.Key;
				TSharedPtr<FNiagaraOutlinerTreeItem> NewRootEntry = AddChildItemToEntry<FNiagaraOutlinerTreeWorldItem>(ExistingItems, InTreeEntry, WorldName, true, ENiagaraOutlinerSystemExpansionState::Collapsed);
				RootEntries.Add(NewRootEntry.ToSharedRef());

				bool bSortDecending = Outliner->ViewSettings.bSortDescending;
				auto SortWorldsByAverageTime = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
				{
					check(A->GetType() == ENiagaraOutlinerTreeItemType::World);
					check(B->GetType() == ENiagaraOutlinerTreeItemType::World);
					const FNiagaraOutlinerWorldData* AData = (const FNiagaraOutlinerWorldData*)A->GetData();
					const FNiagaraOutlinerWorldData* BData = (const FNiagaraOutlinerWorldData*)B->GetData();
					if (AData && BData)
					{
						return (AData->AveragePerFrameTime.GameThread < BData->AveragePerFrameTime.GameThread) != bSortDecending;
					}
					return false;
				};
				auto SortWorldsByMaxTime = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
				{
					check(A->GetType() == ENiagaraOutlinerTreeItemType::World);
					check(B->GetType() == ENiagaraOutlinerTreeItemType::World);
					const FNiagaraOutlinerWorldData* AData = (const FNiagaraOutlinerWorldData*)A->GetData();
					const FNiagaraOutlinerWorldData* BData = (const FNiagaraOutlinerWorldData*)B->GetData();
					if (AData && BData)
					{
						return (AData->MaxPerFrameTime.GameThread < BData->MaxPerFrameTime.GameThread) != bSortDecending;
					}
					return false;
				};
				auto SortWorldsByMatchingFilteredChildren = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
				{
					check(A->GetType() == ENiagaraOutlinerTreeItemType::World);
					check(B->GetType() == ENiagaraOutlinerTreeItemType::World);
					return (A->Children.Num() < B->Children.Num()) != bSortDecending;
				};

				TFunction<bool(const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)> SortFunc;
				switch (Outliner->ViewSettings.GetSortMode())
				{
				case ENiagaraOutlinerSortMode::AverageTime: SortFunc = SortWorldsByAverageTime; break;
				case ENiagaraOutlinerSortMode::MaxTime: SortFunc = SortWorldsByMaxTime; break;
				default: SortFunc = SortWorldsByMatchingFilteredChildren; break;
				}

				RootEntries.Sort(SortFunc);
			}
		}
		else
		{
			//Store off any existing items for us to pull from but remove them from the actual tree child list so any items that are no longer in the data are removed.
			TArray<TSharedRef<FNiagaraOutlinerTreeItem>> ExistingItems = MoveTemp(InTreeEntry->Children);
			ENiagaraOutlinerTreeItemType Type = InTreeEntry->GetType();

			InTreeEntry->bVisible = false;
			InTreeEntry->bAnyChildrenVisible = false;
			bool bFiltered = false;

			if (Type == ENiagaraOutlinerTreeItemType::World)
			{
				//Add all systems in use in the world to the tree
				if (FNiagaraOutlinerWorldData* WorldData = (FNiagaraOutlinerWorldData*)InTreeEntry->GetData())
				{
					for (TPair<FString, FNiagaraOutlinerSystemData>& SystemData : WorldData->Systems)
					{
						FString SystemName = SystemData.Key;
						AddChildItemToEntry<FNiagaraOutlinerTreeSystemItem>(ExistingItems, InTreeEntry, SystemName, true, ENiagaraOutlinerSystemExpansionState::Collapsed);
					}
					InTreeEntry->SortChildren();
				}

				//Worlds are filtered if they have no visible contents.
				bFiltered = InTreeEntry->bAnyChildrenVisible == false;
			}
			else if (Type == ENiagaraOutlinerTreeItemType::System)
			{				
				//Add any child instances of this system to the tree
				if (FNiagaraOutlinerSystemData* SystemData = (FNiagaraOutlinerSystemData*)InTreeEntry->GetData())
				{
					for (FNiagaraOutlinerSystemInstanceData& InstData : SystemData->SystemInstances)
					{
						FString ComponentName = InstData.ComponentName;
						AddChildItemToEntry<FNiagaraOutlinerTreeComponentItem>(ExistingItems, InTreeEntry, ComponentName, true, ENiagaraOutlinerSystemExpansionState::Collapsed);
					}
					InTreeEntry->SortChildren();
				}

				//Systems are filtered if they have no visible contents.
				bFiltered = InTreeEntry->bAnyChildrenVisible == false;
			}
			else if(Type == ENiagaraOutlinerTreeItemType::Component)
			{
				//Add any child emitters to the tree
				if (FNiagaraOutlinerSystemInstanceData* InstData = (FNiagaraOutlinerSystemInstanceData*)InTreeEntry->GetData())
				{
					for (FNiagaraOutlinerEmitterInstanceData& EmtitterData : InstData->Emitters)
					{
						FString EmitterName = EmtitterData.EmitterName;
						AddChildItemToEntry<FNiagaraOutlinerTreeEmitterItem>(ExistingItems, InTreeEntry, EmitterName, true, ENiagaraOutlinerSystemExpansionState::Collapsed);
					}
					InTreeEntry->SortChildren();

					//Apply any system instance filters. No need to generate and check children with these filters.
					if ((Outliner->ViewSettings.FilterSettings.bFilterBySystemExecutionState && Outliner->ViewSettings.FilterSettings.SystemExecutionState != InstData->ActualExecutionState) ||
						(Outliner->ViewSettings.FilterSettings.bFilterBySystemCullState && Outliner->ViewSettings.FilterSettings.bSystemCullState != InstData->ScalabilityState.bCulled) )
					{
						bFiltered = true;
					}
				}
			}
			else if(Type == ENiagaraOutlinerTreeItemType::Emitter)
			{
				//Should this emitter be filtered out.
				if (FNiagaraOutlinerEmitterInstanceData* EmitterData = (FNiagaraOutlinerEmitterInstanceData*)InTreeEntry->GetData())
				{
					bFiltered = (Outliner->ViewSettings.FilterSettings.bFilterByEmitterExecutionState && Outliner->ViewSettings.FilterSettings.EmitterExecutionState != EmitterData->ExecState) ||
						(Outliner->ViewSettings.FilterSettings.bFilterByEmitterSimTarget && Outliner->ViewSettings.FilterSettings.EmitterSimTarget != EmitterData->SimTarget);
				}
			}

			bool bIsLeaf = (Type == ENiagaraOutlinerTreeItemType::Emitter);
			bool bAnyFiltersActive = SearchText.IsEmpty() == false || Outliner->ViewSettings.FilterSettings.AnyActive();

			bool bVisible = false;
			if (bFiltered)
			{
				bVisible = false;
			}
			else
			{
				if (bIsLeaf)
				{
					bVisible = SearchText.IsEmpty() || InTreeEntry->bMatchesSearch;
				}
				else
				{
					bVisible = InTreeEntry->bMatchesSearch || (!bAnyFiltersActive || InTreeEntry->bAnyChildrenVisible);
				}
			}
			InTreeEntry->bVisible = bVisible;
			TreeView->SetItemExpansion(InTreeEntry.ToSharedRef(), InTreeEntry->Expansion == ENiagaraOutlinerSystemExpansionState::Expanded);
		}
	}
}

void SNiagaraOutlinerTree::RefreshTree()
{
	RefreshTree_Helper(nullptr);
	TreeView->RequestTreeRefresh();
}

//////////////////////////////////////////////////////////////////////////

template<typename T>
class SNiagaraOutlinerTreeItemHeaderDataWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraOutlinerTreeItemHeaderDataWidget) 
	: _MinDesiredWidth(FOptionalSize()) 
	{
	}

	SLATE_ARGUMENT(FText, LabelText)
	SLATE_ARGUMENT(FText, ToolTipText)
	SLATE_ARGUMENT(T, Data)
	SLATE_ATTRIBUTE(FOptionalSize, MinDesiredWidth)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FNiagaraOutlinerViewSettings& ViewSettings)
	{
		T Data = InArgs._Data;
		FText ValueText = FText::AsNumber(Data);
		FText Label = InArgs._LabelText;
		
		TSharedRef<SSplitter> HeaderWidget = SNew(SSplitter).PhysicalSplitterHandleSize(1);
		
		if (Label.IsEmpty() == false)
		{
			HeaderWidget->AddSlot()
			.SizeRule(SSplitter::ESizeRule::SizeToContent)
			[
				SNew(STextBlock)
				.Text(Label)
				.ToolTipText(InArgs._ToolTipText)
				.Justification(ETextJustify::Center)
			];
		}

		HeaderWidget->AddSlot()
		.SizeRule(SSplitter::ESizeRule::SizeToContent)
		[
			SNew(STextBlock)
			.Text(ValueText)
			.ToolTipText(InArgs._ToolTipText)
			.Justification(ETextJustify::Center)
		];

		ChildSlot
		[
			SNew(SBox)
			.MinDesiredWidth(InArgs._MinDesiredWidth)
			.ToolTipText(InArgs._ToolTipText)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				.HAlign(HAlign_Center)
				[
					HeaderWidget
				]
			]
		];
	}
};

template<>
class SNiagaraOutlinerTreeItemHeaderDataWidget<FText> : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraOutlinerTreeItemHeaderDataWidget)
		: _MinDesiredWidth(FOptionalSize())
	{
	}

	SLATE_ARGUMENT(FText, LabelText)
	SLATE_ARGUMENT(FText, ToolTipText)
	SLATE_ARGUMENT(FText, Data)
	SLATE_ATTRIBUTE(FOptionalSize, MinDesiredWidth)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FNiagaraOutlinerViewSettings& ViewSettings)
	{
		FText Label = InArgs._LabelText;
		TSharedRef<SSplitter> HeaderWidget = SNew(SSplitter).PhysicalSplitterHandleSize(1);
		
		if (Label.IsEmpty() == false)
		{
			HeaderWidget->AddSlot()
			.SizeRule(SSplitter::ESizeRule::SizeToContent)
			[
				SNew(STextBlock)
				.Text(Label)
				.ToolTipText(InArgs._ToolTipText)
				.Justification(ETextJustify::Center)
			];
		}

		HeaderWidget->AddSlot()
		.SizeRule(SSplitter::ESizeRule::SizeToContent)
		[
			SNew(STextBlock)
			.Text(InArgs._Data)
			.ToolTipText(InArgs._ToolTipText)
			.Justification(ETextJustify::Center)
		];

		ChildSlot
		[
			SNew(SBox)
			.MinDesiredWidth(InArgs._MinDesiredWidth)
			.ToolTipText(InArgs._ToolTipText)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				.HAlign(HAlign_Center)
				[
					HeaderWidget
				]
			]
		];
	}
};

template<>
class SNiagaraOutlinerTreeItemHeaderDataWidget<FNiagaraOutlinerTimingData> : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraOutlinerTreeItemHeaderDataWidget)
		: _MinDesiredWidth(FOptionalSize())
	{
	}

	SLATE_ARGUMENT(FText, LabelText)
	SLATE_ARGUMENT(FText, ToolTipText)
	SLATE_ARGUMENT(FNiagaraOutlinerTimingData, Data)
	SLATE_ATTRIBUTE(FOptionalSize, MinDesiredWidth)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FNiagaraOutlinerViewSettings& ViewSettings)
	{
		FText Label = InArgs._LabelText;
		FNiagaraOutlinerTimingData Data = InArgs._Data;

		float GTVal = Data.GameThread;
		float RTVal = Data.RenderThread;

		if (ViewSettings.TimeUnits == ENiagaraOutlinerTimeUnits::Milliseconds)
		{
			GTVal /= 1000.0f;
			RTVal /= 1000.0f;
		}
		else if (ViewSettings.TimeUnits == ENiagaraOutlinerTimeUnits::Seconds)
		{
			GTVal /= 1000000.0f;
			RTVal /= 1000000.0f;
		}

		FText GameThread = FText::AsNumber(GTVal);
		FText RenderThread = FText::AsNumber(RTVal);

		FText TooltipFmt = LOCTEXT("TimingDataTooltipFmt", "{0} ({1})");
		FText GTTooltip = FText::Format(TooltipFmt, InArgs._ToolTipText, LOCTEXT("TimingDataGameThread","Game Thread"));
		FText RTTooltip = FText::Format(TooltipFmt, InArgs._ToolTipText, LOCTEXT("TimingDataRenderThread", "Render Thread"));

		TSharedRef<SSplitter> HeaderWidget = SNew(SSplitter)
			.PhysicalSplitterHandleSize(2)
			.Style(FAppStyle::Get(), "SplitterDark");
		
		if (Label.IsEmpty() == false)
		{
			HeaderWidget->AddSlot()
			.SizeRule(SSplitter::ESizeRule::SizeToContent)
			[
				SNew(STextBlock)
				.Text(Label)
				.ToolTipText(InArgs._ToolTipText)
				.Justification(ETextJustify::Center)
			];
		}

		HeaderWidget->AddSlot()
		.SizeRule(SSplitter::ESizeRule::SizeToContent)
		[
			SNew(SBox)
			.Padding(FMargin(4, 0, 4, 0))
			.MinDesiredWidth(InArgs._MinDesiredWidth)
			[
				SNew(STextBlock)
				.Text(GameThread)
				.ToolTipText(GTTooltip)
				.Justification(ETextJustify::Center)
			]
		];

		HeaderWidget->AddSlot()
		.SizeRule(SSplitter::ESizeRule::SizeToContent)
		[
			SNew(SBox)
			.Padding(FMargin(4, 0, 4, 0))
			.MinDesiredWidth(InArgs._MinDesiredWidth)
			[
				SNew(STextBlock)
				.Text(RenderThread)
				.ToolTipText(RTTooltip)
				.Justification(ETextJustify::Center)
			]
		];

		ChildSlot
		[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				.ToolTipText(InArgs._ToolTipText)
				.HAlign(HAlign_Center)
				[
					HeaderWidget
				]
		];
	}
};
const float FNiagaraOutlinerTreeItem::HeaderPadding = 6.0f;

void FNiagaraOutlinerTreeItem::RefreshWidget()
{
	if (TSharedPtr<SNiagaraOutlinerTreeItem> PinnedWidget = Widget.Pin())
	{
		PinnedWidget->RefreshContent();
	}
}

TSharedRef<SWidget> FNiagaraOutlinerTreeItem::GetHeaderWidget()
{
	return SNullWidget::NullWidget;
}

TSharedPtr<FStructOnScope>& FNiagaraOutlinerTreeItem::GetDetailsViewContent()
{
	if (DetailsViewData == nullptr)
	{
		DetailsViewData = MakeShared<FStructOnScope>();

		const void* Data = GetData();
		check(Data);

		ENiagaraOutlinerTreeItemType Type = GetType();
		switch (Type)
		{
		case ENiagaraOutlinerTreeItemType::World:
		{
			DetailsViewData->Initialize(FNiagaraOutlinerWorldDataCustomizationWrapper::StaticStruct());
			FNiagaraOutlinerWorldData::StaticStruct()->CopyScriptStruct(DetailsViewData->GetStructMemory(), Data);
		}
		break;
		case ENiagaraOutlinerTreeItemType::System: 
		{
			DetailsViewData->Initialize(FNiagaraOutlinerSystemDataCustomizationWrapper::StaticStruct());
			FNiagaraOutlinerSystemData::StaticStruct()->CopyScriptStruct(DetailsViewData->GetStructMemory(), Data);
		}
		break;
		case ENiagaraOutlinerTreeItemType::Component: 
		{
			DetailsViewData->Initialize(FNiagaraOutlinerSystemInstanceDataCustomizationWrapper::StaticStruct());
			FNiagaraOutlinerSystemInstanceData::StaticStruct()->CopyScriptStruct(DetailsViewData->GetStructMemory(), Data);
		}
		break;
		case ENiagaraOutlinerTreeItemType::Emitter:
		{
			DetailsViewData->Initialize(FNiagaraOutlinerEmitterInstanceDataCustomizationWrapper::StaticStruct());
			FNiagaraOutlinerEmitterInstanceData::StaticStruct()->CopyScriptStruct(DetailsViewData->GetStructMemory(), Data);
		}
		break;
		}
	}
	return DetailsViewData;
}

void FNiagaraOutlinerTreeWorldItem::SortChildren() 
{
	if (TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin())
	{
		if (UNiagaraOutliner* Outliner = Tree->GetDebugger()->GetOutliner())
		{
			bool bSortDecending = Outliner->ViewSettings.bSortDescending;
			auto SortSystemsByAverageTime = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
			{
				check(A->GetType() == ENiagaraOutlinerTreeItemType::System);
				check(B->GetType() == ENiagaraOutlinerTreeItemType::System);
				const FNiagaraOutlinerSystemData* AData = (const FNiagaraOutlinerSystemData*)A->GetData();
				const FNiagaraOutlinerSystemData* BData = (const FNiagaraOutlinerSystemData*)B->GetData();
				if (AData && BData)
				{
					return (AData->AveragePerFrameTime.GameThread < BData->AveragePerFrameTime.GameThread) != bSortDecending;
				}
				return false;
			};
			auto SortSystemsByMaxTime = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
			{
				check(A->GetType() == ENiagaraOutlinerTreeItemType::System);
				check(B->GetType() == ENiagaraOutlinerTreeItemType::System);
				const FNiagaraOutlinerSystemData* AData = (const FNiagaraOutlinerSystemData*)A->GetData();
				const FNiagaraOutlinerSystemData* BData = (const FNiagaraOutlinerSystemData*)B->GetData();
				if (AData && BData)
				{
					return (AData->MaxPerFrameTime.GameThread < BData->MaxPerFrameTime.GameThread) != bSortDecending;
				}
				return false;
			};
			auto SortSystemsByMatchingFilteredChildren = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
			{
				check(A->GetType() == ENiagaraOutlinerTreeItemType::System);
				check(B->GetType() == ENiagaraOutlinerTreeItemType::System);
				return (A->Children.Num() < B->Children.Num()) != bSortDecending;
			};

			TFunction<bool(const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)> SortFunc;
			switch (Outliner->ViewSettings.GetSortMode())
			{
			case ENiagaraOutlinerSortMode::AverageTime: SortFunc = SortSystemsByAverageTime; break;
			case ENiagaraOutlinerSortMode::MaxTime: SortFunc = SortSystemsByMaxTime; break;
			default: SortFunc = SortSystemsByMatchingFilteredChildren; break;
			}

			Children.Sort(SortFunc);
		}
	}
}

const void* FNiagaraOutlinerTreeWorldItem::GetData()const
{
	if (TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin())
	{
		if (UNiagaraOutliner* Outliner = Tree->GetDebugger()->GetOutliner())
		{
			const FString& WorldName = GetFullName();
			return Outliner->FindWorldData(WorldName);
		}
	}
	return nullptr;
}

TSharedRef<SWidget> FNiagaraOutlinerTreeWorldItem::GetHeaderWidget()
{
	if (FNiagaraOutlinerWorldData* Data = (FNiagaraOutlinerWorldData*)GetData())
	{
		TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin();
		check(Tree.IsValid());//If we managed to get valid data then the tree should be valid.
		TSharedPtr<FNiagaraDebugger>& Debugger = Tree->GetDebugger();
		check(Debugger.IsValid());
		UNiagaraOutliner* Outliner = Debugger->GetOutliner();
		check(Outliner);

		TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);

		if (Outliner->ViewSettings.ViewMode == ENiagaraOutlinerViewModes::State)
		{
			//Count system instances matching filters.
			int32 NumMatchingInstances = 0;
			for (const TSharedRef<FNiagaraOutlinerTreeItem>& SystemItem : Children)
			{
				if (SystemItem->GetType() == ENiagaraOutlinerTreeItemType::System && SystemItem->bVisible)
				{
					for (const TSharedRef<FNiagaraOutlinerTreeItem>& InstItem : SystemItem->Children)
					{
						if (InstItem->GetType() == ENiagaraOutlinerTreeItemType::Component && InstItem->bVisible)
						{
							++NumMatchingInstances;
						}
					}
				}
			}

			Box->AddSlot()
				.AutoWidth()
				.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
				[
					SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FText>, Outliner->ViewSettings)
					.ToolTipText(LOCTEXT("WorldHeaderTooltip_WorldType", "World Type"))
					.Data(FText::FromString(LexToString((EWorldType::Type)Data->WorldType)))
				];

			Box->AddSlot()
				.AutoWidth()
				.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
				[
					SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FText>, Outliner->ViewSettings)
					.ToolTipText(LOCTEXT("WorldHeaderTooltip_NetMode", "Net Mode"))
					.Data(FText::FromString(ToString((ENetMode)Data->NetMode)))
				];

			Box->AddSlot()
				.AutoWidth()
				.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
				[
					SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FText>, Outliner->ViewSettings)
					.ToolTipText(LOCTEXT("WorldHeaderTooltip_BegunPlay", "Has Begun Play"))
					.Data(Data->bHasBegunPlay ? FText(LOCTEXT("True", "True")) : FText(LOCTEXT("False", "False")))
					.MinDesiredWidth(50.0f)
				];

			Box->AddSlot()
				.AutoWidth()
				.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
				[
					SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<int32>, Outliner->ViewSettings)
					.ToolTipText(LOCTEXT("VisibleSystemsHeaderName", "Num instances matching current search and filters."))
					.Data(NumMatchingInstances)
				];
		}
		else if (Outliner->ViewSettings.ViewMode == ENiagaraOutlinerViewModes::Performance)
		{
			Box->AddSlot()
				.AutoWidth()
				.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
				[
					SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FNiagaraOutlinerTimingData>, Outliner->ViewSettings)
					//.LabelText(LOCTEXT("WorldAvgPerFrameTime","Frame Avg"))
					.ToolTipText(LOCTEXT("WorldAvgPerFrameTimeToolTip", "Average frame time for all FX work in this World."))
					.Data(Data->AveragePerFrameTime)
					.MinDesiredWidth(50)
				];

			Box->AddSlot()
				.AutoWidth()
				.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
				[
					SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FNiagaraOutlinerTimingData>, Outliner->ViewSettings)
					//.LabelText(LOCTEXT("WorldMaxPerFrameTime", "Frame Max"))
					.ToolTipText(LOCTEXT("WorldMaxPerFrameTimeToolTip", "Max frame time for all FX work in this World."))
					.Data(Data->MaxPerFrameTime)
					.MinDesiredWidth(50)
				];
		}

		return Box;
	}
	return SNullWidget::NullWidget;
}

void FNiagaraOutlinerTreeSystemItem::SortChildren() 
{
	if (TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin())
	{
		if (UNiagaraOutliner* Outliner = Tree->GetDebugger()->GetOutliner())
		{
			bool bSortDecending = Outliner->ViewSettings.bSortDescending;
			auto SortByAvgTime = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
			{
				check(A->GetType() == ENiagaraOutlinerTreeItemType::Component);
				check(B->GetType() == ENiagaraOutlinerTreeItemType::Component);
				const FNiagaraOutlinerSystemInstanceData* AData = (const FNiagaraOutlinerSystemInstanceData*)A->GetData();
				const FNiagaraOutlinerSystemInstanceData* BData = (const FNiagaraOutlinerSystemInstanceData*)B->GetData();
				if (AData && BData)
				{
					return (AData->AverageTime.GameThread < BData->AverageTime.GameThread) != bSortDecending;
				}
				return false;
			};
			auto SortByMaxTime = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
			{
				check(A->GetType() == ENiagaraOutlinerTreeItemType::Component);
				check(B->GetType() == ENiagaraOutlinerTreeItemType::Component);
				const FNiagaraOutlinerSystemInstanceData* AData = (const FNiagaraOutlinerSystemInstanceData*)A->GetData();
				const FNiagaraOutlinerSystemInstanceData* BData = (const FNiagaraOutlinerSystemInstanceData*)B->GetData();
				if (AData && BData)
				{
					return (AData->MaxTime.GameThread < BData->MaxTime.GameThread) != bSortDecending;
				}
				return false;
			};
			auto SortByMatching = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
			{
				check(A->GetType() == ENiagaraOutlinerTreeItemType::Component);
				check(B->GetType() == ENiagaraOutlinerTreeItemType::Component);
				return (A->Children.Num() < B->Children.Num()) != bSortDecending;
			};

			TFunction<bool(const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)> SortFunc;
			switch (Outliner->ViewSettings.GetSortMode())
			{
			case ENiagaraOutlinerSortMode::AverageTime: SortFunc = SortByAvgTime; break;
			case ENiagaraOutlinerSortMode::MaxTime: SortFunc = SortByMaxTime; break;
			default: SortFunc = SortByMatching; break;
			}

			Children.Sort(SortFunc);
		}
	}
}

const void* FNiagaraOutlinerTreeSystemItem::GetData()const
{
	if (TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin())
	{
		if (UNiagaraOutliner* Outliner = Tree->GetDebugger()->GetOutliner())
		{
			TSharedPtr<FNiagaraOutlinerTreeItem> WorldItem = GetParent();

			const FString& SystemName = GetFullName();
			const FString& WorldName = WorldItem->GetFullName();
			return Outliner->FindSystemData(WorldName, SystemName);
		}
	}
	return nullptr;
}

TSharedRef<SWidget> FNiagaraOutlinerTreeSystemItem::GetHeaderWidget()
{
	FNiagaraOutlinerSystemData* Data = (FNiagaraOutlinerSystemData*)GetData();
	if (Data == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin();
	check(Tree.IsValid());//If we managed to get valid data then the tree should be valid.
	TSharedPtr<FNiagaraDebugger>& Debugger = Tree->GetDebugger();
	check(Debugger.IsValid());
	UNiagaraOutliner* Outliner = Debugger->GetOutliner();
	check(Outliner);

	TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);

	if (Data->AssetPath.IsValid())
	{
		Box->AddSlot()
		.AutoWidth()
		.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
		[
			SNew(SButton)
			.OnClicked_Lambda(
				[AssetPath = Data->AssetPath]()
				{
					FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
					FAssetData AssetData;
					if (AssetRegistryModule.Get().TryGetAssetByObjectPath(AssetPath, AssetData) != UE::AssetRegistry::EExists::Unknown)
					{
						GEditor->SyncBrowserToObjects({AssetData});
					}
					return FReply::Handled();
				}
			)
			.ToolTipText(LOCTEXT("NiagaraOutlineCaptureBrowseToAsset", "Browse to this asset in the content browser."))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Icons.BrowseContent")))
			]
		];
	}

	if (Outliner->ViewSettings.ViewMode == ENiagaraOutlinerViewModes::State)
	{
		int32 NumMatchingSystems = 0;
		for (const TSharedRef<FNiagaraOutlinerTreeItem>& Item : Children)
		{
			if (Item->GetType() == ENiagaraOutlinerTreeItemType::Component && Item->bVisible)
			{
				++NumMatchingSystems;
			}
		}

		Box->AddSlot()
			.AutoWidth()
			.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
			[
				SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<int32>, Outliner->ViewSettings)
				.ToolTipText(LOCTEXT("VisibleInstancesHeaderName", "Num system instances matching current search and filters."))
				.Data(NumMatchingSystems)
			];
	}
	else if (Outliner->ViewSettings.ViewMode == ENiagaraOutlinerViewModes::Performance)
	{
		Box->AddSlot()
			.AutoWidth()
			.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
			[
				SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FNiagaraOutlinerTimingData>, Outliner->ViewSettings)
				//.LabelText(LOCTEXT("SystemAvgPerInstanceTime", "Avg"))
				.ToolTipText(LOCTEXT("SystemAvgPerInstanceTimeToolTip", "Average instance time for this System."))
			.Data(Data->AveragePerInstanceTime)
			.MinDesiredWidth(50)
			];

		Box->AddSlot()
			.AutoWidth()
			.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
			[
				SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FNiagaraOutlinerTimingData>, Outliner->ViewSettings)
				//.LabelText(LOCTEXT("SystemMaxPerFrameTime", "Max"))
				.ToolTipText(LOCTEXT("SystemMaxPerFrameTimeToolTip", "Max instance time for this System."))
				.Data(Data->MaxPerInstanceTime)
				.MinDesiredWidth(50)
			];

		Box->AddSlot()
			.AutoWidth()
			.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
			[
				SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FNiagaraOutlinerTimingData>, Outliner->ViewSettings)
				//.LabelText(LOCTEXT("SystemAvgPerFrameTime", "Frame Avg"))
				.ToolTipText(LOCTEXT("SystemAvgPerFrameTimeToolTip", "Average frame time for all instances of this System."))
				.Data(Data->AveragePerFrameTime)
				.MinDesiredWidth(50)
			];

		Box->AddSlot()
			.AutoWidth()
			.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
			[
				SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FNiagaraOutlinerTimingData>, Outliner->ViewSettings)
				//.LabelText(LOCTEXT("SystemMaxPerFrameTime", "Frame Max"))
				.ToolTipText(LOCTEXT("AllInstancesMaxFrameTimeToolTip", "Max frame time for all instances of this System."))
				.Data(Data->MaxPerFrameTime)
				.MinDesiredWidth(50)
			];

		//TODO: Add baseline comparison traffic light icon with tooltip containing details.
	}

	return Box;
}

void FNiagaraOutlinerTreeComponentItem::SortChildren() 
{
	if (TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin())
	{
		if (UNiagaraOutliner* Outliner = Tree->GetDebugger()->GetOutliner())
		{
			bool bSortDecending = Outliner->ViewSettings.bSortDescending;

			auto SortByMatching = [bSortDecending](const TSharedRef<FNiagaraOutlinerTreeItem>& A, const TSharedRef<FNiagaraOutlinerTreeItem>& B)
			{
				check(A->GetType() == ENiagaraOutlinerTreeItemType::Emitter);
				check(B->GetType() == ENiagaraOutlinerTreeItemType::Emitter);
				return (A->Children.Num() < B->Children.Num()) != bSortDecending;
			};

			//For now, always sort emitters by matching but we should gather per emitter perf stats in future too.
			Children.Sort(SortByMatching);
		}
	}
}

const void* FNiagaraOutlinerTreeComponentItem::GetData()const
{

	if (TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin())
	{
		if (UNiagaraOutliner* Outliner = Tree->GetDebugger()->GetOutliner())
		{
			TSharedPtr<FNiagaraOutlinerTreeItem> SystemItem = GetParent();
			TSharedPtr<FNiagaraOutlinerTreeItem> WorldItem = SystemItem->GetParent();

			const FString& ComponentName = GetFullName();
			const FString& SystemName = SystemItem->GetFullName();
			const FString& WorldName = WorldItem->GetFullName();
			return Outliner->FindComponentData(WorldName, SystemName, ComponentName);
		}
	}
	return nullptr;
}

TSharedRef<SWidget> FNiagaraOutlinerTreeComponentItem::GetHeaderWidget()
{
	if (FNiagaraOutlinerSystemInstanceData* Data = (FNiagaraOutlinerSystemInstanceData*)GetData())
	{
		TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin();
		check(Tree.IsValid());//If we managed to get valid data then the tree should be valid.
		TSharedPtr<FNiagaraDebugger>& Debugger = Tree->GetDebugger();
		check(Debugger.IsValid());
		UNiagaraOutliner* Outliner = Debugger->GetOutliner();
		check(Outliner);

		TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);

		if (Outliner->ViewSettings.ViewMode == ENiagaraOutlinerViewModes::State)
		{
			UEnum* ExecStateEnum = StaticEnum<ENiagaraExecutionState>();

			if (Data->ActualExecutionState == ENiagaraExecutionState::Num)
			{
				//System instance is not initialized	
				Box->AddSlot()
					.AutoWidth()
					.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
					[
						SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FText>, Outliner->ViewSettings)
						.ToolTipText(LOCTEXT("UninitializedSystemInstanceTooltip", "Internal data for component is uninitialized. Likely as it has yet to be activated."))
						.Data(LOCTEXT("UninitializedSystemInstanceValue", "Uninitialized"))
					];
			}
			else if (Data->bUsingCullProxy)
			{
				//This instance is using it's cull proxy.
				Box->AddSlot()
					.AutoWidth()
					.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
					[
						SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FText>, Outliner->ViewSettings)
						.ToolTipText(LOCTEXT("CullProxyToolTipText", "This instance is not simulating. Instead it's using a cull proxy to maintain visuals while limiting it's cost."))
						.Data(LOCTEXT("CullProxyInstanceValue", "Cull Proxy"))
					];
			}
			else
			{
				int32 NumMatchingEmitters = 0;
				for (const TSharedRef<FNiagaraOutlinerTreeItem>& Item : Children)
				{
					if (Item->GetType() == ENiagaraOutlinerTreeItemType::Emitter && Item->bVisible)
					{
						++NumMatchingEmitters;
					}
				}

				UEnum* PoolMethodEnum = StaticEnum<ENCPoolMethod>();

				Box->AddSlot()
					.AutoWidth()
					.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
					[
						SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FText>, Outliner->ViewSettings)
						.ToolTipText(LOCTEXT("ComponentHeaderTooltip_PoolMethod", "Pooling Method"))
						.Data(PoolMethodEnum->GetDisplayNameTextByValue((int32)Data->PoolMethod))
						.MinDesiredWidth(50.0f)
					];

				if (Data->ScalabilityState.bCulled)
				{
					Box->AddSlot()
						.AutoWidth()
						.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
						[
							SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FText>, Outliner->ViewSettings)
							.ToolTipText(LOCTEXT("CulledEmittersHeaderTooltip", "State"))
							.Data(LOCTEXT("CulledEmittersHeaderValue", "Culled"))
						];
				}
				else
				{
					Box->AddSlot()
						.AutoWidth()
						.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
						[
							SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FText>, Outliner->ViewSettings)
							.ToolTipText(LOCTEXT("ComponentHeaderTooltip_ExecutionState", "Execution State"))
							.Data(ExecStateEnum->GetDisplayNameTextByValue((int32)Data->ActualExecutionState))
							.MinDesiredWidth(50.0f)
						];
				}

				Box->AddSlot()
					.AutoWidth()
					.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
					[
						SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<int32>, Outliner->ViewSettings)
						.ToolTipText(LOCTEXT("VisibleEmittersHeaderTooltip", "Num emitters matching current search and filters."))
						.Data(NumMatchingEmitters)
					];

			}
		}
		else if (Outliner->ViewSettings.ViewMode == ENiagaraOutlinerViewModes::Performance)
		{
			Box->AddSlot()
				.AutoWidth()
				.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
				[
					SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FNiagaraOutlinerTimingData>, Outliner->ViewSettings)
					.ToolTipText(LOCTEXT("InstanceAvgPerFrameTime", "Average frame time for this System Instance."))
					.Data(Data->AverageTime)
					.MinDesiredWidth(50)
				];

			Box->AddSlot()
				.AutoWidth()
				.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
				[
					SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FNiagaraOutlinerTimingData>, Outliner->ViewSettings)
					.ToolTipText(LOCTEXT("InstanceMaxPerFrameTime", "Max frame time for this System Instance."))
					.Data(Data->MaxTime)
					.MinDesiredWidth(50)
				];

			//TODO: Add baseline comparison traffic light icon with tooltip containing details.
		}
		else if (Outliner->ViewSettings.ViewMode == ENiagaraOutlinerViewModes::Debug)
		{			
			Box->AddSlot()
			.AutoWidth()
			.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
			[
				SNew(SButton)
				.OnClicked(this, &FNiagaraOutlinerTreeComponentItem::CaputreSimCache)
				.ToolTipText(LOCTEXT("NiagaraOutlineCaptureSimCacheTooltip", "Capture a new Sim Cache for this component."))
				.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
				//.ForegroundColor(FLinearColor::White)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NiagaraOutlinerCaptureSimCache", "Capture"))
					//.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
				]	
			];

			if (TObjectPtr<UNiagaraSimCache> SimCache = Outliner->FindSimCache(*Data->ComponentName))
			{
				Box->AddSlot()
				.AutoWidth()
				.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
				[
					SNew(SButton)
					.OnClicked(this, &FNiagaraOutlinerTreeComponentItem::OpenSimCache)
					.ToolTipText(LOCTEXT("NiagaraOutlineOpenSimCacheTooltip", "Sim Cache is available for this component. Open in new window."))
					.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
					//.ForegroundColor(FLinearColor::White)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NiagaraOutlinerOpenSimCache", "Sim Cache"))
						//.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
					]	
				];
			}
			else
			{
				Box->AddSlot()
					.AutoWidth()
					.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
					[
						SNew(SButton)						
						.ToolTipText(LOCTEXT("NiagaraOutlineOpenSimCacheTooltipDisabled", "No Sim Cache available. Please capture one."))
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
						//.ForegroundColor(FLinearColor::White)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NiagaraOutlinerOpenSimCache", "Sim Cache"))
						//.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
					]
				];
			}

			//TODO:	Add playback controls per component.
			//TODO: Add controls to set the current debug hud component filter to this component.
		}

		return Box;
	}
	return SNullWidget::NullWidget;
}

FReply FNiagaraOutlinerTreeComponentItem::OpenSimCache()
{
	if (FNiagaraOutlinerSystemInstanceData* Data = (FNiagaraOutlinerSystemInstanceData*)GetData())
	{
		TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin();
		check(Tree.IsValid());//If we managed to get valid data then the tree should be valid.
		TSharedPtr<FNiagaraDebugger>& Debugger = Tree->GetDebugger();
		check(Debugger.IsValid());
		UNiagaraOutliner* Outliner = Debugger->GetOutliner();
		check(Outliner);

		if (TObjectPtr<UNiagaraSimCache> SimCache = Outliner->FindSimCache(*Data->ComponentName))
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Cast<UObject,UNiagaraSimCache>(SimCache.Get()), EToolkitMode::Standalone);
		}
	}
	return FReply::Handled();
}

FReply FNiagaraOutlinerTreeComponentItem::CaputreSimCache()
{
	if (FNiagaraOutlinerSystemInstanceData* Data = (FNiagaraOutlinerSystemInstanceData*)GetData())
	{
		TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin();
		check(Tree.IsValid());//If we managed to get valid data then the tree should be valid.
		TSharedPtr<FNiagaraDebugger>& Debugger = Tree->GetDebugger();
		check(Debugger.IsValid());
		UNiagaraOutliner* Outliner = Debugger->GetOutliner();
		check(Outliner);

		if (Outliner->CaptureSettings.SimCacheCaptureFrames > 0)
		{
			Debugger->TriggerSimCacheCapture(*Data->ComponentName, Outliner->CaptureSettings.CaptureDelayFrames, Outliner->CaptureSettings.SimCacheCaptureFrames);
		}
	}
	return FReply::Handled();
}

void FNiagaraOutlinerTreeEmitterItem::SortChildren() 
{

}

const void* FNiagaraOutlinerTreeEmitterItem::GetData()const
{
	if (TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin())
	{
		if (UNiagaraOutliner* Outliner = Tree->GetDebugger()->GetOutliner())
		{
			TSharedPtr<FNiagaraOutlinerTreeItem> CompItem = GetParent();
			TSharedPtr<FNiagaraOutlinerTreeItem> SystemItem = CompItem->GetParent();
			TSharedPtr<FNiagaraOutlinerTreeItem> WorldItem = SystemItem->GetParent();

			const FString& EmitterName = GetFullName();
			const FString& ComponentName = CompItem->GetFullName();
			const FString& SystemName = SystemItem->GetFullName();
			const FString& WorldName = WorldItem->GetFullName();
			return Outliner->FindEmitterData(WorldName, SystemName, ComponentName, EmitterName);
		}
	}
	return nullptr;
}

TSharedRef<SWidget> FNiagaraOutlinerTreeEmitterItem::GetHeaderWidget()
{
	if(FNiagaraOutlinerEmitterInstanceData* Data = (FNiagaraOutlinerEmitterInstanceData*) GetData())	
	{
		TSharedPtr<SNiagaraOutlinerTree> Tree = OwnerTree.Pin();
		check(Tree.IsValid());//If we managed to get valid data then the tree should be valid.
		TSharedPtr<FNiagaraDebugger>& Debugger = Tree->GetDebugger();
		check(Debugger.IsValid());
		UNiagaraOutliner* Outliner = Debugger->GetOutliner();
		check(Outliner);

		TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);

		const FSlateBrush* SimTargetBrush = Data->SimTarget == ENiagaraSimTarget::CPUSim ? FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.CPUIcon") : FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.GPUIcon");

		UEnum* ExecStateEnum = StaticEnum<ENiagaraExecutionState>();

		Box->AddSlot()
			.AutoWidth()
			.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
			[
				SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<FText>, Outliner->ViewSettings)
				.ToolTipText(LOCTEXT("EmitterHeaderTooltip_ExecState", "Execution State"))
				.Data(ExecStateEnum->GetDisplayNameTextByValue((int32)Data->ExecState))
				.MinDesiredWidth(50.0f)
			];
		
		Box->AddSlot()
			.AutoWidth()
			.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
			[
				SNew(SImage)
				.Image(SimTargetBrush)
				.ToolTipText(LOCTEXT("EmitterHeaderTooltip_SimTarget", "Sim Target"))
			];

		Box->AddSlot()
			.AutoWidth()
			.Padding(FMargin(HeaderPadding, 0.0f, HeaderPadding, 0.0f))
			[
				SNew(SNiagaraOutlinerTreeItemHeaderDataWidget<int32>, Outliner->ViewSettings)
				.ToolTipText(LOCTEXT("EmitterHeaderTooltip_NumParticles", "Num Particles"))
				.Data(Data->NumParticles)
				.MinDesiredWidth(50.0f)
			];
		return Box;
	}
	return SNullWidget::NullWidget;
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraOutlinerWorldDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FNiagaraOutlinerWorldDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildNum = 0; ChildNum < NumChildren; ++ChildNum)
	{
		TSharedPtr<IPropertyHandle> ChildProperty = StructPropertyHandle->GetChildHandle(ChildNum);

		if (ChildProperty->GetProperty()->GetFName() != GET_MEMBER_NAME_CHECKED(FNiagaraOutlinerWorldData, Systems))
		{
			ChildBuilder.AddProperty(ChildProperty.ToSharedRef());
		}
	}
}
//////////////////////////////////////////////////////////////////////////
void FNiagaraOutlinerSystemDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FNiagaraOutlinerSystemDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildNum = 0; ChildNum < NumChildren; ++ChildNum)
	{
		TSharedPtr<IPropertyHandle> ChildProperty = StructPropertyHandle->GetChildHandle(ChildNum);

		if (ChildProperty->GetProperty()->GetFName() != GET_MEMBER_NAME_CHECKED(FNiagaraOutlinerSystemData, SystemInstances))
		{
			ChildBuilder.AddProperty(ChildProperty.ToSharedRef());
		}
	}
}
//////////////////////////////////////////////////////////////////////////
void FNiagaraOutlinerSystemInstanceDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FNiagaraOutlinerSystemInstanceDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildNum = 0; ChildNum < NumChildren; ++ChildNum)
	{
		TSharedPtr<IPropertyHandle> ChildProperty = StructPropertyHandle->GetChildHandle(ChildNum);

		if (ChildProperty->GetProperty()->GetFName() != GET_MEMBER_NAME_CHECKED(FNiagaraOutlinerSystemInstanceData, Emitters)
		&& ChildProperty->GetProperty()->GetFName() != GET_MEMBER_NAME_CHECKED(FNiagaraOutlinerSystemInstanceData, ComponentName))
		{
			ChildBuilder.AddProperty(ChildProperty.ToSharedRef());
		}
	}
}
//////////////////////////////////////////////////////////////////////////
void FNiagaraOutlinerEmitterInstanceDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FNiagaraOutlinerEmitterInstanceDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildNum = 0; ChildNum < NumChildren; ++ChildNum)
	{
		TSharedPtr<IPropertyHandle> ChildProperty = StructPropertyHandle->GetChildHandle(ChildNum);

		ChildBuilder.AddProperty(ChildProperty.ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_NIAGARA_DEBUGGER

