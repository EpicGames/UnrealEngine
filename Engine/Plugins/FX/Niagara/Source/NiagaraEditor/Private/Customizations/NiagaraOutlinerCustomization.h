// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "NiagaraCommon.h"
#include "EdGraph/EdGraphSchema.h"
#include "Layout/Visibility.h"
#include "NiagaraDebuggerCommon.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/StructOnScope.h"

#include "NiagaraOutlinerCustomization.generated.h"

class FDetailWidgetRow;
enum class ECheckBoxState : uint8;
class FNiagaraDebugger;
class ITableRow;
class SNiagaraOutlinerTree;
class SNiagaraOutlinerTreeItem;
class SSearchBox;
class STableViewBase;
class IStructureDetailsView;

template<typename ItemType> class STreeView;

#if WITH_NIAGARA_DEBUGGER

//////////////////////////////////////////////////////////////////////////

enum class ENiagaraOutlinerSystemExpansionState
{
	Collapsed,
	Expanded,
};

enum class ENiagaraOutlinerTreeItemType
{
	World,
	System,
	Component,
	Emitter,
	Num,
};

struct FNiagaraOutlinerTreeItem : public TSharedFromThis<FNiagaraOutlinerTreeItem>
{
	FNiagaraOutlinerTreeItem()
	{}

	virtual ~FNiagaraOutlinerTreeItem()= default;

	virtual FString GetShortName() const
	{
		FString Ret = GetFullName();
		int32 LastDot;
		if (Ret.FindLastChar(TEXT('.'), LastDot))
		{
			Ret.RightChopInline(LastDot+1);
		}
		return Ret;
	}

	virtual FString GetFullName() const
	{
		return Name;
	}

	FText GetShortNameText(){ return FText::FromString(GetShortName());}
	FText GetFullNameText() { return FText::FromString(GetFullName()); }

	const TSharedPtr<FNiagaraOutlinerTreeItem>& GetParent()const{ return Parent; }

	virtual TArray<TSharedRef<FNiagaraOutlinerTreeItem>>& GetChildren(){ return Children; }


	virtual ENiagaraOutlinerTreeItemType GetType() const { return ENiagaraOutlinerTreeItemType::Num; }
	virtual const void* GetData() const { return nullptr; }

	virtual TSharedRef<SWidget> GetHeaderWidget();

	virtual void SortChildren(){};

	TSharedPtr<FStructOnScope>& GetDetailsViewContent();

	void RefreshWidget();

	TSharedPtr<FNiagaraOutlinerTreeItem> Parent;
	TArray<TSharedRef<FNiagaraOutlinerTreeItem>> Children;
	FString Name;
	ENiagaraOutlinerSystemExpansionState Expansion = ENiagaraOutlinerSystemExpansionState::Collapsed;
	
	bool bVisible = true;
	bool bAnyChildrenVisible = false;
	bool bMatchesSearch = false;

	TWeakPtr<SNiagaraOutlinerTreeItem> Widget;
	TSharedPtr<FStructOnScope> DetailsViewData;

	TWeakPtr<SNiagaraOutlinerTree> OwnerTree;

	static const float HeaderPadding;
};

struct FNiagaraOutlinerTreeWorldItem : public FNiagaraOutlinerTreeItem
{
	virtual ENiagaraOutlinerTreeItemType GetType() const override { return ENiagaraOutlinerTreeItemType::World; }
	virtual const void* GetData() const override;
	virtual TSharedRef<SWidget> GetHeaderWidget() override;
	virtual void SortChildren() override;
};

struct FNiagaraOutlinerTreeSystemItem : public FNiagaraOutlinerTreeItem
{
	virtual ENiagaraOutlinerTreeItemType GetType() const override { return ENiagaraOutlinerTreeItemType::System; }
	virtual const void* GetData() const override;
	virtual TSharedRef<SWidget> GetHeaderWidget() override;
	virtual void SortChildren() override;
};

struct FNiagaraOutlinerTreeComponentItem : public FNiagaraOutlinerTreeItem
{
	virtual ENiagaraOutlinerTreeItemType GetType() const override { return ENiagaraOutlinerTreeItemType::Component; }
	virtual const void* GetData() const override;
	virtual TSharedRef<SWidget> GetHeaderWidget() override;
	virtual void SortChildren() override;
	virtual FString GetShortName() const
	{
		//Remove everything but the component and actor name.
		FString Ret = GetFullName();

		TArray<FString> Split;
		Name.ParseIntoArray(Split, TEXT("."));

		int32 Num = Split.Num();
		if (Num > 1)
		{
			Ret = FString::Printf(TEXT("%s.%s"), *Split[Num-2], *Split[Num-1]);
		}
		else if (Num > 0)
		{
			Ret = Split.Last();
		}

		return Ret;
	}

	//Sim Cache Debugging Controls.
	FReply OpenSimCache();
	FReply CaputreSimCache();
};

struct FNiagaraOutlinerTreeEmitterItem : public FNiagaraOutlinerTreeItem
{
	virtual ENiagaraOutlinerTreeItemType GetType() const override { return ENiagaraOutlinerTreeItemType::Emitter; }
	virtual const void* GetData() const override;
	virtual TSharedRef<SWidget> GetHeaderWidget() override;
	virtual void SortChildren() override;

	virtual FString GetFullName() const override
	{
		return Name;
	}
};

// A widget representing all the Niagara outliner data
class SNiagaraOutlinerTree : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraOutlinerTree) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraDebugger> InDebugger);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)override;

	void CacheVisibility();

	// Handle the data changing under us.
	void HandleOutlinerDataChanged();

	void RequestRefresh() { bNeedsRefresh = true; }

	void ToggleItemExpansion(TSharedPtr<FNiagaraOutlinerTreeItem>& Item);

	const FText& GetSearchText()const {return SearchText;}

	TSharedPtr<FNiagaraDebugger>& GetDebugger(){return Debugger;}

private:
	// Generate a row for the tree
	TSharedRef<ITableRow> OnGenerateRow(TSharedRef<FNiagaraOutlinerTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	// Get the children of an item
	void OnGetChildren(TSharedRef<FNiagaraOutlinerTreeItem> InItem, TArray<TSharedRef<FNiagaraOutlinerTreeItem>>& OutChildren);
	
	void HandleExpansionChanged(TSharedRef<FNiagaraOutlinerTreeItem> InItem, bool bExpanded);

	void HandleSelectionChanged(TSharedPtr<FNiagaraOutlinerTreeItem> SelectedItem, ESelectInfo::Type SelectInfo);

	// Refresh the tree.
	void RefreshTree();

	// Recursive helper refresh the tree.
	void RefreshTree_Helper(const TSharedPtr<FNiagaraOutlinerTreeItem>& InTreeEntry);
	
	template<typename ChildItemType>
	TSharedPtr<FNiagaraOutlinerTreeItem> AddChildItemToEntry(TArray<TSharedRef<FNiagaraOutlinerTreeItem>>& ExistingEntries, const TSharedPtr<FNiagaraOutlinerTreeItem>& InItem, FString ChildName, bool DefaultVisibility, ENiagaraOutlinerSystemExpansionState DefaultExpansion);

	FString OutlinerItemToStringDebug(TSharedRef<FNiagaraOutlinerTreeItem> Item);

	// The search widget
	TSharedPtr<SSearchBox> SearchBox;

	// The list view widget
	TSharedPtr<STreeView<TSharedRef<FNiagaraOutlinerTreeItem>>> TreeView;

	// Text we are searching for
	FText SearchText;

	// Root world items for the tree.
	TArray<TSharedRef<FNiagaraOutlinerTreeItem>> RootEntries;

	TSharedPtr<FNiagaraDebugger> Debugger;

	TSharedPtr<IStructureDetailsView> SelectedItemDetails;

	bool bNeedsRefresh = false;
};

//////////////////////////////////////////////////////////////////////////

class FNiagaraOutlinerWorldDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraOutlinerWorldDetailsCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** IPropertyTypeCustomization interface end */
};

class FNiagaraOutlinerSystemDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraOutlinerSystemDetailsCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** IPropertyTypeCustomization interface end */
};

class FNiagaraOutlinerSystemInstanceDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraOutlinerSystemInstanceDetailsCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** IPropertyTypeCustomization interface end */
};

class FNiagaraOutlinerEmitterInstanceDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraOutlinerEmitterInstanceDetailsCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** IPropertyTypeCustomization interface end */
};

#endif // WITH_NIAGARA_DEBUGGER

/** Due to limitations of the structure details view, we need to wrap up structs we wish to customize. */
USTRUCT()
struct FNiagaraOutlinerWorldDataCustomizationWrapper
{
	GENERATED_BODY()
	UPROPERTY(VisibleAnywhere, Category="World")
	FNiagaraOutlinerWorldData Data;
};
USTRUCT()
struct FNiagaraOutlinerSystemDataCustomizationWrapper
{
	GENERATED_BODY()
	UPROPERTY(VisibleAnywhere, Category = "System")
	FNiagaraOutlinerSystemData Data;
};
USTRUCT()
struct FNiagaraOutlinerSystemInstanceDataCustomizationWrapper
{
	GENERATED_BODY()
	UPROPERTY(VisibleAnywhere, Category = "Instance")
	FNiagaraOutlinerSystemInstanceData Data;
};
USTRUCT()
struct FNiagaraOutlinerEmitterInstanceDataCustomizationWrapper
{
	GENERATED_BODY()
	UPROPERTY(VisibleAnywhere, Category="Emitter")
	FNiagaraOutlinerEmitterInstanceData Data;
};
//////////////////////////////////////////////////////////////////////////
