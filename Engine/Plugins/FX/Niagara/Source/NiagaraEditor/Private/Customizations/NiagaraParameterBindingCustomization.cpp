// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterBindingCustomization.h"

#include "Customizations/NiagaraParameterBindingAdapterCustomization.h"
#include "NiagaraParameterBinding.h"

namespace NiagaraParameterBindingAdapterCustomizationPrivate
{
	class FAdapter : public FNiagaraParameterBindingAdapter
	{
	public:
		explicit FAdapter(uint8* ObjectAddress)
		{
			check(ObjectAddress != nullptr);
			ParameterBinding = reinterpret_cast<FNiagaraParameterBinding*>(ObjectAddress);
		}

		virtual bool IsSetToParameter() const override
		{
			return !ParameterBinding->HasDefaultValueEditorOnly() || ParameterBinding->ResolvedParameter.GetName().IsNone() == false;
		}

		virtual bool AllowConstantValue() const override
		{
			return ParameterBinding->HasDefaultValueEditorOnly();
		}

		virtual FNiagaraTypeDefinition GetConstantTypeDef() const override
		{
			return ParameterBinding->GetDefaultResolvedParameter().GetType();
		}

		virtual TConstArrayView<uint8> GetConstantValue() const override
		{
			return ParameterBinding->GetDefaultValueEditorOnly();
		}

		virtual void SetConstantValue(TConstArrayView<uint8> Memory) const override
		{
			ParameterBinding->SetDefaultValueEditorOnly(Memory);
		}

		virtual const FNiagaraVariableBase& GetBoundParameter() const override
		{
			return ParameterBinding->AliasedParameter;
		}

		virtual void SetBoundParameter(const FInstancedStruct& Parameter) override
		{
			if ( Parameter.IsValid() )
			{
				const FNiagaraParameterBinding& BindingData = Parameter.Get<FNiagaraParameterBinding>();
				ParameterBinding->AliasedParameter = BindingData.AliasedParameter;
				ParameterBinding->ResolvedParameter = BindingData.ResolvedParameter;
			}
			else
			{
				ParameterBinding->AliasedParameter = FNiagaraVariableBase();
				ParameterBinding->ResolvedParameter = FNiagaraVariableBase();
			}
		}

		virtual bool IsSetToDefault() const override
		{
			return ParameterBinding->IsSetToDefault();
		}

		virtual void SetToDefault() override
		{
			ParameterBinding->SetToDefault();
		}

		virtual void CollectBindings(const FNiagaraVariableBase& AliasedVariable, const FNiagaraVariableBase& ResolvedVariable, TArray<TPair<FName, FInstancedStruct>>& OutBindings) const override
		{
			TPair<FName, FInstancedStruct>& BindingData = OutBindings.AddDefaulted_GetRef();
			BindingData.Key = AliasedVariable.GetName();
			FNiagaraParameterBinding& Binding = BindingData.Value.InitializeAs<FNiagaraParameterBinding>();
			Binding.AliasedParameter = AliasedVariable;
			Binding.ResolvedParameter = ResolvedVariable;
		}

		virtual bool AllowUserParameters() const override { return ParameterBinding->AllowUserParameters(); }
		virtual bool AllowSystemParameters() const override { return ParameterBinding->AllowSystemParameters(); }
		virtual bool AllowEmitterParameters() const override { return ParameterBinding->AllowEmitterParameters(); }
		virtual bool AllowParticleParameters() const override { return ParameterBinding->AllowParticleParameters(); }
		virtual bool AllowStaticVariables() const override { return ParameterBinding->AllowStaticVariables(); }

		virtual TConstArrayView<UClass*> GetAllowedDataInterfaces() const override { return ParameterBinding->GetAllowedDataInterfaces(); }
		virtual TConstArrayView<UClass*> GetAllowedObjects() const override { return ParameterBinding->GetAllowedObjects(); }
		virtual TConstArrayView<UClass*> GetAllowedInterfaces() const override { return ParameterBinding->GetAllowedInterfaces(); }
		virtual TConstArrayView<FNiagaraTypeDefinition> GetAllowedTypeDefinitions() const override { return ParameterBinding->GetAllowedTypeDefinitions(); }

		FNiagaraParameterBinding* ParameterBinding = nullptr;
	};

	static TUniquePtr<FNiagaraParameterBindingAdapter> CreateAdapter(uint8* ObjectAddress)
	{
		return MakeUnique<FAdapter>(ObjectAddress);
	}
};

TSharedRef<IPropertyTypeCustomization> FNiagaraParameterBindingCustomization::MakeInstance()
{
	return MakeShared<FNiagaraParameterBindingAdapterCustomization>(&NiagaraParameterBindingAdapterCustomizationPrivate::CreateAdapter);
}
