// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

namespace FNiagaraParameterBindingCustomization
{
	TSharedRef<IPropertyTypeCustomization> MakeInstance();
};
