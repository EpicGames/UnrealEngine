// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomNodeBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ViewModels/NiagaraParameterCollectionViewModel.h"
#include "ViewModels/NiagaraParameterViewModel.h"
#include "NiagaraEditorStyle.h"
#include "IDetailChildrenBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "NiagaraEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "FNiagaraParameterCollectionCustomNodeBuilder"

class FNiagaraParameterCollectionCustomNodeBuilder : public IDetailCustomNodeBuilder
{
public:
	FNiagaraParameterCollectionCustomNodeBuilder(TSharedRef<INiagaraParameterCollectionViewModel> InViewModel, bool bInAllowMetaData = true)
		: ViewModel(InViewModel), bAllowMetaData(bInAllowMetaData)
	{
		ViewModel->OnCollectionChanged().AddRaw(this, &FNiagaraParameterCollectionCustomNodeBuilder::OnCollectionViewModelChanged);
	}

	~FNiagaraParameterCollectionCustomNodeBuilder()
	{
		ViewModel->OnCollectionChanged().RemoveAll(this);
	}

	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override
	{
		OnRebuildChildren = InOnRegenerateChildren;
	}

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) {}
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual bool InitiallyCollapsed() const { return false; }
	virtual FName GetName() const  override
	{
		static const FName NiagaraCustomNodeBuilder("NiagaraCustomNodeBuilder");
		return NiagaraCustomNodeBuilder;
	}
	
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override
	{
		const TArray<TSharedRef<INiagaraParameterViewModel>>& Parameters = ViewModel->GetParameters();

		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

		for (const TSharedRef<INiagaraParameterViewModel>& Parameter : Parameters)
		{
			TSharedPtr<SWidget> NameWidget;

			if (Parameter->CanRenameParameter())
			{
				NameWidget =
					SNew(SInlineEditableTextBlock)
					.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterInlineEditableText")
					.Text(Parameter, &INiagaraParameterViewModel::GetNameText)
					.MaximumLength(NAME_SIZE-1)
					.OnVerifyTextChanged(Parameter, &INiagaraParameterViewModel::VerifyNodeNameTextChanged)
					.OnTextCommitted(Parameter, &INiagaraParameterViewModel::NameTextCommitted);
			}
			else
			{
				NameWidget =
					SNew(STextBlock)
					.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
					.Text(Parameter, &INiagaraParameterViewModel::GetNameText);
			}



			IDetailPropertyRow* Row = nullptr;

			TSharedPtr<SWidget> CustomValueWidget;
			bool bCustomize = true;

			switch (Parameter->GetDefaultValueType())
			{
				case INiagaraParameterViewModel::EDefaultValueType::Struct:
					Row = ChildrenBuilder.AddExternalStructureProperty(Parameter->GetDefaultValueStruct(), NAME_None, FAddPropertyParams().UniqueId(Parameter->GetName()));
					break;

				case INiagaraParameterViewModel::EDefaultValueType::DataInterface:
				{
					UNiagaraDataInterface* DefaultValueObject = Parameter->GetDefaultValueDataInterface();

					if (DefaultValueObject != nullptr)
					{
						TArray<UObject*> Objects;
						Objects.Add(DefaultValueObject);

						FAddPropertyParams Params = FAddPropertyParams()
							.UniqueId(Parameter->GetName())
							.AllowChildren(true)
							.CreateCategoryNodes(false);

						Row = ChildrenBuilder.AddExternalObjectProperty(Objects, NAME_None, Params);
						CustomValueWidget =
							SNew(STextBlock)
							.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
							.Text(FText::FromString(FName::NameToDisplayString(DefaultValueObject->GetClass()->GetName(), false)));
					}
					else
					{
						ChildrenBuilder.AddCustomRow(FText())
							.NameContent()
							[
								NameWidget.ToSharedRef()
							]
							.ValueContent()
							[
								SNew(STextBlock)
								.Text(NSLOCTEXT("NiagaraParameterCollectionCustomNodeBuilder", "NullObjectValue", "(null)"))
							];
					}
					break;
				}

				case INiagaraParameterViewModel::EDefaultValueType::ObjectAsset:
				{
					ChildrenBuilder.AddCustomRow(FText())
						.NameContent()
						[
							NameWidget.ToSharedRef()
						]
						.ValueContent()
						[
							SNew(SObjectPropertyEntryBox)
								.AllowedClass(Parameter->GetType().GetClass())
								.ObjectPath_Lambda(
									[Parameter]() -> FString
									{
										TObjectPtr<UObject> DefaultValueObject = Parameter->GetDefaultValueObjectAsset();
										return DefaultValueObject ? DefaultValueObject.GetPathName() : FString();
									}
								)
								.DisplayBrowse(true)
								.DisplayUseSelected(true)
								.DisplayThumbnail(true)
								.EnableContentPicker(true)
								.OnObjectChanged_Lambda(
									[Parameter](const FAssetData& AssetData)
									{
										Parameter->SetDefaultValueObjectAsset(AssetData.GetAsset());
									}
								)
						];
					break;
				}

				default:
					checkNoEntry();
					break;
			}

			if (Row)
			{

				TSharedPtr<IPropertyHandle> PropertyHandle = Row->GetPropertyHandle();
				PropertyHandle->SetOnPropertyValueChanged(
					FSimpleDelegate::CreateSP(Parameter, &INiagaraParameterViewModel::NotifyDefaultValueChanged));
				PropertyHandle->SetOnChildPropertyValueChanged(
					FSimpleDelegate::CreateSP(Parameter, &INiagaraParameterViewModel::NotifyDefaultValueChanged));

				if (bCustomize)
				{
					TSharedPtr<SWidget> DefaultNameWidget;
					TSharedPtr<SWidget> DefaultValueWidget;
					FDetailWidgetRow& CustomWidget = Row->CustomWidget(true);
					
					Row->GetDefaultWidgets(DefaultNameWidget, DefaultValueWidget, CustomWidget);
					CustomWidget
						.NameContent()
						[
							SNew(SBox)
							.Padding(FMargin(0.0f, 2.0f))
							[
								NameWidget.ToSharedRef()
							]
						];

					if (CustomValueWidget.IsValid())
					{
						CustomWidget.ValueContent()
							[
								CustomValueWidget.ToSharedRef()
							];
					}

					CustomWidget.ExtensionContent()
						[
							SNew(SButton)
							.OnClicked_Raw(this, &FNiagaraParameterCollectionCustomNodeBuilder::DeleteParameter, Parameter->GetVariable())
							.Content()
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Icons.Delete"))
							]
						];
				}
			}
		}
	}

private:
	FReply DeleteParameter(FNiagaraVariable Parameter)
	{
		ViewModel->DeleteParameters({Parameter});
		return FReply::Handled();
	}
	
	void OnCollectionViewModelChanged()
	{
		OnRebuildChildren.ExecuteIfBound();
	}

private:
	TSharedRef<INiagaraParameterCollectionViewModel> ViewModel;
	FSimpleDelegate OnRebuildChildren;
	bool bAllowMetaData;
};

#undef LOCTEXT_NAMESPACE
