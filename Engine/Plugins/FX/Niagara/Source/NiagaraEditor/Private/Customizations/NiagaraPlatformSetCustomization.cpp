// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraPlatformSetCustomization.h"
#include "DetailWidgetRow.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Engine/GameViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "Layout/Visibility.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEmitter.h"
#include "NiagaraPlatformSet.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "NiagaraTypes.h"
#include "PlatformInfo.h"
#include "PropertyHandle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "FNiagaraPlatformSetCustomization"


static TSharedPtr<IPropertyHandle> FindChildPlatformSet(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (FStructProperty* Property = CastField<FStructProperty>(PropertyHandle->GetProperty()))
	{
		if (Property->Struct == FNiagaraPlatformSet::StaticStruct())
		{
			return PropertyHandle;
		}
	}

	// recurse
	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	for (int32 Idx = 0; Idx < static_cast<int32>(NumChildren); ++Idx)
	{
		TSharedPtr<IPropertyHandle> Child = PropertyHandle->GetChildHandle(Idx);

		TSharedPtr<IPropertyHandle> ChildResult = FindChildPlatformSet(Child);
		if (ChildResult.IsValid())
		{
			return ChildResult;
		}
	}

	return TSharedPtr<IPropertyHandle>();
}

uint8* GetBaseAddress(TSharedRef<IPropertyHandle> PropertyHandle)
{
	if (PropertyHandle->GetNumOuterObjects() > 0)
	{
		TArray<UObject*> Objects;
		PropertyHandle->GetOuterObjects(Objects);
		return reinterpret_cast<uint8*>(Objects[0]);
	}

	// walk the struct hierarchy to find the parent StructOnScope
	TSharedPtr<IPropertyHandle> LastParent = PropertyHandle;
	while (true)
	{
		TSharedPtr<IPropertyHandle> NewParent = LastParent->GetParentHandle();
		if (!ensureMsgf(NewParent.IsValid() && NewParent != LastParent, TEXT("Unable to walk property chain")))
		{
			break;
		}
		
		TSharedPtr<IPropertyHandleStruct> StructHandle = NewParent->AsStruct();
		if (StructHandle.IsValid())
		{
			return StructHandle->GetStructData()->GetStructMemory();
		}
		LastParent = NewParent;
	}

	return nullptr;
}

void FNiagaraPlatformSetCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	TArray<TWeakObjectPtr<UObject>> WeakObjectPtrs = CustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();

	uint8* BaseAddress = GetBaseAddress(InPropertyHandle);
	if (!BaseAddress)
	{
		return;
	}
	
	TargetPlatformSet = reinterpret_cast<FNiagaraPlatformSet*>(PropertyHandle->GetValueBaseAddress(BaseAddress));

	if (PlatformSelectionStates.Num() == 0)
	{
		PlatformSelectionStates.Add(MakeShared<ENiagaraPlatformSelectionState>(ENiagaraPlatformSelectionState::Enabled));
		PlatformSelectionStates.Add(MakeShared<ENiagaraPlatformSelectionState>(ENiagaraPlatformSelectionState::Disabled));
		PlatformSelectionStates.Add(MakeShared<ENiagaraPlatformSelectionState>(ENiagaraPlatformSelectionState::Default));
	}

	//Look for outer types for which we need to ensure there are no conflicting settings.
	SystemScalabilitySettings = nullptr;
	EmitterScalabilitySettings = nullptr;
	SystemScalabilityOverrides = nullptr;
	EmitterScalabilityOverrides = nullptr;
	PlatformSetArray.Reset();
	PlatformSetArrayIndex = INDEX_NONE;

	//Look whether this platform set belongs to a class which must keep orthogonal platform sets.
	//We then interrogate the other sets via these ptrs to look for conflicts.
	TSharedPtr<IPropertyHandle> CurrHandle = PropertyHandle->GetParentHandle();
	while (CurrHandle)
	{
		int32 ThisIndex = CurrHandle->GetIndexInArray();
		if (ThisIndex != INDEX_NONE)
		{
			PlatformSetArrayProperty = CurrHandle->GetParentHandle();
			PlatformSetArray = PlatformSetArrayProperty->AsArray();
			PlatformSetArrayIndex = ThisIndex;
		}
		else
		{
			PlatformSetArrayProperty.Reset();
			PlatformSetArray.Reset();
			PlatformSetArrayIndex = INDEX_NONE;
		}

		if (FProperty* CurrProperty = CurrHandle->GetProperty())
		{
			if (UStruct* CurrStruct = CurrProperty->GetOwnerStruct())
			{
				TSharedPtr<IPropertyHandle> ParentHandle = CurrHandle->GetParentHandle();
				if (CurrStruct == FNiagaraSystemScalabilitySettingsArray::StaticStruct())
				{
					SystemScalabilitySettings = reinterpret_cast<FNiagaraSystemScalabilitySettingsArray*>(ParentHandle->GetValueBaseAddress(BaseAddress));
					break;
				}
				else if (CurrStruct == FNiagaraEmitterScalabilitySettingsArray::StaticStruct())
				{
					EmitterScalabilitySettings = reinterpret_cast<FNiagaraEmitterScalabilitySettingsArray*>(ParentHandle->GetValueBaseAddress(BaseAddress));
					break;
				}
				else if (CurrStruct == FNiagaraSystemScalabilityOverrides::StaticStruct())
				{
					SystemScalabilityOverrides = reinterpret_cast<FNiagaraSystemScalabilityOverrides*>(ParentHandle->GetValueBaseAddress(BaseAddress));
					break;
				}
				else if (CurrStruct == FNiagaraEmitterScalabilityOverrides::StaticStruct())
				{
					EmitterScalabilityOverrides = reinterpret_cast<FNiagaraEmitterScalabilityOverrides*>(ParentHandle->GetValueBaseAddress(BaseAddress));
					break;
				}
				else if (PlatformSetArray.IsValid() && PlatformSetArrayProperty->HasMetaData("NiagaraOrthogonalPlatformSets"))
				{
					break;
				}
			}
		}

		CurrHandle = CurrHandle->GetParentHandle();
	}

	UpdateCachedConflicts();
	
	PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FNiagaraPlatformSetCustomization::OnPropertyValueChanged));
	PropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda([&]{ TargetPlatformSet->OnChanged(); }));

	HeaderRow
		.WholeRowContent()
		[
			SAssignNew(QualityLevelWidgetBox, SWrapBox)
			.UseAllottedSize(true)
		];

	GenerateQualityLevelSelectionWidgets();
}

void FNiagaraPlatformSetCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> CVarConditionsHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraPlatformSet, CVarConditions));
	ChildBuilder.AddProperty(CVarConditionsHandle.ToSharedRef());
}

FText FNiagaraPlatformSetCustomization::GetCurrentText() const
{
	return LOCTEXT("Platforms", "Platforms");
}

FText FNiagaraPlatformSetCustomization::GetTooltipText() const
{
	return LOCTEXT("Platforms", "Platforms");
}

void FNiagaraPlatformSetCustomization::GenerateQualityLevelSelectionWidgets()
{
	QualityLevelWidgetBox->ClearChildren();

	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	int32 NumQualityLevels = Settings->QualityLevels.Num();
	
	QualityLevelMenuAnchors.Reset();
	QualityLevelMenuContents.Reset();
	QualityLevelMenuAnchors.SetNum(NumQualityLevels);
	QualityLevelMenuContents.SetNum(NumQualityLevels);

	for (int32 QualityLevel = 0; QualityLevel < NumQualityLevels; ++QualityLevel)
	{
		bool First = QualityLevel == 0;
		bool Last = QualityLevel == (NumQualityLevels - 1);

		if (!QualityLevelMenuAnchors[QualityLevel].IsValid())
		{
			QualityLevelMenuAnchors[QualityLevel] = SNew(SMenuAnchor)
				.ToolTipText(LOCTEXT("AddPlatformOverride", "Add an override for a specific platform."))
				.OnGetMenuContent(this, &FNiagaraPlatformSetCustomization::GenerateDeviceProfileTreeWidget, QualityLevel)
				[
					SNew(SButton)
					.Visibility_Lambda([this, QualityLevel]()
					{
						return QualityLevelWidgetBox->GetChildren()->GetChildAt(QualityLevel)->IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
					})
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ForegroundColor(FSlateColor::UseForeground())
					.OnClicked(this, &FNiagaraPlatformSetCustomization::ToggleMenuOpenForQualityLevel, QualityLevel)
					[
						SNew(SBox)
						.WidthOverride(8)
						.HeightOverride(8)
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.PlatformSet.DropdownButton"))
						]
					]
				];
		}

		QualityLevelWidgetBox->AddSlot()
			.Padding(0, 0, 1, 0)
			[
				SNew(SBox)
				.WidthOverride(100)
				.VAlign(VAlign_Top)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SCheckBox)
						.Style(FNiagaraEditorStyle::Get(), First ? "NiagaraEditor.PlatformSet.StartButton" : 
							(Last ? "NiagaraEditor.PlatformSet.EndButton" : "NiagaraEditor.PlatformSet.MiddleButton"))
						.IsChecked(this, &FNiagaraPlatformSetCustomization::IsQLChecked, QualityLevel)
						.OnCheckStateChanged(this, &FNiagaraPlatformSetCustomization::QLCheckStateChanged, QualityLevel)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Fill)
							.Padding(6,2,2,4)
							[
								SNew(STextBlock)
								.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.PlatformSet.ButtonText")
								.Text(FNiagaraPlatformSet::GetQualityLevelText(QualityLevel))
								.ColorAndOpacity(this, &FNiagaraPlatformSetCustomization::GetQualityLevelButtonTextColor, QualityLevel)
								.ShadowOffset(FVector2D(1, 1))
							]
							// error icon
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							.Padding(0,0,2,0)
							[
								SNew(SBox)
								.WidthOverride(12)
								.HeightOverride(12)
								[
									SNew(SImage)
									.ToolTipText(this, &FNiagaraPlatformSetCustomization::GetQualityLevelErrorToolTip, QualityLevel)
									.Visibility(this, &FNiagaraPlatformSetCustomization::GetQualityLevelErrorVisibility, QualityLevel)
									.Image(FAppStyle::GetBrush("Icons.Error"))
								]
							]
							// dropdown button
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							.Padding(0,0,2,0)
							[
								QualityLevelMenuAnchors[QualityLevel].ToSharedRef()
							]
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						GenerateAdditionalDevicesWidgetForQL(QualityLevel)
					]
				]
			];
	}
}

void FNiagaraPlatformSetCustomization::UpdateCachedConflicts()
{
	TArray<const FNiagaraPlatformSet*> PlatformSets;
	if (SystemScalabilityOverrides != nullptr)
	{
		for (FNiagaraSystemScalabilityOverride& Override : SystemScalabilityOverrides->Overrides)
		{
			PlatformSets.Add(&Override.Platforms);
		}
	}
	else if (EmitterScalabilityOverrides != nullptr)
	{
		for (FNiagaraEmitterScalabilityOverride& Override : EmitterScalabilityOverrides->Overrides)
		{
			PlatformSets.Add(&Override.Platforms);
		}
	}
	else if (SystemScalabilitySettings != nullptr)
	{
		for (FNiagaraSystemScalabilitySettings& Settings : SystemScalabilitySettings->Settings)
		{
			PlatformSets.Add(&Settings.Platforms);
		}
	}
	else if (EmitterScalabilitySettings != nullptr)
	{
		for (FNiagaraEmitterScalabilitySettings& Settings : EmitterScalabilitySettings->Settings)
		{
			PlatformSets.Add(&Settings.Platforms);
		}
	}
	else if (PlatformSetArray.IsValid())
	{
		//Try to handle the general case.
		FName PlatformSetPropertyName = PropertyHandle->GetProperty()->GetFName();

		auto AddPlatformSetProperty = [&PlatformSets](TSharedPtr<IPropertyHandle> Handle)
			{
				FProperty* Prop = Handle->GetProperty();
				if (const FStructProperty* StructProp = CastField<FStructProperty>(Prop))
				{
					if (StructProp->Struct == FNiagaraPlatformSet::StaticStruct())
					{
						TArray<void*> Raw;
						Handle->AccessRawData(Raw);
						if (Raw.Num() == 1 && Raw[0] != nullptr)
						{
							if (FNiagaraPlatformSet* FoundPlatformSet = reinterpret_cast<FNiagaraPlatformSet*>(Raw[0]))
							{
								PlatformSets.Add(FoundPlatformSet);
							}
						}
					}
				}
			};

		uint32 NumElements = 0;
		PlatformSetArray->GetNumElements(NumElements);
		for (uint32 i = 0; i < NumElements; ++i)
		{
			TSharedPtr<IPropertyHandle> ElemProperty = PlatformSetArray->GetElement(i);
			if (TSharedPtr<IPropertyHandle> ElemPlatformSetPropertyHandle = FindChildPlatformSet(ElemProperty))
			{
				AddPlatformSetProperty(ElemPlatformSetPropertyHandle);
			}
		}
	}

	CachedConflicts.Reset();
	FNiagaraPlatformSet::GatherConflicts(PlatformSets, CachedConflicts);
}

void FNiagaraPlatformSetCustomization::InvalidateSiblingConflicts() const
{
	if (!PlatformSetArray.IsValid())
	{
		return;
	}

	uint32 ArrayCount = 0;
	PlatformSetArray->GetNumElements(ArrayCount);
	for (int32 Idx = 0; Idx < static_cast<int32>(ArrayCount); ++Idx)
	{
		if (Idx == PlatformSetArrayIndex)
		{
			continue; // skip self
		}

		TSharedRef<IPropertyHandle> Sibling = PlatformSetArray->GetElement(Idx);

		TSharedPtr<IPropertyHandle> SiblingPlatformSet = FindChildPlatformSet(Sibling);
		if (SiblingPlatformSet.IsValid())
		{
			SiblingPlatformSet->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}
}

EVisibility FNiagaraPlatformSetCustomization::GetQualityLevelErrorVisibility(int32 QualityLevel) const
{
	// not part of an array
	if (PlatformSetArrayIndex == INDEX_NONE)
	{
		return EVisibility::Collapsed;
	}

	for (const FNiagaraPlatformSetConflictInfo& ConflictInfo : CachedConflicts)
	{
		if (ConflictInfo.SetAIndex == PlatformSetArrayIndex || 
			ConflictInfo.SetBIndex == PlatformSetArrayIndex)
		{
			// this conflict applies to this platform set, check if it applies to this quality level button
			const int32 QLMask = FNiagaraPlatformSet::CreateQualityLevelMask(QualityLevel);

			for (const FNiagaraPlatformSetConflictEntry& Conflict : ConflictInfo.Conflicts)
			{
				if ((QLMask & Conflict.QualityLevelMask) != 0)
				{
					return EVisibility::Visible;
				}
			}
		}
	}

	return EVisibility::Collapsed;
}

FText FNiagaraPlatformSetCustomization::GetQualityLevelErrorToolTip(int32 QualityLevel) const
{
	if (PlatformSetArrayIndex == INDEX_NONE)
	{
		return FText::GetEmpty();
	}

	for (const FNiagaraPlatformSetConflictInfo& ConflictInfo : CachedConflicts)
	{
		int32 OtherIndex = INDEX_NONE;

		if (ConflictInfo.SetAIndex == PlatformSetArrayIndex)
		{
			OtherIndex = ConflictInfo.SetBIndex;
		}

		if (ConflictInfo.SetBIndex == PlatformSetArrayIndex)
		{
			OtherIndex = ConflictInfo.SetAIndex;
		}

		if (OtherIndex != INDEX_NONE)
		{
			const int32 QLMask = FNiagaraPlatformSet::CreateQualityLevelMask(QualityLevel);

			for (const FNiagaraPlatformSetConflictEntry& Conflict : ConflictInfo.Conflicts)
			{
				if ((QLMask & Conflict.QualityLevelMask) != 0)
				{
					FText FormatString = LOCTEXT("QualityLevelConflictToolTip", "This effect quality conflicts with the set at index {Index} in this array."); 

					FFormatNamedArguments Args;
					Args.Add(TEXT("Index"), OtherIndex);

					return FText::Format(FormatString, Args);
				}
			}
		}
	}

	return FText::GetEmpty();
}

FSlateColor FNiagaraPlatformSetCustomization::GetQualityLevelButtonTextColor(int32 QualityLevel) const
{
	return TargetPlatformSet->IsEffectQualityEnabled(QualityLevel) ? 
		FSlateColor(FLinearColor(0.95f, 0.95f, 0.95f)) :
		FSlateColor::UseForeground();
}

TSharedRef<SWidget> FNiagaraPlatformSetCustomization::GenerateAdditionalDevicesWidgetForQL(int32 QualityLevel)
{
	TSharedRef<SVerticalBox> Container = SNew(SVerticalBox);

	auto AddDeviceProfileOverrideWidget = [&](UDeviceProfile* Profile, bool bEnabled)
	{
		TSharedPtr<SHorizontalBox> DeviceBox;

		Container->AddSlot()
			.AutoHeight()
			[
				SAssignNew(DeviceBox, SHorizontalBox)
			];

		const FText DeviceNameText = FText::FromName(Profile->GetFName());

		DeviceBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(3, 0, 3, 0)
			[
				SNew(SBox)
				.WidthOverride(8)
				.HeightOverride(8)
				.ToolTipText(bEnabled ?
					LOCTEXT("DeviceIncludedToolTip", "This device is included.") :
					LOCTEXT("DeviceExcludedToolTip", "This device is excluded."))
				[
					SNew(SImage)
					.Image(bEnabled ?
						FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.PlatformSet.Include") :
						FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.PlatformSet.Exclude"))
					.ColorAndOpacity(bEnabled ?
						FLinearColor(0, 1, 0) :
						FLinearColor(1, 0, 0))
					]
				];

		DeviceBox->AddSlot()
			.HAlign(HAlign_Fill)
			.Padding(0, 0, 2, 0)
			[
				SNew(STextBlock)
				.Text(DeviceNameText)
				.ToolTipText(DeviceNameText)
			];

		DeviceBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(0, 0, 2, 0)
			[
				SNew(SBox)
				.WidthOverride(12)
				.HeightOverride(12)
				[
					SNew(SImage)
					.ToolTipText(this, &FNiagaraPlatformSetCustomization::GetDeviceProfileErrorToolTip, Profile, QualityLevel)
					.Visibility(this, &FNiagaraPlatformSetCustomization::GetDeviceProfileErrorVisibility, Profile, QualityLevel)
					.Image(FAppStyle::GetBrush("Icons.Error"))
				]
			];

		DeviceBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(0, 0, 2, 0)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FSlateColor::UseForeground())
				.OnClicked(this, &FNiagaraPlatformSetCustomization::RemoveDeviceProfile, Profile, QualityLevel)
				.ToolTipText(LOCTEXT("RemoveDevice", "Remove this device override."))
				[
					SNew(SBox)
					.WidthOverride(8)
					.HeightOverride(8)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.PlatformSet.Remove"))
					]
				]
			];
		};

	TArray<UDeviceProfile*> EnabledProfiles;
	TArray<UDeviceProfile*> DisabledProfiles;
	TargetPlatformSet->GetOverridenDeviceProfiles(QualityLevel, EnabledProfiles, DisabledProfiles);

	for (UDeviceProfile* Profile : EnabledProfiles)
	{
		AddDeviceProfileOverrideWidget(Profile, true);
	}

	for (UDeviceProfile* Profile : DisabledProfiles)
	{
		AddDeviceProfileOverrideWidget(Profile, false);
	}

	return Container;
}

FReply FNiagaraPlatformSetCustomization::RemoveDeviceProfile(UDeviceProfile* Profile, int32 QualityLevel)
{
	PropertyHandle->NotifyPreChange();
	TargetPlatformSet->SetDeviceProfileState(Profile, QualityLevel, ENiagaraPlatformSelectionState::Default);
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	InvalidateSiblingConflicts();

	return FReply::Handled();
}

EVisibility FNiagaraPlatformSetCustomization::GetDeviceProfileErrorVisibility(UDeviceProfile* Profile, int32 QualityLevel) const
{
	// not part of an array
	if (PlatformSetArrayIndex == INDEX_NONE)
	{
		return EVisibility::Collapsed;
	}

	for (const FNiagaraPlatformSetConflictInfo& ConflictInfo : CachedConflicts)
	{
		if (ConflictInfo.SetAIndex == PlatformSetArrayIndex || 
			ConflictInfo.SetBIndex == PlatformSetArrayIndex)
		{
			const int32 QLMask = FNiagaraPlatformSet::CreateQualityLevelMask(QualityLevel);

			for (const FNiagaraPlatformSetConflictEntry& Entry : ConflictInfo.Conflicts)
			{
				if ((Entry.QualityLevelMask & QLMask) != 0 &&
					Entry.ProfileName == Profile->GetFName())
				{
					return EVisibility::Visible;
				}
			}
		}
	}

	return EVisibility::Collapsed;
}

FText FNiagaraPlatformSetCustomization::GetDeviceProfileErrorToolTip(UDeviceProfile* Profile, int32 QualityLevel) const
{
	for (const FNiagaraPlatformSetConflictInfo& ConflictInfo : CachedConflicts)
	{
		int32 OtherIndex = INDEX_NONE;

		if (ConflictInfo.SetAIndex == PlatformSetArrayIndex)
		{
			OtherIndex = ConflictInfo.SetBIndex;
		}

		if (ConflictInfo.SetBIndex == PlatformSetArrayIndex)
		{
			OtherIndex = ConflictInfo.SetAIndex;
		}

		if (OtherIndex != INDEX_NONE)
		{
			const int32 QLMask = FNiagaraPlatformSet::CreateQualityLevelMask(QualityLevel);

			for (const FNiagaraPlatformSetConflictEntry& Entry : ConflictInfo.Conflicts)
			{
				if ((Entry.QualityLevelMask & QLMask) != 0 &&
					Entry.ProfileName == Profile->GetFName())
				{
					FText FormatString = LOCTEXT("PlatformOverrideConflictToolTip", "This platform override conflicts with the set at index {Index} in this array."); 

					FFormatNamedArguments Args;
					Args.Add(TEXT("Index"), OtherIndex);

					return FText::Format(FormatString, Args);
				}
			}
		}
	}

	return FText::GetEmpty();
}

FReply FNiagaraPlatformSetCustomization::ToggleMenuOpenForQualityLevel(int32 QualityLevel)
{
	check(QualityLevelMenuAnchors.IsValidIndex(QualityLevel));

	QualityLevelMenuContents[QualityLevel].Reset();
	GenerateDeviceProfileTreeWidget(QualityLevel);
	
	TSharedPtr<SMenuAnchor> MenuAnchor = QualityLevelMenuAnchors[QualityLevel];
	MenuAnchor->SetIsOpen(!MenuAnchor->IsOpen());

	return FReply::Handled();
}

// Is or does a viewmodel contain any children active at the given quality?
bool FNiagaraPlatformSetCustomization::IsTreeActiveForQL(const TSharedPtr<FNiagaraDeviceProfileViewModel>& Tree, int32 QualityLevelMask) const
{
	if (TargetPlatformSet->CanConsiderDeviceProfile(Tree->Profile) == false)
	{
		return false;
	}

	int32 AvailableMask = TargetPlatformSet->GetAvailableQualityMaskForDeviceProfile(Tree->Profile);

	return (AvailableMask & QualityLevelMask) != 0;
}

void FNiagaraPlatformSetCustomization::FilterTreeForQL(const TSharedPtr<FNiagaraDeviceProfileViewModel>& SourceTree, TSharedPtr<FNiagaraDeviceProfileViewModel>& FilteredTree, int32 QualityLevelMask)
{
	for (const TSharedPtr<FNiagaraDeviceProfileViewModel>& SourceChild : SourceTree->Children)
	{
		if (IsTreeActiveForQL(SourceChild, QualityLevelMask))
		{
			TSharedPtr<FNiagaraDeviceProfileViewModel>& FilteredChild = FilteredTree->Children.Add_GetRef(MakeShared<FNiagaraDeviceProfileViewModel>());
			FilteredChild->Profile = SourceChild->Profile;

			FilterTreeForQL(SourceChild, FilteredChild, QualityLevelMask);
		}
	}
}

void FNiagaraPlatformSetCustomization::CreateDeviceProfileTree()
{
	//Pull device profiles out by their hierarchy depth.
	TArray<TArray<UDeviceProfile*>> DeviceProfilesByHierarchyLevel;
	for (UObject* Profile : UDeviceProfileManager::Get().Profiles)
	{
		UDeviceProfile* DeviceProfile = CastChecked<UDeviceProfile>(Profile);

		TFunction<void(int32&, UDeviceProfile*)> FindDepth;
		FindDepth = [&](int32& Depth, UDeviceProfile* CurrProfile)
		{
			if (CurrProfile->Parent)
			{
				FindDepth(++Depth, Cast<UDeviceProfile>(CurrProfile->Parent));
			}
		};

		int32 ProfileDepth = 0;
		FindDepth(ProfileDepth, DeviceProfile);
		DeviceProfilesByHierarchyLevel.SetNum(FMath::Max(ProfileDepth+1, DeviceProfilesByHierarchyLevel.Num()));
		DeviceProfilesByHierarchyLevel[ProfileDepth].Add(DeviceProfile);
	}
	
	FullDeviceProfileTree.Reset(DeviceProfilesByHierarchyLevel[0].Num());
	for (int32 RootProfileIdx = 0; RootProfileIdx < DeviceProfilesByHierarchyLevel[0].Num(); ++RootProfileIdx)
	{
		TFunction<void(FNiagaraDeviceProfileViewModel*, int32)> BuildProfileTree;
		BuildProfileTree = [&](FNiagaraDeviceProfileViewModel* CurrRoot, int32 CurrLevel)
		{
			int32 NextLevel = CurrLevel + 1;
			if (NextLevel < DeviceProfilesByHierarchyLevel.Num())
			{
				for (UDeviceProfile* PossibleChild : DeviceProfilesByHierarchyLevel[NextLevel])
				{
					if (PossibleChild->Parent == CurrRoot->Profile)
					{
						TSharedPtr<FNiagaraDeviceProfileViewModel>& NewChild = CurrRoot->Children.Add_GetRef(MakeShared<FNiagaraDeviceProfileViewModel>());
						NewChild->Profile = PossibleChild;
						BuildProfileTree(NewChild.Get(), NextLevel);
					}
				}
			}
		};

		//Add all root nodes and build their trees.
		TSharedPtr<FNiagaraDeviceProfileViewModel> CurrRoot = MakeShared<FNiagaraDeviceProfileViewModel>();
		CurrRoot->Profile = DeviceProfilesByHierarchyLevel[0][RootProfileIdx];
		BuildProfileTree(CurrRoot.Get(), 0);
		FullDeviceProfileTree.Add(CurrRoot);
	}

	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	int32 NumQualityLevels = Settings->QualityLevels.Num();
	FilteredDeviceProfileTrees.Reset();
	FilteredDeviceProfileTrees.SetNum(NumQualityLevels);
	
	for (TSharedPtr<FNiagaraDeviceProfileViewModel>& FullDeviceRoot : FullDeviceProfileTree)
	{
		for (int32 QualityLevel = 0; QualityLevel < NumQualityLevels; ++QualityLevel)
		{
			int32 QualityLevelMask = FNiagaraPlatformSet::CreateQualityLevelMask(QualityLevel);

			if (IsTreeActiveForQL(FullDeviceRoot, QualityLevelMask))
			{
				TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>>& FilteredRoots = FilteredDeviceProfileTrees[QualityLevel];

				TSharedPtr<FNiagaraDeviceProfileViewModel>& FilteredRoot = FilteredRoots.Add_GetRef(MakeShared<FNiagaraDeviceProfileViewModel>());
				FilteredRoot->Profile = FullDeviceRoot->Profile;

				FilterTreeForQL(FullDeviceRoot, FilteredRoot, QualityLevelMask);
			}
		}
	}
}

TSharedRef<SWidget> FNiagaraPlatformSetCustomization::GenerateDeviceProfileTreeWidget(int32 QualityLevel)
{
	FullDeviceProfileTree.Reset();
	if (FullDeviceProfileTree.Num() == 0)
	{
		CreateDeviceProfileTree();	
	}
	
	QualityLevelMenuContents[QualityLevel].Reset();

	TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>>* TreeToUse = &FullDeviceProfileTree;
	if (QualityLevel != INDEX_NONE)
	{
		check(QualityLevel < FilteredDeviceProfileTrees.Num());
		TreeToUse = &FilteredDeviceProfileTrees[QualityLevel];
	}

	return SAssignNew(QualityLevelMenuContents[QualityLevel], SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
		[
			SAssignNew(DeviceProfileTreeWidget, STreeView<TSharedPtr<FNiagaraDeviceProfileViewModel>>)
			.TreeItemsSource(TreeToUse)
			.OnGenerateRow(this, &FNiagaraPlatformSetCustomization::OnGenerateDeviceProfileTreeRow, QualityLevel)
			.OnGetChildren(this, &FNiagaraPlatformSetCustomization::OnGetDeviceProfileTreeChildren, QualityLevel)
			.SelectionMode(ESelectionMode::None)
		];
}

TSharedRef<ITableRow> FNiagaraPlatformSetCustomization::OnGenerateDeviceProfileTreeRow(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem, const TSharedRef<STableViewBase>& OwnerTable, int32 QualityLevel)
{
	TSharedPtr<SHorizontalBox> RowContainer;
	SAssignNew(RowContainer, SHorizontalBox);

	FNiagaraPlatformSetEnabledStateDetails Details;
	FNiagaraPlatformSetEnabledState EnabledState = TargetPlatformSet->IsEnabled(InItem->Profile, QualityLevel, false, &Details);

	FText NameTooltip = FText::Format(LOCTEXT("ProfileQLTooltipFmt", "Effects Quality: {0}"), FNiagaraPlatformSet::GetQualityLevelMaskText(QualityLevel));

	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	int32 NumQualityLevels = Settings->QualityLevels.Num();
	
	
	//Top level profile. Look for a platform icon.
	if (InItem->Profile->Parent == nullptr)
	{
		if (const PlatformInfo::FTargetPlatformInfo* Info = PlatformInfo::FindPlatformInfo(*InItem->Profile->DeviceType))
		{
			const FSlateBrush* DeviceProfileTypeIcon = FAppStyle::GetBrush(Info->GetIconStyleName(EPlatformIconSize::Normal));
			if (DeviceProfileTypeIcon != FAppStyle::Get().GetDefaultBrush())
			{
				RowContainer->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(4, 0, 0, 0)
					[
						SNew(SBox)
						.WidthOverride(16)
						.HeightOverride(16)
						[
							SNew(SImage)
							.Image(DeviceProfileTypeIcon)
						]
					];
			}
		}
	}

	FName TextStyleName("NormalText");
	FSlateColor TextColor(FSlateColor::UseForeground());
	ENiagaraPlatformSelectionState CurrentState = TargetPlatformSet->GetDeviceProfileState(InItem->Profile, QualityLevel);
	
	FSlateColor ActiveColor = FSlateColor::UseForeground();
	FSlateColor InactiveColor = FSlateColor::UseSubduedForeground();
	FSlateColor DisabledColor = FSlateColor(FLinearColor(FVector4(1, 0, 0, 1)));
	if (EnabledState.bIsActive)
	{
		TextColor = ActiveColor;
	}
	else
	{
		if (EnabledState.bCanBeActive)
		{
			TextColor = InactiveColor;
		}
		else
		{
			TextColor = DisabledColor;
		}
	}

	TSharedPtr<SVerticalBox> TooltipContentsBox = SNew(SVerticalBox);
 	if (EnabledState.bIsActive)
 	{
		TooltipContentsBox->AddSlot()
 		[
 			SNew(STextBlock)
 			.TextStyle(FAppStyle::Get(), TextStyleName)
 			.ColorAndOpacity(ActiveColor)
 			.Text(LOCTEXT("DPActiveTooltip", "Active"))
 		];
 	}
 	else
 	{
 		if(EnabledState.bCanBeActive)
 		{
			TooltipContentsBox->AddSlot()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TextStyleName)
				.ColorAndOpacity(InactiveColor)
				.Text(LOCTEXT("DPInActiveTooltip", "Inactive Due To:"))
			];
 			for (FText Reason : Details.ReasonsForInActive)
 			{			
				TooltipContentsBox->AddSlot()
 				[
 					SNew(STextBlock)
 					.TextStyle(FAppStyle::Get(), TextStyleName)
 					.ColorAndOpacity(InactiveColor)
 					.Text(Reason)
 				];
 			}
 		}
 		else
 		{
			TooltipContentsBox->AddSlot()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TextStyleName)
				.ColorAndOpacity(DisabledColor)
				.Text(LOCTEXT("DPDisabledTooltip", "Disabled Due To:"))
			];
 			for (FText Reason : Details.ReasonsForDisabled)
 			{			
				TooltipContentsBox->AddSlot()
 				[
 					SNew(STextBlock)
 					.TextStyle(FAppStyle::Get(), TextStyleName)
 					.ColorAndOpacity(DisabledColor)
 					.Text(Reason)
 				];
 			}
 		}
	}
	
	TooltipContentsBox->AddSlot()
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), TextStyleName)
			.ColorAndOpacity(ActiveColor)
			.Text(LOCTEXT("QLTooltipAvailableListHeader", "Available:"))
		];

	int32 DefaultQL = FNiagaraPlatformSet::QualityLevelFromMask(Details.DefaultQualityMask);
	for (int32 QL = 0; QL < NumQualityLevels; ++QL)
	{
		if(Details.AvailableQualityMask & (1<<QL))
		{
			if (DefaultQL == QL)
			{
				TooltipContentsBox->AddSlot()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), TextStyleName)
						.ColorAndOpacity(ActiveColor)
						.Text(FText::Format(LOCTEXT("QLTooltipAvailableListDefaultFmt", "{0} (Default)"), FNiagaraPlatformSet::GetQualityLevelText(QL)))
					];
			}
			else
			{
				TooltipContentsBox->AddSlot()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), TextStyleName)
						.ColorAndOpacity(ActiveColor)
						.Text(FNiagaraPlatformSet::GetQualityLevelText(QL))
					];
			}
		}
	}

	RowContainer->AddSlot()
		.Padding(4, 2, 0, 2)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.OnClicked(this, &FNiagaraPlatformSetCustomization::OnProfileMenuButtonClicked, InItem, QualityLevel, false)
			.IsEnabled(this, &FNiagaraPlatformSetCustomization::GetProfileMenuItemEnabled, InItem, QualityLevel)
			.ForegroundColor(TextColor)
			.ToolTip(
				SNew(SToolTip)
				[
					TooltipContentsBox.ToSharedRef()
				]
			)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TextStyleName)
				.Text(FText::FromString(InItem->Profile->GetName()))
			]
		];

	RowContainer->AddSlot()
		.AutoWidth()
		.Padding(12, 2, 4, 4)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Fill)
			.Visibility(this, &FNiagaraPlatformSetCustomization::GetProfileMenuButtonVisibility, InItem, QualityLevel)
			.OnClicked(this, &FNiagaraPlatformSetCustomization::OnProfileMenuButtonClicked, InItem, QualityLevel, true)
			.ToolTipText(this, &FNiagaraPlatformSetCustomization::GetProfileMenuButtonToolTip, InItem, QualityLevel)
			[
				SNew(SBox)
				.WidthOverride(8)
				.HeightOverride(8)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(this, &FNiagaraPlatformSetCustomization::GetProfileMenuButtonImage, InItem, QualityLevel)
				]
			]
		];

	return SNew(STableRow<TSharedPtr<FNiagaraDeviceProfileViewModel>>, OwnerTable)
		.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.PlatformSet.TreeView")
		[
			RowContainer.ToSharedRef()
		];
}

enum class EProfileButtonMode
{
	None,
	Include,
	Exclude,
	Remove
};

static EProfileButtonMode GetProfileMenuButtonMode(FNiagaraPlatformSet* PlatformSet, TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel)
{
	int32 Mask = PlatformSet->GetAvailableQualityMaskForDeviceProfile(Item->Profile);
	int32 QLMask = FNiagaraPlatformSet::CreateQualityLevelMask(QualityLevel);

	if ((Mask & QLMask) == 0)
	{
		return EProfileButtonMode::None;
	}

	ENiagaraPlatformSelectionState CurrentState = PlatformSet->GetDeviceProfileState(Item->Profile, QualityLevel);

	bool bQualityEnabled = PlatformSet->IsEffectQualityEnabled(QualityLevel);
	//FNiagaraPlatformSetEnabledState Enabled = PlatformSet->IsEnabled(Item->Profile, QualityLevel);
	bool bIsDefault = CurrentState == ENiagaraPlatformSelectionState::Default;

	if (bIsDefault && bQualityEnabled)
	{
		return EProfileButtonMode::Exclude;
	}

	if (bIsDefault && !bQualityEnabled)
	{
		return EProfileButtonMode::Include;
	}

	return EProfileButtonMode::Remove;
}


FText FNiagaraPlatformSetCustomization::GetProfileMenuButtonToolTip(TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel) const
{
	EProfileButtonMode Mode = GetProfileMenuButtonMode(TargetPlatformSet, Item, QualityLevel);
	switch(Mode)
	{
		case EProfileButtonMode::Include:
			return LOCTEXT("IncludePlatform", "Include this platform.");
		case EProfileButtonMode::Exclude:
			return LOCTEXT("ExcludePlatform", "Exclude this platform.");
		case EProfileButtonMode::Remove:
			return LOCTEXT("RemovePlatform", "Remove this platform override.");
	}

	return FText::GetEmpty();
}

EVisibility FNiagaraPlatformSetCustomization::GetProfileMenuButtonVisibility(TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel) const
{
	EProfileButtonMode Mode = GetProfileMenuButtonMode(TargetPlatformSet, Item, QualityLevel);
	if (Mode == EProfileButtonMode::None)
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

bool FNiagaraPlatformSetCustomization::GetProfileMenuItemEnabled(TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel) const
{
	EProfileButtonMode Mode = GetProfileMenuButtonMode(TargetPlatformSet, Item, QualityLevel);
	if (Mode == EProfileButtonMode::None)
	{
		return false;
	}

	return true;
}

const FSlateBrush* FNiagaraPlatformSetCustomization::GetProfileMenuButtonImage(TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel) const
{
	EProfileButtonMode Mode = GetProfileMenuButtonMode(TargetPlatformSet, Item, QualityLevel);
	switch(Mode)
	{
		case EProfileButtonMode::Include:
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.PlatformSet.Include");
		case EProfileButtonMode::Exclude:
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.PlatformSet.Exclude");
		case EProfileButtonMode::Remove:
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.PlatformSet.Remove");
	}

	return FAppStyle::GetBrush("NoBrush");
}

FReply FNiagaraPlatformSetCustomization::OnProfileMenuButtonClicked(TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel, bool bReopenMenu)
{
	ENiagaraPlatformSelectionState TargetState;

	EProfileButtonMode Mode = GetProfileMenuButtonMode(TargetPlatformSet, Item, QualityLevel);
	switch(Mode)
	{
		case EProfileButtonMode::Include:
			TargetState = ENiagaraPlatformSelectionState::Enabled;
			break;
		case EProfileButtonMode::Exclude:
			TargetState = ENiagaraPlatformSelectionState::Disabled;
			break;
		case EProfileButtonMode::Remove:
			TargetState = ENiagaraPlatformSelectionState::Default;
			break;
		default:
			return FReply::Handled(); // shouldn't happen, button should be collapsed
	}

	PropertyHandle->NotifyPreChange();
	TargetPlatformSet->SetDeviceProfileState(Item->Profile, QualityLevel, TargetState);
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	InvalidateSiblingConflicts();

	if (!bReopenMenu)
	{
		QualityLevelMenuAnchors[QualityLevel]->SetIsOpen(false);
	}

	DeviceProfileTreeWidget->RequestTreeRefresh();

	return FReply::Handled();
}

void FNiagaraPlatformSetCustomization::OnGetDeviceProfileTreeChildren(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem, TArray< TSharedPtr<FNiagaraDeviceProfileViewModel> >& OutChildren, int32 QualityLevel)
{
	{
		OutChildren = InItem->Children;
	}
}

ECheckBoxState FNiagaraPlatformSetCustomization::IsQLChecked(int32 QualityLevel)const
{
	if (TargetPlatformSet)
	{
		return TargetPlatformSet->IsEffectQualityEnabled(QualityLevel) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void FNiagaraPlatformSetCustomization::QLCheckStateChanged(ECheckBoxState CheckState, int32 QualityLevel)
{
	PropertyHandle->NotifyPreChange();
	TargetPlatformSet->SetEnabledForEffectQuality(QualityLevel, CheckState == ECheckBoxState::Checked);
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	InvalidateSiblingConflicts();
}

void FNiagaraPlatformSetCustomization::OnPropertyValueChanged()
{
	UpdateCachedConflicts();
	GenerateQualityLevelSelectionWidgets();
	TargetPlatformSet->OnChanged();
}

//////////////////////////////////////////////////////////////////////////

#include "Widgets/Input/SMultiLineEditableTextBox.h"

/**
 * Input box for entering CVars into Niagara with auto complete for supported variable types.
 */
class SNiagaraConsoleInputBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(FName, FGetCurrentCVar)
	DECLARE_DELEGATE_OneParam(FOnTextCommittedDelegate, const FText& /*Text*/)

	SLATE_BEGIN_ARGS(SNiagaraConsoleInputBox)
		: _SuggestionListPlacement(MenuPlacement_BelowAnchor)
	{}

	/** Where to place the suggestion list */
	SLATE_ARGUMENT(EMenuPlacement, SuggestionListPlacement)

		SLATE_EVENT(FOnTextCommittedDelegate, OnTextCommittedEvent)

		SLATE_EVENT(FGetCurrentCVar, GetCurrentCVar)

		/** Delegate to call to close the console */
		SLATE_EVENT(FSimpleDelegate, OnCloseConsole)

	SLATE_END_ARGS()

	/** Protected console input box widget constructor, called by Slate */
	SNiagaraConsoleInputBox();

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs	Declaration used by the SNew() macro to construct this widget
	 */
	void Construct(const FArguments& InArgs);

	/** Returns the editable text box associated with this widget.  Used to set focus directly. */
	TSharedRef< SMultiLineEditableTextBox > GetEditableTextBox()
	{
		return InputText.ToSharedRef();
	}

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	virtual bool SupportsKeyboardFocus() const override { return true; }

	// e.g. Tab or Key_Up
	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent) override;

	/** Handles entering in a command */
	FText GetTooltipText()const;

	/** Handles entering in a command */
	void OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	void OnTextChanged(const FText& InText);

	/** Get the maximum width of the selection list */
	FOptionalSize GetSelectionListMaxWidth() const;

	/** Makes the widget for the suggestions messages in the list view */
	TSharedRef<ITableRow> MakeSuggestionListItemWidget(TSharedPtr<FString> Message, const TSharedRef<STableViewBase>& OwnerTable);

	void SuggestionSelectionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

	void SetSuggestions(TArray<FString>& Elements, FText Highlight);

	void MarkActiveSuggestion();

	void ClearSuggestions();

	FText GetText() const;

	FReply OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	FReply OnKeyCharHandler(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent);

private:

	struct FSuggestions
	{
		FSuggestions()
			: SelectedSuggestion(INDEX_NONE)
		{
		}

		void Reset()
		{
			SelectedSuggestion = INDEX_NONE;
			SuggestionsList.Reset();
			SuggestionsHighlight = FText::GetEmpty();
		}

		bool HasSuggestions() const
		{
			return SuggestionsList.Num() > 0;
		}

		bool HasSelectedSuggestion() const
		{
			return SuggestionsList.IsValidIndex(SelectedSuggestion);
		}

		void StepSelectedSuggestion(const int32 Step)
		{
			SelectedSuggestion += Step;
			if (SelectedSuggestion < 0)
			{
				SelectedSuggestion = SuggestionsList.Num() - 1;
			}
			else if (SelectedSuggestion >= SuggestionsList.Num())
			{
				SelectedSuggestion = 0;
			}
		}

		TSharedPtr<FString> GetSelectedSuggestion() const
		{
			return SuggestionsList.IsValidIndex(SelectedSuggestion) ? SuggestionsList[SelectedSuggestion] : nullptr;
		}

		/** INDEX_NONE if not set, otherwise index into SuggestionsList */
		int32 SelectedSuggestion;

		/** All log messages stored in this widget for the list view */
		TArray<TSharedPtr<FString>> SuggestionsList;

		/** Highlight text to use for the suggestions list */
		FText SuggestionsHighlight;
	};

	/** Editable text widget */
	TSharedPtr< SMultiLineEditableTextBox > InputText;

	/** history / auto completion elements */
	TSharedPtr< SMenuAnchor > SuggestionBox;

	/** The list view for showing all log messages. Should be replaced by a full text editor */
	TSharedPtr< SListView< TSharedPtr<FString> > > SuggestionListView;

	/** Active list of suggestions */
	FSuggestions Suggestions;

	/** Delegate to call to close the console */
	FSimpleDelegate OnCloseConsole;

	FGetCurrentCVar GetCurrentCVar;

	FOnTextCommittedDelegate OnTextCommittedEvent;

	/** to prevent recursive calls in UI callback */
	bool bIgnoreUIUpdate;

	/** true if this widget has been Ticked at least once */
	bool bHasTicked;

	/** True if we consumed a tab key in OnPreviewKeyDown, so we can ignore it in OnKeyCharHandler as well */
	bool bConsumeTab;

	FText WorkingText;
};

static const TCHAR* NiagaraCVarHistoryKey = TEXT("NiagaraCVarHistory");

SNiagaraConsoleInputBox::SNiagaraConsoleInputBox()
	: bIgnoreUIUpdate(false)
	, bHasTicked(false)
	, bConsumeTab(false)
{
}

void SNiagaraConsoleInputBox::Construct(const FArguments& InArgs)
{
	GetCurrentCVar = InArgs._GetCurrentCVar;
	OnTextCommittedEvent = InArgs._OnTextCommittedEvent;
	OnCloseConsole = InArgs._OnCloseConsole;

	WorkingText = FText::FromName(GetCurrentCVar.Execute());

	EPopupMethod PopupMethod = GIsEditor ? EPopupMethod::CreateNewWindow : EPopupMethod::UseCurrentWindow;
	ChildSlot
	[
		SAssignNew(SuggestionBox, SMenuAnchor)
		.Method(PopupMethod)
		.Placement(InArgs._SuggestionListPlacement)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SAssignNew(InputText, SMultiLineEditableTextBox)
				.ToolTipText_Raw(this, &SNiagaraConsoleInputBox::GetTooltipText)
				.AllowMultiLine(false)
				.ClearTextSelectionOnFocusLoss(true)
				.SelectAllTextWhenFocused(true)
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SNiagaraConsoleInputBox::GetText)))
				.OnTextCommitted(this, &SNiagaraConsoleInputBox::OnTextCommitted)
				.OnTextChanged(this, &SNiagaraConsoleInputBox::OnTextChanged)
				.OnKeyCharHandler(this, &SNiagaraConsoleInputBox::OnKeyCharHandler)
				.OnKeyDownHandler(this, &SNiagaraConsoleInputBox::OnKeyDownHandler)
				.OnIsTypedCharValid(FOnIsTypedCharValid::CreateLambda([](const TCHAR InCh) { return true; })) // allow tabs to be typed into the field
				.ClearKeyboardFocusOnCommit(true)
				.ModiferKeyForNewLine(EModifierKey::Shift)
			]
		]
	.MenuContent
	(
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(FMargin(2))
		[
			SNew(SBox)
			.HeightOverride(250) // avoids flickering, ideally this would be adaptive to the content without flickering
			.MinDesiredWidth(300)
			.MaxDesiredWidth(this, &SNiagaraConsoleInputBox::GetSelectionListMaxWidth)
			[
				SAssignNew(SuggestionListView, SListView< TSharedPtr<FString> >)
				.ListItemsSource(&Suggestions.SuggestionsList)
				.SelectionMode(ESelectionMode::Single)							// Ideally the mouse over would not highlight while keyboard controls the UI
				.OnGenerateRow(this, &SNiagaraConsoleInputBox::MakeSuggestionListItemWidget)
				.OnSelectionChanged(this, &SNiagaraConsoleInputBox::SuggestionSelectionChanged)
			]
		]
	)
	];
}

void SNiagaraConsoleInputBox::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	bHasTicked = true;

	if (!GIntraFrameDebuggingGameThread && !IsEnabled())
	{
		SetEnabled(true);
	}
	else if (GIntraFrameDebuggingGameThread && IsEnabled())
	{
		SetEnabled(false);
	}
}


void SNiagaraConsoleInputBox::SuggestionSelectionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	if (bIgnoreUIUpdate)
	{
		return;
	}

	if (NewValue.IsValid() == false)
	{
		SuggestionBox->SetIsOpen(false);
		return;
	}

	Suggestions.SelectedSuggestion = Suggestions.SuggestionsList.IndexOfByPredicate([&NewValue](const TSharedPtr<FString>& InSuggestion)
		{
			return InSuggestion == NewValue;
		});

	MarkActiveSuggestion();

	// If the user selected this suggestion by clicking on it, then go ahead and close the suggestion
	// box as they've chosen the suggestion they're interested in.
	if (SelectInfo == ESelectInfo::OnMouseClick)
	{
		FString CommitString = *NewValue;
		OnTextCommitted(FText::FromString(CommitString), ETextCommit::Default);
		SuggestionBox->SetIsOpen(false);
	}

	// Ideally this would set the focus back to the edit control
//	FWidgetPath WidgetToFocusPath;
//	FSlateApplication::Get().GeneratePathToWidgetUnchecked( InputText.ToSharedRef(), WidgetToFocusPath );
//	FSlateApplication::Get().SetKeyboardFocus( WidgetToFocusPath, EFocusCause::SetDirectly );
}

FOptionalSize SNiagaraConsoleInputBox::GetSelectionListMaxWidth() const
{
	// Limit the width of the suggestions list to the work area that this widget currently resides on
	const FSlateRect WidgetRect(GetCachedGeometry().GetAbsolutePosition(), GetCachedGeometry().GetAbsolutePosition() + GetCachedGeometry().GetAbsoluteSize());
	const FSlateRect WidgetWorkArea = FSlateApplication::Get().GetWorkArea(WidgetRect);
	return FMath::Max(300.0f, WidgetWorkArea.GetSize().X - 12.0f);
}

TSharedRef<ITableRow> SNiagaraConsoleInputBox::MakeSuggestionListItemWidget(TSharedPtr<FString> Text, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(Text.IsValid());

	FString SanitizedText = *Text;
	SanitizedText.ReplaceInline(TEXT("\r\n"), TEXT("\n"), ESearchCase::CaseSensitive);
	SanitizedText.ReplaceInline(TEXT("\r"), TEXT(" "), ESearchCase::CaseSensitive);
	SanitizedText.ReplaceInline(TEXT("\n"), TEXT(" "), ESearchCase::CaseSensitive);

	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*SanitizedText);
	
	return
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		[
			SNew(STextBlock)
			.Text(FText::FromString(SanitizedText))
			.ToolTipText(CVar ? FText::FromString(CVar->GetHelp()) : FText::GetEmpty())
			.TextStyle(FAppStyle::Get(), "Log.Normal")
			.HighlightText(Suggestions.SuggestionsHighlight)
		];
}

void SNiagaraConsoleInputBox::OnTextChanged(const FText& InText)
{
	if (bIgnoreUIUpdate)
	{
		return;
	}
	
	WorkingText = InText;

	const FString& InputTextStr = InputText->GetText().ToString();
	if (!InputTextStr.IsEmpty())
	{
		TArray<FString> AutoCompleteList;

		auto OnConsoleVariable = [&AutoCompleteList](const TCHAR* Name, IConsoleObject* CVar)
		{
			if (!CVar->IsEnabled())
			{
				return;
			}

			//Don't include commands or types that are not supported
			bool bSupported = CVar->IsVariableBool() || CVar->IsVariableInt() || CVar->IsVariableFloat();
			if (!bSupported)
			{
				return;
			}

			AutoCompleteList.Add(Name);
		};

		IConsoleManager::Get().ForEachConsoleObjectThatContains(FConsoleObjectVisitor::CreateLambda(OnConsoleVariable), *InputTextStr);

		AutoCompleteList.Sort([InputTextStr](const FString& A, const FString& B)
			{
				if (A.StartsWith(InputTextStr))
				{
					if (!B.StartsWith(InputTextStr))
					{
						return true;
					}
				}
				else
				{
					if (B.StartsWith(InputTextStr))
					{
						return false;
					}
				}

				return A < B;

			});


		SetSuggestions(AutoCompleteList, FText::FromString(InputTextStr));
	}
	else
	{
		ClearSuggestions();
	}
}

FText SNiagaraConsoleInputBox::GetTooltipText()const
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*WorkingText.ToString()))
	{
		return FText::FromString(CVar->GetHelp());
	}
	return FText::GetEmpty();
}

void SNiagaraConsoleInputBox::OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	WorkingText = InText;
	IConsoleManager::Get().AddConsoleHistoryEntry(NiagaraCVarHistoryKey, *InText.ToString());
	OnTextCommittedEvent.ExecuteIfBound(InText);
	SuggestionBox->SetIsOpen(false);
}

FReply SNiagaraConsoleInputBox::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent)
{
	if (SuggestionBox->IsOpen())
	{
		if (KeyEvent.GetKey() == EKeys::Up || KeyEvent.GetKey() == EKeys::Down)
		{
			Suggestions.StepSelectedSuggestion(KeyEvent.GetKey() == EKeys::Up ? -1 : +1);
			MarkActiveSuggestion();

			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Tab)
		{
			if (Suggestions.HasSuggestions())
			{
				if (Suggestions.HasSelectedSuggestion())
				{
					Suggestions.StepSelectedSuggestion(KeyEvent.IsShiftDown() ? -1 : +1);
				}
				else
				{
					Suggestions.SelectedSuggestion = 0;
				}
				MarkActiveSuggestion();
			}

			bConsumeTab = true;
			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Escape)
		{
			SuggestionBox->SetIsOpen(false);
			return FReply::Handled();
		}
	}
	else
	{
		if (KeyEvent.GetKey() == EKeys::Up)
		{
			// If the command field isn't empty we need you to have pressed Control+Up to summon the history (to make sure you're not just using caret navigation)
			const bool bIsMultiLine = false;
			const bool bShowHistory = InputText->GetText().IsEmpty() || KeyEvent.IsControlDown();
			if (bShowHistory)
			{
				TArray<FString> History;

				IConsoleManager::Get().GetConsoleHistory(NiagaraCVarHistoryKey, History);

				SetSuggestions(History, FText::GetEmpty());

				if (Suggestions.HasSuggestions())
				{
					Suggestions.StepSelectedSuggestion(-1);
					MarkActiveSuggestion();
				}
				return FReply::Handled();
			}
		}
		else if (KeyEvent.GetKey() == EKeys::Escape)
		{
			if (InputText->GetText().IsEmpty())
			{
				OnCloseConsole.ExecuteIfBound();
			}
			else
			{
				// Clear the console input area
				bIgnoreUIUpdate = true;
				InputText->SetText(FText::GetEmpty());
				bIgnoreUIUpdate = false;

				ClearSuggestions();
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SNiagaraConsoleInputBox::SetSuggestions(TArray<FString>& Elements, FText Highlight)
{
	FString SelectionText;
	if (Suggestions.HasSelectedSuggestion())
	{
		SelectionText = *Suggestions.GetSelectedSuggestion();
	}

	Suggestions.Reset();
	Suggestions.SuggestionsHighlight = Highlight;

	for (int32 i = 0; i < Elements.Num(); ++i)
	{
		Suggestions.SuggestionsList.Add(MakeShared<FString>(Elements[i]));

		if (Elements[i] == SelectionText)
		{
			Suggestions.SelectedSuggestion = i;
		}
	}
	SuggestionListView->RequestListRefresh();

	if (Suggestions.HasSuggestions())
	{
		// Ideally if the selection box is open the output window is not changing it's window title (flickers)
		SuggestionBox->SetIsOpen(true, false);
		if (Suggestions.HasSelectedSuggestion())
		{
			SuggestionListView->RequestScrollIntoView(Suggestions.GetSelectedSuggestion());
		}
		else
		{
			SuggestionListView->ScrollToTop();
		}
	}
	else
	{
		SuggestionBox->SetIsOpen(false);
	}
}

void SNiagaraConsoleInputBox::MarkActiveSuggestion()
{
	bIgnoreUIUpdate = true;
	if (Suggestions.HasSelectedSuggestion())
	{
		TSharedPtr<FString> SelectedSuggestion = Suggestions.GetSelectedSuggestion();

		SuggestionListView->SetSelection(SelectedSuggestion);
		SuggestionListView->RequestScrollIntoView(SelectedSuggestion);	// Ideally this would only scroll if outside of the view

		InputText->SetText(FText::FromString(*SelectedSuggestion));
	}
	else
	{
		SuggestionListView->ClearSelection();
	}
	bIgnoreUIUpdate = false;
}

void SNiagaraConsoleInputBox::ClearSuggestions()
{
	SuggestionBox->SetIsOpen(false);
	Suggestions.Reset();
}

FText SNiagaraConsoleInputBox::GetText() const
{
	return WorkingText;
}

FReply SNiagaraConsoleInputBox::OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return FReply::Unhandled();
}

FReply SNiagaraConsoleInputBox::OnKeyCharHandler(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	// A printable key may be used to open the console, so consume all characters before our first Tick
	if (!bHasTicked)
	{
		return FReply::Handled();
	}

	// Intercept tab if used for auto-complete
	if (InCharacterEvent.GetCharacter() == '\t' && bConsumeTab)
	{
		bConsumeTab = false;
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

//////////////////////////////////////////////////////////////////////////

///
/// 
/// 
void FNiagaraPlatformSetCVarConditionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow.NameContent().Widget = StructPropertyHandle->CreatePropertyNameWidget();
	HeaderRow.ValueContent().Widget = StructPropertyHandle->CreatePropertyValueWidget();
}

void FNiagaraPlatformSetCVarConditionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = StructPropertyHandle;

	//Add the CVar name 
	CVarNameHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraPlatformSetCVarCondition, CVarName));
	
	ChildBuilder.AddProperty(CVarNameHandle.ToSharedRef()).CustomWidget()
	[
		SNew(SNiagaraConsoleInputBox)
		.GetCurrentCVar(this, &FNiagaraPlatformSetCVarConditionCustomization::GetCurrentCVar)
		.OnTextCommittedEvent(this, &FNiagaraPlatformSetCVarConditionCustomization::OnTextCommitted)
		// Always place suggestions above the input line for the output log widget
		.SuggestionListPlacement(MenuPlacement_AboveAnchor)
	];

	BoolValueHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraPlatformSetCVarCondition, Value));

	TAttribute<EVisibility> BoolVisAttr = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FNiagaraPlatformSetCVarConditionCustomization::BoolPropertyVisibility));
	ChildBuilder.AddProperty(BoolValueHandle.ToSharedRef()).Visibility(BoolVisAttr);

	TAttribute<EVisibility> IntVisAttr = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FNiagaraPlatformSetCVarConditionCustomization::IntPropertyVisibility));
	MinIntHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraPlatformSetCVarCondition, MinInt));
	ChildBuilder.AddProperty(MinIntHandle.ToSharedRef()).Visibility(IntVisAttr);
	MaxIntHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraPlatformSetCVarCondition, MaxInt));
	ChildBuilder.AddProperty(MaxIntHandle.ToSharedRef()).Visibility(IntVisAttr);

	TAttribute<EVisibility> FloatVisAttr = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FNiagaraPlatformSetCVarConditionCustomization::FloatPropertyVisibility));
	MinFloatHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraPlatformSetCVarCondition, MinFloat));
	ChildBuilder.AddProperty(MinFloatHandle.ToSharedRef()).Visibility(FloatVisAttr);
	MaxFloatHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraPlatformSetCVarCondition, MaxFloat));
	ChildBuilder.AddProperty(MaxFloatHandle.ToSharedRef()).Visibility(FloatVisAttr);

	TSharedPtr<IPropertyHandle> PassHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraPlatformSetCVarCondition, PassResponse));
	ChildBuilder.AddProperty(PassHandle.ToSharedRef());
	TSharedPtr<IPropertyHandle> FailHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNiagaraPlatformSetCVarCondition, FailResponse));
	ChildBuilder.AddProperty(FailHandle.ToSharedRef());
}

EVisibility FNiagaraPlatformSetCVarConditionCustomization::BoolPropertyVisibility() const
{
	if (FNiagaraPlatformSetCVarCondition* TargetCondition = GetTargetCondition())
	{
		if (IConsoleVariable* CVar = TargetCondition->GetCVar())
		{
			return CVar->IsVariableBool() ? EVisibility::Visible : EVisibility::Hidden;
		}
	}
	return EVisibility::Hidden;
}

EVisibility FNiagaraPlatformSetCVarConditionCustomization::IntPropertyVisibility() const
{
	if (FNiagaraPlatformSetCVarCondition* TargetCondition = GetTargetCondition())
	{
		if (IConsoleVariable* CVar = TargetCondition->GetCVar())
		{
			return CVar->IsVariableInt() ? EVisibility::Visible : EVisibility::Hidden;
		}
	}
	return EVisibility::Hidden;
}

EVisibility FNiagaraPlatformSetCVarConditionCustomization::FloatPropertyVisibility() const
{
	if (FNiagaraPlatformSetCVarCondition* TargetCondition = GetTargetCondition())
	{
		if (IConsoleVariable* CVar = TargetCondition->GetCVar())
		{
			return CVar->IsVariableFloat() ? EVisibility::Visible : EVisibility::Hidden;
		}
	}
	return EVisibility::Hidden;
}

FName FNiagaraPlatformSetCVarConditionCustomization::GetCurrentCVar()
{
	if (FNiagaraPlatformSetCVarCondition* TargetCondition = GetTargetCondition())
	{
		return TargetCondition->CVarName;
	}
	return TEXT("");
}

void FNiagaraPlatformSetCVarConditionCustomization::OnTextCommitted(const FText& Text)
{
	if (FNiagaraPlatformSetCVarCondition* TargetCondition = GetTargetCondition())
	{
		TargetCondition->SetCVar(*Text.ToString());
	}

	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

FNiagaraPlatformSetCVarCondition* FNiagaraPlatformSetCVarConditionCustomization::GetTargetCondition()const
{
	if (uint8* BaseAddress = GetBaseAddress(PropertyHandle.ToSharedRef()))
	{
		return reinterpret_cast<FNiagaraPlatformSetCVarCondition*>(PropertyHandle->GetValueBaseAddress(BaseAddress));
	}
	return nullptr;
}

FNiagaraPlatformSet* FNiagaraPlatformSetCVarConditionCustomization::GetTargetPlatformSet()const
{
	check(PropertyHandle && PropertyHandle->GetParentHandle());
	TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle();
	if (uint8* BaseAddress = GetBaseAddress(ParentHandle.ToSharedRef()))
	{
		return reinterpret_cast<FNiagaraPlatformSet*>(ParentHandle->GetValueBaseAddress(BaseAddress));
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
