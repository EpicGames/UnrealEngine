// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "NiagaraDeviceProfileViewModel.h"
#include "NiagaraPlatformSet.h"
#include "Layout/Visibility.h"
#include "Widgets/Views/STreeView.h"

class FDetailWidgetRow;
class IPropertyHandleArray;
class ITableRow;
class SMenuAnchor;
class STableViewBase;
class SWrapBox;

enum class ECheckBoxState : uint8;

class FNiagaraPlatformSetCustomization : public IPropertyTypeCustomization
{
public:
	/** @return A new instance of this class */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraPlatformSetCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** IPropertyTypeCustomization interface end */
private:

	void CreateDeviceProfileTree();

	TSharedRef<SWidget> GenerateDeviceProfileTreeWidget(int32 QualityLevel);
	TSharedRef<ITableRow> OnGenerateDeviceProfileTreeRow(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem, const TSharedRef<STableViewBase>& OwnerTable, int32 QualityLevel);
	void OnGetDeviceProfileTreeChildren(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem, TArray< TSharedPtr<FNiagaraDeviceProfileViewModel> >& OutChildren, int32 QualityLevel);
	FReply RemoveDeviceProfile(UDeviceProfile* Profile, int32 QualityLevel);
	TSharedRef<SWidget> GetQualityLevelMenuContents(int32 QualityLevel) const;

	EVisibility GetDeviceProfileErrorVisibility(UDeviceProfile* Profile, int32 QualityLevel) const;
	FText GetDeviceProfileErrorToolTip(UDeviceProfile* Profile, int32 QualityLevel) const;

	FSlateColor GetQualityLevelButtonTextColor(int32 QualityLevel) const;
	EVisibility GetQualityLevelErrorVisibility(int32 QualityLevel) const;
	FText GetQualityLevelErrorToolTip(int32 QualityLevel) const;

	const FSlateBrush* GetProfileMenuButtonImage(TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel) const;
	EVisibility GetProfileMenuButtonVisibility(TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel) const;
	bool GetProfileMenuItemEnabled(TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel) const;
	FText GetProfileMenuButtonToolTip(TSharedPtr<FNiagaraDeviceProfileViewModel> Item, int32 QualityLevel) const;
	FReply OnProfileMenuButtonClicked(TSharedPtr<FNiagaraDeviceProfileViewModel> InItem, int32 QualityLevel, bool bReopenMenu);

	TSharedRef<SWidget> GetCurrentDeviceProfileSelectionWidget(TSharedPtr<FNiagaraDeviceProfileViewModel> ProfileView);
	TSharedRef<SWidget> OnGenerateDeviceProfileSelectionWidget(TSharedPtr<ENiagaraPlatformSelectionState> InItem);

	FText GetCurrentText() const;
	FText GetTooltipText() const;

	void GenerateQualityLevelSelectionWidgets();
	TSharedRef<SWidget> GenerateAdditionalDevicesWidgetForQL(int32 QualityLevel);

	bool IsTreeActiveForQL(const TSharedPtr<FNiagaraDeviceProfileViewModel>& Tree, int32 QualityLevelMask) const;
	void FilterTreeForQL(const TSharedPtr<FNiagaraDeviceProfileViewModel>& SourceTree, TSharedPtr<FNiagaraDeviceProfileViewModel>& FilteredTree, int32 QualityLevelMask);

	ECheckBoxState IsQLChecked(int32 QualityLevel) const;
	void QLCheckStateChanged(ECheckBoxState CheckState, int32 QualityLevel);

	FReply ToggleMenuOpenForQualityLevel(int32 QualityLevel);

	void UpdateCachedConflicts();
	void InvalidateSiblingConflicts() const;
	void OnPropertyValueChanged();

private:
	TArray<FNiagaraPlatformSetConflictInfo> CachedConflicts;
	TSharedPtr<IPropertyHandle> PlatformSetArrayProperty;
	TSharedPtr<IPropertyHandleArray> PlatformSetArray;
	int32 PlatformSetArrayIndex;

	TArray<TSharedPtr<SMenuAnchor>> QualityLevelMenuAnchors;
	TArray<TSharedPtr<SWidget>> QualityLevelMenuContents;
	TSharedPtr<SWrapBox> QualityLevelWidgetBox;

	TSharedPtr<IPropertyHandle> PropertyHandle;
	struct FNiagaraPlatformSet* TargetPlatformSet;

	TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>> FullDeviceProfileTree;
	TArray<TArray<TSharedPtr<FNiagaraDeviceProfileViewModel>>> FilteredDeviceProfileTrees;

	TArray<TSharedPtr<ENiagaraPlatformSelectionState>> PlatformSelectionStates;

	TSharedPtr<STreeView<TSharedPtr<FNiagaraDeviceProfileViewModel>>> DeviceProfileTreeWidget;

	struct FNiagaraSystemScalabilitySettingsArray* SystemScalabilitySettings;
	struct FNiagaraEmitterScalabilitySettingsArray* EmitterScalabilitySettings;
	struct FNiagaraSystemScalabilityOverrides* SystemScalabilityOverrides;
	struct FNiagaraEmitterScalabilityOverrides* EmitterScalabilityOverrides;
};

class FNiagaraPlatformSetCVarConditionCustomization : public IPropertyTypeCustomization
{
public:
	/** @return A new instance of this class */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraPlatformSetCVarConditionCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	/** IPropertyTypeCustomization interface end */

private:

	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<IPropertyHandle> CVarNameHandle;
	TSharedPtr<IPropertyHandle> BoolValueHandle;
	TSharedPtr<IPropertyHandle> MinIntHandle;
	TSharedPtr<IPropertyHandle> MaxIntHandle;
	TSharedPtr<IPropertyHandle> MinFloatHandle;
	TSharedPtr<IPropertyHandle> MaxFloatHandle;

	FName GetCurrentCVar();
	void OnTextCommitted(const FText& Text);

	FNiagaraPlatformSetCVarCondition* GetTargetCondition()const;
	FNiagaraPlatformSet* GetTargetPlatformSet()const;

	EVisibility BoolPropertyVisibility() const;
	EVisibility IntPropertyVisibility() const;
	EVisibility FloatPropertyVisibility() const;
};