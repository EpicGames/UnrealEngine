// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "NiagaraEditorStyle.h"
#include "ViewModels/NiagaraScriptInputCollectionViewModel.h"
#include "ViewModels/NiagaraScriptOutputCollectionViewModel.h"
#include "NiagaraParameterCollectionCustomNodeBuilder.h"
#include "ViewModels/NiagaraParameterCollectionViewModel.h"
#include "NiagaraScript.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ScopedTransaction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptDetails"

class SAddParameterButton : public SCompoundWidget
{	
	SLATE_BEGIN_ARGS(SAddParameterButton)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<INiagaraParameterCollectionViewModel> InViewModel)
	{
		CollectionViewModel = InViewModel;

		ChildSlot
		.HAlign(HAlign_Right)
		[
			SAssignNew(ComboButton, SComboButton)
			.HasDownArrow(false)
			.ButtonStyle(FAppStyle::Get(), "RoundButton")
			.ForegroundColor(FSlateColor::UseForeground())
			.OnGetMenuContent(this, &SAddParameterButton::GetAddParameterMenuContent)
			.Visibility(CollectionViewModel.ToSharedRef(), &INiagaraParameterCollectionViewModel::GetAddButtonVisibility)
			.ContentPadding(FMargin(2.0f, 1.0f, 0.0f, 1.0f))
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0, 1, 2, 1)
				.AutoWidth()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("Plus"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(CollectionViewModel.ToSharedRef(), &INiagaraParameterCollectionViewModel::GetAddButtonText)
					.Visibility(this, &SAddParameterButton::OnGetAddParameterTextVisibility)
					.ShadowOffset(FVector2D(1, 1))
				]
			]
		];
	}

private:
	EVisibility OnGetAddParameterTextVisibility() const
	{
		return IsHovered() || ComboButton->IsOpen() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	TSharedRef<SWidget> GetAddParameterMenuContent() const
	{
		FMenuBuilder AddMenuBuilder(true, nullptr);
		TSortedMap<FString, TArray<TSharedPtr<FNiagaraTypeDefinition>>> SubmenusToAdd;
		for (TSharedPtr<FNiagaraTypeDefinition> AvailableType : CollectionViewModel->GetAvailableTypesSorted())
		{
			FText SubmenuText = FNiagaraEditorUtilities::GetTypeDefinitionCategory(*AvailableType);
			if (SubmenuText.IsEmptyOrWhitespace())
			{
				AddMenuBuilder.AddMenuEntry
                (
                    AvailableType->GetNameText(),
                    FText(),
                    FSlateIcon(),
                    FUIAction(FExecuteAction::CreateSP(CollectionViewModel.ToSharedRef(), &INiagaraParameterCollectionViewModel::AddParameter, AvailableType))
                );
			}
			else
			{
				SubmenusToAdd.FindOrAdd(SubmenuText.ToString()).Add(AvailableType);
			}
		}
		for (const auto& Entry : SubmenusToAdd)
		{
			TArray<TSharedPtr<FNiagaraTypeDefinition>> SubmenuEntries = Entry.Value;
			AddMenuBuilder.AddSubMenu(FText::FromString(Entry.Key), FText(), FNewMenuDelegate::CreateLambda([SubmenuEntries, this](FMenuBuilder& InSubMenuBuilder)
            {
                for (TSharedPtr<FNiagaraTypeDefinition> AvailableType : SubmenuEntries)
                {
                    InSubMenuBuilder.AddMenuEntry
                    (
                        AvailableType->GetNameText(),
                        FText(),
                        FSlateIcon(),
                        FUIAction(FExecuteAction::CreateSP(CollectionViewModel.ToSharedRef(), &INiagaraParameterCollectionViewModel::AddParameter, AvailableType))
                    );
                }
            }));
		}
		return AddMenuBuilder.MakeWidget();
	}

	
private:
	TSharedPtr<INiagaraParameterCollectionViewModel> CollectionViewModel;
	TSharedPtr<SComboButton> ComboButton;
};

TSharedRef<IDetailCustomization> FNiagaraScriptDetails::MakeInstance()
{
	return MakeShared<FNiagaraScriptDetails>();
}

FNiagaraScriptDetails::FNiagaraScriptDetails()
{
}

void FNiagaraScriptDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() != 1 || ObjectsBeingCustomized[0]->IsA<UNiagaraScript>() == false)
	{
		return;
	}

	UNiagaraScript* ScriptBeingCustomized = CastChecked<UNiagaraScript>(ObjectsBeingCustomized[0]);
	TArray<TSharedPtr<FNiagaraScriptViewModel>> ExistingViewModels;
	TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>::GetAllViewModelsForObject(ScriptBeingCustomized, ExistingViewModels);
	
	if (ensureMsgf(ExistingViewModels.Num() == 1, TEXT("Missing or duplicate script view models detected.  Can not create script details customization.")))
	{
		ScriptViewModel = ExistingViewModels[0];
	}
	else
	{
		return;
	}

	static const FName InputParamCategoryName = TEXT("NiagaraScript_InputParams");
	static const FName OutputParamCategoryName = TEXT("NiagaraScript_OutputParams");
	static const FName ScriptCategoryName = TEXT("Script");

	TSharedPtr<IPropertyHandle> NumericOutputPropertyHandle = DetailBuilder.GetProperty(TEXT("NumericOutputTypeSelectionMode"));
	if (NumericOutputPropertyHandle.IsValid())
	{
		NumericOutputPropertyHandle->MarkHiddenByCustomization();
	}

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(ScriptCategoryName);

	FVersionedNiagaraScript StandaloneScript = ScriptViewModel->GetStandaloneScript();
	bool bAddParameters = false;
	TSharedPtr<INiagaraParameterCollectionViewModel> InputCollectionViewModel;
	TSharedPtr<INiagaraParameterCollectionViewModel> OutputCollectionViewModel;

	if (StandaloneScript.Script)
	{
		InputCollectionViewModel = ScriptViewModel->GetInputCollectionViewModel();
		OutputCollectionViewModel = ScriptViewModel->GetOutputCollectionViewModel();
		bAddParameters = true;

		if (FVersionedNiagaraScriptData* Data = StandaloneScript.Script->GetScriptData(StandaloneScript.Version))
		{
			TSharedPtr<FStructOnScope> StructData = MakeShareable(new FStructOnScope(FVersionedNiagaraScriptData::StaticStruct(), (uint8*)Data));
			StructData->SetPackage(StandaloneScript.Script.GetPackage());
			TArray<TSharedPtr<IPropertyHandle>> NewHandles = CategoryBuilder.AddAllExternalStructureProperties(StructData.ToSharedRef());

			for(TSharedPtr<IPropertyHandle> NewHandle : NewHandles)
			{
				NewHandle->SetOnPropertyValueChanged((FSimpleDelegate::CreateLambda([StandaloneScript, NewHandle]()
				{
					FPropertyChangedEvent ChangeEvent(NewHandle->GetProperty());
					StandaloneScript.Script->PostEditChangeVersionedProperty(ChangeEvent, StandaloneScript.Version);
				})));
			}
		}		
	}

	if (InputCollectionViewModel.IsValid())
	{
		IDetailCategoryBuilder& InputParamCategory = DetailBuilder.EditCategory(InputParamCategoryName, LOCTEXT("InputParamCategoryName", "Input Parameters"));
		if (bAddParameters)
		{
			InputParamCategory.HeaderContent(
				SNew(SBox)
				.Padding(FMargin(2, 2, 0, 2))
				.VAlign(VAlign_Center)
				[
					SNew(SAddParameterButton, InputCollectionViewModel.ToSharedRef())
				]);
		}
		InputParamCategory.AddCustomBuilder(MakeShared<FNiagaraParameterCollectionCustomNodeBuilder>(InputCollectionViewModel.ToSharedRef()));
	}

	if (OutputCollectionViewModel.IsValid())
	{
		IDetailCategoryBuilder& OutputParamCategory = DetailBuilder.EditCategory(OutputParamCategoryName, LOCTEXT("OutputParamCategoryName", "Output Parameters"));
		if (bAddParameters)
		{
			OutputParamCategory.HeaderContent(
				SNew(SBox)
				.Padding(FMargin(2, 2, 0, 2))
				.VAlign(VAlign_Center)
				[
					SNew(SAddParameterButton, OutputCollectionViewModel.ToSharedRef())
				]);
		}
		OutputParamCategory.AddCustomBuilder(MakeShared<FNiagaraParameterCollectionCustomNodeBuilder>(OutputCollectionViewModel.ToSharedRef()));
	}
}

float ButtonWidth = 50;
float DisplayColorHeight = 16;
float DisplayColorWidth = 24;
float EditColorHeight = 16;
float EditColorWidth = 30;

class SNiagaraDetailsButton : public SButton
{
	SLATE_BEGIN_ARGS(SNiagaraDetailsButton) {}
		SLATE_EVENT(FOnClicked, OnClicked)
		SLATE_ARGUMENT(FText, Text);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SButton::Construct(SButton::FArguments()
			.OnClicked(InArgs._OnClicked)
			.Content()
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(InArgs._Text)
				.Justification(ETextJustify::Center)
				.MinDesiredWidth(ButtonWidth)
			]);
	}
};

#undef LOCTEXT_NAMESPACE
