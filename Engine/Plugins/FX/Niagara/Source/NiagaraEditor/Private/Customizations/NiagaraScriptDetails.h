// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"

class FNiagaraScriptViewModel;
class INiagaraParameterCollectionViewModel;
class FNiagaraMetaDataCustomNodeBuilder;
class IPropertyHandle;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyTypeCustomizationUtils;

class FNiagaraScriptDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	FNiagaraScriptDetails();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder);

private:
	TSharedPtr<FNiagaraScriptViewModel> ScriptViewModel;
};
