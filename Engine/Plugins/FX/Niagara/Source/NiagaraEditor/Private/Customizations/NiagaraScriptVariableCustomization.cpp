// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "NiagaraScriptVariableCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "ScopedTransaction.h"
#include "SEnumCombo.h"

#include "EdGraphSchema_Niagara.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraGraph.h"
#include "NiagaraNode.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraScriptVariable.h"
#include "SNiagaraParameterEditor.h"
#include "GraphEditAction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScriptVariableCustomization)

#define LOCTEXT_NAMESPACE "NiagaraScriptVariableVariableDetails"

const FName FNiagaraScriptVariableDetails::DefaultValueCategoryName = TEXT("Default Value");


TSharedRef<IDetailCustomization> FNiagaraScriptVariableDetails::MakeInstance()
{
	return MakeShareable(new FNiagaraScriptVariableDetails());
}
 
FNiagaraScriptVariableDetails::FNiagaraScriptVariableDetails()
{
	bParameterValueChangedDuringOnValueChanged = false;
	LibraryDefaultModeValue = 0;

	LibrarySynchronizedDefaultModeEnum = StaticEnum<ENiagaraLibrarySynchronizedDefaultMode>();
	LibrarySourceDefaultModeEnum = StaticEnum<ENiagaraLibrarySourceDefaultMode>();
}

FNiagaraScriptVariableDetails::~FNiagaraScriptVariableDetails()
{
	if(ParameterMapWatcherDelegate.IsValid() && Variable.IsValid())
	{
		if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(Variable->GetOuter()))
		{
			Graph->RemoveOnGraphChangedHandler(ParameterMapWatcherDelegate);	
		}
	}
}

UEdGraphPin* FNiagaraScriptVariableDetails::GetAnyDefaultPin()
{
	TArray<UEdGraphPin*> AllPins = GetDefaultPins();
	if (AllPins.Num() > 0)
	{
		return AllPins[0];
	}
	return nullptr;
}

TArray<UEdGraphPin*> FNiagaraScriptVariableDetails::GetDefaultPins()
{
	if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(Variable->GetOuter()))
	{
		return Graph->FindParameterMapDefaultValuePins(Variable->Variable.GetName());
	}
	return TArray<UEdGraphPin*>();
}

void FNiagaraScriptVariableDetails::Refresh()
{
	IDetailLayoutBuilder* DetailBuilderPtr = nullptr;
	if (TSharedPtr<IDetailLayoutBuilder> LockedDetailBuilder = CachedDetailBuilder.Pin())
	{
		// WARNING: We do this because ForceRefresh will lock while pinning...
		DetailBuilderPtr = LockedDetailBuilder.Get();
	}
	if (DetailBuilderPtr != nullptr)
	{
		TSharedRef<IPropertyUtilities> PropertyUtilities = DetailBuilderPtr->GetPropertyUtilities();
		PropertyUtilities->ForceRefresh();
	}
}

void FNiagaraScriptVariableDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CachedDetailBuilder = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

void FNiagaraScriptVariableDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);

	if (ObjectsCustomized.Num() != 1)
	{
		// Only allow selecting one UNiagaraScriptVariable at a time.
		return;
	}
	if (!ObjectsCustomized[0]->IsA<UNiagaraScriptVariable>())
	{
		return;
	}

	Variable = Cast<UNiagaraScriptVariable>(ObjectsCustomized[0].Get());
	if (Variable == nullptr)
	{
		return;
	}
	
	const bool bInLibrary = Variable->GetOuter()->IsA<UNiagaraParameterDefinitions>();
	if (Variable->GetIsStaticSwitch())
	{
		CustomizeDetailsStaticSwitchScriptVariable(DetailBuilder);
	}
	else if (Variable->GetIsSubscribedToParameterDefinitions() || bInLibrary)
	{
		CustomizeDetailsParameterDefinitionsSynchronizedScriptVariable(DetailBuilder);
	}
	else
	{
		CustomizeDetailsGenericScriptVariable(DetailBuilder);
	}

	// Global show/hide logic
	if (Variable->DefaultMode != ENiagaraDefaultMode::Binding)
	{
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultBinding));
	}

	// Always hide the default value variant. We generate a node for this property to enable PostEditChangeProperty(...) events but do not modify it via the default generated customization.
	DetailBuilder.HideProperty("DefaultValueVariant", UNiagaraScriptVariable::StaticClass());

	if (!Variable->Variable.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.bInlineEditConditionToggle));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterBoolOverride));
	}	

	// we hide the enum overrides if the variable type isn't an enum
	if(!Variable->Variable.GetType().GetEnum())
	{
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterEnumOverrides));
	}
	
	// generally we want all the inline parameters only available for inputs & static switches
	if(!Variable->Variable.IsInNameSpace(FNiagaraConstants::ModuleNamespace) && !Variable->GetIsStaticSwitch())
	{
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.bDisplayInOverviewStack));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterSortPriority));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterColorOverride));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterEnumOverrides));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterBoolOverride));
	}
}

void FNiagaraScriptVariableDetails::CustomizeDetailsGenericScriptVariable(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(DefaultValueCategoryName);

	// NOTE: Automatically generated widgets from UProperties are placed below custom properties by default. 
	//       In this case DefaultMode is just a built in combo box, while value widget is custom and added afterwards. 
	//       This guarantees that the combo box always show above the value widget instead of at the bottom of the window. 
	const TSharedPtr<IPropertyHandle> DefaultModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultMode), UNiagaraScriptVariable::StaticClass());
	DefaultModeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FNiagaraScriptVariableDetails::OnComboValueChanged));
	DetailBuilder.HideProperty(DefaultModeHandle);
	DetailBuilder.AddPropertyToCategory(DefaultModeHandle);
	AddGraphDefaultValueCustomRow(CategoryBuilder);
}

void FNiagaraScriptVariableDetails::CustomizeDetailsStaticSwitchScriptVariable(IDetailLayoutBuilder& DetailBuilder)
{
	FNiagaraEditorModule& EditorModule = FNiagaraEditorModule::Get();
	TypeUtilityStaticSwitchValue = EditorModule.GetTypeUtilities(Variable->Variable.GetType());
	if (TypeUtilityStaticSwitchValue && TypeUtilityStaticSwitchValue->CanCreateParameterEditor())
	{
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(DefaultValueCategoryName);
		ParameterEditorStaticSwitchValue = TypeUtilityStaticSwitchValue->CreateParameterEditor(Variable->Variable.GetType());

		TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
		if (Variable->GetDefaultValueData() != nullptr)
		{
			Variable->Variable.SetData(Variable->GetDefaultValueData());
		}
		Variable->Variable.CopyTo(ParameterValue->GetStructMemory());
		ParameterEditorStaticSwitchValue->UpdateInternalValueFromStruct(ParameterValue.ToSharedRef());
		ParameterEditorStaticSwitchValue->SetOnValueChanged(SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &FNiagaraScriptVariableDetails::OnStaticSwitchValueChanged));

		FDetailWidgetRow& DefaultValueWidget = CategoryBuilder.AddCustomRow(LOCTEXT("DefaultValueFilterText", "Default Value"));
		DefaultValueWidget.NameContent()
		[
			SNew(STextBlock)
			.Font(FNiagaraEditorStyle::Get().GetFontStyle("NiagaraEditor.ParameterFont"))
			.Text(FText::FromString(TEXT("Default Value")))
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			ParameterEditorStaticSwitchValue.ToSharedRef()
		];
	}
	else
	{
		TypeUtilityStaticSwitchValue = nullptr;
	}

	// Hide metadata UProperties that aren't useful for static switch variables
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.EditCondition));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.VisibleCondition));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultBinding));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultMode));
}

void FNiagaraScriptVariableDetails::CustomizeDetailsParameterDefinitionsSynchronizedScriptVariable(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(DefaultValueCategoryName);
	FNiagaraEditorModule& EditorModule = FNiagaraEditorModule::Get();
	UObject* VariableOuter = Variable->GetOuter();
	const bool bInLibraryAsset = VariableOuter->IsA<UNiagaraParameterDefinitions>();
	/**
	 *  Default mode is handled as a special case for UNiagaraScriptVariables synchronized with Parameter Libraries. 
	 * 	Hide the actual default mode and instead edit a library default mode in place which maps to the actual default mode and whether or not to override the library default.
	 */
	const TSharedPtr<IPropertyHandle> DefaultModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultMode), UNiagaraScriptVariable::StaticClass());
	DetailBuilder.HideProperty(DefaultModeHandle);
	if (bInLibraryAsset)
	{		
		LibraryDefaultModeValue = GetLibrarySourcedDefaultModeInitialValue();
		FDetailWidgetRow& DefaultModeWidget = CategoryBuilder.AddCustomRow(LOCTEXT("DefaultModeFilterText", "Default Mode"));
		DefaultModeWidget.NameContent()
		[
			SNew(STextBlock)
			.Font(FNiagaraEditorStyle::Get().GetFontStyle("NiagaraEditor.ParameterFont"))
			.Text(FText::FromString(TEXT("Default Mode")))
		]
		.ValueContent()
		[
			SNew(SEnumComboBox, LibrarySourceDefaultModeEnum)
			.CurrentValue(this, &FNiagaraScriptVariableDetails::GetLibraryDefaultModeValue)
			.ContentPadding(FMargin(2, 0))
			.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
			.OnEnumSelectionChanged(SEnumComboBox::FOnEnumSelectionChanged::CreateSP(this, &FNiagaraScriptVariableDetails::OnLibrarySourceDefaultModeChanged))
		];
	}
	else
	{
		LibraryDefaultModeValue = GetLibrarySynchronizedDefaultModeInitialValue();
		FDetailWidgetRow& DefaultModeWidget = CategoryBuilder.AddCustomRow(LOCTEXT("DefaultModeFilterText", "Default Mode"));
		DefaultModeWidget.NameContent()
		[
			SNew(STextBlock)
			.Font(FNiagaraEditorStyle::Get().GetFontStyle("NiagaraEditor.ParameterFont"))
			.Text(FText::FromString(TEXT("Default Mode")))
		]
		.ValueContent()
		[
			SNew(SEnumComboBox, LibrarySynchronizedDefaultModeEnum)
			.CurrentValue(this, &FNiagaraScriptVariableDetails::GetLibraryDefaultModeValue)
			.ContentPadding(FMargin(2, 0))
			.Font(FAppStyle::GetFontStyle("Sequencer.AnimationOutliner.RegularFont"))
			.OnEnumSelectionChanged(SEnumComboBox::FOnEnumSelectionChanged::CreateSP(this, &FNiagaraScriptVariableDetails::OnLibrarySynchronizedDefaultModeChanged))
		];
	}
		
	/** If the default mode is Value, create a default value editor. */
	if (Variable->DefaultMode == ENiagaraDefaultMode::Value)
	{
		if (bInLibraryAsset == false && Variable->GetIsOverridingParameterDefinitionsDefaultValue())
		{
			// If overriding the parameter definitions default value and not in a library, use the graph default value custom row as it modifies the pin default value.
			AddGraphDefaultValueCustomRow(CategoryBuilder);
		}
		else
		{
			// If not overriding the parameter definitions default value, use the library default value custom row as it modifies the script var default value variant.
			AddLibraryDefaultValueCustomRow(CategoryBuilder, bInLibraryAsset);
		}
	}
}

void FNiagaraScriptVariableDetails::OnGraphChanged(const FEdGraphEditAction& EdGraphEditAction,	TWeakObjectPtr<UNiagaraScriptVariable> ScriptVariable)
{
	if(!ScriptVariable.IsValid())
	{
		return;
	}
	
	if(GetAnyDefaultPin() != nullptr)
	{
		Refresh();
		if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(Variable->GetOuter()))
		{
			Graph->RemoveOnGraphChangedHandler(ParameterMapWatcherDelegate);	
		}
	}
}

void FNiagaraScriptVariableDetails::AddGraphDefaultValueCustomRow(IDetailCategoryBuilder& CategoryBuilder)
{
	if (UEdGraphPin* Pin = GetAnyDefaultPin())
	{
		FNiagaraEditorModule& EditorModule = FNiagaraEditorModule::Get();
		TypeUtilityValue = EditorModule.GetTypeUtilities(Variable->Variable.GetType());
		if (TypeUtilityValue && TypeUtilityValue->CanCreateParameterEditor() && Variable->DefaultMode == ENiagaraDefaultMode::Value)
		{
			ParameterEditorValue = TypeUtilityValue->CreateParameterEditor(Variable->Variable.GetType());

			TypeUtilityValue->SetValueFromPinDefaultString(Pin->DefaultValue, Variable->Variable);
			TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
			Variable->Variable.CopyTo(ParameterValue->GetStructMemory());
			ParameterEditorValue->UpdateInternalValueFromStruct(ParameterValue.ToSharedRef());
			ParameterEditorValue->SetOnValueChanged(SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &FNiagaraScriptVariableDetails::OnValueChanged));
			ParameterEditorValue->SetOnBeginValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &FNiagaraScriptVariableDetails::OnBeginValueChanged));
			ParameterEditorValue->SetOnEndValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &FNiagaraScriptVariableDetails::OnEndValueChanged));

			FDetailWidgetRow& DefaultValueWidget = CategoryBuilder.AddCustomRow(LOCTEXT("DefaultValueFilterText", "Default Value"));
			DefaultValueWidget.NameContent()
			[
				SNew(STextBlock)
				.Font(FNiagaraEditorStyle::Get().GetFontStyle("NiagaraEditor.ParameterFont"))
				.Text(FText::FromString(TEXT("Default Value")))
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			[
				ParameterEditorValue.ToSharedRef()
			];
		}
		else
		{
			TypeUtilityValue = nullptr;
		}
	}
	else
	{
		if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(Variable->GetOuter()))
		{
			ParameterMapWatcherDelegate = Graph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraScriptVariableDetails::OnGraphChanged, Variable));
		}
		
		if (Variable->DefaultMode == ENiagaraDefaultMode::Value)
		{
			FDetailWidgetRow& DefaultValueWidget = CategoryBuilder.AddCustomRow(LOCTEXT("DefaultValueFilterText", "Default Value"));
			DefaultValueWidget.WholeRowContent()
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Font(FNiagaraEditorStyle::Get().GetFontStyle("NiagaraEditor.ParameterFont"))
				.Text(NSLOCTEXT("ScriptVariableCustomization", "MissingDefaults", "To set the default value, add this parameter to a Parameter Map Get node."))
			];
		}
	}
}

void FNiagaraScriptVariableDetails::AddLibraryDefaultValueCustomRow(IDetailCategoryBuilder& CategoryBuilder, bool bInLibraryAsset)
{
	FNiagaraEditorModule& EditorModule = FNiagaraEditorModule::Get();
	TypeUtilityLibraryValue = EditorModule.GetTypeUtilities(Variable->Variable.GetType());
	if (TypeUtilityLibraryValue && TypeUtilityLibraryValue->CanCreateParameterEditor() && Variable->DefaultMode == ENiagaraDefaultMode::Value)
	{
		ParameterEditorLibraryValue = TypeUtilityLibraryValue->CreateParameterEditor(Variable->Variable.GetType());

		TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
		Variable->CopyDefaultValueDataTo(ParameterValue->GetStructMemory());
		ParameterEditorLibraryValue->SetEnabled(bInLibraryAsset);
		ParameterEditorLibraryValue->UpdateInternalValueFromStruct(ParameterValue.ToSharedRef());
		ParameterEditorLibraryValue->SetOnBeginValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &FNiagaraScriptVariableDetails::OnBeginLibraryValueChanged));
		ParameterEditorLibraryValue->SetOnValueChanged(SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &FNiagaraScriptVariableDetails::OnLibraryValueChanged));
		ParameterEditorLibraryValue->SetOnEndValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &FNiagaraScriptVariableDetails::OnEndLibraryValueChanged));

		FDetailWidgetRow& DefaultValueWidget = CategoryBuilder.AddCustomRow(LOCTEXT("DefaultValueFilterText", "Default Value"));
		DefaultValueWidget.NameContent()
		[
			SNew(STextBlock)
			.IsEnabled(bInLibraryAsset)
			.Font(FNiagaraEditorStyle::Get().GetFontStyle("NiagaraEditor.ParameterFont"))
			.Text(FText::FromString(TEXT("Default Value")))
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			ParameterEditorLibraryValue.ToSharedRef()
		];
	}
	else
	{
		TypeUtilityLibraryValue = nullptr;
	}
}

void FNiagaraScriptVariableDetails::OnComboValueChanged()
{
	if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(Variable->GetOuter()))
	{
		Graph->ScriptVariableChanged(this->Variable->Variable);
	}
	Refresh();

#if WITH_EDITOR
	if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(Variable->GetOuter()))
	{
		Graph->NotifyGraphNeedsRecompile();
	}
#endif	//#if WITH_EDITOR
}

void FNiagaraScriptVariableDetails::OnValueChanged()
{
	if (TypeUtilityValue && ParameterEditorValue && GetDefaultPins().Num() > 0)
	{
		if (!ParameterEditorValue->CanChangeContinuously())
		{
			const FScopedTransaction Transaction(NSLOCTEXT("ScriptVariableCustomization", "ChangeValue", "Change Default Value"));
			Variable->Modify();

			TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
			ParameterEditorValue->UpdateStructFromInternalValue(ParameterValue.ToSharedRef());
			Variable->Variable.SetData(ParameterValue->GetStructMemory());
			const FString NewDefaultValue = TypeUtilityValue->GetPinDefaultStringFromValue(Variable->Variable);

			for (UEdGraphPin* Pin : GetDefaultPins())
			{
				Pin->Modify();
				GetDefault<UEdGraphSchema_Niagara>()->TrySetDefaultValue(*Pin, NewDefaultValue, true);
			}
		}
		else
		{
			TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
			ParameterEditorValue->UpdateStructFromInternalValue(ParameterValue.ToSharedRef());
			Variable->Variable.SetData(ParameterValue->GetStructMemory());
			const FString NewDefaultValue = TypeUtilityValue->GetPinDefaultStringFromValue(Variable->Variable);

			for (UEdGraphPin* Pin : GetDefaultPins())
			{
				if (Pin->GetDefaultAsString() != NewDefaultValue)
				{
					bParameterValueChangedDuringOnValueChanged = true;
					GetDefault<UEdGraphSchema_Niagara>()->TrySetDefaultValue(*Pin, NewDefaultValue, false);
				}
			}
		}
	}
}
 
void FNiagaraScriptVariableDetails::OnBeginValueChanged()
{
	bParameterValueChangedDuringOnValueChanged = false;
	if (!ParameterEditorValue->CanChangeContinuously())
	{
		return;
	}

	if (TypeUtilityValue && ParameterEditorValue && GetDefaultPins().Num() > 0)
	{
		GEditor->BeginTransaction(NSLOCTEXT("ScriptVariableCustomization", "ChangeValue", "Change Default Value"));
		Variable->Modify();
		TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
		ParameterEditorValue->UpdateStructFromInternalValue(ParameterValue.ToSharedRef());
		Variable->Variable.SetData(ParameterValue->GetStructMemory());
		const FString NewDefaultValue = TypeUtilityValue->GetPinDefaultStringFromValue(Variable->Variable);

		for (UEdGraphPin* Pin : GetDefaultPins())
		{
			if (Pin->GetDefaultAsString() != NewDefaultValue)
			{
				bParameterValueChangedDuringOnValueChanged = true;
				Pin->Modify();
				GetDefault<UEdGraphSchema_Niagara>()->TrySetDefaultValue(*Pin, NewDefaultValue, false);
			}
		}
	}
}

void FNiagaraScriptVariableDetails::OnEndValueChanged()
{
	if (GEditor->IsTransactionActive())
	{
		// If the parameter value was actually changed, invoke PinDefaultValueChanged() on the target pin(s) owning Node(s) to propagate the change.
		if (bParameterValueChangedDuringOnValueChanged)
		{
			bParameterValueChangedDuringOnValueChanged = false;
			for (UEdGraphPin* Pin : GetDefaultPins())
			{
				Pin->GetOwningNode()->PinDefaultValueChanged(Pin);
			}
		}
		GEditor->EndTransaction();
	}
}

void FNiagaraScriptVariableDetails::OnStaticSwitchValueChanged()
{
	if (TypeUtilityStaticSwitchValue && ParameterEditorStaticSwitchValue)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("ScriptVariableCustomization", "ChangeStaticSwitchValue", "Change Static Switch Default Value"));
		Variable->Modify();
		TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
		ParameterEditorStaticSwitchValue->UpdateStructFromInternalValue(ParameterValue.ToSharedRef());
		Variable->Variable.SetData(ParameterValue->GetStructMemory());
		Variable->SetStaticSwitchDefaultValue(Variable->Variable.GetValue<int>()); 
		
#if WITH_EDITOR
		if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(Variable->GetOuter()))
		{
			Graph->NotifyGraphNeedsRecompile();
		}
#endif	//#if WITH_EDITOR
	} 
}

void FNiagaraScriptVariableDetails::OnBeginLibraryValueChanged()
{
	if (!ParameterEditorLibraryValue->CanChangeContinuously())
	{
		return;
	}

	if (TypeUtilityLibraryValue && ParameterEditorLibraryValue && CachedDetailBuilder.IsValid())
	{
		const TSharedPtr<IPropertyHandle> DefaultValueHandle = CachedDetailBuilder.Pin()->GetProperty("DefaultValueVariant", UNiagaraScriptVariable::StaticClass());
		
		TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
		ParameterEditorLibraryValue->UpdateStructFromInternalValue(ParameterValue.ToSharedRef());
		 
		if (FMemory::Memcmp(Variable->GetDefaultValueData(), ParameterValue->GetStructMemory(), Variable->Variable.GetType().GetSize()) != 0)
		{
			GEditor->BeginTransaction(NSLOCTEXT("ScriptVariableCustomization", "ChangeLibraryValue", "Change Default Value"));
			DefaultValueHandle->NotifyPreChange();
			Variable->Modify();
			Variable->SetDefaultValueData(ParameterValue->GetStructMemory());
		}
	}
}

void FNiagaraScriptVariableDetails::OnEndLibraryValueChanged()
{
	Variable->UpdateChangeId();

	if (TypeUtilityLibraryValue && ParameterEditorLibraryValue && CachedDetailBuilder.IsValid())
	{
		const TSharedPtr<IPropertyHandle> DefaultValueHandle = CachedDetailBuilder.Pin()->GetProperty("DefaultValueVariant", UNiagaraScriptVariable::StaticClass());
		DefaultValueHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		DefaultValueHandle->NotifyFinishedChangingProperties();
	}

	if (UNiagaraParameterDefinitions* OuterParameterDefinitions = Cast<UNiagaraParameterDefinitions>(Variable->GetOuter()))
	{
		OuterParameterDefinitions->NotifyParameterDefinitionsChanged();
	}

	OnEndValueChanged();
}

void FNiagaraScriptVariableDetails::OnLibraryValueChanged()
{
	if (TypeUtilityLibraryValue && ParameterEditorLibraryValue && CachedDetailBuilder.IsValid())
	{
		const TSharedPtr<IPropertyHandle> DefaultValueHandle = CachedDetailBuilder.Pin()->GetProperty("DefaultValueVariant", UNiagaraScriptVariable::StaticClass());
		if (!ParameterEditorLibraryValue->CanChangeContinuously())
		{
			const FScopedTransaction Transaction(NSLOCTEXT("ScriptVariableCustomization", "ChangeLibraryValue", "Change Default Value"));
			DefaultValueHandle->NotifyPreChange();
			Variable->Modify();
			TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
			ParameterEditorLibraryValue->UpdateStructFromInternalValue(ParameterValue.ToSharedRef());
			Variable->SetDefaultValueData(ParameterValue->GetStructMemory());

			if (UNiagaraParameterDefinitions* OuterParameterDefinitions = Cast<UNiagaraParameterDefinitions>(Variable->GetOuter()))
			{
				OuterParameterDefinitions->NotifyParameterDefinitionsChanged();
			}
			DefaultValueHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			DefaultValueHandle->NotifyFinishedChangingProperties();
		}
		else
		{
			DefaultValueHandle->NotifyPreChange();
			TSharedPtr<FStructOnScope> ParameterValue = MakeShareable(new FStructOnScope(Variable->Variable.GetType().GetStruct()));
			ParameterEditorLibraryValue->UpdateStructFromInternalValue(ParameterValue.ToSharedRef());
			Variable->SetDefaultValueData(ParameterValue->GetStructMemory());
		}
	}
}

int32 FNiagaraScriptVariableDetails::GetLibrarySourcedDefaultModeInitialValue() const
{
	switch (Variable->DefaultMode) {
	case ENiagaraDefaultMode::Value:
		return static_cast<int32>(ENiagaraLibrarySourceDefaultMode::Value);

	case ENiagaraDefaultMode::Binding:
		return static_cast<int32>(ENiagaraLibrarySourceDefaultMode::Binding);

	case ENiagaraDefaultMode::FailIfPreviouslyNotSet:
		return static_cast<int32>(ENiagaraLibrarySourceDefaultMode::FailIfPreviouslyNotSet);

	default:
		ensureMsgf(false, TEXT("Encountered invalid ENiagaraDefaultMode for definition sourced script variable!"));
		return 0;
	}
}

int32 FNiagaraScriptVariableDetails::GetLibrarySynchronizedDefaultModeInitialValue() const
{
	if (Variable->GetIsOverridingParameterDefinitionsDefaultValue() == false)
	{
		return static_cast<int32>(ENiagaraLibrarySynchronizedDefaultMode::Definition);
	}

	switch (Variable->DefaultMode) {
	case ENiagaraDefaultMode::Value:
		return static_cast<int32>(ENiagaraLibrarySynchronizedDefaultMode::Value);

	case ENiagaraDefaultMode::Binding:
		return static_cast<int32>(ENiagaraLibrarySynchronizedDefaultMode::Binding);

	case ENiagaraDefaultMode::Custom:
		return static_cast<int32>(ENiagaraLibrarySynchronizedDefaultMode::Custom);

	case ENiagaraDefaultMode::FailIfPreviouslyNotSet:
		return static_cast<int32>(ENiagaraLibrarySynchronizedDefaultMode::FailIfPreviouslyNotSet);

	default:
		ensureMsgf(false, TEXT("Encountered invalid ENiagaraDefaultMode for definition linked script variable!"));
		return 0;
	}
}

void FNiagaraScriptVariableDetails::OnLibrarySourceDefaultModeChanged(int32 InValue, ESelectInfo::Type InSelectInfo)
{
	if (CachedDetailBuilder.IsValid())
	{
		const TSharedPtr<IPropertyHandle> DefaultModeHandle = CachedDetailBuilder.Pin()->GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultMode), UNiagaraScriptVariable::StaticClass());
		const FScopedTransaction Transaction(NSLOCTEXT("ScriptVariableCustomization", "ChangeDefaultMode", "Change Default Mode"));

		Variable->Modify();
		DefaultModeHandle->NotifyPreChange();

		const ENiagaraLibrarySourceDefaultMode LibraryDefaultMode = ENiagaraLibrarySourceDefaultMode(InValue);
		switch (LibraryDefaultMode) {
		case ENiagaraLibrarySourceDefaultMode::Value:
			Variable->DefaultMode = ENiagaraDefaultMode::Value;
			break;

		case ENiagaraLibrarySourceDefaultMode::Binding:
			Variable->DefaultMode = ENiagaraDefaultMode::Binding;
			break;

		case ENiagaraLibrarySourceDefaultMode::FailIfPreviouslyNotSet:
			Variable->DefaultMode = ENiagaraDefaultMode::FailIfPreviouslyNotSet;
			break;

		default:
			ensureMsgf(false, TEXT("Encountered unknown ENiagaraLibrarySourceDefaultMode value!"));
			return;
		}

		DefaultModeHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		DefaultModeHandle->NotifyFinishedChangingProperties();

		OnComboValueChanged();
	}
}

void FNiagaraScriptVariableDetails::OnLibrarySynchronizedDefaultModeChanged(int32 InValue, ESelectInfo::Type InSelectInfo)
{
	if (CachedDetailBuilder.IsValid())
	{
		const TSharedPtr<IPropertyHandle> DefaultModeHandle = CachedDetailBuilder.Pin()->GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultMode), UNiagaraScriptVariable::StaticClass());
		const FScopedTransaction Transaction(NSLOCTEXT("ScriptVariableCustomization", "ChangeLibraryDefaultMode", "Change Default Mode"));

		Variable->Modify();
		DefaultModeHandle->NotifyPreChange();

		const ENiagaraLibrarySynchronizedDefaultMode LibraryDefaultMode = ENiagaraLibrarySynchronizedDefaultMode(InValue);
		switch (LibraryDefaultMode) {
		case ENiagaraLibrarySynchronizedDefaultMode::Definition:
			Variable->SetIsOverridingParameterDefinitionsDefaultValue(false);
			// Special case if we're switching back to synchronizing with the parameter definitions, synchronize the library default value immediately.
			FNiagaraStackGraphUtilities::SynchronizeVariableToLibraryAndApplyToGraph(Variable.Get());
			break;

		case ENiagaraLibrarySynchronizedDefaultMode::Value:
			Variable->SetIsOverridingParameterDefinitionsDefaultValue(true);
			Variable->DefaultMode = ENiagaraDefaultMode::Value;
			break;

		case ENiagaraLibrarySynchronizedDefaultMode::Binding:
			Variable->SetIsOverridingParameterDefinitionsDefaultValue(true);
			Variable->DefaultMode = ENiagaraDefaultMode::Binding;
			break;

		case ENiagaraLibrarySynchronizedDefaultMode::Custom:
			Variable->SetIsOverridingParameterDefinitionsDefaultValue(true);
			Variable->DefaultMode = ENiagaraDefaultMode::Custom;
			break;

		case ENiagaraLibrarySynchronizedDefaultMode::FailIfPreviouslyNotSet:
			Variable->SetIsOverridingParameterDefinitionsDefaultValue(true);
			Variable->DefaultMode = ENiagaraDefaultMode::FailIfPreviouslyNotSet;
			break;

		default:
			ensureMsgf(false, TEXT("Encountered unknown ENiagaraLibrarySynchronizedDefaultMode value!"));
			return;
		}

		DefaultModeHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		DefaultModeHandle->NotifyFinishedChangingProperties();

		OnComboValueChanged();
	}
}

TSharedRef<IDetailCustomization> FNiagaraScriptVariableHierarchyDetails::MakeInstance()
{
	return MakeShared<FNiagaraScriptVariableHierarchyDetails>();
}

FNiagaraScriptVariableHierarchyDetails::FNiagaraScriptVariableHierarchyDetails()
{
}

void FNiagaraScriptVariableHierarchyDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);

	if (ObjectsCustomized.Num() != 1)
	{
		// Only allow selecting one UNiagaraScriptVariable at a time.
		return;
	}
	if (!ObjectsCustomized[0]->IsA<UNiagaraScriptVariable>())
	{
		return;
	}

	UNiagaraScriptVariable* Variable = Cast<UNiagaraScriptVariable>(ObjectsCustomized[0].Get());
	if (Variable == nullptr)
	{
		return;
	}

	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultMode));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultBinding));
	// member isn't public so we can't use GET_MEMBER_NAME_CHECKED
	DetailBuilder.HideProperty("DefaultValueVariant", UNiagaraScriptVariable::StaticClass());
	
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.DisplayUnit));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.bAdvancedDisplay));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.bOverrideColor));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.bEnableBoolOverride));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.bInlineEditConditionToggle));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.EditCondition));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.VisibleCondition));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.PropertyMetaData));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.bDisplayInOverviewStack));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterSortPriority));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterEnumOverrides));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterColorOverride));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.InlineParameterBoolOverride));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata.AlternateAliases));
}

#undef LOCTEXT_NAMESPACE

