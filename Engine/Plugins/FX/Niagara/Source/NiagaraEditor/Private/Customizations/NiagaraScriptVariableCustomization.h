// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
 
#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "IDetailCustomization.h"
#include "EditorUndoClient.h"
#include "Types/SlateEnums.h"

#include "NiagaraScriptVariableCustomization.generated.h"
 
class IDetailCategoryBuilder;
class UEdGraphPin;
struct FEdGraphEditAction;

/** Intermediate representations for default mode set on parameter definition script variables. Maps to ENiagaraDefaultMode and bOverrideParameterDefinitionsDefaultValue of UNiagaraScriptVariable. */
UENUM()
enum class ENiagaraLibrarySynchronizedDefaultMode : uint8
{
	// Synchronize with the default value as defined in the synchronized parameter definitions.
	Definition = 0,
	// Default initialize using a value widget in the Selected Details panel. Overrides the parameter definition default value.
	Value,
	// Default initialize using a dropdown widget in the Selected Details panel. Overrides the parameter definition default value.
	Binding,
	// Default initialization is done using a sub-graph. Overrides the parameter definition default value.
	Custom,
	// Fail compilation if this value has not been set previously in the stack. Overrides the parameter definition default value.
	FailIfPreviouslyNotSet
};

UENUM()
enum class ENiagaraLibrarySourceDefaultMode : uint8
{
	// Default initialize using a value widget in the Selected Details panel.
	Value = 0,
	// Default initialize using a dropdown widget in the Selected Details panel.
	Binding,
	// Fail compilation if this value has not been set previously in the stack.
	FailIfPreviouslyNotSet
};

/** This customization sets up a custom details panel for the static switch Variable in the niagara module graph. */
class FNiagaraScriptVariableDetails : public IDetailCustomization
{
public:
 
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
 
	~FNiagaraScriptVariableDetails();
	FNiagaraScriptVariableDetails();
 
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	virtual void CustomizeDetails( const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder ) override;

	// Fully regenerates the details view.
	void Refresh();
 
private:
	void CustomizeDetailsGenericScriptVariable(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeDetailsStaticSwitchScriptVariable(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeDetailsParameterDefinitionsSynchronizedScriptVariable(IDetailLayoutBuilder& DetailBuilder);

	void OnGraphChanged(const FEdGraphEditAction& EdGraphEditAction, TWeakObjectPtr<class UNiagaraScriptVariable> ScriptVariable);
	
	void AddGraphDefaultValueCustomRow(IDetailCategoryBuilder& CategoryBuilder);
	void AddLibraryDefaultValueCustomRow(IDetailCategoryBuilder& CategoryBuilder, bool bInLibraryAsset);

	void OnComboValueChanged();
	void OnBeginValueChanged();
	void OnEndValueChanged();
	void OnValueChanged();

	void OnStaticSwitchValueChanged();

	void OnBeginLibraryValueChanged();
	void OnEndLibraryValueChanged();
	void OnLibraryValueChanged();

	int32 GetLibraryDefaultModeValue() const { return LibraryDefaultModeValue; };
	int32 GetLibrarySourcedDefaultModeInitialValue() const;
	int32 GetLibrarySynchronizedDefaultModeInitialValue() const;
	void OnLibrarySourceDefaultModeChanged(int32 InValue, ESelectInfo::Type InSelectInfo);
	void OnLibrarySynchronizedDefaultModeChanged(int32 InValue, ESelectInfo::Type InSelectInfo);

	bool CanEditCustomDefaultModeRow(bool bVParameterDefinitionsScriptVar) const;
	bool CanEditCustomValueRow(bool bParameterDefinitionsScriptVar) const;

	UEdGraphPin* GetAnyDefaultPin();
	TArray<UEdGraphPin*> GetDefaultPins();

private:
	// Bool to track whether a parameters value has actually changed at any point during OnBeginValueChanged(), OnValueChanged() and OnEndValueChanged() to prevent excessive refreshing.
	bool bParameterValueChangedDuringOnValueChanged;

	FDelegateHandle ParameterMapWatcherDelegate;
	
	// Cached value of enum selector for choosing the library default mode.
	int32 LibraryDefaultModeValue;

	UEnum* LibrarySynchronizedDefaultModeEnum;
	UEnum* LibrarySourceDefaultModeEnum;

	TWeakPtr<class IDetailLayoutBuilder> CachedDetailBuilder;

	TWeakObjectPtr<class UNiagaraScriptVariable> Variable;
	TSharedPtr<class INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeUtilityValue;
	TSharedPtr<class INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeUtilityLibraryValue;
	TSharedPtr<class INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeUtilityStaticSwitchValue;
	TSharedPtr<class SNiagaraParameterEditor> ParameterEditorValue;
	TSharedPtr<class SNiagaraParameterEditor> ParameterEditorLibraryValue;
	TSharedPtr<class SNiagaraParameterEditor> ParameterEditorStaticSwitchValue;

	static const FName DefaultValueCategoryName;
};

/** This customization sets up a custom details panel for script variables found in the hierarchy editor. */
class FNiagaraScriptVariableHierarchyDetails : public IDetailCustomization
{
public:
 
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
 
	FNiagaraScriptVariableHierarchyDetails();
	virtual ~FNiagaraScriptVariableHierarchyDetails() override = default;
 
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};

