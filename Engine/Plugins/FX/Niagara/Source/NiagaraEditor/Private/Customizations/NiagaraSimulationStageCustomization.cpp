// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimulationStageCustomization.h"

#include "Modules/ModuleManager.h"

//Customization
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
//Widgets
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNiagaraDebugger.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
///Niagara
#include "NiagaraEditorModule.h"

#define LOCTEXT_NAMESPACE "NiagaraSimulationStageCustomization"

FNiagaraSimulationStageGenericCustomization::~FNiagaraSimulationStageGenericCustomization()
{
	if (UNiagaraSimulationStageGeneric* SimStage = WeakSimStage.Get())
	{
		SimStage->OnChanged().RemoveAll(this);
	}
}

TSharedRef<IDetailCustomization> FNiagaraSimulationStageGenericCustomization::MakeInstance()
{
	return MakeShareable(new FNiagaraSimulationStageGenericCustomization());
}

void FNiagaraSimulationStageGenericCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	WeakDetailBuilder = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

void FNiagaraSimulationStageGenericCustomization::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	// We only support customization on 1 object
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	if (ObjectsCustomized.Num() != 1 || !ObjectsCustomized[0]->IsA<UNiagaraSimulationStageGeneric>())
	{
		return;
	}

	UNiagaraSimulationStageGeneric* SimStage = CastChecked<UNiagaraSimulationStageGeneric>(ObjectsCustomized[0]);
	WeakSimStage = SimStage;

	SimStage->OnChanged().AddRaw(this, &FNiagaraSimulationStageGenericCustomization::OnPropertyChanged);

	static const FName NAME_SimulationStage = TEXT("Simulation Stage");
	static const FName NAME_ParticleParameters = TEXT("Particle Parameters");
	static const FName NAME_DataInterfaceParameters = TEXT("DataInterface Parameters");
	static const FName NAME_DirectSetParameters = TEXT("Direct Set Parameters");
	static const FName NAME_DispatchParameters = TEXT("Dispatch Parameters");
	IDetailCategoryBuilder& SimStageCategory = DetailBuilder.EditCategory(NAME_SimulationStage);
	IDetailCategoryBuilder& ParticleStageCategory = DetailBuilder.EditCategory(NAME_ParticleParameters);
	IDetailCategoryBuilder& DataInterfaceStageCategory = DetailBuilder.EditCategory(NAME_DataInterfaceParameters);
	IDetailCategoryBuilder& DirectSetStageCategory = DetailBuilder.EditCategory(NAME_DirectSetParameters);
	IDetailCategoryBuilder& DispatchParametersCategory = DetailBuilder.EditCategory(NAME_DispatchParameters);

	// Hide all properties by default
	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	SimStageCategory.GetDefaultProperties(CategoryProperties);
	for (TSharedRef<IPropertyHandle>& PropertyHandle : CategoryProperties)
	{
		const FName PropertyName = PropertyHandle->GetProperty() ? PropertyHandle->GetProperty()->GetFName() : NAME_None;
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageBase, SimulationStageName))
		{
			SimStageCategory.AddProperty(PropertyHandle);
		}
		else
		{
			DetailBuilder.HideProperty(PropertyHandle);
		}
	}

	// Main Category
	SimStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, EnabledBinding)));
	SimStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, IterationSource)));
	SimStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, NumIterations)));
	SimStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ExecuteBehavior)));

	// Paterial Iteration Category
	switch (SimStage->IterationSource)
	{
		case ENiagaraIterationSource::Particles:
		{
			ParticleStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, bDisablePartialParticleUpdate)));

			ParticleStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, bParticleIterationStateEnabled)));
			if (SimStage->bParticleIterationStateEnabled)
			{
				ParticleStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ParticleIterationStateBinding)));
				ParticleStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ParticleIterationStateRange)));
			}
			break;
		}
		case ENiagaraIterationSource::DataInterface:
		{
			DataInterfaceStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, DataInterface)));
			DataInterfaceStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, bGpuDispatchForceLinear)));
			break;
		}
		case ENiagaraIterationSource::DirectSet:
		{
			DirectSetStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, DirectDispatchType)));
			DirectSetStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, DirectDispatchElementType)));
			DirectSetStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ElementCountX)));
			DirectSetStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ElementCountY)));
			DirectSetStageCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, ElementCountZ)));
			break;
		}
	}

	// Common Settings
	DispatchParametersCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, bOverrideGpuDispatchNumThreads)));
	if (SimStage->bOverrideGpuDispatchNumThreads)
	{
		DispatchParametersCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, OverrideGpuDispatchNumThreadsX)));
		DispatchParametersCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, OverrideGpuDispatchNumThreadsY)));
		DispatchParametersCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraSimulationStageGeneric, OverrideGpuDispatchNumThreadsZ)));
	}
}

void FNiagaraSimulationStageGenericCustomization::OnPropertyChanged()
{
	if (IDetailLayoutBuilder* DetailBuilder =  WeakDetailBuilder.Pin().Get() )
	{
		DetailBuilder->ForceRefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE
