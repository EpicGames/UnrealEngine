// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "NiagaraCommon.h"
#include "EdGraph/EdGraphSchema.h"
#include "Layout/Visibility.h"
#include "NiagaraSimulationStageBase.h"

class FDetailWidgetRow;
class IPropertyHandle;
class IPropertyHandleArray;
enum class ECheckBoxState : uint8;

class FNiagaraSimulationStageGenericCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual ~FNiagaraSimulationStageGenericCustomization();

	// Begin IDetailCustomization interface
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface

	void OnPropertyChanged();

private:
	TWeakPtr<IDetailLayoutBuilder> WeakDetailBuilder;
	TWeakObjectPtr<UNiagaraSimulationStageGeneric> WeakSimStage;
};
