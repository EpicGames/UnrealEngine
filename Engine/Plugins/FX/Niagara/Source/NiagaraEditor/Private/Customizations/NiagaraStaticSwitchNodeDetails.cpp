// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStaticSwitchNodeDetails.h"
#include "UObject/WeakObjectPtr.h"
#include "NiagaraNodeStaticSwitch.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Layout/Margin.h"
#include "NiagaraEditorStyle.h"
#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "NiagaraConstants.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraEditorUtilities.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NiagaraStaticSwitchNodeDetails"

TSharedRef<IDetailCustomization> FNiagaraStaticSwitchNodeDetails::MakeInstance()
{
	return MakeShareable(new FNiagaraStaticSwitchNodeDetails);
}

FNiagaraStaticSwitchNodeDetails::FNiagaraStaticSwitchNodeDetails()
{
	DropdownOptions.Add(MakeShareable(new SwitchDropdownOption("Bool")));
	DropdownOptions.Add(MakeShareable(new SwitchDropdownOption("Bool Constant")));
	DropdownOptions.Add(MakeShareable(new SwitchDropdownOption("Integer")));
	DropdownOptions.Add(MakeShareable(new SwitchDropdownOption("Integer Constant")));
	DropdownOptions.Add(MakeShareable(new SwitchDropdownOption("Enum Constant")));

	// Note: Due to static variables Enums can exist in the type registery as both dynamic & static variables so we need to deduplicate them
	TSet<UEnum*> FoundEnums;
	TArray<FNiagaraTypeDefinition> ParameterTypes;
	FNiagaraEditorUtilities::GetAllowedParameterTypes(ParameterTypes);
	for (FNiagaraTypeDefinition Type : ParameterTypes)
	{
		if (Type.IsEnum() && !FoundEnums.Contains(Type.GetEnum()))
		{
			DropdownOptions.Add(MakeShareable(new SwitchDropdownOption(Type.GetEnum()->GetName(), Type.GetEnum())));
			FoundEnums.Add(Type.GetEnum());
		}
	}
}

void FNiagaraStaticSwitchNodeDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	 SwitchTypeDataProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraNodeStaticSwitch, SwitchTypeData));

	static const FName SwitchCategoryName = TEXT("Static Switch");
	static const FName SwitchHiddenCategoryName = TEXT("HiddenMetaData");

	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	if (ObjectsCustomized.Num() == 1 && ObjectsCustomized[0]->IsA<UNiagaraNodeStaticSwitch>())
	{
		Node = CastChecked<UNiagaraNodeStaticSwitch>(ObjectsCustomized[0].Get());

		FNiagaraTypeDefinition TypeDefOnNode = Node->GetInputType();

		// Just in case the actual enum type isn't shown, add to the list.
		if (TypeDefOnNode.IsEnum())
		{
			bool bFoundAlready = false;
			for (TSharedPtr<SwitchDropdownOption>& Option : DropdownOptions)
			{
				if (Option->Enum == TypeDefOnNode.GetEnum())
					bFoundAlready = true;
			}

			if (!bFoundAlready)
				DropdownOptions.Add(MakeShareable(new SwitchDropdownOption(TypeDefOnNode.GetEnum()->GetName(), TypeDefOnNode.GetEnum())));
		}
		UpdateSelectionFromNode();
	
		DetailBuilder.EditCategory(SwitchHiddenCategoryName).SetCategoryVisibility(false);

		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(SwitchCategoryName);		
		FDetailWidgetRow& NameWidget = CategoryBuilder.AddCustomRow(LOCTEXT("NiagaraSwitchNodeNameFilterText", "Input parameter name"));
		FDetailWidgetRow& DropdownWidget = CategoryBuilder.AddCustomRow(LOCTEXT("NiagaraSwitchNodeTypeFilterText", "Input parameter type"));
		FDetailWidgetRow& DefaultValueOption = CategoryBuilder.AddCustomRow(LOCTEXT("NiagaraSwitchNodeDefaultFilterText", "Default value"));
		FDetailWidgetRow& ExternalPinOption = CategoryBuilder.AddCustomRow(LOCTEXT("NiagaraSwitchNodeExteranPinText", "Create Pin"));
		FDetailWidgetRow& ConstantSelection = CategoryBuilder.AddCustomRow(LOCTEXT("NiagaraSwitchNodeConstantFilterText", "Compiler constant"));
		

		NameWidget
		.NameContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(LOCTEXT("NiagaraSwitchNodeNameText", "Input parameter name"))
			]
		]
		.ValueContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(SEditableTextBox)
				.Text(this, &FNiagaraStaticSwitchNodeDetails::GetParameterNameText)
				.IsEnabled(this, &FNiagaraStaticSwitchNodeDetails::GetDefaultOptionEnabled)
				.ToolTipText(LOCTEXT("NiagaraSwitchNodeNameTooltip", "This is the name of the parameter that is exposed to the user calling this function graph."))
				.OnTextCommitted(this, &FNiagaraStaticSwitchNodeDetails::OnParameterNameCommited)
				.OnVerifyTextChanged(this, &FNiagaraStaticSwitchNodeDetails::OnVerifyParameterNameChanged)
				.SelectAllTextWhenFocused(true)
				.RevertTextOnEscape(true)
			]
		];

		DropdownWidget
		.NameContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(LOCTEXT("NiagaraSwitchNodeTypeText", "Static switch type"))
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoWidth()
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(SComboBox<TSharedPtr<SwitchDropdownOption>>)
				.OptionsSource(&DropdownOptions)
				.OnSelectionChanged(this, &FNiagaraStaticSwitchNodeDetails::OnSelectionChanged)
				.OnGenerateWidget(this, &FNiagaraStaticSwitchNodeDetails::CreateWidgetForDropdownOption)
				.InitiallySelectedItem(SelectedDropdownItem)
				[
					SNew(STextBlock)
					.Margin(FMargin(0.0f, 2.0f))
					.Text(this, &FNiagaraStaticSwitchNodeDetails::GetDropdownItemLabel)
				]
			]
		];
		
		DefaultValueOption
		.NameContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(STextBlock)
				.IsEnabled(this, &FNiagaraStaticSwitchNodeDetails::GetDefaultOptionEnabled)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(LOCTEXT("NiagaraSwitchNodeDefaultOptionText", "Default value"))
			]
		]
		.ValueContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(SWidgetSwitcher)
				.IsEnabled(this, &FNiagaraStaticSwitchNodeDetails::GetDefaultOptionEnabled)
		  		.WidgetIndex(this, &FNiagaraStaticSwitchNodeDetails::GetDefaultWidgetIndex)

		  		+ SWidgetSwitcher::Slot()
		  		[
					SNew(SCheckBox)
					.IsChecked_Lambda([this]() { return GetSwitchDefaultValue().Get(0) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged(this, &FNiagaraStaticSwitchNodeDetails::DefaultBoolValueCommitted)
				]

		  		+ SWidgetSwitcher::Slot()
		  		[
					SNew(SNumericEntryBox<int32>)
					.AllowSpin(false)
					.MinValue(0)
					.MaxValue(99)
					.Value(this, &FNiagaraStaticSwitchNodeDetails::GetSwitchDefaultValue)
					.OnValueCommitted(this, &FNiagaraStaticSwitchNodeDetails::DefaultIntValueCommitted)
				]
				
				+ SWidgetSwitcher::Slot()
		  		[
					SNew(SComboBox<TSharedPtr<DefaultEnumOption>>)
					.OptionsSource(&DefaultEnumDropdownOptions)
					.OnSelectionChanged(this, &FNiagaraStaticSwitchNodeDetails::OnSelectionChanged)
					.OnGenerateWidget(this, &FNiagaraStaticSwitchNodeDetails::CreateWidgetForDropdownOption)
					.InitiallySelectedItem(SelectedDefaultValue)
					[
						SNew(STextBlock)
						.Margin(FMargin(0.0f, 2.0f))
						.Text(this, &FNiagaraStaticSwitchNodeDetails::GetDefaultSelectionItemLabel)
					]
				]
			]
		];

		ExternalPinOption.NameContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 2.0f))
				[
					SNew(STextBlock)
					.IsEnabled(this, &FNiagaraStaticSwitchNodeDetails::GetExposeAsPinEnabled)
					.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
					.Text(LOCTEXT("NiagaraSwitchNodeExposeAsPinText", "Expose As Pin"))
				]
			]
		.ValueContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 2.0f))
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([this]() { return GetIsPinExposed() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged(this, &FNiagaraStaticSwitchNodeDetails::ExposeAsPinCommitted)
				]
			];

		ConstantSelection
		.NameContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(STextBlock)
				.IsEnabled(this, &FNiagaraStaticSwitchNodeDetails::IsConstantSelection)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(LOCTEXT("NiagaraSwitchNodeConstantText", "Compiler constant"))
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoWidth()
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(SComboBox<TSharedPtr<ConstantDropdownOption>>)
				.IsEnabled(this, &FNiagaraStaticSwitchNodeDetails::IsConstantSelection)
				.OptionsSource(&ConstantOptions)
				.OnSelectionChanged(this, &FNiagaraStaticSwitchNodeDetails::OnSelectionChanged)
				.OnGenerateWidget(this, &FNiagaraStaticSwitchNodeDetails::CreateWidgetForDropdownOption)
				.InitiallySelectedItem(SelectedConstantItem)
				[
					SNew(STextBlock)
					.Margin(FMargin(0.0f, 2.0f))
					.Text(this, &FNiagaraStaticSwitchNodeDetails::GetConstantSelectionItemLabel)
				]
			]
		];

		RefreshDropdownValues();
	}
}

int32 FNiagaraStaticSwitchNodeDetails::GetDefaultWidgetIndex() const
{
	if (!Node.IsValid())
	{
		return 0;
	}
	ENiagaraStaticSwitchType Type = Node->SwitchTypeData.SwitchType;
	return Type == ENiagaraStaticSwitchType::Bool ? 0 : (Type == ENiagaraStaticSwitchType::Integer ? 1 : 2);
}

TOptional<int32> FNiagaraStaticSwitchNodeDetails::GetSwitchDefaultValue() const
{
	UNiagaraScriptVariable* ScriptVar = GetSwitchParameterScriptVar();
	return ScriptVar ? TOptional<int32>(ScriptVar->GetStaticSwitchDefaultValue()) : TOptional<int32>();
}

void FNiagaraStaticSwitchNodeDetails::DefaultIntValueCommitted(int32 Value, ETextCommit::Type CommitInfo)
{
	UNiagaraScriptVariable* ScriptVar = GetSwitchParameterScriptVar();
	if (ScriptVar)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetStaticSwitchIntValue ", "Set Static Switch Int Value"));
		ScriptVar->Modify();

		ScriptVar->SetStaticSwitchDefaultValue(Value);
		SetSwitchParameterMetadata(ScriptVar->Metadata);
	}
}

void FNiagaraStaticSwitchNodeDetails::DefaultBoolValueCommitted(ECheckBoxState NewState)
{
	UNiagaraScriptVariable* ScriptVar = GetSwitchParameterScriptVar();
	if (ScriptVar)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetStaticSwitchBoolValue ", "Set Static Switch Bool Value"));
		ScriptVar->Modify();

		ScriptVar->SetStaticSwitchDefaultValue((NewState == ECheckBoxState::Checked) ? 1 : 0);
		SetSwitchParameterMetadata(ScriptVar->Metadata);
	}
}


void FNiagaraStaticSwitchNodeDetails::ExposeAsPinCommitted(ECheckBoxState NewState)
{
	if (Node.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetStaticSwitchExposed", "Set Static Switch Exposed"));
		Node->Modify();
		Node->SwitchTypeData.bExposeAsPin = (NewState == ECheckBoxState::Checked);

		if (SwitchTypeDataProperty.IsValid())
		{
			SwitchTypeDataProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}
}

bool FNiagaraStaticSwitchNodeDetails::GetIsPinExposed() const
{
	if (Node.IsValid())
	{
		return Node->SwitchTypeData.bExposeAsPin;
	}
	return false;
}

TSharedRef<SWidget> FNiagaraStaticSwitchNodeDetails::CreateWidgetForDropdownOption(TSharedPtr<SwitchDropdownOption> InOption)
{
	return SNew(STextBlock).Text(FText::FromString(*InOption->Name));
}

TSharedRef<SWidget> FNiagaraStaticSwitchNodeDetails::CreateWidgetForDropdownOption(TSharedPtr<DefaultEnumOption> InOption)
{
	return SNew(STextBlock).Text(InOption->DisplayName);
}

TSharedRef<SWidget> FNiagaraStaticSwitchNodeDetails::CreateWidgetForDropdownOption(TSharedPtr<ConstantDropdownOption> InOption)
{
	return SNew(STextBlock)
		.Text(InOption->DisplayName)
		.ToolTipText(InOption->Tooltip);
}

FText FNiagaraStaticSwitchNodeDetails::GetConstantSelectionItemLabel() const
{
	if (SelectedConstantItem.IsValid())
	{
		return SelectedConstantItem->DisplayName;
	}

	return LOCTEXT("InvalidNiagaraStaticSwitchNodeComboEntryText", "<Invalid selection>");
}

void FNiagaraStaticSwitchNodeDetails::OnSelectionChanged(TSharedPtr<DefaultEnumOption> NewValue, ESelectInfo::Type)
{
	SelectedDefaultValue = NewValue;
	UNiagaraScriptVariable* ScriptVar = GetSwitchParameterScriptVar();
	if (!SelectedDefaultValue.IsValid() || !ScriptVar)
	{
		return;
	}

	UEnum* Enum = Node->SwitchTypeData.Enum;
	if (!Enum)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetStaticSwitchDefaultValue", "Set Static Switch Default Value"));
	ScriptVar->Modify();

	ScriptVar->SetStaticSwitchDefaultValue(SelectedDefaultValue->EnumIndex);
	SetSwitchParameterMetadata(ScriptVar->Metadata);
	if (SwitchTypeDataProperty.IsValid())
	{
		SwitchTypeDataProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

void FNiagaraStaticSwitchNodeDetails::OnSelectionChanged(TSharedPtr<SwitchDropdownOption> NewValue, ESelectInfo::Type)
{
	SelectedDropdownItem = NewValue;
	if (!SelectedDropdownItem.IsValid() || !Node.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetStaticSwitchType", "Set Static Switch Type"));
	Node->Modify();

	FNiagaraTypeDefinition OldType = Node->GetInputType();
	Node->SwitchTypeData.Enum = nullptr;
	Node->SwitchTypeData.SwitchConstant = NAME_None;
	if (SelectedDropdownItem == DropdownOptions[0] || SelectedDropdownItem == DropdownOptions[1])
	{
		Node->SwitchTypeData.SwitchType = ENiagaraStaticSwitchType::Bool;
	}
	else if (SelectedDropdownItem == DropdownOptions[2] || SelectedDropdownItem == DropdownOptions[3])
	{
		Node->SwitchTypeData.SwitchType = ENiagaraStaticSwitchType::Integer;
	}
	else
	{
		Node->SwitchTypeData.SwitchType = ENiagaraStaticSwitchType::Enum;
		if (!IsConstantSelection())
		{
			Node->SwitchTypeData.Enum = SelectedDropdownItem->Enum;
		}
	}
	RefreshDropdownValues();
	Node->OnSwitchParameterTypeChanged(OldType);
	if (SwitchTypeDataProperty.IsValid())
	{
		SwitchTypeDataProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

void FNiagaraStaticSwitchNodeDetails::OnSelectionChanged(TSharedPtr<ConstantDropdownOption> NewValue, ESelectInfo::Type)
{
	SelectedConstantItem = NewValue;
	if (!NewValue.IsValid() || !Node.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetStaticSwitchCompilerConstant", "Set Static Switch Compiler Constant"));
	Node->Modify();

	Node->SwitchTypeData.SwitchConstant = NewValue->Constant.GetName();
	
	// in case of an enum constant we also need the enum type
	if (SelectedDropdownItem == DropdownOptions[4])
	{
		Node->SwitchTypeData.Enum = NewValue->Constant.GetType().GetEnum();
	}
	else
	{
		Node->SwitchTypeData.Enum = nullptr;
	}
	Node->RefreshFromExternalChanges();
	if (SwitchTypeDataProperty.IsValid())
	{
		SwitchTypeDataProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

FText FNiagaraStaticSwitchNodeDetails::GetDropdownItemLabel() const
{
	if (SelectedDropdownItem.IsValid())
	{
		return FText::FromString(*SelectedDropdownItem->Name);
	}

	return LOCTEXT("InvalidNiagaraStaticSwitchNodeComboEntryText", "<Invalid selection>");
}

FText FNiagaraStaticSwitchNodeDetails::GetDefaultSelectionItemLabel() const
{
	if (SelectedDefaultValue.IsValid())
	{
		return SelectedDefaultValue->DisplayName;
	}

	return LOCTEXT("InvalidNiagaraStaticSwitchNodeComboEntryText", "<Invalid selection>");
}

void FNiagaraStaticSwitchNodeDetails::RefreshDropdownValues()
{
	if (!Node.IsValid())
	{
		return;
	}

	// refresh the constant value dropdown values
	ENiagaraStaticSwitchType SwitchType = Node->SwitchTypeData.SwitchType;
	if (IsConstantSelection())
	{
		ConstantOptions.Empty();
		SelectedConstantItem.Reset();

		TArray<FNiagaraVariable> AllConstants;
		AllConstants.Append(FNiagaraConstants::GetStaticSwitchConstants());

		for (const FNiagaraVariable& Var : AllConstants)
		{
			FText DisplayName = FText::FromName(Var.GetName());
			FText Tooltip = FNiagaraConstants::GetEngineConstantDescription(Var);
			if ((SwitchType == ENiagaraStaticSwitchType::Bool && Var.GetType() == FNiagaraTypeDefinition::GetBoolDef()) ||
				(SwitchType == ENiagaraStaticSwitchType::Integer && Var.GetType() == FNiagaraTypeDefinition::GetIntDef()) ||
				(SwitchType == ENiagaraStaticSwitchType::Enum && Var.GetType().IsEnum()))
			{				
				ConstantOptions.Add(MakeShared<ConstantDropdownOption>(DisplayName, Tooltip, Var));
				if (Node->SwitchTypeData.SwitchConstant == Var.GetName())
				{
					SelectedConstantItem = ConstantOptions.Last();
				}
			}
		}
		if (!SelectedConstantItem.IsValid() && ConstantOptions.Num() > 0)
		{
			OnSelectionChanged(ConstantOptions[0], ESelectInfo::Type::Direct);
		}
		if (ConstantOptions.Num() == 0)
		{
			OnSelectionChanged(MakeShared<ConstantDropdownOption>(LOCTEXT("NiagaraNoConstantsForTypeText", "<No entries>"), FText(), FNiagaraVariable()), ESelectInfo::Type::Direct);
		}
	}

	// refresh the default options dropdown values
	if (SwitchType != ENiagaraStaticSwitchType::Enum)
	{
		return;
	}

	UNiagaraScriptVariable* ScriptVar = GetSwitchParameterScriptVar();

	DefaultEnumDropdownOptions.Empty();
	UEnum* Enum = Node->SwitchTypeData.Enum;
	if (Enum)
	{
		SelectedDefaultValue.Reset();
		for (int i = 0; i < Enum->GetMaxEnumValue(); i++)
		{
			if (!Enum->IsValidEnumValue(i))
			{
				continue;
			}
			FText DisplayName = Enum->GetDisplayNameTextByIndex(i);
			DefaultEnumDropdownOptions.Add(MakeShared<DefaultEnumOption>(DisplayName, i));

			if (ScriptVar && i == ScriptVar->GetStaticSwitchDefaultValue())
			{
				SelectedDefaultValue = DefaultEnumDropdownOptions[i];
			}
		}
		if (!SelectedDefaultValue.IsValid() && DefaultEnumDropdownOptions.Num() > 0)
		{
			OnSelectionChanged(DefaultEnumDropdownOptions[0], ESelectInfo::Type::Direct);
		}
	}
}

UNiagaraScriptVariable* FNiagaraStaticSwitchNodeDetails::GetSwitchParameterScriptVar() const
{
	if (!Node.IsValid() || !Node->GetNiagaraGraph())
	{
		return nullptr;
	}
	return Node->GetNiagaraGraph()->GetScriptVariable(FNiagaraVariable(Node->GetInputType(), Node->InputParameterName));
}

void FNiagaraStaticSwitchNodeDetails::SetSwitchParameterMetadata(const FNiagaraVariableMetaData& MetaData)
{
	if (!Node.IsValid() || !Node->GetNiagaraGraph())
	{
		return;
	}
	Node->GetNiagaraGraph()->SetMetaData(FNiagaraVariable(Node->GetInputType(), Node->InputParameterName), MetaData);
	Node->GetNiagaraGraph()->NotifyGraphNeedsRecompile();

	if (SwitchTypeDataProperty.IsValid())
	{
		SwitchTypeDataProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

void FNiagaraStaticSwitchNodeDetails::UpdateSelectionFromNode()
{
	SelectedDropdownItem.Reset();
	ENiagaraStaticSwitchType SwitchType = Node->SwitchTypeData.SwitchType;
	bool IsConstantSelection = Node->IsSetByCompiler();

	if (SwitchType == ENiagaraStaticSwitchType::Bool)
	{
		SelectedDropdownItem = IsConstantSelection ? DropdownOptions[1] : DropdownOptions[0];
	}
	else if (SwitchType == ENiagaraStaticSwitchType::Integer)
	{
		SelectedDropdownItem = IsConstantSelection ? DropdownOptions[3] : DropdownOptions[2];
	}
	else if (SwitchType == ENiagaraStaticSwitchType::Enum && Node->SwitchTypeData.Enum)
	{
		if (IsConstantSelection)
		{
			SelectedDropdownItem = DropdownOptions[4];
		}
		else
		{
			FString SelectedName = Node->SwitchTypeData.Enum->GetName();
			for (TSharedPtr<SwitchDropdownOption>& Option : DropdownOptions)
			{
				if (SelectedName.Equals(*Option->Name))
				{
					SelectedDropdownItem = Option;
					return;
				}
			}
		}
	}
}

bool FNiagaraStaticSwitchNodeDetails::IsConstantSelection() const
{
	return SelectedDropdownItem == DropdownOptions[1] || SelectedDropdownItem == DropdownOptions[3] || SelectedDropdownItem == DropdownOptions[4];
}

bool FNiagaraStaticSwitchNodeDetails::GetExposeAsPinEnabled() const
{
	return !IsConstantSelection();

}

FText FNiagaraStaticSwitchNodeDetails::GetParameterNameText() const
{
	return Node.IsValid() ? FText::FromName(Node->InputParameterName) : FText();
}

void FNiagaraStaticSwitchNodeDetails::OnParameterNameCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	if (Node.IsValid())
	{
		FName NewName(*InText.ToString());
		if ( Node->InputParameterName != NewName )
		{
			const FScopedTransaction Transaction(LOCTEXT("SetStaticSwitchName", "Set Static Switch Name"));
			Node->Modify();
			Node->ChangeSwitchParameterName(NewName);
			Node->GetNiagaraGraph()->NotifyGraphNeedsRecompile();
		}
	}
}

bool FNiagaraStaticSwitchNodeDetails::OnVerifyParameterNameChanged(const FText& InName, FText& OutErrorMessage)
{
	FText TrimmedName = FText::TrimPrecedingAndTrailing(InName);

	if (TrimmedName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("RenameFailed_LeftBlank", "Static switch name cannot be left blank");
		return false;
	}

	if (TrimmedName.ToString().Len() >= NAME_SIZE)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("CharCount"), NAME_SIZE);
		OutErrorMessage = FText::Format(LOCTEXT("RenameFailed_TooLong", "Name must be less than {CharCount} characters long."), Arguments);
		return false;
	}

	if (FName(*TrimmedName.ToString()) == NAME_None)
	{
		OutErrorMessage = LOCTEXT("RenameFailed_ReservedNameNone", "\"None\" is a reserved term and cannot be used for a static switch name");
		return false;
	}

	if (TrimmedName.ToString().Contains("."))
	{
		OutErrorMessage = LOCTEXT("RenameFailed_IncludesDot", "'.' is used to separate namespaces and cannot be used in the name of a static switch");
		return false;
	}

	return true;
}

bool FNiagaraStaticSwitchNodeDetails::GetDefaultOptionEnabled() const
{
	return !IsConstantSelection() && !Node->SwitchTypeData.bExposeAsPin;
}

#undef LOCTEXT_NAMESPACE
