// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"
#include "Misc/Optional.h"
#include "NiagaraTypes.h"

class UNiagaraScriptVariable;

// This data structure is used internally by the dropdown to keep track of the user's choice
struct SwitchDropdownOption
{
	FString Name;
	UEnum* Enum;

	SwitchDropdownOption(FString Name) : Name(Name), Enum(nullptr)
	{}

	SwitchDropdownOption(FString Name, UEnum* Enum) : Name(Name), Enum(Enum)
	{}
};

// This data structure is used internally by the default enum dropdown to keep track of the user's choice
struct DefaultEnumOption
{
	FText DisplayName;
	int32 EnumIndex;	

	DefaultEnumOption(FText DisplayName) : DisplayName(DisplayName), EnumIndex(0)
	{
	}

	DefaultEnumOption(FText DisplayName, int32 EnumIndex) : DisplayName(DisplayName), EnumIndex(EnumIndex)
	{
	}
};

// This data structure is used internally by the constant selection to keep track of the user's choice
struct ConstantDropdownOption
{
	FText DisplayName;
	FText Tooltip;
	FNiagaraVariable Constant;

	ConstantDropdownOption(FText DisplayName) : DisplayName(DisplayName), Tooltip(FText()), Constant(FNiagaraVariable())
	{
	}

	ConstantDropdownOption(FText DisplayName, FText Tooltip, FNiagaraVariable Constant) : DisplayName(DisplayName), Tooltip(Tooltip), Constant(Constant)
	{
	}
};

/** This customization sets up a custom details panel for the static switch node in the niagara module graph. */
class FNiagaraStaticSwitchNodeDetails : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	FNiagaraStaticSwitchNodeDetails();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	
private:
	// type and enum dropdown functions
	TSharedRef<SWidget> CreateWidgetForDropdownOption(TSharedPtr<SwitchDropdownOption> InOption);
	void OnSelectionChanged(TSharedPtr<SwitchDropdownOption> NewValue, ESelectInfo::Type);
	FText GetDropdownItemLabel() const;
	void UpdateSelectionFromNode();

	// parameter name option function
	FText GetParameterNameText() const;
	void OnParameterNameCommited(const FText& InText, ETextCommit::Type InCommitType);
	bool OnVerifyParameterNameChanged(const FText& InLabel, FText& OutErrorMessage);

	// default value option functions
	bool GetDefaultOptionEnabled() const;
	int32 GetDefaultWidgetIndex() const;
	TOptional<int32> GetSwitchDefaultValue() const;
	void DefaultIntValueCommitted(int32 Value, ETextCommit::Type CommitInfo);
	void DefaultBoolValueCommitted(ECheckBoxState NewState);
	TSharedRef<SWidget> CreateWidgetForDropdownOption(TSharedPtr<DefaultEnumOption> InOption);
	void OnSelectionChanged(TSharedPtr<DefaultEnumOption> NewValue, ESelectInfo::Type);
	FText GetDefaultSelectionItemLabel() const;
	void RefreshDropdownValues();

	UNiagaraScriptVariable* GetSwitchParameterScriptVar() const;
	void SetSwitchParameterMetadata(const FNiagaraVariableMetaData& MetaData);

	// constant selection functions
	bool IsConstantSelection() const;
	void OnSelectionChanged(TSharedPtr<ConstantDropdownOption> NewValue, ESelectInfo::Type);
	TSharedRef<SWidget> CreateWidgetForDropdownOption(TSharedPtr<ConstantDropdownOption> InOption);
	FText GetConstantSelectionItemLabel() const;

	bool GetExposeAsPinEnabled() const;
	void ExposeAsPinCommitted(ECheckBoxState NewState);
	bool GetIsPinExposed() const;
	

	TWeakObjectPtr<class UNiagaraNodeStaticSwitch> Node;
	TArray<TSharedPtr<SwitchDropdownOption>> DropdownOptions;
	TSharedPtr<SwitchDropdownOption> SelectedDropdownItem;
	TArray<TSharedPtr<DefaultEnumOption>> DefaultEnumDropdownOptions;
	TSharedPtr<DefaultEnumOption> SelectedDefaultValue;
	TArray<TSharedPtr<ConstantDropdownOption>> ConstantOptions;
	TSharedPtr<ConstantDropdownOption> SelectedConstantItem;
	TSharedPtr<IPropertyHandle> SwitchTypeDataProperty;
};
