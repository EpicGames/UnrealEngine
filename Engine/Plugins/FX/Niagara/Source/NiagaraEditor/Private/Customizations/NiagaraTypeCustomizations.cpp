// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraTypeCustomizations.h"

#include "CoreMinimal.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EdGraphSchema_Niagara.h"
#include "Materials/MaterialParameters.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SGraphActionMenu.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/MaterialInterface.h"
#include "Misc/DefaultValueHelper.h"
#include "Modules/ModuleManager.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

#include "NiagaraConstants.h"
#include "NiagaraDataChannelPublic.h"
#include "NiagaraDataInterfaceRW.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraParameterMapHistory.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"
#include "NiagaraTypes.h"
#include "NiagaraVariableMetaData.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"

//-TODO:stateless:Remove and unify emitter
#include "Stateless/NiagaraStatelessEmitter.h"
//-TODO:stateless:Remove and unify emitter

#include "Widgets/SNiagaraColorEditor.h"
#include "Widgets/SNiagaraParameterMenu.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraTypeCustomizations)

#define LOCTEXT_NAMESPACE "FNiagaraVariableAttributeBindingCustomization"
#define ALLOW_LIBRARY_TO_LIBRARY_DEFAULT_BINDING 0

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FNiagaraMaterialParameterValueHelper : public TSharedFromThis<FNiagaraMaterialParameterValueHelper>
{
	FNiagaraMaterialParameterValueHelper(TSharedPtr<IPropertyHandle> InNamePropertyHandle, TSharedPtr<IPropertyHandle> InValuePropertyHandle, TWeakObjectPtr<UNiagaraRendererProperties> InWeakRenderProperties)
		: NamePropertyHandle(InNamePropertyHandle.ToSharedRef())
		, ValuePropertyHandle(InValuePropertyHandle.ToSharedRef())
		, WeakRenderProperties(InWeakRenderProperties)
	{
	}

	template<typename TType>
	TOptional<TType> GetMaterialDefaultValue(UMaterialInterface* Material, FName ParameterName) const;

	template<>
	TOptional<float> GetMaterialDefaultValue(UMaterialInterface* Material, FName ParameterName) const
	{
		float DefaultValue;
		if (Material->GetScalarParameterDefaultValue(ParameterName, DefaultValue))
		{
			return DefaultValue;
		}
		return {};
	}

	template<>
	TOptional<FLinearColor> GetMaterialDefaultValue(UMaterialInterface* Material, FName ParameterName) const
	{
		FLinearColor DefaultValue;
		if (Material->GetVectorParameterDefaultValue(ParameterName, DefaultValue))
		{
			return DefaultValue;
		}
		return {};
	}

	template<>
	TOptional<UTexture*> GetMaterialDefaultValue(UMaterialInterface* Material, FName ParameterName) const
	{
		UTexture* DefaultValue;
		if (Material->GetTextureParameterDefaultValue(ParameterName, DefaultValue))
		{
			return DefaultValue;
		}
		return {};
	}

	void AppendValueToString(float V, FString& ValuesString) const
	{
		ValuesString.Appendf(TEXT("%f"), V);
	}

	void AppendValueToString(const FLinearColor& V, FString& ValuesString) const
	{
		ValuesString.Appendf(TEXT("{R=%f, G=%f, B=%f, A=%f}"), V.R, V.G, V.B, V.A);
	}

	void AppendValueToString(const UTexture* V, FString& ValuesString) const
	{
		ValuesString.Append(*GetNameSafe(V));
	}

	template<typename TType>
	FText FromValuesAsText(TArray<TType> Values) const
	{
		FString ValuesString;
		for (const TType& V : Values)
		{
			if (!ValuesString.IsEmpty())
			{
				ValuesString.Append(TEXT(", "));
			}
			AppendValueToString(V, ValuesString);
		}
		return FText::FromString(ValuesString);
	}

	FName GetParameterName() const
	{
		FName ParameterName;
		NamePropertyHandle->GetValue(ParameterName);
		return ParameterName;
	}

	template<typename TType>
	TArray<TType> TryResolveMaterialDefaultValue() const
	{
		TArray<TType> DefaultValues;

		if (UNiagaraRendererProperties* RenderProperties = WeakRenderProperties.Get())
		{
			const FName ParameterName = GetParameterName();

			TArray<UMaterialInterface*> UsedMaterials;
			RenderProperties->GetUsedMaterials(nullptr, UsedMaterials);

			for (UMaterialInterface* Material : UsedMaterials)
			{
				if (Material == nullptr)
				{
					continue;
				}

				TOptional<TType> DefaultValue = GetMaterialDefaultValue<TType>(Material, ParameterName);
				if (DefaultValue.IsSet())
				{
					DefaultValues.AddUnique(DefaultValue.GetValue());
				}
			}
		}

		return DefaultValues;
	}

	template<typename TType>
	FText GetToolTipText() const
	{
		const TArray<TType> Values = TryResolveMaterialDefaultValue<TType>();
		const FText ParameterNameText = FText::FromName(GetParameterName());
		if (Values.Num() == 0)
		{
			return FText::Format(LOCTEXT("GetDefaultMaterialValue_InvalidBinding", "Parameter '{0}' is either invalid or not found on the material"), ParameterNameText);
		}
		else if (Values.Num() == 1)
		{
			return FText::Format(LOCTEXT("GetDefaultMaterialValue_Value", "Reset To Default\nSet to parameter '{0}' default value '{1}'"), ParameterNameText, FromValuesAsText(Values));
		}
		else
		{
			return FText::Format(LOCTEXT("GetDefaultMaterialValue_Conflict", "Reset To Default\nParameter '{0}' has conflicting values '{1}'"), ParameterNameText, FromValuesAsText(Values));
		}
	}

	template<typename TType>
	bool GetPropertyValue(TType& OutValue) const
	{
		return ValuePropertyHandle->GetValue(OutValue) == FPropertyAccess::Success;
	}

	template<>
	bool GetPropertyValue(UTexture*& OutValue) const
	{
		UObject* TempValue = nullptr;
		bool bSuccess = ValuePropertyHandle->GetValue(TempValue) == FPropertyAccess::Success;
		OutValue = Cast<UTexture>(TempValue);
		return bSuccess;
	}

	template<>
	bool GetPropertyValue(FLinearColor& OutValue) const
	{
		TArray<void*> RawDatas;
		ValuePropertyHandle->AccessRawData(RawDatas);

		if (RawDatas.Num() == 1 && RawDatas[0] != nullptr)
		{
			FMemory::Memcpy(&OutValue, RawDatas[0], sizeof(FLinearColor));
			return true;
		}
		return false;
	}

	template<typename TType>
	EVisibility IsVisible() const
	{
		TType Value;
		if (GetPropertyValue(Value))
		{
			const TArray<TType> Values = TryResolveMaterialDefaultValue<TType>();
			if (Values.Num() > 0 && Value != Values[0])
			{
				return EVisibility::Visible;
			}
		}
		return EVisibility::Hidden;
	}

	template<typename TType>
	void SetPropertyValue(TType Value) const
	{
		ValuePropertyHandle->SetValue(Value);
	}

	template<>
	void SetPropertyValue(FLinearColor Value) const
	{
		TArray<void*> RawDatas;
		ValuePropertyHandle->AccessRawData(RawDatas);

		for (void* RawData : RawDatas)
		{
			if (RawData)
			{
				FMemory::Memcpy(RawData, &Value, sizeof(FLinearColor));
			}
		}
	}

	template<typename TType>
	FReply OnClicked() const
	{
		const TArray<TType> Values = TryResolveMaterialDefaultValue<TType>();
		if (Values.Num() > 0)
		{
			FScopedTransaction Transaction(LOCTEXT("SetValueToMaterialDefault", "Set Value To Material Default"));

			TArray<UObject*> OuterObjects;
			ValuePropertyHandle->GetOuterObjects(OuterObjects);
			for (UObject* OuterObject : OuterObjects)
			{
				OuterObject->Modify();
			}

			ValuePropertyHandle->NotifyPreChange();

			SetPropertyValue(Values[0]);

			ValuePropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
		return FReply::Handled();
	}

	template<typename TType>
	TSharedRef<SWidget> CreatePropertyWidget()
	{
		return ValuePropertyHandle->CreatePropertyValueWidget();
	}

	template<>
	TSharedRef<SWidget> CreatePropertyWidget<FLinearColor>()
	{
		return SNew(SNiagaraColorEditor)
			.ShowAlpha(true)
			.ShowExpander(true)
			.Color(this, &FNiagaraMaterialParameterValueHelper::GetColorValue)
			.OnColorChanged(this, &FNiagaraMaterialParameterValueHelper::SetColorValue)
			.OnBeginEditing(this, &FNiagaraMaterialParameterValueHelper::BeginContinuousTransaction)
			.OnEndEditing(this, &FNiagaraMaterialParameterValueHelper::EndContinuousTransaction)
			.OnCancelEditing(this, &FNiagaraMaterialParameterValueHelper::CancelContinuousTransaction)
		;
	}

	FLinearColor GetColorValue() const
	{
		FLinearColor Color = FLinearColor::White;
		GetPropertyValue(Color);
		return Color;
	}

	void SetColorValue(FLinearColor InColorValue)
	{
		const FText TransactionText = LOCTEXT("SetColorValue", "Set Color Value");
		if (bContinuousTransactionPending)
		{
			bContinuousTransactionPending = false;
			ContinuousTransactionIndex = GEditor->BeginTransaction(TransactionText);
		}

		const bool bIsContinousTransaction = ContinuousTransactionIndex.IsSet();

		FScopedTransaction Transaction(TransactionText, !bIsContinousTransaction);

		TArray<UObject*> OuterObjects;
		ValuePropertyHandle->GetOuterObjects(OuterObjects);
		for (UObject* OuterObject : OuterObjects)
		{
			OuterObject->Modify();
		}

		ValuePropertyHandle->NotifyPreChange();

		SetPropertyValue(InColorValue);

		ValuePropertyHandle->NotifyPostChange(bIsContinousTransaction ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
	}

	void BeginContinuousTransaction()
	{
		bContinuousTransactionPending = true;
	}

	void EndContinuousTransaction()
	{
		bContinuousTransactionPending = false;
		if (ContinuousTransactionIndex.IsSet())
		{
			GEditor->EndTransaction();
			ContinuousTransactionIndex.Reset();
		}
	}

	void CancelContinuousTransaction(FLinearColor OriginalValue)
	{
		bContinuousTransactionPending = false;
		if (ContinuousTransactionIndex.IsSet())
		{
			GEditor->CancelTransaction(ContinuousTransactionIndex.GetValue());
			ContinuousTransactionIndex.Reset();
		}
		SetPropertyValue(OriginalValue);
	}

	template<typename TType>
	bool CreateWidget(IDetailChildrenBuilder& ChildBuilder)
	{
		ChildBuilder.AddCustomRow(ValuePropertyHandle->GetPropertyDisplayName())
		.NameWidget
		[
			ValuePropertyHandle->CreatePropertyNameWidget()
		]
		.ValueWidget
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				CreatePropertyWidget<TType>()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2, 1)
			[
				SNew(SButton)
				.IsFocusable(false)
				.ToolTipText(this, &FNiagaraMaterialParameterValueHelper::GetToolTipText<TType>)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ContentPadding(0)
				.Visibility(this, &FNiagaraMaterialParameterValueHelper::IsVisible<TType>)
				.OnClicked(this, &FNiagaraMaterialParameterValueHelper::OnClicked<TType>)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];

		return true;
	}

	TSharedRef<IPropertyHandle>					NamePropertyHandle;
	TSharedRef<IPropertyHandle>					ValuePropertyHandle;
	TWeakObjectPtr<UNiagaraRendererProperties>	WeakRenderProperties;

	bool				bContinuousTransactionPending = false;
	TOptional<int32>	ContinuousTransactionIndex;
};

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FNiagaraNumericCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> ValueHandle = PropertyHandle->GetChildHandle(TEXT("Value"));

	HeaderRow
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(ValueHandle.IsValid() ? 125.f : 200.f)
		[
			// Some Niagara numeric types have no value so in that case just display their type name
			ValueHandle.IsValid()
			? ValueHandle->CreatePropertyValueWidget()
			: SNew(STextBlock)
			  .Text(FText::FromString(FName::NameToDisplayString(CastField<FStructProperty>(PropertyHandle->GetProperty())->Struct->GetName(), false)))
			  .Font(IDetailLayoutBuilder::GetDetailFont())
		];
}


void FNiagaraBoolCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	ValueHandle = PropertyHandle->GetChildHandle(TEXT("Value"));

	static const FName DefaultForegroundName("DefaultForeground");

	HeaderRow
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &FNiagaraBoolCustomization::OnCheckStateChanged)
			.IsChecked(this, &FNiagaraBoolCustomization::OnGetCheckState)
			.ForegroundColor(FAppStyle::GetSlateColor(DefaultForegroundName))
			.Padding(0.0f)
		];
}

ECheckBoxState FNiagaraBoolCustomization::OnGetCheckState() const
{
	ECheckBoxState CheckState = ECheckBoxState::Undetermined;
	int32 Value;
	FPropertyAccess::Result Result = ValueHandle->GetValue(Value);
	if (Result == FPropertyAccess::Success)
	{
		CheckState = Value == FNiagaraBool::True ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return CheckState;
}

void FNiagaraBoolCustomization::OnCheckStateChanged(ECheckBoxState InNewState)
{
	if (InNewState == ECheckBoxState::Checked)
	{
		ValueHandle->SetValue(FNiagaraBool::True);
	}
	else
	{
		ValueHandle->SetValue(FNiagaraBool::False);
	}
}

void FNiagaraMatrixCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildNum = 0; ChildNum < NumChildren; ++ChildNum)
	{
		ChildBuilder.AddProperty(PropertyHandle->GetChildHandle(ChildNum).ToSharedRef());
	}
}

TArray<FNiagaraVariableBase> FNiagaraStackAssetAction_VarBind::FindVariables(UNiagaraSystem* NiagaraSystem, const FVersionedNiagaraEmitter& InEmitter, bool bSystem, bool bEmitter, bool bParticles, bool bUser, bool bAllowStatic)
{
	TArray<FNiagaraVariableBase> Bindings;
	TArray<FNiagaraParameterMapHistory> Histories;

	if (FVersionedNiagaraEmitterData* EmitterData = InEmitter.GetEmitterData())
	{
		if ( UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(EmitterData->GraphSource) )
		{
			Histories.Append(UNiagaraNodeParameterMapBase::GetParameterMaps(Source->NodeGraph));
		}
	}

	if (NiagaraSystem && (bSystem || bEmitter))
	{
		if ( UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(NiagaraSystem->GetSystemUpdateScript()->GetLatestSource()) )
		{
			Histories.Append(UNiagaraNodeParameterMapBase::GetParameterMaps(Source->NodeGraph));
		}
	}

	UNiagaraEmitter* NiagaraEmitter = InEmitter.Emitter;
	bEmitter &= NiagaraEmitter != nullptr;
	for (const FNiagaraParameterMapHistory& History : Histories)
	{
		for (const FNiagaraVariable& Var : History.Variables)
		{
			if (Var.GetType().IsStatic() && !bAllowStatic)
				continue;

			if (bParticles && FNiagaraParameterUtilities::IsAttribute(Var))
			{
				Bindings.AddUnique(Var);
			}
			else if (bSystem && FNiagaraParameterUtilities::IsSystemParameter(Var))
			{
				Bindings.AddUnique(Var);
			}
			else if (bEmitter && NiagaraEmitter && Var.IsInNameSpace(NiagaraEmitter->GetUniqueEmitterName()))
			{
				Bindings.AddUnique(
					FNiagaraUtilities::ResolveAliases(Var, FNiagaraAliasContext().ChangeEmitterNameToEmitter(NiagaraEmitter->GetUniqueEmitterName()))
				);
			}
			else if (bEmitter && FNiagaraParameterUtilities::IsAliasedEmitterParameter(Var))
			{
				Bindings.AddUnique(Var);
			}
			else if (bEmitter && Var.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString))
			{
				Bindings.AddUnique(Var);
			}
			else if (bUser && FNiagaraParameterUtilities::IsUserParameter(Var))
			{
				Bindings.AddUnique(Var);
			}
		}
	}

	if (NiagaraSystem && bUser)
	{
		for (const FNiagaraVariable Var : NiagaraSystem->GetExposedParameters().ReadParameterVariables())
		{
			Bindings.AddUnique(Var);
		}
	}
	return Bindings;
}

bool FNiagaraVariableAttributeBindingCustomization::IsValid() const
{
	return TargetVariableBinding && WeakVersionedEmitter.IsValid();
}

FText FNiagaraVariableAttributeBindingCustomization::GetMissingText() const
{
	return LOCTEXT("Missing", "Missing");
}

FName FNiagaraVariableAttributeBindingCustomization::GetVariableName() const
{
	return IsValid() ? TargetVariableBinding->GetName() : NAME_None;
}

FText FNiagaraVariableAttributeBindingCustomization::GetCurrentText() const
{
	return IsValid() ? FText::FromName(TargetVariableBinding->GetName()) : GetMissingText();
}

FText FNiagaraVariableAttributeBindingCustomization::GetTooltipText() const
{
	if (IsValid())
	{
		FText TooltipDesc = FText::Format(LOCTEXT("AttributeBindingTooltip", "Use the variable \"{0}\" if it exists."), FText::FromName(TargetVariableBinding->GetName()));

		const FString DefaultValueStr = TargetVariableBinding->GetDefaultValueString();
		if (DefaultValueStr.Len() > 0)
		{
			TooltipDesc = FText::Format(LOCTEXT("AttributeBindingTooltipValue", "{0}.  Use the value \"{1}\" if the variable does not exist."), TooltipDesc, FText::FromString(DefaultValueStr));
		}

		return TooltipDesc;
	}
	return GetMissingText();
}

TSharedRef<SWidget> FNiagaraVariableAttributeBindingCustomization::OnGetMenuContent() const
{
	FGraphActionMenuBuilder MenuBuilder;

	TSharedPtr<SGraphActionMenu> GraphActionMenu;
	
	TSharedPtr<SWidget> Widget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			[
				SAssignNew(GraphActionMenu, SGraphActionMenu)
				.OnActionSelected(const_cast<FNiagaraVariableAttributeBindingCustomization*>(this), &FNiagaraVariableAttributeBindingCustomization::OnActionSelected)
				.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(const_cast<FNiagaraVariableAttributeBindingCustomization*>(this), &FNiagaraVariableAttributeBindingCustomization::OnCreateWidgetForAction))
				.OnCollectAllActions(const_cast<FNiagaraVariableAttributeBindingCustomization*>(this), &FNiagaraVariableAttributeBindingCustomization::CollectAllActions)
				.AutoExpandActionMenu(false)
				.bAllowPreselectedItemActivation(true)
				.ShowFilterTextBox(true)
			]
		];

	// the widget to focus is retrieved after this function is called via delegate, so setting it here works
	TSharedPtr<SWidget> SearchBoxToFocus = GraphActionMenu->GetFilterTextBox();
	ComboButton->SetMenuContentWidgetToFocus(SearchBoxToFocus);
	
	return Widget.ToSharedRef(); 
}

TArray<FName> FNiagaraVariableAttributeBindingCustomization::GetNames() const
{
	TArray<FName> Names;
	if (!PropertyHandle.IsValid() || !PropertyHandle->GetProperty() || !TargetVariableBinding)
	{
		return Names;
	}

	const FVersionedNiagaraEmitterBase VersionedEmitter = WeakVersionedEmitter.ResolveWeakPtr();
	if (VersionedEmitter.IsValid() == false)
	{
		return Names;
	}

	TArray<UNiagaraGraph*> GraphsToTraverse;
	if ( UNiagaraEmitter* NiagaraEmitter = Cast<UNiagaraEmitter>(VersionedEmitter.Emitter) )
	{
		if (FVersionedNiagaraEmitterData* OwningEmitterData = NiagaraEmitter->GetEmitterData(VersionedEmitter.Version))
		{
			GraphsToTraverse.Add(Cast<UNiagaraScriptSource>(OwningEmitterData->GraphSource)->NodeGraph);
		}
	}

	TSet<FNiagaraVariableBase> Vars;

	if(UNiagaraSystem* System = VersionedEmitter.Emitter->GetTypedOuter<UNiagaraSystem>())
	{
    	GraphsToTraverse.Add(Cast<UNiagaraScriptSource>(System->GetSystemUpdateScript()->GetLatestSource())->NodeGraph);
	
		for (const FNiagaraVariable UserParameter : System->GetExposedParameters().ReadParameterVariables())
		{
			Vars.Add(UserParameter);
		}
	}
	
	TArray<TArray<FNiagaraVariable>> MapHistoryVars;
	for(UNiagaraGraph* Graph : GraphsToTraverse)
	{
		TArray<UNiagaraNodeOutput*> OutputNodes;
		Graph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);

		for(UNiagaraNodeOutput* NodeOutput : OutputNodes)
		{
			FNiagaraParameterMapHistoryBuilder Builder;
			
			if(EmitterHandleGuid.IsValid())
			{
				Builder.ExclusiveEmitterHandle = EmitterHandleGuid;
			}
			
			Builder.BuildParameterMaps(NodeOutput);

			for(FNiagaraParameterMapHistory& History : Builder.Histories)
			{
				MapHistoryVars.Add(History.Variables);
			}
		}
	}

	for(TArray<FNiagaraVariable>& MapHistoryVariables : MapHistoryVars)
	{
		for(FNiagaraVariable& MapHistoryVariable : MapHistoryVariables)
		{
			Vars.Add(MapHistoryVariable);
		}
	}

	bool bAllowStatic = true;
	UNiagaraRendererProperties* RendererProperties = WeakRendererProperties.Get();
	UNiagaraSimulationStageBase* SimulationStage = WeakSimulationStage.Get();

	for (const FNiagaraVariableBase& Var : Vars)
	{
		if (!bAllowStatic && Var.GetType() != TargetVariableBinding->GetType() )
		{
			continue;
		}
		else if (bAllowStatic && !Var.GetType().IsSameBaseDefinition(TargetVariableBinding->GetType()))
		{
			continue;
		}

		if (RendererProperties)
		{
			if (RendererProperties->IsSupportedVariableForBinding(Var, *PropertyHandle->GetProperty()->GetName()))
			{
				Names.AddUnique(Var.GetName());
			}
		}
		else if ( SimulationStage )
		{
			// Unless we have an explicit "particles." binding we are not in the particle namespace
			if (!TargetVariableBinding->GetName().IsNone() && TargetVariableBinding->IsParticleBinding())
			{
				const bool bParticleAttribute = Var.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString);
				if (bParticleAttribute)
				{
					Names.AddUnique(Var.GetName());
				}
			}
			else
			{
				const bool bUserAttribute = Var.IsInNameSpace(FNiagaraConstants::UserNamespaceString);
				const bool bSystemAttribute = Var.IsInNameSpace(FNiagaraConstants::SystemNamespaceString);
				const bool bEmitterAttribute = Var.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString);
				if (bUserAttribute || bSystemAttribute || bEmitterAttribute)
				{
					Names.AddUnique(Var.GetName());
				}
			}
		}
	}

	return Names;
}

void FNiagaraVariableAttributeBindingCustomization::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	TArray<FName> EventNames = GetNames();
	for (FName EventName : EventNames)
	{
		FText CategoryName = FText();
		FString DisplayNameString = FName::NameToDisplayString(EventName.ToString(), false);
		const FText NameText = FText::FromString(DisplayNameString);
		const FText TooltipDesc = FText::Format(LOCTEXT("SetFunctionPopupTooltip", "Use the variable \"{0}\" "), FText::FromString(DisplayNameString));
		TSharedPtr<FNiagaraStackAssetAction_VarBind> NewNodeAction(new FNiagaraStackAssetAction_VarBind(EventName, CategoryName, NameText,
			TooltipDesc, 0, FText()));
		OutAllActions.AddAction(NewNodeAction);
	}
}

TSharedRef<SWidget> FNiagaraVariableAttributeBindingCustomization::OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SNiagaraParameterName)
			.ParameterName(((FNiagaraStackAssetAction_VarBind* const)InCreateData->Action.Get())->VarName)
			.IsReadOnly(true)
			//SNew(STextBlock)
			//.Text(InCreateData->Action->GetMenuDescription())
			.ToolTipText(InCreateData->Action->GetTooltipDescription())
		];
}


void FNiagaraVariableAttributeBindingCustomization::OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType)
{
	if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		for (int32 ActionIndex = 0; ActionIndex < SelectedActions.Num(); ActionIndex++)
		{
			TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedActions[ActionIndex];

			if (CurrentAction.IsValid())
			{
				FSlateApplication::Get().DismissAllMenus();
				FNiagaraStackAssetAction_VarBind* EventSourceAction = (FNiagaraStackAssetAction_VarBind*)CurrentAction.Get();
				ChangeSource(EventSourceAction->VarName);
			}
		}
	}
}

void FNiagaraVariableAttributeBindingCustomization::ChangeSource(FName InVarName)
{
	if (!IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeVariableSource", " Change Variable Source to \"{0}\" "), FText::FromName(InVarName)));
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	for (UObject* Obj : Objects)
	{
		Obj->Modify();
	}

	const FVersionedNiagaraEmitterBase VersionedEmitter(WeakVersionedEmitter);
	const UNiagaraRendererProperties* RendererProperties = WeakRendererProperties.Get();
	const ENiagaraRendererSourceDataMode BindingSourceMode = RendererProperties ? RendererProperties->GetCurrentSourceMode() : ENiagaraRendererSourceDataMode::Emitter;

	PropertyHandle->NotifyPreChange();
	TargetVariableBinding->SetValue(InVarName, VersionedEmitter, BindingSourceMode);
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

void FNiagaraVariableAttributeBindingCustomization::ResetToDefault()
{
	UE_LOGF(LogNiagaraEditor, Warning, "Reset to default!");
}

EVisibility FNiagaraVariableAttributeBindingCustomization::IsResetToDefaultsVisible() const
{
	if (!IsValid())
	{
		return EVisibility::Hidden;
	}

	const UNiagaraRendererProperties* RendererProperties = WeakRendererProperties.Get();
	const ENiagaraRendererSourceDataMode BindingSourceMode = RendererProperties ? RendererProperties->GetCurrentSourceMode() : ENiagaraRendererSourceDataMode::Emitter;
	return TargetVariableBinding->MatchesDefault(*DefaultVariableBinding, BindingSourceMode) ? EVisibility::Hidden : EVisibility::Visible;
}

FReply FNiagaraVariableAttributeBindingCustomization::OnResetToDefaultsClicked()
{
	if (!IsValid())
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("ResetBindingParam", "Reset binding"));
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	for (UObject* Obj : Objects)
	{
		Obj->Modify();
	}

	const FVersionedNiagaraEmitterBase VersionedEmitter(WeakVersionedEmitter);
	const UNiagaraRendererProperties* RendererProperties = WeakRendererProperties.Get();
	const ENiagaraRendererSourceDataMode BindingSourceMode = RendererProperties ? RendererProperties->GetCurrentSourceMode() : ENiagaraRendererSourceDataMode::Emitter;

	PropertyHandle->NotifyPreChange();
	TargetVariableBinding->ResetToDefault(*DefaultVariableBinding, VersionedEmitter, BindingSourceMode);
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
	return FReply::Handled();
}

void FNiagaraVariableAttributeBindingCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	bool bAddDefault = true;

	InPropertyHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateLambda([this]() { ResetToDefault(); }));
	InPropertyHandle->MarkResetToDefaultCustomized(true);
	
	if (Objects.Num() == 1)
	{
		FVersionedNiagaraEmitterBase VersionedEmitter;
		if ( UNiagaraRendererProperties* RenderProperties = Cast<UNiagaraRendererProperties>(Objects[0]) )
		{
			WeakRendererProperties = RenderProperties;
			VersionedEmitter = RenderProperties->GetOuterEmitterBase();
		}
		else if (UNiagaraSimulationStageBase* SimulationStage = Cast<UNiagaraSimulationStageBase>(Objects[0]))
		{
			WeakSimulationStage = SimulationStage;
			VersionedEmitter = SimulationStage->GetOuterEmitterBase();
		}

		WeakVersionedEmitter = FVersionedNiagaraEmitterBaseWeakPtr(VersionedEmitter);

		if (UNiagaraEmitterBase* EmitterBase = VersionedEmitter.Emitter)
		{
			UNiagaraSystem* System = EmitterBase->GetTypedOuter<UNiagaraSystem>();
			TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(System);
			
			if ( SystemViewModel.IsValid() )
			{
				// the owning emitter we retrieve might have the wrong version guid after changing emitter versions, as the sim stage & render props 'OuterEmitterVersion' can be overwritten during merge.
				// so we rely on matching names to find the correct handle guid instead of using the version guid
				const FName EmitterBaseFName(EmitterBase->GetUniqueEmitterName());
				const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& EmitterHandleViewModels = SystemViewModel->GetEmitterHandleViewModels();
				for ( TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels )
				{
					if ( EmitterHandleViewModel->GetName().IsEqual(EmitterBaseFName) )
					{
						EmitterHandleGuid = EmitterHandleViewModel->GetId();
					}
				}

				if(EmitterHandleGuid.IsValid() == false)
				{
					UE_LOGF(LogNiagaraEditor, Warning, "EmitterHandleViewModel was not valid. This shouldn't happen and implies some mismatch in emitter names.");
				}
			}
			
			TargetVariableBinding = (FNiagaraVariableAttributeBinding*)PropertyHandle->GetValueBaseAddress((uint8*)Objects[0]);
			DefaultVariableBinding = (FNiagaraVariableAttributeBinding*)PropertyHandle->GetValueBaseAddress((uint8*)Objects[0]->GetClass()->GetDefaultObject());
				
			HeaderRow
				.NameContent()
				[
					PropertyHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, 4.0f, 0.0f)
					[
						SAssignNew(ComboButton, SComboButton)					
						.OnGetMenuContent(this, &FNiagaraVariableAttributeBindingCustomization::OnGetMenuContent)
						.ContentPadding(1)
						.ToolTipText(this, &FNiagaraVariableAttributeBindingCustomization::GetTooltipText)
						.ButtonContent()
						[
							SNew(SNiagaraParameterName)
							.ParameterName(this, &FNiagaraVariableAttributeBindingCustomization::GetVariableName)
							.IsReadOnly(true)
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2, 1)
					[
						SNew(SButton)
						.IsFocusable(false)
						.ToolTipText(LOCTEXT("ResetToDefaultToolTip", "Reset to Default"))
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ContentPadding(0)
						.Visibility(this, &FNiagaraVariableAttributeBindingCustomization::IsResetToDefaultsVisible)
						.OnClicked(this, &FNiagaraVariableAttributeBindingCustomization::OnResetToDefaultsClicked)
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				];
			bAddDefault = false;
		}
	}
	
	if (bAddDefault)
	{
		HeaderRow
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(200.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FName::NameToDisplayString(CastField<FStructProperty>(PropertyHandle->GetProperty())->Struct->GetName(), false)))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
}

//////////////////////////////////////////////////////////////////////////

FName FNiagaraUserParameterBindingCustomization::GetVariableName() const
{
	if (IsValid() && TargetUserParameterBinding)
	{
		return (TargetUserParameterBinding->Parameter.GetName());
	}
	return FName();
}

FText FNiagaraUserParameterBindingCustomization::GetCurrentText() const
{
	if (IsValid() && TargetUserParameterBinding)
	{
		return FText::FromName(TargetUserParameterBinding->Parameter.GetName());
	}
	return FText::FromString(TEXT("Missing"));
}

FText FNiagaraUserParameterBindingCustomization::GetTooltipText() const
{
	if (IsValid() && TargetUserParameterBinding)
	{
		const FNiagaraTypeDefinition TypeDef = TargetUserParameterBinding->Parameter.GetType();
		if (TypeDef.IsValid())
		{
			FText TooltipFormat;
			if (TargetUserParameterBinding->Parameter.IsValid())
			{
				TooltipFormat = LOCTEXT("UserParameterTooltip_Bound", "Bound to parameter \"{0}\".\nCan be bound to parameters of type \"{1}\".");
			}
			else
			{
				TooltipFormat = LOCTEXT("UserParameterTooltip_Unbound", "Currently not bound to any parameter.\nCan be bound to parameters of type \"{1}\".");
			}

			return FText::Format(
				TooltipFormat,
				FText::FromName(TargetUserParameterBinding->Parameter.GetName()),
				TypeDef.GetNameText()
			);
		}
	}
	return LOCTEXT("InvalidBinding", "Binding is invalid");
}

TSharedRef<SWidget> FNiagaraUserParameterBindingCustomization::OnGetMenuContent() const
{
	FGraphActionMenuBuilder MenuBuilder;

	return SNew(SBorder)
		.Padding(5)
		[
			SNew(SBox)
			[
				SNew(SGraphActionMenu)
				.OnActionSelected(const_cast<FNiagaraUserParameterBindingCustomization*>(this), &FNiagaraUserParameterBindingCustomization::OnActionSelected)
				.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(const_cast<FNiagaraUserParameterBindingCustomization*>(this), &FNiagaraUserParameterBindingCustomization::OnCreateWidgetForAction))
				.OnCollectAllActions(const_cast<FNiagaraUserParameterBindingCustomization*>(this), &FNiagaraUserParameterBindingCustomization::CollectAllActions)
				.AutoExpandActionMenu(false)
				.ShowFilterTextBox(true)
			]
		];
}

TArray<FName> FNiagaraUserParameterBindingCustomization::GetNames() const
{
	TArray<FName> Names;

	if (IsValid() && TargetUserParameterBinding)
	{
		// Build list of allowed types
		TArray<FNiagaraTypeDefinition> AllowedTypes;
		AllowedTypes.Add(TargetUserParameterBinding->Parameter.GetType());
		{
			TArray<FString> AdditionalAllowedClasses;
			const FString& AdditionalAllowedClassesMetadata = PropertyHandle->GetProperty()->GetMetaData(TEXT("AdditionalAllowedClasses"));
			if (!AdditionalAllowedClassesMetadata.IsEmpty())
			{
				AdditionalAllowedClassesMetadata.ParseIntoArray(AdditionalAllowedClasses, TEXT(","), true);
				for (const FString& AllowedClass : AdditionalAllowedClasses)
				{
					if ( UClass* Class = FindObject<UClass>(nullptr, *AllowedClass) )
					{
						AllowedTypes.Add(FNiagaraTypeDefinition(Class));
					}
				}
			}
		}

		UNiagaraSystem* BaseSystem = BaseSystemWeakPtr.Get();
		for (const FNiagaraVariable Var : BaseSystem->GetExposedParameters().ReadParameterVariables())
		{
			if (FNiagaraParameterUtilities::IsUserParameter(Var) && AllowedTypes.Contains(Var.GetType()))
			{
				Names.AddUnique(Var.GetName());
			}
		}
	}
	
	return Names;
}

void FNiagaraUserParameterBindingCustomization::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	TArray<FName> UserParamNames = GetNames();
	for (FName UserParamName : UserParamNames)
	{
		FText CategoryName = FText();
		FString DisplayNameString = FName::NameToDisplayString(UserParamName.ToString(), false);
		const FText NameText = FText::FromString(DisplayNameString);
		const FText TooltipDesc = FText::Format(LOCTEXT("BindToUserParameter", "Bind to the User Parameter \"{0}\" "), FText::FromString(DisplayNameString));
		TSharedPtr<FNiagaraStackAssetAction_VarBind> NewNodeAction(new FNiagaraStackAssetAction_VarBind(UserParamName, CategoryName, NameText,
			TooltipDesc, 0, FText()));
		OutAllActions.AddAction(NewNodeAction);
	}
}

TSharedRef<SWidget> FNiagaraUserParameterBindingCustomization::OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SNiagaraParameterName)
			.ParameterName(((FNiagaraStackAssetAction_VarBind* const)InCreateData->Action.Get())->VarName)
			.IsReadOnly(true)
			//SNew(STextBlock)
			//.Text(InCreateData->Action->GetMenuDescription())
			.ToolTipText(InCreateData->Action->GetTooltipDescription())
		];
}


void FNiagaraUserParameterBindingCustomization::OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType)
{
	if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		for (int32 ActionIndex = 0; ActionIndex < SelectedActions.Num(); ActionIndex++)
		{
			TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedActions[ActionIndex];

			if (CurrentAction.IsValid())
			{
				FSlateApplication::Get().DismissAllMenus();
				FNiagaraStackAssetAction_VarBind* EventSourceAction = (FNiagaraStackAssetAction_VarBind*)CurrentAction.Get();
				ChangeSource(EventSourceAction->VarName);
			}
		}
	}
}

void FNiagaraUserParameterBindingCustomization::ChangeSource(FName InVarName)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeUserParameterSource", " Change User Parameter Source to \"{0}\" "), FText::FromName(InVarName)));
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	for (UObject* Obj : Objects)
	{
		Obj->Modify();
	}

	PropertyHandle->NotifyPreChange();
	TargetUserParameterBinding->Parameter.SetName(InVarName);
	//TargetUserParameterBinding->Parameter.SetType(FNiagaraTypeDefinition::GetUObjectDef()); Do not override the type here!
	//TargetVariableBinding->DataSetVariable = FNiagaraConstants::GetAttributeAsDataSetKey(TargetVariableBinding->BoundVariable);
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

void FNiagaraUserParameterBindingCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	bool bAddDefault = true;

	ObjectCustomizingWeakPtr.Reset();
	BaseSystemWeakPtr.Reset();

	if (Objects.Num() == 1)
	{
		UNiagaraSystem* BaseSystem = Objects[0]->GetTypedOuter<UNiagaraSystem>();
		if (BaseSystem == nullptr)
		{
			if (UNiagaraDataInterface* ObjectAsDataInterface = Cast<UNiagaraDataInterface>(Objects[0]))
			{
				FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
				FVersionedNiagaraEmitter Emitter;
				NiagaraEditorModule.GetTargetSystemAndEmitterForDataInterface(ObjectAsDataInterface, BaseSystem, Emitter);
			}
		}
		
		if (BaseSystem)
		{
			ObjectCustomizingWeakPtr = Objects[0];
			BaseSystemWeakPtr = BaseSystem;

			TargetUserParameterBinding = (FNiagaraUserParameterBinding*)PropertyHandle->GetValueBaseAddress((uint8*)Objects[0]);

			HeaderRow
				.NameContent()
				[
					PropertyHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				.MaxDesiredWidth(200.f)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &FNiagaraUserParameterBindingCustomization::OnGetMenuContent)
					.ContentPadding(1)
					.ToolTipText(this, &FNiagaraUserParameterBindingCustomization::GetTooltipText)
					.ButtonContent()
					[
						SNew(SNiagaraParameterName)
						.ParameterName(this, &FNiagaraUserParameterBindingCustomization::GetVariableName)
						.IsReadOnly(true)
					]
				];

			bAddDefault = false;
		}
	}

	if (bAddDefault)
	{
		HeaderRow
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
		.ValueContent()
			.MaxDesiredWidth(200.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FName::NameToDisplayString(CastField<FStructProperty>(PropertyHandle->GetProperty())->Struct->GetName(), false)))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
}

//////////////////////////////////////////////////////////////////////////
FName FNiagaraMaterialAttributeBindingCustomization::GetNiagaraVariableName() const
{
	if (BaseSystem && TargetParameterBinding)
	{
		return TargetParameterBinding->NiagaraVariable.GetName();
	}
	return FName();
}

FName FNiagaraMaterialAttributeBindingCustomization::GetNiagaraChildVariableName() const
{
	if (BaseSystem && TargetParameterBinding)
	{
		return TargetParameterBinding->NiagaraChildVariable.GetName();
	}
	return FName();
}

FText FNiagaraMaterialAttributeBindingCustomization::GetNiagaraCurrentText() const
{
	if (BaseSystem && TargetParameterBinding)
	{
		return MakeCurrentText(TargetParameterBinding->NiagaraVariable, TargetParameterBinding->NiagaraChildVariable);
	}
	return FText::FromString(TEXT("Missing"));
}


FText FNiagaraMaterialAttributeBindingCustomization::MakeCurrentText(const FNiagaraVariableBase& BaseVar, const FNiagaraVariableBase& ChildVar) 
{
	if (BaseVar.GetName().IsNone())
	{
		return FText::FromName(NAME_None);
	}

	FString DisplayNameString = FName::NameToDisplayString(BaseVar.GetName().ToString(), false);
	FNiagaraTypeDefinition TargetType = BaseVar.GetType();
	if (ChildVar.GetName() != NAME_None)
	{
		DisplayNameString += TEXT(" \"") + FName::NameToDisplayString(ChildVar.GetName().ToString(), false) + TEXT("\"");
		TargetType = ChildVar.GetType();
	}

	DisplayNameString += TEXT(" (") + FName::NameToDisplayString(TargetType.GetFName().ToString(), false) + TEXT(")");

	const FText NameText = FText::FromString(DisplayNameString);
	return NameText;
}

FText FNiagaraMaterialAttributeBindingCustomization::GetNiagaraTooltipText() const
{
	if (BaseSystem && TargetParameterBinding && TargetParameterBinding->NiagaraVariable.IsValid())
	{
		FText TooltipDesc = FText::Format(LOCTEXT("MaterialAttributeBindingTooltip", "Bound to the parameter \"{0}\""), MakeCurrentText(TargetParameterBinding->NiagaraVariable, TargetParameterBinding->NiagaraChildVariable));
		return TooltipDesc;
	}
	return FText::FromString(TEXT("Missing"));
}

TSharedRef<SWidget> FNiagaraMaterialAttributeBindingCustomization::OnGetNiagaraMenuContent() const
{
	FGraphActionMenuBuilder MenuBuilder;

	TSharedRef<SGraphActionMenu> GraphActionMenu = SNew(SGraphActionMenu)
		.OnActionSelected(const_cast<FNiagaraMaterialAttributeBindingCustomization*>(this), &FNiagaraMaterialAttributeBindingCustomization::OnNiagaraActionSelected)
		.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(const_cast<FNiagaraMaterialAttributeBindingCustomization*>(this), &FNiagaraMaterialAttributeBindingCustomization::OnCreateWidgetForNiagaraAction))
		.OnCollectAllActions(const_cast<FNiagaraMaterialAttributeBindingCustomization*>(this), &FNiagaraMaterialAttributeBindingCustomization::CollectAllNiagaraActions)
		.AutoExpandActionMenu(false)
		.ShowFilterTextBox(true);
	
	TSharedRef<SWidget> MenuContent =  SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			[
				GraphActionMenu
			]
		];

	NiagaraParameterButton->SetMenuContentWidgetToFocus(GraphActionMenu->GetFilterTextBox());
	return MenuContent;
}

TArray<FNiagaraTypeDefinition> FNiagaraMaterialAttributeBindingCustomization::GetAllowedVariableTypes() const
{
	TArray<FNiagaraTypeDefinition> AllowedTypes;
	if (BaseSystem && TargetParameterBinding && PropertyHandle)
	{
		TArray<UObject*> Objects;
		PropertyHandle->GetOuterObjects(Objects);
	
		if (Objects.Num() == 1)
		{
			UNiagaraRendererProperties* RendererProperties = Cast<UNiagaraRendererProperties>(Objects[0]);
			TArray<UMaterialInterface*> Materials;
			if (RendererProperties)
			{
				RendererProperties->GetUsedMaterials(nullptr, Materials);
			}
	
			
			auto MatchMaterialParameter =
				[MaterialName=TargetParameterBinding->MaterialParameterName](const FMaterialParameterInfo& InParameter)
				{
					return InParameter.Name == MaterialName;
				};

			TArray<FMaterialParameterInfo> TempParameterInfo;
			TArray<FGuid> ParameterIds;
			bool bHasTextureBinding = false;
			bool bHasScalarBinding = false;
			bool bHasVectorBinding = false;
			bool bHasDoubleVectorBinding = false;
			for (UMaterialInterface* Material : Materials)
			{
				if (!Material)
				{
					continue;
				}
	
				Material->GetAllTextureParameterInfo(TempParameterInfo, ParameterIds);
				bHasTextureBinding |= TempParameterInfo.ContainsByPredicate(MatchMaterialParameter);
				TempParameterInfo.Reset();
	
				Material->GetAllScalarParameterInfo(TempParameterInfo, ParameterIds);
				bHasScalarBinding |= TempParameterInfo.ContainsByPredicate(MatchMaterialParameter);
				TempParameterInfo.Reset();

				Material->GetAllVectorParameterInfo(TempParameterInfo, ParameterIds);
				bHasVectorBinding |= TempParameterInfo.ContainsByPredicate(MatchMaterialParameter);
				TempParameterInfo.Reset();

				Material->GetAllDoubleVectorParameterInfo(TempParameterInfo, ParameterIds);
				bHasDoubleVectorBinding |= TempParameterInfo.ContainsByPredicate(MatchMaterialParameter);
				TempParameterInfo.Reset();

				ParameterIds.Reset();
			}

			if (bHasTextureBinding)
			{
				AllowedTypes.AddUnique(FNiagaraTypeDefinition::GetUObjectDef());
				AllowedTypes.AddUnique(FNiagaraTypeDefinition::GetUTextureDef());
				AllowedTypes.AddUnique(FNiagaraTypeDefinition::GetUTextureRenderTargetDef());
			}
			if (bHasScalarBinding)
			{
				AllowedTypes.AddUnique(FNiagaraTypeDefinition::GetFloatDef());
			}
			if (bHasVectorBinding)
			{
				AllowedTypes.AddUnique(FNiagaraTypeDefinition::GetVec4Def());
				AllowedTypes.AddUnique(FNiagaraTypeDefinition::GetColorDef());
				AllowedTypes.AddUnique(FNiagaraTypeDefinition::GetVec2Def());
				AllowedTypes.AddUnique(FNiagaraTypeDefinition::GetVec3Def());
			}
			if (bHasDoubleVectorBinding)
			{
				AllowedTypes.AddUnique(FNiagaraTypeDefinition::GetPositionDef());
			}
		}
	}
	return AllowedTypes;
}

TArray<TPair<FNiagaraVariableBase, FNiagaraVariableBase> > FNiagaraMaterialAttributeBindingCustomization::GetNiagaraNames() const
{
	TArray<TPair<FNiagaraVariableBase, FNiagaraVariableBase>> Names;
	TArray<FNiagaraVariableBase> BaseVars;

	//-TODO:stateless:Remove and unify emitter
	if (BaseSystem && (BaseEmitter.Emitter || StatelessEmitter) && TargetParameterBinding)
	//-TODO:stateless:Remove and unify emitter
	{
		const bool bSystem = true;
		//-TODO:stateless:Remove and unify emitter
		const bool bEmitter = true;
		//-TODO:stateless:Remove and unify emitter
		const bool bParticles = false;
		const bool bUser = true;
		const bool bStatic = false;
		BaseVars = FNiagaraStackAssetAction_VarBind::FindVariables(BaseSystem, BaseEmitter, bSystem, bEmitter, bParticles, bUser, bStatic);
		
		TArray<FNiagaraScriptDataKey> ScriptDataKeys;
		ScriptDataKeys.Add(FNiagaraScriptDataKey(nullptr, *BaseSystem->GetSystemSpawnScript()));
		ScriptDataKeys.Add(FNiagaraScriptDataKey(nullptr, *BaseSystem->GetSystemUpdateScript()));
		if (FVersionedNiagaraEmitterData* EmitterData = BaseEmitter.GetEmitterData())
		{
			TArray<UNiagaraScript*> Scripts;
			EmitterData->GetScripts(Scripts, false);
			for (UNiagaraScript* Script : Scripts)
			{
				ScriptDataKeys.Add(FNiagaraScriptDataKey(BaseEmitterHandleId, Script->GetUsage(), Script->GetUsageId()));
			}
		}

		TMap<FString, FString> EmitterAlias;
		//-TODO:stateless:Remove and unify emitter
		if (BaseEmitter.Emitter)
		//-TODO:stateless:Remove and unify emitter
		{
			EmitterAlias.Emplace(FNiagaraConstants::EmitterNamespace.ToString(), BaseEmitter.Emitter->GetUniqueEmitterName());
		}

		auto FindCachedDI = 
			[&](const FNiagaraVariableBase& BaseVariable) -> UNiagaraDataInterface*
			{
				FNiagaraVariableBase ResolvedVariable;
				if (BaseVariable.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString))
				{
					//-TODO:stateless:Remove and unify emitter
					check(BaseEmitter.Emitter);
					//-TODO:stateless:Remove and unify emitter
					ResolvedVariable = FNiagaraUtilities::ResolveAliases(BaseVariable, FNiagaraAliasContext()
						.ChangeEmitterToEmitterName(BaseEmitter.Emitter->GetUniqueEmitterName()));
				}
				else
				{
					ResolvedVariable = BaseVariable;
				}

				for (const FNiagaraScriptDataKey& ScriptDataKey : ScriptDataKeys)
				{
					const FNiagaraScriptRuntimeData* ScriptRuntimeData = BaseSystem->GetScriptRuntimeData(ScriptDataKey).Get();
					if (ScriptRuntimeData == nullptr)
					{
						continue;
					}
					
					for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDataInterfaceData : ScriptRuntimeData->GetResolvedDataInterfaces())
					{
						if (ResolvedDataInterfaceData.ParameterStoreVariable == ResolvedVariable)
						{
							return ResolvedDataInterfaceData.ResolvedDataInterface;
						}
					}
				}

				return BaseVariable.GetType().GetClass()->GetDefaultObject<UNiagaraDataInterface>();
			};

		if (BaseVars.Num() > 0)
		{
			const TArray<FNiagaraTypeDefinition> AllowedVariableTypes = GetAllowedVariableTypes();
			for (const FNiagaraVariableBase& BaseVar : BaseVars)
			{
				if (BaseVar.IsDataInterface())
				{
					UNiagaraDataInterface* DI = FindCachedDI(BaseVar);
					if (DI && DI->CanExposeVariables())
					{
						TArray<FNiagaraVariableBase> ChildVars;
						DI->GetExposedVariables(ChildVars);
						for (const FNiagaraVariableBase& ChildVar : ChildVars)
						{
							if (!AllowedVariableTypes.Num() || AllowedVariableTypes.Contains(ChildVar.GetType()))
							{
								Names.AddUnique(TPair<FNiagaraVariableBase, FNiagaraVariableBase>(BaseVar, ChildVar));
							}
						}
					}
				}
				else if (!AllowedVariableTypes.Num() || AllowedVariableTypes.Contains(BaseVar.GetType()))
				{
					if (RenderProps && TargetParameterBinding)
					{
						Names.AddUnique(TPair<FNiagaraVariableBase, FNiagaraVariableBase>(BaseVar, FNiagaraVariableBase()));
					}
				}
			}
		}
	}

	return Names;
}

void FNiagaraMaterialAttributeBindingCustomization::CollectAllNiagaraActions(FGraphActionListBuilderBase& OutAllActions)
{
	TArray<TPair<FNiagaraVariableBase, FNiagaraVariableBase>> ParamNames = GetNiagaraNames();
	for (TPair<FNiagaraVariableBase, FNiagaraVariableBase> ParamPair : ParamNames)
	{
		FText CategoryName = FText();
		const FText NameText = MakeCurrentText(ParamPair.Key, ParamPair.Value);
		const FText TooltipDesc = FText::Format(LOCTEXT("BindToNiagaraParameter", "Bind to the Niagara Variable \"{0}\" "), NameText);
		FNiagaraStackAssetAction_VarBind* VarBind = new FNiagaraStackAssetAction_VarBind(ParamPair.Key.GetName(), CategoryName, NameText,
			TooltipDesc, 0, FText());
		VarBind->BaseVar = ParamPair.Key;
		VarBind->ChildVar = ParamPair.Value;
		TSharedPtr<FNiagaraStackAssetAction_VarBind> NewNodeAction(VarBind);
		OutAllActions.AddAction(NewNodeAction);
	}
}


FText FNiagaraMaterialAttributeBindingCustomization::GetNiagaraChildVariableText() const
{

	FName ChildVarName = GetNiagaraChildVariableName();
	FText ChildVarNameText = ChildVarName.IsNone() == false ? FText::FromString(TEXT("| ") + ChildVarName.ToString()) : FText::GetEmpty();
	return ChildVarNameText;
}

EVisibility FNiagaraMaterialAttributeBindingCustomization::GetNiagaraChildVariableVisibility() const
{
	FName ChildVarName = GetNiagaraChildVariableName();
	return ChildVarName.IsNone() ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedRef<SWidget> FNiagaraMaterialAttributeBindingCustomization::OnCreateWidgetForNiagaraAction(struct FCreateWidgetForActionData* const InCreateData)
{
	FName ChildVarName = (((FNiagaraStackAssetAction_VarBind* const)InCreateData->Action.Get())->ChildVar.GetName());
	FText ChildVarNameText = ChildVarName.IsNone() == false ? FText::FromString(TEXT("| ") + ChildVarName.ToString()) : FText::GetEmpty();
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(3, 0)
			[
				SNew(SNiagaraParameterName)
				.ParameterName(((FNiagaraStackAssetAction_VarBind* const)InCreateData->Action.Get())->VarName)
				.IsReadOnly(true)
				//SNew(STextBlock)
				//.Text(InCreateData->Action->GetMenuDescription())
				.ToolTipText(InCreateData->Action->GetTooltipDescription())
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(3, 0)
			[
				SNew(STextBlock)
				.Visibility(ChildVarName.IsNone() ? EVisibility::Collapsed : EVisibility::Visible)
				.Text(ChildVarNameText)
			]
		];
}


void FNiagaraMaterialAttributeBindingCustomization::OnNiagaraActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType)
{
	if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		for (int32 ActionIndex = 0; ActionIndex < SelectedActions.Num(); ActionIndex++)
		{
			TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedActions[ActionIndex];

			if (CurrentAction.IsValid())
			{
				FSlateApplication::Get().DismissAllMenus();
				FNiagaraStackAssetAction_VarBind* EventSourceAction = (FNiagaraStackAssetAction_VarBind*)CurrentAction.Get();
				ChangeNiagaraSource(EventSourceAction);
			}
		}
	}
}

void FNiagaraMaterialAttributeBindingCustomization::ChangeNiagaraSource(FNiagaraStackAssetAction_VarBind* InVar)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeParameterSource", " Change Parameter Source to \"{0}\" "), FText::FromName(InVar->VarName)));
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	for (UObject* Obj : Objects)
	{
		Obj->Modify();
	}

	PropertyHandle->NotifyPreChange();
	TargetParameterBinding->NiagaraVariable = InVar->BaseVar;
	TargetParameterBinding->NiagaraChildVariable = InVar->ChildVar;
	TargetParameterBinding->CacheValues(BaseEmitter.Emitter);
	//TargetParameterBinding->Parameter.SetType(FNiagaraTypeDefinition::GetUObjectDef()); Do not override the type here!
	//TargetVariableBinding->DataSetVariable = FNiagaraConstants::GetAttributeAsDataSetKey(TargetVariableBinding->BoundVariable);
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

FText FNiagaraMaterialAttributeBindingCustomization::GetMaterialCurrentText() const
{
	if (BaseSystem && TargetParameterBinding)
	{
		return FText::FromName(TargetParameterBinding->MaterialParameterName);
	}
	return FText::FromString(TEXT("Missing"));
}

FText FNiagaraMaterialAttributeBindingCustomization::GetMaterialTooltipText() const
{
	if (BaseSystem && TargetParameterBinding && TargetParameterBinding->MaterialParameterName.IsValid())
	{
		FText TooltipDesc = FText::Format(LOCTEXT("MaterialParameterBindingTooltip", "Bound to the parameter \"{0}\""), FText::FromName(TargetParameterBinding->MaterialParameterName));
		return TooltipDesc;
	}
	return FText::FromString(TEXT("Missing"));
}

TSharedRef<SWidget> FNiagaraMaterialAttributeBindingCustomization::OnGetMaterialMenuContent() const
{
	FGraphActionMenuBuilder MenuBuilder;

	TSharedRef<SGraphActionMenu> GraphActionMenu = SNew(SGraphActionMenu)
		.OnActionSelected(const_cast<FNiagaraMaterialAttributeBindingCustomization*>(this), &FNiagaraMaterialAttributeBindingCustomization::OnMaterialActionSelected)
		.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(const_cast<FNiagaraMaterialAttributeBindingCustomization*>(this), &FNiagaraMaterialAttributeBindingCustomization::OnCreateWidgetForMaterialAction))
		.OnCollectAllActions(const_cast<FNiagaraMaterialAttributeBindingCustomization*>(this), &FNiagaraMaterialAttributeBindingCustomization::CollectAllMaterialActions)
		.AutoExpandActionMenu(false)
		.ShowFilterTextBox(true);

	TSharedRef<SWidget> MenuContent = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			[
				GraphActionMenu
			]
		];

	MaterialParameterButton->SetMenuContentWidgetToFocus(GraphActionMenu->GetFilterTextBox());
	return MenuContent;
}

void FNiagaraMaterialAttributeBindingCustomization::GetMaterialParameters(TArray<TPair<FName, FString>>& OutBindingNameAndDescription) const
{
	TSet<FName> AddedParameterNames;
	auto TransformMaterialParameterInfo =
		[&OutBindingNameAndDescription, &AddedParameterNames](UMaterialInterface* Material, TArray<FMaterialParameterInfo>& ParameterInfos)
		{
			for (const FMaterialParameterInfo& ParameterInfo : ParameterInfos)
			{
				if (ParameterInfo.Association != EMaterialParameterAssociation::GlobalParameter)
				{
					continue;
				}
				if (AddedParameterNames.Contains(ParameterInfo.Name))
				{
					continue;
				}
				AddedParameterNames.Add(ParameterInfo.Name);

				FString ParameterDesc;
				Material->GetParameterDesc(ParameterInfo, ParameterDesc);
				OutBindingNameAndDescription.Emplace(ParameterInfo.Name, ParameterDesc);
			}
			ParameterInfos.Reset();
		};

	if (BaseSystem && TargetParameterBinding && PropertyHandle)
	{
		TArray<UObject*> Objects;
		PropertyHandle->GetOuterObjects(Objects);

		if (Objects.Num() == 1)
		{
			UNiagaraRendererProperties* RendererProperties = Cast<UNiagaraRendererProperties>(Objects[0]);
			TArray<UMaterialInterface*> Materials;
			if (RendererProperties)
			{
				RendererProperties->GetUsedMaterials(nullptr, Materials);
			}

			TArray<FMaterialParameterInfo> TempParameterInfo;
			TArray<FGuid> ParameterIds;
			for (UMaterialInterface* Material : Materials)
			{
				if (!Material)
				{
					continue;
				}

				Material->GetAllTextureParameterInfo(TempParameterInfo, ParameterIds);
				TransformMaterialParameterInfo(Material, TempParameterInfo);

				Material->GetAllScalarParameterInfo(TempParameterInfo, ParameterIds);
				TransformMaterialParameterInfo(Material, TempParameterInfo);

				Material->GetAllVectorParameterInfo(TempParameterInfo, ParameterIds);
				TransformMaterialParameterInfo(Material, TempParameterInfo);

				Material->GetAllDoubleVectorParameterInfo(TempParameterInfo, ParameterIds);
				TransformMaterialParameterInfo(Material, TempParameterInfo);

				ParameterIds.Reset();
			}
		}
	}
}

void FNiagaraMaterialAttributeBindingCustomization::CollectAllMaterialActions(FGraphActionListBuilderBase& OutAllActions)
{
	TArray<TPair<FName, FString>> MaterialParameters;
	GetMaterialParameters(MaterialParameters);

	for (const TPair<FName, FString>& Parameter : MaterialParameters)
	{
		TSharedPtr<FNiagaraStackAssetAction_VarBind> NewNodeAction(
			new FNiagaraStackAssetAction_VarBind(
				Parameter.Key,
				FText(),
				FText::FromName(Parameter.Key),
				FNiagaraRendererMaterialParameterCustomization::GetMaterialBindingTooltip(Parameter.Key, Parameter.Value),
				0,
				FText()
			)
		);
		OutAllActions.AddAction(NewNodeAction);
	}
}

TSharedRef<SWidget> FNiagaraMaterialAttributeBindingCustomization::OnCreateWidgetForMaterialAction(struct FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(InCreateData->Action->GetMenuDescription())
		.ToolTipText(InCreateData->Action->GetTooltipDescription())
		];
}


void FNiagaraMaterialAttributeBindingCustomization::OnMaterialActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType)
{
	if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		for (int32 ActionIndex = 0; ActionIndex < SelectedActions.Num(); ActionIndex++)
		{
			TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedActions[ActionIndex];

			if (CurrentAction.IsValid())
			{
				FSlateApplication::Get().DismissAllMenus();
				FNiagaraStackAssetAction_VarBind* EventSourceAction = (FNiagaraStackAssetAction_VarBind*)CurrentAction.Get();
				ChangeMaterialSource(EventSourceAction->VarName);
			}
		}
	}
}

void FNiagaraMaterialAttributeBindingCustomization::ChangeMaterialSource(FName InVarName)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeParameterSource", " Change Parameter Source to \"{0}\" "), FText::FromName(InVarName)));
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	for (UObject* Obj : Objects)
	{
		Obj->Modify();
	}

	PropertyHandle->NotifyPreChange();
	TargetParameterBinding->MaterialParameterName = InVarName;
	//TargetParameterBinding->Parameter.SetType(FMaterialTypeDefinition::GetUObjectDef()); Do not override the type here!
	//TargetVariableBinding->DataSetVariable = FMaterialConstants::GetAttributeAsDataSetKey(TargetVariableBinding->BoundVariable);
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

void FNiagaraMaterialAttributeBindingCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	bool bAddDefault = true;
	

	if (bAddDefault)
	{
		HeaderRow
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
		.ValueContent()
			.MaxDesiredWidth(200.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ParamHeaderValue", "Binding"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
}


void FNiagaraMaterialAttributeBindingCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	RenderProps = nullptr;
	BaseSystem =  nullptr;
	BaseEmitter = FVersionedNiagaraEmitter();
	BaseEmitterHandleId = FGuid();
	  
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	if (Objects.Num() == 1)
	{
		RenderProps = Cast<UNiagaraRendererProperties>(Objects[0]);
		BaseSystem = Objects[0]->GetTypedOuter<UNiagaraSystem>();
		if (RenderProps)
		{
			BaseEmitter = RenderProps->GetOuterEmitter();
			if (BaseEmitter.Emitter != nullptr)
			{
				const FNiagaraEmitterHandle* EmitterHandle = BaseSystem != nullptr 
					? BaseSystem->GetEmitterHandles().FindByPredicate([this]
						(const FNiagaraEmitterHandle& EmitterHandle) { return EmitterHandle.GetInstance().Emitter == BaseEmitter.Emitter; })
					: nullptr;
				BaseEmitterHandleId = EmitterHandle != nullptr ? EmitterHandle->GetId() : FGuid();
			}
			//-TODO:stateless:Remove and unify emitter
			StatelessEmitter = RenderProps->GetTypedOuter<UNiagaraStatelessEmitter>();
			//-TODO:stateless:Remove and unify emitter
		}
		if (BaseSystem)
		{
			TargetParameterBinding = (FNiagaraMaterialAttributeBinding*)PropertyHandle->GetValueBaseAddress((uint8*)Objects[0]);

			TSharedPtr<IPropertyHandle> ChildPropertyHandle = StructPropertyHandle->GetChildHandle(0);
			FDetailWidgetRow& RowMaterial = ChildBuilder.AddCustomRow(FText::GetEmpty());
			RowMaterial
				.NameContent()
				[
					ChildPropertyHandle->CreatePropertyNameWidget()
				]
			.ValueContent()
				.MaxDesiredWidth(200.f)
				[
					SAssignNew(MaterialParameterButton, SComboButton)
					.OnGetMenuContent(this, &FNiagaraMaterialAttributeBindingCustomization::OnGetMaterialMenuContent)
					.ContentPadding(1)
					.ToolTipText(this, &FNiagaraMaterialAttributeBindingCustomization::GetMaterialTooltipText)
					.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
					.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &FNiagaraMaterialAttributeBindingCustomization::GetMaterialCurrentText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				];

			ChildPropertyHandle = StructPropertyHandle->GetChildHandle(1);
			FDetailWidgetRow& RowNiagara = ChildBuilder.AddCustomRow(FText::GetEmpty());
			RowNiagara
				.NameContent()
				[
					ChildPropertyHandle->CreatePropertyNameWidget()
				]
			.ValueContent()
				.MaxDesiredWidth(200.f)
				[
					SAssignNew(NiagaraParameterButton, SComboButton)
					.OnGetMenuContent(this, &FNiagaraMaterialAttributeBindingCustomization::OnGetNiagaraMenuContent)
					.ContentPadding(1)
					.ToolTipText(this, &FNiagaraMaterialAttributeBindingCustomization::GetNiagaraTooltipText)
					.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
					.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(5, 0)
						[
							SNew(SNiagaraParameterName)
							.ParameterName(this, &FNiagaraMaterialAttributeBindingCustomization::GetNiagaraVariableName)
							.IsReadOnly(true)
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(5, 0)
						[
							SNew(STextBlock)
							.Visibility(this, &FNiagaraMaterialAttributeBindingCustomization::GetNiagaraChildVariableVisibility)
							.Text(this, &FNiagaraMaterialAttributeBindingCustomization::GetNiagaraChildVariableText)
						]
					]
				];
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FName FNiagaraDataInterfaceBindingCustomization::GetVariableName() const
{
	if (BaseStage && TargetDataInterfaceBinding)
	{
		return (TargetDataInterfaceBinding->BoundVariable.GetName());
	}
	return FName();
}

FText FNiagaraDataInterfaceBindingCustomization::GetCurrentText() const
{
	if (BaseStage && TargetDataInterfaceBinding)
	{
		return FText::FromName(TargetDataInterfaceBinding->BoundVariable.GetName());
	}
	return FText::FromString(TEXT("Missing"));
}

FText FNiagaraDataInterfaceBindingCustomization::GetTooltipText() const
{
	if (BaseStage && TargetDataInterfaceBinding)
	{
		if (TargetDataInterfaceBinding->BoundVariable.IsValid())
		{
			return FText::Format(LOCTEXT("DataInterfaceBindingTooltip", "Bound to the user parameter \"{0}\""), FText::FromName(TargetDataInterfaceBinding->BoundVariable.GetName()));
		}
		else
		{
			return LOCTEXT("DataInterfaceBindingTooltip_Unbound", "Currently not bound to any data interface");
		}
	}
	return FText::FromString(TEXT("Missing"));
}

TSharedRef<SWidget> FNiagaraDataInterfaceBindingCustomization::OnGetMenuContent() const
{
	FGraphActionMenuBuilder MenuBuilder;

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			[
				SNew(SGraphActionMenu)
				.OnActionSelected(const_cast<FNiagaraDataInterfaceBindingCustomization*>(this), &FNiagaraDataInterfaceBindingCustomization::OnActionSelected)
				.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(const_cast<FNiagaraDataInterfaceBindingCustomization*>(this), &FNiagaraDataInterfaceBindingCustomization::OnCreateWidgetForAction))
				.OnCollectAllActions(const_cast<FNiagaraDataInterfaceBindingCustomization*>(this), &FNiagaraDataInterfaceBindingCustomization::CollectAllActions)
				.AutoExpandActionMenu(false)
				.ShowFilterTextBox(true)
			]
		];
}

TArray<FName> FNiagaraDataInterfaceBindingCustomization::GetNames() const
{
	TArray<FName> Names;

	if (BaseStage && TargetDataInterfaceBinding)
	{
		if (FVersionedNiagaraEmitterData* EmitterData = BaseStage->GetEmitterData())
		{
			// Find all used emitter and particle data interface variables that can be iterated upon.
			TArray<UNiagaraScript*> AllScripts;
			EmitterData->GetScripts(AllScripts, false);

			TArray<UNiagaraGraph*> Graphs;
			for (const UNiagaraScript* Script : AllScripts)
			{
				if (const UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(Script->GetLatestSource()))
				{
					Graphs.AddUnique(Source->NodeGraph);
				}
			}

			for (const UNiagaraGraph* Graph : Graphs)
			{
				TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection> ParameterReferenceMap = Graph->GetParameterReferenceMap();
				ParameterReferenceMap.KeySort([](const FNiagaraVariable& A, const FNiagaraVariable& B) { return A.GetName().LexicalLess(B.GetName()); });
				for (const auto& ParameterToReferences : ParameterReferenceMap)
				{
					const FNiagaraVariable& ParameterVariable = ParameterToReferences.Key;
					FNiagaraParameterHandle Handle(ParameterVariable.GetName());
					bool bIsValidNamespace = Handle.IsEmitterHandle() || Handle.IsSystemHandle() || Handle.IsOutputHandle() || Handle.IsParticleAttributeHandle() || Handle.IsTransientHandle() || Handle.IsStackContextHandle();
					if (ParameterVariable.IsDataInterface() && bIsValidNamespace)
					{
						const UClass* Class = ParameterVariable.GetType().GetClass();
						if (Class)
						{
							const UObject* DefaultObjDI = Class->GetDefaultObject();
							if (DefaultObjDI != nullptr && DefaultObjDI->IsA<UNiagaraDataInterfaceRWBase>())
							{
								Names.AddUnique(ParameterVariable.GetName());
							}
						}
					}
				}
			}
		}
	}

	return Names;
}

void FNiagaraDataInterfaceBindingCustomization::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	TArray<FName> UserParamNames = GetNames();
	for (FName UserParamName : UserParamNames)
	{
		FText CategoryName = FText();
		FString DisplayNameString = FName::NameToDisplayString(UserParamName.ToString(), false);
		const FText NameText = FText::FromString(DisplayNameString);
		const FText TooltipDesc = FText::Format(LOCTEXT("BindToDataInterface", "Bind to the User Parameter \"{0}\" "), FText::FromString(DisplayNameString));
		TSharedPtr<FNiagaraStackAssetAction_VarBind> NewNodeAction(new FNiagaraStackAssetAction_VarBind(UserParamName, CategoryName, NameText,
			TooltipDesc, 0, FText()));
		OutAllActions.AddAction(NewNodeAction);
	}
}

TSharedRef<SWidget> FNiagaraDataInterfaceBindingCustomization::OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SNiagaraParameterName)
			.ParameterName(((FNiagaraStackAssetAction_VarBind* const)InCreateData->Action.Get())->VarName)
			.IsReadOnly(true)
			//SNew(STextBlock)
			//.Text(InCreateData->Action->GetMenuDescription())
			.ToolTipText(InCreateData->Action->GetTooltipDescription())
		];
}


void FNiagaraDataInterfaceBindingCustomization::OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType)
{
	if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		for (int32 ActionIndex = 0; ActionIndex < SelectedActions.Num(); ActionIndex++)
		{
			TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedActions[ActionIndex];

			if (CurrentAction.IsValid())
			{
				FSlateApplication::Get().DismissAllMenus();
				FNiagaraStackAssetAction_VarBind* EventSourceAction = (FNiagaraStackAssetAction_VarBind*)CurrentAction.Get();
				ChangeSource(EventSourceAction->VarName);
			}
		}
	}
}

void FNiagaraDataInterfaceBindingCustomization::ChangeSource(FName InVarName)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeDataParameterSource", " Change Data Interface Source to \"{0}\" "), FText::FromName(InVarName)));
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	for (UObject* Obj : Objects)
	{
		Obj->Modify();
	}

	PropertyHandle->NotifyPreChange();
	TargetDataInterfaceBinding->BoundVariable.SetName(InVarName);
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

void FNiagaraDataInterfaceBindingCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);

	PropertyHandle->MarkResetToDefaultCustomized(true);

	bool bAddDefault = true;
	if (Objects.Num() == 1)
	{
		BaseStage = Cast<UNiagaraSimulationStageBase>(Objects[0]);
		if (BaseStage)
		{
			TargetDataInterfaceBinding = (FNiagaraVariableDataInterfaceBinding*)PropertyHandle->GetValueBaseAddress((uint8*)Objects[0]);

			HeaderRow
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(250.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &FNiagaraDataInterfaceBindingCustomization::OnGetMenuContent)
					.ContentPadding(1)
					.ToolTipText(this, &FNiagaraDataInterfaceBindingCustomization::GetTooltipText)
					.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
					.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
					.ButtonContent()
					[
						SNew(SNiagaraParameterName)
						.ParameterName(this, &FNiagaraDataInterfaceBindingCustomization::GetVariableName)
						.IsReadOnly(true)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2, 1)
				[
					SNew(SButton)
					.IsFocusable(false)
					.ToolTipText(LOCTEXT("ResetToDefaultToolTip", "Reset to Default"))
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.ContentPadding(0)
					.Visibility(this, &FNiagaraDataInterfaceBindingCustomization::IsResetToDefaultsVisible)
					.OnClicked(this, &FNiagaraDataInterfaceBindingCustomization::OnResetToDefaultsClicked)
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];
			bAddDefault = false;
		}
	}


	if (bAddDefault)
	{
		HeaderRow
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(200.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FName::NameToDisplayString(CastField<FStructProperty>(PropertyHandle->GetProperty())->Struct->GetName(), false)))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
}

EVisibility FNiagaraDataInterfaceBindingCustomization::IsResetToDefaultsVisible() const
{
	return TargetDataInterfaceBinding->BoundVariable.IsValid() ? EVisibility::Visible : EVisibility::Hidden;
}

FReply FNiagaraDataInterfaceBindingCustomization::OnResetToDefaultsClicked()
{
	ChangeSource(NAME_None);

	return FReply::Handled();
}

////////////////////////
FName FNiagaraScriptVariableBindingCustomization::GetVariableName() const
{
	if (TargetVariableBinding && TargetVariableBinding->IsValid())
	{
		return (TargetVariableBinding->Name);
	}
	return FName();
}

FText FNiagaraScriptVariableBindingCustomization::GetCurrentText() const
{
	if (TargetVariableBinding && TargetVariableBinding->IsValid())
	{
		return FText::FromName(TargetVariableBinding->Name);
	}
	return FText::FromString(TEXT("None"));
}

FText FNiagaraScriptVariableBindingCustomization::GetTooltipText() const
{
	if (TargetVariableBinding && TargetVariableBinding->IsValid())
	{
		FText TooltipDesc = FText::Format(LOCTEXT("BindingTooltip", "Use the variable \"{0}\" if it is defined, otherwise use the type's default value."), FText::FromName(TargetVariableBinding->Name));
		return TooltipDesc;
	}
	return FText::FromString(TEXT("There is no default binding selected."));
}

TSharedRef<SWidget> FNiagaraScriptVariableBindingCustomization::OnGetMenuContent() const
{
	FGraphActionMenuBuilder MenuBuilder; // TODO: Is this necessary? It's included in all the other implementations above, but it's never used. Spooky

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			[
				SNew(SGraphActionMenu)
				.OnActionSelected(const_cast<FNiagaraScriptVariableBindingCustomization*>(this), &FNiagaraScriptVariableBindingCustomization::OnActionSelected)
				.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(const_cast<FNiagaraScriptVariableBindingCustomization*>(this), &FNiagaraScriptVariableBindingCustomization::OnCreateWidgetForAction))
				.OnCollectAllActions(const_cast<FNiagaraScriptVariableBindingCustomization*>(this), &FNiagaraScriptVariableBindingCustomization::CollectAllActions)
				.AutoExpandActionMenu(false)
				.ShowFilterTextBox(true)
			]
		];
}

TArray<TSharedPtr<FEdGraphSchemaAction>> FNiagaraScriptVariableBindingCustomization::GetGraphParameterBindingActions(const UNiagaraGraph* Graph)
{
	TArray<FName> Names;
	TArray<TSharedPtr<FEdGraphSchemaAction>> OutActions;

	for (const FNiagaraParameterMapHistory& History : UNiagaraNodeParameterMapBase::GetParameterMaps(Graph))
	{
		for (const FNiagaraVariable& Var : History.Variables)
		{
			FString Namespace = FNiagaraParameterUtilities::GetNamespace(Var);
			if (Namespace == TEXT("Module."))
			{
				// TODO: Skip module inputs for now. Does it make sense to bind module inputs to module inputs?
				continue;
			}
			if (Var.GetType() == BaseScriptVariable->Variable.GetType())
			{
				Names.AddUnique(Var.GetName());
			}
		}
	}

	for (const auto& Var : Graph->GetParameterReferenceMap())
	{
		FString Namespace = FNiagaraParameterUtilities::GetNamespace(Var.Key);
		if (Namespace == TEXT("Module."))
		{
			// TODO: Skip module inputs for now. Does it make sense to bind module inputs to module inputs?
			continue;
		}
		if (Var.Key.GetType() == BaseScriptVariable->Variable.GetType())
		{
			Names.AddUnique(Var.Key.GetName());
		}
	}
	
	for (const FName& Name : Names)
	{
		const FText NameText = FText::FromName(Name);
		const FText TooltipDesc = FText::Format(LOCTEXT("SetFunctionPopupTooltip", "Use the variable \"{0}\" "), NameText);
		TSharedPtr<FNiagaraStackAssetAction_VarBind> NewNodeAction(
			new FNiagaraStackAssetAction_VarBind(Name, FText(), NameText, TooltipDesc, 0, FText())
		);
		OutActions.Add(NewNodeAction);
	}

	return OutActions;
}

TArray<TSharedPtr<FEdGraphSchemaAction>> FNiagaraScriptVariableBindingCustomization::GetEngineConstantBindingActions()
{
	TArray<TSharedPtr<FEdGraphSchemaAction>> OutActions;

	for (const FNiagaraVariable& Parameter : FNiagaraConstants::GetEngineConstants())
	{
		if (BaseScriptVariable->Variable.GetType() != Parameter.GetType())
		{
			// types must match
			continue;
		}

		const FName& Name = Parameter.GetName();
		const FText NameText = FText::FromName(Name);
		const FText TooltipDesc = FText::Format(LOCTEXT("SetFunctionPopupTooltip", "Use the variable \"{0}\" "), NameText);

		TSharedPtr<FNiagaraStackAssetAction_VarBind> NewNodeAction(
			new FNiagaraStackAssetAction_VarBind(Name, FText(), NameText, TooltipDesc, 0, FText())
		);
		OutActions.Add(NewNodeAction);
	}

	return OutActions;
}

TArray<TSharedPtr<FEdGraphSchemaAction>> FNiagaraScriptVariableBindingCustomization::GetLibraryParameterBindingActions()
{
	TArray<TSharedPtr<FEdGraphSchemaAction>> OutActions;

	auto GetAvailableParameterDefinitions = [this]()->TArray<UNiagaraParameterDefinitions*> {
		const bool bSkipSubscribed = false;
		if (BaseGraph)
		{
			TSharedPtr<INiagaraParameterDefinitionsSubscriberViewModel> ParameterDefinitionsSubscriberViewModel = FNiagaraEditorUtilities::GetOwningLibrarySubscriberViewModelForGraph(BaseGraph);
			if (ParameterDefinitionsSubscriberViewModel.IsValid())
			{
				return ParameterDefinitionsSubscriberViewModel->GetAvailableParameterDefinitions(bSkipSubscribed);
			}
		}
		else if (BaseLibrary)
		{
			return BaseLibrary->GetAvailableParameterDefinitions(bSkipSubscribed);
		}
		checkf(false, TEXT("Script variable did not have a valid outer UNiagaraGraph or UNiagaraParameterDefinitions!"));
		return TArray<UNiagaraParameterDefinitions*>();
	};
	
	const TArray<UNiagaraParameterDefinitions*> AvailableParameterDefinitions = GetAvailableParameterDefinitions();
	for (UNiagaraParameterDefinitions* ParameterDefinitions : AvailableParameterDefinitions)
	{
		for(const UNiagaraScriptVariable* LibraryScriptVar : ParameterDefinitions->GetParametersConst())
		{ 
			if (BaseScriptVariable->Variable.GetType() != LibraryScriptVar->Variable.GetType())
			{
				// types must match
				continue;
			}
			else if (BaseScriptVariable->Metadata.GetVariableGuid() == LibraryScriptVar->Metadata.GetVariableGuid())
			{
				// do not bind to self
				continue;
			}

			const FName& Name = LibraryScriptVar->Variable.GetName();
			const FText NameText = FText::FromName(Name);
			const FText TooltipDesc = FText::Format(LOCTEXT("SetFunctionPopupTooltip", "Use the variable \"{0}\" "), NameText);

			FScriptVarBindingNameSubscriptionArgs SubscriptionArgs(ParameterDefinitions, LibraryScriptVar, BaseScriptVariable);

			TSharedPtr<FNiagaraStackAssetAction_VarBind> NewNodeAction(
				new FNiagaraStackAssetAction_VarBind(Name, FText(), NameText, TooltipDesc, 0, FText(), SubscriptionArgs)
			);
			OutActions.Add(NewNodeAction);
		}
	}

	return OutActions;
}

void FNiagaraScriptVariableBindingCustomization::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	if (BaseGraph)
	{
		for (const TSharedPtr<FEdGraphSchemaAction>& Action : GetGraphParameterBindingActions(BaseGraph))
		{
			OutAllActions.AddAction(Action);
		}
		
	}

	for (const TSharedPtr<FEdGraphSchemaAction>& Action : GetEngineConstantBindingActions())
	{
		OutAllActions.AddAction(Action);
	}
	
	//@todo(ng) for now do not allow binding libraries to libraries as we do not handle fixing up the binding if the source library is removed.
	if(BaseGraph != nullptr || ALLOW_LIBRARY_TO_LIBRARY_DEFAULT_BINDING)
	{ 
		for (const TSharedPtr<FEdGraphSchemaAction>& Action : GetLibraryParameterBindingActions())
		{
			OutAllActions.AddAction(Action);
		}
	}
}

TSharedRef<SWidget> FNiagaraScriptVariableBindingCustomization::OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SNiagaraParameterName)
			.ParameterName(((FNiagaraStackAssetAction_VarBind* const)InCreateData->Action.Get())->VarName)
			.IsReadOnly(true)
			.ToolTipText(InCreateData->Action->GetTooltipDescription())
		];
}

void FNiagaraScriptVariableBindingCustomization::OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType)
{
	if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		for (auto& CurrentAction : SelectedActions)
		{
			if (CurrentAction.IsValid())
			{
				FSlateApplication::Get().DismissAllMenus();
				FNiagaraStackAssetAction_VarBind* EventSourceAction = (FNiagaraStackAssetAction_VarBind*)CurrentAction.Get();
				UNiagaraParameterDefinitions* SourceParameterDefinitions = EventSourceAction->LibraryNameSubscriptionArgs.SourceParameterDefinitions;
				const UNiagaraScriptVariable* SourceScriptVar = EventSourceAction->LibraryNameSubscriptionArgs.SourceScriptVar;

				ChangeSource(EventSourceAction->VarName);

				// If in a parameter definitions, consider the parameter definitions binding name subscriptions.
				if (BaseLibrary != nullptr)
				{
					// Force unsubscribe the current binding name.
					BaseLibrary->UnsubscribeBindingNameFromExternalParameterDefinitions(BaseScriptVariable->Metadata.GetVariableGuid());

					// If we are in a parameter definitions and the parameter definitions name binding args are set, set the base library to synchronize the binding name with the source library.
					//@todo(ng) re-enable
					if (ALLOW_LIBRARY_TO_LIBRARY_DEFAULT_BINDING && SourceParameterDefinitions != nullptr)
					{
						BaseLibrary->SubscribeBindingNameToExternalParameterDefinitions(
							SourceParameterDefinitions,
							SourceScriptVar->Metadata.GetVariableGuid(),
							BaseScriptVariable->Metadata.GetVariableGuid()
						);
					}
				}
				else if (BaseGraph != nullptr && SourceParameterDefinitions != nullptr)
				{
					// If we are in a script graph and the parameter definitions name binding args are set, add the parameter specified by the binding as it is not guaranteed to exist in the graph yet.
					if(BaseGraph->GetMetaData(EventSourceAction->LibraryNameSubscriptionArgs.SourceScriptVar->Variable).IsSet() == false)
					{
						BaseGraph->AddParameter(EventSourceAction->LibraryNameSubscriptionArgs.SourceScriptVar);

						TSharedPtr<INiagaraParameterDefinitionsSubscriberViewModel> ParameterDefinitionsSubscriberViewModel = FNiagaraEditorUtilities::GetOwningLibrarySubscriberViewModelForGraph(BaseGraph);
						if (ParameterDefinitionsSubscriberViewModel.IsValid() && ParameterDefinitionsSubscriberViewModel->GetSubscribedParameterDefinitions().Contains(SourceParameterDefinitions) == false)
						{
							ParameterDefinitionsSubscriberViewModel->SubscribeToParameterDefinitions(SourceParameterDefinitions);
						}
					}
				}
			}
		}
	}
}

void FNiagaraScriptVariableBindingCustomization::ChangeSource(FName InVarName)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT("ChangeBinding", " Change default binding to \"{0}\" "), FText::FromName(InVarName)));
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	for (UObject* Obj : Objects)
	{
		Obj->Modify();
	}

	PropertyHandle->NotifyPreChange();
	TargetVariableBinding->Name = InVarName;
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

void FNiagaraScriptVariableBindingCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();
	PropertyHandle = InPropertyHandle;
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	bool bAddDefault = true;
	if (Objects.Num() == 1)
	{
		BaseScriptVariable = Cast<UNiagaraScriptVariable>(Objects[0]);
		if (BaseScriptVariable)
		{
			BaseGraph = Cast<UNiagaraGraph>(BaseScriptVariable->GetOuter());
			BaseLibrary = Cast<UNiagaraParameterDefinitions>(BaseScriptVariable->GetOuter());

			TargetVariableBinding = (FNiagaraScriptVariableBinding*)PropertyHandle->GetValueBaseAddress((uint8*)Objects[0]);

			HeaderRow
			.IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }))
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(200.f)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FNiagaraScriptVariableBindingCustomization::OnGetMenuContent)
				.ContentPadding(1)
				.ToolTipText(this, &FNiagaraScriptVariableBindingCustomization::GetTooltipText)
				.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
				.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
				.ButtonContent()
				[
					SNew(SNiagaraParameterName)
					.ParameterName(this, &FNiagaraScriptVariableBindingCustomization::GetVariableName)
					.IsReadOnly(true)
				]
			];
			bAddDefault = false;
		}
		else
		{
			BaseGraph = nullptr;
			BaseLibrary = nullptr;
		}
	}
	
	if (bAddDefault)
	{
		HeaderRow
		.IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }))
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(200.f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(FName::NameToDisplayString(CastField<FStructProperty>(PropertyHandle->GetProperty())->Struct->GetName(), false)))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
	}
}

void FNiagaraVariableMetaDataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	UNiagaraScriptVariable* OwningScriptVariable = nullptr;
	if (Objects.Num() == 1)
	{
		OwningScriptVariable = Cast<UNiagaraScriptVariable>(Objects[0]);
	}

	bool bOwningScriptVariableSubscribedToDefinition = false;
	bool bOwningScriptVariableOuteredToDefinitions = false;
	if (OwningScriptVariable != nullptr)
	{
		bOwningScriptVariableSubscribedToDefinition = OwningScriptVariable->GetIsSubscribedToParameterDefinitions();
		if (OwningScriptVariable->GetOuter()->IsA<UNiagaraParameterDefinitions>())
		{
			bOwningScriptVariableOuteredToDefinitions = true;
		}
	}

	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);
	TArray<TSharedRef<IPropertyHandle>> ChildPropertyHandles;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		ChildPropertyHandles.Add(PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
	}

	for (const TSharedRef<IPropertyHandle>& ChildPropertyHandle : ChildPropertyHandles)
	{
		/** The customized flag is used for hiding properties.
		 *  The script variable's customization which contains meta data is hiding these properties already,
		 *  but the meta data customization would stomp these by readding them. Hence, we skip them here if they were marked as customized.
		*/
		if(ChildPropertyHandle->IsCustomized())
		{
			continue;
		}
		
		const FProperty* ChildProperty = ChildPropertyHandle->GetProperty();
		const FName ChildPropertyName = ChildProperty->GetFName();

		if (ChildPropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraVariableMetaData, Description))
		{ 
			IDetailPropertyRow & DetailPropertyRow = ChildBuilder.AddProperty(ChildPropertyHandle);
			if (!bOwningScriptVariableOuteredToDefinitions && bOwningScriptVariableSubscribedToDefinition)
			{
				// Parameters linked to definitions but not part of the definitions asset themselves may not edit the description property.
				// NOTE: Parameters outered to definitions, when viewed via the SNiagaraParameterDefinitionsPanel, will not allow editing the definition due to the owning details view being disabled.
				DetailPropertyRow.IsEnabled(false);
			}
		}
		else if(bOwningScriptVariableOuteredToDefinitions)
		{
			// Parameters outered to definitions may only edit the description; all other properties are skipped for the child builder.
			continue;
		}
		else
		{
			ChildBuilder.AddProperty(ChildPropertyHandle);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraSystemScalabilityOverrideCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	];
	HeaderRow.ValueContent()
	[
		InPropertyHandle->CreatePropertyValueWidget()
	];
}

void FNiagaraSystemScalabilityOverrideCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	auto AddOverrideProperties = [&](FName MainPropertyName, TArray<FName> OverrideProperties)
	{
		if (TSharedPtr<IPropertyHandle> MainProperty = PropertyHandle->GetChildHandle(MainPropertyName))
		{
			TAttribute<EVisibility> VisibilityAttribute = TAttribute<EVisibility>::CreateLambda([MainProperty]
			{
				bool bMainProp = false;
				if(MainProperty->GetValue(bMainProp) == FPropertyAccess::Success && bMainProp)
				{
					return EVisibility::Visible;
				}

				return EVisibility::Collapsed;
			});
			
			bool bMainProp;
			if (MainProperty->GetValue(bMainProp) == FPropertyAccess::Success)
			{
				IDetailGroup& PropGroup = ChildBuilder.AddGroup(MainPropertyName, MainProperty->GetPropertyDisplayName());
				PropGroup.HeaderProperty(MainProperty.ToSharedRef());
				for (FName OverridePropName : OverrideProperties)
				{
					if (TSharedPtr<IPropertyHandle> OverrideProp = PropertyHandle->GetChildHandle(OverridePropName))
					{
						PropGroup.AddPropertyRow(OverrideProp.ToSharedRef())
						.Visibility(VisibilityAttribute);
					}
				}
			}
			else
			{
				ChildBuilder.AddProperty(MainProperty.ToSharedRef());
			}
		}
	}; 
	
	TSharedPtr<IPropertyHandle> PlatformsProperty = PropertyHandle->GetChildHandle(TEXT("Platforms"));
	ChildBuilder.AddProperty(PlatformsProperty.ToSharedRef());

	AddOverrideProperties(TEXT("bOverrideDistanceSettings"), { TEXT("bCullByDistance"), TEXT("MaxDistance") });
	AddOverrideProperties(TEXT("bOverrideInstanceCountSettings"), { TEXT("bCullMaxInstanceCount"), TEXT("MaxInstances") });
	AddOverrideProperties(TEXT("bOverridePerSystemInstanceCountSettings"), { TEXT("bCullPerSystemMaxInstanceCount"), TEXT("MaxSystemInstances") });
	AddOverrideProperties(TEXT("bOverrideVisibilitySettings"), { TEXT("VisibilityCulling") });
	AddOverrideProperties(TEXT("bOverrideGlobalBudgetScalingSettings"), { TEXT("BudgetScaling") });
	AddOverrideProperties(TEXT("bOverrideCullProxySettings"), { TEXT("CullProxyMode"), TEXT("MaxSystemProxies") });
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraRendererMaterialParameterCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	if (Objects.Num() > 0)
	{
		WeakRenderProperties = Cast<UNiagaraRendererProperties>(Objects[0]);
	}

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		PropertyHandle->CreatePropertyValueWidget()
	];
}

void FNiagaraRendererMaterialParameterCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	static const FName NAME_MaterialParameterName("MaterialParameterName");
	for (uint32 ChildIndex=0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex);
		if ( ChildHandle->GetProperty()->GetFName() == NAME_MaterialParameterName )
		{
			NamePropertyHandle = ChildHandle;
			ChildBuilder.AddCustomRow(ChildHandle->GetPropertyDisplayName())
			.NameWidget
			[
				ChildHandle->CreatePropertyNameWidget()
			]
			.ValueWidget
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FNiagaraRendererMaterialParameterCustomization::OnGetMaterialBindingNameMenuContent, ChildHandle)
				.ContentPadding(1)
				.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
				.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FNiagaraRendererMaterialParameterCustomization::GetBindingNameText, ChildHandle)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			];
		}
		else if ( CustomizeChildProperty(ChildBuilder, ChildHandle) == false )
		{
			ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}

FText FNiagaraRendererMaterialParameterCustomization::GetBindingNameText(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	FName BindingName;
	if (PropertyHandle.IsValid() )
	{
		PropertyHandle->GetValue(BindingName);
	}
	return FText::FromName(BindingName);
}

FName FNiagaraRendererMaterialParameterCustomization::GetBindingName(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	FName BindingName;
	if (PropertyHandle.IsValid())
	{
		PropertyHandle->GetValue(BindingName);
	}
	return BindingName;
}

FText FNiagaraRendererMaterialParameterCustomization::GetMaterialBindingTooltip(FName ParameterName, const FString& ParameterDesc)
{
	if (ParameterDesc.Len() > 0)
	{
		return FText::Format(LOCTEXT("RendererMaterialParameterTooltip", "Set Material Parameter: {0}\nDescription: {1}"), FText::FromName(ParameterName), FText::FromString(ParameterDesc));
	}
	else
	{
		return FText::Format(LOCTEXT("BasicRendererMaterialParameterTooltip", "Set Material Parameter: {0}"), FText::FromName(ParameterName));
	}
}

TSharedRef<SWidget> FNiagaraRendererMaterialParameterCustomization::OnGetMaterialBindingNameMenuContent(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	FMenuBuilder MenuBuilder(true, NULL);

	UNiagaraRendererProperties* RenderProperties = WeakRenderProperties.Get();
	if (RenderProperties != nullptr && PropertyHandle.IsValid())
	{
		TSet<FName> AddedParameterNames;
		TArray<UMaterialInterface*> UsedMaterials;
		RenderProperties->GetUsedMaterials(nullptr, UsedMaterials);

		for (UMaterialInterface* Material : UsedMaterials)
		{
			if (Material == nullptr)
			{
				continue;
			}

			TArray<FMaterialParameterInfo> MaterialParameterInfos;
			GetMaterialParameterInfos(Material, MaterialParameterInfos);

			for (const FMaterialParameterInfo& MaterialParameterInfo : MaterialParameterInfos)
			{
				if (MaterialParameterInfo.Association != EMaterialParameterAssociation::GlobalParameter)
				{
					continue;
				}

				if (AddedParameterNames.Contains(MaterialParameterInfo.Name))
				{
					continue;
				}
				AddedParameterNames.Add(MaterialParameterInfo.Name);

				FString ParameterDesc;
				Material->GetParameterDesc(MaterialParameterInfo, ParameterDesc);

				MenuBuilder.AddMenuEntry(
					FText::FromName(MaterialParameterInfo.Name),
					GetMaterialBindingTooltip(MaterialParameterInfo.Name, ParameterDesc),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda(
							[BindingName=MaterialParameterInfo.Name, PropertyHandle]()
							{
								PropertyHandle->SetValue(BindingName);
							}
						)
					)
				);
			}
		}

		if (AddedParameterNames.IsEmpty())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("NoParametersFound", "No Parameters Found"),
				FText(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateLambda([]() { return false;} )
				)
			);
		}
	}

	return MenuBuilder.MakeWidget();
}

void FNiagaraRendererMaterialScalarParameterCustomization::GetMaterialParameterInfos(UMaterialInterface* Material, TArray<FMaterialParameterInfo>& OutMaterialParameterInfos) const
{
	TArray<FGuid> Guids;
	Material->GetAllScalarParameterInfo(OutMaterialParameterInfos, Guids);
}

bool FNiagaraRendererMaterialScalarParameterCustomization::CustomizeChildProperty(IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (NamePropertyHandle.IsValid() && PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraRendererMaterialScalarParameter, Value))
	{
		ValueHelper = MakeShared<FNiagaraMaterialParameterValueHelper>(NamePropertyHandle, PropertyHandle, WeakRenderProperties);
		return ValueHelper->CreateWidget<float>(ChildBuilder);
	}
	return false;
}

void FNiagaraRendererMaterialVectorParameterCustomization::GetMaterialParameterInfos(UMaterialInterface* Material, TArray<FMaterialParameterInfo>& OutMaterialParameterInfos) const
{
	TArray<FGuid> Guids;
	Material->GetAllVectorParameterInfo(OutMaterialParameterInfos, Guids);
}

bool FNiagaraRendererMaterialVectorParameterCustomization::CustomizeChildProperty(IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (NamePropertyHandle.IsValid() && PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraRendererMaterialVectorParameter, Value))
	{
		ValueHelper = MakeShared<FNiagaraMaterialParameterValueHelper>(NamePropertyHandle, PropertyHandle, WeakRenderProperties);
		return ValueHelper->CreateWidget<FLinearColor>(ChildBuilder);
	}
	return false;
}

void FNiagaraRendererMaterialTextureParameterCustomization::GetMaterialParameterInfos(UMaterialInterface* Material, TArray<FMaterialParameterInfo>& OutMaterialParameterInfos) const
{
	TArray<FGuid> Guids;
	Material->GetAllTextureParameterInfo(OutMaterialParameterInfos, Guids);
}

bool FNiagaraRendererMaterialTextureParameterCustomization::CustomizeChildProperty(IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (NamePropertyHandle.IsValid() && PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraRendererMaterialTextureParameter, Texture))
	{
		ValueHelper = MakeShared<FNiagaraMaterialParameterValueHelper>(NamePropertyHandle, PropertyHandle, WeakRenderProperties);
		return ValueHelper->CreateWidget<UTexture*>(ChildBuilder);
	}
	return false;
}

bool FNiagaraRendererMaterialStaticBoolParameterCustomization::CustomizeChildProperty(class IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (PropertyHandle->GetProperty()->GetFName() != GET_MEMBER_NAME_CHECKED(FNiagaraRendererMaterialStaticBoolParameter, StaticVariableName))
	{
		return false;
	}

	StructPropertyHandle = PropertyHandle ? PropertyHandle->GetParentHandle() : nullptr;
	FStructProperty* StructProperty = StructPropertyHandle ? CastField<FStructProperty>(StructPropertyHandle->GetProperty()) : nullptr;
	if (StructProperty == nullptr || StructProperty->Struct != FNiagaraRendererMaterialStaticBoolParameter::StaticStruct())
	{
		StructPropertyHandle = nullptr;
		return false;
	}

	ChildBuilder.AddCustomRow(PropertyHandle->GetPropertyDisplayName())
	.NameWidget
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueWidget
	[
		SNew(SComboButton)
		.OnGetMenuContent(this, &FNiagaraRendererMaterialStaticBoolParameterCustomization::OnGetStaticVariablelBindingNameMenuContent)
		.ContentPadding(1)
		.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
		.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
		.ButtonContent()
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &FNiagaraRendererMaterialStaticBoolParameterCustomization::GetWidgetIndex)
			+SWidgetSwitcher::Slot()
			[
				SNew(SNiagaraParameterName)
				.ParameterName(this, &FNiagaraRendererMaterialParameterCustomization::GetBindingName, PropertyHandle)
				.IsReadOnly(true)
			]
			+SWidgetSwitcher::Slot()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FNiagaraRendererMaterialStaticBoolParameterCustomization::GetCurrentConstantValue)
			]
		]
	];

	return true;
}

void FNiagaraRendererMaterialStaticBoolParameterCustomization::GetMaterialParameterInfos(UMaterialInterface* Material, TArray<FMaterialParameterInfo>& OutMaterialParameterInfos) const
{
	TArray<FGuid> Guids;
	Material->GetAllStaticSwitchParameterInfo(OutMaterialParameterInfos, Guids);
	for (FMaterialParameterInfo& MaterialParameterInfo : OutMaterialParameterInfos)
	{
		MaterialParameterInfo.Association = EMaterialParameterAssociation::GlobalParameter;
	}
}

TSharedRef<SWidget> FNiagaraRendererMaterialStaticBoolParameterCustomization::OnGetStaticVariablelBindingNameMenuContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	UNiagaraRendererProperties* RenderProperties = WeakRenderProperties.Get();

	if (RenderProperties != nullptr && StructPropertyHandle.IsValid())
	{
		MenuBuilder.AddMenuEntry(
			GetConstantText(true),
			TAttribute<FText>(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FNiagaraRendererMaterialStaticBoolParameterCustomization::SetValue, true))
		);
		MenuBuilder.AddMenuEntry(
			GetConstantText(false),
			TAttribute<FText>(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FNiagaraRendererMaterialStaticBoolParameterCustomization::SetValue, false))
		);
		MenuBuilder.AddSeparator();

		const FVersionedNiagaraEmitter NiagaraEmitter = RenderProperties->GetOuterEmitter();
		const FNiagaraTypeDefinition StaticBoolDef = FNiagaraTypeDefinition::GetBoolDef().ToStaticDef();
		FNiagaraAliasContext AliasContext;

		auto AddStaticVariables =
			[&](UNiagaraScript* NiagaraScript)
			{
				if (NiagaraScript == nullptr)
				{
					return;
				}
				for ( const FNiagaraVariable& StaticVariable : NiagaraScript->GetVMExecutableData().StaticVariablesWritten )
				{
					if ( StaticVariable.GetType() != StaticBoolDef)
					{
						continue;
					}
					const FNiagaraVariableBase ResolvedVariable = FNiagaraUtilities::ResolveAliases(StaticVariable, AliasContext);
					const bool bSystemAttribute = ResolvedVariable.IsInNameSpace(FNiagaraConstants::SystemNamespaceString);
					const bool bEmitterAttribute = ResolvedVariable.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString);
					const bool bParticleAttribute = ResolvedVariable.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString);
					if (!bSystemAttribute && !bEmitterAttribute && !bParticleAttribute)
					{
						continue;
					}


					MenuBuilder.AddMenuEntry(
						FUIAction(FExecuteAction::CreateSP(this, &FNiagaraRendererMaterialStaticBoolParameterCustomization::SetValue, ResolvedVariable.GetName())),
						SNew(SNiagaraParameterName)
						.ParameterName(ResolvedVariable.GetName())
						.IsReadOnly(true)
					);
				}
			};

		// We must add the emitter alias context before the system scripts as they will contain the emitter spawn & update script
		if (FVersionedNiagaraEmitterData* EmitterData = NiagaraEmitter.GetEmitterData())
		{
			AliasContext.ChangeEmitterNameToEmitter(NiagaraEmitter.Emitter->GetUniqueEmitterName());
			EmitterData->ForEachScript(AddStaticVariables);
		}

		if (UNiagaraSystem* NiagaraSystem = RenderProperties->GetTypedOuter<UNiagaraSystem>())
		{
			AddStaticVariables(NiagaraSystem->GetSystemSpawnScript());
			AddStaticVariables(NiagaraSystem->GetSystemUpdateScript());
		}
	}

	return MenuBuilder.MakeWidget();
}

FNiagaraRendererMaterialStaticBoolParameter* FNiagaraRendererMaterialStaticBoolParameterCustomization::ResolveParameterStruct() const
{
	if (UNiagaraRendererProperties* RenderProperties = WeakRenderProperties.Get())
	{
		return StructPropertyHandle.IsValid() ? reinterpret_cast<FNiagaraRendererMaterialStaticBoolParameter*>(StructPropertyHandle->GetValueBaseAddress(reinterpret_cast<uint8*>(RenderProperties))) : nullptr;
	}
	return nullptr;
}

int32 FNiagaraRendererMaterialStaticBoolParameterCustomization::GetWidgetIndex() const
{
	if (FNiagaraRendererMaterialStaticBoolParameter* Parameter = ResolveParameterStruct())
	{
		return Parameter->StaticValue.IsSet() ? 1 : 0;
	}
	return 0;
}

void FNiagaraRendererMaterialStaticBoolParameterCustomization::SetValue(FName ParameterName) const
{
	if (UNiagaraRendererProperties* RenderProperties = WeakRenderProperties.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("EditStaticBoolBinding", "Edit Static Bool Binding"));
		RenderProperties->Modify();

		StructPropertyHandle->NotifyPreChange();

		FNiagaraRendererMaterialStaticBoolParameter* Parameter = ResolveParameterStruct();
		Parameter->StaticVariableName = ParameterName;
		Parameter->StaticValue.Reset();

		StructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		StructPropertyHandle->NotifyFinishedChangingProperties();
	}
}

void FNiagaraRendererMaterialStaticBoolParameterCustomization::SetValue(bool Value) const
{
	if (UNiagaraRendererProperties* RenderProperties = WeakRenderProperties.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("EditStaticBoolBinding", "Edit Static Bool Binding"));
		RenderProperties->Modify();

		StructPropertyHandle->NotifyPreChange();

		FNiagaraRendererMaterialStaticBoolParameter* Parameter = ResolveParameterStruct();
		Parameter->StaticVariableName = NAME_None;
		Parameter->StaticValue = Value;

		StructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		StructPropertyHandle->NotifyFinishedChangingProperties();
	}
}


FText FNiagaraRendererMaterialStaticBoolParameterCustomization::GetCurrentConstantValue() const
{
	bool bValue = false;
	if (FNiagaraRendererMaterialStaticBoolParameter* Parameter = ResolveParameterStruct())
	{
		bValue = Parameter->StaticValue.Get(false);
	}
	return GetConstantText(bValue);
}

FText FNiagaraRendererMaterialStaticBoolParameterCustomization::GetConstantText(bool bValue) const
{
	return bValue ? LOCTEXT("ConstantTrue", "Constant: True") : LOCTEXT("ConstantFalse", "Constant: False");
}

//////////////////////////////////////////////////////////////////////////

class SNiagaraRawVariableTypeSelectMenu : public SNiagaraParameterMenu
{
public:
	//DECLARE_DELEGATE_RetVal_OneParam(bool, FOnAllowMakeType, const FNiagaraTypeDefinition&);

	SLATE_BEGIN_ARGS(SNiagaraRawVariableTypeSelectMenu)
		: _AutoExpandMenu(false)
	{}
	//~ Begin Required Args
	SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)
	SLATE_ARGUMENT(FNiagaraVariable*, VarToModify)
	//~ End Required Args
	SLATE_ARGUMENT(bool, AutoExpandMenu)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

protected:
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	FNiagaraVariable* VarToModify;
};


void SNiagaraRawVariableTypeSelectMenu::Construct(const FArguments& InArgs)
{
	checkf(InArgs._VarToModify != nullptr, TEXT("Tried to construct change var type menu without valid var ptr!"));
	this->PropertyHandle = InArgs._PropertyHandle;
	this->VarToModify = InArgs._VarToModify;

	SNiagaraParameterMenu::FArguments SuperArgs;
	SuperArgs._AutoExpandMenu = InArgs._AutoExpandMenu;
	SNiagaraParameterMenu::Construct(SuperArgs);
}

void SNiagaraRawVariableTypeSelectMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FNiagaraMenuActionCollector Collector;

	TArray<FNiagaraTypeDefinition> Types;
	FNiagaraEditorUtilities::GetAllowedUserVariableTypes(Types);
	for (const FNiagaraTypeDefinition& Type : Types)
	{
		if(Type.IsDataInterface() || Type.GetClass())
		{
			continue;
		}

		FNiagaraVariable DefaultVar(Type, FName(*Type.GetName()));
		FNiagaraEditorUtilities::ResetVariableToDefaultValue(DefaultVar);

		FText Category = FNiagaraEditorUtilities::GetVariableTypeCategory(DefaultVar);
		const FText DisplayName = Type.GetNameText();
		const FText Tooltip = Type.GetNameText();
		TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
			Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
			FNiagaraMenuAction::FOnExecuteStackAction::CreateLambda(
			[Var=VarToModify, PropHandle=PropertyHandle, Type]()
			{
				FScopedTransaction Transaction(LOCTEXT("Set Raw Niagara Variable Type", "Set Variable Type"));
				check(Var);
				check(PropHandle.IsValid());
				TArray<UObject*> Objects;
				PropHandle->GetOuterObjects(Objects);
				for (UObject* Obj : Objects)
				{
					Obj->Modify();
				}

				PropHandle->NotifyPreChange();
				Var->SetType(Type);
				PropHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				PropHandle->NotifyFinishedChangingProperties();
			}
		)));

		Collector.AddAction(Action, 0);
	}

	Collector.AddAllActionsTo(OutAllActions);
}
// Copy of private SConstrainedBox defined in PropertyEditor/Private/SConstrainedBox.h
class SNiagaraConstrainedBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraConstrainedBox) {}
	SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ATTRIBUTE(TOptional<float>, MinWidth)
		SLATE_ATTRIBUTE(TOptional<float>, MaxWidth)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
	{
		MinWidth = InArgs._MinWidth;
		MaxWidth = InArgs._MaxWidth;

		ChildSlot
			[
				InArgs._Content.Widget
			];
	}

	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
	{
		const float MinWidthVal = MinWidth.Get().Get(0.0f);
		const float MaxWidthVal = MaxWidth.Get().Get(0.0f);

		if (MinWidthVal == 0.0f && MaxWidthVal == 0.0f)
		{
			return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
		}
		else
		{
			FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();

			double XVal = FMath::Max(MinWidthVal, ChildSize.X);
			if (MaxWidthVal > MinWidthVal)
			{
				XVal = FMath::Min(MaxWidthVal, XVal);
			}

			return FVector2D(XVal, ChildSize.Y);
		}
	}

private:
	TAttribute< TOptional<float> > MinWidth;
	TAttribute< TOptional<float> > MaxWidth;
};

void FNiagaraVariableDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	
}

void FNiagaraVariableDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> NameHandle = PropertyHandle->GetChildHandle(TEXT("Name"));
	TSharedPtr<IPropertyHandle> TypeDefHandleHandle = PropertyHandle->GetChildHandle(TEXT("TypeDefHandle"));

	FDetailWidgetRow& Row = ChildBuilder.AddCustomRow(FText::GetEmpty());

	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	if (Objects.Num() > 1)
	{
		Row.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			];
		Row.ValueContent()
			[
				PropertyHandle->CreatePropertyValueWidget()
			];
	}
	else
	{
		void* VarPtr = nullptr;
		if (PropertyHandle->GetValueData(VarPtr) == FPropertyAccess::Success)
		{
			bool bEnforceUniqueName = PropertyHandle->HasMetaData(TEXT("EnforceUniqueNames"));
			FNiagaraVariable* Variable = static_cast<FNiagaraVariable*>(VarPtr);

			Row.NameContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("DataChannelVarNameText", "Name: "))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 4.0f, 4.0f, 4.0f)
				[
					SNew(SNiagaraConstrainedBox)
					.MinWidth(150.0f)
					[
						SNew( SEditableTextBox )
						.Text_Lambda([NameHandle]() -> FText { FText OutText; NameHandle->GetValueAsDisplayText(OutText); return OutText; })
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.SelectAllTextWhenFocused(true)
						.ClearKeyboardFocusOnCommit(false)
						.MaximumLength(NAME_SIZE-1)
						.OnTextCommitted_Lambda([NameHandle](const FText& NewName, ETextCommit::Type)
						{
							FScopedTransaction Transaction(LOCTEXT("ChangeVarName", "Change variable name"));
							TArray<UObject*> OuterObjects;
							NameHandle->GetOuterObjects(OuterObjects);
							for (UObject* OuterObj : OuterObjects)
							{
								OuterObj->Modify();
							}
							NameHandle->SetValue(FName(NewName.ToString()));
						} )
						.OnVerifyTextChanged_Lambda([bEnforceUniqueName, PropertyHandle](const FText& InNewText, FText& OutErrorMessage) -> bool
						{
							// if necessary, validate that the entered name is unique among all entries of the variable array
							if (!bEnforceUniqueName)
							{
								return true;
							}
							FName NewName = FName(InNewText.ToString());
							if (TSharedPtr<IPropertyHandle> Handle = PropertyHandle->GetParentHandle())
							{
								if (TSharedPtr<IPropertyHandleArray> PropertyHandleArray = Handle->AsArray())
								{
									uint32 NumElements;
									PropertyHandleArray->GetNumElements(NumElements);
									bool bNewNameValid = true;
									for (uint32 i = 0; i < NumElements; i++)
									{
										TSharedRef<IPropertyHandle> ElementHandle = PropertyHandleArray->GetElement(i);
										void* VarPtr = nullptr;
										if (!ElementHandle->IsSamePropertyNode(PropertyHandle) && ElementHandle->GetValueData(VarPtr) == FPropertyAccess::Success)
										{
											if (NewName == static_cast<FNiagaraVariable*>(VarPtr)->GetName())
											{
												bNewNameValid = false;
												break;
											}
										}
									}
									if (!bNewNameValid)
									{
										OutErrorMessage = LOCTEXT("DuplicateNameError", "Variable name has to be unique!");
										return false;
									}
								}
							}
							return true;
						})
						.SelectAllTextOnCommit( true )
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 2.0f, 4.0f, 2.0f)
				[
					SNew(SNiagaraConstrainedBox)
					.MinWidth(150.0f)
					[
						PropertyHandle->CreateDefaultPropertyButtonWidgets()
					]
				]
			];
			
			Row.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 2.0f, 0)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("DataChannelVarTypeText", "Type: "))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					SNew(SComboButton)
					.HasDownArrow(true)
					.ContentPadding(0)
					.OnGetMenuContent(this, &FNiagaraVariableDetailsCustomization::GetTypeMenu, TypeDefHandleHandle, Variable)
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.AutoWidth()
						[
							SNew(SImage)
							.ColorAndOpacity_Lambda([Variable]() {return UEdGraphSchema_Niagara::GetTypeColor(Variable->GetType());})
							.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"))
						]
						+ SHorizontalBox::Slot()
						.Padding(4, 2, 2, 2)
						[
							SNew(STextBlock)
							.MinDesiredWidth(150)
							.Text_Lambda([Variable] {return Variable ? Variable->GetType().GetNameText() : FText::GetEmpty(); })
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
				]
			];
		}
	}
}

TSharedRef<SWidget> FNiagaraVariableDetailsCustomization::GetTypeMenu(TSharedPtr<IPropertyHandle> InPropertyHandle, FNiagaraVariable* Var)
{
	return SNew(SNiagaraRawVariableTypeSelectMenu)
	.PropertyHandle(InPropertyHandle)
	.VarToModify(Var);
}

void FNiagaraDataChannelVariableDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FNiagaraDataChannelVariableDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> NameHandle = PropertyHandle->GetChildHandle(TEXT("Name"));
	TSharedPtr<IPropertyHandle> TypeDefHandleHandle = PropertyHandle->GetChildHandle(TEXT("TypeDefHandle"));

	FDetailWidgetRow& Row = ChildBuilder.AddCustomRow(FText::GetEmpty());

	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);
	if (Objects.Num() > 1)
	{
		Row.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			];
		Row.ValueContent()
			[
				PropertyHandle->CreatePropertyValueWidget()
			];
	}
	else
	{
		void* VarPtr = nullptr;
		if (PropertyHandle->GetValueData(VarPtr) == FPropertyAccess::Success)
		{
			bool bEnforceUniqueName = PropertyHandle->HasMetaData(TEXT("EnforceUniqueNames"));
			FNiagaraDataChannelVariable* Variable = static_cast<FNiagaraDataChannelVariable*>(VarPtr);

			Row.NameContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("DataChannelVarNameText", "Name: "))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 4.0f, 4.0f, 4.0f)
				[
					SNew(SNiagaraConstrainedBox)
					.MinWidth(150.0f)
					[
						SNew( SEditableTextBox )
						.Text_Lambda([NameHandle]() -> FText { FText OutText; NameHandle->GetValueAsDisplayText(OutText); return OutText; })
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.SelectAllTextWhenFocused(true)
						.ClearKeyboardFocusOnCommit(false)
						.MaximumLength(NAME_SIZE-1)
						.OnTextCommitted_Lambda([NameHandle](const FText& NewName, ETextCommit::Type)
						{
							FScopedTransaction Transaction(LOCTEXT("ChangeVarName", "Change variable name"));
							TArray<UObject*> OuterObjects;
							NameHandle->GetOuterObjects(OuterObjects);
							for (UObject* OuterObj : OuterObjects)
							{
								OuterObj->Modify();
							}
							NameHandle->SetValue(FName(NewName.ToString()));
						} )
						.OnVerifyTextChanged_Lambda([bEnforceUniqueName, PropertyHandle](const FText& InNewText, FText& OutErrorMessage) -> bool
						{
							// if necessary, validate that the entered name is unique among all entries of the variable array
							if (!bEnforceUniqueName)
							{
								return true;
							}
							FName NewName = FName(InNewText.ToString());
							if (TSharedPtr<IPropertyHandle> Handle = PropertyHandle->GetParentHandle())
							{
								if (TSharedPtr<IPropertyHandleArray> PropertyHandleArray = Handle->AsArray())
								{
									uint32 NumElements;
									PropertyHandleArray->GetNumElements(NumElements);
									bool bNewNameValid = true;
									for (uint32 i = 0; i < NumElements; i++)
									{
										TSharedRef<IPropertyHandle> ElementHandle = PropertyHandleArray->GetElement(i);
										void* VarPtr = nullptr;
										if (!ElementHandle->IsSamePropertyNode(PropertyHandle) && ElementHandle->GetValueData(VarPtr) == FPropertyAccess::Success)
										{
											if (NewName == static_cast<FNiagaraVariable*>(VarPtr)->GetName())
											{
												bNewNameValid = false;
												break;
											}
										}
									}
									if (!bNewNameValid)
									{
										OutErrorMessage = LOCTEXT("DuplicateNameError", "Variable name has to be unique!");
										return false;
									}
								}
							}
							return true;
						})
						.SelectAllTextOnCommit( true )
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 2.0f, 4.0f, 2.0f)
				[
					SNew(SNiagaraConstrainedBox)
					.MinWidth(150.0f)
					[
						PropertyHandle->CreateDefaultPropertyButtonWidgets()
					]
				]
			];
			
			Row.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 2.0f, 0)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("DataChannelVarTypeText", "Type: "))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					SAssignNew(ChangeTypeButton, SComboButton)
					.HasDownArrow(true)
					.ContentPadding(0)
					.OnGetMenuContent(this, &FNiagaraDataChannelVariableDetailsCustomization::GetTypeMenu, TypeDefHandleHandle, Variable)
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.AutoWidth()
						[
							SNew(SImage)
							.ColorAndOpacity_Lambda([Variable]() {return UEdGraphSchema_Niagara::GetTypeColor(Variable->GetType());})
							.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"))
						]
						+ SHorizontalBox::Slot()
						.Padding(4, 2, 2, 2)
						[
							SNew(STextBlock)
							.MinDesiredWidth(150)
							.Text_Lambda([Variable] {return Variable ? Variable->GetType().GetNameText() : FText::GetEmpty(); })
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
				]
			];
		}
	}
}

class SNiagaraDataChannelTypeSelectMenu : public SNiagaraParameterMenu
{
public:
	//DECLARE_DELEGATE_RetVal_OneParam(bool, FOnAllowMakeType, const FNiagaraTypeDefinition&);

	SLATE_BEGIN_ARGS(SNiagaraDataChannelTypeSelectMenu)
		: _AutoExpandMenu(false)
	{}
	//~ Begin Required Args
	SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)
	SLATE_ARGUMENT(FNiagaraDataChannelVariable*, VarToModify)
	//~ End Required Args
	SLATE_ARGUMENT(bool, AutoExpandMenu)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

protected:
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	FNiagaraDataChannelVariable* VarToModify = nullptr;
};


void SNiagaraDataChannelTypeSelectMenu::Construct(const FArguments& InArgs)
{
	checkf(InArgs._VarToModify != nullptr, TEXT("Tried to construct change var type menu without valid var ptr!"));
	this->PropertyHandle = InArgs._PropertyHandle;
	this->VarToModify = InArgs._VarToModify;

	SNiagaraParameterMenu::FArguments SuperArgs;
	SuperArgs._AutoExpandMenu = InArgs._AutoExpandMenu;
	SNiagaraParameterMenu::Construct(SuperArgs);
}

void SNiagaraDataChannelTypeSelectMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FNiagaraMenuActionCollector Collector;

	TArray<FNiagaraTypeDefinition> Types;
	FNiagaraEditorUtilities::GetAllowedUserVariableTypes(Types);
	for (FNiagaraTypeDefinition& Type : Types)
	{
		if (!FNiagaraDataChannelVariable::IsAllowedType(Type))
		{
			continue;
		}
		Type = FNiagaraDataChannelVariable::ToDataChannelType(Type);

		FText Category = FNiagaraEditorUtilities::GetTypeDefinitionCategory(Type);
		const FText DisplayName = Type.GetNameText();
		const FText Tooltip = Type.GetNameText();
		TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
			Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
			FNiagaraMenuAction::FOnExecuteStackAction::CreateLambda(
			[Var=VarToModify, PropHandle=PropertyHandle, Type]()
			{
				FScopedTransaction Transaction(LOCTEXT("Set Raw Niagara Variable Type", "Set Variable Type"));
				check(Var);
				check(PropHandle.IsValid());
				TArray<UObject*> Objects;
				PropHandle->GetOuterObjects(Objects);
				for (UObject* Obj : Objects)
				{
					Obj->Modify();
				}

				PropHandle->NotifyPreChange();
				Var->SetType(Type);
				Var->Version = FGuid::NewGuid();
				PropHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				PropHandle->NotifyFinishedChangingProperties();
			}
		)));

		Collector.AddAction(Action, 0);
	}

	Collector.AddAllActionsTo(OutAllActions);
}

TSharedRef<SWidget> FNiagaraDataChannelVariableDetailsCustomization::GetTypeMenu(TSharedPtr<IPropertyHandle> InPropertyHandle, FNiagaraDataChannelVariable* Var)
{
	TSharedRef<SNiagaraDataChannelTypeSelectMenu> TypeSelectMenu = SNew(SNiagaraDataChannelTypeSelectMenu)
	.PropertyHandle(InPropertyHandle)
	.VarToModify(Var);

	ChangeTypeButton->SetMenuContentWidgetToFocus(TypeSelectMenu->GetSearchBox());
	
	return TypeSelectMenu;
}

void FNDCAccessContextDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		];
	HeaderRow.ValueContent()
		[
			PropertyHandle->CreatePropertyValueWidget()
		];
}

void FNDCAccessContextDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	bool bShowInputContext = true;
	bool bShowOutputContext = false;

	//Walk up our parent chain until we get to a FNDCAccessContextInst and check it's meta data.
	TSharedPtr<IPropertyHandle> ParentPtr = PropertyHandle->GetParentHandle();
	while(ParentPtr.IsValid())
	{
		if (const FStructProperty* StructProp = CastField<FStructProperty>(ParentPtr->GetProperty()))
		{
			UScriptStruct* ParentStruct = StructProp->Struct;
			if (ParentStruct == FNDCAccessContextInst::StaticStruct())
			{
				bShowInputContext = PropertyHandle->HasMetaData(TEXT("ShowNDCAccessContextInput"));
				bShowOutputContext = PropertyHandle->HasMetaData(TEXT("ShowNDCAccessContextOutput"));

				if (!bShowInputContext && !bShowOutputContext)
				{
					//If neither are set then assume the user has neglected to add the meta tags so just show input only.
					bShowInputContext = true;
					bShowOutputContext = false;
				}
				break;
			}
		}

		ParentPtr = ParentPtr->GetParentHandle();
	}

	uint32 NumAccessContextChildren = 0;
	PropertyHandle->GetNumChildren(NumAccessContextChildren);

	for(uint32 ChildIdx = 0; ChildIdx < NumAccessContextChildren; ++ChildIdx)
	{
		TSharedPtr<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(ChildIdx);
		if(!ChildPropertyHandle.IsValid())
		{
			continue;
		}
		FProperty* ChildProperty = ChildPropertyHandle->GetProperty();
		if(!ChildProperty)
		{
			continue;
		}

		bool bIsInputContext = ChildPropertyHandle->HasMetaData(TEXT("NDCAccessContextInput")) && !ChildPropertyHandle->GetProperty()->HasAnyPropertyFlags(CPF_Transient);//Don't show transients in details panels.
		bool bIsOutputContext = ChildPropertyHandle->HasMetaData(TEXT("NDCAccessContextOutput"));
		if(!bIsInputContext && !bIsOutputContext)
		{
			//If neither are set then assume the user has neglected to add the meta tags so just show everything.
			//A cluttered details display is better than an empty one.
			bIsInputContext = true;
			bIsOutputContext = true;
		}

		if((bShowInputContext && bIsInputContext) || (bShowOutputContext && bIsOutputContext))
		{
			ChildBuilder.AddProperty(ChildPropertyHandle.ToSharedRef());
		}
	}
}


void FNiagaraDataChannelReferenceDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		];
	HeaderRow.ValueContent()
		[
			PropertyHandle->CreatePropertyValueWidget()
		];
}

void FNiagaraDataChannelReferenceDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyPtr = PropertyHandle.ToSharedPtr();

	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ++ChildIdx)
	{
		TSharedPtr<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(ChildIdx);
		if (!ChildPropertyHandle.IsValid())
		{
			continue;
		}
		FProperty* ChildProperty = ChildPropertyHandle->GetProperty();
		if (!ChildProperty)
		{
			continue;
		}

		ChildBuilder.AddProperty(ChildPropertyHandle.ToSharedRef());

		ChildPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FNiagaraDataChannelReferenceDetailsCustomization::OnValueChanged));
	}
}

void FNiagaraDataChannelReferenceDetailsCustomization::OnValueChanged()
{
	void* DataPtr = nullptr;
	if (PropertyPtr.IsValid() && PropertyPtr->GetValueData(DataPtr) == FPropertyAccess::Success)
	{
		FNiagaraDataChannelReference* ValuePtr = static_cast<FNiagaraDataChannelReference*>(DataPtr);
		ValuePtr->OnValueChanged();
	}
}

#undef LOCTEXT_NAMESPACE
