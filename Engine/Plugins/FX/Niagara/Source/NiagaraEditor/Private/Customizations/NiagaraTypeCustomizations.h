// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "NiagaraCommon.h"
#include "EdGraph/EdGraphSchema.h"
#include "Layout/Visibility.h"
#include "NiagaraTypes.h"
#include "NiagaraEmitterBase.h"
#include "NiagaraTypeCustomizations.generated.h"

class FDetailWidgetRow;
class FNiagaraScriptViewModel;
class IPropertyHandle;
class IPropertyHandleArray;
class UMaterialInterface;
struct FMaterialParameterInfo;
struct FNiagaraDataChannelVariable;
class UNiagaraGraph;
class UNiagaraParameterDefinitions;
class UNiagaraRendererProperties;
class UNiagaraSimulationStageBase;
class UNiagaraScriptVariable;
class SComboButton;

enum class ECheckBoxState : uint8;


// Args struct for binding parameter definitions names
struct FScriptVarBindingNameSubscriptionArgs
{
	FScriptVarBindingNameSubscriptionArgs()
		: SourceParameterDefinitions(nullptr)
		, SourceScriptVar(nullptr)
		, DestScriptVar(nullptr)
	{
	}

	FScriptVarBindingNameSubscriptionArgs(UNiagaraParameterDefinitions* InSourceParameterDefinitions, const UNiagaraScriptVariable* InSourceScriptVar, const UNiagaraScriptVariable* InDestScriptVar)
		: SourceParameterDefinitions(InSourceParameterDefinitions)
		, SourceScriptVar(InSourceScriptVar)
		, DestScriptVar(InDestScriptVar)
	{
	}

	UNiagaraParameterDefinitions* SourceParameterDefinitions;
	const UNiagaraScriptVariable* SourceScriptVar;
	const UNiagaraScriptVariable* DestScriptVar;
};

class FNiagaraNumericCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraNumericCustomization>();
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils);

	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils )
	{}

};


class FNiagaraBoolCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraBoolCustomization>();
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils);

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
	{}

private:

	ECheckBoxState OnGetCheckState() const;

	void OnCheckStateChanged(ECheckBoxState InNewState);


private:
	TSharedPtr<IPropertyHandle> ValueHandle;
};

class FNiagaraMatrixCustomization : public FNiagaraNumericCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraMatrixCustomization>();
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);


};


USTRUCT()
struct FNiagaraStackAssetAction_VarBind : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	FName VarName;
	FScriptVarBindingNameSubscriptionArgs LibraryNameSubscriptionArgs;
	const UNiagaraScriptVariable* BaseScriptVar;
	FNiagaraVariableBase BaseVar;
	FNiagaraVariableBase ChildVar;

	// Simple type info
	static FName StaticGetTypeId() { static FName Type("FNiagaraStackAssetAction_VarBind"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	FNiagaraStackAssetAction_VarBind()
		: FEdGraphSchemaAction()
	{}

	FNiagaraStackAssetAction_VarBind(FName InVarName,
		FText InNodeCategory,
		FText InMenuDesc,
		FText InToolTip,
		const int32 InGrouping,
		FText InKeywords
	)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords))
		, VarName(InVarName)
		, LibraryNameSubscriptionArgs(FScriptVarBindingNameSubscriptionArgs())
	{}

	FNiagaraStackAssetAction_VarBind(FName InVarName,
		FText InNodeCategory,
		FText InMenuDesc,
		FText InToolTip,
		const int32 InGrouping,
		FText InKeywords,
		FScriptVarBindingNameSubscriptionArgs InLibraryNameSubscriptionArgs
	)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords))
		, VarName(InVarName)
		, LibraryNameSubscriptionArgs(InLibraryNameSubscriptionArgs)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override
	{
		return nullptr;
	}
	//~ End FEdGraphSchemaAction Interface

	static TArray<FNiagaraVariableBase> FindVariables(UNiagaraSystem* NiagaraSystem, const FVersionedNiagaraEmitter& InEmitter, bool bSystem, bool bEmitter, bool bParticles, bool bUser, bool bAllowStatic);
};

class FNiagaraVariableAttributeBindingCustomization : public IPropertyTypeCustomization
{
public:
	/** @return A new instance of this class */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraVariableAttributeBindingCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {};
	/** IPropertyTypeCustomization interface end */
private:
	bool IsValid() const;
	FText GetMissingText() const;

	EVisibility IsResetToDefaultsVisible() const;
	FReply OnResetToDefaultsClicked();
	void ResetToDefault();
	FName GetVariableName() const;
	FText GetCurrentText() const;
	FText GetTooltipText() const;
	TSharedRef<SWidget> OnGetMenuContent() const;
	TArray<FName> GetNames() const;

	void ChangeSource(FName InVarName);
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);
	void OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType);

	TSharedPtr<IPropertyHandle>					PropertyHandle;
	TSharedPtr<SComboButton>					ComboButton;

	FVersionedNiagaraEmitterBaseWeakPtr			WeakVersionedEmitter;
	FGuid										EmitterHandleGuid;		/** The emitter handle guid this binding is used in. Used to gather available parameters for the system & this emitter (not all emitters) */
	TWeakObjectPtr<UNiagaraRendererProperties>	WeakRendererProperties;
	TWeakObjectPtr<UNiagaraSimulationStageBase>	WeakSimulationStage;

	struct FNiagaraVariableAttributeBinding*		TargetVariableBinding = nullptr;
	const struct FNiagaraVariableAttributeBinding*	DefaultVariableBinding = nullptr;
};

class FNiagaraUserParameterBindingCustomization : public IPropertyTypeCustomization
{
public:
	/** @return A new instance of this class */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraUserParameterBindingCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {};
	/** IPropertyTypeCustomization interface end */
private:
	FText GetCurrentText() const;
	FText GetTooltipText() const;
	FName GetVariableName() const;
	TSharedRef<SWidget> OnGetMenuContent() const;
	TArray<FName> GetNames() const;
	void ChangeSource(FName InVarName);
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);
	void OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType);

	bool IsValid() const { return BaseSystemWeakPtr.IsValid() && ObjectCustomizingWeakPtr.IsValid(); }

	TSharedPtr<IPropertyHandle> PropertyHandle;
	TWeakObjectPtr<UObject> ObjectCustomizingWeakPtr;
	TWeakObjectPtr<class UNiagaraSystem> BaseSystemWeakPtr;
	struct FNiagaraUserParameterBinding* TargetUserParameterBinding;

};

class FNiagaraMaterialAttributeBindingCustomization : public IPropertyTypeCustomization
{
public:
	/** @return A new instance of this class */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraMaterialAttributeBindingCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override ;
	/** IPropertyTypeCustomization interface end */
private:
	FText GetNiagaraCurrentText() const;
	FText GetNiagaraTooltipText() const;
	FName GetNiagaraVariableName() const;
	FText GetNiagaraChildVariableText() const;
	FName GetNiagaraChildVariableName() const; 
	EVisibility GetNiagaraChildVariableVisibility() const;

	TSharedRef<SWidget> OnGetNiagaraMenuContent() const;
	TArray<TPair<FNiagaraVariableBase, FNiagaraVariableBase>> GetNiagaraNames() const;
	void ChangeNiagaraSource(FNiagaraStackAssetAction_VarBind* InVar);
	void CollectAllNiagaraActions(FGraphActionListBuilderBase& OutAllActions);
	TSharedRef<SWidget> OnCreateWidgetForNiagaraAction(struct FCreateWidgetForActionData* const InCreateData);
	void OnNiagaraActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType);

	FText GetMaterialCurrentText() const;
	FText GetMaterialTooltipText() const;
	TSharedRef<SWidget> OnGetMaterialMenuContent() const;
	void GetMaterialParameters(TArray<TPair<FName, FString>>& OutBindingNameAndDescription) const;
	void ChangeMaterialSource(FName InVarName);
	void CollectAllMaterialActions(FGraphActionListBuilderBase& OutAllActions);
	TSharedRef<SWidget> OnCreateWidgetForMaterialAction(struct FCreateWidgetForActionData* const InCreateData);
	void OnMaterialActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType);

	TArray<FNiagaraTypeDefinition> GetAllowedVariableTypes() const;
	static FText MakeCurrentText(const FNiagaraVariableBase& BaseVar, const FNiagaraVariableBase& ChildVar);

	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<SComboButton> MaterialParameterButton;
	TSharedPtr<SComboButton> NiagaraParameterButton;
	class UNiagaraSystem* BaseSystem;
	FVersionedNiagaraEmitter BaseEmitter;
	FGuid BaseEmitterHandleId;
	UNiagaraRendererProperties* RenderProps;
	//-TODO:stateless:Remove and unify emitter
	class UNiagaraStatelessEmitter* StatelessEmitter = nullptr;
	//-TODO:stateless:Remove and unify emitter
	struct FNiagaraMaterialAttributeBinding* TargetParameterBinding;

};

class FNiagaraDataInterfaceBindingCustomization : public IPropertyTypeCustomization
{
public:
	/** @return A new instance of this class */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraDataInterfaceBindingCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {};
	/** IPropertyTypeCustomization interface end */
private:
	FText GetCurrentText() const;
	FText GetTooltipText() const;
	FName GetVariableName() const;
	TSharedRef<SWidget> OnGetMenuContent() const;
	TArray<FName> GetNames() const;
	void ChangeSource(FName InVarName);
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);
	void OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType);

	EVisibility IsResetToDefaultsVisible() const;
	FReply OnResetToDefaultsClicked();

	TSharedPtr<IPropertyHandle> PropertyHandle;
	class UNiagaraSimulationStageBase* BaseStage;
	struct FNiagaraVariableDataInterfaceBinding* TargetDataInterfaceBinding;

};


/** The primary goal of this class is to search through type matched and defined Niagara variables 
    in the UNiagaraScriptVariable customization panel to provide a default binding for module inputs. */
class FNiagaraScriptVariableBindingCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraScriptVariableBindingCustomization>();
	}

	/** IPropertyTypeCustomization interface begin */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {};
	/** IPropertyTypeCustomization interface end */
private:
   /** Helpers */
	FName GetVariableName() const;
	FText GetCurrentText() const;
	FText GetTooltipText() const;
	TSharedRef<SWidget> OnGetMenuContent() const;
	TArray<TSharedPtr<FEdGraphSchemaAction>> GetGraphParameterBindingActions(const UNiagaraGraph* Graph);
	TArray<TSharedPtr<FEdGraphSchemaAction>> GetEngineConstantBindingActions();
	TArray<TSharedPtr<FEdGraphSchemaAction>> GetLibraryParameterBindingActions();
	void ChangeSource(FName InVarName);
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);
	void OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType);

	/** State */
	TSharedPtr<IPropertyHandle> PropertyHandle;
	class UNiagaraGraph* BaseGraph;
	class UNiagaraParameterDefinitions* BaseLibrary;
	class UNiagaraScriptVariable* BaseScriptVariable;
	struct FNiagaraScriptVariableBinding* TargetVariableBinding;
};


//** Properties customization for FNiagaraVariableMetadata to hide fields that are not relevant when editing parameter definitions. */
class FNiagaraVariableMetaDataCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraVariableMetaDataCustomization>();
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) {};

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);
};

//** Properties customization for FNiagaraSystemScalabilityOverride to hide fields that are not relevant when editing parameter definitions. */
class FNiagaraSystemScalabilityOverrideCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraSystemScalabilityOverrideCustomization>();
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils);

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);
};

class FNiagaraRendererMaterialParameterCustomization : public IPropertyTypeCustomization
{
public:
	// IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// IPropertyTypeCustomization interface end

	FText GetBindingNameText(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	FName GetBindingName(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	static FText GetMaterialBindingTooltip(FName ParameterName, const FString& ParameterDesc);
	TSharedRef<SWidget> OnGetMaterialBindingNameMenuContent(TSharedPtr<IPropertyHandle> PropertyHandle) const;

	virtual bool CustomizeChildProperty(class IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> PropertyHandle) { return false; }
	virtual void GetMaterialParameterInfos(UMaterialInterface* Material, TArray<FMaterialParameterInfo>& OutMaterialParameterInfos) const = 0;

protected:
	TWeakObjectPtr<UNiagaraRendererProperties>	WeakRenderProperties;
	TSharedPtr<IPropertyHandle>					NamePropertyHandle;
	TSharedPtr<struct FNiagaraMaterialParameterValueHelper>	ValueHelper;
};

class FNiagaraRendererMaterialScalarParameterCustomization : public FNiagaraRendererMaterialParameterCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraRendererMaterialScalarParameterCustomization>();
	}

	virtual void GetMaterialParameterInfos(UMaterialInterface* Material, TArray<FMaterialParameterInfo>& OutMaterialParameterInfos) const override;
	virtual bool CustomizeChildProperty(class IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> PropertyHandle) override;
};

class FNiagaraRendererMaterialVectorParameterCustomization : public FNiagaraRendererMaterialParameterCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraRendererMaterialVectorParameterCustomization>();
	}

	virtual void GetMaterialParameterInfos(UMaterialInterface* Material, TArray<FMaterialParameterInfo>& OutMaterialParameterInfos) const override;
	virtual bool CustomizeChildProperty(IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> PropertyHandle) override;
};

class FNiagaraRendererMaterialTextureParameterCustomization : public FNiagaraRendererMaterialParameterCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraRendererMaterialTextureParameterCustomization>();
	}

	virtual void GetMaterialParameterInfos(UMaterialInterface* Material, TArray<FMaterialParameterInfo>& OutMaterialParameterInfos) const override;
	virtual bool CustomizeChildProperty(IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> PropertyHandle) override;
};

class FNiagaraRendererMaterialStaticBoolParameterCustomization : public FNiagaraRendererMaterialParameterCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraRendererMaterialStaticBoolParameterCustomization>();
	}

	virtual bool CustomizeChildProperty(class IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> PropertyHandle) override;
	virtual void GetMaterialParameterInfos(UMaterialInterface* Material, TArray<FMaterialParameterInfo>& OutMaterialParameterInfos) const override;
	TSharedRef<SWidget> OnGetStaticVariablelBindingNameMenuContent() const;

private:
	struct FNiagaraRendererMaterialStaticBoolParameter* ResolveParameterStruct() const;

	int32 GetWidgetIndex() const;

	void SetValue(FName ParameterName) const;
	void SetValue(bool Value) const;

	FText GetCurrentConstantValue() const;
	FText GetConstantText(bool bValue) const;

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};

//** Properties customization for FNiagaraVariables. */
class FNiagaraVariableDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraVariableDetailsCustomization>();
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	TSharedRef<SWidget> GetTypeMenu(TSharedPtr<IPropertyHandle> InPropertyHandle, FNiagaraVariable* Var);
};

//** Properties customization for FNiagaraDataChannelVariable. */
class FNiagaraDataChannelVariableDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FNiagaraDataChannelVariableDetailsCustomization>();
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	TSharedRef<SWidget> GetTypeMenu(TSharedPtr<IPropertyHandle> InPropertyHandle, FNiagaraDataChannelVariable* Var);
	TSharedPtr<SComboButton> ChangeTypeButton;
};

//** Properties customization for FNDCAccessContext. */
class FNDCAccessContextDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() { return MakeShared<FNDCAccessContextDetailsCustomization>(); }
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
};

//** Properties customization for FNiagaraDataChannelReference. */
class FNiagaraDataChannelReferenceDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() { return MakeShared<FNiagaraDataChannelReferenceDetailsCustomization>(); }
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	void OnValueChanged();

	TSharedPtr<IPropertyHandle> PropertyPtr;
};
