// Copyright Epic Games, Inc. All Rights Reserved.

#include "FNiagaraDataChannelSimCacheVisualizer.h"

#include "NiagaraEditorModule.h"
#include "DataInterface/NiagaraDataInterfaceDataChannelWrite.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/Colors/SColorBlock.h"

#define LOCTEXT_NAMESPACE "NiagaraDataChannelCacheView"

namespace DataChannelVisualizer
{
	const FNDIDataChannelWriteSimCacheFrame& GetFrame(TSharedPtr<FNiagaraSimCacheViewModel> SimCacheViewModel, const UNDIDataChannelWriteSimCacheData* DataChannelData)
	{
		int32 FrameIndex = FMath::Clamp(SimCacheViewModel->GetFrameIndex(), 0, DataChannelData->FrameData.Num() - 1);
		return DataChannelData->FrameData[FrameIndex];
	}

	FText ChannelVariableToString(const FNiagaraVariable& Var)
	{
		if (!Var.IsDataAllocated())
		{
			return FText();
		}
		FNumberFormattingOptions NumberFormat = FNumberFormattingOptions()
				.SetUseGrouping(false)
				.SetMaximumFractionalDigits(4)
				.SetMinimumFractionalDigits(0);
		
		if (Var.GetType() == FNiagaraTypeHelper::GetDoubleDef())
		{
			double Value = Var.GetValue<double>();
			return FText::Format(FText::FromString("{0}"), FText::AsNumber(Value, &NumberFormat));
		}
		if (Var.GetType() == FNiagaraTypeDefinition::GetBoolDef())
		{
			bool Value = Var.GetValue<bool>();
			return Value ? LOCTEXT("true", "true") : LOCTEXT("false", "false");
		}
		if (Var.GetType() == FNiagaraTypeDefinition::GetIntDef())
		{
			int32 Value = Var.GetValue<int32>();
			return FText::AsNumber(Value);
		}
		if (Var.GetType() == FNiagaraTypeHelper::GetVectorDef())
		{
			FVector Value = Var.GetValue<FVector>();
			return FText::Format(FText::FromString("X = {0}\nY = {1}\nZ = {2}"), FText::AsNumber(Value.X, &NumberFormat), FText::AsNumber(Value.Y, &NumberFormat), FText::AsNumber(Value.Z, &NumberFormat));
		}
		if (Var.GetType() == FNiagaraTypeHelper::GetVector2DDef())
		{
			FVector2D Value = Var.GetValue<FVector2D>();
			return FText::Format(FText::FromString("X = {0}\nY = {1}"), FText::AsNumber(Value.X, &NumberFormat), FText::AsNumber(Value.Y, &NumberFormat));
		}
		if (Var.GetType() == FNiagaraTypeHelper::GetVector4Def())
		{
			FVector4 Value = Var.GetValue<FVector4>();
			return FText::Format(FText::FromString("X = {0}, Y = {1}\nZ = {2}, W = {3}"), FText::AsNumber(Value.X, &NumberFormat), FText::AsNumber(Value.Y, &NumberFormat), FText::AsNumber(Value.Z, &NumberFormat), FText::AsNumber(Value.W, &NumberFormat));
		}
		if (Var.GetType() == FNiagaraTypeHelper::GetQuatDef())
		{
			FQuat Value = Var.GetValue<FQuat>();
			return FText::Format(FText::FromString("X = {0}, Y = {1}\nZ = {2}, W = {3}"), FText::AsNumber(Value.X, &NumberFormat), FText::AsNumber(Value.Y, &NumberFormat), FText::AsNumber(Value.Z, &NumberFormat), FText::AsNumber(Value.W, &NumberFormat));
		}
		if (Var.GetType() == FNiagaraTypeDefinition::GetColorDef())
		{
			FLinearColor Value = Var.GetValue<FLinearColor>();
			return FText::Format(FText::FromString("R = {0}, G = {1}\nB = {2}, A = {3}"), FText::AsNumber(Value.R, &NumberFormat), FText::AsNumber(Value.G, &NumberFormat), FText::AsNumber(Value.B, &NumberFormat), FText::AsNumber(Value.A, &NumberFormat));
		}
		if (Var.GetType() == FNiagaraTypeDefinition::GetIDDef())
		{
			FNiagaraID Value = Var.GetValue<FNiagaraID>();
			return FText::Format(LOCTEXT("ID_Format", "{0}/{1}"), FText::AsNumber(Value.Index), FText::AsNumber(Value.AcquireTag));
		}
		if (Var.GetType().IsEnum())
		{
			int32 Value = Var.GetValue<int32>();
			return Var.GetType().GetEnum()->GetDisplayNameTextByValue(Value);
		}
		return LOCTEXT("UnsupportedType", "Unsupported Type");
	}
}

class SNiagaraDataChannelCacheView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDataChannelCacheView)
	{}
	SLATE_END_ARGS()

	virtual ~SNiagaraDataChannelCacheView() override;

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UNDIDataChannelWriteSimCacheData* InData);

	TSharedRef<ITableRow> MakeRowWidget(const TSharedPtr<int32> RowIndexPtr, const TSharedRef<STableViewBase>& OwnerTable) const;

	const FNDIDataChannelWriteSimCacheFrame& GetFrame() const
	{
		return DataChannelVisualizer::GetFrame(SimCacheViewModel, DataChannelData.Get());
	}

private:
	void UpdateRows(bool Refresh = true);
	
	TSharedPtr<FNiagaraSimCacheViewModel> SimCacheViewModel;
	TStrongObjectPtr<const UNDIDataChannelWriteSimCacheData> DataChannelData;

	TSharedPtr<SListView<TSharedPtr<int32>>> ListViewWidget;
	TArray<TSharedPtr<int32>> RowItems;	
	TSharedPtr<SHeaderRow> HeaderRowWidget;
	TMap<FName, int32> NameToIndexMapping;
};

class SDataChannelVisualizerRowWidget : public SMultiColumnTableRow<TSharedPtr<int32>>
{
public:
SLATE_BEGIN_ARGS(SDataChannelVisualizerRowWidget) {}
	SLATE_ARGUMENT(TSharedPtr<int32>, RowIndexPtr)
	SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheViewModel>, SimCacheViewModel)
	SLATE_ARGUMENT(const UNDIDataChannelWriteSimCacheData*, Data)
SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TMap<FName, int32>& NameMapping);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	TSharedPtr<int32> RowIndexPtr;
	TSharedPtr<FNiagaraSimCacheViewModel> SimCacheViewModel;
	TStrongObjectPtr<const UNDIDataChannelWriteSimCacheData> DataChannelData;
	TMap<FName, int32> NameToIndexMapping;
};

SNiagaraDataChannelCacheView::~SNiagaraDataChannelCacheView()
{
	SimCacheViewModel->OnViewDataChanged().RemoveAll(this);
}

void SNiagaraDataChannelCacheView::Construct(const FArguments&, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UNDIDataChannelWriteSimCacheData* InData)
{
	SimCacheViewModel = InViewModel;
	DataChannelData.Reset(InData);

	HeaderRowWidget = SNew(SHeaderRow);
	
	// Generate fixed columns
	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(FName("_ElementIndex"))
		.DefaultLabel(LOCTEXT("ElementName", "Element Index"))
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Fill)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.ManualWidth(100)
		.SortMode(EColumnSortMode::None)
	);
	
	// Generate a column for each variable
	for (int i = 0; i < GetFrame().VariableData.Num(); i++)
	{
		const FNDIDataChannelWriteSimCacheFrameBuffer& Buffer = GetFrame().VariableData[i];
		NameToIndexMapping.Add(Buffer.SourceVar.GetName(), i);
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(Buffer.SourceVar.GetName())
			.DefaultLabel(FText::FromName(Buffer.SourceVar.GetName()))
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Fill)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			.FillWidth(1)
			.ManualWidth(150)
			.SortMode(EColumnSortMode::None)
		);
	}

	SAssignNew(ListViewWidget, SListView<TSharedPtr<int32>>)
		.ListItemsSource(&RowItems)
		.OnGenerateRow(this, &SNiagaraDataChannelCacheView::MakeRowWidget)
		.Visibility(EVisibility::Visible)
		.SelectionMode(ESelectionMode::Single)
		.ConsumeMouseWheel(EConsumeMouseWheel::Never)
		.ScrollbarVisibility(EVisibility::Collapsed)
		.HeaderRow(HeaderRowWidget);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Margin(10)
			.Text_Lambda([this]()
			{
				const FNDIDataChannelWriteSimCacheFrame& FrameData = GetFrame();
				if (FrameData.NumElements == 0)
				{
					return LOCTEXT("NoData", "This frame contains no data channel writes.");
				}
				return FText::Format(LOCTEXT("WriteSummary", "This frame writes {0} elements to the data channel {4}.\n[{1}] Visible to Blueprint\n[{2}] Visible to CPU systems\n[{3}] Visible to GPU systems"),
					FText::AsNumber(FrameData.NumElements),
					FText::FromString(FrameData.bVisibleToGame ? TEXT("X") : TEXT("  ")),
					FText::FromString(FrameData.bVisibleToCPUSims ? TEXT("X") : TEXT("  ")),
					FText::FromString(FrameData.bVisibleToGPUSims ? TEXT("X") : TEXT("  ")),
					FText::FromString(DataChannelData->DataChannelReference.GetAssetPathString()));
			})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ListViewWidget.ToSharedRef()
		]
	];

	UpdateRows();
	SimCacheViewModel->OnViewDataChanged().AddSP(this, &SNiagaraDataChannelCacheView::UpdateRows);
}

TSharedRef<ITableRow> SNiagaraDataChannelCacheView::MakeRowWidget(const TSharedPtr<int32> RowIndexPtr, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return
		SNew(SDataChannelVisualizerRowWidget, OwnerTable, NameToIndexMapping)
		.RowIndexPtr(RowIndexPtr)
		.SimCacheViewModel(SimCacheViewModel)
		.Data(DataChannelData.Get());
}

void SNiagaraDataChannelCacheView::UpdateRows(bool)
{
	const FNDIDataChannelWriteSimCacheFrame& FrameData = GetFrame();
	RowItems.Reset(FrameData.NumElements);
	for (int32 i = 0; i < FrameData.NumElements; ++i)
	{
		RowItems.Emplace(MakeShared<int32>(i));
	}
	ListViewWidget->RequestListRefresh();
}

void SDataChannelVisualizerRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TMap<FName, int32>& NameMapping)
{
	RowIndexPtr = InArgs._RowIndexPtr;
	SimCacheViewModel = InArgs._SimCacheViewModel;
	DataChannelData.Reset(InArgs._Data);
	NameToIndexMapping = NameMapping;

	SMultiColumnTableRow::Construct(
		FSuperRowType::FArguments()
		.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
		InOwnerTableView
	);
}

TSharedRef<SWidget> SDataChannelVisualizerRowWidget::GenerateWidgetForColumn(const FName& InColumnName)
{
	if(!SimCacheViewModel.Get()->IsCacheValid())
	{
		return SNullWidget::NullWidget;
	}

	const int32 InstanceIndex = *RowIndexPtr;
	if (InColumnName == FName("_ElementIndex"))
	{
		return SNew(STextBlock)
			.Text(FText::AsNumber(InstanceIndex));
	}

	int32 Index = NameToIndexMapping.FindChecked(InColumnName);
	const FNDIDataChannelWriteSimCacheFrameBuffer& Buffer = DataChannelVisualizer::GetFrame(SimCacheViewModel, DataChannelData.Get()).VariableData[Index];
	FNiagaraVariable DataVar = Buffer.SourceVar;
	if (DataVar.GetType() == FNiagaraTypeDefinition::GetPositionDef())
	{
		DataVar.SetType(FNiagaraTypeHelper::GetVectorDef());
	}
	else
	{
		DataVar.SetType(FNiagaraTypeHelper::GetLWCType(DataVar.GetType()));
	}
	DataVar.SetData(Buffer.Data.GetData() + (Buffer.Size * InstanceIndex));

	FText DisplayText = DataChannelVisualizer::ChannelVariableToString(DataVar);
	if (DataVar.GetType() == FNiagaraTypeDefinition::GetColorDef())
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(10, 0))
			.AutoWidth()
			[
				SNew(SColorBlock)
				.Color(DataVar.GetValue<FLinearColor>())
				.ShowBackgroundForAlpha(true)
			]
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Margin(FMargin(0, 0, 15, 0))
				.Text(DisplayText)
			];
	}
	return SNew(STextBlock)
		.Margin(FMargin(15, 0))
		.Text(DisplayText);
}

TSharedPtr<SWidget> FNiagaraDataChannelSimCacheVisualizer::CreateWidgetFor(const UObject* CachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel)
{
	if (const UNDIDataChannelWriteSimCacheData* DataChannelData = Cast<const UNDIDataChannelWriteSimCacheData>(CachedData))
	{
		return SNew(SNiagaraDataChannelCacheView, ViewModel, DataChannelData);
	}
	return TSharedPtr<SWidget>();
}

#undef LOCTEXT_NAMESPACE
