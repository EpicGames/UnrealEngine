// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/NiagaraDataInterfaceSimCacheVisualizer.h"

/*
 * Provides a custom widget to show data channel writes stored in the sim cache in a table.
 */
class FNiagaraDataChannelSimCacheVisualizer : public INiagaraDataInterfaceSimCacheVisualizer
{
public:
	virtual ~FNiagaraDataChannelSimCacheVisualizer() override = default;
	
	virtual TSharedPtr<SWidget> CreateWidgetFor(const UObject* CachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel) override;
};
