// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraArraySimCacheVisualizer.h"

#include "NiagaraDataInterfaceArray.h"

#include "UObject/StrongObjectPtr.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailLayoutBuilder.h"
#include "SEnumCombo.h"

#define LOCTEXT_NAMESPACE "NiagaraArraySimCacheVisualizer"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace NDIArraySimCacheVisualizer
{
	static const FName NAME_Index = FName("Index");
	static const FName NAME_CpuValue = FName("Cpu Value");
	static const FName NAME_GpuValue = FName("Gpu Value");

class SVisualizerRowWidget : public SMultiColumnTableRow<TSharedPtr<int32>>
{
public:
	SLATE_BEGIN_ARGS(SVisualizerRowWidget) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheViewModel>,	ViewModel)
		SLATE_ARGUMENT(const UNDIArraySimCacheData*,			CacheData)
		SLATE_ARGUMENT(TSharedPtr<int32>,						RowIndexPtr)
		SLATE_ARGUMENT(UNiagaraDataInterfaceArray*,				DataInterface)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		ViewModel			= InArgs._ViewModel;
		CacheData.Reset(InArgs._CacheData);
		RowIndexPtr			= InArgs._RowIndexPtr;
		DataInterface		= InArgs._DataInterface;

		SMultiColumnTableRow::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (InColumnName == NAME_Index)
		{
			return SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("IndexFormat", "{0}"), FText::AsNumber(*RowIndexPtr)));
		}

		const int32 FrameIndex = ViewModel->GetFrameIndex();
		const int32 ElementIndex = *RowIndexPtr;

		//-TODO: Print the data
		if (InColumnName == NAME_CpuValue)
		{
			if (CacheData->CpuFrameData.IsValidIndex(FrameIndex))
			{
				FString Value = DataInterface->SimCacheVisualizerRead(CacheData.Get(), CacheData->CpuFrameData[FrameIndex], ElementIndex);
				return SNew(STextBlock)
					.Text(FText::FromString(Value));
			}
		}

		if (InColumnName == NAME_GpuValue)
		{
			if (CacheData->GpuFrameData.IsValidIndex(FrameIndex))
			{
				FString Value = DataInterface->SimCacheVisualizerRead(CacheData.Get(), CacheData->GpuFrameData[FrameIndex], ElementIndex);
				return SNew(STextBlock)
					.Text(FText::FromString(Value));
			}
		}

		return SNullWidget::NullWidget;
	}

	TSharedPtr<FNiagaraSimCacheViewModel>			ViewModel;
	TStrongObjectPtr<const UNDIArraySimCacheData>	CacheData;
	TSharedPtr<int32>								RowIndexPtr;
	UNiagaraDataInterfaceArray*						DataInterface = nullptr;
};

class SSimCacheView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSimCacheView)
	{}
	SLATE_END_ARGS()

	virtual ~SSimCacheView() override
	{
		ViewModel->OnViewDataChanged().RemoveAll(this);
	}

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UNDIArraySimCacheData* InCacheData, UNiagaraDataInterfaceArray* InDataInterface)
	{
		ViewModel = InViewModel;
		CacheData.Reset(InCacheData);
		DataInterface = InDataInterface;

		ViewModel->OnViewDataChanged().AddSP(this, &SSimCacheView::UpdateRows);

		HeaderRowWidget = SNew(SHeaderRow);
		UpdateColumns();

		SAssignNew(ListViewWidget, SListView<TSharedPtr<int32>>)
			.ListItemsSource(&RowItems)
			.OnGenerateRow(this, &SSimCacheView::MakeRowWidget)
			.Visibility(EVisibility::Visible)
			.SelectionMode(ESelectionMode::Single)
			.ConsumeMouseWheel(EConsumeMouseWheel::Never)
			.ScrollbarVisibility(EVisibility::Collapsed)
			.HeaderRow(HeaderRowWidget);

		ChildSlot
		[
			SNew(SVerticalBox)
			// Display Array Information
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(this, &SSimCacheView::GetCpuArrayInformation)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(this, &SSimCacheView::GetGpuArrayInformation)
				]
			]
			// Buffer Data View
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 8.0f, 0.0f, 0.0f)
			[
				ListViewWidget.ToSharedRef()
			]
		];
	}

	TSharedRef<ITableRow> MakeRowWidget(const TSharedPtr<int32> RowIndexPtr, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		return
			SNew(SVisualizerRowWidget, OwnerTable)
				.ViewModel(ViewModel)
				.CacheData(CacheData.Get())
				.RowIndexPtr(RowIndexPtr)
				.DataInterface(DataInterface)
				;
	}

	void UpdateColumns(bool bRefresh = true)
	{
		HeaderRowWidget->ClearColumns();
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(NAME_Index)
			.DefaultLabel(LOCTEXT("Index", "Index"))
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Fill)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			.ManualWidth(100)
			.SortMode(EColumnSortMode::None)
		);

		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(NAME_CpuValue)
			.DefaultLabel(LOCTEXT("Cpu Value", "Cpu Value"))
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Fill)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			.ManualWidth(100)
			.SortMode(EColumnSortMode::None)
		);

		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(NAME_GpuValue)
			.DefaultLabel(LOCTEXT("Gpu Value", "Gpu Value"))
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Fill)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			.ManualWidth(100)
			.SortMode(EColumnSortMode::None)
		);

		if (ListViewWidget && bRefresh)
		{
			ListViewWidget->RequestListRefresh();
		}
	}

	void UpdateRows(bool bRefresh = true)
	{
		const int32 FrameIndex = ViewModel->GetFrameIndex();

		int NumElements = 0;
		if (CacheData->CpuFrameData.IsValidIndex(FrameIndex))
		{
			NumElements = FMath::Max(NumElements, CacheData->CpuFrameData[FrameIndex].NumElements);
		}
		if (CacheData->GpuFrameData.IsValidIndex(FrameIndex))
		{
			NumElements = FMath::Max(NumElements, CacheData->GpuFrameData[FrameIndex].NumElements);
		}

		RowItems.Reset(NumElements);
		for ( int32 i=0; i < NumElements; ++i )
		{
			RowItems.Emplace(MakeShared<int32>(i));
		}

		if (ListViewWidget && bRefresh)
		{
			ListViewWidget->RequestListRefresh();
		}
	}


	FText GetCpuArrayInformation() const
	{
		const int32 FrameIndex = ViewModel->GetFrameIndex();
		int NumElements = 0;

		if (CacheData->CpuFrameData.IsValidIndex(FrameIndex))
		{
			NumElements = CacheData->CpuFrameData[FrameIndex].NumElements;
		}

		return FText::Format(LOCTEXT("CpuArrayInfoFormat", "CPU Array Num Elements = {0}"), NumElements);
	}

	FText GetGpuArrayInformation() const
	{
		const int32 FrameIndex = ViewModel->GetFrameIndex();
		int NumElements = 0;

		if (CacheData->GpuFrameData.IsValidIndex(FrameIndex))
		{
			NumElements = CacheData->GpuFrameData[FrameIndex].NumElements;
		}

		return FText::Format(LOCTEXT("GpuArrayInfoFormat", "GPU Array Num Elements = {0}"), NumElements);
	}

private:
	TSharedPtr<FNiagaraSimCacheViewModel>			ViewModel;
	TStrongObjectPtr<const UNDIArraySimCacheData>	CacheData;

	TSharedPtr<SHeaderRow>							HeaderRowWidget;
	TSharedPtr<SListView<TSharedPtr<int32>>>		ListViewWidget;
	TArray<TSharedPtr<int32>>						RowItems;
	UNiagaraDataInterfaceArray*						DataInterface = nullptr;
};

} // NDIArraySimCacheVisualizer

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> FNiagaraArraySimCacheVisualizer::CreateWidgetFor(const UObject* InCachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel)
{
	using namespace NDIArraySimCacheVisualizer;

	if (const UNDIArraySimCacheData* CachedData = Cast<const UNDIArraySimCacheData>(InCachedData))
	{
		check(ArrayDIClass);
		UNiagaraDataInterfaceArray* DataInterface = CastChecked<UNiagaraDataInterfaceArray>(ArrayDIClass->GetDefaultObject());		
		return SNew(SSimCacheView, ViewModel, CachedData, DataInterface);
	}
	return TSharedPtr<SWidget>();
}

#undef LOCTEXT_NAMESPACE
