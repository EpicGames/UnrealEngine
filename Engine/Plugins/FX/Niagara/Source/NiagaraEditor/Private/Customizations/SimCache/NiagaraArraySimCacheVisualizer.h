// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "Customizations/NiagaraDataInterfaceSimCacheVisualizer.h"

//#include "NiagaraArraySimCacheVisualizer.generated.h"

/*
 * Provides a custom widget to show the array DI data in the sim cache in a table.
 */
class FNiagaraArraySimCacheVisualizer : public INiagaraDataInterfaceSimCacheVisualizer
{
public:
	FNiagaraArraySimCacheVisualizer(UClass* InArrayDIClass)
		: ArrayDIClass(InArrayDIClass)
	{
	}
	virtual ~FNiagaraArraySimCacheVisualizer() override = default;
	
	virtual TSharedPtr<SWidget> CreateWidgetFor(const UObject* CachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel) override;

protected:
	UClass* ArrayDIClass = nullptr;
};
