// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDynamicMeshSimCacheVisualizer.h"

#include "DataInterface/NiagaraDataInterfaceDynamicMeshSimCacheData.h"
#include "NiagaraEditorStyle.h"

#include "Engine/StaticMesh.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Modules/ModuleManager.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/StrongObjectPtr.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "MeshDescription.h"
#include "MeshAttributes.h"
#include "PropertyEditorModule.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "IStructureDetailsView.h"
#include "SEnumCombo.h"

#define LOCTEXT_NAMESPACE "NiagaraDynamicMeshSimCacheVisualizer"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace NDIDynamicMeshSimCacheVisualizer
{
	struct FTangentBasis
	{
		FVector3f XAxis = FVector3f::XAxisVector;
		FVector3f YAxis = FVector3f::YAxisVector;
		FVector3f ZAxis = FVector3f::ZAxisVector;
	};

	struct FMeshReadHelper
	{
		explicit FMeshReadHelper(const FNDIDynamicMeshSimCacheFrame* CacheFrame)
		{
			NumVertices = CacheFrame->NumVertices;
			NumTriangles = CacheFrame->NumTriangles;
			NumTexCoords = CacheFrame->NumTexCoords;

			Indicies = MakeArrayView((const uint32*)CacheFrame->IndexData.GetData(), CacheFrame->IndexData.Num() / sizeof(uint32));

			if (CacheFrame->PositionOffset != INDEX_NONE)
			{
				Positions = MakeArrayView((const FVector3f*)(CacheFrame->VertexData.GetData() + CacheFrame->PositionOffset), NumVertices);
			}
			if (CacheFrame->TangentBasisOffset != INDEX_NONE)
			{
				TangentBasis = MakeArrayView((const FUintVector2*)(CacheFrame->VertexData.GetData() + CacheFrame->TangentBasisOffset), NumVertices);
			}
			if (CacheFrame->TexCoordOffset != INDEX_NONE)
			{
				TexCoords = MakeArrayView((const FVector2f*)(CacheFrame->VertexData.GetData() + CacheFrame->TexCoordOffset), NumVertices * NumTexCoords);
			}
			if (CacheFrame->ColorOffset != INDEX_NONE)
			{
				Colors = MakeArrayView((const FColor*)(CacheFrame->VertexData.GetData() + CacheFrame->ColorOffset), NumVertices);
			}
		}

		uint32 GetNumVertices() const { return NumVertices; }
		uint32 GetNumTriangles() const { return NumTriangles; }
		uint32 GetNumTexCoords() const { return NumTexCoords; }

		FUintVector3 GetTriangleIndices(uint32 Triangle) const { return FUintVector3(Indicies[Triangle * 3 + 0], Indicies[Triangle * 3 + 1], Indicies[Triangle * 3 + 2]); }

		FVector3f GetVertexPosition(uint32 VertexIndex) const { return Positions[VertexIndex]; }
		FTangentBasis GeVertexTangents(uint32 VertexIndex) const
		{
			const uint32 PackedTangentX = TangentBasis[VertexIndex].X;
			const uint32 PackedTangentZ = TangentBasis[VertexIndex].Y;

			FTangentBasis Tangents;
			Tangents.XAxis = FVector3f(float((PackedTangentX >> 0) & 0xff), float((PackedTangentX >> 8) & 0xff), float((PackedTangentX >> 16) & 0xff));
			Tangents.ZAxis = FVector3f(float((PackedTangentZ >> 0) & 0xff), float((PackedTangentZ >> 8) & 0xff), float((PackedTangentZ >> 16) & 0xff));
			Tangents.XAxis = (Tangents.XAxis / 127.5f) - 1.0f;
			Tangents.ZAxis = (Tangents.ZAxis / 127.5f) - 1.0f;

			const float TangentSign = (float((PackedTangentZ >> 24) & 0xff) / 127.5f) - 1.0f;

			Tangents.YAxis = FVector3f::CrossProduct(Tangents.ZAxis, Tangents.XAxis) * TangentSign;
			Tangents.XAxis = FVector3f::CrossProduct(Tangents.YAxis, Tangents.ZAxis) * TangentSign;
			return Tangents;
		}
		FVector2f GetVertexTexCoord(uint32 VertexIndex, uint32 Layer) const { return TexCoords[(VertexIndex * NumTexCoords) + Layer]; }
		FColor GetVertexColor(uint32 VertexIndex) const { return Colors[VertexIndex]; }

		bool HasVertexPositions() const { return Positions.Num() > 0; }
		bool HasVertexTangentBasis() const { return TangentBasis.Num() > 0; }
		bool HasVertexTexCoords() const { return TexCoords.Num() > 0; }
		bool HasVertexColors() const { return Colors.Num() > 0; }

	private:
		uint32							NumVertices = 0;
		uint32							NumTriangles = 0;
		uint32							NumTexCoords = 0;

		TConstArrayView<uint32>			Indicies;

		TConstArrayView<FVector3f>		Positions;
		TConstArrayView<FUintVector2>	TangentBasis;
		TConstArrayView<FVector2f>		TexCoords;
		TConstArrayView<FColor>			Colors;
	};


	class SSimCacheView : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SSimCacheView)
		{}
		SLATE_END_ARGS()

		virtual ~SSimCacheView() override
		{
			ViewModel->OnViewDataChanged().RemoveAll(this);
		}

		void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UNiagaraDataInterfaceDynamicMeshSimCacheData* InCacheData)
		{
			ViewModel = InViewModel;
			CacheData.Reset(InCacheData);

			ViewModel->OnViewDataChanged().AddSP(this, &SSimCacheView::RefreshContents);

			{
				FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

				FDetailsViewArgs DetailsViewArgs;
				DetailsViewArgs.bHideSelectionTip = true;
				DetailsViewArgs.bAllowSearch = false;

				FStructureDetailsViewArgs StructureViewArgs;
				StructureViewArgs.bShowObjects = true;
				StructureViewArgs.bShowAssets = true;
				StructureViewArgs.bShowClasses = true;
				StructureViewArgs.bShowInterfaces = true;

				StructureDetailsView = PropertyModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);
				StructureDetailsView->GetDetailsView()->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateLambda([](const FPropertyAndParent&) { return true; }));
			}

			RefreshContents();

			ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						MakeToolbar()
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2.0)
					[
						SNew(STextBlock)
						.Text(this, &SSimCacheView::GetMeshOverviewText)
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						StructureDetailsView->GetWidget().ToSharedRef()
					]
				]
			];
		}

		const FNDIDynamicMeshSimCacheFrame* GetCurrentMeshFrame() const
		{
			return CacheData->GetFrameData(ViewModel->GetFrameIndex(), CurrentSimTarget);
		}

		bool HasValidMeshData() const
		{
			return ViewingMeshFrameValid;
		}

		void RefreshContents(bool bForceRefesh = true)
		{
			if (const FNDIDynamicMeshSimCacheFrame* CacheFrame = GetCurrentMeshFrame())
			{
				ViewingMeshFrameValid = true;
				ViewingMeshFrame = *CacheFrame;
				ViewingMeshFrame.IndexData.Empty();
				ViewingMeshFrame.VertexData.Empty();

				TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FNDIDynamicMeshSimCacheFrame::StaticStruct(), reinterpret_cast<uint8*>(&ViewingMeshFrame));
				StructureDetailsView->SetStructureData(StructOnScope);
			}
			else
			{
				ViewingMeshFrameValid = false;
				StructureDetailsView->SetStructureData(nullptr);
			}
		}

		FText GetMeshOverviewText() const
		{
			if (const FNDIDynamicMeshSimCacheFrame* CacheFrame = GetCurrentMeshFrame())
			{
				return FText::Format(
					LOCTEXT("MeshDataOverviewFmt", "Mesh has {0} Vertices, {1} Triangles and is approximately {2}kb."),
					FText::AsNumber(CacheFrame->NumVertices),
					FText::AsNumber(CacheFrame->NumTriangles),
					FText::AsNumber(FMath::DivideAndRoundUp(CacheFrame->VertexData.Num()  + CacheFrame->IndexData.Num(), 1024))
				);
			}
			return LOCTEXT("NoMeshData", "No mesh data is stored for this simulation type on this frame.");
		}

		TSharedRef<SWidget> MakeToolbar()
		{
			FSlimHorizontalToolBarBuilder ToolbarBuilder(MakeShareable(new FUICommandList), FMultiBoxCustomization::None);

			ToolbarBuilder.BeginSection("SimMode");
			{
				UEnum* SimTargetEnum = StaticEnum<ENiagaraSimTarget>();
				for (ENiagaraSimTarget SimTarget : {ENiagaraSimTarget::CPUSim, ENiagaraSimTarget::GPUComputeSim})
				{
					const FName IconName(SimTarget == ENiagaraSimTarget::CPUSim ? "NiagaraEditor.Stack.CPUIcon" : "NiagaraEditor.Stack.GPUIcon");

					ToolbarBuilder.AddToolBarButton(
						FUIAction(
							FExecuteAction::CreateLambda([this, SimTarget]() { CurrentSimTarget = SimTarget; RefreshContents(); }),
							FCanExecuteAction(),
							FIsActionChecked::CreateLambda([this, SimTarget]() { return CurrentSimTarget == SimTarget; })
						),
						NAME_None,
						SimTargetEnum->GetDisplayNameTextByValue(int(SimTarget)),
						SimTargetEnum->GetToolTipTextByIndex(SimTargetEnum->GetIndexByValue(int(SimTarget))),
						FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), IconName),
						EUserInterfaceActionType::ToggleButton
					);
				}
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("Utils");
			{
				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateSP(this, &SSimCacheView::CopyTrianglesToClipboard),
						FCanExecuteAction::CreateSP(this, &SSimCacheView::HasValidMeshData)
					),
					NAME_None,
					LOCTEXT("TrianglesToClipboard", "Triangles to Clipboard"),
					LOCTEXT("TrianglesToClipboardTooltip", "Copies the mesh triangles indicies into the clipboard in CSV format"),
					FSlateIcon()
				);
				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateSP(this, &SSimCacheView::CopyVerticesToClipboard),
						FCanExecuteAction::CreateSP(this, &SSimCacheView::HasValidMeshData)
					),
					NAME_None,
					LOCTEXT("VerticesToClipboard", "Vertices to Clipboard"),
					LOCTEXT("VerticesToClipboardTooltip", "Copies the mesh vertices into the clipboard in CSV format"),
					FSlateIcon()
				);
				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateSP(this, &SSimCacheView::ExportAsStaticMesh),
						FCanExecuteAction::CreateSP(this, &SSimCacheView::HasValidMeshData)
					),
					NAME_None,
					LOCTEXT("CreateStaticMesh", "Create Static Mesh"),
					LOCTEXT("CreateStaticMeshTooltip", "Creates a Static Mesh asset from the current mesh data"),
					FSlateIcon()
				);
			}
			ToolbarBuilder.EndSection();

			return ToolbarBuilder.MakeWidget();
		}

		void CopyTrianglesToClipboard()
		{
			FString ClipboardString;
			ClipboardString.Append(TEXT("Section,Triangle,Index0,Index1,Index2\n"));
			if (const FNDIDynamicMeshSimCacheFrame* CacheFrame = GetCurrentMeshFrame())
			{
				TConstArrayView<uint32> Indices = MakeArrayView((uint32*)CacheFrame->IndexData.GetData(), CacheFrame->IndexData.Num() / sizeof(uint32));
				for (int32 iSection = 0; iSection < CacheFrame->Sections.Num(); ++iSection)
				{
					const FNDIDynamicMeshSimCacheSection& Section = CacheFrame->Sections[iSection];
					for (uint32 iTriangle=0; iTriangle < Section.AllocatedTriangles; ++iTriangle)
					{
						const uint32 IndexOffset = (Section.TriangleOffset + iTriangle) * 3;
						ClipboardString.Appendf(TEXT("%u,%u,%u,%u,%u\n"), iSection, iTriangle, Indices[IndexOffset + 0], Indices[IndexOffset + 1], Indices[IndexOffset + 2]);
					}
				}
			}
			FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);
		}

		void CopyVerticesToClipboard()
		{
			const FNDIDynamicMeshSimCacheFrame* CacheFrame = GetCurrentMeshFrame();
			if (CacheFrame == nullptr)
			{
				return;
			}

			const FMeshReadHelper MeshReadHelper(CacheFrame);

			// Build used or not information
			TBitArray<TInlineAllocator<1>> VertexUsed;
			{
				VertexUsed.Add(false, CacheFrame->NumVertices);

				for (int32 iSection = 0; iSection < CacheFrame->Sections.Num(); ++iSection)
				{
					const FNDIDynamicMeshSimCacheSection& Section = CacheFrame->Sections[iSection];
					for (uint32 iTriangle = 0; iTriangle < Section.AllocatedTriangles; ++iTriangle)
					{
						const FUintVector3 Indices = MeshReadHelper.GetTriangleIndices(Section.TriangleOffset + iTriangle);
						VertexUsed[Indices.X] = true;
						VertexUsed[Indices.Y] = true;
						VertexUsed[Indices.Z] = true;
					}
				}
			}

			// Build Header String
			FString ClipboardString;
			ClipboardString.Append(TEXT("Vertex,"));
			if (MeshReadHelper.HasVertexPositions())
			{
				ClipboardString.Append(TEXT("Position.X,Position.Y,Position.Z,"));
			}
			if (MeshReadHelper.HasVertexTangentBasis())
			{
				ClipboardString.Append(TEXT("TangentX.X,TangentX.Y,TangentX.Z,"));
				ClipboardString.Append(TEXT("TangentY.X,TangentY.Y,TangentY.Z,"));
				ClipboardString.Append(TEXT("TangentZ.X,TangentZ.Y,TangentZ.Z,"));
			}
			if (MeshReadHelper.HasVertexTexCoords())
			{
				for (uint32 iTexCoord = 0; iTexCoord < MeshReadHelper.GetNumTexCoords(); ++iTexCoord)
				{
					ClipboardString.Appendf(TEXT("TexCoord[%d].X,TexCoord[%d].Y,"), iTexCoord, iTexCoord);
				}
			}
			if (MeshReadHelper.HasVertexColors())
			{
				ClipboardString.Append(TEXT("Color.R,Color.G,Color.B,Color.A,"));
			}

			ClipboardString[ClipboardString.Len() - 1] = '\n';

			// Build Vertices
			for (uint32 iVertex=0; iVertex < MeshReadHelper.GetNumVertices(); ++iVertex)
			{
				if (VertexUsed[iVertex] == false)
				{
					continue;
				}

				ClipboardString.Appendf(TEXT("%u,"), iVertex);
				if (MeshReadHelper.HasVertexPositions())
				{
					const FVector3f Position = MeshReadHelper.GetVertexPosition(iVertex);
					ClipboardString.Appendf(TEXT("%f,%f,%f,"), Position.X, Position.Y, Position.Z);
				}
				if (MeshReadHelper.HasVertexTangentBasis())
				{
					const FTangentBasis Tangents = MeshReadHelper.GeVertexTangents(iVertex);
					ClipboardString.Appendf(TEXT("%f,%f,%f,"), Tangents.XAxis.X, Tangents.XAxis.Y, Tangents.XAxis.Z);
					ClipboardString.Appendf(TEXT("%f,%f,%f,"), Tangents.YAxis.X, Tangents.YAxis.Y, Tangents.YAxis.Z);
					ClipboardString.Appendf(TEXT("%f,%f,%f,"), Tangents.ZAxis.X, Tangents.ZAxis.Y, Tangents.ZAxis.Z);
				}
				if (MeshReadHelper.HasVertexTexCoords())
				{
					for (uint32 iTexCoord = 0; iTexCoord < MeshReadHelper.GetNumTexCoords(); ++iTexCoord)
					{
						const FVector2f TexCoord = MeshReadHelper.GetVertexTexCoord(iVertex, iTexCoord);
						ClipboardString.Appendf(TEXT("%f,%f,"), TexCoord.X, TexCoord.Y);
					}
				}
				if (MeshReadHelper.HasVertexColors())
				{
					const FColor Color = MeshReadHelper.GetVertexColor(iVertex);
					ClipboardString.Appendf(TEXT("%d,%d,%d,%d,"), Color.R, Color.G, Color.B, Color.A);
				}
				ClipboardString[ClipboardString.Len() - 1] = '\n';
			}

			FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);
		}

		void ExportAsStaticMesh()
		{
			const FNDIDynamicMeshSimCacheFrame* CacheFrame = GetCurrentMeshFrame();
			if (CacheFrame == nullptr)
			{
				return;
			}

			const FMeshReadHelper MeshReadHelper(CacheFrame);

		//	const uint32 NumTexCoords = ReadbackResult.VertexTexCoordNum;
		//	const uint32 NumTriangles = ReadbackResult.NumVertices / 3;

			// Create Mesh Description
			FMeshDescription MeshDescription;
			TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();
			TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector3f>(MeshAttribute::VertexInstance::Normal, 1, FVector3f::ZeroVector, EMeshAttributeFlags::Mandatory);
			TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector3f>(MeshAttribute::VertexInstance::Tangent, 1, FVector3f::ZeroVector, EMeshAttributeFlags::Mandatory);
			TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = MeshDescription.VertexInstanceAttributes().RegisterAttribute<float>(MeshAttribute::VertexInstance::BinormalSign, 1, 1.0f, EMeshAttributeFlags::Mandatory);
			TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector4f>(MeshAttribute::VertexInstance::Color, 1, FVector4f(1.0f, 1.0f, 1.0f, 1.0f), EMeshAttributeFlags::Lerpable | EMeshAttributeFlags::Mandatory);
			TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate, MeshReadHelper.GetNumTexCoords(), FVector2f::ZeroVector, EMeshAttributeFlags::Lerpable | EMeshAttributeFlags::Mandatory);

			MeshDescription.EdgeAttributes().RegisterAttribute<bool>(MeshAttribute::Edge::IsHard, 1, false, EMeshAttributeFlags::Mandatory);
			TPolygonGroupAttributesRef<FName> PolygonGroupSlotNames = MeshDescription.PolygonGroupAttributes().RegisterAttribute<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName, 1, NAME_None, EMeshAttributeFlags::Mandatory); //The unique key to match the mesh material slot

			// Reserve space
			MeshDescription.ReserveNewVertices(MeshReadHelper.GetNumVertices());
			MeshDescription.ReserveNewVertexInstances(MeshReadHelper.GetNumTriangles());
			MeshDescription.ReserveNewEdges(MeshReadHelper.GetNumTriangles());
			MeshDescription.ReserveNewPolygons(MeshReadHelper.GetNumTriangles());
			MeshDescription.ReserveNewPolygonGroups(CacheFrame->Sections.Num());

			// Build vertices
			TArray<FVertexInstanceID> VertexInstances;
			VertexInstances.Reserve(MeshReadHelper.GetNumVertices());
			for (uint32 iVertex=0; iVertex < MeshReadHelper.GetNumVertices(); ++iVertex)
			{
				const FVertexID VertexID = MeshDescription.CreateVertex();
				check(VertexID.GetValue() == iVertex);
				VertexPositions[VertexID] = MeshReadHelper.HasVertexPositions() ? MeshReadHelper.GetVertexPosition(iVertex) : FVector3f::ZeroVector;

				FVertexInstanceID VertexInstanceID = MeshDescription.CreateVertexInstance(VertexID);
				VertexInstances.Add(VertexInstanceID);

				const FTangentBasis TangentBasis = MeshReadHelper.HasVertexTangentBasis() ? MeshReadHelper.GeVertexTangents(iVertex) : FTangentBasis();
				const float TangentSign = FVector3f::DotProduct(FVector3f::CrossProduct(TangentBasis.XAxis, TangentBasis.ZAxis), TangentBasis.YAxis) < 0.0f ? -1.0f : 1.0f;
				VertexInstanceNormals.Set(VertexInstanceID, TangentBasis.XAxis);
				VertexInstanceTangents.Set(VertexInstanceID, TangentBasis.ZAxis);
				VertexInstanceBinormalSigns.Set(VertexInstanceID, TangentSign);

				VertexInstanceColors.Set(VertexInstanceID, MeshReadHelper.HasVertexColors() ? MeshReadHelper.GetVertexColor(iVertex).ReinterpretAsLinear() : FLinearColor::White);
				for (uint32 iTexCoord=0; iTexCoord < MeshReadHelper.GetNumTexCoords(); ++iTexCoord)
				{
					VertexInstanceUVs.Set(VertexInstanceID, iTexCoord, MeshReadHelper.GetVertexTexCoord(iVertex, iTexCoord));
				}
			}

			// Build sections / triangles
			TArray<FStaticMaterial> StaticMaterials;
			for (int32 iSection=0; iSection < CacheFrame->Sections.Num(); ++iSection)
			{
				const FNDIDynamicMeshSimCacheSection& Section = CacheFrame->Sections[iSection];

				const FName MaterialSlotName(*FString::Printf(TEXT("Section%d"), iSection));
				const FPolygonGroupID PolyGroupId = MeshDescription.CreatePolygonGroup();
				PolygonGroupSlotNames[PolyGroupId] = MaterialSlotName;

				FStaticMaterial& StaticMaterial			= StaticMaterials.AddDefaulted_GetRef();
				StaticMaterial.MaterialInterface		= Section.Material;
				StaticMaterial.MaterialSlotName			= MaterialSlotName;
				StaticMaterial.ImportedMaterialSlotName	= MaterialSlotName;

				for (uint32 iTriangle=0; iTriangle < Section.AllocatedTriangles; ++iTriangle)
				{
					const FUintVector3 Indices = MeshReadHelper.GetTriangleIndices(Section.TriangleOffset + iTriangle);
					const FPolygonID PolygonID = MeshDescription.CreatePolygon(PolyGroupId, { Indices.X, Indices.Y, Indices.Z });
					MeshDescription.ComputePolygonTriangulation(PolygonID);
				}
			}

			UStaticMesh* StaticMesh = NewObject<UStaticMesh>();
			StaticMesh->SetNumSourceModels(1);
			{
				FMeshBuildSettings& MeshBuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
				MeshBuildSettings.bRecomputeNormals = false;
				MeshBuildSettings.bRecomputeTangents = false;
			}

			StaticMesh->SetStaticMaterials(StaticMaterials);

			// Build Mesh
			UStaticMesh::FBuildMeshDescriptionsParams Params;
			Params.bFastBuild = WITH_EDITOR ? false : true;
			Params.bUseHashAsGuid = true;
			Params.bMarkPackageDirty = false;
			Params.bCommitMeshDescription = true;
			Params.bAllowCpuAccess = false;

			StaticMesh->BuildFromMeshDescriptions({ &MeshDescription }, Params);

			StaticMesh->SetFlags(RF_Public | RF_Standalone);

			TArray<UObject*> AssetsToSave = { StaticMesh };
			TArray<UObject*> SavedAssets;
			FEditorFileUtils::SaveAssetsAs(AssetsToSave, SavedAssets);

			if (SavedAssets.Num() > 0)
			{
				UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				AssetEditorSubsystem->OpenEditorForAssets_Advanced(SavedAssets, EToolkitMode::Standalone);
			}

			StaticMesh->ClearFlags(RF_Public | RF_Standalone);
		}

	private:
		TSharedPtr<FNiagaraSimCacheViewModel>									ViewModel;
		TStrongObjectPtr<const UNiagaraDataInterfaceDynamicMeshSimCacheData>	CacheData;

		bool																	ViewingMeshFrameValid = false;
		FNDIDynamicMeshSimCacheFrame											ViewingMeshFrame;
		TSharedPtr<IStructureDetailsView>										StructureDetailsView;

		ENiagaraSimTarget														CurrentSimTarget = ENiagaraSimTarget::CPUSim;
	};
} // NDIDynamicMeshSimCacheVisualizer

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> FNiagaraDynamicMeshSimCacheVisualizer::CreateWidgetFor(const UObject* InCachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel)
{
	using namespace NDIDynamicMeshSimCacheVisualizer;

	const UNiagaraDataInterfaceDynamicMeshSimCacheData* CachedData = CastChecked<const UNiagaraDataInterfaceDynamicMeshSimCacheData>(InCachedData);
	return SNew(SSimCacheView, ViewModel, CachedData);
}

#undef LOCTEXT_NAMESPACE
