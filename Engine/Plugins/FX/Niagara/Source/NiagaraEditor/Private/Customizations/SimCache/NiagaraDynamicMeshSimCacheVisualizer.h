// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "Customizations/NiagaraDataInterfaceSimCacheVisualizer.h"

// Visualizer for dynamic mesh sim cache data
class FNiagaraDynamicMeshSimCacheVisualizer : public INiagaraDataInterfaceSimCacheVisualizer
{
public:
	virtual ~FNiagaraDynamicMeshSimCacheVisualizer() override = default;
	
	virtual TSharedPtr<SWidget> CreateWidgetFor(const UObject* CachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel) override;
};
