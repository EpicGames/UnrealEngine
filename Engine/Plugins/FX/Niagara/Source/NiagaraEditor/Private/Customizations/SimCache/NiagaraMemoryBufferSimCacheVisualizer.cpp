// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMemoryBufferSimCacheVisualizer.h"

#include "DataInterface/NiagaraDataInterfaceMemoryBuffer.h"

#include "UObject/StrongObjectPtr.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailLayoutBuilder.h"
#include "SEnumCombo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraMemoryBufferSimCacheVisualizer)

#define LOCTEXT_NAMESPACE "NiagaraMemoryBufferSimCacheVisualizer"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace NDIMemoryBufferSimCacheVisualizerLocal
{
static const FName NAME_ElementOffset = FName("_ElementOffset");

template<typename TType>
TConstArrayView<TType> GetCurrentData(const TSharedPtr<FNiagaraSimCacheViewModel>& ViewModel, const UNDIMemoryBufferSimCacheData* CacheData, ENiagaraSimTarget SimTarget)
{
	const int32 FrameIndex = ViewModel->GetFrameIndex();
	if (CacheData->FrameData.IsValidIndex(FrameIndex))
	{
		if (SimTarget == ENiagaraSimTarget::CPUSim)
		{
			return MakeArrayView(
				reinterpret_cast<const TType*>(CacheData->BufferData.GetData() + CacheData->FrameData[FrameIndex].CpuDataOffset),
				CacheData->FrameData[FrameIndex].CpuBufferSize
			);
		}
		else
		{
			return MakeArrayView(
				reinterpret_cast<const TType*>(CacheData->BufferData.GetData() + CacheData->FrameData[FrameIndex].GpuDataOffset),
				CacheData->FrameData[FrameIndex].GpuBufferSize
			);
		}
	}
	return MakeArrayView<const TType>(nullptr, 0);
}

}// namespace NDIMemoryBufferSimCacheVisualizerLocal

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace NDIMemoryBufferSimCacheVisualizer
{
class SVisualizerRowWidget : public SMultiColumnTableRow<TSharedPtr<int32>>
{
public:
	SLATE_BEGIN_ARGS(SVisualizerRowWidget) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheViewModel>,	ViewModel)
		SLATE_ARGUMENT(const UNDIMemoryBufferSimCacheData*,		CacheData)
		SLATE_ARGUMENT(TSharedPtr<int32>,						RowIndexPtr)
		SLATE_ATTRIBUTE(ENiagaraSimTarget,						DisplaySimTarget)
		SLATE_ATTRIBUTE(ENDIMemoryBufferViewType,				DisplayAsType)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		ViewModel			= InArgs._ViewModel;
		CacheData.Reset(InArgs._CacheData);
		RowIndexPtr			= InArgs._RowIndexPtr;
		DisplaySimTarget	= InArgs._DisplaySimTarget;
		DisplayAsType		= InArgs._DisplayAsType;

		SMultiColumnTableRow::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		using namespace NDIMemoryBufferSimCacheVisualizerLocal;
		if (InColumnName == NAME_ElementOffset)
		{
			return SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("ElementFormat", "Element {0}"), FText::AsNumber(*RowIndexPtr)));
		}

		const int32 Offset = *RowIndexPtr + InColumnName.GetNumber();

		FText ItemText;
		switch (DisplayAsType.Get(ENDIMemoryBufferViewType::Integer))
		{
			case ENDIMemoryBufferViewType::Float:
			{
				TConstArrayView<float> BufferData = GetCurrentData<float>(ViewModel, CacheData.Get(), DisplaySimTarget.Get(ENiagaraSimTarget::CPUSim));
				if (BufferData.IsValidIndex(Offset))
				{
					const float Value = BufferData[Offset];
					if (FMath::IsFinite(Value))
					{
						FNumberFormattingOptions FormatOptions;
						FormatOptions.SetMinimumFractionalDigits(2);

						ItemText = FText::AsNumber(Value, &FormatOptions);
					}
					else
					{
						ItemText = LOCTEXT("InvalidFloat", "Invalid Float");
					}
				}
				break;
			}
			case ENDIMemoryBufferViewType::Integer:
			{
				TConstArrayView<int32> BufferData = GetCurrentData<int32>(ViewModel, CacheData.Get(), DisplaySimTarget.Get(ENiagaraSimTarget::CPUSim));
				if (BufferData.IsValidIndex(Offset))
				{
					ItemText = FText::AsNumber(BufferData[Offset]);
				}
				break;
			}
			case ENDIMemoryBufferViewType::UnsignedInteger:
			{
				TConstArrayView<uint32> BufferData = GetCurrentData<uint32>(ViewModel, CacheData.Get(), DisplaySimTarget.Get(ENiagaraSimTarget::CPUSim));
				if (BufferData.IsValidIndex(Offset))
				{
					ItemText = FText::AsNumber(BufferData[Offset]);
				}
				break;
			}
			case ENDIMemoryBufferViewType::Hex:
			{
				TConstArrayView<uint32> BufferData = GetCurrentData<uint32>(ViewModel, CacheData.Get(), DisplaySimTarget.Get(ENiagaraSimTarget::CPUSim));
				if (BufferData.IsValidIndex(Offset))
				{
					ItemText = FText::FromString(FString::Printf(TEXT("%08x"), BufferData[Offset]));
				}
				break;
			}
			default:
				break;
		}

		if (!ItemText.IsEmpty())
		{
			return SNew(STextBlock)
				.Text(ItemText);
		}

		return SNullWidget::NullWidget;
	}

	TSharedPtr<FNiagaraSimCacheViewModel>					ViewModel;
	TStrongObjectPtr<const UNDIMemoryBufferSimCacheData>	CacheData;
	TAttribute<ENiagaraSimTarget>							DisplaySimTarget = ENiagaraSimTarget::CPUSim;
	TAttribute<ENDIMemoryBufferViewType>					DisplayAsType = ENDIMemoryBufferViewType::Integer;
	TSharedPtr<int32>										RowIndexPtr;
};

class SSimCacheView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSimCacheView)
	{}
	SLATE_END_ARGS()

	virtual ~SSimCacheView() override
	{
		ViewModel->OnViewDataChanged().RemoveAll(this);
	}

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UNDIMemoryBufferSimCacheData* InCacheData)
	{
		ViewModel = InViewModel;
		CacheData.Reset(InCacheData);

		ViewModel->OnViewDataChanged().AddSP(this, &SSimCacheView::UpdateRows);

		const UFNiagaraMemoryBufferSimCacheVisualizerSettings* Settings = GetDefault<UFNiagaraMemoryBufferSimCacheVisualizerSettings>();
		DisplayColumns		= Settings->DisplayColumns;
		DisplaySimTarget	= Settings->DisplaySimTarget;
		DisplayAsType		= Settings->DisplayAsType;

		HeaderRowWidget = SNew(SHeaderRow);
		UpdateColumns();

		SAssignNew(ListViewWidget, SListView<TSharedPtr<int32>>)
			.ListItemsSource(&RowItems)
			.OnGenerateRow(this, &SSimCacheView::MakeRowWidget)
			.Visibility(EVisibility::Visible)
			.SelectionMode(ESelectionMode::Single)
			.ConsumeMouseWheel(EConsumeMouseWheel::Never)
			.ScrollbarVisibility(EVisibility::Collapsed)
			.HeaderRow(HeaderRowWidget);

		ChildSlot
		[
			SNew(SVerticalBox)
			// Display Columns
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DisplayColumns", "Display Columns: "))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpinBox<int32>)
					.MinDesiredWidth(50.0f)
					.MinValue(1)
					.MinSliderValue(1)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Value(this, &SSimCacheView::GetDisplayColumns)
					.OnValueChanged(this, &SSimCacheView::SetDisplayColumns)
				]
			]
			// Display Type
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DisplayType", "Display Type: "))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SEnumComboBox, StaticEnum<ENDIMemoryBufferViewType>())
					.CurrentValue(this, &SSimCacheView::GetDisplayAsTypeInt)
					.OnEnumSelectionChanged(this, &SSimCacheView::SetDisplayAsType)
				]
			]
			// Display Target
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DisplaySimTarget", "Display SimTarget: "))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SEnumComboBox, StaticEnum<ENiagaraSimTarget>())
					.CurrentValue(this, &SSimCacheView::GetDisplaySimTargetAsInt)
					.OnEnumSelectionChanged(this, &SSimCacheView::SetDisplaySimTarget)
				]
			]
			// Number of buffer elements
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BufferNumElements", "Buffer Num Elements: "))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(this, &SSimCacheView::GetBufferNumElementsText)
				]
			]
			// Buffer Data View
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.0f, 8.0f, 0.0f, 0.0f)
			[
				ListViewWidget.ToSharedRef()
			]
		];
	}

	TSharedRef<ITableRow> MakeRowWidget(const TSharedPtr<int32> RowIndexPtr, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		return
			SNew(SVisualizerRowWidget, OwnerTable)
				.ViewModel(ViewModel)
				.CacheData(CacheData.Get())
				.RowIndexPtr(RowIndexPtr)
				.DisplaySimTarget(this, &SSimCacheView::GetDisplaySimTarget)
				.DisplayAsType(this, &SSimCacheView::GetDisplayAsType)
				;
	}

	void UpdateColumns(bool bRefresh = true)
	{
		using namespace NDIMemoryBufferSimCacheVisualizerLocal;
		HeaderRowWidget->ClearColumns();
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(NAME_ElementOffset)
			.DefaultLabel(LOCTEXT("ElementOffset", "Element Offset"))
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Fill)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			.ManualWidth(100)
			.SortMode(EColumnSortMode::None)
		);

		for ( int32 i=0; i < DisplayColumns; ++i )
		{
			FName ColumnName(*FString::Printf(TEXT("%d"), i), i);

			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(ColumnName)
				.DefaultLabel(FText::AsNumber(i))
				.HAlignHeader(HAlign_Center)
				.VAlignHeader(VAlign_Fill)
				.HAlignCell(HAlign_Center)
				.VAlignCell(VAlign_Center)
				.ManualWidth(100)
				.SortMode(EColumnSortMode::None)
			);
		}
		if (ListViewWidget && bRefresh)
		{
			ListViewWidget->RequestListRefresh();
		}
	}

	void UpdateRows(bool bRefresh = true)
	{
		using namespace NDIMemoryBufferSimCacheVisualizerLocal;
		TConstArrayView<uint32> BufferData = GetCurrentData<uint32>(ViewModel, CacheData.Get(), DisplaySimTarget);
		const int32 NumRows = FMath::DivideAndRoundUp(BufferData.Num(), DisplayColumns);
		RowItems.Reset(NumRows);
		for ( int32 i=0; i < NumRows; ++i )
		{
			RowItems.Emplace(MakeShared<int32>(i * DisplayColumns));
		}
		if (ListViewWidget && bRefresh)
		{
			ListViewWidget->RequestListRefresh();
		}
	}

	FText GetBufferNumElementsText() const
	{
		using namespace NDIMemoryBufferSimCacheVisualizerLocal;
		TConstArrayView<uint32> BufferData = GetCurrentData<uint32>(ViewModel, CacheData.Get(), DisplaySimTarget);
		return FText::AsNumber(BufferData.Num());
	}

	int32 GetDisplayColumns() const { return DisplayColumns; }

	ENiagaraSimTarget GetDisplaySimTarget() const { return DisplaySimTarget; }
	int32 GetDisplaySimTargetAsInt() const { return (int32)DisplaySimTarget; }

	ENDIMemoryBufferViewType GetDisplayAsType() const { return DisplayAsType; }
	int32 GetDisplayAsTypeInt() const { return (int32)DisplayAsType; }

	void SetDisplayColumns(int32 InDisplayColumns)
	{
		InDisplayColumns = FMath::Max(InDisplayColumns, 1);
		if (DisplayColumns != InDisplayColumns)
		{
			DisplayColumns = InDisplayColumns;
			UpdateColumns(false);
			UpdateRows();

			UFNiagaraMemoryBufferSimCacheVisualizerSettings* Settings = GetMutableDefault<UFNiagaraMemoryBufferSimCacheVisualizerSettings>();
			Settings->DisplayColumns = DisplayColumns;
			Settings->SaveConfig();
		}
	}

	void SetDisplaySimTarget(int32 InNewValue, ESelectInfo::Type InInfo)
	{
		const ENiagaraSimTarget NewValue = (ENiagaraSimTarget)InNewValue;
		if (DisplaySimTarget != NewValue)
		{
			DisplaySimTarget = NewValue;
			UpdateRows();

			UFNiagaraMemoryBufferSimCacheVisualizerSettings* Settings = GetMutableDefault<UFNiagaraMemoryBufferSimCacheVisualizerSettings>();
			Settings->DisplaySimTarget = DisplaySimTarget;
			Settings->SaveConfig();
		}
	}

	void SetDisplayAsType(int32 InNewValue, ESelectInfo::Type InInfo)
	{
		const ENDIMemoryBufferViewType NewValue = (ENDIMemoryBufferViewType)InNewValue;
		if (DisplayAsType != NewValue)
		{
			DisplayAsType = NewValue;
			UpdateRows();

			UFNiagaraMemoryBufferSimCacheVisualizerSettings* Settings = GetMutableDefault<UFNiagaraMemoryBufferSimCacheVisualizerSettings>();
			Settings->DisplayAsType = DisplayAsType;
			Settings->SaveConfig();
		}
	}

private:
	TSharedPtr<FNiagaraSimCacheViewModel>					ViewModel;
	TStrongObjectPtr<const UNDIMemoryBufferSimCacheData>	CacheData;

	int32											DisplayColumns = 1;
	ENiagaraSimTarget								DisplaySimTarget = ENiagaraSimTarget::CPUSim;
	ENDIMemoryBufferViewType						DisplayAsType = ENDIMemoryBufferViewType::Integer;

	TSharedPtr<SHeaderRow>							HeaderRowWidget;
	TSharedPtr<SListView<TSharedPtr<int32>>>		ListViewWidget;
	TArray<TSharedPtr<int32>>						RowItems;
};

} // NDIMemoryBufferSimCacheVisualizer

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> FNiagaraMemoryBufferSimCacheVisualizer::CreateWidgetFor(const UObject* InCachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel)
{
	using namespace NDIMemoryBufferSimCacheVisualizer;

	if (const UNDIMemoryBufferSimCacheData* CachedData = Cast<const UNDIMemoryBufferSimCacheData>(InCachedData))
	{
		return SNew(SSimCacheView, ViewModel, CachedData);
	}
	return TSharedPtr<SWidget>();
}

#undef LOCTEXT_NAMESPACE
