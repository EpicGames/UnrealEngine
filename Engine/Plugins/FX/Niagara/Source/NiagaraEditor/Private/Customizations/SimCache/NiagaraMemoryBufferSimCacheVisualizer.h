// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "Customizations/NiagaraDataInterfaceSimCacheVisualizer.h"

#include "NiagaraMemoryBufferSimCacheVisualizer.generated.h"

UENUM()
enum class ENDIMemoryBufferViewType
{
	Float,
	Integer,
	UnsignedInteger,
	Hex,
};

UCLASS(config=EditorPerProjectUserSettings, defaultconfig)
class UFNiagaraMemoryBufferSimCacheVisualizerSettings : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Config, EditAnywhere, Category = "Settings")
	ENiagaraSimTarget DisplaySimTarget = ENiagaraSimTarget::CPUSim;

	UPROPERTY(Config, EditAnywhere, Category = "Settings")
	ENDIMemoryBufferViewType DisplayAsType = ENDIMemoryBufferViewType::Integer;

	UPROPERTY(Config, EditAnywhere, Category = "Settings")
	int32 DisplayColumns = 1;
};

/*
 * Provides a custom widget to show the memory buffer DI data in the sim cache in a table.
 */
class FNiagaraMemoryBufferSimCacheVisualizer : public INiagaraDataInterfaceSimCacheVisualizer
{
public:
	virtual ~FNiagaraMemoryBufferSimCacheVisualizer() override = default;
	
	virtual TSharedPtr<SWidget> CreateWidgetFor(const UObject* CachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel) override;
};
