// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRenderTargetSimCacheVisualizer.h"

#include "DataInterface/NDIRenderTargetSimCacheData.h"

#include "Engine/Canvas.h"
#include "Engine/Texture2D.h"
#include "UObject/StrongObjectPtr.h"
#include "Slate/SceneViewport.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "CanvasItem.h"
#include "DetailLayoutBuilder.h"
#include "EditorViewportClient.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SEnumCombo.h"
#include "SEditorViewport.h"
#include "TextureResource.h"

#define LOCTEXT_NAMESPACE "NiagaraRenderTargetSimCacheVisualizer"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace NDIRenderTargetSimCacheVisualizer
{
class FTextureViewportClient final : public FEditorViewportClient
{
public:
	FTextureViewportClient(const TSharedRef<SEditorViewport>& InOwnerViewport, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UNDIRenderTargetSimCacheData* InCacheData)
		: FEditorViewportClient(nullptr, nullptr, StaticCastSharedRef<SEditorViewport>(InOwnerViewport))
		, ViewModel(InViewModel)
		, CacheData(InCacheData)
	{
		ViewModel->OnViewDataChanged().AddRaw(this, &FTextureViewportClient::OnViewDataChanged);

		UpdateTexture();
	}

	virtual ~FTextureViewportClient()
	{
		ViewModel->OnViewDataChanged().RemoveAll(this);
	}

	virtual void Draw(FViewport* InViewport, FCanvas* Canvas) override
	{
		Canvas->Clear(FLinearColor::Transparent);

		const float DPIScaleFactor = ShouldDPIScaleSceneCanvas() ? GetDPIScale() : 1.0f;
		const FIntRect ViewRect(
			FMath::Floor(Canvas->GetViewRect().Min.X / DPIScaleFactor) + 2,
			FMath::Floor(Canvas->GetViewRect().Min.Y / DPIScaleFactor) + 2,
			FMath::Floor(Canvas->GetViewRect().Max.X / DPIScaleFactor) - 2,
			FMath::Floor(Canvas->GetViewRect().Max.Y / DPIScaleFactor) - 2
		);
		if (ViewRect.Width() <= 0 || ViewRect.Height() <= 0)
		{
			return;
		}

		const FVector2D HalfPixel(0.5f / float(Texture->GetSizeX()), 0.5f / float(Texture->GetSizeY()));
		FCanvasTileItem TileItem(
			FVector2D(0.0f, 0.0f),
			Texture->GetResource(),
			FVector2D(ViewRect.Width(), ViewRect.Height()),
			FVector2D(HalfPixel.X, HalfPixel.Y),
			FVector2D(1.0f - HalfPixel.X, 1.0f - HalfPixel.Y),
			FLinearColor::White
		);
		Canvas->DrawItem(TileItem);
	}

	void UpdateTexture()
	{
		const int32 FrameIndex = ViewModel->GetFrameIndex();
		const bool bIsValid = CacheData->HasPixelData(FrameIndex);
		const FIntVector TextureSize = bIsValid ? CacheData->GetTextureSize(FrameIndex) : FIntVector(1, 1, 1);
		const EPixelFormat TextureFormat = bIsValid ? CacheData->GetTextureFormat(FrameIndex) : EPixelFormat::PF_B8G8R8A8;

		if (Texture == nullptr || Texture->GetSizeX() != TextureSize.X || Texture->GetSizeY() != TextureSize.Y || Texture->GetPixelFormat() != TextureFormat )
		{
			Texture.Reset(UTexture2D::CreateTransient(TextureSize.X, TextureSize.Y, TextureFormat));
			Texture->MipGenSettings = TMGS_NoMipmaps;
			Texture->UpdateResource();
		}

		void* UntypedMipData = Texture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE);

		if (bIsValid)
		{
			//-TODO: This assumes single slice (i.e. 2D) when we use this viualizer for other types it will need updating.
			checkf(TextureSize.Z == 1, TEXT("Only one texture slice is supported right now"));
			TArray<uint8> Data = CacheData->GetPixelData(FrameIndex);
			FMemory::Memcpy(UntypedMipData, Data.GetData(), Data.Num());
		}
		else
		{
			FColor* TypedMipData = static_cast<FColor*>(UntypedMipData);
			TypedMipData[0] = FColor::Black;
		}

		Texture->GetPlatformData()->Mips[0].BulkData.Unlock();
		Texture->UpdateResource();

		bNeedsRedraw = true;
	}

	void OnViewDataChanged(bool)
	{
		UpdateTexture();
	}

	FVector2D GetTextureSize() const
	{
		return FVector2D(Texture->GetSizeX(), Texture->GetSizeY());
	}

private:
	TSharedPtr<FNiagaraSimCacheViewModel>					ViewModel;
	TStrongObjectPtr<const UNDIRenderTargetSimCacheData>	CacheData;
	TStrongObjectPtr<UTexture2D>							Texture;
};

class STextureViewport : public SEditorViewport
{
public:
	SLATE_BEGIN_ARGS(STextureViewport) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheViewModel>, ViewModel)
		SLATE_ARGUMENT(const UNDIRenderTargetSimCacheData*, CacheData)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ViewModel = InArgs._ViewModel;
		CacheData.Reset(InArgs._CacheData);

		SEditorViewport::Construct(
			SEditorViewport::FArguments()
			.ViewportSize(this, &STextureViewport::GetViewportSize)
		);
	}

	virtual ~STextureViewport() override
	{
	}

	TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override
	{
		ViewportClient = MakeShareable(new FTextureViewportClient(SharedThis(this), ViewModel, CacheData.Get()));
		return ViewportClient.ToSharedRef();
	}

	FVector2D GetViewportSize() const
	{
		return ViewportClient ? ViewportClient->GetTextureSize() * ZoomLevel : FVector2D(1.0f);
	}

	TOptional<float> GetZoomLevel() const
	{
		return TOptional<float>(ZoomLevel);
	}

	void SetZoomLevel(float InZoomLevel)
	{
		if (FMath::IsNearlyEqual(ZoomLevel, InZoomLevel))
		{
			return;
		}
		ViewportClient->RedrawRequested(nullptr);
		ZoomLevel = InZoomLevel;
	}

private:
	TSharedPtr<FNiagaraSimCacheViewModel>					ViewModel;
	TStrongObjectPtr<const UNDIRenderTargetSimCacheData>	CacheData;
	TSharedPtr<FTextureViewportClient>						ViewportClient;

	float	ZoomLevel = 1.0f;
};

class SSimCacheView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSimCacheView)
	{}
	SLATE_END_ARGS()

	virtual ~SSimCacheView() override
	{
		ViewModel->OnViewDataChanged().RemoveAll(this);
	}

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UNDIRenderTargetSimCacheData* InCacheData)
	{
		ViewModel = InViewModel;
		CacheData.Reset(InCacheData);

		TSharedRef<STextureViewport> Viewport =
			SNew(STextureViewport)
			.ViewModel(ViewModel)
			.CacheData(CacheData.Get());

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(10.0f)
			.AutoHeight()
			[
				SNew(SGridPanel)
				+SGridPanel::Slot(0, 0)
				.HAlign(HAlign_Right)
				.Padding(5, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TextureDetails", "Texture Details:"))
				]
				+SGridPanel::Slot(1, 0)
				.Padding(5, 0)
				[
					SNew(STextBlock)
					.Text(this, &SSimCacheView::GetTextureDetails)
				]
				+SGridPanel::Slot(0, 1)
				.HAlign(HAlign_Right)
				.Padding(5, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TextureMemory", "Texture Memory:"))
				]
				+SGridPanel::Slot(1, 1)
				.Padding(5, 0)
				[
					SNew(STextBlock)
					.Text(this, &SSimCacheView::GetTextureMemoryDetails)
				]
				+SGridPanel::Slot(0, 2)
				.HAlign(HAlign_Right)
				.Padding(5, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Zoom", "Zoom:"))
				]
				+SGridPanel::Slot(1, 2)
				.Padding(5, 0)
				[
					SNew(SNumericEntryBox<float>)
					.AllowSpin(true)
					.MinDesiredValueWidth(80)
					.MinSliderValue(0.25f)
					.MaxSliderValue(16.0f)
					.Delta(0.25f)
					.Value(Viewport, &STextureViewport::GetZoomLevel)
					.OnValueChanged(Viewport, &STextureViewport::SetZoomLevel)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					Viewport
				]
			]
		];
	}

	FText GetTextureDetails() const
	{
		const int32 FrameIndex = ViewModel->GetFrameIndex();
		const bool bIsValid = CacheData->IsValidFrame(FrameIndex);
		const FIntVector TextureSize = bIsValid ? CacheData->GetTextureSize(FrameIndex) : FIntVector(0,0,0);
		const EPixelFormat TextureFormat = bIsValid ? CacheData->GetTextureFormat(FrameIndex) : EPixelFormat::PF_Unknown;
		return FText::Format(
			LOCTEXT("TextureDetailsFormat", "{0}x{1}x{2} {3}"),
			TextureSize.X, TextureSize.Y, TextureSize.Z,
			FText::FromString(GPixelFormats[TextureFormat].Name)
		);
	}

	FText GetTextureMemoryDetails() const
	{
		const int32 FrameIndex = ViewModel->GetFrameIndex();
		const bool bIsValid = CacheData->IsValidFrame(FrameIndex);
		const int32 CompressedSize = CacheData->GetCompressedSize(FrameIndex);
		const int32 UncompressedSize = CacheData->GetUncompressedSize(FrameIndex);
		return FText::Format(
			LOCTEXT("TextureMemoryDetailsFormat", "{0}mb (Compressed) {1}mb (Decompressed)"),
			FText::AsNumber(float(CompressedSize) / 1024.0f / 1024.0f),
			FText::AsNumber(float(UncompressedSize) / 1024.0f / 1024.0f)
		);
	}

private:
	TSharedPtr<FNiagaraSimCacheViewModel>					ViewModel;
	TStrongObjectPtr<const UNDIRenderTargetSimCacheData>	CacheData;
};

} // NDIRenderTargetSimCacheVisualizer

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> FNiagaraRenderTargetSimCacheVisualizer::CreateWidgetFor(const UObject* InCachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel)
{
	using namespace NDIRenderTargetSimCacheVisualizer;

	if (const UNDIRenderTargetSimCacheData* CachedData = Cast<const UNDIRenderTargetSimCacheData>(InCachedData))
	{
		return SNew(SSimCacheView, ViewModel, CachedData);
	}
	return TSharedPtr<SWidget>();
}

#undef LOCTEXT_NAMESPACE
