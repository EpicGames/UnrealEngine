// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "Customizations/NiagaraDataInterfaceSimCacheVisualizer.h"

class FNiagaraRenderTargetSimCacheVisualizer : public INiagaraDataInterfaceSimCacheVisualizer
{
public:
	FNiagaraRenderTargetSimCacheVisualizer() {}
	virtual ~FNiagaraRenderTargetSimCacheVisualizer() override = default;
	
	virtual TSharedPtr<SWidget> CreateWidgetFor(const UObject* CachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel) override;
};
