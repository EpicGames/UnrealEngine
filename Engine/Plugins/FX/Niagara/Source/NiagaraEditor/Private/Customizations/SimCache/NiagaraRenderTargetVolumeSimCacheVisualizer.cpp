// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRenderTargetVolumeSimCacheVisualizer.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportCommands.h"
#include "EngineUtils.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SEditorViewport.h"
#include "UnrealEdGlobals.h"
#include "Components/HeterogeneousVolumeComponent.h"
#include "Components/PostProcessComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/TextureCube.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "UObject/StrongObjectPtr.h"
#include "FinalPostProcessSettings.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/Material.h"
#include "SceneView.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "SEnumCombo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraRenderTargetVolumeSimCacheVisualizer)

#define LOCTEXT_NAMESPACE "NiagaraVolumeTextureViewport"

class SNiagaraVolumeTextureViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SNiagaraVolumeTextureViewport) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UAnimatedSparseVolumeTexture* InVolumeTexture);
	virtual ~SNiagaraVolumeTextureViewport() override;

	void UpdatePreviewComponent(bool bReset = true);
	virtual void OnFocusViewportToSelection() override;

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	TStrongObjectPtr<UHeterogeneousVolumeComponent> PreviewComponent;

	void SetTemperatureMask(ENiagaraRenderTargetVolumeVisualizerMask NewMask);
	ENiagaraRenderTargetVolumeVisualizerMask TemparatureMask = ENiagaraRenderTargetVolumeVisualizerMask::G;

	void SetDensityScale(float NewValue);
	float DensityScale = 0.1f;
	
protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	virtual void BindCommands() override;

private:
	TSharedPtr<FAdvancedPreviewScene> AdvancedPreviewScene;
	TSharedPtr<class FNiagaraVolumeTextureViewportClient> ViewportClient;

	TSharedPtr<FNiagaraSimCacheViewModel> ViewModel;
	TStrongObjectPtr<UPostProcessComponent> PostProcessComponent;
	TStrongObjectPtr<const UAnimatedSparseVolumeTexture> VolumeTexture;
};

TSharedPtr<SWidget> FNiagaraRenderTargetVolumeSimCacheVisualizer::CreateWidgetFor(const UObject* CachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel)
{
	const UAnimatedSparseVolumeTexture* SparseVolumeTexture = Cast<const UAnimatedSparseVolumeTexture>(CachedData);
	if (SparseVolumeTexture == nullptr)
	{
		return TSharedPtr<SWidget>();
	}

	static UEnum* MaskEnum = StaticEnum<ENiagaraRenderTargetVolumeVisualizerMask>();
	TSharedPtr<SNiagaraVolumeTextureViewport> Viewport = SNew(SNiagaraVolumeTextureViewport, ViewModel, SparseVolumeTexture);
		
	return SNew(SVerticalBox)
	+SVerticalBox::Slot()
	.Padding(15)
	.AutoHeight()
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MaterialAttributes", "Material Attributes"))
		]
		+SVerticalBox::Slot()
		.Padding(10)
		.AutoHeight()
		[
			SNew(SGridPanel)
			+SGridPanel::Slot(0, 0)
			.Padding(5, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TemperatureMask_Label", "Temperature Mask"))
			]
			+SGridPanel::Slot(1, 0)
			.Padding(5, 0)
			[
				SNew(SEnumComboBox, MaskEnum)
				.ContentPadding(FMargin(2, 0))
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.CurrentValue_Lambda([Viewport]()
				{
					return static_cast<int32>(Viewport->TemparatureMask);
				})
				.OnEnumSelectionChanged_Lambda([Viewport](int32 NewValue, ESelectInfo::Type)
				{
					ENiagaraRenderTargetVolumeVisualizerMask Value = static_cast<ENiagaraRenderTargetVolumeVisualizerMask>(NewValue);
					Viewport->SetTemperatureMask(Value);
				})
			]
			+SGridPanel::Slot(0, 1)
			.Padding(5, 0)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Density_Label", "Density Scale"))
			]
			+SGridPanel::Slot(1, 1)
			.Padding(5, 0)
			[
				SNew(SNumericEntryBox<float>)
				.AllowSpin(true)
				.MinDesiredValueWidth(80)
				.MinSliderValue(0.0f)
				.MaxSliderValue(1.0f)
				.Delta(0.01f)
				.Value_Lambda([Viewport]()
				{
					return Viewport->DensityScale;
				})
				.OnValueChanged_Lambda([Viewport](float Val)
				{
					Viewport->SetDensityScale(Val);
				})
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("SaveSVT", "Save SVT"))
				.OnClicked_Lambda(
					[WeakSparseVolumeTexture =MakeWeakObjectPtr(SparseVolumeTexture)]()
					{
						if (const UObject* TextureObject = WeakSparseVolumeTexture.Get())
						{
							FString PackagePath = TEXT("/Game/SimCacheSVT");
							UPackage* Package = TextureObject->GetPackage();
							if (Package && Package != GetTransientPackage())
							{
								PackagePath = Package->GetPathName() / Package->GetName() + TEXT("_SVT");
							}

							IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
							AssetTools.DuplicateAssetWithDialogAndTitle(
								FString(FPathViews::GetCleanFilename(PackagePath)),
								FString(FPathViews::GetPath(PackagePath)),
								const_cast<UObject*>(TextureObject),
								LOCTEXT("SaveSVTAs", "Save SVT as...")
							);
						}
						return FReply::Handled();
					}
				)
			]
		]
	]
	+SVerticalBox::Slot()
	.AutoHeight()
	[
		Viewport.ToSharedRef()
	];
}

//////////////////////////////////////////////////////////////////////////

/** Viewport Client for the Niagara baseline viewport */
class FNiagaraVolumeTextureViewportClient : public FEditorViewportClient
{
public:
	FNiagaraVolumeTextureViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SNiagaraVolumeTextureViewport>& InNiagaraEditorViewport);

	// FEditorViewportClient interface
	virtual void Tick(float DeltaSeconds) override;
	virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE) override;
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override { return false; }
	virtual bool CanCycleWidgetMode() const override { return false; }

	virtual void SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport) override;

	TWeakPtr<SNiagaraVolumeTextureViewport> NiagaraViewportPtr;
	FAdvancedPreviewScene* AdvancedPreviewScene = nullptr;
};

FNiagaraVolumeTextureViewportClient::FNiagaraVolumeTextureViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SNiagaraVolumeTextureViewport>& InNiagaraEditorViewport)
	: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InNiagaraEditorViewport))
	, AdvancedPreviewScene(&InPreviewScene)
{
	NiagaraViewportPtr = InNiagaraEditorViewport;

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(80,80,80);
	DrawHelper.GridColorMajor = FColor(72,72,72);
	DrawHelper.GridColorMinor = FColor(64,64,64);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
	//ShowWidget(false);

	SetViewMode(VMI_Lit);
	
	EngineShowFlags.SetSnap(0);

	OverrideNearClipPlane(1.0f);

	//This seems to be needed to get the correct world time in the preview.
	FNiagaraVolumeTextureViewportClient::SetIsSimulateInEditorViewport(true);
}


void FNiagaraVolumeTextureViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	if (UWorld* World = PreviewScene->GetWorld())
	{
		if (!World->GetBegunPlay())
		{
			for (FActorIterator It(World); It; ++It)
			{
				It->DispatchBeginPlay();
			}
			World->SetBegunPlay(true);

			// Simulate behavior from GameEngine.cpp
			World->bWorldWasLoadedThisTick = false;
			World->bTriggerPostLoadMap = true;
			NiagaraViewportPtr.Pin()->OnFocusViewportToSelection();
		}

		// Tick the preview scene world.
		World->Tick(LEVELTICK_All, DeltaSeconds);
	}
}

FSceneView* FNiagaraVolumeTextureViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	FSceneView* SceneView = FEditorViewportClient::CalcSceneView(ViewFamily);
	FFinalPostProcessSettings::FCubemapEntry& CubemapEntry = *new(SceneView->FinalPostProcessSettings.ContributingCubemaps) FFinalPostProcessSettings::FCubemapEntry;
	CubemapEntry.AmbientCubemap = GUnrealEd->GetThumbnailManager()->AmbientCubemap;
	CubemapEntry.AmbientCubemapTintMulScaleValue = FLinearColor::White;
	return SceneView;
}

void FNiagaraVolumeTextureViewportClient::SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport)
{
	bIsSimulateInEditorViewport = bInIsSimulateInEditorViewport;
}

//////////////////////////////////////////////////////////////////////////

void SNiagaraVolumeTextureViewport::Construct(const FArguments&, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UAnimatedSparseVolumeTexture* InVolumeTexture)
{
	ViewModel = InViewModel;
	VolumeTexture.Reset(InVolumeTexture);

	PreviewComponent.Reset(NewObject<UHeterogeneousVolumeComponent>(GetTransientPackage(), NAME_None, RF_Transient));
	PostProcessComponent.Reset(NewObject<UPostProcessComponent>(GetTransientPackage(), NAME_None, RF_Transient));
				
	// create HV component and wire all the things
	UMaterialInterface* MaterialInterface = LoadObject<UMaterialInterface>(nullptr, TEXT("/Engine/EngineMaterials/SparseVolumeMaterial"));
	UMaterial *Mat = MaterialInterface->GetMaterial();
	UMaterial *DuplicateMat = DuplicateObject<UMaterial>(Mat, PreviewComponent.Get());

	FGuid ExprGuid;
	FGuid SwitchGuid;
	DuplicateMat->SetStaticComponentMaskParameterValueEditorOnly("Temperature Mask", false, true, false, false, ExprGuid);
	DuplicateMat->SetStaticSwitchParameterValueEditorOnly("Temperature (Attributes B)", false, SwitchGuid);
	DuplicateMat->SetScalarParameterValueEditorOnly("Density Scale", DensityScale);
	DuplicateMat->SetSparseVolumeTextureParameterValueEditorOnly("SparseVolumeTexture", const_cast<UAnimatedSparseVolumeTexture*>(InVolumeTexture));

	PreviewComponent->OverrideMaterials.Add(DuplicateMat);
	PreviewComponent->bIssueBlockingRequests = false;
	PreviewComponent->ConditionalPostLoad();

	PostProcessComponent->bUnbound = true;
	PostProcessComponent->bEnabled = true;
	PostProcessComponent->Settings.bOverride_BloomIntensity = 1;
	PostProcessComponent->Settings.BloomIntensity = 0.1f;

	AdvancedPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues().SetLightBrightness(35)));
	AdvancedPreviewScene->SetFloorVisibility(false);
	AdvancedPreviewScene->AddComponent(PreviewComponent.Get(), PreviewComponent->GetRelativeTransform());
	AdvancedPreviewScene->AddComponent(PostProcessComponent.Get(), PostProcessComponent->GetRelativeTransform(), true);

	SEditorViewport::FArguments ViewportArgs;
	ViewportArgs.ViewportSize(FVector2D(1024, 768));
	SEditorViewport::Construct(ViewportArgs);

	Client->EngineShowFlags.SetGrid(true);
	UpdatePreviewComponent();
	ViewModel->OnViewDataChanged().AddSP(this, &SNiagaraVolumeTextureViewport::UpdatePreviewComponent);
}

SNiagaraVolumeTextureViewport::~SNiagaraVolumeTextureViewport()
{
	ViewModel->OnViewDataChanged().RemoveAll(this);
	AdvancedPreviewScene->RemoveComponent(PreviewComponent.Get());
	AdvancedPreviewScene->RemoveComponent(PostProcessComponent.Get());
	if (ViewportClient.IsValid())
	{
		ViewportClient->Viewport = nullptr;
	}
}

void SNiagaraVolumeTextureViewport::UpdatePreviewComponent(bool)
{
	if (ViewModel.IsValid() &&  VolumeTexture.IsValid())
	{
		int32 FrameIndex = FMath::Clamp(ViewModel->GetFrameIndex(), 0, VolumeTexture->GetNumFrames() - 1);
		PreviewComponent->SetFrame(FrameIndex);
	}
}

TSharedRef<SEditorViewport> SNiagaraVolumeTextureViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SNiagaraVolumeTextureViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SNiagaraVolumeTextureViewport::OnFloatingButtonClicked()
{
}

void SNiagaraVolumeTextureViewport::SetTemperatureMask(ENiagaraRenderTargetVolumeVisualizerMask NewMask)
{
	if (PreviewComponent)
	{
		if (UMaterial* CurrentMat = Cast<UMaterial>(PreviewComponent->OverrideMaterials[0]))
		{
			UMaterial *DuplicateMat = DuplicateObject<UMaterial>(CurrentMat, PreviewComponent.Get());
			FGuid ExprGuid;
			DuplicateMat->SetStaticComponentMaskParameterValueEditorOnly("Temperature Mask",
				NewMask == ENiagaraRenderTargetVolumeVisualizerMask::R,
				NewMask == ENiagaraRenderTargetVolumeVisualizerMask::G,
				NewMask == ENiagaraRenderTargetVolumeVisualizerMask::B,
				NewMask == ENiagaraRenderTargetVolumeVisualizerMask::A,
				ExprGuid);
			PreviewComponent->OverrideMaterials[0] = DuplicateMat;
			PreviewComponent->ConditionalPostLoad();
			SetDensityScale(DensityScale);
			PreviewComponent->MarkRenderStateDirty();
			TemparatureMask = NewMask;
		}
	}
}

void SNiagaraVolumeTextureViewport::SetDensityScale(float NewValue)
{
	DensityScale = NewValue;
	if (PreviewComponent && PreviewComponent->MaterialInstanceDynamic)
	{
		PreviewComponent->MaterialInstanceDynamic->SetScalarParameterValue("Density Scale", NewValue);
	}
}

TSharedRef<FEditorViewportClient> SNiagaraVolumeTextureViewport::MakeEditorViewportClient()
{
	ViewportClient = MakeShareable( new FNiagaraVolumeTextureViewportClient(*AdvancedPreviewScene.Get(), SharedThis(this)) );

	ViewportClient->SetViewLocation( FVector::ZeroVector );
	ViewportClient->SetViewRotation( FRotator(0.0f, 90.0f, 0.0f) );
	ViewportClient->SetViewLocationForOrbiting( FVector::ZeroVector, 750.0f );
	ViewportClient->SetCameraSpeedSettings(FEditorViewportCameraSpeedSettings(0.33f));
	ViewportClient->bSetListenerPosition = false;
	ViewportClient->SetShowGrid();
	ViewportClient->ExposureSettings.bFixed = true;

	ViewportClient->SetRealtime(true);
	ViewportClient->SetGameView(false);
	ViewportClient->VisibilityDelegate.BindSP( this, &SNiagaraVolumeTextureViewport::IsVisible );

	return ViewportClient.ToSharedRef();
}

EVisibility SNiagaraVolumeTextureViewport::OnGetViewportContentVisibility() const
{
	EVisibility BaseVisibility = SEditorViewport::OnGetViewportContentVisibility();
	if (BaseVisibility != EVisibility::Visible)
	{
		return BaseVisibility;
	}
	return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SNiagaraVolumeTextureViewport::BindCommands()
{
	SEditorViewport::BindCommands();
	CommandList->UnmapAction(FEditorViewportCommands::Get().CycleTransformGizmos);
}

void SNiagaraVolumeTextureViewport::OnFocusViewportToSelection()
{
	if (PreviewComponent)
	{
		ViewportClient->FocusViewportOnBox(PreviewComponent->Bounds.GetBox(), true);
	}
}

#undef LOCTEXT_NAMESPACE
