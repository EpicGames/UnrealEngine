// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/NiagaraDataInterfaceSimCacheVisualizer.h"

#include "NiagaraRenderTargetVolumeSimCacheVisualizer.generated.h"

// Used in the visualizer panel to populate the temperature mask dropdown
UENUM()
enum class ENiagaraRenderTargetVolumeVisualizerMask
{
	R, G, B, A
};

class FNiagaraRenderTargetVolumeSimCacheVisualizer : public INiagaraDataInterfaceSimCacheVisualizer
{
public:
	virtual ~FNiagaraRenderTargetVolumeSimCacheVisualizer() override = default;
	
	virtual TSharedPtr<SWidget> CreateWidgetFor(const UObject* CachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel) override;
};
