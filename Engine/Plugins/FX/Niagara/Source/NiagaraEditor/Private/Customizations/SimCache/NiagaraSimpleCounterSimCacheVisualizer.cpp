// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimpleCounterSimCacheVisualizer.h"

#include "DataInterface/NiagaraDataInterfaceSimpleCounter.h"

#include "UObject/StrongObjectPtr.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailLayoutBuilder.h"
#include "SEnumCombo.h"

#define LOCTEXT_NAMESPACE "NiagaraSimpleCounterSimCacheVisualizer"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace NDISimpleCounterSimCacheVisualizer
{
class SSimCacheView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSimCacheView)
	{}
	SLATE_END_ARGS()

	virtual ~SSimCacheView() override
	{
		ViewModel->OnViewDataChanged().RemoveAll(this);
	}

	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraSimCacheViewModel> InViewModel, const UNDISimpleCounterSimCacheData* InCacheData)
	{
		ViewModel = InViewModel;
		CacheData.Reset(InCacheData);

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(this, &SSimCacheView::GetCpuInformation)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(this, &SSimCacheView::GetGpuInformation)
				]
			]
		];
	}

	FText GetValueText(int32 DataOffset) const
	{
		FText DataValue;
		if (CacheData->Values.IsValidIndex(DataOffset))
		{
			return FText::AsNumber(CacheData->Values[DataOffset]);
		}
		else
		{
			return LOCTEXT("DataInvalid", "Invalid");
		}
	}

	FText GetCpuInformation() const
	{
		const int32 FrameIndex = ViewModel->GetFrameIndex();
		const int32 DataOffset = FrameIndex * 2;
		return FText::Format(LOCTEXT("CpuDataFormat", "CPU Visibile Value = {0}"), GetValueText(DataOffset + 0));
	}

	FText GetGpuInformation() const
	{
		const int32 FrameIndex = ViewModel->GetFrameIndex();
		const int32 DataOffset = FrameIndex * 2;
		return FText::Format(LOCTEXT("GpuDataFormat", "GPU Visibile Value = {0}"), GetValueText(DataOffset + 1));
	}

private:
	TSharedPtr<FNiagaraSimCacheViewModel>					ViewModel;
	TStrongObjectPtr<const UNDISimpleCounterSimCacheData>	CacheData;
};

} // NDISimpleCounterSimCacheVisualizer

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> FNiagaraSimpleCounterSimCacheVisualizer::CreateWidgetFor(const UObject* InCachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel)
{
	using namespace NDISimpleCounterSimCacheVisualizer;

	if (const UNDISimpleCounterSimCacheData* CachedData = Cast<const UNDISimpleCounterSimCacheData>(InCachedData))
	{
		return SNew(SSimCacheView, ViewModel, CachedData);
	}
	return TSharedPtr<SWidget>();
}

#undef LOCTEXT_NAMESPACE
