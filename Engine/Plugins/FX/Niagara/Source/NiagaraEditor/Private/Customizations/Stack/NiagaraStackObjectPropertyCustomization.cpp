// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/NiagaraStackObjectPropertyCustomization.h"
