// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters.h"
#include "Utilities/NiagaraDynamicMaterialParameterUtils.h"
#include "Widgets/Text/STextBlock.h"
#include "ViewModels/Stack/NiagaraStackPropertyRow.h"
#include "DetailTreeNode.h"
#include "Containers/StaticBitArray.h"
#include "Materials/MaterialExpressionDynamicParameter.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/Modules/NiagaraStatelessModule_DynamicMaterialParameters.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Materials/Material.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters"

FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters::
FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters()
{
	ParameterIndexMap.Add(FName("Parameter0"), 0);
	ParameterIndexMap.Add(FName("Parameter1"), 1);
	ParameterIndexMap.Add(FName("Parameter2"), 2);
	ParameterIndexMap.Add(FName("Parameter3"), 3);
	ParameterChannelMap.Add(FName("XChannelDistribution"), 0);
	ParameterChannelMap.Add(FName("YChannelDistribution"), 1);
	ParameterChannelMap.Add(FName("ZChannelDistribution"), 2);
	ParameterChannelMap.Add(FName("WChannelDistribution"), 3);
}

TSharedRef<FNiagaraStackObjectPropertyCustomization> FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters::MakeInstance()
{
	return MakeShared<FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters>();
}

TOptional<TSharedPtr<SWidget>> FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters::GenerateNameWidget(UNiagaraStackPropertyRow* StackPropertyRow) const
{
	TSharedRef<FDetailTreeNode> DetailTreeNode = StaticCastSharedRef<FDetailTreeNode>(StackPropertyRow->GetDetailTreeNode());

	int32 ParameterIndex = INDEX_NONE;
	int32 ParameterChannel = INDEX_NONE;
	GetParameterIndexAndChannel(DetailTreeNode, ParameterIndex, ParameterChannel);

	if(ParameterIndex == INDEX_NONE || ParameterChannel == INDEX_NONE)
	{
		return TOptional<TSharedPtr<SWidget>>();	
	}
	
	TWeakPtr<FDetailTreeNode> ParentNode = DetailTreeNode->GetParentNode();
	TSharedPtr<IDetailPropertyRow> PropertyRow = DetailTreeNode->GetRow();

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = StackPropertyRow->GetSystemViewModel();
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = SystemViewModel->GetEmitterHandleViewModelForEmitter(StackPropertyRow->GetEmitterViewModel()->GetEmitter());

	TOptional<FText> DisplayName = TryGetDisplayNameForDynamicMaterialParameter(EmitterHandleViewModel, ParameterIndex, ParameterChannel);

	if(DisplayName.IsSet())
	{
		TOptional<float> MaterialDefault = TryGetDefaultValueForDynamicMaterialParameter(EmitterHandleViewModel, ParameterIndex, ParameterChannel);

		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.Text(DisplayName.GetValue())
			.ToolTipText(MaterialDefault.IsSet()
				? FText::FormatOrdered(LOCTEXT("MaterialDefaultTooltipFormat", "Material Default: {0}"), FText::AsNumber(MaterialDefault.GetValue()))
				: FText::GetEmpty());
		return Widget;
	}

	return TOptional<TSharedPtr<SWidget>>();
}

TOptional<FText> FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters::TryGetDisplayNameForDynamicMaterialParameter(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel, int32 ParameterIndex, int32 ParameterChannel) const
{
	UNiagaraStatelessEmitter* OwningStatelessEmitter = EmitterHandleViewModel->GetEmitterHandle()->GetStatelessEmitter();
	FNiagaraEmitterInstance* ThisEmitterInstance = EmitterHandleViewModel->GetEmitterViewModel()->GetSimulation().Pin().Get();
	TArray<UMaterial*> Materials = NiagaraDynamicMaterialParameterUtils::GetMaterialsFromRenderers(OwningStatelessEmitter->GetRenderers(), ThisEmitterInstance);
	
	TMap<UMaterialExpressionDynamicParameter*, TStaticBitArray<4>> DynamicParameterExpressionToOutputMaskMap;
	TArray<FExpressionInput*> ExpressionInputsToProcess;
	
	// Visit each material and gather all expression inputs for each expression.
	for (UMaterial* Material : Materials)
	{
		if (Material == nullptr)
		{
			continue;
		}

		for (int32 MaterialPropertyIndex = 0; MaterialPropertyIndex < MP_MAX; ++MaterialPropertyIndex)
		{
			FExpressionInput* ExpressionInput = Material->GetExpressionInputForProperty(EMaterialProperty(MaterialPropertyIndex));
			if (ExpressionInput != nullptr)
			{
				ExpressionInputsToProcess.Add(ExpressionInput);
			}
		}
		
		TArray<UMaterialExpression*> Expressions;
		Material->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpression>(Expressions);
		for (UMaterialExpression* Expression : Expressions)
		{
			for (FExpressionInputIterator It{ Expression}; It; ++It)
			{
				ExpressionInputsToProcess.Push(It.Input);
			}
		}
	}
	
	// Then we check these expression inputs to see dynamic parameter node/expression to check whether its corresponding parameter channel is used or not
	bool bAnyDynamicParametersFound = false;
	for (FExpressionInput* ExpressionInput : ExpressionInputsToProcess)
	{
		if (ExpressionInput->Expression == nullptr)
		{
			continue;
		}
	
		UMaterialExpressionDynamicParameter* DynamicParameterExpression = Cast<UMaterialExpressionDynamicParameter>(ExpressionInput->Expression);
		if (DynamicParameterExpression == nullptr)
		{
			continue;
		}
	
		bAnyDynamicParametersFound = true;
		// Each bit tells us whether the corresponding channel for the parameter is used or not
		GetChannelUsedBitMask(ExpressionInput, DynamicParameterExpressionToOutputMaskMap.FindOrAdd(DynamicParameterExpression));
	}
	
	FName ParameterName = NAME_None;
	bool bMultipleNamesFound = false;
	TOptional<bool> bIsParameterUsed;
	for (auto It = DynamicParameterExpressionToOutputMaskMap.CreateConstIterator(); It; ++It)
	{
		UMaterialExpressionDynamicParameter* ExpressionDynamicParameter = It.Key();
		const TStaticBitArray<4>& ExpressionOutputMask = It.Value();
		// We ignore those with a different index, and those where the channel isn't used
		if (ExpressionDynamicParameter->ParameterIndex != ParameterIndex)
		{
			continue;
		}
		
		const FExpressionOutput& Output = ExpressionDynamicParameter->GetOutputs()[ParameterChannel];
		if (ParameterName == NAME_None)
		{
			// We found a name but we continue iterating to check for duplicates
			ParameterName = Output.OutputName;
			
			if(bIsParameterUsed.IsSet() == false)
			{
				bIsParameterUsed = ExpressionOutputMask[ParameterChannel];
			}
		}
		// If we already found a name, but now find one with the same index and channel but a different name, we report that to the user
		else if (ParameterName != Output.OutputName)
		{
			bMultipleNamesFound = true;
		}
	}
	
	// Return the final dynamic param UI name.
	if (bAnyDynamicParametersFound == false)
	{
		return FText::FromString("(No material found using dynamic params)");
	}
	else if (ParameterName != NAME_None)
	{
		if (bMultipleNamesFound == false && bIsParameterUsed.IsSet())
		{
			FText ParameterNameAsText = FText::FromName(ParameterName);
			if(bIsParameterUsed.GetValue() == true)
			{
				return ParameterNameAsText;
			}
			else
			{
				return FText::FormatOrdered(FText::FromString("{0} - {1}"), ParameterNameAsText, FText::FromString("Unused in Material"));
			}
		}
		else
		{
			return FText::FromString(ParameterName.ToString() + TEXT(" (Multiple Aliases Found)"));
		}
	}

	return TOptional<FText>();
}

TOptional<float> FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters::TryGetDefaultValueForDynamicMaterialParameter(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel, int32 ParameterIndex, int32 ParameterChannel) const
{
	UNiagaraStatelessEmitter* OwningStatelessEmitter = EmitterHandleViewModel->GetEmitterHandle()->GetStatelessEmitter();
	FNiagaraEmitterInstance* ThisEmitterInstance = EmitterHandleViewModel->GetEmitterViewModel()->GetSimulation().Pin().Get();
	TArray<UMaterial*> Materials = NiagaraDynamicMaterialParameterUtils::GetMaterialsFromRenderers(OwningStatelessEmitter->GetRenderers(), ThisEmitterInstance);
	return NiagaraDynamicMaterialParameterUtils::GetDynamicParameterDefaultValue(Materials, ParameterIndex, ParameterChannel);
}

void FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters::GenerateContextMenuActions(UNiagaraStackPropertyRow* StackPropertyRow, FMenuBuilder& MenuBuilder) const
{
	Super::GenerateContextMenuActions(StackPropertyRow, MenuBuilder);

	TSharedRef<FDetailTreeNode> DetailTreeNode = StaticCastSharedRef<FDetailTreeNode>(StackPropertyRow->GetDetailTreeNode());

	int32 ParameterIndex = INDEX_NONE;
	int32 ParameterChannel = INDEX_NONE;
	GetParameterIndexAndChannel(DetailTreeNode, ParameterIndex, ParameterChannel);

	if (ParameterIndex == INDEX_NONE || ParameterChannel == INDEX_NONE)
	{
		return;
	}

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = StackPropertyRow->GetSystemViewModel();
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = SystemViewModel->GetEmitterHandleViewModelForEmitter(StackPropertyRow->GetEmitterViewModel()->GetEmitter());

	TOptional<float> MaterialDefault = TryGetDefaultValueForDynamicMaterialParameter(EmitterHandleViewModel, ParameterIndex, ParameterChannel);
	if (!MaterialDefault.IsSet())
	{
		return;
	}

	const float DefaultValue = MaterialDefault.GetValue();
	TWeakObjectPtr<UNiagaraStackPropertyRow> WeakPropertyRow(StackPropertyRow);

	FUIAction SetToDefaultAction(
		FExecuteAction::CreateLambda([WeakPropertyRow, ParameterIndex, ParameterChannel, DefaultValue]()
		{
			UNiagaraStackPropertyRow* PropertyRow = WeakPropertyRow.Get();
			if (PropertyRow == nullptr)
			{
				return;
			}

			TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow->GetDetailTreeNode()->CreatePropertyHandle();
			if (!PropertyHandle.IsValid() || !PropertyHandle->IsValidHandle())
			{
				return;
			}

			TArray<UObject*> OuterObjects;
			PropertyHandle->GetOuterObjects(OuterObjects);

			for (UObject* OuterObject : OuterObjects)
			{
				UNiagaraStatelessModule_DynamicMaterialParameters* Module = Cast<UNiagaraStatelessModule_DynamicMaterialParameters>(OuterObject);
				if (Module == nullptr)
				{
					continue;
				}

				FNiagaraStatelessDynamicParameterSet* ParameterSet = nullptr;
				switch (ParameterIndex)
				{
					case 0: ParameterSet = &Module->Parameter0; break;
					case 1: ParameterSet = &Module->Parameter1; break;
					case 2: ParameterSet = &Module->Parameter2; break;
					case 3: ParameterSet = &Module->Parameter3; break;
					default: break;
				}

				if (ParameterSet == nullptr)
				{
					continue;
				}

				FNiagaraDistributionFloat* Distribution = nullptr;
				switch (ParameterChannel)
				{
					case 0: Distribution = &ParameterSet->XChannelDistribution; break;
					case 1: Distribution = &ParameterSet->YChannelDistribution; break;
					case 2: Distribution = &ParameterSet->ZChannelDistribution; break;
					case 3: Distribution = &ParameterSet->WChannelDistribution; break;
					default: break;
				}

				if (Distribution != nullptr)
				{
					FScopedTransaction Transaction(LOCTEXT("SetToMaterialDefault", "Set Channel to Material Default"));
					PropertyHandle->NotifyPreChange();
					Distribution->InitConstant(DefaultValue);
					PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				}
			}
		})
	);

	MenuBuilder.AddMenuEntry(
		FText::FormatOrdered(LOCTEXT("SetMaterialDefaultFormat", "Set to Material Default ({0})"), FText::AsNumber(DefaultValue)),
		LOCTEXT("SetMaterialDefaultFormatTooltip", "Sets the channel constant to the default value defined in the material's Dynamic Parameter node"),
		FSlateIcon(),
		SetToDefaultAction);
}

void FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters::GetChannelUsedBitMask(FExpressionInput* Input, TStaticBitArray<4>& ChannelUsedMask) const
{
	if (Input->Expression)
	{
		TArray<FExpressionOutput>& Outputs = Input->Expression->GetOutputs();

		if (Outputs.Num() > 0)
		{
			const bool bOutputIndexIsValid = Outputs.IsValidIndex(Input->OutputIndex)
				// Attempt to handle legacy connections before OutputIndex was used that had a mask
				&& (Input->OutputIndex != 0 || Input->Mask == 0);

			for (int32 OutputIndex=0; OutputIndex < Outputs.Num(); ++OutputIndex)
			{
				const FExpressionOutput& Output = Outputs[OutputIndex];

				if ((bOutputIndexIsValid && OutputIndex == Input->OutputIndex)
					|| (!bOutputIndexIsValid
						&& Output.Mask == Input->Mask
						&& Output.MaskR == Input->MaskR
						&& Output.MaskG == Input->MaskG
						&& Output.MaskB == Input->MaskB
						&& Output.MaskA == Input->MaskA))
				{
					ChannelUsedMask[0] = ChannelUsedMask[0] || (Input->MaskR != 0);
					ChannelUsedMask[1] = ChannelUsedMask[1] || (Input->MaskG != 0);
					ChannelUsedMask[2] = ChannelUsedMask[2] || (Input->MaskB != 0);
					ChannelUsedMask[3] = ChannelUsedMask[3] || (Input->MaskA != 0);
					return;
				}
			}
		}
	}
}


void FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters::GetParameterIndexAndChannel(TSharedRef<FDetailTreeNode> DetailTreeNode, int32& OutParameterIndex, int32& OutParameterChannel) const
{
	OutParameterIndex = INDEX_NONE;
	OutParameterChannel = INDEX_NONE;

	FName NodeName = DetailTreeNode->GetNodeName();
	// First we check against the channel map, the individual values within one 'Parameter'
	if(ParameterChannelMap.Contains(NodeName))
	{
		OutParameterChannel = ParameterChannelMap[NodeName];

		TSharedPtr<FDetailTreeNode> ParentTreeNode = DetailTreeNode->GetParentNode().Pin();
		FName ParentNodeName = ParentTreeNode->GetNodeName();
		// If we found one, we can safely assume its parent is valid, which is the entire Parameter
		if(ensure(ParameterIndexMap.Contains(ParentNodeName)))
		{
			OutParameterIndex = ParameterIndexMap[ParentNodeName];
		}
	}
}

#undef LOCTEXT_NAMESPACE
