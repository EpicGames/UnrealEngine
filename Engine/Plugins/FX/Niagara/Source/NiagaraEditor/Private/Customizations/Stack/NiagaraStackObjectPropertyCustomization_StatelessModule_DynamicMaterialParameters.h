// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/StaticBitArray.h"
#include "Customizations/NiagaraStackObjectPropertyCustomization.h"

#define UE_API NIAGARAEDITOR_API

class FNiagaraEmitterHandleViewModel;
class UNiagaraStackPropertyRow;
class FDetailTreeNode;
struct FExpressionInput;
class FMenuBuilder;

class FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters
	: public FNiagaraStackObjectPropertyCustomization
{
	using Super = FNiagaraStackObjectPropertyCustomization;

public:
	UE_API FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters();

	static UE_API TSharedRef<FNiagaraStackObjectPropertyCustomization> MakeInstance();
	
	UE_API virtual TOptional<TSharedPtr<SWidget>> GenerateNameWidget(UNiagaraStackPropertyRow* PropertyRow) const override;
	UE_API virtual void GenerateContextMenuActions(UNiagaraStackPropertyRow* PropertyRow, FMenuBuilder& MenuBuilder) const override;

private:
	UE_API TOptional<FText> TryGetDisplayNameForDynamicMaterialParameter(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel, int32 ParameterIndex, int32 ParameterChannel) const;
	UE_API TOptional<float> TryGetDefaultValueForDynamicMaterialParameter(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel, int32 ParameterIndex, int32 ParameterChannel) const;

	UE_API void GetChannelUsedBitMask(FExpressionInput* Input, TStaticBitArray<4>& ChannelUsedMask) const;
	UE_API void GetParameterIndexAndChannel(TSharedRef<FDetailTreeNode> DetailTreeNode, int32& OutParameterIndex, int32& OutParameterChannel) const;

private:
	TMap<FName, int32> ParameterIndexMap;
	TMap<FName, int32> ParameterChannelMap;
};

#undef UE_API
