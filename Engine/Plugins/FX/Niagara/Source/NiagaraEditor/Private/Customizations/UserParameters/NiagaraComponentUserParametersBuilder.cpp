// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraComponentUserParametersBuilder.h"
#include "ViewModels/HierarchyEditor/NiagaraUserParametersHierarchyViewModel.h"
#include "PropertyHandle.h"
#include "NiagaraComponent.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "NiagaraTypes.h"
#include "ScopedTransaction.h"
#include "NiagaraEditorStyle.h"
#include "IDetailChildrenBuilder.h"
#include "NiagaraConstants.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSettings.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraUserRedirectionParameterStore.h"
#include "SNiagaraParameterEditor.h"
#include "Widgets/SNiagaraDebugger.h"
#include "Widgets/SNiagaraSystemUserParameters.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "NiagaraComponentUserParametersBuilder"

// Proxy class to allow us to override values on the component that are not yet overridden.
class FNiagaraParameterProxy : public TSharedFromThis<FNiagaraParameterProxy>
{
public:
	FNiagaraParameterProxy(TWeakObjectPtr<UNiagaraComponent> InComponent, const FNiagaraVariableBase& InKey, const FNiagaraVariant& InValue, const FSimpleDelegate& InOnRebuild, TArray<TSharedPtr<IPropertyHandle>> InPropertyHandles)
	{
		bResettingToDefault = false;
		Component = InComponent;
		ParameterKey = InKey;
		ParameterValue = InValue;
		OnRebuild = InOnRebuild;
		OverridePropertyHandles = InPropertyHandles;
	}

	void OnPropertyResetToDefault(TSharedPtr<IPropertyHandle> ResetPropertyHandle)
	{
		TWeakObjectPtr<UNiagaraComponent> RawComponent = GetComponent();
		if (RawComponent != nullptr)
		{
			FScopedTransaction ScopedTransaction(FNiagaraUserParameterNodeBuilder::ResetUserParameterTransactionText);
			RawComponent->Modify();

			bResettingToDefault = true;

			for (TSharedPtr<IPropertyHandle> PropertyHandle : OverridePropertyHandles)
			{
				PropertyHandle->NotifyPreChange();
			}

			RawComponent->RemoveParameterOverride(ParameterKey);

			for (TSharedPtr<IPropertyHandle> PropertyHandle : OverridePropertyHandles)
			{
				PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			}

			OnRebuild.ExecuteIfBound();
			bResettingToDefault = false;
		}
	}

	void OnObjectAssetResetToDefault()
	{
		OnPropertyResetToDefault(nullptr);
	}

	bool IsPropertyResetToDefaultVisible(TSharedPtr<IPropertyHandle> ResetPropertyHandle) const
	{
		TWeakObjectPtr<UNiagaraComponent> RawComponent = GetComponent();
		if (RawComponent.IsValid())
		{
			bool bHasParameterOverride = RawComponent->HasParameterOverride(ParameterKey);
			return bHasParameterOverride;
		}
		return false;
	}

	bool IsObjectAssetResetToDefaultVisible() const
	{
		return IsPropertyResetToDefaultVisible(nullptr);		
	}

	FNiagaraVariant FindExistingOverride() const
	{
		TWeakObjectPtr<UNiagaraComponent> RawComponent = GetComponent();
		if (RawComponent.IsValid())
		{
			return RawComponent->FindParameterOverride(ParameterKey);
		}
		return FNiagaraVariant();
	}

	void OnParameterPreChange()
	{
		TWeakObjectPtr<UNiagaraComponent> RawComponent = GetComponent();
		if(RawComponent.IsValid())
		{
			RawComponent->Modify();

			for (TSharedPtr<IPropertyHandle> PropertyHandle : OverridePropertyHandles)
			{
				PropertyHandle->NotifyPreChange();
			}
		}
	}

	/** This function handles value changes coming from the widget. It syncs local parameter editor values -> parameter value. */
	void OnParameterEditorValueChanged()
	{
		if(!ensure(DisplayData.IsValid() && DisplayDataPropertyHandle.IsValid()))
		{
			return;
		}

		TSharedRef<FStructOnScope> ParameterEditorData = MakeShared<FStructOnScope>(ParameterKey.GetType().GetStruct());
		ParameterEditor->UpdateStructFromInternalValue(ParameterEditorData);
		
		if (GIsTransacting)
		{
			// don't start a new transaction if we're currently applying another one (e.g. during an undo)
			return;
		}
		
		FScopedTransaction ScopedTransaction(FNiagaraUserParameterNodeBuilder::ChangedUserParameterTransactionText);
		
		bool bCancelTransaction = true;
		// with 'BroadcastValueChangesPerKey' enabled, this function can get called even when just clicking in or out of the value without modifying it. We don't want that to create a transaction.
		if(FMemory::Memcmp(DisplayData->GetStructMemory(), ParameterEditorData->GetStructMemory(), ParameterKey.GetType().GetSize()) != 0)
		{
			bCancelTransaction = false;
		    Component->Modify();
		}
		
		ParameterValue.SetBytes(ParameterEditorData->GetStructMemory(), ParameterKey.GetType().GetSize());

		DisplayDataPropertyHandle->NotifyPreChange();
		ParameterEditor->UpdateStructFromInternalValue(DisplayData.ToSharedRef());
		// NotifyPostChange will trigger 'OnParameterPostChange', actually setting up the override on the component & calling NotifyPostChange on the override handles, triggering PostEditChangeProperty on the component
		DisplayDataPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		DisplayDataPropertyHandle->NotifyFinishedChangingProperties();
		
		if(bCancelTransaction == true)
		{
			ScopedTransaction.Cancel();
		}
	}
	
	void OnParameterPostChange(const FPropertyChangedEvent& PropertyChangedEvent)
	{
		if (bResettingToDefault)
		{
			return;
		}
	
		TWeakObjectPtr<UNiagaraComponent> RawComponent = GetComponent();
		if(RawComponent.IsValid())
		{
			// if this post change is called on a 'local user parameter' (not a DI or object), we have to sync up the display data -> parameter editor
			if(DisplayData.IsValid())
			{
				if(ParameterEditor.IsValid())
				{
					// we have to make sure the internal parameter editor data is in sync with the display data (copy paste, reset etc. write to display data)
					ParameterEditor->UpdateInternalValueFromStruct(DisplayData.ToSharedRef());
				}
				
				ParameterValue.SetBytes(DisplayData->GetStructMemory(), ParameterValue.GetNumBytes());
			}
			
			RawComponent->SetParameterOverride(ParameterKey, ParameterValue);
	
			for (TSharedPtr<IPropertyHandle> PropertyHandle : OverridePropertyHandles)
			{
				PropertyHandle->NotifyPostChange(PropertyChangedEvent.ChangeType);
			}
		}
	}

	void OnAssetSelectedFromPicker(const FAssetData& InAssetData)
	{
		TWeakObjectPtr<UNiagaraComponent> RawComponent = GetComponent();

		if(RawComponent.IsValid())
		{
			UObject* Asset = InAssetData.GetAsset();
			if (Asset == nullptr || Asset->GetClass()->IsChildOf(ParameterKey.GetType().GetClass()))
			{
				FScopedTransaction ScopedTransaction(LOCTEXT("ChangeAsset", "Change asset"));
				RawComponent->Modify();
				RawComponent->SetParameterOverride(ParameterKey, FNiagaraVariant(Asset));

				for (TSharedPtr<IPropertyHandle> PropertyHandle : OverridePropertyHandles)
				{
					PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				}
			}
		}
	}

	FString GetCurrentAssetPath() const
	{
		TObjectPtr<UObject> CurrentObject = nullptr;
		
		TWeakObjectPtr<UNiagaraComponent> RawComponent = GetComponent();

		if(RawComponent.IsValid())
		{
			FNiagaraVariant CurrentValue = FindExistingOverride();
			if (CurrentValue.IsValid())
			{
				CurrentObject = CurrentValue.GetUObject();
			}
			else
			{
				// fetch from asset
				UNiagaraSystem* System = RawComponent->GetAsset();
				if (System != nullptr)
				{
					FNiagaraUserRedirectionParameterStore& AssetParamStore = System->GetExposedParameters();
					CurrentObject = AssetParamStore.GetUObject(ParameterKey);
				}
			}
		}

		return CurrentObject != nullptr ? CurrentObject.GetPathName() : FString();
	}

	void SetParameterEditor(TSharedRef<SNiagaraParameterEditor> InParameterEditor)
	{
		ParameterEditor = InParameterEditor;
	}

	void SetDisplayData(TSharedRef<FStructOnScope> InDisplayData)
	{
		DisplayData = InDisplayData;
	}

	void SetDisplayDataPropertyHandle(TSharedPtr<IPropertyHandle> InDisplayDataPropertyHandle)
	{
		DisplayDataPropertyHandle = InDisplayDataPropertyHandle;
	}
	
	const FNiagaraVariableBase& Key() const { return ParameterKey; }
	FNiagaraVariant& Value() { return ParameterValue; }

	TWeakObjectPtr<UNiagaraComponent> GetComponent() const { return Component; }

private:
	TWeakObjectPtr<UNiagaraComponent> Component;
	TArray<TSharedPtr<IPropertyHandle>> OverridePropertyHandles;
	FNiagaraVariableBase ParameterKey;
	FNiagaraVariant ParameterValue;
	FSimpleDelegate OnRebuild;
	bool bResettingToDefault;
	TSharedPtr<SNiagaraParameterEditor> ParameterEditor;
	TSharedPtr<FStructOnScope> DisplayData;
	TSharedPtr<IPropertyHandle> DisplayDataPropertyHandle;
};

FNiagaraComponentUserParametersNodeBuilder::FNiagaraComponentUserParametersNodeBuilder(UNiagaraComponent* InComponent, TArray<TSharedPtr<IPropertyHandle>> InOverridePropertyHandles, FName InCustomBuilderRowName)
{
	OverridePropertyHandles = InOverridePropertyHandles;
	CustomBuilderRowName = InCustomBuilderRowName;

	Component = InComponent;
	bDelegatesInitialized = false;
}

FNiagaraComponentUserParametersNodeBuilder::~FNiagaraComponentUserParametersNodeBuilder()
{
	if (Component.IsValid() && bDelegatesInitialized)
	{
		Component->OnSynchronizedWithAssetParameters().RemoveAll(this);
		Component->GetOverrideParameters().RemoveAllOnChangedHandlers(this);

		if(UNiagaraSystem* System = Component->GetAsset())
		{
			TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(System);
			if(SystemViewModel.IsValid() && SystemViewModel->GetUserParametersHierarchyViewModel())
			{
				SystemViewModel->GetUserParametersHierarchyViewModel()->GetOnHierarchyChangedDelegate().RemoveAll(this);
			}
		}
	}
}

void FNiagaraComponentUserParametersNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if(Component.IsValid())
	{
		if (bDelegatesInitialized == false)
		{
			Component->OnSynchronizedWithAssetParameters().AddSP(this, &FNiagaraComponentUserParametersNodeBuilder::Rebuild);
			
			RegisterRebuildOnHierarchyChanged();
			bDelegatesInitialized = true;
		}

		ParameterProxies.Reset();

		UNiagaraSystem* SystemAsset = Component->GetAsset();
		if (SystemAsset == nullptr)
		{
			return;
		}
			
		TArray<FNiagaraVariable> UserParameters;
		SystemAsset->GetExposedParameters().GetUserParameters(UserParameters);
			
		ParameterProxies.Reserve(UserParameters.Num());

		GenerateUserParameterRows(ChildrenBuilder, *Cast<UNiagaraSystemEditorData>(SystemAsset->GetEditorData())->UserParameterHierarchy.Get());
	}
}

FNiagaraVariant FNiagaraComponentUserParametersNodeBuilder::GetCurrentParameterValue(const FNiagaraVariableBase& UserParameter) const
{
	FNiagaraVariant CurrentValue = Component->GetCurrentParameterValue(UserParameter);
	if (CurrentValue.IsValid())
	{
		return CurrentValue;
	}
		
	return GetParameterValueFromSystem(UserParameter, *Component->GetAsset());
}

UNiagaraSystem* FNiagaraComponentUserParametersNodeBuilder::GetSystem() const
{
	return Component->GetAsset();
}

void FNiagaraComponentUserParametersNodeBuilder::GenerateRowForUserParameterInternal(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable UserParameter)
{
	UNiagaraSystem* SystemAsset = GetSystem();

	if(SystemAsset == nullptr)
	{
		return;
	}
	
	FNiagaraVariable ChoppedUserParameter(UserParameter);
	if(UserParameter.IsInNameSpace(FNiagaraConstants::UserNamespaceString))
	{
		ChoppedUserParameter.SetName(FName(UserParameter.GetName().ToString().RightChop(5)));
	}
	
	FNiagaraVariant CurrentParameterValue = GetCurrentParameterValue(ChoppedUserParameter);
	if (!CurrentParameterValue.IsValid())
	{
		return;
	}

	// if the DI is not overridden yet, we don't want to change the original system's DI so we duplicate here just in case
	if (UserParameter.IsDataInterface())
	{
		CurrentParameterValue = FNiagaraVariant(DuplicateObject(CurrentParameterValue.GetDataInterface(), Component.Get()));
	}

	TSharedPtr<FNiagaraParameterProxy> ParameterProxy = ParameterProxies.Add_GetRef(MakeShared<FNiagaraParameterProxy>(Component.Get(), UserParameter, CurrentParameterValue, OnRebuildChildren, OverridePropertyHandles));

	if (UserParameter.IsDataInterface())
	{
		// if no changes are made, then it'll just be the same as the asset
		TArray<UObject*> Objects { CurrentParameterValue.GetDataInterface() };
	
		FAddPropertyParams Params = FAddPropertyParams()
			.UniqueId(ChoppedUserParameter.GetName())
			.AllowChildren(true)
			.CreateCategoryNodes(false);
	
		IDetailPropertyRow* DataInterfaceRow  = ChildrenBuilder.AddExternalObjectProperty(Objects, NAME_None, Params);			
		DataInterfaceRow->DisplayName(FText::FromName(ChoppedUserParameter.GetName()));
		
		FDetailWidgetRow& CustomWidget = DataInterfaceRow->CustomWidget(true);

		DataInterfaceRow->GetPropertyHandle()->SetPropertyDisplayName(FText::FromName(ChoppedUserParameter.GetName()));
		DataInterfaceRow->GetPropertyHandle()->SetOnPropertyValuePreChange(FSimpleDelegate::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterPreChange));
		DataInterfaceRow->GetPropertyHandle()->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterPreChange));
		DataInterfaceRow->GetPropertyHandle()->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterPostChange));
		DataInterfaceRow->GetPropertyHandle()->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterPostChange));
		DataInterfaceRow->OverrideResetToDefault(
			FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::IsPropertyResetToDefaultVisible),
				FResetToDefaultHandler::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnPropertyResetToDefault)));

		CustomWidget
		.FilterString(FText::FromName(ChoppedUserParameter.GetName()))
		.NameContent()
		.HAlign(HAlign_Left)
		[
			CreateUserParameterNameWidget(ChoppedUserParameter)
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(4.0f)
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(FText::FromString(FName::NameToDisplayString(UserParameter.GetType().GetClass()->GetName(), false)))
			]
		];

		AddCustomMenuActionsForParameter(CustomWidget, UserParameter);
	}
	else if(UserParameter.IsUObject())
	{		
		TAttribute<bool> IsObjectAssetResetVisible;
		IsObjectAssetResetVisible.Bind(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::IsObjectAssetResetToDefaultVisible);
		
		FDetailWidgetRow& DetailWidgetRow = ChildrenBuilder.AddCustomRow(FText::FromName(ChoppedUserParameter.GetName()));
		DetailWidgetRow.OverrideResetToDefault(
			FResetToDefaultOverride::Create(
				IsObjectAssetResetVisible,
				FSimpleDelegate::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnObjectAssetResetToDefault)));
		
		DetailWidgetRow
		.NameContent()
		[
			CreateUserParameterNameWidget(ChoppedUserParameter)
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
			.ObjectPath(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::GetCurrentAssetPath)
			.AllowedClass(UserParameter.GetType().GetClass())
			.OnObjectChanged(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnAssetSelectedFromPicker)
			.DisplayUseSelected(true)
			.DisplayBrowse(true)
			.DisplayThumbnail(true)
			.NewAssetFactories(TArray<UFactory*>())
			.OwnerAssetDataArray({ FAssetData(SystemAsset) })
		];
		
		AddCustomMenuActionsForParameter(DetailWidgetRow, UserParameter);		
	}
	else
	{
		IDetailPropertyRow* DetailPropertyRow = AddValueParameterAsRow(ChildrenBuilder, ChoppedUserParameter);

		// DisplayData should always exist after the call above
		if(ensure(DisplayData.Contains(ChoppedUserParameter)))
		{
			// A niagara parameter editor exists only for certain types. If it exists, setup the value change callback.
			if(NiagaraParameterEditors.Contains(ChoppedUserParameter))
			{
				NiagaraParameterEditors[ChoppedUserParameter]->SetOnBeginValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterPreChange));
				NiagaraParameterEditors[ChoppedUserParameter]->SetOnEndValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterEditorValueChanged));
				ParameterProxy->SetParameterEditor(NiagaraParameterEditors[ChoppedUserParameter]);
			}
			
			ParameterProxy->SetDisplayData(DisplayData[ChoppedUserParameter]);
			ParameterProxy->SetDisplayDataPropertyHandle(DetailPropertyRow->GetPropertyHandle());
		}

		DetailPropertyRow->GetPropertyHandle()->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterPreChange));
		DetailPropertyRow->GetPropertyHandle()->SetOnPropertyValuePreChange(FSimpleDelegate::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterPreChange));
		/** We make sure to react properly whenever a value changes. This is also executed after pasting or resetting to default. */
		DetailPropertyRow->GetPropertyHandle()->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterPostChange));
		DetailPropertyRow->GetPropertyHandle()->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnParameterPostChange));
		DetailPropertyRow->GetPropertyHandle()->SetPropertyDisplayName(FText::FromName(ChoppedUserParameter.GetName()));
		DetailPropertyRow->OverrideResetToDefault(
			FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::IsPropertyResetToDefaultVisible),
				FResetToDefaultHandler::CreateSP(ParameterProxy.ToSharedRef(), &FNiagaraParameterProxy::OnPropertyResetToDefault)));
	}
}

void FNiagaraComponentUserParametersNodeBuilder::OnParameterEditorValueChanged(FNiagaraVariable UserParameter)
{
	// this should not be called. FNiagaraParameterProxy::OnParameterEditorValueChanged should be called instead.
	ensure(false);
}

void FNiagaraComponentUserParametersNodeBuilder::OnObjectAssetChanged(const FAssetData& NewObject, FNiagaraVariable UserParameter)
{
	for(TSharedPtr<FNiagaraParameterProxy> Proxy : ParameterProxies)
	{
		if(Proxy->Key() == UserParameter)
		{
			Proxy->OnAssetSelectedFromPicker(NewObject);
			return;
		}
	}
}

void FNiagaraComponentUserParametersNodeBuilder::RegisterRebuildOnHierarchyChanged()
{
	if(Component.IsValid())
	{
		if(UNiagaraSystem* Asset = Component->GetAsset())
		{
			TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(Asset);
			if(SystemViewModel.IsValid())
			{
				SystemViewModel->GetUserParametersHierarchyViewModel()->GetOnHierarchyChangedDelegate().RemoveAll(this);
				SystemViewModel->GetUserParametersHierarchyViewModel()->GetOnHierarchyChangedDelegate().Add(UDataHierarchyViewModelBase::FOnHierarchyChanged::FDelegate::CreateSP(this, &FNiagaraComponentUserParametersNodeBuilder::RebuildOnHierarchyChanged));
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
