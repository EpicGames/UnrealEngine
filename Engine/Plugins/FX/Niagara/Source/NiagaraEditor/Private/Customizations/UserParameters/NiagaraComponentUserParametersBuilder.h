// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraUserParametersBuilderBase.h"
#include "CoreMinimal.h"
#include "NiagaraComponent.h"
#include "ViewModels/HierarchyEditor/NiagaraUserParametersHierarchyViewModel.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class FNiagaraParameterViewModelCustomDetails;
class INiagaraParameterViewModel;
class SNiagaraParameterEditor;
class FNiagaraParameterViewModelCustomDetails;
class FNiagaraSimCacheCapture;
class UNiagaraSystem;

/** A specialized class to generate user parameters for a component. */
class FNiagaraComponentUserParametersNodeBuilder : public FNiagaraUserParameterNodeBuilder
{
public:
	FNiagaraComponentUserParametersNodeBuilder(UNiagaraComponent* InComponent, TArray<TSharedPtr<IPropertyHandle>> InOverridePropertyHandles, FName InCustomBuilderRowName);
	virtual ~FNiagaraComponentUserParametersNodeBuilder() override;
	
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;

private:
	/** NiagaraUserParameterNodeBuilder Interface */
	virtual UNiagaraSystem* GetSystem() const override;
	virtual FNiagaraVariant GetCurrentParameterValue(const FNiagaraVariableBase& UserParameter) const override;
	virtual void GenerateRowForUserParameterInternal(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable UserParameter) override;
	virtual void OnParameterEditorValueChanged(FNiagaraVariable UserParameter) override;
	virtual void OnObjectAssetChanged(const FAssetData& NewObject, FNiagaraVariable UserParameter) override;
	
	void RegisterRebuildOnHierarchyChanged();

private:
	TWeakObjectPtr<UNiagaraComponent> Component;
	TArray<TSharedPtr<class FNiagaraParameterProxy>> ParameterProxies;
	TArray<TSharedPtr<IPropertyHandle>> OverridePropertyHandles;
};
