// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemUserParametersBuilder.h"
#include "ViewModels/HierarchyEditor/NiagaraUserParametersHierarchyViewModel.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Styling/AppStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraTypes.h"
#include "ScopedTransaction.h"
#include "NiagaraNodeInput.h"
#include "IDetailChildrenBuilder.h"
#include "NiagaraConstants.h"
#include "NiagaraModule.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSettings.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraUserRedirectionParameterStore.h"
#include "INiagaraVerseReflectionUtilities.h"
#include "SNiagaraParameterEditor.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SNiagaraDebugger.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SNiagaraSystemUserParameters.h"
#include "Widgets/SNiagaraParameterMenu.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "NiagaraSystemUserParametersBuilder"

FNiagaraSystemUserParameterBuilder::FNiagaraSystemUserParameterBuilder(TSharedPtr<FNiagaraSystemViewModel> InSystemViewModel, FName InCustomBuilderRowName)
{
	CustomBuilderRowName = InCustomBuilderRowName;
	System = &InSystemViewModel->GetSystem();
	SystemViewModel = InSystemViewModel;
	bDelegatesInitialized = false;

	if (INiagaraModule* NiagaraModule = FModuleManager::Get().LoadModulePtr<INiagaraModule>(TEXT("Niagara")))
	{
		VerseReflectionUtilities = NiagaraModule->GetVerseReflectionUtilities();
	}
}

FNiagaraSystemUserParameterBuilder::~FNiagaraSystemUserParameterBuilder()
{
	if (System.IsValid() && bDelegatesInitialized)
	{
		System->GetExposedParameters().OnStructureChanged().RemoveAll(this);
		System->GetExposedParameters().RemoveOnChangedHandler(SystemExposedParametersOnChangedHandle);

		if(SystemViewModel.IsValid() && SystemViewModel.Pin()->GetUserParameterPanelViewModel().IsValid())
		{
			if (TSharedPtr<FNiagaraUserParameterPanelViewModel> PanelViewModel = SystemViewModel.Pin()->GetUserParameterPanelViewModel())
			{
				PanelViewModel->OnRefreshRequested().Unbind();
			}
		}
			
		if(SystemViewModel.IsValid() && SystemViewModel.Pin()->GetUserParametersHierarchyViewModel())
		{
			if (UNiagaraUserParametersHierarchyViewModel* HierarchyViewModel = SystemViewModel.Pin()->GetUserParametersHierarchyViewModel())
			{
				HierarchyViewModel->GetOnHierarchyChangedDelegate().RemoveAll(this);
			}
		}
	}
}

void FNiagaraSystemUserParameterBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	check(System.IsValid());

	if (System.IsValid())
	{
		if(bDelegatesInitialized == false)
		{
			System->GetExposedParameters().OnStructureChanged().Add(FNiagaraParameterStore::FOnStructureChanged::FDelegate::CreateSP(this, &FNiagaraSystemUserParameterBuilder::Rebuild));
			SystemExposedParametersOnChangedHandle = System->GetExposedParameters().AddOnChangedHandler(
				FNiagaraParameterStore::FOnChanged::FDelegate::CreateSP(this, &FNiagaraSystemUserParameterBuilder::SystemExposedParametersChanged));

			SystemViewModel.Pin()->GetUserParametersHierarchyViewModel()->GetOnHierarchyChangedDelegate().RemoveAll(this);
			SystemViewModel.Pin()->GetUserParametersHierarchyViewModel()->GetOnHierarchyChangedDelegate().Add(UDataHierarchyViewModelBase::FOnHierarchyChanged::FDelegate::CreateSP(this, &FNiagaraSystemUserParameterBuilder::RebuildOnHierarchyChanged));
			
			TSharedPtr<FNiagaraUserParameterPanelViewModel> UserParameterPanelViewModel = SystemViewModel.Pin()->GetUserParameterPanelViewModel();
			if(ensure(UserParameterPanelViewModel.IsValid()))
			{
				UserParameterPanelViewModel->OnRefreshRequested().BindSP(this, &FNiagaraSystemUserParameterBuilder::Rebuild);
				UserParameterPanelViewModel->OnParameterAdded().BindSP(this, &FNiagaraSystemUserParameterBuilder::OnParameterAdded);
			}
			
			bDelegatesInitialized = true;
		}
		
		UHierarchyRoot* Root = Cast<UNiagaraSystemEditorData>(System->GetEditorData())->UserParameterHierarchy;
		GenerateUserParameterRows(ChildrenBuilder, *Root);
	}
}

void FNiagaraSystemUserParameterBuilder::AddCustomMenuActionsForParameter(FDetailWidgetRow& WidgetRow, FNiagaraVariable UserParameter)
{
	/** We want to add rename & delete actions within a system asset */
	
	FNiagaraUserParameterNodeBuilder::AddCustomMenuActionsForParameter(WidgetRow, UserParameter);
	
	FUIAction RenameAction(FExecuteAction::CreateSP(this, &FNiagaraSystemUserParameterBuilder::RequestRename, UserParameter));
	WidgetRow.AddCustomContextMenuAction(RenameAction, LOCTEXT("RenameParameterAction", "Rename"), LOCTEXT("RenameParameterActionTooltip", "Rename this user parameter"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"));

	FUIAction DuplicateAction(FExecuteAction::CreateSP(this, &FNiagaraSystemUserParameterBuilder::RequestDuplication, UserParameter));
	WidgetRow.AddCustomContextMenuAction(DuplicateAction, LOCTEXT("DuplicateParameterAction", "Duplicate"), LOCTEXT("DuplicateParameterActionTooltip", "Duplicate this user parameter"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"));

	FUIAction DeleteAction(FExecuteAction::CreateSP(this, &FNiagaraSystemUserParameterBuilder::DeleteParameter, UserParameter));
	WidgetRow.AddCustomContextMenuAction(DeleteAction, LOCTEXT("DeleteParameterAction", "Delete"), LOCTEXT("DeleteParameterActionTooltip", "Delete this user parameter"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"));
}

TSharedRef<SWidget> FNiagaraSystemUserParameterBuilder::GetAdditionalHeaderWidgets()
{
	if(!AdditionalHeaderWidgetsContainer.IsValid())
	{
		AdditionalHeaderWidgetsContainer = SNew(SBox)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(2.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(2.f, 1.f)
			[
				SNew(SButton)
				.OnClicked(this, &FNiagaraSystemUserParameterBuilder::SummonHierarchyEditor)
				.Text(LOCTEXT("SummonUserParametersHierarchyButtonLabel", "Edit Hierarchy"))
				.ButtonStyle(FAppStyle::Get(), "RoundButton")
				.ToolTipText(LOCTEXT("SummonUserParametersHierarchyButtonTooltip", "Summon the Hierarchy Editor to add categories, sections and tooltips to User Parameters."))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(2.f)
			[
				SAssignNew(AddParameterButton, SComboButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "RoundButton")
				.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
				.ContentPadding(FMargin(2, 3.5))
				.ToolTipText(LOCTEXT("AddUserParameterButtonTooltip", "Add a new User Parameter to this system.\nUser Parameters exist once for the entire system and their values can be changed at runtime from Blueprints, C++ or the sequencer, among other systems."))
				.OnGetMenuContent(this, &FNiagaraSystemUserParameterBuilder::GetAddParameterMenu)
				.OnComboBoxOpened_Lambda([this]
				{
					AddParameterButton->SetMenuContentWidgetToFocus(AddParameterMenu->GetSearchBox());
				})
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Plus"))
				]
			]
		];
	}
	
	return AdditionalHeaderWidgetsContainer.ToSharedRef(); 
}

FNiagaraVariant FNiagaraSystemUserParameterBuilder::GetCurrentParameterValue(const FNiagaraVariableBase& UserParameter) const
{
	return GetParameterValueFromSystem(UserParameter, *System.Get());
}

UNiagaraSystem* FNiagaraSystemUserParameterBuilder::GetSystem() const
{
	return System.Get();
}

void FNiagaraSystemUserParameterBuilder::GenerateRowForUserParameterInternal(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable UserParameter)
{
	UNiagaraSystem* SystemAsset = GetSystem();

	if(SystemAsset == nullptr)
	{
		return;
	}
	
	FNiagaraVariable ChoppedUserParameter(UserParameter);
	if(UserParameter.IsInNameSpace(FNiagaraConstants::UserNamespaceString))
	{
		ChoppedUserParameter.SetName(FName(UserParameter.GetName().ToString().RightChop(5)));
	}
	
	FNiagaraVariant CurrentParameterValue = GetCurrentParameterValue(ChoppedUserParameter);
	if (!CurrentParameterValue.IsValid())
	{
		return;
	}
	
	if (UserParameter.IsDataInterface())
	{
		// if no changes are made, then it'll just be the same as the asset
		TArray<UObject*> Objects { CurrentParameterValue.GetDataInterface() };
	
		FAddPropertyParams Params = FAddPropertyParams()
			.UniqueId(ChoppedUserParameter.GetName())
			.AllowChildren(true)
			.CreateCategoryNodes(false);
	
		IDetailPropertyRow* DataInterfaceRow = ChildrenBuilder.AddExternalObjectProperty(Objects, NAME_None, Params);			
		DataInterfaceRow->DisplayName(FText::FromName(ChoppedUserParameter.GetName()));
		
		FDetailWidgetRow& CustomWidget = DataInterfaceRow->CustomWidget(true);

		DataInterfaceRow->GetPropertyHandle()->SetPropertyDisplayName(FText::FromName(ChoppedUserParameter.GetName()));
		DataInterfaceRow->GetPropertyHandle()->SetOnPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FNiagaraSystemUserParameterBuilder::OnDataInterfacePropertyValuePreChange));
		DataInterfaceRow->GetPropertyHandle()->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FNiagaraSystemUserParameterBuilder::OnDataInterfacePropertyValuePreChange));
		DataInterfaceRow->GetPropertyHandle()->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FNiagaraSystemUserParameterBuilder::OnDataInterfacePropertyValueChangedWithData));
		DataInterfaceRow->GetPropertyHandle()->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FNiagaraSystemUserParameterBuilder::OnDataInterfacePropertyValueChangedWithData));
		DataInterfaceRow->OverrideResetToDefault(
			FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateSP(this, &FNiagaraSystemUserParameterBuilder::IsDataInterfaceResetToDefaultVisible, ChoppedUserParameter),
				FResetToDefaultHandler::CreateSP(this, &FNiagaraSystemUserParameterBuilder::OnHandleDataInterfaceReset, ChoppedUserParameter)));
		
		CustomWidget
		.FilterString(FText::FromName(ChoppedUserParameter.GetName()))
		.NameContent()
		.HAlign(HAlign_Left)
		[
			CreateUserParameterNameWidget(UserParameter)
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
			.Text(FText::FromString(FName::NameToDisplayString(UserParameter.GetType().GetClass()->GetName(), false)))
		];
		
		AddCustomMenuActionsForParameter(CustomWidget, UserParameter);
	}
	else if(UserParameter.IsUObject())
	{
		FDetailWidgetRow& DetailWidgetRow = ChildrenBuilder.AddCustomRow(FText::FromName(ChoppedUserParameter.GetName()));
		DetailWidgetRow.OverrideResetToDefault(
					FResetToDefaultOverride::Create(
						TAttribute<bool>::CreateSP(this, &FNiagaraSystemUserParameterBuilder::IsObjectAssetCustomRowResetToDefaultVisible, ChoppedUserParameter),
						FSimpleDelegate::CreateSP(this, &FNiagaraSystemUserParameterBuilder::OnHandleObjectAssetReset, ChoppedUserParameter)));
		
		DetailWidgetRow
		.NameContent()
		[
			CreateUserParameterNameWidget(UserParameter)
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
			.ObjectPath(TAttribute<FString>::CreateSP(this, &FNiagaraSystemUserParameterBuilder::GetObjectAssetPathForUserParameter, ChoppedUserParameter))
			.AllowedClass(UserParameter.GetType().GetClass())
			.OnObjectChanged(this, &FNiagaraSystemUserParameterBuilder::OnObjectAssetChanged, ChoppedUserParameter)
			.DisplayUseSelected(true)
			.DisplayBrowse(true)
			.DisplayThumbnail(true)
			.NewAssetFactories(TArray<UFactory*>())
			.OwnerAssetDataArray({ FAssetData(SystemAsset) })
		];

		AddCustomMenuActionsForParameter(DetailWidgetRow, UserParameter);
	}
	else
	{
		IDetailPropertyRow* DetailPropertyRow = AddValueParameterAsRow(ChildrenBuilder, ChoppedUserParameter);
		if(!ensure(DisplayData.Contains(ChoppedUserParameter) && DisplayDataPropertyHandles.Contains(ChoppedUserParameter)))
		{
			return;
		}
		
		/** We make sure to react properly whenever a value changes. This is also executed after pasting or resetting to default. */
		DetailPropertyRow->GetPropertyHandle()->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FNiagaraSystemUserParameterBuilder::OnDisplayDataPostChange, ChoppedUserParameter));
		DetailPropertyRow->GetPropertyHandle()->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FNiagaraSystemUserParameterBuilder::OnDisplayDataPostChange, ChoppedUserParameter));
		DetailPropertyRow->GetPropertyHandle()->SetPropertyDisplayName(FText::FromName(ChoppedUserParameter.GetName()));
		DetailPropertyRow->OverrideResetToDefault(
			FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateSP(this, &FNiagaraSystemUserParameterBuilder::IsLocalPropertyResetToDefaultVisible, ChoppedUserParameter),
				FResetToDefaultHandler::CreateSP(this, &FNiagaraSystemUserParameterBuilder::OnHandleLocalPropertyReset, ChoppedUserParameter)));
	}
}

TSharedRef<SWidget> FNiagaraSystemUserParameterBuilder::GetAddParameterMenu()
{	
	FNiagaraParameterPanelCategory UserCategory = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces({FNiagaraConstants::UserNamespace});
	AddParameterMenu = SNew(SNiagaraAddParameterFromPanelMenu)
		.Graphs(SystemViewModel.Pin()->GetParameterPanelViewModel()->GetEditableGraphsConst())
		.OnNewParameterRequested(this, &FNiagaraSystemUserParameterBuilder::AddParameter)
		.OnAllowMakeType(this, &FNiagaraSystemUserParameterBuilder::OnAllowMakeType)
		.NamespaceId(UserCategory.NamespaceMetaData.GetGuid())
		.ShowNamespaceCategory(false)
		.ShowGraphParameters(false)
		.AutoExpandMenu(false);

	return AddParameterMenu.ToSharedRef();
}

void FNiagaraSystemUserParameterBuilder::AddParameter(FNiagaraVariable NewParameter) const
{
	TGuardValue<bool> UpdateSystemGuard(bUpdatingSystem, true);
	SystemViewModel.Pin()->AddParameter(NewParameter);
}

bool FNiagaraSystemUserParameterBuilder::OnAllowMakeType(const FNiagaraTypeDefinition& InType) const
{
	FNiagaraNamespaceMetadata UserNameSpaceMetaData = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces({FNiagaraConstants::UserNamespace});
	FNiagaraParameterPanelCategory Category(UserNameSpaceMetaData);
	return SystemViewModel.Pin()->GetParameterPanelViewModel()->CanAddType(InType, Category);
}

FReply FNiagaraSystemUserParameterBuilder::SummonHierarchyEditor() const
{
	if(TSharedPtr<FNiagaraSystemViewModel> SystemViewModelPinned = SystemViewModel.Pin())
	{
		SystemViewModelPinned->FocusTab(FNiagaraSystemToolkitModeBase::UserParametersHierarchyTabID, true);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void FNiagaraSystemUserParameterBuilder::DeleteParameter(FNiagaraVariable UserParameter)
{
	// We re-add the User namespace if necessary
	if(!UserParameter.IsInNameSpace(FNiagaraConstants::UserNamespace))
	{
		UserParameter.SetNamespacedName(FNiagaraConstants::UserNamespaceString, UserParameter.GetName());
	}
	
	FScopedTransaction Transaction(LOCTEXT("DeleteParameterTransaction", "Deleted user parameter"));
	TGuardValue<bool> UpdateSystemGuard(bUpdatingSystem, true);
	System->Modify();
	
	Cast<UNiagaraSystemEditorData>(System->GetEditorData())->UserParameterHierarchy->Modify();

	SystemViewModel.Pin()->DeleteParameter(UserParameter);

	UserParamToWidgetMap.Empty();
	
	Rebuild();
}

TSharedRef<SWidget> FNiagaraSystemUserParameterBuilder::CreateUserParameterNameWidget(FNiagaraVariable UserParameter)
{
	TSharedPtr<SWidget> VerseWidget = SNullWidget::NullWidget;
	if (VerseReflectionUtilities)
	{
		VerseWidget = SNew(SImage)
			.ToolTipText(TAttribute<FText>::CreateSP(this, &FNiagaraSystemUserParameterBuilder::GetParameterAccessLevelTooltip, UserParameter))
			.Image(TAttribute<const FSlateBrush*>::CreateSP(this, &FNiagaraSystemUserParameterBuilder::GetParameterAccessLevelBrush, UserParameter))
			.ColorAndOpacity(FSlateColor::UseForeground());
	}

	// we make sure that within a system context, the name parameter is always prefixed with User.
	FNiagaraUserRedirectionParameterStore::MakeUserVariable(UserParameter);

	/** Instead of pure text, we use the parameter widgets for display in a system asset. */
	TSharedRef<SNiagaraParameterNameTextBlock> ParameterName = SNew(SNiagaraParameterNameTextBlock)
		// if this is specified, we avoid the default behavior, which is "when clicked when the widget had keyboard focus already, enter editing mode"
		// we don't want to be able to rename by clicking, only by context menu
		.IsSelected_Lambda([]() { return false; })
		.ParameterText(FText::FromName(UserParameter.GetName()))
		.OnDragDetected(this, &FNiagaraSystemUserParameterBuilder::GenerateParameterDragDropOp, UserParameter)
		.OnVerifyTextChanged_Lambda([this, UserParameter](const FText& InNewName, FText& OutErrorMessage)
		{
			if(InNewName.IsEmptyOrWhitespace())
			{
				OutErrorMessage = LOCTEXT("UserParameterNameCantBeEmpty", "Can not accept empty name");
				return false;
			}
			
			FNiagaraVariable NewUserParameter = UserParameter;
			NewUserParameter.SetName(FName(InNewName.ToString()));
			if(UserParameter == NewUserParameter)
			{
				// we return true, but renaming won't do anything if the name is the same as before
				return true;
			}

			TArray<FNiagaraVariable> ExistingParameters;
			UserParamToWidgetMap.GenerateKeyArray(ExistingParameters);

			// Check to see if it already exists
			{
				const FString NewNameString = InNewName.ToString();

				bool bNameExistsAlready = false;
				for (FNiagaraVariable& ExistingUserParameter : ExistingParameters)
				{
					if (ExistingUserParameter.GetName().ToString().Equals(NewNameString, ESearchCase::IgnoreCase))
					{
						bNameExistsAlready = true;
						break;
					}
				}

				if (bNameExistsAlready)
				{
					OutErrorMessage = LOCTEXT("UserParameterAlreadyExists", "A User Parameter with the same name already exists");
					return false;
				}
			}

			return true;
		})
		.OnTextCommitted_Lambda([this, UserParameter](const FText& InText, ETextCommit::Type Type) { RenameParameter(UserParameter, FName(InText.ToString())); })
		.DecoratorLocation(ENiagaraParameterDecoratorLocation::BeforeName)
		.Decorator()
		[
			VerseWidget.ToSharedRef()
		];

	UserParamToWidgetMap.Add(UserParameter, ParameterName);

	TAttribute<FText> Tooltip = FText::GetEmpty();
	if(UNiagaraScriptVariable* ScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(UserParameter, *GetSystem()))
	{
		Tooltip = TAttribute<FText>::CreateLambda([ScriptVariable]
		{
			return ScriptVariable->Metadata.Description;
		});
	}
	ParameterName->SetToolTipText(Tooltip);
	
	return ParameterName;
}

void FNiagaraSystemUserParameterBuilder::SelectAllSection()
{
	if(ActiveSection != nullptr)
	{
		ActiveSection = nullptr;
		OnRebuildChildren.Execute();
	}
}

void FNiagaraSystemUserParameterBuilder::OnParameterAdded(FNiagaraVariable UserParameter)
{
	// when adding a new parameter it defaults to the 'All' section, so we select it in order to visualize it before renaming
	SelectAllSection();
	RequestRename(UserParameter);
}

void FNiagaraSystemUserParameterBuilder::RequestRename(FNiagaraVariable UserParameter)
{
	FNiagaraUserRedirectionParameterStore::MakeUserVariable(UserParameter);
	
	// we add a timer for next frame, as requesting a rename on a parameter that has just been created will not preselect the entire text. Waiting a frame fixes that.
	UserParamToWidgetMap[UserParameter]->RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this, UserParameter](double CurrentTime, float DeltaTime)
	{
		UserParamToWidgetMap[UserParameter]->EnterEditingMode();
		return EActiveTimerReturnType::Stop;
	}));
}

void FNiagaraSystemUserParameterBuilder::RenameParameter(FNiagaraVariable UserParameter, FName NewName)
{
	if(UserParameter.GetName() == NewName)
	{
		return;
	}
	
	if(System->GetExposedParameters().IndexOf(UserParameter) != INDEX_NONE)
	{
		TGuardValue<bool> UpdateSystemGuard(bUpdatingSystem, true);
		SystemViewModel.Pin()->RenameParameter(UserParameter, NewName, ENiagaraGetGraphParameterReferencesMode::AllGraphs);
	}

	UserParamToWidgetMap.Empty();

	Rebuild();
}

void FNiagaraSystemUserParameterBuilder::RequestDuplication(FNiagaraVariable UserParameter)
{
	FNiagaraVariable DuplicatedParameter = FNiagaraEditorUtilities::UserParameters::DuplicateUserParameter(UserParameter,*System.Get());
	if(DuplicatedParameter.IsValid())
	{
		RequestRename(DuplicatedParameter);
	}
}

FReply FNiagaraSystemUserParameterBuilder::GenerateParameterDragDropOp(const FGeometry& Geometry, const FPointerEvent& MouseEvent, FNiagaraVariable UserParameter) const
{
	TSharedPtr<FEdGraphSchemaAction> ParameterDragAction = MakeShared<FNiagaraParameterAction>(UserParameter, FText::GetEmpty(), FText::GetEmpty(), FText::GetEmpty(), 0, FText(), nullptr, 0/*SectionID*/);
	TSharedRef<FNiagaraParameterDragOperation> ParameterDragOperation = MakeShared<FNiagaraParameterDragOperation>(ParameterDragAction);
	ParameterDragOperation->SetupDefaults();
	ParameterDragOperation->Construct();
	return FReply::Handled().BeginDragDrop(ParameterDragOperation);
}

void FNiagaraSystemUserParameterBuilder::OnParameterEditorValueChanged(FNiagaraVariable UserParameter)
{
	UNiagaraSystem* SystemAsset = GetSystem();
	
	if(UserParameter.IsDataInterface() || UserParameter.IsUObject())
	{
		ensure(false);
		return;
	}

	TSharedRef<FStructOnScope> ParameterEditorData = MakeShared<FStructOnScope>(UserParameter.GetType().GetStruct());
	NiagaraParameterEditors[UserParameter]->UpdateStructFromInternalValue(ParameterEditorData);
	if(FMemory::Memcmp(ParameterEditorData->GetStructMemory(), DisplayData[UserParameter]->GetStructMemory(), UserParameter.GetType().GetSize()) == 0)
	{
		return;
	}
	
	FScopedTransaction ScopedTransaction(ChangedUserParameterTransactionText, GIsTransacting == false);
	SystemAsset->Modify();

	// we forward the change in the parameter editor into the display data. NotifyPostChange will take care of forwarding the change into the user parameter store.
	DisplayDataPropertyHandles[UserParameter]->NotifyPreChange();
	NiagaraParameterEditors[UserParameter]->UpdateStructFromInternalValue(DisplayData[UserParameter]);
	DisplayDataPropertyHandles[UserParameter]->NotifyPostChange(EPropertyChangeType::ValueSet);
	DisplayDataPropertyHandles[UserParameter]->NotifyFinishedChangingProperties();
}

void FNiagaraSystemUserParameterBuilder::OnDisplayDataPostChange(const FPropertyChangedEvent& ChangedEvent, FNiagaraVariable UserParameter)
{
	if(ChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		UNiagaraSystem* SystemAsset = GetSystem();

		if(NiagaraParameterEditors.Contains(UserParameter))
		{
			// we have to make sure the internal parameter editor data is in sync with the display data (copy paste, reset etc. write to display data)
			NiagaraParameterEditors[UserParameter]->UpdateInternalValueFromStruct(DisplayData[UserParameter]);
		}
		
		TGuardValue<bool> UpdateSystemGuard(bUpdatingSystem, true);
		TSharedRef<FStructOnScope> UserParameterDisplayData = DisplayData[UserParameter];

		// we have to special case the position definition because, while SetParameterData does handle Position variables, it expects 12 Bytes of data
		if(UserParameter.GetType() == FNiagaraTypeDefinition::GetPositionDef())
		{
			FVector Value = *reinterpret_cast<const FVector*>(UserParameterDisplayData->GetStructMemory());
			SystemAsset->GetExposedParameters().SetPositionParameterValue(Value, UserParameter.GetName());
		}
		else
		{
			SystemAsset->GetExposedParameters().SetParameterData(UserParameterDisplayData ->GetStructMemory(), UserParameter);
		}
	}
}

void FNiagaraSystemUserParameterBuilder::OnObjectAssetChanged(const FAssetData& NewObject, FNiagaraVariable UserParameter)
{
	UNiagaraSystem* SystemAsset = GetSystem();
	
	if(UserParameter.IsUObject() == false || UserParameter.IsDataInterface())
	{
		ensure(false);
		return;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("ChangeAsset", "Changed User Parameter Asset"));
	TGuardValue<bool> UpdateSystemGuard(bUpdatingSystem, true);
	SystemAsset->Modify();
	SystemAsset->GetExposedParameters().SetUObject(NewObject.GetAsset(), UserParameter);
}

FString FNiagaraSystemUserParameterBuilder::GetObjectAssetPathForUserParameter(FNiagaraVariable UserParameter) const
{
	ensure(UserParameter.GetType().IsUObject() && UserParameter.GetType().IsDataInterface() == false);
	FNiagaraVariant Variant = GetCurrentParameterValue(UserParameter);

	if (Variant.IsValid() == false)
	{
		return FString();
	}
	
	if (TObjectPtr<UObject> Object = Variant.GetUObject())
	{
		return Object.GetPathName();
	}
	
	return FString();
}

bool FNiagaraSystemUserParameterBuilder::IsDataInterfaceResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle, FNiagaraVariable UserParameter) const
{
	UNiagaraDataInterface* DataInterface = GetSystem()->GetExposedParameters().GetDataInterface(UserParameter);

	if(DataInterface == nullptr)
	{
		return false;
	}

	bool bIdentical = true;
	UObject* DataInterfaceCDO = DataInterface->GetClass()->GetDefaultObject();
	for(TFieldIterator<FProperty> It(DataInterface->GetClass()); It; ++It)
	{
		if(It->HasAnyPropertyFlags(CPF_Edit) == false)
		{
			continue;
		}
		
		bIdentical &= It->Identical_InContainer(DataInterface, DataInterfaceCDO, 0, PPF_DeepComparison);

		if(bIdentical == false)
		{
			break;
		}
	}

	return bIdentical == false;
}

void FNiagaraSystemUserParameterBuilder::OnHandleDataInterfaceReset(TSharedPtr<IPropertyHandle> PropertyHandle,	FNiagaraVariable UserParameter)
{
	UNiagaraDataInterface* DataInterface = GetSystem()->GetExposedParameters().GetDataInterface(UserParameter);

	if(DataInterface == nullptr)
	{
		return;
	}
	
	UObject* DataInterfaceCDO = DataInterface->GetClass()->GetDefaultObject();
	for(TFieldIterator<FProperty> It(DataInterface->GetClass()); It; ++It)
	{
		if(It->HasAnyPropertyFlags(CPF_Edit) == false)
		{
			continue;
		}
		
		It->CopyCompleteValue_InContainer(DataInterface, DataInterfaceCDO);
	}

	// to clean up reset widgets for child properties, 
	Rebuild();
}

bool FNiagaraSystemUserParameterBuilder::IsLocalPropertyResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle, FNiagaraVariable UserParameter) const
{
	if(UserParameter.GetType() == FNiagaraTypeDefinition::GetPositionDef())
	{
		static UPackage* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
		static UScriptStruct* VectorStruct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector"));

		TSharedRef<FStructOnScope> VectorData = MakeShared<FStructOnScope>(VectorStruct);
		VectorStruct->InitializeDefaultValue(VectorData->GetStructMemory());
		return FMemory::Memcmp(DisplayData[UserParameter]->GetStructMemory(), VectorData->GetStructMemory(), VectorStruct->GetStructureSize()) != 0;
	}
	
	FNiagaraEditorUtilities::ResetVariableToDefaultValue(UserParameter);
	return FMemory::Memcmp(DisplayData[UserParameter]->GetStructMemory(), UserParameter.GetData(), UserParameter.GetType().GetSize()) != 0;
}

void FNiagaraSystemUserParameterBuilder::OnHandleLocalPropertyReset(TSharedPtr<IPropertyHandle> PropertyHandle, FNiagaraVariable UserParameter)
{
	FScopedTransaction Transaction(ResetUserParameterTransactionText);
	GetSystem()->Modify();
	
	FNiagaraTypeDefinition Type = UserParameter.GetType();

	UStruct* Struct = Type.GetStruct();
	TSharedPtr<FStructOnScope> DefaultData;
	if(UserParameter.GetType() == FNiagaraTypeDefinition::GetPositionDef())
	{
		static UPackage* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
		static UScriptStruct* VectorStruct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector"));

		DefaultData = MakeShared<FStructOnScope>(VectorStruct);
		VectorStruct->InitializeDefaultValue(DefaultData->GetStructMemory());
		Struct = VectorStruct;

		FVector* Vector = reinterpret_cast<FVector*>(DefaultData->GetStructMemory());
		FVector* CurrentVector = reinterpret_cast<FVector*>(DisplayData[UserParameter]->GetStructMemory());

		UE_LOGF(LogNiagaraEditor, Warning, "Current: %ls", *CurrentVector->ToString());
		UE_LOGF(LogNiagaraEditor, Warning, "Default: %ls", *Vector->ToString());

	}
	else
	{
		FNiagaraEditorUtilities::ResetVariableToDefaultValue(UserParameter);
		DefaultData = MakeShared<FStructOnScope>(Struct, UserParameter.GetData());
	}
	
	
	DisplayDataPropertyHandles[UserParameter]->NotifyPreChange();
	FMemory::Memcpy(DisplayData[UserParameter]->GetStructMemory(), DefaultData->GetStructMemory(), Struct->GetStructureSize());
	DisplayDataPropertyHandles[UserParameter]->NotifyPostChange(EPropertyChangeType::ValueSet);
	DisplayDataPropertyHandles[UserParameter]->NotifyFinishedChangingProperties();
}

bool FNiagaraSystemUserParameterBuilder::IsObjectAssetCustomRowResetToDefaultVisible(FNiagaraVariable UserParameter) const
{	
	return GetSystem()->GetExposedParameters().GetUObject(UserParameter) != nullptr;
}

void FNiagaraSystemUserParameterBuilder::OnHandleObjectAssetReset(FNiagaraVariable UserParameter)
{
	FScopedTransaction Transaction(ResetUserParameterTransactionText);
	TGuardValue<bool> UpdateSystemGuard(bUpdatingSystem, true);
	GetSystem()->Modify();

	GetSystem()->GetExposedParameters().SetUObject(nullptr, UserParameter);
}

void FNiagaraSystemUserParameterBuilder::OnDataInterfacePropertyValuePreChange()
{
	GetSystem()->Modify();
}

void FNiagaraSystemUserParameterBuilder::OnDataInterfacePropertyValueChangedWithData(const FPropertyChangedEvent& PropertyChangedEvent)
{
	GetSystem()->GetExposedParameters().OnInterfaceChange();
}

bool FNiagaraSystemUserParameterBuilder::IsParameterExposedToVerse(FNiagaraVariable UserParameter) const
{
	return GetParameterAccessLevel(UserParameter) != ENiagaraParameterAccessLevel::Private;
}

ENiagaraParameterAccessLevel FNiagaraSystemUserParameterBuilder::GetParameterAccessLevel(FNiagaraVariable UserParameter) const
{
	TOptional<ENiagaraParameterAccessLevel> AccessValue;
	if ( VerseReflectionUtilities )
	{
		UserParameter.RemoveRootNamespace(FNiagaraConstants::UserNamespaceString);
		if (!VerseReflectionUtilities->IsValidVariable(UserParameter).IsSet())
		{
			return ENiagaraParameterAccessLevel::Public;
		}
	}
	return AccessValue.Get(ENiagaraParameterAccessLevel::Private);
}

const FSlateBrush* FNiagaraSystemUserParameterBuilder::GetParameterAccessLevelBrush(FNiagaraVariable UserParameter) const
{
	switch (GetParameterAccessLevel(UserParameter))
	{
		case ENiagaraParameterAccessLevel::Public:
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Verse.Public");

		default:
		case ENiagaraParameterAccessLevel::Private:
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Verse.Private");
	}
}

FText FNiagaraSystemUserParameterBuilder::GetParameterAccessLevelTooltip(FNiagaraVariable UserParameter) const
{
	if (VerseReflectionUtilities)
	{
		UserParameter.RemoveRootNamespace(FNiagaraConstants::UserNamespaceString);
		TOptional<FText> ErrorText = VerseReflectionUtilities->IsValidVariable(UserParameter);
		if (ErrorText.IsSet())
		{
			return ErrorText.GetValue();
		}
	}

	const UEnum* Enum = StaticEnum<ENiagaraParameterAccessLevel>();
	return Enum ? Enum->GetToolTipTextByIndex(static_cast<int32>(GetParameterAccessLevel(UserParameter))) : FText();
}

bool FNiagaraSystemUserParameterBuilder::CanExposeParameterToVerse(FNiagaraVariable UserParameter) const
{
	return VerseReflectionUtilities ? !VerseReflectionUtilities->IsValidVariable(UserParameter).IsSet() : false;
}

void FNiagaraSystemUserParameterBuilder::SystemExposedParametersChanged()
{
	if (bUpdatingSystem == false)
	{
		Rebuild();
	}
}

#undef LOCTEXT_NAMESPACE
