// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraUserParametersBuilderBase.h"
#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "DataHierarchyViewModelBase.h"
#include "ViewModels/HierarchyEditor/NiagaraUserParametersHierarchyViewModel.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class FNiagaraParameterViewModelCustomDetails;
class INiagaraParameterViewModel;
class SNiagaraParameterEditor;
class FNiagaraParameterViewModelCustomDetails;
class FNiagaraSimCacheCapture;
class UNiagaraSystem;
class INiagaraVerseReflectionUtilities;

/** A specialized class to generate user parameters for a system asset's editor. Due to reliance on the system view model this should only be used within the asset editor. */
class FNiagaraSystemUserParameterBuilder : public FNiagaraUserParameterNodeBuilder
{
public:
	FNiagaraSystemUserParameterBuilder(TSharedPtr<FNiagaraSystemViewModel> InSystemViewModel, FName InCustomBuilderRowName);
	virtual ~FNiagaraSystemUserParameterBuilder() override;

	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	
	TSharedRef<SWidget> GetAdditionalHeaderWidgets();
private:
	/** NiagaraUserParameterNodeBuilder Interface */
	virtual UNiagaraSystem* GetSystem() const override;
	virtual FNiagaraVariant GetCurrentParameterValue(const FNiagaraVariableBase& UserParameter) const override;
	virtual void GenerateRowForUserParameterInternal(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable UserParameter) override;
	virtual void AddCustomMenuActionsForParameter(FDetailWidgetRow& WidgetRow, FNiagaraVariable UserParameter) override;
	virtual void OnParameterEditorValueChanged(FNiagaraVariable UserParameter) override;
	virtual void OnObjectAssetChanged(const FAssetData& NewObject, FNiagaraVariable UserParameter) override;

	void OnDisplayDataPostChange(const FPropertyChangedEvent& ChangedEvent, FNiagaraVariable UserParameter);

	TSharedRef<SWidget> GetAddParameterMenu();
	void AddParameter(FNiagaraVariable NewParameter) const;
	void OnParameterAdded(FNiagaraVariable UserParameter);
	bool OnAllowMakeType(const FNiagaraTypeDefinition& InType) const;
	void DeleteParameter(FNiagaraVariable UserParameter);

	virtual TSharedRef<SWidget> CreateUserParameterNameWidget(FNiagaraVariable UserParameter) override;
	
	FReply SummonHierarchyEditor() const;
	void SelectAllSection();
	
	void RequestRename(FNiagaraVariable UserParameter);
	void RenameParameter(FNiagaraVariable UserParameter, FName NewName);

	void RequestDuplication(FNiagaraVariable NiagaraVariable);

	FReply GenerateParameterDragDropOp(const FGeometry& Geometry, const FPointerEvent& MouseEvent, FNiagaraVariable UserParameter) const;
private:
	FString GetObjectAssetPathForUserParameter(FNiagaraVariable UserParameter) const;

	bool IsDataInterfaceResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle, FNiagaraVariable UserParameter) const;
	void OnHandleDataInterfaceReset(TSharedPtr<IPropertyHandle> PropertyHandle, FNiagaraVariable UserParameter);
	
	/** These can be used for Reset to Default overrides that are added to rows with valid property nodes (external properties etc.) */
	bool IsLocalPropertyResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle, FNiagaraVariable UserParameter) const;
	void OnHandleLocalPropertyReset(TSharedPtr<IPropertyHandle> PropertyHandle, FNiagaraVariable UserParameter);

	/** These can be used for Reset to Default overrides that are added to custom rows (no valid property nodes) */
	bool IsObjectAssetCustomRowResetToDefaultVisible(FNiagaraVariable UserParameter) const;
	void OnHandleObjectAssetReset(FNiagaraVariable UserParameter);
	
	void OnDataInterfacePropertyValuePreChange();
	void OnDataInterfacePropertyValueChangedWithData(const FPropertyChangedEvent& PropertyChangedEvent);
	
	bool IsParameterExposedToVerse(FNiagaraVariable UserParameter) const;
	ENiagaraParameterAccessLevel GetParameterAccessLevel(FNiagaraVariable UserParameter) const;
	const FSlateBrush* GetParameterAccessLevelBrush(FNiagaraVariable UserParameter) const;
	FText GetParameterAccessLevelTooltip(FNiagaraVariable UserParameter) const;
	bool CanExposeParameterToVerse(FNiagaraVariable UserParameter) const;

	void SystemExposedParametersChanged();

private:
	TWeakObjectPtr<UNiagaraSystem> System;
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;
	TMap<FNiagaraVariable, TSharedPtr<class SNiagaraParameterNameTextBlock>> UserParamToWidgetMap;
	TSharedPtr<SWidget> AdditionalHeaderWidgetsContainer;
	TSharedPtr<class SComboButton> AddParameterButton;
	TSharedPtr<class SNiagaraAddParameterFromPanelMenu> AddParameterMenu;
	const INiagaraVerseReflectionUtilities* VerseReflectionUtilities = nullptr;

	FDelegateHandle SystemExposedParametersOnChangedHandle;
	mutable bool bUpdatingSystem = false;
};
