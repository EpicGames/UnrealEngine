// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraUserParametersBuilderBase.h"
#include "PropertyCustomizationHelpers.h"
#include "NiagaraTypes.h"
#include "IDetailChildrenBuilder.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraClipboard.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSettings.h"
#include "NiagaraUserRedirectionParameterStore.h"
#include "ViewModels/HierarchyEditor/NiagaraUserParametersHierarchyViewModel.h"
#include "Widgets/SNiagaraDebugger.h"
#include "Widgets/SNiagaraSystemUserParameters.h"
#include "Widgets/Input/SCheckBox.h"
#include "NiagaraEditorDataBase.h"
#include "NiagaraEditorUtilities.h"
#include "PropertyEditorClipboard.h"
#include "Customizations/NiagaraComponentDetails.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "NiagaraUserParametersBuilderBase"

FText FNiagaraUserParameterNodeBuilder::ResetUserParameterTransactionText = LOCTEXT("ResetUserParameter", "User Parameter reset to default value");
FText FNiagaraUserParameterNodeBuilder::ChangedUserParameterTransactionText = LOCTEXT("EditUserParameter", "Edit User Parameter");

FNiagaraVariant FNiagaraUserParameterNodeBuilder::GetParameterValueFromSystem(const FNiagaraVariableBase& Parameter, const UNiagaraSystem& System)
{
	const FNiagaraUserRedirectionParameterStore& UserParameterStore = System.GetExposedParameters();
	if (Parameter.IsDataInterface())
	{
		int32 Index = UserParameterStore.IndexOf(Parameter);
		if (Index != INDEX_NONE)
		{
			return FNiagaraVariant(UserParameterStore.GetDataInterfaces()[Index]);
		}
	}
	
	if (Parameter.IsUObject())
	{
		int32 Index = UserParameterStore.IndexOf(Parameter);
		if (Index != INDEX_NONE)
		{
			return FNiagaraVariant(UserParameterStore.GetUObjects()[Index]);
		}
	}

	if (Parameter.GetType() == FNiagaraTypeDefinition::GetPositionDef())
	{
		const FVector* Value = UserParameterStore.GetPositionParameterValue(Parameter.GetName());
		return Value == nullptr ? FNiagaraVariant() : FNiagaraVariant(Value, sizeof(FVector));
	}

	TArray<uint8> DataValue;
	DataValue.AddUninitialized(Parameter.GetSizeInBytes());
	if (UserParameterStore.CopyParameterData(Parameter, DataValue.GetData()))
	{
		return FNiagaraVariant(DataValue);
	}
	return FNiagaraVariant();
}

IDetailPropertyRow* FNiagaraUserParameterNodeBuilder::AddValueParameterAsRow(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable ChoppedUserParameter)
{
	FNiagaraVariant CurrentParameterValue = GetCurrentParameterValue(ChoppedUserParameter);
	if(!ensure(CurrentParameterValue.GetMode() == ENiagaraVariantMode::Bytes))
	{
		return nullptr;
	}
	
	FNiagaraTypeDefinition ParameterType = ChoppedUserParameter.GetType();
	UStruct* Struct = ParameterType.GetStruct();

	// we don't want to create niagara editors for certain types.
	// - Position types would use the Vector3f widget right now which wouldn't give us double precision
	bool bShouldCreateNiagaraEditor = ParameterType != FNiagaraTypeDefinition::GetPositionDef();

	// for position types, we want to display an FVector instead of the FVector3f that the position actually is. So we replace the struct here
    if(ParameterType == FNiagaraTypeDefinition::GetPositionDef())
    {
    	static UPackage* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));
    	static UScriptStruct* VectorStruct = FindObjectChecked<UScriptStruct>(CoreUObjectPkg, TEXT("Vector"));
    	Struct = VectorStruct;
    }
	
	TSharedRef<FStructOnScope> CurrentParameterValueData = MakeShared<FStructOnScope>(Struct);
	FMemory::Memcpy(CurrentParameterValueData->GetStructMemory(), CurrentParameterValue.GetBytes(), Struct->GetStructureSize());

	DisplayData.Add(ChoppedUserParameter, CurrentParameterValueData);
	
	if(bShouldCreateNiagaraEditor)
	{
		UNiagaraScriptVariable* UserParameterScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(ChoppedUserParameter, *GetSystem());
		FNiagaraEditorModule& NiagaraEditorModule = FNiagaraEditorModule::Get();
		if(TSharedPtr<INiagaraEditorTypeUtilities> TypeUtilities = NiagaraEditorModule.GetTypeUtilities(ParameterType))
		{
			if(TypeUtilities->CanCreateParameterEditor())
			{
				TSharedPtr<SNiagaraParameterEditor> ParameterEditor = TypeUtilities->CreateParameterEditor(ParameterType, UserParameterScriptVariable->Metadata.DisplayUnit, UserParameterScriptVariable->Metadata.WidgetCustomization).ToSharedRef();
				// since the color editor is in a wrap box using the allotted size, we have to make sure to give it plenty of space so it doesn't start wrapping immediately
				if(ParameterType == FNiagaraTypeDefinition::GetColorDef())
				{
					ParameterEditor->SetMinimumDesiredWidth(350.f);
				}
				ParameterEditor->UpdateInternalValueFromStruct(CurrentParameterValueData);
				ParameterEditor->SetOnEndValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &FNiagaraUserParameterNodeBuilder::OnParameterEditorValueChanged, ChoppedUserParameter));			
				
				NiagaraParameterEditors.Add(ChoppedUserParameter, ParameterEditor.ToSharedRef());
			}
		}
	}
	
	FAddPropertyParams Params;
	Params.UniqueId(ChoppedUserParameter.GetName());

	/** We add an external property here, pointing at the display data we have created
	 *  Data can be written in three ways:
	 *  1) Using the custom parameter editor. This requires syncing from parameter editor -> display data -> user parameter store
	 *  2) Using copy & paste. This requires syncing from display data -> parameter editor -> user parameter store
	 *  3) Using reset to default. This requires syncing the default value to display data -> parameter editor -> user parameter store
	 */
	IDetailPropertyRow* DetailPropertyRow = ChildrenBuilder.AddExternalStructure(DisplayData[ChoppedUserParameter], ChoppedUserParameter.GetName());
	DisplayDataPropertyHandles.Add(ChoppedUserParameter, DetailPropertyRow->GetPropertyHandle());
	
	TSharedPtr<SWidget> DefaultNameWidget;
	TSharedPtr<SWidget> DefaultValueWidget;
	DetailPropertyRow->GetDefaultWidgets(DefaultNameWidget, DefaultValueWidget);

	bool bHasCustomEditor = NiagaraParameterEditors.Contains(ChoppedUserParameter);
	FDetailWidgetRow& DetailWidgetRow = DetailPropertyRow->CustomWidget(!bHasCustomEditor);
	DetailWidgetRow
	.FilterString(FText::FromName(ChoppedUserParameter.GetName()))
	.NameContent()
	[
		CreateUserParameterNameWidget(ChoppedUserParameter)
	]
	.ValueContent()
	[
		bHasCustomEditor ? NiagaraParameterEditors[ChoppedUserParameter] : DefaultValueWidget.ToSharedRef()
	];

	AddCustomMenuActionsForParameter(DetailWidgetRow, ChoppedUserParameter);
	
	return DetailPropertyRow;
}

IDetailPropertyRow* FNiagaraUserParameterNodeBuilder::AddObjectAssetParameterAsRow(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable ChoppedUserParameter)
{
	UObject* AssetHelper = ObjectAssetHelpers.Add_GetRef(NewObject<UNiagaraObjectAssetHelper>());

	FAddPropertyParams AddPropertyParams;
	AddPropertyParams.UniqueId(ChoppedUserParameter.GetName());
	return ChildrenBuilder.AddExternalObjectProperty({AssetHelper}, FName("Path"), AddPropertyParams);
}

void FNiagaraUserParameterNodeBuilder::GenerateUserParameterRows(IDetailChildrenBuilder& ChildrenBuilder, UHierarchyRoot& UserParameterHierarchyRoot)
{
	UNiagaraSystem* SystemAsset = GetSystem();

	if(SystemAsset == nullptr)
	{
		return;
	}

	TArray<const UHierarchySection*> OrderedSections;
	TMap<const UHierarchySection*, TArray<const UHierarchyCategory*>> SectionCategoryMap;

	UHierarchyRoot* Root = &UserParameterHierarchyRoot;
	for(const UHierarchyElement* Child : Root->GetChildren())
	{
		if(const UHierarchyCategory* HierarchyCategory = Cast<UHierarchyCategory>(Child))
		{
			const FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchyCategory->FindMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
			if(SectionAssociation && SectionAssociation->Section.IsValid())
			{
				SectionCategoryMap.FindOrAdd(SectionAssociation->Section.Get()).Add(HierarchyCategory);	
			}
		}
	}

	// we create a custom row for user param sections here, but only if we have at least one section specified
	if(SectionCategoryMap.Num() > 0)
	{
		TSharedRef<SWrapBox> SectionsBox = SNew(SWrapBox).UseAllottedSize(true);

		// maps don't guarantee order, so we iterate over the original section data instead
		for(const TObjectPtr<UHierarchySection>& Section : Root->GetSectionData())
		{
			// if the map doesn't contain at the entry, it means there are 0 categories for that section, so we skip it
			if(!SectionCategoryMap.Contains(Section))
			{
				continue;
			}
			
			TArray<const UHierarchyCategory*>& HierarchyCategories = SectionCategoryMap[Section];
			
			if(HierarchyCategories.Num() > 0)
			{
				// we only want to add a section if any of its categories contain at least one user parameter
				bool bDoesUserParamExist = false;
				for(const UHierarchyCategory* Category : HierarchyCategories)
				{
					bDoesUserParamExist |= Category->DoesOneChildExist<UNiagaraHierarchyUserParameter>(true);

					if(bDoesUserParamExist)
					{
						break;
					}
				}

				if(bDoesUserParamExist)
				{
					SectionsBox->AddSlot()
					.Padding(FMargin(2.f))
					[
						SNew(SBox)
						.Padding(FMargin(0.f, 4.f, 0.f, 0.f))
						[
							SNew(SCheckBox)
							.Style(FAppStyle::Get(), "DetailsView.SectionButton")
							.OnCheckStateChanged_Lambda([this, Section](ECheckBoxState NewState)
							{
								ActiveSection = Section;
								OnRebuildChildren.ExecuteIfBound();
							})
							.IsChecked_Lambda([this, Section]()
							{
								return ActiveSection == Section ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							})
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "SmallText")
								.Text_UObject(Section.Get(), &UHierarchySection::GetSectionNameAsText)
								.ToolTipText_UObject(Section.Get(), &UHierarchySection::GetTooltip)
							]
						]
					];
				}
			}
		}

		// if we have at least one custom section, we add a default "All" section
		if(SectionsBox->GetChildren()->Num() > 0)
		{
			SectionsBox->AddSlot()
			.Padding(FMargin(2.f))
			[
				SNew(SBox)
				.Padding(FMargin(0.f, 4.f, 0.f, 0.f))
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "DetailsView.SectionButton")
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
					{
						ActiveSection = nullptr;
						OnRebuildChildren.ExecuteIfBound();
					})
					.IsChecked_Lambda([this]()
					{
						return ActiveSection == nullptr ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "SmallText")
						.Text(FText::FromString("All"))
					]
				]
			];
			
			FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(FText::FromString("User Parameter Sections"));
			Row.RowTag(FName("User Parameter Sections"));
			
			Row.WholeRowContent()
			[
				SectionsBox
			];
		}
	}
	
	// we add the categories now if the active section is set to all (nullptr) or if the section it belongs to is active
	for(const UHierarchyElement* Child : Root->GetChildren())
	{		
		if(const UHierarchyCategory* HierarchyCategory = Cast<UHierarchyCategory>(Child))
		{
			const UHierarchySection* Section = nullptr;
			if(const FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchyCategory->FindMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>())
			{
				if(SectionAssociation->Section.IsValid())
				{
					Section = SectionAssociation->Section.Get();
				}
			}
			
			if((ActiveSection == nullptr || Section == ActiveSection) && HierarchyCategory->DoesOneChildExist<UNiagaraHierarchyUserParameter>(true))
			{
				GenerateCategory(ChildrenBuilder, *HierarchyCategory);
			}
		}
		else if(const UNiagaraHierarchyUserParameter* UserParameter = Cast<UNiagaraHierarchyUserParameter>(Child))
		{
			// we only want to display parameters within the root in the all section
			if(ActiveSection == nullptr)
			{
				GenerateRowForUserParameter(ChildrenBuilder, UserParameter->GetUserParameter());
			}
		}
	}

	// at last we add all parameters that haven't been setup at all, if the active section is set to "All"
	if(ActiveSection == nullptr)
	{
		TArray<FNiagaraVariable> LeftoverUserParameters;
		SystemAsset->GetExposedParameters().GetUserParameters(LeftoverUserParameters);

		TArray<UNiagaraHierarchyUserParameter*> HierarchyUserParameters;
		Root->GetChildrenOfType<>(HierarchyUserParameters, true);

		TArray<FNiagaraVariable> RedirectedUserParameters;
		for(FNiagaraVariable& Parameter : LeftoverUserParameters)
		{
			SystemAsset->GetExposedParameters().RedirectUserVariable(Parameter);
		}
		
		for(UNiagaraHierarchyUserParameter* HierarchyUserParameter : HierarchyUserParameters)
		{
			if(LeftoverUserParameters.Contains(HierarchyUserParameter->GetUserParameter()))
			{
				LeftoverUserParameters.Remove(HierarchyUserParameter->GetUserParameter());
			}
		}

		for(FNiagaraVariable& LeftoverUserParameter : LeftoverUserParameters)
		{			
			GenerateRowForUserParameter(ChildrenBuilder, LeftoverUserParameter);
		}
	}
}

void FNiagaraUserParameterNodeBuilder::GenerateRowForUserParameter(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable UserParameter)
{
	GenerateRowForUserParameterInternal(ChildrenBuilder, UserParameter);
}

void FNiagaraUserParameterNodeBuilder::GenerateCategory(IDetailChildrenBuilder& ChildrenBuilder, const UHierarchyCategory& HierarchyCategory)
{
	ChildrenBuilder.AddCustomBuilder(MakeShared<FNiagaraUserParameterCategoryBuilder>(HierarchyCategory, HierarchyCategory.GetCategoryName(), *this));
}

void FNiagaraUserParameterNodeBuilder::AddCustomMenuActionsForParameter(FDetailWidgetRow& WidgetRow, FNiagaraVariable UserParameter)
{
	if(UserParameter.IsDataInterface())
	{
		FUIAction CopyAction;
		CopyAction.ExecuteAction = FExecuteAction::CreateLambda([this, UserParameter]()
		{
			FNiagaraVariant CurrentParameterValue = GetCurrentParameterValue(UserParameter);
			if(CurrentParameterValue.IsValid() == false || CurrentParameterValue.GetDataInterface() == nullptr)
			{
				return;
			}
			
			UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
			const UNiagaraClipboardFunctionInput* Input = UNiagaraClipboardFunctionInput::CreateDataValue(ClipboardContent, UserParameter.GetName(), UserParameter.GetType().GetClass(), TOptional<bool>(), CurrentParameterValue.GetDataInterface());
			ClipboardContent->FunctionInputs.Add(Input);
			
			FNiagaraEditorModule::Get().GetClipboard().SetClipboardContent(ClipboardContent);
		});
		WidgetRow.CopyAction(CopyAction);

		FUIAction PasteAction;
		PasteAction.ExecuteAction = FExecuteAction::CreateLambda([this, UserParameter]()
		{
			FNiagaraVariant CurrentParameterValue = GetCurrentParameterValue(UserParameter);
			if(CurrentParameterValue.IsValid() == false || CurrentParameterValue.GetDataInterface() == nullptr)
			{
				return;
			}
			
			const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent();

			if(ClipboardContent->FunctionInputs.Num() == 1 && ClipboardContent->FunctionInputs[0]->InputType.IsDataInterface())
			{
				UNiagaraDataInterface* CopiedDI = ClipboardContent->FunctionInputs[0]->Data;
				if(CopiedDI->IsA(CurrentParameterValue.GetDataInterface()->GetClass()))
				{
					CopiedDI->CopyTo(CurrentParameterValue.GetDataInterface());
				}
			}			
		});
		WidgetRow.PasteAction(PasteAction);
	}
	else if(UserParameter.IsUObject())
	{
		FUIAction CopyAction;
		CopyAction.ExecuteAction = FExecuteAction::CreateLambda([this, UserParameter]()
		{
			FNiagaraVariant CurrentParameterValue = GetCurrentParameterValue(UserParameter);
			if(CurrentParameterValue.IsValid() == false)
			{
				return;
			}

			FPropertyEditorClipboard::ClipboardCopy(*CurrentParameterValue.GetUObject().GetPathName());
		});
		WidgetRow.CopyAction(CopyAction);

		FUIAction PasteAction;
		PasteAction.ExecuteAction = FExecuteAction::CreateLambda([this, UserParameter]()
		{
			FString PastedString;
			FPropertyEditorClipboard::ClipboardPaste(PastedString);

			FSoftObjectPath Path(PastedString);
			if(Path.IsAsset())
			{
				UObject* Asset = Path.TryLoad();
				OnObjectAssetChanged(FAssetData(Asset), UserParameter);
			}
			else
			{
				OnObjectAssetChanged(FAssetData(nullptr), UserParameter);
			}
		});
		WidgetRow.PasteAction(PasteAction);
	}
}

TSharedRef<SWidget> FNiagaraUserParameterNodeBuilder::CreateUserParameterNameWidget(FNiagaraVariable UserParameter)
{
	UNiagaraScriptVariable* ScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(UserParameter, *GetSystem());
	TAttribute<FText> Tooltip = FText::GetEmpty();
	if(ScriptVariable)
	{
		Tooltip = TAttribute<FText>::CreateLambda([ScriptVariable]
		{
			return ScriptVariable->Metadata.Description;
		});
	}
	
	return SNew(SBox)
	.ToolTipText(Tooltip)
	.Padding(FMargin(0.0f, 2.0f))
	[
		SNew(STextBlock)
		.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
		.Text(FText::FromName(UserParameter.GetName()))
	];
}

void FNiagaraUserParameterNodeBuilder::OnScriptVariablePropertyChanged(UNiagaraScriptVariable* ScriptVariable, const FPropertyChangedEvent& PropertyChangedEvent) const
{
	if(ScriptVariable->GetOuter() == GetSystem()->GetEditorData())
	{
		Rebuild();
	}
}

void FNiagaraUserParameterNodeBuilder::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ObjectAssetHelpers);
}

FString FNiagaraUserParameterNodeBuilder::GetReferencerName() const
{
	return "UserParametersBuilder";
}

FNiagaraUserParameterCategoryBuilder::FNiagaraUserParameterCategoryBuilder(const UHierarchyCategory& InNiagaraHierarchyCategory, FName InCustomBuilderRowName, FNiagaraUserParameterNodeBuilder& InNodeBuilder) : NodeBuilder(InNodeBuilder)
{
	HierarchyCategory = &InNiagaraHierarchyCategory;
	CustomBuilderRowName = InCustomBuilderRowName;
}

void FNiagaraUserParameterCategoryBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.NameContent()
	[
		SNew(STextBlock)
		.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
		.Text_UObject(HierarchyCategory.Get(), &UHierarchyCategory::GetCategoryAsText)
		.ToolTipText_UObject(HierarchyCategory.Get(), &UHierarchyCategory::GetTooltip)
	];

	NodeRow.FilterString(FText::FromName(CustomBuilderRowName));
}

void FNiagaraUserParameterCategoryBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	TArray<const UNiagaraHierarchyUserParameter*> ContainedParameters;
	TArray<const UHierarchyCategory*> ContainedCategories;

	for(const UHierarchyElement* Child : HierarchyCategory->GetChildren())
	{
		if(const UNiagaraHierarchyUserParameter* UserParameter = Cast<UNiagaraHierarchyUserParameter>(Child))
		{
			ContainedParameters.Add(UserParameter);
		}
	}

	for(UHierarchyElement* Child : HierarchyCategory->GetChildren())
	{
		if(UHierarchyCategory* Category = Cast<UHierarchyCategory>(Child))
		{
			ContainedCategories.Add(Category);
		}
	}

	for(const UHierarchyCategory* Category : ContainedCategories)
	{
		if(Category->DoesOneChildExist<UNiagaraHierarchyUserParameter>(true))
		{
			NodeBuilder.GenerateCategory(ChildrenBuilder, *Category);
		}
	}
		
	for(const UNiagaraHierarchyUserParameter* HierarchyUserParameter : ContainedParameters)
	{
		NodeBuilder.GenerateRowForUserParameter(ChildrenBuilder, HierarchyUserParameter->GetUserParameter());
	}
}

#undef LOCTEXT_NAMESPACE
