// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomNodeBuilder.h"
#include "NiagaraComponent.h"
#include "SNiagaraParameterEditor.h"
#include "DataHierarchyViewModelBase.h"

class UNiagaraScriptVariable;

/** A base class for generating rows based on a given hierarchy root that contains sections, categories and user parameters. */
class FNiagaraUserParameterNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FNiagaraUserParameterNodeBuilder>, FSelfRegisteringEditorUndoClient, FGCObject
{
public:
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override {}
	virtual bool InitiallyCollapsed() const override { return false; }
	
	virtual FName GetName() const  override
	{
		return CustomBuilderRowName;
	}
	
	void Rebuild() const
	{
		OnRebuildChildren.ExecuteIfBound();
	}
	
	void RebuildOnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload) const
	{
		Rebuild();
	}
	
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override
	{
		OnRebuildChildren = InOnRegenerateChildren;
	}
	
	/** Generates all rows for the node builder. Includes setting up sections, categories and user parameters */
	void GenerateUserParameterRows(IDetailChildrenBuilder& ChildrenBuilder, UHierarchyRoot& UserParameterHierarchyRoot);
	/** Generates a single row for a specific user parameter. */
	void GenerateRowForUserParameter(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable UserParameter);
	/** Generates a category for a given hierarchy category. */
	void GenerateCategory(IDetailChildrenBuilder& ChildrenBuilder, const UHierarchyCategory& HierarchyCategory);

	void OnActiveSectionChanged(ECheckBoxState State, const UHierarchySection* NewSelection)
	{
		ActiveSection = NewSelection;
		OnRebuildChildren.Execute();
	}

	ECheckBoxState IsSectionActive(const UHierarchySection* Section) const
	{
		return ActiveSection == Section ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	static FText ResetUserParameterTransactionText;
	static FText ChangedUserParameterTransactionText;	
protected:

	virtual UNiagaraSystem* GetSystem() const = 0;
	virtual FNiagaraVariant GetCurrentParameterValue(const FNiagaraVariableBase& UserParameter) const = 0;
	/** Allows the addition of right-click context actions for a specific user parameter */
	virtual void AddCustomMenuActionsForParameter(FDetailWidgetRow& WidgetRow, FNiagaraVariable UserParameter);
	virtual void GenerateRowForUserParameterInternal(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable UserParameter) = 0;
	virtual void OnParameterEditorValueChanged(FNiagaraVariable UserParameter) = 0;
	/** Since object asset references are handled individually via custom rows, they need to implement this themselves. This will be called with copy & paste. */
	virtual void OnObjectAssetChanged(const FAssetData& NewObject, FNiagaraVariable UserParameter) = 0;

	static FNiagaraVariant GetParameterValueFromSystem(const FNiagaraVariableBase& UserParameter, const UNiagaraSystem& System);
	
	/** Main function to generate a value parameter row. Sets up the parameter editor & value changed callback, display data & property handle. */
	class IDetailPropertyRow* AddValueParameterAsRow(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable ChoppedUserParameter);
	class IDetailPropertyRow* AddObjectAssetParameterAsRow(IDetailChildrenBuilder& ChildrenBuilder, FNiagaraVariable ChoppedUserParameter);
	
	virtual TSharedRef<SWidget> CreateUserParameterNameWidget(FNiagaraVariable UserParameter);
	
	virtual void PostUndo(bool bSuccess) override { Rebuild(); }
	virtual void PostRedo(bool bSuccess) override { Rebuild(); }

	void OnScriptVariablePropertyChanged(UNiagaraScriptVariable* ScriptVariable, const FPropertyChangedEvent& PropertyChangedEvent) const;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
protected:
	bool bDelegatesInitialized = false;
	FSimpleDelegate OnRebuildChildren;
	FName CustomBuilderRowName;
	const UHierarchySection* ActiveSection = nullptr;
	TArray<UHierarchySection*> AvailableSections;
	TMap<FNiagaraVariable, TSharedRef<SNiagaraParameterEditor>> NiagaraParameterEditors;
	/** The struct on scopes contained here are owned by the builder. When parameter editor value changes, they get copied into the display data, and from there into the parameter store.
	 * This is done because the types the parameter editors operate on are not the same as the struct on scopes we are editing (bool vs. FNiagaraBool etc.) */
	TMap<FNiagaraVariable, TSharedRef<FStructOnScope>> DisplayData;
	TMap<FNiagaraVariable, TSharedPtr<IPropertyHandle>> DisplayDataPropertyHandles;
	TArray<TObjectPtr<UObject>> ObjectAssetHelpers;
};

/** The category builder will display all sub-categories & parameters contained within a given category in the details panel. */
class FNiagaraUserParameterCategoryBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FNiagaraUserParameterCategoryBuilder>
{
public:
	FNiagaraUserParameterCategoryBuilder(const UHierarchyCategory& InNiagaraHierarchyCategory, FName InCustomBuilderRowName, FNiagaraUserParameterNodeBuilder& InNodeBuilder);
	
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	
	virtual bool InitiallyCollapsed() const override { return false; }

	virtual FName GetName() const  override
	{
		return CustomBuilderRowName;
	}
	
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;

private:
	TWeakObjectPtr<const UHierarchyCategory> HierarchyCategory;
	FName CustomBuilderRowName;
	FNiagaraUserParameterNodeBuilder& NodeBuilder;
};
