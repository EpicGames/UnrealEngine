// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphSchema_Niagara.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor/AssetReferenceFilter.h"
#include "EdGraphNode_Comment.h"
#include "GraphEditorSettings.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraCommon.h"
#include "NiagaraComponent.h"
#include "NiagaraConstants.h"
#include "NiagaraDataInterface.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraGraph.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraNodeConvert.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOp.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeOutputTag.h"
#include "NiagaraNodeParameterMapFor.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeReadDataSet.h"
#include "NiagaraNodeReroute.h"
#include "NiagaraNodeSelect.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraNodeWriteDataSet.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSettings.h"
#include "ObjectEditorUtils.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "Settings/EditorStyleSettings.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "UObject/LinkerLoad.h"
#include "Algo/Sort.h"
#include "Config/NiagaraFavoriteActionsConfig.h"
#include "NiagaraVariant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphSchema_Niagara)

#define LOCTEXT_NAMESPACE "NiagaraSchema"

const FLinearColor UEdGraphSchema_Niagara::NodeTitleColor_Attribute = FLinearColor::Green;
const FLinearColor UEdGraphSchema_Niagara::NodeTitleColor_Constant = FLinearColor::Red;
const FLinearColor UEdGraphSchema_Niagara::NodeTitleColor_SystemConstant = FLinearColor::White;
const FLinearColor UEdGraphSchema_Niagara::NodeTitleColor_FunctionCall = FLinearColor::Blue;
const FLinearColor UEdGraphSchema_Niagara::NodeTitleColor_CustomHlsl = FLinearColor::Yellow;
const FLinearColor UEdGraphSchema_Niagara::NodeTitleColor_Event = FLinearColor::Red;
const FLinearColor UEdGraphSchema_Niagara::NodeTitleColor_TranslatorConstant = FLinearColor::Gray;
const FLinearColor UEdGraphSchema_Niagara::NodeTitleColor_RapidIteration = FLinearColor::Black;

const FText UEdGraphSchema_Niagara::ReplaceExistingInputConnectionsText = LOCTEXT("BreakExistingConnectionsText", "Replace existing input connections.");
const FText UEdGraphSchema_Niagara::TypesAreNotCompatibleText = LOCTEXT("TypesNotCompatible", "Types are not compatible");
const FText UEdGraphSchema_Niagara::ConvertText = LOCTEXT("ConvertTypesText", "Convert {0} to {1}");
const FText UEdGraphSchema_Niagara::ConvertLossyText = LOCTEXT("ConvertTypesLossyText", "Convert {0} to {1} implicitly. Be aware that this can be a lossy conversion.");
const FText UEdGraphSchema_Niagara::PinNotConnectableText = LOCTEXT("PinNotConnectable", "Pin is not connectable");
const FText UEdGraphSchema_Niagara::SameNodeConnectionForbiddenText = LOCTEXT("SameNodeNotAllowed", "Both pins are on the same node");
const FText UEdGraphSchema_Niagara::DirectionsNotCompatibleText = LOCTEXT("DirectionsNotCompatible", "Pin directions are incompatible");
const FText UEdGraphSchema_Niagara::AddPinIncompatibleTypeText = LOCTEXT("AddPinCompatibleConnection", "Cannot make connections to or from add pins for non-parameter types");
const FText UEdGraphSchema_Niagara::CircularConnectionFoundText = LOCTEXT("CircularConnectionFound", "Circular connection found");

const FName UEdGraphSchema_Niagara::PinCategoryType("Type");
const FName UEdGraphSchema_Niagara::PinCategoryMisc("Misc");
const FName UEdGraphSchema_Niagara::PinCategoryClass("Class");
const FName UEdGraphSchema_Niagara::PinCategoryEnum("Enum");

const FName UEdGraphSchema_Niagara::PinCategoryStaticType("StaticType");
const FName UEdGraphSchema_Niagara::PinCategoryStaticClass("StaticClass");
const FName UEdGraphSchema_Niagara::PinCategoryStaticEnum("StaticEnum");

namespace NiagaraNodeNumbers
{
	// Maximum distance a drag can be off a node edge to require 'push off' from node
	constexpr int32 NiagaraMinNodeDistance = 60;
}

UEdGraphNode* FNiagaraSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	UEdGraphNode* ResultNode = NULL;

	// If there is a template, we actually use it
	if (NodeTemplate != NULL)
	{
		FString OutErrorMsg;
		UNiagaraNode* NiagaraNodeTemplate = Cast<UNiagaraNode>(NodeTemplate);
		if (NiagaraNodeTemplate && !NiagaraNodeTemplate->CanAddToGraph(CastChecked<UNiagaraGraph>(ParentGraph), OutErrorMsg))
		{
			if (OutErrorMsg.Len() > 0)
			{
				FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(OutErrorMsg));
			}
			return ResultNode;
		}

		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorNewNode", "Niagara Editor: New Node"));
		ParentGraph->Modify();

		NodeTemplate->SetFlags(RF_Transactional);

		// set outer to be the graph so it doesn't go away
		NodeTemplate->Rename(NULL, ParentGraph, REN_NonTransactional);
		ParentGraph->AddNode(NodeTemplate, true, bSelectNewNode);

		NodeTemplate->CreateNewGuid();
		NodeTemplate->PostPlacedNewNode();
		NodeTemplate->AllocateDefaultPins();
		NodeTemplate->AutowireNewNode(FromPin);

		// For input pins, new node will generally overlap node being dragged off
		// Work out if we want to visually push away from connected node
		int32 XLocation = Location.X;
		if (FromPin && FromPin->Direction == EGPD_Input)
		{
			UEdGraphNode* PinNode = FromPin->GetOwningNode();
			const float XDelta = FMath::Abs(PinNode->NodePosX - Location.X);

			if (XDelta < NiagaraNodeNumbers::NiagaraMinNodeDistance)
			{
				// Set location to edge of current node minus the max move distance
				// to force node to push off from connect node enough to give selection handle
				XLocation = PinNode->NodePosX - NiagaraNodeNumbers::NiagaraMinNodeDistance;
			}
		}

		NodeTemplate->NodePosX = XLocation;
		NodeTemplate->NodePosY = Location.Y;
		NodeTemplate->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

		ResultNode = NodeTemplate;

		//ParentGraph->NotifyGraphChanged();
	}

	return ResultNode;
}

UEdGraphNode* FNiagaraSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode/* = true*/) 
{
	UEdGraphNode* ResultNode = NULL;

	if (FromPins.Num() > 0)
	{
		ResultNode = PerformAction(ParentGraph, FromPins[0], Location, bSelectNewNode);

		if (ResultNode)
		{
			// Try autowiring the rest of the pins
			for (int32 Index = 1; Index < FromPins.Num(); ++Index)
			{
				ResultNode->AutowireNewNode(FromPins[Index]);
			}
		}
	}
	else
	{
		ResultNode = PerformAction(ParentGraph, NULL, Location, bSelectNewNode);
	}

	return ResultNode;
}

void FNiagaraSchemaAction_NewNode::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEdGraphSchemaAction::AddReferencedObjects( Collector );

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject( NodeTemplate );
}

UEdGraphNode* FNiagaraSchemaAction_NewComment::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode /*= true*/)
{
	// Add menu item for creating comment boxes
	UEdGraphNode_Comment* CommentTemplate = NewObject<UEdGraphNode_Comment>();

	FVector2f SpawnLocation = Location;
	FSlateRect Bounds;
	
	if (GraphEditor->GetBoundsForSelectedNodes(Bounds, 50.0f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}
	CommentTemplate->bCommentBubbleVisible_InDetailsPanel = false;
	CommentTemplate->bCommentBubbleVisible = false; 
	CommentTemplate->bCommentBubblePinned = false;

	UEdGraphNode* NewNode = FNiagaraSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(ParentGraph, CommentTemplate, SpawnLocation, bSelectNewNode);
	return NewNode;
}

//////////////////////////////////////////////////////////////////////////

static int32 GbAllowAllNiagaraNodesInEmitterGraphs = 1;
static FAutoConsoleVariableRef CVarAllowAllNiagaraNodesInEmitterGraphs(
	TEXT("niagara.AllowAllNiagaraNodesInEmitterGraphs"),
	GbAllowAllNiagaraNodesInEmitterGraphs,
	TEXT("If true, all nodes will be allowed in the Niagara emitter graphs. \n"),
	ECVF_Default
);

UEdGraphSchema_Niagara::UEdGraphSchema_Niagara(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<FNiagaraAction_NewNode> AddNewNodeMenuAction(
	TArray<TSharedPtr<FNiagaraAction_NewNode>>& NewActions,
	UEdGraphNode* InNodeTemplate,
	const FText& DisplayName,
	TArray<FString> NestedCategories,
	TOptional<FNiagaraFavoritesActionData> InFavoritesActionData,
	const FText& Tooltip,
	FText Keywords = FText(),
	FNiagaraActionSourceData SourceData = FNiagaraActionSourceData(
		EScriptSource::Niagara, FText::FromString(TEXT("Niagara")), true))
{
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	TSharedPtr<FNiagaraAction_NewNode> NewAction = MakeShared<FNiagaraAction_NewNode>(DisplayName, NestedCategories, InFavoritesActionData, Tooltip, Keywords);
	if (ensureMsgf(InNodeTemplate == nullptr || NiagaraEditorSettings->IsVisibleClass(InNodeTemplate->GetClass()),
		TEXT("Can not create a menu action for a node of class %s."), *InNodeTemplate->GetClass()->GetName()))
	{
		NewAction->WeakNodeTemplate = InNodeTemplate;
	}
	NewAction->SourceData = SourceData;
	NewActions.Add(NewAction);

	return NewAction;
}

bool IsSystemGraph(const UNiagaraGraph* NiagaraGraph)
{
	TArray<UNiagaraNodeEmitter*> Emitters;
	NiagaraGraph->GetNodesOfClass<UNiagaraNodeEmitter>(Emitters);
	bool bSystemGraph = Emitters.Num() != 0 || NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::SystemSpawnScript) != nullptr || NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::SystemUpdateScript) != nullptr;
	return bSystemGraph;
}

bool IsParticleGraph(const UNiagaraGraph* NiagaraGraph)
{
	bool bParticleGraph = NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::ParticleSpawnScriptInterpolated) != nullptr || NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::ParticleSpawnScript) != nullptr || NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::ParticleUpdateScript) != nullptr;
	return bParticleGraph;
}

bool IsModuleGraph(const UNiagaraGraph* NiagaraGraph)
{
	bool bModuleGraph = NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::Module) != nullptr;
	return bModuleGraph;
}

bool IsDynamicInputGraph(const UNiagaraGraph* NiagaraGraph)
{
	bool bDynamicInputGraph = NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::DynamicInput) != nullptr;
	return bDynamicInputGraph;
}


bool IsFunctionGraph(const UNiagaraGraph* NiagaraGraph)
{
	bool bFunctionGraph = NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::Function) != nullptr;
	return bFunctionGraph;
}


bool IsParticleUpdateGraph(const UNiagaraGraph* NiagaraGraph)
{
	bool bUpdateGraph = NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::ParticleUpdateScript) != nullptr;
	return bUpdateGraph;
}


const UNiagaraGraph* GetAlternateGraph(const UNiagaraGraph* NiagaraGraph)
{
	UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(NiagaraGraph->GetOuter());
	if (ScriptSource != nullptr)
	{
		UNiagaraScript* Script = Cast<UNiagaraScript>(ScriptSource->GetOuter());
		if (Script != nullptr)
		{
			FVersionedNiagaraEmitterData* EmitterProperties = Script->GetOuterEmitter().GetEmitterData();
			if (EmitterProperties != nullptr)
			{
				if (EmitterProperties->SpawnScriptProps.Script == Script)
				{
					return CastChecked<UNiagaraScriptSource>(EmitterProperties->UpdateScriptProps.Script->GetLatestSource())->NodeGraph;
				}
				else if (EmitterProperties->UpdateScriptProps.Script == Script)
				{
					return CastChecked<UNiagaraScriptSource>(EmitterProperties->SpawnScriptProps.Script->GetLatestSource())->NodeGraph;
				}
			}
		}
	}
	return nullptr;
}

FText GetGraphTypeTitle(const UNiagaraGraph* NiagaraGraph)
{
	UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(NiagaraGraph->GetOuter());
	if (ScriptSource != nullptr)
	{
		UNiagaraScript* Script = Cast<UNiagaraScript>(ScriptSource->GetOuter());
		if (Script != nullptr)
		{
			if (Script->IsParticleSpawnScript())
			{
				return LOCTEXT("Parameter Menu Title Spawn", "Spawn Parameters");
			}
			else if (Script->IsParticleUpdateScript())
			{
				return LOCTEXT("Parameter Menu Title Update", "Update Parameters");
			}
		}
	}
	return LOCTEXT("Parameter Menu Title Generic", "Script Parameters");
}

void AddParametersForGraph(TArray<TSharedPtr<FNiagaraAction_NewNode>>& NewActions, const UNiagaraGraph* CurrentGraph,  UEdGraph* OwnerOfTemporaries, const UNiagaraGraph* NiagaraGraph)
{
	FText GraphParameterCategory = GetGraphTypeTitle(NiagaraGraph);
	TArray<UNiagaraNodeInput*> InputNodes;
	NiagaraGraph->GetNodesOfClass(InputNodes);

	TArray<FNiagaraVariable> SeenParams;
	for (UNiagaraNodeInput* InputNode : InputNodes)
	{
		if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter && !SeenParams.Contains(InputNode->Input))
		{
			SeenParams.Add(InputNode->Input);
			FName Name = InputNode->Input.GetName();
			FText DisplayName = FText::FromName(Name);
			
			if (NiagaraGraph != CurrentGraph)
			{
				Name = UNiagaraNodeInput::GenerateUniqueName(CastChecked<UNiagaraGraph>(CurrentGraph), Name, InputNode->Usage);
				DisplayName = FText::Format(LOCTEXT("Parameter Menu Copy Param","Copy \"{0}\" to this Graph"), FText::FromName(Name));
			}

			UNiagaraNodeInput* InputNodeTemplate = NewObject<UNiagaraNodeInput>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, InputNodeTemplate, DisplayName, {GraphParameterCategory.ToString()}, {}, FText::GetEmpty());
			InputNodeTemplate->Input = InputNode->Input;
			InputNodeTemplate->Usage = InputNode->Usage;
			InputNodeTemplate->ExposureOptions = InputNode->ExposureOptions;
			InputNodeTemplate->SetDataInterface(nullptr);

			// We also support parameters from an alternate graph. If that was used, then we need to take special care
			// to make the parameter unique to that graph.
			if (NiagaraGraph != CurrentGraph)
			{
				InputNodeTemplate->Input.SetName(Name);

				if (InputNode->GetDataInterface())
				{
					InputNodeTemplate->SetDataInterface(Cast<UNiagaraDataInterface>(StaticDuplicateObject(InputNode->GetDataInterface(), InputNodeTemplate, NAME_None, ~RF_Transient)));
				}
				if (InputNode->GetObjectAsset())
				{
					InputNodeTemplate->SetObjectAsset(InputNode->GetObjectAsset());
				}
			}
		}
	}
}

void AddParameterMenuOptions(TArray<TSharedPtr<FNiagaraAction_NewNode>>& NewActions, const UNiagaraGraph* CurrentGraph, UEdGraph* OwnerOfTemporaries, const UNiagaraGraph* NiagaraGraph)
{
	AddParametersForGraph(NewActions, CurrentGraph, OwnerOfTemporaries, NiagaraGraph);

	const UNiagaraGraph* AltGraph = GetAlternateGraph(NiagaraGraph);
	if (AltGraph != nullptr)
	{
		AddParametersForGraph(NewActions, CurrentGraph, OwnerOfTemporaries, AltGraph);
	}
}

TArray<TSharedPtr<FNiagaraAction_NewNode>> UEdGraphSchema_Niagara::GetGraphActions(const UEdGraph* CurrentGraph, const UEdGraphPin* FromPin, UEdGraph* OwnerOfTemporaries) const
{
	TArray<TSharedPtr<FNiagaraAction_NewNode>> NewActions;

	const UNiagaraGraph* NiagaraGraph = CastChecked<UNiagaraGraph>(CurrentGraph);
	
	const bool bSystemGraph = IsSystemGraph(NiagaraGraph);
	const bool bModuleGraph = IsModuleGraph(NiagaraGraph);
	const bool bDynamicInputGraph = IsDynamicInputGraph(NiagaraGraph);
	const bool bFunctionGraph = IsFunctionGraph(NiagaraGraph);
	const bool bParticleUpdateGraph = IsParticleUpdateGraph(NiagaraGraph);

	// What node types should we allow
	//-TODO: Needs further clean-up for FromPin
	bool bAllowOpNodes = true;
	bool bAllowCustomNode = true;
	bool bAllowFunctionNodes = true;
	bool bAllowModuleNodes = true;
	bool bAllowEventNodes = true;
	bool bAllowParameterMapGetSetNodes = true;
	bool bAllowOutputTagNodes = true;
	bool bAllowSelectNodes = true;
	bool bAllowStaticSwitchNodes = true;

	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();

	TOptional<FNiagaraTypeDefinition> FromType;
	TOptional<EEdGraphPinDirection> FromDirection;

	UNiagaraDataInterface* FromPinDataInterface = nullptr;
	if ( FromPin )
	{
		FromType = PinToTypeDefinition(FromPin);
		FromDirection = FromPin->Direction;
		if (const UClass* FromPinClass = FromType->GetClass() )
		{
			bAllowOpNodes = false;
			bAllowFunctionNodes = false;
			bAllowEventNodes = false;
			bAllowOutputTagNodes = false;
			bAllowSelectNodes = false;
			bAllowStaticSwitchNodes = false;

			if (UNiagaraNodeInput* FromPinInputNode = Cast<UNiagaraNodeInput>(FromPin->GetOwningNode()))
			{
				FromPinDataInterface = FromPinInputNode->GetDataInterface();
			}
			else
			{
				FromPinDataInterface = Cast<UNiagaraDataInterface>(const_cast<UClass*>(FromPinClass)->GetDefaultObject());
			}
		}
	}
	
	// Add operations (add / mul / etc)
	if (bAllowOpNodes &&
		(GbAllowAllNiagaraNodesInEmitterGraphs || bModuleGraph || bFunctionGraph || bSystemGraph) &&
		NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeOp::StaticClass()))
	{
		const TArray<FNiagaraOpInfo>& OpInfos = FNiagaraOpInfo::GetOpInfoArray();

		bool bFilterTypes = false;
		if(FromType.IsSet() && FromDirection.IsSet())
		{
			bFilterTypes = true;
		}
		
		for (const FNiagaraOpInfo& OpInfo : OpInfos)
		{
			bool bAddOp = true;
			if(bFilterTypes)
			{
				bAddOp = false;
				if(FromDirection == EGPD_Output)
				{
					for(const FNiagaraOpInOutInfo& InputInfo : OpInfo.Inputs)
					{
						if(FNiagaraTypeDefinition::TypesAreAssignable(InputInfo.DataType, FromType.GetValue()))
						{
							bAddOp = true;
							break;
						}
					}
				}
				else
				{
					for(const FNiagaraOpInOutInfo& OutputInfo : OpInfo.Outputs)
					{
						if(FNiagaraTypeDefinition::TypesAreAssignable(FromType.GetValue(), OutputInfo.DataType))
						{
							bAddOp = true;
							break;
						}
					}
				}
			}

			if(bAddOp)
			{
				FNiagaraFavoritesActionData FavoritesActionData;
				FavoritesActionData.ActionIdentifier.Names.Add(OpInfo.Name);
				FavoritesActionData.bFavoriteByDefault = false;
				
				UNiagaraNodeOp* OpNode = NewObject<UNiagaraNodeOp>(OwnerOfTemporaries);
				OpNode->OpName = OpInfo.Name;
				TSharedPtr<FNiagaraAction_NewNode> MathOpAction = AddNewNodeMenuAction(NewActions, OpNode, OpInfo.FriendlyName, {OpInfo.Category.ToString()}, FavoritesActionData, OpInfo.Description,  OpInfo.Keywords);
				MathOpAction->AlternateSearchName = OpInfo.AlternateSearchName.IsSet() ? FText::FromName(OpInfo.AlternateSearchName.GetValue()) : TOptional<FText>();
			}
		}
	}

	// Add custom code
	if (bAllowCustomNode && NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeCustomHlsl::StaticClass()))
	{
		const FText DisplayName = LOCTEXT("CustomHLSLNode","Custom Hlsl");
		const FText TooltipDesc = LOCTEXT("CustomHlslPopupTooltip", "Add a node with custom hlsl content");
		
		UNiagaraNodeCustomHlsl* CustomHlslNode = NewObject<UNiagaraNodeCustomHlsl>(OwnerOfTemporaries);
		CustomHlslNode->SetCustomHlsl(TEXT("// Insert the body of the function here and add any inputs\r\n// and outputs by name using the add pins above.\r\n// Currently, complicated branches, for loops, switches, etc are not advised."));
		AddNewNodeMenuAction(NewActions, CustomHlslNode, DisplayName, {LOCTEXT("Function Menu Title", "Functions").ToString()}, {}, TooltipDesc,  FText::GetEmpty());
	}

	auto AddScriptFunctionAction = [&NewActions, OwnerOfTemporaries](const TArray<FString>& Categories, const FAssetData& ScriptAsset)
	{
		FText AssetDesc;
		ScriptAsset.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData , Description), AssetDesc);

		FText Keywords;
		ScriptAsset.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Keywords), Keywords);

		bool bSuggested = ScriptAsset.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, bSuggested));

		bool bIsInLibrary = FNiagaraEditorUtilities::IsScriptAssetInLibrary(ScriptAsset);
		const FText DisplayName = FNiagaraEditorUtilities::FormatScriptName(ScriptAsset.AssetName, bIsInLibrary);
		const FText TooltipDesc = FNiagaraEditorUtilities::FormatScriptDescription(AssetDesc, ScriptAsset.GetSoftObjectPath(), bIsInLibrary);
		const TTuple<EScriptSource, FText> Source = FNiagaraEditorUtilities::GetScriptSource(ScriptAsset);
		FNiagaraActionSourceData SourceData(Source.Key, Source.Value, true);
		
		FNiagaraFavoritesActionData FavoritesActionData;
		FavoritesActionData.ActionIdentifier.Names.Add(FName(ScriptAsset.GetObjectPathString()));
		FavoritesActionData.bFavoriteByDefault = bSuggested;
		
		UNiagaraNodeFunctionCall* FunctionCallNode = NewObject<UNiagaraNodeFunctionCall>(OwnerOfTemporaries);
		FunctionCallNode->FunctionScriptAssetObjectPath = FName(ScriptAsset.GetSoftObjectPath().ToString());

		TSharedPtr<FNiagaraAction_NewNode> Action = AddNewNodeMenuAction(NewActions, FunctionCallNode, DisplayName, Categories, FavoritesActionData, TooltipDesc, Keywords, SourceData);
		Action->bIsInLibrary = bIsInLibrary;
		Action->PreviewMoviePath = FNiagaraEditorUtilities::Preview::GetPreviewMovieObjectPath(ScriptAsset);

		return Action;
	};

	//Add functions
	if (bAllowFunctionNodes &&
		(GbAllowAllNiagaraNodesInEmitterGraphs || bModuleGraph || bFunctionGraph || bDynamicInputGraph) &&
		NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeFunctionCall::StaticClass()))
	{
		TArray<FAssetData> FunctionScriptAssets;
		FNiagaraEditorUtilities::FGetFilteredScriptAssetsOptions FunctionScriptFilterOptions;
		FunctionScriptFilterOptions.bIncludeNonLibraryScripts = true;
		FunctionScriptFilterOptions.ScriptUsageToInclude = ENiagaraScriptUsage::Function;
		FNiagaraEditorUtilities::GetFilteredScriptAssets(FunctionScriptFilterOptions, FunctionScriptAssets);

		for (const FAssetData& FunctionScriptAsset : FunctionScriptAssets)
		{
			AddScriptFunctionAction({LOCTEXT("Function Menu Title", "Functions").ToString()}, FunctionScriptAsset);
		}

		// Insert fast path function nodes here (visible with fx.AllowFastPathFunctionLibrary)...
		const TArray<FNiagaraFunctionSignature>& VMFastPathOps = UNiagaraFunctionLibrary::GetVectorVMFastPathOps();
		for (const FNiagaraFunctionSignature& Sig : VMFastPathOps)
		{
			UNiagaraNodeFunctionCall* FunctionCallNode = NewObject<UNiagaraNodeFunctionCall>(OwnerOfTemporaries);
			FunctionCallNode->Signature = Sig;

			const FText MenuDesc = FText::FromString(FName::NameToDisplayString(Sig.Name.ToString(), false));
			TSharedPtr<FNiagaraAction_NewNode> FunctionCallAction = AddNewNodeMenuAction(NewActions, FunctionCallNode, MenuDesc, TArray<FString>(), {},  MenuDesc);
			FunctionCallAction->WeakNodeTemplate = FunctionCallNode;
		}
	}

	//Add modules
	if (bAllowModuleNodes && !bFunctionGraph && !bModuleGraph && !bDynamicInputGraph &&
		NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeFunctionCall::StaticClass()))
	{
		TArray<FAssetData> ModuleScriptAssets;
		FNiagaraEditorUtilities::FGetFilteredScriptAssetsOptions ModuleScriptFilterOptions;
		ModuleScriptFilterOptions.bIncludeNonLibraryScripts = true;
		ModuleScriptFilterOptions.ScriptUsageToInclude = ENiagaraScriptUsage::Module;
		FNiagaraEditorUtilities::GetFilteredScriptAssets(ModuleScriptFilterOptions, ModuleScriptAssets);

		for (const FAssetData& ModuleScriptAsset : ModuleScriptAssets)
		{
			TSharedPtr<FNiagaraAction_NewNode> ModuleAction = AddScriptFunctionAction({LOCTEXT("Module Menu Title", "Modules").ToString()}, ModuleScriptAsset);
			ModuleAction->SearchWeightMultiplier = 0.5f;
		}
	}

	//Add event read and writes nodes
	if (bAllowEventNodes && bModuleGraph &&
		NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeReadDataSet::StaticClass()) &&
		NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeWriteDataSet::StaticClass()))
	{
		const FText MenuCat = LOCTEXT("NiagaraEventMenuCat", "Events");
		TArray<FString> ReadCategories = {MenuCat.ToString(), LOCTEXT("NiagaraEventCategory_Read", "Read").ToString()};
		TArray<FString> WriteCategories = {MenuCat.ToString(), LOCTEXT("NiagaraEventCategory_Write", "Write").ToString()};

		TArray<FNiagaraTypeDefinition> PayloadTypes;
		FNiagaraEditorUtilities::GetAllowedPayloadTypes(PayloadTypes);
		for (const FNiagaraTypeDefinition& PayloadType : PayloadTypes)
		{
			if (PayloadType.IsInternalType())
			{
				continue;
			}

			if (PayloadType.GetStruct() && !PayloadType.GetStruct()->IsChildOf(UNiagaraDataInterface::StaticClass()))
			{
				{
					const FText DisplayNameFormat = LOCTEXT("AddEventReadFmt", "Add {0} Event Read");
					FString NameString = PayloadType.GetNameText().ToString();
					NameString = NameString.Replace(TEXT(" Event"), TEXT(""), ESearchCase::IgnoreCase);
					const FText DisplayName = FText::Format(DisplayNameFormat, FText::FromString(NameString));
					
					UNiagaraNodeReadDataSet* EventReadNode = NewObject<UNiagaraNodeReadDataSet>(OwnerOfTemporaries);
					EventReadNode->InitializeFromStruct(PayloadType.GetStruct());

					AddNewNodeMenuAction(NewActions, EventReadNode, DisplayName, ReadCategories, {}, FText::GetEmpty(), FText::GetEmpty());
				}
				{
					const FText DisplayNameFormat = LOCTEXT("AddEventWriteFmt", "Add {0} Event Write");
					FString NameString = PayloadType.GetNameText().ToString();
					NameString = NameString.Replace(TEXT("Event"), TEXT(""), ESearchCase::IgnoreCase);
					const FText DisplayName = FText::Format(DisplayNameFormat, FText::FromString(NameString));
					
					UNiagaraNodeWriteDataSet* EventWriteNode = NewObject<UNiagaraNodeWriteDataSet>(OwnerOfTemporaries);
					EventWriteNode->InitializeFromStruct(PayloadType.GetStruct());
					
					AddNewNodeMenuAction(NewActions, EventWriteNode, DisplayName,  WriteCategories, {}, FText::GetEmpty(), FText::GetEmpty());
				}
			}
		}
	}
	
	TArray<ENiagaraScriptUsage> UsageTypesToAdd;
	if (bParticleUpdateGraph)
	{
		UsageTypesToAdd.Add(ENiagaraScriptUsage::ParticleEventScript);
		UsageTypesToAdd.Add(ENiagaraScriptUsage::EmitterSpawnScript);
		UsageTypesToAdd.Add(ENiagaraScriptUsage::EmitterUpdateScript);
	}

	if (bSystemGraph)
	{
		UsageTypesToAdd.Add(ENiagaraScriptUsage::SystemSpawnScript);
		UsageTypesToAdd.Add(ENiagaraScriptUsage::SystemUpdateScript);
	}

	if (UsageTypesToAdd.Num() != 0 && NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeOutput::StaticClass()))
	{
		for (ENiagaraScriptUsage Usage : UsageTypesToAdd)
		{
			const FText MenuCat = LOCTEXT("NiagaraUsageMenuCat", "Output Nodes");

			UNiagaraNodeOutput* OutputNode = NewObject<UNiagaraNodeOutput>(OwnerOfTemporaries);
			OutputNode->SetUsage(Usage);

			FText DisplayName = FText::Format(LOCTEXT("AddOutput", "Add {0}"), OutputNode->GetNodeTitle(ENodeTitleType::FullTitle));

			AddNewNodeMenuAction(NewActions, OutputNode, DisplayName,  {MenuCat.ToString()}, {}, FText::GetEmpty(), FText::GetEmpty());

			UNiagaraNodeOutput* UpdateOutputNode = NiagaraGraph->FindOutputNode(ENiagaraScriptUsage::ParticleUpdateScript);
			if (UpdateOutputNode)
			{
				OutputNode->Outputs = UpdateOutputNode->Outputs;
			}
			else
			{
				OutputNode->Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("Out")));
			}
		}
	}


	// Add Convert Nodes
	if (NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeConvert::StaticClass()))
	{
		FNiagaraTypeDefinition PinType = FNiagaraTypeDefinition::GetGenericNumericDef();

		if (FromPin)
		{
			PinType = PinToTypeDefinition(FromPin);
		}
		
		/** Make & Break */
		if (PinType.GetScriptStruct())
		{
			FText MakeCat = LOCTEXT("NiagaraMake", "Make");
			FText BreakCat = LOCTEXT("NiagaraBreak", "Break");

			FText DescFmt = LOCTEXT("NiagaraMakeBreakFmt", "{0}");
			auto MakeBreakType = [&](FNiagaraTypeDefinition Type, bool bMake, bool bInLibrary)
			{
				FText DisplayName = Type.GetNameText();
				FText Desc = FText::Format(DescFmt, DisplayName);
				
				UNiagaraNodeConvert* ConvertNode = NewObject<UNiagaraNodeConvert>(OwnerOfTemporaries);
				TSharedPtr<FNiagaraAction_NewNode> Action = AddNewNodeMenuAction(NewActions, ConvertNode, DisplayName,  {bMake ? MakeCat.ToString() : BreakCat.ToString()}, {}, FText::GetEmpty(), FText::GetEmpty());
				if (bMake)
				{
					ConvertNode->InitAsMake(Type);
				}
				else
				{
					ConvertNode->InitAsBreak(Type);
				}
				Action->bIsInLibrary = bInLibrary;
			};

			// we don't allow make on the following types
			TSet<FNiagaraTypeDefinition> NoMakeAllowed;
			NoMakeAllowed.Add(FNiagaraTypeDefinition::GetGenericNumericDef());
			NoMakeAllowed.Add(FNiagaraTypeDefinition::GetParameterMapDef());
			NoMakeAllowed.Add(FNiagaraTypeHelper::GetDoubleDef());
			NoMakeAllowed.Add(FNiagaraTypeHelper::GetQuatDef());
			NoMakeAllowed.Add(FNiagaraTypeHelper::GetVector4Def());
			NoMakeAllowed.Add(FNiagaraTypeHelper::GetVectorDef());
			NoMakeAllowed.Add(FNiagaraTypeHelper::GetVector2DDef());
			
			// Dynamic Makes from type registry. Should include the pin type as well.
			TArray<FNiagaraTypeDefinition> CandidateTypes;
			FNiagaraEditorUtilities::GetAllowedParameterTypes(CandidateTypes);
			for (const FNiagaraTypeDefinition& CandidateType : FNiagaraTypeRegistry::GetRegisteredTypes())
			{
				bool bAddMake = false;
				bool bIsInLibrary = true;
				
				if(!CandidateType.IsValid() || CandidateType.IsUObject() || NoMakeAllowed.Contains(CandidateType))
				{
					continue;
				}
				
				// if we have a "from pin", we generally only want applicable make types
				if(FromPin)
				{
					// if our "from pin" is an output pin, we require at least one input pin type of the candidate type to be the same or assignable
					if(FromPin->Direction == EGPD_Output)
					{
						// we are checking here if the candidate type has a property that is the same as our pin
						for (TFieldIterator<FProperty> CandidatePropertyIt(CandidateType.GetStruct(), EFieldIteratorFlags::IncludeSuper); CandidatePropertyIt; ++CandidatePropertyIt)
						{
							if (IsValidNiagaraPropertyType(*CandidatePropertyIt))
							{
								FNiagaraTypeDefinition CandidatePropertyType = GetTypeDefForProperty(*CandidatePropertyIt);
								if(FNiagaraTypeDefinition::TypesAreAssignable(CandidatePropertyType, PinType))
								{
									bAddMake = true;
									break;
								}
							}
						}
					}
					else
					{
						// if our "from pin" is an input pin, we generally only allow that exact type for making, with some exceptions like NiagaraGenerics
						if(PinType == CandidateType)
						{
								bAddMake = true;
						}
						
						if(PinType == FNiagaraTypeDefinition::GetGenericNumericDef() && FNiagaraTypeDefinition::IsValidNumericInput(CandidateType))
						{
							bAddMake = true;
						}
					}
				}
				// if we don't, just generally allow make types. We are stricter about break types however
				else
				{
					bAddMake = true;
					if (!FNiagaraTypeRegistry::GetRegisteredParameterTypes().Contains(CandidateType) &&
						!FNiagaraTypeRegistry::GetRegisteredPayloadTypes().Contains(CandidateType) && 
						!FNiagaraTypeRegistry::GetUserDefinedTypes().Contains(CandidateType))
					{
						bIsInLibrary = false;
					}
				}

				if(bAddMake)
				{
					MakeBreakType(CandidateType, true, bIsInLibrary);
				}
			}
			
			// Break for the current "from pin"
			{
				// Don't break scalars. Allow makes for now as a convenient method of getting internal script constants when dealing with numeric pins.
				// Object and data interfaces can't be broken.
				if (FromPin && !PinType.IsInternalType() && !PinType.IsUObject() && !FNiagaraTypeDefinition::IsScalarDefinition(PinType) && PinType != FNiagaraTypeDefinition::GetGenericNumericDef() && PinType != FNiagaraTypeDefinition::GetParameterMapDef())
				{
					if(FromPin->Direction == EGPD_Output)
					{
						MakeBreakType(PinType, false, true);
					}					
				}
			}				
			

			//Always add generic convert as an option.
			FText Desc = LOCTEXT("NiagaraConvert", "Convert");
			
			UNiagaraNodeConvert* ConvertNode = NewObject<UNiagaraNodeConvert>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, ConvertNode, Desc,  {}, {},FText::GetEmpty(), FText::GetEmpty());
		}
	}

	if (FromPin)
	{
		//Add pin specific menu options.
		if ( FromPinDataInterface && (FromPin->Direction == EGPD_Output) )
		{
			FText MenuCat = FromPinDataInterface->GetClass()->GetDisplayNameText();
			TArray<FNiagaraFunctionSignature> Functions;
			FromPinDataInterface->GetFunctionSignatures(Functions);
			for (FNiagaraFunctionSignature& Sig : Functions)
			{
				if (Sig.bSoftDeprecatedFunction || Sig.bHidden)
					continue;

				UNiagaraNodeFunctionCall* FuncNode = NewObject<UNiagaraNodeFunctionCall>(OwnerOfTemporaries);
				AddNewNodeMenuAction(NewActions, FuncNode, FText::FromString(FName::NameToDisplayString(Sig.GetNameString(), false)),  {MenuCat.ToString()}, {}, Sig.GetDescription(), FText::GetEmpty());
				FuncNode->Signature = Sig;
			}
		}

		if (FromPin->Direction == EGPD_Output)
		{
			FNiagaraTypeDefinition PinType = PinToTypeDefinition(FromPin);

			//Add all swizzles for this type if it's a vector.
			if (FNiagaraHlslTranslator::IsHlslBuiltinVector(PinType))
			{
				TArray<FString> Components;
				for (TFieldIterator<FProperty> PropertyIt(PinType.GetStruct(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
				{
					FProperty* Property = *PropertyIt;
					Components.Add(Property->GetName().ToLower());
				}

				TArray<FString> Swizzles;
				TFunction<void(FString)> GenSwizzles = [&](FString CurrStr)
				{
					if (CurrStr.Len() == 4) return;//Only generate down to float4
					for (FString& CompStr : Components)
					{
						Swizzles.Add(CurrStr + CompStr);
						GenSwizzles(CurrStr + CompStr);
					}
				};

				GenSwizzles(FString());

				for (FString Swiz : Swizzles)
				{
					const FText Category = LOCTEXT("NiagaraSwizzles", "Swizzles");
					
					UNiagaraNodeConvert* ConvertNode = NewObject<UNiagaraNodeConvert>(OwnerOfTemporaries);
					AddNewNodeMenuAction(NewActions, ConvertNode, FText::FromString(Swiz),  {Category.ToString()}, {}, FText::GetEmpty(), FText::GetEmpty());

					ConvertNode->InitAsSwizzle(Swiz);
				}
			}
		}
	}

	// Handle parameter map get/set/for
	if (bAllowParameterMapGetSetNodes)
	{
		FText MenuCat = FText::FromString("Parameter Map");
		if (NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeParameterMapGet::StaticClass()))
		{
			FString Name = TEXT("Parameter Map Get");
			
			FNiagaraFavoritesActionData FavoritesActionData;
			FavoritesActionData.ActionIdentifier.Names.Add(FName(Name));
			FavoritesActionData.bFavoriteByDefault = true;
			
			UNiagaraNodeParameterMapGet* BaseNode = NewObject<UNiagaraNodeParameterMapGet>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, BaseNode, FText::FromString(Name), {MenuCat.ToString()}, FavoritesActionData, FText::GetEmpty(), FText::GetEmpty());
		}
		if (NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeParameterMapSet::StaticClass()))
		{
			FString Name = TEXT("Parameter Map Set");
			
			FNiagaraFavoritesActionData FavoritesActionData;
			FavoritesActionData.ActionIdentifier.Names.Add(FName(Name));
			FavoritesActionData.bFavoriteByDefault = true;
			
			UNiagaraNodeParameterMapSet* BaseNode = NewObject<UNiagaraNodeParameterMapSet>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, BaseNode, FText::FromString(Name), {MenuCat.ToString()}, FavoritesActionData, FText::GetEmpty(), FText::GetEmpty());
		}
		if (NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeParameterMapFor::StaticClass()))
		{
			FString Name = TEXT("Parameter Map For");
			UNiagaraNodeParameterMapFor* BaseNode = NewObject<UNiagaraNodeParameterMapFor>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, BaseNode, FText::FromString(Name),  {MenuCat.ToString()}, {}, FText::GetEmpty(), FText::GetEmpty());
		}
		if (NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeParameterMapForWithContinue::StaticClass()))
		{
			FString Name = TEXT("Parameter Map For With Continue");
			UNiagaraNodeParameterMapForWithContinue* BaseNode = NewObject<UNiagaraNodeParameterMapForWithContinue>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, BaseNode, FText::FromString(Name),  { MenuCat.ToString() }, {}, FText::GetEmpty(), FText::GetEmpty());
		}
		if (NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeParameterMapForIndex::StaticClass()))
		{
			FString Name = TEXT("Parameter Map For Current Index");
			UNiagaraNodeParameterMapForIndex* BaseNode = NewObject<UNiagaraNodeParameterMapForIndex>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, BaseNode, FText::FromString(Name),  { MenuCat.ToString() }, {}, FText::GetEmpty(), FText::GetEmpty());
		}
	}

	// Handle comment nodes
	if (NiagaraEditorSettings->IsVisibleClass(UEdGraphNode_Comment::StaticClass()))
	{
		FText MenuCat = FText::FromString("Comments");

		{
			FString Name = TEXT("Add Comment");
			UEdGraphNode_Comment* BaseNode = NewObject<UEdGraphNode_Comment>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, BaseNode, FText::FromString(Name),  {MenuCat.ToString()}, {}, FText::GetEmpty(), FText::GetEmpty());
		}		
	}

	// Handle output tag nodes
	if (bAllowOutputTagNodes && NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeOutputTag::StaticClass()))
	{
		FText MenuCat = FText::FromString("Special Purpose Parameters");

		{
			FString Name = TEXT("Add Compiler Output Tag");
			UNiagaraNodeOutputTag* BaseNode = NewObject<UNiagaraNodeOutputTag>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, BaseNode, FText::FromString(Name),  {MenuCat.ToString()}, {}, FText::GetEmpty(), FText::GetEmpty());
		}
	}
	
	//Add all input node options for input pins or no pin.
	if ((FromPin == nullptr || FromPin->Direction == EGPD_Input) && NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeInput::StaticClass()))
	{
		TArray<UNiagaraNodeInput*> InputNodes;
		NiagaraGraph->GetNodesOfClass(InputNodes);

		if (bFunctionGraph)
		{
			//Emitter constants managed by the system.
			const TArray<FNiagaraVariable>& SystemConstants = FNiagaraConstants::GetEngineConstants();
			for (const FNiagaraVariable& SysConst : SystemConstants)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Constant"), FText::FromName(SysConst.GetName()));
				const FText DisplayName = FText::Format(LOCTEXT("GetSystemConstant", "Get {Constant}"), Args);
				
				UNiagaraNodeInput* InputNode = NewObject<UNiagaraNodeInput>(OwnerOfTemporaries);
				AddNewNodeMenuAction(NewActions, InputNode, DisplayName,  {LOCTEXT("System Parameters Menu Title", "System Parameters").ToString()}, {}, FText::GetEmpty(), FText::GetEmpty());

				InputNode->Usage = ENiagaraInputNodeUsage::SystemConstant;
				InputNode->Input = SysConst;
			}
		}

		//Emitter constants managed by the Translator.
		const TArray<FNiagaraVariable>& TranslatorConstants = FNiagaraConstants::GetTranslatorConstants();
		for (const FNiagaraVariable& TransConst : TranslatorConstants)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Constant"), FText::FromName(TransConst.GetName()));
			const FText DisplayName = FText::Format(LOCTEXT("GetTranslatorConstant", "{Constant}"), Args);
			
			UNiagaraNodeInput* InputNode = NewObject<UNiagaraNodeInput>(OwnerOfTemporaries);
			AddNewNodeMenuAction(NewActions, InputNode, DisplayName,  {LOCTEXT("Translator Parameters Menu Title", "Special Purpose Parameters").ToString()}, {}, FText::GetEmpty(), FText::GetEmpty());

			InputNode->Usage = ENiagaraInputNodeUsage::TranslatorConstant;
			InputNode->ExposureOptions.bCanAutoBind = true;
			InputNode->ExposureOptions.bHidden = true;
			InputNode->ExposureOptions.bRequired = false;
			InputNode->ExposureOptions.bExposed = false;
			InputNode->Input = TransConst;
		}

		AddParameterMenuOptions(NewActions, NiagaraGraph, OwnerOfTemporaries, NiagaraGraph);

		//Add a generic Parameter node to allow easy creation of parameters.
		{
			FNiagaraTypeDefinition PinType = FNiagaraTypeDefinition::GetGenericNumericDef();
			if (FromPin)
			{
				PinType = PinToTypeDefinition(FromPin);
			}

			// we don't want the add parameter list in module or dynamic input graphs
			if (PinType.GetStruct() && !bModuleGraph && !bDynamicInputGraph)
			{
				const FText MenuDescFmt = LOCTEXT("Add ParameterFmt", "Add {0} Parameter");
				const FText AddParameterCategory = LOCTEXT("AddParameterCat", "Add Parameter");
				TArray<FNiagaraTypeDefinition> RegisteredTypes;
				FNiagaraEditorUtilities::GetAllowedParameterTypes(RegisteredTypes);
				for (const FNiagaraTypeDefinition& Type : RegisteredTypes)
				{
					if (Type.IsUObject() && Type.IsDataInterface() == false)
					{
						continue;
					}

					TArray<FString> Categories;
					Categories.Add(AddParameterCategory.ToString());
					
					if (const UClass* Class = Type.GetClass())
					{						
						Categories.Add(FObjectEditorUtils::GetCategoryText(Class).ToString());
					}
					else
					{
						// If you are in dynamic inputs or modules, we only allow free-range variables for 
						// data interfaces and parameter maps.
						if (bDynamicInputGraph || bModuleGraph)
						{
							if (Type != FNiagaraTypeDefinition::GetParameterMapDef())
							{
								continue;
							}
						}
					}
						
					const FText DisplayName = FText::Format(MenuDescFmt, Type.GetNameText());
					
					UNiagaraNodeInput* InputNode = NewObject<UNiagaraNodeInput>(OwnerOfTemporaries);
					AddNewNodeMenuAction(NewActions, InputNode, DisplayName,  Categories, {}, FText::GetEmpty(), FText::GetEmpty());
					FNiagaraEditorUtilities::InitializeParameterInputNode(*InputNode, Type, NiagaraGraph);
				}

				// this allows adding a parameter of the type of the dragged-from input pin
				if (PinType != FNiagaraTypeDefinition::GetGenericNumericDef())
				{
					//For correctly typed pins, offer the correct type at the top level.				
					const FText DisplayName = FText::Format(MenuDescFmt, PinType.GetNameText());
					
					UNiagaraNodeInput* InputNode = NewObject<UNiagaraNodeInput>(OwnerOfTemporaries);
					AddNewNodeMenuAction(NewActions, InputNode, DisplayName,  {}, {}, FText::GetEmpty(), FText::GetEmpty());
					FNiagaraEditorUtilities::InitializeParameterInputNode(*InputNode, PinType, NiagaraGraph);
				}
			}
		}
	}

	const FText UtilMenuCat = LOCTEXT("NiagaraUsageSelectorMenuCat", "Utility");

	// Add reroute node
	if (NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeReroute::StaticClass()))
	{
		const FText RerouteMenuDesc = LOCTEXT("NiagaraRerouteMenuDesc", "Reroute");
		
		UNiagaraNodeReroute* RerouteNode = NewObject<UNiagaraNodeReroute>(OwnerOfTemporaries);
		AddNewNodeMenuAction(NewActions, RerouteNode, RerouteMenuDesc,  {UtilMenuCat.ToString()}, {}, FText::GetEmpty(), FText::GetEmpty());
	}
	
	// Add select node
	// Note: Data Interfaces are not supported for this type
	if (bAllowSelectNodes && NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeSelect::StaticClass()))
	{
		const FText SelectMenuDesc = LOCTEXT("NiagaraSelectMenuDesc", "Select / If");

		FNiagaraFavoritesActionData FavoritesActionData;
		FavoritesActionData.ActionIdentifier.Names.Add(FName(FText::AsCultureInvariant(SelectMenuDesc).ToString()));
		FavoritesActionData.bFavoriteByDefault = true;
		
		UNiagaraNodeSelect* Node = NewObject<UNiagaraNodeSelect>(OwnerOfTemporaries);
		AddNewNodeMenuAction(NewActions, Node, SelectMenuDesc,  {UtilMenuCat.ToString()}, FavoritesActionData, FText::GetEmpty(), FText::FromString(TEXT("If Branch Bool Select")));
	}

	// Add static switch node
	// Note: Data Interfaces are not supported for this type
	if (bAllowStaticSwitchNodes && NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeStaticSwitch::StaticClass()))
	{
		const FText UsageSelectorMenuDesc = LOCTEXT("NiagaraStaticSwitchMenuDesc", "Static Switch");

		FNiagaraFavoritesActionData FavoritesActionData;
		FavoritesActionData.ActionIdentifier.Names.Add(FName(FText::AsCultureInvariant(UsageSelectorMenuDesc).ToString()));
		FavoritesActionData.bFavoriteByDefault = true;
		
		UNiagaraNodeStaticSwitch* Node = NewObject<UNiagaraNodeStaticSwitch>(OwnerOfTemporaries);
		// new nodes should auto refresh
		Node->SwitchTypeData.bAutoRefreshEnabled = true;
		AddNewNodeMenuAction(NewActions, Node, UsageSelectorMenuDesc,  {UtilMenuCat.ToString()}, FavoritesActionData, FText::GetEmpty(), FText::FromString(TEXT("")));
	}

	return NewActions;
}

const FPinConnectionResponse UEdGraphSchema_Niagara::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, SameNodeConnectionForbiddenText);
	}

	// Check both pins support connections
	if(PinA->bNotConnectable || PinB->bNotConnectable)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, PinNotConnectableText);
	}
	
	// Compare the directions
	const UEdGraphPin* InputPin = NULL;
	const UEdGraphPin* OutputPin = NULL;

	if (!CategorizePinsByDirection(PinA, PinB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, DirectionsNotCompatibleText);
	}
	
	ECanCreateConnectionResponse AllowConnectionResponse = CONNECT_RESPONSE_MAKE;
	bool bInputPinHasConnection = InputPin->LinkedTo.Num() > 0;
	if(bInputPinHasConnection)
	{
		AllowConnectionResponse = (InputPin == PinA) ? CONNECT_RESPONSE_BREAK_OTHERS_A : CONNECT_RESPONSE_BREAK_OTHERS_B; 
	}

	// Do not allow making connections off of dynamic add pins to non parameter map associated pins 
	auto GetPinsAreInvalidAddPinCombination = [](const UEdGraphPin* A, const UEdGraphPin* B)->bool {
		if (A->PinType.PinSubCategory == UNiagaraNodeWithDynamicPins::AddPinSubCategory)
		{
			if (B->PinType.PinCategory == PinCategoryMisc)
			{
				return true;
			}
			else
			{
				return false;
			}
		}
		return false;
	};

	if (GetPinsAreInvalidAddPinCombination(PinA, PinB) || GetPinsAreInvalidAddPinCombination(PinB, PinA))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, AddPinIncompatibleTypeText);
	}

	// Check for a circular connection before checking any type compatibility
	TSet<const UEdGraphNode*> VisitedNodes;
	if (UEdGraphSchema_Niagara::CheckCircularConnection(VisitedNodes, OutputPin->GetOwningNode(), InputPin->GetOwningNode()))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, CircularConnectionFoundText);
	}

	if (!IsPinWildcard(PinA) && !IsPinWildcard(PinB))
	{
		// Check for compatible type pins.
		if ((PinA->PinType.PinCategory == PinCategoryType || PinA->PinType.PinCategory == PinCategoryStaticType) &&
			(PinB->PinType.PinCategory == PinCategoryType || PinB->PinType.PinCategory == PinCategoryStaticType) &&
			PinA->PinType != PinB->PinType)
		{
			FNiagaraTypeDefinition PinTypeInput = PinToTypeDefinition(InputPin);
			FNiagaraTypeDefinition PinTypeOutput = PinToTypeDefinition(OutputPin);
			
			if (PinTypeInput == FNiagaraTypeDefinition::GetParameterMapDef() || PinTypeOutput == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TypesAreNotCompatibleText);
			}
			else if (FNiagaraTypeDefinition::TypesAreAssignable(PinTypeInput, PinTypeOutput, GetDefault<UNiagaraSettings>()->bEnforceStrictStackTypes == false))
			{
				// if we have a lossy conversion but are assignable, let the user connect but inform them
				if(FNiagaraTypeDefinition::IsLossyConversion(PinTypeOutput, PinTypeInput))
				{
					FText ConvertMessage = FText::Format(ConvertLossyText, PinTypeOutput.GetNameText(), PinTypeInput.GetNameText());
					if(bInputPinHasConnection)
					{
						ConvertMessage = FText::FromString(ReplaceExistingInputConnectionsText.ToString() + TEXT("\n") + ConvertMessage.ToString()); 
					}
					return FPinConnectionResponse(AllowConnectionResponse, ConvertMessage);
				}
				// if we are assignable and not lossy, we simply allow the connection
				else
				{
					return FPinConnectionResponse(AllowConnectionResponse, bInputPinHasConnection ? ReplaceExistingInputConnectionsText.ToString() : FString());
				}
			}
			// if the types are not directly assignable at all, give the user a chance to use a conversion node
			else
			{
				//Do some limiting on auto conversions here?
				if (PinTypeInput.GetClass() || PinTypeInput.IsStatic() || PinTypeOutput.IsStatic())
				{
					return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TypesAreNotCompatibleText);
				}
				else
				{
					FText ConvertMessage = FText::Format(ConvertText, PinTypeOutput.GetNameText(), PinTypeInput.GetNameText());
					if(bInputPinHasConnection)
					{
						ConvertMessage = FText::FromString(ReplaceExistingInputConnectionsText.ToString() + TEXT("\n") + ConvertMessage.ToString()); 
					}
					return FPinConnectionResponse(CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE, ConvertMessage);
				}
			}
		}

		// Check for compatible misc pins
		if (PinA->PinType.PinCategory == PinCategoryMisc ||
			PinB->PinType.PinCategory == PinCategoryMisc)
		{
			UNiagaraNodeWithDynamicPins* NodeA = Cast<UNiagaraNodeWithDynamicPins>(PinA->GetOwningNode());
			UNiagaraNodeWithDynamicPins* NodeB = Cast<UNiagaraNodeWithDynamicPins>(PinB->GetOwningNode());
			
			// Handle Direct connection or Inferred connection between numerics
			FNiagaraTypeDefinition PinAType = PinToTypeDefinition(PinA);
			FNiagaraTypeDefinition PinBType = PinToTypeDefinition(PinB);
			FNiagaraTypeDefinition PinATypeAlt = PinAType;
			FNiagaraTypeDefinition PinBTypeAlt = PinBType;

			if (PinA->GetOwningNode())
			{
				UNiagaraGraph* Graph = Cast< UNiagaraGraph>(PinA->GetOwningNode()->GetGraph());
				if (Graph)
				{
					PinATypeAlt = Graph->GetCachedNumericConversion(PinA);
				}
			}

			if (PinB->GetOwningNode())
			{
				UNiagaraGraph* Graph = Cast< UNiagaraGraph>(PinB->GetOwningNode()->GetGraph());
				if (Graph)
				{
					PinBTypeAlt = Graph->GetCachedNumericConversion(PinB);
				}
			}


			auto CanConnect = [](const UEdGraphPin* SrcPin, const UEdGraphPin* DestPin, const FNiagaraTypeDefinition& DestPinType, UNiagaraNodeWithDynamicPins* SrcNode)->bool{
				return SrcPin->PinType.PinCategory == PinCategoryMisc && SrcPin->PinType.PinSubCategory == UNiagaraNodeWithDynamicPins::AddPinSubCategory &&
					(
						DestPin->PinType.PinCategory == PinCategoryType &&
						SrcNode && DestPinType == FNiagaraTypeDefinition::GetGenericNumericDef() ?
						SrcNode->AllowNiagaraTypeForAddPin(DestPinType) : DestPinType != FNiagaraTypeDefinition::GetGenericNumericDef()
						)
					&& PinToTypeDefinition(DestPin) != FNiagaraTypeDefinition::GetParameterMapDef();
			};

			bool bPinAIsAddAndAcceptsPinB = CanConnect(PinA, PinB, PinBType, NodeA);
			bool bPinAIsAddAndAcceptsPinBAlt = CanConnect(PinA, PinB, PinBTypeAlt, NodeA);

			bool bPinBIsAddAndAcceptsPinA = CanConnect(PinB, PinA, PinAType, NodeB);
			bool bPinBIsAddAndAcceptsPinAAlt = CanConnect(PinB, PinA, PinATypeAlt, NodeB);

			// Disallow only if both paths are invalid. I.e. a Numeric that doesn't have an inferred type doesn't match, 
			// but one that does have an inferred type can match.
			if (bPinAIsAddAndAcceptsPinB == false && bPinBIsAddAndAcceptsPinA == false && 
				bPinAIsAddAndAcceptsPinBAlt == false && bPinBIsAddAndAcceptsPinAAlt == false)
			{
				return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TypesAreNotCompatibleText);
			}

		}
		else
		{

			if (PinA->PinType.PinCategory == PinCategoryClass || PinB->PinType.PinCategory == PinCategoryClass ||
				PinA->PinType.PinCategory == PinCategoryStaticClass || PinB->PinType.PinCategory == PinCategoryStaticClass)
			{
				FNiagaraTypeDefinition AType = PinToTypeDefinition(PinA);
				FNiagaraTypeDefinition BType = PinToTypeDefinition(PinB);

				if (AType != BType)
				{
					return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TypesAreNotCompatibleText);
				}
			}

			if (PinA->PinType.PinCategory == PinCategoryEnum || PinB->PinType.PinCategory == PinCategoryEnum ||
				PinA->PinType.PinCategory == PinCategoryStaticEnum || PinB->PinType.PinCategory == PinCategoryStaticEnum)
			{
				FNiagaraTypeDefinition PinTypeInput = PinToTypeDefinition(InputPin);
				FNiagaraTypeDefinition PinTypeOutput = PinToTypeDefinition(OutputPin);
				if (FNiagaraTypeDefinition::TypesAreAssignable(PinTypeInput, PinTypeOutput) == false)
				{
					return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TypesAreNotCompatibleText);
				}
			}
		}

	}
	// at least one pin is a wildcard
	else
	{
		return GetWildcardConnectionResponse(PinA, PinB);
	}
	
	// See if we want to break existing connections (if its an input with an existing connection)
	const bool bBreakExistingDueToDataInput = (InputPin->LinkedTo.Num() > 0);
	if (bBreakExistingDueToDataInput)
	{
		return FPinConnectionResponse(AllowConnectionResponse, ReplaceExistingInputConnectionsText.ToString());
	}
	else
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, FString());
	}
}

void UEdGraphSchema_Niagara::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorBreakConnection", "Niagara Editor: Break Connection"));

	Super::BreakSinglePinLink(SourcePin, TargetPin);
}

void UEdGraphSchema_Niagara::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorBreakPinLinks", "Niagara Editor: Break Pin Links"));

	Super::BreakPinLinks(TargetPin, bSendsNodeNotification);
}

FConnectionDrawingPolicy* UEdGraphSchema_Niagara::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FNiagaraConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}

void UEdGraphSchema_Niagara::ResetPinToAutogeneratedDefaultValue(UEdGraphPin* Pin, bool bCallModifyCallbacks) const
{
	const FScopedTransaction Transaction(LOCTEXT("ResetPinToDefault", "Reset pin to default."), GIsTransacting == false);
	Pin->Modify();
	Pin->DefaultValue = Pin->AutogeneratedDefaultValue;
	if (bCallModifyCallbacks)
	{
		Pin->GetOwningNode()->PinDefaultValueChanged(Pin);
	}
}

void UEdGraphSchema_Niagara::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	const FScopedTransaction Transaction(LOCTEXT("CreateRerouteNodeOnWire", "Create Reroute Node"));

	//@TODO: This constant is duplicated from inside of SGraphNodeKnot
	const FVector2f NodeSpacerSize(42.0f, 24.0f);
	const FVector2f KnotTopLeft = GraphPosition - (NodeSpacerSize * 0.5f);

	// Create a new knot
	UEdGraph* ParentGraph = PinA->GetOwningNode()->GetGraph();
	UNiagaraNodeReroute* NewReroute = FNiagaraSchemaAction_NewNode::SpawnNodeFromTemplate<UNiagaraNodeReroute>(ParentGraph, NewObject<UNiagaraNodeReroute>(), KnotTopLeft);

	// Move the connections across (only notifying the knot, as the other two didn't really change)
	PinA->BreakLinkTo(PinB);
	PinA->MakeLinkTo((PinA->Direction == EGPD_Output) ? NewReroute->GetInputPin(0) : NewReroute->GetOutputPin(0));
	PinB->MakeLinkTo((PinB->Direction == EGPD_Output) ? NewReroute->GetInputPin(0) : NewReroute->GetOutputPin(0));
	NewReroute->PropagatePinType();
}


void UEdGraphSchema_Niagara::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	uint32 Offset = 0;
	TArray<UEnum*> Enums;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);
	for (const FAssetData& Data : Assets)
	{
		UObject* Asset = Data.GetAsset();
		if (UEnum* Enum = Cast<UEnum>(Asset))
		{
			if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(Data))
			{
				Enums.Add(Enum);
			}
		}
	}

	if (Enums.Num() > 0)
	{
		FScopedTransaction AddSwitchTransaction(LOCTEXT("NiagaraModuleEditorDropEnum", "Niagara Module: Drag and Drop Enum"));
		Graph->Modify();

		for (UEnum* Enum : Enums)
		{
			FGraphNodeCreator<UNiagaraNodeStaticSwitch> SwitchNodeCreator(*Graph);
			FString NewName = FString::Printf(TEXT("Switch on %s"), *Enum->GetName());
			UNiagaraNodeStaticSwitch* SwitchNode = SwitchNodeCreator.CreateNode();
			SwitchNode->NodePosX = GraphPosition.X;
			SwitchNode->NodePosY = GraphPosition.Y + Offset * 50.f;
			SwitchNode->InputParameterName = FName(NewName);
			SwitchNode->SwitchTypeData.SwitchType = ENiagaraStaticSwitchType::Enum;
			SwitchNode->SwitchTypeData.Enum = Enum;
			SwitchNodeCreator.Finalize();
			Offset++;
		}
	}
}

void UEdGraphSchema_Niagara::GetAssetsGraphHoverMessage(const TArray<FAssetData>&Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);
	for (const FAssetData& AssetData : Assets)
	{
		UEnum* Enum = Cast<UEnum>(AssetData.GetAsset());
		if (Enum)
		{
			if (AssetReferenceFilter)
			{
				FText FailureReason;
				if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
				{
					if (OutTooltipText.IsEmpty())
					{
						OutTooltipText = FailureReason.ToString();
					}
					continue;
				}
			}

			OutTooltipText = LOCTEXT("CreateStaticMeshSwitch", "Create a static switch using the selected enum").ToString();
			OutOkIcon = true;
			break;
		}
	}
}

void UEdGraphSchema_Niagara::TrySetDefaultValue(UEdGraphPin& Pin, const FString& NewDefaultValue, bool bMarkAsModified /*= true*/) const
{
	Pin.DefaultValue = NewDefaultValue;

	if (bMarkAsModified)
	{
		UEdGraphNode* Node = Pin.GetOwningNode();
		checkf(Node != nullptr, TEXT("Encountered null node owning pin!"));
		Node->PinDefaultValueChanged(&Pin);
	}
}

FNiagaraTypeDefinition ResolveNumericPinType(const UEdGraphPin* Pin)
{
	FNiagaraTypeDefinition PinType = UEdGraphSchema_Niagara::PinToTypeDefinition(Pin);
	if (PinType == FNiagaraTypeDefinition::GetGenericNumericDef())
	{
		if (UNiagaraNode* Node = Cast<UNiagaraNode>(Pin->GetOwningNode()))
		{
			if (UNiagaraGraph* NiagaraGraph = Node->GetNiagaraGraph())
			{
				return NiagaraGraph->GetCachedNumericConversion(Pin);
			}
		}
	}
	return PinType;
}

bool UEdGraphSchema_Niagara::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorCreateConnection", "Niagara Editor: Create Connection"));

	const FPinConnectionResponse Response = CanCreateConnection(PinA, PinB);
	bool bModified = false;

	FNiagaraTypeDefinition TypeA = ResolveNumericPinType(PinA);
	FNiagaraTypeDefinition TypeB = ResolveNumericPinType(PinB);
	
	switch (Response.Response)
	{
	case CONNECT_RESPONSE_MAKE:
		PinA->Modify();
		PinB->Modify();
		PinA->MakeLinkTo(PinB);
		bModified = true;
		break;

	case CONNECT_RESPONSE_BREAK_OTHERS_A:
		PinA->Modify();
		PinB->Modify();
		PinA->BreakAllPinLinks();
		PinA->MakeLinkTo(PinB);
		bModified = true;
		break;

	case CONNECT_RESPONSE_BREAK_OTHERS_B:
		PinA->Modify();
		PinB->Modify();
		PinB->BreakAllPinLinks();
		
		PinA->MakeLinkTo(PinB);
		bModified = true;
		break;

	case CONNECT_RESPONSE_BREAK_OTHERS_AB:
		PinA->Modify();
		PinB->Modify();
		PinA->BreakAllPinLinks();
		PinB->BreakAllPinLinks();
		PinA->MakeLinkTo(PinB);
		bModified = true;
		break;

	case CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE:
	{
		if (PinA->Direction == EGPD_Input)
		{
			//Swap so that A is the from pin and B is the to pin.
			UEdGraphPin* Temp = PinA;
			PinA = PinB;
			PinB = Temp;
		}

		FNiagaraTypeDefinition LocalTypeA = PinToTypeDefinition(PinA);
		FNiagaraTypeDefinition LocalTypeB = PinToTypeDefinition(PinB);
		
		if (LocalTypeA != LocalTypeB && LocalTypeA.GetClass() == nullptr && LocalTypeB.GetClass() == nullptr)
		{
			UEdGraphNode* ANode = PinA->GetOwningNode();
			UEdGraphNode* BNode = PinB->GetOwningNode();
			UEdGraph* Graph = ANode->GetTypedOuter<UEdGraph>();
			
			// Since we'll be adding a node, make sure to modify the graph itself.
			Graph->Modify();
			FGraphNodeCreator<UNiagaraNodeConvert> NodeCreator(*Graph);
			UNiagaraNodeConvert* AutoConvertNode = NodeCreator.CreateNode(false);
			AutoConvertNode->AllocateDefaultPins();
			AutoConvertNode->NodePosX = (ANode->NodePosX + BNode->NodePosX) >> 1;
			AutoConvertNode->NodePosY = (ANode->NodePosY + BNode->NodePosY) >> 1;
			NodeCreator.Finalize();

			if (AutoConvertNode->InitConversion(PinA, PinB))
			{
				PinA->Modify();
				PinB->Modify();
				bModified = true;
			}
			else
			{
				Graph->RemoveNode(AutoConvertNode);
			}
		}
	}
	break;

	case CONNECT_RESPONSE_DISALLOW:
	default:
		break;
	}

	if(Response.Response != CONNECT_RESPONSE_DISALLOW)
	{
		if (IsPinWildcard(PinA))
		{
			ConvertPinToType(PinA, TypeB);
		}
		
		if (IsPinWildcard(PinB))
		{
			ConvertPinToType(PinB, TypeA);
		}
	}
	
#if WITH_EDITOR
	if (bModified)
	{
		// nodes might not be valid if above code reconstructed new pins
		if (UEdGraphNode* NodeA = PinA->GetOwningNodeUnchecked())
		{
			NodeA->PinConnectionListChanged(PinA);
		}

		if (UEdGraphNode* NodeB = PinB->GetOwningNodeUnchecked())
		{
			NodeB->PinConnectionListChanged(PinB);
		}
	}
#endif	//#if WITH_EDITOR

	return bModified;
}

FLinearColor UEdGraphSchema_Niagara::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	if (PinType.PinCategory == PinCategoryType || PinType.PinCategory == PinCategoryStaticType)
	{
		FNiagaraTypeDefinition Type(CastChecked<UScriptStruct>(PinType.PinSubCategoryObject.Get()));
		return GetTypeColor(Type);
	}
		
	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
	if (PinType.PinCategory == PinCategoryEnum || PinType.PinCategory == PinCategoryStaticEnum)
	{
		return Settings->BytePinTypeColor;
	}
	if (PinType.PinCategory == PinCategoryClass || PinType.PinCategory == PinCategoryStaticClass)
	{
		UClass* PinClass = Cast<UClass>(PinType.PinSubCategoryObject.Get());
		// Note: we use the interface color for UObjects as most Niagara work involves DIs not UObjects and we didn't want to confuse users
		return PinClass && PinClass->IsChildOf(UNiagaraDataInterface::StaticClass()) ? Settings->ObjectPinTypeColor : Settings->InterfacePinTypeColor;
	}
	return Settings->WildcardPinTypeColor;
}

FLinearColor UEdGraphSchema_Niagara::GetTypeColor(const FNiagaraTypeDefinition& Type)
{
	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
	const UNiagaraSettings* NiagaraSettings = GetDefault<UNiagaraSettings>();
	if (Type == FNiagaraTypeDefinition::GetFloatDef() || Type == FNiagaraTypeHelper::GetDoubleDef())
	{
		return Settings->FloatPinTypeColor;
	}
	else if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()))
	{
		return Settings->IntPinTypeColor;
	}
	else if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		return Settings->BooleanPinTypeColor;
	}
	else if (Type == FNiagaraTypeDefinition::GetVec3Def() || Type == FNiagaraTypeHelper::GetVectorDef())
	{
		return Settings->VectorPinTypeColor;
	}
	else if (Type == FNiagaraTypeDefinition::GetPositionDef())
	{
		return NiagaraSettings->PositionPinTypeColor;
	}
	else if (Type == FNiagaraTypeDefinition::GetParameterMapDef())
	{
		return Settings->ExecutionPinTypeColor;
	}
	else if (Type.IsDataInterface())
	{
		return Settings->ObjectPinTypeColor;
	}
	else if (Type.IsUObject())
	{
		// Note: we use the interface color for UObjects as most Niagara work involves DIs not UObjects and we didn't want to confuse users
		return Settings->InterfacePinTypeColor;
	}
	else if (Type.IsEnum())
	{
		return Settings->BytePinTypeColor;
	}
	else if(Type == FNiagaraTypeDefinition::GetWildcardDef())
	{
		return Settings->WildcardPinTypeColor;
	}
	else
	{
		return Settings->StructPinTypeColor;
	}
}

bool UEdGraphSchema_Niagara::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	check(Pin != NULL);

	if (Pin->bDefaultValueIsIgnored)
	{
		return true;
	}

	return false;
}

FNiagaraVariable UEdGraphSchema_Niagara::PinToNiagaraVariable(const UEdGraphPin* Pin, bool bNeedsValue, ENiagaraStructConversion StructConversion)
{
	FNiagaraVariable Var = FNiagaraVariable(PinToTypeDefinition(Pin, StructConversion), Pin->PinName);
	bool bHasValue = false;
	if (Pin->bDefaultValueIsIgnored == false && Pin->DefaultValue.IsEmpty() == false)
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(Var.GetType());
		if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanHandlePinDefaults())
		{
			bHasValue = TypeEditorUtilities->SetValueFromPinDefaultString(Pin->DefaultValue, Var);
			if (bHasValue == false)
			{
				FString OwningNodePath = Pin->GetOwningNode() != nullptr ? Pin->GetOwningNode()->GetPathName() : TEXT("Unknown");
				UE_LOGF(LogNiagaraEditor, Warning, "PinToNiagaraVariable: Failed to convert default value '%ls' to type %ls. Owning node path: %ls", *Pin->DefaultValue, *Var.GetType().GetName(), *OwningNodePath);
			}
		}
		else
		{
			if (Pin->GetOwningNode() != nullptr && nullptr == Cast<UNiagaraNodeOp>(Pin->GetOwningNode()))
			{
				FString OwningNodePath = Pin->GetOwningNode() != nullptr ? Pin->GetOwningNode()->GetPathName() : TEXT("Unknown");
				UE_LOGF(LogNiagaraEditor, Warning, "Pin had default value string, but default values aren't supported for variables of type {%ls}. Owning node path: %ls", *Var.GetType().GetName(), *OwningNodePath);
			}
		}
	}

	if (bNeedsValue && bHasValue == false)
	{
		FNiagaraEditorUtilities::ResetVariableToDefaultValue(Var);
		if (Var.GetData() == nullptr)
		{
			UE_LOGF(LogNiagaraEditor, Warning, "ResetVariableToDefaultValue called, but failed on var %ls type %ls. ", *Var.GetName().ToString(), *Var.GetType().GetName());
		}
	}

	return Var;
}

bool UEdGraphSchema_Niagara::TryGetPinDefaultValueFromNiagaraVariable(const FNiagaraVariable& Variable, FString& OutPinDefaultValue) const
{
	// Create a variable we can be sure is allocated since it's required for the call to GetPinDefaultStringFromValue.
	FNiagaraVariable PinDefaultVariable = Variable;
	if (Variable.IsDataAllocated() == false)
	{
		FNiagaraEditorUtilities::ResetVariableToDefaultValue(PinDefaultVariable);
	}

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(PinDefaultVariable.GetType());
	if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanHandlePinDefaults())
	{
		OutPinDefaultValue = TypeEditorUtilities->GetPinDefaultStringFromValue(PinDefaultVariable);
		return true;
	}
	
	OutPinDefaultValue = FString();
	return false;
}

bool UEdGraphSchema_Niagara::TryGetPinDefaultValueFromNiagaraVariant(const FNiagaraVariableBase& Variable, const FNiagaraVariant& Value, FString& OutPinDefaultValue) const
{
	bool bValid = Value.IsValid(Variable.GetType());
	OutPinDefaultValue.Reset();
	if (Variable.IsDataInterface())
	{
		if (bValid)
		{
			OutPinDefaultValue = FSoftObjectPath(Value.GetDataInterface()).ToString();
			return true;
		}
	}
	else if (Variable.IsUObject())
	{
		if (bValid)
		{
			OutPinDefaultValue = FSoftObjectPath(Value.GetUObject()).ToString();
			return true;
		}
	}
	else
	{
		FNiagaraVariable PinDefaultVariable = Variable;//Just deal with data in the variable since that's what other code deals with
		if (bValid)
		{
			PinDefaultVariable.AllocateData();
			FMemory::Memcpy(PinDefaultVariable.GetData(), Value.GetBytes(), PinDefaultVariable.GetType().GetSize());
		}
		else
		{
			FNiagaraEditorUtilities::ResetVariableToDefaultValue(PinDefaultVariable);
		}

		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(Variable.GetType());
		if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanHandlePinDefaults())
		{
			OutPinDefaultValue = TypeEditorUtilities->GetPinDefaultStringFromValue(PinDefaultVariable);
			return true;
		}
	}
	OutPinDefaultValue = FString();
	return false;
}

void UEdGraphSchema_Niagara::ConvertIllegalPinsInPlace(UEdGraphPin* Pin)
{
	if (Pin == nullptr)
		return;

	if (Pin->PinType.PinCategory == PinCategoryType && Pin->PinType.PinSubCategoryObject.IsValid())
	{
		UScriptStruct* Struct = Cast<UScriptStruct>(Pin->PinType.PinSubCategoryObject.Get());
		UScriptStruct* ConvertedStruct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(Struct, ENiagaraStructConversion::UserFacing);
		if (Struct != ConvertedStruct)
		{
			Pin->PinType.PinSubCategoryObject = ConvertedStruct;
		}
	}
}

FNiagaraTypeDefinition UEdGraphSchema_Niagara::PinToTypeDefinition(const UEdGraphPin* Pin, ENiagaraStructConversion StructConversion)
{
	if (Pin == nullptr)
	{
		return FNiagaraTypeDefinition();
	}
	UEdGraphNode* OwningNode = Pin->GetOwningNodeUnchecked();
	if ((Pin->PinType.PinCategory == PinCategoryType || Pin->PinType.PinCategory == PinCategoryStaticType) && Pin->PinType.PinSubCategoryObject.IsValid())
	{
		UScriptStruct* Struct = Cast<UScriptStruct>(Pin->PinType.PinSubCategoryObject.Get());
		if (Struct == nullptr)
		{
			UE_LOGF(LogNiagaraEditor, Error, "Pin states that it is of struct type, but is missing its struct object. This is usually the result of a registered type going away. Pin Name '%ls' Owning Node '%ls'.",
				*Pin->PinName.ToString(), OwningNode ? *OwningNode->GetName() : TEXT("Invalid"));
			return FNiagaraTypeDefinition();
		}

		if (Struct && !FNiagaraTypeHelper::IsNiagaraFriendlyTopLevelStruct(Struct, StructConversion)) // LWC swapover support
		{
			if (OwningNode && OwningNode->HasAnyFlags(EObjectFlags::RF_NeedPostLoad))
			{
				UE_LOGF(LogNiagaraEditor, Verbose, "Pin states that it is not a niagara friendly struct, but didn't get converted by PostLoad yet, it still has RF_NeedPostLoad . Pin Name '%ls' Owning Node '%ls'.",
					*Pin->PinName.ToString(), OwningNode ? *OwningNode->GetPathName() : TEXT("Invalid"));
			}
			else 
			{
				UE_LOGF(LogNiagaraEditor, Verbose, "Pin states that it is not a niagara friendly struct, but didn't get converted by PostLoad yet, it does not have RF_NeedPostLoad though. Pin Name '%ls' Owning Node '%ls'.",
					*Pin->PinName.ToString(), OwningNode ? *OwningNode->GetPathName() : TEXT("Invalid"));
			}
			return FNiagaraTypeDefinition(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(Struct, StructConversion));
		}

		if (Pin->PinType.PinCategory == PinCategoryType)
			return FNiagaraTypeDefinition(Struct);
		else if (Pin->PinType.PinCategory == PinCategoryStaticType)
			return FNiagaraTypeDefinition(Struct).ToStaticDef();
	}
	else if (Pin->PinType.PinCategory == PinCategoryClass || Pin->PinType.PinCategory == PinCategoryStaticClass)
	{
		UClass* Class = Cast<UClass>(Pin->PinType.PinSubCategoryObject.Get());
		if (Class == nullptr)
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Pin states that it is of class type, but is missing its class object. This is usually the result of a registered type going away. Pin Name '%ls' Owning Node '%ls'.",
				*Pin->PinName.ToString(), OwningNode ? *OwningNode->GetFullName() : TEXT("Invalid"));
			return FNiagaraTypeDefinition();
		}

		if (Pin->PinType.PinCategory == PinCategoryClass)	
			return FNiagaraTypeDefinition(Class);
		else if (Pin->PinType.PinCategory == PinCategoryStaticClass)
			return FNiagaraTypeDefinition(Class).ToStaticDef();
	}
	else if (Pin->PinType.PinCategory == PinCategoryEnum || Pin->PinType.PinCategory == PinCategoryStaticEnum)
	{
		UEnum* Enum = Cast<UEnum>(Pin->PinType.PinSubCategoryObject.Get());
		if (Enum == nullptr)
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Pin states that it is of Enum type, but is missing its Enum! Pin Name '%ls' Owning Node '%ls'. Turning into standard int definition!", *Pin->PinName.ToString(),
				OwningNode ? *OwningNode->GetFullName() : TEXT("Invalid"));
			return FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetIntDef());
		}

		if (Pin->PinType.PinCategory == PinCategoryEnum)
			return FNiagaraTypeDefinition(Enum);
		else if (Pin->PinType.PinCategory == PinCategoryStaticEnum)
			return FNiagaraTypeDefinition(Enum).ToStaticDef();
	}
	return FNiagaraTypeDefinition();
}

FNiagaraTypeDefinition UEdGraphSchema_Niagara::PinTypeToTypeDefinition(const FEdGraphPinType& PinType)
{
	if ((PinType.PinCategory == PinCategoryType || PinType.PinCategory == PinCategoryStaticType) && PinType.PinSubCategoryObject.IsValid())
	{
		UScriptStruct* Struct = Cast<UScriptStruct>(PinType.PinSubCategoryObject.Get());
		if (Struct == nullptr)
		{
			UE_LOGF(LogNiagaraEditor, Error, "Pin states that it is of struct type, but is missing its struct object. This is usually the result of a registered type going away.");
			return FNiagaraTypeDefinition();
		}

		if (PinType.PinCategory == PinCategoryType)
			return FNiagaraTypeDefinition(Struct);
		else if (PinType.PinCategory == PinCategoryStaticType)
			return FNiagaraTypeDefinition(Struct).ToStaticDef();
	}
	else if (PinType.PinCategory == PinCategoryClass || PinType.PinCategory == PinCategoryStaticClass)
	{
		UClass* Class = Cast<UClass>(PinType.PinSubCategoryObject.Get());
		if (Class == nullptr)
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Pin states that it is of class type, but is missing its class object. This is usually the result of a registered type going away.");
			return FNiagaraTypeDefinition();
		}

		if (PinType.PinCategory == PinCategoryClass)
			return FNiagaraTypeDefinition(Class);
		else if (PinType.PinCategory == PinCategoryStaticClass)
			return FNiagaraTypeDefinition(Class).ToStaticDef();
	}
	else if (PinType.PinCategory == PinCategoryEnum || PinType.PinCategory == PinCategoryStaticEnum)
	{
		UEnum* Enum = Cast<UEnum>(PinType.PinSubCategoryObject.Get());
		if (Enum == nullptr)
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Pin states that it is of Enum type, but is missing its Enum! Turning into standard int definition!");
			return FNiagaraTypeDefinition(FNiagaraTypeDefinition::GetIntDef());
		}

		if (PinType.PinCategory == PinCategoryEnum)
			return FNiagaraTypeDefinition(Enum);
		//else if (PinType.PinCategory == PinCategoryEnum)
		//	return FNiagaraTypeDefinition(Enum).ToStaticDef();
	}
	
	return FNiagaraTypeDefinition();
}

FEdGraphPinType UEdGraphSchema_Niagara::TypeDefinitionToPinType(FNiagaraTypeDefinition TypeDef)
{
	if (TypeDef.GetClass())
	{
		if (TypeDef.IsStatic())
			return FEdGraphPinType(PinCategoryStaticClass, NAME_None, const_cast<UClass*>(TypeDef.GetClass()), EPinContainerType::None, false, FEdGraphTerminalType());
		else
			return FEdGraphPinType(PinCategoryClass, NAME_None, const_cast<UClass*>(TypeDef.GetClass()), EPinContainerType::None, false, FEdGraphTerminalType());
	}
	else if (TypeDef.GetEnum())
	{
		if (TypeDef.IsStatic())
			return FEdGraphPinType(PinCategoryStaticEnum, NAME_None, const_cast<UEnum*>(TypeDef.GetEnum()), EPinContainerType::None, false, FEdGraphTerminalType());
		else
			return FEdGraphPinType(PinCategoryEnum, NAME_None, const_cast<UEnum*>(TypeDef.GetEnum()), EPinContainerType::None, false, FEdGraphTerminalType());
	}	
	else
	{
		//TODO: Are base types better as structs or done like BPS as a special name?
		if (TypeDef.IsStatic())
			return FEdGraphPinType(PinCategoryStaticType, NAME_None, const_cast<UScriptStruct*>(TypeDef.GetScriptStruct()), EPinContainerType::None, false, FEdGraphTerminalType());
		else
			return FEdGraphPinType(PinCategoryType, NAME_None, const_cast<UScriptStruct*>(TypeDef.GetScriptStruct()), EPinContainerType::None, false, FEdGraphTerminalType());

	}
}
bool UEdGraphSchema_Niagara::IsPinStatic(const UEdGraphPin* Pin)
{
	return Pin->PinType.PinCategory == PinCategoryStaticEnum || Pin->PinType.PinCategory == PinCategoryStaticClass || Pin->PinType.PinCategory == PinCategoryStaticType;
}

bool UEdGraphSchema_Niagara::IsPinWildcard(const UEdGraphPin* Pin)
{
	return Pin->PinType.PinCategory == PinCategoryType && Pin->PinType.PinSubCategoryObject == FNiagaraTypeDefinition::GetWildcardStruct();
}

FPinConnectionResponse UEdGraphSchema_Niagara::GetWildcardConnectionResponse(const UEdGraphPin* PinA, const UEdGraphPin* PinB)
{
	FNiagaraTypeDefinition PinAType = PinToTypeDefinition(PinA);
	FNiagaraTypeDefinition PinBType	= PinToTypeDefinition(PinB);
	
	ensure(PinAType == FNiagaraTypeDefinition::GetWildcardDef() || PinBType == FNiagaraTypeDefinition::GetWildcardDef());
	
	const UNiagaraNode* NodeA = CastChecked<UNiagaraNode>(PinA->GetOwningNode());
	const UNiagaraNode* NodeB = CastChecked<UNiagaraNode>(PinB->GetOwningNode());

	bool bPinsSwapped = false;
	// ensure that at least PinA is a wildcard
	if (IsPinWildcard(PinB))
	{
		const UEdGraphPin* TmpPin = PinA;
		const UNiagaraNode* TmpNode = NodeA;
		const FNiagaraTypeDefinition TmpType = PinAType;
		PinA = PinB;
		PinB = TmpPin;
		NodeA = NodeB;
		NodeB = TmpNode;
		PinAType = PinBType;
		PinBType = TmpType;
		bPinsSwapped = true;
	}

	// we can't link to numeric types, but maybe it was already resolved to a usable type
	if (PinBType == FNiagaraTypeDefinition::GetGenericNumericDef())
	{
		if (UNiagaraGraph* NiagaraGraph = const_cast<UNiagaraGraph*>(NodeB->GetNiagaraGraph()))
		{
			FNiagaraTypeDefinition ResolvedType = NiagaraGraph->GetCachedNumericConversion(PinB);
			PinBType = ResolvedType;
		}
	}
	
	FString Message;
	ECanCreateConnectionResponse Response = CONNECT_RESPONSE_DISALLOW;
	if (PinBType == FNiagaraTypeDefinition::GetWildcardDef())
	{
		Response = ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW;
		Message = TEXT("Can't convert wildcard to wildcard.");
		return FPinConnectionResponse(Response, Message);
	}
	else
	{
		if (NodeA->AllowNiagaraTypeForPinTypeChange(PinBType, const_cast<UEdGraphPin*>(PinA)))
		{
			if(PinA->Direction == EGPD_Input && PinA->LinkedTo.Num() > 0)
			{
				if(!bPinsSwapped)
				{
					Response = CONNECT_RESPONSE_BREAK_OTHERS_A;			
				}
				else
				{
					Response = CONNECT_RESPONSE_BREAK_OTHERS_B;
				}
			}
			else if(PinB->Direction == EGPD_Input && PinB->LinkedTo.Num() > 0)
			{
				if(!bPinsSwapped)
				{
					Response = CONNECT_RESPONSE_BREAK_OTHERS_B;			
				}
				else
				{
					Response = CONNECT_RESPONSE_BREAK_OTHERS_A;
				}
			}
			else
			{
				Response = CONNECT_RESPONSE_MAKE;
			}
			
			Message = FString::Printf(TEXT("Convert wildcard to %s."), *PinBType.GetName());
		}
		else
		{
			Response = CONNECT_RESPONSE_DISALLOW;;
			Message = FString::Printf(TEXT("Can't convert wildcard to %s."), *PinBType.GetName());
		}
	}

	NodeA->GetWildcardPinHoverConnectionTextAddition(PinA, PinB, Response, Message);
	return FPinConnectionResponse(Response, Message);
}

bool UEdGraphSchema_Niagara::IsSystemConstant(const FNiagaraVariable& Variable)const
{
	return FNiagaraConstants::GetEngineConstants().Find(Variable) != INDEX_NONE;
}

UNiagaraParameterCollection* UEdGraphSchema_Niagara::VariableIsFromParameterCollection(const FNiagaraVariable& Var)const
{
	FNameBuilder VarName(Var.GetName());
	if (VarName.ToView().StartsWith(TEXT("NPC.")))
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		if (UNiagaraParameterCollection* Collection = NiagaraEditorModule.FindCollectionForVariable(FString(VarName.ToView())))
		{
			return Collection;
		}
	}
	return nullptr;
}

UNiagaraParameterCollection* UEdGraphSchema_Niagara::VariableIsFromParameterCollection(const FString& VarName, bool bAllowPartialMatch, FNiagaraVariable& OutVar)const
{
	OutVar = FNiagaraVariable();

	if (VarName.StartsWith(TEXT("NPC.")))
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		if (UNiagaraParameterCollection* Collection = NiagaraEditorModule.FindCollectionForVariable(VarName))
		{
			const TArray<FNiagaraVariable>& CollectionVariables = Collection->GetParameters();
			FString BestMatchSoFar;

			for (const FNiagaraVariable& CollVar : CollectionVariables)
			{
				FString CollVarName = CollVar.GetName().ToString();
				if (CollVarName == VarName)
				{
					OutVar = CollVar;
					break;
				}
				else if (bAllowPartialMatch && VarName.StartsWith(CollVarName + TEXT(".")) && (BestMatchSoFar.Len() == 0 || CollVarName.Len() > BestMatchSoFar.Len()))
				{
					OutVar = CollVar;
					BestMatchSoFar = CollVarName;
				}
			}
			return Collection;
		}
	}
	return nullptr;
}

bool UEdGraphSchema_Niagara::IsValidNiagaraPropertyType(const FProperty* Property) const
{
	return Property != nullptr &&
		(Property->IsA(FFloatProperty::StaticClass()) ||
		Property->IsA(FDoubleProperty::StaticClass()) ||
		Property->IsA(FIntProperty::StaticClass()) ||
		Property->IsA(FBoolProperty::StaticClass()) ||
		Property->IsA(FEnumProperty::StaticClass()) ||
		Property->IsA(FStructProperty::StaticClass()) ||
		Property->IsA(FUInt16Property::StaticClass()));
}

FNiagaraTypeDefinition UEdGraphSchema_Niagara::GetTypeDefForProperty(const FProperty* Property)const
{
	if (Property->IsA(FFloatProperty::StaticClass()) || Property->IsA(FDoubleProperty::StaticClass()))
	{
		return FNiagaraTypeDefinition::GetFloatDef();
	}
	if (Property->IsA(FIntProperty::StaticClass()))
	{
		return FNiagaraTypeDefinition::GetIntDef();
	}
	if (Property->IsA(FBoolProperty::StaticClass()))
	{
		return FNiagaraTypeDefinition::GetBoolDef();
	}	
	if (Property->IsA(FEnumProperty::StaticClass()))
	{
		const FEnumProperty* EnumProp = CastField<FEnumProperty>(Property);
		return FNiagaraTypeDefinition(EnumProp->GetEnum());
	}
	if (const FStructProperty* StructProp = CastField<const FStructProperty>(Property))
	{
		return FNiagaraTypeDefinition(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::UserFacing));
	}
	if(Property->IsA(FUInt16Property::StaticClass()))
	{
		return FNiagaraTypeDefinition::GetHalfDef();
	}

	checkf(false, TEXT("Could not find type for Property(%s) Type(%s)"), *Property->GetName(), *Property->GetClass()->GetName());
	return FNiagaraTypeDefinition::GetFloatDef();//Some invalid type?
}

void UEdGraphSchema_Niagara::ConvertNumericPinToTypeAll(UNiagaraNode* InNode, FNiagaraTypeDefinition TypeDef)
{
	if (InNode)
	{
		FScopedTransaction AllTransaction(NSLOCTEXT("UnrealEd", "NiagaraEditorChangeNumericPinTypeAll", "Change Pin Type For All"));
		for (auto Pin : InNode->Pins)
		{
			if (PinToTypeDefinition(Pin) == FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorChangeNumericPinType", "Change Pin Type"));
				if (false == InNode->ConvertNumericPinToType(Pin, TypeDef))
				{
					Transaction.Cancel();
				}
			}
		}
	}
}

void UEdGraphSchema_Niagara::ConvertPinToType(UEdGraphPin* InPin, FNiagaraTypeDefinition TypeDef) const
{
	if (PinToTypeDefinition(InPin) != TypeDef)
	{
		UNiagaraNode* Node = Cast<UNiagaraNode>(InPin->GetOwningNode());
		if (Node)
		{
			FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorChangeNumericPinType", "Change Pin Type"));
			Node->RequestNewPinType(InPin, TypeDef);
		}
	}
}

void UEdGraphSchema_Niagara::ConvertNumericPinToType(UEdGraphPin* InGraphPin, FNiagaraTypeDefinition TypeDef)
{
	if (PinToTypeDefinition(InGraphPin) != TypeDef)
	{
		UNiagaraNode* Node = Cast<UNiagaraNode>(InGraphPin->GetOwningNode());
		if (Node)
		{
			FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorChangeNumericPinType", "Change Pin Type"));
			if (false == Node->ConvertNumericPinToType(InGraphPin, TypeDef))
			{
				Transaction.Cancel();
			}
		}
	}
}

bool UEdGraphSchema_Niagara::PinTypesValidForNumericConversion(FEdGraphPinType AType, FEdGraphPinType BType) const
{
	if (AType == BType)
	{
		return true;
	}
	else if ((AType.PinCategory == PinCategoryStaticType || AType.PinCategory == PinCategoryStaticEnum) && BType.PinCategory == PinCategoryType)
	{
		return true;
	}
	else if ((AType.PinCategory == PinCategoryType || AType.PinCategory == PinCategoryEnum) && BType.PinCategory == PinCategoryType)
	{
		return true;
	}

	return false;
}

bool UEdGraphSchema_Niagara::IsStaticPin(const UEdGraphPin* Pin)
{
	if (Pin)
	{
		if (Pin->PinType.PinCategory == PinCategoryStaticType)
			return true;
		if (Pin->PinType.PinCategory == PinCategoryStaticClass)
			return true;
		if (Pin->PinType.PinCategory == PinCategoryStaticEnum)
			return true;
	}
	return false;
}

bool UEdGraphSchema_Niagara::CheckCircularConnection(TSet<const UEdGraphNode*>& VisitedNodes, const UEdGraphNode* InNode, const UEdGraphNode* InTestNode)
{
	bool AlreadyAdded = false;

	VisitedNodes.Add(InNode, &AlreadyAdded);
	if (AlreadyAdded)
	{
		// node is already in our set, so return so we don't reprocess it
		return false;
	}

	if (InNode == InTestNode)
	{
		// we've found a match, so we have a circular reference
		return true;
	}

	// iterate over all of the nodes that are inputs to InNode
	for (const UEdGraphPin* Pin : InNode->GetAllPins())
	{
		if (Pin && Pin->Direction == EGPD_Input)
		{
			for (const UEdGraphPin* OutputPin : Pin->LinkedTo)
			{
				if (const UEdGraphNode* InputNode = OutputPin ? OutputPin->GetOwningNode() : nullptr)
				{
					if (CheckCircularConnection(VisitedNodes, InputNode, InTestNode))
					{
						return true;
					}
				}
			}

		}
	}

	return false;
}

void UEdGraphSchema_Niagara::GetNumericConversionToSubMenuActions(UToolMenu* Menu, const FName SectionName, UEdGraphPin* InGraphPin)
{
	FToolMenuSection& Section = Menu->FindOrAddSection(SectionName);

	// Add all the types we could convert to
	for (const FNiagaraTypeDefinition& TypeDef : FNiagaraTypeRegistry::GetNumericTypes())
	{
		FText Title = TypeDef.GetNameText();

		FText Description;
		FFormatNamedArguments Args;
		Args.Add(TEXT("TypeTitle"), Title);
		Description = FText::Format(LOCTEXT("NumericConversionText", "{TypeTitle}"), Args);
		Section.AddMenuEntry(NAME_None, Description, Description, FSlateIcon(), FUIAction(
			FExecuteAction::CreateUObject((UEdGraphSchema_Niagara*const)this, &UEdGraphSchema_Niagara::ConvertNumericPinToType, const_cast<UEdGraphPin*>(InGraphPin), FNiagaraTypeDefinition(TypeDef))));
	}
}

void UEdGraphSchema_Niagara::GetNumericConversionToSubMenuActionsAll(UToolMenu* Menu, const FName SectionName, UNiagaraNode* InNode)
{
	FToolMenuSection& Section = Menu->FindOrAddSection(SectionName);

	// Add all the types we could convert to
	for (const FNiagaraTypeDefinition& TypeDef : FNiagaraTypeRegistry::GetNumericTypes())
	{
		FText Title = TypeDef.GetNameText();

		FText Description;
		FFormatNamedArguments Args;
		Args.Add(TEXT("TypeTitle"), Title);
		Description = FText::Format(LOCTEXT("NumericConversionText", "{TypeTitle}"), Args);
		Section.AddMenuEntry(NAME_None, Description, Description, FSlateIcon(), FUIAction(
			FExecuteAction::CreateUObject((UEdGraphSchema_Niagara*const)this, &UEdGraphSchema_Niagara::ConvertNumericPinToTypeAll, InNode, FNiagaraTypeDefinition(TypeDef))));
	}
}

TArray<TTuple<FString, FString, FText>> UEdGraphSchema_Niagara::GetDataInterfaceFunctionPrototypes(const UEdGraphPin* InGraphPin)
{
	TArray<TTuple<FString, FString, FText>> FunctionPrototypes;
	if (InGraphPin)
	{
		FNiagaraTypeDefinition GraphPinType = UEdGraphSchema_Niagara::PinToTypeDefinition(InGraphPin);
		if (GraphPinType.IsDataInterface())
		{
			UNiagaraDataInterface* DataInterface = CastChecked<UNiagaraDataInterface>(GraphPinType.GetClass()->GetDefaultObject());
			
			// No functions, don't add an option
			TArray<FNiagaraFunctionSignature> FunctionSignatures;
			DataInterface->GetFunctionSignatures(FunctionSignatures);
			if (FunctionSignatures.Num() == 0)
			{
				return FunctionPrototypes;
			}

			// Make the data interface name
			FString DataInterfaceName = InGraphPin->GetName();
			{
				int32 LastPinNameDot = 0;
				if (DataInterfaceName.FindLastChar('.', LastPinNameDot))
				{
					DataInterfaceName.RightChopInline(LastPinNameDot + 1);
				}
				DataInterfaceName = FNiagaraHlslTranslator::GetSanitizedSymbolName(DataInterfaceName);
			}

			// Generate all prototypes
			FunctionPrototypes.Reserve(FunctionSignatures.Num());
			for (const FNiagaraFunctionSignature& FunctionSignature : FunctionSignatures)
			{
				if (FunctionSignature.bSoftDeprecatedFunction || FunctionSignature.bHidden )
				{
					continue;
				}

				if (!FunctionSignature.bSupportsGPU && !FunctionSignature.bSupportsCPU)
				{
					continue;
				}

				FString Prototype = FNiagaraHlslTranslator::GenerateFunctionHlslPrototype(DataInterfaceName, FunctionSignature);
				if ( Prototype.Len() > 0 )
				{
					Prototype.Append(TEXT("\r\n"));
					FText Description = FText::Format(LOCTEXT("DataInterfacePrototypeDescriptionFormat", "{0}\n{1}"), FText::FromString(Prototype), FunctionSignature.GetDescription());
					FunctionPrototypes.Emplace(FunctionSignature.GetNameString(), Prototype, Description);
				}
			}
		}
	}
	return FunctionPrototypes;
}

void UEdGraphSchema_Niagara::GenerateDataInterfacePinMenu(UToolMenu* ToolMenu, const FName SectionName, const UEdGraphPin* GraphPin, FNiagaraTypeDefinition TypeDef) const
{
	TArray<TTuple<FString, FString, FText>> FunctionPrototypes = GetDataInterfaceFunctionPrototypes(GraphPin);
	if (FunctionPrototypes.Num () == 0)
	{
		return;
	}

	Algo::Sort(FunctionPrototypes, [](const TTuple<FString, FString, FText>& Lhs, const TTuple<FString, FString, FText>& Rhs) { return Lhs.Get<0>() < Rhs.Get<0>(); });

	// Make the menu
	FToolMenuSection& Section = ToolMenu->FindOrAddSection(SectionName);
	Section.AddSubMenu(
		"DataInterfaceFunctionHLSL",
		LOCTEXT("DataInterfaceFunctionHLSL", "Data Interface Function HLSL..."),
		LOCTEXT("DataInterfaceFunctionHLSLToolTip", "List of all data interface functions, selecting will copy the prototype into the clipboard for easy HLSL coding."),
		FNewToolMenuDelegate::CreateLambda(
			[SectionName, FunctionPrototypes](UToolMenu* ToolMenu)
			{
				FString AllPrototypes;
				for ( const auto& Prototype : FunctionPrototypes )
				{
					AllPrototypes += Prototype.Get<1>();
				}

				FToolMenuSection& Section = ToolMenu->FindOrAddSection(SectionName);
				Section.AddMenuEntry(
					NAME_None,
					LOCTEXT("CopyAllToClipboard", "Copy All To Clipboard"),
					LOCTEXT("CopyAllToClipboardTooltip", "Copies all functions into the clipboard"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
						FUIAction(FExecuteAction::CreateLambda([ClipboardText=MoveTemp(AllPrototypes)]() { FPlatformApplicationMisc::ClipboardCopy(*ClipboardText); }))
				);
				Section.AddSeparator(NAME_None);

				for (const auto& Prototype : FunctionPrototypes)
				{
					const FString SaveClipboardText = Prototype.Get<1>();
					Section.AddMenuEntry(
						NAME_None,
						FText::FromString(Prototype.Get<0>()),
						Prototype.Get<2>(),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
						FUIAction(FExecuteAction::CreateLambda([ClipboardText=SaveClipboardText]() { FPlatformApplicationMisc::ClipboardCopy(*ClipboardText); }))
					);
				}
			}
		)
	);
}

void UEdGraphSchema_Niagara::ToggleNodeEnabledState(UNiagaraNode* InNode) const
{
	if (InNode != nullptr)
	{
		if (InNode->GetDesiredEnabledState() == ENodeEnabledState::Disabled)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorSetNodeEnabled", "Enabled Node"));
			InNode->Modify();
			InNode->SetEnabledState(ENodeEnabledState::Enabled, true);
			InNode->MarkNodeRequiresSynchronization(__FUNCTION__, true);
		}
		else if (InNode->GetDesiredEnabledState() == ENodeEnabledState::Enabled)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorSetNodeDisabled", "Disabled Node"));
			InNode->Modify();
			InNode->SetEnabledState(ENodeEnabledState::Disabled, true);
			InNode->MarkNodeRequiresSynchronization(__FUNCTION__, true);
		}
	}
}

void UEdGraphSchema_Niagara::RefreshNode(UNiagaraNode* InNode) const
{
	if (InNode != nullptr)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorRefreshNode", "Refresh Node"));
		InNode->Modify();
		if (InNode->RefreshFromExternalChanges())
		{
			InNode->MarkNodeRequiresSynchronization(__FUNCTION__, true);
		}
	}
}

bool UEdGraphSchema_Niagara::CanPromoteSinglePinToParameter(const UEdGraphPin* SourcePin) 
{
	const UNiagaraGraph* NiagaraGraph = Cast<UNiagaraGraph>(SourcePin->GetOwningNode()->GetGraph());
	if (IsFunctionGraph(NiagaraGraph))
	{
		return true;
	}
	return false;
}

void UEdGraphSchema_Niagara::PromoteSinglePinToParameter(UEdGraphPin* SourcePin)
{
	if (SourcePin)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorPromote", "Promote To Parameter"));
		{
			TSharedPtr<FNiagaraSchemaAction_NewNode> InputAction = TSharedPtr<FNiagaraSchemaAction_NewNode>(new FNiagaraSchemaAction_NewNode(FText::GetEmpty(), FText::GetEmpty(), NAME_None, FText::GetEmpty(), 0));
			UNiagaraNodeInput* InputNode = NewObject<UNiagaraNodeInput>(GetTransientPackage());
			FNiagaraVariable Var = PinToNiagaraVariable(SourcePin);
			UNiagaraGraph* Graph = Cast<UNiagaraGraph>(SourcePin->GetOwningNode()->GetGraph());
			FNiagaraEditorUtilities::InitializeParameterInputNode(*InputNode, Var.GetType(), Graph);
			InputAction->NodeTemplate = InputNode;

			UEdGraphNode* PinNode = SourcePin->GetOwningNode();

			const float PinVisualOffsetX = 175.0f;
			InputAction->PerformAction(Graph, SourcePin, FVector2f(PinNode->NodePosX - PinVisualOffsetX, PinNode->NodePosY));
		}
	}
}

bool CanResetPinToDefault(const UEdGraphSchema_Niagara* Schema, const UEdGraphPin* Pin)
{
	return Schema->DoesDefaultValueMatchAutogenerated(*Pin) == false;
}

void UEdGraphSchema_Niagara::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	const UEdGraphNode* InGraphNode = Context->Node;
	const UEdGraphPin* InGraphPin = Context->Pin;
	if (InGraphPin)
	{
		const FName SectionName = "EdGraphSchemaPinActions";
		FToolMenuSection& Section = Menu->FindOrAddSection(SectionName);
		FNiagaraTypeDefinition InGraphPinType = PinToTypeDefinition(InGraphPin);
		if (InGraphPinType == FNiagaraTypeDefinition::GetGenericNumericDef() && InGraphPin->LinkedTo.Num() == 0)
		{
			Section.AddSubMenu(
				"ConvertNumericSpecific",
				LOCTEXT("ConvertNumericSpecific", "Convert Numeric To..."),
				LOCTEXT("ConvertNumericSpecificToolTip", "Convert Numeric pin to the specific typed pin."),
			FNewToolMenuDelegate::CreateUObject((UEdGraphSchema_Niagara*const)this, &UEdGraphSchema_Niagara::GetNumericConversionToSubMenuActions, SectionName, const_cast<UEdGraphPin*>(InGraphPin)));
		}
		else if ( InGraphPinType.IsDataInterface() )
		{
			GenerateDataInterfacePinMenu(Menu, SectionName, InGraphPin, InGraphPinType);
		}

		if (InGraphPin->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			FUIAction PromoteToParameterAction(FExecuteAction::CreateUObject((UEdGraphSchema_Niagara*const)this, &UEdGraphSchema_Niagara::PromoteSinglePinToParameter, const_cast<UEdGraphPin*>(InGraphPin)),					
				FCanExecuteAction::CreateStatic(&UEdGraphSchema_Niagara::CanPromoteSinglePinToParameter, InGraphPin));
			PromoteToParameterAction.IsActionVisibleDelegate = FIsActionButtonVisible::CreateStatic(&UEdGraphSchema_Niagara::CanPromoteSinglePinToParameter, InGraphPin);
			
			Section.AddMenuEntry(
			"PromoteToParameter",
			LOCTEXT("PromoteToParameter", "Promote to Parameter"),
			LOCTEXT("PromoteToParameterTooltip", "Create a parameter argument and connect this pin to that parameter."),
				FSlateIcon(),
				PromoteToParameterAction
			);
			
			if (InGraphPin->LinkedTo.Num() == 0 && InGraphPin->bDefaultValueIsIgnored == false)
			{
				FUIAction ResetToDefaultAction(FExecuteAction::CreateUObject((UEdGraphSchema_Niagara*const)this, &UEdGraphSchema_Niagara::ResetPinToAutogeneratedDefaultValue, const_cast<UEdGraphPin*>(InGraphPin), true),
					FCanExecuteAction::CreateStatic(&CanResetPinToDefault, this, InGraphPin));
				ResetToDefaultAction.IsActionVisibleDelegate = FIsActionButtonVisible::CreateStatic(&CanResetPinToDefault, this, InGraphPin);
				
				Section.AddMenuEntry(
					"ResetInputToDefault",
					LOCTEXT("ResetInputToDefault", "Reset to Default"),
					LOCTEXT("ResetInputToDefaultToolTip", "Reset this input to its default value."),
					FSlateIcon(),
					ResetToDefaultAction
				);
			}
		}
	}
	else if (InGraphNode)
	{
		if (InGraphNode->IsA<UEdGraphNode_Comment>())
		{
			//Comment boxes do not support enable/disable or pin handling, so exit out now
			return;
		}

		const UNiagaraNode* Node = Cast<UNiagaraNode>(InGraphNode);
		if (Node == nullptr)
		{
			ensureMsgf(false, TEXT("Encountered unexpected node type when creating context menu actions for Niagara Script Graph!"));
			return;
		}

		bool bHasNumerics = false;
		for (auto Pin : Node->Pins)
		{
			if (PinToTypeDefinition(Pin) == FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				bHasNumerics = true;
				break;
			}
		}
		if (bHasNumerics)
		{
			const FName SectionName = "EdGraphSchema_NiagaraNodeActions";
			FToolMenuSection& Section = Menu->AddSection(SectionName, LOCTEXT("PinConversionMenuHeader", "Convert Pins"));
			Section.AddSubMenu(
				"ConvertAllNumericSpecific",
				LOCTEXT("ConvertAllNumericSpecific", "Convert All Numerics To..."),
				LOCTEXT("ConvertAllNumericSpecificToolTip", "Convert all Numeric pins to the specific typed pin."),
				FNewToolMenuDelegate::CreateUObject((UEdGraphSchema_Niagara*const)this, &UEdGraphSchema_Niagara::GetNumericConversionToSubMenuActionsAll, SectionName, const_cast<UNiagaraNode*>(Node)));
		}
		
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchema_NiagaraNodeActions", LOCTEXT("NodeActionsMenuHeader", "Niagara Node Actions"));
		Section.AddMenuEntry("ToggleEnabledState", LOCTEXT("ToggleEnabledState", "Toggle Enabled State"), LOCTEXT("ToggleEnabledStateTooltip", "Toggle this node between Enbled (default) and Disabled (skipped from compilation)."), FSlateIcon(),
			FUIAction(FExecuteAction::CreateUObject((UEdGraphSchema_Niagara*const)this, &UEdGraphSchema_Niagara::ToggleNodeEnabledState, const_cast<UNiagaraNode*>(Node))));
		Section.AddMenuEntry("RefreshNode", LOCTEXT("RefreshNode", "Refresh Node"), LOCTEXT("RefreshNodeTooltip", "Refresh this node."), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Refresh"),
			FUIAction(FExecuteAction::CreateUObject((UEdGraphSchema_Niagara*const)this, &UEdGraphSchema_Niagara::RefreshNode, const_cast<UNiagaraNode*>(Node))));
	}

	Super::GetContextMenuActions(Menu, Context);
}

TSharedPtr<IAssetReferenceFilter> UEdGraphSchema_Niagara::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (Graph)
	{
		if (UNiagaraScript* Script = Graph->GetTypedOuter<UNiagaraScript>())
		{
			TArray<TSharedPtr<FNiagaraScriptViewModel>> ViewModels;
			TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>::GetAllViewModelsForObject(Script, ViewModels);
			if (ViewModels.Num() == 1)
			{
				return ViewModels[0]->MakeAssetReferenceFilter();
			}
			else
			{
				ensureMsgf(false, TEXT("Missing or duplicate script view models detected.  Can not make IAssetReferenceFilter."));
			}
		}
	}

	return {};
}

FNiagaraConnectionDrawingPolicy::FNiagaraConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraph)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
	, GraphObj(CastChecked<UNiagaraGraph>(InGraph))
{
	ArrowImage = nullptr;
	ArrowRadius = FVector2D::ZeroVector;
	
	DefaultDataWireThickness = Settings->DefaultDataWireThickness;
	DefaultExecutionWireThickness = Settings->DefaultExecutionWireThickness;
}

void FNiagaraConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	FConnectionDrawingPolicy::DetermineWiringStyle(OutputPin, InputPin, Params);
	
	if (GraphObj)
	{
		const UEdGraphSchema_Niagara* NSchema = Cast<UEdGraphSchema_Niagara>(GraphObj->GetSchema());
		if (NSchema && OutputPin)
		{
			Params.WireColor = NSchema->GetPinTypeColor(OutputPin->PinType);
			if (NSchema->PinToTypeDefinition(OutputPin) == FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				FNiagaraTypeDefinition NewDef = GraphObj->GetCachedNumericConversion(OutputPin);
				if (NewDef.IsValid())
				{
					FEdGraphPinType NewPinType = NSchema->TypeDefinitionToPinType(NewDef);
					Params.WireColor = NSchema->GetPinTypeColor(NewPinType);
				}
			}
		}
	
		if(OutputPin && InputPin)
		{
			if(OutputPin->bOrphanedPin || InputPin->bOrphanedPin)
			{
				Params.WireColor = FLinearColor::Red;
			}
		}
	}

	Params.WireThickness = DefaultDataWireThickness;
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;
	
	// Get the schema and grab the default color from it
	check(OutputPin);
	check(GraphObj);
	
	UEdGraphNode* OutputNode = (OutputPin != nullptr) ? OutputPin->GetOwningNode() : nullptr;
	UEdGraphNode* InputNode = (InputPin != nullptr) ? InputPin->GetOwningNode() : nullptr;
	
	const bool bDeemphasizeUnhoveredPins = HoveredPins.Num() > 0;
	
	// If the output or input connect to a knot that is going backwards, we will flip the direction on values going into them
	if (UNiagaraNodeReroute* OutputRerouteNode = Cast<UNiagaraNodeReroute>(OutputNode))
	{
		if (ShouldChangeTangentForReroute(OutputRerouteNode))
		{
			Params.StartDirection = EGPD_Input;
		}
	}
	
	if (UNiagaraNodeReroute* InputRerouteNode = Cast<UNiagaraNodeReroute>(InputNode))
	{
		if (ShouldChangeTangentForReroute(InputRerouteNode))
		{
			Params.EndDirection = EGPD_Output;
		}
	}
	
	// Container types should draw thicker
	if ((InputPin && InputPin->PinType.IsContainer()) || (OutputPin && OutputPin->PinType.IsContainer()))
	{
		Params.WireThickness = DefaultExecutionWireThickness;
	}
	
	// If either end of the connection is not enabled (and not a passthru to something else), draw the wire differently
	bool bWireIsOnDisabledNodeAndNotPassthru = false;
	if (OutputNode && (OutputNode->IsDisplayAsDisabledForced() || !OutputNode->IsNodeEnabled()))
	{
		if (OutputNode->GetPassThroughPin(OutputPin) == nullptr)
		{
			bWireIsOnDisabledNodeAndNotPassthru = true;
		}
	}
	
	if (InputNode && (InputNode->IsDisplayAsDisabledForced() || !InputNode->IsNodeEnabled()))
	{
		if (InputNode->GetPassThroughPin(InputPin) == nullptr)
		{
			bWireIsOnDisabledNodeAndNotPassthru = true;
		}
	}
	
	if ((OutputPin && OutputPin->GetOwningNode()->IsNodeUnrelated()) || (InputPin && InputPin->GetOwningNode()->IsNodeUnrelated()))
	{
		bWireIsOnDisabledNodeAndNotPassthru = true;
	}
	
	if (bWireIsOnDisabledNodeAndNotPassthru)
	{
		Params.WireColor *= 0.5f;
		Params.WireThickness = 0.5f;
	}
	
	if (bDeemphasizeUnhoveredPins)
	{
		ApplyHoverDeemphasis(OutputPin, InputPin, /*inout*/ Params.WireThickness, /*inout*/ Params.WireColor);
	}
}

bool FNiagaraConnectionDrawingPolicy::ShouldChangeTangentForReroute(UNiagaraNodeReroute* Node)
{
	if (bool* pResult = RerouteToReversedDirectionMap.Find(Node))
	{
		return *pResult;
	}
	else
	{
		bool bPinReversed = false;

		FVector2D AverageLeftPin;
		FVector2D AverageRightPin;
		FVector2D CenterPin;
		bool bCenterValid = FindPinCenter(Node->GetOutputPin(0), /*out*/ CenterPin);
		bool bLeftValid = GetAverageConnectedPosition(Node, EGPD_Input, /*out*/ AverageLeftPin);
		bool bRightValid = GetAverageConnectedPosition(Node, EGPD_Output, /*out*/ AverageRightPin);

		if (bLeftValid && bRightValid)
		{
			bPinReversed = AverageRightPin.X < AverageLeftPin.X;
		}
		else if (bCenterValid)
		{
			if (bLeftValid)
			{
				bPinReversed = CenterPin.X < AverageLeftPin.X;
			}
			else if (bRightValid)
			{
				bPinReversed = AverageRightPin.X < CenterPin.X;
			}
		}

		RerouteToReversedDirectionMap.Add(Node, bPinReversed);

		return bPinReversed;
	}
}

bool FNiagaraConnectionDrawingPolicy::GetAverageConnectedPosition(UNiagaraNodeReroute* Reroute, EEdGraphPinDirection Direction, FVector2D& OutPos) const
{
	FVector2D Result = FVector2D::ZeroVector;
	int32 ResultCount = 0;

	UEdGraphPin* Pin = (Direction == EGPD_Input) ? Reroute->GetInputPin(0) : Reroute->GetOutputPin(0);
	for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
	{
		FVector2D CenterPoint;
		if (FindPinCenter(LinkedPin, /*out*/ CenterPoint))
		{
			Result += CenterPoint;
			ResultCount++;
		}
	}

	if (ResultCount > 0)
	{
		OutPos = Result * (1.0f / ResultCount);
		return true;
	}
	else
	{
		return false;
	}
}

bool FNiagaraConnectionDrawingPolicy::FindPinCenter(UEdGraphPin* Pin, FVector2D& OutCenter) const
{
	if (const TSharedPtr<SGraphPin>* pPinWidget = PinToPinWidgetMap.Find(Pin))
	{
		if (FArrangedWidget* pPinEntry = PinGeometries->Find((*pPinWidget).ToSharedRef()))
		{
			OutCenter = FGeometryHelper::CenterOf(pPinEntry->Geometry);
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

