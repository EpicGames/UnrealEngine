// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphSchema_NiagaraSystemOverview.h"
#include "EdGraphNode_Comment.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphSchema_NiagaraSystemOverview)

#define LOCTEXT_NAMESPACE "NiagaraSchema"

int32 UEdGraphSchema_NiagaraSystemOverview::CurrentCacheRefreshID = 0;

UEdGraphSchema_NiagaraSystemOverview::UEdGraphSchema_NiagaraSystemOverview(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UEdGraphSchema_NiagaraSystemOverview::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	//@TODO System Overview: enable/disable emitter, solo, etc. 
}

bool UEdGraphSchema_NiagaraSystemOverview::IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const
{
	return InVisualizationCacheID != CurrentCacheRefreshID;
}

int32 UEdGraphSchema_NiagaraSystemOverview::GetCurrentVisualizationCacheID() const
{
	return CurrentCacheRefreshID;
}

void UEdGraphSchema_NiagaraSystemOverview::ForceVisualizationCacheClear() const
{
	CurrentCacheRefreshID++;
}

void UEdGraphSchema_NiagaraSystemOverview::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);
	TArray<UNiagaraEmitter*> Emitters;

	for (const FAssetData& Data : Assets)
	{
		UObject* Asset = Data.GetAsset();
		if (UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Asset))
		{
			if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(Asset))
			{
				Emitters.Add(Emitter);
			}
		}
	}

	UNiagaraSystem* System = Graph->GetTypedOuter<UNiagaraSystem>();
	if (Emitters.Num() > 0 && System)
	{
		UNiagaraSystemEditorData* SystemEditorData = CastChecked<UNiagaraSystemEditorData>(System->GetEditorData(), ECastCheckedType::NullChecked);
		// only insert emitters if we're in a system graph and not another emitter
		if (SystemEditorData->GetOwningSystemIsPlaceholder() == false)
		{
			FScopedTransaction AddEmitterTransaction(LOCTEXT("NiagaraEditorDropEmitter", "Niagara: Drag and Drop Emitter"));

			FVector2f NewNodePosition = GraphPosition;
			for (UNiagaraEmitter* Emitter : Emitters)
			{
				FGuid Handle = FNiagaraEditorUtilities::AddEmitterToSystem(*System, *Emitter, Emitter->GetExposedVersion().VersionGuid);

				// place the node at the mouse position from the drag operation
				TArray<UNiagaraOverviewNode*> OverviewNodes;
				SystemEditorData->GetSystemOverviewGraph()->GetNodesOfClass<UNiagaraOverviewNode>(OverviewNodes);
				UNiagaraOverviewNode* OverviewNode = OverviewNodes.Last();
				if (OverviewNode->GetEmitterHandleGuid() == Handle)
				{
					OverviewNode->NodePosX = NewNodePosition.X;
					OverviewNode->NodePosY = NewNodePosition.Y;
					NewNodePosition.X += 250;
				}
			}
			System->OnSystemPostEditChange().Broadcast(System);
		}
	}
}

void UEdGraphSchema_NiagaraSystemOverview::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	UNiagaraSystem* System = HoverGraph->GetTypedOuter<UNiagaraSystem>();
	if (!System)
	{
		return;
	}
	UNiagaraSystemEditorData* SystemEditorData = CastChecked<UNiagaraSystemEditorData>(System->GetEditorData(), ECastCheckedType::NullChecked);
	if (SystemEditorData->GetOwningSystemIsPlaceholder())
	{
		for (const FAssetData& AssetData : Assets)
		{
			if (UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(AssetData.GetAsset()))
			{
				OutTooltipText = LOCTEXT("NiagaraEditorDropEmitterInEmitterTooltip", "Emitters can only be added to systems").ToString();
				break;
			}
		}
		return;
	}
	
	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);
	for (const FAssetData& AssetData : Assets)
	{
		if (UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(AssetData.GetAsset()))
		{
			if (AssetReferenceFilter)
			{
				FText FailureReason;
				if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
				{
					if (OutTooltipText.IsEmpty())
					{
						OutTooltipText = FailureReason.ToString();
					}
					continue;
				}
			}

			OutTooltipText = LOCTEXT("NiagaraEditorDropEmitterTooltip", "Add emitter to system").ToString();
			OutOkIcon = true;
			break;
		}
	}
}

TSharedPtr<IAssetReferenceFilter> UEdGraphSchema_NiagaraSystemOverview::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (Graph)
	{
		if (const UNiagaraSystem* NiagaraSystem = Graph->GetTypedOuter<UNiagaraSystem>())
		{
			if (GEditor)
			{
				FAssetReferenceFilterContext AssetReferenceFilterContext;
				AssetReferenceFilterContext.AddReferencingAsset(NiagaraSystem);
				return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
			}
		}
	}

	return {};
}

#undef LOCTEXT_NAMESPACE

