// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalDistanceFieldCapture.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Engine/VolumeTexture.h"
#include "GlobalDistanceFieldReadback.h"
#include "HAL/IConsoleManager.h"
#include "IContentBrowserSingleton.h"
#include "LevelEditorViewport.h"
#include "NiagaraEditorModule.h"
#include "Stats/Stats.h"



FGlobalDistanceFieldCapture* FGlobalDistanceFieldCapture::Singleton = nullptr;


void FGlobalDistanceFieldCapture::Request(UVolumeTexture* VolumeTexture, bool bRangeCompress, FOnDistanceFieldCaptureComplete& InCompletionDelegate)
{
	RequestCommon(VolumeTexture, bRangeCompress, false, FVector(), InCompletionDelegate);
}

void FGlobalDistanceFieldCapture::Request(UVolumeTexture* VolumeTexture, bool bRangeCompress, const FVector& CamPos, FOnDistanceFieldCaptureComplete& InCompletionDelegate)
{
	RequestCommon(VolumeTexture, bRangeCompress, true, CamPos, InCompletionDelegate);
}

void FGlobalDistanceFieldCapture::RequestCommon(UVolumeTexture* VolumeTexture, bool bRangeCompress, bool bSetCamPos, const FVector& CamPos, FOnDistanceFieldCaptureComplete& InCompletionDelegate)
{
	if (Singleton != nullptr)
	{
		UE_LOGF(LogConsoleResponse, Error, "Cannot capture the global distance field because a capture is already in progress.");
		return;
	}

	Singleton = new FGlobalDistanceFieldCapture(VolumeTexture, bRangeCompress, bSetCamPos, CamPos, InCompletionDelegate);
}

FGlobalDistanceFieldCapture::FGlobalDistanceFieldCapture(UVolumeTexture* Tex, bool bCompress, bool bSetCamPos, const FVector& CamPos, FOnDistanceFieldCaptureComplete& InCompletionDelegate)
{
	check(Tex != nullptr);
	
	VolumeTex = Tex;
	bRangeCompress = bCompress;
	CompletionDelegate = InCompletionDelegate;

	if (bSetCamPos && GCurrentLevelEditingViewportClient != nullptr)
	{
		StoredCamPos = GCurrentLevelEditingViewportClient->GetViewLocation();

		// Small kludge: Add a tiny bit to the camera position to ensure the grid gets snapped to the same position. This is because they're likely using
		// the Console Output to position the camera, and the grid is snapped to cell-sized coords, and placing the camera exactly at that position
		// was causing it to snap one cell back in each dimension.
		FVector ShiftedCamPos = CamPos + FVector(0.001f);

		GCurrentLevelEditingViewportClient->SetViewLocation(ShiftedCamPos);
		bRestoreCamera = true;
	}

	Readback = new FGlobalDistanceFieldReadback();
	Readback->ReadbackComplete = FGlobalDistanceFieldReadback::FCompleteDelegate::CreateRaw(this, &FGlobalDistanceFieldCapture::OnReadbackComplete);
	Readback->CallbackThread = ENamedThreads::GameThread;
	RequestGlobalDistanceFieldReadback_GameThread(Readback);
}

FGlobalDistanceFieldCapture::~FGlobalDistanceFieldCapture()
{
	ensure(Singleton == this);

	delete Readback;
	Singleton = nullptr;
}

void FGlobalDistanceFieldCapture::OnReadbackComplete()
{
	if (!ensureMsgf(Readback, TEXT("FGlobalDistanceFieldCapture::OnReadbackComplete called without a pending request!")))
	{
		return;
	}

	float MaxDist = -MAX_FLT;
	float MinDist = MAX_FLT;
	for (FFloat16Color Color : Readback->ReadbackData)
	{
		float Dist = Color.R;
		MaxDist = FMath::Max(MaxDist, Dist);
		MinDist = FMath::Min(MinDist, Dist);
	}

	if (VolumeTex.IsValid())
	{
		FIntVector Size = Readback->Size;
		ETextureSourceFormat Format = TSF_RGBA16F;
		const uint8* PixelData = (const uint8*)Readback->ReadbackData.GetData();
		TArray<uint16> CompressedData;
		if (bRangeCompress)
		{
			CompressedData.Empty(Size.X * Size.Y * Size.Z * sizeof(uint16));
			for (FFloat16Color Color : Readback->ReadbackData)
			{
				CompressedData.Add(uint16(65535.0f * FMath::GetRangePct(MinDist, MaxDist, Color.R.GetFloat())));
			}
			PixelData = (const uint8*)CompressedData.GetData();
			Format = TSF_G16;
		}

		// Overwrite the volume texture with the new data
		VolumeTex->Source.Init(Size.X, Size.Y, Size.Z, 1, Format, PixelData);
		VolumeTex->Source2DTexture = nullptr; // we're no longer sourced from a 2D texture
		VolumeTex->SetLightingGuid();
		VolumeTex->UpdateResource();
	}

	FVector Center, Extents;
	Readback->Bounds.GetCenterAndExtents(Center, Extents);

	UE_LOGF(LogConsoleResponse, Display, "GLOBAL DISTANCE FIELD CAPTURE: Texture: %ls, Position: (%g, %g, %g), Extents: (%g, %g, %g), Distance Range: (%g, %g)",
		*VolumeTex->GetName(), Center.X, Center.Y, Center.Z, Extents.X, Extents.Y, Extents.Z, MinDist, MaxDist);	

	if (CompletionDelegate.IsBound())
	{
		CompletionDelegate.Execute(Extents, MinDist, MaxDist);
	}

	if (bRestoreCamera && GCurrentLevelEditingViewportClient != nullptr)
	{
		GCurrentLevelEditingViewportClient->SetViewLocation(StoredCamPos);
	}

	delete Readback;
	Readback = nullptr;
	VolumeTex = nullptr;

	// Work is done, we can kill the singleton instance now
	delete this;
}
