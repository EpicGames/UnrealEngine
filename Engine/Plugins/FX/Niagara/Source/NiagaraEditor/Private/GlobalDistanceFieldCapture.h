// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

class FGlobalDistanceFieldReadback;
class UVolumeTexture;


/**
* Delegate called when distance field capture is complete
*/
DECLARE_DELEGATE_ThreeParams(FOnDistanceFieldCaptureComplete, FVector /*Extents*/, float /*MinRange*/, float /*MaxRange*/);


/**
 * Utility interface used by an editor console command to capture the global distance field and store it into a volume
 * texture selected in the Content Browser
 */
class FGlobalDistanceFieldCapture
{
public:
	NIAGARAEDITOR_API virtual ~FGlobalDistanceFieldCapture();
	
	/**	Requests a capture. Will overwrite the volume texture selected in the Content Browser or create a new one. */
	static NIAGARAEDITOR_API void Request(UVolumeTexture* VolumeTexture, bool bRangeCompress, FOnDistanceFieldCaptureComplete& InCompletionDelegate);

	/**	Requests a capture at the specified camera location. Will overwrite the volume texture selected in the Content Browser or create a new one. */
	static NIAGARAEDITOR_API void Request(UVolumeTexture* VolumeTexture, bool bRangeCompress, const FVector& CameraPos, FOnDistanceFieldCaptureComplete& InCompletionDelegate);

private:
	TWeakObjectPtr<UVolumeTexture> VolumeTex;
	FGlobalDistanceFieldReadback* Readback = nullptr;
	FVector StoredCamPos;
	bool bRestoreCamera = false;
	bool bRangeCompress = false;
	FOnDistanceFieldCaptureComplete CompletionDelegate;

	// Only allow singleton access of constructors	
	FGlobalDistanceFieldCapture(UVolumeTexture* Tex, bool bCompress, bool bSetCamPos, const FVector& CamPos, FOnDistanceFieldCaptureComplete& CompletionDelegate);
	FGlobalDistanceFieldCapture(const FGlobalDistanceFieldCapture&) = delete;

	void OnReadbackComplete();

	static void RequestCommon(UVolumeTexture* VolumeTexture, bool bRangeCompress, bool bSetCamPos, const FVector& CamPos, FOnDistanceFieldCaptureComplete& CompletionDelegate);

	static FGlobalDistanceFieldCapture* Singleton;
};
