// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraActions.h"
#include "DataInterface/NiagaraDataInterfaceDataTable.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraScriptVariable.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#include "ScopedTransaction.h"
#include "Settings/EditorStyleSettings.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "Misc/MessageDialog.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/SNiagaraParameterMenu.h"
#include "Widgets/SToolTip.h"

#include "NiagaraDataChannel.h"
#include "Config/NiagaraFavoriteActionsConfig.h"
#include "Engine/DataTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraActions)

#define LOCTEXT_NAMESPACE "NiagaraActions"

/************************************************************************/
/* FNiagaraMenuAction													*/
/************************************************************************/
FNiagaraMenuAction::FNiagaraMenuAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords, FOnExecuteStackAction InAction, int32 InSectionID)
	: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords), InSectionID)
	, Action(InAction)
{}

FNiagaraMenuAction::FNiagaraMenuAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords, FOnExecuteStackAction InAction, FCanExecuteStackAction InCanPerformAction, int32 InSectionID)
	: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords), InSectionID)
	, Action(InAction)
	, CanPerformAction(InCanPerformAction)
{}

TOptional<FNiagaraVariable> FNiagaraMenuAction::GetParameterVariable() const
{
	return ParameterVariable;
}

void FNiagaraMenuAction::SetParameterVariable(const FNiagaraVariable& InParameterVariable)
{
	ParameterVariable = InParameterVariable;
}

void FNiagaraMenuActionCollector::AddAction(TSharedPtr<FNiagaraMenuAction> Action, int32 SortOrder)
{
	Actions.Add({Action, SortOrder});
}

void FNiagaraMenuActionCollector::AddAllActionsTo(FGraphActionListBuilderBase& ActionBuilder)
{
	Actions.Sort([](const FCollectedAction& Lhs, const FCollectedAction& Rhs)
	{
		// First check configured sort order
		if (Lhs.SortOrder != Rhs.SortOrder)
		{
			return Lhs.SortOrder < Rhs.SortOrder;
		}

		// Then check the defined category (and subcategory)
		const FText CategoryA = Lhs.Action->GetCategory();
		const FText CategoryB = Rhs.Action->GetCategory();
		int32 CategoryCompare = CategoryA.CompareTo(CategoryB);
		if (CategoryCompare != 0)
		{
			return CategoryCompare < 0;
		}

		// Then compare the actual variable names
		FNiagaraParameterHandle HandleA(FName(Lhs.Action->GetMenuDescription().ToString()));
		FNiagaraParameterHandle HandleB(FName(Rhs.Action->GetMenuDescription().ToString()));

		const TArray<FName> NamesA = HandleA.GetHandleParts();
		const TArray<FName> NamesB = HandleB.GetHandleParts();
		if (NamesA.Num() == NamesB.Num())
		{
			for (int i = 0; i < NamesA.Num(); i++)
			{
				if (NamesA[i] != NamesB[i])
				{
					return NamesA[i].LexicalLess(NamesB[i]);
				}
			}
		}
		return NamesA.Num() < NamesB.Num();
	});
	
	for (const FCollectedAction& Entry : Actions)
	{
		ActionBuilder.AddAction(Entry.Action);
	}
}

FNiagaraMenuAction_Base::FNiagaraMenuAction_Base(FText InDisplayName, TArray<FString> InNodeCategories, TOptional<FNiagaraFavoritesActionData> InFavoritesActionData, FText InToolTip, FText InKeywords, float InIntrinsicWeightMultiplier)
{
	DisplayName = InDisplayName;
	Categories = InNodeCategories;
	FavoritesActionData = InFavoritesActionData;
	ToolTip = InToolTip;
	Keywords = InKeywords;
	SearchWeightMultiplier = InIntrinsicWeightMultiplier;

	if(FavoritesActionData.IsSet() == false)
	{
		FNiagaraFavoritesActionData DefaultFavoritesActionData;
		DefaultFavoritesActionData.ActionIdentifier.Names.Add(FName(DisplayName.ToString()));
		DefaultFavoritesActionData.bFavoriteByDefault = false;
		FavoritesActionData = DefaultFavoritesActionData;
	}
	
	UpdateFullSearchText();
}

void FNiagaraMenuAction_Base::UpdateFullSearchText()
{
	FullSearchString.Reset();
	
	TArray<FString> KeywordsArray;
	Keywords.ToString().ParseIntoArray(KeywordsArray, TEXT(" "), true);

	TArray<FString> TooltipArray;
	ToolTip.ToString().ParseIntoArray(TooltipArray, TEXT(" "), true);

	for (FString& Entry : KeywordsArray)
	{
		Entry.ToLowerInline();
		FullSearchString += Entry;
	}

	FullSearchString.Append(LINE_TERMINATOR);

	for (FString& Entry : TooltipArray)
	{
		Entry.ToLowerInline();
		FullSearchString += Entry;
	}

	FullSearchString.Append(LINE_TERMINATOR);

	for (FString Entry : Categories)
	{
		Entry.ToLowerInline();
		FullSearchString += Entry;
	}
}

/************************************************************************/
/* FNiagaraParameterAction												*/
/************************************************************************/
FNiagaraParameterAction::FNiagaraParameterAction(const FNiagaraVariable& InParameter,
	const TArray<FNiagaraGraphParameterReferenceCollection>& InReferenceCollection,
	FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords,
	TSharedPtr<TArray<FName>> ParametersWithNamespaceModifierRenamePending, 
	int32 InSectionID)
	: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords), InSectionID)
	, ScriptVar(nullptr)
	, Parameter(InParameter)
	, ReferenceCollection(InReferenceCollection)
	, bIsExternallyReferenced(false)
	, ParametersWithNamespaceModifierRenamePendingWeak(ParametersWithNamespaceModifierRenamePending)
{
}

FNiagaraParameterAction::FNiagaraParameterAction(const FNiagaraVariable& InParameter,
	FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords,
	TSharedPtr<TArray<FName>> ParametersWithNamespaceModifierRenamePending, 
	int32 InSectionID)
	: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords), InSectionID)
	, ScriptVar(nullptr)
	, Parameter(InParameter)
	, bIsExternallyReferenced(false)
	, ParametersWithNamespaceModifierRenamePendingWeak(ParametersWithNamespaceModifierRenamePending)
{
}

FNiagaraParameterAction::FNiagaraParameterAction(const UNiagaraScriptVariable* InScriptVar,
	FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords,
	int32 InSectionID)
	: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords), InSectionID)
	, ScriptVar(InScriptVar)
	, Parameter(InScriptVar ? InScriptVar->Variable : FNiagaraVariable())
	, bIsExternallyReferenced(false)
{
}

const UNiagaraScriptVariable* FNiagaraParameterAction::GetScriptVar() const
{
	return ScriptVar;
}

const FNiagaraVariable& FNiagaraParameterAction::GetParameter() const
{
	return ScriptVar ? ScriptVar->Variable : Parameter;
}

TArray<FNiagaraGraphParameterReferenceCollection>& FNiagaraParameterAction::GetReferenceCollection()
{
	return ReferenceCollection;
}

bool FNiagaraParameterAction::GetIsNamespaceModifierRenamePending() const
{
	TSharedPtr<TArray<FName>> ParameterNamesWithNamespaceModifierRenamePending = ParametersWithNamespaceModifierRenamePendingWeak.Pin();
	if (ParameterNamesWithNamespaceModifierRenamePending.IsValid())
	{
		return ParameterNamesWithNamespaceModifierRenamePending->Contains(Parameter.GetName());
	}
	return false;
}

void FNiagaraParameterAction::SetIsNamespaceModifierRenamePending(bool bIsNamespaceModifierRenamePending)
{
	TSharedPtr<TArray<FName>> ParameterNamesWithNamespaceModifierRenamePending = ParametersWithNamespaceModifierRenamePendingWeak.Pin();
	if (ParameterNamesWithNamespaceModifierRenamePending.IsValid())
	{
		if (bIsNamespaceModifierRenamePending)
		{
			ParameterNamesWithNamespaceModifierRenamePending->AddUnique(Parameter.GetName());
		}
		else
		{
			ParameterNamesWithNamespaceModifierRenamePending->Remove(Parameter.GetName());
		}
	}
}

UEdGraphNode* FNiagaraAction_NewNode::CreateNode(UEdGraph* ParentGraph, UEdGraphPin* FromPin, FVector2D NodePosition, bool bSelectNewNode) const
{
	// see niagara schema 
	int32 NiagaraNodeDistance = 60;
	
	UEdGraphNode* ResultNode = nullptr;

	// If there is a template, we actually use it
	if (UEdGraphNode* NodeTemplate = WeakNodeTemplate.Get())
	{
		FString OutErrorMsg;
		UNiagaraNode* NiagaraNodeTemplate = Cast<UNiagaraNode>(NodeTemplate);
		if (NiagaraNodeTemplate && !NiagaraNodeTemplate->CanAddToGraph(CastChecked<UNiagaraGraph>(ParentGraph), OutErrorMsg))
		{
			if (OutErrorMsg.Len() > 0)
			{
				FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(OutErrorMsg));
			}
			return ResultNode;
		}

		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "NiagaraEditorNewNode", "Niagara Editor: New Node"));
		ParentGraph->Modify();

		NodeTemplate->SetFlags(RF_Transactional);

		// set outer to be the graph so it doesn't go away
		NodeTemplate->Rename(NULL, ParentGraph, REN_NonTransactional);
		ParentGraph->AddNode(NodeTemplate, true, bSelectNewNode);

		NodeTemplate->CreateNewGuid();
		NodeTemplate->PostPlacedNewNode();
		NodeTemplate->AllocateDefaultPins();
		NodeTemplate->AutowireNewNode(FromPin);

		// For input pins, new node will generally overlap node being dragged off
		// Work out if we want to visually push away from connected node
		int32 XLocation = NodePosition.X;
		if (FromPin && FromPin->Direction == EGPD_Input)
		{
			UEdGraphNode* PinNode = FromPin->GetOwningNode();
			const float XDelta = FMath::Abs(PinNode->NodePosX - NodePosition.X);

			if (XDelta < NiagaraNodeDistance)
			{
				// Set location to edge of current node minus the max move distance
				// to force node to push off from connect node enough to give selection handle
				XLocation = PinNode->NodePosX - NiagaraNodeDistance;
			}
		}

		NodeTemplate->NodePosX = XLocation;
		NodeTemplate->NodePosY = NodePosition.Y;
		NodeTemplate->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

		ResultNode = NodeTemplate;

		ParentGraph->NotifyGraphChanged();
	}

	return ResultNode;
}

UEdGraphNode* FNiagaraAction_NewNode::CreateNode(UEdGraph* Graph, TArray<UEdGraphPin*>& FromPins, FVector2D NodePosition,	bool bSelectNewNode) const
{
	UEdGraphNode* ResultNode = NULL;

	if (FromPins.Num() > 0)
	{
		ResultNode = CreateNode(Graph, FromPins[0], NodePosition, bSelectNewNode);

		if (ResultNode)
		{
			// Try autowiring the rest of the pins
			for (int32 Index = 1; Index < FromPins.Num(); ++Index)
			{
				ResultNode->AutowireNewNode(FromPins[Index]);
			}
		}
	}
	else
	{
		ResultNode = CreateNode(Graph, nullptr, NodePosition, bSelectNewNode);
	}

	return ResultNode;
}

TOptional<FNiagaraVariable> FNiagaraMenuAction_Generic::GetParameterVariable() const
{
	return ParameterVariable;
}

void FNiagaraMenuAction_Generic::SetParameterVariable(const FNiagaraVariable& InParameterVariable)
{
	ParameterVariable = InParameterVariable;
}

bool FNiagaraParameterAction::GetIsExternallyReferenced() const
{
	return bIsExternallyReferenced;	
}

void FNiagaraParameterAction::SetIsExternallyReferenced(bool bInIsExternallyReferenced)
{
	bIsExternallyReferenced = bInIsExternallyReferenced;
}

bool FNiagaraParameterAction::GetIsSourcedFromCustomStackContext() const
{
	return bIsSourcedFromCustomStackContext;
}

void FNiagaraParameterAction::SetIsSourcedFromCustomStackContext(bool bInIsSourcedFromCustomStackContext)
{
	bIsSourcedFromCustomStackContext = bInIsSourcedFromCustomStackContext;
}

/************************************************************************/
/* FNiagaraParameterGraphDragOperation									*/
/************************************************************************/
FNiagaraParameterGraphDragOperation::FNiagaraParameterGraphDragOperation()
	: bControlDrag(false)
	, bAltDrag(false)
{

}

TSharedRef<FNiagaraParameterGraphDragOperation> FNiagaraParameterGraphDragOperation::New(const TSharedPtr<FEdGraphSchemaAction>& InActionNode)
{
	TSharedRef<FNiagaraParameterGraphDragOperation> Operation = MakeShareable(new FNiagaraParameterGraphDragOperation);
	Operation->SourceAction = InActionNode;
	Operation->Construct();
	return Operation;
}

void FNiagaraParameterGraphDragOperation::HoverTargetChanged()
{
	if (SourceAction.IsValid())
	{
		if (!HoveredCategoryName.IsEmpty())
		{
			return;
		}
		else if (HoveredAction.IsValid())
		{
			const FSlateBrush* StatusSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
			TSharedPtr<FNiagaraParameterAction> ParameterAction = StaticCastSharedPtr<FNiagaraParameterAction>(SourceAction);
			if (ParameterAction.IsValid())
			{
				const FLinearColor TypeColor = UEdGraphSchema_Niagara::GetTypeColor(ParameterAction->GetParameter().GetType());
				SetSimpleFeedbackMessage(StatusSymbol, TypeColor, SourceAction->GetMenuDescription());
			}
			return;
		}
	}

	FGraphSchemaActionDragDropAction::HoverTargetChanged();
}

FReply FNiagaraParameterGraphDragOperation::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	FNiagaraParameterAction* ParameterAction = (FNiagaraParameterAction*)SourceAction.Get();
	if (ParameterAction)
	{
		if (const UNiagaraScriptVariable* ScriptVar = ParameterAction->GetScriptVar())
		{
			if (UNiagaraNodeParameterMapGet* GetMapNode = Cast<UNiagaraNodeParameterMapGet>(GetHoveredNode()))
			{
				if(!GetMapNode->DoesParameterExistOnNode(ScriptVar->Variable))
				{
					FScopedTransaction AddNewPinTransaction(LOCTEXT("Drop Onto Get Pin", "Drop parameter onto Get node"));
					FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(GetMapNode, false, ScriptVar);
				}
			}
			else if (UNiagaraNodeParameterMapSet* SetMapNode = Cast<UNiagaraNodeParameterMapSet>(GetHoveredNode()))
			{
				if(!SetMapNode->DoesParameterExistOnNode(ScriptVar->Variable))
				{
					FScopedTransaction AddNewPinTransaction(LOCTEXT("Drop Onto Set Pin", "Drop parameter onto Set node"));
					FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(SetMapNode, true, ScriptVar);
				}
			}
		}

		// Legacy codepath for drag actions that do not carry the UNiagaraScriptVariable.
		else
		{
			const FNiagaraVariable& Parameter = ParameterAction->GetParameter();
			if (UNiagaraNodeParameterMapGet* GetMapNode = Cast<UNiagaraNodeParameterMapGet>(GetHoveredNode()))
			{
				FScopedTransaction AddNewPinTransaction(LOCTEXT("Drop Onto Get Pin", "Drop parameter onto Get node"));
				FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(GetMapNode, false, Parameter);
			}
			else if (UNiagaraNodeParameterMapSet* SetMapNode = Cast<UNiagaraNodeParameterMapSet>(GetHoveredNode()))
			{
				FScopedTransaction AddNewPinTransaction(LOCTEXT("Drop Onto Set Pin", "Drop parameter onto Set node"));
				FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(SetMapNode, true, Parameter);
			}
		}
	}

	return FReply::Handled();
}

FReply FNiagaraParameterGraphDragOperation::DroppedOnPanel(const TSharedRef<SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	if (Graph.GetSchema()->IsA<UEdGraphSchema_Niagara>())
	{
		FNiagaraParameterAction* ParameterAction = (FNiagaraParameterAction*)SourceAction.Get();
		if (ParameterAction)
		{
			UNiagaraGraph* NiagaraGraph = Cast<UNiagaraGraph>(&Graph);

			auto GetScriptVar = [&ParameterAction, &NiagaraGraph]()->const UNiagaraScriptVariable* {
				const UNiagaraScriptVariable* ActionScriptVar = ParameterAction->GetScriptVar();
				if (ActionScriptVar != nullptr)
				{
					return ActionScriptVar;
				}
				return NiagaraGraph->GetScriptVariable(ParameterAction->GetParameter());
			};

			const UNiagaraScriptVariable* ScriptVariable = GetScriptVar();
			// if the ScriptVariable is a nullptr, it is likely that the action was dropped on a panel different than the original
			if(ScriptVariable == nullptr)
			{
				return FReply::Handled();
			}

			FNiagaraParameterNodeConstructionParams NewNodeParams(
				GraphPosition,
				&Graph,
				ParameterAction->GetParameter(),
				ScriptVariable
			);

			// Take into account the current state of modifier keys in case the user changed their mind
			FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
			const bool bModifiedKeysActive = ModifierKeys.IsControlDown() || ModifierKeys.IsAltDown();
			const bool bAutoCreateGetter = bModifiedKeysActive ? ModifierKeys.IsControlDown() : bControlDrag;
			const bool bAutoCreateSetter = bModifiedKeysActive ? ModifierKeys.IsAltDown() : bAltDrag;
			
			if(ScriptVariable != nullptr && ScriptVariable->GetIsStaticSwitch())
			{
				MakeStaticSwitch(NewNodeParams, ScriptVariable);
				return FReply::Handled();
			}
			
			// Handle Getter/Setters
			if (bAutoCreateGetter || bAutoCreateSetter)
			{
				if (bAutoCreateGetter)
				{
					MakeGetMap(NewNodeParams);
				}
				if (bAutoCreateSetter)
				{
					MakeSetMap(NewNodeParams);
				}
			}
			// Show selection menu
			else
			{
				FMenuBuilder MenuBuilder(true, NULL);
				const FText ParameterNameText = FText::FromName(NewNodeParams.Parameter.GetName());

				MenuBuilder.BeginSection("NiagaraParameterDroppedOnPanel", ParameterNameText);
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("CreateGetMap", "Get Map including {0}"), ParameterNameText),
					FText::Format(LOCTEXT("CreateGetMapToolTip", "Create Getter for variable '{0}'\n(Ctrl-drag to automatically create a getter)"), ParameterNameText),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FNiagaraParameterGraphDragOperation::MakeGetMap, NewNodeParams), 
						FCanExecuteAction())
				);

				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("CreateSetMap", "Set Map including {0}"), ParameterNameText),
					FText::Format(LOCTEXT("CreateSetMapToolTip", "Create Set Map for parameter '{0}'\n(Alt-drag to automatically create a setter)"), ParameterNameText),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FNiagaraParameterGraphDragOperation::MakeSetMap, NewNodeParams),
						FCanExecuteAction())
				);

				TSharedRef< SWidget > PanelWidget = Panel;
				// Show dialog to choose getter vs setter
				FSlateApplication::Get().PushMenu(
					PanelWidget,
					FWidgetPath(),
					MenuBuilder.MakeWidget(),
					ScreenPosition,
					FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
				);

				MenuBuilder.EndSection();
			}
		}
	}

	return FReply::Handled();
}


bool FNiagaraParameterGraphDragOperation::IsCurrentlyHoveringNode(const UEdGraphNode* TestNode) const
{
	return TestNode == GetHoveredNode();
}

void FNiagaraParameterGraphDragOperation::MakeGetMap(FNiagaraParameterNodeConstructionParams InParams)
{

	FScopedTransaction AddNewPinTransaction(LOCTEXT("MakeGetMap", "Make Get Node For Variable"));
	check(InParams.Graph);
	InParams.Graph->Modify();
	FGraphNodeCreator<UNiagaraNodeParameterMapGet> GetNodeCreator(*InParams.Graph);
	UNiagaraNodeParameterMapGet* GetNode = GetNodeCreator.CreateNode();
	GetNode->NodePosX = InParams.GraphPosition.X;
	GetNode->NodePosY = InParams.GraphPosition.Y;
	GetNodeCreator.Finalize();

	if (const UNiagaraScriptVariable* ScriptVar = InParams.ScriptVar)
	{
		FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(GetNode, false, ScriptVar);
	}
	else
	{
		FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(GetNode, false, InParams.Parameter);
	}
}

void FNiagaraParameterGraphDragOperation::MakeSetMap(FNiagaraParameterNodeConstructionParams InParams)
{
	FScopedTransaction AddNewPinTransaction(LOCTEXT("MakeSetMap", "Make Set Node For Variable"));
	check(InParams.Graph);
	InParams.Graph->Modify();
	FGraphNodeCreator<UNiagaraNodeParameterMapSet> SetNodeCreator(*InParams.Graph);
	UNiagaraNodeParameterMapSet* SetNode = SetNodeCreator.CreateNode();
	SetNode->NodePosX = InParams.GraphPosition.X;
	SetNode->NodePosY = InParams.GraphPosition.Y;
	SetNodeCreator.Finalize();

	if (const UNiagaraScriptVariable* ScriptVar = InParams.ScriptVar)
	{
		FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(SetNode, true, ScriptVar);
	}
	else
	{
		FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(SetNode, true, InParams.Parameter);
	}
}

void FNiagaraParameterGraphDragOperation::MakeStaticSwitch(FNiagaraParameterNodeConstructionParams InParams, const UNiagaraScriptVariable* ScriptVariable)
{
	FScopedTransaction AddNewPinTransaction(LOCTEXT("MakeStaticSwitch", "Make Static Switch"));
	check(InParams.Graph);
	InParams.Graph->Modify();

	// copy metadata
	if (UNiagaraGraph* NiagaraGraph = Cast<UNiagaraGraph>(InParams.Graph))
	{
		if(NiagaraGraph->GetScriptVariable(ScriptVariable->Variable) == nullptr)
		{
			NiagaraGraph->AddParameter(ScriptVariable);
		}
	}
	
	FGraphNodeCreator<UNiagaraNodeStaticSwitch> SetNodeCreator(*InParams.Graph);
	UNiagaraNodeStaticSwitch* SwitchNode = SetNodeCreator.CreateNode();
	SwitchNode->NodePosX = InParams.GraphPosition.X;
	SwitchNode->NodePosY = InParams.GraphPosition.Y;
	SwitchNode->InputParameterName = InParams.Parameter.GetName();
	const FNiagaraTypeDefinition& Type = InParams.Parameter.GetType();
	
	if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		SwitchNode->SwitchTypeData.SwitchType = ENiagaraStaticSwitchType::Bool;
	}
	else if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()))
	{
		SwitchNode->SwitchTypeData.SwitchType = ENiagaraStaticSwitchType::Integer;
	}
	else if (Type.IsEnum())
	{
		SwitchNode->SwitchTypeData.SwitchType = ENiagaraStaticSwitchType::Enum;
		SwitchNode->SwitchTypeData.Enum = Type.GetEnum();
	}
	
	SetNodeCreator.Finalize();
}

EVisibility FNiagaraParameterGraphDragOperation::GetIconVisible() const
{
	return EVisibility::Collapsed;
}

EVisibility FNiagaraParameterGraphDragOperation::GetErrorIconVisible() const
{
	return EVisibility::Collapsed;
}

TSharedPtr<SWidget> FNiagaraParameterDragOperation::GetDefaultDecorator() const
{
	TSharedPtr<SWidget> Decorator = SNew(SToolTip)
	.Content()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			FNiagaraParameterUtilities::GetParameterWidget(StaticCastSharedPtr<FNiagaraParameterAction>(SourceAction)->GetParameter(), true, false)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.MaxDesiredWidth(250.f)
			.Padding(5.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::White)
				.Text(this, &FNiagaraParameterDragOperation::GetHoverText)
				.Visibility(this, &FNiagaraParameterDragOperation::IsTextVisible)
				.AutoWrapText(true)
			]
		]
	];
	
	return Decorator;
}

EVisibility FNiagaraParameterDragOperation::IsTextVisible() const
{
	return CurrentHoverText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

//////////////////////////////////////////////////////////////////////////

TMap<FName, TUniquePtr<INiagaraDataInterfaceNodeActionProvider>> INiagaraDataInterfaceNodeActionProvider::RegisteredActionProviders;

void INiagaraDataInterfaceNodeActionProvider::GetNodeContextMenuActions(UClass* DIClass, UToolMenu* Menu, UGraphNodeContextMenuContext* Context, const FNiagaraFunctionSignature& Signature)
{
	if(ensure(DIClass))
	{
		while(DIClass && DIClass != UObject::StaticClass())
		{
			if (TUniquePtr<INiagaraDataInterfaceNodeActionProvider>* Provider = RegisteredActionProviders.Find(DIClass->GetFName()))
			{
				(*Provider)->GetNodeContextMenuActionsImpl(Menu, Context, Signature);
			}
			DIClass = DIClass->GetSuperClass();
		}
	}
}

void INiagaraDataInterfaceNodeActionProvider::GetInlineNodeContextMenuActions(UClass* DIClass, UToolMenu* ToolMenu)
{
	if (ensure(DIClass))
	{
		while (DIClass && DIClass != UObject::StaticClass())
		{
			if (TUniquePtr<INiagaraDataInterfaceNodeActionProvider>* Provider = RegisteredActionProviders.Find(DIClass->GetFName()))
			{
				(*Provider)->GetInlineNodeContextMenuActionsImpl(ToolMenu);
			}
			DIClass = DIClass->GetSuperClass();
		}
	}
}

INiagaraDataInterfaceNodeActionProvider::FInlineMenuDisplayOptions INiagaraDataInterfaceNodeActionProvider::GetInlineMenuDisplayOptions(UClass* DIClass, UEdGraphNode* Source)
{
	if(ensure(DIClass))
	{
		if (TUniquePtr<INiagaraDataInterfaceNodeActionProvider>* Provider = RegisteredActionProviders.Find(DIClass->GetFName()))
		{
			return (*Provider)->GetInlineMenuDisplayOptionsImpl(DIClass, Source);
		}
	}

	return {};
}

void INiagaraDataInterfaceNodeActionProvider::CollectAddPinActions(UClass* DIClass, FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin)
{
	if (ensure(DIClass))
	{
		while (DIClass && DIClass != UObject::StaticClass())
		{
			if (TUniquePtr<INiagaraDataInterfaceNodeActionProvider>* Provider = RegisteredActionProviders.Find(DIClass->GetFName()))
			{
				(*Provider)->CollectAddPinActionsImpl(Collector, AddPin);
			}
			DIClass = DIClass->GetSuperClass();
		}
	}
}

TSharedPtr<SWidget> INiagaraDataInterfaceNodeActionProvider::GetCustomFunctionSpecifierWidget(UClass* DIClass, UNiagaraNodeFunctionCall* FunctionCallNode)
{
	if (DIClass && FunctionCallNode)
	{
		while (DIClass && DIClass != UObject::StaticClass())
		{
			if (TUniquePtr<INiagaraDataInterfaceNodeActionProvider>* Provider = RegisteredActionProviders.Find(DIClass->GetFName()))
			{
				return (*Provider)->GetCustomFunctionSpecifierWidgetImpl(FunctionCallNode);
			}
			DIClass = DIClass->GetSuperClass();
		}
	}
	return nullptr;
}

////////////////////////

namespace NiagaraActionsLocal
{
	FText InitForDataChannelHeaderText = LOCTEXT("DataChannelsHeader", "Data Channels");
	FText InitForDataChannelSectionText = LOCTEXT("DataChannelsSection","Data Channels");
	FText InitForDataChannelMenuText = LOCTEXT("InitForDataChannelMenu", "Init For Data Channel...");
	FText InitForDataChannelMenuTooltipText = LOCTEXT("InitForDataChannelTooltip", "Initializes this node to write to all members of a given data channel.");
}

void FNiagaraDataInterfaceNodeActionProvider_DataChannelWrite::GetNodeContextMenuActionsImpl(UToolMenu* Menu, UGraphNodeContextMenuContext* Context, const FNiagaraFunctionSignature& Signature) const
{
	using namespace NiagaraActionsLocal;

	//For all functions except "Num", add a context menu to initialized to a specific data channel.
	if (Signature.Name == TEXT("Num") || Signature.Name == TEXT("SpawnConditional"))
	{
		return;
	}

	FToolMenuSection& Section = Menu->AddSection("DataChannelWrite", InitForDataChannelHeaderText);

	const UNiagaraNodeFunctionCall* FuncNode = CastChecked<UNiagaraNodeFunctionCall>(Context->Node);
	TWeakObjectPtr<const UNiagaraNodeFunctionCall> WeakNode = FuncNode;

	auto CreateNodeContextMenu = [WeakNode](UToolMenu* InNewToolMenu)
	{
		FToolMenuSection& SubSection = InNewToolMenu->AddSection("InitForDataChannelSection", InitForDataChannelSectionText);

		auto InitForDataChannelSection = [&SubSection, WeakNode](UNiagaraDataChannel* DataChannel)
		{
			check(DataChannel);

			TWeakObjectPtr<UNiagaraDataChannel> WeakChannel = DataChannel;
			auto CreateDataChannelActionEntry = [WeakChannel, WeakNode]()
			{
				UNiagaraDataChannel* Channel = WeakChannel.Get();
				UNiagaraNodeFunctionCall* Node = const_cast<UNiagaraNodeFunctionCall*>(WeakNode.Get());
				if (Channel && Node)
				{
					Node->RemoveAllDynamicPins();
					TConstArrayView<FNiagaraDataChannelVariable> ChannelVars = Channel->GetVariables();
					for (const FNiagaraDataChannelVariable& Var : ChannelVars)
					{
						FNiagaraTypeDefinition Type = Var.GetType();
						if (Type.IsEnum() == false)
						{
							Type = FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(Var.GetType().GetScriptStruct()));
						}
						FNiagaraVariable SWCVar(Type, Var.GetName());
						Node->AddParameter(SWCVar, EEdGraphPinDirection::EGPD_Input);
					}
				}
			};
			SubSection.AddMenuEntry(
				NAME_None,
				FText::FromString(DataChannel->GetAsset()->GetName()),
				FText::FromString(DataChannel->GetAsset()->GetName()),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Import"),
				FUIAction(FExecuteAction::CreateLambda(CreateDataChannelActionEntry), FCanExecuteAction()));
		};

		UNiagaraDataChannel::ForEachDataChannel(InitForDataChannelSection);
	};

	Section.AddSubMenu("InitForDataChannelMenu", InitForDataChannelMenuText, InitForDataChannelMenuTooltipText,	FNewToolMenuDelegate::CreateLambda(CreateNodeContextMenu));
}

void FNiagaraDataInterfaceNodeActionProvider_DataChannelWrite::CollectAddPinActionsImpl(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin)const
{
	auto GatherAddPinsForChannel = [&](UNiagaraDataChannel* Channel)
	{
		TConstArrayView<FNiagaraDataChannelVariable> ChannelVars = Channel->GetVariables();
		for (const FNiagaraDataChannelVariable& Var : ChannelVars)
		{
			FNiagaraTypeDefinition Type = Var.GetType();
			if (Type.IsEnum() == false)
			{
				Type = FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(Var.GetType().GetScriptStruct()));
			}
			FNiagaraVariable SWCVar(Type, Var.GetName());
			FNiagaraEditorUtilities::ResetVariableToDefaultValue(SWCVar);

			const UEdGraphPin* ConstAddPin = AddPin;

			// The script variable is not a duplicate, add an entry for it.
			FText Category;
			FText DisplayName;
			FText Tooltip;
			{
				Category = FText::Format(LOCTEXT("NDIWriteAddPinCatFmt", "Write to NDC {0}"), FText::FromString(Channel->GetAsset()->GetName()));
				DisplayName = FText::FromName(SWCVar.GetName());
				Tooltip = FText::Format(
					LOCTEXT("NDIWritelAddPinTooltipFmt", "Write to the variable {0} from NDC {1}."),
					FText::FromName(SWCVar.GetName()),
					FText::FromString(Channel->GetAsset()->GetName()));
			}

			TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
				Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
				FNiagaraMenuAction::FOnExecuteStackAction::CreateUObject(CastChecked<UNiagaraNodeWithDynamicPins>(AddPin->GetOwningNode()), &UNiagaraNodeWithDynamicPins::AddParameter, SWCVar, ConstAddPin->Direction)));
			Action->SetParameterVariable(SWCVar);
			Collector.AddAction(Action, 3);
		}
	};
	UNiagaraDataChannel::ForEachDataChannel(GatherAddPinsForChannel);
}

void FNiagaraDataInterfaceNodeActionProvider_DataChannelWrite::GetInlineNodeContextMenuActionsImpl(UToolMenu* ToolMenu) const
{
	AddDataChannelInitActions(ToolMenu);
}

INiagaraDataInterfaceNodeActionProvider::FInlineMenuDisplayOptions FNiagaraDataInterfaceNodeActionProvider_DataChannelWrite::GetInlineMenuDisplayOptionsImpl(UClass* DIClass, UEdGraphNode* Source) const
{
	UNiagaraNodeFunctionCall* FunctionCall = Cast<UNiagaraNodeFunctionCall>(Source);
	if(FunctionCall == nullptr)
	{
		return {};
	}

	if (FunctionCall->Signature.Name != TEXT("Write") && FunctionCall->Signature.Name != TEXT("Append"))
	{
		return {};
	}
	
	FInlineMenuDisplayOptions DisplayOptions;
	DisplayOptions.bDisplayInline = true;
	DisplayOptions.DisplayBrush = FAppStyle::GetBrush("Icons.Edit");
	DisplayOptions.TooltipText = LOCTEXT("InitWriteWithDataChannel", "Initialize the write node using the selected Niagara Data Channel asset.");

	return DisplayOptions;
}

void FNiagaraDataInterfaceNodeActionProvider_DataChannelWrite::AddDataChannelInitActions(UToolMenu* ToolMenu)
{
	UGraphNodeContextMenuContext* Context = ToolMenu->FindContext<UGraphNodeContextMenuContext>();

	if(Context == nullptr || Context->Node == nullptr)
	{
		return;
	}
	
	const UNiagaraNodeFunctionCall* FuncNode = CastChecked<UNiagaraNodeFunctionCall>(Context->Node);
	TWeakObjectPtr<const UNiagaraNodeFunctionCall> WeakNode = FuncNode;

	FToolMenuSection& MenuSection = ToolMenu->AddSection("DataChannelWrite", NiagaraActionsLocal::InitForDataChannelHeaderText);
	auto InitForDataChannelSection = [&MenuSection, WeakNode](UNiagaraDataChannel* DataChannel)
	{
		check(DataChannel);

		TWeakObjectPtr<UNiagaraDataChannel> WeakChannel = DataChannel;
		auto CreateDataChannelActionEntry = [WeakChannel, WeakNode]()
		{
			UNiagaraDataChannel* Channel = WeakChannel.Get();
			UNiagaraNodeFunctionCall* Node = const_cast<UNiagaraNodeFunctionCall*>(WeakNode.Get());

			if (Channel && Node)
			{
				Node->RemoveAllDynamicPins();
				TConstArrayView<FNiagaraDataChannelVariable> ChannelVars = Channel->GetVariables();
				for (const FNiagaraDataChannelVariable& Var : ChannelVars)
				{
					FNiagaraTypeDefinition Type = Var.GetType();
					if (Type.IsEnum() == false)
					{
						Type = FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(Var.GetType().GetScriptStruct()));
					}
					FNiagaraVariable SWCVar(Type, Var.GetName());
					Node->AddParameter(SWCVar, EEdGraphPinDirection::EGPD_Input);
				}
			}
		};

		MenuSection.AddMenuEntry(NAME_None, FText::FromString(DataChannel->GetAsset()->GetName()), FText::FromString(DataChannel->GetAsset()->GetName()), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Import"),
			FUIAction(FExecuteAction::CreateLambda(CreateDataChannelActionEntry), FCanExecuteAction()));
	};

	UNiagaraDataChannel::ForEachDataChannel(InitForDataChannelSection);
}


void FNiagaraDataInterfaceNodeActionProvider_DataChannelRead::GetNodeContextMenuActionsImpl(UToolMenu* Menu, UGraphNodeContextMenuContext* Context, const FNiagaraFunctionSignature& Signature) const
{
	using namespace NiagaraActionsLocal;

	//For all functions except "Num", add a context menu to initialized to a specific data channel.
	if (Signature.Name != TEXT("Read") && Signature.Name != TEXT("Consume"))
	{
		return;
	}
	
	const UNiagaraNodeFunctionCall* FuncNode = CastChecked<UNiagaraNodeFunctionCall>(Context->Node);
	TWeakObjectPtr<const UNiagaraNodeFunctionCall> WeakNode = FuncNode;

	// as the lambda gives us the tool menu, it's important to use that instead of the menu we are being passed in from the function
	auto CreateNodeContextMenu = [WeakNode](UToolMenu* InToolMenu)
	{
		AddDataChannelInitActions(InToolMenu);
	};

	FToolMenuSection& Section = Menu->AddSection("DataChannelRead", InitForDataChannelHeaderText);
	Section.AddSubMenu("InitForDataChannelMenu", InitForDataChannelMenuText, InitForDataChannelMenuTooltipText,	FNewToolMenuDelegate::CreateLambda(CreateNodeContextMenu));
}

void FNiagaraDataInterfaceNodeActionProvider_DataChannelRead::GetInlineNodeContextMenuActionsImpl(UToolMenu* ToolMenu) const
{
	AddDataChannelInitActions(ToolMenu);
}

INiagaraDataInterfaceNodeActionProvider::FInlineMenuDisplayOptions FNiagaraDataInterfaceNodeActionProvider_DataChannelRead::GetInlineMenuDisplayOptionsImpl(UClass* DIClass, UEdGraphNode* Source) const
{
	UNiagaraNodeFunctionCall* FunctionCall = Cast<UNiagaraNodeFunctionCall>(Source);
	if(FunctionCall == nullptr)
	{
		return {};
	}

	if (FunctionCall->Signature.Name != TEXT("Read") && FunctionCall->Signature.Name != TEXT("Consume"))
	{
		return {};
	}
	
	FInlineMenuDisplayOptions DisplayOptions;
	DisplayOptions.bDisplayInline = true;
	DisplayOptions.DisplayBrush = FAppStyle::GetBrush("Icons.Edit");
	DisplayOptions.TooltipText = LOCTEXT("InitReadWithDataChannel", "Initialize the read node using the selected Niagara Data Channel asset.");

	return DisplayOptions;
}

void FNiagaraDataInterfaceNodeActionProvider_DataChannelRead::CollectAddPinActionsImpl(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin)const
{
	FNiagaraFunctionSignature* Sig = nullptr;
	FString FuncName;
	if (UNiagaraNodeFunctionCall* FuncNode = Cast<UNiagaraNodeFunctionCall>(AddPin->GetOwningNode()))
	{
		Sig = &FuncNode->Signature;
		FuncName = Sig->GetNameString();
	}

	auto GatherAddPinsForChannel = [&](UNiagaraDataChannel* Channel)
	{
		TConstArrayView<FNiagaraDataChannelVariable> ChannelVars = Channel->GetVariables();
		for (const FNiagaraDataChannelVariable& Var : ChannelVars)
		{
			FNiagaraTypeDefinition Type = Var.GetType();
			if (Type.IsEnum() == false)
			{
				Type = FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(Var.GetType().GetScriptStruct()));
			}
			FNiagaraVariable SWCVar(Type, Var.GetName());
			FNiagaraEditorUtilities::ResetVariableToDefaultValue(SWCVar);

			const UEdGraphPin* ConstAddPin = AddPin;

			// The script variable is not a duplicate, add an entry for it.
			FText Category;
			FText DisplayName;
			FText Tooltip;
			if (FuncName == TEXT("SpawnConditional"))
			{
				Category = FText::Format(LOCTEXT("NDISpawnConditionalAddPinCatFmt", "Conditions on {0} variables"), FText::FromString(Channel->GetAsset()->GetName()));
				DisplayName = FText::FromName(SWCVar.GetName());
				Tooltip = FText::Format(
					LOCTEXT("NDISpawnConditionalAddPinTooltipFmt", "Make this function conditional on {0} from Data Channel {1}.\nThe function will only have an effect for Data Channel entries that pass the comparisson test with the passed value."),
					FText::FromName(SWCVar.GetName()),
					FText::FromString(Channel->GetAsset()->GetName()));
			}
			else
			{
				Category = FText::Format(LOCTEXT("NDIReadAddPinCatFmt", "{0}"), FText::FromString(Channel->GetAsset()->GetName()));
				DisplayName = FText::FromName(SWCVar.GetName());
				Tooltip = FText::Format(LOCTEXT("NDIReadAddPinTooltipFmt", "Read {0} from Data Channel {1}."), DisplayName, FText::FromString(Channel->GetAsset()->GetName()));
			}

			TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
				Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
				FNiagaraMenuAction::FOnExecuteStackAction::CreateUObject(CastChecked<UNiagaraNodeWithDynamicPins>(AddPin->GetOwningNode()), &UNiagaraNodeWithDynamicPins::AddParameter, SWCVar, ConstAddPin->Direction)));
			Action->SetParameterVariable(SWCVar);
			Collector.AddAction(Action, 3);
		}
	};
	UNiagaraDataChannel::ForEachDataChannel(GatherAddPinsForChannel);
}

void FNiagaraDataInterfaceNodeActionProvider_DataChannelRead::AddDataChannelInitActions(UToolMenu* ToolMenu)
{
	UGraphNodeContextMenuContext* Context = ToolMenu->FindContext<UGraphNodeContextMenuContext>();

	if(Context == nullptr || Context->Node == nullptr)
	{
		return;
	}
	
	const UNiagaraNodeFunctionCall* FuncNode = CastChecked<UNiagaraNodeFunctionCall>(Context->Node);
	TWeakObjectPtr<const UNiagaraNodeFunctionCall> WeakNode = FuncNode;

	FToolMenuSection& MenuSection = ToolMenu->AddSection("DataChannelRead", NiagaraActionsLocal::InitForDataChannelHeaderText);
	auto InitForDataChannelSection = [&MenuSection, WeakNode](UNiagaraDataChannel* DataChannel)
	{
		check(DataChannel);

		TWeakObjectPtr<UNiagaraDataChannel> WeakChannel = DataChannel;
		auto CreateDataChannelActionEntry = [WeakChannel, WeakNode]()
		{
			UNiagaraDataChannel* Channel = WeakChannel.Get();
			UNiagaraNodeFunctionCall* Node = const_cast<UNiagaraNodeFunctionCall*>(WeakNode.Get());

			if (Channel && Node)
			{
				Node->RemoveAllDynamicPins();
				TConstArrayView<FNiagaraDataChannelVariable> ChannelVars = Channel->GetVariables();
				for (const FNiagaraDataChannelVariable& Var : ChannelVars)
				{
					FNiagaraTypeDefinition Type = Var.GetType();
					if (Type.IsEnum() == false)
					{
						Type = FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(Var.GetType().GetScriptStruct()));
					}
					FNiagaraVariable SWCVar(Type, Var.GetName());
					Node->AddParameter(SWCVar, EEdGraphPinDirection::EGPD_Output);
				}
			}
		};

		MenuSection.AddMenuEntry(NAME_None, FText::FromString(DataChannel->GetAsset()->GetName()), FText::FromString(DataChannel->GetAsset()->GetName()), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Import"),
			FUIAction(FExecuteAction::CreateLambda(CreateDataChannelActionEntry), FCanExecuteAction()));
	};

	UNiagaraDataChannel::ForEachDataChannel(InitForDataChannelSection);
}

TSharedPtr<SWidget> FNiagaraDataInterfaceNodeActionProvider_DataChannelRead::GetCustomFunctionSpecifierWidgetImpl(UNiagaraNodeFunctionCall* FunctionCallNode) const
{
	TArray<FNiagaraTypeDefinition> AllowedTypes;
	if(FunctionCallNode->Signature.Name == TEXT("SpawnDirect"))
	{
		AllowedTypes.Add(FNiagaraTypeDefinition::GetIntDef());
	}
	else if(FunctionCallNode->Signature.Name == TEXT("ScaleSpawnCount"))
	{
		AllowedTypes.Add(FNiagaraTypeDefinition::GetIntDef());
		AllowedTypes.Add(FNiagaraTypeHelper::GetDoubleDef());
		AllowedTypes.Add(FNiagaraTypeHelper::GetVector2DDef());
		AllowedTypes.Add(FNiagaraTypeHelper::GetVectorDef());
		AllowedTypes.Add(FNiagaraTypeHelper::GetVector4Def());
	}

	return SNew(SNiagaraFunctionSpecifierNDCVariablesSelector)
		.WeakNodeToModify(FunctionCallNode)
		.AllowedTypes(MoveTemp(AllowedTypes));
}

/////////////////////////////////////////////////////////////////////////////////////////////

void FNiagaraDataInterfaceNodeActionProvider_DataTable::GetInlineNodeContextMenuActionsImpl(UToolMenu* ToolMenu) const
{
	AddInitializeActions(ToolMenu);
}

INiagaraDataInterfaceNodeActionProvider::FInlineMenuDisplayOptions FNiagaraDataInterfaceNodeActionProvider_DataTable::GetInlineMenuDisplayOptionsImpl(UClass* DIClass, UEdGraphNode* Source) const
{
	if (UNiagaraNodeFunctionCall* FunctionCall = Cast<UNiagaraNodeFunctionCall>(Source))
	{
		if (UNiagaraDataInterfaceDataTable::IsReadFunction(FunctionCall->Signature))
		{
			FInlineMenuDisplayOptions DisplayOptions;
			DisplayOptions.bDisplayInline = true;
			DisplayOptions.DisplayBrush = FAppStyle::GetBrush("Icons.Edit");
			DisplayOptions.TooltipText = LOCTEXT("InitDataTableReadNode", "Initialize the data table read using the selected data table structure.");

			return DisplayOptions;
		}
	}
	return {};
}

void FNiagaraDataInterfaceNodeActionProvider_DataTable::AddInitializeActions(UToolMenu* ToolMenu)
{
	UGraphNodeContextMenuContext* Context = ToolMenu->FindContext<UGraphNodeContextMenuContext>();
	if (Context == nullptr || Context->Node == nullptr)
	{
		return;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths.Add(FTopLevelAssetPath(UDataTable::StaticClass()));

	TArray<FAssetData> DataTableAssets;
	AssetRegistryModule.Get().GetAssets(Filter, DataTableAssets);

	if (DataTableAssets.Num() == 0)
	{
		return;
	}

	FToolMenuSection& MenuSection = ToolMenu->AddSection("DataTable", LOCTEXT("DataTable", "Data Table"));
	TWeakObjectPtr<UNiagaraNodeFunctionCall> WeakFunctionNode = CastChecked<UNiagaraNodeFunctionCall>(ConstCast(Context->Node));

	for (const FAssetData& AssetData : DataTableAssets)
	{
		FText AssetName = FText::FromName(AssetData.AssetName);
		MenuSection.AddMenuEntry(
			NAME_None,
			AssetName,
			FText::Format(LOCTEXT("SetNodeToDataTableTooltip", "Set node to read data table '{0}'"), AssetName),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Import"),
			FUIAction(FExecuteAction::CreateStatic(&AllocateNodePins, AssetData.ToSoftObjectPath(), WeakFunctionNode))
		);
	}
}

void FNiagaraDataInterfaceNodeActionProvider_DataTable::AllocateNodePins(FSoftObjectPath DataTablePath, TWeakObjectPtr<UNiagaraNodeFunctionCall> WeakFunctionNode)
{
	UDataTable* DataTable = Cast<UDataTable>(DataTablePath.TryLoad());
	UNiagaraNodeFunctionCall* FunctionNode = WeakFunctionNode.Get();
	if (!DataTable || !FunctionNode)
	{
		return;
	}

	FunctionNode->RemoveAllDynamicPins();

	for (const FNiagaraVariableBase& Variable : UNiagaraDataInterfaceDataTable::GetVariablesFromDataTable(DataTable))
	{
		FunctionNode->AddParameter(Variable, EEdGraphPinDirection::EGPD_Output);
	}
}

#undef LOCTEXT_NAMESPACE
