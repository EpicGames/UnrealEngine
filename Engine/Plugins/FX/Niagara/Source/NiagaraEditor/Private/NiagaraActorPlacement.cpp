// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraActorPlacement.h"
#include "ActorFactoryNiagara.h"

#include "IPlacementModeModule.h"

namespace FNiagaraActorPlacement
{
	static TOptional<FPlacementModeID> PlacementModeID;

	void Register()
	{
		if (GIsEditor == false || PlacementModeID.IsSet() )
		{
			return;
		}

		if (IPlacementModeModule::IsAvailable())
		{
			PlacementModeID = IPlacementModeModule::Get().RegisterPlaceableItem(
				FBuiltInPlacementCategories::Visual(),
				MakeShared<FPlaceableItem>(*UActorFactoryNiagara::StaticClass())
			);
		}
	}

	void Unregister()
	{
		if (PlacementModeID.IsSet() == false)
		{
			return;
		}
		if (IPlacementModeModule::IsAvailable())
		{
			IPlacementModeModule::Get().UnregisterPlaceableItem(PlacementModeID.GetValue());
		}
		PlacementModeID.Reset();
	}
}
