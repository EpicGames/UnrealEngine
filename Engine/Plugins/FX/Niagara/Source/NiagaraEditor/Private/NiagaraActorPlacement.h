// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace FNiagaraActorPlacement
{
	void Register();
	void Unregister();
}
