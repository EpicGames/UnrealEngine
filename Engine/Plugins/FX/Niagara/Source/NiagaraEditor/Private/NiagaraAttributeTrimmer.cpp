// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraAttributeTrimmer.h"

#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraCompilationBridge.h"
#include "NiagaraConstants.h"
#include "NiagaraGraphDigest.h"
#include "NiagaraModule.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraScriptVariable.h"

template<typename GraphBridge>
class FNiagaraAttributeTrimmerHelper<GraphBridge>::FFunctionInputResolver
{
public:
	FFunctionInputResolver(const FGraph* Graph, ENiagaraScriptUsage Usage, const FGuid& UsageId)
	{
		TMap<FName, FModuleScopedPin> UnresolvedInputs;

		Traverse(TEXT(""), Graph, Usage, UsageId, UnresolvedInputs);
	}

	FModuleScopedPin ResolveInput(const FModuleScopedPin& Input) const
	{
		if (const FModuleScopedPin* FunctionInput = FunctionInputMap.Find(Input))
		{
			return *FunctionInput;
		}

		return FModuleScopedPin();
	}

private:
	void Traverse(FStringView Namespace, const FGraph* Graph, ENiagaraScriptUsage Usage, const FGuid& UsageId, TMap<FName, FModuleScopedPin>& UnresolvedInputs)
	{
		FNameBuilder NamespacePrefix;
		if (Namespace.Len() > 0)
		{
			NamespacePrefix << Namespace;
			NamespacePrefix << TEXT(".");
		}
		const FName NamespaceFName(Namespace);

		if (const FOutputNode* OutputNode = GraphBridge::FindOutputNode(Graph, Usage, UsageId))
		{
			TArray<const FNode*> TraversedNodes;
			GraphBridge::BuildTraversal(Graph, OutputNode, TraversedNodes);

			for (const FNode* Node : TraversedNodes)
			{
				if (const FFunctionCallNode* FunctionNode = GraphBridge::AsFunctionCallNode(Node))
				{
					if (const FGraph* FunctionGraph = GraphBridge::GetFunctionNodeGraph(FunctionNode))
					{
						FNameBuilder NamespacedFunction;
						NamespacedFunction << NamespacePrefix.ToView();
						NamespacedFunction << GraphBridge::GetFunctionName(FunctionNode);

						TMap<FName, FModuleScopedPin> FunctionInputs;
						Traverse(NamespacedFunction.ToView(), FunctionGraph, GraphBridge::GetFunctionUsage(FunctionNode), FGuid(), FunctionInputs);

						// resolve any of the UnresolvedInputs that we've accumulated
						for (const FInputPin* NodePin : GraphBridge::GetInputPins(FunctionNode))
						{
							if (FModuleScopedPin* InnerPin = FunctionInputs.Find(NodePin->PinName))
							{
								FunctionInputMap.Add(*InnerPin, FModuleScopedPin(NodePin, NamespaceFName));
								FunctionInputs.Remove(NodePin->PinName);
							}
						}
					}
				}
				if (const FInputNode* InputNode = GraphBridge::AsInputNode(Node))
				{
					int32 OutputPinCount = 0;
					for (const FOutputPin* InputNodePin : GraphBridge::GetOutputPins(InputNode))
					{
						check(OutputPinCount == 0);
						++OutputPinCount;

						UnresolvedInputs.Add(GraphBridge::GetInputVariable(InputNode).GetName(), FModuleScopedPin(InputNodePin, NamespaceFName));
					}
				}
			}
		}
	}

	TMap<FModuleScopedPin, FModuleScopedPin> FunctionInputMap;
};

template<typename GraphBridge>
class FNiagaraAttributeTrimmerHelper<GraphBridge>::FImpureFunctionParser
{
public:
	FImpureFunctionParser(const FGraph* Graph, ENiagaraScriptUsage Usage, const FGuid& UsageId, FTrimAttributeCache& InCache)
		: Cache(InCache)
	{
		Traverse(TEXT(""), Graph, Usage, UsageId);
	}

	const TArray<FModuleScopedPin>& ReadFunctionInputs() const
	{
		return ImpureFunctionInputs;
	}

private:
	void Traverse(FString Namespace, const FGraph* Graph, ENiagaraScriptUsage Usage, const FGuid& UsageId)
	{
		const FString NamespacePrefix = (Namespace.Len() > 0) ? (Namespace + TEXT(".")) : TEXT("");

		if (const FOutputNode* OutputNode = GraphBridge::FindOutputNode(Graph, Usage, UsageId))
		{
			TArray<const FNode*> TraversedNodes;
			GraphBridge::BuildTraversal(Graph, OutputNode, TraversedNodes);

			for (const FNode* Node : TraversedNodes)
			{
				if (const FFunctionCallNode* FunctionNode = GraphBridge::AsFunctionCallNode(Node))
				{
					if (const FGraph* FunctionGraph = GraphBridge::GetFunctionNodeGraph(FunctionNode))
					{
						Traverse(NamespacePrefix + GraphBridge::GetFunctionName(FunctionNode), FunctionGraph, GraphBridge::GetFunctionUsage(FunctionNode), FGuid());
					}

					bool bEvaluatePins = false;
					if (FunctionNode->Signature.bRequiresExecPin)
					{
						bEvaluatePins = true;
					}
					else if(const FCustomHlslNode* CustomHlslNode = GraphBridge::AsCustomHlslNode(Node))
					{
						FCustomHlslNodeInfo NodeInfo;
						BuildCustomHlslNodeInfo(Cache, CustomHlslNode, NodeInfo);

						bEvaluatePins = NodeInfo.bHasImpureFunctionText;
					}

					if (bEvaluatePins)
					{
						for (const FInputPin* Pin : GraphBridge::GetInputPins(FunctionNode))
						{
							// setting the following to false will allow the trimmer to remove additional attributes.  A value of false
							// will limit our dependency search to the function inputs rather than the execution path.
							constexpr bool bBackCompat_IncludeExecPinsForImpureFunctions = true;

							if (!bBackCompat_IncludeExecPinsForImpureFunctions)
							{
								if (GraphBridge::GetPinType(Pin, ENiagaraStructConversion::Simulation) == FNiagaraTypeDefinition::GetParameterMapDef())
								{
									continue;
								}
							}

							ImpureFunctionInputs.Emplace(Pin, *Namespace);
						}
					}
				}
			}
		}
	}

	FTrimAttributeCache& Cache;
	TArray<FModuleScopedPin> ImpureFunctionInputs;
};

template<typename GraphBridge>
class FNiagaraAttributeTrimmerHelper<GraphBridge>::FExpressionBuilder
{
public:

	FExpressionBuilder(const FParamMapHistory& InParamMap, const FFunctionInputResolver& InInputResolver, FTrimAttributeCache& InCache)
		: ParamMap(InParamMap)
		, InputResolver(InInputResolver)
		, Cache(InCache)
	{}

	// generates a set of dependencies for a specific pin
	//	-generates the set of input pins to nodes that contribute to the 'expression' of the supplied output pin (EndPin)
	//		An expression is the set of nodes that define what is being set in a MapSet.  It does not traverse through
	//		Exec pins as those nodes would be part of another expression
	//	-generates the list of custom HLSL nodes that are encountered in the traversal
	void FindDependencies(const FModuleScopedPin& EndPin, FDependencyChain& Dependencies)
	{
		FindDependenciesInternal(EndPin, Dependencies, nullptr);
	}

private:
	// generates a set of dependencies for a specific pin
	//	-generates the set of input pins to nodes that contribute to the 'expression' of the supplied output pin (EndPin)
	//		An expression is the set of nodes that define what is being set in a MapSet.  It does not traverse through
	//		Exec pins as those nodes would be part of another expression
	//	-generates the list of custom HLSL nodes that are encountered in the traversal
	void FindDependenciesInternal(const FModuleScopedPin& EndPin, FDependencyChain& Dependencies, TArray<const FInputNode*>* InputNodes)
	{
		TStringBuilder<1024> NamespaceBuilder;

		TArray<FModuleScopedPin> PinsToEvaluate;
		TSet<const FNode*> EvaluatedNodes;

		FModuleScopedPin CurrentPin = EndPin;
		while (CurrentPin.Pin)
		{
			if (CurrentPin.Pin->Direction == EEdGraphPinDirection::EGPD_Output)
			{
				continue;
			}

			const FInputPin* CurrentInputPin = static_cast<const FInputPin*>(CurrentPin.Pin);
			if (const FOutputPin* TracedPin = GraphBridge::GetLinkedOutputPin(CurrentInputPin))
			{
				const FNode* CurrentNode = GraphBridge::GetOwningNode(TracedPin);
				bool AlreadyEvaluated = false;

				// record any output pins to a parameter get node
				if (const FParamMapGetNode* ParamGetNode = GraphBridge::AsParamMapGetNode(CurrentNode))
				{
					Dependencies.Pins.Emplace(FModuleScopedPin(TracedPin, CurrentPin.ModuleName));
				}
				else if (const FFunctionCallNode* FunctionCallNode = GraphBridge::AsFunctionCallNode(CurrentNode))
				{
					if (const FGraph* NodeGraph = GraphBridge::GetFunctionNodeGraph(FunctionCallNode))
					{
						TArray<const FOutputNode*> InnerOutputNodes;
						GraphBridge::FindOutputNodes(NodeGraph, InnerOutputNodes);

						NamespaceBuilder.Reset();
						if (CurrentPin.ModuleName != NAME_None)
						{
							CurrentPin.ModuleName.AppendString(NamespaceBuilder);
							NamespaceBuilder << TEXT(".");
						}
						NamespaceBuilder << GraphBridge::GetFunctionName(FunctionCallNode);

						const FName AggregateModuleName = *NamespaceBuilder;

						for (const FOutputNode* InnerOutputNode : InnerOutputNodes)
						{
							for (const FInputPin* InnerOutputNodePin : GraphBridge::GetInputPins(InnerOutputNode))
							{
								if (InnerOutputNodePin->PinName == TracedPin->PinName)
								{
									TArray<const FInputNode*> InnerInputNodes;
									FindDependenciesInternal(FModuleScopedPin(InnerOutputNodePin, AggregateModuleName), Dependencies, &InnerInputNodes);

									// map the InputNodes we found to the input pins on the FunctionCallNode
									for (const FInputNode* InnerInputNode : InnerInputNodes)
									{
										for (const FInputPin* NodePin : GraphBridge::GetInputPins(FunctionCallNode))
										{
											if (NodePin->PinName == GraphBridge::GetInputVariable(InnerInputNode).GetName())
											{
												PinsToEvaluate.Emplace(NodePin, CurrentPin.ModuleName);
												break;
											}
										}
									}
								}
							}
						}

						// skip adding the inputs to the list of pins we need to evaluate because we've already search through the internal graph
						// to figure out the minimal set of items we care about
						AlreadyEvaluated = true;
					}
				}
				else if (const FInputNode* InputNode = GraphBridge::AsInputNode(CurrentNode))
				{
					if (InputNodes)
					{
						InputNodes->AddUnique(InputNode);
					}
					else
					{
						const FModuleScopedPin ResolvedPin = InputResolver.ResolveInput(FModuleScopedPin(TracedPin, CurrentPin.ModuleName));
						if (ResolvedPin.Pin)
						{
							PinsToEvaluate.Add(ResolvedPin);
							EvaluatedNodes.Add(InputNode);
							AlreadyEvaluated = true;
						}
					}
				}

				if (!AlreadyEvaluated)
				{
					EvaluatedNodes.Add(CurrentNode, &AlreadyEvaluated);
				}

				if (!AlreadyEvaluated)
				{
					for (const FInputPin* NodePin : GraphBridge::GetInputPins(CurrentNode))
					{
						// we want to exclude execution pins
						const bool IsParamMapPin = GraphBridge::GetPinType(NodePin, ENiagaraStructConversion::Simulation) == FNiagaraTypeDefinition::GetParameterMapDef();
						const bool IsExecPin = NodePin->PinName == UNiagaraNodeParameterMapBase::SourcePinName;
						if (IsParamMapPin || IsExecPin)
						{
							continue;
						}

						PinsToEvaluate.Emplace(NodePin, CurrentPin.ModuleName);
					}

					if (const FCustomHlslNode* CustomHlslNode = GraphBridge::AsCustomHlslNode(CurrentNode))
					{
						if (!Dependencies.CustomNodes.Contains(CustomHlslNode))
						{
							FCustomHlslNodeInfo& NodeInfo = Dependencies.CustomNodes.Add(CustomHlslNode);
							BuildCustomHlslNodeInfo(Cache, CustomHlslNode, NodeInfo);
						}
					}
				}
			}

			if (PinsToEvaluate.Num())
			{
				CurrentPin = PinsToEvaluate.Pop(EAllowShrinking::No);
			}
			else
			{
				break;
			}
		}
	}

	const FParamMapHistory& ParamMap;
	const FFunctionInputResolver& InputResolver;
	FTrimAttributeCache& Cache;

	const bool EvaluateStaticSwitches = false;
};

template<typename GraphBridge>
void FNiagaraAttributeTrimmerHelper<GraphBridge>::BuildCustomHlslNodeInfo(FTrimAttributeCache& InCache, const FCustomHlslNode* CustomNode, FCustomHlslNodeInfo& NodeInfo)
{
	if (CustomNode == nullptr)
	{
		return;
	}

	if (FCustomHlslNodeInfo* ExistingNodeInfo = InCache.CustomNodeCache.Find(CustomNode))
	{
		NodeInfo = *ExistingNodeInfo;
		return;
	}

	NodeInfo.bHasImpureFunctionText = GraphBridge::GetCustomNodeUsesImpureFunctions(CustomNode);

	InCache.CustomNodeCache.Add(CustomNode, NodeInfo);
}

// For a specific read of a variable finds the corresponding PreviousWritePin if one exists (only considers actual writes
// rather than default pins on a MapGet)
template<typename GraphBridge>
typename FNiagaraAttributeTrimmerHelper<GraphBridge>::FModuleScopedPin FNiagaraAttributeTrimmerHelper<GraphBridge>::FindWriteForRead(const FParamMapHistory& ParamMap, const FModuleScopedPin& ReadPin, int32* OutVariableIndex)
{
	if (ReadPin.Pin == nullptr)
	{
		return FModuleScopedPin();
	}

	int32 VariableCount = ParamMap.Variables.Num();

	// without more context we need to do a full linear search through the ReadHistory
	for (int32 VariableIt = 0; VariableIt < VariableCount; ++VariableIt)
	{
		const TArray<typename FParamMapHistory::FReadHistory>& ReadHistoryEntries = ParamMap.PerVariableReadHistory[VariableIt];

		for (const typename FParamMapHistory::FReadHistory& ReadHistoryEntry : ReadHistoryEntries)
		{
			if (ReadHistoryEntry.ReadPin == ReadPin)
			{
				if (OutVariableIndex)
				{
					*OutVariableIndex = VariableIt;
				}

				if (ReadHistoryEntry.PreviousWritePin.Pin)
				{
					// we only care about writes (MapSets).  This will exclude default variables that are set as the input to the MapGet
					if (const FParamMapSetNode* WriteNode = GraphBridge::AsParamMapSetNode(GraphBridge::GetOwningNode(ReadHistoryEntry.PreviousWritePin.Pin)))
					{
						return ReadHistoryEntry.PreviousWritePin;
					}
				}

				return FModuleScopedPin();
			}
		}
	}

	return FModuleScopedPin();
}

template<typename GraphBridge>
int32 FNiagaraAttributeTrimmerHelper<GraphBridge>::FindDefaultBinding(const FParamMapHistory& ParamMap, int32 VariableIndex, const FModuleScopedPin& ReadPin)
{
	if (const FGraph* OwnerGraph = GraphBridge::GetOwningGraph(GraphBridge::GetOwningNode(ReadPin.Pin)))
	{
		FNiagaraVariable ParamVariable = ParamMap.Variables[VariableIndex];
		FString ParamName = ParamVariable.GetName().ToString();

		// now we need to replace the module name with 'module'
		ParamName.ReplaceInline(*ReadPin.ModuleName.ToString(), *FNiagaraConstants::ModuleNamespace.ToString());

		ParamVariable.SetName(*ParamName);

		FNiagaraScriptVariableBinding DefaultBinding;
		TOptional<ENiagaraDefaultMode> DefaultMode = GraphBridge::GetGraphDefaultMode(OwnerGraph, ParamVariable, DefaultBinding);
		if (DefaultMode.IsSet() && *DefaultMode == ENiagaraDefaultMode::Binding && DefaultBinding.IsValid())
		{
			return ParamMap.FindVariableByName(DefaultBinding.GetName());
		}
	}

	return INDEX_NONE;
}

// for a specific read of a variable finds the actual name of the attribute being read
template<typename GraphBridge>
FName FNiagaraAttributeTrimmerHelper<GraphBridge>::FindAttributeForRead(const FParamMapHistory& ParamMap, const FModuleScopedPin& ReadPin)
{
	const int32 VariableCount = ParamMap.Variables.Num();

	// wihtout more context we need to do a full linear search through the ReadHistory
	for (int32 VariableIt = 0; VariableIt < VariableCount; ++VariableIt)
	{
		const TArray<typename FParamMapHistory::FReadHistory>& ReadHistoryEntries = ParamMap.PerVariableReadHistory[VariableIt];

		for (const typename FParamMapHistory::FReadHistory& ReadHistoryEntry : ReadHistoryEntries)
		{
			if (ReadHistoryEntry.ReadPin == ReadPin)
			{
				return ParamMap.Variables[VariableIt].GetName();
			}
		}
	}

	return NAME_None;
}

// for a given pin search through the variables of the parameter map and try to resolve any potential ambiguity with module namespaces
template<typename GraphBridge>
FName FNiagaraAttributeTrimmerHelper<GraphBridge>::FindParameterMapVariable(const FParamMapHistory& ParamMap, const FModuleScopedPin& Pin)
{
	auto FindVariableName = [&ParamMap](FName NameToEvaluate, FName& OutVariableName) -> bool
	{
		if (ParamMap.FindVariableByName(NameToEvaluate) != INDEX_NONE)
		{
			OutVariableName = NameToEvaluate;
			return true;
		}

		// we also need to check to see if the attribute has resolved namespaces (i.e. Particles.Module.MyAttribute or StackContext.MyAttribute)
		// for which we'll just search through the parameter maps variables directly
		const int32 AliasedIndex = ParamMap.VariablesWithOriginalAliasesIntact.IndexOfByPredicate([&](const FNiagaraVariable& Variable)
		{
			return Variable.GetName() == NameToEvaluate;
		});

		if (ParamMap.Variables.IsValidIndex(AliasedIndex))
		{
			OutVariableName = ParamMap.Variables[AliasedIndex].GetName();
			return true;
		}

		return false;
	};

	FName FoundVariableName;

	// try the pin name
	if (FindVariableName(Pin.Pin->PinName, FoundVariableName))
	{
		return FoundVariableName;
	}
	else if (Pin.ModuleName != NAME_None)
	{
		FNameBuilder ModuleNameString(Pin.ModuleName);

		// check if we need to replace an explicit module name with 'Module'
		FString GenericPinName = Pin.Pin->PinName.ToString();
		if (GenericPinName.ReplaceInline(*ModuleNameString, *FNiagaraConstants::ModuleNamespaceString))
		{
			if (FindVariableName(*GenericPinName, FoundVariableName))
			{
				return FoundVariableName;
			}
		}

		// try replacing 'Module' with the explicit ModuleName
		FString ExplicitPinName = Pin.Pin->PinName.ToString();
		if (ExplicitPinName.ReplaceInline(*FNiagaraConstants::ModuleNamespaceString, *ModuleNameString))
		{
			if (FindVariableName(*ExplicitPinName, FoundVariableName))
			{
				return FoundVariableName;
			}
		}
	}

	return NAME_None;
}

// given the set of expressions (as defined in FindDependencies above) we resolve the named attribute aggregating the dependent reads and custom nodes
template<typename GraphBridge>
void FNiagaraAttributeTrimmerHelper<GraphBridge>::ResolveDependencyChain(const FParamMapHistory& ParamMap, const FDependencyMap& DependencyData, const FName& AttributeName, FDependencyChain& ResolvedDependencies)
{
	const int32 VariableIndex = ParamMap.FindVariableByName(AttributeName);
	if (VariableIndex != INDEX_NONE)
	{
		if (ParamMap.PerVariableWriteHistory[VariableIndex].Num())
		{
			const FModuleScopedPin& LastWritePin = ParamMap.PerVariableWriteHistory[VariableIndex].Last();

			TArray<FModuleScopedPin> PinsToResolve;
			PinsToResolve.Add(LastWritePin);

			while (PinsToResolve.Num() > 0)
			{
				const FModuleScopedPin PinToResolve = PinsToResolve.Pop(EAllowShrinking::No);

				if (const FDependencyChain* Dependencies = DependencyData.Find(PinToResolve))
				{
					for (const FModuleScopedPin& ReadPin : Dependencies->Pins)
					{
						bool AlreadyAdded = false;
						ResolvedDependencies.Pins.Add(ReadPin, &AlreadyAdded);

						if (!AlreadyAdded)
						{
							int32 SourceVariableIndex = INDEX_NONE;
							const FModuleScopedPin WritePin = FindWriteForRead(ParamMap, ReadPin, &SourceVariableIndex);

							if (WritePin.Pin)
							{
								PinsToResolve.Add(WritePin);
							}
							else if (SourceVariableIndex != INDEX_NONE)
							{
								const int32 DefaultBoundVariable = FindDefaultBinding(ParamMap, SourceVariableIndex, ReadPin);

								// if we are bound to a variable we need to grab the last relevant write to that variable and continue our resolution
								if (DefaultBoundVariable != INDEX_NONE)
								{
									bool bValidDefaultWriteFound = false;

									const FNode* ReadPinOwningNode = GraphBridge::GetOwningNode(ReadPin.Pin);
									const int32 ReadNodeVisitationIndex = ParamMap.MapNodeVisitations.IndexOfByKey(ReadPinOwningNode);
									if (ReadNodeVisitationIndex != INDEX_NONE)
									{
										const int32 VariableWriteCount = ParamMap.PerVariableWriteHistory[DefaultBoundVariable].Num();
										for (int32 VariableWriteIt = VariableWriteCount - 1; VariableWriteIt >= 0; --VariableWriteIt)
										{
											const FModuleScopedPin& DefaultWritePin = ParamMap.PerVariableWriteHistory[DefaultBoundVariable][VariableWriteIt];
											const FNode* DefaultWritePinOwningNode = GraphBridge::GetOwningNode(DefaultWritePin.Pin);
											const int32 DefaultWriteNodeVisitationIndex = ParamMap.MapNodeVisitations.IndexOfByKey(DefaultWritePinOwningNode);
											if (DefaultWriteNodeVisitationIndex != INDEX_NONE && DefaultWriteNodeVisitationIndex < ReadNodeVisitationIndex)
											{
												PinsToResolve.Add(DefaultWritePin);
												bValidDefaultWriteFound = true;
												break;
											}
										}
									}

									// if no valid pin was found that writes into the default bound variable within this parameter map, then we assume
									// that it will be present in a previous parameter map, and so we just record the earliest read from this parameter
									// map as a dependency

									if (!ParamMap.PerVariableReadHistory[DefaultBoundVariable].IsEmpty())
									{
										// setting the following to false will allow the trimmer to remove additional attributes.  In the cases
										// where an attribute is written to and used only in a single stage the value could be considered local and not
										// need to be an attribute.  If bBackCompat_AlwaysIncludeFirstReadPin is true then any time we read from that
										// variable we'll treat it as a dependent.  If it's false then the writes that contribute to that variable will 
										// be resolved as part of the dependency resolution.
										constexpr bool bBackCompat_AlwaysIncludeFirstReadPin = true;

										if (!bValidDefaultWriteFound || bBackCompat_AlwaysIncludeFirstReadPin)
										{
											const typename FParamMapHistory::FReadHistory& InitialReadHistory = ParamMap.PerVariableReadHistory[DefaultBoundVariable][0];

											if (InitialReadHistory.PreviousWritePin.Pin)
											{
												PinsToResolve.Add(InitialReadHistory.PreviousWritePin);
											}
											ResolvedDependencies.Pins.Add(InitialReadHistory.ReadPin);
										}
									}
								}
							}
						}
					}

					ResolvedDependencies.CustomNodes.Append(Dependencies->CustomNodes);
				}
			}
		}
	}
}

template<typename GraphBridge>
void FNiagaraAttributeTrimmerHelper<GraphBridge>::TrimAttributes_Safe(TConstArrayView<const FParamMapHistory*> LocalParamHistories, TSet<FName>& AttributesToPreserve, TArray<FNiagaraVariable>& Attributes)
{
	// go through the ParamMapHistories and collect any CustomHlsl nodes so that we can give a best effort to
	// find references to our variable.  If a reference is found we'll assume that we can't trim the attribute
	TArray<const FCustomHlslNode*, TInlineAllocator<8>> CustomHlslNodes;
	for (const FParamMapHistory* ParamMap : LocalParamHistories)
	{
		for (const FPin* Pin : ParamMap->MapPinHistory)
		{
			if (const FCustomHlslNode* CustomHlslNode = GraphBridge::AsCustomHlslNode(GraphBridge::GetOwningNode(Pin)))
			{
				CustomHlslNodes.AddUnique(CustomHlslNode);
			}
		}
	}

	Attributes.SetNum(Algo::StableRemoveIf(Attributes, [=](const FNiagaraVariable& Var)
	{
		// preserve attributes which have a record of being read
		for (const FParamMapHistory* ParamMap : LocalParamHistories)
		{
			const int32 VarIdx = ParamMap->FindVariable(Var.GetName(), Var.GetType());
			if (VarIdx != INDEX_NONE)
			{
				check(ParamMap->PerVariableReadHistory.IsValidIndex(VarIdx));
				if (ParamMap->PerVariableReadHistory[VarIdx].Num())
				{
					return false;
				}
			}
		}

		if (AttributesToPreserve.Contains(Var.GetName()))
		{
			return false;
		}

		FNameBuilder VariableName(Var.GetName());

		for (const FCustomHlslNode* CustomHlslNode : CustomHlslNodes)
		{
			if (GraphBridge::CustomHlslReferencesTokens(CustomHlslNode, { VariableName.ToView() }))
			{
				return false;
			}
		}

		return true;
	}));
}

template<typename GraphBridge>
void FNiagaraAttributeTrimmerHelper<GraphBridge>::TrimAttributes_Aggressive(const FCompilationCopy* CompileDuplicateData, TConstArrayView<const FParamMapHistory*> LocalParamHistories, TSet<FName>& AttributesToPreserve, TArray<FNiagaraVariable>& Attributes)
{
	FTrimAttributeCache Cache;

	// variable references hidden in custom hlsl nodes may not be present in a specific stages parameter map
	// so we consolidate all the variables into one list to use when going through custom hlsl nodes
	TSet<FNiagaraVariableBase> UnifiedVariables;

	for (const FParamMapHistory* ParamMap : LocalParamHistories)
	{
		for (const FNiagaraVariableBase& Variable : ParamMap->Variables)
		{
			UnifiedVariables.Add(Variable);
		}
	}

	FDependencyMap PerVariableDependencySets;

	for (const FParamMapHistory* ParamMap : LocalParamHistories)
	{
		FFunctionInputResolver InputResolver(GraphBridge::GetGraph(CompileDuplicateData), ParamMap->OriginatingScriptUsage, ParamMap->UsageGuid);

		const int32 VariableCount = ParamMap->Variables.Num();

		for (int32 VariableIt = 0; VariableIt < VariableCount; ++VariableIt)
		{
			const FNiagaraVariableBase& Var = ParamMap->Variables[VariableIt];

			for (const FModuleScopedPin& WritePin : ParamMap->PerVariableWriteHistory[VariableIt])
			{
				FDependencyChain& Dependencies = PerVariableDependencySets.Add(WritePin);

				FExpressionBuilder Builder(*ParamMap, InputResolver, Cache);
				Builder.FindDependencies(WritePin, Dependencies);

				for (typename FCustomHlslNodeMap::TConstIterator CustomNodeIt = Dependencies.CustomNodes.CreateConstIterator(); CustomNodeIt; ++CustomNodeIt)
				{
					if (CustomNodeIt.Value().bHasImpureFunctionText)
					{
						AttributesToPreserve.Add(Var.GetName());
					}
				}
			}
		}

		FImpureFunctionParser ImpureFunctionParser(GraphBridge::GetGraph(CompileDuplicateData), ParamMap->OriginatingScriptUsage, ParamMap->UsageGuid, Cache);
		for (const FModuleScopedPin& RequiredFunctionInput : ImpureFunctionParser.ReadFunctionInputs())
		{
			FDependencyChain Dependencies;

			FExpressionBuilder Builder(*ParamMap, InputResolver, Cache);
			Builder.FindDependencies(RequiredFunctionInput, Dependencies);

			for (const FModuleScopedPin& DependentPin : Dependencies.Pins)
			{
				// see if this variable corresponds to something in our parameter map's list of variables
				// if it does then we need to mark it as something that needs to be preserved
				// Note that the parameter map can have the variable represented with either an explicit namespace
				// or the generic Module, so we need to try both.
				const FName MatchingVariableName = FindParameterMapVariable(*ParamMap, DependentPin);
				if (MatchingVariableName != NAME_None)
				{
					AttributesToPreserve.Add(MatchingVariableName);
				}
			}
		}
	}

	TArray<FName> SearchList = AttributesToPreserve.Array();

	auto ConditionalAddAttribute = [&](const FName AttributeName)
	{
		bool AlreadyAdded = false;
		AttributesToPreserve.Add(AttributeName, &AlreadyAdded);
		if (!AlreadyAdded)
		{
			SearchList.Add(AttributeName);
		}
	};

	while (SearchList.Num())
	{
		const FName AttributeName = SearchList.Pop(EAllowShrinking::No);

		for (const FParamMapHistory* ParamMap : LocalParamHistories)
		{
			FDependencyChain ResolvedChain;
			ResolveDependencyChain(*ParamMap, PerVariableDependencySets, AttributeName, ResolvedChain);

			for (const FModuleScopedPin& ParameterRead : ResolvedChain.Pins)
			{
				const FName DependentAttributeName = FindAttributeForRead(*ParamMap, ParameterRead);

				if (DependentAttributeName != NAME_None)
				{
					// for each of the reads we are dependent on, check to see if we have a write associated with it
					// if we don't then it's value will be coming from the previous frame/stage and so we need to preserve
					// the attribute.
					if (!FindWriteForRead(*ParamMap, ParameterRead, nullptr).Pin)
					{
						ConditionalAddAttribute(DependentAttributeName);
					}

					// if we're dependent on a Previous value, then make sure we add the attribute to the list of things to be preserved as well
					if (FNiagaraParameterUtilities::IsPreviousName(DependentAttributeName))
					{
						ConditionalAddAttribute(FNiagaraParameterUtilities::GetSourceForPreviousValue(DependentAttributeName));
					}

					// likewise for Initial value
					if (FNiagaraParameterUtilities::IsInitialName(DependentAttributeName))
					{
						ConditionalAddAttribute(FNiagaraParameterUtilities::GetSourceForInitialValue(DependentAttributeName));
					}
				}
			}

			for (const auto& CustomHlslInfo : ResolvedChain.CustomNodes)
			{
				// go through all of the variables in the attributes to see if they are referenced by any encountered CustomNodes
				TArray<FName> VariableNames;
				for (const FNiagaraVariableBase& Variable : UnifiedVariables)
				{
					// skip searching through the hlsl code if we're already included
					if (AttributesToPreserve.Find(Variable.GetName()))
					{
						continue;
					}
					VariableNames.Push(Variable.GetName());
				}

				const int32 VariableCount = VariableNames.Num();
				TArray<bool> VariableReferenced;
				VariableReferenced.SetNumZeroed(VariableCount);
				GraphBridge::CustomHlslReferencesTokens(CustomHlslInfo.Key, VariableNames, VariableReferenced);
				for (int32 VariableIt = 0; VariableIt < VariableCount; ++VariableIt)
				{
					if (VariableReferenced[VariableIt])
					{
						ConditionalAddAttribute(VariableNames[VariableIt]);
					}
				}
			}
		}
	}

	auto SafeToRemove = [&](const FNiagaraVariable& Var)
	{
		return !AttributesToPreserve.Contains(Var.GetName());
	};

	int32 AttributeIt = 0;
	while (AttributeIt < Attributes.Num())
	{
		const FNiagaraVariableBase& Attribute = Attributes[AttributeIt];

		if (!AttributesToPreserve.Contains(Attribute.GetName()))
		{
			// when we have something to do with this information, we can re-enable the compile tag
			//TranslateResults.CompileTags.Emplace(Attribute, TEXT("Trimmed"));
			Attributes.RemoveAt(AttributeIt, EAllowShrinking::No);
		}
		else
		{
			++AttributeIt;
		}
	}
}

template class FNiagaraAttributeTrimmerHelper<FNiagaraCompilationGraphBridge>;
template class FNiagaraAttributeTrimmerHelper<FNiagaraCompilationDigestBridge>;
