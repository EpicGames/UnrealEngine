// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"

template<typename GraphBridge>
class FNiagaraAttributeTrimmerHelper : public GraphBridge
{
	using typename GraphBridge::FGraph;
	using typename GraphBridge::FPin;
	using typename GraphBridge::FCustomHlslNode;
	using typename GraphBridge::FNode;
	using typename GraphBridge::FOutputNode;
	using typename GraphBridge::FInputNode;
	using typename GraphBridge::FFunctionCallNode;
	using typename GraphBridge::FParamMapGetNode;
	using typename GraphBridge::FParamMapSetNode;
	using typename GraphBridge::FInputPin;
	using typename GraphBridge::FOutputPin;
	using typename GraphBridge::FParamMapHistory;
	using typename GraphBridge::FCompilationCopy;
	using typename GraphBridge::FModuleScopedPin;

	using FDependencySet = TSet<FModuleScopedPin>;
	struct FCustomHlslNodeInfo
	{
		bool bHasImpureFunctionText = false;
	};

	using FCustomHlslNodeMap = TMap<const FCustomHlslNode*, FCustomHlslNodeInfo>;

	struct FDependencyChain
	{
		FDependencySet Pins;
		FCustomHlslNodeMap CustomNodes;
	};

	struct FTrimAttributeCache
	{
		FCustomHlslNodeMap CustomNodeCache;
	};

	using FDependencyMap = TMap<FModuleScopedPin, FDependencyChain>;

	class FFunctionInputResolver;
	class FImpureFunctionParser;
	class FExpressionBuilder;

	// For a specific read of a variable finds the corresponding PreviousWritePin if one exists (only considers actual writes
	// rather than default pins on a MapGet)
	static FModuleScopedPin FindWriteForRead(const FParamMapHistory& ParamMap, const FModuleScopedPin& ReadPin, int32* OutVariableIndex);
	static int32 FindDefaultBinding(const FParamMapHistory& ParamMap, int32 VariableIndex, const FModuleScopedPin& ReadPin);

	// for a specific read of a variable finds the actual name of the attribute being read
	static FName FindAttributeForRead(const FParamMapHistory& ParamMap, const FModuleScopedPin& ReadPin);

	// for a given pin search through the variables of the parameter map and try to resolve any potential ambiguity with module namespaces
	static FName FindParameterMapVariable(const FParamMapHistory& ParamMap, const FModuleScopedPin& Pin);

	// given the set of expressions (as defined in FindDependencies above) we resolve the named attribute aggregating the dependent reads and custom nodes
	static void ResolveDependencyChain(const FParamMapHistory& ParamMap, const FDependencyMap& DependencyData, const FName& AttributeName, FDependencyChain& ResolvedDependencies);

	static void BuildCustomHlslNodeInfo(FTrimAttributeCache& Cache, const FCustomHlslNode* CustomNode, FCustomHlslNodeInfo& NodeInfo);

public:
	static void TrimAttributes_Safe(TConstArrayView<const FParamMapHistory*> LocalParamHistories, TSet<FName>& AttributesToPreserve, TArray<FNiagaraVariable>& Attributes);
	static void TrimAttributes_Aggressive(const FCompilationCopy* CompileDuplicateData, TConstArrayView<const FParamMapHistory*> LocalParamHistories, TSet<FName>& AttributesToPreserve, TArray<FNiagaraVariable>& Attributes);
};


