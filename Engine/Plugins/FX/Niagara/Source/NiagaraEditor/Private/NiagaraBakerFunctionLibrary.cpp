// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerFunctionLibrary.h"

#include "NiagaraBakerRendererOutputStaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerFunctionLibrary)

void UNiagaraBakerFunctionLibrary::CaptureNiagaraToStaticMesh(UNiagaraComponent* ComponentToCapture, UStaticMesh* StaticMeshOutput, FNiagaraRendererReadbackParameters ReadbackParameters)
{
#if WITH_NIAGARA_RENDERER_READBACK
	NiagaraRendererReadback::EnqueueReadback(
		ComponentToCapture,
		[ComponentToCapture, StaticMeshOutput](const FNiagaraRendererReadbackResult& ReadbackResult)
		{
			// Failed or no data
			if (ReadbackResult.NumVertices == 0)
			{
				return;
			}

			FNiagaraBakerRendererOutputStaticMesh::ConvertReadbackResultsToStaticMesh(ReadbackResult, StaticMeshOutput);
		},
		ReadbackParameters
	);
#endif
}
