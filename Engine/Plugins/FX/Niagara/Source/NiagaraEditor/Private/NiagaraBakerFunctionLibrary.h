// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "NiagaraRendererReadback.h"

#include "NiagaraBakerFunctionLibrary.generated.h"

class UNiagaraComponent;
class UStaticMesh;

UCLASS(MinimalAPI)
class UNiagaraBakerFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Niagara Baker", meta = (Keywords = "Niagara Baker", WorldContext = "WorldContextObject", UnsafeDuringActorConstruction = "true"))
	static void CaptureNiagaraToStaticMesh(UNiagaraComponent* ComponentToCapture, UStaticMesh* StaticMeshOutput, FNiagaraRendererReadbackParameters ReadbackParameters = FNiagaraRendererReadbackParameters());
};
