// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerOutputRegistry.h"
#include "IDetailsView.h"
#include "NiagaraBakerOutputSimCache.h"
#include "NiagaraBakerOutputStaticMesh.h"
#include "NiagaraBakerOutputTexture2D.h"
#include "NiagaraBakerOutputVolumeTexture.h"
#include "NiagaraBakerOutputSparseVolumeTexture.h"
#include "NiagaraBakerRendererOutputSimCache.h"
#include "NiagaraBakerRendererOutputStaticMesh.h"
#include "NiagaraBakerRendererOutputTexture2D.h"
#include "NiagaraBakerRendererOutputVolumeTexture.h"
#include "NiagaraBakerRendererOutputSparseVolumeTexture.h"
#include "Customizations/NiagaraBakerOutputCustomization.h"

FNiagaraBakerOutputRegistry& FNiagaraBakerOutputRegistry::Get()
{
	static TUniquePtr<FNiagaraBakerOutputRegistry> Singleton;
	if ( Singleton.IsValid() == false )
	{
		Singleton = MakeUnique<FNiagaraBakerOutputRegistry>();

		// Sim Cache
		{
			FRegistryEntry& Entry = Singleton->Registry.AddDefaulted_GetRef();
			Entry.WeakClass				= UNiagaraBakerOutputSimCache::StaticClass();
			Entry.CreateRenderer		= []() -> FNiagaraBakerOutputRenderer* { return new FNiagaraBakerRendererOutputSimCache(); };
			Entry.CreateCustomization	= FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraBakerOutputSimCacheDetails::MakeInstance);
		}
		// Static Mesh
		{
			FRegistryEntry& Entry = Singleton->Registry.AddDefaulted_GetRef();
			Entry.WeakClass				= UNiagaraBakerOutputStaticMesh::StaticClass();
			Entry.CreateRenderer		= []() -> FNiagaraBakerOutputRenderer* { return new FNiagaraBakerRendererOutputStaticMesh(); };
			Entry.CreateCustomization	= FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraBakerOutputStaticMeshDetails::MakeInstance);
		}
		// Texture 2D
		{
			FRegistryEntry& Entry		= Singleton->Registry.AddDefaulted_GetRef();
			Entry.WeakClass				= UNiagaraBakerOutputTexture2D::StaticClass();
			Entry.CreateRenderer		= []() -> FNiagaraBakerOutputRenderer* { return new FNiagaraBakerRendererOutputTexture2D(); };
			Entry.CreateCustomization	= FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraBakerOutputTexture2DDetails::MakeInstance);
		}
		// Volume Texture
		{
			FRegistryEntry& Entry = Singleton->Registry.AddDefaulted_GetRef();
			Entry.WeakClass				= UNiagaraBakerOutputVolumeTexture::StaticClass();
			Entry.CreateRenderer		= []() -> FNiagaraBakerOutputRenderer* { return new FNiagaraBakerRendererOutputVolumeTexture(); };
			Entry.CreateCustomization	= FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraBakerOutputVolumeTextureDetails::MakeInstance);
		}
		// Sparse Volume Texture
		{
			FRegistryEntry& Entry = Singleton->Registry.AddDefaulted_GetRef();
			Entry.WeakClass = UNiagaraBakerOutputSparseVolumeTexture::StaticClass();
			Entry.CreateRenderer = []() -> FNiagaraBakerOutputRenderer* { return new FNiagaraBakerRendererOutputSparseVolumeTexture(); };
			Entry.CreateCustomization = FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraBakerOutputSparseVolumeTextureDetails::MakeInstance);
		}
	}
	return *Singleton.Get();
}

TArray<UClass*> FNiagaraBakerOutputRegistry::GetOutputClasses() const
{
	TArray<UClass*> Classes;
	for ( const FRegistryEntry& Entry : Registry )
	{
		if (Entry.WeakClass.Get() == nullptr)
		{
			continue;
		}
		if ( Entry.ShouldShowCVar != nullptr && Entry.ShouldShowCVar->GetBool() == false )
		{
			continue;
		}
		Classes.Add(Entry.WeakClass.Get());
	}
	return Classes;
}

FNiagaraBakerOutputRenderer* FNiagaraBakerOutputRegistry::GetRendererForClass(UClass* Class) const
{
	for (const FRegistryEntry& Entry : Registry)
	{
		if (Entry.WeakClass.Get() == Class)
		{
			return Entry.CreateRenderer();
		}
	}
	return nullptr;
}

void FNiagaraBakerOutputRegistry::RegisterCustomizations(IDetailsView* DetailsView) const
{
	for (const FRegistryEntry& Entry : Registry)
	{
		if (Entry.WeakClass.Get() && Entry.CreateCustomization.IsBound() )
		{
			DetailsView->RegisterInstancedCustomPropertyLayout(Entry.WeakClass.Get(), Entry.CreateCustomization);
		}
	}
}
