// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyEditorDelegates.h"

class FNiagaraBakerOutputRenderer;
class IDetailsView;

class FNiagaraBakerOutputRegistry
{
	struct FRegistryEntry
	{
		TWeakObjectPtr<UClass>						WeakClass;
		IConsoleVariable*							ShouldShowCVar = nullptr;
		TFunction<FNiagaraBakerOutputRenderer*()>	CreateRenderer;
		FOnGetDetailCustomizationInstance			CreateCustomization;
	};

public:
	static FNiagaraBakerOutputRegistry& Get();

	TArray<UClass*> GetOutputClasses() const;
	FNiagaraBakerOutputRenderer* GetRendererForClass(UClass* Class) const;
	void RegisterCustomizations(IDetailsView* DetailsView) const;

private:
	TArray<FRegistryEntry>	Registry;
};

