// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerRenderer.h"
#include "NiagaraBakerSettings.h"
#include "NiagaraBakerOutputRegistry.h"
#include "NiagaraComponent.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraBatchedElements.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraScriptRuntimeData.h"

#include "NiagaraDataInterfaceGrid2DCollection.h"

#include "Components/SceneCaptureComponent2D.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "AdvancedPreviewScene.h"
#include "BufferVisualizationData.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "EngineModule.h"
#include "IImageWrapperModule.h"
#include "IImageWrapper.h"
#include "ImageWrapperHelper.h"
#include "LegacyScreenPercentageDriver.h"
#include "UObject/Package.h"
#include "TextureResource.h"
#include "SceneInterface.h"
#include "RendererInterface.h"

#include "NiagaraDataInterfaceGrid3DCollection.h"
#include "NiagaraDataInterfaceRenderTargetVolume.h"

#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Components/HeterogeneousVolumeComponent.h"

DEFINE_LOG_CATEGORY(LogNiagaraBaker);

//////////////////////////////////////////////////////////////////////////

namespace NiagaraBakerRendererPrivate
{
	static void CreatePreviewScene(UNiagaraSystem* NiagaraSystem, TObjectPtr<UNiagaraComponent>& OutComponent, TSharedPtr<FAdvancedPreviewScene>& OutPreviewScene)
	{
		check(NiagaraSystem);
		OutComponent = NewObject<UNiagaraComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		OutComponent->CastShadow = 1;
		OutComponent->bCastDynamicShadow = 1;
		OutComponent->SetAllowScalability(false);
		OutComponent->SetAsset(NiagaraSystem);
		OutComponent->SetForceSolo(true);
		OutComponent->SetAgeUpdateMode(ENiagaraAgeUpdateMode::DesiredAge);
		OutComponent->SetCanRenderWhileSeeking(true);
		OutComponent->SetMaxSimTime(0.0f);
		OutComponent->Activate(true);

		OutPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));
		OutPreviewScene->SetFloorVisibility(false);
		OutPreviewScene->AddComponent(OutComponent, OutComponent->GetRelativeTransform());
	}

	template<typename TComponentType>
	static void CreatePreviewScene(TObjectPtr<TComponentType>& OutComponent, TSharedPtr<FAdvancedPreviewScene>& OutPreviewScene)
	{
		OutComponent = NewObject<TComponentType>(GetTransientPackage(), NAME_None, RF_Transient);

		OutPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));
		OutPreviewScene->SetFloorVisibility(false);
		OutPreviewScene->AddComponent(OutComponent, OutComponent->GetRelativeTransform());
	}

	template<typename TComponentType>
	static void DestroyPreviewScene(TObjectPtr<TComponentType>& InOutComponent, TSharedPtr<FAdvancedPreviewScene>& InOutPreviewScene)
	{
		if (InOutPreviewScene && InOutComponent)
		{
			InOutPreviewScene->RemoveComponent(InOutComponent);
			InOutPreviewScene = nullptr;
		}

		if (InOutComponent)
		{
			InOutComponent->DestroyComponent();
			InOutComponent = nullptr;
		}
	}
}

//////////////////////////////////////////////////////////////////////////

const FString FNiagaraBakerOutputBindingHelper::STRING_SceneCaptureSource("SceneCaptureSource");
const FString FNiagaraBakerOutputBindingHelper::STRING_BufferVisualization("BufferVisualization");
const FString FNiagaraBakerOutputBindingHelper::STRING_EmitterDI("EmitterDI");
const FString FNiagaraBakerOutputBindingHelper::STRING_EmitterParticles("EmitterParticles");

FNiagaraBakerOutputBindingHelper::ERenderType FNiagaraBakerOutputBindingHelper::GetRenderType(FName BindingName, FName& OutName)
{
	OutName = FName();
	if (BindingName.IsNone())
	{
		return ERenderType::SceneCapture;
	}

	FString SourceBindingString = BindingName.ToString();
	TArray<FString> SplitNames;
	SourceBindingString.ParseIntoArray(SplitNames, TEXT("."));

	if (!ensure(SplitNames.Num() > 0))
	{
		return ERenderType::None;
	}

	// Scene Capture mode
	if (SplitNames[0] == STRING_SceneCaptureSource)
	{
		if (!ensure(SplitNames.Num() == 2))
		{
			return ERenderType::None;
		}

		OutName = FName(SplitNames[1]);
		return ERenderType::SceneCapture;
	}

	// Buffer Visualization Mode
	if (SplitNames[0] == STRING_BufferVisualization)
	{
		if (!ensure(SplitNames.Num() == 2))
		{
			return ERenderType::None;
		}
		OutName = FName(SplitNames[1]);
		return ERenderType::BufferVisualization;
	}

	// Emitter Data Interface
	if (SplitNames[0] == STRING_EmitterDI)
	{
		OutName = FName(*SourceBindingString.RightChop(SplitNames[0].Len() + 1));
		return ERenderType::DataInterface;
	}

	// Emitter Data Interface
	if (SplitNames[0] == STRING_EmitterParticles)
	{
		OutName = FName(*SourceBindingString.RightChop(SplitNames[0].Len() + 1));
		return ERenderType::Particle;
	}
	return ERenderType::None;
}

void FNiagaraBakerOutputBindingHelper::GetSceneCaptureBindings(TArray<FNiagaraBakerOutputBinding>& OutBindings)
{
	static UEnum* SceneCaptureOptions = StaticEnum<ESceneCaptureSource>();
	for (int i = 0; i < SceneCaptureOptions->GetMaxEnumValue(); ++i)
	{
		FNiagaraBakerOutputBinding& NewBinding = OutBindings.AddDefaulted_GetRef();
		NewBinding.BindingName = FName(STRING_SceneCaptureSource + TEXT(".") + SceneCaptureOptions->GetNameStringByIndex(i));
		NewBinding.MenuCategory = FText::FromString(STRING_SceneCaptureSource);
		NewBinding.MenuEntry = SceneCaptureOptions->GetDisplayNameTextByIndex(i);
	}
}

void FNiagaraBakerOutputBindingHelper::GetBufferVisualizationBindings(TArray<FNiagaraBakerOutputBinding>& OutBindings)
{
	// Gather all buffer visualization options
	struct FIterator
	{
		TArray<FNiagaraBakerOutputBinding>& OutBindings;

		FIterator(TArray<FNiagaraBakerOutputBinding>& InOutBindings)
			: OutBindings(InOutBindings)
		{}

		void ProcessValue(const FString& MaterialName, UMaterialInterface* Material, const FText& DisplayName)
		{
			FNiagaraBakerOutputBinding& NewBinding = OutBindings.AddDefaulted_GetRef();
			NewBinding.BindingName = FName(STRING_BufferVisualization + TEXT(".") + MaterialName);
			NewBinding.MenuCategory = FText::FromString(STRING_BufferVisualization);
			NewBinding.MenuEntry = DisplayName;
		}
	} Iterator(OutBindings);
	GetBufferVisualizationData().IterateOverAvailableMaterials(Iterator);
}

void FNiagaraBakerOutputBindingHelper::ForEachEmitterDataInterface(UNiagaraSystem* NiagaraSystem, FEmitterDIFunction Function)
{
	check(NiagaraSystem);

	for (int32 EmitterIndex=0; EmitterIndex < NiagaraSystem->GetEmitterHandles().Num(); ++EmitterIndex)
	{
		const FNiagaraEmitterHandle& EmitterHandle = NiagaraSystem->GetEmitterHandle(EmitterIndex);
		FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetInstance().GetEmitterData();
		if (!EmitterHandle.IsValid() || !EmitterHandle.GetIsEnabled() || !EmitterData)
		{
			continue;
		}

		const FString EmitterName = EmitterHandle.GetName().ToString();
		const FString EmitterPrefix = EmitterName + TEXT(".");

		EmitterData->ForEachScript(
			[&](UNiagaraScript* NiagaraScript)
			{
				const FNiagaraScriptRuntimeData* ScriptRuntimeData = NiagaraSystem->GetScriptRuntimeData(&EmitterHandle, *NiagaraScript).Get();
				const FNiagaraScriptExecutionParameterStore* SrcStore = ScriptRuntimeData != nullptr ? ScriptRuntimeData->GetExecutionReadyParameterStore() : nullptr;
				if (SrcStore != nullptr)
				{
					for (const FNiagaraVariableWithOffset& Variable : SrcStore->ReadParameterVariables())
					{
						if (Variable.IsDataInterface() == false)
						{
							continue;
						}

						const FString VariableName = Variable.GetName().ToString();
						if (!VariableName.StartsWith(EmitterPrefix))
						{
							continue;
						}

						UNiagaraDataInterface* DataInterface = SrcStore->GetDataInterface(Variable.Offset);
						check(DataInterface);

						Function(EmitterName, VariableName.Mid(EmitterPrefix.Len()), DataInterface);
					}
				}
			}
		);
	}
}

UNiagaraDataInterface* FNiagaraBakerOutputBindingHelper::GetDataInterface(UNiagaraComponent* NiagaraComponent, FName DataInterfaceName)
{
	// Find data interface
	FNiagaraSystemInstanceControllerPtr SystemInstanceController = NiagaraComponent->GetSystemInstanceController();
	if ( SystemInstanceController.IsValid() == false )
	{
		return nullptr;
	}

	FNiagaraSystemInstance* SystemInstance = SystemInstanceController->GetSoloSystemInstance();
	for (auto EmitterInstance : SystemInstance->GetEmitters())
	{
		if ( FNiagaraComputeExecutionContext* ComputeContext = EmitterInstance->GetGPUContext() )
		{
			for (const FNiagaraVariableWithOffset& Variable : ComputeContext->CombinedParamStore.ReadParameterVariables())
			{
				if (Variable.IsDataInterface() && (Variable.GetName() == DataInterfaceName))
				{
					return ComputeContext->CombinedParamStore.GetDataInterface(Variable.Offset);
				}
			}
		}
	}
	return nullptr;
}

void FNiagaraBakerOutputBindingHelper::GetDataInterfaceBindingsForCanvas(TArray<FNiagaraBakerOutputBinding>& OutBindings, UNiagaraSystem* NiagaraSystem)
{
	check(NiagaraSystem);

	ForEachEmitterDataInterface(
		NiagaraSystem,
		[&](const FString& EmitterName, const FString& VariableName, UNiagaraDataInterface* DataInterface)
		{
			if (DataInterface->CanRenderVariablesToCanvas())
			{
				TArray<FNiagaraVariableBase> RendererableVariables;
				DataInterface->GetCanvasVariables(RendererableVariables);
				for (const FNiagaraVariableBase& RendererableVariable : RendererableVariables)
				{
					const FString VariableString = VariableName + TEXT(".") + RendererableVariable.GetName().ToString();

					FNiagaraBakerOutputBinding& NewBinding = OutBindings.AddDefaulted_GetRef();
					NewBinding.BindingName = FName(STRING_EmitterDI + TEXT(".") + EmitterName + TEXT(".") + VariableString);
					NewBinding.MenuCategory = FText::FromString(TEXT("DataInterface") + EmitterName);
					NewBinding.MenuEntry = FText::FromString(VariableString);
				}
			}
		}
	);
}

void FNiagaraBakerOutputBindingHelper::GetParticleAttributeBindings(TArray<FNiagaraBakerOutputBinding>& OutBindings, UNiagaraSystem* NiagaraSystem)
{
	check(NiagaraSystem);

	const TArray<TSharedRef<const FNiagaraEmitterCompiledData>>& AllEmitterCompiledData = NiagaraSystem->GetEmitterCompiledData();

	for (int32 EmitterIndex = 0; EmitterIndex < NiagaraSystem->GetEmitterHandles().Num(); ++EmitterIndex)
	{
		const FNiagaraEmitterHandle& EmitterHandle = NiagaraSystem->GetEmitterHandle(EmitterIndex);
		FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetInstance().GetEmitterData();
		if (!EmitterHandle.IsValid() || !EmitterHandle.GetIsEnabled() || !EmitterData)
		{
			continue;
		}

		const FString EmitterName = EmitterHandle.GetName().ToString();

		if (ensure(AllEmitterCompiledData.IsValidIndex(EmitterIndex)))
		{
			const FNiagaraDataSetCompiledData& ParticleDataSet = AllEmitterCompiledData[EmitterIndex]->DataSetCompiledData;
			for (int32 iVariable = 0; iVariable < ParticleDataSet.VariableLayouts.Num(); ++iVariable)
			{
				const FNiagaraVariableBase& Variable = ParticleDataSet.Variables[iVariable];
				const FNiagaraVariableLayoutInfo& VariableLayout = ParticleDataSet.VariableLayouts[iVariable];
				if (VariableLayout.GetNumFloatComponents() > 0)
				{
					const FString VariableString = Variable.GetName().ToString();

					FNiagaraBakerOutputBinding& NewBinding = OutBindings.AddDefaulted_GetRef();
					NewBinding.BindingName = FName(STRING_EmitterParticles + TEXT(".") + EmitterName + TEXT(".") + VariableString);
					NewBinding.MenuCategory = FText::FromString(TEXT("ParticleAttribute ") + EmitterName);
					NewBinding.MenuEntry = FText::FromString(VariableString);
				}
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FNiagaraBakerRenderer::FNiagaraBakerRenderer(UNiagaraSystem* InNiagaraSystem)
	: NiagaraSystem(InNiagaraSystem)
{
}

FNiagaraBakerRenderer::~FNiagaraBakerRenderer()
{
	using namespace NiagaraBakerRendererPrivate;
	DestroyPreviewScene(PreviewComponent, AdvancedPreviewScene);
	DestroyPreviewScene(SimCachePreviewComponent, SimCacheAdvancedPreviewScene);
	DestroyPreviewScene(StaticMeshPreviewComponent, StaticMeshPreviewScene);
	DestroyPreviewScene(SVTPreviewComponent, SVTPreviewScene);
}

void FNiagaraBakerRenderer::SetAbsoluteTime(float AbsoluteTime, bool bShouldTickComponent)
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	if ( !ensure(BakerSettings) || !PreviewComponent )
	{
		return;
	}

	if (!PreviewComponent->IsActive() && (AbsoluteTime < PreviewComponent->GetDesiredAge()))
	{
		PreviewComponent->ReinitializeSystem();
	}

	PreviewComponent->SetSeekDelta(BakerSettings->GetSeekDelta());
	PreviewComponent->SeekToDesiredAge(AbsoluteTime);

	UWorld* World = PreviewComponent->GetWorld();
	if (ensure(World))
	{
		World->TimeSeconds			= AbsoluteTime;
		World->UnpausedTimeSeconds	= AbsoluteTime;
		World->RealTimeSeconds		= AbsoluteTime;
		World->DeltaRealTimeSeconds	= BakerSettings->GetSeekDelta();
		World->DeltaTimeSeconds		= BakerSettings->GetSeekDelta();
		World->Tick(ELevelTick::LEVELTICK_PauseTick, 0.0f);
	}

	if (bShouldTickComponent)
	{
		PreviewComponent->TickComponent(BakerSettings->GetSeekDelta(), ELevelTick::LEVELTICK_All, nullptr);

		// World should be guaranteed but let's be safe
		if ( World )
		{
			// Send EOF updates before we flush our pending ticks to ensure everything is ready for Niagara
			World->SendAllEndOfFrameUpdates();

			if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
			{
				WorldManager->FlushComputeAndDeferredQueues(false);
			}
		}
	}
}

void FNiagaraBakerRenderer::RenderSceneCapture(UTextureRenderTarget2D* RenderTarget, ESceneCaptureSource CaptureSource) const
{
	RenderSceneCapture(RenderTarget, GetPreviewComponent(), CaptureSource);
}

void FNiagaraBakerRenderer::RenderSceneCapture(UTextureRenderTarget2D* RenderTarget, UPrimitiveComponent* BakedDataComponent, ESceneCaptureSource CaptureSource) const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	if (!BakedDataComponent || !RenderTarget || !BakerSettings)
	{
		return;
	}

	const float WorldTime = GetWorldTime();
	UWorld* World = BakedDataComponent->GetWorld();

	FCanvas Canvas(RenderTarget->GameThread_GetRenderTargetResource(), nullptr, FGameTime::CreateUndilated(WorldTime, FApp::GetDeltaTime()), GetFeatureLevel());
	Canvas.Clear(FLinearColor::Black);

	// Lazy create the component to avoid it always being present in every Niagara editor
	if (SceneCaptureComponent == nullptr)
	{
		SceneCaptureComponent = NewObject<USceneCaptureComponent2D>(GetTransientPackage(), NAME_None, RF_Transient);
		SceneCaptureComponent->bTickInEditor = false;
		SceneCaptureComponent->SetComponentTickEnabled(false);
		SceneCaptureComponent->SetVisibility(true);
		SceneCaptureComponent->bCaptureEveryFrame = false;
		SceneCaptureComponent->bCaptureOnMovement = false;
	}

	SceneCaptureComponent->RegisterComponentWithWorld(World);
	SceneCaptureComponent->TextureTarget = RenderTarget;
	SceneCaptureComponent->CaptureSource = CaptureSource;

	// Set view location
	const FNiagaraBakerCameraSettings& CurrentCamera = BakerSettings->GetCurrentCamera();
	if (CurrentCamera.IsOrthographic())
	{
		SceneCaptureComponent->ProjectionType = ECameraProjectionMode::Orthographic;
		SceneCaptureComponent->OrthoWidth = CurrentCamera.OrthoWidth;
	}
	else
	{
		SceneCaptureComponent->ProjectionType = ECameraProjectionMode::Perspective;
		SceneCaptureComponent->FOVAngle = CurrentCamera.FOV;
	}

	const FMatrix SceneCaptureMatrix = FMatrix(FPlane(0, 0, 1, 0), FPlane(1, 0, 0, 0), FPlane(0, 1, 0, 0), FPlane(0, 0, 0, 1));
	FMatrix ViewMatrix = SceneCaptureMatrix * BakerSettings->GetViewportMatrix().Inverse() * FRotationTranslationMatrix(BakerSettings->GetCameraRotation(), BakerSettings->GetCameraLocation());
	SceneCaptureComponent->SetWorldLocationAndRotation(ViewMatrix.GetOrigin(), ViewMatrix.Rotator());

	SceneCaptureComponent->bUseCustomProjectionMatrix = true;
	SceneCaptureComponent->CustomProjectionMatrix = BakerSettings->GetProjectionMatrix();

	if (BakerSettings->bRenderComponentOnly)
	{
		const TArray<TObjectPtr<USceneComponent>>& AttachChildren = BakedDataComponent->GetAttachChildren();

		SceneCaptureComponent->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
		SceneCaptureComponent->ShowOnlyComponents.Empty(1 + AttachChildren.Num());
		SceneCaptureComponent->ShowOnlyComponents.Add(BakedDataComponent);
		for (TWeakObjectPtr<USceneComponent> WeakChildComponent : AttachChildren)
		{
			if (UPrimitiveComponent* ChildComponent = Cast<UPrimitiveComponent>(WeakChildComponent.Get()))
			{
				SceneCaptureComponent->ShowOnlyComponents.Add(ChildComponent);
			}
		}
	}
	else
	{
		SceneCaptureComponent->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_RenderScenePrimitives;
	}

	SceneCaptureComponent->CaptureScene();

	SceneCaptureComponent->TextureTarget = nullptr;
	SceneCaptureComponent->UnregisterComponent();

	// Alpha from a scene capture is 1- so we need to invert
	if (SceneCaptureComponent->CaptureSource == ESceneCaptureSource::SCS_SceneColorHDR)
	{
		FCanvasTileItem TileItem(FVector2D(0, 0), FVector2D(RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceHeight()), FLinearColor::White);
		TileItem.BlendMode = SE_BLEND_Opaque;
		TileItem.BatchedElementParameters = new FBatchedElementNiagaraInvertColorChannel(0);
		Canvas.DrawItem(TileItem);
	}
	Canvas.Flush_GameThread();
}

void FNiagaraBakerRenderer::RenderBufferVisualization(UTextureRenderTarget2D* RenderTarget, FName BufferVisualizationMode) const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	if (!RenderTarget || !BakerSettings)
	{
		return;
	}

	const FIntRect ViewRect = FIntRect(0, 0, RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceHeight());
	const float WorldTime = GetWorldTime();
	UWorld* World = GetWorld();

	FCanvas Canvas(RenderTarget->GameThread_GetRenderTargetResource(), nullptr, FGameTime::CreateUndilated(WorldTime, FApp::GetDeltaTime()), GetFeatureLevel());
	Canvas.Clear(FLinearColor::Black);

	// Create View Family
	FSceneViewFamilyContext ViewFamily(
		FSceneViewFamily::ConstructionValues(RenderTarget->GameThread_GetRenderTargetResource(), World->Scene, FEngineShowFlags(ESFIM_Game))
		.SetTime(FGameTime::CreateUndilated(WorldTime, FApp::GetDeltaTime()))
	);
	
	ViewFamily.EngineShowFlags.SetScreenPercentage(false);
	//ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
	//ViewFamily.EngineShowFlags.MotionBlur = 0;
	//ViewFamily.EngineShowFlags.SetDistanceCulledPrimitives(true); // show distance culled objects
	//ViewFamily.EngineShowFlags.SetPostProcessing(false);
	
	if (BufferVisualizationMode.IsValid())
	{
		ViewFamily.EngineShowFlags.SetPostProcessing(true);
		ViewFamily.EngineShowFlags.SetVisualizeBuffer(true);
		ViewFamily.EngineShowFlags.SetTonemapper(false);
		ViewFamily.EngineShowFlags.SetScreenPercentage(false);
	}
	
	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.ViewOrigin = BakerSettings->GetCameraLocation();
	ViewInitOptions.ViewRotationMatrix = BakerSettings->GetViewMatrix();
	ViewInitOptions.ProjectionMatrix = BakerSettings->GetProjectionMatrix();
	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	if (BakerSettings->bRenderComponentOnly)
	{
		ViewInitOptions.ShowOnlyPrimitives.Emplace();
		ViewInitOptions.ShowOnlyPrimitives->Add(GetPreviewComponent()->GetPrimitiveSceneId());
	}
	
	FSceneView* NewView = new FSceneView(ViewInitOptions);
	NewView->CurrentBufferVisualizationMode = BufferVisualizationMode;
	ViewFamily.Views.Add(NewView);
	
	ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(ViewFamily, 1.0f));
	
	GetRendererModule().BeginRenderingViewFamily(&Canvas, &ViewFamily);

	Canvas.Flush_GameThread();
}

void FNiagaraBakerRenderer::RenderDataInterface(UTextureRenderTarget2D* RenderTarget, FName BindingName) const
{
	const float WorldTime = GetWorldTime();
	FCanvas Canvas(RenderTarget->GameThread_GetRenderTargetResource(), nullptr, FGameTime::CreateUndilated(WorldTime, FApp::GetDeltaTime()), GetFeatureLevel());
	Canvas.Clear(FLinearColor::Black);

	ON_SCOPE_EXIT{ Canvas.Flush_GameThread(); };

	// Gather data interface / attribute name
	FString SourceString = BindingName.ToString();
	int32 DotIndex;
	if (!SourceString.FindLastChar('.', DotIndex))
	{
		return;
	}

	const FName DataInterfaceName = FName(SourceString.LeftChop(SourceString.Len() - DotIndex));
	const FName VariableName = FName(SourceString.RightChop(DotIndex + 1));

	// Find data interface
	FNiagaraSystemInstanceControllerPtr SystemInstanceController = GetPreviewComponent()->GetSystemInstanceController();
	if ( SystemInstanceController.IsValid() == false )
	{
		return;
	}

	FNiagaraSystemInstance* SystemInstance = SystemInstanceController->GetSoloSystemInstance();
	const FNiagaraSystemInstanceID SystemInstanceID = SystemInstance->GetId();
	for (auto EmitterInstance : SystemInstance->GetEmitters())
	{
		FNiagaraComputeExecutionContext* ExecContext = EmitterInstance->GetGPUContext();
		if ( ExecContext == nullptr )
		{
			continue;
		}

		for (const FNiagaraVariableWithOffset& Variable : ExecContext->CombinedParamStore.ReadParameterVariables())
		{
			if (Variable.IsDataInterface())
			{
				if (Variable.GetName() == DataInterfaceName)
				{
					if ( UNiagaraDataInterface* DataInterface = ExecContext->CombinedParamStore.GetDataInterface(Variable.Offset) )
					{
						const FIntRect ViewRect(0, 0, RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceHeight());
						DataInterface->RenderVariableToCanvas(SystemInstanceID, VariableName, &Canvas, ViewRect);
						return;
					}
				}
			}
		}
	}
}

void FNiagaraBakerRenderer::RenderParticleAttribute(UTextureRenderTarget2D* RenderTarget, FName BindingName) const
{
	const float WorldTime = GetWorldTime();
	FCanvas Canvas(RenderTarget->GameThread_GetRenderTargetResource(), nullptr, FGameTime::CreateUndilated(WorldTime, FApp::GetDeltaTime()), GetFeatureLevel());
	Canvas.Clear(FLinearColor::Black);

	ON_SCOPE_EXIT { Canvas.Flush_GameThread(); };

	FString SourceString = BindingName.ToString();
	int32 DotIndex;
	if ( !SourceString.FindChar('.', DotIndex) )
	{
		return;
	}
	
	const FString EmitterName = SourceString.LeftChop(SourceString.Len() - DotIndex);
	const FName AttributeName = FName(SourceString.RightChop(DotIndex + 1));
	
	FNiagaraSystemInstanceControllerPtr SystemInstanceController = GetPreviewComponent()->GetSystemInstanceController();
	if (!ensure(SystemInstanceController.IsValid()))
	{
		return;
	}
	
	FNiagaraSystemInstance* SystemInstance = SystemInstanceController->GetSoloSystemInstance();
	if ( !ensure(SystemInstance) )
	{
		return;
	}
	
	for ( const auto& EmitterInstance : SystemInstance->GetEmitters() )
	{
		const UNiagaraEmitter* NiagaraEmitter = EmitterInstance->GetEmitter();
		if ( !NiagaraEmitter || (NiagaraEmitter->GetUniqueEmitterName() != EmitterName) )
		{
			continue;
		}
	
		if (EmitterInstance->GetGPUContext() != nullptr)
		{
			return;
		}
	
		const FNiagaraDataSet& ParticleDataSet = EmitterInstance->GetParticleData();
		const FNiagaraDataBuffer* ParticleDataBuffer = ParticleDataSet.GetCurrentData();
		FNiagaraDataSetReaderInt32<int32> UniqueIDAccessor = FNiagaraDataSetAccessor<int32>::CreateReader(ParticleDataSet, FName("UniqueID"));
		if ( !ParticleDataBuffer || !UniqueIDAccessor.IsValid() )
		{
			return;
		}
	
		const int32 VariableIndex = ParticleDataSet.GetCompiledData().Variables.IndexOfByPredicate([&AttributeName](const FNiagaraVariable& Variable) { return Variable.GetName() == AttributeName; });
		if (VariableIndex == INDEX_NONE)
		{
			return;
		}
		const FNiagaraVariableLayoutInfo& VariableInfo = ParticleDataSet.GetCompiledData().VariableLayouts[VariableIndex];
	
		const float* FloatChannels[4];
		FloatChannels[0] = reinterpret_cast<const float*>(ParticleDataBuffer->GetComponentPtrFloat(VariableInfo.GetFloatComponentStart()));
		FloatChannels[1] = VariableInfo.GetNumFloatComponents() > 1 ? reinterpret_cast<const float*>(ParticleDataBuffer->GetComponentPtrFloat(VariableInfo.GetFloatComponentStart() + 1)) : nullptr;
		FloatChannels[2] = VariableInfo.GetNumFloatComponents() > 2 ? reinterpret_cast<const float*>(ParticleDataBuffer->GetComponentPtrFloat(VariableInfo.GetFloatComponentStart() + 2)) : nullptr;
		FloatChannels[3] = VariableInfo.GetNumFloatComponents() > 3 ? reinterpret_cast<const float*>(ParticleDataBuffer->GetComponentPtrFloat(VariableInfo.GetFloatComponentStart() + 3)) : nullptr;
	
		const FIntPoint RenderTargetSize(RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceHeight());
		const int32 ParticleBufferStore = RenderTargetSize.X * RenderTargetSize.Y;
		for ( uint32 i=0; i < ParticleDataBuffer->GetNumInstances(); ++i )
		{
			const int32 UniqueID = UniqueIDAccessor[i];
			if (UniqueID >= ParticleBufferStore)
			{
				continue;
			}
	
			FLinearColor OutputColor;
			OutputColor.R = FloatChannels[0] ? FloatChannels[0][i] : 0.0f;
			OutputColor.G = FloatChannels[1] ? FloatChannels[1][i] : 0.0f;
			OutputColor.B = FloatChannels[2] ? FloatChannels[2][i] : 0.0f;
			OutputColor.A = FloatChannels[3] ? FloatChannels[3][i] : 0.0f;
	
			const int32 TexelX = UniqueID % RenderTargetSize.X;
			const int32 TexelY = UniqueID / RenderTargetSize.X;
			Canvas.DrawTile(TexelX, TexelY, 1.0f, 1.0f, 0.0f, 0.0f, 1.0f, 1.0f, OutputColor);
		}

		// We are done
		break;
	}
}

void FNiagaraBakerRenderer::RenderSimCache(UTextureRenderTarget2D* RenderTarget, UNiagaraSimCache* SimCache) const
{
	using namespace NiagaraBakerRendererPrivate;

	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	if (!SimCache)
	{
		return;
	}

	if (SimCachePreviewComponent == nullptr)
	{
		CreatePreviewScene(NiagaraSystem, SimCachePreviewComponent, SimCacheAdvancedPreviewScene);
	}

	const float SeekDelta = BakerSettings->GetSeekDelta();

	SimCachePreviewComponent->SetSimCache(SimCache);
	SimCachePreviewComponent->SetSeekDelta(SeekDelta);
	SimCachePreviewComponent->SeekToDesiredAge(GetWorldTime());
	SimCachePreviewComponent->TickComponent(SeekDelta, ELevelTick::LEVELTICK_All, nullptr);

	SimCachePreviewComponent->MarkRenderDynamicDataDirty();
	UWorld* World = SimCachePreviewComponent->GetWorld();
	World->SendAllEndOfFrameUpdates();

	RenderSceneCapture(RenderTarget, SimCachePreviewComponent, ESceneCaptureSource::SCS_SceneColorHDR);

	SimCachePreviewComponent->SetSimCache(nullptr);
}

void FNiagaraBakerRenderer::RenderSparseVolumeTexture(UTextureRenderTarget2D* RenderTarget, const FNiagaraBakerOutputFrameIndices Indices, UAnimatedSparseVolumeTexture* SVT) const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	if (!SVT)
	{
		return;
	}

	if (SVTPreviewComponent == nullptr)
	{
		SVTPreviewComponent = NewObject<UHeterogeneousVolumeComponent>(GetTransientPackage(), NAME_None, RF_Transient);
				
		// create HV component and wire all the things
		UMaterialInterface* MaterialInterface = LoadObject<UMaterialInterface>(nullptr, TEXT("/Engine/EngineMaterials/SparseVolumeMaterial"));
		
		UMaterial *Mat = MaterialInterface->GetMaterial();

		// #todo(dmp): we had to duplicate the material itself because we cannot make a mid and send that to HV
		// HV internally makes a MID from whatever is bound, and the MID of a MID workflow appears broken	
		UMaterial* DuplicateMat = DuplicateObject<UMaterial>(Mat, SVTPreviewComponent);
		DuplicateMat->ClearFlags(RF_Standalone);

		FGuid ExprGuid;
		DuplicateMat->SetStaticComponentMaskParameterValueEditorOnly("Temperature Mask", false, true, false, false, ExprGuid);

		FGuid SwitchGuid;
		DuplicateMat->SetStaticSwitchParameterValueEditorOnly("Temperature (Attributes B)", false, SwitchGuid);

		DuplicateMat->SetSparseVolumeTextureParameterValueEditorOnly("SparseVolumeTexture", SVT);

		SVTPreviewComponent->OverrideMaterials.Add(DuplicateMat);

		SVTPreviewComponent->bIssueBlockingRequests = true;
		SVTPreviewComponent->ConditionalPostLoad();

		SVTPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));
		SVTPreviewScene->SetFloorVisibility(false);
		SVTPreviewScene->AddComponent(SVTPreviewComponent, SVTPreviewComponent->GetRelativeTransform());
	}
	
	TArray<FMaterialParameterInfo> ParameterInfo;
	TArray<FGuid> ParameterIds;
	SVTPreviewComponent->OverrideMaterials[0]->GetAllSparseVolumeTextureParameterInfo(ParameterInfo, ParameterIds);
	USparseVolumeTexture* OldSVT;
	SVTPreviewComponent->OverrideMaterials[0]->GetSparseVolumeTextureParameterValue(ParameterInfo[0], OldSVT);
	
	if (OldSVT != SVT)
	{
		SVTPreviewComponent->OverrideMaterials[0]->GetMaterial()->SetSparseVolumeTextureParameterValueEditorOnly("SparseVolumeTexture", SVT);
	}
	
	int32 NumFrames = SVT->GetNumFrames();
	if (NumFrames == 0)
	{		
		return;
	}

	// do mod if we are previewing the looped result and the timeline doesn't line up with the baked number of frames for the looped
	// sim.  This will at least loop it, but there could be a frame pop when the timeline resets
	SVTPreviewComponent->SetFrame(Indices.FrameIndexA % SVT->GetNumFrames());
	
	// #todo(dmp): apply world scale and pivot to HV actor

	const float SeekDelta = BakerSettings->GetSeekDelta();
	SVTPreviewComponent->TickComponent(SeekDelta, ELevelTick::LEVELTICK_All, nullptr);

	SVTPreviewComponent->MarkRenderDynamicDataDirty();
	UWorld* World = SVTPreviewComponent->GetWorld();
	World->SendAllEndOfFrameUpdates();

	RenderSceneCapture(RenderTarget, SVTPreviewComponent, ESceneCaptureSource::SCS_SceneColorHDR);
}

void FNiagaraBakerRenderer::RenderStaticMesh(UTextureRenderTarget2D* RenderTarget, UStaticMesh* StaticMesh) const
{
	using namespace NiagaraBakerRendererPrivate;

	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	if (!StaticMesh)
	{
		return;
	}

	if (StaticMeshPreviewComponent == nullptr)
	{
		CreatePreviewScene<UStaticMeshComponent>(StaticMeshPreviewComponent, StaticMeshPreviewScene);
	}

	StaticMeshPreviewComponent->SetStaticMesh(StaticMesh);

	UWorld* World = StaticMeshPreviewComponent->GetWorld();
	World->SendAllEndOfFrameUpdates();

	RenderSceneCapture(RenderTarget, StaticMeshPreviewComponent, ESceneCaptureSource::SCS_SceneColorHDR);

	StaticMeshPreviewComponent->SetStaticMesh(nullptr);
}

UWorld* FNiagaraBakerRenderer::GetWorld() const
{
	return GetPreviewComponent()->GetWorld();
}

float FNiagaraBakerRenderer::GetWorldTime() const
{
	return GetPreviewComponent()->GetDesiredAge();
}

ERHIFeatureLevel::Type FNiagaraBakerRenderer::GetFeatureLevel() const
{
	return GetPreviewComponent()->GetWorld()->Scene->GetFeatureLevel();
}

UNiagaraComponent* FNiagaraBakerRenderer::GetPreviewComponent() const
{
	if (PreviewComponent == nullptr)
	{
		using namespace NiagaraBakerRendererPrivate;
		CreatePreviewScene(NiagaraSystem, PreviewComponent, AdvancedPreviewScene);
	}

	return PreviewComponent;
}

UNiagaraSystem* FNiagaraBakerRenderer::GetNiagaraSystem() const
{
	return NiagaraSystem;
}

void FNiagaraBakerRenderer::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(NiagaraSystem);
	Collector.AddReferencedObject(PreviewComponent);
	Collector.AddReferencedObject(SceneCaptureComponent);
	Collector.AddReferencedObject(SimCachePreviewComponent);
	Collector.AddReferencedObject(SVTPreviewComponent);
}

FNiagaraBakerOutputRenderer* FNiagaraBakerRenderer::GetOutputRenderer(UClass* Class)
{
	return FNiagaraBakerOutputRegistry::Get().GetRendererForClass(Class);
}

bool FNiagaraBakerRenderer::ExportImage(FStringView FilePath, FIntPoint ImageSize, TArrayView<FFloat16Color> ImageData)
{
	const FString FileExtension = FPaths::GetExtension(FilePath.GetData(), true);
	const EImageFormat ImageFormat = ImageWrapperHelper::GetImageFormat(FileExtension);
	if (ImageFormat == EImageFormat::Invalid)
	{
		return false;
	}

	IImageWrapperModule & ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>("ImageWrapper");
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);
	if ( ImageWrapper.IsValid() == false )
	{
		return false;
	}

	if ( ImageFormat == EImageFormat::EXR || ImageFormat == EImageFormat::HDR )
	{
		TArray<FLinearColor> TempImageData;
		TempImageData.Reserve(ImageData.Num());
		for (const FFloat16Color& HalfColor : ImageData)
		{
			TempImageData.Emplace(HalfColor.GetFloats());
		}

		if (ImageWrapper->SetRaw(TempImageData.GetData(), TempImageData.Num() * TempImageData.GetTypeSize(), ImageSize.X, ImageSize.Y, ERGBFormat::RGBAF, 32) == false)
		{
			return false;
		}
	}
	else
	{
		TArray<FColor> TempImageData;
		TempImageData.Reserve(ImageData.Num());
		for (const FFloat16Color& HalfColor : ImageData)
		{
			TempImageData.Add(HalfColor.GetFloats().ToFColor(true));
		}

		if (ImageWrapper->SetRaw(TempImageData.GetData(), TempImageData.Num() * TempImageData.GetTypeSize(), ImageSize.X, ImageSize.Y, ERGBFormat::BGRA, 8) == false)
		{
			return false;
		}
	}

	const TArray64<uint8> TempData = ImageWrapper->GetCompressed();
	return FFileHelper::SaveArrayToFile(TempData, FilePath.GetData());
}

bool FNiagaraBakerRenderer::ExportVolume(FStringView FilePath, FIntVector ImageSize, TArrayView<FFloat16Color> ImageData)
{
	const FString FileExtension = FPaths::GetExtension(FilePath.GetData(), true);
	if (FileExtension == TEXT(".vdb"))
	{
		UE_LOGF(LogNiagaraEditor, Warning, "Exporting vdb grids from the Niagara Baker is no longer supported.");

		return false;
	}
	else
	{
		return ExportImage(FilePath, FIntPoint(ImageSize.X, ImageSize.Y * ImageSize.Z), ImageData);
	}
}

bool FVolumeDataInterfaceHelper::Initialize(const TArray<FString>& InputDataInterfacePath, UNiagaraComponent* InNiagaraComponent)
{
	NiagaraComponent = InNiagaraComponent;

	DataInterfacePath = InputDataInterfacePath;
	
	if (DataInterfacePath.Num() < 2)
	{
		return false;
	}

	const FName DataInterfaceName(DataInterfacePath[0] + "." + DataInterfacePath[1]);
	UNiagaraDataInterface* DataInterface = FNiagaraBakerOutputBindingHelper::GetDataInterface(NiagaraComponent, DataInterfaceName);
	if (DataInterface == nullptr)
	{
		return false;
	}

	// Guaranteed since we got a data interface
	SystemInstance = NiagaraComponent->GetSystemInstanceController()->GetSoloSystemInstance();

	// Render Target Volume
	if (DataInterface->IsA<UNiagaraDataInterfaceRenderTargetVolume>())
	{
		VolumeRenderTargetDataInterface = CastChecked<UNiagaraDataInterfaceRenderTargetVolume>(DataInterface);
		VolumeRenderTargetProxy = static_cast<FNiagaraDataInterfaceProxyRenderTargetVolumeProxy*>(VolumeRenderTargetDataInterface->GetProxy());
		VolumeRenderTargetInstanceData_GameThread = static_cast<FRenderTargetVolumeRWInstanceData_GameThread*>(SystemInstance->FindDataInterfaceInstanceData(VolumeRenderTargetDataInterface));
		if (VolumeRenderTargetInstanceData_GameThread == nullptr)
		{
			return false;
		}
	}
	// Grid 3D
	else if (DataInterface->IsA<UNiagaraDataInterfaceGrid3DCollection>())
	{
		Grid3DDataInterface = CastChecked<UNiagaraDataInterfaceGrid3DCollection>(DataInterface);
		Grid3DProxy = static_cast<FNiagaraDataInterfaceProxyGrid3DCollectionProxy*>(Grid3DDataInterface->GetProxy());
		Grid3DInstanceData_GameThread = static_cast<FGrid3DCollectionRWInstanceData_GameThread*>(SystemInstance->FindDataInterfaceInstanceData(Grid3DDataInterface));
		if (Grid3DInstanceData_GameThread == nullptr)
		{
			return false;
		}

		if (DataInterfacePath.Num() != 3)
		{
			// Perhaps a path to pull all attributes, i.e. whole texture?
			return false;
		}

		Grid3DAttributeName = FName(DataInterfacePath[2]);
		Grid3DVariableIndex = Grid3DInstanceData_GameThread->Vars.IndexOfByPredicate([&](const FNiagaraVariableBase& VariableBase) { return VariableBase.GetName() == Grid3DAttributeName; });
		if (Grid3DVariableIndex == INDEX_NONE)
		{
			return false;
		}
		Grid3DAttributeStart = Grid3DInstanceData_GameThread->Offsets[Grid3DVariableIndex];
		Grid3DAttributeChannels = Grid3DInstanceData_GameThread->Vars[Grid3DVariableIndex].GetType().GetSize() / sizeof(float);
		Grid3DTextureSize.X = Grid3DInstanceData_GameThread->NumCells.X * Grid3DInstanceData_GameThread->NumTiles.X;
		Grid3DTextureSize.Y = Grid3DInstanceData_GameThread->NumCells.Y * Grid3DInstanceData_GameThread->NumTiles.Y;
		Grid3DTextureSize.Z = Grid3DInstanceData_GameThread->NumCells.Z * Grid3DInstanceData_GameThread->NumTiles.Z;
	}
	// Unsupported type
	else
	{
		return false;
	}

	return true;
}