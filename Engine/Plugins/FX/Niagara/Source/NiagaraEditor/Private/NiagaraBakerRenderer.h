// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "NiagaraBakerOutput.h"
#include "NiagaraBakerSettings.h"
#include "NiagaraSystem.h"
#include "UObject/GCObject.h"

DECLARE_LOG_CATEGORY_EXTERN(LogNiagaraBaker, Log, All);

class FNiagaraBakerRenderer;
class UNiagaraComponent;
class UNiagaraBakerSettings;
class UNiagaraSimCache;
class UTextureRenderTarget2D;
class USceneCaptureComponent2D;
class FAdvancedPreviewScene;
class FCanvas;
class UAnimatedSparseVolumeTexture;
class UHeterogeneousVolumeComponent;
class UStaticMesh;
class UStaticMeshComponent;

struct FNiagaraBakerFeedbackContext
{
	bool HasIssues() const { return Errors.Num() + Warnings.Num() > 0; }

	TArray<FString>	Errors;
	TArray<FString>	Warnings;
};

struct FNiagaraBakerOutputBinding
{
	FName	BindingName;
	FText	MenuCategory;
	FText	MenuEntry;
};

struct FNiagaraBakerOutputBindingHelper
{
	typedef TFunction<void(const FString& EmitterName, const FString& VariableName, UNiagaraDataInterface* DataInterface)> FEmitterDIFunction;

	enum class ERenderType
	{
		None,
		SceneCapture,
		BufferVisualization,
		DataInterface,
		Particle
	};

	static const FString STRING_SceneCaptureSource;
	static const FString STRING_BufferVisualization;
	static const FString STRING_EmitterDI;
	static const FString STRING_EmitterParticles;

	static ERenderType GetRenderType(FName BindingName, FName& OutName);

	static void ForEachEmitterDataInterface(UNiagaraSystem* NiagaraSystem, FEmitterDIFunction Function);
	static UNiagaraDataInterface* GetDataInterface(UNiagaraComponent* NiagaraComponent, FName DataInterfaceName);

	static void GetSceneCaptureBindings(TArray<FNiagaraBakerOutputBinding>& OutBindings);
	static void GetBufferVisualizationBindings(TArray<FNiagaraBakerOutputBinding>& OutBindings);
	static void GetDataInterfaceBindingsForCanvas(TArray<FNiagaraBakerOutputBinding>& OutBindings, UNiagaraSystem* NiagaraSystem);
	static void GetParticleAttributeBindings(TArray<FNiagaraBakerOutputBinding>& OutBindings, UNiagaraSystem* NiagaraSystem);
};

class FNiagaraBakerOutputRenderer
{
public:
	FNiagaraBakerOutputRenderer() {}
	virtual ~FNiagaraBakerOutputRenderer() = default;

	/**
	Creates a list of all possible renderer bindings for the output, i.e. which sources you can pull from
	*/
	virtual TArray<FNiagaraBakerOutputBinding> GetRendererBindings(UNiagaraBakerOutput* InBakerOutput) const { return TArray<FNiagaraBakerOutputBinding>(); }

	/**
	Get the size we want to render the preview into.
	Returning a negative or zero area will result in the preview being considered invalid
	*/
	virtual FIntPoint GetPreviewSize(UNiagaraBakerOutput* BakerOutput, FIntPoint InAvailableSize) const { return FIntPoint::ZeroValue; }

	/**
	Capture a preview of the baker output to the render target.
	The render target will already be sized based on the result from GetPreviewSize
	*/
	virtual void RenderPreview(UNiagaraBakerOutput* BakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const = 0;

	virtual FIntPoint GetGeneratedSize(UNiagaraBakerOutput* BakerOutput, FIntPoint InAvailableSize) const { return FIntPoint::ZeroValue; }
	virtual void RenderGenerated(UNiagaraBakerOutput* BakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const = 0;

	virtual bool BeginBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput) = 0;
	virtual void BakeFrame(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput, int FrameIndex, const FNiagaraBakerRenderer& BakerRenderer) = 0;
	virtual void EndBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput) = 0;
};

class FNiagaraBakerRenderer : FGCObject
{
public:
	FNiagaraBakerRenderer(UNiagaraSystem* NiagaraSystem);
	virtual ~FNiagaraBakerRenderer() override;

	void SetAbsoluteTime(float AbsoluteTime, bool bShouldTickComponent = true);

	void RenderSceneCapture(UTextureRenderTarget2D* RenderTarget, ESceneCaptureSource CaptureSource) const;
	void RenderSceneCapture(UTextureRenderTarget2D* RenderTarget, UPrimitiveComponent* BakedDataComponent, ESceneCaptureSource CaptureSource) const;
	void RenderBufferVisualization(UTextureRenderTarget2D* RenderTarget, FName BufferVisualizationMode = NAME_None) const;
	void RenderDataInterface(UTextureRenderTarget2D* RenderTarget, FName BindingName) const;
	void RenderParticleAttribute(UTextureRenderTarget2D* RenderTarget, FName BindingName) const;
	void RenderSimCache(UTextureRenderTarget2D* RenderTarget, UNiagaraSimCache* SimCache) const;
	void RenderSparseVolumeTexture(UTextureRenderTarget2D* RenderTarget, const FNiagaraBakerOutputFrameIndices Indices, UAnimatedSparseVolumeTexture *SVT) const;
	void RenderStaticMesh(UTextureRenderTarget2D* RenderTarget, UStaticMesh* StaticMesh) const;

	UWorld* GetWorld() const;
	float GetWorldTime() const;
	ERHIFeatureLevel::Type GetFeatureLevel() const;
	UNiagaraComponent* GetPreviewComponent() const;
	UNiagaraSystem* GetNiagaraSystem() const;
	UNiagaraBakerSettings* GetBakerSettings() const { return NiagaraSystem->GetBakerSettings(); }
	const UNiagaraBakerSettings* GetBakerGeneratedSettings() const { return NiagaraSystem->GetBakerGeneratedSettings(); }

	// FGCObject Impl
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraBakerRenderer");
	}
	// FGCObject Impl

	static FNiagaraBakerOutputRenderer* GetOutputRenderer(UClass* Class);

	static bool ExportImage(FStringView FilePath, FIntPoint ImageSize, TArrayView<FFloat16Color> ImageData);
	static bool ExportVolume(FStringView FilePath, FIntVector ImageSize, TArrayView<FFloat16Color> ImageData);

private:
	TObjectPtr<UNiagaraSystem> NiagaraSystem = nullptr;
	mutable TObjectPtr<UNiagaraComponent> PreviewComponent = nullptr;
	mutable TSharedPtr<FAdvancedPreviewScene> AdvancedPreviewScene;
	mutable TObjectPtr<USceneCaptureComponent2D> SceneCaptureComponent = nullptr;

	mutable TObjectPtr<UNiagaraComponent> SimCachePreviewComponent = nullptr;
	mutable TSharedPtr<FAdvancedPreviewScene> SimCacheAdvancedPreviewScene;

	mutable TObjectPtr<UStaticMeshComponent> StaticMeshPreviewComponent = nullptr;
	mutable TSharedPtr<FAdvancedPreviewScene> StaticMeshPreviewScene;

	mutable TObjectPtr<UHeterogeneousVolumeComponent> SVTPreviewComponent = nullptr;
	mutable TSharedPtr<FAdvancedPreviewScene> SVTPreviewScene;
};

class UNiagaraComponent;
class FNiagaraSystemInstance;
class UNiagaraDataInterfaceGrid3DCollection;
struct FNiagaraDataInterfaceProxyGrid3DCollectionProxy;
struct FGrid3DCollectionRWInstanceData_GameThread;
class UNiagaraDataInterfaceRenderTargetVolume;
struct FNiagaraDataInterfaceProxyRenderTargetVolumeProxy;
struct FRenderTargetVolumeRWInstanceData_GameThread;

class FVolumeDataInterfaceHelper
{
public:

	FVolumeDataInterfaceHelper() {};

	UNiagaraComponent* NiagaraComponent = nullptr;
	FNiagaraSystemInstance* SystemInstance = nullptr;
	TArray<FString>											DataInterfacePath;

	UNiagaraDataInterfaceGrid3DCollection* Grid3DDataInterface = nullptr;
	FNiagaraDataInterfaceProxyGrid3DCollectionProxy* Grid3DProxy = nullptr;
	FGrid3DCollectionRWInstanceData_GameThread* Grid3DInstanceData_GameThread = nullptr;
	FName													Grid3DAttributeName;
	int32													Grid3DVariableIndex = INDEX_NONE;
	int32													Grid3DAttributeStart = INDEX_NONE;
	int32													Grid3DAttributeChannels = 0;
	FIntVector												Grid3DTextureSize = FIntVector::ZeroValue;

	UNiagaraDataInterfaceRenderTargetVolume* VolumeRenderTargetDataInterface = nullptr;
	FNiagaraDataInterfaceProxyRenderTargetVolumeProxy* VolumeRenderTargetProxy = nullptr;
	FRenderTargetVolumeRWInstanceData_GameThread* VolumeRenderTargetInstanceData_GameThread = nullptr;

	bool Initialize(const TArray<FString>& InputDataInterfacePath, UNiagaraComponent* InNiagaraComponent);
};

