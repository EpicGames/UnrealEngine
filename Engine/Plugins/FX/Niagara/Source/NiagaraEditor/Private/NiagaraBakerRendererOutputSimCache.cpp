// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerRendererOutputSimCache.h"
#include "NiagaraBakerOutputSimCache.h"

#include "NiagaraBakerSettings.h"
#include "NiagaraComponent.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraSimCacheFactoryNew.h"
#include "NiagaraSystemInstance.h"

TArray<FNiagaraBakerOutputBinding> FNiagaraBakerRendererOutputSimCache::GetRendererBindings(UNiagaraBakerOutput* InBakerOutput) const
{
	return TArray<FNiagaraBakerOutputBinding>();
}

FIntPoint FNiagaraBakerRendererOutputSimCache::GetPreviewSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	return InAvailableSize;
}

void FNiagaraBakerRendererOutputSimCache::RenderPreview(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	BakerRenderer.RenderSceneCapture(RenderTarget, ESceneCaptureSource::SCS_SceneColorHDR);
}

FIntPoint FNiagaraBakerRendererOutputSimCache::GetGeneratedSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	return InAvailableSize;
}

void FNiagaraBakerRendererOutputSimCache::RenderGenerated(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	static FString SimCacheNotFoundError(TEXT("SimCache asset not found.\nPlease bake to see the result."));

	UNiagaraBakerOutputSimCache* BakerOutput = CastChecked<UNiagaraBakerOutputSimCache>(InBakerOutput);
	UNiagaraBakerSettings* BakerSettings = BakerOutput->GetTypedOuter<UNiagaraBakerSettings>();

	UNiagaraSimCache* SimCache = BakerOutput->GetAsset<UNiagaraSimCache>(BakerOutput->SimCacheAssetPathFormat, 0);
	if (SimCache == nullptr)
	{
		OutErrorString = SimCacheNotFoundError;
		return;
	}

	BakerRenderer.RenderSimCache(RenderTarget, SimCache);
}

bool FNiagaraBakerRendererOutputSimCache::BeginBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
	UNiagaraBakerOutputSimCache* BakerOutput = CastChecked<UNiagaraBakerOutputSimCache>(InBakerOutput);
	UNiagaraBakerSettings* BakerSettings = BakerOutput->GetTypedOuter<UNiagaraBakerSettings>();
	check(BakerSettings && BakerOutput);

	const FString AssetFullName = BakerOutput->GetAssetPath(BakerOutput->SimCacheAssetPathFormat, 0);
	BakeSimCache = UNiagaraBakerOutput::GetOrCreateAsset<UNiagaraSimCache, UNiagaraSimCacheFactoryNew>(AssetFullName);
	if (BakeSimCache == nullptr)
	{
		return false;
	}

	BakeSimCache->AddToRoot();

	return true;
}

void FNiagaraBakerRendererOutputSimCache::BakeFrame(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput, int FrameIndex, const FNiagaraBakerRenderer& BakerRenderer)
{
	UNiagaraBakerOutputSimCache* BakerOutput = CastChecked<UNiagaraBakerOutputSimCache>(InBakerOutput);
	if (BakeSimCache == nullptr)
	{
		return;
	}

	UNiagaraComponent* PreviewComponent = BakerRenderer.GetPreviewComponent();
	if (PreviewComponent && !PreviewComponent->IsComplete())
	{
		FNiagaraSimCacheFeedbackContext SimCacheFeedbackContext(false);
		if ( FrameIndex == 0 )
		{
			BakeSimCache->BeginWrite(BakerOutput->CreateParameters, BakerRenderer.GetPreviewComponent(), SimCacheFeedbackContext);
		}
		BakeSimCache->WriteFrame(BakerRenderer.GetPreviewComponent(), SimCacheFeedbackContext);

		FeedbackContext.Errors.Append(SimCacheFeedbackContext.Errors);
		FeedbackContext.Warnings.Append(SimCacheFeedbackContext.Warnings);
	}
}

void FNiagaraBakerRendererOutputSimCache::EndBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
	UNiagaraBakerOutputSimCache* BakerOutput = CastChecked<UNiagaraBakerOutputSimCache>(InBakerOutput);
	if (BakeSimCache == nullptr)
	{
		return;
	}

	BakeSimCache->EndWrite(true);

	BakeSimCache->RemoveFromRoot();
}
