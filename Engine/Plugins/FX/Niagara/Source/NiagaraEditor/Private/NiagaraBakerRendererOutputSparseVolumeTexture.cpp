// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerRendererOutputSparseVolumeTexture.h"
#include "NiagaraBakerOutputSparseVolumeTexture.h"

#include "NiagaraBakerSettings.h"
#include "NiagaraComponent.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraDataInterfaceGrid3DCollection.h"
#include "NiagaraDataInterfaceRenderTargetVolume.h"
#include "NiagaraBatchedElements.h"
#include "NiagaraSVTShaders.h"
#include "NiagaraShader.h"

#include "Engine/Canvas.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/VolumeTexture.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/PathViews.h"
#include "Modules/ModuleManager.h"
#include "Factories/VolumeTextureFactory.h"
#include "TextureResource.h"
#include "UObject/UObjectGlobals.h"
#include "RenderGraphUtils.h"

#include "TextureRenderTargetVolumeResource.h"
#include "Engine/TextureRenderTargetVolume.h"

#include "Editor/SparseVolumeTexture/Public/SparseVolumeTextureFactory.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "SparseVolumeTexture/SparseVolumeTextureData.h"
#include "SparseVolumeTexture/ISparseVolumeTextureStreamingManager.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Factories/Factory.h"
#include "IAssetTools.h"
#include "Misc/MessageDialog.h"
#include "AssetToolsModule.h"
#endif //WITH_EDITOR

TArray<FNiagaraBakerOutputBinding> FNiagaraBakerRendererOutputSparseVolumeTexture::GetRendererBindings(UNiagaraBakerOutput* InBakerOutput) const
{
	TArray<FNiagaraBakerOutputBinding> OutBindings;
	if (UNiagaraSystem* NiagaraSystem = InBakerOutput->GetTypedOuter<UNiagaraSystem>())
	{
		FNiagaraBakerOutputBindingHelper::ForEachEmitterDataInterface(
			NiagaraSystem,
			[&](const FString& EmitterName, const FString& VariableName, UNiagaraDataInterface* DataInterface)
			{
				if (UNiagaraDataInterfaceGrid3DCollection* Grid3D = Cast<UNiagaraDataInterfaceGrid3DCollection>(DataInterface))
				{
					TArray<FNiagaraVariableBase> GridVariables;
					TArray<uint32> GridVariableOffsets;
					int32 NumAttribChannelsFound;
					Grid3D->FindAttributes(GridVariables, GridVariableOffsets, NumAttribChannelsFound);

					for (const FNiagaraVariableBase& GridVariable : GridVariables)
					{
						const FString GridVariableString = GridVariable.GetName().ToString();

						FNiagaraBakerOutputBinding& NewBinding = OutBindings.AddDefaulted_GetRef();
						NewBinding.BindingName = FName(EmitterName + "." + VariableName + "." + GridVariableString);
						NewBinding.MenuCategory = FText::FromString(EmitterName + " " + TEXT("Grid3DCollection"));
						NewBinding.MenuEntry = FText::FromString(VariableName + "." + GridVariableString);
					}
				}
				else if (UNiagaraDataInterfaceRenderTargetVolume* VolumeRT = Cast<UNiagaraDataInterfaceRenderTargetVolume>(DataInterface))
				{
					FNiagaraBakerOutputBinding& NewBinding = OutBindings.AddDefaulted_GetRef();
					NewBinding.BindingName = FName(EmitterName + "." + VariableName);
					NewBinding.MenuCategory = FText::FromString(EmitterName + " " + TEXT("VolumeRenderTarget"));
					NewBinding.MenuEntry = FText::FromString(VariableName);
				}
			}
		);
	}
	return OutBindings;
}

FIntPoint FNiagaraBakerRendererOutputSparseVolumeTexture::GetPreviewSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	return InAvailableSize;
}

void FNiagaraBakerRendererOutputSparseVolumeTexture::RenderPreview(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	BakerRenderer.RenderSceneCapture(RenderTarget, ESceneCaptureSource::SCS_SceneColorHDR);
}

FIntPoint FNiagaraBakerRendererOutputSparseVolumeTexture::GetGeneratedSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	return InAvailableSize;
}

void FNiagaraBakerRendererOutputSparseVolumeTexture::RenderGenerated(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	static FString SVTNotFoundError(TEXT("Sparse Volume Texture asset not found.\nPlease bake to see the result."));
	static FString LoopedSVTNotFoundError(TEXT("Looped Sparse Volume Texture asset not found.\reverting to full frame range baked result."));

	UNiagaraBakerOutputSparseVolumeTexture* BakerOutput = CastChecked<UNiagaraBakerOutputSparseVolumeTexture>(InBakerOutput);
	UNiagaraBakerSettings* BakerSettings = BakerRenderer.GetBakerSettings();

	UAnimatedSparseVolumeTexture* SVT = nullptr;
	
	bool bPreviewLoopedOutput = true;
	if (BakerOutput->bEnableLoopedOutput && BakerSettings->bPreviewLoopedOutput)
	{
		SVT = BakerOutput->GetAsset<UAnimatedSparseVolumeTexture>(BakerOutput->LoopedSparseVolumeTextureAssetPathFormat, 0);
		if (SVT == nullptr)
		{
			OutErrorString = SVTNotFoundError;			
		}
	}

	if (SVT == nullptr)
	{
		SVT = BakerOutput->GetAsset<UAnimatedSparseVolumeTexture>(BakerOutput->SparseVolumeTextureAssetPathFormat, 0);
		if (SVT == nullptr)
		{
			OutErrorString = SVTNotFoundError;
			return;
		}
	}

	const float WorldTime = BakerRenderer.GetWorldTime();
	FCanvas Canvas(RenderTarget->GameThread_GetRenderTargetResource(), nullptr, FGameTime::CreateUndilated(WorldTime, FApp::GetDeltaTime()), BakerRenderer.GetFeatureLevel());

	const FNiagaraBakerOutputFrameIndices FrameIndices = BakerSettings->GetOutputFrameIndices(BakerOutput, WorldTime);

	BakerRenderer.RenderSparseVolumeTexture(RenderTarget, FrameIndices, SVT);
}

bool FNiagaraBakerRendererOutputSparseVolumeTexture::BeginBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
	UNiagaraBakerOutputSparseVolumeTexture* OutputVolumeTexture = CastChecked<UNiagaraBakerOutputSparseVolumeTexture>(InBakerOutput);

	const FString AssetFullName = OutputVolumeTexture->GetAssetPath(OutputVolumeTexture->SparseVolumeTextureAssetPathFormat, 0);
	
	
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	TArray<FAssetData> FoundAssets;
	bool FoundAsset = false;
	if (AssetRegistry->GetAssetsByPackageName(FName(AssetFullName), FoundAssets))
	{
		if (FoundAssets.Num() > 0)
		{
			if (UObject* ExistingOject = StaticLoadObject(UAnimatedSparseVolumeTexture::StaticClass(), nullptr, *AssetFullName))
			{
				FoundAsset = true;
			}
		}
	}	

	UNiagaraBakerOutputSparseVolumeTexture* BakerOutput = CastChecked<UNiagaraBakerOutputSparseVolumeTexture>(InBakerOutput);
	UAnimatedSparseVolumeTexture* SVT = BakerOutput->GetAsset<UAnimatedSparseVolumeTexture>(BakerOutput->SparseVolumeTextureAssetPathFormat, 0);

	if (SVT == nullptr)
	{
		SVTAsset = UNiagaraBakerOutput::GetOrCreateAsset<UAnimatedSparseVolumeTexture, USparseVolumeTextureFactory>(AssetFullName);
	}
	else
	{				
		SVTAsset = NewObject<UAnimatedSparseVolumeTexture>(SVT->GetOuter(), UAnimatedSparseVolumeTexture::StaticClass(), *SVT->GetName(), RF_Public | RF_Standalone);
		SVTAsset->PostEditChange();				
	}
	
	if (!SVTAsset->BeginInitialize(1))
	{
		UE_LOGF(LogNiagaraBaker, Error, "Cannot initialize SVT for baking");
		return false;
	}

	return true;
}

void FNiagaraBakerRendererOutputSparseVolumeTexture::BakeFrame(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput, int FrameIndex, const FNiagaraBakerRenderer& BakerRenderer)
{
	UNiagaraBakerOutputSparseVolumeTexture* OutputVolumeTexture = CastChecked<UNiagaraBakerOutputSparseVolumeTexture>(InBakerOutput);

	FVolumeDataInterfaceHelper DataInterface;

	TArray<FString> DataInterfacePath;
	OutputVolumeTexture->SourceBinding.SourceName.ToString().ParseIntoArray(DataInterfacePath, TEXT("."));
	if ( DataInterface.Initialize(DataInterfacePath, BakerRenderer.GetPreviewComponent()) == false )
	{
		return;
	}
	
	FNiagaraDataInterfaceProxyRenderTargetVolumeProxy* RT_Proxy_RenderTargetVolume = nullptr;
	FNiagaraDataInterfaceProxyGrid3DCollectionProxy* RT_Proxy_Grid3D = nullptr;
	if (DataInterface.VolumeRenderTargetDataInterface != nullptr)
	{					
		RT_Proxy_RenderTargetVolume = DataInterface.VolumeRenderTargetProxy;		
	}
	else if (DataInterface.Grid3DDataInterface != nullptr)
	{
		if (!DataInterface.Grid3DInstanceData_GameThread->UseRGBATexture)
		{
			UE_LOGF(LogNiagaraBaker, Error, "Cannot bake SVTs from non RGBA Grid3D Collections");
		}		
		RT_Proxy_Grid3D = DataInterface.Grid3DProxy;	
	}
	else
	{
		UE_LOGF(LogNiagaraBaker, Error, "Cannot bake from data interface");
	}
					
	//-OPT: Currently we are flushing rendering commands.  Do not remove this until making access to the frame data safe across threads.
	TArray<uint8> TextureData;
	FIntVector VolumeResolution = FIntVector(-1, -1, -1);
	EPixelFormat VolumeFormat = EPixelFormat::PF_A1;

	ENQUEUE_RENDER_COMMAND(NDIRenderTargetVolume_CacheFrame)
		(
			[RT_Proxy_RenderTargetVolume, RT_Proxy_Grid3D, RT_InstanceID = DataInterface.SystemInstance->GetId(), 
			RT_TextureData = &TextureData, RT_VolumeResolution = &VolumeResolution, RT_VolumeFormat = &VolumeFormat](FRHICommandListImmediate& RHICmdList)
			{
				FRHIGPUTextureReadback RenderTargetReadback("ReadVolumeTexture");
				uint32 BlockBytes = -1;				

				if (RT_Proxy_RenderTargetVolume)
				{
					if (const FRenderTargetVolumeRWInstanceData_RenderThread* InstanceData_RT = RT_Proxy_RenderTargetVolume->SystemInstancesToProxyData_RT.Find(RT_InstanceID))
					{
						*RT_VolumeResolution = InstanceData_RT->Size;
						RenderTargetReadback.EnqueueCopy(RHICmdList, InstanceData_RT->RenderTarget->GetRHI(), FIntVector(0, 0, 0), 0, *RT_VolumeResolution);
						BlockBytes = GPixelFormats[InstanceData_RT->RenderTarget->GetRHI()->GetFormat()].BlockBytes;
						*RT_VolumeFormat = InstanceData_RT->RenderTarget->GetRHI()->GetFormat();
					}
					else
					{
						UE_LOGF(LogNiagaraBaker, Error, "No valid volume RT DI to do readback from");
						return;
					}
				}
				else if (RT_Proxy_Grid3D)
				{
					if (const FGrid3DCollectionRWInstanceData_RenderThread* InstanceData_RT = RT_Proxy_Grid3D->SystemInstancesToProxyData_RT.Find(RT_InstanceID))
					{
						if (InstanceData_RT->CurrentData)
						{
							*RT_VolumeResolution = InstanceData_RT->NumCells;
							RenderTargetReadback.EnqueueCopy(RHICmdList, InstanceData_RT->CurrentData->GetPooledTexture()->GetRHI(), FIntVector(0, 0, 0), 0, *RT_VolumeResolution);
							BlockBytes = GPixelFormats[InstanceData_RT->CurrentData->GetPooledTexture()->GetRHI()->GetFormat()].BlockBytes;
							*RT_VolumeFormat = InstanceData_RT->CurrentData->GetPooledTexture()->GetRHI()->GetFormat();
						}
						else
						{
							UE_LOGF(LogNiagaraBaker, Error, "No valid grid DI to do readback from");
							return;
						}
					}
					else
					{
						UE_LOGF(LogNiagaraBaker, Error, "No valid grid DI to do readback from");
						return;
					}
				}
				else
				{
					UE_LOGF(LogNiagaraBaker, Error, "No valid grid DI to do readback from");
					return;
				}

				// Sync the GPU. Unfortunately we can't use the fences because not all RHIs implement them yet.
				constexpr bool bFlushResources = true;
				RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

				//Lock the readback staging texture
				int32 RowPitchInPixels;
				int32 BufferHeight;
				const uint8* LockedData = (const uint8*)RenderTargetReadback.Lock(RowPitchInPixels, &BufferHeight);
						
				int32 Count = RT_VolumeResolution->X * RT_VolumeResolution->Y * RT_VolumeResolution->Z * BlockBytes;
				RT_TextureData->AddUninitialized(Count);

				const uint8* SliceStart = LockedData;
				for (int32 Z = 0; Z < RT_VolumeResolution->Z; ++Z)
				{
					const uint8* RowStart = SliceStart;
					for (int32 Y = 0; Y < RT_VolumeResolution->Y; ++Y)
					{
						int32 Offset = 0 + Y * RT_VolumeResolution->X + Z * RT_VolumeResolution->X * RT_VolumeResolution->Y;
						FMemory::Memcpy(RT_TextureData->GetData() + Offset * BlockBytes, RowStart, BlockBytes * RT_VolumeResolution->X);

						RowStart += RowPitchInPixels * BlockBytes;
					}

					SliceStart += BufferHeight * RowPitchInPixels * BlockBytes;
				}

				//Unlock the staging texture
				RenderTargetReadback.Unlock();					
			}
	);
	FlushRenderingCommands();

	if (TextureData.Num() > 0)
	{

#if WITH_EDITOR			
		UE::SVT::FTextureDataCreateInfo SVTCreateInfo;
		SVTCreateInfo.VirtualVolumeAABBMin = FIntVector3::ZeroValue;
		SVTCreateInfo.VirtualVolumeAABBMax = VolumeResolution;
		SVTCreateInfo.FallbackValues[0] = FVector4f(0, 0, 0, 0);
		SVTCreateInfo.FallbackValues[1] = FVector4f(0, 0, 0, 0);
		SVTCreateInfo.AttributesFormats[0] = VolumeFormat;
		SVTCreateInfo.AttributesFormats[1] = PF_Unknown;

		UE::SVT::FTextureData SparseTextureData{};
		bool Success = SparseTextureData.CreateFromDense(SVTCreateInfo, TArrayView<uint8, int64>((uint8*)TextureData.GetData(), (int64)TextureData.Num() * sizeof(TextureData[0])), TArrayView<uint8>());

		if (!Success)
		{
			UE_LOGF(LogNiagaraBaker, Error, "Cannot create SVT for data interface");

			return;
		}

		FTransform TransformToUse = FTransform::Identity;
								
		FNiagaraVariableBase& BoundWorldSizeVar = OutputVolumeTexture->VolumeWorldSpaceSizeBinding.ResolvedParameter;

		if (BoundWorldSizeVar.IsValid())
		{
			FVector3f WorldGridScale =
				BakerRenderer.GetPreviewComponent()->GetOverrideParameters().GetParameterValueOrDefault<FVector3f>(BoundWorldSizeVar, FVector3f(1, 1, 1));
			
			if (WorldGridScale.Length() < SMALL_NUMBER)
			{
				WorldGridScale = FVector3f(1, 1, 1);
			}

			FVector WorldScaleFVector(WorldGridScale.X, WorldGridScale.Y, WorldGridScale.Z);

			// scale by volume resolution to get proper world space scale rendering
			const FVector FloatResolution(VolumeResolution.X, VolumeResolution.Y, VolumeResolution.Z);
			WorldScaleFVector /= FloatResolution;
			TransformToUse.SetScale3D(WorldScaleFVector);
		}

		if (!SVTAsset->AppendFrame(SparseTextureData, TransformToUse))
		{
			UE_LOGF(LogNiagaraBaker, Error, "Cannot append frame to SVT");
		}
#endif			
	}

}

void FNiagaraBakerRendererOutputSparseVolumeTexture::EndBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
	UNiagaraBakerSettings* BakerSettings = InBakerOutput->GetTypedOuter<UNiagaraBakerSettings>();
	
	UNiagaraBakerOutputSparseVolumeTexture* OutputVolumeTexture = CastChecked<UNiagaraBakerOutputSparseVolumeTexture>(InBakerOutput);
		
	if (!SVTAsset->EndInitialize())
	{
		UE_LOGF(LogNiagaraBaker, Error, "Cannot EndInitialize on creating SVT");
	}

	SVTAsset->ConditionalPostLoad();

	// make a second pass over the data and loop it
	if (OutputVolumeTexture->bEnableLoopedOutput)
	{		
		const FString LoopedAssetFullName = OutputVolumeTexture->GetAssetPath(OutputVolumeTexture->LoopedSparseVolumeTextureAssetPathFormat, 0);

		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		TArray<FAssetData> FoundAssets;
		bool FoundAsset = false;
		if (AssetRegistry->GetAssetsByPackageName(FName(LoopedAssetFullName), FoundAssets))
		{
			if (FoundAssets.Num() > 0)
			{
				if (UObject* ExistingOject = StaticLoadObject(UAnimatedSparseVolumeTexture::StaticClass(), nullptr, *LoopedAssetFullName))
				{
					FoundAsset = true;
				}
			}
		}

		UNiagaraBakerOutputSparseVolumeTexture* BakerOutput = CastChecked<UNiagaraBakerOutputSparseVolumeTexture>(InBakerOutput);
		UAnimatedSparseVolumeTexture* LoopedSVT = BakerOutput->GetAsset<UAnimatedSparseVolumeTexture>(BakerOutput->LoopedSparseVolumeTextureAssetPathFormat, 0);

		if (LoopedSVT == nullptr)
		{
			LoopedSVTAsset = UNiagaraBakerOutput::GetOrCreateAsset<UAnimatedSparseVolumeTexture, USparseVolumeTextureFactory>(LoopedAssetFullName);
		}
		else
		{
			LoopedSVTAsset = NewObject<UAnimatedSparseVolumeTexture>(LoopedSVT->GetOuter(), UAnimatedSparseVolumeTexture::StaticClass(), *LoopedSVT->GetName(), RF_Public | RF_Standalone);
			LoopedSVTAsset->PostEditChange();
		}

		if (!LoopedSVTAsset->BeginInitialize(1))
		{
			UE_LOGF(LogNiagaraBaker, Error, "Cannot initialize looped SVT for baking");
			return;
		}

		// create dense output buffer
		const float FrameRate = BakerSettings->FramesPerSecond;
		const int32 TotalNumFrames = SVTAsset->GetNumFrames();		
		const int32 StartFrame = OutputVolumeTexture->StartTime * FrameRate;
		const int32 BlendFrames = OutputVolumeTexture->BlendDuration * FrameRate;
		const int32 LoopedFrames = TotalNumFrames - StartFrame - BlendFrames;
		const int32 BlendStartFrame = TotalNumFrames - BlendFrames;

		if (TotalNumFrames <= 0)
		{
			UE_LOGF(LogNiagaraBaker, Error, "SVT sequence to loop must have > 0 frames");
			return;
		}

		if (StartFrame < 0)
		{
			UE_LOGF(LogNiagaraBaker, Error, "Start frame must be greater than or equal to 0");
			return;
		}

		if (BlendFrames <= 0)
		{
			UE_LOGF(LogNiagaraBaker, Error, "Blend amount must be greater than 0");
			return;
		}

		if (LoopedFrames <= 0)
		{
			UE_LOGF(LogNiagaraBaker, Error, "Cannot loop SVT due to insufficient frames.  Please reduce blend amount and start time or bake more frames.");
			return;
		}

		for (int i = 0; i < LoopedFrames; ++i)
		{
			int32 OutFrameA = StartFrame + BlendFrames + i;
			float LerpAmount = FMath::Clamp((1.0 * OutFrameA - BlendStartFrame) / BlendFrames, 0.0f, 1.0f);

			int32 OutFrameB = StartFrame + 1.0 * LerpAmount * BlendFrames;

			//
			// nonblocking read SVT frames
			//			
			const int32 MipLevel = 0;
			const float SVTFrameRate = 0.0f;
			const bool bBlocking = true;
			const bool bHasFrameRate = false;

			USparseVolumeTextureFrame* SparseVolumeTextureFrameA = 
				USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(SVTAsset, GetTypeHash(InBakerOutput), SVTFrameRate, OutFrameA, MipLevel, bBlocking, bHasFrameRate);

			USparseVolumeTextureFrame* SparseVolumeTextureFrameB =
				USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(SVTAsset, GetTypeHash(InBakerOutput+12345), SVTFrameRate, OutFrameB, MipLevel, bBlocking, bHasFrameRate);

			// The streaming manager normally ticks in FDeferredShadingSceneRenderer::Render(), but the SVT->DenseTexture conversion compute shader happens in a render command before that.
			// At execution time of that command, the streamer hasn't had the chance to do any streaming yet, so we force another tick here.
			// Assuming blocking requests are used, this guarantees that the requested frame is fully streamed in (if there is memory available).
			UE::SVT::GetStreamingManager().Update_GameThread();

			if (SparseVolumeTextureFrameA == nullptr)
			{
				UE_LOGF(LogNiagaraBaker, Error, "Invalid frame from baked SVT for looping");
				return;
			}

			if (SparseVolumeTextureFrameB == nullptr)
			{
				UE_LOGF(LogNiagaraBaker, Error, "Invalid frame from baked SVT for looping");
				return;
			}

			FIntVector VolumeResolution = SparseVolumeTextureFrameA->GetVolumeResolution();
			EPixelFormat VolumeFormat = SparseVolumeTextureFrameA->GetFormat(0);
			
			//
			// Perform blend and output results
			//

			TArray<uint8> TextureData;		
			ENQUEUE_RENDER_COMMAND(NDIRenderTargetVolumeUpdate)
				(
					[RT_TextureData = &TextureData, RT_VolumeResolution = VolumeResolution, RT_VolumeFormat = VolumeFormat, RT_LerpAmount = LerpAmount,
					RT_SVTRenderResources_A = SparseVolumeTextureFrameA->GetTextureRenderResources(),
					RT_SVTRenderResources_B = SparseVolumeTextureFrameB->GetTextureRenderResources()](FRHICommandListImmediate& RHICmdList)
					{
						if (RT_SVTRenderResources_A == nullptr)
						{
							UE_LOGF(LogNiagaraBaker, Error, "Null svt resource");
							return;
						}

						if (RT_SVTRenderResources_B == nullptr)
						{
							UE_LOGF(LogNiagaraBaker, Error, "Null svt resource");
							return;
						}

						//
						// execute compute shader to output blended result
						//
						FRDGBuilder GraphBuilder(RHICmdList);
						
						TShaderMapRef<FNiagaraBlendSVTsToDenseBufferCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
						FNiagaraBlendSVTsToDenseBufferCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FNiagaraBlendSVTsToDenseBufferCS::FParameters>();

						FRDGTextureDesc TempTextureDesc = FRDGTextureDesc::Create3D(
							RT_VolumeResolution,
							RT_VolumeFormat,
							FClearValueBinding::Black,
							ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV);

						FRDGTextureRef TempTexture = GraphBuilder.CreateTexture(TempTextureDesc, TEXT("TempOutput"));
						PassParameters->DestinationBuffer = GraphBuilder.CreateUAV(TempTexture);
					
						FRHITexture* PageTableTexture_A = RT_SVTRenderResources_A->GetPageTableTexture();
						FRHITexture* TextureA_A = RT_SVTRenderResources_A->GetPhysicalTileDataATexture();
						if (PageTableTexture_A == nullptr || TextureA_A == nullptr)
						{
							UE_LOGF(LogNiagaraBaker, Error, "Null svt texture");
							return;
						}

						// build pass parameters for the "A" SVT frame
						FUintVector4 CurrentPackedUniforms0_A = FUintVector4();
						FUintVector4 CurrentPackedUniforms1_A = FUintVector4();
						RT_SVTRenderResources_A->GetPackedUniforms(CurrentPackedUniforms0_A, CurrentPackedUniforms1_A);

						PassParameters->TileDataTextureSampler_A = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
						PassParameters->SparseVolumeTexturePageTable_A = PageTableTexture_A;
						PassParameters->SparseVolumeTextureA_A = TextureA_A;
						PassParameters->PackedSVTUniforms0_A = CurrentPackedUniforms0_A;
						PassParameters->PackedSVTUniforms1_A = CurrentPackedUniforms1_A;
						PassParameters->TextureSize_A = RT_VolumeResolution;
						PassParameters->MipLevels_A = 0;

						// build pass parameters for the "B" SVT frame
						FRHITexture* PageTableTexture_B = RT_SVTRenderResources_B->GetPageTableTexture();
						FRHITexture* TextureA_B = RT_SVTRenderResources_B->GetPhysicalTileDataATexture();

						FUintVector4 CurrentPackedUniforms0_B = FUintVector4();
						FUintVector4 CurrentPackedUniforms1_B = FUintVector4();
						RT_SVTRenderResources_B->GetPackedUniforms(CurrentPackedUniforms0_B, CurrentPackedUniforms1_B);

						PassParameters->TileDataTextureSampler_B = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
						PassParameters->SparseVolumeTexturePageTable_B = PageTableTexture_B;
						PassParameters->SparseVolumeTextureA_B = TextureA_B;
						PassParameters->PackedSVTUniforms0_B = CurrentPackedUniforms0_B;
						PassParameters->PackedSVTUniforms1_B = CurrentPackedUniforms1_B;
						PassParameters->TextureSize_B = RT_VolumeResolution;
						PassParameters->MipLevels_B = 0;

						PassParameters->LerpAmount = RT_LerpAmount;

						FIntVector ThreadGroupSize = FNiagaraShader::GetDefaultThreadGroupSize(ENiagaraGpuDispatchType::ThreeD);
						const FIntVector NumThreadGroups(
							FMath::DivideAndRoundUp(RT_VolumeResolution.X, ThreadGroupSize.X),
							FMath::DivideAndRoundUp(RT_VolumeResolution.Y, ThreadGroupSize.Y),
							FMath::DivideAndRoundUp(RT_VolumeResolution.Z, ThreadGroupSize.Z)
						);

						GraphBuilder.AddPass(
							// Friendly name of the pass for profilers using printf semantics.
							RDG_EVENT_NAME("Blend SVTs"),
							// Parameters provided to RDG.
							PassParameters,
							// Issues compute commands.
							ERDGPassFlags::Compute,
							// This is deferred until Execute. May execute in parallel with other passes.
							[PassParameters, ComputeShader, NumThreadGroups](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
							{
								FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *PassParameters, NumThreadGroups);
							});
			
						//
						// Readback dense texture		
						//
						FRHIGPUTextureReadback RenderTargetReadback("ReadVolumeTexture");						
						AddEnqueueCopyPass(GraphBuilder, &RenderTargetReadback, TempTexture);
						
						// Execute the graph.
						GraphBuilder.Execute();

						RHICmdList.SubmitAndBlockUntilGPUIdle();

						check(RenderTargetReadback.IsReady());

						//Lock the readback staging texture
						int32 RowPitchInPixels;
						int32 BufferHeight;
						const uint8* LockedData = (const uint8*)RenderTargetReadback.Lock(RowPitchInPixels, &BufferHeight);

						if (LockedData == nullptr)
						{
							UE_LOGF(LogNiagaraBaker, Error, "Readback failed and returned null locked data");
							return;
						}

						if (RowPitchInPixels == 0 || BufferHeight == 0)
						{
							UE_LOGF(LogNiagaraBaker, Error, "Readback failed and returned data with zero pitch/buffer height");
							return;
						}
						
						uint32 BlockBytes = GPixelFormats[RT_VolumeFormat].BlockBytes;
						int32 Count = RT_VolumeResolution.X * RT_VolumeResolution.Y * RT_VolumeResolution.Z * BlockBytes;
						RT_TextureData->AddUninitialized(Count);

						if (RT_TextureData->Num() == 0)
						{
							UE_LOGF(LogNiagaraBaker, Error, "Output looped texture data has no elements");
							return;
						}
						
						const uint8* SliceStart = LockedData;
						for (int32 Z = 0; Z < RT_VolumeResolution.Z; ++Z)
						{
							const uint8* RowStart = SliceStart;
							for (int32 Y = 0; Y < RT_VolumeResolution.Y; ++Y)
							{
								int32 Offset = 0 + Y * RT_VolumeResolution.X + Z * RT_VolumeResolution.X * RT_VolumeResolution.Y;
								FMemory::Memcpy(RT_TextureData->GetData() + Offset * BlockBytes, RowStart, BlockBytes * RT_VolumeResolution.X);

								RowStart += RowPitchInPixels * BlockBytes;
							}

							SliceStart += BufferHeight * RowPitchInPixels * BlockBytes;
						}						

						//Unlock the staging texture
						RenderTargetReadback.Unlock();					
					}					
				);

			FlushRenderingCommands();

			if (TextureData.Num() == 0)
			{
				UE_LOGF(LogNiagaraBaker, Error, "Readback failed when trying to add looped render target to SVT");
				return;
			}

			//
			// add dense texture to LoopedSVTAsset
			//
#if WITH_EDITOR			
			UE::SVT::FTextureDataCreateInfo SVTCreateInfo;
			SVTCreateInfo.VirtualVolumeAABBMin = FIntVector3::ZeroValue;
			SVTCreateInfo.VirtualVolumeAABBMax = VolumeResolution;
			SVTCreateInfo.FallbackValues[0] = FVector4f(0, 0, 0, 0);
			SVTCreateInfo.FallbackValues[1] = FVector4f(0, 0, 0, 0);
			SVTCreateInfo.AttributesFormats[0] = VolumeFormat;
			SVTCreateInfo.AttributesFormats[1] = PF_Unknown;

			UE::SVT::FTextureData SparseTextureData{};
			bool Success = SparseTextureData.CreateFromDense(SVTCreateInfo, TArrayView<uint8, int64>((uint8*)TextureData.GetData(), (int64)TextureData.Num() * sizeof(TextureData[0])), TArrayView<uint8>());

			if (!Success)
			{
				UE_LOGF(LogNiagaraBaker, Error, "Cannot create looped SVT output");
				return;
			}

			FTransform TransformToUse = SparseVolumeTextureFrameA->GetFrameTransform();

			if (!LoopedSVTAsset->AppendFrame(SparseTextureData, TransformToUse))
			{
				UE_LOGF(LogNiagaraBaker, Error, "Cannot append frame to looped SVT");
				return;
			}
#endif
		}		

		if (!LoopedSVTAsset->EndInitialize())
		{
			UE_LOGF(LogNiagaraBaker, Error, "Cannot EndInitialize on creating looped SVT");
		}

		LoopedSVTAsset->ConditionalPostLoad();
	}
}

