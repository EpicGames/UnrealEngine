// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerRendererOutputStaticMesh.h"
#include "NiagaraBakerOutputStaticMesh.h"

#include "NiagaraBakerSettings.h"
#include "NiagaraComponent.h"
#include "NiagaraGpuComputeDispatchInterface.h"
#include "NiagaraRendererReadback.h"
#include "NiagaraSystemInstance.h"

#include "Engine/StaticMesh.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MeshDescription.h"
#include "MeshAttributes.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraBakerRendererOutputStaticMesh)

UNiagaraBakerStaticMeshFactoryNew::UNiagaraBakerStaticMeshFactoryNew()
{
	SupportedClass = UStaticMesh::StaticClass();
}

UObject* UNiagaraBakerStaticMeshFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UStaticMesh>(InParent, InClass, InName, Flags);
}

TArray<FNiagaraBakerOutputBinding> FNiagaraBakerRendererOutputStaticMesh::GetRendererBindings(UNiagaraBakerOutput* InBakerOutput) const
{
	return TArray<FNiagaraBakerOutputBinding>();
}

FIntPoint FNiagaraBakerRendererOutputStaticMesh::GetPreviewSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	return InAvailableSize;
}

void FNiagaraBakerRendererOutputStaticMesh::RenderPreview(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	BakerRenderer.RenderSceneCapture(RenderTarget, ESceneCaptureSource::SCS_SceneColorHDR);
}

FIntPoint FNiagaraBakerRendererOutputStaticMesh::GetGeneratedSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	return InAvailableSize;
}

void FNiagaraBakerRendererOutputStaticMesh::RenderGenerated(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	static FString StaticMeshNotFoundError(TEXT("StaticMesh asset not found.\nPlease bake to see the result."));

	UNiagaraBakerOutputStaticMesh* BakerOutput = CastChecked<UNiagaraBakerOutputStaticMesh>(InBakerOutput);
	UNiagaraBakerSettings* BakerGeneratedSettings = BakerOutput->GetTypedOuter<UNiagaraBakerSettings>();

	const float WorldTime = BakerRenderer.GetWorldTime();
	const FNiagaraBakerOutputFrameIndices FrameIndices = BakerGeneratedSettings->GetOutputFrameIndices(BakerOutput, WorldTime);

	UStaticMesh* StaticMesh = BakerOutput->GetAsset<UStaticMesh>(BakerOutput->FramesAssetPathFormat, FrameIndices.FrameIndexA);
	if (StaticMesh == nullptr)
	{
		OutErrorString = StaticMeshNotFoundError;
		return;
	}

	BakerRenderer.RenderStaticMesh(RenderTarget, StaticMesh);
}

bool FNiagaraBakerRendererOutputStaticMesh::BeginBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
#if WITH_NIAGARA_RENDERER_READBACK
	BakeRenderTarget = NewObject<UTextureRenderTarget2D>();
	BakeRenderTarget->AddToRoot();
	BakeRenderTarget->ClearColor = FLinearColor::Transparent;
	BakeRenderTarget->TargetGamma = 1.0f;
	BakeRenderTarget->InitCustomFormat(128, 128, PF_FloatRGBA, false);

	return true;
#else
	FeedbackContext.Errors.Add(TEXT("Niagara Renderer Readback not enabled, failed to bake"));
	return false;
#endif
}

void FNiagaraBakerRendererOutputStaticMesh::BakeFrame(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput, int FrameIndex, const FNiagaraBakerRenderer& BakerRenderer)
{
	UNiagaraBakerOutputStaticMesh* BakerOutput = CastChecked<UNiagaraBakerOutputStaticMesh>(InBakerOutput);

	UNiagaraComponent* PreviewComponent = BakerRenderer.GetPreviewComponent();
	if (!PreviewComponent || PreviewComponent->IsComplete())
	{
		return;
	}

#if WITH_NIAGARA_RENDERER_READBACK
	NiagaraRendererReadback::EnqueueReadback(
		PreviewComponent,
		[BakerOutput, FrameIndex](const FNiagaraRendererReadbackResult& ReadbackResult)
		{
			// Failed or no data
			if (ReadbackResult.NumVertices == 0)
			{
				return;
			}

			// Create asset
			const FString AssetFullName = BakerOutput->GetAssetPath(BakerOutput->FramesAssetPathFormat, FrameIndex);
			UStaticMesh* StaticMesh = UNiagaraBakerOutput::GetOrCreateAsset<UStaticMesh, UNiagaraBakerStaticMeshFactoryNew>(AssetFullName);
			if (StaticMesh == nullptr)
			{
				return;
			}

			ConvertReadbackResultsToStaticMesh(ReadbackResult, StaticMesh);
		},
		BakerOutput->ExportParameters
	);

	BakerRenderer.RenderSceneCapture(BakeRenderTarget, ESceneCaptureSource::SCS_SceneColorHDR);
#endif
}

void FNiagaraBakerRendererOutputStaticMesh::EndBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
	if (BakeRenderTarget)
	{
		BakeRenderTarget->RemoveFromRoot();
		BakeRenderTarget->MarkAsGarbage();
		BakeRenderTarget = nullptr;
	}
}

bool FNiagaraBakerRendererOutputStaticMesh::ConvertReadbackResultsToStaticMesh(const FNiagaraRendererReadbackResult& ReadbackResult, UStaticMesh* StaticMesh)
{
#if WITH_NIAGARA_RENDERER_READBACK
	// Failed or no data
	if (ReadbackResult.NumVertices == 0 || StaticMesh == nullptr)
	{
		return false;
	}

	const uint32 NumTexCoords = ReadbackResult.VertexTexCoordNum;
	const uint32 NumTriangles = ReadbackResult.NumVertices / 3;

	// Create Mesh Description
	FMeshDescription MeshDescription;
	TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector3f>(MeshAttribute::VertexInstance::Normal, 1, FVector3f::ZeroVector, EMeshAttributeFlags::Mandatory);
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector3f>(MeshAttribute::VertexInstance::Tangent, 1, FVector3f::ZeroVector, EMeshAttributeFlags::Mandatory);
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = MeshDescription.VertexInstanceAttributes().RegisterAttribute<float>(MeshAttribute::VertexInstance::BinormalSign, 1, 1.0f, EMeshAttributeFlags::Mandatory);
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector4f>(MeshAttribute::VertexInstance::Color, 1, FVector4f(1.0f, 1.0f, 1.0f, 1.0f), EMeshAttributeFlags::Lerpable | EMeshAttributeFlags::Mandatory);
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate, NumTexCoords, FVector2f::ZeroVector, EMeshAttributeFlags::Lerpable | EMeshAttributeFlags::Mandatory);

	MeshDescription.EdgeAttributes().RegisterAttribute<bool>(MeshAttribute::Edge::IsHard, 1, false, EMeshAttributeFlags::Mandatory);
	TPolygonGroupAttributesRef<FName> PolygonGroupSlotNames = MeshDescription.PolygonGroupAttributes().RegisterAttribute<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName, 1, NAME_None, EMeshAttributeFlags::Mandatory); //The unique key to match the mesh material slot

	// Reserve space
	MeshDescription.ReserveNewVertices(ReadbackResult.NumVertices);
	MeshDescription.ReserveNewVertexInstances(NumTriangles);
	MeshDescription.ReserveNewEdges(NumTriangles);
	MeshDescription.ReserveNewPolygons(NumTriangles);
	MeshDescription.ReserveNewPolygonGroups(ReadbackResult.Sections.Num());

	// Build vertices
	TArray<FVertexInstanceID> VertexInstances;
	VertexInstances.Reserve(ReadbackResult.NumVertices);
	for (uint32 iVertex=0; iVertex < ReadbackResult.NumVertices; ++iVertex)
	{
		const FVertexID VertexID = MeshDescription.CreateVertex();
		check(VertexID.GetValue() == iVertex);
		VertexPositions[VertexID] = ReadbackResult.HasPosition() ? ReadbackResult.GetPosition(iVertex) : FVector3f::ZeroVector;

		FVertexInstanceID VertexInstanceID = MeshDescription.CreateVertexInstance(VertexID);
		VertexInstances.Add(VertexInstanceID);

		const FVector3f TangentX = ReadbackResult.HasTangentBasis() ? ReadbackResult.GetTangentX(iVertex) : FVector3f::XAxisVector;
		const FVector3f TangentY = ReadbackResult.HasTangentBasis() ? ReadbackResult.GetTangentY(iVertex) : FVector3f::YAxisVector;
		const FVector3f TangentZ = ReadbackResult.HasTangentBasis() ? ReadbackResult.GetTangentZ(iVertex) : FVector3f::ZAxisVector;
		const float TangentSign = FVector3f::DotProduct(FVector3f::CrossProduct(TangentX, TangentZ), TangentY) < 0.0f ? -1.0f : 1.0f;
		VertexInstanceNormals.Set(VertexInstanceID, TangentX);
		VertexInstanceTangents.Set(VertexInstanceID, TangentZ);
		VertexInstanceBinormalSigns.Set(VertexInstanceID, TangentSign);

		VertexInstanceColors.Set(VertexInstanceID, ReadbackResult.HasColor() ? ReadbackResult.GetColor(iVertex) : FLinearColor::White);
		for (uint32 iTexCoord=0; iTexCoord < NumTexCoords; ++iTexCoord)
		{
			VertexInstanceUVs.Set(VertexInstanceID, iTexCoord, ReadbackResult.GetTexCoord(iVertex, iTexCoord));
		}
	}

	// Build sections / triangles
	TArray<FStaticMaterial> StaticMaterials;
	for (int32 iSection=0; iSection < ReadbackResult.Sections.Num(); ++iSection)
	{
		const FNiagaraRendererReadbackResult::FSection& Section = ReadbackResult.Sections[iSection];

		const FName MaterialSlotName(*FString::Printf(TEXT("Section%d"), iSection));
		const FPolygonGroupID PolyGroupId = MeshDescription.CreatePolygonGroup();
		PolygonGroupSlotNames[PolyGroupId] = MaterialSlotName;

		FStaticMaterial& StaticMaterial			= StaticMaterials.AddDefaulted_GetRef();
		StaticMaterial.MaterialInterface		= Section.WeakMaterialInterface.Get();
		StaticMaterial.MaterialSlotName			= MaterialSlotName;
		StaticMaterial.ImportedMaterialSlotName	= MaterialSlotName;

		for (uint32 iTriangle=0; iTriangle < Section.NumTriangles; ++iTriangle)
		{
			const uint32 BaseIndex = (Section.FirstTriangle + iTriangle) * 3;
			const FPolygonID PolygonID = MeshDescription.CreatePolygon(PolyGroupId, { VertexInstances[BaseIndex + 0], VertexInstances[BaseIndex + 1], VertexInstances[BaseIndex + 2] });
			MeshDescription.ComputePolygonTriangulation(PolygonID);
		}
	}

	StaticMesh->SetNumSourceModels(1);
	{
		FMeshBuildSettings& MeshBuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
		MeshBuildSettings.bRecomputeNormals = false;
		MeshBuildSettings.bRecomputeTangents = false;
		MeshBuildSettings.bGenerateLightmapUVs = false;
	}

	StaticMesh->SetStaticMaterials(StaticMaterials);

	// Build Mesh
	UStaticMesh::FBuildMeshDescriptionsParams Params;
	Params.bFastBuild = WITH_EDITOR ? false : true;
	Params.bUseHashAsGuid = true;
	Params.bMarkPackageDirty = false;
	Params.bCommitMeshDescription = true;
	Params.bAllowCpuAccess = false;

	StaticMesh->BuildFromMeshDescriptions({ &MeshDescription }, Params);

	return true;
#endif
}
