// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerRendererOutputTexture2D.h"
#include "NiagaraBakerOutputTexture2D.h"

#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Factories/Texture2dFactoryNew.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "NiagaraBakerSettings.h"
#include "TextureResource.h"

TArray<FNiagaraBakerOutputBinding> FNiagaraBakerRendererOutputTexture2D::GetRendererBindings(UNiagaraBakerOutput* InBakerOutput) const
{
	TArray<FNiagaraBakerOutputBinding> OutBindings;
	FNiagaraBakerOutputBindingHelper::GetSceneCaptureBindings(OutBindings);
	FNiagaraBakerOutputBindingHelper::GetBufferVisualizationBindings(OutBindings);
	if ( UNiagaraSystem* NiagaraSystem = InBakerOutput->GetTypedOuter<UNiagaraSystem>() )
	{
		FNiagaraBakerOutputBindingHelper::GetDataInterfaceBindingsForCanvas(OutBindings, NiagaraSystem);
		FNiagaraBakerOutputBindingHelper::GetParticleAttributeBindings(OutBindings, NiagaraSystem);
	}
	return OutBindings;
}

FIntPoint FNiagaraBakerRendererOutputTexture2D::GetPreviewSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	UNiagaraBakerOutputTexture2D* BakerOutput = CastChecked<UNiagaraBakerOutputTexture2D>(InBakerOutput);
	return BakerOutput->FrameSize;
}

void FNiagaraBakerRendererOutputTexture2D::RenderPreview(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	UNiagaraBakerOutputTexture2D* BakerOutput = Cast<UNiagaraBakerOutputTexture2D>(InBakerOutput);

	FName SourceName;
	const FNiagaraBakerOutputBindingHelper::ERenderType RenderType = FNiagaraBakerOutputBindingHelper::GetRenderType(BakerOutput->SourceBinding.SourceName, SourceName);
	switch ( RenderType )
	{
		case FNiagaraBakerOutputBindingHelper::ERenderType::SceneCapture:
		{
			static UEnum* SceneCaptureSourceEnum = StaticEnum<ESceneCaptureSource>();
			const int32 SceneCaptureSource = SceneCaptureSourceEnum->GetIndexByName(SourceName);
			BakerRenderer.RenderSceneCapture(RenderTarget, SceneCaptureSource == INDEX_NONE ? ESceneCaptureSource::SCS_SceneColorHDR : ESceneCaptureSource(SceneCaptureSource));
			break;
		}
		case FNiagaraBakerOutputBindingHelper::ERenderType::BufferVisualization:
		{
			BakerRenderer.RenderBufferVisualization(RenderTarget, SourceName);
			break;
		}
		case FNiagaraBakerOutputBindingHelper::ERenderType::DataInterface:
		{
			BakerRenderer.RenderDataInterface(RenderTarget, SourceName);
			break;
		}
		case FNiagaraBakerOutputBindingHelper::ERenderType::Particle:
		{
			BakerRenderer.RenderParticleAttribute(RenderTarget, SourceName);
			break;
		}
		default:
		{
			break;
		}
	}
}

FIntPoint FNiagaraBakerRendererOutputTexture2D::GetGeneratedSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	UNiagaraBakerOutputTexture2D* BakerOutput = Cast<UNiagaraBakerOutputTexture2D>(InBakerOutput);
	return BakerOutput ? BakerOutput->FrameSize : FIntPoint::ZeroValue;
}

void FNiagaraBakerRendererOutputTexture2D::RenderGenerated(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	static FString AtlasNotFoundError(TEXT("Atlas texture not found.\nPlease bake an atlas to see the result."));

	UNiagaraBakerOutputTexture2D* BakerOutput = Cast<UNiagaraBakerOutputTexture2D>(InBakerOutput);
	UNiagaraBakerSettings* BakerGeneratedSettings = BakerOutput->GetTypedOuter<UNiagaraBakerSettings>();
	if ( BakerOutput->bGenerateAtlas == false || BakerGeneratedSettings == nullptr)
	{
		OutErrorString = AtlasNotFoundError;
		return;
	}

	UTexture2D* AtlasTexture = BakerOutput->GetAsset<UTexture2D>(BakerOutput->AtlasAssetPathFormat, 0);
	if (AtlasTexture == nullptr)
	{
		OutErrorString = AtlasNotFoundError;
		return;
	}

	const float WorldTime = BakerRenderer.GetWorldTime();
	FCanvas Canvas(RenderTarget->GameThread_GetRenderTargetResource(), nullptr, FGameTime::CreateUndilated(WorldTime, FApp::GetDeltaTime()), BakerRenderer.GetFeatureLevel());

	const FNiagaraBakerOutputFrameIndices FrameIndices = BakerGeneratedSettings->GetOutputFrameIndices(BakerOutput, WorldTime);

	const FIntPoint TextureSize = FIntPoint(FMath::Max(AtlasTexture->GetSizeX(), 1), FMath::Max(AtlasTexture->GetSizeY(), 1));
	const FIntPoint FrameSize = BakerOutput->FrameSize;
	const FIntPoint FramesPerDimension = BakerGeneratedSettings->FramesPerDimension;
	const FIntPoint FrameIndexA = FIntPoint(FrameIndices.FrameIndexA % FramesPerDimension.X, FrameIndices.FrameIndexA / FramesPerDimension.X);
	const FIntPoint FrameIndexB = FIntPoint(FrameIndices.FrameIndexB % FramesPerDimension.X, FrameIndices.FrameIndexB / FramesPerDimension.X);
	const FIntPoint FramePixelA = FIntPoint(FrameIndexA.X * FrameSize.X, FrameIndexA.Y * FrameSize.Y);
	const FIntPoint FramePixelB = FIntPoint(FrameIndexB.X * FrameSize.X, FrameIndexB.Y * FrameSize.Y);

	Canvas.DrawTile(
		0.0f, 0.0f,
		RenderTarget->GetSurfaceWidth(), RenderTarget->GetSurfaceHeight(),
		(float(FramePixelA.X) + 0.5f) / float(TextureSize.X), (float(FramePixelA.Y) + 0.5f) / float(TextureSize.Y),
		(float(FramePixelA.X + FrameSize.X) - 0.5f) / float(TextureSize.X), (float(FramePixelA.Y + FrameSize.Y) - 0.5f) / float(TextureSize.Y),
		FLinearColor::White,
		AtlasTexture->GetResource(),
		SE_BLEND_Opaque
	);

	Canvas.Flush_GameThread();
}

bool FNiagaraBakerRendererOutputTexture2D::BeginBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
	UNiagaraBakerOutputTexture2D* BakerOutput = CastChecked<UNiagaraBakerOutputTexture2D>(InBakerOutput);
	if ( !BakerOutput->bGenerateAtlas && !BakerOutput->bGenerateFrames && !BakerOutput->bExportFrames )
	{
		return false;
	}

	BakeRenderTarget = NewObject<UTextureRenderTarget2D>();
	BakeRenderTarget->AddToRoot();
	BakeRenderTarget->ClearColor = FLinearColor::Transparent;
	BakeRenderTarget->TargetGamma = 1.0f;
	BakeRenderTarget->InitCustomFormat(BakerOutput->FrameSize.X, BakerOutput->FrameSize.Y, PF_FloatRGBA, false);

	if (BakerOutput->bGenerateAtlas)
	{
		BakeAtlasTextureData.Empty();
		BakeAtlasTextureData.SetNumZeroed(BakerOutput->AtlasTextureSize.X * BakerOutput->AtlasTextureSize.Y);
	}

	return true;
}

void FNiagaraBakerRendererOutputTexture2D::BakeFrame(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput, int FrameIndex, const FNiagaraBakerRenderer& BakerRenderer)
{
	UNiagaraBakerOutputTexture2D* BakerOutput = CastChecked<UNiagaraBakerOutputTexture2D>(InBakerOutput);
	UNiagaraBakerSettings* BakerSettings = BakerRenderer.GetBakerSettings();

	TOptional<FString> ErrorString;
	RenderPreview(InBakerOutput, BakerRenderer, BakeRenderTarget, ErrorString);

	TArray<FFloat16Color> FrameResults;
	BakeRenderTarget->GameThread_GetRenderTargetResource()->ReadFloat16Pixels(FrameResults);

	const FIntPoint ViewportOffset = FIntPoint(
		BakerOutput->FrameSize.X * (FrameIndex % BakerSettings->FramesPerDimension.X),
		BakerOutput->FrameSize.Y * (FrameIndex / BakerSettings->FramesPerDimension.X)
	);

	// Are we generating an atlas
	if ( BakerOutput->bGenerateAtlas )
	{
		for (int y = 0; y < BakerOutput->FrameSize.Y; ++y)
		{
			FFloat16Color* SrcPixel = FrameResults.GetData() + (y * BakerOutput->FrameSize.X);
			FFloat16Color* DstPixel = BakeAtlasTextureData.GetData() + ViewportOffset.X + ((y + ViewportOffset.Y) * BakerOutput->AtlasTextureSize.X);
			FMemory::Memcpy(DstPixel, SrcPixel, sizeof(FFloat16Color) * BakerOutput->FrameSize.X);
		}
	}

	// Are generating frames?
	if ( BakerOutput->bGenerateFrames )
	{
		const FString AssetFullName = BakerOutput->GetAssetPath(BakerOutput->FramesAssetPathFormat, FrameIndex);
		if (UTexture2D* OutputTexture = UNiagaraBakerOutput::GetOrCreateAsset<UTexture2D, UTexture2DFactoryNew>(AssetFullName))
		{
			const bool bIsPoT = FMath::IsPowerOfTwo(BakerOutput->FrameSize.X) && FMath::IsPowerOfTwo(BakerOutput->FrameSize.Y);

			OutputTexture->PreEditChange(nullptr);
			OutputTexture->Source.Init(BakerOutput->FrameSize.X, BakerOutput->FrameSize.Y, 1, 1, TSF_RGBA16F, (const uint8*)(FrameResults.GetData()));
			OutputTexture->PowerOfTwoMode = ETexturePowerOfTwoSetting::None;
			OutputTexture->MipGenSettings = bIsPoT ? TextureMipGenSettings::TMGS_FromTextureGroup : TextureMipGenSettings::TMGS_NoMipmaps;
			OutputTexture->AddressX = BakerOutput->bSetTextureAddressX ? BakerOutput->TextureAddressX : OutputTexture->AddressX;
			OutputTexture->AddressY = BakerOutput->bSetTextureAddressY ? BakerOutput->TextureAddressY : OutputTexture->AddressY;
			OutputTexture->PostEditChange();
		}
	}

	// Are we exporting frames
	if ( BakerOutput->bExportFrames )
	{
		const FString FilePath = BakerOutput->GetExportPath(BakerOutput->FramesExportPathFormat, FrameIndex);
		FNiagaraBakerRenderer::ExportImage(FilePath, BakerOutput->FrameSize, FrameResults);
	}
}

void FNiagaraBakerRendererOutputTexture2D::EndBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
	UNiagaraBakerOutputTexture2D* BakerOutput = CastChecked<UNiagaraBakerOutputTexture2D>(InBakerOutput);

	if ( BakerOutput->bGenerateAtlas )
	{
		const FString AssetFullName = BakerOutput->GetAssetPath(BakerOutput->AtlasAssetPathFormat, 0);
		if (UTexture2D* OutputTexture = UNiagaraBakerOutput::GetOrCreateAsset<UTexture2D, UTexture2DFactoryNew>(AssetFullName))
		{
			const bool bIsPoT = FMath::IsPowerOfTwo(BakerOutput->AtlasTextureSize.X) && FMath::IsPowerOfTwo(BakerOutput->AtlasTextureSize.Y);

			OutputTexture->PreEditChange(nullptr);
			OutputTexture->Source.Init(BakerOutput->AtlasTextureSize.X, BakerOutput->AtlasTextureSize.Y, 1, 1, TSF_RGBA16F, (const uint8*)(BakeAtlasTextureData.GetData()));
			OutputTexture->PowerOfTwoMode = ETexturePowerOfTwoSetting::None;
			OutputTexture->MipGenSettings = bIsPoT ? TextureMipGenSettings::TMGS_FromTextureGroup : TextureMipGenSettings::TMGS_NoMipmaps;
			OutputTexture->AddressX = BakerOutput->bSetTextureAddressX ? BakerOutput->TextureAddressX : OutputTexture->AddressX;
			OutputTexture->AddressY = BakerOutput->bSetTextureAddressY ? BakerOutput->TextureAddressY : OutputTexture->AddressY;
			OutputTexture->UpdateResource();
			OutputTexture->PostEditChange();
			OutputTexture->MarkPackageDirty();
		}
	}

	// Clean up temporary storage
	BakeRenderTarget->RemoveFromRoot();
	BakeRenderTarget->MarkAsGarbage();
	BakeRenderTarget = nullptr;

	BakeAtlasTextureData.Empty();
}
