// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraBakerRenderer.h"

class UTextureRenderTarget2D;

class FNiagaraBakerRendererOutputTexture2D : public FNiagaraBakerOutputRenderer
{
public:
	virtual TArray<FNiagaraBakerOutputBinding> GetRendererBindings(UNiagaraBakerOutput* InBakerOutput) const override;

	virtual FIntPoint GetPreviewSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const override;
	virtual void RenderPreview(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const override;

	virtual FIntPoint GetGeneratedSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const override;
	virtual void RenderGenerated(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const override;

	virtual bool BeginBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput) override;
	virtual void BakeFrame(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput, int FrameIndex, const FNiagaraBakerRenderer& BakerRenderer) override;
	virtual void EndBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput) override;

protected:
	UTextureRenderTarget2D* BakeRenderTarget = nullptr;
	TArray<FFloat16Color> BakeAtlasTextureData;
};
