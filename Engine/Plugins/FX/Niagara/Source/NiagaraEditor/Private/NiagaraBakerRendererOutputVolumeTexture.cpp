// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBakerRendererOutputVolumeTexture.h"
#include "NiagaraBakerOutputVolumeTexture.h"

#include "NiagaraBakerSettings.h"
#include "NiagaraComponent.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraComputeExecutionContext.h"
#include "NiagaraBatchedElements.h"
#include "NiagaraDataInterfaceGrid3DCollection.h"
#include "NiagaraDataInterfaceRenderTargetVolume.h"
#include "Engine/Canvas.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/VolumeTexture.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/PathViews.h"
#include "Modules/ModuleManager.h"
#include "Factories/VolumeTextureFactory.h"
#include "TextureResource.h"
#include "UObject/UObjectGlobals.h"

#include "VolumeCache.h"
#include "VolumeCacheFactory.h"

TArray<FNiagaraBakerOutputBinding> FNiagaraBakerRendererOutputVolumeTexture::GetRendererBindings(UNiagaraBakerOutput* InBakerOutput) const
{
	TArray<FNiagaraBakerOutputBinding> OutBindings;
	if (UNiagaraSystem* NiagaraSystem = InBakerOutput->GetTypedOuter<UNiagaraSystem>())
	{
		FNiagaraBakerOutputBindingHelper::ForEachEmitterDataInterface(
			NiagaraSystem,
			[&](const FString& EmitterName, const FString& VariableName, UNiagaraDataInterface* DataInterface)
			{
				if ( UNiagaraDataInterfaceGrid3DCollection* Grid3D = Cast<UNiagaraDataInterfaceGrid3DCollection>(DataInterface) )
				{
					TArray<FNiagaraVariableBase> GridVariables;
					TArray<uint32> GridVariableOffsets;
					int32 NumAttribChannelsFound;
				 	Grid3D->FindAttributes(GridVariables, GridVariableOffsets, NumAttribChannelsFound);

					for ( const FNiagaraVariableBase& GridVariable : GridVariables)
					{
						const FString GridVariableString = GridVariable.GetName().ToString();

						FNiagaraBakerOutputBinding& NewBinding = OutBindings.AddDefaulted_GetRef();
						NewBinding.BindingName = FName(EmitterName + "." + VariableName + "." + GridVariableString);
						NewBinding.MenuCategory = FText::FromString(EmitterName + " " + TEXT("Grid3DCollection"));
						NewBinding.MenuEntry = FText::FromString(VariableName + "." + GridVariableString);
					}
				}
				else if ( UNiagaraDataInterfaceRenderTargetVolume* VolumeRT = Cast<UNiagaraDataInterfaceRenderTargetVolume>(DataInterface) )
				{
					FNiagaraBakerOutputBinding& NewBinding = OutBindings.AddDefaulted_GetRef();
					NewBinding.BindingName	= FName(EmitterName + "." + VariableName);
					NewBinding.MenuCategory	= FText::FromString(EmitterName + " " + TEXT("VolumeRenderTarget"));
					NewBinding.MenuEntry	= FText::FromString(VariableName);
				}
			}
		);
	}
	return OutBindings;
}

FIntPoint FNiagaraBakerRendererOutputVolumeTexture::GetPreviewSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	return InAvailableSize;
}

void FNiagaraBakerRendererOutputVolumeTexture::RenderPreview(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	UNiagaraBakerOutputVolumeTexture* OutputVolumeTexture = CastChecked<UNiagaraBakerOutputVolumeTexture>(InBakerOutput);

	const float WorldTime = BakerRenderer.GetWorldTime();
	FCanvas Canvas(RenderTarget->GameThread_GetRenderTargetResource(), nullptr, FGameTime::CreateUndilated(WorldTime, FApp::GetDeltaTime()), BakerRenderer.GetFeatureLevel());
	Canvas.Clear(FLinearColor::Black);
	ON_SCOPE_EXIT{ Canvas.Flush_GameThread(); };

	FVolumeDataInterfaceHelper DataInterface;

	TArray<FString> DataInterfacePath;
	OutputVolumeTexture->SourceBinding.SourceName.ToString().ParseIntoArray(DataInterfacePath, TEXT("."));
	if (DataInterface.Initialize(DataInterfacePath, BakerRenderer.GetPreviewComponent()) == false)
	{
		OutErrorString = TEXT("Could not find data to preview from.\nPlease ensure binding is set to a valid source.");
		return;
	}

	// Volume Render Target
	if ( DataInterface.VolumeRenderTargetDataInterface != nullptr )
	{
		const int32 NumSlices		= DataInterface.VolumeRenderTargetInstanceData_GameThread->Size.Z;
		const int32 SlicePerAxis	= FMath::CeilToInt(FMath::Sqrt((float)NumSlices));
		const int32 TileWidth		= RenderTarget->GetSurfaceWidth()  / SlicePerAxis;
		const int32 TileHeight		= RenderTarget->GetSurfaceHeight() / SlicePerAxis;
		if (TileWidth <= 0 && TileHeight <= 0)
		{
			return;
		}

		const int32 AttributeChannels[] = {0, 1, 2, 3};
		const FIntVector TextureSize = DataInterface.VolumeRenderTargetInstanceData_GameThread->Size;
		const FVector2f TileUVOffset = FVector2f(0.5f / float(TextureSize.X), 0.5f / float(TextureSize.Y));
		const FVector2f TileUVSize = FVector2f(float(TextureSize.X - 1) / float(TextureSize.X), float(TextureSize.Y - 1) / float(TextureSize.Y));

		for (int32 Slice=0; Slice < NumSlices; ++Slice)
		{
			const float SliceW = (float(Slice) + 0.5f) / float(NumSlices - 1);
			const FVector3f AttributeUVs[] =
			{
				FVector3f(TileUVOffset.X, TileUVOffset.Y, SliceW),
				FVector3f(TileUVOffset.X, TileUVOffset.Y, SliceW),
				FVector3f(TileUVOffset.X, TileUVOffset.Y, SliceW),
				FVector3f(TileUVOffset.X, TileUVOffset.Y, SliceW),
			};
			const int32 TileX = (Slice % SlicePerAxis) * TileWidth;
			const int32 TileY = (Slice / SlicePerAxis) * TileHeight;

			FCanvasTileItem TileItem(FVector2D(TileX, TileY), GWhiteTexture, FVector2D(TileWidth, TileHeight), FVector2D(0.0, 1.0f), FVector2D(1.0, 0.0f), FLinearColor::White);
			TileItem.BlendMode = SE_BLEND_Opaque;
			TileItem.BatchedElementParameters = new FBatchedElementNiagaraVolumeAttribute(
				FVector2f(1.0f, 1.0f),
				MakeArrayView(AttributeUVs),
				MakeArrayView(AttributeChannels),
				[Proxy_RT=DataInterface.VolumeRenderTargetProxy, SystemInstanceID_RT= DataInterface.SystemInstance->GetId()](FRHITexture*& OutTexture, FRHISamplerState*& OutSamplerState)
				{
					if ( const FRenderTargetVolumeRWInstanceData_RenderThread* InstanceData_RT = Proxy_RT->SystemInstancesToProxyData_RT.Find(SystemInstanceID_RT) )
					{
						OutTexture = InstanceData_RT->RenderTarget->GetRHI();
						OutSamplerState = InstanceData_RT->SamplerStateRHI;
					}
				}
			);
			Canvas.DrawItem(TileItem);
		}
	}
	// Grid Collection
	else if ( DataInterface.Grid3DDataInterface != nullptr )
	{
		const FVector2f TileUVSize = FVector2f(
			float(DataInterface.Grid3DInstanceData_GameThread->NumCells.X - 1) / float(DataInterface.Grid3DTextureSize.X),
			float(DataInterface.Grid3DInstanceData_GameThread->NumCells.Y - 1) / float(DataInterface.Grid3DTextureSize.Y)
		);
		TArray<FVector3f, TInlineAllocator<4>> AttributeUVs;
		for ( int32 i=0; i < DataInterface.Grid3DAttributeChannels; ++i )
		{
			const int32 AttributeIndex = DataInterface.Grid3DAttributeStart + i;
			const FIntVector AttributeTileIndex(
				AttributeIndex % DataInterface.Grid3DInstanceData_GameThread->NumTiles.X,
				(AttributeIndex / DataInterface.Grid3DInstanceData_GameThread->NumTiles.X) % DataInterface.Grid3DInstanceData_GameThread->NumTiles.Y,
				AttributeIndex / (DataInterface.Grid3DInstanceData_GameThread->NumTiles.X * DataInterface.Grid3DInstanceData_GameThread->NumTiles.Y)
			);
			FVector3f& AttributeUV = AttributeUVs.AddDefaulted_GetRef();
			AttributeUV.X = (float(AttributeTileIndex.X * DataInterface.Grid3DInstanceData_GameThread->NumCells.X) + 0.5f) / DataInterface.Grid3DTextureSize.X;
			AttributeUV.Y = (float(AttributeTileIndex.Y * DataInterface.Grid3DInstanceData_GameThread->NumCells.Y) + 0.5f) / DataInterface.Grid3DTextureSize.Y;
			AttributeUV.Z = (float(AttributeTileIndex.Z * DataInterface.Grid3DInstanceData_GameThread->NumCells.Z) + 0.5f) / DataInterface.Grid3DTextureSize.Z;
		}

		const int32 NumSlices		= DataInterface.Grid3DInstanceData_GameThread->NumCells.Z;
		const int32 SlicePerAxis	= FMath::CeilToInt(FMath::Sqrt((float)NumSlices));
		const int32 TileWidth		= RenderTarget->GetSurfaceWidth()  / SlicePerAxis;
		const int32 TileHeight		= RenderTarget->GetSurfaceHeight() / SlicePerAxis;
		if (TileWidth > 0 && TileHeight > 0)
		{
			for (int32 Slice=0; Slice < NumSlices; ++Slice)
			{
				const int32 TileX = (Slice % SlicePerAxis) * TileWidth;
				const int32 TileY = (Slice / SlicePerAxis) * TileHeight;

				TArray<FVector3f, TInlineAllocator<4>> SliceAttributeUVs;
				for ( FVector3f SliceUV : AttributeUVs )
				{
					SliceUV.Z += float(Slice) / float(DataInterface.Grid3DTextureSize.Z);
					SliceAttributeUVs.Add(SliceUV);
				}

				FCanvasTileItem TileItem(FVector2D(TileX, TileY), GWhiteTexture, FVector2D(TileWidth, TileHeight), FVector2D(0.0, 1.0f), FVector2D(1.0, 0.0f), FLinearColor::White);
				TileItem.BlendMode = SE_BLEND_Opaque;
				TileItem.BatchedElementParameters = new FBatchedElementNiagaraVolumeAttribute(
					TileUVSize,
					MakeArrayView(SliceAttributeUVs),
					MakeArrayView<int32>(nullptr, 0),
					[RT_Proxy=DataInterface.Grid3DProxy, RT_SystemInstanceID=DataInterface.SystemInstance->GetId()](FRHITexture*& OutTexture, FRHISamplerState*& OutSamplerState)
					{
						if ( const FGrid3DCollectionRWInstanceData_RenderThread* RT_InstanceData = RT_Proxy->SystemInstancesToProxyData_RT.Find(RT_SystemInstanceID) )
						{
							if ( RT_InstanceData->CurrentData != nullptr )
							{
								OutTexture = RT_InstanceData->CurrentData->GetPooledTexture()->GetRHI();
								OutSamplerState = TStaticSamplerState<SF_Bilinear>::GetRHI();
							}
						}
					}
				);
				Canvas.DrawItem(TileItem);
			}
		}
	}
}

FIntPoint FNiagaraBakerRendererOutputVolumeTexture::GetGeneratedSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const
{
	UNiagaraBakerOutputVolumeTexture* OutputVolumeTexture = CastChecked<UNiagaraBakerOutputVolumeTexture>(InBakerOutput);
	UNiagaraBakerSettings* BakerSettings = OutputVolumeTexture->GetTypedOuter<UNiagaraBakerSettings>();

	FIntPoint RequiredSize = FIntPoint::ZeroValue;
	if ( OutputVolumeTexture->bGenerateAtlas )
	{
		if ( UVolumeTexture* AtlasVolumeTexture = OutputVolumeTexture->GetAsset<UVolumeTexture>(OutputVolumeTexture->AtlasAssetPathFormat, 0) )
		{
			const FIntPoint FrameSize(
				AtlasVolumeTexture->GetSizeX() / BakerSettings->FramesPerDimension.X,
				AtlasVolumeTexture->GetSizeY() / BakerSettings->FramesPerDimension.Y
			);
			const float Ratio = FMath::Min(float(InAvailableSize.X) / float(FrameSize.X), float(InAvailableSize.Y) / float(FrameSize.Y));
			RequiredSize.X = FMath::RoundToInt(FrameSize.X * Ratio);
			RequiredSize.Y = FMath::RoundToInt(FrameSize.Y * Ratio);
		}
	}

	return RequiredSize;
}

void FNiagaraBakerRendererOutputVolumeTexture::RenderGenerated(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const
{
	static FString AtlasNotFoundError(TEXT("Atlas texture not found.\nPlease bake an atlas to see the result."));

	UNiagaraBakerOutputVolumeTexture* OutputVolumeTexture = CastChecked<UNiagaraBakerOutputVolumeTexture>(InBakerOutput);
	UNiagaraBakerSettings* BakerGeneratedSettings = OutputVolumeTexture->GetTypedOuter<UNiagaraBakerSettings>();

	if (OutputVolumeTexture->bGenerateAtlas == false || !BakerGeneratedSettings)
	{
		OutErrorString = AtlasNotFoundError;
		return;
	}

	UVolumeTexture* AtlasVolumeTexture = OutputVolumeTexture->GetAsset<UVolumeTexture>(OutputVolumeTexture->AtlasAssetPathFormat, 0);
	if (AtlasVolumeTexture == nullptr)
	{
		OutErrorString = AtlasNotFoundError;
		return;
	}

	const float WorldTime = BakerRenderer.GetWorldTime();
	FCanvas Canvas(RenderTarget->GameThread_GetRenderTargetResource(), nullptr, FGameTime::CreateUndilated(WorldTime, FApp::GetDeltaTime()), BakerRenderer.GetFeatureLevel());
	Canvas.Clear(FLinearColor::Black);

	const FIntVector AtlasTextureSize(AtlasVolumeTexture->GetSizeX(), AtlasVolumeTexture->GetSizeY(), AtlasVolumeTexture->GetSizeZ());
	if (AtlasTextureSize.X > 0 && AtlasTextureSize.Y > 0 && AtlasTextureSize.Z > 0)
	{
		const FNiagaraBakerOutputFrameIndices FrameIndices = BakerGeneratedSettings->GetOutputFrameIndices(OutputVolumeTexture, WorldTime);
		const FIntPoint TileIndex(FrameIndices.FrameIndexA % BakerGeneratedSettings->FramesPerDimension.X, FrameIndices.FrameIndexA / BakerGeneratedSettings->FramesPerDimension.X);
		const FIntPoint TileSize(AtlasVolumeTexture->GetSizeX() / BakerGeneratedSettings->FramesPerDimension.X, AtlasVolumeTexture->GetSizeY() / BakerGeneratedSettings->FramesPerDimension.Y);
		const FVector2f TileUV((float(TileSize.X * TileIndex.X) + 0.5f) / float(AtlasTextureSize.X), (float(TileSize.Y * TileIndex.Y) + 0.5f) / float(AtlasTextureSize.Y));
		const FVector2f TileUVSize(float(TileSize.X - 1) / float(AtlasTextureSize.X), float(TileSize.Y - 1) / float(AtlasTextureSize.Y));
		const int32 AttributeChannels[] = { 0, 1, 2, 3 };

		const int32 NumSlices = AtlasTextureSize.Z;
		const int32 SlicePerAxis = FMath::CeilToInt(FMath::Sqrt((float)NumSlices));
		const int32 TileWidth = RenderTarget->GetSurfaceWidth() / SlicePerAxis;
		const int32 TileHeight = RenderTarget->GetSurfaceHeight() / SlicePerAxis;
		if (TileWidth > 0 && TileHeight > 0)
		{
			for (int32 Slice=0; Slice < NumSlices; ++Slice)
			{
				const int32 TileX = (Slice % SlicePerAxis) * TileWidth;
				const int32 TileY = (Slice / SlicePerAxis) * TileHeight;

				FVector3f SliceAttributeUVs[4];
				for (int i=0; i < UE_ARRAY_COUNT(SliceAttributeUVs); ++i)
				{
					SliceAttributeUVs[i].X = TileUV.X;
					SliceAttributeUVs[i].Y = TileUV.Y;
					SliceAttributeUVs[i].Z = (float(Slice) + 0.5f) / float(AtlasTextureSize.Z);
				}

				FCanvasTileItem TileItem(FVector2D(TileX, TileY), GWhiteTexture, FVector2D(TileWidth, TileHeight), FVector2D(0.0, 1.0f), FVector2D(1.0, 0.0f), FLinearColor::White);
				TileItem.BlendMode = SE_BLEND_Opaque;
				TileItem.BatchedElementParameters = new FBatchedElementNiagaraVolumeAttribute(
					TileUVSize,
					MakeArrayView(SliceAttributeUVs),
					MakeArrayView(AttributeChannels),
					[Resource_RT=AtlasVolumeTexture->GetResource()](FRHITexture*& OutTexture, FRHISamplerState*& OutSamplerState)
					{
						if (Resource_RT)
						{
							OutTexture = Resource_RT->GetTexture3DRHI();
							OutSamplerState = Resource_RT->SamplerStateRHI;
						}
						//else
						//{
						//	OutTexture = &GBlackVolumeTexture;
						//	OutSamplerState = TStaticSamplerState<SF_Bilinear>::GetRHI();
						//}
					}
				);
				Canvas.DrawItem(TileItem);
			}
		}
	}
	Canvas.Flush_GameThread();
}

bool FNiagaraBakerRendererOutputVolumeTexture::BeginBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
	UNiagaraBakerOutputVolumeTexture* OutputVolumeTexture = CastChecked<UNiagaraBakerOutputVolumeTexture>(InBakerOutput);

	// @todo: 
	BakedFrameRange = FIntVector2(TNumericLimits<int32>::Lowest(), TNumericLimits<int32>::Lowest());

	return OutputVolumeTexture->bGenerateAtlas || OutputVolumeTexture->bGenerateFrames || OutputVolumeTexture->bExportFrames;
}

void FNiagaraBakerRendererOutputVolumeTexture::BakeFrame(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput, int FrameIndex, const FNiagaraBakerRenderer& BakerRenderer)
{
	UNiagaraBakerOutputVolumeTexture* OutputVolumeTexture = CastChecked<UNiagaraBakerOutputVolumeTexture>(InBakerOutput);

	FVolumeDataInterfaceHelper DataInterface;

	TArray<FString> DataInterfacePath;
	OutputVolumeTexture->SourceBinding.SourceName.ToString().ParseIntoArray(DataInterfacePath, TEXT("."));
	if ( DataInterface.Initialize(DataInterfacePath, BakerRenderer.GetPreviewComponent()) == false )
	{
		return;
	}

	FIntVector TextureSize = FIntVector::ZeroValue;
	TArray<FFloat16Color> TextureData;

	// Are we a volume render target?
	if ( DataInterface.VolumeRenderTargetDataInterface )
	{
		ENQUEUE_RENDER_COMMAND(CopyVolumeRenderTarget)
		(
			[Proxy_RT=DataInterface.VolumeRenderTargetProxy, SystemInstanceId_RT=DataInterface.SystemInstance->GetId(), OutTextureSize_RT=&TextureSize, OutTextureData_RT=&TextureData](FRHICommandListImmediate& RHICmdList)
			{
				if ( const FRenderTargetVolumeRWInstanceData_RenderThread* InstanceData_RT = Proxy_RT->SystemInstancesToProxyData_RT.Find(SystemInstanceId_RT) )
				{
					*OutTextureSize_RT = InstanceData_RT->Size;
					RHICmdList.Read3DSurfaceFloatData(
						InstanceData_RT->RenderTarget->GetRHI(),
						FIntRect(0, 0, InstanceData_RT->Size.X, InstanceData_RT->Size.Y),
						FIntPoint(0, InstanceData_RT->Size.Z),
						*OutTextureData_RT
					);
				}
			}
		);
		FlushRenderingCommands();
	}
	// Are we a grid collection?
	else if ( DataInterface.Grid3DDataInterface )
	{
		ENQUEUE_RENDER_COMMAND(CopyVolumeRenderTarget)
		(
			[Proxy_RT=DataInterface.Grid3DProxy, SystemInstanceId_RT=DataInterface.SystemInstance->GetId(), OutTextureSize_RT=&TextureSize, OutTextureData_RT=&TextureData](FRHICommandListImmediate& RHICmdList)
			{
				if ( const FGrid3DCollectionRWInstanceData_RenderThread* InstanceData_RT = Proxy_RT->SystemInstancesToProxyData_RT.Find(SystemInstanceId_RT) )
				{
					if (InstanceData_RT->CurrentData)
					{
						FIntVector TextureSize;
						TextureSize.X = InstanceData_RT->NumCells.X * InstanceData_RT->NumTiles.X;
						TextureSize.Y = InstanceData_RT->NumCells.Y * InstanceData_RT->NumTiles.Y;
						TextureSize.Z = InstanceData_RT->NumCells.Z * InstanceData_RT->NumTiles.Z;
						*OutTextureSize_RT = TextureSize;

						RHICmdList.Read3DSurfaceFloatData(
							InstanceData_RT->CurrentData->GetPooledTexture()->GetRHI(),
							FIntRect(0, 0, TextureSize.X, TextureSize.Y),
							FIntPoint(0, TextureSize.Z),
							*OutTextureData_RT
						);
					}
				}
			}
		);
		FlushRenderingCommands();

		if ( TextureSize == DataInterface.Grid3DTextureSize )
		{
			TArray<FFloat16Color> TileTextureData;
			const FIntVector TileTextureSize = DataInterface.Grid3DInstanceData_GameThread->NumCells;
			TileTextureData.AddUninitialized(TileTextureSize.X * TileTextureSize.Y * TileTextureSize.Z);

			FFloat16 Channels[4] = { 0.0f, 0.0f, 0.0f, 1.0f };
			int32 AttributeOffset[4] = { 0, 0, 0, 0 };
			for (int32 c=0; c < DataInterface.Grid3DAttributeChannels; ++c)
			{
				const int32 AttributeIndex = DataInterface.Grid3DAttributeStart + c;
				const FIntVector AttributeTileIndex(
					AttributeIndex % DataInterface.Grid3DInstanceData_GameThread->NumTiles.X,
					(AttributeIndex / DataInterface.Grid3DInstanceData_GameThread->NumTiles.X) % DataInterface.Grid3DInstanceData_GameThread->NumTiles.Y,
					AttributeIndex / (DataInterface.Grid3DInstanceData_GameThread->NumTiles.X * DataInterface.Grid3DInstanceData_GameThread->NumTiles.Y)
				);

				AttributeOffset[c] = AttributeTileIndex.X * TileTextureSize.X;
				AttributeOffset[c] += AttributeTileIndex.Y * TextureSize.X;
				AttributeOffset[c] += AttributeTileIndex.Z * TextureSize.X * TextureSize.Y;
			}

			int32 OutTexel = 0;
			for ( int32 z=0; z < TileTextureSize.Z; ++z )
			{
				const int32 SliceOffset = z * TextureSize.X * TextureSize.Y;
				for ( int32 y=0; y < TileTextureSize.Y; ++y )
				{
					const int32 SliceRowOffset = SliceOffset + (y * TextureSize.X);
					for ( int32 x=0; x < TileTextureSize.X; ++x )
					{
						const int32 Offset = SliceRowOffset + x;
						for ( int32 c=0; c < DataInterface.Grid3DAttributeChannels; ++c )
						{
							Channels[c] = TextureData[AttributeOffset[c] + Offset].R;
						}
						TileTextureData[OutTexel].R = Channels[0];
						TileTextureData[OutTexel].G = Channels[1];
						TileTextureData[OutTexel].B = Channels[2];
						TileTextureData[OutTexel].A = Channels[3];
						++OutTexel;
					}
				}
			}

			TextureSize = TileTextureSize;
			TextureData = TileTextureData;
		}
		else
		{
			TextureData.Empty();
		}
	}

	// Process the volume data
	if ( TextureData.Num() > 0 )
	{
		// Create atlas
		if (OutputVolumeTexture->bGenerateAtlas)
		{
			UNiagaraBakerSettings* BakerSettings = OutputVolumeTexture->GetTypedOuter<UNiagaraBakerSettings>();
			check(BakerSettings);
			if (BakeAtlasTextureData.Num() == 0)
			{
				BakeAtlasFrameSize = TextureSize;
				BakeAtlasTextureSize = FIntVector(
					TextureSize.X * BakerSettings->FramesPerDimension.X,
					TextureSize.Y * BakerSettings->FramesPerDimension.Y,
					TextureSize.Z
				);
				BakeAtlasTextureData.AddDefaulted(BakeAtlasTextureSize.X * BakeAtlasTextureSize.Y * BakeAtlasTextureSize.Z);
			}

			for ( int32 z=0; z < BakeAtlasFrameSize.Z; ++z )
			{
				for ( int32 y=0; y < BakeAtlasFrameSize.Y; ++y )
				{
					const int32 TileX = FrameIndex % BakerSettings->FramesPerDimension.X;
					const int32 TileY = FrameIndex / BakerSettings->FramesPerDimension.X;
					const int32 DstOffset =
						(BakeAtlasTextureSize.X * BakeAtlasTextureSize.Y * z) +
						(BakeAtlasFrameSize.Y * BakeAtlasTextureSize.X * TileY) +
						(BakeAtlasFrameSize.X * TileX) +
						(BakeAtlasTextureSize.X * y);
					const int32 SrcOffset =
						(TextureSize.X * TextureSize.Y * z) +
						(TextureSize.X * y);
					FMemory::Memcpy(BakeAtlasTextureData.GetData() + DstOffset, TextureData.GetData() + SrcOffset, BakeAtlasFrameSize.X * sizeof(FFloat16Color));
				}
			}
		}

		// Create asset per frame
		if (OutputVolumeTexture->bGenerateFrames)
		{
			const FString AssetFullName	= OutputVolumeTexture->GetAssetPath(OutputVolumeTexture->FramesAssetPathFormat, FrameIndex);
			if (UVolumeTexture* OutputTexture = UNiagaraBakerOutput::GetOrCreateAsset<UVolumeTexture, UVolumeTextureFactory>(AssetFullName))
			{
				OutputTexture->PreEditChange(nullptr);
				OutputTexture->Source.Init(TextureSize.X, TextureSize.Y, TextureSize.Z, 1, TSF_RGBA16F, (const uint8*)(TextureData.GetData()));
				OutputTexture->PowerOfTwoMode = ETexturePowerOfTwoSetting::None;
				OutputTexture->MipGenSettings = TextureMipGenSettings::TMGS_NoMipmaps;
				OutputTexture->PostEditChange();
			}
		}

		// Export each frame
		if (OutputVolumeTexture->bExportFrames)
		{
			BakedFrameSize = TextureSize;

			if (BakedFrameRange.X == TNumericLimits<int32>::Lowest() && BakedFrameRange.Y == TNumericLimits<int32>::Lowest())
			{
				BakedFrameRange.X = FrameIndex;
				BakedFrameRange.Y = FrameIndex;
			}
			else
			{
				BakedFrameRange.X = FMath::Min(BakedFrameRange.X, FrameIndex);
				BakedFrameRange.Y = FMath::Max(BakedFrameRange.Y, FrameIndex);
			}

			const FString FilePath = OutputVolumeTexture->GetExportPath(OutputVolumeTexture->FramesExportPathFormat, FrameIndex);
			const FString FileDirectory = FString(FPathViews::GetPath(FilePath));

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (!PlatformFile.DirectoryExists(*FileDirectory))
			{
				if (!PlatformFile.CreateDirectoryTree(*FileDirectory))
				{
					UE_LOGF(LogNiagaraBaker, Warning, "Cannot Create Directory : %ls", *FileDirectory);
					return;
				}
			}

			FNiagaraBakerRenderer::ExportVolume(FilePath, TextureSize, TextureData);
		}
	}
}

void FNiagaraBakerRendererOutputVolumeTexture::EndBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput)
{
	UNiagaraBakerOutputVolumeTexture* OutputVolumeTexture = CastChecked<UNiagaraBakerOutputVolumeTexture>(InBakerOutput);
	if (BakeAtlasTextureData.Num() > 0)
	{
		const FString AssetFullName = OutputVolumeTexture->GetAssetPath(OutputVolumeTexture->AtlasAssetPathFormat, 0);
		if (UVolumeTexture* OutputTexture = UNiagaraBakerOutput::GetOrCreateAsset<UVolumeTexture, UVolumeTextureFactory>(AssetFullName))
		{
			OutputTexture->PreEditChange(nullptr);
			OutputTexture->Source.Init(BakeAtlasTextureSize.X, BakeAtlasTextureSize.Y, BakeAtlasTextureSize.Z, 1, TSF_RGBA16F, (const uint8*)(BakeAtlasTextureData.GetData()));
			OutputTexture->PowerOfTwoMode = ETexturePowerOfTwoSetting::None;
			OutputTexture->MipGenSettings = TextureMipGenSettings::TMGS_NoMipmaps;
			OutputTexture->UpdateResource();
			OutputTexture->PostEditChange();
			OutputTexture->MarkPackageDirty();
		}

		BakeAtlasFrameSize = FIntVector::ZeroValue;
		BakeAtlasTextureSize = FIntVector::ZeroValue;
		BakeAtlasTextureData.Empty();
	}

	// @todo: move this to the right place
	if (OutputVolumeTexture->bExportFrames)
	{
		const FString AssetFullName = OutputVolumeTexture->GetAssetPath(OutputVolumeTexture->AtlasAssetPathFormat, 0);
		if (UVolumeCache* OutputCache = UNiagaraBakerOutput::GetOrCreateAsset<UVolumeCache, UVolumeCacheFactory>(AssetFullName))
		{
			FString FullPath = OutputVolumeTexture->GetExportPath(OutputVolumeTexture->FramesExportPathFormat, 0);
			FullPath = FString(FPathViews::GetPath(FullPath));

			FString FullFileName = FString(FPathViews::GetCleanFilename(OutputVolumeTexture->FramesExportPathFormat));

			FullPath = FullPath + "/" + FullFileName;

			OutputCache->FilePath = FullPath;
		
			OutputCache->FrameRangeStart = BakedFrameRange.X;
			OutputCache->FrameRangeEnd = BakedFrameRange.Y;
			OutputCache->Resolution = BakedFrameSize;
			OutputCache->CacheType = EVolumeCacheType::OpenVDB;

			OutputCache->Modify();
			OutputCache->PostEditChange();		

			if (BakedFrameRange.Y - BakedFrameRange.X > 0)
			{
				if (BakedFrameSize.X + BakedFrameSize.Y + BakedFrameSize.Z > 0)
				{
					OutputCache->InitData();
		
					// @todo: could be cool to have an option where the whole cache is read into memory after writing it so you can preview results super quick
					// bool LoadedCache = OutputCache->LoadRange();

					// if (!LoadedCache)
					// {
					// 	UE_LOGF(LogNiagaraBaker, Warning, "Could not load cache into memory: %ls", *FullPath);
					// }
				}
				else
				{
					UE_LOGF(LogNiagaraBaker, Warning, "Output cache has no voxels : %ls", *FullPath);
				}
			}
			else
			{
				UE_LOGF(LogNiagaraBaker, Warning, "No frames written : %ls", *FullPath);
			}
		}
	}
}

