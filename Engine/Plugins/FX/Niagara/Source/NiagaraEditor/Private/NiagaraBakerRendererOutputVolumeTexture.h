// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraBakerRenderer.h"

class UNiagaraBakerOutputVolumeTexture;

class FNiagaraBakerRendererOutputVolumeTexture : public FNiagaraBakerOutputRenderer
{
public:
	virtual TArray<FNiagaraBakerOutputBinding> GetRendererBindings(UNiagaraBakerOutput* InBakerOutput) const override;

	virtual FIntPoint GetPreviewSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const override;
	virtual void RenderPreview(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const override;

	virtual FIntPoint GetGeneratedSize(UNiagaraBakerOutput* InBakerOutput, FIntPoint InAvailableSize) const override;
	virtual void RenderGenerated(UNiagaraBakerOutput* InBakerOutput, const FNiagaraBakerRenderer& BakerRenderer, UTextureRenderTarget2D* RenderTarget, TOptional<FString>& OutErrorString) const override;

	virtual bool BeginBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput) override;
	virtual void BakeFrame(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput, int FrameIndex, const FNiagaraBakerRenderer& BakerRenderer) override;
	virtual void EndBake(FNiagaraBakerFeedbackContext& FeedbackContext, UNiagaraBakerOutput* InBakerOutput) override;

protected:
	FIntVector				BakeAtlasFrameSize = FIntVector::ZeroValue;
	FIntVector				BakeAtlasTextureSize = FIntVector::ZeroValue;
	TArray<FFloat16Color>	BakeAtlasTextureData;

	FIntVector				BakedFrameSize = FIntVector::ZeroValue;
	FIntVector2				BakedFrameRange = FIntVector2(-1, -1);
};
