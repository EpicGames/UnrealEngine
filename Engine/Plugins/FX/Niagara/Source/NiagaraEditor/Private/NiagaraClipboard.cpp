// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraClipboard.h"
#include "NiagaraDataInterface.h"

#include "Factories.h"
#include "UObject/UObjectMarks.h"
#include "UObject/PropertyPortFlags.h"
#include "Containers/UnrealString.h"
#include "UnrealExporter.h"
#include "Exporters/Exporter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/StringOutputDevice.h"
#include "NiagaraNodeFunctionCall.h"
#include "Engine/UserDefinedEnum.h"
#include "StructUtils/UserDefinedStruct.h"
#include "NiagaraEditorModule.h"
#include "INiagaraEditorTypeUtilities.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "PropertyHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraClipboard)

struct FNiagaraClipboardContentTextObjectFactory : public FCustomizableTextObjectFactory
{
public:
	UNiagaraClipboardContent* ClipboardContent;

public:
	FNiagaraClipboardContentTextObjectFactory()
		: FCustomizableTextObjectFactory(GWarn)
		, ClipboardContent(nullptr)
	{
	}

protected:
	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		return ObjectClass == UNiagaraClipboardContent::StaticClass();
	}

	virtual void ProcessConstructedObject(UObject* CreatedObject) override
	{
		if (CreatedObject->IsA<UNiagaraClipboardContent>())
		{
			ClipboardContent = CastChecked<UNiagaraClipboardContent>(CreatedObject);
		}
	}
};

UNiagaraClipboardFunctionInput* MakeNewInput(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, ENiagaraClipboardFunctionInputValueMode InValueMode)
{
	UNiagaraClipboardFunctionInput* NewInput = Cast<UNiagaraClipboardFunctionInput>(NewObject<UNiagaraClipboardFunctionInput>(InOuter));
	NewInput->InputName = InInputName;
	NewInput->InputType = InInputType;
	NewInput->bHasEditCondition = bInEditConditionValue.IsSet();
	NewInput->bEditConditionValue = bInEditConditionValue.Get(false);
	NewInput->ValueMode = InValueMode;
	return NewInput;
}

const UNiagaraClipboardFunctionInput* UNiagaraClipboardFunctionInput::CreateLocalValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, TArray<uint8>& InLocalValueData)
{
	checkf(InLocalValueData.Num() == InInputType.GetSize(), TEXT("Input data size (%d) didn't match type size (%d)."), InLocalValueData.Num(), InInputType.GetSize());
	UNiagaraClipboardFunctionInput* NewInput = MakeNewInput(InOuter, InInputName, InInputType, bInEditConditionValue, ENiagaraClipboardFunctionInputValueMode::Local);
	NewInput->Local = InLocalValueData;
	return NewInput;
}

const UNiagaraClipboardFunctionInput* UNiagaraClipboardFunctionInput::CreateLinkedValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, const FNiagaraVariableBase& InLinkedValue)
{
	UNiagaraClipboardFunctionInput* NewInput = MakeNewInput(InOuter, InInputName, InInputType, bInEditConditionValue, ENiagaraClipboardFunctionInputValueMode::Linked);
	NewInput->Linked = InLinkedValue;
	return NewInput;
}

const UNiagaraClipboardFunctionInput* UNiagaraClipboardFunctionInput::CreateDataValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, UNiagaraDataInterface* InDataValue)
{
	UNiagaraClipboardFunctionInput* NewInput = MakeNewInput(InOuter, InInputName, InInputType, bInEditConditionValue, ENiagaraClipboardFunctionInputValueMode::Data);
	NewInput->Data = NewObject<UNiagaraDataInterface>(NewInput, InDataValue->GetClass());
	InDataValue->CopyTo(NewInput->Data);
	return NewInput;
}

const UNiagaraClipboardFunctionInput* UNiagaraClipboardFunctionInput::CreateObjectAssetValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, UObject* InObject)
{
	UNiagaraClipboardFunctionInput* NewInput = MakeNewInput(InOuter, InInputName, InInputType, bInEditConditionValue, ENiagaraClipboardFunctionInputValueMode::ObjectAsset);
	NewInput->ObjectAsset = InObject;
	return NewInput;
}

const UNiagaraClipboardFunctionInput* UNiagaraClipboardFunctionInput::CreateExpressionValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, const FString& InExpressionValue)
{
	UNiagaraClipboardFunctionInput* NewInput = MakeNewInput(InOuter, InInputName, InInputType, bInEditConditionValue, ENiagaraClipboardFunctionInputValueMode::Expression);
	NewInput->Expression = InExpressionValue;
	return NewInput;
}

const UNiagaraClipboardFunctionInput* UNiagaraClipboardFunctionInput::CreateDynamicValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, FString InDynamicValueName, UNiagaraScript* InDynamicValue, const FGuid& InScriptVersion)
{
	UNiagaraClipboardFunctionInput* NewInput = MakeNewInput(InOuter, InInputName, InInputType, bInEditConditionValue, ENiagaraClipboardFunctionInputValueMode::Dynamic);
	NewInput->Dynamic = UNiagaraClipboardFunction::CreateScriptFunction(NewInput, InDynamicValueName, InDynamicValue, InScriptVersion);
	return NewInput;
}

const UNiagaraClipboardFunctionInput* UNiagaraClipboardFunctionInput::CreateDefaultInputValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType)
{
	UNiagaraClipboardFunctionInput* NewInput = MakeNewInput(InOuter, InInputName, InInputType, TOptional<bool>(), ENiagaraClipboardFunctionInputValueMode::ResetToDefault);
	NewInput->Local.SetNumZeroed(InInputType.GetSize());
	return NewInput;
}


bool UNiagaraClipboardFunctionInput::CopyValuesFrom(const UNiagaraClipboardFunctionInput* InOther)
{
	if (InputType != InOther->InputType)
		return false;

	ValueMode = InOther->ValueMode;
	Local = InOther->Local;
	Linked = InOther->Linked;

	Data = nullptr;
	if (InOther->Data)
		Data = Cast<UNiagaraDataInterface>(StaticDuplicateObject(InOther->Data, this));
	Expression = InOther->Expression;
	Dynamic = nullptr;
	if (InOther->Dynamic)
		Dynamic = Cast<UNiagaraClipboardFunction>(StaticDuplicateObject(InOther->Dynamic, this));

	return true;
}

const UNiagaraClipboardRenderer* UNiagaraClipboardRenderer::CreateRenderer(UObject* InOuter, UNiagaraRendererProperties* Renderer, TOptional<FNiagaraStackNoteData> StackNoteData, TOptional<FText> AlternateDisplayName)
{
	UNiagaraClipboardRenderer* NewRenderer = Cast<UNiagaraClipboardRenderer>(NewObject<UNiagaraClipboardRenderer>(InOuter));
	NewRenderer->RendererProperties = CastChecked<UNiagaraRendererProperties>(StaticDuplicateObject(Renderer, InOuter));

	if (StackNoteData.IsSet() && StackNoteData.GetValue().IsValid())
	{
		NewRenderer->StackNoteData = StackNoteData;
	}
	if (AlternateDisplayName.IsSet() && !AlternateDisplayName.GetValue().IsEmpty())
	{
		NewRenderer->AlternateDisplayName = AlternateDisplayName;
	}

	return NewRenderer;
}

UNiagaraClipboardFunction* UNiagaraClipboardFunction::CreateScriptFunction(UObject* InOuter, FString InFunctionName, UNiagaraScript* InScript, const FGuid& InScriptVersion, const TOptional<FNiagaraStackNoteData>& InStackNote)
{
	UNiagaraClipboardFunction* NewFunction = Cast<UNiagaraClipboardFunction>(NewObject<UNiagaraClipboardFunction>(InOuter));
	NewFunction->ScriptMode = ENiagaraClipboardFunctionScriptMode::ScriptAsset;
	NewFunction->FunctionName = InFunctionName;
	NewFunction->Script = InScript;
	NewFunction->ScriptVersion = InScriptVersion;

	if(InStackNote.IsSet())
	{
		NewFunction->StackNoteData = InStackNote.GetValue();
	}
	
	return NewFunction;
}

UNiagaraClipboardFunction* UNiagaraClipboardFunction::CreateAssignmentFunction(UObject* InOuter, FString InFunctionName, const TArray<FNiagaraVariable>& InAssignmentTargets, const TArray<FString>& InAssignmentDefaults, TOptional<FNiagaraStackNoteData> InStackNoteData)
{
	UNiagaraClipboardFunction* NewFunction = Cast<UNiagaraClipboardFunction>(NewObject<UNiagaraClipboardFunction>(InOuter));
	NewFunction->ScriptMode = ENiagaraClipboardFunctionScriptMode::Assignment;
	NewFunction->FunctionName = InFunctionName;
	NewFunction->AssignmentTargets = InAssignmentTargets;
	NewFunction->AssignmentDefaults = InAssignmentDefaults;

	if(InStackNoteData.IsSet())
	{
		NewFunction->StackNoteData = InStackNoteData.GetValue();
	}
	
	return NewFunction;
}

FNiagaraClipboardPortableValue FNiagaraClipboardPortableValue::CreateFromStructValue(const UScriptStruct& TargetStruct, uint8* StructMemory)
{
	FNiagaraClipboardPortableValue PortableValue;
	TargetStruct.ExportText(PortableValue.ValueString, StructMemory, nullptr, nullptr, PPF_Copy, nullptr);
	return PortableValue;
}

FNiagaraClipboardPortableValue FNiagaraClipboardPortableValue::CreateFromTypedValue(const FNiagaraTypeDefinition& InType, const FNiagaraVariant& InValue)
{
	FNiagaraClipboardPortableValue PortableValue;

	TSharedPtr<INiagaraEditorTypeUtilities> InputTypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(InType);
	if (InputTypeUtilities.IsValid() &&	InputTypeUtilities->SupportsClipboardPortableValues())
	{
		if (InputTypeUtilities->TryUpdateClipboardPortableValueFromTypedValue(InType, InValue, PortableValue) == false)
		{
			PortableValue.Reset();
		}
	}

	if (PortableValue.IsValid() == false)
	{
		if (InType.GetStruct() != nullptr &&
			InType.GetStruct()->IsA<UScriptStruct>() &&
			InType.GetSize() == InValue.GetNumBytes())
		{
			UScriptStruct* ValueStruct = CastChecked<UScriptStruct>(InType.GetStruct());
			PortableValue = CreateFromStructValue(*ValueStruct, InValue.GetBytes());
		}
	}

	if (PortableValue.IsValid() == false)
	{
		PortableValue.Reset();
	}
	return PortableValue;
}

FNiagaraClipboardPortableValue FNiagaraClipboardPortableValue::CreateFromPropertyHandle(const IPropertyHandle& InPropertyHandle)
{
	FNiagaraClipboardPortableValue PortableValue;
	FProperty* Property = InPropertyHandle.GetProperty();
	if (Property != nullptr)
	{
		UScriptStruct* ValueStruct = nullptr;
		if (Property->IsA(FStructProperty::StaticClass()))
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(Property);
			ValueStruct = StructProperty->Struct;
			TSharedPtr<INiagaraEditorPropertyUtilities, ESPMode::ThreadSafe> PropertyUtilities = FNiagaraEditorModule::Get().GetPropertyUtilities(*ValueStruct);
			if (PropertyUtilities.IsValid() && PropertyUtilities->SupportsClipboardPortableValues())
			{
				if (PropertyUtilities->TryUpdateClipboardPortableValueFromProperty(InPropertyHandle, PortableValue) == false)
				{
					PortableValue.Reset();
				}
			}
		}

		if (PortableValue.IsValid() == false)
		{
			if (InPropertyHandle.GetValueAsFormattedString(PortableValue.ValueString, PPF_Copy) != FPropertyAccess::Success)
			{
				PortableValue.Reset();
			}
		}
	}

	return PortableValue;
}

bool FNiagaraClipboardPortableValue::CanUpdateTypedValue(const FNiagaraTypeDefinition& InTargetType) const
{
	if (IsValid() == false)
	{
		return false;
	}

	TSharedPtr<INiagaraEditorTypeUtilities> TargetTypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(InTargetType);
	if (TargetTypeUtilities.IsValid() && TargetTypeUtilities->SupportsClipboardPortableValues())
	{
		return TargetTypeUtilities->CanUpdateTypedValueFromClipboardPortableValue(*this, InTargetType);
	}
	if (InTargetType.GetStruct() != nullptr && InTargetType.GetStruct()->IsA<UScriptStruct>())
	{
		UScriptStruct* TargetStruct = CastChecked<UScriptStruct>(InTargetType.GetStruct());
		TArray<uint8> ValueBytes;
		ValueBytes.AddDefaulted(InTargetType.GetSize());
		return TryUpdateStructValue(*TargetStruct, ValueBytes.GetData());
	}
	return false;
}

bool FNiagaraClipboardPortableValue::TryUpdateStructValue(const UScriptStruct& TargetStruct, uint8* StructMemory) const
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ELogVerbosity::Verbose);
	FErrorPipe ErrorPipe;
	return TargetStruct.ImportText(*ValueString, StructMemory, nullptr, PPF_Copy, &ErrorPipe, TargetStruct.GetName()) != nullptr && ErrorPipe.NumErrors == 0;
}

bool FNiagaraClipboardPortableValue::TryUpdateTypedValue(const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const
{
	if (IsValid() == false)
	{
		return false;
	}

	TSharedPtr<INiagaraEditorTypeUtilities> TargetTypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(InTargetType);
	if (TargetTypeUtilities.IsValid() && TargetTypeUtilities->SupportsClipboardPortableValues())
	{
		if (TargetTypeUtilities->TryUpdateTypedValueFromClipboardPortableValue(*this, InTargetType, InTargetValue))
		{
			return true;
		}
	}
	if (InTargetType.GetStruct() != nullptr && InTargetType.GetStruct()->IsA<UScriptStruct>())
	{
		UScriptStruct* TargetStruct = CastChecked<UScriptStruct>(InTargetType.GetStruct());
		TArray<uint8> ValueBytes;
		ValueBytes.AddDefaulted(TargetStruct->GetStructureSize());
		if (TryUpdateStructValue(*TargetStruct, ValueBytes.GetData()))
		{
			InTargetValue.SetBytes(ValueBytes.GetData(), ValueBytes.Num());
			return true;
		}
	}
	return false;
}

bool FNiagaraClipboardPortableValue::TryUpdatePropertyHandle(IPropertyHandle& InTargetPropertyHandle) const
{
	if (IsValid() == false)
	{
		return false;
	}

	FProperty* Property = InTargetPropertyHandle.GetProperty();
	if (Property != nullptr)
	{
		if (Property->IsA(FStructProperty::StaticClass()))
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(Property);
			TSharedPtr<INiagaraEditorPropertyUtilities, ESPMode::ThreadSafe> PropertyUtilities = FNiagaraEditorModule::Get().GetPropertyUtilities(*StructProperty->Struct);
			if (PropertyUtilities.IsValid() && PropertyUtilities->SupportsClipboardPortableValues())
			{
				if (PropertyUtilities->TryUpdatePropertyFromClipboardPortableValue(*this, InTargetPropertyHandle))
				{
					return true;
				}
			}
		}
		return InTargetPropertyHandle.SetValueFromFormattedString(ValueString) == FPropertyAccess::Success;
	}
	return false;
}

UNiagaraClipboardContent* UNiagaraClipboardContent::Create()
{
	return NewObject<UNiagaraClipboardContent>(GetTransientPackage());
}

FNiagaraClipboard::FNiagaraClipboard()
{
}

void FNiagaraClipboard::SetClipboardContent(UNiagaraClipboardContent* ClipboardContent)
{
	FPlatformApplicationMisc::ClipboardCopy(*ClipboardContentToString(ClipboardContent));
}

const UNiagaraClipboardContent* FNiagaraClipboard::GetClipboardContent() const
{
	// Get the text from the clipboard.
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	return ClipboardContentFromString(ClipboardText);
}

FString FNiagaraClipboard::ClipboardContentToString(UNiagaraClipboardContent* ClipboardContent)
{
	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	// Export the clipboard to text.
	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;
	UExporter::ExportToOutputDevice(&Context, ClipboardContent, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, ClipboardContent->GetOuter());
	return *Archive;
}

UNiagaraClipboardContent* FNiagaraClipboard::ClipboardContentFromString(const FString& ClipboardText)
{
	// Try to create niagara clipboard content from that.
	FNiagaraClipboardContentTextObjectFactory ClipboardContentFactory;
	if (ClipboardContentFactory.CanCreateObjectsFromText(ClipboardText))
	{
		ClipboardContentFactory.ProcessBuffer(GetTransientPackage(), RF_Transactional, ClipboardText);
		return ClipboardContentFactory.ClipboardContent;
	}

	if (ClipboardText.IsEmpty() == false)
	{
		// If the clipboard text wasn't a niagara clipboard object, it's likely been copied from elsewhere in the editor and may be 
		// a valid portable value so we construct an empty clipboard object and set the portable data.
		UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();

		// Check to see if the clipboard string can represent an external object reference and if so, check to see if we have a registered
		// handler.
		FSoftObjectPath ClipboardObjectPath = FSoftObjectPath(ClipboardText);
		UObject* ClipboardObject = ClipboardObjectPath.ResolveObject();
		if (ClipboardObject != nullptr)
		{
			TSharedPtr<const INiagaraEditorExternalObjectUtilities, ESPMode::ThreadSafe> ExternalObjectUtilities = FNiagaraEditorModule::Get().GetExternalObjectUtilities(*ClipboardObject->GetClass());
			FNiagaraClipboardPortableValue ObjectValue;
			if (ExternalObjectUtilities.IsValid() &&
				ExternalObjectUtilities->SupportsClipboardPortableValues() &&
				ExternalObjectUtilities->TryUpdateClipboardPortableValueFromObject(*ClipboardObject, ObjectValue))
			{
				ClipboardContent->PortableValues.Add(ObjectValue);
			}
		}

		// If it was not handled by an external object handler, just put the raw clipboard text in the portable values.
		if (ClipboardContent->PortableValues.Num() == 0)
		{
			FNiagaraClipboardPortableValue& PortableValue = ClipboardContent->PortableValues.AddDefaulted_GetRef();
			PortableValue.ValueString = ClipboardText;
		}

		return ClipboardContent;
	}

	return nullptr;
}

void UNiagaraClipboardEditorScriptingUtilities::TryGetInputByName(const TArray<UNiagaraClipboardFunctionInput*>& InInputs, FName InInputName, bool& bOutSucceeded, UNiagaraClipboardFunctionInput*& OutInput)
{
	for (UNiagaraClipboardFunctionInput* Input : InInputs)
	{
		if (Input->InputName == InInputName)
		{
			bOutSucceeded = true;
			OutInput = const_cast<UNiagaraClipboardFunctionInput*>(Input);
		}
	}
	bOutSucceeded = false;
	OutInput = nullptr;
}

void UNiagaraClipboardEditorScriptingUtilities::TryGetLocalValueAsFloat(const UNiagaraClipboardFunctionInput* InInput, bool& bOutSucceeded, float& OutValue)
{
	if (InInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::Local && InInput->InputType == FNiagaraTypeDefinition::GetFloatDef() && InInput->Local.Num() == InInput->InputType.GetSize())
	{
		bOutSucceeded = true;
		OutValue = *((float*)InInput->Local.GetData());
	}
	else
	{
		bOutSucceeded = false;
		OutValue = 0.0f;
	}
}

void UNiagaraClipboardEditorScriptingUtilities::TryGetLocalValueAsInt(const UNiagaraClipboardFunctionInput* InInput, bool& bOutSucceeded, int32& OutValue)
{
	bool bCompatibleType = InInput->InputType == FNiagaraTypeDefinition::GetIntDef() || (InInput->InputType.IsEnum() && InInput->InputType.GetSize() == sizeof(int32)) ||
		InInput->InputType == FNiagaraTypeDefinition::GetBoolDef();
	if (InInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::Local && bCompatibleType && InInput->Local.Num() == InInput->InputType.GetSize())
	{
		bOutSucceeded = true;
		OutValue = *((int32*)InInput->Local.GetData());
	}
	else
	{
		bOutSucceeded = false;
		OutValue = 0;
	}
}

void UNiagaraClipboardEditorScriptingUtilities::TrySetLocalValueAsInt(UNiagaraClipboardFunctionInput* InInput, bool& bOutSucceeded, int32 InValue,bool bLooseTyping)
{
	bool bCompatibleType = InInput->InputType == FNiagaraTypeDefinition::GetIntDef() || (bLooseTyping && InInput->InputType.IsEnum() && InInput->InputType.GetSize() == sizeof(int32)) ||
		(bLooseTyping && InInput->InputType == FNiagaraTypeDefinition::GetBoolDef() && InInput->InputType.GetSize() == sizeof(int32));
	if (InInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::Local && bCompatibleType && InInput->Local.Num() == InInput->InputType.GetSize())
	{
		bOutSucceeded = true;
		*((int32*)InInput->Local.GetData()) = InValue;
	}
	else
	{
		bOutSucceeded = false;
	}
}

FName UNiagaraClipboardEditorScriptingUtilities::GetTypeName(const UNiagaraClipboardFunctionInput* InInput)
{
	return InInput->InputType.GetFName();
}

const UNiagaraClipboardFunctionInput* CreateLocalValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, bool bInHasEditCondition, bool bInEditConditionValue, TArray<uint8>& InLocalValueData)
{
	return UNiagaraClipboardFunctionInput::CreateLocalValue(
		InOuter != nullptr ? InOuter : GetTransientPackage(),
		InInputName,
		InInputType,
		bInHasEditCondition ? TOptional<bool>(bInEditConditionValue) : TOptional<bool>(),
		InLocalValueData);
}

FNiagaraTypeDefinition UNiagaraClipboardEditorScriptingUtilities::GetRegisteredTypeDefinitionByName(FName InTypeName)
{
	return FNiagaraTypeRegistry::GetRegisteredTypeByName(InTypeName).Get(FNiagaraTypeDefinition());
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateColorLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FLinearColor InColorValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition::GetColorDef();
	TArray<uint8> ColorValue;
	ColorValue.AddUninitialized(InputType.GetSize());
	FMemory::Memcpy(ColorValue.GetData(), &InColorValue, InputType.GetSize());

	return const_cast<UNiagaraClipboardFunctionInput*>(CreateLocalValue(InOuter, InInputName, InputType, bInHasEditCondition, bInEditConditionValue, ColorValue));
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateFloatLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, float InFloatValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition::GetFloatDef();
	TArray<uint8> FloatValue;
	FloatValue.AddUninitialized(InputType.GetSize());
	FMemory::Memcpy(FloatValue.GetData(), &InFloatValue, InputType.GetSize());

	return const_cast<UNiagaraClipboardFunctionInput*>(CreateLocalValue(InOuter, InInputName, InputType, bInHasEditCondition, bInEditConditionValue, FloatValue));
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateMatrixLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FMatrix44f InMatrixValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition::GetMatrix4Def();
	FNiagaraMatrix Matrix;
	Matrix.Row0 = FVector4f(InMatrixValue.M[0][0], InMatrixValue.M[0][1], InMatrixValue.M[0][2], InMatrixValue.M[0][3]);
	Matrix.Row1 = FVector4f(InMatrixValue.M[1][0], InMatrixValue.M[1][1], InMatrixValue.M[1][2], InMatrixValue.M[1][3]);
	Matrix.Row2 = FVector4f(InMatrixValue.M[2][0], InMatrixValue.M[2][1], InMatrixValue.M[2][2], InMatrixValue.M[2][3]);
	Matrix.Row3 = FVector4f(InMatrixValue.M[3][0], InMatrixValue.M[3][1], InMatrixValue.M[3][2], InMatrixValue.M[3][3]);
	TArray<uint8> MatrixValue;
	MatrixValue.AddUninitialized(InputType.GetSize());
	FMemory::Memcpy(MatrixValue.GetData(), &Matrix, InputType.GetSize());

	return const_cast<UNiagaraClipboardFunctionInput*>(CreateLocalValue(InOuter, InInputName, InputType, bInHasEditCondition, bInEditConditionValue, MatrixValue));
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateQuatLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FQuat4f InQuatValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition::GetQuatDef();
	TArray<uint8> QuatValue;
	QuatValue.AddUninitialized(InputType.GetSize());
	FMemory::Memcpy(QuatValue.GetData(), &InQuatValue, InputType.GetSize());

	return const_cast<UNiagaraClipboardFunctionInput*>(CreateLocalValue(InOuter, InInputName, InputType, bInHasEditCondition, bInEditConditionValue, QuatValue));
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateVec2LocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FVector2D InVec2Value)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition::GetVec2Def();
	const FVector2f InVec2ValueFloat = FVector2f(InVec2Value);	// LWC_TODO: Precision loss
	TArray<uint8> Vec2ValueFloat;
	Vec2ValueFloat.AddUninitialized(InputType.GetSize());
	FMemory::Memcpy(Vec2ValueFloat.GetData(), &InVec2ValueFloat, InputType.GetSize());

	return const_cast<UNiagaraClipboardFunctionInput*>(CreateLocalValue(InOuter, InInputName, InputType, bInHasEditCondition, bInEditConditionValue, Vec2ValueFloat));
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateVec3LocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FVector InVec3Value)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition::GetVec3Def();
	const FVector3f InVec3ValueFloat = (FVector3f)InVec3Value;
	TArray<uint8> Vec3ValueFloat;
	Vec3ValueFloat.AddUninitialized(InputType.GetSize());
	FMemory::Memcpy(Vec3ValueFloat.GetData(), &InVec3ValueFloat, InputType.GetSize());

	return const_cast<UNiagaraClipboardFunctionInput*>(CreateLocalValue(InOuter, InInputName, InputType, bInHasEditCondition, bInEditConditionValue, Vec3ValueFloat));
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateVec4LocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FVector4f InVec4Value)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition::GetVec4Def();
	TArray<uint8> Vec4Value;
	Vec4Value.AddUninitialized(InputType.GetSize());
	FMemory::Memcpy(Vec4Value.GetData(), &InVec4Value, InputType.GetSize());

	return const_cast<UNiagaraClipboardFunctionInput*>(CreateLocalValue(InOuter, InInputName, InputType, bInHasEditCondition, bInEditConditionValue, Vec4Value));
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateIntLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, int32 InIntValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition::GetIntDef();
	TArray<uint8> IntValue;
	IntValue.AddUninitialized(InputType.GetSize());
	FMemory::Memcpy(IntValue.GetData(), &InIntValue, InputType.GetSize());

	return const_cast<UNiagaraClipboardFunctionInput*>(CreateLocalValue(InOuter, InInputName, InputType, bInHasEditCondition, bInEditConditionValue, IntValue));
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateBoolLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, bool InBoolValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition::GetBoolDef();
	const int32 BoolAsIntValue = InBoolValue ? 1 : 0;
	TArray<uint8> IntValue;
	IntValue.AddUninitialized(InputType.GetSize());
	FMemory::Memcpy(IntValue.GetData(), &BoolAsIntValue, InputType.GetSize());

	return const_cast<UNiagaraClipboardFunctionInput*>(CreateLocalValue(InOuter, InInputName, InputType, bInHasEditCondition, bInEditConditionValue, IntValue));
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateStructLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, UUserDefinedStruct* InStructValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition(InStructValue);
	FStructOnScope StructOnScope = FStructOnScope(InStructValue);
	TArray<uint8> StructValue;
	const int32 StructSize = InStructValue->GetStructureSize();
	StructValue.AddUninitialized(StructSize);
	FMemory::Memcpy(StructValue.GetData(), StructOnScope.GetStructMemory(), StructSize);
	
	return const_cast<UNiagaraClipboardFunctionInput*>(UNiagaraClipboardFunctionInput::CreateLocalValue(
		InOuter != nullptr ? InOuter : GetTransientPackage(),
		InInputName,
		InputType,
		bInHasEditCondition ? TOptional<bool>(bInEditConditionValue) : TOptional<bool>(),
		StructValue)
	);
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateEnumLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditCoditionValue, UUserDefinedEnum* InEnumType, int32 InEnumValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition(Cast<UEnum>(InEnumType));
	TArray<uint8> EnumValue;
	EnumValue.AddUninitialized(sizeof(int32));
	FMemory::Memcpy(EnumValue.GetData(), &InEnumValue, sizeof(int32));

	return const_cast<UNiagaraClipboardFunctionInput*>(UNiagaraClipboardFunctionInput::CreateLocalValue(
		InOuter != nullptr ? InOuter : GetTransientPackage(),
		InInputName,
		InputType,
		bInHasEditCondition ? TOptional<bool>(bInEditCoditionValue) : TOptional<bool>(),
		EnumValue)
	);
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateEnumLinkedValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, UUserDefinedEnum* InEnumType, FName InLinkedValue)
{
	FNiagaraTypeDefinition InputType = FNiagaraTypeDefinition(Cast<UEnum>(InEnumType));
	if (InputType.IsValid())
	{
		FNiagaraVariableBase LinkedParameter = FNiagaraVariableBase(InputType, InLinkedValue);
		return const_cast<UNiagaraClipboardFunctionInput*>(UNiagaraClipboardFunctionInput::CreateLinkedValue(
			InOuter != nullptr ? InOuter : GetTransientPackage(),
			InInputName,
			InputType,
			bInHasEditCondition ? TOptional<bool>(bInEditConditionValue) : TOptional<bool>(),
			LinkedParameter));
	}
	return nullptr;
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateLinkedValueInput(UObject* InOuter, FName InInputName, FName InInputTypeName, bool bInHasEditCondition, bool bInEditConditionValue, FName InLinkedValue)
{
	FNiagaraTypeDefinition InputType = GetRegisteredTypeDefinitionByName(InInputTypeName);
	if (InputType.IsValid())
	{
		FNiagaraVariableBase LinkedParameter = FNiagaraVariableBase(InputType, InLinkedValue);
		return const_cast<UNiagaraClipboardFunctionInput*>(UNiagaraClipboardFunctionInput::CreateLinkedValue(
			InOuter != nullptr ? InOuter : GetTransientPackage(),
			InInputName,
			InputType,
			bInHasEditCondition ? TOptional<bool>(bInEditConditionValue) : TOptional<bool>(),
			LinkedParameter));
	}
	return nullptr;
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateDataValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, UNiagaraDataInterface* InDataValue)
{
	if (InDataValue != nullptr)
	{ 
		return const_cast<UNiagaraClipboardFunctionInput*>(UNiagaraClipboardFunctionInput::CreateDataValue(
			InOuter != nullptr ? InOuter : GetTransientPackage(),
			InInputName,
			FNiagaraTypeDefinition(InDataValue->GetClass()),
			bInHasEditCondition ? TOptional<bool>(bInEditConditionValue) : TOptional<bool>(),
			InDataValue));
	}
	return nullptr;
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateExpressionValueInput(UObject* InOuter, FName InInputName, FName InInputTypeName, bool bInHasEditCondition, bool bInEditConditionValue, const FString& InExpressionValue)
{
	FNiagaraTypeDefinition InputType = GetRegisteredTypeDefinitionByName(InInputTypeName);
	if (InputType.IsValid())
	{
		return const_cast<UNiagaraClipboardFunctionInput*>(UNiagaraClipboardFunctionInput::CreateExpressionValue(
			InOuter != nullptr ? InOuter : GetTransientPackage(),
			InInputName,
			InputType,
			bInHasEditCondition ? TOptional<bool>(bInEditConditionValue) : TOptional<bool>(),
			InExpressionValue));
	}
	return nullptr;
}

UNiagaraClipboardFunctionInput* UNiagaraClipboardEditorScriptingUtilities::CreateDynamicValueInput(UObject* InOuter, FName InInputName, FName InInputTypeName, bool bInHasEditCondition, bool bInEditConditionValue, FString InDynamicValueName, UNiagaraScript* InDynamicValue)
{
	FNiagaraTypeDefinition InputType = GetRegisteredTypeDefinitionByName(InInputTypeName);
	if (InputType.IsValid())
	{
		return const_cast<UNiagaraClipboardFunctionInput*>(UNiagaraClipboardFunctionInput::CreateDynamicValue(
			InOuter != nullptr ? InOuter : GetTransientPackage(),
			InInputName,
			InputType,
			bInHasEditCondition ? TOptional<bool>(bInEditConditionValue) : TOptional<bool>(),
			InDynamicValueName,
			InDynamicValue,
			FGuid()));
	}
	return nullptr;
}

