// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCompiler.h"

#include "EdGraphSchema_Niagara.h"
#include "EdGraphUtilities.h"
#include "INiagaraEditorTypeUtilities.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "NiagaraCompilationBridge.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterface.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraGraph.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraPrecompileContainer.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraShader.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemCompilingManager.h"
#include "NiagaraTrace.h"
#include "Serialization/MemoryReader.h"
#include "ShaderCompiler.h"
#include "ShaderCore.h"
#include "ShaderFormatVectorVM.h"
#include "NiagaraGraphDigest.h"
#include "Misc/LazySingleton.h"
#include "NiagaraSystemImpl.h"
#include "NiagaraDigestDatabase.h"
#include "DerivedDataCache.h"
#include "Tasks/Task.h"
#include "DerivedDataRequestOwner.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "NiagaraCompilationTasks.h"

// remove me
#include "Algo/RemoveIf.h"
#include "NiagaraNodeReroute.h"

#define LOCTEXT_NAMESPACE "NiagaraCompiler"

void FNiagaraPrecompileData::SortOutputNodesByDependencies(TArray<const FNiagaraCompilationNodeOutput*>& NodesToSort, TConstArrayView<FNiagaraSimulationStageInfo> SimStages)
{
	if (SimStages.IsEmpty())
		return;

	TArray<const FNiagaraCompilationNodeOutput*> NewArray;
	NewArray.Reserve(NodesToSort.Num());

	// First gather up the non-simstage items
	bool bFoundAnySimStages = false;
	for (const FNiagaraCompilationNodeOutput* OutputNode : NodesToSort)
	{
		// Add any non sim stage entries back to the array in the order of encounter
		if (OutputNode->Usage != ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			NewArray.Emplace(OutputNode);
		}
		else
		{
			bFoundAnySimStages = true;
		}
	}

	// No Sim stages, no problem! Just return
	if (!bFoundAnySimStages)
	{
		return;
	}

	ensure(SimStages.Num() == (NodesToSort.Num() - NewArray.Num()));

	// Add any sim stage entries back to the array in the order of encounter in the SimStage entry list from the Emitter (Handles reordering)
	for (const FNiagaraSimulationStageInfo& Stage : SimStages)
	{
		for (const FNiagaraCompilationNodeOutput* OutputNode : NodesToSort)
		{
			if (OutputNode->Usage == ENiagaraScriptUsage::ParticleSimulationStageScript && OutputNode->UsageId == Stage.StageId)
			{
				NewArray.Emplace(OutputNode);
				break;
			}
		}
	}

	ensure(NodesToSort.Num() == NewArray.Num());

	// Copy out final results
	NodesToSort = NewArray;
}


FName FNiagaraPrecompileData::ResolveEmitterAlias(FName VariableName) const
{
	return FNiagaraParameterUtilities::ResolveEmitterAlias(VariableName, EmitterUniqueName);
}

void FNiagaraPrecompileData::GatherPreCompiledVariables(const FString& InNamespaceFilter, TArray<FNiagaraVariable>& OutVars) const
{
	if (InNamespaceFilter.Len() == 0)
	{
		OutVars.Append(EncounteredVariables);
	}
	else
	{
		for (const FNiagaraVariable& EncounteredVariable : EncounteredVariables)
		{
			if (FNiagaraParameterUtilities::IsInNamespace(EncounteredVariable, InNamespaceFilter))
			{
				FNiagaraVariable NewVar = EncounteredVariable;
				if (NewVar.IsDataAllocated() == false && !NewVar.IsDataInterface())
				{
					FNiagaraEditorUtilities::ResetVariableToDefaultValue(NewVar);
				}
				OutVars.AddUnique(NewVar);
			}
		}
	}
}

static bool DoesCustomHlslIncludeDirectParticleAcccess(const FNiagaraCompilationNode* Node)
{
	static const FString UseParticleReadTokens[] =
	{
		TEXT("InputDataFloat"),
		TEXT("InputDataInt"),
		TEXT("InputDataBool"),
		TEXT("InputDataHalf"),
	};

	if (const FNiagaraCompilationNodeCustomHlsl* CustomHlslNode = Node->AsType<FNiagaraCompilationNodeCustomHlsl>())
	{
		for (const FString& Token : CustomHlslNode->Tokens)
		{
			for (const FString& BannedToken : UseParticleReadTokens)
			{
				if (Token == BannedToken)
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FNiagaraPrecompileData::FinishPrecompile(
	const FNiagaraSystemCompilationTask& CompilationTask,
	TConstArrayView<FNiagaraVariable> EncounterableVariables,
	TConstArrayView<FNiagaraVariable> InStaticVariables,
	const FNiagaraFixedConstantResolver& ConstantResolver,
	TConstArrayView<ENiagaraScriptUsage> UsagesToProcess,
	TConstArrayView<FNiagaraSimulationStageInfo> SimStages,
	TConstArrayView<FString> EmitterNames)
{
	using FParameterMapHistoryWithMetaDataBuilder = TNiagaraParameterMapHistoryWithMetaDataBuilder<FNiagaraCompilationDigestBridge>;
	using FParameterMapHistoryBuilder = TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationDigestBridge>;
	using FParameterMapHistory = TNiagaraParameterMapHistory<FNiagaraCompilationDigestBridge>;

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	ENiagaraScriptCompileStatusEnum = StaticEnum<ENiagaraScriptCompileStatus>();
	ENiagaraScriptUsageEnum = StaticEnum<ENiagaraScriptUsage>();

	TArray<const FNiagaraCompilationNodeOutput*> OutputNodes;

	if (DigestedSourceGraph)
	{
		DigestedSourceGraph->FindOutputNodes(OutputNodes);
	}

	SortOutputNodesByDependencies(OutputNodes, SimStages);

	bool bFilterByEmitterAlias = true;
	for (const FNiagaraCompilationNodeOutput* FoundOutputNode : OutputNodes)
	{
		if (UNiagaraScript::IsSystemScript(FoundOutputNode->Usage))
		{
			bFilterByEmitterAlias = false;
		}
	}

	// we assume that we will support PartialParticleUpdate, but if we encounter a customhlsl node
	// that is directly referencing banned tokens, then we'll disable it for all sim stages
	bool bPartialParticleUpdateSimStageSupport = true;

	// Only use the static variables that match up with our expectations for this script. IE for emitters, filter things out for resolution.
	FNiagaraParameterUtilities::FilterToRelevantStaticVariables(InStaticVariables, StaticVariables, *GetUniqueEmitterName(), TEXT("Emitter"), bFilterByEmitterAlias);

	int32 NumSimStageNodes = 0;
	for (const FNiagaraCompilationNodeOutput* FoundOutputNode : OutputNodes)
	{
		if (UsagesToProcess.Contains(FoundOutputNode->Usage) == false)
		{
			continue;
		}

		FName SimStageName;
		bool bStageEnabled = true;
		if (FoundOutputNode->Usage == ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			// Find the simulation stage for this output node.
			const FGuid& UsageId = FoundOutputNode->UsageId;
			const FNiagaraSimulationStageInfo* MatchingStagePtr = SimStages.FindByPredicate([UsageId](const FNiagaraSimulationStageInfo& SimStage)
			{
				return SimStage.StageId == UsageId;
			});

			// Set whether or not the stage is enabled, and get the iteration source name if available.
			if (MatchingStagePtr && MatchingStagePtr->bEnabled && MatchingStagePtr->bGenericStage)
			{
				SimStageName = MatchingStagePtr->IterationSource == ENiagaraIterationSource::DataInterface ? MatchingStagePtr->DataInterfaceBindingName : FName();
			}
		}


		//////////////////////////////////////////////////////////////////////////
		// IS THIS EVEN NECESSARY?  IN THE PRECOMPILEDUPLICATE WE DO ANOTHER
		// TRAVERSAL (OVER THE DUPLICATED GRAPH) AND WE WILL STILL COLLECT THE
		// DATA THERE...IS THAT SUFFICIENT?
		// 
		// DATA THAT SEEMS TO BE SERIALIZED FROM THE HISTORIES:
		//	-EncounteredVariables
		//	-StaticVariablesWithMultipleWrites
		//	-PinToConstantValues
		//	-StaticVariables << this is also already calculated in System->GetCachedTraversalData()
		//////////////////////////////////////////////////////////////////////////






		if (bStageEnabled)
		{
			// Map all for this output node
			FParameterMapHistoryWithMetaDataBuilder Builder;
			*Builder.ConstantResolver = ConstantResolver;
			Builder.AddGraphToCallingGraphContextStack(DigestedSourceGraph.Get());
			Builder.RegisterEncounterableVariables(EncounterableVariables);
			Builder.RegisterExternalStaticVariables(StaticVariables);
			CompilationTask.GetAvailableCollections(Builder.AvailableCollections->EditCollections());
			Builder.OnNodeVisitedDelegate.AddLambda([&bPartialParticleUpdateSimStageSupport](const FNiagaraCompilationNode* CompilationNode) -> void
			{
				if (bPartialParticleUpdateSimStageSupport && DoesCustomHlslIncludeDirectParticleAcccess(CompilationNode))
				{
					bPartialParticleUpdateSimStageSupport = false;
				}
			});

			FString TranslationName = TEXT("Emitter");// Note that this cannot be GetUniqueEmitterName() as it would break downstream logic for some reason for data interfaces.
			Builder.BeginTranslation(TranslationName);
			Builder.BeginUsage(FoundOutputNode->Usage, SimStageName);
			Builder.EnableScriptAllowList(true, FoundOutputNode->Usage);
			Builder.BuildParameterMaps(FoundOutputNode, true);
			Builder.EndUsage();

			int HistoryIdx = 0;
			for (FParameterMapHistory& History : Builder.Histories)
			{
				History.OriginatingScriptUsage = FoundOutputNode->Usage;
				History.UsageGuid = FoundOutputNode->UsageId;
				History.UsageName = SimStageName;
				for (int32 i = 0; i < History.Variables.Num(); i++)
				{
					FNiagaraVariable& Var = History.Variables[i];
					EncounteredVariables.AddUnique(Var);
					if (Var.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
					{
						UE_LOGF(LogNiagaraEditor, Log, "Invalid numeric parameter found! %ls", *Var.GetName().ToString())
					}

					if (Var.GetType().IsStatic())
					{
						int32 NumValues = 0;
						int32 LastIndex = INDEX_NONE;

						// The logic for the static variables array adds the full payload static variable to the builder list. This will result
						// in duplicates with the same name and type, but *different* value payloads. We detect this and error out.
						for (int32 StaticIdx = 0; StaticIdx < Builder.StaticVariables.Num(); StaticIdx++)
						{
							const FNiagaraVariable& BuilderVar = Builder.StaticVariables[StaticIdx];
							if (Var == BuilderVar) // operator == ignores the value field, which is what we want here 
							{
								if (NumValues == 0)
								{
									LastIndex = StaticIdx;
									NumValues++;
								}
								else if (LastIndex != INDEX_NONE && !BuilderVar.HoldsSameData(Builder.StaticVariables[LastIndex]))
								{
									if (UNiagaraScript::LogCompileStaticVars > 0)
									{
										UE_LOGF(LogNiagaraEditor, Log, "Mismatch in static vars %ls: \"%ls\" vs \"%ls\"", *BuilderVar.GetName().ToString(),
											*BuilderVar.ToString(), *Builder.StaticVariables[LastIndex].ToString());
									}

									StaticVariablesWithMultipleWrites.AddUnique(Var);
									NumValues++;
									break;
								}
							}
						}

						if (History.PerVariableConstantValue[i].Num() > 1)
						{
							StaticVariablesWithMultipleWrites.AddUnique(Var);
						}
					}
				}

				if (UNiagaraScript::LogCompileStaticVars > 0)
				{
					for (auto Iter : History.PinToConstantValues)
					{
						UE_LOGF(LogNiagaraEditor, Log, "History [%d] Pin: %ls Value: %ls", HistoryIdx, *Iter.Key.ToString(), *Iter.Value);
					}
				}
				PinToConstantValues.Append(History.PinToConstantValues);
				++HistoryIdx;
			}

			if (FoundOutputNode->Usage == ENiagaraScriptUsage::ParticleSimulationStageScript)
			{
				NumSimStageNodes++;
			}

			if (UNiagaraScript::LogCompileStaticVars > 0)
			{
				UE_LOGF(LogNiagaraEditor, Log, "Builder.StaticVariables After Param Map Traversal............................");
			}

			for (int32 StaticIdx = 0; StaticIdx < Builder.StaticVariables.Num(); StaticIdx++)
			{
				const FNiagaraVariable& Var = Builder.StaticVariables[StaticIdx];
				bool bProcess = Builder.StaticVariableExportable[StaticIdx];

				if (UNiagaraScript::LogCompileStaticVars > 0)
				{
					UE_LOGF(LogNiagaraEditor, Log, "%ls > %ls", *Var.ToString(), bProcess ? TEXT("EXPORT") : TEXT("SkipExport"));
				}

				if (!bProcess)
				{
					continue;
				}

				StaticVariables.AddUnique(Var);

			}

			if (UNiagaraScript::LogCompileStaticVars > 0)
			{
				for (auto Iter : PinToConstantValues)
				{
					UE_LOGF(LogNiagaraEditor, Log, "Pin: %ls Value: %ls", *Iter.Key.ToString(), *Iter.Value);
				}
			}

			Builder.EndTranslation(TranslationName);

			// Collect data interface information.
			TMap<UNiagaraDataInterface*, FNiagaraVariable> DataInterfaceToTopLevelNiagaraVariable;
			TMap<FNiagaraVariable, TArray<FNiagaraVariable>> DataInterfaceParameterMapReferences;
			for (const FParameterMapHistory& History : Builder.Histories)
			{
				// Find the variable indices for data interfaces.
				TArray<int32> DataInterfaceVariableIndices;
				for (int32 i = 0; i < History.Variables.Num(); i++)
				{
					const FNiagaraVariable& Variable = History.Variables[i];
					if (Variable.IsDataInterface())
					{
						DataInterfaceVariableIndices.Add(i);
					}
				}

				// Find the data interface input nodes and collect data from the data interfaces.
				for (int32 i = 0; i < DataInterfaceVariableIndices.Num(); i++)
				{
					int32 VariableIndex = DataInterfaceVariableIndices[i];
					const FNiagaraVariable& Variable = History.Variables[VariableIndex];
					for (const FParameterMapHistory::FModuleScopedPin& WritePin : History.PerVariableWriteHistory[VariableIndex])
					{
						FNiagaraCompilationPin::FLinkedPinView LinkedPins = WritePin.Pin ? WritePin.Pin->GetLinkedPins() : FNiagaraCompilationPin::FLinkedPinView();

						if (LinkedPins.Num() == 1 && LinkedPins[0] != nullptr)
						{
							if (LinkedPins[0]->OwningNode->NodeType == FNiagaraCompilationNode::ENodeType::Input)
							{
								if (const FNiagaraCompilationNodeInput* InputNode = static_cast<const FNiagaraCompilationNodeInput*>(LinkedPins[0]->OwningNode))
								{
									FCompileDataInterfaceData* DataInterfaceData = nullptr;
									for (const FString& EmitterReference : InputNode->DataInterfaceEmitterReferences)
									{
										if (EmitterNames.Contains(EmitterReference))
										{
											if (DataInterfaceData == nullptr)
											{
												DataInterfaceData = &SharedCompileDataInterfaceData->AddDefaulted_GetRef();
												DataInterfaceData->EmitterName = EmitterUniqueName;
												DataInterfaceData->Usage = FoundOutputNode->Usage;
												DataInterfaceData->UsageId = FoundOutputNode->UsageId;
												DataInterfaceData->Variable = Variable;
											}

											DataInterfaceData->ReadsEmitterParticleData.Add(EmitterReference);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	{
		auto VariableLess = [](const FNiagaraVariableBase& Lhs, const FNiagaraVariableBase& Rhs) -> bool
		{
			return Lhs.GetName().LexicalLess(Rhs.GetName());
		};

		StaticVariables.StableSort(VariableLess);
	}

	if (NumSimStageNodes)
	{
		CompileSimStageData.Reserve(NumSimStageNodes);

		const int32 NumProvidedStages = SimStages.Num();
		for (int32 i = 0, ActiveStageCount = 0; ActiveStageCount < NumSimStageNodes && i < NumProvidedStages; ++i)
		{
			const FNiagaraSimulationStageInfo& SimStage = SimStages[i];
			if (SimStage.bEnabled && SimStage.bHasCompilationData)
			{
				FNiagaraSimulationStageCompilationData& CompileData = CompileSimStageData.Add_GetRef(SimStage.CompilationData);

				// note whether we've detected any incorrect usage of particle data, such that we need to
				// bad partial particle updates
				if (!bPartialParticleUpdateSimStageSupport)
				{
					CompileData.PartialParticleUpdate = false;
				}

				++ActiveStageCount;
			}
		}
	}
}

void FNiagaraPrecompileData::CollectBakedRapidIterationParameters(const FNiagaraSystemCompilationTask& CompilationTask, TConstArrayView<TObjectKey<UNiagaraScript>> OwnedScriptKeys)
{
	// if we're using rapid iteration parameters then none of them will be baked into the script and we can carry on
	if (GetUseRapidIterationParams())
	{
		return;
	}

	// we also need to aggregate all of the rapid iteration parameters for the owned scripts
	for (const TObjectKey<UNiagaraScript>& ScriptKey : OwnedScriptKeys)
	{
		if (const FNiagaraSystemCompilationTask::FScriptInfo* ScriptInfo = CompilationTask.GetScriptInfo(ScriptKey))
		{
			for (const FNiagaraVariableWithOffset& Param : ScriptInfo->RapidIterationParameters.ReadParameterVariables())
			{
				// skip the variable if it already exists
				const bool AlreadyAdded = BakedRapidIterationParameters.Contains(Param);
				if (!AlreadyAdded)
				{
					FNiagaraVariable& BakedParameter = BakedRapidIterationParameters.Add_GetRef(Param);
					BakedParameter.SetData(ScriptInfo->RapidIterationParameters.GetParameterData(Param));
				}
			}
		}
	}
}

UNiagaraDataInterface* FNiagaraCompilationCopyData::GetDuplicatedDataInterfaceCDOForClass(UClass* Class) const
{
	if (UNiagaraDataInterface* ClassDefault = AggregatedDataInterfaceCDODuplicates.FindRef(Class))
	{
		return ClassDefault;
	}
	else
	{
		ensureMsgf(false, TEXT("GetDuplicatedDataInterfaceCDOForClass - %s"), *Class->GetName());
	}
	return nullptr;
}

void FNiagaraCompilationCopyData::InstantiateCompilationCopy(const FNiagaraCompilationGraphDigested& SourceGraph, const FNiagaraPrecompileData* PrecompileData, ENiagaraScriptUsage InUsage, const FNiagaraFixedConstantResolver& ConstantResolver)
{
	InstantiatedGraph = SourceGraph.Instantiate(PrecompileData, this, ValidUsages, ConstantResolver);

	if (InstantiatedGraph)
	{
		SourceGraph.CollectReferencedDataInterfaceCDO(AggregatedDataInterfaceCDODuplicates);
	}
}

void FNiagaraCompilationCopyData::CreateParameterMapHistory(const FNiagaraSystemCompilationTask& CompilationTask, const TArray<FNiagaraVariable>& EncounterableVariables, const TArray<FNiagaraVariable>& InStaticVariables, const FNiagaraFixedConstantResolver& ConstantResolver, TConstArrayView<FNiagaraSimulationStageInfo> SimStages)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	PrecompiledHistories.Empty();

	TArray<const FNiagaraCompilationNodeOutput*> OutputNodes;
	if (InstantiatedGraph.IsValid())
	{
		InstantiatedGraph->FindOutputNodes(OutputNodes);
	}

	FNiagaraPrecompileData::SortOutputNodesByDependencies(OutputNodes, SimStages);

	for (const FNiagaraCompilationNodeOutput* FoundOutputNode : OutputNodes)
	{
		FName SimStageName;
		bool bStageEnabled = true;
		if (FoundOutputNode->Usage == ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			// Find the simulation stage for this output node.
			const FGuid& UsageId = FoundOutputNode->UsageId;
			const FNiagaraSimulationStageInfo* MatchingStagePtr = SimStages.FindByPredicate([UsageId](const FNiagaraSimulationStageInfo& SimStage)
			{
				return SimStage.StageId == UsageId;
			});

			// Set whether or not the stage is enabled, and get the iteration source name if available.
			bStageEnabled = MatchingStagePtr && MatchingStagePtr->bEnabled;
			if (bStageEnabled && MatchingStagePtr->bGenericStage)
			{
				SimStageName = MatchingStagePtr->IterationSource == ENiagaraIterationSource::DataInterface ? MatchingStagePtr->DataInterfaceBindingName : FName();
			}
		}

		if (bStageEnabled)
		{
			// Map all for this output node
			FParameterMapHistoryWithMetaDataBuilder Builder;
			*Builder.ConstantResolver = ConstantResolver;
			Builder.AddGraphToCallingGraphContextStack(InstantiatedGraph.Get());
			Builder.RegisterEncounterableVariables(EncounterableVariables);
			Builder.RegisterExternalStaticVariables(InStaticVariables);
			CompilationTask.GetAvailableCollections(Builder.AvailableCollections->EditCollections());

			FString TranslationName = TEXT("Emitter");
			Builder.BeginTranslation(TranslationName);
			Builder.BeginUsage(FoundOutputNode->Usage, SimStageName);
			Builder.EnableScriptAllowList(true, FoundOutputNode->Usage);
			Builder.BuildParameterMaps(FoundOutputNode, true);
			Builder.EndUsage();

			for (FParameterMapHistory& History : Builder.Histories)
			{
				History.OriginatingScriptUsage = FoundOutputNode->Usage;
				History.UsageGuid = FoundOutputNode->UsageId;
				History.UsageName = SimStageName;
				for (int32 VarIdx = 0; VarIdx < History.Variables.Num(); VarIdx++)
				{
					const FNiagaraVariable& Var = History.Variables[VarIdx];
					if (Var.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
					{
						UE_LOGF(LogNiagaraEditor, Log, "Invalid numeric parameter found! %ls", *Var.GetName().ToString())
					}
				}
			}

			PrecompiledHistories.Append(Builder.Histories);
			Builder.EndTranslation(TranslationName);
		}
		else
		{
			// Add in a blank spot
			PrecompiledHistories.Emplace();
		}
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
