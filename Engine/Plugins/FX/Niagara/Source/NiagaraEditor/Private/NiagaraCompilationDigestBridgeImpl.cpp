// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCompilationBridge.h"

#include "NiagaraCompilationPrivate.h"
#include "NiagaraConstants.h"
#include "NiagaraDigestDatabase.h"
#include "NiagaraGraphDigest.h"
#include "NiagaraTraversalStateContext.h"

const FNiagaraCompilationGraph* FNiagaraCompilationDigestBridge::GetGraph(const FCompilationCopy* CompilationCopy)
{
	return CompilationCopy->InstantiatedGraph.Get();
}

const FNiagaraCompilationNode* FNiagaraCompilationDigestBridge::GetOwningNode(const FPin* Pin)
{
	return Pin->OwningNode;
}

// left intentionally unimplemented because the digest path shouldn't ever require mutable nodes
//FNiagaraCompilationNode* FNiagaraCompilationDigestBridge::GetMutableOwningNode(const FPin* Pin);
//{
//	return nullptr;
//}

const FNiagaraCompilationGraph* FNiagaraCompilationDigestBridge::GetOwningGraph(const FNode* Node)
{
	return Node->OwningGraph;
}

bool FNiagaraCompilationDigestBridge::CustomHlslReferencesTokens(const FCustomHlslNode* CustomNode, TConstArrayView<FStringView> TokenStrings)
{
	for (FStringView TokenString : TokenStrings)
	{
		const bool TokenMatched = CustomNode->Tokens.ContainsByPredicate([TokenString](const FString& Token) -> bool
		{
			return Token.Contains(TokenString);
		});

		if (TokenMatched)
		{
			return true;
		}
	}

	return false;
}

void FNiagaraCompilationDigestBridge::CustomHlslReferencesTokens(const FCustomHlslNode* CustomNode, TConstArrayView<FName> TokenStrings, TArrayView<bool> Results)
{
	checkf(TokenStrings.Num() == Results.Num(), TEXT("Number of results must match the number of tokens queried for"));
	if (TokenStrings.Num() == 0)
	{
		return;
	}

	for (int32 i = 0; i < TokenStrings.Num(); i++)
	{
		Results[i] = CustomNode->TokensContainsName(TokenStrings[i]);
	}
}

ENiagaraScriptUsage FNiagaraCompilationDigestBridge::GetCustomHlslUsage(const FCustomHlslNode* CustomNode)
{
	return CustomNode->CustomScriptUsage;
}

FString FNiagaraCompilationDigestBridge::GetCustomHlslString(const FCustomHlslNode* CustomNode)
{
	return CustomNode->CustomHlsl;
}

void FNiagaraCompilationDigestBridge::GetCustomHlslIncludePaths(const FCustomHlslNode* CustomNode, TArray<FNiagaraCustomHlslInclude>& IncludePaths)
{
	IncludePaths = CustomNode->CustomIncludePaths;
}

const TArray<FNiagaraCompilationCachedConnection>& FNiagaraCompilationDigestBridge::GetConvertConnections(const FConvertNode* ConvertNode)
{
	return ConvertNode->Connections;
}

const FNiagaraCompilationNodeFunctionCall* FNiagaraCompilationDigestBridge::AsFunctionCallNode(const FNode* Node)
{
	return Node->AsType<FNiagaraCompilationNodeFunctionCall>();
}

const FNiagaraCompilationNodeInput* FNiagaraCompilationDigestBridge::AsInputNode(const FNode* Node)
{
	return Node->AsType<FNiagaraCompilationNodeInput>();
}

const FNiagaraCompilationNodeParameterMapGet* FNiagaraCompilationDigestBridge::AsParamMapGetNode(const FNode* Node)
{
	return Node->AsType<FNiagaraCompilationNodeParameterMapGet>();
}

const FNiagaraCompilationNodeCustomHlsl* FNiagaraCompilationDigestBridge::AsCustomHlslNode(const FNode* Node)
{
	return Node->AsType<FNiagaraCompilationNodeCustomHlsl>();
}

const FNiagaraCompilationNodeParameterMapSet* FNiagaraCompilationDigestBridge::AsParamMapSetNode(const FNode* Node)
{
	return Node->AsType<FNiagaraCompilationNodeParameterMapSet>();
}

bool FNiagaraCompilationDigestBridge::GraphHasParametersOfType(const FGraph* Graph, const FNiagaraTypeDefinition& TypeDef)
{
	return Graph->HasParametersOfType(TypeDef);
}

TArray<FNiagaraVariableBase> FNiagaraCompilationDigestBridge::GraphGetStaticSwitchInputs(const FGraph* Graph)
{
	return Graph->StaticSwitchInputs;
}

void FNiagaraCompilationDigestBridge::FindOutputNodes(const FGraph* Graph, ENiagaraScriptUsage ScriptUsage, TArray<const FOutputNode*>& OutputNodes)
{
	Graph->FindOutputNodes(ScriptUsage, OutputNodes);
}

const FNiagaraCompilationDigestBridge::FOutputNode* FNiagaraCompilationDigestBridge::FindOutputNode(const FGraph* Graph, ENiagaraScriptUsage ScriptUsage, const FGuid& UsageId)
{
	return Graph->FindOutputNode(ScriptUsage, UsageId);
}

void FNiagaraCompilationDigestBridge::FindOutputNodes(const FGraph* Graph, TArray<const FOutputNode*>& OutputNodes)
{
	Graph->FindOutputNodes(OutputNodes);
}

void FNiagaraCompilationDigestBridge::BuildTraversal(const FGraph* Graph, const FNode* OutputNode, TArray<const FNode*>& TraversedNodes)
{
	Graph->BuildTraversal(OutputNode, TraversedNodes);
}

const FNiagaraCompilationGraph* FNiagaraCompilationDigestBridge::GetEmitterGraph(const FEmitterNode* EmitterNode)
{
	return EmitterNode->CalledGraph.Get();
}

FNiagaraEmitterID FNiagaraCompilationDigestBridge::GetEmitterID(const FEmitterNode* EmitterNode)
{
	return EmitterNode->EmitterID;
}

FString FNiagaraCompilationDigestBridge::GetEmitterUniqueName(const FEmitterNode* EmitterNode)
{
	return EmitterNode->EmitterUniqueName;
}

ENiagaraScriptUsage FNiagaraCompilationDigestBridge::GetEmitterUsage(const FEmitterNode* EmitterNode)
{
	return EmitterNode->Usage;
}

FString FNiagaraCompilationDigestBridge::GetEmitterHandleIdString(const FEmitterNode* EmitterNode)
{
	return EmitterNode->EmitterHandleIdString;
}

FString FNiagaraCompilationDigestBridge::GetEmitterName(const FEmitterNode* EmitterNode)
{
	return EmitterNode->EmitterName;
}

FString FNiagaraCompilationDigestBridge::GetEmitterPathName(const FEmitterNode* EmitterNode)
{
	return EmitterNode->EmitterPathName;
}

const FNiagaraCompilationGraph* FNiagaraCompilationDigestBridge::GetFunctionNodeGraph(const FFunctionCallNode* FunctionCall)
{
	return FunctionCall->CalledGraph.Get();
}

FString FNiagaraCompilationDigestBridge::GetFunctionFullName(const FFunctionCallNode* FunctionCall)
{
	if (const FNiagaraCompilationGraph* Graph = FunctionCall->CalledGraph.Get())
	{
		return Graph->SourceScriptFullName;
	}
	return FString();
}

FString FNiagaraCompilationDigestBridge::GetFunctionName(const FFunctionCallNode* FunctionCall)
{
	return FunctionCall->FunctionName;
}

FString FNiagaraCompilationDigestBridge::GetFunctionScriptName(const FFunctionCallNode* FunctionCall)
{
	return FunctionCall->FunctionScriptName;
}

ENiagaraScriptUsage FNiagaraCompilationDigestBridge::GetFunctionUsage(const FFunctionCallNode* FunctionCall)
{
	return FunctionCall->CalledScriptUsage;
}

const FNiagaraCompilationInputPin* FNiagaraCompilationDigestBridge::GetDefaultPin(const FParamMapGetNode* GetNode, const FOutputPin* OutputPin)
{
	const FInputPin* DefaultPin = nullptr;
	const int32 OutputPinIndex = UE_PTRDIFF_TO_INT32(OutputPin - GetNode->OutputPins.GetData());
	if (ensure(GetNode->DefaultInputPinIndices.IsValidIndex(OutputPinIndex)))
	{
		const int32 InputPinIndex = GetNode->DefaultInputPinIndices[OutputPinIndex];
		if (InputPinIndex != INDEX_NONE)
		{
			ensure(GetNode->InputPins.IsValidIndex(InputPinIndex));
			DefaultPin = &GetNode->InputPins[InputPinIndex];
		}
	}
	return DefaultPin;
}

TOptional<ENiagaraDefaultMode> FNiagaraCompilationDigestBridge::GetGraphDefaultMode(const FGraph* Graph, const FNiagaraVariable& Variable, FNiagaraScriptVariableBinding& Binding)
{
	return Graph->GetDefaultMode(Variable, Binding);
}

TArray<const FNiagaraCompilationNodeInput*> FNiagaraCompilationDigestBridge::GetGraphInputNodes(const FGraph* Graph, const UNiagaraGraph::FFindInputNodeOptions& Options)
{
	TArray<const FInputNode*> InputNodes;
	Graph->FindInputNodes(InputNodes, Options);
	return InputNodes;
}

TArray<const FNiagaraCompilationInputPin*> FNiagaraCompilationDigestBridge::GetInputPins(const FNode* Node)
{
	TArray<const FInputPin*> InputPins;
	InputPins.Reserve(Node->InputPins.Num());
	Algo::Transform(Node->InputPins, InputPins, [](const FInputPin& InputPin) -> const FInputPin*
		{
			return &InputPin;
		});

	return InputPins;
}

TArray<const FNiagaraCompilationOutputPin*> FNiagaraCompilationDigestBridge::GetOutputPins(const FNode* Node)
{
	TArray<const FOutputPin*> OutputPins;
	OutputPins.Reserve(Node->OutputPins.Num());
	Algo::Transform(Node->OutputPins, OutputPins, [](const FOutputPin& OutputPin) -> const FOutputPin*
		{
			return &OutputPin;
		});

	return OutputPins;
}

TArray<const FNiagaraCompilationPin*> FNiagaraCompilationDigestBridge::GetPins(const FNode* Node)
{
	TArray<const FPin*> Pins;
	Pins.Reserve(Node->InputPins.Num() + Node->OutputPins.Num());

	auto GetPinPointers = [](const FPin& InPin) -> const FPin*
	{
		return &InPin;
	};

	Algo::Transform(Node->InputPins, Pins, GetPinPointers);
	Algo::Transform(Node->OutputPins, Pins, GetPinPointers);

	return Pins;
}

FNiagaraTypeDefinition FNiagaraCompilationDigestBridge::GetPinType(const FPin* Pin, ENiagaraStructConversion Conversion)
{
	return Pin->Variable.GetType();
}

FText FNiagaraCompilationDigestBridge::GetPinFriendlyName(const FPin* Pin)
{
	return FText::FromName(Pin->PinName);
}

FText FNiagaraCompilationDigestBridge::GetPinDisplayName(const FPin* Pin)
{
	return FText::FromName(Pin->PinName);
}

FNiagaraVariable FNiagaraCompilationDigestBridge::GetPinVariable(const FPin* Pin, bool bNeedsValue, ENiagaraStructConversion Conversion)
{
	return Pin->Variable;
}

FNiagaraVariable FNiagaraCompilationDigestBridge::GetInputVariable(const FInputNode* InputNode)
{
	return InputNode->InputVariable;
}

const TArray<FNiagaraVariable>& FNiagaraCompilationDigestBridge::GetOutputVariables(const FOutputNode* OutputNode)
{
	return OutputNode->Outputs;
}

TArray<FNiagaraVariable> FNiagaraCompilationDigestBridge::GetGraphOutputNodeVariables(const FGraph* Graph, ENiagaraScriptUsage Usage)
{
	TArray<FNiagaraVariable> OutputNodeVariables;
	TArray<const FOutputNode*> OutputNodes;
	Graph->FindOutputNodes(Usage, OutputNodes);
	for (const FNiagaraCompilationNodeOutput* OutputNode : OutputNodes)
	{
		OutputNodeVariables.Append(OutputNode->Outputs);
	}
	return OutputNodeVariables;
}

const FNiagaraCompilationInputPin* FNiagaraCompilationDigestBridge::GetPinAsInput(const FPin* Pin)
{
	if (Pin && Pin->Direction == EGPD_Input)
	{
		return static_cast<const FInputPin*>(Pin);
	}
	return nullptr;
}

const FNiagaraCompilationOutputPin* FNiagaraCompilationDigestBridge::GetLinkedOutputPin(const FInputPin* InputPin)
{
	return InputPin ? InputPin->LinkedTo : nullptr;
}

FGuid FNiagaraCompilationDigestBridge::GetOutputNodeUsageId(const FOutputNode* OutputNode)
{
	return OutputNode->UsageId;
}

bool FNiagaraCompilationDigestBridge::CanCreateConnection(const FOutputPin* OutputPin, const FInputPin* InputPin, FText& FailureMessage)
{
	// pin connections have already been verified at this point
	return true;
}

ENiagaraScriptUsage FNiagaraCompilationDigestBridge::GetOutputNodeScriptType(const FOutputNode* OutputNode)
{
	return OutputNode->Usage;
}

FGuid FNiagaraCompilationDigestBridge::GetOutputNodeScriptTypeId(const FOutputNode* OutputNode)
{
	return OutputNode->UsageId;
}

bool FNiagaraCompilationDigestBridge::IsGraphEmpty(const FGraph* Graph)
{
	return Graph->Nodes.IsEmpty();
}

void FNiagaraCompilationDigestBridge::AddCollectionPaths(const FParamMapHistory& History, TArray<FString>& Paths)
{
	for (const FNiagaraCompilationNPCHandle& Handle : History.EncounteredParameterCollections.Handles)
	{
		Paths.AddUnique(Handle.Resolve()->CollectionPath);
	}
}

bool FNiagaraCompilationDigestBridge::NodeIsEnabled(const FNode* Node)
{
	return Node->NodeEnabled;
}

TOptional<ENiagaraDefaultMode> FNiagaraCompilationDigestBridge::GraphGetDefaultMode(const FGraph* Graph, const FNiagaraVariableBase& Variable, FNiagaraScriptVariableBinding& Binding)
{
	return Graph->GetDefaultMode(Variable, Binding);
}

const FNiagaraCompilationOutputPin* FNiagaraCompilationDigestBridge::GetSelectOutputPin(const FSelectNode* SelectNode, const FNiagaraVariableBase& Variable)
{
	return SelectNode->FindOutputPin(Variable);
}

FString FNiagaraCompilationDigestBridge::GetNodeName(const FNode* Node)
{
	return Node->NodeName;
}

FString FNiagaraCompilationDigestBridge::GetNodeTitle(const FNode* Node)
{
	return Node->FullTitle;
}

const FNiagaraCompilationInputPin* FNiagaraCompilationDigestBridge::GetInputPin(const FNode* Node, int32 PinIndex)
{
	return Node->InputPins.IsValidIndex(PinIndex) ? &Node->InputPins[PinIndex] : nullptr;
}

int32 FNiagaraCompilationDigestBridge::GetPinIndexById(TConstArrayView<const FInputPin*> Pins, const FGuid& PinId)
{
	const int32 PinCount = Pins.Num();
	for (int32 PinIt = 0; PinIt < PinCount; ++PinIt)
	{
		if (Pins[PinIt]->UniquePinId == PinId)
		{
			return PinIt;
		}
	}
	return INDEX_NONE;
}

int32 FNiagaraCompilationDigestBridge::GetPinIndexById(TConstArrayView<const FOutputPin*> Pins, const FGuid& PinId)
{
	const int32 PinCount = Pins.Num();
	for (int32 PinIt = 0; PinIt < PinCount; ++PinIt)
	{
		if (Pins[PinIt]->UniquePinId == PinId)
		{
			return PinIt;
		}
	}
	return INDEX_NONE;
}

FString FNiagaraCompilationDigestBridge::GetCollectionFullName(FParameterCollection Collection)
{
	return Collection.Resolve()->CollectionFullName;
}

bool FNiagaraCompilationDigestBridge::IsCollectionValid(FParameterCollection Collection)
{
	return Collection.IsValid();
}

UNiagaraDataInterface* FNiagaraCompilationDigestBridge::GetCollectionDataInterface(FParameterCollection Collection, const FNiagaraVariable& Variable)
{
	check(false);
	return nullptr;
}

UObject* FNiagaraCompilationDigestBridge::GetCollectionUObject(FParameterCollection Collection, const FNiagaraVariable& Variable)
{
	check(false);
	return nullptr;
}

FNiagaraCompilationDigestBridge::FParameterCollection FNiagaraCompilationDigestBridge::IsParameterCollectionParameter(const FParamMapHistory& ParamMapHistory, const FNiagaraVariable& InVar, bool& bMissingParameter)
{
	if (ParamMapHistory.bParameterCollectionsSkipped)
	{
		UE_LOGF(LogNiagaraEditor, Error, "NiagaraParameterCollection was skipped during history building.  History which require NPC data can not be generated during PostLoad().");
		return nullptr;
	}

	bMissingParameter = false;
	FNameBuilder VarName(InVar.GetName());
	for (const FNiagaraCompilationNPCHandle& CollectionHandle : ParamMapHistory.EncounteredParameterCollections.Handles)
	{
		FNameBuilder CollectionName(CollectionHandle.Namespace);
		if (VarName.ToView().StartsWith(CollectionName.ToView()))
		{
			bMissingParameter = CollectionHandle.Resolve()->Variables.Contains(InVar);
			return CollectionHandle;
		}
	}

	return FNiagaraCompilationNPCHandle();
}

FNiagaraCompilationDigestBridge::FBuilderExtraData::FBuilderExtraData()
{
	TraversalStateContext = MakePimpl<FNiagaraTraversalStateContext>();
	AvailableCollections = MakeUnique<FNiagaraDigestedParameterCollections>();
}

void FNiagaraCompilationDigestBridge::FParameterCollectionStore::Append(const FParameterCollectionStore& Other)
{
	Handles.Append(Other.Handles);
}

bool FNiagaraCompilationDigestBridge::IsStaticPin(const FPin* Pin)
{
	return Pin->Variable.GetType().IsStatic();
}

const FNiagaraCompilationNodeOutput* FNiagaraCompilationDigestBridge::AsOutputNode(const FNode* Node)
{
	return Node->AsType<FNiagaraCompilationNodeOutput>();
}

bool FNiagaraCompilationDigestBridge::IsParameterMapGet(const FNode* Node)
{
	return Node->AsType<FNiagaraCompilationNodeParameterMapGet>() != nullptr;
}

ENiagaraScriptUsage FNiagaraCompilationDigestBridge::GetOutputNodeUsage(const FOutputNode* OutputNode)
{
	return OutputNode->Usage;
}

TOptional<FName> FNiagaraCompilationDigestBridge::GetOutputNodeStackContextOverride(const FOutputNode* OutputNode)
{
	return OutputNode->StackContextOverrideName;
}

FString FNiagaraCompilationDigestBridge::GetNodeClassName(const FNode* Node)
{
	return Node->GetSourceNodeClass()->GetName();
}

bool FNiagaraCompilationDigestBridge::IsParameterMapPin(const FPin* Pin)
{
	return Pin->Variable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef();
}

bool FNiagaraCompilationDigestBridge::GetGraphReferencesStaticVariables(const FGraph* Graph, FNiagaraStaticVariableSearchContext& StaticVariableContext)
{
	return Graph->bContainsStaticVariables;
}

const FNiagaraCompilationNodeEmitter* FNiagaraCompilationDigestBridge::GetNodeAsEmitter(const FNode* Node)
{
	return Node->AsType<FEmitterNode>();
}

bool FNiagaraCompilationDigestBridge::GetCustomNodeUsesImpureFunctions(const FNiagaraCompilationNodeCustomHlsl* CustomNode)
{
	return CustomNode->bCallsImpureFunctions;
}

//////////////////////////////////////////////////////////////////////////


FNiagaraCompilationNPCHandle FNiagaraDigestedParameterCollections::FindMatchingCollection(FName VariableName, bool bAllowPartialMatch, FNiagaraVariable& OutVar) const
{
	FNiagaraCompilationNPCHandle Handle = FindCollectionByName(VariableName);

	if (Handle.IsValid())
	{
		FNiagaraCompilationNPCHandle::FDigestPtr Collection = Handle.Resolve();
		for (const FNiagaraVariableBase& Variable : Collection->Variables)
		{
			if (Variable.GetName() == VariableName)
			{
				OutVar = Variable;
				return Handle;
			}
		}

		if (bAllowPartialMatch)
		{
			int32 BestMatchLength = 0;

			FNameBuilder TargetBuilder(VariableName);
			for (const FNiagaraVariableBase& Variable : Collection->Variables)
			{
				FNameBuilder CurrentBuilder(Variable.GetName());
				CurrentBuilder << TEXT(".");

				const int32 CurrentBuilderLen = CurrentBuilder.Len();

				if (BestMatchLength == 0 || BestMatchLength < CurrentBuilderLen)
				{
					if (TargetBuilder.ToView().StartsWith(CurrentBuilder.ToView()))
					{
						OutVar = Variable;
						BestMatchLength = CurrentBuilderLen;
					}
				}
			}
		}
	}

	return Handle;
}

FNiagaraCompilationNPCHandle FNiagaraDigestedParameterCollections::FindCollection(const FNiagaraVariable& Variable) const
{
	return FindCollectionByName(Variable.GetName());
}

FNiagaraCompilationNPCHandle FNiagaraDigestedParameterCollections::FindCollectionByName(FName VariableName) const
{
	FNameBuilder VarName(VariableName);
	FStringView VarNameView = VarName.ToView();
	if (VarNameView.StartsWith(PARAM_MAP_NPC_STR))
	{
		for (FNiagaraCompilationNPCHandle Handle : Collections)
		{
			FNameBuilder FullNamespace;
			FullNamespace << PARAM_MAP_NPC_STR;
			FullNamespace << Handle.Namespace;
			FullNamespace << TEXT(".");

			if (VarNameView.StartsWith(FullNamespace.ToView()))
			{
				return Handle;
			}
		}
	}

	return FNiagaraCompilationNPCHandle();
}
