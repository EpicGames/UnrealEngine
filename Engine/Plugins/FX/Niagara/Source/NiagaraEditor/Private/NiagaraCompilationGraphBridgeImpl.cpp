// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCompilationBridge.h"

#include "EdGraphSchema_Niagara.h"
#include "NiagaraCompilationPrivate.h"
#include "NiagaraConstants.h"
#include "NiagaraModule.h"
#include "NiagaraNodeConvert.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeIf.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOp.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapFor.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeSelect.h"
#include "NiagaraScriptVariable.h"

namespace NiagaraCompilationGraphBridgeImpl
{

template<typename ArrayType>
static void AppendCompilationPins(const UNiagaraNode* Node, EEdGraphPinDirection PinDirection, ArrayType& OutPins)
{
	const UNiagaraNodeWithDynamicPins* DynNode = Cast<const UNiagaraNodeWithDynamicPins>(Node);
	for (UEdGraphPin* Pin : Node->Pins)
	{
		if (Pin->Direction == PinDirection)
		{
			if (Pin->bOrphanedPin)
			{
				continue;
			}

			if (DynNode != nullptr && DynNode->IsAddPin(Pin))
			{
				continue;
			}

			OutPins.Add(Pin);
		}
	}
}

}


FNiagaraCompilationGraphBridge::FBuilderExtraData::FBuilderExtraData()
{
	AvailableCollections = MakeUnique<FAvailableParameterCollections>();
}

void FNiagaraCompilationGraphBridge::FParameterCollectionStore::Append(const FParameterCollectionStore& Other)
{
	Collections.Append(Other.Collections);
	CollectionVariables.Append(Other.CollectionVariables);
	CollectionNamespaces.Append(Other.CollectionNamespaces);
}

const UNiagaraGraph* FNiagaraCompilationGraphBridge::GetGraph(const FCompilationCopy* CompilationCopy)
{
	return CompilationCopy->NodeGraphDeepCopy.Get();
}

const UNiagaraNode* FNiagaraCompilationGraphBridge::GetOwningNode(const FPin* Pin)
{
	return Cast<const FNode>(Pin->GetOwningNode());
}

UNiagaraNode* FNiagaraCompilationGraphBridge::GetMutableOwningNode(const FPin* Pin)
{
	return Cast<FNode>(Pin->GetOwningNode());
}

const UNiagaraGraph* FNiagaraCompilationGraphBridge::GetOwningGraph(const FNode* Node)
{
	if (const UNiagaraNode* NiagaraNode = Cast<const UNiagaraNode>(Node))
	{
		return NiagaraNode->GetNiagaraGraph();
	}
	return nullptr;
}

bool FNiagaraCompilationGraphBridge::CustomHlslReferencesTokens(const FCustomHlslNode* CustomNode, TConstArrayView<FStringView> TokenStrings)
{
	TArray<FStringView> StringTokens;
	UNiagaraNodeCustomHlsl::GetTokensFromString(CustomNode->GetCustomHlsl(), StringTokens, false, false);

	for (FStringView Token : TokenStrings)
	{
		const bool TokenMatched = StringTokens.ContainsByPredicate([&Token](const FStringView& HlslToken) -> bool
			{
				return HlslToken.Contains(Token);
			});

		if (TokenMatched)
		{
			return true;
		}
	}

	return false;
}

void FNiagaraCompilationGraphBridge::CustomHlslReferencesTokens(const FCustomHlslNode* CustomNode, TConstArrayView<FName> TokenStrings, TArrayView<bool> Results)
{
	checkf(TokenStrings.Num() == Results.Num(), TEXT("Number of results must match the number of tokens queried for"));
	if (TokenStrings.Num() == 0)
	{
		return;
	}
	TArray<FStringView> StringTokens;
	UNiagaraNodeCustomHlsl::GetTokensFromString(CustomNode->GetCustomHlsl(), StringTokens, false, false);

	for (int32 i = 0; i < TokenStrings.Num(); i++)
	{
		FNameBuilder NameBuilder(TokenStrings[i]);
		FStringView NameString(NameBuilder.ToView());
		Results[i] = StringTokens.ContainsByPredicate([NameString](const FStringView& HlslToken) -> bool
			{
				return HlslToken.Contains(NameString);
			});
	}
}

ENiagaraScriptUsage FNiagaraCompilationGraphBridge::GetCustomHlslUsage(const FCustomHlslNode* CustomNode)
{
	return CustomNode->ScriptUsage;
}

FString FNiagaraCompilationGraphBridge::GetCustomHlslString(const FCustomHlslNode* CustomNode)
{
	return CustomNode->GetCustomHlsl();
}

void FNiagaraCompilationGraphBridge::GetCustomHlslIncludePaths(const FCustomHlslNode* CustomNode, TArray<FNiagaraCustomHlslInclude>& Includes)
{
	CustomNode->GetIncludeFilePaths(Includes);
}

const TArray<FNiagaraConvertConnection>& FNiagaraCompilationGraphBridge::GetConvertConnections(const FConvertNode* ConvertNode)
{
	return ConvertNode->GetConnections();
}

const UNiagaraNodeFunctionCall* FNiagaraCompilationGraphBridge::AsFunctionCallNode(const FNode* Node)
{
	return Cast<const UNiagaraNodeFunctionCall>(Node);
}

const UNiagaraNodeInput* FNiagaraCompilationGraphBridge::AsInputNode(const FNode* Node)
{
	return Cast<const UNiagaraNodeInput>(Node);
}

const UNiagaraNodeParameterMapGet* FNiagaraCompilationGraphBridge::AsParamMapGetNode(const FNode* Node)
{
	return Cast<const UNiagaraNodeParameterMapGet>(Node);
}

const UNiagaraNodeCustomHlsl* FNiagaraCompilationGraphBridge::AsCustomHlslNode(const FNode* Node)
{
	return Cast<const UNiagaraNodeCustomHlsl>(Node);
}

const UNiagaraNodeParameterMapSet* FNiagaraCompilationGraphBridge::AsParamMapSetNode(const FNode* Node)
{
	return Cast<const UNiagaraNodeParameterMapSet>(Node);
}

bool FNiagaraCompilationGraphBridge::GraphHasParametersOfType(const FGraph* Graph, const FNiagaraTypeDefinition& TypeDef)
{
	TArray<FNiagaraVariable> Inputs;
	TArray<FNiagaraVariable> Outputs;
	Graph->GetParameters(Inputs, Outputs);

	auto ContainsVariableOfType = [TypeDef](const FNiagaraVariable& Variable) -> bool
	{
		return Variable.GetType() == TypeDef;
	};

	return Inputs.ContainsByPredicate(ContainsVariableOfType)
		|| Outputs.ContainsByPredicate(ContainsVariableOfType);
}

TArray<FNiagaraVariableBase> FNiagaraCompilationGraphBridge::GraphGetStaticSwitchInputs(const FGraph* Graph)
{
	TArray<FNiagaraVariableBase> StaticSwitchInputs;
	const TArray<FNiagaraVariable>& Variables = Graph->FindStaticSwitchInputs();

	StaticSwitchInputs.Reserve(Variables.Num());
	Algo::Transform(Variables, StaticSwitchInputs, [](const FNiagaraVariable& InVar) -> FNiagaraVariableBase
	{
		return InVar;
	});

	return StaticSwitchInputs;
}

void FNiagaraCompilationGraphBridge::FindOutputNodes(const FGraph* Graph, ENiagaraScriptUsage ScriptUsage, TArray<const FOutputNode*>& OutputNodes)
{
	TArray<FOutputNode*> MutableOutputNodes;
	Graph->FindOutputNodes(ScriptUsage, MutableOutputNodes);
	OutputNodes.Reserve(MutableOutputNodes.Num());
	Algo::Transform(MutableOutputNodes, OutputNodes, [](FOutputNode* OutputNode) -> const FOutputNode*
	{
		return OutputNode;
	});
}

void FNiagaraCompilationGraphBridge::FindOutputNodes(const FGraph* Graph, TArray<const FOutputNode*>& OutputNodes)
{
	TArray<FOutputNode*> MutableOutputNodes;
	Graph->FindOutputNodes(MutableOutputNodes);
	OutputNodes.Reserve(MutableOutputNodes.Num());
	Algo::Transform(MutableOutputNodes, OutputNodes, [](FOutputNode* OutputNode) -> const FOutputNode*
	{
		return OutputNode;
	});
}

const FNiagaraCompilationGraphBridge::FOutputNode* FNiagaraCompilationGraphBridge::FindOutputNode(const FGraph* Graph, ENiagaraScriptUsage ScriptUsage, const FGuid& UsageId)
{
	TArray<FOutputNode*> MutableOutputNodes;
	Graph->FindOutputNodes(MutableOutputNodes);
	UNiagaraNodeOutput** FoundEntry = Algo::FindByPredicate(MutableOutputNodes, [ScriptUsage, UsageId](FOutputNode* OutputNode) -> bool
	{
		return OutputNode->GetUsage() == ScriptUsage && OutputNode->GetUsageId() == UsageId;
	});

	return FoundEntry ? *FoundEntry : nullptr;
}

void FNiagaraCompilationGraphBridge::BuildTraversal(const FGraph* Graph, const FNode* OutputNode, TArray<const FNode*>& TraversedNodes)
{
	TArray<UNiagaraNode*> MutableTraversedNodes;
	Graph->BuildTraversal(MutableTraversedNodes, const_cast<FNode*>(OutputNode), true);
	TraversedNodes.Reserve(MutableTraversedNodes.Num());
	Algo::Transform(MutableTraversedNodes, TraversedNodes, [](UNiagaraNode* Node) -> const FNode*
	{
		return Node;
	});
}

const UNiagaraGraph* FNiagaraCompilationGraphBridge::GetEmitterGraph(const FEmitterNode* EmitterNode)
{
	return EmitterNode->GetCalledGraph();
}

FNiagaraEmitterID FNiagaraCompilationGraphBridge::GetEmitterID(const FEmitterNode* EmitterNode)
{
	return EmitterNode->GetEmitterID();
}

FString FNiagaraCompilationGraphBridge::GetEmitterUniqueName(const FEmitterNode* EmitterNode)
{
	return EmitterNode->GetEmitterUniqueName();
}

ENiagaraScriptUsage FNiagaraCompilationGraphBridge::GetEmitterUsage(const FEmitterNode* EmitterNode)
{
	return EmitterNode->GetUsage();
}

FString FNiagaraCompilationGraphBridge::GetEmitterName(const FEmitterNode* EmitterNode)
{
	return EmitterNode->GetName();
}

FString FNiagaraCompilationGraphBridge::GetEmitterPathName(const FEmitterNode* EmitterNode)
{
	return EmitterNode->GetFullName();
}

FString FNiagaraCompilationGraphBridge::GetEmitterHandleIdString(const FEmitterNode* EmitterNode)
{
	return EmitterNode->GetEmitterHandleId().ToString(EGuidFormats::Digits);
}

const UNiagaraGraph* FNiagaraCompilationGraphBridge::GetFunctionNodeGraph(const FFunctionCallNode* FunctionCall)
{
	return FunctionCall->GetCalledGraph();
}

FString FNiagaraCompilationGraphBridge::GetFunctionFullName(const FFunctionCallNode* FunctionCall)
{
	return FunctionCall->FunctionScript ? FunctionCall->FunctionScript->GetFullName() : FString();
}

FString FNiagaraCompilationGraphBridge::GetFunctionScriptName(const FFunctionCallNode* FunctionCall)
{
	return FunctionCall->FunctionScript ? FunctionCall->FunctionScript->GetName() : FString();
}

FString FNiagaraCompilationGraphBridge::GetFunctionName(const FFunctionCallNode* FunctionCall)
{
	return FunctionCall ? FunctionCall->GetFunctionName() : FString();
}

ENiagaraScriptUsage FNiagaraCompilationGraphBridge::GetFunctionUsage(const FFunctionCallNode* FunctionCall)
{
	return FunctionCall->GetCalledUsage();
}

TOptional<ENiagaraDefaultMode> FNiagaraCompilationGraphBridge::GetGraphDefaultMode(const FGraph* Graph, const FNiagaraVariable& Variable, FNiagaraScriptVariableBinding& Binding)
{
	return Graph->GetDefaultMode(Variable, &Binding);
}

const UEdGraphPin* FNiagaraCompilationGraphBridge::GetDefaultPin(const FParamMapGetNode* GetNode, const FOutputPin* OutputPin)
{
	return GetNode->GetDefaultPin(const_cast<FOutputPin*>(OutputPin));
}

bool FNiagaraCompilationGraphBridge::IsStaticPin(const FPin* Pin)
{
	return UEdGraphSchema_Niagara::IsStaticPin(Pin);
}

// retrieves all input pins (excluding any add pins that may be present)
TArray<const UEdGraphPin*> FNiagaraCompilationGraphBridge::GetInputPins(const FNode* Node)
{
	TArray<const FInputPin*> InputPins;
	NiagaraCompilationGraphBridgeImpl::AppendCompilationPins(Node, EEdGraphPinDirection::EGPD_Input, InputPins);
	return InputPins;
}

// retrieves all output pins (excluding both orphaned pins and add pins)
TArray<const UEdGraphPin*> FNiagaraCompilationGraphBridge::GetOutputPins(const FNode* Node)
{
	TArray<const FInputPin*> OutputPins;
	NiagaraCompilationGraphBridgeImpl::AppendCompilationPins(Node, EEdGraphPinDirection::EGPD_Output, OutputPins);
	return OutputPins;
}

// gets all pins assoicated with the node
TArray<const UEdGraphPin*> FNiagaraCompilationGraphBridge::GetPins(const FNode* Node)
{
	TArray<const FPin*> Pins;
	Pins.Reserve(Node->Pins.Num());
	Algo::Transform(Node->Pins, Pins, [](FPin* InPin) -> const FPin*
		{
			return InPin;
		});
	return Pins;
}

FNiagaraTypeDefinition FNiagaraCompilationGraphBridge::GetPinType(const FPin* Pin, ENiagaraStructConversion Conversion)
{
	return UEdGraphSchema_Niagara::PinToTypeDefinition(Pin, Conversion);
}

FText FNiagaraCompilationGraphBridge::GetPinFriendlyName(const FPin* Pin)
{
	return Pin->PinFriendlyName;
}

FText FNiagaraCompilationGraphBridge::GetPinDisplayName(const FPin* Pin)
{
	return Pin->GetDisplayName();
}

FNiagaraVariable FNiagaraCompilationGraphBridge::GetPinVariable(const FPin* Pin, bool bNeedsValue, ENiagaraStructConversion Conversion)
{
	return UEdGraphSchema_Niagara::PinToNiagaraVariable(Pin, bNeedsValue, Conversion);
}

const UEdGraphPin* FNiagaraCompilationGraphBridge::GetPinAsInput(const FPin* Pin)
{
	if (Pin && Pin->Direction == EGPD_Input)
	{
		return Pin;
	}
	return nullptr;
}

FNiagaraVariable FNiagaraCompilationGraphBridge::GetInputVariable(const FInputNode* InputNode)
{
	return InputNode->Input;
}

const TArray<FNiagaraVariable>& FNiagaraCompilationGraphBridge::GetOutputVariables(const FOutputNode* OutputNode)
{
	return OutputNode->GetOutputs();
}

TArray<FNiagaraVariable> FNiagaraCompilationGraphBridge::GetGraphOutputNodeVariables(const FGraph* Graph, ENiagaraScriptUsage Usage)
{
	TArray<FNiagaraVariable> OutputNodeVariables;
	Graph->GetOutputNodeVariables(Usage, OutputNodeVariables);
	return OutputNodeVariables;
}

TArray<const UNiagaraNodeInput*> FNiagaraCompilationGraphBridge::GetGraphInputNodes(const FGraph* Graph, const UNiagaraGraph::FFindInputNodeOptions& Options)
{
	TArray<FInputNode*> MutableInputNodes;
	Graph->FindInputNodes(MutableInputNodes, Options);

	TArray<const FInputNode*> InputNodes;
	InputNodes.Reserve(MutableInputNodes.Num());

	Algo::Transform(MutableInputNodes, InputNodes, [](FInputNode* InputNode) -> const FInputNode*
		{
			return InputNode;
		});

	return InputNodes;
}

const UEdGraphPin* FNiagaraCompilationGraphBridge::GetLinkedOutputPin(const FInputPin* InputPin)
{
	if (InputPin && !InputPin->LinkedTo.IsEmpty())
	{
		return InputPin->LinkedTo[0];
	}
	return nullptr;
}

bool FNiagaraCompilationGraphBridge::CanCreateConnection(const FOutputPin* OutputPin, const FInputPin* InputPin, FText& FailureMessage)
{
	FPinConnectionResponse PinResponse = GetDefault<UEdGraphSchema_Niagara>()->CanCreateConnection(OutputPin, InputPin);
	if (PinResponse.Response == ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW)
	{
		FailureMessage = PinResponse.Message;
		return false;
	}

	return true;
}

ENiagaraScriptUsage FNiagaraCompilationGraphBridge::GetOutputNodeUsage(const FOutputNode* OutputNode)
{
	return OutputNode->GetUsage();
}

FGuid FNiagaraCompilationGraphBridge::GetOutputNodeUsageId(const FOutputNode* OutputNode)
{
	return OutputNode->GetUsageId();
}

ENiagaraScriptUsage FNiagaraCompilationGraphBridge::GetOutputNodeScriptType(const FOutputNode* OutputNode)
{
	return OutputNode->ScriptType;
}

FGuid FNiagaraCompilationGraphBridge::GetOutputNodeScriptTypeId(const FOutputNode* OutputNode)
{
	return OutputNode->ScriptTypeId;
}

bool FNiagaraCompilationGraphBridge::IsGraphEmpty(const FGraph* Graph)
{
	return Graph->IsEmpty();
}

void FNiagaraCompilationGraphBridge::AddCollectionPaths(const FParamMapHistory& History, TArray<FString>& Paths)
{
	for (UNiagaraParameterCollection* Collection : History.EncounteredParameterCollections.Collections)
	{
		Paths.AddUnique(FSoftObjectPath(Collection).ToString());
	}
}

bool FNiagaraCompilationGraphBridge::NodeIsEnabled(const FNode* Node)
{
	return Node->IsNodeEnabled();
}

TOptional<ENiagaraDefaultMode> FNiagaraCompilationGraphBridge::GraphGetDefaultMode(const FGraph* Graph, const FNiagaraVariableBase& Variable, FNiagaraScriptVariableBinding& Binding)
{
	return Graph->GetDefaultMode(Variable, &Binding);
}

const UEdGraphPin* FNiagaraCompilationGraphBridge::GetSelectOutputPin(const FSelectNode* SelectNode, const FNiagaraVariableBase& Variable)
{
	return SelectNode->GetOutputPin(Variable);
}

FString FNiagaraCompilationGraphBridge::GetNodeName(const FNode* Node)
{
	return Node->GetName();
}

FString FNiagaraCompilationGraphBridge::GetNodeTitle(const FNode* Node)
{
	return Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString();
}

const UEdGraphPin* FNiagaraCompilationGraphBridge::GetInputPin(const FNode* Node, int32 PinIndex)
{
	return Node->GetInputPin(PinIndex);
}

int32 FNiagaraCompilationGraphBridge::GetPinIndexById(TConstArrayView<const FPin*> Pins, const FGuid& PinId)
{
	const int32 PinCount = Pins.Num();
	for (int32 PinIt = 0; PinIt < PinCount; ++PinIt)
	{
		if (Pins[PinIt]->PinId == PinId)
		{
			return PinIt;
		}
	}
	return INDEX_NONE;
}

FString FNiagaraCompilationGraphBridge::GetCollectionFullName(FParameterCollection Collection)
{
	return GetFullNameSafe(Collection);
}

bool FNiagaraCompilationGraphBridge::IsCollectionValid(FParameterCollection Collection)
{
	return IsValid(Collection);
}

UNiagaraDataInterface* FNiagaraCompilationGraphBridge::GetCollectionDataInterface(FParameterCollection Collection, const FNiagaraVariable& Variable)
{
	return Collection->GetDefaultInstance()->GetParameterStore().GetDataInterface(Variable);
}

TObjectPtr<UObject> FNiagaraCompilationGraphBridge::GetCollectionUObject(FParameterCollection Collection, const FNiagaraVariable& Variable)
{
	return Collection->GetDefaultInstance()->GetParameterStore().GetUObject(Variable);
}

FNiagaraCompilationGraphBridge::FParameterCollection FNiagaraCompilationGraphBridge::IsParameterCollectionParameter(const FParamMapHistory& ParamMapHistory, const FNiagaraVariable& InVar, bool& bMissingParameter)
{
	if (ParamMapHistory.bParameterCollectionsSkipped)
	{
		UE_LOGF(LogNiagaraEditor, Error, "NiagaraParameterCollection was skipped during history building.  History which require NPC data can not be generated during PostLoad().");
		return nullptr;
	}

	bMissingParameter = false;
	FString VarName = InVar.GetName().ToString();
	for (int32 i = 0; i < ParamMapHistory.EncounteredParameterCollections.Collections.Num(); ++i)
	{
		FNameBuilder CollectionNameBuilder(ParamMapHistory.EncounteredParameterCollections.CollectionNamespaces[i]);

		if (VarName.StartsWith(CollectionNameBuilder))
		{
			bMissingParameter = !ParamMapHistory.EncounteredParameterCollections.CollectionVariables[i].Contains(InVar);
			return ParamMapHistory.EncounteredParameterCollections.Collections[i];
		}
	}
	return nullptr;
}

const UNiagaraNodeOutput* FNiagaraCompilationGraphBridge::AsOutputNode(const FNode* Node)
{
	return Cast<const FOutputNode>(Node);
}

bool FNiagaraCompilationGraphBridge::IsParameterMapGet(const FNode* Node)
{
	return Node->IsA<UNiagaraNodeParameterMapGet>();
}

TOptional<FName> FNiagaraCompilationGraphBridge::GetOutputNodeStackContextOverride(const FOutputNode* OutputNode)
{
	return OutputNode->GetStackContextOverride();
}

FString FNiagaraCompilationGraphBridge::GetNodeClassName(const FNode* Node)
{
	return Node->GetClass()->GetName();
}

bool FNiagaraCompilationGraphBridge::IsParameterMapPin(const FPin* Pin)
{
	return UEdGraphSchema_Niagara::PinToTypeDefinition(Pin) == FNiagaraTypeDefinition::GetParameterMapDef();
}

bool FNiagaraCompilationGraphBridge::GetGraphReferencesStaticVariables(const FGraph* Graph, FNiagaraStaticVariableSearchContext& StaticVariableContext)
{
	return Graph->ReferencesStaticVariable(StaticVariableContext);
}

const UNiagaraNodeEmitter* FNiagaraCompilationGraphBridge::GetNodeAsEmitter(const FNode* Node)
{
	return Cast<const UNiagaraNodeEmitter>(Node);
}

bool FNiagaraCompilationGraphBridge::GetCustomNodeUsesImpureFunctions(const UNiagaraNodeCustomHlsl* CustomNode)
{
	return CustomNode->CallsImpureDataInterfaceFunctions();
}

UNiagaraParameterCollection* FNiagaraCompilationGraphBridge::FAvailableParameterCollections::FindCollection(const FNiagaraVariable& Variable) const
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	return Schema->VariableIsFromParameterCollection(Variable);
}

UNiagaraParameterCollection* FNiagaraCompilationGraphBridge::FAvailableParameterCollections::FindMatchingCollection(FName VariableName, bool bAllowPartialMatch, FNiagaraVariable& OutVar) const
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	return Schema->VariableIsFromParameterCollection(VariableName.ToString(), true, OutVar);
}
