// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraDigestDatabase.h"
#include "NiagaraGraphDigestTypes.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraParameterMapHistory.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraSimulationStageCompileData.h"
#include "NiagaraTypes.h"
#include "Templates/SharedPointer.h"

class FCompileConstantResolver;
class FNiagaraCompilationGraph;
class FNiagaraCompilationNodeEmitter;
class FNiagaraCompileRequestData;
class FNiagaraCompileRequestDuplicateData;
struct FNiagaraCompilationDigestBridge;
class FNiagaraFixedConstantResolver;
enum class ENiagaraFunctionDebugState : uint8;
class UNiagaraGraph;
struct FNiagaraGraphCachedBuiltHistory;
template<typename GraphInterface> class TNiagaraHlslTranslator;
class UNiagaraScriptSourceBase;
struct FNiagaraSystemCompilationTask;

// a version of the FCompileConstantResolver that is immutable (the constants are evaluated at the time of
// construction rather than on demand).  Useful for digested graphs where their state is also immutable and
// means we don't need to hold and reference UObjects (like UNiagaraSystem & UNiagaraEmitter)
class FNiagaraFixedConstantResolver
{
public:
	using FTranslator = TNiagaraHlslTranslator<FNiagaraCompilationDigestBridge>;

	FNiagaraFixedConstantResolver();
	FNiagaraFixedConstantResolver(const FTranslator* InTranslator, ENiagaraScriptUsage InUsage = ENiagaraScriptUsage::Function, ENiagaraFunctionDebugState InDebugState = ENiagaraFunctionDebugState::NoDebug);
	FNiagaraFixedConstantResolver(const FCompileConstantResolver& SrcConstantResolver);

	bool ResolveConstant(FNiagaraVariable& OutConstant) const;

	FNiagaraFixedConstantResolver WithDebugState(ENiagaraFunctionDebugState InDebugState) const;
	FNiagaraFixedConstantResolver WithUsage(ENiagaraScriptUsage ScriptUsage) const;

	ENiagaraFunctionDebugState GetDebugState() const;

	void AddChildResolver(const FGuid& ChildId, const FNiagaraFixedConstantResolver& ChildResolver);
	const FNiagaraFixedConstantResolver* FindChildResolver(const FGuid& ChildId) const;

private:
	void InitConstants();
	void SetScriptUsage(ENiagaraScriptUsage ScriptUsage);
	void SetDebugState(ENiagaraFunctionDebugState DebugState);

	const FTranslator* Translator = nullptr;

	enum class EResolvedConstant : uint8
	{
		FunctionDebugState = 0,
		ScriptUsage,
		ScriptContext,
		EmitterLocalspace,
		EmitterDeterminism,
		EmitterInterpolatedSpawn,
		EmitterSimulationTarget,
		Count
	};

	TArray<FNiagaraVariable, TFixedAllocator<(uint8)EResolvedConstant::Count>> ResolvedConstants;

	using FNamedResolverPair = TTuple<FGuid, FNiagaraFixedConstantResolver>;
	TArray<FNamedResolverPair> ChildResolvers;
};

struct FNiagaraSimulationStageInfo
{
	FGuid StageId;
	bool bEnabled = true;
	bool bGenericStage = false;
	ENiagaraIterationSource IterationSource = ENiagaraIterationSource::Particles;
	FName DataInterfaceBindingName;
	bool bHasCompilationData = false;
	FNiagaraSimulationStageCompilationData CompilationData;
};

//////////////////////////////////////////////////////////////////////////
// Helper structures for the FNiagaraActiveCompilationAsyncTask
//////////////////////////////////////////////////////////////////////////

// holds the precompile data for the compilation of a NiagaraSystem
class FNiagaraPrecompileData
{
public:
	using FSharedPrecompileData = TSharedPtr<FNiagaraPrecompileData, ESPMode::ThreadSafe>;

	void GatherPreCompiledVariables(const FString& InNamespaceFilter, TArray<FNiagaraVariable>& OutVars) const;
	FName ResolveEmitterAlias(FName VariableName) const;

	const FString& GetUniqueEmitterName() const { return EmitterUniqueName; }
	FNiagaraEmitterID GetEmitterID()const { return EmitterID; }

	void FinishPrecompile(
		const FNiagaraSystemCompilationTask& CompilationTask,
		TConstArrayView<FNiagaraVariable> EncounterableVariables,
		TConstArrayView<FNiagaraVariable> InStaticVariables,
		const FNiagaraFixedConstantResolver& ConstantResolver,
		TConstArrayView<ENiagaraScriptUsage> UsagesToProcess,
		TConstArrayView<FNiagaraSimulationStageInfo> SimStages,
		TConstArrayView<FString> EmitterNames);

	void CollectBakedRapidIterationParameters(const FNiagaraSystemCompilationTask& CompilationTask, TConstArrayView<TObjectKey<UNiagaraScript>> OwnedScriptKeys);

	int32 GetDependentRequestCount() const
	{
		return EmitterData.Num();
	};
	FSharedPrecompileData GetDependentRequest(int32 Index)
	{
		return EmitterData[Index];
	}
	const FNiagaraPrecompileData* GetDependentRequest(int32 Index) const
	{
		return EmitterData[Index].Get();
	}

	bool GetUseRapidIterationParams() const { return bUseRapidIterationParams; }

	bool GetDisableDebugSwitches() const { return bDisableDebugSwitches; }

	bool Matches(const FNiagaraCompileRequestData& Other) const;

	// Simulation Stage Variables. Sim stage of 0 is always Spawn/Update
	TArray<FNiagaraSimulationStageCompilationData> CompileSimStageData;

	struct FCompileDataInterfaceData
	{
		FString EmitterName;
		ENiagaraScriptUsage Usage;
		FGuid UsageId;
		FNiagaraVariable Variable;
		TArray<FString> ReadsEmitterParticleData;
	};
	TSharedPtr<TArray<FCompileDataInterfaceData>> SharedCompileDataInterfaceData;

	TArray<FNiagaraVariable> EncounteredVariables;
	FString EmitterUniqueName;
	FNiagaraEmitterID EmitterID;
	TArray<FSharedPrecompileData> EmitterData;
	FNiagaraDigestedGraphPtr DigestedSourceGraph;
	FString SourceName;
	bool bUseRapidIterationParams = true;
	bool bDisableDebugSwitches = false;

	UEnum* ENiagaraScriptCompileStatusEnum = nullptr;
	UEnum* ENiagaraScriptUsageEnum = nullptr;

	TMap<FGraphTraversalHandle, FString> PinToConstantValues;
	TArray<FNiagaraVariable> StaticVariables;
	TArray<FNiagaraVariable> StaticVariablesWithMultipleWrites;

	TArray<FNiagaraVariable> BakedRapidIterationParameters;

	static void SortOutputNodesByDependencies(TArray<const FNiagaraCompilationNodeOutput*>& NodesToSort, TConstArrayView<FNiagaraSimulationStageInfo> SimStages);

	template<typename T> T GetStaticVariableValue(const FNiagaraVariableBase Variable, const T DefaultValue) const
	{
		const int32 Index = StaticVariables.Find(Variable);
		return Index != INDEX_NONE && StaticVariables[Index].IsDataAllocated() ? StaticVariables[Index].GetValue<T>() : DefaultValue;
	}
};


class FNiagaraCompilationCopyData
{
public:
	using FSharedCompilationCopy = TSharedPtr<FNiagaraCompilationCopyData, ESPMode::ThreadSafe>;
	using FParameterMapHistory = TNiagaraParameterMapHistory<FNiagaraCompilationDigestBridge>;
	using FParameterMapHistoryWithMetaDataBuilder = TNiagaraParameterMapHistoryWithMetaDataBuilder<FNiagaraCompilationDigestBridge>;

	UNiagaraDataInterface* GetDuplicatedDataInterfaceCDOForClass(UClass* Class) const;

	TArray<FParameterMapHistory>& GetPrecomputedHistories() { return PrecompiledHistories; }
	const TArray<FParameterMapHistory>& GetPrecomputedHistories() const { return PrecompiledHistories; }

	void InstantiateCompilationCopy(const FNiagaraCompilationGraphDigested& SourceGraph, const FNiagaraPrecompileData* PrecompileData, ENiagaraScriptUsage InUsage, const FNiagaraFixedConstantResolver& ConstantResolver);
	void CreateParameterMapHistory(const FNiagaraSystemCompilationTask& CompilationTask, const TArray<FNiagaraVariable>& EncounterableVariables, const TArray<FNiagaraVariable>& InStaticVariables, const FNiagaraFixedConstantResolver& ConstantResolver, TConstArrayView<FNiagaraSimulationStageInfo> SimStages);

	int32 GetDependentRequestCount() const { return EmitterData.Num(); }
	FSharedCompilationCopy GetDependentRequest(int32 Index) { return EmitterData[Index]; }

	TArray<ENiagaraScriptUsage> ValidUsages;

	TSharedPtr<FNiagaraCompilationGraphInstanced, ESPMode::ThreadSafe> InstantiatedGraph;

	TArray<FParameterMapHistory> PrecompiledHistories;

	TArray<FNiagaraVariable> ChangedFromNumericVars;
	FString EmitterUniqueName;

	TArray<FSharedCompilationCopy> EmitterData;

	TMap<TObjectPtr<UClass>, TObjectPtr<UNiagaraDataInterface>> AggregatedDataInterfaceCDODuplicates;
};

//////////////////////////////////////////////////////////////////////////
// Helper structures for the FNiagaraActiveCompilationDefault
//////////////////////////////////////////////////////////////////////////

// Implements the FNiagaraCompileRequestDataBase interface that is used to represent the precompile data
// in order to perform the compilation of a Niagara asset.
class FNiagaraCompileRequestData : public FNiagaraCompileRequestDataBase
{
public:
	FNiagaraCompileRequestData()
	{

	}

	virtual void GatherPreCompiledVariables(const FString& InNamespaceFilter, TArray<FNiagaraVariable>& OutVars) const override;
	virtual FName ResolveEmitterAlias(FName VariableName) const override;

	const FString& GetUniqueEmitterName() const { return EmitterUniqueName; }
	void FinishPrecompile(const TArray<FNiagaraVariable>& EncounterableVariables, const TArray<FNiagaraVariable>& InStaticVariables, FCompileConstantResolver ConstantResolver, const TArray<ENiagaraScriptUsage>& UsagesToProcess, const TArray<class UNiagaraSimulationStageBase*>* SimStages, const TArray<FString> EmitterNames);
	virtual int32 GetDependentRequestCount() const override {
		return EmitterData.Num();
	};
	virtual TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> GetDependentRequest(int32 Index) override {
		return EmitterData[Index];
	}
	virtual const FNiagaraCompileRequestDataBase* GetDependentRequest(int32 Index) const override
	{
		return EmitterData[Index].Get();
	}
	void AddRapidIterationParameters(const FNiagaraParameterStore& InParamStore, FCompileConstantResolver InResolver);
	virtual bool GetUseRapidIterationParams() const override { return bUseRapidIterationParams; }

	virtual bool GetDisableDebugSwitches() const override { return bDisableDebugSwitches; }

	// Simulation Stage Variables. Sim stage of 0 is always Spawn/Update
	TArray<FNiagaraSimulationStageCompilationData> CompileSimStageData;

	struct FCompileDataInterfaceData
	{
		FString EmitterName;
		ENiagaraScriptUsage Usage;
		FGuid UsageId;
		FNiagaraVariable Variable;
		TArray<FString> ReadsEmitterParticleData;
	};
	TSharedPtr<TArray<FCompileDataInterfaceData>> SharedCompileDataInterfaceData;

	TArray<FNiagaraVariable> EncounteredVariables;
	FString EmitterUniqueName;
	FNiagaraEmitterID EmitterID;
	TArray<TSharedPtr<FNiagaraCompileRequestData, ESPMode::ThreadSafe>> EmitterData;
	TWeakObjectPtr<UNiagaraScriptSource> Source;
	FString SourceName;
	bool bUseRapidIterationParams = true;
	bool bDisableDebugSwitches = false;

	UEnum* ENiagaraScriptCompileStatusEnum = nullptr;
	UEnum* ENiagaraScriptUsageEnum = nullptr;

	TArray<FNiagaraVariable> RapidIterationParams;

	TMap<FGraphTraversalHandle, FString> PinToConstantValues;
	TArray<FNiagaraVariable> StaticVariables;
	TArray<FNiagaraVariable> StaticVariablesWithMultipleWrites;

	template<typename T> T GetStaticVariableValue(const FNiagaraVariableBase Variable, const T DefaultValue) const
	{
		const int32 Index = StaticVariables.Find(Variable);
		return Index != INDEX_NONE && StaticVariables[Index].IsDataAllocated() ? StaticVariables[Index].GetValue<T>() : DefaultValue;
	}

	void CompareAgainst(FNiagaraGraphCachedBuiltHistory* InCachedDataBase);

	static void SortOutputNodesByDependencies(TArray<class UNiagaraNodeOutput*>& NodesToSort, const TArray<class UNiagaraSimulationStageBase*>* SimStages);
};

// Implements the FNiagaraCompileRequestDuplicateDataBase interface that is used to represent the data that
// is duplicated (for example graphs) in order to perform the compilation of a Niagara asset.
class FNiagaraCompileRequestDuplicateData : public FNiagaraCompileRequestDuplicateDataBase
{
public:
	FNiagaraCompileRequestDuplicateData()
	{
	}

	virtual bool IsDuplicateDataFor(UNiagaraSystem* InSystem, UNiagaraEmitter* InEmitter, UNiagaraScript* InScript) const override;
	virtual void GetDuplicatedObjects(TArray<UObject*>& Objects) override;
	virtual const TMap<FName, UNiagaraDataInterface*>& GetObjectNameMap() override;
	virtual const UNiagaraScriptSourceBase* GetScriptSource() const override;
	UNiagaraDataInterface* GetDuplicatedDataInterfaceCDOForClass(UClass* Class) const;

	TArray<FNiagaraParameterMapHistory>& GetPrecomputedHistories() { return PrecompiledHistories; }
	const TArray<FNiagaraParameterMapHistory>& GetPrecomputedHistories() const { return PrecompiledHistories; }

	void DuplicateReferencedGraphs(UNiagaraGraph* InSrcGraph, UNiagaraGraph* InDupeGraph, ENiagaraScriptUsage InUsage, FCompileConstantResolver ConstantResolver, TMap<UNiagaraNodeFunctionCall*, ENiagaraScriptUsage> FunctionsWithUsage = TMap<UNiagaraNodeFunctionCall*, ENiagaraScriptUsage>());
	void DeepCopyGraphs(UNiagaraScriptSource* ScriptSource, ENiagaraScriptUsage InUsage, FCompileConstantResolver ConstantResolver);
	void DeepCopyGraphs(const FVersionedNiagaraEmitter& Emitter);

	void FinishPrecompileDuplicate(const TArray<FNiagaraVariable>& EncounterableVariables, const TArray<FNiagaraVariable>& InStaticVariables, FCompileConstantResolver ConstantResolver, const TArray<class UNiagaraSimulationStageBase*>* SimStages, const TArray<FNiagaraVariable>& InParamStore);
	void CreateDataInterfaceCDO(TArrayView<UClass*> VariableDataInterfaces);

	virtual int32 GetDependentRequestCount() const override { return EmitterData.Num(); }
	virtual TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> GetDependentRequest(int32 Index) override { return EmitterData[Index]; }

	virtual void ReleaseCompilationCopies() override;

	TWeakObjectPtr<UNiagaraSystem> OwningSystem;
	TWeakObjectPtr<UNiagaraEmitter> OwningEmitter;
	TArray<ENiagaraScriptUsage> ValidUsages;

	TWeakObjectPtr<UNiagaraScriptSource> SourceDeepCopy;
	TWeakObjectPtr<UNiagaraGraph> NodeGraphDeepCopy;

	TArray<FNiagaraParameterMapHistory> PrecompiledHistories;

	TArray<FNiagaraVariable> ChangedFromNumericVars;
	FString EmitterUniqueName;
	FNiagaraEmitterID EmitterID;
	TArray<TSharedPtr<FNiagaraCompileRequestDuplicateData, ESPMode::ThreadSafe>> EmitterData;

	struct FDuplicatedGraphData
	{
		UNiagaraScript* ClonedScript;
		UNiagaraGraph* ClonedGraph;
		TArray<UEdGraphPin*> CallInputs;
		TArray<UEdGraphPin*> CallOutputs;
		ENiagaraScriptUsage Usage;
		bool bHasNumericParameters;
	};

	TSharedPtr<TMap<const UNiagaraGraph*, TArray<FDuplicatedGraphData>>> SharedSourceGraphToDuplicatedGraphsMap;
	TSharedPtr<TMap<FName, UNiagaraDataInterface*>> SharedNameToDuplicatedDataInterfaceMap;
	TSharedPtr<TMap<UClass*, UNiagaraDataInterface*>> SharedDataInterfaceClassToDuplicatedCDOMap;

protected:
	void DuplicateReferencedGraphsRecursive(UNiagaraGraph* InGraph, const FCompileConstantResolver& ConstantResolver, TMap<UNiagaraNodeFunctionCall*, ENiagaraScriptUsage> FunctionsWithUsage);
private:
	TArray<TWeakObjectPtr<UNiagaraScriptSource>> TrackedScriptSourceCopies;
};

enum class ENiagaraCompileRIParamMode : uint8
{
	// the issue here is that we aren't strictly allowed to call ComputeVMCompilationId on the worker threads which leaves us in a
	// state where we could have broken RI parameters.  We ensure that the RI parameters are up to date before we begin.  This is a
	// costly operation, but sadly seems necessary for some content.
	GameThread = 0,

	// for now we need RI generation to be synchronous to match the behavior of the default
	// compilation mode.  In the future we need to get RI preparation out of the compilation
	// and instead we are provided with overridden RI values and don't need to feed back the
	// entire list of RI parameters.
	WorkerThread_Synchronous,

	// desirable behavior that currently doesn't work
	WorkerThread_Asynchronous,
};
