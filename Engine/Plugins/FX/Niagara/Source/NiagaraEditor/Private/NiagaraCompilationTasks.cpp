// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCompilationTasks.h"

#include "Algo/RemoveIf.h"
#include "AssetCompilingManager.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Logging/StructuredLog.h"
#include "Misc/PathViews.h"
#include "NiagaraCompilationTypes.h"
#include "NiagaraDigestDatabase.h"
#include "NiagaraEmitter.h"
#include "NiagaraGraphDigest.h"
#include "NiagaraCompilationBridge.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSettings.h"
#include "NiagaraShader.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemCompilingManager.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ShaderSerialization.h"
#include "VectorVM.h"
#include "VectorVMBridge.h"
#include "VectorVMOptimizer.h"

// needed for AsType...pretty ugly
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeFunctionCall.h"

static int GNiagaraCompilationForceOneThread = 0;
static FAutoConsoleVariableRef CVarNiagaraCompilationForceOneThread(
	TEXT("fx.Niagara.Compilation.ForceOneThread"),
	GNiagaraCompilationForceOneThread,
	TEXT("Forces the compile tasks to run inline on one thread."),
	ECVF_Default
);

#if ENABLE_COOK_STATS
namespace NiagaraSystemCookStats
{
	extern FCookStats::FDDCResourceUsageStats UsageStats;

	static std::atomic<int32> ScriptTranslationCount(0);
	static int32 CpuScriptCompileCount = 0;
	static int32 CpuScriptDdcCacheHitCount = 0;
	static int32 CpuScriptDdcCacheMissCount = 0;
	static int32 GpuScriptCompileCount = 0;
	static int32 GpuScriptDdcCacheHitCount = 0;
	static int32 GpuScriptDdcCacheMissCount = 0;

	static FCookStatsManager::FAutoRegisterCallback RegisterNiagaraCompilationCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		AddStat(TEXT("Niagara"), FCookStatsManager::CreateKeyValueArray(TEXT("NiagaraScriptTranslationCount"), ScriptTranslationCount.load(std::memory_order_relaxed)));
		AddStat(TEXT("Niagara"), FCookStatsManager::CreateKeyValueArray(TEXT("NiagaraCpuScriptCompileCount"), CpuScriptCompileCount));
		AddStat(TEXT("Niagara"), FCookStatsManager::CreateKeyValueArray(TEXT("NiagaraCpuScriptDdcCacheHitCount"), CpuScriptDdcCacheHitCount));
		AddStat(TEXT("Niagara"), FCookStatsManager::CreateKeyValueArray(TEXT("NiagaraCpuScriptDdcCacheMissCount"), CpuScriptDdcCacheMissCount));
		AddStat(TEXT("Niagara"), FCookStatsManager::CreateKeyValueArray(TEXT("NiagaraGpuScriptCompileCount"), GpuScriptCompileCount));
		AddStat(TEXT("Niagara"), FCookStatsManager::CreateKeyValueArray(TEXT("NiagaraGpuScriptDdcCacheHitCount"), GpuScriptDdcCacheHitCount));
		AddStat(TEXT("Niagara"), FCookStatsManager::CreateKeyValueArray(TEXT("NiagaraGpuScriptDdcCacheMissCount"), GpuScriptDdcCacheMissCount));
	});
}
#endif


namespace NiagaraCompilationTasksImpl
{
	static const FGuid UE_NIAGARA_ASYNC_TASK_COMPILER_VER = FGuid(0x73F57E8B, 0xD0AC44E8, 0xBEE372D4, 0x1CD85A69);
	
	static const UE::DerivedData::FCacheBucket NiagaraDDCBucket("NiagaraScript");
	static const UE::DerivedData::FValueId ScriptDataValue = UE::DerivedData::FValueId::FromName(TEXT("ScriptData"));

	static bool GDebugParseCompileMetaDataOnGet = false;
	static FAutoConsoleVariableRef CVarGDebugParseCompileMetaDataOnGet(
		TEXT("fx.Niagara.ParseCompileMetaDataOnGet"),
		GDebugParseCompileMetaDataOnGet,
		TEXT("When true the compilation metadata from DDC will be parsed on get."),
		ECVF_Default
	);

	void GetUsagesToDuplicate(ENiagaraScriptUsage TargetUsage, TArray<ENiagaraScriptUsage>& DuplicateUsages)
	{
		// For now we need to include both spawn and update for each target usage, otherwise attribute lists in the precompiled histories aren't generated correctly.
		switch (TargetUsage)
		{
		case ENiagaraScriptUsage::SystemSpawnScript:
		case ENiagaraScriptUsage::SystemUpdateScript:
			DuplicateUsages.Add(ENiagaraScriptUsage::SystemSpawnScript);
			DuplicateUsages.Add(ENiagaraScriptUsage::SystemUpdateScript);
			DuplicateUsages.Add(ENiagaraScriptUsage::EmitterSpawnScript);
			DuplicateUsages.Add(ENiagaraScriptUsage::EmitterUpdateScript);
			break;
		case ENiagaraScriptUsage::ParticleSpawnScript:
		case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
		case ENiagaraScriptUsage::ParticleUpdateScript:
		case ENiagaraScriptUsage::ParticleEventScript:
		case ENiagaraScriptUsage::ParticleSimulationStageScript:
		case ENiagaraScriptUsage::ParticleGPUComputeScript:
			DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSpawnScript);
			DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSpawnScriptInterpolated);
			DuplicateUsages.Add(ENiagaraScriptUsage::ParticleUpdateScript);
			DuplicateUsages.Add(ENiagaraScriptUsage::ParticleEventScript);
			DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSimulationStageScript);
			DuplicateUsages.Add(ENiagaraScriptUsage::ParticleGPUComputeScript);
			break;
		}
	}

	TArray<UNiagaraScript*> FindDependentScripts(UNiagaraSystem* System, FVersionedNiagaraEmitterData* EmitterData, UNiagaraScript* Script)
	{
		TArray<UNiagaraScript*> DependentScripts;

		const ENiagaraScriptUsage ScriptUsage = Script->GetUsage();

		if (System)
		{
			if (UNiagaraScript::IsEquivalentUsage(ScriptUsage, ENiagaraScriptUsage::EmitterSpawnScript))
			{
				DependentScripts.Add(System->GetSystemSpawnScript());
			}

			if (UNiagaraScript::IsEquivalentUsage(ScriptUsage, ENiagaraScriptUsage::EmitterUpdateScript))
			{
				DependentScripts.Add(System->GetSystemUpdateScript());
			}
		}

		if (EmitterData)
		{
			const bool bIsGpuEmitter = EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim;

			if (UNiagaraScript::IsEquivalentUsage(ScriptUsage, ENiagaraScriptUsage::ParticleSpawnScript))
			{
				if (bIsGpuEmitter)
				{
					DependentScripts.Add(EmitterData->GetGPUComputeScript());
				}
			}

			if (UNiagaraScript::IsEquivalentUsage(ScriptUsage, ENiagaraScriptUsage::ParticleUpdateScript))
			{
				if (bIsGpuEmitter)
				{
					DependentScripts.Add(EmitterData->GetGPUComputeScript());
				}
				else if (EmitterData->UsesInterpolatedSpawning())
				{
					DependentScripts.Add(EmitterData->SpawnScriptProps.Script);
				}
			}
		}

		return DependentScripts;
	}

	static UE::DerivedData::FCacheKey BuildNiagaraDDCCacheKey(const FNiagaraVMExecutableDataId& CompileId, const FString& ScriptPath)
	{
		FString KeyString;
		KeyString.Reserve(1024);

		NiagaraCompilationTasksImpl::UE_NIAGARA_ASYNC_TASK_COMPILER_VER.AppendString(KeyString);
		KeyString.AppendChar(TCHAR('_'));
		KeyString.Appendf(TEXT("%i"), GNiagaraSkipVectorVMBackendOptimizations);
		KeyString.AppendChar(TCHAR('_'));
		KeyString.Append(ScriptPath);
		KeyString.AppendChar(TCHAR('_'));

		CompileId.AppendKeyString(KeyString);

		return { NiagaraDDCBucket, FIoHash::HashBuffer(MakeMemoryView(FTCHARToUTF8(KeyString))) };
	}

	static UE::DerivedData::FCacheKey BuildNiagaraComputeDDCCacheKey(const FNiagaraShaderMapId& ShaderMapId, EShaderPlatform ShaderPlatform, const FString& ScriptPath)
	{
		static const FString NIAGARASHADERMAP_DERIVEDDATA_VER = FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().NIAGARASHADERMAP_DERIVEDDATA_VER).ToString(EGuidFormats::DigitsWithHyphens);

		FName Format = LegacyShaderPlatformToShaderFormat(ShaderPlatform);

		FString KeyString;
		KeyString.Reserve(1024);

		Format.ToString(KeyString);
		KeyString.Appendf(TEXT("_%d_"), GetTargetPlatformManagerRef().ShaderFormatVersion(Format));
		ShaderMapAppendKeyString(ShaderPlatform, KeyString);
		ShaderMapId.AppendKeyString(KeyString);
		KeyString.AppendChar(TCHAR('_'));
		KeyString.Append(ScriptPath);
		KeyString.AppendChar(TCHAR('_'));
		KeyString.Append(NIAGARASHADERMAP_DERIVEDDATA_VER);

		return { NiagaraDDCBucket, FIoHash::HashBuffer(MakeMemoryView(FTCHARToUTF8(KeyString))) };
	}

	static bool ValidateExecData(const UNiagaraScript* Script, const FNiagaraVMExecutableData& ExecData, FString& ErrorString)
	{
		bool IsValid = true;

		for (const auto& Attribute : ExecData.Attributes)
		{
			if (!Attribute.IsValid())
			{
				ErrorString.Appendf(TEXT("Failure - %s - Attribute [%s] is invalid!\n"), Script ? *Script->GetFullName() : TEXT("<unknown>"), *Attribute.GetName().ToString());
				IsValid = false;
			}
		}

		for (const auto& Parameter : ExecData.Parameters.Parameters)
		{
			if (!Parameter.IsValid())
			{
				ErrorString.Appendf(TEXT("Failure - %s - Parameter [%s] is invalid!\n"), Script ? *Script->GetFullName() : TEXT("<unknown>"), *Parameter.GetName().ToString());
				IsValid = false;
			}
		}

		return IsValid;
	}

	static bool BinaryToExecData(const UNiagaraScript* Script, FSharedBuffer SharedBuffer, FNiagaraVMExecutableData& OutExecData)
	{
		if (!UNiagaraScript::ShouldUseDDC())
		{
			return false;
		}

		if (SharedBuffer.GetSize() == 0)
		{
			return false;
		}

		FMemoryReaderView Ar(SharedBuffer.GetView(), true);

		// Read the archive version from the header of the payload so we can setup our reader with the right version
		FPackageFileVersion ArchiveVersion;
		Ar << ArchiveVersion;

		if (!ArchiveVersion.IsCompatible(GOldestLoadablePackageFileUEVersion))
		{
			UE_LOGF(LogNiagaraEditor, Display, "Failed to validate FNiagaraVMExecutableData received from DDC, rejecting!  Reasons:\nDeprecated object version");
			return false;
		}

		FObjectAndNameAsStringProxyArchive SafeAr(Ar, false);
		SafeAr.SetUEVer(ArchiveVersion);
		OutExecData.SerializeData(SafeAr, true);

		FString ValidationErrors;
		if (!ValidateExecData(Script, OutExecData, ValidationErrors))
		{
			UE_LOGF(LogNiagaraEditor, Display, "Failed to validate FNiagaraVMExecutableData received from DDC, rejecting!  Reasons:\n%ls", *ValidationErrors);
			return false;
		}

		return !SafeAr.IsError();
	}

	static FSharedBuffer ExecToBinaryData(const UNiagaraScript* Script, FNiagaraVMExecutableData& InExecData)
	{
		if (!UNiagaraScript::ShouldUseDDC())
		{
			return FSharedBuffer();
		}

		FString ValidationErrors;
		if (!ValidateExecData(Script, InExecData, ValidationErrors))
		{
			UE_LOGF(LogNiagaraEditor, Error, "Failed to validate FNiagaraVMExecutableData being pushed to DDC, rejecting!  Errors:\n%ls", *ValidationErrors);
			return FSharedBuffer();
		}

		TArray<uint8> BinaryData;
		FMemoryWriter Ar(BinaryData, true);

		// include the archive version into the payload since we're using struct serialization for the ExecData
		FPackageFileVersion ArchiveVersion = GPackageFileUEVersion;
		Ar << ArchiveVersion;

		FObjectAndNameAsStringProxyArchive SafeAr(Ar, false);
		InExecData.SerializeData(SafeAr, true);

		if (!BinaryData.IsEmpty() && !SafeAr.IsError())
		{
			return MakeSharedBufferFromArray(MoveTemp(BinaryData));
		}

		return FSharedBuffer();
	}

	static FSharedBuffer ShaderMapToBinaryData(FNiagaraShaderMap* ShaderMap)
	{
		if (!UNiagaraScript::ShouldUseDDC())
		{
			return FSharedBuffer();
		}

		TArray<uint8> BinaryData;
		FMemoryWriter Ar(BinaryData, true);
		FShaderSerializeContext Ctx(Ar);
		ShaderMap->Serialize(Ctx);

		if (!BinaryData.IsEmpty() && !Ar.IsError())
		{
			return MakeSharedBufferFromArray(MoveTemp(BinaryData));
		}

		return FSharedBuffer();
	}

	static FNiagaraShaderMapRef BinaryDataToShaderMap(FSharedBuffer SharedBuffer)
	{
		if (!UNiagaraScript::ShouldUseDDC() || SharedBuffer.GetSize() == 0)
		{
			return nullptr;
		}

		FMemoryReaderView Ar(SharedBuffer.GetView(), true);
		FNiagaraShaderMapRef ShaderMap = new FNiagaraShaderMap(FNiagaraShaderMap::WorkerThread);
		FShaderSerializeContext Ctx(Ar);
		ShaderMap->Serialize(Ctx);

		if (Ar.IsError())
		{
			return nullptr;
		}

		return ShaderMap;
	}

	static TArray<FNiagaraVariable> ExtractInputVariablesFromHistory(const TNiagaraParameterMapHistory<FNiagaraCompilationDigestBridge>& History, const FNiagaraCompilationGraph* FunctionGraph)
	{
		TArray<FNiagaraVariable> InputVariables;

		const int32 ModuleNamespaceLength = FNiagaraConstants::ModuleNamespaceString.Len();

		for (int32 i = 0; i < History.Variables.Num(); i++)
		{
			const FNiagaraVariable& Variable = History.Variables[i];
			const TArray<TNiagaraParameterMapHistory<FNiagaraCompilationDigestBridge>::FReadHistory>& ReadHistory = History.PerVariableReadHistory[i];

			// A read is only really exposed if it's the first read and it has no corresponding write.
			if (ReadHistory.Num() > 0 && ReadHistory[0].PreviousWritePin.Pin == nullptr)
			{
				const FNiagaraCompilationPin* InputPin = ReadHistory[0].ReadPin.Pin;

				if (FNiagaraStackGraphUtilities::IsRapidIterationType(InputPin->Variable.GetType()))
				{
					// Make sure that the module input is from the called graph, and not a nested graph.
					const FNiagaraCompilationGraph* NodeGraph = InputPin->OwningNode->OwningGraph;
					if (NodeGraph == FunctionGraph)
					{
						FNameBuilder InputPinName(InputPin->PinName);
						FStringView InputPinView(InputPinName);
						if (InputPinView.StartsWith(FNiagaraConstants::ModuleNamespaceString)
							&& InputPinView.Len() > ModuleNamespaceLength
							&& InputPinView[ModuleNamespaceLength] == TCHAR('.'))
						{
							InputVariables.Add(InputPin->Variable);
						}
					}
				}
			}
		}

		return InputVariables;
	}

	struct FScriptCompileMetaData
	{
		FScriptCompileMetaData(const FCbObject& MetaData)
		{
			Version = static_cast<EMetaDataVersion>(MetaData.Find(ANSITEXTVIEW("Version")).AsInt32());
			if (Version == EMetaDataVersion::Latest)
			{
				BuildString = FString(MetaData.Find(ANSITEXTVIEW("Build")).AsString());
				DateTime = MetaData.Find(ANSITEXTVIEW("Time")).AsDateTime();
				CompileIdString = FString(MetaData.Find(ANSITEXTVIEW("CompileId")).AsString());
				CompileTaskId = FGuid(FString(MetaData.Find(ANSITEXTVIEW("CompileTaskId")).AsString()));

				const FCbArray RIParamArray = MetaData.Find(ANSITEXTVIEW("RIParam")).AsArray();
				for (FCbFieldViewIterator It = RIParamArray.CreateViewIterator(); It; ++It)
				{
					TTuple<FName, FNiagaraVariant>& NamedVariant = BakedRapidIterationParameters.AddDefaulted_GetRef();

					FCbObjectView RIParamView = It->AsObjectView();

					NamedVariant.Key = *FString(RIParamView.FindView(ANSITEXTVIEW("VarName")).AsString());

					FMemoryView VariantDataView = RIParamView.FindView(ANSITEXTVIEW("VarData")).AsBinaryView();

					NamedVariant.Value = FNiagaraVariant(VariantDataView.GetData(), VariantDataView.GetSize());
				}
			}
		}

		FScriptCompileMetaData(const FNiagaraSystemCompilationTask* SystemCompileTask, const FNiagaraSystemCompilationTask::FCompileTaskInfo& CompileTask)
		{
			Version = EMetaDataVersion::Latest;
			BuildString = FApp::GetBuildVersion();
			DateTime = FDateTime::UtcNow();
			CompileTask.CompileId.AppendKeyString(CompileIdString);
			CompileTaskId = SystemCompileTask->GetCompilationTaskId();

			BakedRapidIterationParameters.Reserve(CompileTask.BakedRapidIterationParameters.Num());
			Algo::Transform(CompileTask.BakedRapidIterationParameters, BakedRapidIterationParameters, [](const FNiagaraVariable& InVar) -> TTuple<FName, FNiagaraVariant>
			{
				return MakeTuple(InVar.GetName(), FNiagaraVariant(InVar.GetData(), InVar.GetAllocatedSizeInBytes()));
			});
		}

		FCbObject ToCompactBinary() const
		{
			FCbWriter Writer;
			Writer.BeginObject();
			{
				Writer.AddInteger(ANSITEXTVIEW("Version"), static_cast<int32>(Version));
				Writer.AddString(ANSITEXTVIEW("Build"), BuildString);
				Writer.AddDateTime(ANSITEXTVIEW("Time"), DateTime);
				Writer.AddString(ANSITEXTVIEW("CompileId"), CompileIdString);
				Writer.AddString(ANSITEXTVIEW("CompileTaskId"), CompileTaskId.ToString());

				Writer.BeginArray(ANSITEXTVIEW("RIParam"));
				{
					for (const TTuple<FName, FNiagaraVariant>& Var : BakedRapidIterationParameters)
					{
						Writer.BeginObject();
						{
							Writer.AddString(ANSITEXTVIEW("VarName"), *Var.Key.ToString());
							Writer.AddBinary(ANSITEXTVIEW("VarData"), Var.Value.GetBytes(), Var.Value.GetNumBytes());
						}

						Writer.EndObject();
					}
				}
				Writer.EndArray();
			}
			Writer.EndObject();

			return Writer.Save().AsObject();
		}

		enum class EMetaDataVersion : int32
		{
			Invalid = 0,

			// new entries
			WithCompilationTaskId,

			// 
			Last,
			Latest = Last - 1,
		};

		EMetaDataVersion Version = EMetaDataVersion::Invalid;
		FString BuildString;
		FDateTime DateTime;
		FString CompileIdString;
		FGuid CompileTaskId;
		TArray<TTuple<FName, FNiagaraVariant>> BakedRapidIterationParameters;
	};

	static FCbObject GenerateScriptCompileMetaData(const FNiagaraSystemCompilationTask* SystemCompileTask, const FNiagaraSystemCompilationTask::FCompileTaskInfo& CompileTask)
	{
		return FScriptCompileMetaData(SystemCompileTask, CompileTask).ToCompactBinary();
	}

	static void ParseScriptCompileMetaData(const FCbObject& MetaData)
	{
		FScriptCompileMetaData Data(MetaData);
	}

	void AddRapidIterationParameter(
		const FNiagaraParameterHandle& ParameterHandle,
		const FNiagaraVariable& InputVariable,
		const FNiagaraVariable& Parameter,
		const FNiagaraCompilationNodeFunctionCall* FunctionCallNode,
		FNiagaraParameterStore& ParameterStore)
	{
		const int32 ExistingParameterIndex = ParameterStore.IndexOf(Parameter);
		if (ExistingParameterIndex == INDEX_NONE)
		{
			// we only try to add the variable if we've already determined that it's got a reasonable
			// default value and there's no override pin for the function input
			if (Parameter.IsDataAllocated() && !FunctionCallNode->HasOverridePin(ParameterHandle))
			{
				constexpr bool bAddParameterIfMissing = true;
				ParameterStore.SetParameterData(Parameter.GetData(), Parameter, bAddParameterIfMissing);
			}
		}
	}

	void FixupRenamedParameter(
		const FNiagaraVariable& InputVariable,
		const FNiagaraVariable& Parameter,
		const FNiagaraCompilationNodeFunctionCall* FunctionCallNode,
		const TMultiMap<FGuid, FNiagaraVariable>& GuidMapping,
		FNiagaraParameterStore& ParameterStore)
	{
		TOptional<FNiagaraVariableMetaData> VariableMetaData = FunctionCallNode->CalledGraph->GetMetaData(InputVariable);
		if (!VariableMetaData.IsSet())
		{
			return;
		}

		TArray<FNiagaraVariable> ExistingParameters;
		GuidMapping.MultiFind(VariableMetaData->GetVariableGuid(), ExistingParameters);

		// if we didn't match anything then there's nothing left for us to do
		if (ExistingParameters.IsEmpty())
		{
			return;
		}

		for (const FNiagaraVariable& ExistingParameter : ExistingParameters)
		{
			// if we've found a match then there's nothing to rename
			if (ExistingParameter.GetName() == Parameter.GetName())
			{
				return;
			}
		}

		// next go through to see if we may need to actually do a rename.  We'll only do this if we have something
		// that matches the GUID and is in the same namespace
		FNameBuilder NameBuilder(Parameter.GetName());
		int32 NamespaceStrCount;
		if (NameBuilder.ToView().FindLastChar(TEXT('.'), NamespaceStrCount))
		{
			FStringView ParameterNamespace = NameBuilder.ToView().Left(NamespaceStrCount);

			for (const FNiagaraVariable& ExistingParameter : ExistingParameters)
			{
				if (ExistingParameter.IsInNameSpace(ParameterNamespace))
				{
					ParameterStore.RenameParameter(ExistingParameter, Parameter.GetName());
					return;
				}
			}
		}
	}

	template<typename T, typename InAllocatorType, typename OutAllocatorType>
	static void AppendUnique(const TArray<T, InAllocatorType>& Input, TArray<T, OutAllocatorType>& Output)
	{
		Output.Reserve(Input.Num() + Output.Num());
		for (const T& InputValue : Input)
		{
			Output.AddUnique(InputValue);
		}
	}

	const UNiagaraGraph* GetGraphFromScriptSource(const UNiagaraScriptSourceBase* ScriptSourceBase)
	{
		if (const UNiagaraScriptSource* ScriptSource = CastChecked<const UNiagaraScriptSource>(ScriptSourceBase))
		{
			return ScriptSource->NodeGraph;
		}
		return nullptr;
	}

	const UNiagaraGraph* GetGraphFromScript(const UNiagaraScript* Script)
	{
		if (Script)
		{
			return GetGraphFromScriptSource(Script->GetLatestSource());
		}
		return nullptr;
	}

	TSharedPtr<FNiagaraShaderScriptParametersMetadata> BuildScriptParametersMetadata(const FNiagaraCompilationCopyData* CompilationCopyData, const FNiagaraShaderScriptParametersMetadata& InScriptParametersMetadata)
	{
		TSharedPtr<FNiagaraShaderScriptParametersMetadata> ScriptParametersMetadata = MakeShared<FNiagaraShaderScriptParametersMetadata>();
		ScriptParametersMetadata->DataInterfaceParamInfo = InScriptParametersMetadata.DataInterfaceParamInfo;

		FShaderParametersMetadataBuilder ShaderMetadataBuilder(TShaderParameterStructTypeInfo<FNiagaraShader::FParameters>::GetStructMetadata());

		auto FindDataInterfaceByName = [CompilationCopyData](const FString& DIName) -> const UNiagaraDataInterfaceBase*
		{
			for (const auto& DataInterfacePair : CompilationCopyData->AggregatedDataInterfaceCDODuplicates)
			{
				if (DataInterfacePair.Key->GetName() == DIName)
				{
					return DataInterfacePair.Value;
				}
			}

			return nullptr;
		};

		// Build meta data for each data interface
		for (FNiagaraDataInterfaceGPUParamInfo& DataInterfaceParamInfo : ScriptParametersMetadata->DataInterfaceParamInfo)
		{
			const UNiagaraDataInterfaceBase* DataInterface = FindDataInterfaceByName(DataInterfaceParamInfo.DIClassName);
			if (ensure(DataInterface))
			{
				const uint32 NextMemberOffset = ShaderMetadataBuilder.GetNextMemberOffset();
				FNiagaraShaderParametersBuilder ShaderParametersBuilder(DataInterfaceParamInfo, ScriptParametersMetadata->LooseMetadataNames, ScriptParametersMetadata->StructIncludeInfos, ShaderMetadataBuilder);
				DataInterface->BuildShaderParameters(ShaderParametersBuilder);
				DataInterfaceParamInfo.ShaderParametersOffset = NextMemberOffset;
			}
		}

		ScriptParametersMetadata->ShaderParametersMetadata = MakeShareable<FShaderParametersMetadata>(ShaderMetadataBuilder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("FNiagaraShaderScript")));

		return ScriptParametersMetadata;
	}

	void BuildDebugGroupName(const FNiagaraPrecompileData& PrecompileData, const FNiagaraCompileOptions& CompileOptions, FStringBuilderBase& DebugGroupName)
	{
		FPathViews::Append(DebugGroupName, PrecompileData.SourceName);
		FPathViews::Append(DebugGroupName, PrecompileData.EmitterUniqueName);
		FPathViews::Append(DebugGroupName, PrecompileData.ENiagaraScriptUsageEnum->GetNameStringByValue((int64)CompileOptions.TargetUsage));

		if (CompileOptions.TargetUsageId.IsValid())
		{
			DebugGroupName << TEXT("_");
			CompileOptions.TargetUsageId.AppendString(DebugGroupName, EGuidFormats::Digits);
		}
	}
} // NiagaraCompilationTasksImpl

FNiagaraSystemCompilationTask::FNiagaraSystemCompilationTask(FNiagaraCompilationTaskHandle InTaskHandle, UNiagaraSystem* InSystem, ENiagaraCompileRIParamMode InRIParamMode)
	: TaskHandle(InTaskHandle)
	, RIParamMode(InRIParamMode)
	, System_GT(InSystem)
{
	bGenerateCompileMetaData = GetDefault<UNiagaraSettings>()->bGenerateMetaDataOnCompile;

	CompilationTaskId = FGuid::NewGuid();
}

void FNiagaraSystemCompilationTask::Abort()
{
	bAborting = true;
	CompileCompletionEvent.Trigger();
}

FString FNiagaraSystemCompilationTask::GetDescription() const
{
	FString TaskDescription;
	TaskDescription.Append(SystemInfo.SystemName);
	if (bCompileForEdit)
	{
		TaskDescription.Append(TEXT(" [Edit]"));
	}
	return TaskDescription;
}

FString FNiagaraSystemCompilationTask::GetStatusString() const
{
	FString StatusString;

	switch (CompilationState)
	{
		case EState::Invalid:
		{
			StatusString = TEXT("Invalid");
			break;
		}

		case EState::WaitingForProcessing:
		{
			StatusString = TEXT("WaitingForProcessing");
			break;
		}

		case EState::ResultsProcessed:
		{
			StatusString = TEXT("ResultsProcessed");
			break;
		}

		case EState::Completed:
		{
			StatusString = TEXT("Completed");
			break;
		}

		case EState::Aborted:
		{
			StatusString = TEXT("Aborted");
			break;
		}
	}

	return StatusString;
}

const FNiagaraSystemCompilationTask::FEmitterInfo* FNiagaraSystemCompilationTask::FSystemInfo::EmitterInfoBySourceEmitter(int32 InSourceEmitterIndex) const
{
	return EmitterInfo.FindByPredicate([InSourceEmitterIndex](const FEmitterInfo& Info) -> bool
	{
		return Info.SourceEmitterIndex == InSourceEmitterIndex;
	});
}

FNiagaraSystemCompilationTask::FEmitterInfo* FNiagaraSystemCompilationTask::FSystemInfo::EmitterInfoBySourceEmitter(int32 InSourceEmitterIndex)
{
	return EmitterInfo.FindByPredicate([InSourceEmitterIndex](const FEmitterInfo& Info) -> bool
	{
		return Info.SourceEmitterIndex == InSourceEmitterIndex;
	});
}

FNiagaraSystemCompilationTask::FCompileGroupInfo::FCompileGroupInfo(int32 InSourceEmitterIndex)
: SourceEmitterIndex(InSourceEmitterIndex)
{}

bool FNiagaraSystemCompilationTask::FCompileGroupInfo::HasOutstandingCompileTasks(const FNiagaraSystemCompilationTask& ParentTask) const
{
	for (int32 CompileTaskIndex : CompileTaskIndices)
	{
		const FCompileTaskInfo& CompileTask = ParentTask.CompileTasks[CompileTaskIndex];

		if (CompileTask.IsOutstanding(ParentTask))
		{
			return true;
		}
	}

	return false;
}

void FNiagaraSystemCompilationTask::FCompileGroupInfo::InstantiateCompileGraph(const FNiagaraSystemCompilationTask& ParentTask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAsyncTask_InstantiateGraph);

	CompilationCopy = MakeShared<FNiagaraCompilationCopyData, ESPMode::ThreadSafe>();
	FNiagaraCompilationCopyData& BasePtr = *CompilationCopy.Get();

	TArray<TSharedPtr<FNiagaraCompilationCopyData, ESPMode::ThreadSafe>> DependentRequests;
	FCompileConstantResolver EmptyResolver;

	BasePtr.ValidUsages = ValidUsages;

	// First deep copy all the emitter graphs referenced by the system so that we can later hook up emitter handles in the system traversal.
	const int32 EmitterCount = ParentTask.SystemInfo.EmitterInfo.Num();
	BasePtr.EmitterData.Reserve(EmitterCount);

	for (const FEmitterInfo& EmitterInfo : ParentTask.SystemInfo.EmitterInfo)
	{
		TSharedPtr<FNiagaraCompilationCopyData, ESPMode::ThreadSafe> EmitterPtr = MakeShared<FNiagaraCompilationCopyData, ESPMode::ThreadSafe>();
		EmitterPtr->EmitterUniqueName = EmitterInfo.UniqueEmitterName;
		EmitterPtr->ValidUsages = BasePtr.ValidUsages;

		// Don't need to copy the graph if we aren't going to use it.
		if (EmitterInfo.Enabled && ((SourceEmitterIndex == INDEX_NONE) || (SourceEmitterIndex == EmitterInfo.SourceEmitterIndex)))
		{
			const FNiagaraPrecompileData* EmitterRequestData = static_cast<const FNiagaraPrecompileData*>(ParentTask.SystemPrecompileData->GetDependentRequest(EmitterInfo.DigestedEmitterIndex).Get());

			if (const FNiagaraCompilationGraphDigested* SourceGraph = EmitterInfo.SourceGraph.Get())
			{
				EmitterPtr->InstantiateCompilationCopy(*SourceGraph, EmitterRequestData, ENiagaraScriptUsage::EmitterSpawnScript, EmitterInfo.ConstantResolver);
			}
		}
		EmitterPtr->ValidUsages = BasePtr.ValidUsages;
		BasePtr.EmitterData.Add(EmitterPtr);
	}

	// Now deep copy the system graphs, skipping traversal into any emitter references.
	{
		TArray<FNiagaraVariable> EncounterableSystemVariables;
		ParentTask.SystemPrecompileData->GatherPreCompiledVariables(FString(), EncounterableSystemVariables);

		// skip the deep copy if we're not compiling the system scripts
		if (BasePtr.ValidUsages.Contains(ENiagaraScriptUsage::SystemSpawnScript))
		{
			if (const FNiagaraCompilationGraphDigested* SourceGraph = ParentTask.SystemInfo.SystemSourceGraph.Get())
			{
				BasePtr.InstantiateCompilationCopy(*SourceGraph, ParentTask.SystemPrecompileData.Get(), ENiagaraScriptUsage::SystemSpawnScript, ParentTask.SystemInfo.ConstantResolver);
			}

			BasePtr.CreateParameterMapHistory(ParentTask, EncounterableSystemVariables, ParentTask.SystemInfo.StaticVariableResults, ParentTask.SystemInfo.ConstantResolver, {});

			// bubble up the referenced DI classes to the system
			for (const FNiagaraCompilationCopyData::FSharedCompilationCopy& EmitterData : BasePtr.EmitterData)
			{
				BasePtr.AggregatedDataInterfaceCDODuplicates.Append(EmitterData->AggregatedDataInterfaceCDODuplicates);
			}
		}
	}

	// Now we can finish off the emitters.
	for (const FEmitterInfo& EmitterInfo : ParentTask.SystemInfo.EmitterInfo)
	{
		TArray<FNiagaraVariable> EncounterableEmitterVariables;
		ParentTask.SystemPrecompileData->GetDependentRequest(EmitterInfo.DigestedEmitterIndex)->GatherPreCompiledVariables(FString(), EncounterableEmitterVariables);

		if (EmitterInfo.Enabled && ((SourceEmitterIndex == INDEX_NONE) || (SourceEmitterIndex == EmitterInfo.SourceEmitterIndex)))
		{
			TArray<FNiagaraVariable> StaticVariablesFromEmitter = ParentTask.SystemInfo.StaticVariableResults;
			StaticVariablesFromEmitter.Append(EmitterInfo.StaticVariableResults);

			BasePtr.EmitterData[EmitterInfo.DigestedEmitterIndex]->CreateParameterMapHistory(ParentTask, EncounterableEmitterVariables, StaticVariablesFromEmitter, EmitterInfo.ConstantResolver, EmitterInfo.SimStages);
		}
	}
}

bool FNiagaraSystemCompilationTask::FCompileComputeShaderTaskInfo::IsOutstanding() const
{
	return !ShaderMap.IsValid();
}

bool FNiagaraSystemCompilationTask::FCompileTaskInfo::IsOutstanding(const FNiagaraSystemCompilationTask& ParentTask) const
{
	if (!ExeData.IsValid())
	{
		return true;
	}
	
	for (int32 TaskIndex : ComputeShaderTaskIndices)
	{
		if (ParentTask.CompileComputeShaderTasks[TaskIndex].IsOutstanding())
		{
			return true;
		}
	}

	return false;
}

FNiagaraSystemCompilationTask::FCompileGroupInfo* FNiagaraSystemCompilationTask::GetCompileGroupInfo(int32 EmitterIndex)
{
	return CompileGroups.FindByPredicate([EmitterIndex](const FCompileGroupInfo& GroupInfo) -> bool
	{
		return GroupInfo.SourceEmitterIndex == EmitterIndex;
	});
}

const FNiagaraSystemCompilationTask::FScriptInfo* FNiagaraSystemCompilationTask::GetScriptInfo(const TObjectKey<UNiagaraScript>& ScriptKey) const
{
	return DigestedScriptInfo.Find(ScriptKey);
}

void FNiagaraSystemCompilationTask::FCompileTaskInfo::CollectNamedDataInterfaces(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo)
{
	FString UniqueEmitterName;
	const FNiagaraCompilationCopyData* CompilationCopyData = nullptr;

	if (const FEmitterInfo* EmitterInfo = SystemCompileTask->SystemInfo.EmitterInfoBySourceEmitter(GroupInfo.SourceEmitterIndex))
	{
		CompilationCopyData = static_cast<const FNiagaraCompilationCopyData*>(GroupInfo.CompilationCopy->GetDependentRequest(EmitterInfo->DigestedEmitterIndex).Get());
		UniqueEmitterName = EmitterInfo->UniqueEmitterName;
	}
	else
	{
		CompilationCopyData = GroupInfo.CompilationCopy.Get();
	}

	const FScriptInfo& ScriptInfo = SystemCompileTask->DigestedScriptInfo.FindChecked(ScriptKey);

	auto AccumulateInputDataInterfaces = [this, &UniqueEmitterName](const FNiagaraCompilationNode& Node) -> bool
	{
		if (const FNiagaraCompilationNodeInput* InputNode = Node.AsType<FNiagaraCompilationNodeInput>())
		{
			if (InputNode->DataInterfaceName != NAME_None && InputNode->DuplicatedDataInterface)
			{
				constexpr bool bIsParameterMapDataInterface = false;
				FName DIName = FNiagaraHlslTranslator::GetDataInterfaceName(InputNode->DataInterfaceName, UniqueEmitterName, bIsParameterMapDataInterface);

				NamedDataInterfaces.Add(DIName, InputNode->DuplicatedDataInterface);
			}
		}

		return true;
	};

	auto DefaultNodeFinder = [&ScriptInfo](const FNiagaraCompilationNodeOutput& OutputNode) -> bool
	{
		if (UNiagaraScript::IsEquivalentUsage(OutputNode.Usage, ScriptInfo.Usage) && OutputNode.UsageId == ScriptInfo.UsageId)
		{
			return true;
		}

		if (ScriptInfo.Usage == ENiagaraScriptUsage::ParticleGPUComputeScript)
		{
			return UNiagaraScript::IsParticleScript(OutputNode.Usage);
		}
		else if (ScriptInfo.Usage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated)
		{
			return UNiagaraScript::IsEquivalentUsage(OutputNode.Usage, ENiagaraScriptUsage::ParticleUpdateScript);
		}

		return OutputNode.Usage == ScriptInfo.Usage && OutputNode.UsageId == ScriptInfo.UsageId;
	};

	CompilationCopyData->InstantiatedGraph->NodeTraversal(true, false, DefaultNodeFinder, AccumulateInputDataInterfaces);

	if (UNiagaraScript::IsSystemScript(ScriptInfo.Usage))
	{
		check(GroupInfo.SourceEmitterIndex == INDEX_NONE);

		const ENiagaraScriptUsage EmitterUsage = ScriptInfo.Usage == ENiagaraScriptUsage::SystemSpawnScript
			? ENiagaraScriptUsage::EmitterSpawnScript
			: ENiagaraScriptUsage::EmitterUpdateScript;
		const FGuid EmitterUsageId = ScriptInfo.UsageId;
		auto EmitterNodeFinder = [EmitterUsage, EmitterUsageId](const FNiagaraCompilationNodeOutput& OutputNode) -> bool
		{
			return UNiagaraScript::IsEquivalentUsage(OutputNode.Usage, EmitterUsage) && OutputNode.UsageId == EmitterUsageId;
		};

		for (const FNiagaraCompilationCopyData::FSharedCompilationCopy& EmitterCompilationCopy : CompilationCopyData->EmitterData)
		{
			if (const FNiagaraCompilationGraph* EmitterGraph = EmitterCompilationCopy->InstantiatedGraph.Get())
			{
				UniqueEmitterName = EmitterCompilationCopy->EmitterUniqueName;
				EmitterGraph->NodeTraversal(true, false, EmitterNodeFinder, AccumulateInputDataInterfaces);
			}
		}
	}
}

void FNiagaraSystemCompilationTask::FCompileTaskInfo::Translate(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAsyncTask_Translate);

	COOK_STAT(NiagaraSystemCookStats::ScriptTranslationCount.fetch_add(1, std::memory_order_relaxed));

	const FNiagaraPrecompileData* PrecompileData = nullptr;
	const FNiagaraCompilationCopyData* CompilationCopyData = nullptr;

	if (const FEmitterInfo* EmitterInfo = SystemCompileTask->SystemInfo.EmitterInfoBySourceEmitter(GroupInfo.SourceEmitterIndex))
	{
		PrecompileData = static_cast<const FNiagaraPrecompileData*>(SystemCompileTask->SystemPrecompileData->GetDependentRequest(EmitterInfo->DigestedEmitterIndex).Get());
		CompilationCopyData = static_cast<const FNiagaraCompilationCopyData*>(GroupInfo.CompilationCopy->GetDependentRequest(EmitterInfo->DigestedEmitterIndex).Get());
	}
	else
	{
		PrecompileData = SystemCompileTask->SystemPrecompileData.Get();
		CompilationCopyData = GroupInfo.CompilationCopy.Get();
	}

	const FScriptInfo& ScriptInfo = SystemCompileTask->DigestedScriptInfo.FindChecked(ScriptKey);

	// TRANSLATION
	{
		double TranslationStartTime = FPlatformTime::Seconds();

		BakedRapidIterationParameters = PrecompileData->BakedRapidIterationParameters;

		FHlslNiagaraTranslatorOptions TranslateOptions;

		if (CompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleGPUComputeScript)
		{
			TranslateOptions.SimTarget = ENiagaraSimTarget::GPUComputeSim;
		}
		else
		{
			TranslateOptions.SimTarget = ENiagaraSimTarget::CPUSim;
		}
		TranslateOptions.OverrideModuleConstants = BakedRapidIterationParameters;
		TranslateOptions.bParameterRapidIteration = PrecompileData->GetUseRapidIterationParams();
		TranslateOptions.bDisableDebugSwitches = PrecompileData->GetDisableDebugSwitches();

		FString PathName = SourceScript->GetPathName();

		TUniquePtr<INiagaraHlslTranslator> Translator = INiagaraHlslTranslator::CreateTranslator(PrecompileData, CompilationCopyData);
		TranslateResults = Translator->Translate(CompileOptions, TranslateOptions);
		TranslateOutput = Translator->GetTranslateOutput();
		TranslatedHlsl = Translator->GetTranslatedHLSL();

		TranslationTime = (float) (FPlatformTime::Seconds() - TranslationStartTime);
	}
}

void FNiagaraSystemCompilationTask::FCompileTaskInfo::IssueCompileVm(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo)
{
	COOK_STAT(NiagaraSystemCookStats::CpuScriptCompileCount++);

	const FNiagaraPrecompileData* PrecompileData = nullptr;

	if (const FEmitterInfo* EmitterInfo = SystemCompileTask->SystemInfo.EmitterInfoBySourceEmitter(GroupInfo.SourceEmitterIndex))
	{
		PrecompileData = static_cast<const FNiagaraPrecompileData*>(SystemCompileTask->SystemPrecompileData->GetDependentRequest(EmitterInfo->DigestedEmitterIndex).Get());
	}
	else
	{
		PrecompileData = SystemCompileTask->SystemPrecompileData.Get();
	}

	// COMPILATION
	TStringBuilder<512> DebugGroupName;
	NiagaraCompilationTasksImpl::BuildDebugGroupName(*PrecompileData, CompileOptions, DebugGroupName);

	ScriptCompiler = MakeUnique<FHlslNiagaraCompiler>();
	ScriptCompilationJobId = ScriptCompiler->CompileScriptVM(DebugGroupName, CompileOptions, TranslateResults, TranslateOutput, TranslatedHlsl, SystemCompileTask->NiagaraShaderType);
}

void FNiagaraSystemCompilationTask::FCompileTaskInfo::IssueTranslateGpu(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo)
{
	const FNiagaraPrecompileData* PrecompileData = nullptr;

	if (const FEmitterInfo* EmitterInfo = SystemCompileTask->SystemInfo.EmitterInfoBySourceEmitter(GroupInfo.SourceEmitterIndex))
	{
		PrecompileData = static_cast<const FNiagaraPrecompileData*>(SystemCompileTask->SystemPrecompileData->GetDependentRequest(EmitterInfo->DigestedEmitterIndex).Get());
	}
	else
	{
		PrecompileData = SystemCompileTask->SystemPrecompileData.Get();
	}

	// COMPILATION
	TStringBuilder<512> DebugGroupName;
	NiagaraCompilationTasksImpl::BuildDebugGroupName(*PrecompileData, CompileOptions, DebugGroupName);

	ScriptCompiler = MakeUnique<FHlslNiagaraCompiler>();
	ScriptCompilationJobId = ScriptCompiler->CreateShaderIntermediateData(DebugGroupName, CompileOptions, TranslateResults, TranslateOutput, TranslatedHlsl);
}

void FNiagaraSystemCompilationTask::FCompileTaskInfo::IssueCompileGpu(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo)
{
	COOK_STAT(NiagaraSystemCookStats::GpuScriptCompileCount++);

	// if we don't have a valid shader file, then there's no point in issuing a compilation
	if (!TranslateResults.bHLSLGenSucceeded)
	{
		return;
	}

	const FNiagaraPrecompileData* PrecompileData = nullptr;
	const FNiagaraCompilationCopyData* CompilationCopyData = nullptr;

	if (const FEmitterInfo* EmitterInfo = SystemCompileTask->SystemInfo.EmitterInfoBySourceEmitter(GroupInfo.SourceEmitterIndex))
	{
		PrecompileData = static_cast<const FNiagaraPrecompileData*>(SystemCompileTask->SystemPrecompileData->GetDependentRequest(EmitterInfo->DigestedEmitterIndex).Get());
		CompilationCopyData = static_cast<const FNiagaraCompilationCopyData*>(GroupInfo.CompilationCopy->GetDependentRequest(EmitterInfo->DigestedEmitterIndex).Get());
	}
	else
	{
		PrecompileData = SystemCompileTask->SystemPrecompileData.Get();
		CompilationCopyData = GroupInfo.CompilationCopy.Get();
	}

	check(!ComputeShaderTaskIndices.IsEmpty());

	TStringBuilder<512> DebugGroupName;
	NiagaraCompilationTasksImpl::BuildDebugGroupName(*PrecompileData, CompileOptions, DebugGroupName);

	TSharedPtr<FNiagaraShaderScriptParametersMetadata> ShaderParameters =
		NiagaraCompilationTasksImpl::BuildScriptParametersMetadata(CompilationCopyData, TranslateOutput.ScriptData.ShaderScriptParametersMetadata);

	ShaderMapCompiler = MakeUnique<FNiagaraShaderMapCompiler>(SystemCompileTask->NiagaraShaderType, ShaderParameters);

	for (int32 ShaderTaskIndex : ComputeShaderTaskIndices)
	{
		const FCompileComputeShaderTaskInfo& ComputeInfo = SystemCompileTask->CompileComputeShaderTasks[ShaderTaskIndex];
		ShaderMapCompiler->AddShaderPlatform(ComputeInfo.ShaderMapId, ComputeInfo.ShaderPlatform);
	}

	TArray<UNiagaraDataInterface*> UniqueDI;
	for (const auto& It : CompilationCopyData->AggregatedDataInterfaceCDODuplicates)
	{
		UniqueDI.AddUnique(It.Value);
	}

	ShaderMapCompiler->CompileScript(CompileId, PrecompileData->SourceName, DebugGroupName, CompileOptions, TranslateResults, TranslateOutput, TranslatedHlsl, UniqueDI);
}

TOptional<FNiagaraCompileResults> FNiagaraSystemCompilationTask::FCompileTaskInfo::HandleDeprecatedGpuScriptResults() const
{
	return TOptional<FNiagaraCompileResults>();
}

/** Returns true if the task has valid results */
bool FNiagaraSystemCompilationTask::FCompileTaskInfo::RetrieveCompilationResult(bool bWait)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAsyncTask_RetrieveResult);

	check(CompileResultsReadyEvent.IsValid() && CompileResultsReadyEvent->IsValid());

	if (CompileResultsReadyEvent->IsCompleted())
	{
		return true;
	}
	check(!ExeData.IsValid());
	check(ScriptCompiler.IsValid());

	// in cases where asynchronous shader compiling isn't allowed we'll simply block execution here till the task
	// is complete
	if (!GShaderCompilingManager->AllowAsynchronousShaderCompiling())
	{
		bWait = true;
	}

	TOptional<FNiagaraCompileResults> CompileResult = ScriptCompiler->GetCompileResult(ScriptCompilationJobId, bWait);
	if (!CompileResult)
	{
		return false;
	}

	FNiagaraCompileResults& Results = CompileResult.GetValue();

	ExeData = Results.Data;

	RetrieveTranslateResult();

	ExeData->LastCompileStatus = (FNiagaraCompileResults::CompileResultsToSummary(&Results));
	if (ExeData->LastCompileStatus != ENiagaraScriptCompileStatus::NCS_Error)
	{
		// When there are no errors the compile events get emptied, so add them back here.
		ExeData->LastCompileEvents.Append(Results.CompileEvents);
	}

	CompilerWallTime = Results.CompilerWallTime;
	CompilerPreprocessTime = Results.CompilerPreprocessTime;
	CompilerWorkerTime = Results.CompilerWorkerTime;

	return true;
}

void FNiagaraSystemCompilationTask::FCompileTaskInfo::RetrieveTranslateResult()
{
	if (!ExeData.IsValid())
	{
		ExeData = MakeShared<FNiagaraVMExecutableData>(TranslateOutput.ScriptData);
	}

	for (const FNiagaraCompileEvent& Message : TranslateResults.CompileEvents)
	{
#if defined(NIAGARA_SCRIPT_COMPILE_LOGGING_MEDIUM)
		UE_LOGF(LogNiagaraEditor, Log, "%ls", *Message.Message);
#endif
		if (Message.Severity == FNiagaraCompileEventSeverity::Error)
		{
			// Write the error messages to the string as well so that they can be echoed up the chain.
			if (ExeData->ErrorMsg.Len() > 0)
			{
				ExeData->ErrorMsg += "\n";
			}
			ExeData->ErrorMsg += Message.Message;
		}
	}

	for (const FNiagaraCompileEvent& Event : TranslateResults.CompileEvents)
	{
		ExeData->LastCompileEvents.AddUnique(Event);
	}
	ExeData->ExternalDependencies = TranslateResults.CompileDependencies;
	ExeData->CompileTags = TranslateResults.CompileTags;
	ExeData->CompileTagsEditorOnly = TranslateResults.CompileTagsEditorOnly;

	// we also need to include the information about the rapid iteration parameters that were used
	// to generate the ExeData
	ExeData->BakedRapidIterationParameters = BakedRapidIterationParameters;

	const bool bHasTranslationErrors = TranslateResults.NumErrors > 0;
	const bool bHasTranslationWarnings = TranslateResults.NumWarnings > 0;

	ExeData->LastCompileStatus = bHasTranslationErrors
		? ENiagaraScriptCompileStatus::NCS_Error
		: bHasTranslationWarnings
			? ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings
			: ENiagaraScriptCompileStatus::NCS_UpToDate;
}

bool FNiagaraSystemCompilationTask::FCompileTaskInfo::RetrieveShaderMap(bool bWait)
{
	check(CompileResultsReadyEvent.IsValid() && CompileResultsReadyEvent->IsValid());

	if (CompileResultsReadyEvent->IsCompleted())
	{
		return true;
	}

	// if we don't have a shader map compiler, then just return the translation results
	if (!ShaderMapCompiler.IsValid())
	{
		RetrieveTranslateResult();
		return true;
	}

	ExeData = ShaderMapCompiler->ReadScriptMetaData();

	if (ShaderMapCompiler->ProcessCompileResults(bWait))
	{
		ExeData->LastCompileStatus = ENiagaraScriptCompileStatus::NCS_UpToDate;
		return true;
	}

	return false;
}

void FNiagaraSystemCompilationTask::FCompileTaskInfo::GenerateOptimizedVMByteCode(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAsyncTask_OptimizeVM);

	if (const uint8* ByteCode = ExeData->ByteCode.GetDataPtr())
	{
		const double OptimizeStartTime = FPlatformTime::Seconds();

		//this is just necessary because VectorVM doesn't know about FVMExternalFunctionBindingInfo
		const int32 NumExtFns = ExeData->CalledVMExternalFunctions.Num();

		TArray<FVectorVMExtFunctionData> ExtFnTable;
		ExtFnTable.SetNumZeroed(NumExtFns);
		for (int i = 0; i < NumExtFns; ++i)
		{
			ExtFnTable[i].NumInputs = ExeData->CalledVMExternalFunctions[i].GetNumInputs();
			ExtFnTable[i].NumOutputs = ExeData->CalledVMExternalFunctions[i].GetNumOutputs();
		}

		{
			const uint64 AssetPathHash = CityHash64((char*)AssetPath.GetCharArray().GetData(), AssetPath.GetCharArray().Num());

			VectorVM::Optimizer::FVectorVMOptimizerContext* OptimizerContext = VectorVM::Optimizer::AllocOptimizerContext();
			VectorVM::Optimizer::OptimizeVectorVMScript(ByteCode, ExeData->ByteCode.GetLength(), ExtFnTable.GetData(), ExtFnTable.Num(), OptimizerContext, AssetPathHash, VectorVM::VVMFlag_OptOmitStats | VectorVM::VVMFlag_OptSaveIntermediateState);

			// extract a human readable version of the script
			VectorVM::Optimizer::GenerateHumanReadableScript(*OptimizerContext, ExeData->LastExperimentalAssemblyScript);

			// freeze the OptimizeContext
			VectorVM::Bridge::FreezeOptimizerContext(*OptimizerContext, ExeData->ExperimentalContextData);

			VectorVM::Optimizer::FreeOptimizerContext(OptimizerContext);
		}

		ByteCodeOptimizeTime = (float)(OptimizeStartTime - FPlatformTime::Seconds());
	}
}

void FNiagaraSystemCompilationTask::FCompileTaskInfo::ProcessCompilation(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo)
{
	ON_SCOPE_EXIT
	{
		CompileResultsProcessedEvent->Trigger();
	};

	if (ScriptCompileType == EScriptCompileType::CompileForVm)
	{
		GenerateOptimizedVMByteCode(SystemCompileTask, GroupInfo);
	}

	// we need to update our ComputeShaderTasks with the generated shader map so that we can put the data to the DDC
	if (ShaderMapCompiler.IsValid())
	{
		check(ScriptCompileType == EScriptCompileType::CompileForGpu);
		for (int32 ComputeShaderTaskIndex : ComputeShaderTaskIndices)
		{
			FCompileComputeShaderTaskInfo& ShaderTaskInfo = SystemCompileTask->CompileComputeShaderTasks[ComputeShaderTaskIndex];
			ShaderMapCompiler->GetShaderMap(ShaderTaskInfo.ShaderMapId, ShaderTaskInfo.ShaderMap, ShaderTaskInfo.CompilationErrors);
		}
	}
}

bool FNiagaraSystemCompilationTask::FDDCTaskInfo::ResolveGet(bool bSuccess)
{
	if (DataCacheGetKey != UE::DerivedData::FCacheKey::Empty)
	{
		if (bSuccess)
		{
			bFromDerivedDataCache = true;
		}
		else
		{
			DataCachePutKeys.AddUnique(DataCacheGetKey);
		}
	}

	return bSuccess;
}

FNiagaraSystemCompilationTask::FDataCacheRequestBase::FDataCacheRequestBase()
	: RequestOwner(UE::DerivedData::EPriority::Normal)
{
	RequestOwner.KeepAlive();
}

void FNiagaraSystemCompilationTask::FDispatchAndProcessDataCacheGetRequests::Launch(FNiagaraSystemCompilationTask* SystemCompileTask)
{
	// todo - incorporate something here where we skip if we try to update the compileid (post ri collection)
	// and there's no actual change to the compileid...in that case we shouldn't worry about trying to grab
	// stuff from the ddc again

	union FDDCUserData
	{
		uint64 UserData;
		struct 
		{
			int32 CompileTaskIndex;
			bool bShaderCompileTask;
		};
	};

	using namespace UE::DerivedData;

	TArray<FCacheGetRequest> GetRequests;

	const int32 CompileTaskCount = SystemCompileTask->CompileTasks.Num();
	const int32 ShaderCompileTaskCount = SystemCompileTask->CompileComputeShaderTasks.Num();

	GetRequests.Reserve(CompileTaskCount + ShaderCompileTaskCount);

	for (int32 CompileTaskIt = 0; CompileTaskIt < CompileTaskCount; ++CompileTaskIt)
	{
		FNiagaraSystemCompilationTask::FCompileTaskInfo& CompileTask = SystemCompileTask->CompileTasks[CompileTaskIt];
		if (!CompileTask.ExeData.IsValid())
		{
			const UE::DerivedData::FCacheKey CacheKey = NiagaraCompilationTasksImpl::BuildNiagaraDDCCacheKey(CompileTask.CompileId, CompileTask.AssetPath);

			if (CacheKey != CompileTask.DDCTaskInfo.DataCacheGetKey)
			{
				FDDCUserData Index;
				Index.CompileTaskIndex = CompileTaskIt;
				Index.bShaderCompileTask = false;

				FCacheGetRequest& GetValueRequest = GetRequests.AddDefaulted_GetRef();
				GetValueRequest.Name = CompileTask.AssetPath;
				GetValueRequest.Key = CacheKey;
				GetValueRequest.UserData = Index.UserData;

				CompileTask.DDCTaskInfo.DataCacheGetKey = CacheKey;
			}
		}
	}

	for (int32 ShaderCompileTaskIt = 0; ShaderCompileTaskIt < ShaderCompileTaskCount; ++ShaderCompileTaskIt)
	{
		FNiagaraSystemCompilationTask::FCompileComputeShaderTaskInfo& ShaderCompileTask = SystemCompileTask->CompileComputeShaderTasks[ShaderCompileTaskIt];
		if (ShaderCompileTask.IsOutstanding())
		{
			FNiagaraSystemCompilationTask::FCompileTaskInfo& CompileTask = SystemCompileTask->CompileTasks[ShaderCompileTask.ParentCompileTaskIndex];
			const UE::DerivedData::FCacheKey CacheKey = NiagaraCompilationTasksImpl::BuildNiagaraComputeDDCCacheKey(ShaderCompileTask.ShaderMapId, ShaderCompileTask.ShaderPlatform, CompileTask.AssetPath);

			if (CacheKey != ShaderCompileTask.DDCTaskInfo.DataCacheGetKey)
			{
				FDDCUserData Index;
				Index.CompileTaskIndex = ShaderCompileTaskIt;
				Index.bShaderCompileTask = true;

				FCacheGetRequest& ShaderGetValueRequest = GetRequests.AddDefaulted_GetRef();
				ShaderGetValueRequest.Name = CompileTask.AssetPath;
				ShaderGetValueRequest.Key = CacheKey;
				ShaderGetValueRequest.UserData = Index.UserData;

				ShaderCompileTask.DDCTaskInfo.DataCacheGetKey = CacheKey;
			}
		}
	}

	PendingGetRequestCount = GetRequests.Num();

	if (PendingGetRequestCount > 0)
	{
		FRequestBarrier RequestBarrier(RequestOwner);
		GetCache().Get(GetRequests, RequestOwner, [this, SystemCompileTask = SystemCompileTask->AsShared()](FCacheGetResponse&& Response)
		{
			FDDCUserData Index;
			Index.UserData = Response.UserData;

			FDDCTaskInfo* DDCTaskInfo = nullptr;
			if (Index.bShaderCompileTask)
			{
				if (ensure(SystemCompileTask->CompileComputeShaderTasks.IsValidIndex(Index.CompileTaskIndex)))
				{
					DDCTaskInfo = &SystemCompileTask->CompileComputeShaderTasks[Index.CompileTaskIndex].DDCTaskInfo;
				}
			}
			else
			{
				if (ensure(SystemCompileTask->CompileTasks.IsValidIndex(Index.CompileTaskIndex)))
				{
					DDCTaskInfo = &SystemCompileTask->CompileTasks[Index.CompileTaskIndex].DDCTaskInfo;
				}
			}

			if (DDCTaskInfo && Response.Status == EStatus::Ok)
			{
				const FValueWithId& ValueWithId = Response.Record.GetValue(NiagaraCompilationTasksImpl::ScriptDataValue);
				DDCTaskInfo->PendingDDCData = ValueWithId.GetData().Decompress().MoveToUnique();

				if (NiagaraCompilationTasksImpl::GDebugParseCompileMetaDataOnGet)
				{
					if (!Index.bShaderCompileTask)
					{
						NiagaraCompilationTasksImpl::ParseScriptCompileMetaData(Response.Record.GetMeta());
					}
				}
			}

			if (PendingGetRequestCount.fetch_sub(1, std::memory_order_relaxed) == 1)
			{
				// in order to make sure that processing the binary data from the DDC can properly serialize
				// objects (based on their path name) we must run BinaryToExecData() on the game thread (to
				// avoid conflicts with GC or async loading).
				FNiagaraSystemCompilingManager::Get().QueueGameThreadFunction([CompletionEvent = this->CompletionEvent, SystemCompileTask]() mutable
				{
					check(IsInGameThread());
					for (FNiagaraSystemCompilationTask::FCompileTaskInfo& CompileTask : SystemCompileTask->CompileTasks)
					{
						if (!CompileTask.DDCTaskInfo.PendingDDCData.IsNull())
						{
							FNiagaraVMExecutableData ExeData;
							if (NiagaraCompilationTasksImpl::BinaryToExecData(CompileTask.SourceScript.Get(), CompileTask.DDCTaskInfo.PendingDDCData.MoveToShared(), ExeData))
							{
								CompileTask.ExeData = MakeShared<FNiagaraVMExecutableData>(MoveTemp(ExeData));
							}
						}

						if (CompileTask.DDCTaskInfo.ResolveGet(CompileTask.ExeData.IsValid()))
						{
							COOK_STAT(NiagaraSystemCookStats::CpuScriptDdcCacheHitCount++);
						}
						else
						{
							COOK_STAT(NiagaraSystemCookStats::CpuScriptDdcCacheMissCount++);
						}
					}

					for (FNiagaraSystemCompilationTask::FCompileComputeShaderTaskInfo& CompileTask : SystemCompileTask->CompileComputeShaderTasks)
					{
						if (!CompileTask.DDCTaskInfo.PendingDDCData.IsNull())
						{
							CompileTask.ShaderMap = NiagaraCompilationTasksImpl::BinaryDataToShaderMap(CompileTask.DDCTaskInfo.PendingDDCData.MoveToShared());
						}

						if (CompileTask.DDCTaskInfo.ResolveGet(CompileTask.ShaderMap.IsValid()))
						{
							COOK_STAT(NiagaraSystemCookStats::GpuScriptDdcCacheHitCount++);
						}
						else
						{
							COOK_STAT(NiagaraSystemCookStats::GpuScriptDdcCacheMissCount++);
						}

					}

					CompletionEvent.Trigger();
				});
			}
		});
	}
	else
	{
		CompletionEvent.Trigger();
	}
}

void FNiagaraSystemCompilationTask::FDispatchDataCachePutRequests::Launch(FNiagaraSystemCompilationTask* SystemCompileTask)
{
	using namespace UE::DerivedData;

	const bool bIncludeMetaData = SystemCompileTask->GenerateCompileMetaData();

	TArray<FCachePutRequest> PutRequests;

	const int32 CompileTaskCount = SystemCompileTask->CompileTasks.Num();
	const int32 ShaderCompileTaskCount = SystemCompileTask->CompileComputeShaderTasks.Num();

	PutRequests.Reserve(CompileTaskCount + ShaderCompileTaskCount);

	for (const FNiagaraSystemCompilationTask::FCompileTaskInfo& CompileTask : SystemCompileTask->CompileTasks)
	{
		if (CompileTask.ExeData.IsValid() && !CompileTask.DDCTaskInfo.DataCachePutKeys.IsEmpty())
		{
			if (FSharedBuffer SharedBuffer = NiagaraCompilationTasksImpl::ExecToBinaryData(CompileTask.SourceScript.Get(), *CompileTask.ExeData))
			{
				FValue PutValue = FValue::Compress(SharedBuffer);
				for (const FCacheKey& CachePutKey : CompileTask.DDCTaskInfo.DataCachePutKeys)
				{
					FCacheRecordBuilder RecordBuilder(CachePutKey);
					RecordBuilder.AddValue(NiagaraCompilationTasksImpl::ScriptDataValue, PutValue);
					if (bIncludeMetaData)
					{
						RecordBuilder.SetMeta(NiagaraCompilationTasksImpl::GenerateScriptCompileMetaData(SystemCompileTask, CompileTask));
					}

					FCachePutRequest& PutRequest = PutRequests.AddZeroed_GetRef();
					PutRequest.Name = CompileTask.AssetPath;
					PutRequest.Record = RecordBuilder.Build();
					PutRequest.Policy = ECachePolicy::Default;
				}
			}
		}
	}

	for (const FNiagaraSystemCompilationTask::FCompileComputeShaderTaskInfo& CompileTask : SystemCompileTask->CompileComputeShaderTasks)
	{
		if (CompileTask.ShaderMap.IsValid() && CompileTask.ShaderMap->IsValid() && !CompileTask.DDCTaskInfo.DataCachePutKeys.IsEmpty())
		{
			const FNiagaraSystemCompilationTask::FCompileTaskInfo& ParentCompileTask = SystemCompileTask->CompileTasks[CompileTask.ParentCompileTaskIndex];

			if (FSharedBuffer SharedBuffer = NiagaraCompilationTasksImpl::ShaderMapToBinaryData(CompileTask.ShaderMap.GetReference()))
			{
				FValue PutValue = FValue::Compress(SharedBuffer);
				for (const FCacheKey& CachePutKey : CompileTask.DDCTaskInfo.DataCachePutKeys)
				{
					FCacheRecordBuilder RecordBuilder(CachePutKey);
					RecordBuilder.AddValue(NiagaraCompilationTasksImpl::ScriptDataValue, PutValue);

					FCachePutRequest& PutRequest = PutRequests.AddZeroed_GetRef();
					PutRequest.Name = ParentCompileTask.AssetPath;
					PutRequest.Record = RecordBuilder.Build();
					PutRequest.Policy = ECachePolicy::Default;
				}
			}
		}
	}

	PendingPutRequestCount = PutRequests.Num();

	if (!PutRequests.IsEmpty())
	{
		FRequestBarrier RequestBarrier(RequestOwner);
		GetCache().Put(PutRequests, RequestOwner, [this, SystemCompileTask = SystemCompileTask->AsShared()](FCachePutResponse&& Response)
		{
			if (PendingPutRequestCount.fetch_sub(1, std::memory_order_relaxed) == 1)
			{
				CompletionEvent.Trigger();
			}
		});
	}
	else
	{
		CompletionEvent.Trigger();
	}
}

void FNiagaraSystemCompilationTask::DigestSystemInfo()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAsyncTask_DigestSystem);

	using namespace NiagaraCompilationTasksImpl;

	if (RIParamMode == ENiagaraCompileRIParamMode::GameThread)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAsyncTask_PrepareRIParam);
		System_GT->PrepareRapidIterationParametersForCompilation();
	}

	FNiagaraDigestDatabase& DigestDatabase = FNiagaraDigestDatabase::Get();
	const UNiagaraScript* SystemSpawnScript = System_GT->GetSystemSpawnScript();
	const UNiagaraScript* SystemUpdateScript = System_GT->GetSystemUpdateScript();

	const UNiagaraGraph* SystemGraph = GetGraphFromScript(SystemSpawnScript);
	check(SystemGraph == GetGraphFromScript(SystemUpdateScript));

	FNiagaraGraphChangeIdBuilder ChangeIdBuilder;
	ChangeIdBuilder.ParseReferencedGraphs(SystemGraph);

	SystemInfo.SystemName = System_GT->GetName();
	SystemInfo.SystemPackageName = System_GT->GetOutermost()->GetFName();
	SystemInfo.bUseRapidIterationParams = System_GT->ShouldUseRapidIterationParameters();
	SystemInfo.bDisableDebugSwitches = System_GT->ShouldDisableDebugSwitches();
	SystemInfo.ConstantResolver = FNiagaraFixedConstantResolver(FCompileConstantResolver(System_GT.Get(), ENiagaraScriptUsage::SystemSpawnScript));
	SystemInfo.OwnedScriptKeys = { SystemSpawnScript, SystemUpdateScript };
	SystemInfo.SystemSourceGraph = DigestDatabase.CreateGraphDigest(SystemGraph, ChangeIdBuilder);

	if (RIParamMode == ENiagaraCompileRIParamMode::GameThread)
	{
		TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalSystemData = System_GT->GetCachedTraversalData();
		SystemInfo.StaticVariableResults = static_cast<FNiagaraGraphCachedBuiltHistory*>(CachedTraversalSystemData.Get())->StaticVariables;
	}
	else
	{
		TArray<FNiagaraVariable> StaticVariablesFromEmitters;

		System_GT->GatherStaticVariables(SystemInfo.InitialStaticVariables, StaticVariablesFromEmitters);
		for (const FNiagaraVariable& EmitterVariable : StaticVariablesFromEmitters)
		{
			SystemInfo.InitialStaticVariables.AddUnique(EmitterVariable);
		}
	}

	const TArray<FNiagaraEmitterHandle>& EmitterHandles = System_GT->GetEmitterHandles();
	const int32 SourceEmitterCount = EmitterHandles.Num();

	SystemInfo.EmitterInfo.Reserve(SourceEmitterCount);
	for (int32 SourceEmitterIndex = 0; SourceEmitterIndex < SourceEmitterCount; ++SourceEmitterIndex)
	{
		const FNiagaraEmitterHandle& Handle = EmitterHandles[SourceEmitterIndex];
		if (!Handle.GetIsEnabled())
		{
			continue;
		}

		const int32 DigestedEmitterIndex = SystemInfo.EmitterInfo.Num();

		const FVersionedNiagaraEmitter& HandleInstance = Handle.GetInstance();
		if (FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData())
		{
			FEmitterInfo& EmitterInfo = SystemInfo.EmitterInfo.AddDefaulted_GetRef();

			const UNiagaraGraph* EmitterGraph = GetGraphFromScriptSource(EmitterData->GraphSource);
			ChangeIdBuilder.ParseReferencedGraphs(EmitterGraph);

			if (const UNiagaraEmitter* Emitter = HandleInstance.Emitter)
			{
				EmitterInfo.UniqueEmitterName = HandleInstance.Emitter->GetUniqueEmitterName();
				EmitterInfo.EmitterHandleId = Handle.GetId();
				EmitterInfo.UniqueInstanceName = Handle.GetUniqueInstanceName();
				EmitterInfo.Enabled = Handle.GetIsEnabled();
				EmitterInfo.ConstantResolver = FNiagaraFixedConstantResolver(FCompileConstantResolver(HandleInstance, ENiagaraScriptUsage::EmitterSpawnScript));
				EmitterInfo.SourceEmitterIndex = SourceEmitterIndex;
				EmitterInfo.DigestedEmitterIndex = DigestedEmitterIndex;
				EmitterInfo.SourceGraph = DigestDatabase.CreateGraphDigest(EmitterGraph, ChangeIdBuilder);

				if (RIParamMode == ENiagaraCompileRIParamMode::GameThread)
				{
					EmitterInfo.StaticVariableResults = SystemInfo.StaticVariableResults;

					TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalEmitterData = Handle.GetInstance().Emitter->GetCachedTraversalData(Handle.GetInstance().Version);
					EmitterInfo.StaticVariableResults.Append(static_cast<FNiagaraGraphCachedBuiltHistory*>(CachedTraversalEmitterData.Get())->StaticVariables);
				}
				else
				{
					EmitterData->GatherStaticVariables(EmitterInfo.InitialStaticVariables);
				}

				// be sure to incorporate our constant resolver into the top level SystemInfo.ConstantResolver
				SystemInfo.ConstantResolver.AddChildResolver(EmitterInfo.EmitterHandleId, EmitterInfo.ConstantResolver);
			}

			{
				constexpr bool bCompilableOnly = false;
				constexpr bool bEnabledOnly = false;
				TArray<UNiagaraScript*> EmitterScripts;
				EmitterData->GetScripts(EmitterScripts, bCompilableOnly, bEnabledOnly);
				EmitterInfo.OwnedScriptKeys.Reserve(EmitterScripts.Num());
				for (const UNiagaraScript* EmitterScript : EmitterScripts)
				{
					EmitterInfo.OwnedScriptKeys.Add(EmitterScript);
				}
			}

			const TArray<UNiagaraSimulationStageBase*> SimStages = EmitterData->GetSimulationStages();
			EmitterInfo.SimStages.Reserve(SimStages.Num());
			for (const UNiagaraSimulationStageBase* SourceSimStage : SimStages)
			{
				if (SourceSimStage)
				{
					FNiagaraSimulationStageInfo& SimStageInfo = EmitterInfo.SimStages.AddDefaulted_GetRef();
					SimStageInfo.bEnabled = SourceSimStage->bEnabled;
					SimStageInfo.StageId = SourceSimStage->Script->GetUsageId();
					
					if (const UNiagaraSimulationStageGeneric* SourceGenericSimStage = Cast<const UNiagaraSimulationStageGeneric>(SourceSimStage))
					{
						SimStageInfo.bGenericStage = true;
						SimStageInfo.IterationSource = SourceGenericSimStage->IterationSource;
						if (SimStageInfo.IterationSource == ENiagaraIterationSource::DataInterface)
						{
							SimStageInfo.DataInterfaceBindingName = SourceGenericSimStage->DataInterface.BoundVariable.GetName();
						}
					}

					{
						TArray<FNiagaraSimulationStageCompilationData> SimStageCompilationData;

						SimStageInfo.bHasCompilationData = SourceSimStage->FillCompilationData(SimStageCompilationData);
						if (SimStageInfo.bHasCompilationData)
						{
							SimStageInfo.CompilationData = SimStageCompilationData[0];
						}
					}
				}
			}
		}
	}

	System_GT->GetExposedParameters().GetParameters(SystemInfo.OriginalExposedParams);
}

void FNiagaraSystemCompilationTask::DigestParameterCollections(TConstArrayView<TWeakObjectPtr<UNiagaraParameterCollection>> Collections)
{
	DigestedParameterCollections.Reserve(Collections.Num());
	for (TWeakObjectPtr<UNiagaraParameterCollection> Collection : Collections)
	{
		if (const UNiagaraParameterCollection* CollectionPtr = Collection.Get())
		{
			DigestedParameterCollections.Add(CollectionPtr, FNiagaraDigestDatabase::Get().CreateCompilationCopy(CollectionPtr));
		}
	}
}

void FNiagaraSystemCompilationTask::DigestShaderInfo(const ITargetPlatform* InTargetPlatform, FNiagaraShaderType* InShaderType)
{
	TargetPlatform = InTargetPlatform;
	NiagaraShaderType = InShaderType;
}

void FNiagaraSystemCompilationTask::AddScript(int32 SourceEmitterIndex, UNiagaraScript* Script, const FNiagaraVMExecutableDataId& CompileId, bool bRequiresCompilation, TConstArrayView<FShaderCompileRequest> ShaderRequests)
{
	if (!Script)
	{
		return;
	}

	if (bRequiresCompilation)
	{
		FVersionedNiagaraScriptData* ScriptData = Script->GetScriptData(CompileId.ScriptVersionID);
		if (ensure(ScriptData))
		{
			FCompileGroupInfo* GroupInfo = GetCompileGroupInfo(SourceEmitterIndex);
			if (!GroupInfo)
			{
				GroupInfo = &CompileGroups.Emplace_GetRef(SourceEmitterIndex);
			}

			TArray<ENiagaraScriptUsage> DuplicateUsages;
			NiagaraCompilationTasksImpl::GetUsagesToDuplicate(Script->GetUsage(), DuplicateUsages);
			for (ENiagaraScriptUsage DuplicateUsage : DuplicateUsages)
			{
				GroupInfo->ValidUsages.AddUnique(DuplicateUsage);
			}

			const int32 CompileTaskIndex = CompileTasks.AddDefaulted();
			GroupInfo->CompileTaskIndices.Add(CompileTaskIndex);

			FCompileTaskInfo& TaskInfo = CompileTasks[CompileTaskIndex];
			TaskInfo.CompileOptions = FNiagaraCompileOptions(
				Script->GetUsage(),
				Script->GetUsageId(),
				ScriptData->ModuleUsageBitmask,
				Script->GetPathName(),
				Script->GetFullName(),
				Script->GetName());

			TaskInfo.CompileOptions.AdditionalDefines = CompileId.AdditionalDefines;
			TaskInfo.CompileOptions.AdditionalVariables = CompileId.AdditionalVariables;
			TaskInfo.SourceScript = Script;
			TaskInfo.ScriptKey = Script;
			TaskInfo.AssetPath = Script->GetPathName();
			TaskInfo.CompileId = CompileId;

			const bool bIsGpuScript = TaskInfo.CompileOptions.IsGpuScript() && UNiagaraScript::IsGPUScript(TaskInfo.CompileOptions.TargetUsage);

			TaskInfo.ScriptCompileType = bIsGpuScript
				? UNiagaraScript::AreGpuScriptsCompiledBySystem()
					? EScriptCompileType::CompileForGpu
					: EScriptCompileType::TranslateForGpu
				: TaskInfo.CompileOptions.IsGpuScript()
					? EScriptCompileType::DummyCompileForCpuStubs
					: EScriptCompileType::CompileForVm;

			if (TaskInfo.ScriptCompileType == EScriptCompileType::CompileForGpu)
			{
				const FEmitterInfo* EmitterInfo = SystemInfo.EmitterInfoBySourceEmitter(SourceEmitterIndex);
				if (EmitterInfo && EmitterInfo->Enabled)
				{
					TaskInfo.ComputeShaderTaskIndices.Reserve(ShaderRequests.Num());
					for (const FShaderCompileRequest& ShaderRequest : ShaderRequests)
					{
						TaskInfo.ComputeShaderTaskIndices.Add(CompileComputeShaderTasks.Num());
						FCompileComputeShaderTaskInfo& ShaderTaskInfo = CompileComputeShaderTasks.AddDefaulted_GetRef();

						ShaderTaskInfo.ParentCompileTaskIndex = CompileTaskIndex;
						ShaderTaskInfo.ShaderMapId = ShaderRequest.ShaderMapId;
						ShaderTaskInfo.ShaderPlatform = ShaderRequest.ShaderPlatform;
					}
				}
			}

			TaskInfo.TaskStartTime = FPlatformTime::Seconds();
		}
	}

	FVersionedNiagaraEmitterData* EmitterData = nullptr;

	if (SourceEmitterIndex != INDEX_NONE)
	{
		EmitterData = System_GT->GetEmitterHandle(SourceEmitterIndex).GetEmitterData();
	}

	FScriptInfo& Info = DigestedScriptInfo.Add(Script);
	Script->RapidIterationParameters.CopyParametersTo(Info.RapidIterationParameters, false, FNiagaraParameterStore::EDataInterfaceCopyMethod::None);
	Info.RapidIterationParameters.ParameterGuidMapping = Script->RapidIterationParameters.ParameterGuidMapping;

	{
		const FNiagaraTypeDefinition& PositionTypeDef = FNiagaraTypeDefinition::GetPositionDef();
		const FNiagaraTypeDefinition& Vec3TypeDef = FNiagaraTypeDefinition::GetVec3Def();

		// sanitize the guid mapping to account for parameters that might have been auto upgraded from Vec3 to Position
		// for LWC.  The ParameterStore will be updated, but the guid mapping may not have been
		for (const FNiagaraVariableWithOffset& ParamStoreVariable : Info.RapidIterationParameters.ReadParameterVariables())
		{
			if (ParamStoreVariable.GetType() == PositionTypeDef)
			{
				if (!Info.RapidIterationParameters.ParameterGuidMapping.Contains(ParamStoreVariable))
				{
					FNiagaraVariable VariableAsVector(Vec3TypeDef, ParamStoreVariable.GetName());

					FGuid ExistingGuid;
					if (Info.RapidIterationParameters.ParameterGuidMapping.RemoveAndCopyValue(ParamStoreVariable, ExistingGuid))
					{
						Info.RapidIterationParameters.ParameterGuidMapping.Add(ParamStoreVariable, ExistingGuid);
					}
				}
			}
		}
	}

	Info.Usage = Script->GetUsage();
	Info.UsageId = Script->GetUsageId();
	Info.SourceEmitterIndex = SourceEmitterIndex;
	for (UNiagaraScript* DependentScript : NiagaraCompilationTasksImpl::FindDependentScripts(System_GT.Get(), EmitterData, Script))
	{
		Info.DependentScripts.AddUnique(DependentScript);
	}
}

void FNiagaraSystemCompilationTask::Tick()
{
	UE_LOG_CONTEXT("NiagaraSystem", SystemInfo.GetSystemPath());
	switch (CompilationState)
	{
		case EState::WaitingForProcessing:
		{
			// check to see if the task's compiler has completed
			for (FCompileTaskInfo& CompileTask : CompileTasks)
			{
				const bool HasPendingCompilationTask = CompileTask.CompileResultsReadyEvent.IsValid()
					&& CompileTask.CompileResultsReadyEvent->IsValid()
					&& !CompileTask.CompileResultsReadyEvent->IsCompleted();

				if (HasPendingCompilationTask)
				{
					bool bRetrieveCompleted = false;

					switch (CompileTask.ScriptCompileType)
					{
						case EScriptCompileType::CompileForVm:
						case EScriptCompileType::TranslateForGpu:
							bRetrieveCompleted = CompileTask.RetrieveCompilationResult(false);
						break;

						case EScriptCompileType::DummyCompileForCpuStubs:
							CompileTask.RetrieveTranslateResult();
							bRetrieveCompleted = true;
						break;

						case EScriptCompileType::CompileForGpu:
							bRetrieveCompleted = CompileTask.RetrieveShaderMap(false);
						break;
					}

					if (bRetrieveCompleted)
					{
						CompileTask.CompileResultsReadyEvent->Trigger();
					}
				}
			}
		} break;

		case EState::Completed:
		{
			if (bAborting)
			{
				CompilationState = EState::Aborted;
			}
		} break;
	}
}

bool FNiagaraSystemCompilationTask::Poll(FNiagaraSystemAsyncCompileResults& Results) const
{
	// ignore actually collecting the data if we're in the middle of aborting and just let the caller know
	// that there's no tasks waiting for them
	if (bAborting)
	{
		return true;
	}

	switch (CompilationState)
	{
		case EState::ResultsProcessed:
		case EState::Completed:
		{
			for (const FCompileTaskInfo& TaskInfo : CompileTasks)
			{
				if (UNiagaraScript* SourceScript = TaskInfo.SourceScript.Get())
				{
					FNiagaraScriptAsyncCompileData& CompileData = Results.CompileResultMap.Add(SourceScript);

					CompileData.bFromDerivedDataCache = TaskInfo.DDCTaskInfo.bFromDerivedDataCache;
					CompileData.CompileId = TaskInfo.CompileId;
					CompileData.ExeData = TaskInfo.ExeData;

					for (int32 ShaderTaskIndex : TaskInfo.ComputeShaderTaskIndices)
					{
						const FCompileComputeShaderTaskInfo& ShaderTaskInfo = CompileComputeShaderTasks[ShaderTaskIndex];

						FNiagaraCompiledShaderInfo& ResultsInfo = CompileData.CompiledShaders.AddDefaulted_GetRef();
						ResultsInfo.TargetPlatform = TargetPlatform;
						ResultsInfo.ShaderPlatform = ShaderTaskInfo.ShaderPlatform;
						ResultsInfo.FeatureLevel = ShaderTaskInfo.ShaderMapId.FeatureLevel;
						ResultsInfo.CompiledShader = ShaderTaskInfo.ShaderMap;
						ResultsInfo.CompilationErrors = ShaderTaskInfo.CompilationErrors;
					}

					CompileData.NamedDataInterfaces.Reserve(TaskInfo.NamedDataInterfaces.Num());
					Algo::Transform(TaskInfo.NamedDataInterfaces, CompileData.NamedDataInterfaces, [](const TTuple<FName, UNiagaraDataInterface*>& InElement)
					{
						return MakeTuple(InElement.Key, InElement.Value);
					});

					CompileData.CompileMetrics.TaskWallTime = (float) (FPlatformTime::Seconds() - TaskInfo.TaskStartTime);
					CompileData.CompileMetrics.DDCFetchTime = 0.0f;
					CompileData.CompileMetrics.CompilerWallTime = TaskInfo.CompilerWallTime;
					CompileData.CompileMetrics.CompilerWorkerTime = TaskInfo.CompilerWorkerTime;
					CompileData.CompileMetrics.CompilerPreprocessTime = TaskInfo.CompilerPreprocessTime;
					CompileData.CompileMetrics.TranslateTime = TaskInfo.TranslationTime;
					CompileData.CompileMetrics.ByteCodeOptimizeTime = TaskInfo.ByteCodeOptimizeTime;

					if (const FScriptInfo* ScriptInfo = DigestedScriptInfo.Find(SourceScript))
					{
						if (const FEmitterInfo* EmitterInfo = SystemInfo.EmitterInfoBySourceEmitter(ScriptInfo->SourceEmitterIndex))
						{
							CompileData.UniqueEmitterName = EmitterInfo->UniqueEmitterName;
						}

						// we also need to incorporate the rapid iteration parameters that we encountered
						TConstArrayView<FNiagaraVariableWithOffset> RapidIterationParameters = ScriptInfo->RapidIterationParameters.ReadParameterVariables();

						CompileData.RapidIterationParameters.Reserve(RapidIterationParameters.Num());
						for (const FNiagaraVariableWithOffset& ParamWithOffset : RapidIterationParameters)
						{
							FNiagaraVariable& Parameter = CompileData.RapidIterationParameters.Add_GetRef(ParamWithOffset);
							Parameter.SetData(ScriptInfo->RapidIterationParameters.GetParameterData(ParamWithOffset));
						}
					}
				}
			}

			Results.bForced = bForced;
			Results.ExposedVariables = SystemExposedVariables.Array();

			return true;
		}
	}

	return false;
}

bool FNiagaraSystemCompilationTask::CanRemove() const
{
	return (CompilationState == EState::Completed && ResultsRetrieved)
		|| (CompilationState == EState::Aborted);
}

bool FNiagaraSystemCompilationTask::AreResultsPending() const
{
	return CompilationState == EState::Completed && !ResultsRetrieved;
}

void FNiagaraSystemCompilationTask::WaitTillCompileCompletion()
{
	const FTimespan WaitTimeout = FTimespan::FromMilliseconds(50.0);
	while (!CompileCompletionEvent.IsCompleted())
	{
		// if the wait doesn't complete the task then we need to make sure to poke the
		// compilation manager since there we may be waiting on it
		if (!CompileCompletionEvent.Wait(WaitTimeout))
		{
			FNiagaraSystemCompilingManager::Get().AdvanceAsyncTasks();
		}
	}
}

void FNiagaraSystemCompilationTask::WaitTillCachePutCompletion()
{
	if (PutRequestHelper.IsValid())
	{
		PutRequestHelper->CompletionEvent.Wait();
	}
}

void FNiagaraSystemCompilationTask::Precompile()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAsyncTask_Precompile);

	SystemPrecompileData = MakeShared<FNiagaraPrecompileData, ESPMode::ThreadSafe>();

	SystemPrecompileData->SharedCompileDataInterfaceData = MakeShared<TArray<FNiagaraPrecompileData::FCompileDataInterfaceData>>();
	TArray<TSharedPtr<FNiagaraPrecompileData, ESPMode::ThreadSafe>> DependentRequests;
	FCompileConstantResolver EmptyResolver;

	SystemPrecompileData->SourceName = SystemInfo.SystemName;

	SystemPrecompileData->DigestedSourceGraph = SystemInfo.SystemSourceGraph;
	SystemPrecompileData->bUseRapidIterationParams = SystemInfo.bUseRapidIterationParams;
	SystemPrecompileData->bDisableDebugSwitches = SystemInfo.bDisableDebugSwitches;

	TArray<FString> EmitterNames;

	// Create an array of variables that we might encounter when traversing the graphs (include the originally exposed vars above)
	TArray<FNiagaraVariable> EncounterableVars = SystemInfo.OriginalExposedParams;

	const int32 EmitterCount = SystemInfo.EmitterInfo.Num();
	SystemPrecompileData->EmitterData.Empty(EmitterCount);

	for (const FEmitterInfo& EmitterInfo : SystemInfo.EmitterInfo)
	{
		TSharedPtr<FNiagaraPrecompileData, ESPMode::ThreadSafe> EmitterPtr = MakeShared<FNiagaraPrecompileData, ESPMode::ThreadSafe>();
		EmitterPtr->EmitterUniqueName = EmitterInfo.UniqueEmitterName;
		EmitterPtr->EmitterID = FNiagaraEmitterID(EmitterInfo.SourceEmitterIndex);
		EmitterPtr->SourceName = SystemPrecompileData->SourceName;
		EmitterPtr->DigestedSourceGraph = EmitterInfo.SourceGraph;
		EmitterPtr->bUseRapidIterationParams = SystemPrecompileData->bUseRapidIterationParams;
		EmitterPtr->bDisableDebugSwitches = SystemPrecompileData->bDisableDebugSwitches;
		EmitterPtr->SharedCompileDataInterfaceData = SystemPrecompileData->SharedCompileDataInterfaceData;
		SystemPrecompileData->EmitterData.Add(EmitterPtr);
		EmitterNames.Add(EmitterInfo.UniqueInstanceName);
	}

	// Now deep copy the system graphs, skipping traversal into any emitter references.
	{
		SystemPrecompileData->FinishPrecompile(
			*this,
			EncounterableVars,
			SystemInfo.StaticVariableResults,
			SystemInfo.ConstantResolver,
			{ ENiagaraScriptUsage::SystemSpawnScript, ENiagaraScriptUsage::SystemUpdateScript },
			{},
			EmitterNames);

		SystemPrecompileData->CollectBakedRapidIterationParameters(*this, SystemInfo.OwnedScriptKeys);
	}

	// Add the User and System variables that we did encounter to the list that emitters might also encounter.
	TArray<FNiagaraVariable> SystemEncounteredUserVariables;
	SystemPrecompileData->GatherPreCompiledVariables(FNiagaraConstants::UserNamespaceString, SystemEncounteredUserVariables);
	NiagaraCompilationTasksImpl::AppendUnique(SystemEncounteredUserVariables, EncounterableVars);
	SystemExposedVariables.Append(SystemEncounteredUserVariables);

	SystemPrecompileData->GatherPreCompiledVariables(FNiagaraConstants::SystemNamespaceString, EncounterableVars);

	// Now we can finish off the emitters.
	for (int32 EmitterIt = 0; EmitterIt < EmitterCount; ++EmitterIt)
	{
		const FEmitterInfo& EmitterInfo = SystemInfo.EmitterInfo[EmitterIt];
		UE_LOG_CONTEXT("NiagaraEmitter", EmitterInfo.UniqueEmitterName);

		if (EmitterInfo.Enabled) // Don't pull in the emitter if it isn't going to be used.
		{
			TArray<FNiagaraVariable> StaticVariablesFromEmitter = SystemInfo.StaticVariableResults;
			StaticVariablesFromEmitter.Append(EmitterInfo.StaticVariableResults);

			FNiagaraPrecompileData* EmitterPrecompileData = SystemPrecompileData->EmitterData[EmitterIt].Get();

			EmitterPrecompileData->FinishPrecompile(
				*this,
				EncounterableVars,
				StaticVariablesFromEmitter,
				EmitterInfo.ConstantResolver,
				{ ENiagaraScriptUsage::EmitterSpawnScript, ENiagaraScriptUsage::EmitterUpdateScript },
				{},
				EmitterNames);

			// Then finish the precompile for the particle scripts once we've gathered the emitter vars which might be referenced.
			TArray<FNiagaraVariable> ParticleEncounterableVars = EncounterableVars;
			EmitterPrecompileData->GatherPreCompiledVariables(FNiagaraConstants::EmitterNamespaceString, ParticleEncounterableVars);

			TArray<FNiagaraVariable> EmitterEncounteredUserVariables;
			EmitterPrecompileData->GatherPreCompiledVariables(FNiagaraConstants::UserNamespaceString, EmitterEncounteredUserVariables);
			SystemExposedVariables.Append(EmitterEncounteredUserVariables);

			TArray<FNiagaraVariable> OldStaticVars = EmitterPrecompileData->StaticVariables;
			EmitterPrecompileData->FinishPrecompile(
				*this,
				ParticleEncounterableVars,
				OldStaticVars,
				EmitterInfo.ConstantResolver,
				{
					ENiagaraScriptUsage::ParticleSpawnScript,
					ENiagaraScriptUsage::ParticleSpawnScriptInterpolated,
					ENiagaraScriptUsage::ParticleUpdateScript,
					ENiagaraScriptUsage::ParticleEventScript,
					ENiagaraScriptUsage::ParticleGPUComputeScript,
					ENiagaraScriptUsage::ParticleSimulationStageScript 
				},
				EmitterInfo.SimStages,
				EmitterNames);

			EmitterPrecompileData->CollectBakedRapidIterationParameters(*this, EmitterInfo.OwnedScriptKeys);
		}
	}
}

void FNiagaraSystemCompilationTask::GetAvailableCollections(TArray<FNiagaraCompilationNPCHandle>& OutCollections) const
{
	DigestedParameterCollections.GenerateValueArray(OutCollections);
}

bool FNiagaraSystemCompilationTask::GenerateCompileMetaData() const
{
	return bGenerateCompileMetaData;
}

const FGuid& FNiagaraSystemCompilationTask::GetCompilationTaskId() const
{
	return CompilationTaskId;
}

bool FNiagaraSystemCompilationTask::GetStageName(int32 EmitterIndex, const FNiagaraCompilationNodeOutput* OutputNode, FName& OutStageName) const
{
	OutStageName = NAME_None;

	if (EmitterIndex == INDEX_NONE)
	{
		return true;
	}

	bool bStageEnabled = true;
	if (OutputNode->Usage == ENiagaraScriptUsage::ParticleSimulationStageScript)
	{
		const FNiagaraSimulationStageInfo* SimStageInfo = SystemInfo.EmitterInfo[EmitterIndex].SimStages.FindByPredicate([OutputNode](const FNiagaraSimulationStageInfo& SimStageInfo)
		{
			return SimStageInfo.StageId == OutputNode->UsageId;
		});

		bStageEnabled = SimStageInfo && SimStageInfo->bEnabled;
		if (bStageEnabled && SimStageInfo->bGenericStage)
		{
			OutStageName = SimStageInfo->DataInterfaceBindingName;
		}
	}

	return bStageEnabled;		
}

struct FNiagaraSystemCompilationTask::FCollectStaticVariablesTaskBuilder
{
	FCollectStaticVariablesTaskBuilder(FNiagaraSystemCompilationTask& CompilationTask)
	: FoundStaticVariables(CompilationTask.SystemInfo.StaticVariableResults)
	{
		if (FNiagaraCompilationGraphDigested* SystemGraph = CompilationTask.SystemInfo.SystemSourceGraph.Get())
		{
			FStaticVariableBuilderTaskHandle BuilderTask = MakeShared<FStaticVariableBuilderTask, ESPMode::ThreadSafe>();
			BuilderTask->CompilationTask = &CompilationTask;
			BuilderTask->ConstantResolver = CompilationTask.SystemInfo.ConstantResolver;
			BuilderTask->GraphContext = SystemGraph;
			BuilderTask->InitialStaticVariables = CompilationTask.SystemInfo.InitialStaticVariables;
			SystemGraph->FindOutputNodes(BuilderTask->OutputNodes);
			BuilderTask->StageNames.Init(NAME_None, BuilderTask->OutputNodes.Num());

			StaticVariableTasksToProcess.Add(BuilderTask);
		}
	}

	FCollectStaticVariablesTaskBuilder(const FNiagaraSystemCompilationTask& CompilationTask, FEmitterInfo& EmitterInfo)
	: FoundStaticVariables(EmitterInfo.StaticVariableResults)
	{
		if (FNiagaraCompilationGraphDigested* EmitterGraph = EmitterInfo.SourceGraph.Get())
		{
			TArray<const FNiagaraCompilationNodeOutput*> OutputNodes;
			EmitterGraph->FindOutputNodes(OutputNodes);

			// collect sim stage info for each of the output nodes (if they are even sim stages)
			TArray<TTuple<bool, FName>> OutputNodeStageInfo;
			OutputNodeStageInfo.Reserve(OutputNodes.Num());
			int32 EnabledOutputNodeCount = 0;
			Algo::Transform(OutputNodes, OutputNodeStageInfo, [&EmitterInfo, &EnabledOutputNodeCount](const FNiagaraCompilationNodeOutput* OutputNode) -> TTuple<bool, FName>
			{
				bool bEnabled = true;
				FName StageName = NAME_None;

				if (OutputNode->Usage == ENiagaraScriptUsage::ParticleSimulationStageScript)
				{
					const FNiagaraSimulationStageInfo* SimStage = EmitterInfo.SimStages.FindByPredicate([OutputNode](const FNiagaraSimulationStageInfo& StageInfo) -> bool
					{
						return StageInfo.StageId == OutputNode->UsageId;
					});

					if (SimStage)
					{
						bEnabled = SimStage->bEnabled;
						StageName = SimStage->DataInterfaceBindingName;
					}
				}

				if (bEnabled)
				{
					++EnabledOutputNodeCount;
				}

				return MakeTuple(bEnabled, StageName);
			});

			if (EnabledOutputNodeCount)
			{
				FStaticVariableBuilderTaskHandle BuilderTask = MakeShared<FStaticVariableBuilderTask, ESPMode::ThreadSafe>();
				BuilderTask->CompilationTask = &CompilationTask;
				BuilderTask->ConstantResolver = EmitterInfo.ConstantResolver;
				BuilderTask->GraphContext = EmitterGraph;
				BuilderTask->UniqueEmitterName = EmitterInfo.UniqueEmitterName;

				// the initial static variables needs to come from gathering data from the Emitter
				// but also what has been calculated in the SystemInfo
				BuilderTask->InitialStaticVariables = CompilationTask.SystemInfo.StaticVariableResults;
				for (const FNiagaraVariable& EmitterGatheredVariable : EmitterInfo.InitialStaticVariables)
				{
					BuilderTask->InitialStaticVariables.AddUnique(EmitterGatheredVariable);
				}

				// Only use the static variables that match up with our expectations for this script. IE for emitters, filter things out for resolution.
				{ // FNiagaraParameterUtilities::FilterToRelevantStaticVariables
					FNiagaraAliasContext RenameContext(ENiagaraScriptUsage::ParticleSpawnScript);
					RenameContext.ChangeEmitterName(EmitterInfo.UniqueEmitterName, FNiagaraConstants::EmitterNamespaceString);

					for (FNiagaraVariable& StaticVariable : BuilderTask->InitialStaticVariables)
					{
						const FNiagaraVariable ResolvedVariable = FNiagaraUtilities::ResolveAliases(StaticVariable, RenameContext);
						StaticVariable.SetName(ResolvedVariable.GetName());
					}
				}

				//BuilderTask->InitialStaticVariables = EmitterInfo.InitialStaticVariables;
				BuilderTask->OutputNodes.Reserve(EnabledOutputNodeCount);
				BuilderTask->StageNames.Reserve(EnabledOutputNodeCount);

				for (int32 OutputNodeIt = 0; OutputNodeIt < OutputNodes.Num(); ++OutputNodeIt)
				{
					if (OutputNodeStageInfo[OutputNodeIt].Key)
					{
						BuilderTask->OutputNodes.Add(OutputNodes[OutputNodeIt]);
						BuilderTask->StageNames.Add(OutputNodeStageInfo[OutputNodeIt].Value);
					}
				}

				StaticVariableTasksToProcess.Add(BuilderTask);
			}
		}
	}

	struct FStaticVariableBuilderTask
	{
		const FNiagaraSystemCompilationTask* CompilationTask = nullptr;
		FNiagaraFixedConstantResolver ConstantResolver;
		const FNiagaraCompilationGraph* GraphContext = nullptr;
		FString UniqueEmitterName;
		TArray<const FNiagaraCompilationNodeOutput*> OutputNodes;
		TArray<FNiagaraVariable> InitialStaticVariables;
		TArray<FName> StageNames;

		TArray<FNiagaraVariable> FoundStaticVariables;
	};

	using FStaticVariableBuilderTaskHandle = TSharedPtr<FStaticVariableBuilderTask, ESPMode::ThreadSafe>;

	static UE::Tasks::FTask LaunchCollectedTasks(FCollectStaticVariablesTaskBuilder&& Context)
	{
		using namespace UE::Tasks;
		if (!Context.StaticVariableTasksToProcess.IsEmpty())
		{
			// fire off the tasks related to collecting static variables
			const int32 TaskCount = Context.StaticVariableTasksToProcess.Num();
			TArray<FTask> PendingTasks;
			PendingTasks.Reserve(TaskCount);

			for (FStaticVariableBuilderTaskHandle& BuilderTask : Context.StaticVariableTasksToProcess)
			{
				PendingTasks.Add(Launch(UE_SOURCE_LOCATION, [BuilderTask]
				{
					TArray<FNiagaraVariable> ActiveStaticVariables = BuilderTask->InitialStaticVariables;

					const int32 OutputNodeCount = BuilderTask->OutputNodes.Num();
					for (int32 OutputNodeIt = 0; OutputNodeIt < OutputNodeCount; ++OutputNodeIt)
					{
						const FNiagaraCompilationNodeOutput* OutputNode = BuilderTask->OutputNodes[OutputNodeIt];

						TNiagaraParameterMapHistoryWithMetaDataBuilder<FNiagaraCompilationDigestBridge> Builder;
						*Builder.ConstantResolver = BuilderTask->ConstantResolver;
						Builder.AddGraphToCallingGraphContextStack(BuilderTask->GraphContext);
						Builder.RegisterExternalStaticVariables(ActiveStaticVariables);
						BuilderTask->CompilationTask->GetAvailableCollections(Builder.AvailableCollections->EditCollections());
						Builder.BeginTranslation(FNiagaraConstants::EmitterNamespaceString);
						Builder.EnableScriptAllowList(true, OutputNode->Usage);
						Builder.IncludeStaticVariablesOnly();
						Builder.BeginUsage(OutputNode->Usage, BuilderTask->StageNames[OutputNodeIt]);
						Builder.BuildParameterMaps(OutputNode, true);
						Builder.EndUsage();

						for (const FNiagaraVariable& FoundVariable : Builder.StaticVariables)
						{
							ActiveStaticVariables.AddUnique(FoundVariable);
						}
					}

					BuilderTask->FoundStaticVariables = MoveTemp(ActiveStaticVariables);
				}));
			}

			TArray<FStaticVariableBuilderTaskHandle> MergeTaskInput = Context.StaticVariableTasksToProcess;
			TArray<FNiagaraVariable>& MergeTaskOutput = Context.FoundStaticVariables;
			return Launch(UE_SOURCE_LOCATION, [MergeTaskInput, &MergeTaskOutput]
			{
				if (MergeTaskInput.Num() == 1)
				{
					MergeTaskOutput = MergeTaskInput[0]->FoundStaticVariables;
				}
				else
				{
					for (const FStaticVariableBuilderTaskHandle& BuilderTask : MergeTaskInput)
					{
						for (const FNiagaraVariable& BuilderTaskVariable : BuilderTask->FoundStaticVariables)
						{
							MergeTaskOutput.AddUnique(BuilderTaskVariable);
						}
					}
				}
			}, PendingTasks);
		}

		return FTask();
	}

	TArray<FStaticVariableBuilderTaskHandle> StaticVariableTasksToProcess;
	TArray<FNiagaraVariable>& FoundStaticVariables;
};

struct FNiagaraSystemCompilationTask::FBuildRapidIterationTaskBuilder
{
	FBuildRapidIterationTaskBuilder(FNiagaraSystemCompilationTask& CompilationTask, FScriptInfo& ScriptInfo)
		: ParameterStore(ScriptInfo.RapidIterationParameters)
	{
		const FNiagaraCompilationGraph* Graph;
		FString UniqueEmitterName;
		FNiagaraFixedConstantResolver ConstantResolver;
	
		if (const FEmitterInfo* EmitterInfo = CompilationTask.SystemInfo.EmitterInfoBySourceEmitter(ScriptInfo.SourceEmitterIndex))
		{
			Graph = EmitterInfo->SourceGraph.Get();
			FoundStaticVariables = &EmitterInfo->StaticVariableResults;
			UniqueEmitterName = EmitterInfo->UniqueEmitterName;
			ConstantResolver = EmitterInfo->ConstantResolver;
		}
		else
		{
			Graph = CompilationTask.SystemInfo.SystemSourceGraph.Get();
			FoundStaticVariables = &CompilationTask.SystemInfo.StaticVariableResults;
			ConstantResolver = CompilationTask.SystemInfo.ConstantResolver;
		}

		if (Graph)
		{
			TArray<const FNiagaraCompilationNodeOutput*> OutputNodes;
			if (ScriptInfo.Usage == ENiagaraScriptUsage::ParticleGPUComputeScript)
			{
				Graph->FindOutputNodes(OutputNodes);
				OutputNodes.SetNum(Algo::RemoveIf(OutputNodes, [](const FNiagaraCompilationNodeOutput* OutputNode) -> bool
				{
					return !UNiagaraScript::IsParticleScript(OutputNode->Usage);
				}));
			}
			else
			{
				if (const FNiagaraCompilationNodeOutput* OutputNode = Graph->FindEquivalentOutputNode(ScriptInfo.Usage, ScriptInfo.UsageId))
				{
					OutputNodes.Add(OutputNode);
				}
			}

			for (const FNiagaraCompilationNodeOutput* OutputNode : OutputNodes)
			{
				TArray<const FNiagaraCompilationNode*> TraversalNodes;
				Graph->BuildTraversal(OutputNode, TraversalNodes);
				for (const FNiagaraCompilationNode* TraversalNode : TraversalNodes)
				{
					if (const FNiagaraCompilationNodeFunctionCall* FunctionCallNode = TraversalNode->AsType<FNiagaraCompilationNodeFunctionCall>())
					{
						if (FunctionCallNode->CalledGraph.IsValid())
						{
							FRapidIterationBuilderTaskHandle BuilderTask = MakeShared<FRapidIterationBuilderTask, ESPMode::ThreadSafe>();
							BuilderTask->CompilationTask = &CompilationTask;
							BuilderTask->ConstantResolver = ConstantResolver;
							BuilderTask->FunctionCallNode = FunctionCallNode;
							BuilderTask->UniqueEmitterName = UniqueEmitterName;
							BuilderTask->ScriptUsage = ScriptInfo.Usage;

							RapidIterationTasksToProcess.Add(BuilderTask);
						}
					}
				}
			}
		}
	}

	struct FRapidIterationBuilderTask
	{
		const FNiagaraSystemCompilationTask* CompilationTask = nullptr;
		FNiagaraFixedConstantResolver ConstantResolver;
		const FNiagaraCompilationNodeFunctionCall* FunctionCallNode = nullptr;
		TArray<FNiagaraVariable> ModuleInputVariables;
		FString UniqueEmitterName;
		ENiagaraScriptUsage ScriptUsage;
	};

	using FRapidIterationBuilderTaskHandle = TSharedPtr<FRapidIterationBuilderTask, ESPMode::ThreadSafe>;

	UE::Tasks::FTask LaunchCollectedTasks()
	{
		using namespace UE::Tasks;

		const int32 TaskCount = RapidIterationTasksToProcess.Num();
		TArray<FTask> PendingTasks;
		PendingTasks.Reserve(TaskCount);

		const TArray<FNiagaraVariable>* OwnerStaticVariables = FoundStaticVariables;

		for (FRapidIterationBuilderTaskHandle& BuilderTask : RapidIterationTasksToProcess)
		{
			PendingTasks.Add(Launch(UE_SOURCE_LOCATION, [BuilderTask, OwnerStaticVariables]
			{
				constexpr bool bIgnoreDisabled = false;
				constexpr bool bFilterForCompilation = false;

				TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationDigestBridge> Builder;
				Builder.SetIgnoreDisabled(bIgnoreDisabled);
				*Builder.ConstantResolver = BuilderTask->ConstantResolver;
				Builder.RegisterExternalStaticVariables(*OwnerStaticVariables);
				BuilderTask->CompilationTask->GetAvailableCollections(Builder.AvailableCollections->EditCollections());

				/* The line below represents a compromise. It basically says don't look up the static variable from the
				static variables list set in RegisterExternalStaticVariables as a rapid iteration parameter. Why? Rapid Iteration Parameters
				may be stale and linger around from prior changes that have been abandoned. We keep them around so that when you toggle back
				in the UI, you haven't also lost your last set values for that version. So therefore, we can't officially remove them.

				The issue comes in when a static variable is set via some other means, linking to another variable for example. When generating
				the UI, it *should* show anything that would match up to the linked variable, but because this is a parameter map history
				traversal that is in isolation on the node and doesn't include upstream set nodes, it will fail and just look up the rapid iteration
				version if left to its' own devices.

				Trial 1 at a fix was to cache all the values at the root of the graph and look them up when we build the static variables list
				we use above. That isn't good because that version skips all disabled modules and we want the UI to be consistent regardless
				of disabled module status.

				Trial 2 was to back up and include the precursor gets/sets. This fails for the same reason, ultimately something will
				be bound to a variable that will assume that it needs to look up a bogus rapid iteration value and we end up back here.

				Trial 3 just says, we know the static variables list above includes basically all relevant intermediate static values. So if
				we just circumvent the logic in the parameter map history traversal that looks up by rapid iteration parameter, it will
				find the right "cached" value. So we just do that below.
					*/
				Builder.SetIgnoreStaticRapidIterationParameters(true);

				// if we are only dealing with the module input pins then we don't need to delve deep into the graph
				Builder.MaxGraphDepthTraversal = 1;

				BuilderTask->FunctionCallNode->BuildParameterMapHistory(Builder, false, bFilterForCompilation);

				if (Builder.Histories.Num() == 1)
				{
					BuilderTask->ModuleInputVariables = NiagaraCompilationTasksImpl::ExtractInputVariablesFromHistory(
						Builder.Histories[0], BuilderTask->FunctionCallNode->CalledGraph.Get());

					BuilderTask->FunctionCallNode->MultiFindParameterMapDefaultValues(BuilderTask->ScriptUsage, BuilderTask->ConstantResolver, BuilderTask->ModuleInputVariables);
				}
			}));
		}

		TArray<FRapidIterationBuilderTaskHandle> MergeTaskInput = RapidIterationTasksToProcess;
		FNiagaraParameterStore& MergeTaskOutput = ParameterStore;
		return Launch(UE_SOURCE_LOCATION, [MergeTaskInput, &MergeTaskOutput]
		{
			TSet<FName> ValidParameterNames;

			// we need to build a reverse mapping of the ParameterGuidMapping so that we can quickly find renamed parameters
			TMultiMap<FGuid, FNiagaraVariable> GuidMapping;
			for (const FNiagaraVariableBase& RapidIterationParameter : MergeTaskOutput.ReadParameterVariables())
			{
				if (FGuid* ParameterGuid = MergeTaskOutput.ParameterGuidMapping.Find(RapidIterationParameter))
				{
					GuidMapping.Add(*ParameterGuid, RapidIterationParameter);
				}
			}

			for (FRapidIterationBuilderTaskHandle InputTask : MergeTaskInput)
			{
				for (const FNiagaraVariable& ModuleInputVariable : InputTask->ModuleInputVariables)
				{
					if (FNiagaraStackGraphUtilities::IsRapidIterationType(ModuleInputVariable.GetType()))
					{
						const FName FunctionName = FName(InputTask->FunctionCallNode->FunctionName);

						FNiagaraParameterHandle AliasedFunctionInputHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(ModuleInputVariable.GetName(), FunctionName);
						const FName AliasedName = AliasedFunctionInputHandle.GetParameterHandleString();

						FNiagaraVariable RapidIterationParameter(ModuleInputVariable);
						const TCHAR* EmitterName = InputTask->UniqueEmitterName.IsEmpty() ? nullptr	: *InputTask->UniqueEmitterName;
						const FString RapidIterationConstantName = FNiagaraUtilities::CreateRapidIterationConstantName(AliasedName, EmitterName, InputTask->ScriptUsage);

						RapidIterationParameter.SetName(*RapidIterationConstantName);

						ValidParameterNames.Add(RapidIterationParameter.GetName());

						NiagaraCompilationTasksImpl::FixupRenamedParameter(
							ModuleInputVariable,
							RapidIterationParameter,
							InputTask->FunctionCallNode,
							GuidMapping,
							MergeTaskOutput);

						NiagaraCompilationTasksImpl::AddRapidIterationParameter(
							AliasedFunctionInputHandle,
							ModuleInputVariable,
							RapidIterationParameter,
							InputTask->FunctionCallNode,
							MergeTaskOutput);
					}
				}
			}

			// go through and remove any entries in the ParameterStore that weren't found
			TArray<FNiagaraVariableBase> ParametersToRemove;
			for (const FNiagaraVariableWithOffset& Parameter : MergeTaskOutput.ReadParameterVariables())
			{
				if (!ValidParameterNames.Contains(Parameter.GetName()))
				{
					ParametersToRemove.Add(Parameter);
				}
			}

			for (const FNiagaraVariableBase& ParameterToRemove : ParametersToRemove)
			{
				MergeTaskOutput.RemoveParameter(ParameterToRemove);
			}
		}, PendingTasks);
	}

	TArray<FRapidIterationBuilderTaskHandle> RapidIterationTasksToProcess;
	const TArray<FNiagaraVariable>* FoundStaticVariables = nullptr;
	FNiagaraParameterStore& ParameterStore;
};

UE::Tasks::FTask FNiagaraSystemCompilationTask::BuildRapidIterationParametersAsync()
{
	using namespace UE::Tasks;

	COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeSyncWork(); Timer.TrackCyclesOnly(););

	TMap<int32 /*EmitterIndex*/, FTask> CollectStaticVariableTasks;
	
	FTask SystemCollectStaticVariableTask = FCollectStaticVariablesTaskBuilder::LaunchCollectedTasks({*this});
	CollectStaticVariableTasks.Add(INDEX_NONE, SystemCollectStaticVariableTask);

	if (SystemCollectStaticVariableTask.IsValid())
	{
		for (FEmitterInfo& EmitterInfo : SystemInfo.EmitterInfo)
		{
			if (!EmitterInfo.Enabled)
			{
				continue;
			}
			FTask EmitterCollectStaticVariableTask = Launch(UE_SOURCE_LOCATION, [Self = AsShared(), &EmitterInfo]
			{
				AddNested(FCollectStaticVariablesTaskBuilder::LaunchCollectedTasks({*Self, EmitterInfo}));
			}, SystemCollectStaticVariableTask);

			CollectStaticVariableTasks.Add(EmitterInfo.DigestedEmitterIndex, EmitterCollectStaticVariableTask);
		}

		TArray<FTask> PendingTasks;
		for (TMap<TObjectKey<UNiagaraScript>, FScriptInfo>::ElementType& CurrentIt : DigestedScriptInfo)
		{
			FScriptInfo& ScriptInfo = CurrentIt.Value;

			const FEmitterInfo* EmitterInfo = SystemInfo.EmitterInfoBySourceEmitter(ScriptInfo.SourceEmitterIndex);

			PendingTasks.Add(Launch(UE_SOURCE_LOCATION, [Self = AsShared(), &ScriptInfo]
			{
				FBuildRapidIterationTaskBuilder TaskBuilder(*Self, ScriptInfo);
				AddNested(TaskBuilder.LaunchCollectedTasks());
			}, CollectStaticVariableTasks.FindRef(EmitterInfo ? EmitterInfo->DigestedEmitterIndex : INDEX_NONE)));
		}

		// We need a single task to copy over parameters between dependent scripts and make sure that the static variable
		// list is up to date
		TMap<TObjectKey<UNiagaraScript>, FScriptInfo>* ScriptInfoMapPtr = &DigestedScriptInfo;
		return Launch(UE_SOURCE_LOCATION, [ScriptInfoMapPtr, Self = AsShared()]
		{
			for (TMap<TObjectKey<UNiagaraScript>, FScriptInfo>::ElementType& CurrentIt : *ScriptInfoMapPtr)
			{
				FScriptInfo& SrcScriptInfo = CurrentIt.Value;
				for (TObjectKey<UNiagaraScript>& DependentScriptKey : SrcScriptInfo.DependentScripts)
				{
					FScriptInfo& DstScriptInfo = ScriptInfoMapPtr->FindChecked(DependentScriptKey);
					SrcScriptInfo.RapidIterationParameters.CopyParametersTo(DstScriptInfo.RapidIterationParameters, false, FNiagaraParameterStore::EDataInterfaceCopyMethod::None);
				}

				// make sure that all rapid iteration parameters on the script have made it to the system/emitter info
				for (const FNiagaraVariableWithOffset& ScriptRI : SrcScriptInfo.RapidIterationParameters.ReadParameterVariables())
				{
					if (ScriptRI.GetType().IsStatic())
					{
						FNiagaraVariable StaticVariable = FNiagaraVariable(ScriptRI);
						StaticVariable.SetData(SrcScriptInfo.RapidIterationParameters.GetParameterData(ScriptRI));

						if (SrcScriptInfo.SourceEmitterIndex == INDEX_NONE)
						{
							Self->SystemInfo.StaticVariableResults.AddUnique(StaticVariable);
						}
						else if (FEmitterInfo* EmitterInfo = Self->SystemInfo.EmitterInfoBySourceEmitter(SrcScriptInfo.SourceEmitterIndex))
						{
							EmitterInfo->StaticVariableResults.AddUnique(StaticVariable);
						}
					}
				}
			}
		}, PendingTasks);
	}

	return FTask();
}

void FNiagaraSystemCompilationTask::BuildAndApplyRapidIterationParameters()
{
	check(IsInGameThread());
	{
		COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeAsyncWait(); Timer.TrackCyclesOnly(););
		BuildRapidIterationParametersAsync().Wait();
	}

	// see FNiagaraActiveCompilationAsyncTask::Apply() for why we don't allow removal
	constexpr bool bAllowParameterRemoval = false;

	for (const TMap<TObjectKey<UNiagaraScript>, FScriptInfo>::ElementType& CurrentIt : DigestedScriptInfo)
	{
		if (UNiagaraScript* Script = CurrentIt.Key.ResolveObjectPtr())
		{
			const FScriptInfo& ScriptInfo = CurrentIt.Value;

			TConstArrayView<FNiagaraVariableWithOffset> SrcRapidIterationParameters = ScriptInfo.RapidIterationParameters.ReadParameterVariables();
			TArray<FNiagaraVariable> DstRapidIterationParameters;
			DstRapidIterationParameters.Reserve(SrcRapidIterationParameters.Num());
			for (const FNiagaraVariableWithOffset& ParamWithOffset : SrcRapidIterationParameters)
			{
				FNiagaraVariable& Parameter = DstRapidIterationParameters.Add_GetRef(ParamWithOffset);
				Parameter.SetData(ScriptInfo.RapidIterationParameters.GetParameterData(ParamWithOffset));
			}

			Script->ApplyRapidIterationParameters(DstRapidIterationParameters, bAllowParameterRemoval);
		}
	}
}

void FNiagaraSystemCompilationTask::IssuePostResultsProcessedTasks()
{
	CompileCompletionEvent.Trigger();
	CompilationState = EState::ResultsProcessed;
	PutRequestHelper = MakeUnique<FDispatchDataCachePutRequests>();
	PutRequestHelper->Launch(this);

	Launch(UE_SOURCE_LOCATION, [Self = AsShared()]
	{
		Self->CompilationState = EState::Completed;
	}, PutRequestHelper->CompletionEvent);
}

void FNiagaraSystemCompilationTask::IssueCompilationTasks()
{
	using namespace UE::Tasks;

	TArray<FTask> IssuedCompilationTasks;
	TArray<FTaskEvent> CompilationTasks;

	ETaskPriority TaskPriority = ETaskPriority::Normal;
	EExtendedTaskPriority TaskExtendedPriority = (GNiagaraCompilationForceOneThread > 0 || FApp::ShouldUseThreadingForPerformance() == false)
		? EExtendedTaskPriority::Inline 
		: EExtendedTaskPriority::None;

	if (HasOutstandingCompileTasks())
	{
		FTask PrecompileTask = Launch(UE_SOURCE_LOCATION, [Self = AsShared()]
		{
			COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeSyncWork(); Timer.TrackCyclesOnly(););
			UE_LOG_CONTEXT("NiagaraSystem", Self->SystemInfo.GetSystemPath());

			Self->Precompile();
		}, TaskPriority, TaskExtendedPriority);

		// async compilation copy tasks
		for (FCompileGroupInfo& GroupInfo : CompileGroups)
		{
			if (GroupInfo.HasOutstandingCompileTasks(*this))
			{
				FTask InstantiateGraphTask = Launch(UE_SOURCE_LOCATION, [Self = AsShared(), &GroupInfo]
				{
					COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeSyncWork(); Timer.TrackCyclesOnly(););
					UE_LOG_CONTEXT("NiagaraSystem", Self->SystemInfo.GetSystemPath());

					GroupInfo.InstantiateCompileGraph(*Self);
				}, PrecompileTask, TaskPriority, TaskExtendedPriority);

				for (int32 CompileTaskIndex : GroupInfo.CompileTaskIndices)
				{
					FCompileTaskInfo& CompileTask = CompileTasks[CompileTaskIndex];
					if (CompileTask.IsOutstanding(*this))
					{
						CompileTask.CompileResultsReadyEvent = MakeUnique<FTaskEvent>(UE_SOURCE_LOCATION);
						CompileTask.CompileResultsProcessedEvent = MakeUnique<FTaskEvent>(UE_SOURCE_LOCATION);

						IssuedCompilationTasks.Add(Launch(UE_SOURCE_LOCATION, [Self = AsShared(), &GroupInfo, &CompileTask]
						{
							COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeSyncWork(); Timer.TrackCyclesOnly(););

							FNiagaraSystemCompilationTask* TaskPtr = &Self.Get();
							UE_LOG_CONTEXT("NiagaraSystem", TaskPtr->SystemInfo.GetSystemPath());

							CompileTask.CollectNamedDataInterfaces(TaskPtr, GroupInfo);
							switch (CompileTask.ScriptCompileType)
							{
								case EScriptCompileType::CompileForVm:
									CompileTask.Translate(TaskPtr, GroupInfo);
									CompileTask.IssueCompileVm(TaskPtr, GroupInfo);
								break;

								case EScriptCompileType::CompileForGpu:
									CompileTask.Translate(TaskPtr, GroupInfo);
									CompileTask.IssueCompileGpu(TaskPtr, GroupInfo);
								break;

								// path for handling the translation of GPU scripts but we're not handling the compilation of
								// the compute shaders ourselves, rather it's using the old path of the FNiagaraShaderScript getting
								// cached by the UNiagaraScript post CPU translation/compilation
								case EScriptCompileType::TranslateForGpu:
									CompileTask.Translate(TaskPtr, GroupInfo);
									CompileTask.IssueTranslateGpu(TaskPtr, GroupInfo);
								break;

								// path for dealing with the CPU scripts (Particle spawn/update) when the emitter is GPU
								// These scripts shouldn't need to be processed, but currently they are required for supplying
								// RI parameters to the GPU execution context
								case EScriptCompileType::DummyCompileForCpuStubs:
									CompileTask.Translate(TaskPtr, GroupInfo);
								break;
							}
						}, InstantiateGraphTask, TaskPriority, TaskExtendedPriority));

						Launch(UE_SOURCE_LOCATION, [Self = AsShared(), &GroupInfo, &CompileTask]
						{
							COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeSyncWork(); Timer.TrackCyclesOnly(););
							UE_LOG_CONTEXT("NiagaraSystem", Self->SystemInfo.GetSystemPath());

							CompileTask.ProcessCompilation(&Self.Get(), GroupInfo);
						}, *CompileTask.CompileResultsReadyEvent, TaskPriority, TaskExtendedPriority);

						CompilationTasks.Add(*CompileTask.CompileResultsProcessedEvent);
					}
				}
			}
		}
	}

	Launch(UE_SOURCE_LOCATION, [Self = AsShared()]
	{
		Self->CompilationState = EState::WaitingForProcessing;
	}, IssuedCompilationTasks, TaskPriority, TaskExtendedPriority);

	Launch(UE_SOURCE_LOCATION, [Self = AsShared()]
	{
		UE_LOG_CONTEXT("NiagaraSystem", Self->SystemInfo.GetSystemPath());
		Self->IssuePostResultsProcessedTasks();
	}, CompilationTasks, TaskPriority, TaskExtendedPriority);
}

bool FNiagaraSystemCompilationTask::HasOutstandingCompileTasks() const
{
	for (const FCompileTaskInfo& CompileTask : CompileTasks)
	{
		if (CompileTask.IsOutstanding(*this))
		{
			return true;
		}
	}

	return false;
}

UE::Tasks::FTask FNiagaraSystemCompilationTask::BeginTasks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraAsyncTask_BeginTasks);

	using namespace UE::Tasks;

	COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeSyncWork(); Timer.TrackCyclesOnly(););

	InitialGetRequestHelper.Launch(this);

	TArray<FTask> SystemTaskPrerequisites;
	SystemTaskPrerequisites.Add(InitialGetRequestHelper.CompletionEvent);

	// if we are baking rapid iteration parameters then we can rely on the results from the DDC to 
	// supply everything that we need.  If however we're not baking the rapid iteration parameters
	// then we need to supplement the results from the DDC with our own collection of the RI so
	// that we can apply them to the script if something has changed.
	if (SystemInfo.bUseRapidIterationParams)
	{
		constexpr bool bSyncPrepareRapidIterationParameter = true;

		if (RIParamMode == ENiagaraCompileRIParamMode::WorkerThread_Synchronous)
		{
			BuildAndApplyRapidIterationParameters();
		}
		else if (RIParamMode == ENiagaraCompileRIParamMode::WorkerThread_Asynchronous)
		{
			SystemTaskPrerequisites.Add(BuildRapidIterationParametersAsync());
		}
	}

	FTask SystemTask = Launch(UE_SOURCE_LOCATION, [Self = AsShared()]
	{
		COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeSyncWork(); Timer.TrackCyclesOnly(););

		if (Self->HasOutstandingCompileTasks())
		{
			TArray<FTask> CompilationTaskPrerequisites;

			if (Self->RIParamMode != ENiagaraCompileRIParamMode::GameThread && !Self->SystemInfo.bUseRapidIterationParams)
			{
				FTask BuildRIParamTask = Self->BuildRapidIterationParametersAsync();
				Self->PostRIParameterGetRequestHelper = MakeUnique<FDispatchAndProcessDataCacheGetRequests>();

				Launch(UE_SOURCE_LOCATION, [Self]
				{
					Self->PostRIParameterGetRequestHelper->Launch(&Self.Get());
				}, BuildRIParamTask);

				CompilationTaskPrerequisites.Add(Self->PostRIParameterGetRequestHelper->CompletionEvent);
			}

			FTask InnerTask = Launch(UE_SOURCE_LOCATION, [Self]
			{
				Self->IssueCompilationTasks();
			}, CompilationTaskPrerequisites);
		}
		else
		{
			Self->IssuePostResultsProcessedTasks();
		}
	}, SystemTaskPrerequisites);

	return SystemTask;
}
