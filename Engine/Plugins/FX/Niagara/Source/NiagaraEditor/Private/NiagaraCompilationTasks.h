// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DerivedDataCache.h"
#include "Tasks/Task.h"
#include "DerivedDataRequestOwner.h"
#include "NiagaraCompiler.h"
#include "NiagaraCompilationPrivate.h"
#include "Templates/SharedPointer.h"

struct FNiagaraSystemCompilationTask : public TSharedFromThis<FNiagaraSystemCompilationTask, ESPMode::ThreadSafe>
{
	struct FDataCacheRequestBase
	{
		FDataCacheRequestBase();

	protected:
		UE::DerivedData::FRequestOwner RequestOwner;
	};

	// helper for script DDC get requests
	struct FDispatchAndProcessDataCacheGetRequests : public FDataCacheRequestBase
	{
		void Launch(FNiagaraSystemCompilationTask* SystemCompileTask);

		UE::Tasks::FTaskEvent CompletionEvent = UE::Tasks::FTaskEvent(UE_SOURCE_LOCATION);
		std::atomic<int32> PendingGetRequestCount = 0;
	};

	// helper for script DDC put requests
	struct FDispatchDataCachePutRequests : public FDataCacheRequestBase
	{
		void Launch(FNiagaraSystemCompilationTask* SystemCompileTask);

		UE::Tasks::FTaskEvent CompletionEvent = UE::Tasks::FTaskEvent(UE_SOURCE_LOCATION);
		std::atomic<int32> PendingPutRequestCount = 0;
	};

	struct FShaderCompileRequest
	{
		FNiagaraShaderMapId ShaderMapId;
		EShaderPlatform ShaderPlatform;
	};

	FNiagaraSystemCompilationTask(FNiagaraCompilationTaskHandle InTaskHandle, UNiagaraSystem* InSystem, ENiagaraCompileRIParamMode InRIParamMode);

	void Abort();

	void DigestSystemInfo();
	void DigestParameterCollections(TConstArrayView<TWeakObjectPtr<UNiagaraParameterCollection>> Collections);
	void DigestShaderInfo(const ITargetPlatform* TargetPlatform, FNiagaraShaderType* ShaderType);
	void AddScript(int32 EmitterIndex, UNiagaraScript* ScriptToCompile, const FNiagaraVMExecutableDataId& CompileId, bool bRequiresCompilation, TConstArrayView<FShaderCompileRequest> ShaderRequests);
	UE::Tasks::FTask BeginTasks();

	UE::Tasks::FTask BuildRapidIterationParametersAsync();
	void BuildAndApplyRapidIterationParameters();
	void IssueCompilationTasks();
	void IssuePostResultsProcessedTasks();
	bool HasOutstandingCompileTasks() const;
	FString GetDescription() const;
	FString GetStatusString() const;

	double PrepareStartTime = 0.0;
	double QueueStartTime = 0.0;
	double LaunchStartTime = 0.0;
	double LastStallWarningTime = 0.0;

	// compilation was forced
	bool bForced = false;

	// task has been deemed to be stalled
	bool bStalled = false;

	enum class EState : uint8
	{
		Invalid = 0,
		WaitingForProcessing,
		ResultsProcessed,
		Completed,
		Aborted,
	};

	std::atomic<EState> CompilationState = EState::Invalid;
	std::atomic<bool> ResultsRetrieved = false;

	struct FEmitterInfo
	{
		FGuid EmitterHandleId;
		FString UniqueEmitterName;
		FString UniqueInstanceName;
		FNiagaraDigestedGraphPtr SourceGraph;
		TArray<FNiagaraVariable> InitialStaticVariables;
		TArray<FNiagaraVariable> StaticVariableResults;
		TArray<FNiagaraSimulationStageInfo> SimStages;
		FNiagaraFixedConstantResolver ConstantResolver;
		TArray<TObjectKey<UNiagaraScript>> OwnedScriptKeys;

		// Index into the EmitterHandles[] of the owning System
		int32 SourceEmitterIndex = INDEX_NONE;

		// Index of the this in FSystemInfo::EmitterInfo
		int32 DigestedEmitterIndex = INDEX_NONE;
		bool Enabled = false;
	};

	struct FSystemInfo
	{
		TArray<FEmitterInfo> EmitterInfo;

		FString SystemName;
		FName SystemPackageName;
		TArray<FNiagaraVariable> InitialStaticVariables;
		TArray<FNiagaraVariable> StaticVariableResults;
		TArray<FNiagaraVariable> OriginalExposedParams;
		FNiagaraDigestedGraphPtr SystemSourceGraph;
		FNiagaraFixedConstantResolver ConstantResolver;
		TArray<TObjectKey<UNiagaraScript>> OwnedScriptKeys;

		bool bUseRapidIterationParams = false;
		bool bDisableDebugSwitches = false;

		FSoftObjectPath GetSystemPath() 
		{ 
			return FSoftObjectPath(FTopLevelAssetPath(FName(SystemPackageName), FName(SystemName)), FString());
		}

		const FEmitterInfo* EmitterInfoBySourceEmitter(int32 InSourceEmitterIndex) const;
		FEmitterInfo* EmitterInfoBySourceEmitter(int32 InSourceEmitterIndex);
	};

	struct FCompileGroupInfo
	{
		FCompileGroupInfo() = delete;
		FCompileGroupInfo(int32 InSourceEmitterIndex);

		bool HasOutstandingCompileTasks(const FNiagaraSystemCompilationTask& ParentTask) const;
		void InstantiateCompileGraph(const FNiagaraSystemCompilationTask& ParentTask);

		const int32 SourceEmitterIndex;
		TArray<int32> CompileTaskIndices;
		TArray<ENiagaraScriptUsage> ValidUsages;
		TSharedPtr<FNiagaraCompilationCopyData> CompilationCopy;
	};

	struct FScriptInfo
	{
		ENiagaraScriptUsage Usage = ENiagaraScriptUsage::Function;
		FGuid UsageId;
		FNiagaraParameterStore RapidIterationParameters;
		TArray<TObjectKey<UNiagaraScript>> DependentScripts;
		int32 SourceEmitterIndex = INDEX_NONE;
	};

	struct FDDCTaskInfo
	{
		bool ResolveGet(bool bSuccess);

		UE::DerivedData::FCacheKey DataCacheGetKey;
		TArray< UE::DerivedData::FCacheKey> DataCachePutKeys;
		FUniqueBuffer PendingDDCData;
		bool bFromDerivedDataCache = false;
	};

	struct FCompileComputeShaderTaskInfo
	{
		int32 ParentCompileTaskIndex = INDEX_NONE;

		FNiagaraShaderMapId ShaderMapId;
		EShaderPlatform ShaderPlatform;
		FNiagaraShaderMapRef ShaderMap;
		TArray<FShaderCompilerError> CompilationErrors;
		FDDCTaskInfo DDCTaskInfo;

		bool IsOutstanding() const;
	};

	enum class EScriptCompileType
	{
		Invalid = 0,
		CompileForVm,
		TranslateForGpu,
		CompileForGpu,
		DummyCompileForCpuStubs,
	};

	struct FCompileTaskInfo
	{
		FNiagaraCompileOptions CompileOptions;
		FNiagaraTranslateResults TranslateResults;
		FNiagaraTranslatorOutput TranslateOutput;
		FString TranslatedHlsl;
		TSharedPtr<FNiagaraVMExecutableData> ExeData;

		// complier for VM script compiles
		TUniquePtr<FHlslNiagaraCompiler> ScriptCompiler;

		// information for GPU script compiles
		TUniquePtr<FNiagaraShaderMapCompiler> ShaderMapCompiler;
		TArray<int32> ComputeShaderTaskIndices;

		TWeakObjectPtr<UNiagaraScript> SourceScript;
		TObjectKey<UNiagaraScript> ScriptKey;
		FString AssetPath;
		FNiagaraVMExecutableDataId CompileId;
		uint32 ScriptCompilationJobId;

		FDDCTaskInfo DDCTaskInfo;

		TUniquePtr<UE::Tasks::FTaskEvent> CompileResultsReadyEvent;
		TUniquePtr<UE::Tasks::FTaskEvent> CompileResultsProcessedEvent;
		TMap<FName, UNiagaraDataInterface*> NamedDataInterfaces;
		TArray<FNiagaraVariable> BakedRapidIterationParameters; // todo - remove and have the data pulled directly from the precompile data
		double TaskStartTime = 0;
		float TranslationTime = 0.0f;
		float CompilerWallTime = 0.0f;
		float CompilerWorkerTime = 0.0f;
		float CompilerPreprocessTime = 0.0f;
		float ByteCodeOptimizeTime = 0.0f;
		EScriptCompileType ScriptCompileType = EScriptCompileType::Invalid;

		bool IsOutstanding(const FNiagaraSystemCompilationTask& ParentTask) const;
		void CollectNamedDataInterfaces(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo);
		void Translate(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo);
		void IssueCompileVm(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo);
		void IssueTranslateGpu(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo);
		void IssueCompileGpu(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo);
		void ProcessCompilation(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo);
		void GenerateOptimizedVMByteCode(FNiagaraSystemCompilationTask* SystemCompileTask, const FCompileGroupInfo& GroupInfo);
		void RetrieveTranslateResult();
		bool RetrieveCompilationResult(bool bWait);
		bool RetrieveShaderMap(bool bWait);

		TOptional<FNiagaraCompileResults> HandleDeprecatedGpuScriptResults() const;
	};

	struct FCollectStaticVariablesTaskBuilder;
	struct FBuildRapidIterationTaskBuilder;

	FCompileGroupInfo* GetCompileGroupInfo(int32 EmitterIndex);
	const FScriptInfo* GetScriptInfo(const TObjectKey<UNiagaraScript>& ScriptKey) const;

	TSharedPtr<FNiagaraPrecompileData> SystemPrecompileData;
	TSet<FNiagaraVariable> SystemExposedVariables;

	TArray<FCompileGroupInfo> CompileGroups;
	TArray<FCompileTaskInfo> CompileTasks;
	TArray<FCompileComputeShaderTaskInfo> CompileComputeShaderTasks;

	const FNiagaraCompilationTaskHandle TaskHandle;

	const ITargetPlatform* TargetPlatform = nullptr;
	FNiagaraShaderType* NiagaraShaderType = nullptr;

	int32 TasksAwaitingDDCGetResults = 0;
	bool bCompileForEdit = false;

	const ENiagaraCompileRIParamMode RIParamMode;

	void Tick();
	bool Poll(FNiagaraSystemAsyncCompileResults& Results) const;
	bool CanRemove() const;
	bool AreResultsPending() const;

	void WaitTillCompileCompletion();
	void WaitTillCachePutCompletion();
	void Precompile();

	void GetAvailableCollections(TArray<FNiagaraCompilationNPCHandle>& OutCollections) const;

	bool GenerateCompileMetaData() const;
	const FGuid& GetCompilationTaskId() const;

private:
	FNiagaraSystemCompilationTask() = delete;

	bool GetStageName(int32 EmitterIndex, const FNiagaraCompilationNodeOutput* OutputNode, FName& OutStageName) const;

	TWeakObjectPtr<UNiagaraSystem> System_GT;

	FSystemInfo SystemInfo;

	FDispatchAndProcessDataCacheGetRequests InitialGetRequestHelper;
	TUniquePtr<FDispatchAndProcessDataCacheGetRequests> PostRIParameterGetRequestHelper;
	TUniquePtr<FDispatchDataCachePutRequests> PutRequestHelper;

	TMap<TObjectKey<UNiagaraScript>, FScriptInfo> DigestedScriptInfo;

	TMap<TObjectKey<UNiagaraParameterCollection>, FNiagaraCompilationNPCHandle> DigestedParameterCollections;

	UE::Tasks::FTaskEvent CompileCompletionEvent = UE::Tasks::FTaskEvent(UE_SOURCE_LOCATION);

	bool bAborting = false;

	bool bGenerateCompileMetaData = false;

	// Guid just to track this specific compilation task.  Used in the DDC metadata to help see if in the case of a problem whether
	// the data pulled from DDC all came from the same job.
	FGuid CompilationTaskId;
};
