// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCompiler.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "EdGraphSchema_Niagara.h"
#include "EdGraphUtilities.h"
#include "Interfaces/IShaderFormat.h"
#include "INiagaraEditorTypeUtilities.h"
#include "Logging/StructuredLog.h"
#include "Misc/FileHelper.h"
#include "Misc/PathViews.h"
#include "Modules/ModuleManager.h"
#include "NiagaraCompilationPrivate.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterface.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraGraph.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraPrecompileContainer.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraShader.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"
#include "NiagaraTrace.h"
#include "Serialization/MemoryReader.h"
#include "ShaderCompiler.h"
#include "ShaderCore.h"
#include "ShaderDiagnostics.h"
#include "VectorVMTestCompile.h"

#define LOCTEXT_NAMESPACE "NiagaraCompiler"

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraCompiler, All, All);

DECLARE_CYCLE_STAT(TEXT("Niagara - Module - CompileScript"), STAT_NiagaraEditor_Module_CompileScript, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslCompiler - CompileScript"), STAT_NiagaraEditor_HlslCompiler_CompileScript, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslCompiler - CompileShader_VectorVM"), STAT_NiagaraEditor_HlslCompiler_CompileShader_VectorVM, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - Module - CompileShader_VectorVMSucceeded"), STAT_NiagaraEditor_HlslCompiler_CompileShader_VectorVMSucceeded, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - ScriptSource - PreCompile"), STAT_NiagaraEditor_ScriptSource_PreCompile, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - ScriptSource - PreCompileDuplicate"), STAT_NiagaraEditor_ScriptSource_PreCompileDuplicate, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslCompiler - TestCompileShader_VectorVM"), STAT_NiagaraEditor_HlslCompiler_TestCompileShader_VectorVM, STATGROUP_NiagaraEditor);

static int32 GbForceNiagaraTranslatorSingleThreaded = 1;
static FAutoConsoleVariableRef CVarForceNiagaraTranslatorSingleThreaded(
	TEXT("fx.ForceNiagaraTranslatorSingleThreaded"),
	GbForceNiagaraTranslatorSingleThreaded,
	TEXT("If > 0 all translation will occur one at a time, useful for debugging. \n"),
	ECVF_Default
);

// Enable this to log out generated HLSL for debugging purposes.
static int32 GbForceNiagaraTranslatorDump = 0;
static FAutoConsoleVariableRef CVarForceNiagaraTranslatorDump(
	TEXT("fx.ForceNiagaraTranslatorDump"),
	GbForceNiagaraTranslatorDump,
	TEXT("If > 0 all translation generated HLSL will be dumped \n"),
	ECVF_Default
);

static int32 GbForceNiagaraVMBinaryDump = 0;
static FAutoConsoleVariableRef CVarForceNiagaraVMBinaryDump(
	TEXT("fx.ForceNiagaraVMBinaryDump"),
	GbForceNiagaraVMBinaryDump,
	TEXT("If > 0 all translation generated binary text will be dumped \n"),
	ECVF_Default
);

static int32 GbForceNiagaraCacheDump = 0;
static FAutoConsoleVariableRef CVarForceNiagaraCacheDump(
	TEXT("fx.ForceNiagaraCacheDump"),
	GbForceNiagaraCacheDump,
	TEXT("If > 0 all cached graph traversal data will be dumped \n"),
	ECVF_Default
);

static int32 GNiagaraEnablePrecompilerNamespaceFixup = 0;
static FAutoConsoleVariableRef CVarNiagaraEnablePrecompilerNamespaceFixup(
	TEXT("fx.NiagaraEnablePrecompilerNamespaceFixup"),
	GNiagaraEnablePrecompilerNamespaceFixup,
	TEXT("Enable a precompiler stage to discover parameter name matches and convert matched parameter hlsl name tokens to appropriate namespaces. \n"),
	ECVF_Default
);

namespace NiagaraCompileRequestHelper
{
	using FDebugGroupNameBuilder = TStringBuilder<512>;
	static void BuildScriptDebugGroupName(const FNiagaraCompileRequestData* InCompileRequest, const FNiagaraCompileOptions& InCompileOptions, FDebugGroupNameBuilder& NameBuilder)
	{
		FPathViews::Append(NameBuilder, InCompileRequest->SourceName);
		FPathViews::Append(NameBuilder, InCompileRequest->EmitterUniqueName);
		FPathViews::Append(NameBuilder, InCompileRequest->ENiagaraScriptUsageEnum->GetNameStringByValue((int64)InCompileOptions.TargetUsage));

		if (InCompileOptions.TargetUsageId.IsValid())
		{
			NameBuilder << TEXT("_");
			InCompileOptions.TargetUsageId.AppendString(NameBuilder, EGuidFormats::Digits);
		}
	}

};

static FCriticalSection TranslationCritSec;

void DumpHLSLText(const FString& SourceCode, const FString& DebugName)
{
	FScopeLock Lock(&TranslationCritSec);
	FNiagaraUtilities::DumpHLSLText(SourceCode, DebugName);
}

template< class T >
T* PrecompileDuplicateObject(T const* SourceObject, UObject* Outer, const FName Name = NAME_None)
{
	//double StartTime = FPlatformTime::Seconds();
	T* DupeObj = DuplicateObject<T>(SourceObject, Outer, Name);
	//float DeltaTime = (float)(FPlatformTime::Seconds() - StartTime);
	//if (DeltaTime > 0.01f)
	//{
	//	UE_LOGF(LogNiagaraEditor, Log, "\tPrecompile Duplicate %ls took %f sec", *SourceObject->GetPathName(), DeltaTime);
	//}
	return DupeObj;

}

void FNiagaraCompileRequestDuplicateData::DuplicateReferencedGraphs(UNiagaraGraph* InSrcGraph, UNiagaraGraph* InDupeGraph, ENiagaraScriptUsage InUsage, FCompileConstantResolver ConstantResolver, TMap<UNiagaraNodeFunctionCall*, ENiagaraScriptUsage> FunctionsWithUsage)
{
	if (!InDupeGraph || !InSrcGraph)
	{
		return;
	}

	TArray<FDuplicatedGraphData>& DuplicatedGraphDataArray = SharedSourceGraphToDuplicatedGraphsMap->FindOrAdd(InSrcGraph);
	FDuplicatedGraphData& DuplicatedGraphData = DuplicatedGraphDataArray.AddDefaulted_GetRef();
	DuplicatedGraphData.ClonedScript = nullptr;
	DuplicatedGraphData.ClonedGraph = InDupeGraph;
	DuplicatedGraphData.Usage = InUsage;
	DuplicatedGraphData.bHasNumericParameters = false;

	bool bStandaloneScript = false;
	TArray<UNiagaraNodeOutput*> OutputNodes;
	InDupeGraph->FindOutputNodes(OutputNodes);
	if (OutputNodes.Num() == 1 && UNiagaraScript::IsStandaloneScript(OutputNodes[0]->GetUsage()))
	{
		bStandaloneScript = true;
	}

	FNiagaraEditorUtilities::ResolveNumerics(InDupeGraph, bStandaloneScript, ChangedFromNumericVars);
	DuplicateReferencedGraphsRecursive(InDupeGraph, ConstantResolver, FunctionsWithUsage);
}

void FNiagaraCompileRequestDuplicateData::DuplicateReferencedGraphsRecursive(UNiagaraGraph* InGraph, const FCompileConstantResolver& ConstantResolver, TMap<UNiagaraNodeFunctionCall*, ENiagaraScriptUsage> FunctionsWithUsage)
{
	if (!InGraph)
	{
		return;
	}

	TArray<UNiagaraNode*> Nodes;
	InGraph->GetNodesOfClass(Nodes);
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	for (UEdGraphNode* Node : Nodes)
	{
		if (UNiagaraNode* InNode = Cast<UNiagaraNode>(Node))
		{
			UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(InNode);
			if (InputNode)
			{
				if (InputNode->Input.IsDataInterface())
				{
					UNiagaraDataInterface* DataInterface = InputNode->GetDataInterface();
					bool bIsParameterMapDataInterface = false;
					FName DIName = FNiagaraHlslTranslator::GetDataInterfaceName(InputNode->Input.GetName(), EmitterUniqueName, bIsParameterMapDataInterface);
					UNiagaraDataInterface* Dupe = PrecompileDuplicateObject<UNiagaraDataInterface>(DataInterface, GetTransientPackage());
					SharedNameToDuplicatedDataInterfaceMap->Add(DIName, Dupe);
				}
				continue;
			}

			UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(InNode);
			if (FunctionCallNode)
			{
				UNiagaraScript* FunctionScript = FunctionCallNode->FunctionScript;
				bool bFunctionCallOwnsScript = FunctionScript != nullptr && FunctionScript->GetOuter() == FunctionCallNode;
				if(FunctionCallNode->HasValidScriptAndGraph() && bFunctionCallOwnsScript == false)
				{
					// If the function call doesn't already own the script it's pointing at then script needs to be duplicated since it's a referenced
					// script and will need to be preprocessed.
					ENiagaraScriptUsage ScriptUsage = FunctionCallNode->GetCalledUsage();

					FCompileConstantResolver FunctionConstantResolver = ConstantResolver;
					if (FunctionsWithUsage.Contains(FunctionCallNode))
					{
						FunctionConstantResolver = ConstantResolver.WithUsage(FunctionsWithUsage[FunctionCallNode]);
					}

					UNiagaraGraph* FunctionGraph = FunctionCallNode->GetCalledGraph();
					bool bHasNumericParams = FunctionGraph->HasNumericParameters();
					if (bHasNumericParams || SharedSourceGraphToDuplicatedGraphsMap->Contains(FunctionGraph) == false)
					{
						// Duplicate the script, the source, and graph
						UNiagaraScript* DupeScript = FunctionScript->CreateCompilationCopy();
						TArray<ENiagaraScriptUsage> CompileUsages = { DupeScript->GetUsage() };
						UNiagaraScriptSource* DupeScriptSource = CastChecked<UNiagaraScriptSource>(DupeScript->GetSource(FunctionCallNode->SelectedScriptVersion))->CreateCompilationCopy(CompileUsages);
						TrackedScriptSourceCopies.Add(DupeScriptSource);
						UNiagaraGraph* DupeGraph = DupeScriptSource->NodeGraph;
						DupeScript->SetSource(DupeScriptSource, FunctionCallNode->SelectedScriptVersion);
						
						// Do any preprocessing necessary
						FEdGraphUtilities::MergeChildrenGraphsIn(DupeGraph, DupeGraph, /*bRequireSchemaMatch=*/ true);	
						FPinCollectorArray CallOutputs;
						FPinCollectorArray CallInputs;
						InNode->GetOutputPins(CallOutputs);
						InNode->GetInputPins(CallInputs);
						FNiagaraEditorUtilities::PreprocessFunctionGraph(Schema, DupeGraph, CallInputs, CallOutputs, ScriptUsage, FunctionConstantResolver);
							
						// Record the data for this duplicate.
						TArray<FDuplicatedGraphData>& DuplicatedGraphDataArray = SharedSourceGraphToDuplicatedGraphsMap->FindOrAdd(FunctionGraph);
						FDuplicatedGraphData& DuplicatedGraphData = DuplicatedGraphDataArray.AddDefaulted_GetRef();
						DuplicatedGraphData.ClonedScript = DupeScript;
						DuplicatedGraphData.ClonedGraph = DupeGraph;
						DuplicatedGraphData.CallInputs = CallInputs;
						DuplicatedGraphData.CallOutputs = CallOutputs;
						DuplicatedGraphData.Usage = ScriptUsage;
						DuplicatedGraphData.bHasNumericParameters = bHasNumericParams;

						// Assign the copied script and process any child scripts.
						FunctionCallNode->FunctionScript = DupeScript;
						DuplicateReferencedGraphsRecursive(DupeGraph, FunctionConstantResolver, FunctionsWithUsage);
					}
					else
					{
						// This graph was already processed and doesn't need per-call duplication so use the previous copy.
						TArray<FDuplicatedGraphData>* DuplicatedGraphDataArray = SharedSourceGraphToDuplicatedGraphsMap->Find(FunctionGraph);
						check(DuplicatedGraphDataArray != nullptr && DuplicatedGraphDataArray->Num() != 0);
						FunctionCallNode->FunctionScript = (*DuplicatedGraphDataArray)[0].ClonedScript;
					}
				}
			}

			UNiagaraNodeEmitter* EmitterNode = Cast<UNiagaraNodeEmitter>(InNode);
			if (EmitterNode)
			{
				for (TSharedPtr<FNiagaraCompileRequestDuplicateData, ESPMode::ThreadSafe>& Ptr : EmitterData)
				{
					if (Ptr->EmitterUniqueName == EmitterNode->GetEmitterUniqueName())
					{
						EmitterNode->SyncEnabledState(); // Just to be safe, sync here while we likely still have the handle source.
						EmitterNode->SetOwnerSystem(nullptr);
						EmitterNode->SetCachedVariablesForCompilation(*Ptr->EmitterUniqueName, Ptr->EmitterID, Ptr->NodeGraphDeepCopy.Get(), Ptr->SourceDeepCopy.Get());
					}
				}
			}
		}
	}
}

const TMap<FName, UNiagaraDataInterface*>& FNiagaraCompileRequestDuplicateData::GetObjectNameMap()
{
	return *SharedNameToDuplicatedDataInterfaceMap.Get();
}

const UNiagaraScriptSourceBase* FNiagaraCompileRequestDuplicateData::GetScriptSource() const
{
	return SourceDeepCopy.Get();
}

UNiagaraDataInterface* FNiagaraCompileRequestDuplicateData::GetDuplicatedDataInterfaceCDOForClass(UClass* Class) const
{
	if (SharedDataInterfaceClassToDuplicatedCDOMap.IsValid())
	{
		UNiagaraDataInterface*const* DuplicatedCDOPtr = SharedDataInterfaceClassToDuplicatedCDOMap->Find(Class);
		if (DuplicatedCDOPtr != nullptr)
		{
			return *DuplicatedCDOPtr;
		}
	}
	return nullptr;
}

void FNiagaraCompileRequestData::SortOutputNodesByDependencies(TArray<class UNiagaraNodeOutput*>& NodesToSort, const TArray<class UNiagaraSimulationStageBase*>* SimStages)
{
	if (!SimStages)
		return;

	TArray<class UNiagaraNodeOutput*> NewArray;
	NewArray.Reserve(NodesToSort.Num());

	// First gather up the non-simstage items
	bool bFoundAnySimStages = false;
	for (class UNiagaraNodeOutput* OutputNode : NodesToSort)
	{
		// Add any non sim stage entries back to the array in the order of encounter
		if (OutputNode->GetUsage() != ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			NewArray.Emplace(OutputNode);
		}
		else
		{
			bFoundAnySimStages = true;
		}
	}

	// No Sim stages, no problem! Just return
	if (!bFoundAnySimStages)
	{
		return;
	}

	ensure(SimStages->Num() == (NodesToSort.Num() - NewArray.Num()));

	// Add any sim stage entries back to the array in the order of encounter in the SimStage entry list from the Emitter (Handles reordering)
	for (const UNiagaraSimulationStageBase* Stage : *SimStages)
	{
		if (Stage && Stage->Script)
		{
			const FGuid & StageId = Stage->Script->GetUsageId();

			for (class UNiagaraNodeOutput* OutputNode : NodesToSort)
			{
				if (OutputNode->GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript && OutputNode->GetUsageId() == StageId)
				{
					NewArray.Emplace(OutputNode);
					break;
				}
			}
		}
	}

	ensure(NodesToSort.Num() == NewArray.Num());

	// Copy out final results
	NodesToSort = NewArray;
}


FName FNiagaraCompileRequestData::ResolveEmitterAlias(FName VariableName) const
{
	return FNiagaraParameterUtilities::ResolveEmitterAlias(VariableName, EmitterUniqueName);
}

bool FNiagaraCompileRequestDuplicateData::IsDuplicateDataFor(UNiagaraSystem* InSystem, UNiagaraEmitter* InEmitter, UNiagaraScript* InScript) const
{
	return OwningSystem.Get() == InSystem && OwningEmitter.Get() == InEmitter && ValidUsages.Contains(InScript->GetUsage());
}

void FNiagaraCompileRequestDuplicateData::GetDuplicatedObjects(TArray<UObject*>& Objects)
{
	Objects.Add(SourceDeepCopy.Get());
	Objects.Add(NodeGraphDeepCopy.Get());

	if (SharedNameToDuplicatedDataInterfaceMap.IsValid())
	{
		TArray<UNiagaraDataInterface*> DIs;
		SharedNameToDuplicatedDataInterfaceMap->GenerateValueArray(DIs);
		for (UNiagaraDataInterface* DI : DIs)
		{
			Objects.Add(DI);
		}
	}

	if (SharedDataInterfaceClassToDuplicatedCDOMap.IsValid())
	{
		auto Iter = SharedDataInterfaceClassToDuplicatedCDOMap->CreateIterator();
		while (Iter)
		{
			Objects.Add(Iter.Value());
			++Iter;
		}
	}

	if (SharedSourceGraphToDuplicatedGraphsMap.IsValid())
	{
		auto Iter = SharedSourceGraphToDuplicatedGraphsMap->CreateIterator();
		while (Iter)
		{
			for (int32 i = 0; i < Iter.Value().Num(); i++)
			{
				Objects.Add(Iter.Value()[i].ClonedScript);
				Objects.Add(Iter.Value()[i].ClonedGraph);
			}
			++Iter;
		}
	}
}

void FNiagaraCompileRequestData::GatherPreCompiledVariables(const FString& InNamespaceFilter, TArray<FNiagaraVariable>& OutVars) const
{
	if (InNamespaceFilter.Len() == 0)
	{
		OutVars.Append(EncounteredVariables);
	}
	else
	{
		for (const FNiagaraVariable& EncounteredVariable : EncounteredVariables)
		{
			if (FNiagaraParameterUtilities::IsInNamespace(EncounteredVariable, InNamespaceFilter))
			{
				FNiagaraVariable NewVar = EncounteredVariable;
				if (NewVar.IsDataAllocated() == false && !NewVar.IsDataInterface() && !NewVar.IsUObject())
				{
					FNiagaraEditorUtilities::ResetVariableToDefaultValue(NewVar);
				}
				OutVars.AddUnique(NewVar);
			}
		}
	}
}

void FNiagaraCompileRequestDuplicateData::DeepCopyGraphs(UNiagaraScriptSource* ScriptSource, ENiagaraScriptUsage InUsage, FCompileConstantResolver ConstantResolver)
{
	// Clone the source graph so we can modify it as needed; merging in the child graphs
	SourceDeepCopy = ScriptSource->CreateCompilationCopy(ValidUsages);
	NodeGraphDeepCopy = SourceDeepCopy->NodeGraph;
	FEdGraphUtilities::MergeChildrenGraphsIn(NodeGraphDeepCopy.Get(), NodeGraphDeepCopy.Get(), /*bRequireSchemaMatch=*/ true);
	DuplicateReferencedGraphs(ScriptSource->NodeGraph, NodeGraphDeepCopy.Get(), InUsage, ConstantResolver);
}

void FNiagaraCompileRequestDuplicateData::DeepCopyGraphs(const FVersionedNiagaraEmitter& Emitter)
{
	UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(Emitter.GetEmitterData()->GraphSource);

	SourceDeepCopy = ScriptSource->CreateCompilationCopy(ValidUsages);
	NodeGraphDeepCopy = SourceDeepCopy->NodeGraph;
	FEdGraphUtilities::MergeChildrenGraphsIn(NodeGraphDeepCopy.Get(), NodeGraphDeepCopy.Get(), /*bRequireSchemaMatch=*/ true);

	TMap<UNiagaraNodeFunctionCall*, ENiagaraScriptUsage> FunctionsWithUsage;
	TArray<UNiagaraNodeOutput*> OutputNodes;
	NodeGraphDeepCopy->GetNodesOfClass(OutputNodes);
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		TArray<UNiagaraNode*> TraversedNodes;
		NodeGraphDeepCopy->BuildTraversal(TraversedNodes, OutputNode);
		for (UNiagaraNode* TraversedNode : TraversedNodes)
		{
			UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(TraversedNode);
			if (FunctionCallNode != nullptr)
			{
				FunctionsWithUsage.Add(FunctionCallNode, OutputNode->GetUsage());
			}
		}
	}

	FCompileConstantResolver ConstantResolver(Emitter, ENiagaraScriptUsage::EmitterSpawnScript);
	DuplicateReferencedGraphs(ScriptSource->NodeGraph, NodeGraphDeepCopy.Get(), ENiagaraScriptUsage::EmitterSpawnScript, ConstantResolver, FunctionsWithUsage);
}


void FNiagaraCompileRequestData::AddRapidIterationParameters(const FNiagaraParameterStore& InParamStore, FCompileConstantResolver InResolver)
{
	TArray<FNiagaraVariable> StoreParams;
	InParamStore.GetParameters(StoreParams);

	for (int32 i = 0; i < StoreParams.Num(); i++)
	{
		// Only support POD data...
		if (StoreParams[i].IsDataInterface() || StoreParams[i].IsUObject())
		{
			continue;
		}

		if (InResolver.ResolveConstant(StoreParams[i]))
		{
			continue;
		}

		// Check to see if we already have this RI var...
		int32 OurFoundIdx = INDEX_NONE;
		for (int32 OurIdx = 0; OurIdx < RapidIterationParams.Num(); OurIdx++)
		{
			if (RapidIterationParams[OurIdx].GetType() == StoreParams[i].GetType() && RapidIterationParams[OurIdx].GetName() == StoreParams[i].GetName())
			{
				OurFoundIdx = OurIdx;
				break;
			}
		}

		// If we don't already have it, add it with the up-to-date value.
		if (OurFoundIdx == INDEX_NONE)
		{
			// In parameter stores, the data isn't always up-to-date in the variable, so make sure to get the most up-to-date data before passing in.
			const int32* Index = InParamStore.FindParameterOffset(StoreParams[i]);
			if (Index != nullptr)
			{
				StoreParams[i].SetData(InParamStore.GetParameterData(*Index, StoreParams[i].GetType())); // This will memcopy the data in.
				RapidIterationParams.Add(StoreParams[i]);
			}
		}
		else
		{
			FNiagaraVariable ExistingVar = RapidIterationParams[OurFoundIdx];

			const int32* Index = InParamStore.FindParameterOffset(StoreParams[i]);
			if (Index != nullptr)
			{
				StoreParams[i].SetData(InParamStore.GetParameterData(*Index, StoreParams[i].GetType())); // This will memcopy the data in.

				if (StoreParams[i] != ExistingVar)
				{
					UE_LOGF(LogNiagaraEditor, Display, "Mismatch in values for Rapid iteration param: %ls vs %ls", *StoreParams[i].ToString(), *ExistingVar.ToString());
				}
			}
		}
	}
}

void FNiagaraCompileRequestDuplicateData::ReleaseCompilationCopies()
{
	// clean up graph copies
	if (SharedSourceGraphToDuplicatedGraphsMap.IsValid())
	{
		for (const TPair<const UNiagaraGraph*, TArray<FDuplicatedGraphData>>& SourceGraphToDuplicatedGraphs : *(SharedSourceGraphToDuplicatedGraphsMap.Get()))
		{
			for (const FDuplicatedGraphData& DuplicatedGraphData : SourceGraphToDuplicatedGraphs.Value)
			{
				DuplicatedGraphData.ClonedGraph->ReleaseCompilationCopy();
			}
		}
		SharedSourceGraphToDuplicatedGraphsMap->Empty();
	}

	// clean up script sources
	for (TWeakObjectPtr<UNiagaraScriptSource> Source : TrackedScriptSourceCopies)
	{
		if (Source.IsValid())
		{
			Source->ReleaseCompilationCopy();
		}
	}
	TrackedScriptSourceCopies.Empty();
	if (SourceDeepCopy.IsValid())
	{
		SourceDeepCopy->ReleaseCompilationCopy();
	}
	SourceDeepCopy = nullptr;

	// clean up emitter data
	for (TSharedPtr<FNiagaraCompileRequestDuplicateData, ESPMode::ThreadSafe> EmitterRequest : EmitterData)
	{
		EmitterRequest->ReleaseCompilationCopies();
	}
}

void FNiagaraCompileRequestData::CompareAgainst(FNiagaraGraphCachedBuiltHistory* InCachedDataBase)
{
	if (InCachedDataBase)
	{
		bool bDumpVars = false;
		if (StaticVariables.Num() != InCachedDataBase->StaticVariables.Num())
		{
			UE_LOGF(LogNiagaraEditor, Warning, "FNiagaraCompileRequestData::CompareAgainst> StaticVariables.Num() != InCachedDataBase->StaticVariables.Num()");
			bDumpVars = true;
		}
		
		for (const FNiagaraVariable& Var : StaticVariables)
		{
			bool bFound = false;
			for (const FNiagaraVariable& OtherVar : InCachedDataBase->StaticVariables)
			{
				if (OtherVar == Var)
				{
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				UE_LOGF(LogNiagaraEditor, Warning, "FNiagaraCompileRequestData::CompareAgainst> Could not find %ls", *Var.ToString());
				bDumpVars = true;
			}
		}

		if (bDumpVars)
		{
			for (const FNiagaraVariable& Var : InCachedDataBase->StaticVariables)
			{
				UE_LOGF(LogNiagaraEditor, Warning, "%ls", *Var.ToString());
			}
		}
	}
}


void FNiagaraCompileRequestData::FinishPrecompile(const TArray<FNiagaraVariable>& EncounterableVariables, const TArray<FNiagaraVariable>& InStaticVariables, FCompileConstantResolver ConstantResolver, const TArray<ENiagaraScriptUsage>& UsagesToProcess, const TArray<class UNiagaraSimulationStageBase*>* SimStages, const TArray<FString> EmitterNames)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	{
		ENiagaraScriptCompileStatusEnum = StaticEnum<ENiagaraScriptCompileStatus>();
		ENiagaraScriptUsageEnum = StaticEnum<ENiagaraScriptUsage>();

		TArray<UNiagaraNodeOutput*> OutputNodes;
		if (Source.IsValid() && Source->NodeGraph != nullptr)
		{
			Source->NodeGraph->FindOutputNodes(OutputNodes);
		}

		SortOutputNodesByDependencies(OutputNodes, SimStages);

		bool bFilterByEmitterAlias = true;
		for (UNiagaraNodeOutput* FoundOutputNode : OutputNodes)
		{
			if (UNiagaraScript::IsSystemScript(FoundOutputNode->GetUsage()))
			{
				bFilterByEmitterAlias = false;
			}
		}
		
		// Only use the static variables that match up with our expectations for this script. IE for emitters, filter things out for resolution.
		FNiagaraParameterUtilities::FilterToRelevantStaticVariables(InStaticVariables, StaticVariables, *GetUniqueEmitterName(), TEXT("Emitter"), bFilterByEmitterAlias);

		int32 NumSimStageNodes = 0;
		for (UNiagaraNodeOutput* FoundOutputNode : OutputNodes)
		{
			if (UsagesToProcess.Contains(FoundOutputNode->GetUsage()) == false)
			{
				continue;
			}

			FName SimStageName;
			bool bStageEnabled = true;
			if (FoundOutputNode->GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript && SimStages)
			{
				// Find the simulation stage for this output node.
				const FGuid& UsageId = FoundOutputNode->GetUsageId();
				UNiagaraSimulationStageBase*const* MatchingStagePtr = SimStages->FindByPredicate([UsageId](UNiagaraSimulationStageBase* SimStage)
				{ 
					return SimStage != nullptr && SimStage->Script != nullptr && SimStage->Script->GetUsageId() == UsageId;
				});

				// Set whether or not the stage is enabled, and get the iteration source name if available.
				bStageEnabled = MatchingStagePtr != nullptr && (*MatchingStagePtr)->bEnabled;
				if(bStageEnabled && (*MatchingStagePtr)->IsA<UNiagaraSimulationStageGeneric>())
				{
					UNiagaraSimulationStageGeneric* GenericStage = CastChecked<UNiagaraSimulationStageGeneric>(*MatchingStagePtr);
					SimStageName = GenericStage->IterationSource == ENiagaraIterationSource::DataInterface ? GenericStage->DataInterface.BoundVariable.GetName() : FName();
				}
			}

			if (bStageEnabled)
			{
				// Map all for this output node
				TNiagaraParameterMapHistoryWithMetaDataBuilder<FNiagaraCompilationGraphBridge> Builder;
				*Builder.ConstantResolver = ConstantResolver;
				Builder.AddGraphToCallingGraphContextStack(Source->NodeGraph);
				Builder.RegisterEncounterableVariables(EncounterableVariables);
				Builder.RegisterExternalStaticVariables(StaticVariables);

				FString TranslationName = TEXT("Emitter");// Note that this cannot be GetUniqueEmitterName() as it would break downstream logic for some reason for data interfaces.
				Builder.BeginTranslation(TranslationName);
				Builder.BeginUsage(FoundOutputNode->GetUsage(), SimStageName);
				Builder.EnableScriptAllowList(true, FoundOutputNode->GetUsage());
				Builder.BuildParameterMaps(FoundOutputNode, true);
				Builder.EndUsage();

				int HistoryIdx = 0;
				for (FNiagaraParameterMapHistory& History : Builder.Histories)
				{
					History.OriginatingScriptUsage = FoundOutputNode->GetUsage();
					History.UsageGuid = FoundOutputNode->ScriptTypeId;
					History.UsageName = SimStageName;
					for (int32 i = 0; i < History.Variables.Num(); i++)
					{
						FNiagaraVariable& Var = History.Variables[i];
						EncounteredVariables.AddUnique(Var);
						if (Var.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
						{
							UE_LOGF(LogNiagaraEditor, Log, "Invalid numeric parameter found! %ls", *Var.GetName().ToString())
						}

						if (Var.GetType().IsStatic())
						{
							int32 NumValues = 0;
							int32 LastIndex = INDEX_NONE;

							// The logic for the static variables array adds the full payload static variable to the builder list. This will result
							// in duplicates with the same name and type, but *different* value payloads. We detect this and error out.
							for (int32 StaticIdx = 0; StaticIdx < Builder.StaticVariables.Num(); StaticIdx++)
							{
								const FNiagaraVariable& BuilderVar = Builder.StaticVariables[StaticIdx];
								if (Var == BuilderVar) // operator == ignores the value field, which is what we want here 
								{
									if (NumValues == 0)
									{
										LastIndex = StaticIdx;
										NumValues++;
									}
									else if (LastIndex != INDEX_NONE && !BuilderVar.HoldsSameData(Builder.StaticVariables[LastIndex]))
									{
										if (UNiagaraScript::LogCompileStaticVars > 0)
										{
											UE_LOGF(LogNiagaraEditor, Log, "Mismatch in static vars %ls: \"%ls\" vs \"%ls\"", *BuilderVar.GetName().ToString(),
												*BuilderVar.ToString(), *Builder.StaticVariables[LastIndex].ToString());
										}

										StaticVariablesWithMultipleWrites.AddUnique(Var);
										NumValues++;
										break;
									}
								} 
							}

							if (History.PerVariableConstantValue[i].Num() > 1)
							{
								StaticVariablesWithMultipleWrites.AddUnique(Var);
							}
						}
					}

					if (UNiagaraScript::LogCompileStaticVars > 0)
					{
						for (auto Iter : History.PinToConstantValues)
						{
							UE_LOGF(LogNiagaraEditor, Log, "History [%d] Pin: %ls Value: %ls", HistoryIdx , *Iter.Key.ToString(), *Iter.Value);
						}
					}
					PinToConstantValues.Append(History.PinToConstantValues);
					++HistoryIdx;
				}

				if (FoundOutputNode->GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript)
				{
					NumSimStageNodes++;
				}

				if (UNiagaraScript::LogCompileStaticVars > 0)
				{
					UE_LOGF(LogNiagaraEditor, Log, "Builder.StaticVariables After Param Map Traversal............................");
				}

				for (int32 StaticIdx = 0; StaticIdx < Builder.StaticVariables.Num(); StaticIdx++)
				{
					const FNiagaraVariable& Var = Builder.StaticVariables[StaticIdx];
					bool bProcess = Builder.StaticVariableExportable[StaticIdx];			

					if (UNiagaraScript::LogCompileStaticVars > 0)
					{
						UE_LOGF(LogNiagaraEditor, Log, "%ls > %ls", *Var.ToString(), bProcess ? TEXT("EXPORT") : TEXT("SkipExport"));
					}

					if (!bProcess)
					{
						continue;
					}

					StaticVariables.AddUnique(Var);

				}

				if (UNiagaraScript::LogCompileStaticVars > 0)
				{
					for (auto Iter : PinToConstantValues)
					{
						UE_LOGF(LogNiagaraEditor, Log, "Pin: %ls Value: %ls", *Iter.Key.ToString(), *Iter.Value);
					}
				}

				Builder.EndTranslation(TranslationName);

				// Collect data interface information.
				TMap<UNiagaraDataInterface*, FNiagaraVariable> DataInterfaceToTopLevelNiagaraVariable;
				TMap<FNiagaraVariable, TArray<FNiagaraVariable>> DataInterfaceParameterMapReferences;
				for (const FNiagaraParameterMapHistory& History : Builder.Histories)
				{
					// Find the variable indices for data interfaces.
					TArray<int32> DataInterfaceVariableIndices;
					for (int32 i = 0; i < History.Variables.Num(); i++)
					{
						const FNiagaraVariable& Variable = History.Variables[i];
						if (Variable.IsDataInterface())
						{
							DataInterfaceVariableIndices.Add(i);
						}
					}

					// Find the data interface input nodes and collect data from the data interfaces.
					for (int32 i = 0; i < DataInterfaceVariableIndices.Num(); i++)
					{
						int32 VariableIndex = DataInterfaceVariableIndices[i];
						const FNiagaraVariable& Variable = History.Variables[VariableIndex];
						for (const FNiagaraParameterMapHistory::FModuleScopedPin& WritePin : History.PerVariableWriteHistory[VariableIndex])
						{
							if (WritePin.Pin != nullptr && WritePin.Pin->LinkedTo.Num() == 1 && WritePin.Pin->LinkedTo[0] != nullptr)
							{
								UNiagaraNode* LinkedNode = Cast<UNiagaraNode>(WritePin.Pin->LinkedTo[0]->GetOwningNode());
								if (LinkedNode != nullptr && LinkedNode->IsA<UNiagaraNodeInput>())
								{
									UNiagaraDataInterface* DataInterface = CastChecked<UNiagaraNodeInput>(LinkedNode)->GetDataInterface();
									FCompileDataInterfaceData* DataInterfaceData = nullptr;
									if (DataInterface != nullptr)
									{
										TArray<FString> EmitterReferences;
										DataInterface->GetEmitterReferencesByName(EmitterReferences);

										for (const FString& EmitterName : EmitterNames)
										{
											if (EmitterReferences.Contains(EmitterName))
											{
												if (DataInterfaceData == nullptr)
												{
													DataInterfaceData = &SharedCompileDataInterfaceData->AddDefaulted_GetRef();
													DataInterfaceData->EmitterName = EmitterUniqueName;
													DataInterfaceData->Usage = FoundOutputNode->GetUsage();
													DataInterfaceData->UsageId = FoundOutputNode->GetUsageId();
													DataInterfaceData->Variable = Variable;
												}
												DataInterfaceData->ReadsEmitterParticleData.Add(EmitterName);
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}

		if (SimStages && NumSimStageNodes)
		{
			CompileSimStageData.Reserve(NumSimStageNodes);

			const int32 NumProvidedStages = SimStages->Num();
			for (int32 i=0, ActiveStageCount = 0; ActiveStageCount < NumSimStageNodes && i < NumProvidedStages; ++i)
			{
				UNiagaraSimulationStageBase* SimStage = (*SimStages)[i];
				if (SimStage == nullptr || !SimStage->bEnabled)
				{
					continue;
				}

				if ( SimStage->FillCompilationData(CompileSimStageData) )
				{
					++ActiveStageCount;
				}
			}
		}
	}
}

void FNiagaraCompileRequestDuplicateData::FinishPrecompileDuplicate(const TArray<FNiagaraVariable>& EncounterableVariables, const TArray<FNiagaraVariable>& InStaticVariables, FCompileConstantResolver ConstantResolver, const TArray<class UNiagaraSimulationStageBase*>* SimStages, const TArray<FNiagaraVariable>& InParamStore)
{

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	PrecompiledHistories.Empty();


	TArray<UNiagaraNodeOutput*> OutputNodes;
	if (NodeGraphDeepCopy.IsValid())
	{
		NodeGraphDeepCopy->FindOutputNodes(OutputNodes);
	}

	FNiagaraCompileRequestData::SortOutputNodesByDependencies(OutputNodes, SimStages);

	for (UNiagaraNodeOutput* FoundOutputNode : OutputNodes)
	{
		FName SimStageName;
		bool bStageEnabled = true;
		if (FoundOutputNode->GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript && /*bSimulationStagesEnabled &&*/ SimStages)
		{
			// Find the simulation stage for this output node.
			const FGuid& UsageId = FoundOutputNode->GetUsageId();
			UNiagaraSimulationStageBase* const* MatchingStagePtr = SimStages->FindByPredicate([UsageId](UNiagaraSimulationStageBase* SimStage)
				{
					return SimStage != nullptr && SimStage->Script != nullptr && SimStage->Script->GetUsageId() == UsageId;
				});

			// Set whether or not the stage is enabled, and get the iteration source name if available.
			bStageEnabled = MatchingStagePtr != nullptr && (*MatchingStagePtr)->bEnabled;
			if (bStageEnabled && (*MatchingStagePtr)->IsA<UNiagaraSimulationStageGeneric>())
			{
				UNiagaraSimulationStageGeneric* GenericStage = CastChecked<UNiagaraSimulationStageGeneric>(*MatchingStagePtr);
				SimStageName = GenericStage->IterationSource == ENiagaraIterationSource::DataInterface ? GenericStage->DataInterface.BoundVariable.GetName() : FName();
			}
		}

		if (bStageEnabled)
		{
			// Map all for this output node
			FNiagaraParameterMapHistoryWithMetaDataBuilder Builder;
			*Builder.ConstantResolver = ConstantResolver;
			Builder.AddGraphToCallingGraphContextStack(NodeGraphDeepCopy.Get());
			Builder.RegisterEncounterableVariables(EncounterableVariables);
			Builder.RegisterExternalStaticVariables(InStaticVariables);

			FString TranslationName = TEXT("Emitter");
			Builder.BeginTranslation(TranslationName);
			Builder.BeginUsage(FoundOutputNode->GetUsage(), SimStageName);
			Builder.EnableScriptAllowList(true, FoundOutputNode->GetUsage());
			Builder.BuildParameterMaps(FoundOutputNode, true);
			Builder.EndUsage();

			for (FNiagaraParameterMapHistory& History : Builder.Histories)
			{
				History.OriginatingScriptUsage = FoundOutputNode->GetUsage();
				History.UsageGuid = FoundOutputNode->ScriptTypeId;
				History.UsageName = SimStageName;
				for (int32 VarIdx = 0; VarIdx < History.Variables.Num(); VarIdx++)
				{
					const FNiagaraVariable& Var = History.Variables[VarIdx];
					if (Var.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
					{
						UE_LOGF(LogNiagaraEditor, Log, "Invalid numeric parameter found! %ls", *Var.GetName().ToString())
					}
				}
			}

			PrecompiledHistories.Append(Builder.Histories);
			Builder.EndTranslation(TranslationName);
		}
		else
		{
			// Add in a blank spot
			PrecompiledHistories.Emplace();
		}
	}
}

void FNiagaraCompileRequestDuplicateData::CreateDataInterfaceCDO(TArrayView<UClass*> VariableDataInterfaces)
{
	// Collect classes for any data interfaces found in the duplicated graphs
	TArray<UClass*> DataInterfaceClasses(VariableDataInterfaces);
	for (const TPair<const UNiagaraGraph*, TArray<FDuplicatedGraphData>>& SourceGraphToDuplicatedGraphs : *(SharedSourceGraphToDuplicatedGraphsMap.Get()))
	{
		for (const FDuplicatedGraphData& DuplicatedGraphData : SourceGraphToDuplicatedGraphs.Value)
		{
			TArray<UNiagaraNodeInput*> InputNodes;
			DuplicatedGraphData.ClonedGraph->FindInputNodes(InputNodes);
			for (const UNiagaraNodeInput* InputNode : InputNodes)
			{
				if (InputNode->Input.IsDataInterface())
				{
					DataInterfaceClasses.AddUnique(InputNode->Input.GetType().GetClass());
				}
			}
		}
	}

	// Generate copies of the CDOs for any encountered data interfaces.
	for (UClass* DataInterfaceClass : DataInterfaceClasses)
	{
		UNiagaraDataInterface* DuplicateCDO = CastChecked<UNiagaraDataInterface>(PrecompileDuplicateObject(DataInterfaceClass->GetDefaultObject(true), GetTransientPackage()));
		SharedDataInterfaceClassToDuplicatedCDOMap->Add(DataInterfaceClass, DuplicateCDO);
	}
}

TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> FNiagaraEditorModule::Precompile(UObject* InObj, FGuid Version)
{
	UNiagaraScript* Script = Cast<UNiagaraScript>(InObj);
	UNiagaraPrecompileContainer* Container = Cast<UNiagaraPrecompileContainer>(InObj);
	UNiagaraSystem* System = Cast<UNiagaraSystem>(InObj);
	UPackage* LogPackage = nullptr;

	if (Container)
	{
		System = Container->System;
		if (System)
		{
			LogPackage = System->GetOutermost();
		}
	}
	else if (Script)
	{
		LogPackage = Script->GetOutermost();
	}

	if (!LogPackage || (!Script && !System))
	{
		TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> InvalidPtr;
		return InvalidPtr;
	}

	FString LogName = LogPackage ? LogPackage->GetName() : InObj->GetName();

	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraPrecompile);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*LogName, NiagaraChannel);

	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_ScriptSource_PreCompile);
	double StartTime = FPlatformTime::Seconds();

	TSharedPtr<FNiagaraCompileRequestData, ESPMode::ThreadSafe> BasePtr = MakeShared<FNiagaraCompileRequestData, ESPMode::ThreadSafe>();
	BasePtr->SharedCompileDataInterfaceData = MakeShared<TArray<FNiagaraCompileRequestData::FCompileDataInterfaceData>>();
	TArray<TSharedPtr<FNiagaraCompileRequestData, ESPMode::ThreadSafe>> DependentRequests;
	FCompileConstantResolver EmptyResolver;

	BasePtr->SourceName = LogName;

	if (Script)
	{
		BasePtr->Source = Cast<UNiagaraScriptSource>(Script->GetSource(Version));
		const TArray<FNiagaraVariable> EncounterableVariables;
		const TArray<ENiagaraScriptUsage> ValidUsages = { ENiagaraScriptUsage::Function, ENiagaraScriptUsage::Module, ENiagaraScriptUsage::DynamicInput };
		const TArray<FString> EmitterNames;
		TArray<FNiagaraVariable> StaticVars;
		BasePtr->FinishPrecompile(EncounterableVariables, StaticVars, EmptyResolver, ValidUsages, nullptr, EmitterNames);
		
	}
	else if (System)
	{
		TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalSystemData = System->GetCachedTraversalData();

		check(System->GetSystemSpawnScript()->GetLatestSource() == System->GetSystemUpdateScript()->GetLatestSource());
		BasePtr->Source = Cast<UNiagaraScriptSource>(System->GetSystemSpawnScript()->GetLatestSource());
		BasePtr->bUseRapidIterationParams = System->ShouldUseRapidIterationParameters();
		BasePtr->bDisableDebugSwitches = System->ShouldDisableDebugSwitches();
		
		TArray<FString> EmitterNames;

		// Store off the current variables in the exposed parameters list.
		TArray<FNiagaraVariable> OriginalExposedParams;
		System->GetExposedParameters().GetParameters(OriginalExposedParams);

		// Create an array of variables that we might encounter when traversing the graphs (include the originally exposed vars above)
		TArray<FNiagaraVariable> EncounterableVars(OriginalExposedParams);

		// First deep copy all the emitter graphs referenced by the system so that we can later hook up emitter handles in the system traversal.
		BasePtr->EmitterData.Empty();
		for (int32 i = 0; i < System->GetEmitterHandles().Num(); i++)
		{
			const FNiagaraEmitterHandle& Handle = System->GetEmitterHandle(i);
			TSharedPtr<FNiagaraCompileRequestData, ESPMode::ThreadSafe> EmitterPtr = MakeShared<FNiagaraCompileRequestData, ESPMode::ThreadSafe>();
			EmitterPtr->EmitterUniqueName = Handle.GetUniqueInstanceName();
			EmitterPtr->EmitterID = FNiagaraEmitterID(i);
			EmitterPtr->SourceName = BasePtr->SourceName;
			//-TODO:Stateless: We need to handle the stateless path here
			EmitterPtr->Source = Handle.GetEmitterData() ? Cast<UNiagaraScriptSource>(Handle.GetEmitterData()->GraphSource) : nullptr;
			EmitterPtr->bUseRapidIterationParams = BasePtr->bUseRapidIterationParams;
			EmitterPtr->bDisableDebugSwitches = BasePtr->bDisableDebugSwitches;
			EmitterPtr->SharedCompileDataInterfaceData = BasePtr->SharedCompileDataInterfaceData;
			BasePtr->EmitterData.Add(EmitterPtr);
			EmitterNames.Add(Handle.GetUniqueInstanceName());
		}

		// Now deep copy the system graphs, skipping traversal into any emitter references.
		TArray<FNiagaraVariable> StaticVariablesFromSystem = ((FNiagaraGraphCachedBuiltHistory*)CachedTraversalSystemData.Get())->StaticVariables;
		{
			FCompileConstantResolver ConstantResolver(System, ENiagaraScriptUsage::SystemSpawnScript);
			UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(System->GetSystemSpawnScript()->GetLatestSource());
			static TArray<ENiagaraScriptUsage> SystemUsages = { ENiagaraScriptUsage::SystemSpawnScript, ENiagaraScriptUsage::SystemUpdateScript };
			
			BasePtr->FinishPrecompile(EncounterableVars, StaticVariablesFromSystem, ConstantResolver, SystemUsages, nullptr, EmitterNames);

			BasePtr->CompareAgainst(((FNiagaraGraphCachedBuiltHistory*)CachedTraversalSystemData.Get()));
		}

		// Add the User and System variables that we did encounter to the list that emitters might also encounter.
		BasePtr->GatherPreCompiledVariables(TEXT("User"), EncounterableVars);
		BasePtr->GatherPreCompiledVariables(TEXT("System"), EncounterableVars);

		// now that the scripts have been precompiled we can prepare the rapid iteration parameters, which we need to do before we
		// actually generate the hlsl in the case of baking out the parameters
		TArray<UNiagaraScript*> Scripts;
		TMap<UNiagaraScript*, FVersionedNiagaraEmitter> ScriptToEmitterMap;

		// Now we can finish off the emitters.
		for (int32 i = 0; i < System->GetEmitterHandles().Num(); i++)
		{
			const FNiagaraEmitterHandle& Handle = System->GetEmitterHandle(i);
			FCompileConstantResolver ConstantResolver(Handle.GetInstance(), ENiagaraScriptUsage::EmitterSpawnScript);
			if (Handle.GetIsEnabled() && Handle.GetEmitterData()) // Don't pull in the emitter if it isn't going to be used.
			{
				TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalEmitterData = Handle.GetInstance().Emitter->GetCachedTraversalData(Handle.GetInstance().Version);
				TArray<FNiagaraVariable> StaticVariablesFromEmitter = StaticVariablesFromSystem;
				StaticVariablesFromEmitter.Append(((FNiagaraGraphCachedBuiltHistory*)CachedTraversalEmitterData.Get())->StaticVariables);

				TArray<UNiagaraScript*> EmitterScripts;
				FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData();
				EmitterData->GetScripts(EmitterScripts, false, true);

				for (int32 ScriptIdx = 0; ScriptIdx < EmitterScripts.Num(); ScriptIdx++)
				{
					if (EmitterScripts[ScriptIdx])
					{
						Scripts.AddUnique(EmitterScripts[ScriptIdx]);
						ScriptToEmitterMap.Add(EmitterScripts[ScriptIdx], Handle.GetInstance());
					}
				}
				static TArray<ENiagaraScriptUsage> EmitterUsages = { ENiagaraScriptUsage::EmitterSpawnScript, ENiagaraScriptUsage::EmitterUpdateScript };

				BasePtr->EmitterData[i]->FinishPrecompile(EncounterableVars, StaticVariablesFromEmitter, ConstantResolver, EmitterUsages, nullptr, EmitterNames);
				BasePtr->EmitterData[i]->CompareAgainst(((FNiagaraGraphCachedBuiltHistory*)CachedTraversalEmitterData.Get()));
				

				// Then finish the precompile for the particle scripts once we've gathered the emitter vars which might be referenced.
				TArray<FNiagaraVariable> ParticleEncounterableVars = EncounterableVars;
				BasePtr->EmitterData[i]->GatherPreCompiledVariables(TEXT("Emitter"), ParticleEncounterableVars);
				static TArray<ENiagaraScriptUsage> ParticleUsages = {
					ENiagaraScriptUsage::ParticleSpawnScript,
					ENiagaraScriptUsage::ParticleSpawnScriptInterpolated,
					ENiagaraScriptUsage::ParticleUpdateScript,
					ENiagaraScriptUsage::ParticleEventScript,
					ENiagaraScriptUsage::ParticleGPUComputeScript,
					ENiagaraScriptUsage::ParticleSimulationStageScript };

				TArray<FNiagaraVariable> OldStaticVars = BasePtr->EmitterData[i]->StaticVariables;
				BasePtr->EmitterData[i]->FinishPrecompile(ParticleEncounterableVars, OldStaticVars, ConstantResolver, ParticleUsages, &EmitterData->GetSimulationStages(), EmitterNames);
			}
		}

		{
			TMap<UNiagaraScript*, UNiagaraScript*> ScriptDependencyMap;

			// Prepare rapid iteration parameters for execution.
			TArray<TTuple<UNiagaraScript*, FVersionedNiagaraEmitter>> ScriptsToIterate;
			for (const auto& Entry : ScriptToEmitterMap)
			{
				ScriptsToIterate.Add(Entry);
			}
			for (int Index = 0; Index < ScriptsToIterate.Num(); Index++)
			{
				TTuple<UNiagaraScript*, FVersionedNiagaraEmitter> ScriptEmitterPair = ScriptsToIterate[Index];
				UNiagaraScript* CompiledScript = ScriptEmitterPair.Key;
				FVersionedNiagaraEmitterData* EmitterData = ScriptEmitterPair.Value.GetEmitterData();

				if (UNiagaraScript::IsEquivalentUsage(CompiledScript->GetUsage(), ENiagaraScriptUsage::EmitterSpawnScript))
				{
					UNiagaraScript* SystemSpawnScript = System->GetSystemSpawnScript();
					Scripts.AddUnique(SystemSpawnScript);
					ScriptDependencyMap.Add(CompiledScript, SystemSpawnScript);
					if (!ScriptToEmitterMap.Contains(SystemSpawnScript))
					{
						ScriptToEmitterMap.Add(SystemSpawnScript);
						ScriptsToIterate.Emplace(SystemSpawnScript, FVersionedNiagaraEmitter());
					}
				}

				if (UNiagaraScript::IsEquivalentUsage(CompiledScript->GetUsage(), ENiagaraScriptUsage::EmitterUpdateScript))
				{
					UNiagaraScript* SystemUpdateScript = System->GetSystemUpdateScript();
					Scripts.AddUnique(SystemUpdateScript);
					ScriptDependencyMap.Add(CompiledScript, SystemUpdateScript);
					ScriptToEmitterMap.Add(SystemUpdateScript);
					if (!ScriptToEmitterMap.Contains(SystemUpdateScript))
					{
						ScriptToEmitterMap.Add(SystemUpdateScript);
						ScriptsToIterate.Emplace(SystemUpdateScript, FVersionedNiagaraEmitter());
					}
				}

				if (UNiagaraScript::IsEquivalentUsage(CompiledScript->GetUsage(), ENiagaraScriptUsage::ParticleSpawnScript))
				{
					if (EmitterData && EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
					{
						UNiagaraScript* ComputeScript = EmitterData->GetGPUComputeScript();

						Scripts.AddUnique(ComputeScript);
						ScriptDependencyMap.Add(CompiledScript, ComputeScript);
						if (!ScriptToEmitterMap.Contains(ComputeScript))
						{
							ScriptToEmitterMap.Add(ComputeScript, ScriptEmitterPair.Value);
							ScriptsToIterate.Emplace(ComputeScript, ScriptEmitterPair.Value);
						}
					}
				}

				if (UNiagaraScript::IsEquivalentUsage(CompiledScript->GetUsage(), ENiagaraScriptUsage::ParticleUpdateScript))
				{
					if (EmitterData && EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
					{
						UNiagaraScript* ComputeScript = EmitterData->GetGPUComputeScript();

						Scripts.AddUnique(ComputeScript);
						ScriptDependencyMap.Add(CompiledScript, ComputeScript);
						if (!ScriptToEmitterMap.Contains(ComputeScript))
						{
							ScriptToEmitterMap.Add(ComputeScript, ScriptEmitterPair.Value);
							ScriptsToIterate.Emplace(ComputeScript, ScriptEmitterPair.Value);
						}
					}
					else if (EmitterData && EmitterData->UsesInterpolatedSpawning())
					{
						Scripts.AddUnique(EmitterData->SpawnScriptProps.Script);
						ScriptDependencyMap.Add(CompiledScript, EmitterData->SpawnScriptProps.Script);
						if (!ScriptToEmitterMap.Contains(EmitterData->SpawnScriptProps.Script))
						{
							ScriptToEmitterMap.Add(EmitterData->SpawnScriptProps.Script, ScriptEmitterPair.Value);
							ScriptsToIterate.Emplace(EmitterData->SpawnScriptProps.Script, ScriptEmitterPair.Value);
						}
					}
				}
			}

			FNiagaraUtilities::PrepareRapidIterationParameters(Scripts, ScriptDependencyMap, ScriptToEmitterMap);

			BasePtr->AddRapidIterationParameters(System->GetSystemSpawnScript()->RapidIterationParameters, FCompileConstantResolver(System, ENiagaraScriptUsage::SystemSpawnScript));
			BasePtr->AddRapidIterationParameters(System->GetSystemUpdateScript()->RapidIterationParameters, FCompileConstantResolver(System, ENiagaraScriptUsage::SystemUpdateScript));

			// Now we can finish off the emitters.
			for (int32 i = 0; i < System->GetEmitterHandles().Num(); i++)
			{
				const FNiagaraEmitterHandle& Handle = System->GetEmitterHandle(i);
				FCompileConstantResolver ConstantResolver(Handle.GetInstance(), ENiagaraScriptUsage::EmitterSpawnScript);
				if (Handle.GetIsEnabled() && Handle.GetEmitterData()) // Don't pull in the emitter if it isn't going to be used.
				{
					TArray<UNiagaraScript*> EmitterScripts;
					Handle.GetEmitterData()->GetScripts(EmitterScripts, false);

					for (int32 ScriptIdx = 0; ScriptIdx < EmitterScripts.Num(); ScriptIdx++)
					{
						if (EmitterScripts[ScriptIdx])
						{
							BasePtr->EmitterData[i]->AddRapidIterationParameters(EmitterScripts[ScriptIdx]->RapidIterationParameters, ConstantResolver);
						}
					}
				}
			}
		}
	}

	UE_LOGF(LogNiagaraEditor, Verbose, "'%ls' Precompile took %f sec.", *LogName,
		(float)(FPlatformTime::Seconds() - StartTime));

	return BasePtr;
}

void GetUsagesToDuplicate(ENiagaraScriptUsage TargetUsage, TArray<ENiagaraScriptUsage>& DuplicateUsages)
{
	/*switch (TargetUsage)
	{
	case ENiagaraScriptUsage::SystemSpawnScript:
		DuplicateUsages.Add(ENiagaraScriptUsage::SystemSpawnScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::EmitterSpawnScript);
		break;
	case ENiagaraScriptUsage::SystemUpdateScript:
		DuplicateUsages.Add(ENiagaraScriptUsage::SystemUpdateScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::EmitterUpdateScript);
		break;
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
		DuplicateUsages.Add(TargetUsage);
		break;
	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSpawnScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSpawnScriptInterpolated);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleUpdateScript);
		break;
	case ENiagaraScriptUsage::ParticleGPUComputeScript:
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSpawnScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSpawnScriptInterpolated);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleUpdateScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSimulationStageScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleGPUComputeScript);
		break;
	}*/

	// For now we need to include both spawn and update for each target usage, otherwise attribute lists in the precompiled histories aren't generated correctly.
	switch (TargetUsage)
	{
	case ENiagaraScriptUsage::SystemSpawnScript:
	case ENiagaraScriptUsage::SystemUpdateScript:
		DuplicateUsages.Add(ENiagaraScriptUsage::SystemSpawnScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::SystemUpdateScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::EmitterSpawnScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::EmitterUpdateScript);
		break;
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
	case ENiagaraScriptUsage::ParticleGPUComputeScript:
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSpawnScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSpawnScriptInterpolated);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleUpdateScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleEventScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleSimulationStageScript);
		DuplicateUsages.Add(ENiagaraScriptUsage::ParticleGPUComputeScript);
		break;
	}
}

TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> FNiagaraEditorModule::PrecompileDuplicate(
	const FNiagaraCompileRequestDataBase* OwningSystemRequestData,
	UNiagaraSystem* OwningSystem,
	UNiagaraEmitter* OwningEmitter,
	UNiagaraScript* TargetScript,
	FGuid TargetVersion)
{
	FString LogName;
	if (OwningSystem != nullptr)
	{
		LogName = OwningSystem->GetOutermost() != nullptr ? OwningSystem->GetOutermost()->GetName() : OwningSystem->GetName();
	}
	else if (TargetScript != nullptr)
	{
		LogName = TargetScript->GetOutermost() != nullptr ? TargetScript->GetOutermost()->GetName() : TargetScript->GetName();
	}
	else
	{
		TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> InvalidPtr;
		return InvalidPtr;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraPrecompileDuplicate);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*LogName, NiagaraChannel);

	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_ScriptSource_PreCompileDuplicate);
	double StartTime = FPlatformTime::Seconds();

	TSharedPtr<FNiagaraCompileRequestDuplicateData, ESPMode::ThreadSafe> BasePtr = MakeShared<FNiagaraCompileRequestDuplicateData, ESPMode::ThreadSafe>();
	TArray<TSharedPtr<FNiagaraCompileRequestDuplicateData, ESPMode::ThreadSafe>> DependentRequests;
	FCompileConstantResolver EmptyResolver;

	BasePtr->SharedSourceGraphToDuplicatedGraphsMap = MakeShared<TMap<const UNiagaraGraph*, TArray<FNiagaraCompileRequestDuplicateData::FDuplicatedGraphData>>>();
	BasePtr->SharedNameToDuplicatedDataInterfaceMap = MakeShared<TMap<FName, UNiagaraDataInterface*>>();
	BasePtr->SharedDataInterfaceClassToDuplicatedCDOMap = MakeShared<TMap<UClass*, UNiagaraDataInterface*>>();
	BasePtr->OwningSystem = OwningSystem;
	BasePtr->OwningEmitter = OwningEmitter;

	TArray<UClass*> DataInterfaceClasses;

	auto CollectDataInterfaceClasses = [&](TConstArrayView<FNiagaraVariable> Variables)
	{
		// Collect classes for external encounterable variables
		for (const FNiagaraVariable& EncounterableVariable : Variables)
		{
			if (EncounterableVariable.IsDataInterface())
			{
				DataInterfaceClasses.AddUnique(EncounterableVariable.GetType().GetClass());
			}
		}
	};

	if (OwningSystem == nullptr)
	{
		UNiagaraScriptSource* Source = CastChecked<UNiagaraScriptSource>(TargetScript->GetSource(TargetVersion));
		BasePtr->ValidUsages.Add(TargetScript->GetUsage());
		BasePtr->DeepCopyGraphs(Source, TargetScript->GetUsage(), EmptyResolver);
		TArray<FNiagaraVariable> EncounterableScriptVariables;
		OwningSystemRequestData->GatherPreCompiledVariables(FString(), EncounterableScriptVariables);
		BasePtr->FinishPrecompileDuplicate(EncounterableScriptVariables, TArray<FNiagaraVariable>(), EmptyResolver, nullptr, ((FNiagaraCompileRequestData*)OwningSystemRequestData)->RapidIterationParams);
		CollectDataInterfaceClasses(EncounterableScriptVariables);
	}
	else
	{
		TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalSystemData = OwningSystem->GetCachedTraversalData();
		GetUsagesToDuplicate(TargetScript->GetUsage(), BasePtr->ValidUsages);

		check(OwningSystem->GetSystemSpawnScript()->GetLatestSource() == OwningSystem->GetSystemUpdateScript()->GetLatestSource());

		// First deep copy all the emitter graphs referenced by the system so that we can later hook up emitter handles in the system traversal.
		BasePtr->EmitterData.Empty();
		for (int32 i = 0; i < OwningSystem->GetEmitterHandles().Num(); i++)
		{
			const FNiagaraEmitterHandle& Handle = OwningSystem->GetEmitterHandle(i);
			TSharedPtr<FNiagaraCompileRequestDuplicateData, ESPMode::ThreadSafe> EmitterPtr = MakeShared<FNiagaraCompileRequestDuplicateData, ESPMode::ThreadSafe>();
			EmitterPtr->EmitterUniqueName = Handle.GetUniqueInstanceName();
			EmitterPtr->EmitterID = FNiagaraEmitterID(i);
			EmitterPtr->ValidUsages = BasePtr->ValidUsages;
			EmitterPtr->SharedSourceGraphToDuplicatedGraphsMap = BasePtr->SharedSourceGraphToDuplicatedGraphsMap;
			EmitterPtr->SharedNameToDuplicatedDataInterfaceMap = BasePtr->SharedNameToDuplicatedDataInterfaceMap;
			EmitterPtr->SharedDataInterfaceClassToDuplicatedCDOMap = BasePtr->SharedDataInterfaceClassToDuplicatedCDOMap;
			//EmitterPtr->bSimulationStagesEnabled = Handle.GetInstance()->bSimulationStagesEnabled;
			if (Handle.GetIsEnabled() && Handle.GetInstance().Emitter && (OwningEmitter == nullptr || OwningEmitter == Handle.GetInstance().Emitter)) // Don't need to copy the graph if we aren't going to use it.
			{
				EmitterPtr->DeepCopyGraphs(Handle.GetInstance());
			}
			EmitterPtr->ValidUsages = BasePtr->ValidUsages;
			BasePtr->EmitterData.Add(EmitterPtr);
		}

		// Now deep copy the system graphs, skipping traversal into any emitter references.
		TArray<FNiagaraVariable> StaticVariablesFromSystem = ((FNiagaraGraphCachedBuiltHistory*)CachedTraversalSystemData.Get())->StaticVariables;
		{
			UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(OwningSystem->GetSystemSpawnScript()->GetLatestSource());

			TArray<FNiagaraVariable> EncounterableSystemVariables;
			OwningSystemRequestData->GatherPreCompiledVariables(FString(), EncounterableSystemVariables);

			// skip the deep copy if we're not compiling the system scripts
			if (BasePtr->ValidUsages.Contains(ENiagaraScriptUsage::SystemSpawnScript))
			{
				FCompileConstantResolver ConstantResolver(OwningSystem, ENiagaraScriptUsage::SystemSpawnScript);
				BasePtr->DeepCopyGraphs(Source, ENiagaraScriptUsage::SystemSpawnScript, ConstantResolver);
				BasePtr->FinishPrecompileDuplicate(EncounterableSystemVariables, StaticVariablesFromSystem, ConstantResolver, nullptr, ((FNiagaraCompileRequestData*)OwningSystemRequestData)->RapidIterationParams);
			}

			CollectDataInterfaceClasses(EncounterableSystemVariables);
		}

		// Now we can finish off the emitters.
		for (int32 i = 0; i < OwningSystem->GetEmitterHandles().Num(); i++)
		{
			const FNiagaraEmitterHandle& Handle = OwningSystem->GetEmitterHandle(i);

			TArray<FNiagaraVariable> EncounterableEmitterVariables;
			OwningSystemRequestData->GetDependentRequest(i)->GatherPreCompiledVariables(FString(), EncounterableEmitterVariables);

			if (Handle.GetIsEnabled() && Handle.GetInstance().Emitter && (OwningEmitter == nullptr || OwningEmitter == Handle.GetInstance().Emitter))
			{
				TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalEmitterData = Handle.GetInstance().Emitter->GetCachedTraversalData(Handle.GetInstance().Version);
				TArray<FNiagaraVariable> StaticVariablesFromEmitter = StaticVariablesFromSystem;
				StaticVariablesFromEmitter.Append(((FNiagaraGraphCachedBuiltHistory*)CachedTraversalEmitterData.Get())->StaticVariables);

				FCompileConstantResolver ConstantResolver(Handle.GetInstance(), ENiagaraScriptUsage::EmitterSpawnScript);
				BasePtr->EmitterData[i]->FinishPrecompileDuplicate(EncounterableEmitterVariables, StaticVariablesFromEmitter, ConstantResolver, &Handle.GetEmitterData()->GetSimulationStages(), ((FNiagaraCompileRequestData*)OwningSystemRequestData)->EmitterData[i]->RapidIterationParams);

			}

			CollectDataInterfaceClasses(EncounterableEmitterVariables);
		}
	}

	BasePtr->CreateDataInterfaceCDO(DataInterfaceClasses);

	UE_LOGF(LogNiagaraEditor, Verbose, "'%ls' PrecompileDuplicate took %f sec.", *LogName,
		(float)(FPlatformTime::Seconds() - StartTime));

	return BasePtr;
}

int32 FNiagaraEditorModule::CompileScript(const FNiagaraCompileRequestDataBase* InCompileRequest, const FNiagaraCompileRequestDuplicateDataBase* InCompileRequestDuplicate, const FNiagaraCompileOptions& InCompileOptions)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_CompileScript);

	check(InCompileRequest != NULL);
	const FNiagaraCompileRequestData* CompileRequest = (const FNiagaraCompileRequestData*)InCompileRequest;
	const FNiagaraCompileRequestDuplicateData* CompileRequestDuplicate = (const FNiagaraCompileRequestDuplicateData*)InCompileRequestDuplicate;
	TArray<FNiagaraVariable> CookedRapidIterationParams = CompileRequest->GetUseRapidIterationParams() ? TArray<FNiagaraVariable>() : CompileRequest->RapidIterationParams;

	UE_LOGF(LogNiagaraEditor, Verbose, "Compiling System %ls ...................................................................", *InCompileOptions.FullName);

	FNiagaraCompileResults Results;
	FActiveCompilation ActiveCompilation;
	ActiveCompilation.Compiler = MakeShared<FHlslNiagaraCompiler>();
	FNiagaraTranslateResults TranslateResults;
	TUniquePtr<INiagaraHlslTranslator> Translator = INiagaraHlslTranslator::CreateTranslator(CompileRequest, CompileRequestDuplicate);

	FHlslNiagaraTranslatorOptions TranslateOptions;

	if (InCompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleGPUComputeScript)
	{
		TranslateOptions.SimTarget = ENiagaraSimTarget::GPUComputeSim;
	}
	else
	{
		TranslateOptions.SimTarget = ENiagaraSimTarget::CPUSim;
	}
	TranslateOptions.OverrideModuleConstants = CookedRapidIterationParams;
	TranslateOptions.bParameterRapidIteration = InCompileRequest->GetUseRapidIterationParams();
	TranslateOptions.bDisableDebugSwitches = InCompileRequest->GetDisableDebugSwitches();


	double TranslationStartTime = FPlatformTime::Seconds();
	if (GbForceNiagaraTranslatorSingleThreaded > 0)
	{
		FScopeLock Lock(&TranslationCritSec);
		TranslateResults = Translator->Translate(InCompileOptions, TranslateOptions);
	}
	else
	{
		TranslateResults = Translator->Translate(InCompileOptions, TranslateOptions);
	}

	ActiveCompilation.TranslationTime = (float) (FPlatformTime::Seconds() - TranslationStartTime);
	UE_LOGF(LogNiagaraEditor, Verbose, "Translating System %ls took %f sec.", *InCompileOptions.FullName, (float)(FPlatformTime::Seconds() - TranslationStartTime));

	if (GbForceNiagaraTranslatorDump != 0)
	{
		DumpHLSLText(Translator->GetTranslatedHLSL(), InCompileOptions.FullName);
		if (GbForceNiagaraVMBinaryDump != 0 && Results.Data.IsValid())
		{
			DumpHLSLText(Results.Data->LastAssemblyTranslation, InCompileOptions.FullName);
		}
	}

	NiagaraCompileRequestHelper::FDebugGroupNameBuilder DebugGroupName;
	NiagaraCompileRequestHelper::BuildScriptDebugGroupName(CompileRequest, InCompileOptions, DebugGroupName);

	int32 JobID = ActiveCompilation.Compiler->CompileScript(DebugGroupName, InCompileOptions, TranslateResults, Translator->GetTranslateOutput(), Translator->GetTranslatedHLSL());
	ActiveCompilations.Add(JobID, ActiveCompilation);
	return JobID;
}

TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> FNiagaraEditorModule::CacheGraphTraversal(const UObject* Obj, FGuid Version)
{
	TSharedPtr<FNiagaraGraphCachedBuiltHistory, ESPMode::ThreadSafe> CachedPtr = MakeShared<FNiagaraGraphCachedBuiltHistory>();

	const UNiagaraSystem* System = Cast<UNiagaraSystem>(Obj);
	const UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Obj);

	/*
	* const TArray<FNiagaraVariable>& EncounterableVariables, const TArray<FNiagaraVariable>& InStaticVariables, FCompileConstantResolver ConstantResolver, const TArray<ENiagaraScriptUsage>& UsagesToProcess, const 
	*/
	const TArray<class UNiagaraSimulationStageBase*>* SimStages = nullptr;
	const UNiagaraScriptSource* ScriptSource = nullptr;
	FCompileConstantResolver ConstantResolver;
	TArray<FNiagaraVariable> EncounterableVariables;
	TArray<FNiagaraVariable> StaticVariablesFromSystem;
	TArray<FNiagaraVariable> StaticVariablesFromSystemAndEmitters;
	TArray<FNiagaraVariable> SrcStaticVariables;
	TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> ParentCachedData;
	FString SrcUniqueEmitterName;

	if (System)
	{
		CachedPtr->SetSourceSystem(System);
		ScriptSource = CastChecked<UNiagaraScriptSource>(System->GetSystemSpawnScript()->GetLatestSource());
		ConstantResolver = FCompileConstantResolver(System, ENiagaraScriptUsage::SystemSpawnScript);
		System->GatherStaticVariables(StaticVariablesFromSystem, StaticVariablesFromSystemAndEmitters);

		SrcStaticVariables = StaticVariablesFromSystem;

		for (const FNiagaraVariable& Var : StaticVariablesFromSystemAndEmitters)
			SrcStaticVariables.AddUnique(Var);
	}
	else if (Emitter)
	{
		const FVersionedNiagaraEmitterData* EmitterData = Emitter->GetEmitterData(Version);

		CachedPtr->SetSourceEmitter(EmitterData);
		UNiagaraSystem* SysParent = Cast<UNiagaraSystem>(Emitter->GetOuter());
		if (SysParent)
		{
			ParentCachedData = SysParent->GetCachedTraversalData();

			if (ParentCachedData.IsValid())
			{
				SrcStaticVariables = ((FNiagaraGraphCachedBuiltHistory*)ParentCachedData.Get())->StaticVariables;
			}
			/*for (const FNiagaraEmitterHandle& Handle : SysParent->GetEmitterHandles())
			{
				if (Handle.GetInstance() == Emitter)
				{
					SrcUniqueEmitterName = Handle.GetUniqueEmitterName().ToString();
				}
			}*/
		}
		SrcUniqueEmitterName = Emitter->GetUniqueEmitterName();
		EmitterData->GatherStaticVariables(SrcStaticVariables);

		ScriptSource = CastChecked<UNiagaraScriptSource>(EmitterData->GraphSource);
		SimStages = &EmitterData->GetSimulationStages();
		ConstantResolver = FCompileConstantResolver(FVersionedNiagaraEmitter(const_cast<UNiagaraEmitter*>(Emitter), Version), ENiagaraScriptUsage::EmitterSpawnScript);
	}


	TArray<UNiagaraNodeOutput*> OutputNodes;
	if (ScriptSource != nullptr && ScriptSource->NodeGraph != nullptr)
	{
		ScriptSource->NodeGraph->FindOutputNodes(OutputNodes);
	}

	bool bFilterByEmitterAlias = true;
	for (UNiagaraNodeOutput* FoundOutputNode : OutputNodes)
	{
		if (UNiagaraScript::IsSystemScript(FoundOutputNode->GetUsage()))
		{
			bFilterByEmitterAlias = false;
		}
	}

	// Only use the static variables that match up with our expectations for this script. IE for emitters, filter things out for resolution.
	TArray<FNiagaraVariable> StaticVariables;
	FNiagaraParameterUtilities::FilterToRelevantStaticVariables(SrcStaticVariables, StaticVariables, *SrcUniqueEmitterName, TEXT("Emitter"), bFilterByEmitterAlias);

	int32 NumSimStageNodes = 0;
	for (UNiagaraNodeOutput* FoundOutputNode : OutputNodes)
	{
		FName SimStageName;
		bool bStageEnabled = true;
		if (FoundOutputNode->GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript && SimStages)
		{
			// Find the simulation stage for this output node.
			const FGuid& UsageId = FoundOutputNode->GetUsageId();
			UNiagaraSimulationStageBase* const* MatchingStagePtr = SimStages->FindByPredicate([UsageId](UNiagaraSimulationStageBase* SimStage)
				{
					return SimStage != nullptr && SimStage->Script != nullptr && SimStage->Script->GetUsageId() == UsageId;
				});

			// Set whether or not the stage is enabled, and get the iteration source name if available.
			bStageEnabled = MatchingStagePtr != nullptr && (*MatchingStagePtr)->bEnabled;
			if (bStageEnabled && (*MatchingStagePtr)->IsA<UNiagaraSimulationStageGeneric>())
			{
				UNiagaraSimulationStageGeneric* GenericStage = CastChecked<UNiagaraSimulationStageGeneric>(*MatchingStagePtr);
				SimStageName = GenericStage->IterationSource == ENiagaraIterationSource::DataInterface ? GenericStage->DataInterface.BoundVariable.GetName() : FName();
			}
		}

		if (bStageEnabled)
		{

			// Map all for this output node
			FNiagaraParameterMapHistoryWithMetaDataBuilder Builder;
			*Builder.ConstantResolver = ConstantResolver;
			if (ScriptSource != nullptr)
			{
				Builder.AddGraphToCallingGraphContextStack(ScriptSource->NodeGraph);
			}
			Builder.RegisterEncounterableVariables(EncounterableVariables);
			Builder.RegisterExternalStaticVariables(StaticVariables);

			FString TranslationName = TEXT("Emitter");
			Builder.BeginTranslation(TranslationName);
			Builder.BeginUsage(FoundOutputNode->GetUsage(), SimStageName);
			Builder.EnableScriptAllowList(true, FoundOutputNode->GetUsage());
			Builder.IncludeStaticVariablesOnly();
			Builder.BuildParameterMaps(FoundOutputNode, true);
			Builder.EndUsage();

			for (int32 BuilderVarIndex = 0; BuilderVarIndex < Builder.StaticVariables.Num() && BuilderVarIndex < Builder.StaticVariableExportable.Num(); ++BuilderVarIndex)
			{
				StaticVariables.AddUnique(Builder.StaticVariables[BuilderVarIndex]);
			}
		}
	}


	CachedPtr->StaticVariables = StaticVariables;
	if (GbForceNiagaraCacheDump != 0 && Obj )
	{
		UE_LOGF(LogNiagaraEditor, Log, "==================================================================\nCacheGraphTraversal %ls\n==================================================================", *Obj->GetPathName());
		int32 i = 0;
		UE_LOGF(LogNiagaraEditor, Log, "Static Variables: %d", StaticVariables.Num());
		for (const FNiagaraVariable& Var : StaticVariables)
		{
			UE_LOGF(LogNiagaraEditor, Log, "[%d] %ls", i, *Var.ToString());
			++i;
		}
	}
	return CachedPtr;
}

TSharedPtr<FNiagaraVMExecutableData> FNiagaraEditorModule::GetCompilationResult(int32 JobID, bool bWait, FNiagaraScriptCompileMetrics& ScriptMetrics)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraEditorModule::GetCompilationResult);

	FActiveCompilation* MapEntry = ActiveCompilations.Find(JobID);
	check(MapEntry && MapEntry->Compiler.IsValid());

	TSharedPtr<FHlslNiagaraCompiler> Compiler = MapEntry->Compiler;
	TOptional<FNiagaraCompileResults> CompileResult = Compiler->GetCompileResult(JobID, bWait);
	if (!CompileResult)
	{
		return TSharedPtr<FNiagaraVMExecutableData>();
	}

	ScriptMetrics.TranslateTime = MapEntry->TranslationTime;

	ActiveCompilations.Remove(JobID);
	FNiagaraCompileResults& Results = CompileResult.GetValue();

	FString OutGraphLevelErrorMessages;
	for (const FNiagaraCompileEvent& Message : Results.CompileEvents)
	{
		#if defined(NIAGARA_SCRIPT_COMPILE_LOGGING_MEDIUM)
			UE_LOGF(LogNiagaraCompiler, Log, "%ls", *Message.Message);
		#endif
		if (Message.Severity == FNiagaraCompileEventSeverity::Error)
		{
			// Write the error messages to the string as well so that they can be echoed up the chain.
			if (OutGraphLevelErrorMessages.Len() > 0)
			{
				OutGraphLevelErrorMessages += "\n";
			}
			OutGraphLevelErrorMessages += Message.Message;
		}
	}
	
	Results.Data->ErrorMsg = OutGraphLevelErrorMessages;
	Results.Data->LastCompileStatus = (FNiagaraCompileResults::CompileResultsToSummary(&Results));
	if (Results.Data->LastCompileStatus != ENiagaraScriptCompileStatus::NCS_Error)
	{
		// When there are no errors the compile events get emptied, so add them back here.
		Results.Data->LastCompileEvents.Append(Results.CompileEvents);
	}

	ScriptMetrics.CompilerWallTime = CompileResult->CompilerWallTime;
	ScriptMetrics.CompilerPreprocessTime = CompileResult->CompilerPreprocessTime;
	ScriptMetrics.CompilerWorkerTime = CompileResult->CompilerWorkerTime;

	return CompileResult->Data;
}

void FNiagaraEditorModule::TestCompileScriptFromConsole(const TArray<FString>& Arguments)
{
	if (Arguments.Num() == 1)
	{
		FString TranslatedHLSL;
		FFileHelper::LoadFileToString(TranslatedHLSL, *Arguments[0]);
		if (TranslatedHLSL.Len() != 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslCompiler_TestCompileShader_VectorVM);
			FShaderCompilerInput Input;
			Input.Target = FShaderTarget(SF_Compute, SP_PCD3D_SM5);
			Input.VirtualSourceFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf");
			Input.EntryPointName = TEXT("SimulateMain");
			Input.Environment.SetDefine(TEXT("VM_SIMULATION"), 1);
			Input.Environment.SetDefine(TEXT("COMPUTESHADER"), 1);
			Input.Environment.SetDefine(TEXT("PIXELSHADER"), 0);
			Input.Environment.SetDefine(TEXT("DOMAINSHADER"), 0);
			Input.Environment.SetDefine(TEXT("HULLSHADER"), 0);
			Input.Environment.SetDefine(TEXT("VERTEXSHADER"), 0);
			Input.Environment.SetDefine(TEXT("GEOMETRYSHADER"), 0);
			Input.Environment.SetDefine(TEXT("MESHSHADER"), 0);
			Input.Environment.SetDefine(TEXT("AMPLIFICATIONSHADER"), 0);
			Input.Environment.IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/NiagaraEmitterInstance.ush"), TranslatedHLSL);

			FVectorVMCompilationOutput CompilationOutput;
			double StartTime = FPlatformTime::Seconds();
			bool bSucceeded = TestCompileVectorVMShader(Input, CompilationOutput, GNiagaraSkipVectorVMBackendOptimizations != 0);
			float DeltaTime = (float)(FPlatformTime::Seconds() - StartTime);

			if (bSucceeded)
			{
				UE_LOGF(LogNiagaraCompiler, Log, "Test compile of %ls took %f seconds and succeeded.", *Arguments[0], DeltaTime);
			}
			else
			{
				UE_LOGF(LogNiagaraCompiler, Error, "Test compile of %ls took %f seconds and failed.  Errors: %ls", *Arguments[0], DeltaTime, *CompilationOutput.Errors);
			}
		}
		else
		{
			UE_LOGF(LogNiagaraCompiler, Error, "Test compile of %ls failed, the file could not be loaded or it was empty.", *Arguments[0]);
		}
	}
	else
	{
		UE_LOGF(LogNiagaraCompiler, Error, "Test compile failed, file name argument was missing.");
	}
}


ENiagaraScriptCompileStatus FNiagaraCompileResults::CompileResultsToSummary(const FNiagaraCompileResults* CompileResults)
{
	ENiagaraScriptCompileStatus SummaryStatus = ENiagaraScriptCompileStatus::NCS_Unknown;
	if (CompileResults != nullptr)
	{
		if (CompileResults->NumErrors > 0)
		{
			SummaryStatus = ENiagaraScriptCompileStatus::NCS_Error;
		}
		else
		{
			if (CompileResults->bVMSucceeded)
			{
				if (CompileResults->NumWarnings)
				{
					SummaryStatus = ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings;
				}
				else
				{
					SummaryStatus = ENiagaraScriptCompileStatus::NCS_UpToDate;
				}
			}

			if (CompileResults->bComputeSucceeded)
			{
				if (CompileResults->NumWarnings)
				{
					SummaryStatus = ENiagaraScriptCompileStatus::NCS_ComputeUpToDateWithWarnings;
				}
				else
				{
					SummaryStatus = ENiagaraScriptCompileStatus::NCS_UpToDate;
				}
			}
		}
	}
	return SummaryStatus;
}

int32 FHlslNiagaraCompiler::CompileScript(const FNiagaraCompileRequestData* InCompileRequest, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, FNiagaraTranslatorOutput* TranslatorOutput, FString& TranslatedHLSL)
{
	NiagaraCompileRequestHelper::FDebugGroupNameBuilder DebugGroupName;
	NiagaraCompileRequestHelper::BuildScriptDebugGroupName(InCompileRequest, InOptions, DebugGroupName);

	return CompileScript(DebugGroupName, InOptions, InTranslateResults, TranslatorOutput ? *TranslatorOutput : FNiagaraTranslatorOutput(), TranslatedHLSL);
}

int32 FHlslNiagaraCompiler::CompileScript(const FStringView GroupName, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, const FNiagaraTranslatorOutput& TranslatorOutput, const FString& TranslatedHLSL)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslCompiler_CompileScript);

	CompileResults.Data = MakeShared<FNiagaraVMExecutableData>();

	//TODO: This should probably be done via the same route that other shaders take through the shader compiler etc.
	//But that adds the complexity of a new shader type, new shader class and a new shader map to contain them etc.
	//Can do things simply for now.

	CompileResults.Data->LastHlslTranslation = TEXT("");

	FShaderCompilerInput Input;
	Input.Target = FShaderTarget(SF_Compute, SP_PCD3D_SM5);
	Input.VirtualSourceFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf");
	Input.EntryPointName = TEXT("SimulateMain");
	Input.Environment.SetDefine(TEXT("VM_SIMULATION"), 1);
	Input.Environment.SetDefine(TEXT("COMPUTESHADER"), 1);
	Input.Environment.SetDefine(TEXT("PIXELSHADER"), 0);
	Input.Environment.SetDefine(TEXT("DOMAINSHADER"), 0);
	Input.Environment.SetDefine(TEXT("HULLSHADER"), 0);
	Input.Environment.SetDefine(TEXT("VERTEXSHADER"), 0);
	Input.Environment.SetDefine(TEXT("GEOMETRYSHADER"), 0);
	Input.Environment.SetDefine(TEXT("MESHSHADER"), 0);
	Input.Environment.SetDefine(TEXT("AMPLIFICATIONSHADER"), 0);
	Input.Environment.IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/NiagaraEmitterInstance.ush"), TranslatedHLSL);
	Input.DebugInfoFlags = GShaderCompilingManager->GetDumpShaderDebugInfoFlags();
	Input.DumpDebugInfoRootPath = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory() / TEXT("VM");
	Input.DebugGroupName = GroupName;
	Input.DebugExtension.Empty();
	Input.DumpDebugInfoPath.Empty();

	FName VVMFormatName = FName(TEXT("VVM_1_0"));
	//TODO: This is normally invoked by GlobalBeginCompileShader, which is not called in this path. Should it be?
	const IShaderFormat* VVMShaderFormat = GetTargetPlatformManagerRef().FindShaderFormat(VVMFormatName);
	VVMShaderFormat->ModifyShaderCompilerInput(Input);

	if (GShaderCompilingManager->GetDumpShaderDebugInfo() == FShaderCompilingManager::EDumpShaderDebugInfo::Always)
	{
		Input.DumpDebugInfoPath = GShaderCompilingManager->CreateShaderDebugInfoPath(Input);
	}
	CompileResults.DumpDebugInfoPath = Input.DumpDebugInfoPath;

	uint32 JobID = FShaderCommonCompileJob::GetNextJobId();
	CompilationJob = MakeUnique<FNiagaraCompilerJob>();
	CompilationJob->TranslatorOutput = TranslatorOutput;

	CompileResults.bVMSucceeded = (CompilationJob->TranslatorOutput.Errors.Len() == 0) && (TranslatedHLSL.Len() > 0) && !InTranslateResults.NumErrors;

	// only issue jobs for VM compilation if we're going to be using the resulting byte code.  This excludes particle scripts when we're using
	// a GPU simulation
	const bool bCompilingGPUParticleScript = InOptions.IsGpuScript() && UNiagaraScript::IsParticleScript(InOptions.TargetUsage);
	if (bCompilingGPUParticleScript)
	{
		CompileResults.bComputeSucceeded = false;
		if (CompileResults.bVMSucceeded)
		{
			//Clear out current contents of compile results.
			*(CompileResults.Data) = CompilationJob->TranslatorOutput.ScriptData;
			CompileResults.Data->ByteCode.Reset();
			CompileResults.bComputeSucceeded = true;
		}
	}

	CompileResults.AppendCompileEvents(MakeArrayView(InTranslateResults.CompileEvents));
	CompileResults.Data->LastCompileEvents.Append(InTranslateResults.CompileEvents);
	CompileResults.Data->ExternalDependencies = InTranslateResults.CompileDependencies;
	CompileResults.Data->CompileTags = InTranslateResults.CompileTags;
	CompileResults.Data->CompileTagsEditorOnly = InTranslateResults.CompileTagsEditorOnly;

	// Early out if compiling a GPU particle script as we do not need to submit a CPU compile request.
	// This must be done after we add in the translator errors etc so tha they are passed to the compile job correctly.
	if (bCompilingGPUParticleScript)
	{
		CompileResults.Data->LastHlslTranslationGPU = TranslatedHLSL;
		DumpDebugInfo(CompileResults, Input, true);
		CompilationJob->CompileResults = CompileResults;
		return JobID;
	}

	CompilationJob->TranslatorOutput.ScriptData.LastHlslTranslation = TranslatedHLSL;
	CompilationJob->TranslatorOutput.ScriptData.ExternalDependencies = InTranslateResults.CompileDependencies;
	CompilationJob->TranslatorOutput.ScriptData.CompileTags = InTranslateResults.CompileTags;
	CompilationJob->TranslatorOutput.ScriptData.CompileTagsEditorOnly = InTranslateResults.CompileTagsEditorOnly;

	bool bJobScheduled = false;
	if (CompileResults.bVMSucceeded)
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslCompiler_CompileShader_VectorVM);
		CompilationJob->StartTime = FPlatformTime::Seconds();

		FShaderType* NiagaraShaderType = nullptr;
		for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
		{
			if (FNiagaraShaderType* ShaderType = ShaderTypeIt->GetNiagaraShaderType())
			{
				NiagaraShaderType = ShaderType;
				break;
			}
		}
		if (NiagaraShaderType)
		{
			TRefCountPtr<FShaderCompileJob> Job = GShaderCompilingManager->PrepareShaderCompileJob(JobID, FShaderCompileJobKey(NiagaraShaderType, EShaderPlatform::SP_PCD3D_SM5), EShaderCompileJobPriority::Normal);
			if (Job)
			{
				TArray<FShaderCommonCompileJobPtr> NewJobs;
				CompilationJob->ShaderCompileJob = Job;
				Input.ShaderFormat = VVMFormatName;
				if (GNiagaraSkipVectorVMBackendOptimizations != 0)
				{
					Input.Environment.CompilerFlags.Add(CFLAG_SkipOptimizations);
				}
				Job->Input = Input;
				NewJobs.Add(FShaderCommonCompileJobPtr(Job));

				GShaderCompilingManager->SubmitJobs(NewJobs, FString(), FString());
			}
			bJobScheduled = true;
		}
	}
	CompileResults.Data->LastHlslTranslation = TranslatedHLSL;

	if (!bJobScheduled)
	{

		CompileResults.Data->ByteCode.Reset();
		CompileResults.Data->Attributes.Empty();
		CompileResults.Data->Parameters.Empty();
		CompileResults.Data->InternalParameters.Empty();
		CompileResults.Data->DataInterfaceInfo.Empty();
		CompileResults.Data->UObjectInfos.Empty();

	}
	CompilationJob->CompileResults = CompileResults;

	return JobID;
}

uint32 FHlslNiagaraCompiler::CompileScriptVM(const FStringView GroupName, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, const FNiagaraTranslatorOutput& TranslatorOutput, const FString& TranslatedHLSL, FNiagaraShaderType* NiagaraShaderType)
{
	check(!InOptions.IsGpuScript() || !UNiagaraScript::IsParticleScript(InOptions.TargetUsage));

	CompileResults.Data = MakeShared<FNiagaraVMExecutableData>();

	CompileResults.bVMSucceeded = (TranslatorOutput.Errors.Len() == 0) && (TranslatedHLSL.Len() > 0) && !InTranslateResults.NumErrors;
	CompileResults.AppendCompileEvents(MakeArrayView(InTranslateResults.CompileEvents));
	CompileResults.Data->LastCompileEvents.Append(InTranslateResults.CompileEvents);
	CompileResults.Data->ExternalDependencies = InTranslateResults.CompileDependencies;
	CompileResults.Data->CompileTags = InTranslateResults.CompileTags;
	CompileResults.Data->CompileTagsEditorOnly = InTranslateResults.CompileTagsEditorOnly;
	CompileResults.Data->LastHlslTranslation = TranslatedHLSL;
	CompileResults.DumpDebugInfoPath.Reset();

	CompilationJob = MakeUnique<FNiagaraCompilerJob>();
	CompilationJob->TranslatorOutput = TranslatorOutput;
	CompilationJob->TranslatorOutput.ScriptData.LastHlslTranslation = TranslatedHLSL;
	CompilationJob->TranslatorOutput.ScriptData.ExternalDependencies = InTranslateResults.CompileDependencies;
	CompilationJob->TranslatorOutput.ScriptData.CompileTags = InTranslateResults.CompileTags;
	CompilationJob->TranslatorOutput.ScriptData.CompileTagsEditorOnly = InTranslateResults.CompileTagsEditorOnly;
	CompilationJob->StartTime = FPlatformTime::Seconds();
	CompilationJob->CompileResults = CompileResults;

	if (CompileResults.bVMSucceeded && NiagaraShaderType)
	{
		const uint32 JobID = FShaderCommonCompileJob::GetNextJobId();
		TRefCountPtr<FShaderCompileJob> Job = GShaderCompilingManager->PrepareShaderCompileJob(JobID, FShaderCompileJobKey(NiagaraShaderType, EShaderPlatform::SP_PCD3D_SM5), EShaderCompileJobPriority::Normal);
		if (Job)
		{
			Job->Input.Target = FShaderTarget(SF_Compute, SP_PCD3D_SM5);
			Job->Input.VirtualSourceFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf");
			Job->Input.EntryPointName = TEXT("SimulateMain");
			Job->Input.Environment.SetDefine(TEXT("VM_SIMULATION"), 1);
			Job->Input.Environment.SetDefine(TEXT("COMPUTESHADER"), 1);
			Job->Input.Environment.SetDefine(TEXT("PIXELSHADER"), 0);
			Job->Input.Environment.SetDefine(TEXT("DOMAINSHADER"), 0);
			Job->Input.Environment.SetDefine(TEXT("HULLSHADER"), 0);
			Job->Input.Environment.SetDefine(TEXT("VERTEXSHADER"), 0);
			Job->Input.Environment.SetDefine(TEXT("GEOMETRYSHADER"), 0);
			Job->Input.Environment.SetDefine(TEXT("MESHSHADER"), 0);
			Job->Input.Environment.SetDefine(TEXT("AMPLIFICATIONSHADER"), 0);
			Job->Input.Environment.IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/NiagaraEmitterInstance.ush"), TranslatedHLSL);
			Job->Input.DebugInfoFlags = GShaderCompilingManager->GetDumpShaderDebugInfoFlags();
			Job->Input.DumpDebugInfoRootPath = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory() / TEXT("VM");
			Job->Input.DumpDebugInfoPath = CompileResults.DumpDebugInfoPath;
			Job->Input.DebugGroupName = GroupName;
			Job->Input.DebugExtension.Empty();
			Job->Input.ShaderFormat = FName(TEXT("VVM_1_0"));

			//TODO: This is normally invoked by GlobalBeginCompileShader, which is not called in this path. Should it be?
			if (const IShaderFormat* VVMShaderFormat = GetTargetPlatformManagerRef().FindShaderFormat(Job->Input.ShaderFormat))
			{
				VVMShaderFormat->ModifyShaderCompilerInput(Job->Input);
			}

			if (GNiagaraSkipVectorVMBackendOptimizations != 0)
			{
				Job->Input.Environment.CompilerFlags.Add(CFLAG_SkipOptimizations);
			}

			if (GShaderCompilingManager->GetDumpShaderDebugInfo() == FShaderCompilingManager::EDumpShaderDebugInfo::Always)
			{
				Job->Input.DumpDebugInfoPath = GShaderCompilingManager->CreateShaderDebugInfoPath(Job->Input);
				CompileResults.DumpDebugInfoPath = Job->Input.DumpDebugInfoPath;
			}

			CompilationJob->ShaderCompileJob = Job;

			TArray<FShaderCommonCompileJobPtr> NewJobs;
			NewJobs.Emplace(Job);

			GShaderCompilingManager->SubmitJobs(NewJobs, FString(), FString());

			return JobID;
		}
	}

	return INDEX_NONE;
}

int32 FHlslNiagaraCompiler::CreateShaderIntermediateData(const FStringView GroupName, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, const FNiagaraTranslatorOutput& TranslatorOutput, const FString& TranslatedHLSL)
{
	check(InOptions.IsGpuScript() && UNiagaraScript::IsParticleScript(InOptions.TargetUsage));

	CompileResults.Data = MakeShared<FNiagaraVMExecutableData>();

	//TODO: This should probably be done via the same route that other shaders take through the shader compiler etc.
	//But that adds the complexity of a new shader type, new shader class and a new shader map to contain them etc.
	//Can do things simply for now.

	CompileResults.Data->LastHlslTranslation = TEXT("");

	FShaderCompilerInput Input;
	Input.Target = FShaderTarget(SF_Compute, SP_PCD3D_SM5);
	Input.VirtualSourceFilePath = TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf");
	Input.EntryPointName = TEXT("SimulateMain");
	Input.Environment.SetDefine(TEXT("VM_SIMULATION"), 1);
	Input.Environment.SetDefine(TEXT("COMPUTESHADER"), 1);
	Input.Environment.SetDefine(TEXT("PIXELSHADER"), 0);
	Input.Environment.SetDefine(TEXT("DOMAINSHADER"), 0);
	Input.Environment.SetDefine(TEXT("HULLSHADER"), 0);
	Input.Environment.SetDefine(TEXT("VERTEXSHADER"), 0);
	Input.Environment.SetDefine(TEXT("GEOMETRYSHADER"), 0);
	Input.Environment.SetDefine(TEXT("MESHSHADER"), 0);
	Input.Environment.SetDefine(TEXT("AMPLIFICATIONSHADER"), 0);
	Input.Environment.IncludeVirtualPathToContentsMap.Add(TEXT("/Engine/Generated/NiagaraEmitterInstance.ush"), TranslatedHLSL);
	Input.DebugInfoFlags = GShaderCompilingManager->GetDumpShaderDebugInfoFlags();
	Input.DumpDebugInfoRootPath = GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory() / TEXT("VM");
	Input.DebugGroupName = GroupName;
	Input.DebugExtension.Empty();
	Input.DumpDebugInfoPath.Empty();

	CompileResults.DumpDebugInfoPath = Input.DumpDebugInfoPath;

	uint32 JobID = FShaderCommonCompileJob::GetNextJobId();
	CompilationJob = MakeUnique<FNiagaraCompilerJob>();
	CompilationJob->TranslatorOutput = TranslatorOutput;

	CompileResults.bVMSucceeded = (CompilationJob->TranslatorOutput.Errors.Len() == 0) && (TranslatedHLSL.Len() > 0) && !InTranslateResults.NumErrors;

	// only issue jobs for VM compilation if we're going to be using the resulting byte code.  This excludes particle scripts when we're using
	// a GPU simulation
	CompileResults.bComputeSucceeded = false;
	if (CompileResults.bVMSucceeded)
	{
		//Clear out current contents of compile results.
		*(CompileResults.Data) = CompilationJob->TranslatorOutput.ScriptData;
		CompileResults.Data->ByteCode.Reset();
		CompileResults.bComputeSucceeded = true;
	}

	CompileResults.AppendCompileEvents(MakeArrayView(InTranslateResults.CompileEvents));
	CompileResults.Data->LastCompileEvents.Append(InTranslateResults.CompileEvents);
	CompileResults.Data->ExternalDependencies = InTranslateResults.CompileDependencies;
	CompileResults.Data->CompileTags = InTranslateResults.CompileTags;
	CompileResults.Data->CompileTagsEditorOnly = InTranslateResults.CompileTagsEditorOnly;

	// Early out if compiling a GPU particle script as we do not need to submit a CPU compile request.
	// This must be done after we add in the translator errors etc so tha they are passed to the compile job correctly.
	CompileResults.Data->LastHlslTranslationGPU = TranslatedHLSL;
	DumpDebugInfo(CompileResults, Input, true);
	CompilationJob->CompileResults = CompileResults;
	return JobID;
}

void FHlslNiagaraCompiler::FixupVMAssembly(FString& Asm)
{
	const TCHAR* OpTag = TEXT("__OP__");
	const TCHAR* OpDelimTags[] = { TEXT("("), TEXT(";") };
	const int32 OpTagLen = FCString::Strlen(OpTag);
	const int32 OpCount = VectorVM::GetNumOpCodes();
	constexpr int32 MaxDigitLength = 8;

	const FString OriginalAsm = MoveTemp(Asm);
	const FStringView OriginalAsmView(OriginalAsm);

	Asm.Reserve(OriginalAsm.Len());

	int32 SearchStartLocation = 0;
	while (SearchStartLocation != INDEX_NONE)
	{
		const int32 OpStartLocation = OriginalAsmView.Find(OpTag, SearchStartLocation);

		// append the string leading up to the OpCode
		const int32 SubStringLength = OpStartLocation == INDEX_NONE ? MAX_int32 : OpStartLocation - SearchStartLocation;
		if (SubStringLength)
		{
			Asm.Append(OriginalAsmView.Mid(SearchStartLocation, SubStringLength));
			SearchStartLocation = OpStartLocation;
		}

		if (OpStartLocation == INDEX_NONE)
		{
			break;
		}

		// find the next delimiter
		const int32 OpEndLocation = OpStartLocation + OpTagLen;
		FStringView DelimSearchView = OriginalAsmView.Mid(OpEndLocation, MaxDigitLength + 1);
		int32 DelimOffset = INDEX_NONE;

		for (const TCHAR* DelimTag : OpDelimTags)
		{
			int32 NextLocation = DelimSearchView.Find(DelimTag);
			if (NextLocation != INDEX_NONE && (DelimOffset == INDEX_NONE || NextLocation < DelimOffset))
			{
				DelimOffset = NextLocation;
			}
		}

		// check if the intervening spaces are digits (op index)
		int32 OpIndex = INDEX_NONE;

		if (DelimOffset != INDEX_NONE)
		{
			FStringView OpIndexStrView = OriginalAsmView.Mid(OpEndLocation, DelimOffset);
			if (OpIndexStrView.Len() <= MaxDigitLength)
			{
				TStringBuilder<MaxDigitLength + 1> OpIndexStr;
				OpIndexStr.Append(OpIndexStrView);
				if (!LexTryParseString<int32>(OpIndex, *OpIndexStr))
				{
					OpIndex = INDEX_NONE;
				}
			}
		}

		if (OpIndex != INDEX_NONE && OpIndex < OpCount)
		{
			Asm.Append(VectorVM::GetOpName(EVectorVMOp(OpIndex)));
			SearchStartLocation = OpEndLocation + DelimOffset;
		}
		else
		{
			Asm.Append(OriginalAsmView.Mid(OpStartLocation, OpTagLen));
			SearchStartLocation = OpEndLocation;
		}
	}
}

//TODO: Map Lines of HLSL to their source Nodes and flag those nodes with errors associated with their lines.
void FHlslNiagaraCompiler::DumpDebugInfo(const FNiagaraCompileResults& CompileResult, const FShaderCompilerInput& Input, bool bGPUScript)
{
	if (CompileResults.Data.IsValid())
	{
		// Support dumping debug info only on failure or warnings
		FString DumpDebugInfoPath = CompileResult.DumpDebugInfoPath;
		if (DumpDebugInfoPath.IsEmpty())
		{
			const FShaderCompilingManager::EDumpShaderDebugInfo DumpShaderDebugInfo = GShaderCompilingManager->GetDumpShaderDebugInfo();
			bool bDumpDebugInfo = false;
			if (DumpShaderDebugInfo == FShaderCompilingManager::EDumpShaderDebugInfo::OnError)
			{
				bDumpDebugInfo = !CompileResult.bVMSucceeded;
			}
			else if (DumpShaderDebugInfo == FShaderCompilingManager::EDumpShaderDebugInfo::OnErrorOrWarning)
			{
				bDumpDebugInfo = !CompileResult.bVMSucceeded || (CompileResult.NumErrors + CompileResult.NumWarnings) > 0;
			}

			if (bDumpDebugInfo)
			{
				DumpDebugInfoPath = GShaderCompilingManager->CreateShaderDebugInfoPath(Input);
			}
		}

		if (!DumpDebugInfoPath.IsEmpty())
		{
			FString ExportText = CompileResults.Data->LastHlslTranslation;
			FString ExportTextAsm = CompileResults.Data->LastAssemblyTranslation;
			if (bGPUScript)
			{
				ExportText = CompileResults.Data->LastHlslTranslationGPU;
				ExportTextAsm = "";
			}
			FString ExportTextParams;
			for (const FNiagaraVariable& Var : CompileResults.Data->Parameters.Parameters)
			{
				ExportTextParams += Var.ToString();
				ExportTextParams += "\n";
			}

			FNiagaraEditorUtilities::WriteTextFileToDisk(DumpDebugInfoPath, TEXT("NiagaraEmitterInstance.ush"), ExportText, true);
			FNiagaraEditorUtilities::WriteTextFileToDisk(DumpDebugInfoPath, TEXT("NiagaraEmitterInstance.asm"), ExportTextAsm, true);
			FNiagaraEditorUtilities::WriteTextFileToDisk(DumpDebugInfoPath, TEXT("NiagaraEmitterInstance.params"), ExportTextParams, true);
		}
	}
}

TOptional<FNiagaraCompileResults> FHlslNiagaraCompiler::GetCompileResult(int32 JobID, bool bWait /*= false*/)
{
	check(IsInGameThread());

	if (!CompilationJob)
	{
		return TOptional<FNiagaraCompileResults>();
	}
	if (!CompilationJob->ShaderCompileJob)
	{
		// In case we did not schedule any compile jobs but have a static result (e.g. in case of previous translator errors)
		FNiagaraCompileResults Results = CompilationJob->CompileResults;
		CompilationJob.Reset();
		return Results;
	}

	TArray<int32> ShaderMapIDs;
	ShaderMapIDs.Add(JobID);
	if (bWait && (CompilationJob->ShaderCompileJob->GetStatus() != EShaderCompileJobStatus::Released))
	{
		GShaderCompilingManager->FinishCompilation(NULL, ShaderMapIDs);
		check(CompilationJob->ShaderCompileJob->GetStatus() == EShaderCompileJobStatus::Released);
	}

	if (CompilationJob->ShaderCompileJob->GetStatus() != EShaderCompileJobStatus::Released)
	{
		return TOptional<FNiagaraCompileResults>();
	}
	else
	{
		// We do this because otherwise the shader compiling manager might still reference the deleted job at the end of this method.
		// The finalization flag is set by another thread, so the manager might not have had a change to process the result.
		GShaderCompilingManager->FinishCompilation(NULL, ShaderMapIDs);
	}

	FNiagaraCompileResults Results = CompilationJob->CompileResults;
	Results.bVMSucceeded = false;
	FVectorVMCompilationOutput CompilationOutput;
	if (CompilationJob->ShaderCompileJob->bSucceeded)
	{
		TConstArrayView<uint8> Code = CompilationJob->ShaderCompileJob->Output.ShaderCode.GetReadView();
		FShaderCodeReader ShaderCode(Code);
		FMemoryReaderView Ar(Code, true);
		Ar.SetLimitSize(ShaderCode.GetActualShaderCodeSize());
		Ar << CompilationOutput;

		if (!CompilationOutput.Errors.IsEmpty())
		{
			Warning(FText::Format(LOCTEXT("VectorVMCompileWarningMessageFormat", "The Vector VM compile generated warnings:\n{0}"), FText::FromString(CompilationOutput.Errors)));
		}

		Results.bVMSucceeded = true;
	}
	else if (CompilationJob->ShaderCompileJob->Output.Errors.Num() > 0)
	{
		TArray<FShaderCommonCompileJobPtr> FailedJobs;
		FailedJobs.Emplace(CompilationJob->ShaderCompileJob);

		const FShaderDiagnosticInfo ShaderDiagInfo(FailedJobs);
		if (ShaderDiagInfo.UniqueErrors.Num())
		{
			const FString AllUniqueErrors = FString::Join(ShaderDiagInfo.UniqueErrors, TEXT("\n"));

			UE_LOGFMT_LOC(LogNiagaraCompiler, Error, "VectorVMShaderCompilationFailed", "{NumErrors} errors encountered compiling Vector VM shaders:\n{Errors}",
				("NumErrors", ShaderDiagInfo.UniqueErrors.Num()),
				("Errors", AllUniqueErrors)
			);

			Error(FText::Format(LOCTEXT("VectorVMCompileErrorMessageFormat", "The Vector VM compile failed.  Errors:\n{0}"), FText::FromString(AllUniqueErrors)));

			if (UE_LOG_ACTIVE(LogNiagaraCompiler, Verbose))
			{
				DumpHLSLText(Results.Data->LastHlslTranslation, CompilationJob->CompileResults.DumpDebugInfoPath);
			}
		}
	}

	if (Results.bVMSucceeded)
	{
			//Build internal parameters
		SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslCompiler_CompileShader_VectorVMSucceeded);
		*Results.Data = CompilationJob->TranslatorOutput.ScriptData;
		Results.Data->ByteCode.SetData(CompilationOutput.ByteCode);
		Results.Data->NumTempRegisters = CompilationOutput.MaxTempRegistersUsed + 1;
		Results.Data->LastAssemblyTranslation = CompilationOutput.AssemblyAsString;
		FixupVMAssembly(Results.Data->LastAssemblyTranslation);
		Results.Data->LastOpCount = CompilationOutput.NumOps;

		if (GbForceNiagaraVMBinaryDump != 0 && Results.Data.IsValid())
		{
			DumpHLSLText(Results.Data->LastAssemblyTranslation, CompilationJob->CompileResults.DumpDebugInfoPath);
		}

		Results.Data->InternalParameters.Empty();
		bool bAllFloatsFinite = true;
		for (int32 i = 0; i < CompilationOutput.InternalConstantOffsets.Num(); ++i)
		{
			const FName ConstantName(TEXT("InternalConstant"), i);
			EVectorVMBaseTypes Type = CompilationOutput.InternalConstantTypes[i];
			int32 Offset = CompilationOutput.InternalConstantOffsets[i];
			switch (Type)
			{
			case EVectorVMBaseTypes::Float:
			{
				float Val = *(float*)(CompilationOutput.InternalConstantData.GetData() + Offset);
				Results.Data->InternalParameters.SetOrAdd(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), ConstantName))->SetValue(Val);
				bAllFloatsFinite &= FMath::IsFinite(Val);
			}
			break;
			case EVectorVMBaseTypes::Int:
			{
				int32 Val = *(int32*)(CompilationOutput.InternalConstantData.GetData() + Offset);
				Results.Data->InternalParameters.SetOrAdd(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), ConstantName))->SetValue(Val);
			}
			break;
			case EVectorVMBaseTypes::Bool:
			{
				int32 Val = *(int32*)(CompilationOutput.InternalConstantData.GetData() + Offset);
				Results.Data->InternalParameters.SetOrAdd(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), ConstantName))->SetValue(Val);
			}
			break;
			}
		}

		if (!bAllFloatsFinite)
		{
			Warning(LOCTEXT("FloatConstantsNanOrInf", "Float constant table contains NaN or Inf, this may result in invalid simulation results."));
		}

		Results.CompilerWallTime = (float)(FPlatformTime::Seconds() - CompilationJob->StartTime);
		Results.CompilerWorkerTime = (float)CompilationJob->ShaderCompileJob->Output.CompileTime;
		Results.CompilerPreprocessTime = (float)CompilationJob->ShaderCompileJob->Output.PreprocessTime;

		Results.Data->CalledVMExternalFunctions.Empty(CompilationOutput.CalledVMFunctionTable.Num());
		for (FCalledVMFunction& FuncInfo : CompilationOutput.CalledVMFunctionTable)
		{
			//Extract the external function call table binding info.
			const FNiagaraFunctionSignature* Sig = nullptr;
			for (FNiagaraScriptDataInterfaceCompileInfo& NDIInfo : CompilationJob->TranslatorOutput.ScriptData.DataInterfaceInfo)
			{
				Sig = NDIInfo.RegisteredFunctions.FindByPredicate([&](const FNiagaraFunctionSignature& CheckSig)
				{
					FString SigSymbol = FNiagaraHlslTranslator::GetFunctionSignatureSymbol(CheckSig);
					return SigSymbol == FuncInfo.Name;
				});
				if (Sig)
				{
					break;
				}
			}

			// Look in function library
			if (Sig == nullptr)
			{
				Sig = UNiagaraFunctionLibrary::GetVectorVMFastPathOps(true).FindByPredicate(
					[&](const FNiagaraFunctionSignature& CheckSig)
				{
					FString SigSymbol = FNiagaraHlslTranslator::GetFunctionSignatureSymbol(CheckSig);
					return SigSymbol == FuncInfo.Name;
				}
				);
			}

			if (Sig)
			{
				FVMExternalFunctionBindingInfo& NewBinding = Results.Data->CalledVMExternalFunctions.AddDefaulted_GetRef();
				NewBinding.Name = Sig->Name;
				NewBinding.OwnerName = Sig->OwnerName;
				NewBinding.InputParamLocations = FuncInfo.InputParamLocations;
				NewBinding.NumOutputs = FuncInfo.NumOutputs;
				for (auto it = Sig->FunctionSpecifiers.CreateConstIterator(); it; ++it)
				{
					// we convert the map into an array here to save runtime memory
					NewBinding.FunctionSpecifiers.Emplace(it->Key, it->Value);
				}

				//Write out our variadic parameters to allow proper binding for VM external functions.
				Sig->GetVariadicInputs(NewBinding.VariadicInputs);
				Sig->GetVariadicOutputs(NewBinding.VariadicOutputs);
			}
			else
			{
				Error(FText::Format(LOCTEXT("VectorVMExternalFunctionBindingError", "Failed to bind the external function call:  {0}"), FText::FromString(FuncInfo.Name)));
				Results.bVMSucceeded = false;
			}
		}
	}

	if (!Results.bVMSucceeded)
	{
		Results.Data->ByteCode.Reset();
		Results.Data->Attributes.Empty();
		Results.Data->Parameters.Empty();
		Results.Data->InternalParameters.Empty();
		Results.Data->DataInterfaceInfo.Empty();
		Results.Data->UObjectInfos.Empty();
	}

	DumpDebugInfo(CompileResults, CompilationJob->ShaderCompileJob->Input, false);

	//Seems like Results is a bit of a cobbled together mess at this point.
	//Ideally we can tidy this up in future.
	//Doing this as a minimal risk free fix for not having errors passed through into the compile results.
	Results.NumErrors = CompileResults.NumErrors;
	Results.CompileEvents = CompileResults.CompileEvents;
	Results.Data->CompileTags = CompileResults.Data->CompileTags;
	Results.Data->CompileTagsEditorOnly = CompileResults.Data->CompileTagsEditorOnly;

	CompilationJob.Reset();
	return Results;
}

FHlslNiagaraCompiler::FHlslNiagaraCompiler()
	: CompileResults()
{
}



void FHlslNiagaraCompiler::Error(FText ErrorText)
{
	FString ErrorString = FString::Printf(TEXT("%s"), *ErrorText.ToString());
	CompileResults.Data->LastCompileEvents.Add(FNiagaraCompileEvent(FNiagaraCompileEventSeverity::Error, ErrorString));
	CompileResults.CompileEvents.Add(FNiagaraCompileEvent(FNiagaraCompileEventSeverity::Error, ErrorString));
	CompileResults.NumErrors++;
}

void FHlslNiagaraCompiler::Warning(FText WarningText)
{
	FString WarnString = FString::Printf(TEXT("%s"), *WarningText.ToString());
	CompileResults.Data->LastCompileEvents.Add(FNiagaraCompileEvent(FNiagaraCompileEventSeverity::Warning, WarnString));
	CompileResults.CompileEvents.Add(FNiagaraCompileEvent(FNiagaraCompileEventSeverity::Warning, WarnString));
	CompileResults.NumWarnings++;
}

FNiagaraShaderMapCompiler::FNiagaraShaderMapCompiler(
	const FNiagaraShaderType* InShaderType,
	TSharedPtr<FNiagaraShaderScriptParametersMetadata> InShaderParameters)
	: ShaderType(InShaderType)
	, ShaderParameters(InShaderParameters)
{
}

void FNiagaraShaderMapCompiler::AddShaderPlatform(const FNiagaraShaderMapId& ShaderMapId, EShaderPlatform ShaderPlatform)
{
	FActiveCompilation& ActiveCompilation = ActiveCompilations.AddDefaulted_GetRef();
	ActiveCompilation.ShaderMapId = ShaderMapId;
	ActiveCompilation.ShaderPlatform = ShaderPlatform;
	ActiveCompilation.ShaderMap = new FNiagaraShaderMap(FNiagaraShaderMap::WorkerThread);
}

void FNiagaraShaderMapCompiler::CompileScript(
	const FNiagaraVMExecutableDataId& ScriptCompileId,
	const FStringView SourceName,
	const FStringView DebugGroupName,
	const FNiagaraCompileOptions& CompileOptions,
	const FNiagaraTranslateResults& TranslateResults,
	const FNiagaraTranslatorOutput& TranslatorOutput,
	const FString& TranslatedHLSL,
	TConstArrayView<UNiagaraDataInterface*> DataInterfaces)
{
	TArray<TRefCountPtr<FShaderCommonCompileJob>> CompileJobs;

	for (FActiveCompilation& ActiveCompilation : ActiveCompilations)
	{
		TRefCountPtr<FSharedShaderCompilerEnvironment> CompilationEnvironment = new FSharedShaderCompilerEnvironment();
		CompilationEnvironment->SetDefine(TEXT("GPU_SIMULATION_SHADER"), TEXT("1"));
		CompilationEnvironment->SetDefine(TEXT("NIAGARA_COMPRESSED_ATTRIBUTES_ENABLED"),
			ScriptCompileId.AdditionalDefines.Contains(TEXT("CompressAttributes")) ? 1 : 0);

		// Fast math breaks The ExecGrid layout script because floor(x/y) returns a bad value if x == y. Yay.
		if (IsMetalPlatform(ActiveCompilation.ShaderPlatform))
		{
			CompilationEnvironment->CompilerFlags.Add(CFLAG_NoFastMath);
		}

		for (UNiagaraDataInterface* DataInterface : DataInterfaces)
		{
			DataInterface->ModifyCompilationEnvironment(ActiveCompilation.ShaderPlatform, *CompilationEnvironment.GetReference());
		}

		ActiveCompilation.ShaderMap->CreateCompileJobs(
			ShaderType,
			DebugGroupName,
			ActiveCompilation.ShaderMapId,
			TranslatedHLSL,
			CompilationEnvironment,
			TranslatorOutput.ScriptData.SimulationStageMetaData,
			TranslatorOutput.ScriptData.SimulationStageMetaData.Num(),
			ActiveCompilation.ShaderPlatform,
			ShaderParameters,
			ActiveCompilation.ShaderCompileJobs
		);

		CompileJobs.Append(ActiveCompilation.ShaderCompileJobs);
	}

	// we also need to populate the ExeData for the script based on the translator results.  This handles all the meta data of the script.
	ScriptExeData = MakeShared<FNiagaraVMExecutableData>(TranslatorOutput.ScriptData);
	ScriptExeData->LastCompileEvents.Append(TranslateResults.CompileEvents);
	ScriptExeData->ExternalDependencies = TranslateResults.CompileDependencies;
	ScriptExeData->CompileTags = TranslateResults.CompileTags;
	ScriptExeData->CompileTagsEditorOnly = TranslateResults.CompileTagsEditorOnly;
	ScriptExeData->LastHlslTranslationGPU = TranslatedHLSL;

	GShaderCompilingManager->SubmitJobs(CompileJobs, FString(SourceName));
}

bool FNiagaraShaderMapCompiler::ProcessCompileResults(bool bWait)
{
	check(!bWait); // not currently implemented
	check(IsInGameThread());

	for (TArray<FActiveCompilation>::TIterator CompileIt = ActiveCompilations.CreateIterator(); CompileIt; ++CompileIt)
	{
		auto IsCompileJobIncomplete = [](const FShaderCommonCompileJobPtr& CompileJob) -> bool
		{
			return !CompileJob.IsValid() || (CompileJob->GetStatus() != EShaderCompileJobStatus::Released);
		};

		auto IsCompileJobError = [](const FShaderCommonCompileJobPtr& CompileJob) -> bool
		{
			return !CompileJob->bSucceeded;
		};

		// make sure that all of the shader compile jobs have been released 
		const bool bReadyToProcess = !CompileIt->ShaderCompileJobs.ContainsByPredicate(IsCompileJobIncomplete);

		if (!bReadyToProcess)
		{
			// todo - it might be worth keeping track of jobs that aren't getting handled because of the above
			// condition.  Either it's taking a long time and so it could be worth reporting, or because the
			// job is lost and we'll never complete.
			continue;
		}

		FActiveCompilation& CurrentCompilation = *CompileIt;
		FCompletedCompilation& CompletedCompilation = CompletedCompilations.AddDefaulted_GetRef();

		// do a first pass over all of the ShaderCompileJobs to see if any of them failed.  If it did, then we don't need
		// to worry about pushing out our incomplete ShaderMap and we should just report the errors
		const bool bSuccessfulCompilation = !CurrentCompilation.ShaderCompileJobs.ContainsByPredicate(IsCompileJobError);

		if (bSuccessfulCompilation)
		{
			// for now we'll process all shaders at once (need to measure the cost here)
			for (const FShaderCommonCompileJobPtr& ShaderCompileJob : CurrentCompilation.ShaderCompileJobs)
			{
				CurrentCompilation.ShaderMap->ProcessAndFinalizeShaderCompileJob(ShaderCompileJob);
			}
		}
		else
		{
			CurrentCompilation.ShaderMap->SetCompiledSuccessfully(false);
		}

		// pass on error/warning info
		for (const FShaderCommonCompileJobPtr& ShaderCompileJob : CurrentCompilation.ShaderCompileJobs)
		{
			if (const FShaderCompileJob* SingleShaderJob = ShaderCompileJob->GetSingleShaderJob())
			{
				CompletedCompilation.CompilationErrors.Append(SingleShaderJob->Output.Errors);
			}
		}

		// now that we've added all the results into the shader map we can move it over to CompletedCompilations
		CompletedCompilation.ShaderMap = CurrentCompilation.ShaderMap;

		// and remove it from the ActiveCompilations
		CompileIt.RemoveCurrentSwap();
	}

	return ActiveCompilations.IsEmpty();
}

bool FNiagaraShaderMapCompiler::GetShaderMap(const FNiagaraShaderMapId& ShaderMapId, FNiagaraShaderMapRef& OutShaderMap, TArray<FShaderCompilerError>& OutCompilationErrors) const
{
	for (const FCompletedCompilation& CompletedCompilation : CompletedCompilations)
	{
		if (CompletedCompilation.ShaderMap->GetShaderMapId() == ShaderMapId)
		{
			OutShaderMap = CompletedCompilation.ShaderMap;
			OutCompilationErrors = CompletedCompilation.CompilationErrors;
			return true;
		}
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
