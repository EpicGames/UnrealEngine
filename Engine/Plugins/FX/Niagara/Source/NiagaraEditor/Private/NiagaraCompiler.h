// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraHlslTranslator.h"
#include "INiagaraCompiler.h"
#include "ShaderCompiler.h"

class Error;
class UNiagaraGraph;
class UNiagaraNode;
class UNiagaraNodeFunctionCall;
class UNiagaraScript;
class UNiagaraScriptSource;
struct FNiagaraTranslatorOutput;

struct FNiagaraCompilerJob
{
	TRefCountPtr<FShaderCompileJob> ShaderCompileJob;
	FNiagaraCompileResults CompileResults;
	double StartTime;
	FNiagaraTranslatorOutput TranslatorOutput;

	FNiagaraCompilerJob()
	{
		StartTime = FPlatformTime::Seconds();
	}
};

class FHlslNiagaraCompiler : public INiagaraCompiler
{
protected:	
	/** Captures information about a script compile. */
	FNiagaraCompileResults CompileResults;

public:

	NIAGARAEDITOR_API FHlslNiagaraCompiler();
	virtual ~FHlslNiagaraCompiler() = default;

	//Begin INiagaraCompiler Interface
	UE_DEPRECATED(5.4, "Please update to supply the GroupName directly as this will be removed in a future version.")
	NIAGARAEDITOR_API virtual int32 CompileScript(const class FNiagaraCompileRequestData* InCompileRequest, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, FNiagaraTranslatorOutput* TranslatorOutput, FString& TranslatedHLSL) override;

	NIAGARAEDITOR_API virtual int32 CompileScript(const FStringView GroupName, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, const FNiagaraTranslatorOutput& TranslatorOutput, const FString& TranslatedHLSL) override;
	NIAGARAEDITOR_API virtual uint32 CompileScriptVM(const FStringView GroupName, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, const FNiagaraTranslatorOutput& TranslatorOutput, const FString& TranslatedHLSL, FNiagaraShaderType* NiagaraShaderType);
	NIAGARAEDITOR_API virtual int32 CreateShaderIntermediateData(const FStringView GroupName, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, const FNiagaraTranslatorOutput& TranslatorOutput, const FString& TranslatedHLSL);
	virtual TOptional<FNiagaraCompileResults> GetCompileResult(int32 JobID, bool bWait = false) override;

	NIAGARAEDITOR_API virtual void Error(FText ErrorText) override;
	NIAGARAEDITOR_API virtual void Warning(FText WarningText) override;

private:
	TUniquePtr<FNiagaraCompilerJob> CompilationJob;

	NIAGARAEDITOR_API void DumpDebugInfo(const FNiagaraCompileResults& CompileResult, const FShaderCompilerInput& Input, bool bGPUScript);

	/** SCW doesn't have access to the VM op code names so we do a fixup pass to make these human readable after we get the data back from SCW. */
	NIAGARAEDITOR_API void FixupVMAssembly(FString& Asm);
};

class FNiagaraShaderMapCompiler
{
public:
	FNiagaraShaderMapCompiler(
		const FNiagaraShaderType* InShaderType,
		TSharedPtr<FNiagaraShaderScriptParametersMetadata> InShaderParameters);

	FNiagaraShaderMapCompiler() = delete;

	void CompileScript(
		const FNiagaraVMExecutableDataId& ScriptCompileId,
		FStringView SourceName,
		FStringView GroupName,
		const FNiagaraCompileOptions& InOptions,
		const FNiagaraTranslateResults& InTranslateResults,
		const FNiagaraTranslatorOutput& TranslatorOutput,
		const FString& TranslatedHLSL,
		TConstArrayView<UNiagaraDataInterface*> DataInterfaces);
	void AddShaderPlatform(const FNiagaraShaderMapId& ShaderMapId, EShaderPlatform ShaderPlatform);

	// returns true if there are no more results requiring processing
	bool ProcessCompileResults(bool bWait = false);

	struct FCompletedCompilation
	{
		FNiagaraShaderMapRef ShaderMap;
		TArray<FShaderCompilerError> CompilationErrors;
	};

	TConstArrayView<FCompletedCompilation> ReadCompletedCompilations() const
	{
		return CompletedCompilations;
	}

	bool GetShaderMap(const FNiagaraShaderMapId& ShaderMapId, FNiagaraShaderMapRef& OutShaderMap, TArray<FShaderCompilerError>& OutCompilationErrors) const;

	TSharedPtr<FNiagaraVMExecutableData> ReadScriptMetaData() const
	{
		return ScriptExeData;
	}

	TSharedPtr<FNiagaraShaderScriptParametersMetadata> GetShaderParameters() const
	{
		return ShaderParameters;
	}

private:
	struct FActiveCompilation
	{
		FNiagaraShaderMapId ShaderMapId;
		EShaderPlatform ShaderPlatform;
		FNiagaraShaderMapRef ShaderMap;
		TArray<FShaderCommonCompileJobPtr> ShaderCompileJobs;
	};

	const FNiagaraShaderType* ShaderType = nullptr;
	TSharedPtr<FNiagaraShaderScriptParametersMetadata> ShaderParameters;
	TArray<FActiveCompilation> ActiveCompilations;
	TArray<FCompletedCompilation> CompletedCompilations;
	TSharedPtr<FNiagaraVMExecutableData> ScriptExeData;
};
