// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentAssetBroker.h"
#include "NiagaraComponent.h"
#include "NiagaraSystem.h"

class FNiagaraComponentBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override
	{
		return UNiagaraSystem::StaticClass();
	}

	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
	{
		if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(InComponent))
		{
			if (UNiagaraSystem* NiagaraSystem = Cast<UNiagaraSystem>(InAsset))
			{
				NiagaraComponent->SetAsset(NiagaraSystem);
				return true;
			}
		}
		return false;
	}

	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
	{
		if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(InComponent))
		{
			return NiagaraComponent->GetFXSystemAsset();
		}
		return nullptr;
	}
};