// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraConvertInPlaceEmitterAndSystemState.h"
#include "NiagaraClipboard.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackScriptHierarchyRoot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraConvertInPlaceEmitterAndSystemState)

bool UNiagaraConvertInPlaceEmitterAndSystemState::Convert(UNiagaraScript* InOldScript, UNiagaraClipboardContent* InOldClipboardContent, UNiagaraScript* InNewScript, UNiagaraStackScriptHierarchyRoot* InHierarchyRoot, UNiagaraClipboardContent* InNewClipboardContent, UNiagaraNodeFunctionCall* InCallingNode, FText& OutMessage)
{
	UE_LOGF(LogNiagaraEditor, Log, "%ls to %ls", *InOldScript->GetPathName(), *InNewScript->GetPathName());

	if (InOldScript->GetPathName() == TEXT("/Niagara/Modules/Emitter/EmitterLifeCycle.EmitterLifeCycle") && InNewScript->GetPathName() == TEXT("/Niagara/Modules/Emitter/EmitterState.EmitterState"))
	{
		if (InOldClipboardContent->Functions.Num() == 1 && InNewClipboardContent->Functions.Num() == 1 &&
			InOldClipboardContent->Functions[0] && InNewClipboardContent->Functions[0])
		{
			int32 OldMaxLoopCount = 0;
			const UNiagaraClipboardFunctionInput* InputOldLoopDuration = nullptr;
			const UNiagaraClipboardFunctionInput* InputOldMaxLoopCount = nullptr;
			const UNiagaraClipboardFunctionInput* InputOldDurationRecalcEachLoop = nullptr;
			const UNiagaraClipboardFunctionInput* InputOldNextLoopDelay = nullptr;
			const UNiagaraClipboardFunctionInput* InputOldDelayFirstLoopOnly = nullptr;
			int32 AutoComplete = 0;
			int32 CompleteOnInactive = 0;
			bool bAutoCompleteWasLocal = false;
			bool bCompleteOnInactiveWasLocal = false;

			for (const UNiagaraClipboardFunctionInput* Input : InOldClipboardContent->Functions[0]->Inputs)
			{
				if (Input)
				{
					UE_LOGF(LogNiagaraEditor, Log, "OldInput \"%ls\"", *Input->InputName.ToString());

					bool bSuccess = false;
					if (Input->InputName == TEXT("MaxLoopCount"))
					{
						InputOldMaxLoopCount = Input;
						UNiagaraClipboardEditorScriptingUtilities::TryGetLocalValueAsInt(Input, bSuccess, OldMaxLoopCount);
					}
					else if (Input->InputName == TEXT("NextLoopDuration"))
					{
						InputOldLoopDuration = Input;
					}
					else if (Input->InputName == TEXT("DurationRecalcEachLoop"))
					{
						InputOldDurationRecalcEachLoop = Input;
					}
					else if (Input->InputName == TEXT("NextLoopDelay"))
					{
						InputOldNextLoopDelay = Input;
					}
					else if (Input->InputName == TEXT("AutoComplete"))
					{
						UNiagaraClipboardEditorScriptingUtilities::TryGetLocalValueAsInt(Input, bAutoCompleteWasLocal, AutoComplete);
					}
					else if (Input->InputName == TEXT("CompleteOnInactive"))
					{
						UNiagaraClipboardEditorScriptingUtilities::TryGetLocalValueAsInt(Input, bCompleteOnInactiveWasLocal, CompleteOnInactive);
					}
					else if (Input->InputName == TEXT("DelayFirstLoopOnly"))
					{
						InputOldDelayFirstLoopOnly = Input;						 
					}
				}
			}

			TArray<const UNiagaraClipboardFunctionInput*> NewInputs;
			bool bMultipleLoopsFound = false;
			bool bUseLoopDelay = false;
			TArray<FName> ConvertedItems;
			
			for (const UNiagaraClipboardFunctionInput* Input : InNewClipboardContent->Functions[0]->Inputs)
			{
				if (Input)
				{
					UE_LOGF(LogNiagaraEditor, Log, "New Input \"%ls\"", *Input->InputName.ToString());
					bool bSuccess = false;
					if (Input->InputName == TEXT("Life Cycle Mode"))
					{
						// When converting old Emitter Life Cycle, always assume that new type is Self
						UNiagaraClipboardFunctionInput* NewInput = Cast< UNiagaraClipboardFunctionInput>(StaticDuplicateObject(Input, GetTransientPackage()));
						UNiagaraClipboardEditorScriptingUtilities::TrySetLocalValueAsInt(NewInput, bSuccess, 1);
						NewInputs.Add(NewInput);
						if (bSuccess)
							ConvertedItems.Add(Input->InputName);
					}
					else if (Input->InputName == TEXT("Loop Behavior"))
					{
						UNiagaraClipboardFunctionInput* NewInput = Cast< UNiagaraClipboardFunctionInput>(StaticDuplicateObject(Input, GetTransientPackage()));
						int32 NewValue = -1;
						if (InputOldMaxLoopCount && InputOldMaxLoopCount->ValueMode == ENiagaraClipboardFunctionInputValueMode::Local)
						{
							if (OldMaxLoopCount == 0)
								NewValue = 0; // Infinite
							else if (OldMaxLoopCount == 1)
								NewValue = 1; // Once
							else if (OldMaxLoopCount > 1)
								NewValue = 2; // Multiple
						}
						
						// Force to multiple if anything other than local
						if (NewValue < 0)
							NewValue = 2; // Multiple

						if (NewValue == 2)
							bMultipleLoopsFound = true;

						UNiagaraClipboardEditorScriptingUtilities::TrySetLocalValueAsInt(NewInput, bSuccess, NewValue);
						NewInputs.Add(NewInput);
						if (bSuccess)
							ConvertedItems.Add(Input->InputName);
					}
					else if (Input->InputName == TEXT("Inactive Response"))
					{

						UNiagaraClipboardFunctionInput* NewInput = Cast< UNiagaraClipboardFunctionInput>(StaticDuplicateObject(Input, GetTransientPackage()));
						NewInputs.Add(NewInput);
						
						int32 NewValue = -1;
						if (AutoComplete && bAutoCompleteWasLocal)
							NewValue = 0; // Complete
						else if (CompleteOnInactive && bCompleteOnInactiveWasLocal)
							NewValue = 1; // Kill

						// Force to complete if anything else
						if (NewValue < 0)
							NewValue = 0; // Complete

						UNiagaraClipboardEditorScriptingUtilities::TrySetLocalValueAsInt(NewInput, bSuccess, NewValue);
						if (bSuccess)
							ConvertedItems.Add(Input->InputName);
					}
					else if (Input->InputName == TEXT("Loop Duration"))
					{
						if (InputOldLoopDuration)
						{
							UNiagaraClipboardFunctionInput* NewInput = Cast< UNiagaraClipboardFunctionInput>(StaticDuplicateObject(InputOldLoopDuration, GetTransientPackage()));
							NewInput->InputName = Input->InputName;
							NewInputs.Add(NewInput);
							ConvertedItems.Add(Input->InputName);
						}
					}
					else if (Input->InputName == TEXT("Recalculate Duration Each Loop"))
					{
						if (InputOldDurationRecalcEachLoop)
						{
							UNiagaraClipboardFunctionInput* NewInput = Cast< UNiagaraClipboardFunctionInput>(StaticDuplicateObject(InputOldDurationRecalcEachLoop, GetTransientPackage()));
							NewInput->InputName = Input->InputName;
							NewInputs.Add(NewInput);
							ConvertedItems.Add(Input->InputName);
						}
					}
					else if (Input->InputName == TEXT("Loop Delay"))
					{
						if (InputOldNextLoopDelay)
						{
							UNiagaraClipboardFunctionInput* NewInput = Cast< UNiagaraClipboardFunctionInput>(StaticDuplicateObject(InputOldNextLoopDelay, GetTransientPackage()));
							NewInput->InputName = Input->InputName;
							NewInputs.Add(NewInput);
							ConvertedItems.Add(Input->InputName);
						}
					}
					else if (Input->InputName == TEXT("UseLoopDelay"))
					{
						if (InputOldNextLoopDelay)
						{
							float OutFloat = 0.0f;
							bool bLoopDelayFound = false;
							UNiagaraClipboardEditorScriptingUtilities::TryGetLocalValueAsFloat(InputOldNextLoopDelay, bLoopDelayFound, OutFloat);
							if ((bLoopDelayFound && OutFloat != 0.0f) || InputOldNextLoopDelay->ValueMode != ENiagaraClipboardFunctionInputValueMode::Local)
							{
								bUseLoopDelay = true;
								int32 NewValue = -1;
								UNiagaraClipboardFunctionInput* NewInput = Cast< UNiagaraClipboardFunctionInput>(StaticDuplicateObject(Input, GetTransientPackage())); UNiagaraClipboardEditorScriptingUtilities::TrySetLocalValueAsInt(NewInput, bSuccess, NewValue);
								NewInputs.Add(NewInput);
								if (bSuccess)
									ConvertedItems.Add(Input->InputName);
							}
						}
					}
				}
			}

			InHierarchyRoot->SetValuesFromClipboardFunctionInputs(NewInputs);
			//InCallingNode->RefreshFromExternalChanges();

			// We have to do this as a second pass as Loop Count is behind a static switch set above.
			NewInputs.Empty();
			if (InputOldMaxLoopCount && bMultipleLoopsFound)
			{
				UNiagaraClipboardFunctionInput* NewInput = Cast< UNiagaraClipboardFunctionInput>(StaticDuplicateObject(InputOldMaxLoopCount, GetTransientPackage()));
				NewInput->InputName = TEXT("Loop Count");
				NewInputs.Add(NewInput);
				ConvertedItems.Add(NewInput->InputName);

				InHierarchyRoot->SetValuesFromClipboardFunctionInputs(NewInputs);
			}

			// Use first frame delay is behind a static switch set above.
			NewInputs.Empty();
			if (bUseLoopDelay && InputOldDelayFirstLoopOnly)
			{
				UNiagaraClipboardFunctionInput* NewInput = Cast< UNiagaraClipboardFunctionInput>(StaticDuplicateObject(InputOldDelayFirstLoopOnly, GetTransientPackage()));
				NewInput->InputName = TEXT("Delay First Loop Only");
				NewInputs.Add(NewInput);
				ConvertedItems.Add(NewInput->InputName);

				InHierarchyRoot->SetValuesFromClipboardFunctionInputs(NewInputs);
			}

			FString ConcatString = FString::Printf(TEXT("Copied over %d properties\nPlease double-check for correctness\n"), ConvertedItems.Num());
			for (int32 i = 0; i < ConvertedItems.Num(); i++)
			{
				ConcatString += ConvertedItems[i].ToString();

				if (i < ConvertedItems.Num() - 1)
					ConcatString += TEXT(",\n");
				else
					ConcatString += TEXT("\n");

			}
			OutMessage = FText::FromString(ConcatString);
			if (ConvertedItems.Num() > 0)
				return true;
			else
				return false;
		}
	}
	return false;
};
