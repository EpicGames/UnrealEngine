// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraConvertInPlaceUtilityBase.h"
#include "NiagaraScript.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraConvertInPlaceEmitterAndSystemState.generated.h"

UCLASS(MinimalAPI)
class  UNiagaraConvertInPlaceEmitterAndSystemState : public UNiagaraConvertInPlaceUtilityBase
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API virtual bool Convert(UNiagaraScript* InOldScript, UNiagaraClipboardContent* InOldClipboardContent, UNiagaraScript* InNewScript, UNiagaraStackScriptHierarchyRoot* InHierarchyRoot, UNiagaraClipboardContent* InNewClipboardContent, UNiagaraNodeFunctionCall* InCallingNode, FText& OutMessage) override;
};
