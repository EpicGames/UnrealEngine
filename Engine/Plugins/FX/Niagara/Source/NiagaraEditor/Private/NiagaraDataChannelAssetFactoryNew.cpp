// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelAssetFactoryNew.h"
#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "NiagaraSettings.h"
#include "NiagaraDataChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelAssetFactoryNew)

UNiagaraDataChannelAssetFactoryNew::UNiagaraDataChannelAssetFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UNiagaraDataChannelAsset::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
}

UObject* UNiagaraDataChannelAssetFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UNiagaraDataChannelAsset::StaticClass()));

	UNiagaraDataChannelAsset* NewChannel = NewObject<UNiagaraDataChannelAsset>(InParent, Class, Name, Flags | RF_Transactional);

	return NewChannel;
}

bool UNiagaraDataChannelAssetFactoryNew::CanCreateNew()const
{
	return INiagaraModule::DataChannelsEnabled();
}
