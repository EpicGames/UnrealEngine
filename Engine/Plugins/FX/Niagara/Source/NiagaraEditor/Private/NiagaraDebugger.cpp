// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDebugger.h"
#include "Modules/ModuleManager.h"
#include "NiagaraEditorModule.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "NiagaraDebuggerCommon.h"
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
#include "ISessionServicesModule.h"
#endif
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

#include "Framework/Application/SlateApplication.h"

#if WITH_NIAGARA_DEBUGGER

#define LOCTEXT_NAMESPACE "NiagaraDebugger"

DEFINE_LOG_CATEGORY(LogNiagaraDebugger);

FNiagaraDebugger::FNiagaraDebugger()
{
}

FNiagaraDebugger::~FNiagaraDebugger()
{
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
	if (SessionManager.IsValid())
	{
		SessionManager->OnSelectedSessionChanged().RemoveAll(this);
		SessionManager->OnInstanceSelectionChanged().RemoveAll(this);
	}
#endif

	GetMutableDefault<UNiagaraDebugHUDSettings>()->OnChangedDelegate.RemoveAll(this);
	GetMutableDefault<UNiagaraOutliner>()->OnChangedDelegate.RemoveAll(this);
}

void FNiagaraDebugger::Init()
{
	MessageEndpoint = FMessageEndpoint::Builder("SNiagaraDebugger")
		.Handling<FNiagaraDebuggerAcceptConnection>(this, &FNiagaraDebugger::HandleConnectionAcceptedMessage)
		.Handling<FNiagaraDebuggerConnectionClosed>(this, &FNiagaraDebugger::HandleConnectionClosedMessage)
		.Handling<FNiagaraDebuggerOutlinerUpdate>(this, &FNiagaraDebugger::HandleOutlinerUpdateMessage)
		.Handling<FNiagaraSimpleClientInfo>(this, &FNiagaraDebugger::UpdateSimpleClientInfo)
		.Handling<FNiagaraSystemSimCacheCaptureReply>(this, &FNiagaraDebugger::HandleSimCacheCaptureReply);

#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
	ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
	SessionManager = SessionServicesModule.GetSessionManager();

	if (SessionManager.IsValid())
	{
		SessionManager->OnSelectedSessionChanged().AddSP(this, &FNiagaraDebugger::SessionManager_OnSessionSelectionChanged);
		SessionManager->OnInstanceSelectionChanged().AddSP(this, &FNiagaraDebugger::SessionManager_OnInstanceSelectionChanged);
	}
#endif

	GetMutableDefault<UNiagaraDebugHUDSettings>()->OnChangedDelegate.AddSP(this, &FNiagaraDebugger::UpdateDebugHUDSettings);
	GetMutableDefault<UNiagaraOutliner>()->OnChangedDelegate.AddSP(this, &FNiagaraDebugger::TriggerOutlinerCapture);
}

void FNiagaraDebugger::ExecConsoleCommand(const TCHAR* Cmd, bool bRequiresWorld)
{
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS

	auto SendExecCommand = [&](FNiagaraDebugger::FClientInfo& Client)
	{
		UE_LOGF(LogNiagaraDebugger, Log, "Sending console command %ls. | Session: %ls | Instance: %ls |", Cmd, *Client.SessionId.ToString(), *Client.InstanceId.ToString());
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FNiagaraDebuggerExecuteConsoleCommand>(Cmd, bRequiresWorld), EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Client.Address), FTimespan::Zero(), FDateTime::MaxValue());
	};

	ForAllConnectedClients(SendExecCommand);

#else

	//Session/Messaging system is not available so just send to local client.
	if (INiagaraDebuggerClient* LocalClient = INiagaraDebuggerClient::Get())
	{
		LocalClient->ExecConsoleCommand(FNiagaraDebuggerExecuteConsoleCommand(Cmd, bRequiresWorld));
	}

#endif//WITH_UNREAL_TARGET_DEVELOPER_TOOLS
}

void FNiagaraDebugger::UpdateDebugHUDSettings()
{
	//Send the current state as a message to all connected clients.
	if (const UNiagaraDebugHUDSettings* Settings = GetDefault<UNiagaraDebugHUDSettings>())
	{
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS

		auto SendSettingsUpdate = [&](FNiagaraDebugger::FClientInfo& Client)
		{
			//Create the message and copy the current state of the settings into it.
			FNiagaraDebugHUDSettingsData* Message = FMessageEndpoint::MakeMessage<FNiagaraDebugHUDSettingsData>();
			FNiagaraDebugHUDSettingsData::StaticStruct()->CopyScriptStruct(Message, &Settings->Data);

			UE_LOGF(LogNiagaraDebugger, Log, "Sending updated debug HUD settings. | Session: %ls | Instance: %ls |", *Client.SessionId.ToString(), *Client.InstanceId.ToString());
			MessageEndpoint->Send(Message, EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Client.Address), FTimespan::Zero(), FDateTime::MaxValue());
		};

		ForAllConnectedClients(SendSettingsUpdate);

#else

		//Session/Messaging system is not available so just send to local client.
		if (INiagaraDebuggerClient* LocalClient = INiagaraDebuggerClient::Get())
		{
			LocalClient->UpdateDebugHUDSettings(Settings->Data);
		}

#endif//WITH_UNREAL_TARGET_DEVELOPER_TOOLS
	}
}

void FNiagaraDebugger::RequestUpdatedClientInfo()
{
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS

	auto RequestUpdate = [&](FNiagaraDebugger::FClientInfo& Client)
	{
		FNiagaraRequestSimpleClientInfoMessage* Message = FMessageEndpoint::MakeMessage<FNiagaraRequestSimpleClientInfoMessage>();
		UE_LOGF(LogNiagaraDebugger, Log, "Requesting updated simple client info. | Session: %ls | Instance: %ls |", *Client.SessionId.ToString(), *Client.InstanceId.ToString());
		MessageEndpoint->Send(Message, EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Client.Address), FTimespan::Zero(), FDateTime::MaxValue());
	};

	ForAllConnectedClients(RequestUpdate);

#else

	//Session/Messaging system is not available so just send to local client.
	if (INiagaraDebuggerClient* LocalClient = INiagaraDebuggerClient::Get())
	{
		LocalClient->GetSimpleClientInfo(SimpleClientInfo);
		OnSimpleClientInfoChangedDelegate.Broadcast(SimpleClientInfo);
	}

#endif//WITH_UNREAL_TARGET_DEVELOPER_TOOLS
}

const FName NiagaraDebuggerCaptureOutlinerOnKeyPressMessageKey(TEXT("NiagaraDebuggerCaptureOutlinerOnKeyPressMessageKey"));
void FNiagaraDebugger::ToggleOutlinerCaptureOnKeyPress()
{
	bTriggerOutlinerCaptureOnKeyPress = !bTriggerOutlinerCaptureOnKeyPress;

	const FString HudMessage = FString::Printf(TEXT("Press 'C' To Capture Niagara Outliner..."));//TODOL Customizable Key?

	if(bTriggerOutlinerCaptureOnKeyPress)
	{
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().RegisterInputPreProcessor(AsShared());
		}

		FNiagaraWorldManager::ForAllWorldManagers([&HudMessage](FNiagaraWorldManager& WorldMan) { WorldMan.AddDebugHUDMessage(NiagaraDebuggerCaptureOutlinerOnKeyPressMessageKey, FNiagaraDebugMessage(ENiagaraDebugMessageType::Info, HudMessage, FLT_MAX)); });
	}
	else
	{
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().UnregisterInputPreProcessor(AsShared());
		}

		FNiagaraWorldManager::ForAllWorldManagers([&HudMessage](FNiagaraWorldManager& WorldMan) { WorldMan.AddDebugHUDMessage(NiagaraDebuggerCaptureOutlinerOnKeyPressMessageKey, FNiagaraDebugMessage(ENiagaraDebugMessageType::Info, HudMessage, -1.0f)); });
	}
}

void FNiagaraDebugger::TriggerOutlinerCapture()
{
	//Send the current settings as a message to all connected clients.
	if (UNiagaraOutliner* Outliner = GetMutableDefault<UNiagaraOutliner>())
	{
		if(Outliner->CaptureSettings.bTriggerCapture)
		{
		#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
			auto SendSettingsUpdate = [&](FNiagaraDebugger::FClientInfo& Client)
			{
				//Create the message and copy the current state of the settings into it.
				FNiagaraOutlinerCaptureSettings* Message = FMessageEndpoint::MakeMessage<FNiagaraOutlinerCaptureSettings>();
				FNiagaraOutlinerCaptureSettings::StaticStruct()->CopyScriptStruct(Message, &Outliner->CaptureSettings);

				UE_LOGF(LogNiagaraDebugger, Log, "Sending updated outliner settings. | Session: %ls | Instance: %ls |", *Client.SessionId.ToString(), *Client.InstanceId.ToString());
				MessageEndpoint->Send(Message, EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Client.Address), FTimespan::Zero(), FDateTime::MaxValue());
			};

			ForAllConnectedClients(SendSettingsUpdate);
		
		#else

			//Session/Messaging system is not available so just send to local client.
			if (INiagaraDebuggerClient* LocalClient = INiagaraDebuggerClient::Get())
			{
				FNiagaraOutlinerCaptureSettings Request;
				FNiagaraOutlinerCaptureSettings::StaticStruct()->CopyScriptStruct(&Request, &Outliner->CaptureSettings);

				LocalClient->UpdateOutlinerSettings(Request, FOnNiagaraDebuggerClientOutlinerCapture::CreateLambda(
					[&](const FNiagaraOutlinerData& NewOutlinerData)
					{
						if (UNiagaraOutliner* Outliner = GetOutliner())
						{
							Outliner->UpdateData(NewOutlinerData);
						}
					}));
			}

		#endif//WITH_UNREAL_TARGET_DEVELOPER_TOOLS

			Outliner->CaptureSettings.bTriggerCapture = false;

			//We also need at least minimal debug hud verbosity so we see the countdown timer.
			if (Outliner->CaptureSettings.CaptureDelayFrames > 0)
			{
				if (UNiagaraDebugHUDSettings* Settings = GetMutableDefault<UNiagaraDebugHUDSettings>())
				{
					if (
#if WITH_EDITORONLY_DATA
						!Settings->Data.bWidgetEnabled ||
#endif
						!Settings->Data.bHudEnabled )
					{
#if WITH_EDITORONLY_DATA
						Settings->Data.bWidgetEnabled = true;
#endif
						Settings->Data.bHudEnabled = true;
						Settings->PostEditChange();
					}
				}
			}
		}
	}
}

void FNiagaraDebugger::TriggerSimCacheCapture(FName ComponentName, int32 CaptureDelay, int32 CaptureFrames)
{
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
	auto SendCaptureRequest = [&](FNiagaraDebugger::FClientInfo& Client)
	{
		FNiagaraSystemSimCacheCaptureRequest* Message = FMessageEndpoint::MakeMessage<FNiagaraSystemSimCacheCaptureRequest>();
		Message->CaptureDelayFrames = CaptureDelay;
		Message->CaptureFrames = CaptureFrames;
		Message->ComponentName = ComponentName;

		UE_LOGF(LogNiagaraDebugger, Log, "Sending request to capture sim cache for component %ls. | Session: %ls | Instance: %ls |", *ComponentName.ToString(), *Client.SessionId.ToString(), *Client.InstanceId.ToString());
		MessageEndpoint->Send(Message, EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Client.Address), FTimespan::Zero(), FDateTime::MaxValue());
	};

	ForAllConnectedClients(SendCaptureRequest);

#else
	
	if(INiagaraDebuggerClient* LocalClient = INiagaraDebuggerClient::Get())
	{
		FNiagaraSystemSimCacheCaptureRequest Request;
		Request.CaptureDelayFrames = CaptureDelay;
		Request.CaptureFrames = CaptureFrames;
		Request.ComponentName = ComponentName;			
		LocalClient->SimCacheCaptureRequest(Request, FOnNiagaraDebuggerClientSimCacheCapture::CreateLambda(
			[&](const FNiagaraSystemSimCacheCaptureRequest& Request, TObjectPtr<UNiagaraSimCache> CapturedSimCache)
			{
				if (UNiagaraOutliner* Outliner = GetOutliner())
				{
					Outliner->UpdateSystemSimCache(Request.ComponentName, CapturedSimCache);
				}
		}));
	}

#endif//WITH_UNREAL_TARGET_DEVELOPER_TOOLS
}

void FNiagaraDebugger::SessionManager_OnSessionSelectionChanged(const TSharedPtr<ISessionInfo>& Session)
{
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
	//Drop all existing and pending connections when the session selection changes.
	TArray<FClientInfo> ToClose;
	if (Session.IsValid())
	{
		UE_LOGF(LogNiagaraDebugger, Log, "Session selection changed. Dropping connections not from this session. | Session: %ls (%ls)", *Session->GetSessionId().ToString(), *Session->GetSessionName());
		for (FClientInfo& Client : ConnectedClients)
		{
			if (Client.SessionId != Session->GetSessionId())
			{
				ToClose.Add(Client);
			}
		}
		for (FClientInfo& Client : PendingClients)
		{
			if (Client.SessionId != Session->GetSessionId())
			{
				ToClose.Add(Client);
			}
		}
	}
	else
	{
		UE_LOGF(LogNiagaraDebugger, Log, "Session selection changed. Dropping all connections.");
		ToClose.Append(ConnectedClients);
		ToClose.Append(PendingClients);
	}

	for (FClientInfo& Client : ToClose)
	{
		CloseConnection(Client.SessionId, Client.InstanceId);
	}
#endif
}

void FNiagaraDebugger::SessionManager_OnInstanceSelectionChanged(const TSharedPtr<ISessionInstanceInfo>& Instance, bool Selected)
{
#if WITH_UNREAL_TARGET_DEVELOPER_TOOLS
	if (MessageEndpoint.IsValid())
	{
		if (Selected)
		{
			const FGuid& SessionId = Instance->GetOwnerSession()->GetSessionId();
			const FGuid& InstanceId = Instance->GetInstanceId();

			int32 FoundActive = FindActiveConnection(SessionId, InstanceId);
			if (FoundActive != INDEX_NONE)
			{
				UE_LOGF(LogNiagaraDebugger, Log, "Session Instance selection callback for existing active connection. Ignored. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *Instance->GetInstanceName());
				return;
			}

			int32 FoundPending = FindPendingConnection(SessionId, InstanceId);
			if (FoundPending != INDEX_NONE)
			{
				UE_LOGF(LogNiagaraDebugger, Log, "Session Instance selection callback for existing pending connection. Ignored. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *Instance->GetInstanceName());
				return;
			}

			FNiagaraDebuggerRequestConnection* ConnectionRequestMessage = FMessageEndpoint::MakeMessage<FNiagaraDebuggerRequestConnection>(SessionId, InstanceId);
			UE_LOGF(LogNiagaraDebugger, Log, "Establishing connection. | Session: %ls | Instance: %ls (%ls).", *SessionId.ToString(), *InstanceId.ToString(), *Instance->GetInstanceName());
			MessageEndpoint->Publish(ConnectionRequestMessage);

			FClientInfo& NewPendingConnection = PendingClients.AddDefaulted_GetRef();
			NewPendingConnection.SessionId = SessionId;
			NewPendingConnection.InstanceId = InstanceId;
			NewPendingConnection.StartTime = FPlatformTime::Seconds();
		}
		else
		{
			CloseConnection(Instance->GetOwnerSession()->GetSessionId(), Instance->GetInstanceId());
		}
	}
#endif
}

int32 FNiagaraDebugger::FindPendingConnection(FGuid SessionId, FGuid InstanceId)const
{
	return PendingClients.IndexOfByPredicate([&](const FClientInfo& Pending) { return Pending.SessionId == SessionId && Pending.InstanceId == InstanceId; });
}

int32 FNiagaraDebugger::FindActiveConnection(FGuid SessionId, FGuid InstanceId)const
{
	return ConnectedClients.IndexOfByPredicate([&](const FClientInfo& Active) { return Active.SessionId == SessionId && Active.InstanceId == InstanceId; });
}

void FNiagaraDebugger::CloseConnection(FGuid SessionId, FGuid InstanceId)
{
	int32 FoundPending = FindPendingConnection(SessionId, InstanceId);
	int32 FoundActive = FindActiveConnection(SessionId, InstanceId);

	checkf(FoundActive == INDEX_NONE || FoundPending == INDEX_NONE, TEXT("Same client info is on both the pending and active connections lists."));

	if (FoundPending != INDEX_NONE)
	{
		FClientInfo Pending = PendingClients[FoundPending];
		PendingClients.RemoveAtSwap(FoundPending);
		MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FNiagaraDebuggerConnectionClosed>(Pending.SessionId, Pending.InstanceId));
		UE_LOGF(LogNiagaraDebugger, Log, "Closing pending connection. | Session: %ls | Instance: %ls |", *SessionId.ToString(), *InstanceId.ToString());

		OnConnectionClosedDelegate.Broadcast(Pending);
	}
	if (FoundActive != INDEX_NONE)
	{
		FClientInfo Active = ConnectedClients[FoundActive];
		ConnectedClients.RemoveAtSwap(FoundActive);
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FNiagaraDebuggerConnectionClosed>(Active.SessionId, Active.InstanceId), EMessageFlags::Reliable, nullptr, TArrayBuilder<FMessageAddress>().Add(Active.Address), FTimespan::Zero(), FDateTime::MaxValue());
		UE_LOGF(LogNiagaraDebugger, Log, "Closing active connection. | Session: %ls | Instance: %ls |", *SessionId.ToString(), *InstanceId.ToString());

		OnConnectionClosedDelegate.Broadcast(Active);
	}
}

void FNiagaraDebugger::HandleConnectionAcceptedMessage(const FNiagaraDebuggerAcceptConnection& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	int32 FoundActive = FindActiveConnection(Message.SessionId, Message.InstanceId);
	if (FoundActive != INDEX_NONE)
	{
		UE_LOGF(LogNiagaraDebugger, Log, "Received connection accepted message from an already connected client. Ignored. | Session: %ls | Instance: %ls |", *Message.SessionId.ToString(), *Message.InstanceId.ToString());
		return;
	}

	int32 FoundPending = FindPendingConnection(Message.SessionId, Message.InstanceId);
	if (FoundPending != INDEX_NONE)
	{
		UE_LOGF(LogNiagaraDebugger, Log, "Connection accepted. | Session: %ls | Instance: %ls |", *Message.SessionId.ToString(), *Message.InstanceId.ToString());
		PendingClients.RemoveAtSwap(FoundPending);

		FClientInfo& NewConnection = ConnectedClients.AddDefaulted_GetRef();
		NewConnection.Address = Context->GetSender();
		NewConnection.SessionId = Message.SessionId;
		NewConnection.InstanceId = Message.InstanceId;
		NewConnection.StartTime = FPlatformTime::Seconds();

		UpdateDebugHUDSettings();

		if (UNiagaraOutliner* Outliner = GetOutliner())
		{
			Outliner->Reset();
		}

		OnConnectionMadeDelegate.Broadcast(NewConnection);
	}
	else
	{
		UE_LOGF(LogNiagaraDebugger, Log, "Received connection accepted message from a client that is not in our pending list. Ignored. | Session: %ls | Instance: %ls |", *Message.SessionId.ToString(), *Message.InstanceId.ToString());
	}
}

void FNiagaraDebugger::HandleConnectionClosedMessage(const FNiagaraDebuggerConnectionClosed& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	int32 FoundPending = FindPendingConnection(Message.SessionId, Message.InstanceId);
	int32 FoundActive = FindActiveConnection(Message.SessionId, Message.InstanceId);

	checkf(FoundActive == INDEX_NONE || FoundPending == INDEX_NONE, TEXT("Same client info is on both the pending and active connections lists."));

	if (FoundPending != INDEX_NONE)
	{
		UE_LOGF(LogNiagaraDebugger, Log, "Pending connection closed by the client. | Session: %ls | Instance: %ls |", *Message.SessionId.ToString(), *Message.InstanceId.ToString());
		OnConnectionClosedDelegate.Broadcast(PendingClients[FoundPending]);
		PendingClients.RemoveAtSwap(FoundPending);
	}
	if (FoundActive != INDEX_NONE)
	{
		UE_LOGF(LogNiagaraDebugger, Log, "Active connection closed by the client. | Session: %ls | Instance: %ls |", *Message.SessionId.ToString(), *Message.InstanceId.ToString());
		OnConnectionClosedDelegate.Broadcast(ConnectedClients[FoundActive]);
		ConnectedClients.RemoveAtSwap(FoundActive);
	}
}

void FNiagaraDebugger::HandleOutlinerUpdateMessage(const FNiagaraDebuggerOutlinerUpdate& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogNiagaraDebugger, Log, "Received outliner update. | Sender: %ls ", *Context->GetSender().ToString());
	if(UNiagaraOutliner* Outliner = GetOutliner())
	{
		Outliner->UpdateData(Message.OutlinerData);
	}
}

void FNiagaraDebugger::UpdateSimpleClientInfo(const FNiagaraSimpleClientInfo& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogNiagaraDebugger, Log, "Received simple client info update. | Sender: %ls ", *Context->GetSender().ToString());
	SimpleClientInfo = Message;
	OnSimpleClientInfoChangedDelegate.Broadcast(SimpleClientInfo);
}

void FNiagaraDebugger::HandleSimCacheCaptureReply(const FNiagaraSystemSimCacheCaptureReply& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	UE_LOGF(LogNiagaraDebugger, Log, "Received Niagara Sim Cache Capture Reply. | Sender: %ls ", *Context->GetSender().ToString());
	
	if (UNiagaraOutliner* Outliner = GetOutliner())
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TObjectPtr<UNiagaraSimCache> NewSimCache = NewObject<UNiagaraSimCache>(NiagaraEditorModule.GetTempPackage());//Outer to the temp package so that we can "Edit" it.

		FMemoryReader ArReader(Message.SimCacheData);
		FObjectAndNameAsStringProxyArchive ProxyArReader(ArReader, false);
		NewSimCache->Serialize(ProxyArReader);

		Outliner->UpdateSystemSimCache(Message.ComponentName, NewSimCache);
	}
}

//IInputProcessor Interface

bool FNiagaraDebugger::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	if (IsCapturingOutlinerOnKeyPress() && InKeyEvent.GetKey() == EKeys::C)
	{
		if (UNiagaraOutliner* Outliner = GetOutliner())
		{
			Outliner->CaptureSettings.bTriggerCapture = true;
			Outliner->OnChanged();
		}
		ToggleOutlinerCaptureOnKeyPress();
	}
	return IInputProcessor::HandleKeyDownEvent(SlateApp, InKeyEvent);;
}

//IInputProcessor Interface END


#undef LOCTEXT_NAMESPACE


#endif//WITH_NIAGARA_DEBUGGER