// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "NiagaraDebuggerCommon.h"
#include "NiagaraOutliner.h"
#include "NiagaraWorldManager.h"
#include "Framework/Application/IInputProcessor.h"

class ISessionManager;
class FMessageEndpoint;
class ISessionInfo;
class ISessionInstanceInfo;

DECLARE_LOG_CATEGORY_EXTERN(LogNiagaraDebugger, Log, All);

#if WITH_NIAGARA_DEBUGGER


class FNiagaraDebugger : public TSharedFromThis<FNiagaraDebugger>, public IInputProcessor
{
public:
	struct FClientInfo
	{
		FMessageAddress Address;
		FGuid SessionId;
		FGuid InstanceId;

		/** Time this connection began it's current state, pending or connected. Used to timeout pending connections and track uptime of connected clients. */
		double StartTime;

		FORCEINLINE void Clear()
		{
			Address.Invalidate();
			SessionId.Invalidate();
			InstanceId.Invalidate();
			StartTime = 0.0;
		}
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnConnectionMade, const FClientInfo&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnConnectionClosed, const FClientInfo&);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSimpleClientInfoChanged, const FNiagaraSimpleClientInfo&);

	FNiagaraDebugger();
	virtual ~FNiagaraDebugger();
	void Init();

	template<typename TAction>
	void ForAllConnectedClients(TAction Func);

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& GetMessageEndpoint() { return MessageEndpoint; }

	void ExecConsoleCommand(const TCHAR* Cmd, bool bRequiresWorld);
	void UpdateDebugHUDSettings();
	void TriggerOutlinerCapture();
	void TriggerSimCacheCapture(FName ComponentName, int32 CaptureDelay, int32 CaptureFrames);

	void ToggleOutlinerCaptureOnKeyPress();
	bool IsCapturingOutlinerOnKeyPress(){return bTriggerOutlinerCaptureOnKeyPress;}

	void RequestUpdatedClientInfo();

	FOnConnectionMade& GetOnConnectionMade() { return OnConnectionMadeDelegate; }
	FOnConnectionClosed& GetOnConnectionClosed() { return OnConnectionClosedDelegate; }
	FOnSimpleClientInfoChanged& GetOnSimpleClientInfoChanged() { return OnSimpleClientInfoChangedDelegate;	}

	UNiagaraOutliner* GetOutliner()const
	{
		//Keep access via debugger incase we chose to not use the default object in future.
		return GetMutableDefault<UNiagaraOutliner>(); 
	}

	const FNiagaraSimpleClientInfo& GetSimpleClientInfo() { return SimpleClientInfo; }

	//IInputProcessor Interface
	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor){};
	virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)override;
	virtual const TCHAR* GetDebugName() const { return TEXT("NiagaraDebugger"); }
	//IInputProcessor Interface END

protected:

	// Handles a connection accepted message and finalizes establishing the connection to a debugger client.
	void HandleConnectionAcceptedMessage(const FNiagaraDebuggerAcceptConnection& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	// Handles a connection closed message can be called from debugged clients if their instance shutsdown etc.
	void HandleConnectionClosedMessage(const FNiagaraDebuggerConnectionClosed& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	// Handles an update to the Niagara Outliner data from the client.
	void HandleOutlinerUpdateMessage(const FNiagaraDebuggerOutlinerUpdate& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	// Handles a message that updates the simple client info.
	void UpdateSimpleClientInfo(const FNiagaraSimpleClientInfo& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	// Handles an reply messages from a sim cache capture request.
	void HandleSimCacheCaptureReply(const FNiagaraSystemSimCacheCaptureReply& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback from session manager when the selected session is changed. */
	void SessionManager_OnSessionSelectionChanged(const TSharedPtr<ISessionInfo>& Session);

	/** Callback from session manager when the selected instance is changed. */
	void SessionManager_OnInstanceSelectionChanged(const TSharedPtr<ISessionInstanceInfo>& Instance, bool Selected);

	/** Removes any active or pending connection to the given client and sends a message informing the client. */
	void CloseConnection(FGuid SessionId, FGuid InstanceId);

	int32 FindPendingConnection(FGuid SessionId, FGuid InstanceId)const;
	int32 FindActiveConnection(FGuid SessionId, FGuid InstanceId)const;

	/** Holds a pointer to the session manager. */
	TSharedPtr<ISessionManager> SessionManager;

	/** Holds the messaging endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Clients that we are actively connected to. */
	TArray<FClientInfo> ConnectedClients;

	/** Clients that we are awaiting an connection acceptance message from. */
	TArray<FClientInfo> PendingClients;

	FOnConnectionMade OnConnectionMadeDelegate;
	FOnConnectionClosed OnConnectionClosedDelegate;
	FOnSimpleClientInfoChanged OnSimpleClientInfoChangedDelegate;

	/** Some basic info on the connected client. */
	FNiagaraSimpleClientInfo SimpleClientInfo;

	bool bTriggerOutlinerCaptureOnKeyPress = false;
};

template<typename TAction>
void FNiagaraDebugger::ForAllConnectedClients(TAction Func)
{
	for (FNiagaraDebugger::FClientInfo& Client : ConnectedClients)
	{
		Func(Client);
	}
}

#endif//WITH_NIAGARA_DEBUGGER
