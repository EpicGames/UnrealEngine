// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDigestDatabase.h"

#include "Misc/LazySingleton.h"
#include "NiagaraEditorModule.h"
#include "NiagaraGraph.h"
#include "NiagaraGraphDigest.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "UObject/UObjectGlobals.h"

namespace NiagaraGraphDigestDatabaseImpl
{

static int32 GDigestGraphCacheSize = 512;
static FAutoConsoleVariableRef CVarDigestGraphCacheSize(
	TEXT("fx.Niagara.DigestGraphCacheSize"),
	GDigestGraphCacheSize,
	TEXT("Defines the size of the cache for digested Niagara graphs."),
	ECVF_ReadOnly
);

FNiagaraDigestDatabase* GDigestDatabase = nullptr;

} // NiagaraGraphDigestDatabaseImpl::

FNiagaraDigestDatabase& FNiagaraDigestDatabase::Get()
{
	if (!NiagaraGraphDigestDatabaseImpl::GDigestDatabase)
	{
		NiagaraGraphDigestDatabaseImpl::GDigestDatabase = &TLazySingleton<FNiagaraDigestDatabase>::Get();
	}

	return *NiagaraGraphDigestDatabaseImpl::GDigestDatabase;
}

void FNiagaraDigestDatabase::Shutdown()
{
	Get().ReleaseDatabase();
}

FNiagaraDigestDatabase::FNiagaraDigestDatabase()
	: CompilationGraphCache(NiagaraGraphDigestDatabaseImpl::GDigestGraphCacheSize)
{
	// we use GC as an opportunity to clear the graph cache, preventing the database from holding onto references and
	// extending object lifetimes unnecessarily.  In particular the digested graphs contain duplicates of the DI
	// which could be referencing real objects.  Note that the NPC cache isn't cleared as it is a duplicate owned by
	// this database that mirrors the universal data (i.e. the NPC themselves aren't released when running in the editor)
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FNiagaraDigestDatabase::ReleaseGraphCache);
}

FNiagaraDigestDatabase::~FNiagaraDigestDatabase()
{
	ReleaseDatabase();

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
}

void FNiagaraDigestDatabase::ReleaseGraphCache()
{
	FWriteScopeLock WriteScope(DigestCacheLock);

	CompilationGraphCache.Empty(NiagaraGraphDigestDatabaseImpl::GDigestGraphCacheSize);
}

void FNiagaraDigestDatabase::ReleaseDatabase()
{
	FWriteScopeLock WriteScope(DigestCacheLock);

	CompilationGraphCache.Empty(NiagaraGraphDigestDatabaseImpl::GDigestGraphCacheSize);
	CompilationNPCCache.Empty();
}

void FNiagaraDigestDatabase::AddReferencedObjects(FReferenceCollector& Collector)
{
	FReadScopeLock ReadScope(DigestCacheLock);

	for (FCompilationNPCCache::TIterator It(CompilationNPCCache); It; ++It)
	{
		It.Value()->AddReferencedObjects(Collector);
	}
}

FString FNiagaraDigestDatabase::GetReferencerName() const
{
	static const FString ReferencerName = TEXT("NiagaraDigestDatabsae");
	return ReferencerName;
}

//////////////////////////////////////////////////////////////////////////
/// Digested graphs

FNiagaraDigestedGraphPtr FNiagaraDigestDatabase::CreateGraphDigest(const UNiagaraGraph* Graph, const FNiagaraGraphChangeIdBuilder& Digester)
{
	check(IsInGameThread());
	FNiagaraCompilationGraphHandle GraphHash(Graph, Digester);
	FNiagaraDigestedGraphPtr PendingGraph;

	{
		FWriteScopeLock WriteScope(DigestCacheLock);

		if (const FNiagaraDigestedGraphPtr* CompilationGraph = CompilationGraphCache.FindAndTouch(GraphHash))
		{
			++GraphCacheHits;
			return *CompilationGraph;
		}

		++GraphCacheMisses;
		PendingGraph = MakeShared<FNiagaraDigestedGraphPtr::ElementType, FNiagaraDigestedGraphPtr::Mode>();

		CompilationGraphCache.Add(GraphHash, PendingGraph);
	}

	if (PendingGraph)
	{
		PendingGraph->Digest(Graph, Digester);
	}

	return PendingGraph;
}


FNiagaraCompilationGraphHandle::FNiagaraCompilationGraphHandle(const UNiagaraGraph* Graph, const FNiagaraGraphChangeIdBuilder& Builder)
{
	AssetKey = Graph;

	// generate the hash key based on the provided ChangeIdBuilder
	Hash = Builder.FindChangeId(Graph);
}

//////////////////////////////////////////////////////////////////////////
/// Digested parameter collection

FNiagaraCompilationNPCHandle FNiagaraDigestDatabase::CreateCompilationCopy(const UNiagaraParameterCollection* Collection)
{
	check(IsInGameThread());
	FNiagaraCompilationNPCHandle CollectionHash(Collection);
	FNiagaraCompilationNPC* PendingCollection = nullptr;

	{
		FWriteScopeLock WriteScope(DigestCacheLock);

		if (const FNiagaraCompilationNPCHandle::FDigestPtr* CompilationCollection = CompilationNPCCache.Find(CollectionHash))
		{
			++CollectionCacheHits;
			return CollectionHash;
		}

		++CollectionCacheMisses;
		FNiagaraCompilationNPCHandle::FDigestPtr CompilationCollection = MakeShared<FNiagaraCompilationNPCHandle::FDigestPtr::ElementType, FNiagaraCompilationNPCHandle::FDigestPtr::Mode>();
		PendingCollection = CompilationCollection.Get();

		CompilationNPCCache.Add(CollectionHash, CompilationCollection);
	}

	if (PendingCollection)
	{
		PendingCollection->Create(Collection);
	}

	return CollectionHash;
}

FNiagaraCompilationNPCHandle::FDigestPtr FNiagaraDigestDatabase::Resolve(const FNiagaraCompilationNPCHandle& Handle) const
{
	FReadScopeLock ReadScope(DigestCacheLock);

	return CompilationNPCCache.FindRef(Handle);
}

void FNiagaraCompilationNPC::Create(const UNiagaraParameterCollection* Collection)
{
	SourceCollection = Collection;
	Namespace = Collection->GetNamespace();
	CollectionPath = FSoftObjectPath(Collection).ToString();
	CollectionFullName = GetFullNameSafe(Collection);

	Variables.Reserve(Collection->GetParameters().Num());
	for (const FNiagaraVariable& Parameter : Collection->GetParameters())
	{
		Variables.Emplace(Parameter);
	}

	{ // we also need to deal with any data interfaces that might be stored in the NPC
		UPackage* TransientPackage = GetTransientPackage();

		const FNiagaraParameterStore& DefaultParamStore = Collection->GetDefaultInstance()->GetParameterStore();
		for (const FNiagaraVariableBase& Variable : Variables)
		{
			const int32 VariableOffset = DefaultParamStore.IndexOf(Variable);
			if (VariableOffset != INDEX_NONE)
			{
				if (UNiagaraDataInterface* DefaultDataInterface = DefaultParamStore.GetDataInterface(VariableOffset))
				{
					UNiagaraDataInterface* DuplicateDataInterface = DuplicateObject<UNiagaraDataInterface>(DefaultDataInterface, TransientPackage);

					DefaultDataInterfaces.Add(Variable, DuplicateDataInterface);
				}
			}
		}
	}
}

void FNiagaraCompilationNPC::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddStableReferenceMap(DefaultDataInterfaces);
}

UNiagaraDataInterface* FNiagaraCompilationNPC::GetDataInterface(const FNiagaraVariableBase& Variable) const
{
	return DefaultDataInterfaces.FindRef(Variable);
}

FNiagaraCompilationNPCHandle::FNiagaraCompilationNPCHandle(const UNiagaraParameterCollection* Connection)
{
	if (Connection)
	{
		Namespace = Connection->GetNamespace();
		AssetKey = Connection;
		Hash = Connection->GetCompileHash();
	}
}

FNiagaraCompilationNPCHandle::FDigestPtr FNiagaraCompilationNPCHandle::Resolve() const
{
	return FNiagaraDigestDatabase::Get().Resolve(*this);
}