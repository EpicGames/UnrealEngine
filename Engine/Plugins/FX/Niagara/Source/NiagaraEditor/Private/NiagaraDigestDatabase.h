// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/LruCache.h"
#include "NiagaraCompileHash.h"
#include "NiagaraGraphDigest.h"
#include "UObject/GCObject.h"

class UNiagaraCompilationGraph;
class UNiagaraGraph;
class UNiagaraParameterCollection;
class UNiagaraScriptSourceBase;

using FNiagaraDigestedGraphPtr = TSharedPtr<FNiagaraCompilationGraphDigested, ESPMode::ThreadSafe>;

class FNiagaraCompilationGraphHandle
{
public:
	FNiagaraCompilationGraphHandle() = default;
	FNiagaraCompilationGraphHandle(const UNiagaraGraph* Graph, const FNiagaraGraphChangeIdBuilder& ChangeIdBuilder);

protected:
	TObjectKey<UNiagaraGraph> AssetKey;
	FGuid Hash;

	friend FORCEINLINE uint32 GetTypeHash(const FNiagaraCompilationGraphHandle& CompilationCopy)
	{
		return HashCombine(GetTypeHash(CompilationCopy.AssetKey), GetTypeHash(CompilationCopy.Hash));
	}

	friend FORCEINLINE bool operator==(const FNiagaraCompilationGraphHandle& Lhs, const FNiagaraCompilationGraphHandle& Rhs)
	{
		return Lhs.AssetKey == Rhs.AssetKey && Lhs.Hash == Rhs.Hash;
	}
};

class FNiagaraCompilationNPC
{
public:
	void Create(const UNiagaraParameterCollection* Collection);
	void AddReferencedObjects(FReferenceCollector& Collector);

	UNiagaraDataInterface* GetDataInterface(const FNiagaraVariableBase& Variable) const;

	TArray<FNiagaraVariableBase> Variables;
	FName Namespace;
	TWeakObjectPtr<const UNiagaraParameterCollection> SourceCollection;
	FString CollectionPath;
	FString CollectionFullName;
	TMap<FNiagaraVariableBase, TObjectPtr<UNiagaraDataInterface>> DefaultDataInterfaces;
};

class FNiagaraCompilationNPCHandle
{
public:
	using FDigestPtr = TSharedPtr<FNiagaraCompilationNPC, ESPMode::ThreadSafe>;

	FNiagaraCompilationNPCHandle() = default;
	FNiagaraCompilationNPCHandle(const FNiagaraCompilationNPCHandle& Handle) = default;
	FNiagaraCompilationNPCHandle(const UNiagaraParameterCollection* Collection);

	bool IsValid() const
	{
		return Resolve().IsValid();
	}

	FDigestPtr Resolve() const;

	FName Namespace = NAME_None;

protected:
	TObjectKey<UNiagaraParameterCollection> AssetKey;
	FNiagaraCompileHash Hash;

	friend FORCEINLINE uint32 GetTypeHash(const FNiagaraCompilationNPCHandle& CompilationCopy)
	{
		return HashCombine(GetTypeHash(CompilationCopy.AssetKey), GetTypeHash(CompilationCopy.Hash));
	}

	friend FORCEINLINE bool operator==(const FNiagaraCompilationNPCHandle& Lhs, const FNiagaraCompilationNPCHandle& Rhs)
	{
		return Lhs.AssetKey == Rhs.AssetKey && Lhs.Hash == Rhs.Hash;
	}
};

class FNiagaraDigestedParameterCollections
{
public:
	TConstArrayView<FNiagaraCompilationNPCHandle> ReadCollections() const { return Collections; }
	TArray<FNiagaraCompilationNPCHandle>& EditCollections() { return Collections; };

	FNiagaraCompilationNPCHandle FindMatchingCollection(FName VariableName, bool bAllowPartialMatch, FNiagaraVariable& OutVar) const;
	FNiagaraCompilationNPCHandle FindCollection(const FNiagaraVariable& Variable) const;

protected:
	FNiagaraCompilationNPCHandle FindCollectionByName(FName VariableName) const;

	TArray<FNiagaraCompilationNPCHandle> Collections;
};

class FNiagaraDigestDatabase : public FGCObject
{
public:
	FNiagaraDigestDatabase();
	virtual ~FNiagaraDigestDatabase();

	static FNiagaraDigestDatabase& Get();
	static void Shutdown();

	FNiagaraDigestedGraphPtr CreateGraphDigest(const UNiagaraGraph* Graph, const FNiagaraGraphChangeIdBuilder& Digester);

	FNiagaraCompilationNPCHandle CreateCompilationCopy(const UNiagaraParameterCollection* Collection);
	FNiagaraCompilationNPCHandle::FDigestPtr Resolve(const FNiagaraCompilationNPCHandle& Handle) const;

	void ReleaseDatabase();

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

protected:
	void ReleaseGraphCache();

	uint32 GraphCacheHits = 0;
	uint32 GraphCacheMisses = 0;
	uint32 CollectionCacheHits = 0;
	uint32 CollectionCacheMisses = 0;

	using FCompilationGraphCache = TLruCache<FNiagaraCompilationGraphHandle, FNiagaraDigestedGraphPtr>;
	FCompilationGraphCache CompilationGraphCache;

	using FCompilationNPCCache = TMap<FNiagaraCompilationNPCHandle, FNiagaraCompilationNPCHandle::FDigestPtr>;
	FCompilationNPCCache CompilationNPCCache;

	mutable FRWLock DigestCacheLock;
};
