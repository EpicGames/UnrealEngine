// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorCommands.h"

#include "Framework/Commands/Commands.h"

#define LOCTEXT_NAMESPACE "NiagaraEditorCommands"

UE_DEFINE_TCOMMANDS(FNiagaraEditorCommands)

void FNiagaraEditorCommands::RegisterCommands()
{
	UI_COMMAND(Apply, "Apply", "Apply unsaved changes to the current object.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ApplyScratchPadChanges, "Apply Scratch", "Applies changes to all scratch pad scripts which have pending changes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Discard, "Discard", "Discard unsaved changes from the current object.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Compile, "Compile", "Compile the current scripts", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RefreshNodes, "Refresh", "Refreshes the current graph nodes, and updates pins due to external changes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ModuleVersioning, "Versioning", "Manage different versions of a module script.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EmitterVersioning, "Versioning", "Manage different versions of an emittert.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectNextUsage, "Select Next Usage", "Selects the next usage of the selected item.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateAssetFromSelection, "Create Asset...", "Creates an asset from the current selection.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(OpenAddEmitterMenu, "Add Emitter...", "Adds an emitter to the system.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::None, EKeys::E));

	UI_COMMAND(ResetSimulation, "Reset", "Resets the current simulation", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar));
	UI_COMMAND(TogglePreviewGrid, "Grid", "Toggles the preview pane's grid.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND(ToggleOriginAxis, "Origin Axis", "Toggles the preview pane's origin axis.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleInstructionCounts, "InstructionCounts", "Display Instruction Counts", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleParticleCounts, "ParticleCounts", "Display Particle Counts", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleEmitterExecutionOrder, "Emitter Execution Order", "Display Emitter Execution Order", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleGpuTickInformation, "Gpu Tick Information", "Display Gpu Tick Information", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleMemoryInfo, "Memory", "Show approximate memory usage", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleStatelessInfo, "Lightweight", "Show information about lightweight emitters and execution path", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(TogglePreviewBackground, "Background", "Toggles the preview pane's background.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleUnlockToChanges, "Lock/Unlock To Changes", "Toggles whether or not changes in the source asset get pulled into this asset automatically.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleStatPerformance, "Performance", "Show runtime performance for particle scripts. Not supported for optimized VM byte code.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleStatPerformanceGPU, "GPU Profiling", "Measure gpu runtime cost of scripts. Can be expensive for simulation stages with lots of iterations.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleProfileInEditMode, "Profile In Edit Mode", "When enabled and profiling is active choose how we compile the system.  Edit mode is worse performance but won't recompile when tweaking parameters.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ClearStatPerformance, "Clear Stats", "Clear all existing stat captures.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleStatPerformanceTypeAvg, "Display Average", "Displays the average of captured stats.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleStatPerformanceTypeMax, "Display Maximum", "Displays the maximum of captured stats.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleStatPerformanceModePercent, "Display Relative Values", "Displays the captured module stats in percent of the parent script.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleStatPerformanceModeAbsolute, "Display Absolute Values", "Displays the captured module stats times directly.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleBounds, "Bounds", "Display Bounds", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleBounds_SetFixedBounds_SelectedEmitters, "Set Fixed Bounds (Emitters)", "Set Fixed Bounds on emitters (only the selected emitters when in a System asset) ", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleBounds_SetFixedBounds_System, "Set Fixed Bounds (System)", "Set Fixed Bounds on the system", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleOrbit, "Orbit Mode", "Toggle Orbit Navigation", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleMotion, "Motion", "Toggle Motion On Component", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SaveThumbnailImage, "Thumbnail", "Generate Thumbnail", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleAutoPlay, "Auto-play", "Toggles whether or not simulations auto-play when their asset editor is opened, and when the asset is modified.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleResetSimulationOnChange, "Reset on change", "Toggles whether or not the simulation is reset whenever a change is made in the asset editor.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleResimulateOnChangeWhilePaused, "Resimulate when paused", "Toggles whether or not the simulation is rerun to the current time when making changes while paused.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleResetDependentSystems, "Reset Dependent Systems", "Toggles whether or not to reset all systems that include this emitter when it is reset by the user.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(CollapseStackToHeaders, "Collapse to Headers", "Expands all headers and collapse all items.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::O));

	UI_COMMAND(IsolateSelectedEmitters, "Isolate selected emitters", "Isolate all currently selected emitters.", EUserInterfaceActionType::Button, FInputChord(EKeys::S));
	UI_COMMAND(EnableSelectedEmitters, "Enable selected emitters", "Enables all currently selected emitters and recompiles the system.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::D));
	UI_COMMAND(DisableSelectedEmitters, "Disable selected emitters", "Disables all currently selected emitters and recompiles the system.", EUserInterfaceActionType::Button, FInputChord(EKeys::D));
	UI_COMMAND(HideDisabledModules, "Collapse disabled modules", "Collapses disabled modules in the stack, so they won't take up as much space.", EUserInterfaceActionType::Check, FInputChord(EModifierKey::Control, EKeys::M));
	
	UI_COMMAND(OpenDebugHUD, "Debug HUD", "Open the Niagara Debug HUD.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenDebugOutliner, "FX Outliner", "Open the Niagara FX Outliner.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenAttributeSpreadsheet, "Attribute Spreadsheet", "Open the Niagara Debug Outliner.", EUserInterfaceActionType::Button, FInputChord());

	// todo
	/*UI_COMMAND(SelectNextEmitter, "Select Next Emitter", "Toggles whether or not simulations auto-play when their asset editor is opened, and when the asset is modified.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Shift, EKeys::D));
	UI_COMMAND(SelectPreviousEmitter, "Select Previous Emitter", "Toggles whether or not simulations auto-play when their asset editor is opened, and when the asset is modified.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Shift, EKeys::A));
	UI_COMMAND(ShowLowLevelOfDetail, "Show Low Level of Detail", "Toggles whether or not simulations auto-play when their asset editor is opened, and when the asset is modified.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::One));
	UI_COMMAND(ShowMediumLevelOfDetail, "Show Medium Level of Detail", "Toggles whether or not simulations auto-play when their asset editor is opened, and when the asset is modified.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::Two));
	UI_COMMAND(ShowHighLevelOfDetail, "Show High Level of Detail", "Toggles whether or not simulations auto-play when their asset editor is opened, and when the asset is modified.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::Three));*/

	UI_COMMAND(FindInCurrentView, "Find", "Contextually finds items in current view.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));

	UI_COMMAND(ZoomToFit, "Zoom to Fit", "Zooms and pans to fit the current selection.", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(ZoomToFitAll, "Zoom to Fit All", "Zooms and pans to fix all items.", EUserInterfaceActionType::Button, FInputChord(EKeys::A));
}

#undef LOCTEXT_NAMESPACE // NiagaraEditorCommands
