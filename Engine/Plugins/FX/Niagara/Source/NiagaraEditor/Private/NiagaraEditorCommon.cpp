// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorCommon.h"
#include "INiagaraCompiler.h"
#include "NiagaraSystem.h"

#include "ActorFactoryNiagara.h"
#include "NiagaraActor.h"
#include "NiagaraComponent.h"
#include "NiagaraSettings.h"
#include "Misc/StringFormatter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEditorCommon)

#define LOCTEXT_NAMESPACE "NiagaraEditor"

DEFINE_LOG_CATEGORY(LogNiagaraEditor);


///////////////////////////////////////////////////////////////////////////////
/// Common Editor Strings													///
///////////////////////////////////////////////////////////////////////////////
namespace FNiagaraEditorStrings
{
	const FName DefaultValueCustomRowName = TEXT("Default Value");
	const FName DefaultModeCustomRowName = TEXT("Default Mode");

	const FName FNiagaraParameterActionId = TEXT("FNiagaraParameterAction");
}

///////////////////////////////////////////////////////////////////////////////
/// Op Info																	///
///////////////////////////////////////////////////////////////////////////////
TMap<FName, int32> FNiagaraOpInfo::OpInfoMap;
TArray<FNiagaraOpInfo> FNiagaraOpInfo::OpInfos;

const FNiagaraOpInfo* FNiagaraOpInfo::GetOpInfo(FName OpName)
{
	const int32* Idx = OpInfoMap.Find(OpName);
	if (Idx)
	{
		return &OpInfos[*Idx];
	}
	return nullptr;
}

const TArray<FNiagaraOpInfo>& FNiagaraOpInfo::GetOpInfoArray()
{
	return OpInfos;
}

void FNiagaraOpInfo::BuildName(FString InName, FString InCategory)
{	
	Name = FName(*(InCategory + TEXT("::") + InName));
}

int32 CountType(TConstArrayView<FNiagaraTypeDefinition> Types, const FNiagaraTypeDefinition& SearchType)
{
	int32 Count = 0;
	for (const FNiagaraTypeDefinition& Type : Types)
	{
		if (Type == SearchType)
		{
			Count++;
		}
	}
	return Count;
}

bool FNiagaraOpInfo::CreateHlslForAddedInputs(int32 InputCount, FString & HlslResult) const
{
	if (!bSupportsAddedInputs || AddedInputFormatting.IsEmpty() || InputCount < 2)
	{
		return false;
	}

	FString Result = TEXT("{0}");
	FString KeyA = TEXT("A");
	FString KeyB = TEXT("B");
	TMap<FString, FStringFormatArg> FormatArgs;
	FormatArgs.Add(KeyA, FStringFormatArg(Result));
	FormatArgs.Add(KeyB, FStringFormatArg(Result));
	for (int32 i = 1; i < InputCount; i++)
	{
		FormatArgs[KeyB] = FStringFormatArg(TEXT("{") + FString::FromInt(i) + TEXT("}"));
		Result = FString::Format(*AddedInputFormatting, FormatArgs);
		FormatArgs[KeyA] = FStringFormatArg(Result);
	}

	HlslResult = Result;
	return true;
}

BEGIN_FUNCTION_BUILD_OPTIMIZATION
void FNiagaraOpInfo::Init()
{
	//Common input and output names.
	static FName Result(TEXT("Result"));
	static FText ResultText = NSLOCTEXT("NiagaraOpInfo", "Operation Result", "Result");
	static FText InText = NSLOCTEXT("NiagaraOpInfo", "Single Function Param", "In"); 

	static FName A(TEXT("A"));
	static FText AText = NSLOCTEXT("NiagaraOpInfo", "First Function Param", "A");

	static FName B(TEXT("B"));
	static FText BText = NSLOCTEXT("NiagaraOpInfo", "Second Function Param", "B");

	static FName C(TEXT("C"));
	static FText CText = NSLOCTEXT("NiagaraOpInfo", "Third Function Param", "C");

	FName X(TEXT("X"));	FText XText = NSLOCTEXT("NiagaraOpInfo", "First Vector Component", "X");

	FText PositionCategory = NSLOCTEXT("NiagaraOpInfo", "PositionOpCategory", "Position");

	static FText SelectionText = NSLOCTEXT("NiagaraOpInfo", "Selection", "Selection");
	static FText MaskText = NSLOCTEXT("NiagaraOpInfo", "Mask", "Mask");
	
	static FName Min(TEXT("Min"));
	static FName Max(TEXT("Max"));
	static FText MinText = NSLOCTEXT("NiagaraOpInfo", "Min", "Min");
	static FText MaxText = NSLOCTEXT("NiagaraOpInfo", "Max", "Max");

	auto NoPositionTypeCheck = [=](const TArray<FNiagaraTypeDefinition>& InputTypes, FText& ErrorMessage)
	{
		int32 PositionTypes = CountType(InputTypes, FNiagaraTypeDefinition::GetPositionDef());
		if (PositionTypes > 0)
		{
			ErrorMessage = NSLOCTEXT("NiagaraOpInfo", "Position Type Error", "This operation cannot be used to for position types, as it would break large world coordinates. To resolve this error, you can explicitly truncate the position to vectors first.");
			return false;
		}
		return true;
	};

	//Add all numeric ops
	FText NumericCategory = NSLOCTEXT("NiagaraOpInfo", "NumericOpCategory", "Numeric");
	{
		FNiagaraTypeDefinition NumericType = FNiagaraTypeDefinition::GetGenericNumericDef();
		FString DefaultStr_Zero(TEXT("0.0"));
		FString DefaultStr_One(TEXT("1.0"));
		FString CategoryName(TEXT("Numeric"));

		int32 Idx = OpInfos.AddDefaulted();
		FNiagaraOpInfo* Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Add Name", "Add");
		Op->CompactName = FText::FromString(TEXT("+"));
		Op->CompactNameFontSizeOverride = 30.f;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Add Desc", "Result = A + B");
		Op->AlternateSearchName = FName("+");
		Op->Keywords = FText::FromString(TEXT("+"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_Zero, TEXT("{0} + {1}")));
		Op->BuildName(TEXT("Add"), CategoryName);
		Op->bSupportsAddedInputs = true;
		Op->AddedInputTypeRestrictions.Add(NumericType);
		Op->AddedInputFormatting = TEXT("{A} + {B}");
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
			{
				if (InPinValues.Num() == 0)
					return 0;
				if (InPinValues.Num() == 1)
					return InPinValues[0];

				int32 Return = InPinValues[0];
				for (int32 i = 1; i < InPinValues.Num(); i++)
				{
					Return = Return + InPinValues[i];
				}

				return Return;
			});
		Op->InputTypeValidationFunction.BindLambda([=](const TArray<FNiagaraTypeDefinition>& InputTypes, FText& ErrorMessage)
			{
				int32 PositionTypes = CountType(InputTypes, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes > 1)
				{
					ErrorMessage = NSLOCTEXT("NiagaraOpInfo", "Add Type Error", "The add operation cannot be used to add two positions. If this is intended, convert one or all of them to vectors first.");
					return false;
				}
				return true;
			});
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Custom;
		Op->CustomNumericResolveFunction.BindLambda([=](TConstArrayView<FNiagaraTypeDefinition> NonNumericInputs)
			{
				int32 PositionTypes = CountType(NonNumericInputs, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes > 0)
				{
					return FNiagaraTypeDefinition::GetPositionDef();
				}
				return FNiagaraTypeDefinition::GetNumericOutputType(NonNumericInputs, ENiagaraNumericOutputTypeSelectionMode::Largest);
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Subtract Name", "Subtract");
		Op->CompactName = FText::FromString(TEXT("-"));
		Op->CompactNameFontSizeOverride = 28.f;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Subtract Desc", "Result = A - B");
		Op->AlternateSearchName = FName("-");
		Op->Keywords = FText::FromString(TEXT("-"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_Zero, TEXT("{0} - {1}")));		
		Op->BuildName(TEXT("Subtract"), CategoryName);
		Op->bSupportsAddedInputs = true;
		Op->AddedInputTypeRestrictions.Add(NumericType);
		Op->AddedInputFormatting = TEXT("{A} - {B}");
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
			{
				if (InPinValues.Num() == 0)
					return 0;
				if (InPinValues.Num() == 1)
					return InPinValues[0];

				int32 Return = InPinValues[0];
				for (int32 i = 1; i < InPinValues.Num(); i++)
				{
					Return = Return - InPinValues[i];
				}

				return Return;
			});
		Op->InputTypeValidationFunction.BindLambda([=](const TArray<FNiagaraTypeDefinition>& InputTypes, FText& ErrorMessage)
			{
				int32 PositionTypes = CountType(InputTypes, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes > 2)
				{
					ErrorMessage = NSLOCTEXT("NiagaraOpInfo", "Subtract Type Error", "The subtract operation cannot be used for more than two positions.");
					return false;
				}
				return true;
			});
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Custom;
		Op->CustomNumericResolveFunction.BindLambda([=](TConstArrayView<FNiagaraTypeDefinition> NonNumericInputs)
			{
				int32 PositionTypes = CountType(NonNumericInputs, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes == 1)
				{
					return FNiagaraTypeDefinition::GetPositionDef();
				}
				if (PositionTypes > 1)
				{
					 return FNiagaraTypeDefinition::GetVec3Def();
				 }
				return FNiagaraTypeDefinition::GetNumericOutputType(NonNumericInputs, ENiagaraNumericOutputTypeSelectionMode::Largest);
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Multiply Name", "Multiply");
		Op->CompactName = FText::FromString(TEXT("\xD7"));
		Op->CompactNameFontSizeOverride = 30.f;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Multiply Desc", "Result = A * B");
		Op->AlternateSearchName = FName("*");
		Op->Keywords = FText::FromString(TEXT("*"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("{0} * {1}")));
		Op->BuildName(TEXT("Mul"), CategoryName);
		Op->bSupportsAddedInputs = true;
		Op->AddedInputTypeRestrictions.Add(NumericType);
		Op->AddedInputFormatting = TEXT("{A} * {B}");
		Op->bSupportsStaticResolution = true;
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
			{
				if (InPinValues.Num() == 0)
					return 0;
				if (InPinValues.Num() == 1)
					return InPinValues[0];

				int32 Return = InPinValues[0];
				for (int32 i = 1; i < InPinValues.Num(); i++)
				{
					Return = Return * InPinValues[i];
				}

				return Return;
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Divide Name", "Divide");
		Op->CompactName = FText::FromString(TEXT("\xF7"));
		Op->CompactNameFontSizeOverride = 25.f;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Divide Desc", "Result = A / B");
		Op->AlternateSearchName = FName("/");
		Op->Keywords = FText::FromString(TEXT("/"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("{0} / {1}")));
		Op->BuildName(TEXT("Div"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
			{
				if (InPinValues.Num() == 0)
					return 0;
				if (InPinValues.Num() == 1)
					return InPinValues[0];

				int32 Return = InPinValues[0];
				for (int32 i = 1; i < InPinValues.Num(); i++)
				{
					if (InPinValues[i] != 0)
						Return = Return / InPinValues[i];
					else
						Return = 0;
				}

				return Return;
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "MultiplyAdd Name", "MultiplyAdd");
		FString CompactNameString = TEXT("(A");
		CompactNameString.Append(TEXT("\xD7"));
		CompactNameString.Append(TEXT("B)+C"));
		Op->CompactName = FText::FromString(CompactNameString);
		Op->bShowPinNamesInCompactMode = true;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "MultiplyAdd Desc", "Result = (A * B) + C");
		Op->AlternateSearchName = FName("*+");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(C, NumericType, CText, CText, DefaultStr_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("{0} * {1} + {2}")));
		Op->BuildName(TEXT("Madd"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda([=](const TArray<FNiagaraTypeDefinition>& InputTypes, FText& ErrorMessage)
			{
				int32 PositionTypes = CountType(InputTypes, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes == 0)
				{
					return true;
				}
				if (PositionTypes > 1 || (InputTypes[2] != FNiagaraTypeDefinition::GetPositionDef()))
				{
					ErrorMessage = NSLOCTEXT("NiagaraOpInfo", "MultiplyAdd Type Error", "The MultiplyAdd operation can only use positions for the addition, not the multiplication part.");
					return false;
				}
				return true;
			});
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Custom;
		Op->CustomNumericResolveFunction.BindLambda([=](TConstArrayView<FNiagaraTypeDefinition> NonNumericInputs)
			{
				int32 PositionTypes = CountType(NonNumericInputs, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes == 1)
				{
					return FNiagaraTypeDefinition::GetPositionDef();
				}
				return FNiagaraTypeDefinition::GetNumericOutputType(NonNumericInputs, ENiagaraNumericOutputTypeSelectionMode::Largest);
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Lerp Name", "Lerp");
		Op->CompactName = FText::FromString(TEXT("Lerp"));
		Op->bShowPinNamesInCompactMode = true;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Lerp Desc", "Result = (A * (1 - Alpha)) + (B * Alpha)");
		Op->Keywords = FText::FromString(TEXT("lerp linear interpolate"));

		FText AlphaPinFriendlyName = FText::FromString("Alpha");
		FText AlphaTooltip = NSLOCTEXT("NiagaraOpInfo", "Lerp Alpha Tooltip", "A value typically between 0 and 1. Determines the percentage to use for interpolating from A to B.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(C, NumericType, AlphaPinFriendlyName, AlphaTooltip, DefaultStr_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_Zero, TEXT("lerp({0},{1},{2})")));
		Op->BuildName(TEXT("Lerp"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda([=](const TArray<FNiagaraTypeDefinition>& InputTypes, FText& ErrorMessage)
			{
				int32 PositionTypes = CountType(InputTypes, FNiagaraTypeDefinition::GetPositionDef());
				 if (PositionTypes == 0)
				 {
					 return true;
				 }
				if (PositionTypes != 2 || InputTypes[2] == FNiagaraTypeDefinition::GetPositionDef())
				{
					ErrorMessage = NSLOCTEXT("NiagaraOpInfo", "Lerp Type Error", "The Lerp operation can only lerp between two positions, not between a position and another type.");
					return false;
				}
				return true;
			});
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Custom;
		Op->CustomNumericResolveFunction.BindLambda([=](TConstArrayView<FNiagaraTypeDefinition> NonNumericInputs)
			{
				 int32 PositionTypes = CountType(NonNumericInputs, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes == 2)
				{
					return FNiagaraTypeDefinition::GetPositionDef();
				}
				return FNiagaraTypeDefinition::GetNumericOutputType(NonNumericInputs, ENiagaraNumericOutputTypeSelectionMode::Largest);
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Reciprocal Fast Name", "Reciprocal Fast");
		Op->CompactName = FText::FromString("Rcp Fast");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Reciprocal Fast Desc", "12-bits of accuracy, but faster. Result = 1 / A using Newton/Raphson approximation.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("rcp({0})")));
		Op->BuildName(TEXT("RcpFast"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Reciprocal Name", "Reciprocal");
		Op->CompactName = FText::FromString("Reciprocal");
		Op->CompactNameFontSizeOverride = 18.f;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Reciprocal Desc", "More accurate than Reciprocal Fast. Result = 1 / A");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("Reciprocal({0})")));
		Op->BuildName(TEXT("Rcp"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Reciprocal Sqrt Name", "Reciprocal Sqrt");
		Op->CompactName = FText::FromString("Rcp Sqrt");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Reciprocal Sqrt Desc", "Result = 1 / sqrt(A)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("rsqrt({0})")));
		Op->BuildName(TEXT("RSqrt"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Sqrt Name", "Sqrt");
		Op->CompactName = FText::FromString("Sqrt");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Sqrt Desc", "Result = sqrt(A)");
		Op->Keywords = FText::FromString(TEXT("sqrt"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("sqrt({0})")));
		Op->BuildName(TEXT("Sqrt"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "One Minus Name", "One Minus");
		Op->CompactName = FText::FromString(TEXT("1-A"));
		Op->bShowPinNamesInCompactMode = true;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "One Minus Desc", "Result = 1 - A");
		Op->AlternateSearchName = FName("1-");
		Op->Keywords = FText::FromString(TEXT("1-x"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("1 - {0}")));
		Op->BuildName(TEXT("OneMinus"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
			{
				if (InPinValues.Num() == 1)
					return 1 - InPinValues[0];
				else
					return 0;
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Negate Name", "Negate");
		Op->CompactName = FText::FromString("-A");
		Op->bShowPinNamesInCompactMode = true;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Negate Desc", "Result = -A");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("-({0})")));
		Op->BuildName(TEXT("Negate"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
			{
				if (InPinValues.Num() == 1)
					return -InPinValues[0];
				else
					return 0;
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Abs Name", "Abs");
		Op->CompactName = FText::FromString("Abs");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Abs Desc", "Result = abs(A)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("abs({0})")));
		Op->BuildName(TEXT("Abs"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Exp Name", "Exp");
		Op->CompactName = FText::FromString("Exp");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Exp Desc", "Result = exp(A)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("exp({0})")));
		Op->BuildName(TEXT("Exp"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Exp2 Name", "Exp2");
		Op->CompactName = FText::FromString("Exp2");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Exp2 Desc", "Result = exp2(A)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("exp2({0})")));
		Op->BuildName(TEXT("Exp2"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Log Name", "Log");
		Op->CompactName = FText::FromString("Log");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Log Desc", "Result = log(A)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("log({0})")));
		Op->BuildName(TEXT("Log"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Log2 Name", "Log2");
		Op->CompactName = FText::FromString("Log2");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Log2 Desc", "Result = log2(A)");
		Op->Keywords = FText::FromString(TEXT("log2"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("log2({0})")));
		Op->BuildName(TEXT("Log2"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		FText AngleFriendlyNameText = NSLOCTEXT("NiagaraOpInfo", "Angle Name", "Angle");
		FText AngleDescText = NSLOCTEXT("NiagaraOpInfo", "Angle Desc", "Angle as specified by the period range.");
		FText PeriodFriendlyNameText = NSLOCTEXT("NiagaraOpInfo", "Period Name", "Period");
		FText PeriodDescText = NSLOCTEXT("NiagaraOpInfo", "Period Desc", "Value in which a complete rotation has occurred.");
		FNiagaraTypeDefinition PeriodType = FNiagaraTypeDefinition::GetFloatDef();
		FNiagaraTypeDefinition AngleType = FNiagaraTypeDefinition::GetFloatDef();

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Sine Name", "Sine");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Sine Desc", "Result = sin(Angle*(TWO_PI/Period))");
		Op->Keywords = FText::FromString(TEXT("sine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Angle"), AngleType, AngleFriendlyNameText, AngleDescText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Period"), PeriodType, PeriodFriendlyNameText, PeriodDescText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("sin({0}*(TWO_PI/{1}))")));
		Op->BuildName(TEXT("Sine"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "SinRad Name", "Sine(Radians)");
		Op->CompactName = FText::FromString("Sin(R)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "SinRad Desc", "Result = sin(AngleInRadians)");
		Op->Keywords = FText::FromString(TEXT("sine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("sin({0})")));
		Op->BuildName(TEXT("Sine(Radians)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "SinDeg Name", "Sine(Degrees)");
		Op->CompactName = FText::FromString("Sin(D)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "SinDeg Desc", "Result = sin(AngleInDegrees*DegreesToRadians)");
		Op->Keywords = FText::FromString(TEXT("sine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("sin((PI/180.0f)*{0})")));
		Op->BuildName(TEXT("Sine(Degrees)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Cosine Name", "Cosine");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Cosine Desc", "Result = cos(Angle*(TWO_PI/Period))");
		Op->Keywords = FText::FromString(TEXT("Cosine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Angle"), AngleType, AngleFriendlyNameText, AngleDescText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Period"), PeriodType, PeriodFriendlyNameText, PeriodDescText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("cos({0}*(TWO_PI/{1}))")));
		Op->BuildName(TEXT("Cosine"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "CosRad Name", "Cosine(Radians)");
		Op->CompactName = FText::FromString("Cos(R)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "CosRad Desc", "Result = cos(AngleInRadians)");
		Op->Keywords = FText::FromString(TEXT("Cosine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("cos({0})")));
		Op->BuildName(TEXT("Cosine(Radians)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "CosDeg Name", "Cosine(Degrees)");
		Op->CompactName = FText::FromString("Cos(D)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "CosDeg Desc", "Result = cos(AngleInDegrees*DegreesToRadians)");
		Op->Keywords = FText::FromString(TEXT("Cosine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("cos((PI/180.0f)*{0})")));
		Op->BuildName(TEXT("Cosine(Degrees)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Tangent Name", "Tangent");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Tangent Desc", "Result = tan(Angle*(TWO_PI/Period))");
		Op->Keywords = FText::FromString(TEXT("Tangent"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Angle"), AngleType, AngleFriendlyNameText, AngleDescText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Period"), PeriodType, PeriodFriendlyNameText, PeriodDescText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("tan({0}*(TWO_PI/{1}))")));
		Op->BuildName(TEXT("Tangent"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "TanRad Name", "Tangent(Radians)");
		Op->CompactName = FText::FromString("Tan(R)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "TanRad Desc", "Result = tan(AngleInRadians)");
		Op->Keywords = FText::FromString(TEXT("Tangent"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("tan({0})")));
		Op->BuildName(TEXT("Tangent(Radians)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "TanDeg Name", "Tangent(Degrees)");
		Op->CompactName = FText::FromString("Tan(D)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "TanDeg Desc", "Result = tan(AngleInDegrees*DegreesToRadians)");
		Op->Keywords = FText::FromString(TEXT("Tangent"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("tan((PI/180.0f)*{0})")));
		Op->BuildName(TEXT("Tangent(Degrees)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;


		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ArcSine Name", "ArcSine");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ArcSine Desc", "Result = asin(A)*(Period/TWO_PI)");
		Op->Keywords = FText::FromString(TEXT("ArcSine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Period"), PeriodType, PeriodFriendlyNameText, PeriodDescText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("asin(clamp({0}, -1.0f, 1.0f))*({1}/TWO_PI)")));
		Op->BuildName(TEXT("ArcSine"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ArcSineRad Name", "ArcSine(Radians)");
		Op->CompactName = FText::FromString("ArcSin(R)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ArcSineRad Desc", "Result = asin(A)");
		Op->Keywords = FText::FromString(TEXT("ArcSine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("asin(clamp({0}, -1.0f, 1.0f))")));
		Op->BuildName(TEXT("ArcSine(Radians)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ArcSineDeg Name", "ArcSine(Degrees)");
		Op->CompactName = FText::FromString("ArcSin(D)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ArcSineDeg Desc", "Result = asin(A)*RadiansToDegrees");
		Op->Keywords = FText::FromString(TEXT("ArcSine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("(180.0f/PI)*asin(clamp({0}, -1.0f, 1.0f))")));
		Op->BuildName(TEXT("ArcSine(Degrees)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Pi Name", "PI");
		Op->CompactName = FText::FromString(UTF8_TO_TCHAR("\xCF\x80"));
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Pi Desc", "The constant PI");
		Op->Keywords = FText::FromString(TEXT("pi"));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetFloatDef(), ResultText, ResultText, DefaultStr_One, TEXT("PI")));
		Op->BuildName(TEXT("PI"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Two Pi Name", "TWO_PI");
		Op->CompactName = FText::FormatOrdered(FText::FromString("2 {0}"), FText::FromString(UTF8_TO_TCHAR("\xCF\x80")));
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Two Pi Desc", "The constant PI * 2");
		Op->Keywords = FText::FromString(TEXT("pi"));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetFloatDef(), ResultText, ResultText, DefaultStr_One, TEXT("TWO_PI")));
		Op->BuildName(TEXT("TWO_PI"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;


		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ArcCosine Name", "ArcCosine");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ArcCosine Desc", "Result = acos(A)*(Period/TWO_PI)");
		Op->Keywords = FText::FromString(TEXT("ArcCosine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Period"), PeriodType, PeriodFriendlyNameText, PeriodDescText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("acos(clamp({0}, -1.0f, 1.0f))*({1}/TWO_PI)")));
		Op->BuildName(TEXT("ArcCosine"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ArcCosineRad Name", "ArcCosine(Radians)");
		Op->CompactName = FText::FromString("ArcCos(R)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ArcCosineRad Desc", "Result = acos(A)");
		Op->Keywords = FText::FromString(TEXT("ArcCosine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("acos(clamp({0}, -1.0f, 1.0f))")));
		Op->BuildName(TEXT("ArcCosine(Radians)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ArcCosineDeg Name", "ArcCosine(Degrees)");
		Op->CompactName = FText::FromString("ArcCos(D)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ArcCosineDeg Desc", "Result = acos(A)*RadiansToDegrees");
		Op->Keywords = FText::FromString(TEXT("ArcCosine"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("(180.0f/PI)*acos(clamp({0}, -1.0f, 1.0f))")));
		Op->BuildName(TEXT("ArcCosine(Degrees)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;


		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ArcTangent Name", "ArcTangent");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ArcTangent Desc", "Result = atan(A)*(Period/TWO_PI)");
		Op->Keywords = FText::FromString(TEXT("ArcTangent"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Period"), PeriodType, PeriodFriendlyNameText, PeriodDescText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("atan({0})*({1}/TWO_PI)")));
		Op->BuildName(TEXT("ArcTangent"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ArcTangentRad Name", "ArcTangent(Radians)");
		Op->CompactName = FText::FromString("ArcTan(R)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ArcTangentRad Desc", "Result = atan(A)");
		Op->Keywords = FText::FromString(TEXT("ArcTangent"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("atan({0})")));
		Op->BuildName(TEXT("ArcTangent(Radians)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ArcTangentDeg Name", "ArcTangent(Degrees)");
		Op->CompactName = FText::FromString("ArcTan(D)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ArcTangentDeg Desc", "Result = atan(A)*RadiansToDegrees");
		Op->Keywords = FText::FromString(TEXT("ArcTangent"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("(180.0f/PI)*atan({0})")));
		Op->BuildName(TEXT("ArcTangent(Degrees)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ATan2 Name", "ArcTangent2");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ATan2 Desc", "ResultInPeriod = Period * atan2(A, B) / 2PI");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(TEXT("Period"), PeriodType, PeriodFriendlyNameText, PeriodDescText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("atan2({0},{1})*({2}/TWO_PI)")));
		Op->BuildName(TEXT("ArcTangent2"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ATan2Rad Name", "ArcTangent2(Radians)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ATan2Rad Desc", "ResultInRadians = atan2(A, B)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("atan2({0},{1})")));
		Op->BuildName(TEXT("ArcTangent2(Radians)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ATan2Deg Name", "ArcTangent2(Degrees)");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "ATan2Deg Desc", "ResultInPeriod = 180 * atan2(A, B) / PI");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("atan2({0},{1})*(180.0f/PI)")));
		Op->BuildName(TEXT("ArcTangent2(Degrees)"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Degrees To Radians", "DegreesToRadians");
		Op->CompactName = FText::FromString("DegToRad");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Degrees To Radians Desc", "Degrees to Radians");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, FNiagaraTypeDefinition::GetFloatDef(), AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetFloatDef(), ResultText, ResultText, DefaultStr_One, TEXT("(PI/180.0f)*({0})")));
		Op->BuildName(TEXT("DegreesToRadians"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Radians To Degrees", "RadiansToDegrees");
		Op->CompactName = FText::FromString("RadToDeg");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "RadiansToDegrees Desc", "Radians to Degrees");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, FNiagaraTypeDefinition::GetFloatDef(), AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetFloatDef(), ResultText, ResultText, DefaultStr_One, TEXT("(180.0f/PI)*({0})")));
		Op->BuildName(TEXT("RadiansToDegrees"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Ceil Name", "Ceil");
		Op->CompactName = FText::FromString("Ceil");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Ceil Desc", "Rounds A to the nearest integer higher than A.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("ceil({0})")));
		Op->BuildName(TEXT("Ceil"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Floor Name", "Floor");
		Op->CompactName = FText::FromString("Floor");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Floor Desc", "Rounds A to the nearest integer lower than A.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("floor({0})")));
		Op->BuildName(TEXT("Floor"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Round Name", "Round");
		Op->CompactName = FText::FromString("Round");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Round Desc", "Rounds A to the nearest integer.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("round({0})")));
		Op->BuildName(TEXT("Round"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Fmod Name", "Modulo");
		Op->CompactName = FText::FromString("%");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Fmod Desc", "Result = A % B (Modulo)");
		Op->CompactNameFontSizeOverride = 25.f;
		Op->Keywords = FText::FromString(TEXT("%"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("ModuloPrecise({0}, {1})")));
		Op->BuildName(TEXT("FMod"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Fmod Name Fast", "Modulo Fast");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Fmod Desc Fast", "Result = A % B. May be less precise than regular FMod.");
		Op->Keywords = FText::FromString(TEXT("%"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("Modulo({0}, {1})")));
		Op->BuildName(TEXT("FModFast"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Frac Name", "Frac");
		Op->CompactName = FText::FromString("Frac");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Frac Desc", "Result = frac(A)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("frac({0})")));
		Op->BuildName(TEXT("Frac"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Trunc Name", "Trunc");
		Op->CompactName = FText::FromString("Trunc");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Trunc Desc", "Result = trunc(A)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("trunc({0})")));
		Op->BuildName(TEXT("Trunc"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Clamp Name", "Clamp");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Clamp Desc", "Result = clamp(A, Min, Max)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(Min, NumericType, MinText, MinText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(Max, NumericType, MaxText, MaxText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("clamp({0},{1},{2})")));
		Op->BuildName(TEXT("Clamp"), CategoryName);
		Op->InputTypeValidationFunction.BindLambda([=](const TArray<FNiagaraTypeDefinition>& InputTypes, FText& ErrorMessage)
			{
				int32 PositionTypes = CountType(InputTypes, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes > 0 && PositionTypes != 3)
				{
					ErrorMessage = NSLOCTEXT("NiagaraOpInfo", "Clamp Type Error", "The clamp operation can only clamp between positions, not between a position and another type.");
					return false;
				}
				return true;
			});
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Custom;
		Op->CustomNumericResolveFunction.BindLambda([=](TConstArrayView<FNiagaraTypeDefinition> NonNumericInputs)
			{
				 int32 PositionTypes = CountType(NonNumericInputs, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes == 3)
				{
					return FNiagaraTypeDefinition::GetPositionDef();
				}
				return FNiagaraTypeDefinition::GetNumericOutputType(NonNumericInputs, ENiagaraNumericOutputTypeSelectionMode::Largest);
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Min Name", "Min");
		Op->CompactName = FText::FromString("Min");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Min Desc", "Result = min(A, B)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("min({0},{1})")));
		Op->BuildName(TEXT("Min"), CategoryName);
		Op->bSupportsAddedInputs = true;
		Op->AddedInputTypeRestrictions.Add(NumericType);
		Op->AddedInputFormatting = TEXT("min({A}, {B})");
		Op->InputTypeValidationFunction.BindLambda([=](const TArray<FNiagaraTypeDefinition>& InputTypes, FText& ErrorMessage)
			{
				int32 PositionTypes = CountType(InputTypes, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes > 0 && PositionTypes != 2)
				{
					ErrorMessage = NSLOCTEXT("NiagaraOpInfo", "Min Type Error", "The min operation cannot be used for a position and a non-position type.");
					return false;
				}
				return true;
			});
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Custom;
		Op->CustomNumericResolveFunction.BindLambda([=](TConstArrayView<FNiagaraTypeDefinition> NonNumericInputs)
			{
				 int32 PositionTypes = CountType(NonNumericInputs, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes == 2)
				{
					return FNiagaraTypeDefinition::GetPositionDef();
				}
				return FNiagaraTypeDefinition::GetNumericOutputType(NonNumericInputs, ENiagaraNumericOutputTypeSelectionMode::Largest);
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Max Name", "Max");
		Op->CompactName = FText::FromString("Max");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Max Desc", "Result = max(A, B)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("max({0},{1})")));
		Op->BuildName(TEXT("Max"), CategoryName);
		Op->bSupportsAddedInputs = true;
		Op->AddedInputTypeRestrictions.Add(NumericType);
		Op->AddedInputFormatting = TEXT("max({A}, {B})");
		Op->InputTypeValidationFunction.BindLambda([=](const TArray<FNiagaraTypeDefinition>& InputTypes, FText& ErrorMessage)
			{
				int32 PositionTypes = CountType(InputTypes, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes > 0 && PositionTypes != 2)
				{
					ErrorMessage = NSLOCTEXT("NiagaraOpInfo", "Max Type Error", "The max operation cannot be used for a position and a non-position type.");
					return false;
				}
				return true;
			});
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Custom;
		Op->CustomNumericResolveFunction.BindLambda([=](TConstArrayView<FNiagaraTypeDefinition> NonNumericInputs)
			{
				 int32 PositionTypes = CountType(NonNumericInputs, FNiagaraTypeDefinition::GetPositionDef());
				if (PositionTypes == 2)
				{
					return FNiagaraTypeDefinition::GetPositionDef();
				}
				return FNiagaraTypeDefinition::GetNumericOutputType(NonNumericInputs, ENiagaraNumericOutputTypeSelectionMode::Largest);
			});
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Pow Name", "Pow");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Pow Desc", "Result = pow(A, B). When A < 0, it will be clamped to 0.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("pow(max(0, {0}),{1})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("Pow"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Sign Name", "Sign");
		Op->CompactName = FText::FromString("Sign");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Sign Desc", "Result = sign(A)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("sign({0})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("Sign"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Step Name", "Step");
		Op->CompactName = FText::FromString("Step");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Step Desc", "Result = step(0, A)");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("step(0, {0})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("Step"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Noise Name", "Noise");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Noise Desc", "A continuous pseudo random noise function.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(X, NumericType, XText, XText, DefaultStr_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("noise({0})")));
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Scalar;
		Op->BuildName(TEXT("Noise"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Dot Name", "Dot");
		Op->CompactName = FText::FromString("Dot");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Dot Desc", "Dot product of two vectors.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("dot({0},{1})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Scalar;
		Op->BuildName(TEXT("Dot"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Normalize Name", "Normalize");
		Op->CompactName = FText::FromString("Normalize");
		Op->CompactNameFontSizeOverride = 18.f;
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Normalize Desc", "Normalizes the passed value.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("normalize({0})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("Normalize"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Length Name", "Length");
		Op->CompactName = FText::FromString("Len");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Length Desc", "Returns the length of the passed value.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("length({0})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::Scalar;
		Op->BuildName(TEXT("Length"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = PositionCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Distance Name", "Distance");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Distance Desc", "Returns the distance between two positions.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, FNiagaraTypeDefinition::GetPositionDef(), AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, FNiagaraTypeDefinition::GetPositionDef(), BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetFloatDef(), ResultText, ResultText, DefaultStr_One, TEXT("length({0} - {1})")));
		Op->NumericOuputTypeSelectionMode = ENiagaraNumericOutputTypeSelectionMode::None;
		Op->BuildName(TEXT("DistancePos"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		// Non-deterministic random number generation. Calls FRandomStream on the CPU. 
		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Rand Name", "Random");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Rand Desc", "Returns a non-deterministic random value between 0 and A.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("rand({0})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("Rand"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		// Non-deterministic integer random number generation. Calls FRandomStream on the CPU. 
		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Rand Integer Name", "Random Integer");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Rand Integer Desc", "Returns a non-deterministic random integer value between 0 and Max-1");
		Op->Inputs.Add(FNiagaraOpInOutInfo(Max, NumericType, MaxText, MaxText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("rand_int({0})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("Rand Integer"), CategoryName);
		Op->bNumericsCanBeIntegers = true;
		Op->bNumericsCanBeFloats = false;
		OpInfoMap.Add(Op->Name) = Idx;

		// Non-deterministic float random number generation. Calls FRandomStream on the CPU. 
		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Rand Float Name", "Random Float");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Rand Float Desc", "Returns a non-deterministic random float value between 0 and Max");
		Op->Inputs.Add(FNiagaraOpInOutInfo(Max, NumericType, MaxText, MaxText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("rand_float({0})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("Rand Float"), CategoryName);
		Op->bNumericsCanBeIntegers = false;
		Op->bNumericsCanBeFloats = true;
		OpInfoMap.Add(Op->Name) = Idx;

		// Deterministic/seeded random number generation. 
		static FName SeedName1(TEXT("Seed 1"));
		static FName SeedName2(TEXT("Seed 2"));
		static FName SeedName3(TEXT("Seed 3"));
		static FText SeedText1 = NSLOCTEXT("NiagaraOpInfo", "Seed1 Desc", "Seed 1");
		static FText SeedText2 = NSLOCTEXT("NiagaraOpInfo", "Seed2 Desc", "Seed 2");
		static FText SeedText3 = NSLOCTEXT("NiagaraOpInfo", "Seed3 Desc", "Seed 3");
		FString DefaultSeed_Zero(TEXT("0"));
		FNiagaraTypeDefinition SeedType = FNiagaraTypeDefinition::GetIntDef();

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Seeded Rand Name", "Seeded Random");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Seeded Rand Desc", "Returns a deterministic random value between 0 and A.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(SeedName1, SeedType, SeedText1, SeedText1, DefaultSeed_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(SeedName2, SeedType, SeedText2, SeedText2, DefaultSeed_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(SeedName3, SeedType, SeedText3, SeedText3, DefaultSeed_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("rand({0}, {1}, {2}, {3})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("SeededRand"), CategoryName);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Seeded Integer Rand Name", "Seeded Integer Random");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Seeded Integer Rand Desc", "Returns a deterministic random integer value between 0 and Max-1.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(Max, NumericType, MaxText, MaxText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(SeedName1, SeedType, SeedText1, SeedText1, DefaultSeed_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(SeedName2, SeedType, SeedText2, SeedText2, DefaultSeed_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(SeedName3, SeedType, SeedText3, SeedText3, DefaultSeed_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("rand_int({0}, {1}, {2}, {3})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("SeededRand Integer"), CategoryName);
		Op->bNumericsCanBeIntegers = true;
		Op->bNumericsCanBeFloats = false;
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Seeded Float Rand Name", "Seeded Float Random");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "Seeded Float Rand Desc", "Returns a deterministic random float value between 0 and Max.");
		Op->Inputs.Add(FNiagaraOpInOutInfo(Max, NumericType, MaxText, MaxText, DefaultStr_One));
		Op->Inputs.Add(FNiagaraOpInOutInfo(SeedName1, SeedType, SeedText1, SeedText1, DefaultSeed_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(SeedName2, SeedType, SeedText2, SeedText2, DefaultSeed_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(SeedName3, SeedType, SeedText3, SeedText3, DefaultSeed_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, NumericType, ResultText, ResultText, DefaultStr_One, TEXT("rand_float({0}, {1}, {2}, {3})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("SeededRand Float"), CategoryName);
		Op->bNumericsCanBeIntegers = false;
		Op->bNumericsCanBeFloats = true;
		OpInfoMap.Add(Op->Name) = Idx;

		//Hash ops
		FNiagaraTypeDefinition IntHashType = FNiagaraTypeDefinition::GetIntDef();
		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "HashInt Name", "Hash Integer");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "HashInt Desc", "Returns a hashed value for the given inputs. The output is in the range 0 to (2^31 - 1).");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntHashType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntHashType, BText, BText, DefaultStr_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, IntHashType, ResultText, ResultText, DefaultStr_One, TEXT("hash({0}, {1})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("Hash Integer"), CategoryName);
		Op->bSupportsAddedInputs = true;
		Op->AddedInputTypeRestrictions.Add(IntHashType);
		Op->AddedInputFormatting = TEXT("hash({A}, {B})");
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "HashFloat Name", "Hash Float");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "HashFloat Desc", "Returns a hashed value for the given inputs. The result is normalized to the range 0 (inclusive) to 1 (exclusive).");
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntHashType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntHashType, BText, BText, DefaultStr_Zero));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetFloatDef(), ResultText, ResultText, DefaultStr_One, TEXT("hash_float({0}, {1})")));
		Op->InputTypeValidationFunction.BindLambda(NoPositionTypeCheck);
		Op->BuildName(TEXT("Hash Float"), CategoryName);
		Op->bSupportsAddedInputs = true;
		Op->AddedInputTypeRestrictions.Add(IntHashType);
		Op->AddedInputFormatting = TEXT("hash_float({A}, {B})");
		OpInfoMap.Add(Op->Name) = Idx;

		//Comparison ops
		auto ComparisonOpTypeValidation = [=](const TArray<FNiagaraTypeDefinition>& InputTypes, FText& ErrorMessage)
		{
			int32 PositionTypes = CountType(InputTypes, FNiagaraTypeDefinition::GetPositionDef());
			if (PositionTypes > 0 && PositionTypes != 2)
			{
				ErrorMessage = NSLOCTEXT("NiagaraOpInfo", "Comparison Type Error", "The comparison operation cannot be used for a position and a non-position type.");
				return false;
			}
			return true;
		};
		
		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "CmpLT Name", "Less Than");
		Op->CompactName = FText::FromString("<");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "CmpLT Desc", "Result = A < B");
		Op->Keywords = FText::FromString(TEXT("<"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetBoolDef(), ResultText, ResultText, DefaultStr_One, TEXT("NiagaraAll({0} < {1})")));
		Op->BuildName(TEXT("CmpLT"), CategoryName);
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
		{
			if (InPinValues.Num() != 2)
				return 0;

			int32 Return = InPinValues[0] < InPinValues[1];
			return Return;
		});
		Op->InputTypeValidationFunction.BindLambda(ComparisonOpTypeValidation);
		OpInfoMap.Add(Op->Name) = Idx;

		

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "CmpLE Name", "Less Than Or Equal");
		Op->CompactName = FText::FromString("<=");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "CmpLE Desc", "Result = A <= B");
		Op->Keywords = FText::FromString(TEXT("<="));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetBoolDef(), ResultText, ResultText, DefaultStr_One, TEXT("NiagaraAll({0} <= {1})")));
		Op->BuildName(TEXT("CmpLE"), CategoryName);
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
		{
			if (InPinValues.Num() != 2)
				return 0;

			int32 Return = InPinValues[0] <= InPinValues[1];
			return Return;
		});
		Op->InputTypeValidationFunction.BindLambda(ComparisonOpTypeValidation);
		OpInfoMap.Add(Op->Name) = Idx;


		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "CmpGT Name", "Greater Than");
		Op->CompactName = FText::FromString(">");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "CmpGT Desc", "Result = A > B");
		Op->Keywords = FText::FromString(TEXT(">"));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetBoolDef(), ResultText, ResultText, DefaultStr_One, TEXT("NiagaraAll({0} > {1})")));
		Op->BuildName(TEXT("CmpGT"), CategoryName);
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
		{
			if (InPinValues.Num() != 2)
				return 0;

			int32 Return = InPinValues[0] > InPinValues[1];
			return Return;
		});
		Op->InputTypeValidationFunction.BindLambda(ComparisonOpTypeValidation);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "CmpGE Name", "Greater Than Or Equal");
		Op->CompactName = FText::FromString(">=");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "CmpGE Desc", "Result = A >= B");
		Op->Keywords = FText::FromString(TEXT(">="));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetBoolDef(), ResultText, ResultText, DefaultStr_One, TEXT("NiagaraAll({0} >= {1})")));
		Op->BuildName(TEXT("CmpGE"), CategoryName);
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
		{
			if (InPinValues.Num() != 2)
				return 0;

			int32 Return = InPinValues[0] >= InPinValues[1];
			return Return;
		});Op->InputTypeValidationFunction.BindLambda(ComparisonOpTypeValidation);
		OpInfoMap.Add(Op->Name) = Idx;

		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "CmpEQ Name", "Equal");
		Op->CompactName = FText::FromString("==");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "CmpEQ Desc", "Result = A == B");
		Op->Keywords = FText::FromString(TEXT("=="));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetBoolDef(), ResultText, ResultText, DefaultStr_One, TEXT("NiagaraAll({0} == {1})")));
		Op->BuildName(TEXT("CmpEQ"), CategoryName);
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
		{
			if (InPinValues.Num() != 2)
				return 0;

			int32 Return = InPinValues[0] == InPinValues[1];
			return Return;
		});
		Op->InputTypeValidationFunction.BindLambda(ComparisonOpTypeValidation);
		OpInfoMap.Add(Op->Name) = Idx;

		OpInfoMap.Add(Op->Name) = Idx;
		
		Idx = OpInfos.AddDefaulted();
		Op = &OpInfos[Idx];
		Op->Category = NumericCategory;
		Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "CmpNEQ Name", "Not Equal");
		Op->CompactName = FText::FromString("!=");
		Op->Description = NSLOCTEXT("NiagaraOpInfo", "CmpNEQ Desc", "Result = A != B");
		Op->Keywords = FText::FromString(TEXT("!="));
		Op->Inputs.Add(FNiagaraOpInOutInfo(A, NumericType, AText, AText, DefaultStr_Zero));
		Op->Inputs.Add(FNiagaraOpInOutInfo(B, NumericType, BText, BText, DefaultStr_One));
		Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetBoolDef(), ResultText, ResultText, DefaultStr_One, TEXT("NiagaraAny({0} != {1})")));
		Op->BuildName(TEXT("CmpNEQ"), CategoryName);
		Op->bSupportsStaticResolution = true;
		Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
		{
			if (InPinValues.Num() != 2)
				return 0;

			int32 Return = InPinValues[0] != InPinValues[1];
			return Return;
		});
		Op->InputTypeValidationFunction.BindLambda(ComparisonOpTypeValidation);
		OpInfoMap.Add(Op->Name) = Idx;

		
		OpInfoMap.Add(Op->Name) = Idx;
	}

	//////////////////////////////////////////////////////////////////////////
	//Integer Only Ops

	FString Default_IntZero(TEXT("0"));
	FString Default_IntOne(TEXT("1"));
	FText IntCategory = NSLOCTEXT("NiagaraOpInfo", "IntOpCategory", "Integer");
	FString IntCategoryName(TEXT("Integer"));
	FNiagaraTypeDefinition IntType = FNiagaraTypeDefinition::GetIntDef();

	int32 Idx = OpInfos.AddDefaulted();
	FNiagaraOpInfo* Op = &OpInfos[Idx];
	Op->Category = IntCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "BitAnd Name", "Bitwise AND");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "BitAnd Desc", "Result = A & B");
	Op->Keywords = FText::FromString(TEXT("&"));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntType, AText, AText, Default_IntZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntType, BText, BText, Default_IntOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, IntType, ResultText, ResultText, Default_IntOne, TEXT("{0} & {1}")));
	Op->BuildName(TEXT("BitAnd"), IntCategoryName);
	Op->bSupportsAddedInputs = true;
	Op->AddedInputTypeRestrictions.Add(IntType);
	Op->AddedInputFormatting = TEXT("{A} & {B}");
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = IntCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "BitOr Name", "Bitwise OR");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "BitOr Desc", "Result = A | B");
	Op->Keywords = FText::FromString(TEXT("|"));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntType, AText, AText, Default_IntZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntType, BText, BText, Default_IntOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, IntType, ResultText, ResultText, Default_IntOne, TEXT("{0} | {1}")));
	Op->BuildName(TEXT("BitOr"), IntCategoryName);
	Op->bSupportsAddedInputs = true;
	Op->AddedInputTypeRestrictions.Add(IntType);
	Op->AddedInputFormatting = TEXT("{A} | {B}");
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = IntCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "BitXOr Name", "Bitwise XOR");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "BitXOr Desc", "Result = A ^ B");
	Op->Keywords = FText::FromString(TEXT("^"));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntType, AText, AText, Default_IntZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntType, BText, BText, Default_IntOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, IntType, ResultText, ResultText, Default_IntOne, TEXT("{0} ^ {1}")));
	Op->BuildName(TEXT("BitXOr"), IntCategoryName);
	Op->bSupportsAddedInputs = true;
	Op->AddedInputTypeRestrictions.Add(IntType);
	Op->AddedInputFormatting = TEXT("{A} ^ {B}");
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = IntCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "BitNot Name", "Bitwise NOT");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "BitNot Desc", "Result =  ~B");
	Op->Keywords = FText::FromString(TEXT("~"));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntType, AText, AText, Default_IntOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, IntType, ResultText, ResultText, Default_IntOne, TEXT("~{0}")));
	Op->BuildName(TEXT("BitNot"), IntCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = IntCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "BitLShift Name", "Bitwise Left Shift");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "BitLShift Desc", "Shifts A left by B bits, padding with zeroes on the right. B should be between 0 and 31 or there will be undefined behavior.");
	Op->Keywords = FText::FromString(TEXT("<<"));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntType, AText, AText, Default_IntOne));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntType, BText, BText, Default_IntOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, IntType, ResultText, ResultText, Default_IntOne, TEXT("{0} << {1}")));
	Op->BuildName(TEXT("BitLShift"), IntCategoryName);
	Op->bSupportsAddedInputs = true;
	Op->AddedInputTypeRestrictions.Add(IntType);
	Op->AddedInputFormatting = TEXT("{A} << {B}");
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = IntCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "BitRShift Name", "Bitwise Right Shift");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "BitRShift Desc", "Shifts A right by B bits, taking the sign bit and propagating it to fill in on left (i.e. negative numbers fill with 1's, positive fill with 0's. B should be between 0 and 31 or there will be undefined behavior.");
	Op->Keywords = FText::FromString(TEXT(">>"));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntType, AText, AText, Default_IntOne));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntType, BText, BText, Default_IntOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, IntType, ResultText, ResultText, Default_IntOne, TEXT("{0} >> {1}")));
	Op->BuildName(TEXT("BitRShift"), IntCategoryName);
	Op->bSupportsAddedInputs = true;
	Op->AddedInputTypeRestrictions.Add(IntType);
	Op->AddedInputFormatting = TEXT("{A} >> {B}");
	OpInfoMap.Add(Op->Name) = Idx;


	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = IntCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "SelectStaticIntName", "Select Int (Static)");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "SelectStaticIntDesc", "Result = A ? B : C");
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, FNiagaraTypeDefinition::GetBoolDef().ToStaticDef(), AText, AText, TEXT("false")));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntType.ToStaticDef(), BText, BText, Default_IntZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(C, IntType.ToStaticDef(), CText, CText, Default_IntZero));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, IntType.ToStaticDef(), ResultText, ResultText, Default_IntZero, TEXT("A ? B : C")));
	Op->BuildName(TEXT("SelectStaticInt"), IntCategoryName);
	Op->bSupportsStaticResolution = true;
	Op->StaticVariableResolveFunction.BindLambda(
		[=](const TArray<int32>& InPinValues)
	{
		if (InPinValues.Num() != 3)
			return 0;

		const int32 Return = InPinValues[0] != 0 ? InPinValues[1] : InPinValues[2];
		return Return;
	}
	);
	OpInfoMap.Add(Op->Name) = Idx;


	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = IntCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "EnumEq Name", "Enum Equal");
	Op->CompactName = FText::FromString("==");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "EnumEq Desc", "Result = A == B");
	Op->Keywords = FText::FromString(TEXT("=="));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntType, AText, AText, Default_IntZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntType, BText, BText, Default_IntZero));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetBoolDef(), ResultText, ResultText, Default_IntZero, TEXT("NiagaraAll({0} == {1})")));
	Op->BuildName(TEXT("EnumEq"), IntCategoryName);
	Op->bSupportsStaticResolution = true;
	Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
	{
		if (InPinValues.Num() != 2)
			return 0;

		int32 Return = InPinValues[0] == InPinValues[1];
		return Return;
	});
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = IntCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "EnumNEq Name", "Enum Not Equal");
	Op->CompactName = FText::FromString("!=");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "EnumNEq Desc", "Result = A != B");
	Op->Keywords = FText::FromString(TEXT("!="));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, IntType, AText, AText, Default_IntZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, IntType, BText, BText, Default_IntZero));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetBoolDef(), ResultText, ResultText, Default_IntZero, TEXT("NiagaraAll({0} != {1})")));
	Op->BuildName(TEXT("EnumNEq"), IntCategoryName);
	Op->bSupportsStaticResolution = true;
	Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
	{
		if (InPinValues.Num() != 2)
			return 0;

		int32 Return = InPinValues[0] != InPinValues[1];
		return Return;
	});
	OpInfoMap.Add(Op->Name) = Idx;


	//////////////////////////////////////////////////////////////////////////
	// Boolean Only Ops
	FString Default_BoolZero(TEXT("false"));
	FString Default_BoolOne(TEXT("true"));
	FText BoolCategory = NSLOCTEXT("NiagaraOpInfo", "BoolOpCategory", "Boolean");
	FString BoolCategoryName(TEXT("Boolean"));
	FNiagaraTypeDefinition BoolType = FNiagaraTypeDefinition::GetBoolDef();

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = BoolCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "LogicAnd Name", "Logic AND");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "LogicAnd Desc", "Result = A && B");
	Op->Keywords = FText::FromString(TEXT("&&"));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, BoolType, AText, AText, Default_BoolZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, BoolType, BText, BText, Default_BoolOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, BoolType, ResultText, ResultText, Default_BoolOne, TEXT("{0} && {1}")));
	Op->BuildName(TEXT("LogicAnd"), BoolCategoryName);
	Op->bSupportsAddedInputs = true;
	Op->AddedInputTypeRestrictions.Add(BoolType);
	Op->AddedInputFormatting = TEXT("{A} && {B}");
	Op->bSupportsStaticResolution = true;
	Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
	{
			if (InPinValues.Num() == 0)
				return 0;
			if (InPinValues.Num() == 1)
				return InPinValues[0];

			int32 Return = InPinValues[0];
			for (int32 i = 1; i < InPinValues.Num(); i++)
			{
				Return = Return && InPinValues[i];
			}

			return Return;
	});
	OpInfoMap.Add(Op->Name) = Idx;

	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = BoolCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "LogicOr Name", "Logic OR");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "LogicOr Desc", "Logic = A || B");
	Op->Keywords = FText::FromString(TEXT("||"));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, BoolType, AText, AText, Default_BoolZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, BoolType, BText, BText, Default_BoolOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, BoolType, ResultText, ResultText, Default_BoolOne, TEXT("{0} || {1}")));
	Op->BuildName(TEXT("LogicOr"), BoolCategoryName);
	Op->bSupportsAddedInputs = true;
	Op->AddedInputTypeRestrictions.Add(BoolType);
	Op->AddedInputFormatting = TEXT("{A} || {B}");
	Op->bSupportsStaticResolution = true;
	Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
	{
		if (InPinValues.Num() == 0)
			return 0;
		if (InPinValues.Num() == 1)
			return InPinValues[0];

		int32 Return = InPinValues[0];
		for (int32 i = 1; i < InPinValues.Num(); i++)
		{
			Return = Return || InPinValues[i];
		}

		return Return;
	});
	OpInfoMap.Add(Op->Name) = Idx;

	

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = BoolCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "LogicNot Name", "Logic NOT");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "LogicNot Desc", "Result = !B");
	Op->Keywords = FText::FromString(TEXT("!"));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, BoolType, AText, AText, Default_BoolOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, BoolType, ResultText, ResultText, Default_BoolOne, TEXT("!{0}")));
	Op->BuildName(TEXT("LogicNot"), BoolCategoryName);
	Op->bSupportsStaticResolution = true;
	Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
	{
		if (InPinValues.Num() != 1)
			return 0;

		int32 Return = !InPinValues[0];
		return Return;
	});
	OpInfoMap.Add(Op->Name) = Idx;


	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = BoolCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "LogicEq Name", "Bool Equal");
	Op->CompactName = FText::FromString("==");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "LogicEq Desc", "Result = A == B");
	Op->Keywords = FText::FromString(TEXT("=="));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, BoolType, AText, AText, Default_BoolZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, BoolType, BText, BText, Default_BoolOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, BoolType, ResultText, ResultText, Default_BoolOne, TEXT("NiagaraAll({0} == {1})")));
	Op->BuildName(TEXT("LogicEq"), BoolCategoryName);
	Op->bSupportsStaticResolution = true;
	Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
	{
		if (InPinValues.Num() != 2)
			return 0;

		int32 Return = InPinValues[0] == InPinValues[1];
		return Return;
	});
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = BoolCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "LogicNEq Name", "Bool Not Equal");
	Op->CompactName = FText::FromString("!=");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "LogicNEq Desc", "Result = A != B");
	Op->Keywords = FText::FromString(TEXT("!="));
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, BoolType, AText, AText, Default_BoolZero));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, BoolType, BText, BText, Default_BoolOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, BoolType, ResultText, ResultText, Default_BoolOne, TEXT("NiagaraAll({0} != {1})")));
	Op->BuildName(TEXT("LogicNEq"), BoolCategoryName);
	Op->bSupportsStaticResolution = true;
	Op->StaticVariableResolveFunction.BindLambda([=](const TArray<int32>& InPinValues)
	{
		if (InPinValues.Num() != 2)
			return 0;

		int32 Return = InPinValues[0] != InPinValues[1];
		return Return;
	});
	OpInfoMap.Add(Op->Name) = Idx;



	//////////////////////////////////////////////////////////////////////////
	//Matrix only ops
	FString Default_MatrixZero(TEXT(
		"0.0,0.0,0.0,0.0,\
		0.0,0.0,0.0,0.0,\
		0.0,0.0,0.0,0.0,\
		0.0,0.0,0.0,0.0"));

	FString Default_MatrixOne(TEXT(
		"1.0,0.0,0.0,0.0,\
		0.0,1.0,0.0,0.0,\
		0.0,0.0,1.0,0.0,\
		0.0,0.0,0.0,1.0"));

	FText MatrixCategory = NSLOCTEXT("NiagaraOpInfo", "MatrixOpCategory", "Matrix");
	FString MatrixCategoryName(TEXT("Matrix"));
	FNiagaraTypeDefinition MatrixType = FNiagaraTypeDefinition::GetMatrix4Def();

	static FName M(TEXT("M"));
	static FText MText = NSLOCTEXT("NiagaraOpInfo", "Matrix Param", "M");

	static FName V(TEXT("V"));
	static FText VText = NSLOCTEXT("NiagaraOpInfo", "Vector Param", "V");

	FString Default_VectorOne(TEXT("1.0,1.0,1.0"));
	FString Default_Vector4One(TEXT("1.0,1.0,1.0,1.0"));
	FString Default_VectorX(TEXT("1.0,0.0,0.0"));
	FString Default_VectorY(TEXT("0.0,1.0,0.0"));
	FString Default_VectorZ(TEXT("0.0,0.0,1.0"));

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = MatrixCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Transpose Name", "Transpose");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "Transpose Desc", "Returns the trasnpose of the passd matrix.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(M, MatrixType, MText, MText, Default_MatrixOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, MatrixType, ResultText, ResultText, Default_MatrixOne, TEXT("transpose({0})")));
	Op->BuildName(TEXT("Transpose"), MatrixCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = MatrixCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Row0 Name", "Row 0");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "Row0 Desc", "Returns Row 0 of this matrix.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(M, MatrixType, MText, MText, Default_MatrixOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetVec4Def(), ResultText, ResultText, Default_MatrixOne, TEXT("{0}[0]")));
	Op->BuildName(TEXT("Row0"), MatrixCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = MatrixCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Row1 Name", "Row 1");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "Row1 Desc", "Returns Row 1 of this matrix.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(M, MatrixType, MText, MText, Default_MatrixOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetVec4Def(), ResultText, ResultText, Default_MatrixOne, TEXT("{0}[1]")));
	Op->BuildName(TEXT("Row1"), MatrixCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = MatrixCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Row2 Name", "Row 2");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "Row2 Desc", "Returns Row 2 of this matrix.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(M, MatrixType, MText, MText, Default_MatrixOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetVec4Def(), ResultText, ResultText, Default_MatrixOne, TEXT("{0}[2]")));
	Op->BuildName(TEXT("Row2"), MatrixCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = MatrixCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Row3 Name", "Row 3");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "Row3 Desc", "Returns Row 3 of this matrix.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(M, MatrixType, MText, MText, Default_MatrixOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetVec4Def(), ResultText, ResultText, Default_MatrixOne, TEXT("{0}[3]")));
	Op->BuildName(TEXT("Row3"), MatrixCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = MatrixCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "MatrixMatrix Mul Name", "Multiply (Matrix * Matrix)");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "MatrixMatrix Desc", "Multiplies one matrix by another.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, MatrixType, AText, AText, Default_MatrixOne));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, MatrixType, BText, BText, Default_MatrixOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, MatrixType, ResultText, ResultText, Default_MatrixOne, TEXT("mul({0},{1})")));
	Op->BuildName(TEXT("MatrixMultiply"), MatrixCategoryName);
	Op->bSupportsAddedInputs = true;
	Op->AddedInputTypeRestrictions.Add(MatrixType);
	Op->AddedInputFormatting = TEXT("mul({A},{B})");
	OpInfoMap.Add(Op->Name) = Idx;
	
	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = MatrixCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "MatrixVector Mul Name", "Multiply (Matrix * Vector4)");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "MatrixVector Mul Desc", "Multiplies a matrix by a vector4.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(M, MatrixType, MText, MText, Default_MatrixOne));
	Op->Inputs.Add(FNiagaraOpInOutInfo(V, FNiagaraTypeDefinition::GetVec4Def(), VText, VText, Default_VectorOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetVec4Def(), ResultText, ResultText, Default_VectorOne, TEXT("mul({1},{0})")));
	Op->BuildName(TEXT("MatrixVectorMultiply"), MatrixCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = MatrixCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "TransformPosition Name", "Matrix Transform Position");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "TransformPosition Desc", "Transforms a position vector, meaning it multiplies it with the full matrix, including translation.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(M, MatrixType, MText, MText, Default_MatrixOne));
	Op->Inputs.Add(FNiagaraOpInOutInfo(V, FNiagaraTypeDefinition::GetPositionDef(), VText, VText, Default_VectorOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetPositionDef(), ResultText, ResultText, Default_VectorOne, TEXT("mul(float4({1},1.0),{0}).xyz")));
	Op->BuildName(TEXT("TransformPosition"), MatrixCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = MatrixCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "TransformVector Name", "Matrix Transform Vector");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "TransformVector Desc", "Transforms a Vector3 as a vector, meaning it does not apply the translation part of the matrix.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(M, MatrixType, MText, MText, Default_MatrixOne));
	Op->Inputs.Add(FNiagaraOpInOutInfo(V, FNiagaraTypeDefinition::GetVec3Def(), VText, VText, Default_VectorOne));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetVec3Def(), ResultText, ResultText, Default_VectorOne, TEXT("mul(float4({1},0.0),{0}).xyz")));
	Op->BuildName(TEXT("TransformVector"), MatrixCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	/* Vector3 only ops **/
	//TODO: Replace all categories with "Math" or something like that?
	FString Vec3CategoryName(TEXT("Vector3"));
	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = NumericCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "Vector Cross Name", "Cross");
	Op->CompactName = FText::FromString("Cross");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "Vector Cross Desc", "Cross product of two vectors.");
	Op->Inputs.Add(FNiagaraOpInOutInfo(A, FNiagaraTypeDefinition::GetVec3Def(), AText, AText, Default_VectorX));
	Op->Inputs.Add(FNiagaraOpInOutInfo(B, FNiagaraTypeDefinition::GetVec3Def(), BText, BText, Default_VectorY));
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetVec3Def(), ResultText, ResultText, Default_VectorZ, TEXT("cross({0},{1})")));
	Op->BuildName(TEXT("Cross"), Vec3CategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	FText UtilCategory = NSLOCTEXT("NiagaraOpInfo", "UtilOpCategory", "Utility");
	FString UtilCategoryName(TEXT("Util"));
	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = UtilCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "ExecIndex Name", "Execution Index");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "ExexIndex Desc", "Returns the index of this particle in the current execution. For example, in a spawn script this gives the index of the particle being spawned which can be used to interpolate it's position.");	
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, IntType, ResultText, ResultText, Default_IntZero, TEXT("ExecIndex()")));
	Op->BuildName(TEXT("ExecIndex"), UtilCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;

	Idx = OpInfos.AddDefaulted();
	Op = &OpInfos[Idx];
	Op->Category = UtilCategory;
	Op->FriendlyName = NSLOCTEXT("NiagaraOpInfo", "SpawnInterp Name", "Spawn Interpolation");
	Op->Description = NSLOCTEXT("NiagaraOpInfo", "SpawnInterp Desc", "Returns the fraction used for interpolated spawning. i.e. A fraction defining where this particle was spawned between this frame and the last.");
	Op->Outputs.Add(FNiagaraOpInOutInfo(Result, FNiagaraTypeDefinition::GetFloatDef(), ResultText, ResultText, Default_IntZero, TEXT("GetSpawnInterpolation()")));
	Op->BuildName(TEXT("SpawnInterpolation"), UtilCategoryName);
	OpInfoMap.Add(Op->Name) = Idx;
}
END_FUNCTION_BUILD_OPTIMIZATION
////////////////////////////////////////////////////////////////////////// 


///////////////////////////////////////////////////////////////////////////////
/// UActorFactoryNiagara													///
///////////////////////////////////////////////////////////////////////////////
UActorFactoryNiagara::UActorFactoryNiagara(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("NiagaraSystemDisplayName", "NiagaraSystem");
	NewActorClass = ANiagaraActor::StaticClass();
}

bool UActorFactoryNiagara::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UNiagaraSystem::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoSystem", "A valid Niagara System must be specified.");
		return false;
	}

	const UNiagaraSettings* NiagarSettings = GetDefault<UNiagaraSettings>();
	if (NiagarSettings && !NiagarSettings->bAllowCreateActorFromSystemWithNoEffectType)
	{
		const UNiagaraSystem* NiagaraSystem = Cast<UNiagaraSystem>(AssetData.GetAsset());
		if (NiagaraSystem && NiagaraSystem->GetEffectType() == nullptr)
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "RequiredEffectType", "An effect type is required to create actor instances.");
			return false;
		}
	}

	return true;
}

void UActorFactoryNiagara::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UNiagaraSystem* System = CastChecked<UNiagaraSystem>(Asset);
	ANiagaraActor* NiagaraActor = CastChecked<ANiagaraActor>(NewActor);

	// Term Component
	NiagaraActor->GetNiagaraComponent()->UnregisterComponent();

	// Change properties
	NiagaraActor->GetNiagaraComponent()->SetAsset(System);

	// if we're created by Kismet on the server during gameplay, we need to replicate the emitter
	if (NiagaraActor->GetWorld()->HasBegunPlay() && NiagaraActor->GetWorld()->GetNetMode() != NM_Client)
	{
		NiagaraActor->SetReplicates(true);
		NiagaraActor->bAlwaysRelevant = true;
		NiagaraActor->SetNetUpdateFrequency(0.1f); // could also set bNetTemporary but LD might further trigger it or something
	}

	// Init Component
	NiagaraActor->GetNiagaraComponent()->RegisterComponent();
}

UObject* UActorFactoryNiagara::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	ANiagaraActor* NewActor = CastChecked<ANiagaraActor>(Instance);
	if (NewActor->GetNiagaraComponent())
	{
		return NewActor->GetNiagaraComponent()->GetAsset();
	}
	else
	{
		return NULL;
	}
}


///////////////////////////////////////////////////////////////////////////////
/// INiagaraScriptGraphFocusInfo											///
///////////////////////////////////////////////////////////////////////////////
INiagaraScriptGraphFocusInfo::~INiagaraScriptGraphFocusInfo()
{
	//Stand-in definition for abstract INiagaraScriptGraphFocusInfo's pure virtual dtor
}

#undef LOCTEXT_NAMESPACE

