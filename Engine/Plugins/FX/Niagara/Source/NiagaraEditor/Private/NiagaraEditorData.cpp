// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorData.h"

#include "NiagaraConstants.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraParameterDefinitionsBase.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraScriptVariable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEditorData)


void UNiagaraEditorParametersAdapter::PostLoad()
{
	Super::PostLoad();

	SetFlags(RF_Transactional);
	
	// Remove null entries from EditorOnlyScriptVars predating null checks on add methods.
	for (int32 Idx = EditorOnlyScriptVars.Num() - 1; Idx > -1; --Idx)
	{
		if (EditorOnlyScriptVars[Idx] == nullptr)
		{
			EditorOnlyScriptVars.RemoveAtSwap(Idx);
		}
	}

	TArray<FNiagaraVariable> OldTypes = FNiagaraConstants::GetOldPositionTypeVariables();
	for (FNiagaraVariable OldVarType : OldTypes)
	{
		for (const TObjectPtr<UNiagaraScriptVariable>& ScriptVariable : EditorOnlyScriptVars)
		{
			if (ScriptVariable && ScriptVariable->Variable == OldVarType)
			{
				ScriptVariable->Variable.SetType(FNiagaraTypeDefinition::GetPositionDef());
			}
		}
	}
}

TArray<TTuple<FName /*SyncedOldName*/, FName /*SyncedNewName*/>> UNiagaraEditorParametersAdapter::SynchronizeParametersWithParameterDefinitions(
	const TArray<UNiagaraParameterDefinitionsBase*> TargetDefinitions,
	const TArray<UNiagaraParameterDefinitionsBase*> AllDefinitions,
	const TSet<FGuid>& AllDefinitionsParameterIds,
	INiagaraParameterDefinitionsSubscriber* Subscriber,
	FSynchronizeWithParameterDefinitionsArgs Args)
{
	TArray<UNiagaraScriptVariable*> TargetScriptVariables;
	TArray<TTuple<FName, FName>> OldToNewNameArr;

	// Filter script variables that will be synchronized if specific script variable ids are specified.
	if (Args.SpecificDestScriptVarIds.Num() > 0)
	{
		TargetScriptVariables = EditorOnlyScriptVars.FilterByPredicate([&Args](const UNiagaraScriptVariable* DestScriptVar) { return Args.SpecificDestScriptVarIds.Contains(DestScriptVar->Metadata.GetVariableGuid()); });
	}
	else
	{
		TargetScriptVariables = EditorOnlyScriptVars;
	}

	// Get all script variables from target definitions.
	TArray<const UNiagaraScriptVariable*> TargetLibraryScriptVariables;
	for (const UNiagaraParameterDefinitionsBase* TargetParameterDefinitionsItr : TargetDefinitions)
	{
		TargetLibraryScriptVariables.Append(CastChecked<const UNiagaraParameterDefinitions>(TargetParameterDefinitionsItr)->GetParametersConst());
	}

	auto GetTargetDefinitionScriptVarWithSameId = [&TargetLibraryScriptVariables](const UNiagaraScriptVariable* GraphScriptVar)->const UNiagaraScriptVariable* {
		const FGuid& GraphScriptVarId = GraphScriptVar->Metadata.GetVariableGuid();
		if (const UNiagaraScriptVariable* const* FoundLibraryScriptVarPtr = TargetLibraryScriptVariables.FindByPredicate([GraphScriptVarId](const UNiagaraScriptVariable* LibraryScriptVar) { return LibraryScriptVar->Metadata.GetVariableGuid() == GraphScriptVarId; }))
		{
			return *FoundLibraryScriptVarPtr;
		}
		return nullptr;
	};

	// If subscribing all name match parameters; 
	// If a destination parameter has the same name as a source parameter, create a subscription for the source parameter definition. 
	// Retain the destination parameter default value if it does not match the source parameters.
	if (Args.bSubscribeAllNameMatchParameters)
	{
		// Get all script variables from all definitions.
		TArray<const UNiagaraScriptVariable*> AllDefinitionsScriptVariables;
		for (const UNiagaraParameterDefinitionsBase* AllDefinitionsItr : AllDefinitions)
		{
			AllDefinitionsScriptVariables.Append(CastChecked<const UNiagaraParameterDefinitions>(AllDefinitionsItr)->GetParametersConst());
		}

		auto GetDefinitionScriptVarWithSameNameAndType = [&AllDefinitionsScriptVariables](const UNiagaraScriptVariable* GraphScriptVar)->const UNiagaraScriptVariable* {
			if (const UNiagaraScriptVariable* const* FoundLibraryScriptVarPtr = AllDefinitionsScriptVariables.FindByPredicate([&GraphScriptVar](const UNiagaraScriptVariable* LibraryScriptVar) { return LibraryScriptVar->Variable == GraphScriptVar->Variable; }))
			{
				return *FoundLibraryScriptVarPtr;
			}
			return nullptr;
		};

		for (UNiagaraScriptVariable* TargetScriptVar : TargetScriptVariables)
		{
			if (const UNiagaraScriptVariable* LibraryScriptVar = GetDefinitionScriptVarWithSameNameAndType(TargetScriptVar))
			{
				// Add the found definition script var as a target script var so that it can be synchronized with later.
				TargetLibraryScriptVariables.Add(LibraryScriptVar);

				const bool bDoNotAssetIfAlreadySubscribed = true;
				Subscriber->SubscribeToParameterDefinitions(CastChecked<UNiagaraParameterDefinitions>(LibraryScriptVar->GetOuter()), bDoNotAssetIfAlreadySubscribed);
				TargetScriptVar->SetIsSubscribedToParameterDefinitions(true);
				TargetScriptVar->Metadata.SetVariableGuid(LibraryScriptVar->Metadata.GetVariableGuid());
				if (UNiagaraScriptVariable::DefaultsAreEquivalent(TargetScriptVar, LibraryScriptVar) == false)
				{
					// Preserve the TargetScriptVars default value if it is not equivalent to prevent breaking changes from subscribing new parameters.
					TargetScriptVar->SetIsOverridingParameterDefinitionsDefaultValue(true);
				}
				SynchronizeEditorOnlyScriptVar(LibraryScriptVar, TargetScriptVar);
			}
		}
	}

	for (UNiagaraScriptVariable* TargetScriptVar : TargetScriptVariables)
	{
		if (TargetScriptVar->GetIsSubscribedToParameterDefinitions())
		{
			if (const UNiagaraScriptVariable* TargetLibraryScriptVar = GetTargetDefinitionScriptVarWithSameId(TargetScriptVar))
			{
				TOptional<TTuple<FName, FName>> OptionalOldToNewName = SynchronizeEditorOnlyScriptVar(TargetLibraryScriptVar, TargetScriptVar);
				if (OptionalOldToNewName.IsSet())
				{
					OldToNewNameArr.Add(OptionalOldToNewName.GetValue());
				}
			}
			else if (AllDefinitionsParameterIds.Contains(TargetScriptVar->Metadata.GetVariableGuid()) == false)
			{
				// ScriptVar is marked as being sourced from a parameter definitions but no matching library script variables were found, break the link to the parameter definitions for ScriptVar.
				TargetScriptVar->SetIsSubscribedToParameterDefinitions(false);
			}
		}
	}

	return OldToNewNameArr;
}

TOptional<TTuple<FName /*SyncedOldName*/, FName /*SyncedNewName*/>> UNiagaraEditorParametersAdapter::SynchronizeEditorOnlyScriptVar(const UNiagaraScriptVariable* SourceScriptVar, UNiagaraScriptVariable* DestScriptVar /*= nullptr*/)
{
	if (DestScriptVar == nullptr)
	{
		const FGuid& SourceScriptVarId = SourceScriptVar->Metadata.GetVariableGuid();
		TObjectPtr<UNiagaraScriptVariable>* ScriptVarPtr = EditorOnlyScriptVars.FindByPredicate([&SourceScriptVarId](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Metadata.GetVariableGuid() == SourceScriptVarId; });
		if (ScriptVarPtr == nullptr)
		{
			// Failed to find a DestScriptVar with an Id matching that of SourceScriptVar.
			return TOptional<TTuple<FName, FName>>();
		}
		DestScriptVar = *ScriptVarPtr;
	}

	if (DestScriptVar->GetChangeId() != SourceScriptVar->GetChangeId())
	{
		const FName OldParameterName = DestScriptVar->Variable.GetName();
		const FName NewParameterName = SourceScriptVar->Variable.GetName();

		DestScriptVar->Variable = SourceScriptVar->Variable;
		DestScriptVar->Metadata.Description = SourceScriptVar->Metadata.Description;
		DestScriptVar->SetChangeId(SourceScriptVar->GetChangeId());

		if (OldParameterName != NewParameterName)
		{
			return TOptional<TTuple<FName, FName>>(TTuple<FName, FName>(OldParameterName, NewParameterName));
		}
	}
	return TOptional<TTuple<FName, FName>>();
}

bool UNiagaraEditorParametersAdapter::SynchronizeParameterDefinitionsScriptVariableRemoved(const FGuid& RemovedScriptVarId)
{
	for (UNiagaraScriptVariable* ScriptVar : EditorOnlyScriptVars)
	{
		if (ScriptVar->Metadata.GetVariableGuid() == RemovedScriptVarId)
		{
			ScriptVar->SetIsSubscribedToParameterDefinitions(false);
			return true;
		}
	}
	return false;
}

