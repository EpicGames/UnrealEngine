// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorModule.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "NiagaraModule.h"
#include "NiagaraEditorTickables.h"
#include "Modules/ModuleManager.h"
#include "AssetToolsModule.h"
#include "ContentBrowserMenuContexts.h"
#include "ISequencerModule.h"
#include "ISettingsModule.h"
#include "SequencerChannelInterface.h"
#include "SequencerSettings.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Stats/Stats.h"
#include "Subsystems/ImportSubsystem.h"
#include "UObject/UObjectThreadContext.h"
#include "Misc/ScopedSlowTask.h"

#include "EdGraphSchema_Niagara.h"
#include "EdGraphUtilities.h"
#include "KismetPins/SGraphPinBool.h"
#include "KismetPins/SGraphPinColor.h"
#include "KismetPins/SGraphPinEnum.h"
#include "KismetPins/SGraphPinExec.h"
#include "KismetPins/SGraphPinInteger.h"
#include "KismetPins/SGraphPinNum.h"
#include "KismetPins/SGraphPinVector.h"
#include "KismetPins/SGraphPinVector2D.h"
#include "KismetPins/SGraphPinVector4.h"
#include "NiagaraNodeAssignment.h"
#include "SGraphPin.h"
#include "Widgets/SNiagaraGraphPinAdd.h"
#include "SNiagaraGraphPinNumeric.h"

#include "TypeEditorUtilities/NiagaraFloatTypeEditorUtilities.h"
#include "TypeEditorUtilities/NiagaraIntegerTypeEditorUtilities.h"
#include "TypeEditorUtilities/NiagaraEnumTypeEditorUtilities.h"
#include "TypeEditorUtilities/NiagaraBoolTypeEditorUtilities.h"
#include "TypeEditorUtilities/NiagaraVectorTypeEditorUtilities.h"
#include "TypeEditorUtilities/NiagaraColorTypeEditorUtilities.h"
#include "TypeEditorUtilities/NiagaraMatrixTypeEditorUtilities.h"
#include "TypeEditorUtilities/NiagaraDataInterfaceCurveTypeEditorUtilities.h"
#include "TypeEditorUtilities/NiagaraDistributionPropertyEditorUtilities.h"
#include "TypeEditorUtilities/NiagaraCascadeDistributionExternalObjectUtilities.h"

#include "NiagaraActorPlacement.h"
#include "NiagaraSystemCompilingManager.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorCommands.h"
#include "PropertyEditorModule.h"
#include "NiagaraSettings.h"
#include "NiagaraShaderModule.h"
#include "NiagaraSystemEmitterState.h"

#include "NiagaraDataInterfaceArray.h"
#include "NiagaraDataInterfaceCurve.h"
#include "NiagaraDataInterfaceVector2DCurve.h"
#include "NiagaraDataInterfaceVectorCurve.h"
#include "NiagaraDataInterfaceVector4Curve.h"
#include "NiagaraDataInterfaceColorCurve.h"
#include "DataInterface/NiagaraDataInterfaceDataChannelRead.h"
#include "DataInterface/NiagaraDataInterfaceDataChannelWrite.h"
#include "DataInterface/NiagaraDataInterfaceDataTable.h"
#include "DataInterface/NiagaraDataInterfaceDynamicMesh.h"
#include "DataInterface/NiagaraDataInterfaceMemoryBuffer.h"
#include "DataInterface/NiagaraDataInterfaceSimpleCounter.h"
#include "NiagaraDataInterfaceRenderTarget2D.h"
#include "NiagaraDataInterfaceRenderTargetVolume.h"

#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraPlaceholderDataInterfaceManager.h"
#include "TNiagaraGraphPinEditableName.h"
#include "UObject/Class.h"
#include "UObject/ICookInfo.h"
#include "NiagaraScriptMergeManager.h"
#include "NiagaraDigestDatabase.h"
#include "NiagaraEmitter.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSystem.h"
#include "NiagaraTypes.h"
#include "NiagaraSystemFactoryNew.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraClipboard.h"
#include "NiagaraMessageManager.h"
#include "NiagaraMessages.h"
#include "NiagaraComponentBroker.h"
#include "NiagaraBakerSettings.h"
#include "ContentBrowserModule.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraActions.h"
#include "NiagaraSystemStateDataResolver.h"

#include "MovieScene/Parameters/MovieSceneNiagaraBoolParameterTrack.h"
#include "MovieScene/Parameters/MovieSceneNiagaraFloatParameterTrack.h"
#include "MovieScene/Parameters/MovieSceneNiagaraIntegerParameterTrack.h"
#include "MovieScene/Parameters/MovieSceneNiagaraVectorParameterTrack.h"
#include "MovieScene/Parameters/MovieSceneNiagaraColorParameterTrack.h"

#include "Sections/MovieSceneBoolSection.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Sections/MovieSceneIntegerSection.h"
#include "Sections/MovieSceneVectorSection.h"
#include "Sections/MovieSceneColorSection.h"

#include "Sequencer/NiagaraSequence/Sections/MovieSceneNiagaraEmitterSection.h"
#include "Sequencer/NiagaraSequence/NiagaraEmitterTrackEditor.h"
#include "Sequencer/LevelSequence/NiagaraSystemTrackEditor.h"

#include "ISequencerSection.h"
#include "Sections/BoolPropertySection.h"
#include "Sections/ColorPropertySection.h"

#include "Customizations/NiagaraComponentDetails.h"
#include "Customizations/NiagaraEditorPreviewActorCustomization.h"
#include "Customizations/NiagaraFunctionCallNodeDetails.h"
#include "Customizations/NiagaraParameterBindingCustomization.h"
#include "Customizations/NiagaraPlatformSetCustomization.h"
#include "Customizations/NiagaraScriptVariableCustomization.h"
#include "Customizations/NiagaraScriptDetails.h"
#include "Customizations/NiagaraStaticSwitchNodeDetails.h"
#include "Customizations/NiagaraTypeCustomizations.h"
#include "Customizations/NiagaraComponentRendererPropertiesDetails.h"
#include "Customizations/NiagaraDataInterfaceEmitterBindingCustomization.h"
#include "Customizations/NiagaraDebugHUDCustomization.h"
#include "Customizations/NiagaraBakerSettingsDetails.h"
#include "Customizations/NiagaraOutlinerCustomization.h"
#include "Customizations/NiagaraSimulationStageCustomization.h"
#include "Customizations/NiagaraDataChannelDetails.h"
#include "Customizations/SimCache/NiagaraArraySimCacheVisualizer.h"
#include "Customizations/SimCache/FNiagaraDataChannelSimCacheVisualizer.h"
#include "Customizations/SimCache/NiagaraDynamicMeshSimCacheVisualizer.h"
#include "Customizations/SimCache/NiagaraMemoryBufferSimCacheVisualizer.h"
#include "Customizations/SimCache/NiagaraRenderTargetSimCacheVisualizer.h"
#include "Customizations/SimCache/NiagaraRenderTargetVolumeSimCacheVisualizer.h"
#include "Customizations/SimCache/NiagaraSimpleCounterSimCacheVisualizer.h"

#include "NiagaraComponent.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraScript.h"
#include "NiagaraCommon.h"
#include "NiagaraComponentRendererProperties.h"
#include "NiagaraMeshRendererProperties.h"

#include "HAL/IConsoleManager.h"
#include "NiagaraThumbnailRenderer.h"
#include "Misc/FeedbackContext.h"
#include "NiagaraNodeFunctionCall.h"
#include "Engine/Selection.h"
#include "NiagaraActor.h"
#include "INiagaraEditorOnlyDataUtlities.h"

#include "AssetCompilingManager.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "FileMediaSource.h"
#include "ILevelSequenceModule.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Containers/Ticker.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Distributions/Distribution.h"

#include "ViewModels/Stack/NiagaraStackObjectIssueGenerator.h"
#include "NiagaraPlatformSet.h"
#include "NiagaraEffectType.h"
#include "Widgets/SNiagaraSystemViewport.h"

#include "LevelEditor.h"
#include "LevelEditorOutlinerSettings.h"
#include "NiagaraAssetTagDefinitions.h"
#include "Filters/CustomClassFilterData.h"

#include "Widgets/SNiagaraDebugger.h"

#include "NiagaraDebugVis.h"
#include "NiagaraPerfBaseline.h"
#include "NiagaraGraphDataCache.h"
#include "NiagaraDecalRendererProperties.h"
#include "NiagaraLightRendererProperties.h"
#include "NiagaraRibbonRendererProperties.h"
#include "NiagaraSpriteRendererProperties.h"
#include "NiagaraVolumeRendererProperties.h"

#include "Engine/AssetManager.h"
#include "NiagaraRecentAndFavoritesManager.h"
#include "Customizations/Stack/NiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters.h"
#include "Stateless/Modules/NiagaraStatelessModule_DynamicMaterialParameters.h"
#include "Widgets/DataChannel/NiagaraDataChannelWizard.h"

#include "TraversalCache/TraversalBuilder.h"
#include "TraversalCache/TraversalCache.h"

#include "UserAssetTagMenuHelpers.h"
#include "Config/TaggedAssetBrowserConfig.h"
#include "UserAssetTagEditorUtilities.h"
#include "Widgets/STaggedAssetBrowser.h"
#include "Widgets/SUserAssetTagBrowserSelectedAssetDetails.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEditorModule)

IMPLEMENT_MODULE( FNiagaraEditorModule, NiagaraEditor );

#define LOCTEXT_NAMESPACE "NiagaraEditorModule"

const FName FNiagaraEditorModule::NiagaraEditorAppIdentifier( TEXT( "NiagaraEditorApp" ) );
const FLinearColor FNiagaraEditorModule::WorldCentricTabColorScale(0.0f, 0.0f, 0.2f, 0.5f);

int32 GbShowNiagaraDeveloperWindows = 0;
static FAutoConsoleVariableRef CVarShowNiagaraDeveloperWindows(
	TEXT("fx.ShowNiagaraDeveloperWindows"),
	GbShowNiagaraDeveloperWindows,
	TEXT("If > 0 the niagara system, emitter, and script editors will show additional developer windows.\nThese windows are for niagara tool development and debugging and editing the data\n directly in these windows can cause instability.\n"),
	ECVF_Default
	);

int32 GbPreloadSelectablePluginAssetsOnDemand = 1;
static FAutoConsoleVariableRef CVarPreloadSelectablePluginAssetsOnDemand(
	TEXT("fx.Niagara.PreloadSelectablePluginAssetsOnDemand"),
	GbPreloadSelectablePluginAssetsOnDemand,
	TEXT("If > 0 then niagara system, emitter, and script assets provided by the niagara plugin will be preloaded when a dialog is opened to select them. This is a temoporary workaround for asset registry issues in cooked editor builds.\n"),
	ECVF_Default);

int32 GbEnableExperimentalInlineDynamicInputs = 0;
static FAutoConsoleVariableRef CVarEnableExperimentalInlineDynamicInputs(
	TEXT("fx.Niagara.EnableExperimentalInlineDynamicInputs"),
	GbEnableExperimentalInlineDynamicInputs,
	TEXT("If > 0 experimental inline editors for dynamic input trees will be available via right click menu in the stack.\n"),
	ECVF_Default);

int32 GbEnableCustomInlineDynamicInputFormats = 1;
static FAutoConsoleVariableRef CVarEnableCustomInlineDynamicInputFormats(
	TEXT("fx.Niagara.EnableCustomInlineDynamicInputFormats"),
	GbEnableCustomInlineDynamicInputFormats,
	TEXT("If > 0 and experimental inline editors for dynamic input trees are enabled, custom formats which are defined on scripts will be applied.\n"),
	ECVF_Default);

static FAutoConsoleCommand LogFolderPermissionsCommand = FAutoConsoleCommand(
	TEXT("fx.Niagara.MigrateInternalTagsToUserAssetTags"),
	TEXT("Finds and resaves Niagara assets within the current Content Browser view (recursive) that were tagged with an internal Niagara Asset Tag (such as Template, Hidden, Deprecated etc.)"),
	FConsoleCommandWithArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args)
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			FContentBrowserItemPath CurrentPath = ContentBrowserModule.Get().GetCurrentPath();
				
			if(CurrentPath.HasInternalPath())
			{
				UE_LOGF(LogNiagaraEditor, Display, "Trying to find tagged Niagara assets under path %ls", *CurrentPath.GetInternalPathString());
				
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
				
				FARFilter Filter;
				Filter.ClassPaths.Add(UNiagaraEmitter::StaticClass()->GetClassPathName());
				Filter.ClassPaths.Add(UNiagaraSystem::StaticClass()->GetClassPathName());
				Filter.PackagePaths.Add(CurrentPath.GetInternalPathName());
				Filter.bRecursivePaths = true;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				for(const FNiagaraAssetTagDefinition* Definition : INiagaraModule::Get().GetInternalAssetTagDefinitions())
				{
					Filter.TagsAndValues.Add(FName(Definition->GetGuidAsString()));
				}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				
				TArray<FAssetData> OutAssetData;
				AssetRegistryModule.Get().GetAssets(Filter, OutAssetData);

				TArray<UPackage*> PackagesToResave;
				Algo::Transform(OutAssetData, PackagesToResave, [](const FAssetData& AssetData)
				{
					return AssetData.GetPackage();
				});

				for(UPackage* Package : PackagesToResave)
				{
					Package->MarkPackageDirty();
				}

				if(PackagesToResave.Num() > 0)
				{
					UEditorLoadingAndSavingUtils::SavePackagesWithDialog(PackagesToResave, false);
				}
				else
				{
					UE_LOGF(LogNiagaraEditor, Warning, "Didn't find any assets to resave under %ls", *CurrentPath.GetInternalPathString());
				}
			}
			else
			{
				UE_LOGF(LogNiagaraEditor, Warning, "No valid path found");	
			}
		}));
// this is required for gpu script compilation ticks
static FNiagaraShaderQueueTickable NiagaraShaderQueueProcessor;

#if !IS_MONOLITHIC
namespace NiagaraDebugVisHelper
{
	UE_SELECT_ANY const FNiagaraTypeRegistry*& GTypeRegistrySingletonPtr = GCoreTypeRegistrySingletonPtr;
}
#endif

//////////////////////////////////////////////////////////////////////////

class FNiagaraEditorOnlyDataUtilities : public INiagaraEditorOnlyDataUtilities
{
	UNiagaraScriptSourceBase* CreateDefaultScriptSource(UObject* InOuter) const override
	{
		return NewObject<UNiagaraScriptSource>(InOuter);
	}

	virtual UNiagaraEditorDataBase* CreateDefaultEditorData(UObject* InOuter) const override
	{
		if (UNiagaraSystem* System = Cast<UNiagaraSystem>(InOuter))
		{
			UNiagaraSystemEditorData* SystemEditorData = NewObject<UNiagaraSystemEditorData>(InOuter);
			SystemEditorData->SetFlags(RF_Transactional);
			SystemEditorData->SynchronizeOverviewGraphWithSystem(*System);
			SystemEditorData->InitOnSyncScriptVariables(*System);
			return SystemEditorData;
		}
		else if(UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(InOuter))
		{
			UNiagaraEmitterEditorData* EmitterEditorData = NewObject<UNiagaraEmitterEditorData>(InOuter);
			EmitterEditorData->SetFlags(RF_Transactional);
			return EmitterEditorData;
		}
		return nullptr;
	}

	virtual UNiagaraEditorParametersAdapterBase* CreateDefaultEditorParameters(UObject* InOuter) const override
	{
		UNiagaraEditorParametersAdapter* Adapter = NewObject<UNiagaraEditorParametersAdapter>(InOuter);
		Adapter->SetFlags(RF_Transactional);
		
		return Adapter;
	}

	virtual UObject::FAssetRegistryTag CreateClassUsageAssetRegistryTag(const UObject* SourceObject) const override
	{
		const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
		return NiagaraEditorSettings->CreateClassUsageAssetRegistryTag(SourceObject);
	}

	virtual UNiagaraMessageDataBase* CreateErrorMessage(UObject* InOuter, FText InMessageShort, FText InMessageLong, FName InTopicName, bool bInAllowDismissal = false) const override
	{
		return CreateMessage(InOuter, ENiagaraMessageSeverity::Error, InMessageShort, InMessageLong, InTopicName, bInAllowDismissal);
	}

	virtual UNiagaraMessageDataBase* CreateWarningMessage(UObject* InOuter, FText InMessageShort, FText InMessageLong, FName InTopicName, bool bInAllowDismissal = false) const override
	{
		return CreateMessage(InOuter, ENiagaraMessageSeverity::Warning, InMessageShort, InMessageLong, InTopicName, bInAllowDismissal);
	}

	UNiagaraMessageDataBase* CreateMessage(UObject* InOuter, ENiagaraMessageSeverity Severity, FText InMessageShort, FText InMessageLong, FName InTopicName, bool bInAllowDismissal) const
	{
		UNiagaraMessageDataText* NewMessage = NewObject<UNiagaraMessageDataText>(InOuter);
		NewMessage->Init(InMessageLong, InMessageShort, Severity, InTopicName);
		NewMessage->SetAllowDismissal(bInAllowDismissal);
		return NewMessage;
	}

	bool IsEditorDataInterfaceInstance(const UNiagaraDataInterface* DataInterface) const override
	{
		return FNiagaraEditorUtilities::IsEditorDataInterfaceInstance(DataInterface);
	}

	UNiagaraDataInterface* GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(const UNiagaraSystem& OwningSystem, UNiagaraDataInterface& EditorDataInterfaceInstance) const override
	{
		return FNiagaraEditorUtilities::GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(OwningSystem, EditorDataInterfaceInstance);
	}

	virtual TOptional<FNiagaraSystemStateData> GetSystemStateData(const UNiagaraSystem& System) const override
	{
		return FNiagaraSystemStateDataResolver::TryResolve(System);
	}

	virtual bool GetResetSimulationOnParameterChange() const override
	{
		const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
		return NiagaraEditorSettings->GetResetSimulationOnChange();
	}

	virtual void MoveUserAssetTagsToMetaData(UObject* NiagaraAsset) const override
	{
		check(NiagaraAsset->IsA<UNiagaraEmitter>() || NiagaraAsset->IsA<UNiagaraSystem>());

		TArray<FNiagaraAssetTagDefinitionReference>* References = nullptr;
		if(UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(NiagaraAsset))
		{
			References = &Emitter->AssetTags_DEPRECATED;
		}
		else if(UNiagaraSystem* System = Cast<UNiagaraSystem>(NiagaraAsset))
		{
			References = &System->AssetTags_DEPRECATED;
		}

		if(References && References->Num() > 0)
		{
			TMap<FGuid, const FNiagaraAssetTagDefinition*> AllTags;
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

			FARFilter Filter;
			Filter.ClassPaths.Add(UDEPRECATED_NiagaraAssetTagDefinitions::StaticClass()->GetClassPathName());

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			for(const FNiagaraAssetTagDefinition* InternalTagDefinition : INiagaraModule::Get().GetInternalAssetTagDefinitions())
			{
				AllTags.Add(InternalTagDefinition->TagGuid, InternalTagDefinition);
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// When cooking, only enumerate on-disk assets- avoids indeterministic results

			TArray<FAssetData> AssetTagDefinitionAssets;
			AssetRegistryModule.Get().GetAssets(Filter, AssetTagDefinitionAssets);

			{
				FCookLoadScope EditorOnlyLoad(ECookLoadType::EditorOnly);
				for (const FAssetData& AssetTagDefinitionAsset : AssetTagDefinitionAssets)
				{
					UDEPRECATED_NiagaraAssetTagDefinitions* TagDefinitionsAsset = Cast<UDEPRECATED_NiagaraAssetTagDefinitions>(AssetTagDefinitionAsset.GetAsset());

					for (const FNiagaraAssetTagDefinition& TagDefinition : TagDefinitionsAsset->GetAssetTagDefinitions())
					{
						AllTags.Add(TagDefinition.TagGuid, &TagDefinition);
					}
				}
			}

			TArray<const FNiagaraAssetTagDefinition*> FoundTagDefinitions;
			for(const FNiagaraAssetTagDefinitionReference& AssetTagDefinitionReference : *References)
			{
				FGuid TagGuid = AssetTagDefinitionReference.GetTagDefinitionReferenceGuid();
				if(AllTags.Contains(TagGuid))
				{
					FoundTagDefinitions.Add(AllTags[TagGuid]);
				}
			}

			for(const FNiagaraAssetTagDefinition* Definition : FoundTagDefinitions)
			{
				if(Definition == nullptr)
				{
					continue;
				}

				UE::UserAssetTags::AddUserAssetTag(NiagaraAsset, FName(Definition->AssetTag.ToString()));
			}

			References->Empty();
		}
	}
};

class FNiagaraScriptGraphPanelPinFactory : public FGraphPanelPinFactory
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<class SGraphPin>, FCreateGraphPin, UEdGraphPin*);

	/** Registers a delegate for creating a pin for a specific type. */
	void RegisterTypePin(const UScriptStruct* Type, FCreateGraphPin CreateGraphPin)
	{
		TypeToCreatePinDelegateMap.Add(Type, CreateGraphPin);
	}

	/** Registers a delegate for creating a pin for for a specific miscellaneous sub category. */
	void RegisterMiscSubCategoryPin(FName SubCategory, FCreateGraphPin CreateGraphPin)
	{
		MiscSubCategoryToCreatePinDelegateMap.Add(SubCategory, CreateGraphPin);
	}

	//~ FGraphPanelPinFactory interface
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override
	{
		TSharedPtr<class SGraphPin> OutPin = InternalCreatePin(InPin);

		if (OutPin.IsValid() && UEdGraphSchema_Niagara::IsStaticPin(InPin))
			OutPin->SetCustomPinIcon(FNiagaraEditorStyle::Get().GetBrush(TEXT("NiagaraEditor.Pins.StaticConnected")), FNiagaraEditorStyle::Get().GetBrush(TEXT("NiagaraEditor.Pins.StaticDisconnected")));
		return OutPin;
	}

	TSharedPtr<class SGraphPin> InternalCreatePin(class UEdGraphPin* InPin) const
	{
		if (const UEdGraphSchema_Niagara* NSchema = Cast<UEdGraphSchema_Niagara>(InPin->GetSchema()))
		{
			if (InPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType || InPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType)
			{
				if (InPin->PinType.PinSubCategoryObject != nullptr && InPin->PinType.PinSubCategoryObject->IsA<UScriptStruct>())
				{
					const UScriptStruct* Struct = CastChecked<const UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get());
					const FCreateGraphPin* CreateGraphPin = TypeToCreatePinDelegateMap.Find(Struct);
					if (CreateGraphPin != nullptr)
					{
						return (*CreateGraphPin).Execute(InPin);
					}
					// Otherwise, fall back to the generic pin for Niagara types. Previous iterations put out an error here, but this 
					// was not correct as the above list is just overrides from the default renamable pin, usually numeric types with their own custom 
					// editors for default values. Things like the parameter map can safely just fall through to the end condition and create a
					// generic renamable pin.
				}
				else
				{
					UE_LOGF(LogNiagaraEditor, Warning, "Pin type is invalid! Pin Name '%ls' Owning Node '%ls'. Turning into standard int definition!", *InPin->PinName.ToString(),
						*InPin->GetOwningNode()->GetFullName());
					InPin->PinType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UScriptStruct*>(FNiagaraTypeDefinition::GetIntStruct()));
					InPin->DefaultValue.Empty();
					return CreatePin(InPin);
				}
			}
			else if (InPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum || InPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum)
			{
				const UEnum* Enum = Cast<const UEnum>(InPin->PinType.PinSubCategoryObject.Get());
				if (Enum == nullptr)
				{
					UE_LOGF(LogNiagaraEditor, Warning, "Pin states that it is of Enum type, but is missing its Enum! Pin Name '%ls' Owning Node '%ls'. Turning into standard int definition!", *InPin->PinName.ToString(),
						*InPin->GetOwningNode()->GetFullName());
					InPin->PinType.PinCategory = UEdGraphSchema_Niagara::PinCategoryType;
					InPin->PinType.PinSubCategoryObject = MakeWeakObjectPtr(const_cast<UScriptStruct*>(FNiagaraTypeDefinition::GetIntStruct()));
					InPin->DefaultValue.Empty();
					return CreatePin(InPin);
				}
				return SNew(TNiagaraGraphPinEditableName<SGraphPinEnum>, InPin);
			}
			else if (InPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryMisc)
			{
				const FCreateGraphPin* CreateGraphPin = MiscSubCategoryToCreatePinDelegateMap.Find(InPin->PinType.PinSubCategory);
				if (CreateGraphPin != nullptr)
				{
					return (*CreateGraphPin).Execute(InPin);
				}
			}

			return SNew(TNiagaraGraphPinEditableName<SGraphPin>, InPin);
		}
		return nullptr;
	}

private:
	TMap<const UScriptStruct*, FCreateGraphPin> TypeToCreatePinDelegateMap;
	TMap<FName, FCreateGraphPin> MiscSubCategoryToCreatePinDelegateMap;
};

void FNiagaraStackObjectCustomizationRegistry::RegisterStackObjectCustomization(UClass& Class, FOnGetStackObjectCustomizationInstance CustomizationFactory)
{
	Customizations.Add(&Class) = CustomizationFactory.Execute();
}

void FNiagaraStackObjectCustomizationRegistry::UnregisterStackObjectCustomization(UClass& Class)
{
	Customizations.Remove(&Class);
}

TSharedPtr<const FNiagaraStackObjectPropertyCustomization> FNiagaraStackObjectCustomizationRegistry::GetCustomizationForStackObject(UNiagaraStackObject& StackObject)
{
	if(Customizations.Contains(StackObject.GetObject()->GetClass()))
	{
		return Customizations[StackObject.GetObject()->GetClass()];
	}

	return nullptr;
}

FNiagaraEditorModule::FNiagaraEditorModule() 
	: SequencerSettings(nullptr)
	, TestCompileScriptCommand(nullptr)
	, DumpCompileIdDataForAssetCommand(nullptr)
	, Clipboard(MakeShared<FNiagaraClipboard>())
	, ReservedParametersManagerSingleton(nullptr)
{
}

void DumpParameterStore(const FNiagaraParameterStore& ParameterStore)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::Get().GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	TArray<FNiagaraVariable> ParameterVariables;
	ParameterStore.GetParameters(ParameterVariables);
	for (const FNiagaraVariable& ParameterVariable : ParameterVariables)
	{
		FString Name = ParameterVariable.GetName().ToString();
		FString Type = ParameterVariable.GetType().GetName();
		FString Value;
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> ParameterTypeUtilities = NiagaraEditorModule.GetTypeUtilities(ParameterVariable.GetType());
		if (ParameterTypeUtilities.IsValid() && ParameterTypeUtilities->CanHandlePinDefaults())
		{
			FNiagaraVariable ParameterVariableWithValue = ParameterVariable;
			ParameterVariableWithValue.SetData(ParameterStore.GetParameterData(ParameterVariable));
			Value = ParameterTypeUtilities->GetPinDefaultStringFromValue(ParameterVariableWithValue);
		}
		else
		{
			Value = "(unsupported)";
		}
		UE_LOGF(LogNiagaraEditor, Log, "%ls\t%ls\t%ls", *Name, *Type, *Value);
	}
}

void DumpRapidIterationParametersForScript(UNiagaraScript* Script, const FString& HeaderName)
{
	UEnum* NiagaraScriptUsageEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Niagara.ENiagaraScriptUsage"), EFindObjectFlags::ExactClass);
	FString UsageName = NiagaraScriptUsageEnum->GetNameByValue((int64)Script->GetUsage()).ToString();
	UE_LOGF(LogNiagaraEditor, Log, "%ls - %ls - %ls", *Script->GetPathName(), *HeaderName, *UsageName);
	DumpParameterStore(Script->RapidIterationParameters);
}

void DumpRapidIterationParametersForEmitter(FVersionedNiagaraEmitter Emitter, const FString& EmitterName)
{
	if (FVersionedNiagaraEmitterData* EmitterData = Emitter.GetEmitterData())
	{
		TArray<UNiagaraScript*> Scripts;
		EmitterData->GetScripts(Scripts, false);
		for (UNiagaraScript* Script : Scripts)
		{
			DumpRapidIterationParametersForScript(Script, EmitterName);
		}
	}
}

void DumpRapidIterationParamersForAsset(const TArray<FString>& Arguments)
{
	if (Arguments.Num() == 1)
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		const FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(Arguments[0]));
		UObject* Asset = AssetData.GetAsset();
		if (Asset != nullptr)
		{
			UNiagaraSystem* SystemAsset = Cast<UNiagaraSystem>(Asset);
			if (SystemAsset != nullptr)
			{
				DumpRapidIterationParametersForScript(SystemAsset->GetSystemSpawnScript(), SystemAsset->GetName());
				DumpRapidIterationParametersForScript(SystemAsset->GetSystemUpdateScript(), SystemAsset->GetName());
				for (const FNiagaraEmitterHandle& EmitterHandle : SystemAsset->GetEmitterHandles())
				{
					DumpRapidIterationParametersForEmitter(EmitterHandle.GetInstance(), EmitterHandle.GetName().ToString());
				}
			}
			else
			{
				UNiagaraEmitter* EmitterAsset = Cast<UNiagaraEmitter>(Asset);
				if (EmitterAsset != nullptr)
				{
					DumpRapidIterationParametersForEmitter(FVersionedNiagaraEmitter(EmitterAsset, EmitterAsset->GetExposedVersion().VersionGuid), EmitterAsset->GetName());
				}
				else
				{
					UE_LOGF(LogNiagaraEditor, Warning, "DumpRapidIterationParameters - Only niagara system and niagara emitter assets are supported");
				}
			}
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Warning, "DumpRapidIterationParameters - Asset not found");
		}
	}
	else
	{
		UE_LOGF(LogNiagaraEditor, Warning, "DumpRapidIterationParameters - Must supply an asset path to dump");
	}
}

void CompileEmitterStandAlone(FVersionedNiagaraEmitter VersionedEmitter, TSet<FVersionedNiagaraEmitter>& InOutCompiledEmitters)
{
	if (InOutCompiledEmitters.Contains(VersionedEmitter) == false)
	{
		FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
		if (EmitterData && EmitterData->GetParent().Emitter != nullptr)
		{
			// If the emitter has a parent emitter make sure to compile that one first.
			CompileEmitterStandAlone(EmitterData->GetParent(), InOutCompiledEmitters);

			if (VersionedEmitter.Emitter->IsSynchronizedWithParent() == false)
			{
				// If compiling the parent caused it to become out of sync with the current emitter merge in changes before compiling.
				VersionedEmitter.Emitter->MergeChangesFromParent();
			}
		}

		VersionedEmitter.Emitter->MarkPackageDirty();
		UNiagaraSystem* TransientSystem = NewObject<UNiagaraSystem>(GetTransientPackage(), FName("StandaloneEmitter_TempSystem"), RF_Transient);
		UNiagaraSystemFactoryNew::InitializeSystem(TransientSystem, true);
		TransientSystem->AddEmitterHandle(*VersionedEmitter.Emitter, TEXT("Emitter"), VersionedEmitter.Version);
		FNiagaraStackGraphUtilities::RebuildEmitterNodes(*TransientSystem);
		TransientSystem->RequestCompile(false);
		TransientSystem->WaitForCompilationComplete();

		InOutCompiledEmitters.Add(VersionedEmitter);
	}
}

void PreventSystemRecompile(FAssetData SystemAsset, TSet<FVersionedNiagaraEmitter>& InOutCompiledEmitters)
{
	UNiagaraSystem* System = Cast<UNiagaraSystem>(SystemAsset.GetAsset());
	if (System != nullptr)
	{
		for (const FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
		{
			CompileEmitterStandAlone(EmitterHandle.GetInstance(), InOutCompiledEmitters);
		}
		
		System->MarkPackageDirty();
		System->RequestCompile(false);
		System->WaitForCompilationComplete();
	}
}

void PreventSystemRecompile(const TArray<FString>& Arguments)
{
	if (Arguments.Num() > 0)
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FAssetData SystemAsset = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(Arguments[0]));
		if (SystemAsset.IsValid() == false)
		{
			TArray<FAssetData> AssetsInPackage;
			AssetRegistryModule.Get().GetAssetsByPackageName(*Arguments[0], AssetsInPackage);
			if (AssetsInPackage.Num() == 1)
			{
				SystemAsset = AssetsInPackage[0];
			}
		}
		TSet<FVersionedNiagaraEmitter> CompiledEmitters;
		PreventSystemRecompile(SystemAsset, CompiledEmitters);
	}
}

void PreventAllSystemRecompiles()
{
	const FText SlowTaskText = NSLOCTEXT("NiagaraEditor", "PreventAllSystemRecompiles", "Refreshing all systems to prevent recompiles.");
	GWarn->BeginSlowTask(SlowTaskText, true, true);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TArray<FAssetData> SystemAssets;
	AssetRegistryModule.Get().GetAssetsByClass(UNiagaraSystem::StaticClass()->GetClassPathName(), SystemAssets);

	TSet<FVersionedNiagaraEmitter> CompiledEmitters;
	int32 ItemIndex = 0;
	for (FAssetData& SystemAsset : SystemAssets)
	{
		if (GWarn->ReceivedUserCancel())
		{
			return;
		}
		GWarn->UpdateProgress(ItemIndex++, SystemAssets.Num());

		PreventSystemRecompile(SystemAsset, CompiledEmitters);
	}

	GWarn->EndSlowTask();
}

void UpgradeAllNiagaraAssets()
{
	//First Load All Niagara Assets.
	const FText SlowTaskText_Load = NSLOCTEXT("NiagaraEditor", "UpgradeAllNiagaraAssets_Load", "Loading all Niagara Assets ready to upgrade.");
	GWarn->BeginSlowTask(SlowTaskText_Load, true, true);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TArray<FAssetData> SystemAssets;
	AssetRegistryModule.Get().GetAssetsByClass(UNiagaraSystem::StaticClass()->GetClassPathName(), SystemAssets);

	TArray<UNiagaraSystem*> Systems;
	Systems.Reserve(SystemAssets.Num());
	TSet<UNiagaraEmitter*> CompiledEmitters;
	int32 ItemIndex = 0;
	for (FAssetData& SystemAsset : SystemAssets)
	{
		if (GWarn->ReceivedUserCancel())
		{
			return;
		}
		GWarn->UpdateProgress(ItemIndex++, SystemAssets.Num());

		UNiagaraSystem* System = Cast<UNiagaraSystem>(SystemAsset.GetAsset());
		if (System != nullptr)
		{
			Systems.Add(System);
		}
	}

	GWarn->EndSlowTask();

	//////////////////////////////////////////////////////////////////////////

	//Now process any data that needs to be updated.
	const FText SlowTaskText_Upgrade = NSLOCTEXT("NiagaraEditor", "UpgradeAllNiagaraAssets_Upgrade", "Upgrading All Niagara Assets.");
	GWarn->BeginSlowTask(SlowTaskText_Upgrade, true, true);

	//Upgrade any data interface function call nodes.
	TArray<UObject*> FunctionCallNodes;
	GetObjectsOfClass(UNiagaraNodeFunctionCall::StaticClass(), FunctionCallNodes);
	ItemIndex = 0;
	for (UObject* Object : FunctionCallNodes)
	{
		if (GWarn->ReceivedUserCancel())
		{
			return;
		}

		if (UNiagaraNodeFunctionCall* FuncCallNode = Cast<UNiagaraNodeFunctionCall>(Object))
		{
			FuncCallNode->UpgradeDIFunctionCalls();
		}

		GWarn->UpdateProgress(ItemIndex++, FunctionCallNodes.Num());
	}

	GWarn->EndSlowTask();
}

void MakeIndent(int32 IndentLevel, FString& OutIndentString)
{
	OutIndentString.Reserve(IndentLevel * 2);
	for (int32 i = 0; i < IndentLevel * 2; i++)
	{
		OutIndentString.AppendChar(TCHAR(' '));
	}
}

void DumpCompileIdDataForScript(UNiagaraScript* Script, int32 IndentLevel, FString& Dump)
{
	FString Indent;
	MakeIndent(IndentLevel, Indent);
	Dump.Append(FString::Printf(TEXT("%sScript: %s\n"), *Indent, *Script->GetPathName()));
	UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(Script->GetLatestSource());
	TArray<UNiagaraNode*> Nodes;
	ScriptSource->NodeGraph->GetNodesOfClass<UNiagaraNode>(Nodes);
	for (UNiagaraNode* Node : Nodes)
	{
		Dump.Append(FString::Printf(TEXT("%s%s - %s-%s\n"), *Indent, *Node->GetFullName(), *Node->NodeGuid.ToString(EGuidFormats::Digits), *Node->GetChangeId().ToString(EGuidFormats::Digits)));
		UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(Node);
		if (FunctionCallNode != nullptr)
		{
			UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(FunctionCallNode);
			if (AssignmentNode != nullptr)
			{
				Dump.Append(FString::Printf(TEXT("%sAssignment Node: %s\n"), *Indent, *FunctionCallNode->GetFunctionName()));
				for (const FNiagaraVariable& AssignmentTarget : AssignmentNode->GetAssignmentTargets())
				{
					Dump.Append(FString::Printf(TEXT("%s  Assignment Target: %s - %s\n"), *Indent, *AssignmentTarget.GetName().ToString(), *AssignmentTarget.GetType().GetName()));
				}
			}
			else if (FunctionCallNode->FunctionScript != nullptr)
			{
				Dump.Append(FString::Printf(TEXT("%sFunction Call: %s\n"), *Indent, *FunctionCallNode->GetFunctionName()));
				DumpCompileIdDataForScript(FunctionCallNode->FunctionScript, IndentLevel + 1, Dump);
			}
		}
	}
}

void DumpCompileIdDataForEmitter(const FVersionedNiagaraEmitter& VersionedEmitter, int32 IndentLevel, FString& Dump)
{
	FString Indent;
	MakeIndent(IndentLevel, Indent);
	Dump.Append(FString::Printf(TEXT("%sEmitter: %s\n"), *Indent, *VersionedEmitter.Emitter->GetUniqueEmitterName()));

	if (FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData())
	{
		TArray<UNiagaraScript*> Scripts;
		EmitterData->GetScripts(Scripts, false);
		for (UNiagaraScript* Script : Scripts)
		{
			DumpCompileIdDataForScript(Script, IndentLevel + 1, Dump);
		}
	}
}

void DumpCompileIdDataForSystem(UNiagaraSystem* System, FString& Dump)
{
	Dump.Append(FString::Printf(TEXT("\nSystem %s\n"), *System->GetPathName()));
	DumpCompileIdDataForScript(System->GetSystemSpawnScript(), 1, Dump);
	DumpCompileIdDataForScript(System->GetSystemUpdateScript(), 1, Dump);
	for (const FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
	{
		DumpCompileIdDataForEmitter(EmitterHandle.GetInstance(), 1, Dump);
	}
}

void DumpCompileIdDataForAsset(const TArray<FString>& Arguments)
{
	if (Arguments.Num() > 0)
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FAssetData SystemAsset = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(Arguments[0]));
		if (SystemAsset.IsValid() == false)
		{
			TArray<FAssetData> AssetsInPackage;
			AssetRegistryModule.Get().GetAssetsByPackageName(*Arguments[0], AssetsInPackage);
			if (AssetsInPackage.Num() == 1)
			{
				SystemAsset = AssetsInPackage[0];
			}
		}
		if (SystemAsset.IsValid())
		{
			UNiagaraSystem* System = Cast<UNiagaraSystem>(SystemAsset.GetAsset());
			if (System != nullptr)
			{
				FString Dump;
				DumpCompileIdDataForSystem(System, Dump);
				UE_LOGF(LogNiagaraEditor, Log, "%ls", *Dump);
			}
			else
			{
				UE_LOGF(LogNiagaraEditor, Warning, "Could not load system asset for argument: %ls", *Arguments[0]);
			}
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Could not find asset for argument: %ls", *Arguments[0]);
		}
	}
	else
	{
		UE_LOGF(LogNiagaraEditor, Warning, "Command required an asset reference to be passed in.");
	}
}

void LoadAllSystemsInFolder(const TArray<FString>& Arguments)
{
	if (Arguments.Num() == 1)
	{
		TArray<FAssetData> SystemAssetsInFolder;
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FARFilter Filter;
		Filter.ClassPaths.Add(UNiagaraSystem::StaticClass()->GetClassPathName());
		Filter.PackagePaths.Add(*Arguments[0]);
		Filter.bRecursivePaths = true;
		AssetRegistryModule.Get().GetAssets(Filter, SystemAssetsInFolder);
		if (SystemAssetsInFolder.Num() > 0)
		{
			const FText SlowTaskText = FText::Format(NSLOCTEXT("NiagaraEditor", "LoadAllSystemsInFolderFormat", "Loading {0} systems in folder {1}"), FText::AsNumber(SystemAssetsInFolder.Num()), FText::FromString(Arguments[0]));
			FScopedSlowTask SlowTask(SystemAssetsInFolder.Num(), SlowTaskText);
			SlowTask.MakeDialog(true);
			int32 ItemNumber = 0;
			for (FAssetData& SystemAsset : SystemAssetsInFolder)
			{
				if (SlowTask.ShouldCancel())
				{
					return;
				}
				ItemNumber++;
				FText SlowTaskUpdateText = FText::Format(NSLOCTEXT("NiagaraEditor", "LoadAllSystemsInFolderProgressFormat", "Loading system {0} of {1}\n{2}"),
					FText::AsNumber(ItemNumber), FText::AsNumber(SystemAssetsInFolder.Num()),  FText::FromString(SystemAsset.GetFullName()));
				SlowTask.EnterProgressFrame(1, SlowTaskUpdateText);
				SystemAsset.GetAsset();
			}
		}
	}
}

void DumpEmitterDependenciesInFolder(const TArray<FString>& Arguments)
{
	if (Arguments.Num() == 1)
	{
		TArray<FAssetData> SystemAssetsInFolder;
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FARFilter Filter;
		Filter.ClassPaths.Add(UNiagaraSystem::StaticClass()->GetClassPathName());
		Filter.PackagePaths.Add(*Arguments[0]);
		Filter.bRecursivePaths = true;
		AssetRegistryModule.Get().GetAssets(Filter, SystemAssetsInFolder);
		SystemAssetsInFolder.RemoveAll([](const FAssetData& AssetData) { return AssetData.GetObjectPathString().StartsWith("/Game/Developers"); });
		if (SystemAssetsInFolder.Num() > 0)
		{
			const FText SlowTaskText = FText::Format(NSLOCTEXT("NiagaraEditor", "LoadAllSystemsInFolderFormat", "Loading {0} systems in folder {1}"), FText::AsNumber(SystemAssetsInFolder.Num()), FText::FromString(Arguments[0]));
			FScopedSlowTask SlowTask(SystemAssetsInFolder.Num(), SlowTaskText);
			SlowTask.MakeDialog(true);
			int32 ItemNumber = 0;

			bool bWasCancelled = false;
			TArray<TPair<FString, TArray<int32>>> SystemPathEmitterIndicesPairs;
			for (FAssetData& SystemAsset : SystemAssetsInFolder)
			{
				if (SlowTask.ShouldCancel())
				{
					bWasCancelled = true;
					break;
				}
				ItemNumber++;
				FText SlowTaskUpdateText = FText::Format(NSLOCTEXT("NiagaraEditor", "DumpEmitterDependenciesInFolder", "Dumping system {0} of {1}\n{2}"),
					FText::AsNumber(ItemNumber), FText::AsNumber(SystemAssetsInFolder.Num()), FText::FromString(SystemAsset.GetFullName()));
				SlowTask.EnterProgressFrame(1, SlowTaskUpdateText);
				UNiagaraSystem* System = Cast<UNiagaraSystem>(SystemAsset.GetAsset());
				if (System != nullptr)
				{
					System->WaitForCompilationComplete(true, false);
					TPair<FString, TArray<int32>>& SystemPathEmitterIndicesPair = SystemPathEmitterIndicesPairs.AddDefaulted_GetRef();
					SystemPathEmitterIndicesPair.Key = System->GetPathName();
					for (const FNiagaraEmitterExecutionIndex& EmitterExecutionIndex : System->GetEmitterExecutionOrder())
					{
						SystemPathEmitterIndicesPair.Value.Add(EmitterExecutionIndex.EmitterIndex);
					}
				}
			}

			SystemPathEmitterIndicesPairs.Sort(
				[](const TPair<FString, TArray<int32>>& A, const TPair<FString, TArray<int32>>& B)
				{
					return A.Key.Compare(B.Key) < 0;
				});

			TStringBuilder<1024> StringBuilder;
			auto IntToString = [](const int32& Int) { return FString::Printf(TEXT("%i"), Int); };
			for (const TPair<FString, TArray<int32>>& SystemEmitterIndiciesPair : SystemPathEmitterIndicesPairs)
			{
				StringBuilder.Appendf(TEXT("%s, %s\n"), *SystemEmitterIndiciesPair.Key, *FString::JoinBy(SystemEmitterIndiciesPair.Value, TEXT(","), IntToString));
			}
			FNiagaraEditorUtilities::WriteTextFileToDisk(FPaths::ProjectLogDir(), FString(TEXT("EmitterDependencies")) + TEXT(".csv"), StringBuilder.ToString(), true);
		}
	}
}

void ExecuteInvalidateNiagaraCachedScripts(const TArray< FString >& Args)
{
	if (Args.Num() == 0)
	{
		// todo: log error, at least one command is needed
		UE_LOGF(LogConsoleResponse, Display, "fx.InvalidateCachedScripts failed\nAs this command should not be executed accidentally it requires you to specify an extra parameter.");
		return;
	}

	FString FileName = FPaths::EngineDir() + TEXT("Plugins/FX/Niagara/Shaders/Private/NiagaraShaderVersion.ush");

	FileName = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FileName);

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	SourceControlProvider.Init();

	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(FileName, EStateCacheUsage::ForceUpdate);
	if (SourceControlState.IsValid())
	{
		if (SourceControlState->CanCheckout() || SourceControlState->IsCheckedOutOther())
		{
			if (SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), FileName) == ECommandResult::Failed)
			{
				UE_LOGF(LogConsoleResponse, Display, "fx.InvalidateCachedScripts failed\nCouldn't check out \"NiagaraShaderVersion.ush\"");
				return;
			}
		}
		else if (!SourceControlState->IsSourceControlled())
		{
			UE_LOGF(LogConsoleResponse, Display, "fx.InvalidateCachedScripts failed\n\"NiagaraShaderVersion.ush\" is not under revision control.");
		}
		else if (SourceControlState->IsCheckedOutOther())
		{
			UE_LOGF(LogConsoleResponse, Display, "fx.InvalidateCachedScripts failed\n\"NiagaraShaderVersion.ush\" is already checked out by someone else\n(Unreal revision control needs to be fixed to allow multiple checkout.)");
			return;
		}
		else if (SourceControlState->IsDeleted())
		{
			UE_LOGF(LogConsoleResponse, Display, "fx.InvalidateCachedScripts failed\n\"NiagaraShaderVersion.ush\" is marked for delete");
			return;
		}
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	IFileHandle* FileHandle = PlatformFile.OpenWrite(*FileName);
	if (FileHandle)
	{
		FString Guid = FString::Printf(
			TEXT("// Copyright Epic Games, Inc. All Rights Reserved.\n")
			TEXT("// This file is automatically generated by the console command fx.InvalidateCachedScripts\n")
			TEXT("// Each time the console command is executed it generates a new GUID. As a hash of this file is included\n")
			TEXT("// in the DDC key, it will automatically invalidate.\n")
			TEXT("// \n")
			TEXT("// If you are merging streams and there is a conflict with this GUID you should make a new GUID rather than taking one or the other.\n")
			TEXT("#pragma message(\"UESHADERMETADATA_VERSION %s\")"), *FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphens));

		FileHandle->Write((const uint8*)TCHAR_TO_ANSI(*Guid), Guid.Len());
		delete FileHandle;

		UE_LOGF(LogConsoleResponse, Display, "fx.InvalidateCachedScripts succeeded\n\"NiagaraShaderVersion.ush\" was updated.\n");
	}
	else
	{
		UE_LOGF(LogConsoleResponse, Display, "fx.InvalidateCachedScripts failed\nCouldn't open \"NiagaraShaderVersion.ush\".\n");
	}
}

FAutoConsoleCommand InvalidateCachedNiagaraScripts(
	TEXT("fx.InvalidateCachedScripts"),
	TEXT("Invalidate Niagara script cache by making a unique change to NiagaraShaderVersion.ush which is included in common.usf.")
	TEXT("To initiate actual the recompile of all shaders use \"recompileshaders changed\" or press \"Ctrl Shift .\".\n")
	TEXT("The NiagaraShaderVersion.ush file should be automatically checked out but it needs to be checked in to have effect on other machines."),
	FConsoleCommandWithArgsDelegate::CreateStatic(ExecuteInvalidateNiagaraCachedScripts)
);

void ExecuteRebuildNiagaraCachedScripts(const TArray< FString >& Args)
{
	UE_LOGF(LogConsoleResponse, Display, "fx.RebuildDirtyScripts started.\n");

	// Need to flush the cache to make sure that we have the latest files.
	FlushShaderFileCache();
	for (TObjectIterator<UNiagaraSystem> SystemIterator; SystemIterator; ++SystemIterator)
	{
		SystemIterator->RequestCompile(false);
	}
}

FAutoConsoleCommand ExecuteRebuildNiagaraCachedScriptsCmd(
	TEXT("fx.RebuildDirtyScripts"),
	TEXT("Go through all loaded assets and force them to recompute their script hash. If dirty, regenerate."),
	FConsoleCommandWithArgsDelegate::CreateStatic(ExecuteRebuildNiagaraCachedScripts)
);


class FNiagaraSystemBoolParameterTrackEditor : public FNiagaraSystemParameterTrackEditor<UMovieSceneNiagaraBoolParameterTrack, UMovieSceneBoolSection>
{
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override
	{
		checkf(SectionObject.GetClass()->IsChildOf<UMovieSceneBoolSection>(), TEXT("Unsupported section."));
		return MakeShareable(new FBoolPropertySection(SectionObject));
	}
};

class FNiagaraSystemColorParameterTrackEditor : public FNiagaraSystemParameterTrackEditor<UMovieSceneNiagaraColorParameterTrack, UMovieSceneColorSection>
{
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override
	{
		checkf(SectionObject.GetClass()->IsChildOf<UMovieSceneColorSection>(), TEXT("Unsupported section."));
		return MakeShareable(new FColorPropertySection(*Cast<UMovieSceneColorSection>(&SectionObject), ObjectBinding, GetSequencer()));
	}
};

// This will be called before UObjects are destroyed, so clean up anything we need to related to UObjects here
void FNiagaraEditorModule::OnPreExit()
{
#if WITH_NIAGARA_DEBUGGER
	Debugger.Reset();
	SNiagaraDebugger::UnregisterTabSpawner();
#endif

	UDeviceProfileManager::Get().OnManagerUpdated().Remove(DeviceProfileManagerUpdatedHandle);

	FEditorDelegates::OnAssetsPreDelete.Remove(OnAssetsPreDeleteHandle);

	if (GEditor)
	{
		CastChecked<UEditorEngine>(GEngine)->OnPreviewShaderPlatformChanged().Remove(PreviewPlatformChangedHandle);
	}

	// Ensure that we don't have any lingering compiles laying around that will explode after this module shuts down.
	for (TObjectIterator<UNiagaraSystem> It; It; ++It)
	{
		UNiagaraSystem* Sys = *It;
		if (Sys)
		{
			Sys->KillAllActiveCompilations();
		}
	}

	FNiagaraDigestDatabase::Shutdown();

	ClearObjectPool();
	
	INiagaraDataInterfaceNodeActionProvider::Unregister<UNiagaraDataInterfaceDataChannelWrite>();
	INiagaraDataInterfaceNodeActionProvider::Unregister<UNiagaraDataInterfaceDataChannelRead>();

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.OnFilesLoaded().Remove(AssetRegistryOnLoadCompleteHandle);

	TempPackage->RemoveFromRoot();
	TempPackage = nullptr;
}

void FNiagaraEditorModule::PostGarbageCollect()
{
	// could be that some of the asset data was garbage collected, so we reset the cache
	InvalidateCachedScriptAssetData();
}

void FNiagaraEditorModule::StartupModule()
{
	bThumbnailRenderersRegistered = false;

	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	RecentAndFavoritesManager = MakeShared<FNiagaraRecentAndFavoritesManager>();
	RecentAndFavoritesManager->Initialize();
	// We have to call shutdown before the exit actually happens to be able to save to disk
	FEditorDelegates::OnEditorPreExit.AddLambda([]()
	{
		UNiagaraFavoriteActionsConfig::Shutdown();
	});
	
	InitializeTaggedAssetBrowserDetails();

	RegisterMenuExtensions();

	TempPackage = NewObject<UPackage>(nullptr, TEXT("/Temp/NiagaraEditor"), RF_Transient);
	TempPackage->AddToRoot();

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.RegisterDynamicTagAssetClass(UNiagaraSystem::StaticClass()->GetClassPathName());
	ContentBrowserModule.RegisterDynamicTagAssetClass(UNiagaraEmitter::StaticClass()->GetClassPathName());

	// Preload all parameter definition & collection assets so that they will be postloaded before postload calls to scripts/emitters/systems that rely on them.
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().OnFilesLoaded().AddLambda([this]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraEditorModule::OnFilesLoaded);
			ParameterCollectionAssetCache.RefreshCache(true /*bAllowLoading*/);
			ParameterDefinitionsAssetCache.RefreshCache(true /*bAllowLoading*/);
		});
		AssetRegistryModule.Get().OnAssetsAdded().AddLambda([this](TConstArrayView<FAssetData> InAssets)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraEditorModule::OnAssetsAdded);

			bool FoundParameterCollection = false;
			bool FoundParameterDefinitions = false;
			for (const FAssetData& Asset : InAssets)
			{
				if (!FoundParameterCollection && Asset.IsInstanceOf(UNiagaraParameterCollection::StaticClass()))
				{
					FoundParameterCollection = true;
					continue;
				}
				else if (!FoundParameterDefinitions && Asset.IsInstanceOf(UNiagaraParameterDefinitions::StaticClass()))
				{
					FoundParameterDefinitions = true;
					continue;
				}
				if (FoundParameterDefinitions && FoundParameterCollection)
				{
					break;
				}
			}
			if (FoundParameterCollection)
			{
				ParameterCollectionAssetCache.RefreshCache(false);
			}
			if (FoundParameterDefinitions)
			{
				ParameterDefinitionsAssetCache.RefreshCache(false);
			}
		});
	}

	UNiagaraSettings::OnSettingsChanged().AddRaw(this, &FNiagaraEditorModule::OnNiagaraSettingsChangedEvent);
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FNiagaraEditorModule::OnPreGarbageCollection);
	FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FNiagaraEditorModule::PostGarbageCollect);
	
	// Any attempt to use GEditor right now will fail as it hasn't been initialized yet. Waiting for post engine init resolves that.
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FNiagaraEditorModule::OnPostEngineInit);

	DeviceProfileManagerUpdatedHandle = UDeviceProfileManager::Get().OnManagerUpdated().AddRaw(this, &FNiagaraEditorModule::OnDeviceProfileManagerUpdated);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FNiagaraEditorModule::OnPreExit);

	OnAssetsPreDeleteHandle = FEditorDelegates::OnAssetsPreDelete.AddRaw(this, &FNiagaraEditorModule::OnAssetsPreDelete);

	OnModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FNiagaraEditorModule::OnModulesChanged);
	
	// register details customization
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout(
		UNiagaraComponent::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraComponentDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		FNiagaraEditorPreviewActorCustomization::GetCustomizationTypeName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraEditorPreviewActorCustomization::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		UNiagaraNodeStaticSwitch::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraStaticSwitchNodeDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		UNiagaraScriptVariable::StaticClass()->GetFName(), 
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraScriptVariableDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		UNiagaraNodeFunctionCall::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraFunctionCallNodeDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		UNiagaraScript::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraScriptDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		UNiagaraComponentRendererProperties::StaticClass()->GetFName(), 
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraComponentRendererPropertiesDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		UNiagaraDataChannelAsset::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataChannelAssetDetails::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraFloat::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraNumericCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraInt32::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraNumericCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraNumeric::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraNumericCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraParameterMap::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraNumericCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraEnumToByteHelper::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraNumericCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraBool::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraBoolCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraMatrix::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraMatrixCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraVariableAttributeBinding::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraVariableAttributeBindingCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraParameterBinding::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraParameterBindingCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraParameterBindingWithValue::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraParameterBindingCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraScriptVariableBinding::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraScriptVariableBindingCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraPlatformSet::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraPlatformSetCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraPlatformSetCVarCondition::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraPlatformSetCVarConditionCustomization::MakeInstance));
	
	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraUserParameterBinding::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraUserParameterBindingCustomization::MakeInstance));
	
	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraMaterialAttributeBinding::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraMaterialAttributeBindingCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
	    FNiagaraVariableDataInterfaceBinding::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceBindingCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraBakerTextureSource::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraBakerTextureSourceDetails::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraVariableMetaData::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraVariableMetaDataCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraSystemScalabilityOverride::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraSystemScalabilityOverrideCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(
		UNiagaraSimulationStageGeneric::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraSimulationStageGenericCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraDataInterfaceEmitterBinding::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceEmitterBindingCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraVariable::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraVariableDetailsCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraDataChannelVariable::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDataChannelVariableDetailsCustomization::MakeInstance));

#if WITH_NIAGARA_DEBUGGER
	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraDebugHUDVariable::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDebugHUDVariableCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraOutlinerWorldData::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraOutlinerWorldDetailsCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraOutlinerSystemData::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraOutlinerSystemDetailsCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraOutlinerSystemInstanceData::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraOutlinerSystemInstanceDetailsCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraOutlinerEmitterInstanceData::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraOutlinerEmitterInstanceDetailsCustomization::MakeInstance));
#endif
	
	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraRendererMaterialScalarParameter::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraRendererMaterialScalarParameterCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraRendererMaterialVectorParameter::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraRendererMaterialVectorParameterCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraRendererMaterialTextureParameter::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraRendererMaterialTextureParameterCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraRendererMaterialStaticBoolParameter::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraRendererMaterialStaticBoolParameterCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		FNiagaraDataChannelReference::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDataChannelReferenceDetailsCustomization::MakeInstance));

	RegisterDynamicCustomizations();

	// details customizations end.

	//Register Stack Object Issue Generators.
	RegisterStackIssueGenerator(FNiagaraPlatformSet::StaticStruct()->GetFName(), new FNiagaraPlatformSetIssueGenerator());

	RegisterDefaultStackObjectCustomizations();
	
	FNiagaraEditorStyle::Register();
	ReinitializeStyleCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.NiagaraEditor.ReinitializeStyle"),
		TEXT("Reinitializes the style for the niagara editor module.  Used in conjuction with live coding for UI tweaks.  May crash the editor if style objects are in use."),
		FConsoleCommandDelegate::CreateRaw(this, &FNiagaraEditorModule::ReinitializeStyle));

	FNiagaraEditorCommands::Register();

	NiagaraComponentBroker = MakeShareable(new FNiagaraComponentBroker);
	FComponentAssetBrokerage::RegisterBroker(NiagaraComponentBroker, UNiagaraComponent::StaticClass(), true, true);

	TSharedPtr<FNiagaraScriptGraphPanelPinFactory> GraphPanelPinFactory = MakeShareable(new FNiagaraScriptGraphPanelPinFactory());

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetFloatStruct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(TNiagaraGraphPinEditableName<SGraphPinNum<float>>, GraphPin); }));

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetIntStruct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(TNiagaraGraphPinEditableName<SGraphPinInteger>, GraphPin); }));

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetVec2Struct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(TNiagaraGraphPinEditableName<SGraphPinVector2D<float>>, GraphPin); }));

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetVec3Struct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(TNiagaraGraphPinEditableName<SGraphPinVector<float>>, GraphPin); }));

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetPositionStruct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(TNiagaraGraphPinEditableName<SGraphPinVector<float>>, GraphPin); }));

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetVec4Struct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(TNiagaraGraphPinEditableName<SGraphPinVector4<float>>, GraphPin); }));

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetColorStruct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(TNiagaraGraphPinEditableName<SGraphPinColor>, GraphPin); }));

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetBoolStruct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(TNiagaraGraphPinEditableName<SGraphPinBool>, GraphPin); }));

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetGenericNumericStruct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(TNiagaraGraphPinEditableName<SNiagaraGraphPinNumeric>, GraphPin); }));

	// TODO: Don't register this here.
	GraphPanelPinFactory->RegisterMiscSubCategoryPin(UNiagaraNodeWithDynamicPins::AddPinSubCategory, FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(SNiagaraGraphPinAdd, GraphPin); }));

	GraphPanelPinFactory->RegisterTypePin(FNiagaraTypeDefinition::GetParameterMapStruct(), FNiagaraScriptGraphPanelPinFactory::FCreateGraphPin::CreateLambda(
		[](UEdGraphPin* GraphPin) -> TSharedRef<SGraphPin> { return SNew(SGraphPinExec, GraphPin); }));

	EnumTypeUtilities = MakeShareable(new FNiagaraEditorEnumTypeUtilities());
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetFloatDef(), MakeShareable(new FNiagaraEditorFloatTypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetIntDef(), MakeShareable(new FNiagaraEditorIntegerTypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetIntDef().ToStaticDef(), MakeShareable(new FNiagaraEditorIntegerTypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetBoolDef(), MakeShareable(new FNiagaraEditorBoolTypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetBoolDef().ToStaticDef(), MakeShareable(new FNiagaraEditorBoolTypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetVec2Def(), MakeShareable(new FNiagaraEditorVector2TypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetVec3Def(), MakeShareable(new FNiagaraEditorVector3TypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetVec4Def(), MakeShareable(new FNiagaraEditorVector4TypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetQuatDef(), MakeShareable(new FNiagaraEditorQuatTypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetPositionDef(), MakeShareable(new FNiagaraEditorVector3TypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetColorDef(), MakeShareable(new FNiagaraEditorColorTypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetMatrix4Def(), MakeShareable(new FNiagaraEditorMatrixTypeUtilities()));
	RegisterTypeUtilities(FNiagaraTypeDefinition::GetIDDef(), MakeShareable(new FNiagaraEditorNiagaraIDTypeUtilities()));

	RegisterTypeUtilities(FNiagaraTypeDefinition(UNiagaraDataInterfaceCurve::StaticClass()), MakeShared<FNiagaraDataInterfaceCurveTypeEditorUtilities, ESPMode::ThreadSafe>());
	RegisterTypeUtilities(FNiagaraTypeDefinition(UNiagaraDataInterfaceVector2DCurve::StaticClass()), MakeShared<FNiagaraDataInterfaceCurveTypeEditorUtilities, ESPMode::ThreadSafe>());
	RegisterTypeUtilities(FNiagaraTypeDefinition(UNiagaraDataInterfaceVectorCurve::StaticClass()), MakeShared<FNiagaraDataInterfaceVectorCurveTypeEditorUtilities, ESPMode::ThreadSafe>());
	RegisterTypeUtilities(FNiagaraTypeDefinition(UNiagaraDataInterfaceVector4Curve::StaticClass()), MakeShared<FNiagaraDataInterfaceVectorCurveTypeEditorUtilities, ESPMode::ThreadSafe>());
	RegisterTypeUtilities(FNiagaraTypeDefinition(UNiagaraDataInterfaceColorCurve::StaticClass()), MakeShared<FNiagaraDataInterfaceColorCurveTypeEditorUtilities, ESPMode::ThreadSafe>());

	TSharedRef<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe> DistributionPropertyUtilities = MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>();
	RegisterPropertyUtilities(FNiagaraDistributionFloat::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());
	RegisterPropertyUtilities(FNiagaraDistributionVector2::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());
	RegisterPropertyUtilities(FNiagaraDistributionVector3::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());
	RegisterPropertyUtilities(FNiagaraDistributionColor::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());
	RegisterPropertyUtilities(FNiagaraDistributionRangeFloat::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());
	RegisterPropertyUtilities(FNiagaraDistributionRangeVector2::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());
	RegisterPropertyUtilities(FNiagaraDistributionRangeVector3::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());
	RegisterPropertyUtilities(FNiagaraDistributionRangeColor::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());
	RegisterPropertyUtilities(FNiagaraDistributionCurveFloat::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());
	RegisterPropertyUtilities(FNiagaraDistributionCurveVector3::StaticStruct(), MakeShared<FNiagaraDistributionPropertyEditorUtilities, ESPMode::ThreadSafe>());

	RegisterExternalObjectUtilities(*UDistribution::StaticClass(), true, MakeShared<FNiagaraCascadeDistributionExternalObjectUtilities, ESPMode::ThreadSafe>());

	FEdGraphUtilities::RegisterVisualPinFactory(GraphPanelPinFactory);

	FNiagaraOpInfo::Init();

	RegisterSettings();

	RegisterDefaultRendererFactories();
	
	// Register sequencer track editors
	ISequencerModule &SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	CreateEmitterTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FNiagaraEmitterTrackEditor::CreateTrackEditor));
	CreateSystemTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FNiagaraSystemTrackEditor::CreateTrackEditor));

	SequencerModule.RegisterChannelInterface<FMovieSceneNiagaraEmitterChannel>();

	CreateBoolParameterTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(
		&FNiagaraSystemBoolParameterTrackEditor::CreateTrackEditor));
	CreateFloatParameterTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(
		&FNiagaraSystemParameterTrackEditor<UMovieSceneNiagaraFloatParameterTrack, UMovieSceneFloatSection>::CreateTrackEditor));
	CreateIntegerParameterTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(
		&FNiagaraSystemParameterTrackEditor<UMovieSceneNiagaraIntegerParameterTrack, UMovieSceneIntegerSection>::CreateTrackEditor));
	CreateVectorParameterTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(
		&FNiagaraSystemParameterTrackEditor<UMovieSceneNiagaraVectorParameterTrack, UMovieSceneFloatVectorSection>::CreateTrackEditor));
	CreateColorParameterTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(
		&FNiagaraSystemColorParameterTrackEditor::CreateTrackEditor));

	RegisterParameterTrackCreatorForType(*FNiagaraBool::StaticStruct(), FOnCreateMovieSceneTrackForParameter::CreateLambda([](FNiagaraVariable InParameter) {
		return NewObject<UMovieSceneNiagaraBoolParameterTrack>(); }));
	RegisterParameterTrackCreatorForType(*FNiagaraFloat::StaticStruct(), FOnCreateMovieSceneTrackForParameter::CreateLambda([](FNiagaraVariable InParameter) {
		return NewObject<UMovieSceneNiagaraFloatParameterTrack>(); }));
	RegisterParameterTrackCreatorForType(*FNiagaraInt32::StaticStruct(), FOnCreateMovieSceneTrackForParameter::CreateLambda([](FNiagaraVariable InParameter) {
		return NewObject<UMovieSceneNiagaraIntegerParameterTrack>(); }));
	RegisterParameterTrackCreatorForType(*FNiagaraTypeDefinition::GetVec2Struct(), FOnCreateMovieSceneTrackForParameter::CreateLambda([](FNiagaraVariable InParameter) 
	{
		UMovieSceneNiagaraVectorParameterTrack* VectorTrack = NewObject<UMovieSceneNiagaraVectorParameterTrack>();
		VectorTrack->SetChannelsUsed(2);
		return VectorTrack;
	}));
	RegisterParameterTrackCreatorForType(*FNiagaraTypeDefinition::GetVec3Struct(), FOnCreateMovieSceneTrackForParameter::CreateLambda([](FNiagaraVariable InParameter)
	{
		UMovieSceneNiagaraVectorParameterTrack* VectorTrack = NewObject<UMovieSceneNiagaraVectorParameterTrack>();
		VectorTrack->SetChannelsUsed(3);
		return VectorTrack;
	}));
	RegisterParameterTrackCreatorForType(*FNiagaraTypeDefinition::GetVec4Struct(), FOnCreateMovieSceneTrackForParameter::CreateLambda([](FNiagaraVariable InParameter)
	{
		UMovieSceneNiagaraVectorParameterTrack* VectorTrack = NewObject<UMovieSceneNiagaraVectorParameterTrack>();
		VectorTrack->SetChannelsUsed(4);
		return VectorTrack;
	}));
	RegisterParameterTrackCreatorForType(*FNiagaraTypeDefinition::GetPositionStruct(), FOnCreateMovieSceneTrackForParameter::CreateLambda([](FNiagaraVariable InParameter)
	{
		UMovieSceneNiagaraVectorParameterTrack* VectorTrack = NewObject<UMovieSceneNiagaraVectorParameterTrack>();
		VectorTrack->SetChannelsUsed(3);
		return VectorTrack;
	}));
	RegisterParameterTrackCreatorForType(*FNiagaraTypeDefinition::GetColorStruct(), FOnCreateMovieSceneTrackForParameter::CreateLambda([](FNiagaraVariable InParameter) {
		return NewObject<UMovieSceneNiagaraColorParameterTrack>(); }));

	// Register the shader queue processor (for cooking)
	INiagaraModule& NiagaraModule = FModuleManager::LoadModuleChecked<INiagaraModule>("Niagara");
	NiagaraModule.SetOnProcessShaderCompilationQueue(INiagaraModule::FOnProcessQueue::CreateLambda([]()
	{
		FNiagaraShaderQueueTickable::ProcessQueue();
	}));

	INiagaraShaderModule& NiagaraShaderModule = FModuleManager::LoadModuleChecked<INiagaraShaderModule>("NiagaraShader");
	NiagaraShaderModule.SetOnProcessShaderCompilationQueue(INiagaraShaderModule::FOnProcessQueue::CreateLambda([]()
	{
		FNiagaraShaderQueueTickable::ProcessQueue();
	}));

	// Register the emitter merge handler and editor data utilities.
	ScriptMergeManager = MakeShared<FNiagaraScriptMergeManager>();
	NiagaraModule.RegisterMergeManager(ScriptMergeManager.ToSharedRef());

	EditorOnlyDataUtilities = MakeShared<FNiagaraEditorOnlyDataUtilities>();
	NiagaraModule.RegisterEditorOnlyDataUtilities(EditorOnlyDataUtilities.ToSharedRef());

	// Register the script compiler
	ScriptCompilerHandle = NiagaraModule.RegisterScriptCompiler(INiagaraModule::FScriptCompiler::CreateLambda([this](const FNiagaraCompileRequestDataBase* CompileRequest, const FNiagaraCompileRequestDuplicateDataBase* CompileRequestDuplicate, const FNiagaraCompileOptions& Options)
	{
		return CompileScript(CompileRequest, CompileRequestDuplicate, Options);
	}));

	CompileResultHandle = NiagaraModule.RegisterCompileResultDelegate(INiagaraModule::FCheckCompilationResult::CreateLambda([this](int32 JobID, bool bWait, FNiagaraScriptCompileMetrics& ScriptMetrics)
	{
		return GetCompilationResult(JobID, bWait, ScriptMetrics);
	}));

	PrecompilerHandle = NiagaraModule.RegisterPrecompiler(INiagaraModule::FOnPrecompile::CreateLambda([this](UObject* InObj, FGuid Version)
	{
		return Precompile(InObj, Version);
	}));

	PrecompileDuplicatorHandle = NiagaraModule.RegisterPrecompileDuplicator(INiagaraModule::FOnPrecompileDuplicate::CreateLambda([this](const FNiagaraCompileRequestDataBase* OwningSystemRequestData, UNiagaraSystem* OwningSystem, UNiagaraEmitter* OwningEmitter, UNiagaraScript* TargetScript,  FGuid Version)
	{
		return PrecompileDuplicate(OwningSystemRequestData, OwningSystem, OwningEmitter, TargetScript, Version);
	}));

	GraphCacheTraversalHandle = NiagaraModule.RegisterGraphTraversalCacher(INiagaraModule::FOnCacheGraphTraversal::CreateLambda([this](const UObject* InObj, FGuid Version)
	{
		return CacheGraphTraversal(InObj, Version);
	}));

	RequestCompileSystemHandle = NiagaraModule.RegisterRequestCompileSystem(INiagaraModule::FOnRequestCompileSystem::CreateLambda([this](UNiagaraSystem* System, bool bForced, const ITargetPlatform* TargetPlatform)
	{
		return RequestCompileSystem(System, bForced, TargetPlatform);
	}));

	PollSystemCompileHandle = NiagaraModule.RegisterPollSystemCompile(INiagaraModule::FOnPollSystemCompile::CreateLambda([this](FNiagaraCompilationTaskHandle TaskHandle, FNiagaraSystemAsyncCompileResults& Results, bool bWait, bool bPeek)
	{
		return PollSystemCompile(TaskHandle, Results, bWait, bPeek);
	}));

	AbortSystemCompileHandle = NiagaraModule.RegisterAbortSystemCompile(INiagaraModule::FOnAbortSystemCompile::CreateLambda([this](FNiagaraCompilationTaskHandle TaskHandle)
	{
		AbortSystemCompile(TaskHandle);
	}));

	TestCompileScriptCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.TestCompileNiagaraScript"),
		TEXT("Compiles the specified script on disk for the niagara vector vm"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FNiagaraEditorModule::TestCompileScriptFromConsole));

	ValidateScriptVariableGuidsCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.Niagara.ValidateDuplicateVariableGuids"),
		TEXT("Validate the script guids of a given script."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FNiagaraEditorModule::ValidateScriptVariableIds, false));
		
	ValidateAndFixScriptVariableGuidsCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.Niagara.FixDuplicateVariableGuids"),
		TEXT("Validates and fixes the script guids of a given script, if duplicates exist."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FNiagaraEditorModule::ValidateScriptVariableIds, true));

	DumpRapidIterationParametersForAsset = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.DumpRapidIterationParametersForAsset"),
		TEXT("Dumps the values of the rapid iteration parameters for the specified asset by path."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&DumpRapidIterationParamersForAsset));

	PreventSystemRecompileCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.PreventSystemRecompile"),
		TEXT("Forces the system to refresh all it's dependencies so it won't recompile on load.  This may mark multiple assets dirty for re-saving."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&PreventSystemRecompile));

	PreventAllSystemRecompilesCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.PreventAllSystemRecompiles"),
		TEXT("Loads all of the systems in the project and forces each system to refresh all it's dependencies so it won't recompile on load.  This may mark multiple assets dirty for re-saving."),
		FConsoleCommandDelegate::CreateStatic(&PreventAllSystemRecompiles));

	UpgradeAllNiagaraAssetsCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.UpgradeAllNiagaraAssets"),
		TEXT("Loads all Niagara assets and preforms any data upgrade processes required. This may mark multiple assets dirty for re-saving."),
		FConsoleCommandDelegate::CreateStatic(&UpgradeAllNiagaraAssets));

	DumpCompileIdDataForAssetCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.DumpCompileIdDataForAsset"),
		TEXT("Dumps data relevant to generating the compile id for an asset."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&DumpCompileIdDataForAsset));

	LoadAllSystemsInFolderCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.LoadAllNiagaraSystemsInFolder"),
		TEXT("Loads all niagara systems in the supplied directory and sub-directories."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&LoadAllSystemsInFolder));

	DumpEmitterDependenciesCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.DumpEmitterDepencenciesInFolder"),
		TEXT("Dumps emitter dependencies for all systems in the supplied folder and sub-folders."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&DumpEmitterDependenciesInFolder));

	FNiagaraMessageManager* MessageManager = FNiagaraMessageManager::Get();
	MessageManager->RegisterMessageTopic(FNiagaraMessageTopics::CompilerTopicName);
	MessageManager->RegisterMessageTopic(FNiagaraMessageTopics::ObjectTopicName);

	// Register sim cache visualizers
	RegisterDataInterfaceCacheVisualizer(UNiagaraDataInterfaceDataChannelWrite::StaticClass(), MakeShared<FNiagaraDataChannelSimCacheVisualizer>());
	RegisterDataInterfaceCacheVisualizer(UNiagaraDataInterfaceDynamicMesh::StaticClass(), MakeShared<FNiagaraDynamicMeshSimCacheVisualizer>());
	RegisterDataInterfaceCacheVisualizer(UNiagaraDataInterfaceMemoryBuffer::StaticClass(), MakeShared<FNiagaraMemoryBufferSimCacheVisualizer>());
	RegisterDataInterfaceCacheVisualizer(UNiagaraDataInterfaceRenderTarget2D::StaticClass(), MakeShared<FNiagaraRenderTargetSimCacheVisualizer>());
	RegisterDataInterfaceCacheVisualizer(UNiagaraDataInterfaceRenderTargetVolume::StaticClass(), MakeShared<FNiagaraRenderTargetVolumeSimCacheVisualizer>());
	RegisterDataInterfaceCacheVisualizer(UNiagaraDataInterfaceSimpleCounter::StaticClass(), MakeShared<FNiagaraSimpleCounterSimCacheVisualizer>());
	for (TObjectIterator<UClass> It; It; ++It)
	{
		if (It->IsChildOf(UNiagaraDataInterfaceArray::StaticClass()))
		{
			RegisterDataInterfaceCacheVisualizer(*It, MakeShared<FNiagaraArraySimCacheVisualizer>(*It));
		}
	}

	RegisterModuleWizards(UE::Niagara::Wizard::DataChannel::CreateNDCWizardGenerator());

#if NIAGARA_PERF_BASELINES
	UNiagaraEffectType::OnGeneratePerfBaselines().BindRaw(this, &FNiagaraEditorModule::GeneratePerfBaselines);
#endif

	GraphDataCache = MakeUnique<FNiagaraGraphDataCache>();
	
	//Register node action providers for data interface functions.
	INiagaraDataInterfaceNodeActionProvider::Register<UNiagaraDataInterfaceDataChannelWrite, FNiagaraDataInterfaceNodeActionProvider_DataChannelWrite>();
	INiagaraDataInterfaceNodeActionProvider::Register<UNiagaraDataInterfaceDataChannelRead, FNiagaraDataInterfaceNodeActionProvider_DataChannelRead>();
	INiagaraDataInterfaceNodeActionProvider::Register<UNiagaraDataInterfaceDataTable, FNiagaraDataInterfaceNodeActionProvider_DataTable>();

	FAssetCompilingManager::Get().RegisterManager(&FNiagaraSystemCompilingManager::Get());
	ILevelSequenceModule& LevelSequenceModule = FModuleManager::LoadModuleChecked<ILevelSequenceModule>("LevelSequence");
	DefaultTrackHandle = LevelSequenceModule.OnNewActorTrackAdded().AddStatic(&FNiagaraSystemTrackEditor::AddDefaultSystemTracks);

	TraversalCache = MakeShared<UE::Niagara::TraversalCache::FTraversalCache>();
	TraversalCache->Initialize();
}

void FNiagaraEditorModule::ShutdownModule()
{
	FNiagaraActorPlacement::Unregister();

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();

	RecentAndFavoritesManager->Shutdown();
	RecentAndFavoritesManager.Reset();
	
	// Clean up asset registry callbacks
	if (FModuleManager::Get().IsModuleLoaded("AssetRegistry"))
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().OnFilesLoaded().RemoveAll(this);
		AssetRegistryModule.Get().OnAssetAdded().RemoveAll(this);
	}

	UNiagaraSettings::OnSettingsChanged().RemoveAll(this);

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);

	FModuleManager::Get().OnModulesChanged().Remove(OnModulesChangedHandle);
	
	if (GEditor)
	{
		GEditor->OnExecParticleInvoked().RemoveAll(this);
	}
	
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("NiagaraComponent");
	}
	if (FModuleManager::Get().IsModuleLoaded("LevelSequence"))
	{
		ILevelSequenceModule& LevelSequenceModule = FModuleManager::LoadModuleChecked<ILevelSequenceModule>("LevelSequence");
		LevelSequenceModule.OnNewActorTrackAdded().Remove(DefaultTrackHandle);
	}

	UnregisterDynamicCustomizations();

	FNiagaraEditorStyle::Unregister();

	UnregisterSettings();

	if (UObjectInitialized())
	{
		FComponentAssetBrokerage::UnregisterBroker(NiagaraComponentBroker);
	}

	ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
	if (SequencerModule != nullptr)
	{
		SequencerModule->UnRegisterTrackEditor(CreateEmitterTrackEditorHandle);
		SequencerModule->UnRegisterTrackEditor(CreateSystemTrackEditorHandle);
		SequencerModule->UnRegisterTrackEditor(CreateBoolParameterTrackEditorHandle);
		SequencerModule->UnRegisterTrackEditor(CreateFloatParameterTrackEditorHandle);
		SequencerModule->UnRegisterTrackEditor(CreateIntegerParameterTrackEditorHandle);
		SequencerModule->UnRegisterTrackEditor(CreateVectorParameterTrackEditorHandle);
		SequencerModule->UnRegisterTrackEditor(CreateColorParameterTrackEditorHandle);
	}

	INiagaraModule* NiagaraModule = FModuleManager::GetModulePtr<INiagaraModule>("Niagara");
	if (NiagaraModule != nullptr)
	{
		NiagaraModule->UnregisterMergeManager(ScriptMergeManager.ToSharedRef());
		NiagaraModule->UnregisterEditorOnlyDataUtilities(EditorOnlyDataUtilities.ToSharedRef());
		NiagaraModule->UnregisterScriptCompiler(ScriptCompilerHandle);
		NiagaraModule->UnregisterCompileResultDelegate(CompileResultHandle);
		NiagaraModule->UnregisterPrecompiler(PrecompilerHandle);
		NiagaraModule->UnregisterPrecompileDuplicator(PrecompileDuplicatorHandle);
		NiagaraModule->UnregisterGraphTraversalCacher(GraphCacheTraversalHandle);
		NiagaraModule->UnregisterRequestCompileSystem(RequestCompileSystemHandle);
		NiagaraModule->UnregisterPollSystemCompile(PollSystemCompileHandle);
		NiagaraModule->UnregisterAbortSystemCompile(AbortSystemCompileHandle);
	}

	// Verify that we've cleaned up all the view models in the world.
	FNiagaraSystemViewModel::CleanAll();
	FNiagaraEmitterViewModel::CleanAll();
	FNiagaraScriptViewModel::CleanAll();

	if (TestCompileScriptCommand != nullptr)
	{
		IConsoleManager::Get().UnregisterConsoleObject(TestCompileScriptCommand);
	}

	if (DumpRapidIterationParametersForAsset != nullptr)
	{
		IConsoleManager::Get().UnregisterConsoleObject(DumpRapidIterationParametersForAsset);
	}

	if (PreventSystemRecompileCommand != nullptr)
	{
		IConsoleManager::Get().UnregisterConsoleObject(PreventSystemRecompileCommand);
	}

	if (PreventAllSystemRecompilesCommand != nullptr)
	{
		IConsoleManager::Get().UnregisterConsoleObject(PreventAllSystemRecompilesCommand);
	}

	if (DumpCompileIdDataForAssetCommand != nullptr)
	{
		IConsoleManager::Get().UnregisterConsoleObject(DumpCompileIdDataForAssetCommand);
		DumpCompileIdDataForAssetCommand = nullptr;
	}

	if (UObjectInitialized() && GIsEditor && bThumbnailRenderersRegistered)
	{
		UThumbnailManager::Get().UnregisterCustomRenderer(UNiagaraEmitter::StaticClass());
		UThumbnailManager::Get().UnregisterCustomRenderer(UNiagaraSystem::StaticClass());
	}

	for (auto& Pair : StackIssueGenerators)
	{
		if (Pair.Value)
		{
			delete Pair.Value;
		}
	}
	StackIssueGenerators.Empty();

#if NIAGARA_PERF_BASELINES
	UNiagaraEffectType::OnGeneratePerfBaselines().Unbind();
#endif

	GraphDataCache.Reset();

	FAssetCompilingManager::Get().UnregisterManager(&FNiagaraSystemCompilingManager::Get());

	TraversalCache.Reset();
	
	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser");
		(ContentBrowserModule != nullptr) && UObjectInitialized())
	{
		ContentBrowserModule->UnregisterDynamicTagAssetClass(UNiagaraSystem::StaticClass()->GetClassPathName());
		ContentBrowserModule->UnregisterDynamicTagAssetClass(UNiagaraEmitter::StaticClass()->GetClassPathName());
	}
}

void FNiagaraEditorModule::OnPostEngineInit()
{
	if (GIsEditor)
	{
		FNiagaraActorPlacement::Register();

		UThumbnailManager::Get().RegisterCustomRenderer(UNiagaraEmitter::StaticClass(), UNiagaraEmitterThumbnailRenderer::StaticClass());
		UThumbnailManager::Get().RegisterCustomRenderer(UNiagaraSystem::StaticClass(), UNiagaraSystemThumbnailRenderer::StaticClass());
		bThumbnailRenderersRegistered = true;
	}

	// The editor should be valid at this point.. log a warning if not!
	if (GEditor)
	{
		GEditor->OnExecParticleInvoked().AddRaw(this, &FNiagaraEditorModule::OnExecParticleInvoked);

		UEditorEngine* EditorEngine = CastChecked<UEditorEngine>(GEngine);
		PreviewPlatformChangedHandle = EditorEngine->OnPreviewShaderPlatformChanged().AddRaw(this, &FNiagaraEditorModule::OnPreviewPlatformChanged);

		// Ensure we have the right feature level set as the editor may already be in one before we get here
		UNiagaraScript::SetPreviewFeatureLevel(GEditor->DefaultWorldFeatureLevel);

		// Handle a re-import for mesh renderers
		if (UImportSubsystem* ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>())
		{
			ImportSubsystem->OnAssetReimport.AddLambda(
				[](UObject* ObjectReimported)
				{
					UNiagaraMeshRendererProperties::OnMeshChanged(ObjectReimported);
				}
			);
		}
	}
	else
	{
		UE_LOGF(LogNiagaraEditor, Log, "GEditor isn't valid! Particle reset commands will not work for Niagara components!");
	}

	// ensure that all cached asset types are fully loaded.
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	bool bForceSilentLoadingOfCachedAssets = NiagaraEditorSettings->GetForceSilentLoadingOfCachedAssets();

	ParameterCollectionAssetCache.SetForceLoadSilent(bForceSilentLoadingOfCachedAssets);
	ParameterDefinitionsAssetCache.SetForceLoadSilent(bForceSilentLoadingOfCachedAssets);

	ParameterCollectionAssetCache.RefreshCache(true /*bAllowLoading*/);
	ParameterDefinitionsAssetCache.RefreshCache(true /*bAllowLoading*/);

	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		
		TSharedPtr<FFilterCategory> FXFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("FXFilterCategory", "FX"), FText::GetEmpty());
		TSharedRef<FCustomClassFilterData> NewClassData = MakeShared<FCustomClassFilterData>(ANiagaraActor::StaticClass(), FXFilterCategory, FLinearColor::White);

		if(TSharedPtr<FFilterCategory> EssentialCategory = LevelEditorModule.GetOutlinerFilterCategory(FLevelEditorOutlinerBuiltInCategories::Common()))
		{
			NewClassData->AddCategory(EssentialCategory);
		}
		LevelEditorModule.AddCustomClassFilterToOutliner(NewClassData);
	}

#if WITH_NIAGARA_DEBUGGER
	Debugger = MakeShared<FNiagaraDebugger>();
	Debugger->Init();
	SNiagaraDebugger::RegisterTabSpawner();
#endif

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistryOnLoadCompleteHandle = AssetRegistry.OnFilesLoaded().AddRaw(this, &FNiagaraEditorModule::OnAssetRegistryLoadComplete);
}

void FNiagaraEditorModule::OnDeviceProfileManagerUpdated()
{
	FNiagaraPlatformSet::InvalidateCachedData();
}

void FNiagaraEditorModule::OnPreviewPlatformChanged(EShaderPlatform NewShaderPlatform)
{
	FNiagaraPlatformSet::InvalidateCachedData();

	UNiagaraScript::SetPreviewFeatureLevel(GetMaxSupportedFeatureLevel(NewShaderPlatform));

	for (TObjectIterator<UNiagaraSystem> It; It; ++It)
	{
		UNiagaraSystem* System = *It;
		check(System);
		System->UpdateScalability();
	}
}

FNiagaraEditorModule& FNiagaraEditorModule::Get()
{
	return FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
}

FNiagaraRecentAndFavoritesManager* FNiagaraEditorModule::GetRecentsManager()
{
	return RecentAndFavoritesManager.Get();
}

void FNiagaraEditorModule::OnNiagaraSettingsChangedEvent(const FName& PropertyName, const UNiagaraSettings* Settings)
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSettings, AdditionalParameterTypes)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSettings, AdditionalPayloadTypes)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraSettings, AdditionalParameterEnums))
	{
		FNiagaraTypeDefinition::RecreateUserDefinedTypeRegistry();
	}
}

void FNiagaraEditorModule::RegisterTypeUtilities(FNiagaraTypeDefinition Type, TSharedRef<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> EditorUtilities)
{
	UE::TScopeLock ScopeLock(EditorUtilitiesGuard);
	TypeToEditorUtilitiesMap.Add(Type, EditorUtilities);
}

void FNiagaraEditorModule::RegisterPropertyUtilities(const UScriptStruct* InStruct, TSharedRef<INiagaraEditorPropertyUtilities, ESPMode::ThreadSafe> InPropertyUtilities)
{
	UE::TScopeLock ScopeLock(EditorUtilitiesGuard);
	StructToPropertyUtilitiesMap.Add(InStruct, InPropertyUtilities);
}

void FNiagaraEditorModule::RegisterExternalObjectUtilities(const UClass& InClass, bool bHandleSubclasses, TSharedRef<INiagaraEditorExternalObjectUtilities, ESPMode::ThreadSafe> InExternalObjectUtilities)
{
	UE::TScopeLock ScopeLock(EditorUtilitiesGuard);
	if (RegisteredExternalObjectUtilities.ContainsByPredicate([&InClass]
		(const FRegisteredExternalObjectUtilities& RegisteredUtilities) { return RegisteredUtilities.Class == &InClass; }) == false)
	{
		FRegisteredExternalObjectUtilities& AddedUtilities = RegisteredExternalObjectUtilities.AddDefaulted_GetRef();
		AddedUtilities.Class = &InClass;
		AddedUtilities.bHandleSubclasses = bHandleSubclasses;
		AddedUtilities.ExternalObjectUtilities = InExternalObjectUtilities;
	}
}

void FNiagaraEditorModule::RegisterModuleWizards(TSharedRef<UE::Niagara::Wizard::FModuleWizardGenerator> WizardGenerator)
{
	ModuleWizards.Add(WizardGenerator);
}

TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> FNiagaraEditorModule::GetTypeUtilities(const FNiagaraTypeDefinition& Type) const
{
	const TSharedRef<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe>* TypeUtilities = nullptr;
	{
		UE::TScopeLock ScopeLock(EditorUtilitiesGuard);
		TypeUtilities = TypeToEditorUtilitiesMap.Find(Type);
	}

	if(TypeUtilities != nullptr)
	{
		return *TypeUtilities;
	}

	if (Type.IsEnum())
	{
		return EnumTypeUtilities;
	}

	return TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe>();
}

TSharedPtr<INiagaraEditorPropertyUtilities, ESPMode::ThreadSafe> FNiagaraEditorModule::GetPropertyUtilities(const UScriptStruct& Struct) const
{
	const TSharedRef<INiagaraEditorPropertyUtilities, ESPMode::ThreadSafe>* PropertyUtilities = nullptr;
	{
		UE::TScopeLock ScopeLock(EditorUtilitiesGuard);
		PropertyUtilities = StructToPropertyUtilitiesMap.Find(&Struct);
	}
	return PropertyUtilities != nullptr ? *PropertyUtilities : TSharedPtr<INiagaraEditorPropertyUtilities>();
}

TSharedPtr<const INiagaraEditorExternalObjectUtilities, ESPMode::ThreadSafe> FNiagaraEditorModule::GetExternalObjectUtilities(const UClass& Class) const
{
	const FRegisteredExternalObjectUtilities* MatchingRegisteredUtilities = nullptr;
	{
		UE::TScopeLock ScopeLock(EditorUtilitiesGuard);
		MatchingRegisteredUtilities = RegisteredExternalObjectUtilities.FindByPredicate([&Class](const FRegisteredExternalObjectUtilities& RegisteredUtilities)
		{
			return RegisteredUtilities.Class == &Class || (RegisteredUtilities.bHandleSubclasses && Class.IsChildOf(RegisteredUtilities.Class));
		});
	}
	return MatchingRegisteredUtilities != nullptr ? MatchingRegisteredUtilities->ExternalObjectUtilities : TSharedPtr<const INiagaraEditorExternalObjectUtilities>();
}

void FNiagaraEditorModule::RegisterWidgetProvider(TSharedRef<INiagaraEditorWidgetProvider> InWidgetProvider)
{
	checkf(WidgetProvider.IsValid() == false, TEXT("Widget provider has already been set."));
	WidgetProvider = InWidgetProvider;
}

void FNiagaraEditorModule::UnregisterWidgetProvider(TSharedRef<INiagaraEditorWidgetProvider> InWidgetProvider)
{
	checkf(WidgetProvider.IsValid() && WidgetProvider == InWidgetProvider, TEXT("Can only unregister the widget provider that was originally registered."));
	WidgetProvider.Reset();
}

TSharedRef<INiagaraEditorWidgetProvider> FNiagaraEditorModule::GetWidgetProvider() const
{
	return WidgetProvider.ToSharedRef();
}

void FNiagaraEditorModule::RegisterDataInterfaceCacheVisualizer(UClass* DataInterfaceClass, TSharedRef<INiagaraDataInterfaceSimCacheVisualizer> InCacheVisualizer)
{
	DataInterfaceVisualizers.FindOrAdd(DataInterfaceClass).AddUnique(InCacheVisualizer);
}

void FNiagaraEditorModule::UnregisterDataInterfaceCacheVisualizer(UClass* DataInterfaceClass, TSharedRef<INiagaraDataInterfaceSimCacheVisualizer> InCacheVisualizer)
{
	if (TArray<TSharedRef<INiagaraDataInterfaceSimCacheVisualizer>>* CacheVisualizers = DataInterfaceVisualizers.Find(DataInterfaceClass))
	{
		CacheVisualizers->Remove(InCacheVisualizer);
	}
}

TArrayView<TSharedRef<INiagaraDataInterfaceSimCacheVisualizer>> FNiagaraEditorModule::FindDataInterfaceCacheVisualizer(UClass* DataInterfaceClass)
{
	if (TArray<TSharedRef<INiagaraDataInterfaceSimCacheVisualizer>>* CacheVisualizers = DataInterfaceVisualizers.Find(DataInterfaceClass))
	{
		return *CacheVisualizers;
	}
	return TArrayView<TSharedRef<INiagaraDataInterfaceSimCacheVisualizer>>();
}

TSharedRef<FNiagaraScriptMergeManager> FNiagaraEditorModule::GetScriptMergeManager() const
{
	return ScriptMergeManager.ToSharedRef();
}

UObject* FNiagaraEditorModule::GetPooledDuplicateObject(UObject* Source, EFieldIteratorFlags::SuperClassFlags CopySuperProperties)
{
	check(IsInGameThread());
	TArray<UObject*>& Pool = ObjectPool.FindOrAdd(Source->GetClass());
	UObject* OutPooledObj;
	if (Pool.Num() > 0)
	{
		OutPooledObj = Pool.Pop();
		for (TFieldIterator<FProperty> PropertyIt(OutPooledObj->GetClass(), CopySuperProperties); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			const uint8* SourceAddr = Property->ContainerPtrToValuePtr<uint8>(Source);
			uint8* DestinationAddr = Property->ContainerPtrToValuePtr<uint8>(OutPooledObj);
			Property->CopyCompleteValue(DestinationAddr, SourceAddr);
		}
	}
	else
	{
		bool bIsTransactionalSource = Source->HasAllFlags(RF_Transactional);
		Source->ClearFlags(RF_Transactional);
		OutPooledObj = DuplicateObject(Source, GetTransientPackage());
		OutPooledObj->AddToRoot();
		if (bIsTransactionalSource)
		{
			Source->SetFlags(RF_Transactional);
		}
	}
	return OutPooledObj;
}

void FNiagaraEditorModule::ReleaseObjectToPool(UObject* Obj)
{
	check(IsInGameThread());
	if (Obj->GetOuter() == GetTransientPackage())
	{
		Obj->AddToRoot();
		ObjectPool.FindOrAdd(Obj->GetClass()).Push(Obj);
	}
}

void FNiagaraEditorModule::ClearObjectPool()
{
	check(IsInGameThread());
	for (auto& Pool : ObjectPool)
	{
		for (UObject* Var : Pool.Value)
		{
			Var->RemoveFromRoot();
			Var->MarkAsGarbage();
		}
	}
	ObjectPool.Empty();
}

void FNiagaraEditorModule::RegisterRendererCreationInfo(FNiagaraRendererCreationInfo InRendererCreationInfo)
{
	RendererCreationInfo.Add(InRendererCreationInfo);
}

void FNiagaraEditorModule::RegisterParameterTrackCreatorForType(const UScriptStruct& StructType, FOnCreateMovieSceneTrackForParameter CreateTrack)
{
	checkf(TypeToParameterTrackCreatorMap.Contains(&StructType) == false, TEXT("Type already registered"));
	TypeToParameterTrackCreatorMap.Add(&StructType, CreateTrack);
}

void FNiagaraEditorModule::UnregisterParameterTrackCreatorForType(const UScriptStruct& StructType)
{
	TypeToParameterTrackCreatorMap.Remove(&StructType);
}

bool FNiagaraEditorModule::CanCreateParameterTrackForType(const UScriptStruct& StructType)
{
	return TypeToParameterTrackCreatorMap.Contains(&StructType);
}

UMovieSceneNiagaraParameterTrack* FNiagaraEditorModule::CreateParameterTrackForType(const UScriptStruct& StructType, FNiagaraVariable Parameter)
{
	FOnCreateMovieSceneTrackForParameter* CreateTrack = TypeToParameterTrackCreatorMap.Find(&StructType);
	checkf(CreateTrack != nullptr, TEXT("Type not supported"));
	UMovieSceneNiagaraParameterTrack* ParameterTrack = CreateTrack->Execute(Parameter);
	ParameterTrack->SetParameter(Parameter);
	return ParameterTrack;
}

const FNiagaraEditorCommands& FNiagaraEditorModule::Commands()
{
	return FNiagaraEditorCommands::Get();
}

TSharedPtr<FNiagaraSystemViewModel> FNiagaraEditorModule::GetExistingViewModelForSystem(UNiagaraSystem* InSystem)
{
	return FNiagaraSystemViewModel::GetExistingViewModelForObject(InSystem);
}

const FNiagaraEditorCommands& FNiagaraEditorModule::GetCommands() const
{
	return FNiagaraEditorCommands::Get();
}

void FNiagaraEditorModule::InvalidateCachedScriptAssetData()
{
	CachedScriptAssetHighlights.Reset();
	TypeConversionScriptCache.Reset();
}

const TArray<UNiagaraScript*>& FNiagaraEditorModule::GetCachedTypeConversionScripts() const
{
	if (!TypeConversionScriptCache.IsSet())
	{
		TArray<FAssetData> DynamicInputAssets;
	    FNiagaraEditorUtilities::FGetFilteredScriptAssetsOptions DynamicInputScriptFilterOptions;
	    DynamicInputScriptFilterOptions.ScriptUsageToInclude = ENiagaraScriptUsage::DynamicInput;
	    FNiagaraEditorUtilities::GetFilteredScriptAssets(DynamicInputScriptFilterOptions, DynamicInputAssets);

		TArray<UNiagaraScript*> AvailableDynamicInputs;
	    for (const FAssetData& DynamicInputAsset : DynamicInputAssets)
	    {
    		UNiagaraScript* DynamicInputScript = Cast<UNiagaraScript>(DynamicInputAsset.GetAsset());
    		if (DynamicInputScript != nullptr)
    		{
    			UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(DynamicInputScript->GetLatestSource());
    			if (ScriptSource && DynamicInputScript->GetLatestScriptData()->bCanBeUsedForTypeConversions)
    			{
    				AvailableDynamicInputs.Add(DynamicInputScript);
    			}
    		}
	    }
		TypeConversionScriptCache = AvailableDynamicInputs;
	}
	return TypeConversionScriptCache.GetValue();
}

FNiagaraClipboard& FNiagaraEditorModule::GetClipboard() const
{
	return Clipboard.Get();
}

void FNiagaraEditorModule::GetDataInterfaceFeedbackSafe(UNiagaraDataInterface* InDataInterface, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& OutWarnings, TArray<FNiagaraDataInterfaceFeedback>& OutInfo)
{
	if (!InDataInterface)
		return;

	UNiagaraSystem* OwningSystem = InDataInterface->GetTypedOuter<UNiagaraSystem>();
	UNiagaraComponent* OwningComponent = InDataInterface->GetTypedOuter<UNiagaraComponent>();

	if (OwningSystem == nullptr)
	{
		// If no outer was found, try to find one by component.
		if (OwningComponent != nullptr)
		{
			OwningSystem = OwningComponent->GetAsset();
		}
	}

	if (OwningSystem == nullptr)
	{
		// If no outer information is available, check system view models for placeholder DIs.
		TArray<TSharedRef<FNiagaraSystemViewModel>> SystemViewModels;
		FNiagaraSystemViewModel::GetAllViewModels(SystemViewModels);
		for (TSharedRef<FNiagaraSystemViewModel> SystemViewModel : SystemViewModels)
		{
			FGuid OwningEmitterHandle;
			UNiagaraNodeFunctionCall* OwningFunctionCallNode;
			if (SystemViewModel->IsValid() && SystemViewModel->GetPlaceholderDataInterfaceManager()->TryGetOwnerInformation(InDataInterface, OwningEmitterHandle, OwningFunctionCallNode))
			{
				OwningSystem = &SystemViewModel->GetSystem();
				break;
			}
		}
	}

	InDataInterface->GetFeedback(OwningSystem, OwningComponent, OutErrors, OutWarnings, OutInfo);

}

void FNiagaraEditorModule::EnsureReservedDefinitionUnique(FGuid& UniqueId)
{
	if (!UniqueId.IsValid() || ReservedDefinitionIds.Contains(UniqueId))
	{
		UniqueId = FGuid::NewGuid();
	}

	ReservedDefinitionIds.Add(UniqueId);
}

const TArray<TWeakObjectPtr<UNiagaraParameterDefinitions>>& FNiagaraEditorModule::GetCachedParameterDefinitionsAssets()
{
	return ParameterDefinitionsAssetCache.Get();
}

const TArray<TWeakObjectPtr<UNiagaraParameterCollection>>& FNiagaraEditorModule::GetCachedParameterCollectionAssets()
{
	return ParameterCollectionAssetCache.Get();
}

void FNiagaraEditorModule::GetTargetSystemAndEmitterForDataInterface(UNiagaraDataInterface* InDataInterface, UNiagaraSystem*& OutOwningSystem, FVersionedNiagaraEmitter& OutOwningEmitter)
{
	OutOwningSystem = InDataInterface->GetTypedOuter<UNiagaraSystem>();
	OutOwningEmitter.Emitter = InDataInterface->GetTypedOuter<UNiagaraEmitter>();
	
	if (OutOwningSystem == nullptr)
	{
		// If no outer was find try to find one by componenet.
		UNiagaraComponent* OwningComponent = InDataInterface->GetTypedOuter<UNiagaraComponent>();
		if (OwningComponent != nullptr)
		{
			OutOwningSystem = OwningComponent->GetAsset();
		}
	}

	if (OutOwningSystem == nullptr)
	{
		// If no outer information is available check system view models for placeholder DIs.
		TArray<TSharedRef<FNiagaraSystemViewModel>> SystemViewModels;
		FNiagaraSystemViewModel::GetAllViewModels(SystemViewModels);
		for (TSharedRef<FNiagaraSystemViewModel> SystemViewModel : SystemViewModels)
		{
			FGuid OwningEmitterHandle;
			UNiagaraNodeFunctionCall* OwningFunctionCallNode;
			if (SystemViewModel->IsValid() && SystemViewModel->GetPlaceholderDataInterfaceManager()->TryGetOwnerInformation(InDataInterface, OwningEmitterHandle, OwningFunctionCallNode))
			{
				OutOwningSystem = &SystemViewModel->GetSystem();
				if (OwningEmitterHandle.IsValid())
				{
					OutOwningEmitter = SystemViewModel->GetEmitterHandleViewModelById(OwningEmitterHandle)->GetEmitterViewModel()->GetEmitter();
				}
				break;
			}
		}
	}
}

void FNiagaraEditorModule::RegisterDefaultRendererFactories()
{
	RegisterRendererCreationInfo(FNiagaraRendererCreationInfo(
		UNiagaraMeshRendererProperties::StaticClass()->GetDisplayNameText(),
		FText::FromString(UNiagaraMeshRendererProperties::StaticClass()->GetDescription()),
		UNiagaraMeshRendererProperties::StaticClass()->GetClassPathName(),
		FNiagaraRendererCreationInfo::FRendererFactory::CreateLambda([](UObject* OuterEmitter)
		{
			UNiagaraMeshRendererProperties* NewRenderer = NewObject<UNiagaraMeshRendererProperties>(OuterEmitter, NAME_None, RF_Transactional);

			// we have an empty entry in the constructor. Due to CDO default value propagation being unwanted, we have to keep it in there
			if(ensure(NewRenderer->Meshes.Num() == 1))
			{
				const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
				NewRenderer->Meshes[0].Mesh = Cast<UStaticMesh>(NiagaraEditorSettings->DefaultMeshRendererMesh.TryLoad());
			}
			return NewRenderer;
		})));
	
	RegisterRendererCreationInfo(FNiagaraRendererCreationInfo(
		UNiagaraSpriteRendererProperties::StaticClass()->GetDisplayNameText(),
		FText::FromString(UNiagaraSpriteRendererProperties::StaticClass()->GetDescription()),
		UNiagaraSpriteRendererProperties::StaticClass()->GetClassPathName(),
		FNiagaraRendererCreationInfo::FRendererFactory::CreateLambda([](UObject* OuterEmitter)
		{
			UNiagaraSpriteRendererProperties* NewRenderer = NewObject<UNiagaraSpriteRendererProperties>(OuterEmitter, NAME_None, RF_Transactional);
			const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
			NewRenderer->Material = Cast<UMaterialInterface>(NiagaraEditorSettings->DefaultSpriteRendererMaterial.TryLoad());
			return NewRenderer;
		})));
	
	RegisterRendererCreationInfo(FNiagaraRendererCreationInfo(
		UNiagaraRibbonRendererProperties::StaticClass()->GetDisplayNameText(),
		FText::FromString(UNiagaraRibbonRendererProperties::StaticClass()->GetDescription()),
		UNiagaraRibbonRendererProperties::StaticClass()->GetClassPathName(),
		FNiagaraRendererCreationInfo::FRendererFactory::CreateLambda([](UObject* OuterEmitter)
		{
			UNiagaraRibbonRendererProperties* NewRenderer = NewObject<UNiagaraRibbonRendererProperties>(OuterEmitter, NAME_None, RF_Transactional);
			const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
			NewRenderer->Material = Cast<UMaterialInterface>(NiagaraEditorSettings->DefaultRibbonRendererMaterial.TryLoad());
			return NewRenderer;
		})));

	RegisterRendererCreationInfo(FNiagaraRendererCreationInfo(
		UNiagaraComponentRendererProperties::StaticClass()->GetDisplayNameText(),
		FText::FromString(UNiagaraComponentRendererProperties::StaticClass()->GetDescription()),
		UNiagaraComponentRendererProperties::StaticClass()->GetClassPathName(),
		FNiagaraRendererCreationInfo::FRendererFactory::CreateLambda([](UObject* OuterEmitter)
		{
			UNiagaraComponentRendererProperties* NewRenderer = NewObject<UNiagaraComponentRendererProperties>(OuterEmitter, NAME_None, RF_Transactional);
			return NewRenderer;
		})));

	RegisterRendererCreationInfo(FNiagaraRendererCreationInfo(
		UNiagaraLightRendererProperties::StaticClass()->GetDisplayNameText(),
		FText::FromString(UNiagaraLightRendererProperties::StaticClass()->GetDescription()),
		UNiagaraLightRendererProperties::StaticClass()->GetClassPathName(),
		FNiagaraRendererCreationInfo::FRendererFactory::CreateLambda([](UObject* OuterEmitter)
		{
			UNiagaraLightRendererProperties* NewRenderer = NewObject<UNiagaraLightRendererProperties>(OuterEmitter, NAME_None, RF_Transactional);
			return NewRenderer;
		})));

	RegisterRendererCreationInfo(FNiagaraRendererCreationInfo(
		UNiagaraDecalRendererProperties::StaticClass()->GetDisplayNameText(),
		FText::FromString(UNiagaraDecalRendererProperties::StaticClass()->GetDescription()),
		UNiagaraDecalRendererProperties::StaticClass()->GetClassPathName(),
		FNiagaraRendererCreationInfo::FRendererFactory::CreateLambda([](UObject* OuterEmitter)
		{
			UNiagaraDecalRendererProperties* NewRenderer = NewObject<UNiagaraDecalRendererProperties>(OuterEmitter, NAME_None, RF_Transactional);
			const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
			NewRenderer->Material = Cast<UMaterialInterface>(NiagaraEditorSettings->DefaultDecalRendererMaterial.TryLoad());
			return NewRenderer;
		})));

	RegisterRendererCreationInfo(FNiagaraRendererCreationInfo(
		UNiagaraVolumeRendererProperties::StaticClass()->GetDisplayNameText(),
		FText::FromString(UNiagaraVolumeRendererProperties::StaticClass()->GetDescription()),
		UNiagaraVolumeRendererProperties::StaticClass()->GetClassPathName(),
		FNiagaraRendererCreationInfo::FRendererFactory::CreateLambda([](UObject* OuterEmitter)
		{
			UNiagaraVolumeRendererProperties* NewRenderer = NewObject<UNiagaraVolumeRendererProperties>(OuterEmitter, NAME_None, RF_Transactional);
			return NewRenderer;
		})));
}

void FNiagaraEditorModule::RegisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SequencerSettings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("NiagaraSequenceEditor"));
		SequencerSettings->SetStopPlayingWhenJumpingToStartOrEnd(true);

		SettingsModule->RegisterSettings("Editor", "ContentEditors", "NiagaraSequenceEditor",
			LOCTEXT("NiagaraSequenceEditorSettingsName", "Niagara Sequence Editor"),
			LOCTEXT("NiagaraSequenceEditorSettingsDescription", "Configure the look and feel of the Niagara Sequence Editor."),
			SequencerSettings);	
	}
}

void FNiagaraEditorModule::UnregisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Editor", "ContentEditors", "NiagaraSequenceEditor");
	}
}

void FNiagaraEditorModule::InitializeTaggedAssetBrowserDetails()
{	
	// Now we add information on how to display emitters & systems in the tagged asset browser
	// Emitters
	{
		FAssetDetailsDisplayInfo EmitterClassInfo;
	
		// Inheritance
		FDisplayedPropertyData InheritanceProperty;
		InheritanceProperty.ShouldDisplayPropertyDelegate = FDisplayedPropertyData::FShouldDisplayProperty::CreateLambda([](const FAssetData& AssetData)
		{
			if(AssetData.FindTag(GET_MEMBER_NAME_CHECKED(UNiagaraEmitter, bIsInheritable)))
			{
				return true;
			}
			
			return false; 
		});
		InheritanceProperty.NameWidgetDelegate = FDisplayedPropertyData::FGenerateWidget::CreateLambda([](const FAssetData& AssetData) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock)
			.Text(LOCTEXT("Inheritance", "Inheritance"));
		});
		InheritanceProperty.ValueWidgetDelegate = FDisplayedPropertyData::FGenerateWidget::CreateLambda([](const FAssetData& AssetData) -> TSharedRef<SWidget>
		{
			bool bUseInheritance = false;
			if(FNiagaraEditorUtilities::GetIsInheritableFromAssetRegistryTags(AssetData, bUseInheritance))
			{
				return SNew(STextBlock).Text(bUseInheritance
					? LOCTEXT("Emitter_UseInheritance_Yes", "Yes")
					: LOCTEXT("Emitter_UseInheritance_No", "No")
					);
			}
	
			return SNullWidget::NullWidget;
		});
		
		// Simulation Target
		FName SimulationTargetTag("HasGPUEmitter");
	
		FDisplayedPropertyData SimulationTargetProperty;
		SimulationTargetProperty.ShouldDisplayPropertyDelegate = FDisplayedPropertyData::FShouldDisplayProperty::CreateLambda([SimulationTargetTag](const FAssetData& Asset)
		{
			return Asset.FindTag(SimulationTargetTag);
		});
		SimulationTargetProperty.NameWidgetDelegate = FDisplayedPropertyData::FGenerateWidget::CreateLambda([](const FAssetData& AssetData) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock)
			.Text(LOCTEXT("SimulationTargetLabel", "Runs on"));
		});
		SimulationTargetProperty.ValueWidgetDelegate = FDisplayedPropertyData::FGenerateWidget::CreateLambda([SimulationTargetTag](const FAssetData& Asset)
		{
			return SNew(SImage)
			.Image_Lambda([Asset, SimulationTargetTag]() -> const FSlateBrush*
			{
				if(Asset.FindTag(SimulationTargetTag))
				{
					FString Value;
					if(Asset.GetTagValue(SimulationTargetTag, Value))
					{
						if(Value == "True")
						{
							return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.GPUIcon");
						}
						else
						{
							return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.CPUIcon");
						}
					}
				}
	
				return FAppStyle::GetNoBrush();
			});
		});
	
		EmitterClassInfo.DisplayedProperties =
		{
			InheritanceProperty,
			SimulationTargetProperty
		};
		
		EmitterClassInfo.GetDescriptionDelegate = FAssetDetailsDisplayInfo::FGetDescription::CreateLambda([](const FAssetData& AssetData)
		{
			FName DescriptionTagName("TemplateAssetDescription");
			if(AssetData.FindTag(DescriptionTagName))
			{
				FString Description;
				AssetData.GetTagValue(DescriptionTagName, Description);
				return FText::FromString(Description);
			}
	
			return FText::GetEmpty();
		});

		FTaggedAssetBrowserDetailsDisplayDatabase::RegisterClass(UNiagaraEmitter::StaticClass(), EmitterClassInfo);
	}
	
	// Systems
	{
		FAssetDetailsDisplayInfo SystemClassInfo;
	
		// Number of emitters
		FDisplayedPropertyData NumEmittersProperty;
		NumEmittersProperty.ShouldDisplayPropertyDelegate = FDisplayedPropertyData::FShouldDisplayProperty::CreateLambda([](const FAssetData& AssetData) -> bool
		{
			return AssetData.FindTag("NumEmitters");
		});
		NumEmittersProperty.NameWidgetDelegate = FDisplayedPropertyData::FGenerateWidget::CreateLambda([](const FAssetData& AssetData) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock)
			.Text(LOCTEXT("NumberOfEmitters", "Number of Emitters"));
		});
		NumEmittersProperty.ValueWidgetDelegate =  FDisplayedPropertyData::FGenerateWidget::CreateLambda([](const FAssetData& AssetData) -> TSharedRef<SWidget>
		{
			int32 NumEmitters = INDEX_NONE;
			if(AssetData.GetTagValue("NumEmitters", NumEmitters))
			{
				return SNew(STextBlock).Text(FText::AsNumber(NumEmitters));
			}
	
			return SNullWidget::NullWidget;
		});
		
		SystemClassInfo.DisplayedProperties =
		{
			NumEmittersProperty
		};
		
		SystemClassInfo.GetDescriptionDelegate = FAssetDetailsDisplayInfo::FGetDescription::CreateLambda([](const FAssetData& AssetData)
		{
			FName DescriptionTagName("TemplateAssetDescription");
			if(AssetData.FindTag(DescriptionTagName))
			{
				FString Description;
				AssetData.GetTagValue(DescriptionTagName, Description);
				return FText::FromString(Description);
			}
		
			return FText::GetEmpty();
		});
		
		FTaggedAssetBrowserDetailsDisplayDatabase::RegisterClass(UNiagaraSystem::StaticClass(), SystemClassInfo);
	}
}

void FNiagaraEditorModule::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (SequencerSettings)
	{
		Collector.AddReferencedObject(SequencerSettings);
	}
}

void FNiagaraEditorModule::OnPreGarbageCollection()
{
	if (IsRunningCommandlet() && !IsEngineExitRequested())
	{
		// For commandlets like GenerateDistillFileSetsCommandlet, they just load the package and do some hierarchy navigation within it 
		// tracking sub-assets, then they garbage collect. Since nothing is holding onto the system at the root level, it will be summarily
		// killed and any of references will also be killed. To thwart this for now, we are forcing the compilations to complete BEFORE
		// garbage collection kicks in. To do otherwise for now has too many loose ends (a system may be left around after the level has been
		// unloaded, leaving behind weird external references, etc). This should be revisited when more time is available (i.e. not days before a 
		// release is due to go out).
		for (TObjectIterator<UNiagaraSystem> It; It; ++It)
		{
			if (UNiagaraSystem* System = *It)
			{
				if (System->CompileRequestsShouldBlockGC())
				{
					System->WaitForCompilationComplete();
				}
			}
		}
	}
}

void FNiagaraEditorModule::OnExecParticleInvoked(const TCHAR* Str)
{
	// Very similar logic to UEditorEngine::Exec_Particle
	if (FParse::Command(&Str, TEXT("RESET")))
	{
		auto ReinitActorComponents =
			[](AActor* Actor)
			{
				TArray<UNiagaraComponent*, TInlineAllocator<1>> ComponentsToReset;
				for (UActorComponent* AC : Actor->GetComponents())
				{
					if (UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(AC))
					{
						ComponentsToReset.Add(NiagaraComponent);
					}
				}

				for (UNiagaraComponent* NiagaraComponent : ComponentsToReset)
				{
					NiagaraComponent->ReinitializeSystem();
				}
			};

		if (FParse::Command(&Str, TEXT("SELECTED")))
		{
			// Reset any selected emitters in the level
			for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
			{
				AActor* Actor = CastChecked<AActor>(*It);
				ReinitActorComponents(Actor);
			}
		}
		else if (FParse::Command(&Str, TEXT("ALL")))
		{
			// Reset ALL emitters in the level
			for (TObjectIterator<AActor> It; It; ++It)
			{
				ReinitActorComponents(*It);
			}
		}
	}
}

void FNiagaraEditorModule::ReinitializeStyle()
{
	FNiagaraEditorStyle::ReinitializeStyle();
}

void FNiagaraEditorModule::EnqueueObjectForDeferredDestructionInternal(FDeferredDestructionContainerBase* InObjectToDestruct)
{
	if (EnqueuedForDeferredDestruction.Num() == 0)
	{
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FNiagaraEditorModule::DeferredDestructObjects));
	}
	EnqueuedForDeferredDestruction.Add(MakeShareable<FDeferredDestructionContainerBase>(InObjectToDestruct));
}

bool FNiagaraEditorModule::DeferredDestructObjects(float InDeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FNiagaraEditorModule_DeferredDestructObjects);
	EnqueuedForDeferredDestruction.Empty();
	return false;
}

void FNiagaraEditorModule::RegisterDefaultStackObjectCustomizations()
{
	StackObjectCustomizationRegistry.RegisterStackObjectCustomization(*UNiagaraStatelessModule_DynamicMaterialParameters::StaticClass(),
		FNiagaraStackObjectCustomizationRegistry::FOnGetStackObjectCustomizationInstance::CreateStatic(&FNiagaraStackObjectPropertyCustomization_StatelessModule_DynamicMaterialParameters::MakeInstance));
}

void FNiagaraEditorModule::OnAssetRegistryLoadComplete()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraEditorModule::OnAssetRegistryLoadComplete);
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	AssetRegistry.OnFilesLoaded().Remove(AssetRegistryOnLoadCompleteHandle);

	check(AssetRegistry.IsLoadingAssets() == false);
	// NDC is only for when running in the Editor with UI
	if (!IsRunningCookCommandlet())
	{
		IAssetTools& AssetTools = IAssetTools::Get();

		//Ensure All Data Channel Assets are loaded and available for use in editor.
		TArray<FAssetData> AllDataChannels;
		AssetRegistry.GetAssetsByClass(UNiagaraDataChannelAsset::StaticClass()->GetClassPathName(), AllDataChannels);
		for (const FAssetData& DataChannelAsset : AllDataChannels)
		{
			if (AssetTools.IsAssetVisible(DataChannelAsset, true))
			{
				DataChannelAsset.GetAsset();
			}
		}
	}
}

void FNiagaraEditorModule::OnAssetsPreDelete(const TArray<UObject*>& Objects)
{
	for(UObject* Obj : Objects)
	{
		if(UNiagaraDataChannelAsset* NDCAsset = Cast<UNiagaraDataChannelAsset>(Obj))
		{
			FNiagaraWorldManager::ForAllWorldManagers(
				[DataChannel = NDCAsset->Get()](FNiagaraWorldManager& WorldMan)
			{
				WorldMan.RemoveDataChannel(DataChannel);
			});
		}
	}
}

void FNiagaraEditorModule::ConditionalRegisterDynamicStruct(const UScriptStruct* Struct)
{
	//Register customizations for Niagara Data Channel Access Context types.
	if (Struct->IsChildOf(FNDCAccessContextBase::StaticStruct()))
	{
		const FName StructName = Struct->GetFName();

		//Ensure we don't register the struct again.
		//If the module is unloaded / reloaded we'll still have it's correct customizer in place.
		if (RegisteredDynamicStructs.Contains(StructName))
		{
			return;
		}

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		PropertyModule.RegisterCustomPropertyTypeLayout(
			StructName,
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNDCAccessContextDetailsCustomization::MakeInstance));

		RegisteredDynamicStructs.Add(StructName);
	}
}

void FNiagaraEditorModule::OnModulesChanged(FName Module, EModuleChangeReason Reason)
{
	//Refresh any dynamic customizations for potentially newly loaded types.
	if (Reason == EModuleChangeReason::ModuleLoaded)
	{
		// if we are running a cook commandlet we don't need to setup the NDC types or refresh them
		if (!IsRunningCookCommandlet())
		{
			if (FModuleManager::Get().DoesLoadedModuleHaveUObjects(Module))
			{
				FNameBuilder ScriptPackageName(FPackageName::GetModuleScriptPackageName(Module));
				if (const UPackage* ScriptPackage = FindPackage(nullptr, ScriptPackageName.ToString()))
				{
					ForEachObjectWithPackage(ScriptPackage, [this](UObject* Object) -> bool
					{
						if (const UScriptStruct* Struct = Cast<const UScriptStruct>(Object))
						{
							ConditionalRegisterDynamicStruct(Struct);
						}

						return true;
					});
				}
			}
		}
	}
}

void FNiagaraEditorModule::RegisterDynamicCustomizations()
{
	// if we are running a cook commandlet we don't need to setup the NDC types or refresh them
	if (IsRunningCookCommandlet())
	{
		return;
	}

	//Register any dynamic/discovered class customizations.
	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		ConditionalRegisterDynamicStruct(*It);
	}
}

void FNiagaraEditorModule::UnregisterDynamicCustomizations()
{
	if (!RegisteredDynamicStructs.IsEmpty())
	{
		if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			for (const FName& StructName : RegisteredDynamicStructs)
			{
				PropertyModule.UnregisterCustomPropertyTypeLayout(StructName);
			}
		}

		RegisteredDynamicStructs.Reset();
	}
}

void FNiagaraEditorModule::RegisterMenuExtensions()
{
	RegisterMenuExtension_MigrateToUserAssetTags();
}

void FNiagaraEditorModule::RegisterMenuExtension_MigrateToUserAssetTags()
{
	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
	{
		UToolMenu* MenuExtension = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.NiagaraAssetTagDefinitions");
		FToolMenuSection& Section = MenuExtension->FindOrAddSection("GetAssetActions");
		Section.AddDynamicEntry("MigrateToUserAssetTagsDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
				if (Context)
				{
					if(Context->SelectedAssets.Num() > 0)
					{
						InSection.AddMenuEntry(
							"MigrateToUserAssetTags",
							LOCTEXT("MigrateToUserAssetTags_Label", "Migrate Tagged Assets to User Asset Tags"),
							LOCTEXT("MigrateToUserAssetTags_Tooltip",
								"Migrate the assets that were tagged using tags from this Tag Definitions assets to the new User Asset Tag system."),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Tag"),
							FExecuteAction::CreateLambda([InSelectedAssets = Context->SelectedAssets]()
								{
									TArray<UObject*> SelectedObjects;
									SelectedObjects.Reserve(InSelectedAssets.Num());

									for(const FAssetData& Asset : InSelectedAssets)
									{
										if(UObject* LoadedAsset = Asset.GetAsset())
										{
											SelectedObjects.Add(LoadedAsset);
										}
									}

									FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
									FARFilter Filter;
									Filter.ClassPaths.Add(UNiagaraEmitter::StaticClass()->GetClassPathName());
									Filter.ClassPaths.Add(UNiagaraSystem::StaticClass()->GetClassPathName());
									
									for(UObject* SelectedObject : SelectedObjects)
									{
										UDEPRECATED_NiagaraAssetTagDefinitions* DefinitionsAsset = Cast<UDEPRECATED_NiagaraAssetTagDefinitions>(SelectedObject);
										for(const FNiagaraAssetTagDefinition& Definition : DefinitionsAsset->GetAssetTagDefinitions())
										{
											Filter.TagsAndValues.Add(FName(Definition.GetGuidAsString()));
										}
									}

									TArray<FAssetData> OutAssetData;
									AssetRegistryModule.Get().GetAssets(Filter, OutAssetData);

									TArray<UPackage*> PackagesToResave;
									Algo::Transform(OutAssetData, PackagesToResave, [](const FAssetData& AssetData)
									{
										return AssetData.GetPackage();
									});

									for(UPackage* Package : PackagesToResave)
									{
										Package->MarkPackageDirty();
									}
									
									if(PackagesToResave.Num() > 0)
									{
										UEditorLoadingAndSavingUtils::SavePackagesWithDialog(PackagesToResave, false);
									}
									else
									{
										UE_LOGF(LogNiagaraEditor, Warning, "Didn't find any assets to resave");
									}
								}));
					}
				}
			}));
	}));
}

FNiagaraEditorModule::FOnScriptApplied& FNiagaraEditorModule::OnScriptApplied()
{
	return OnScriptAppliedDelegate;
}

const FNiagaraEditorModule::FMoviePreviewObjects& FNiagaraEditorModule::FindOrAddMoviePreviewObjects(const FAssetData& AssetData)
{
	if(MoviePreviewObjects.Contains(AssetData))
	{
		return MoviePreviewObjects[AssetData];
	}
	
	FString PreviewMoviePathString;
	bool bHasMovieSequence = AssetData.GetTagValue("PreviewMoviePath", PreviewMoviePathString);
	FSoftObjectPath PreviewMoviePath(PreviewMoviePathString);
	UObject* MoviePreviewObject = PreviewMoviePath.TryLoad();
	UFileMediaSource* MediaSource = Cast<UFileMediaSource>(MoviePreviewObject);
	
	FMoviePreviewObjects& PreviewObjects = MoviePreviewObjects.Add(AssetData);
	PreviewObjects.MediaPlayer.Reset(NewObject<UMediaPlayer>(GetTransientPackage(), UMediaPlayer::StaticClass()));
	PreviewObjects.MediaTexture.Reset(NewObject<UMediaTexture>(GetTransientPackage()));

	PreviewObjects.MediaPlayer->SetLooping(true);
	PreviewObjects.MediaPlayer->AffectedByPIEHandling = false;
	PreviewObjects.MediaPlayer->OpenSource(MediaSource);
	PreviewObjects.MediaTexture->SetMediaPlayer(PreviewObjects.MediaPlayer.Get());
	PreviewObjects.MediaTexture->UpdateResource();
	PreviewObjects.MediaPlayer->Play();
	PreviewObjects.ImageBrush = new FSlateImageBrush(PreviewObjects.MediaTexture.Get(), {80.f, 80.f});

	return PreviewObjects;
}

UNiagaraParameterCollection* FNiagaraEditorModule::FindCollectionForVariable(const FString& VariableName)
{
	auto FindCachedCollectionByPrefix = [this](const FString& Prefix)
	{
		for (const TWeakObjectPtr<UNiagaraParameterCollection>& CollectionPtr : ParameterCollectionAssetCache.Get())
		{
			if (UNiagaraParameterCollection* Collection = CollectionPtr.Get())
			{
				if (Prefix.StartsWith(Collection->GetFullNamespaceName().ToString()))
				{
					return Collection;
				}
			}
		}

		return (UNiagaraParameterCollection*)nullptr;
	};

	if (UNiagaraParameterCollection* Collection = FindCachedCollectionByPrefix(VariableName))
	{
		return Collection;
	}

	ParameterCollectionAssetCache.RefreshCache(!FUObjectThreadContext::Get().IsRoutingPostLoad /*bAllowLoading*/);

	return FindCachedCollectionByPrefix(VariableName);
}

void FNiagaraEditorModule::ValidateScriptVariableIds(const TArray<FString>& ScriptPathArgs, bool bFix)
{
	for(const FString& ScriptPath : ScriptPathArgs)
	{
		FSoftObjectPath ScriptSoftPath(ScriptPath);
		if(ScriptSoftPath.IsAsset())
		{
			UObject* ScriptObject = ScriptSoftPath.TryLoad();
			if(UNiagaraScript* Script = Cast<UNiagaraScript>(ScriptObject))
			{
				if(bFix)
				{
					FNiagaraEditorUtilities::Scripts::Validation::FixupDuplicateScriptVariableGuids(Script);
				}
				else
				{
					for(FNiagaraAssetVersion& AssetVersion : Script->GetAllAvailableVersions())
					{
						FNiagaraEditorUtilities::Scripts::Validation::ValidateScriptVariableIds(Script, AssetVersion.VersionGuid);
					}
				}
			}
		}
	}	
}

#if NIAGARA_PERF_BASELINES
void FNiagaraEditorModule::GeneratePerfBaselines(TArray<UNiagaraEffectType*>& BaselinesToGenerate)
{
	if (BaselinesToGenerate.Num() == 0)
	{
		return;
	}

	if (!BaselineViewport.IsValid())
	{
		//Spawn a new window and preview scene to run the baseline inside.
		TSharedRef<SWindow> NewWindow = SNew(SWindow)
			.Title(LOCTEXT("NiagaraBaselineWindow", "Gathering Niagara Performance Baselines...."))
			.SizingRule(ESizingRule::FixedSize)
			.ClientSize(FVector2D(1920, 1080))
			//.IsTopmostWindow(true)
			.SupportsMaximize(false)
			.SupportsMinimize(false);

		TSharedPtr<SWindow> WindowPtr;
		WindowPtr = NewWindow;

		BaselineViewport = SNew(SNiagaraBaselineViewport);
		BaselineViewport->Init(WindowPtr);

		NewWindow->SetContent(BaselineViewport.ToSharedRef());
		FSlateApplication::Get().AddWindow(NewWindow);

		NewWindow->GetOnWindowClosedEvent().AddRaw(this, &FNiagaraEditorModule::OnPerfBaselineWindowClosed);
	}
	
	
	for (UNiagaraEffectType* EffectType : BaselinesToGenerate)
	{
		if (EffectType->IsPerfBaselineValid() == false && EffectType->GetPerfBaselineController())
		{
			if (!BaselineViewport->AddBaseline(EffectType))
			{
				// We may want to do something smarter than this in the future, but right now we will infinitely loop on these settings. 
				// Might as well make them defaults (0.0f) and have everything fail relative to them.
				FNiagaraPerfBaselineStats Stats;
				EffectType->UpdatePerfBaselineStats(Stats);

				UE_LOGF(LogNiagaraEditor, Warning, "Failed to add baseline! %ls", *EffectType->GetName());
			}
		}
	}
}

void FNiagaraEditorModule::OnPerfBaselineWindowClosed(const TSharedRef<SWindow>& ClosedWindow)
{
	ClosedWindow->SetContent(SNullWidget::NullWidget);
	BaselineViewport.Reset();
}
void FNiagaraEditorModule::PreloadSelectablePluginAssetsByClass(UClass* InClass)
{
	if (GbPreloadSelectablePluginAssetsOnDemand && PluginAssetClassesPreloaded.Contains(InClass) == false)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		TArray<FAssetData> AllClassAssets;
		AssetRegistryModule.Get().GetAssetsByClass(InClass->GetClassPathName(), AllClassAssets);
		for (FAssetData& ClassAsset : AllClassAssets)
		{
			if (ClassAsset.HasAnyPackageFlags(PKG_Cooked) && ClassAsset.IsAssetLoaded() == false && FNiagaraEditorUtilities::IsEnginePluginAsset(FTopLevelAssetPath(ClassAsset.GetSoftObjectPath().ToString())))
			{
				ClassAsset.GetAsset();
			}
		}
		PluginAssetClassesPreloaded.Add(InClass);
	}
}

void FNiagaraEditorModule::ScriptApplied(UNiagaraScript* Script, FGuid VersionGuid) const
{
	ensure(Script != nullptr);
	OnScriptAppliedDelegate.Broadcast(Script, VersionGuid);
}

#endif

#undef LOCTEXT_NAMESPACE

