// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorSettings.h"

#include "NiagaraActions.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorModule.h"
#include "NiagaraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEditorSettings)

// These GUIDs are now statically defined so that they can be serialized out in settings for the parameter panel category expansion.
const FGuid FNiagaraEditorGuids::SystemNamespaceMetaDataGuid = FGuid(TEXT("7B4AFB34D0DF46189A05349E361CE735"));
const FGuid FNiagaraEditorGuids::EmitterNamespaceMetaDataGuid = FGuid(TEXT("1BA31433B3314F6BB258AECFBB466AC7"));
const FGuid FNiagaraEditorGuids::ParticleAttributeNamespaceMetaDataGuid = FGuid(TEXT("530A57A84EF444F482CCC0A4B6D8D364"));
const FGuid FNiagaraEditorGuids::ModuleNamespaceMetaDataGuid = FGuid(TEXT("FCA14ABEFAD14BB58786362887FEED09"));
const FGuid FNiagaraEditorGuids::ModuleOutputNamespaceMetaDataGuid = FGuid(TEXT("8945EAA4041E43F6824AC3B0AF2AEA20"));
const FGuid FNiagaraEditorGuids::ModuleLocalNamespaceMetaDataGuid = FGuid(TEXT("1DAA70FAC2ED4512914BD788D5BC3C9D"));
const FGuid FNiagaraEditorGuids::TransientNamespaceMetaDataGuid = FGuid(TEXT("39BA69CE1CB74A168469C1F49C4E37DE"));
const FGuid FNiagaraEditorGuids::StackContextNamespaceMetaDataGuid = FGuid(TEXT("65275FF7723C4CF28A82EB033EB3FAA6"));
const FGuid FNiagaraEditorGuids::EngineNamespaceMetaDataGuid = FGuid(TEXT("18363BDD94D549038AA825175F92DDF9"));
const FGuid FNiagaraEditorGuids::UserNamespaceMetaDataGuid = FGuid(TEXT("A3AC42514BF24B84AD2C52D2276414DA"));
const FGuid FNiagaraEditorGuids::ParameterCollectionNamespaceMetaDataGuid = FGuid(TEXT("A5843E1CF9924F16B52016676332DEDC"));
const FGuid FNiagaraEditorGuids::DataInstanceNamespaceMetaDataGuid = FGuid(TEXT("17F5FC610A914BA58D7EA8FD39B9A1D0"));
const FGuid FNiagaraEditorGuids::StaticSwitchNamespaceMetaDataGuid = FGuid(TEXT("6A44CDD2EC3D4495BDC7FE28CAE00604"));

int32 GbLogFoundButNotAllowedAssets = 0;
static FAutoConsoleVariableRef CVarLogFoundButNotAllowedAssets(
	TEXT("fx.Niagara.LogFoundButNotAllowedAssets"),
	GbShowNiagaraDeveloperWindows,
	TEXT("If > 0 assets which are found, but not allowed in the current editor context will be printed to the log."),
	ECVF_Default
);

FNiagaraNamespaceMetadata::FNiagaraNamespaceMetadata()
	: BackgroundColor(FLinearColor::Black)
	, ForegroundStyle("NiagaraEditor.ParameterName.NamespaceText")
	, SortId(TNumericLimits<int32>::Max())
{
}

FNiagaraNamespaceMetadata::FNiagaraNamespaceMetadata(TArray<FName> InNamespaces, FName InRequiredNamespaceModifier)
	: Namespaces(InNamespaces)
	, RequiredNamespaceModifier(InRequiredNamespaceModifier)
	, BackgroundColor(FLinearColor::Black)
	, ForegroundStyle("NiagaraEditor.ParameterName.NamespaceText")
	, SortId(TNumericLimits<int32>::Max())
{
}

FNiagaraActionColors::FNiagaraActionColors()
	: NiagaraColor(EForceInit::ForceInitToZero)
	, GameColor(EForceInit::ForceInitToZero)
	, PluginColor(EForceInit::ForceInitToZero)
	, DeveloperColor(EForceInit::ForceInitToZero)
{
}

UNiagaraEditorSettings::UNiagaraEditorSettings(const FObjectInitializer& ObjectInitlaizer)
	: Super(ObjectInitlaizer)
{
	bAutoCompile = true;
	bAutoPlay = true;
	bResetSimulationOnChange = true;
	bResimulateOnChangeWhilePaused = true;
	bResetDependentSystemsWhenEditingEmitters = false;
	SetupNamespaceMetadata();
	TrackedUsageBaseClasses = { UNiagaraNode::StaticClass(), UNiagaraDataInterface::StaticClass(), UNiagaraRendererProperties::StaticClass() };
	bForceSilentLoadingOfCachedAssets = false;
}

#define LOCTEXT_NAMESPACE "NamespaceMetadata"

FLinearColor UNiagaraEditorSettings::GetSourceColor(EScriptSource  Source) const
{
	if(Source == EScriptSource::Niagara)
	{
		return ActionColors.NiagaraColor;
	}
	else if(Source == EScriptSource::Game)
	{
		return ActionColors.GameColor;
	}
	else if(Source == EScriptSource::Plugins)
	{
		return ActionColors.PluginColor;
	}
	else if(Source == EScriptSource::Developer)
	{
		return ActionColors.DeveloperColor;
	}

	return FLinearColor(1.f,1.f,1.f,0.3f);
}

FLinearColor UNiagaraEditorSettings::GetDefaultNoteColor() const
{
	return DefaultNoteColor;
}

void UNiagaraEditorSettings::SetupNamespaceMetadata()
{
	DefaultNamespaceMetadata = FNiagaraNamespaceMetadata({ NAME_None })
		.SetDisplayName(LOCTEXT("DefaultDisplayName", "None"))
		.SetDescription(LOCTEXT("DefaultDescription", "Non-standard unknown namespace."))
		.SetBackgroundColor(FLinearColor(FColor(102, 102, 102)))
		.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace)
		.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier)
		.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingName);

	NamespaceMetadata =
	{
		FNiagaraNamespaceMetadata({FNiagaraConstants::SystemNamespace})
			.SetDisplayName(LOCTEXT("SystemDisplayName", "System"))
			.SetDisplayNameLong(LOCTEXT("SystemDisplayNameLong", "System Attributes"))
			.SetDescription(LOCTEXT("SystemDescription", "Persistent attribute in the system which is written in a system\n stage and can be read anywhere."))
			.SetBackgroundColor(FLinearColor(FColor(49, 113, 142)))
			.SetSortId(10)
			.AddOptionalNamespaceModifier(FNiagaraConstants::ModuleNamespace)
			.AddOptionalNamespaceModifier(FNiagaraConstants::InitialNamespace)
			.AddOptionalNamespaceModifier(FNiagaraConstants::PreviousNamespace)
			.SetGuid(FNiagaraEditorGuids::SystemNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::EmitterNamespace})
			.SetDisplayName(LOCTEXT("EmitterDisplayName", "Emitter"))
			.SetDisplayNameLong(LOCTEXT("EmitterDisplayNameLong", "Emitter Attributes"))
			.SetDescription(LOCTEXT("EmitterDescription", "Persistent attribute which is written in a emitter\nstage and can be read in emitter and particle stages."))
			.SetBackgroundColor(FLinearColor(FColor(145, 99, 56)))
			.SetSortId(20)
			.AddOptionalNamespaceModifier(FNiagaraConstants::ModuleNamespace)
			.AddOptionalNamespaceModifier(FNiagaraConstants::InitialNamespace)
			.AddOptionalNamespaceModifier(FNiagaraConstants::PreviousNamespace)
			.SetGuid(FNiagaraEditorGuids::EmitterNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::ParticleAttributeNamespace})
			.SetDisplayName(LOCTEXT("ParticleDisplayName", "Particles"))
			.SetDisplayNameLong(LOCTEXT("ParticleDisplayNameLong", "Particle Attributes"))
			.SetDescription(LOCTEXT("ParticleDescription", "Persistent attribute which is written in a particle\nstage and can be read in particle stages."))
			.SetBackgroundColor(FLinearColor(FColor(72, 130, 71)))
			.SetSortId(30)
			.AddOptionalNamespaceModifier(FNiagaraConstants::ModuleNamespace)
			.AddOptionalNamespaceModifier(FNiagaraConstants::InitialNamespace)
			.AddOptionalNamespaceModifier(FNiagaraConstants::PreviousNamespace)
			.SetGuid(FNiagaraEditorGuids::ParticleAttributeNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::ModuleNamespace})
			.SetDisplayName(LOCTEXT("ModuleDisplayName", "Input"))
			.SetDisplayNameLong(LOCTEXT("ModuleDisplayNameLong", "Module Inputs"))
			.SetDescription(LOCTEXT("ModuleDescription", "A value which exposes a module input to the system and emitter editor."))
			.SetBackgroundColor(FLinearColor(FColor(136, 66, 65)))
			.SetSortId(40)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInSystem)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
			.SetGuid(FNiagaraEditorGuids::ModuleNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::OutputNamespace}, FNiagaraConstants::ModuleNamespace)
			.SetDisplayName(LOCTEXT("ModuleOutputDisplayName", "Output"))
			.SetDisplayNameLong(LOCTEXT("ModuleOutputDisplayNameLong", "Module Outputs"))
			.SetDescription(LOCTEXT("ModuleOutputDescription", "A transient value which the module author has decided might be useful to other modules further down in the stage.\nTransient values do not persist from frame to frame, or between stages, e.g. emitter to particle, or spawn to update."))
			.SetBackgroundColor(FLinearColor(FColor(108, 87, 131)))
			.SetSortId(60)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInScript)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInSystem)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventCreatingInSystemEditor)
			.SetGuid(FNiagaraEditorGuids::ModuleOutputNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::LocalNamespace, FNiagaraConstants::ModuleNamespace})
			.SetDisplayName(LOCTEXT("ModuleLocalDisplayName", "Local"))
			.SetDisplayNameLong(LOCTEXT("ModuleLocalDisplayNameLong", "Module Locals"))
			.SetDescription(LOCTEXT("ModuleLocalDescription", "A transient value which can be written to and read from within a single module.\nTransient values do not persist from frame to frame, or between stages, e.g. emitter to particle, or spawn to update."))
			.SetBackgroundColor(FLinearColor(FColor(191, 176, 84)))
			.SetForegroundStyle("NiagaraEditor.ParameterName.NamespaceTextDark")
			.SetSortId(50)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInSystem)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier)
			.SetGuid(FNiagaraEditorGuids::ModuleLocalNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::TransientNamespace})
			.SetDisplayName(LOCTEXT("TransientDisplayName", "Transient"))
			.SetDisplayNameLong(LOCTEXT("TransientDisplayNameLong", "Stage Transients"))
			.SetDescription(LOCTEXT("TransientDescription", "A transient value which can be written to and read from from any module.\nTransient values do not persist from frame to frame, or between stages, e.g. emitter to particle, or spawn to update."))
			.SetBackgroundColor(FLinearColor(FColor(108, 87, 131)))
			.SetSortId(80)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInScript)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInSystem)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier)
			.SetGuid(FNiagaraEditorGuids::TransientNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::StackContextNamespace})
			.SetDisplayName(LOCTEXT("StackContextDisplayName", "StackContext"))
			.SetDisplayNameLong(LOCTEXT("StackContextDisplayNameLong", "Stack Context Sensitive"))
			.SetDescription(LOCTEXT("StackContextDescription", "A value which can be written to and read from from any module.\nStackContext values do  persist from frame to frame, or between stages, e.g. emitter to particle, or spawn to update.\nThey take on the namespace most relevant to where they are used.\nSystem Spawn/Update: \"System\"\nEmitter Spawn/Update: \"Emitter\".\nParticle Spawn/Update/Events: \"Particles\".\nParticle Simulation Stage iterating Particles: \"Particles\".\nParticle Simulation Stage with Iteration Source: Writes to the iteration source directly."))
			.SetBackgroundColor(FLinearColor(FColor(87, 131, 121)))
			.SetSortId(80)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInScript)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInSystem)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
		//.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier),
			.AddOptionalNamespaceModifier(FNiagaraConstants::ModuleNamespace)
			.AddOptionalNamespaceModifier(FNiagaraConstants::InitialNamespace)
			.AddOptionalNamespaceModifier(FNiagaraConstants::PreviousNamespace)
			.SetGuid(FNiagaraEditorGuids::StackContextNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::EngineNamespace})
			.SetDisplayName(LOCTEXT("EngineDisplayName", "Engine"))
			.SetDisplayNameLong(LOCTEXT("EngineDisplayNameLong", "Engine Provided"))
			.SetDescription(LOCTEXT("EngineDescription", "A read only value which is provided by the engine.\nThis value's source can be the simulation itsef\ne.g. ExecutionCount, or the owner of the simulation (The component), e.g. (Owner) Scale."))
			.SetBackgroundColor(FLinearColor(FColor(170, 170, 170)))
			.SetForegroundStyle("NiagaraEditor.ParameterName.NamespaceTextDark")
			.SetSortId(70)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingName)
			.SetGuid(FNiagaraEditorGuids::EngineNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::UserNamespace})
			.SetDisplayName(LOCTEXT("UserDisplayName", "User"))
			.SetDisplayNameLong(LOCTEXT("UserDisplayNameLong", "User Exposed"))
			.SetDescription(LOCTEXT("UserDescription", "A read only value which can be initialized per system and\nmodified externally in the level, by blueprint, or by c++."))
			.SetBackgroundColor(FLinearColor(FColor(91, 161, 194)))
			.SetSortId(0)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInScript)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace)
			.SetGuid(FNiagaraEditorGuids::UserNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::ParameterCollectionNamespace})
			.SetDisplayName(LOCTEXT("NiagaraParameterCollectionDisplayName", "NPC"))
			.SetDisplayNameLong(LOCTEXT("NiagaraParameterCollectionDisplayNameLong", "Niagara Parameter Collection"))
			.SetDescription(LOCTEXT("NiagaraParameterCollectionDescription", "Values read from a niagara parameter collection asset.\nRead only in a niagara system."))
			.SetBackgroundColor(FLinearColor(FColor(170, 170, 170)))
			.SetForegroundStyle("NiagaraEditor.ParameterName.NamespaceTextDark")
			.SetSortId(90)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInScript)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInSystem)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingName)
			.SetGuid(FNiagaraEditorGuids::ParameterCollectionNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::DataInstanceNamespace})
			.SetDisplayName(LOCTEXT("DataInstanceDisplayName", "Data Instance"))
			.SetDescription(LOCTEXT("DataInstanceDescription", "A special value which has a single bool IsAlive value, which determines if a particle is alive or not."))
			.SetBackgroundColor(FLinearColor(FColor(170, 170, 170)))
			.SetForegroundStyle("NiagaraEditor.ParameterName.NamespaceTextDark")
			.SetSortId(100)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInSystem)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInScript)
			.AddOption(ENiagaraNamespaceMetadataOptions::AdvancedInSystem)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingName)
			.SetGuid(FNiagaraEditorGuids::DataInstanceNamespaceMetaDataGuid),
		FNiagaraNamespaceMetadata({FNiagaraConstants::StaticSwitchNamespace})
			.SetDisplayName(LOCTEXT("StatisSwitchDisplayName", "Static Switch Inputs"))
			.SetDescription(LOCTEXT("StaticSwitchDescription", "Values which can only be set at edit time."))
			.SetSortId(45)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInSystem)
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInDefinitions)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier)
			.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingName)
			.SetGuid(FNiagaraEditorGuids::StaticSwitchNamespaceMetaDataGuid),
	};

	DefaultNamespaceModifierMetadata = FNiagaraNamespaceMetadata({ NAME_None })
		.SetDisplayName(LOCTEXT("DefaultModifierDisplayName", "None"))
		.SetDescription(LOCTEXT("DefaultModifierDescription", "Arbitrary sub-namespace for specifying module specific dataset attributes, or calling nested modules."))
		.SetBackgroundColor(FLinearColor(FColor(102, 102, 102)))
		.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace)
		.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier)
		.AddOption(ENiagaraNamespaceMetadataOptions::PreventEditingName);

	NamespaceModifierMetadata =
	{
		FNiagaraNamespaceMetadata({FNiagaraConstants::InitialNamespace})
			.SetDisplayName(LOCTEXT("InitialModifierDisplayName", "Initial"))
			.SetDescription(LOCTEXT("InitialModifierDescription", "A namespace modifier for dataset attributes which when used in\na linked input in an update script will get the initial value from the spawn script."))
			.SetBackgroundColor(FLinearColor(FColor(170, 170, 170)))
			.SetForegroundStyle("NiagaraEditor.ParameterName.NamespaceTextDark"),
		FNiagaraNamespaceMetadata({FNiagaraConstants::PreviousNamespace})
			.SetDisplayName(LOCTEXT("PreviousModifierDisplayName", "Previous"))
			.SetDescription(LOCTEXT("PreviousModifierDescription", "A namespace modifier for dataset attributes which when used in\na linked input in an update script will get the value from the start of the update script."))
			.SetBackgroundColor(FLinearColor(FColor(152, 152, 102)))
			.SetForegroundStyle("NiagaraEditor.ParameterName.NamespaceTextDark"),
		FNiagaraNamespaceMetadata({FNiagaraConstants::ModuleNamespace})
			.SetDisplayName(LOCTEXT("ModuleModifierDisplayName", "Module"))
			.SetDescription(LOCTEXT("ModuleModifierDescription", "A namespace modifier which makes that attribute unique to the module\ninstance by appending the unique module name."))
			.SetBackgroundColor(FLinearColor(FColor(102, 102, 152)))
			.AddOption(ENiagaraNamespaceMetadataOptions::HideInSystem),
		FNiagaraNamespaceMetadata({FNiagaraConstants::SystemNamespace})
			.SetDisplayName(LOCTEXT("SystemModifierDisplayName", "System"))
			.SetDescription(LOCTEXT("SystemModifierDescription", "A namespace modifier which specifies that an engine provided parameter comes from the system."))
			.SetBackgroundColor(FLinearColor(FColor(49, 113, 142))),
		FNiagaraNamespaceMetadata({FNiagaraConstants::EmitterNamespace})
			.SetDisplayName(LOCTEXT("EmitterModifierDisplayName", "Emitter"))
			.SetDescription(LOCTEXT("EmitterModifierDescription", "A namespace modifier which specifies that an engine provided parameter comes from the emitter."))
			.SetBackgroundColor(FLinearColor(FColor(145, 99, 56))),
		FNiagaraNamespaceMetadata({FNiagaraConstants::OwnerNamespace})
			.SetDisplayName(LOCTEXT("OwnerDisplayName", "Owner"))
			.SetDescription(LOCTEXT("OwnerDescription", "A namespace modifier which specifies that an engine provided parameter comes from the owner, or component."))
			.SetBackgroundColor(FLinearColor(FColor(170, 170, 170)))
			.SetForegroundStyle("NiagaraEditor.ParameterName.NamespaceTextDark"),
		FNiagaraNamespaceMetadata({FNiagaraConstants::SharedNamespace})
			.SetDisplayName(LOCTEXT("SharedDisplayName", "Shared"))
			.SetDescription(LOCTEXT("SharedDescription", "A namespace modifier which specifies that a parameter is being used to 'share' a value across multiple inputs and the UI should display it accordingly."))
			.SetBackgroundColor(FLinearColor(FColor(170, 170, 170)))
			.SetForegroundStyle("NiagaraEditor.ParameterName.NamespaceTextDark"),
	};
}

void UNiagaraEditorSettings::BuildCachedPlaybackSpeeds() const
{
	CachedPlaybackSpeeds = PlaybackSpeeds;
	if (!CachedPlaybackSpeeds.GetValue().Contains(1.f))
	{
		CachedPlaybackSpeeds.GetValue().Add(1.f);
	}
	
	CachedPlaybackSpeeds.GetValue().Sort();
}

bool UNiagaraEditorSettings::IsShowGridInViewport() const
{
	return ViewportSettings.bShowGridInViewport;
}

void UNiagaraEditorSettings::SetShowGridInViewport(bool bInShowGridInViewport)
{
	if (this->ViewportSettings.bShowGridInViewport != bInShowGridInViewport)
	{
		this->ViewportSettings.bShowGridInViewport = bInShowGridInViewport;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::IsShowInstructionsCount() const
{
	return ViewportSettings.bShowInstructionsCount;
}

void UNiagaraEditorSettings::SetShowInstructionsCount(bool bInShowInstructionsCount)
{
	if (this->ViewportSettings.bShowInstructionsCount != bInShowInstructionsCount)
	{
		this->ViewportSettings.bShowInstructionsCount = bInShowInstructionsCount;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::IsShowParticleCountsInViewport() const
{
	return ViewportSettings.bShowParticleCountsInViewport;
}

void UNiagaraEditorSettings::SetShowParticleCountsInViewport(bool bInShowParticleCountsInViewport)
{
	if (this->ViewportSettings.bShowParticleCountsInViewport != bInShowParticleCountsInViewport)
	{
		this->ViewportSettings.bShowParticleCountsInViewport = bInShowParticleCountsInViewport;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::IsShowEmitterExecutionOrder() const
{
	return ViewportSettings.bShowEmitterExecutionOrder;
}

void UNiagaraEditorSettings::SetShowEmitterExecutionOrder(bool bInShowEmitterExecutionOrder)
{
	if (this->ViewportSettings.bShowEmitterExecutionOrder != bInShowEmitterExecutionOrder)
	{
		this->ViewportSettings.bShowEmitterExecutionOrder = bInShowEmitterExecutionOrder;
		SaveConfig();		
	}
}

bool UNiagaraEditorSettings::IsShowGpuTickInformation() const
{
	return ViewportSettings.bShowGpuTickInformation;
}

void UNiagaraEditorSettings::SetShowGpuTickInformation(bool bInShowGpuTickInformation)
{
	if (ViewportSettings.bShowGpuTickInformation != bInShowGpuTickInformation)
	{
		ViewportSettings.bShowGpuTickInformation = bInShowGpuTickInformation;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::IsShowMemoryInfo() const
{
	return ViewportSettings.bShowMemoryInfo;
}

void UNiagaraEditorSettings::SetShowMemoryInfo(bool bInShowInfo)
{
	if (ViewportSettings.bShowMemoryInfo != bInShowInfo)
	{
		ViewportSettings.bShowMemoryInfo = bInShowInfo;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::IsShowStatelessInfo() const
{
	return ViewportSettings.bShowStatelessInfo;
}

void UNiagaraEditorSettings::SetShowStatelessInfo(bool bInShowInfo)
{
	if (ViewportSettings.bShowStatelessInfo != bInShowInfo)
	{
		ViewportSettings.bShowStatelessInfo = bInShowInfo;
		SaveConfig();
	}
}

TArray<float> UNiagaraEditorSettings::GetPlaybackSpeeds() const
{
	if(!CachedPlaybackSpeeds.IsSet())
	{
		BuildCachedPlaybackSpeeds();
	}

	return CachedPlaybackSpeeds.GetValue();	
}

bool UNiagaraEditorSettings::GetAutoCompile() const
{
	return bAutoCompile;
}

void UNiagaraEditorSettings::SetAutoCompile(bool bInAutoCompile)
{
	if (bAutoCompile != bInAutoCompile)
	{
		bAutoCompile = bInAutoCompile;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::GetAutoPlay() const
{
	return bAutoPlay;
}

void UNiagaraEditorSettings::SetAutoPlay(bool bInAutoPlay)
{
	if (bAutoPlay != bInAutoPlay)
	{
		bAutoPlay = bInAutoPlay;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::GetResetSimulationOnChange() const
{
	return bResetSimulationOnChange;
}

void UNiagaraEditorSettings::SetResetSimulationOnChange(bool bInResetSimulationOnChange)
{
	if (bResetSimulationOnChange != bInResetSimulationOnChange)
	{
		bResetSimulationOnChange = bInResetSimulationOnChange;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::GetResimulateOnChangeWhilePaused() const
{
	return bResimulateOnChangeWhilePaused;
}

void UNiagaraEditorSettings::SetResimulateOnChangeWhilePaused(bool bInResimulateOnChangeWhilePaused)
{
	if (bResimulateOnChangeWhilePaused != bInResimulateOnChangeWhilePaused)
	{
		bResimulateOnChangeWhilePaused = bInResimulateOnChangeWhilePaused;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::GetResetDependentSystemsWhenEditingEmitters() const
{
	return bResetDependentSystemsWhenEditingEmitters;
}

void UNiagaraEditorSettings::SetResetDependentSystemsWhenEditingEmitters(bool bInResetDependentSystemsWhenEditingEmitters)
{
	if (bResetDependentSystemsWhenEditingEmitters != bInResetDependentSystemsWhenEditingEmitters)
	{
		bResetDependentSystemsWhenEditingEmitters = bInResetDependentSystemsWhenEditingEmitters;
		SaveConfig();
	}
}

bool UNiagaraEditorSettings::GetDisplayAdvancedParameterPanelCategories() const
{
	return bDisplayAdvancedParameterPanelCategories;
}

void UNiagaraEditorSettings::SetDisplayAdvancedParameterPanelCategories(bool bInDisplayAdvancedParameterPanelCategories)
{
	if (bDisplayAdvancedParameterPanelCategories != bInDisplayAdvancedParameterPanelCategories)
	{
		bDisplayAdvancedParameterPanelCategories = bInDisplayAdvancedParameterPanelCategories;
		SaveConfig();
		SettingsChangedDelegate.Broadcast(GET_MEMBER_NAME_CHECKED(UNiagaraEditorSettings, bDisplayAdvancedParameterPanelCategories).ToString(), this);
	}
}

// Use the guids from the list at the top of this file to register last known expansion state.
FNiagaraParameterPanelSectionStorage& UNiagaraEditorSettings::FindOrAddParameterPanelSectionStorage(FGuid PanelSectionId, bool& bOutAdded)
{
	bOutAdded = false;
	check(PanelSectionId.IsValid());
	for (FNiagaraParameterPanelSectionStorage& Storage : SystemParameterPanelSectionData)
	{
		if (PanelSectionId == Storage.ParamStorageId)
		{
			return Storage;
		}
	}

	int32 Idx = SystemParameterPanelSectionData.Emplace(PanelSectionId);
	bOutAdded = true;
	return SystemParameterPanelSectionData[Idx];

}

bool UNiagaraEditorSettings::GetDisplayAffectedAssetStats() const
{
	return bDisplayAffectedAssetStats;
}

int32 UNiagaraEditorSettings::GetAssetStatsSearchLimit() const
{
	return AffectedAssetSearchLimit;
}

FNiagaraNamespaceMetadata UNiagaraEditorSettings::GetDefaultNamespaceMetadata() const
{
	return DefaultNamespaceMetadata;
}

FNiagaraNamespaceMetadata UNiagaraEditorSettings::GetMetaDataForNamespaces(TArray<FName> InNamespaces) const
{
	TArray<FNiagaraNamespaceMetadata> MatchingMetadata;
	for (const FNiagaraNamespaceMetadata& NamespaceMetadataItem : NamespaceMetadata)
	{
		if (NamespaceMetadataItem.Namespaces.Num() <= InNamespaces.Num())
		{	
			bool bNamespacesMatch = true;
			for (int32 i = 0; i < NamespaceMetadataItem.Namespaces.Num() && bNamespacesMatch; i++)
			{
				if(NamespaceMetadataItem.Namespaces[i] != InNamespaces[i])
				{
					bNamespacesMatch = false;
				}
			}
			if (bNamespacesMatch)
			{
				MatchingMetadata.Add(NamespaceMetadataItem);
			}
		}
	}
	if (MatchingMetadata.Num() == 0)
	{
		return FNiagaraNamespaceMetadata();
	}
	else if (MatchingMetadata.Num() == 1)
	{
		return MatchingMetadata[0];
	}
	else
	{
		int32 IndexOfLargestMatch = 0;
		for (int32 i = 1; i < MatchingMetadata.Num(); i++)
		{
			if (MatchingMetadata[i].Namespaces.Num() > MatchingMetadata[IndexOfLargestMatch].Namespaces.Num())
			{
				IndexOfLargestMatch = i;
			}
		}
		return MatchingMetadata[IndexOfLargestMatch];
	}
}

FNiagaraNamespaceMetadata UNiagaraEditorSettings::GetMetaDataForId(const FGuid& NamespaceId) const
{
	for (const FNiagaraNamespaceMetadata& NamespaceMetaDatum : NamespaceMetadata)
	{
		if (NamespaceMetaDatum.GetGuid() == NamespaceId)
		{
			return NamespaceMetaDatum;
		}
	}
	ensureMsgf(false, TEXT("Failed to find namespace metadata by ID!"));
	return DefaultNamespaceMetadata;
}

const FGuid& UNiagaraEditorSettings::GetIdForUsage(ENiagaraScriptUsage Usage) const
{
	switch (Usage) {
	case ENiagaraScriptUsage::SystemSpawnScript:
	case ENiagaraScriptUsage::SystemUpdateScript:
		return FNiagaraEditorGuids::SystemNamespaceMetaDataGuid;
	case ENiagaraScriptUsage::EmitterSpawnScript:
	case ENiagaraScriptUsage::EmitterUpdateScript:
		return FNiagaraEditorGuids::EmitterNamespaceMetaDataGuid;
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
	case ENiagaraScriptUsage::ParticleGPUComputeScript:
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
		return FNiagaraEditorGuids::ParticleAttributeNamespaceMetaDataGuid;
	default:
		ensureMsgf(false, TEXT("Encounted unexpected usage when finding namespace metadata!"));
		return DefaultNamespaceMetadata.GetGuid();
	}
}

const TArray<FNiagaraNamespaceMetadata>& UNiagaraEditorSettings::GetAllNamespaceMetadata() const
{
	return NamespaceMetadata;
}

FNiagaraNamespaceMetadata UNiagaraEditorSettings::GetDefaultNamespaceModifierMetadata() const
{
	return DefaultNamespaceModifierMetadata;
}

FNiagaraNamespaceMetadata UNiagaraEditorSettings::GetMetaDataForNamespaceModifier(FName NamespaceModifier) const
{
	for (const FNiagaraNamespaceMetadata& NamespaceModifierMetadataItem : NamespaceModifierMetadata)
	{
		if (NamespaceModifierMetadataItem.Namespaces.Num() == 1 && NamespaceModifierMetadataItem.Namespaces[0] == NamespaceModifier)
		{
			return NamespaceModifierMetadataItem;
		}
	}
	return FNiagaraNamespaceMetadata();
}

const TArray<FNiagaraNamespaceMetadata>& UNiagaraEditorSettings::GetAllNamespaceModifierMetadata() const
{
	return NamespaceModifierMetadata;
}

const TArray<FNiagaraCurveTemplate>& UNiagaraEditorSettings::GetCurveTemplates() const
{
	return CurveTemplates;
}

FName UNiagaraEditorSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

FText UNiagaraEditorSettings::GetSectionText() const
{
	return NSLOCTEXT("NiagaraEditorPlugin", "NiagaraEditorSettingsSection", "Niagara Editor");
}

void UNiagaraEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	CachedPlaybackSpeeds.Reset();
	
	if (PropertyChangedEvent.Property != nullptr)
	{
		SettingsChangedDelegate.Broadcast(PropertyChangedEvent.Property->GetName(), this);
	}
}

void UNiagaraEditorSettings::SetViewportSharedSettings(const FNiagaraViewportSharedSettings& InViewportSharedSettings)
{
	ViewportSettings = InViewportSharedSettings;
	SaveConfig();
}

UNiagaraEditorSettings::FOnNiagaraEditorSettingsChanged& UNiagaraEditorSettings::OnSettingsChanged()
{
	return GetMutableDefault<UNiagaraEditorSettings>()->SettingsChangedDelegate;
}

void UNiagaraEditorSettings::SetOnIsClassVisible(const FOnIsClassAllowed& InOnIsClassAllowed)
{
	OnIsClassVisibleDelegate = InOnIsClassAllowed;
}

void UNiagaraEditorSettings::SetOnIsClassReferenceable(const FOnIsClassAllowed& InOnIsClassAllowed)
{
	OnIsClassReferenceableDelegate = InOnIsClassAllowed;
}

void UNiagaraEditorSettings::SetOnShouldFilterAssetByClassUsage(const FOnShouldFilterAsset& InOnShouldFilterAssetByClassUsage)
{
	OnShouldFilterAssetByClassUsage = InOnShouldFilterAssetByClassUsage;
}

void UNiagaraEditorSettings::SetOnShouldFilterAssetInNiagaraAssetBrowser(const FOnShouldFilterAsset& InOnShouldFilterAssetInNiagaraAssetBrowser)
{
	OnShouldFilterAssetInNiagaraAssetBrowser = InOnShouldFilterAssetInNiagaraAssetBrowser;
}

void UNiagaraEditorSettings::SetOnGetExternalAssetTabSpawners(FOnGetExternalTabsForAsset InOnGetExternalTabsForAsset, FOnExternalExtendLayoutForAsset InOnExternalExtendLayoutForAsset)
{
	OnGetExternalTabsForAsset = InOnGetExternalTabsForAsset;
	OnExternalExtendLayoutForAsset = InOnExternalExtendLayoutForAsset;
}

bool UNiagaraEditorSettings::IsVisibleClass(const UClass* InClass) const
{
	return OnIsClassVisibleDelegate.IsBound() == false || OnIsClassVisibleDelegate.Execute(InClass);
}

bool UNiagaraEditorSettings::IsReferenceableClass(const UClass* InClass) const
{
	return OnIsClassReferenceableDelegate.IsBound() == false || OnIsClassReferenceableDelegate.Execute(InClass);
}

bool UNiagaraEditorSettings::IsVisibleTypeDefinition(const FNiagaraTypeDefinition& InTypeDefinition) const
{
	return InTypeDefinition.GetClass() == nullptr || IsVisibleClass(InTypeDefinition.GetClass());
}

const FName ClassUsageListTagName = "ClassUsageList";

bool UNiagaraEditorSettings::ShouldTrackClassUsage(const UClass* InClass) const
{
	for (const UClass* TrackedUsageBaseClass : TrackedUsageBaseClasses)
	{
		if (InClass->IsChildOf(TrackedUsageBaseClass))
		{
			return true;
		}
	}
	return false;
}

void UNiagaraEditorSettings::GetExternalTabSpawnersForAsset(TSharedRef<FAssetEditorToolkit> InHostToolkit, UObject& InAsset, TArray<TSharedRef<FWorkflowTabFactory>>& OutExternalTabSpawners) const
{
	OnGetExternalTabsForAsset.ExecuteIfBound(InHostToolkit, InAsset, OutExternalTabSpawners);
}

void UNiagaraEditorSettings::ExternalExtendLayoutForAsset(TSharedRef<FLayoutExtender> InLayoutExtender, UObject& InAsset) const
{
	OnExternalExtendLayoutForAsset.ExecuteIfBound(InLayoutExtender, InAsset);
}

UObject::FAssetRegistryTag UNiagaraEditorSettings::CreateClassUsageAssetRegistryTag(const UObject* SourceObject) const
{
	FString ClassUsageList;
	if (SourceObject->IsAsset())
	{
		TSet<FString> ClassPaths;
		TArray<UObject*> ObjectsInPackage;
		GetObjectsWithPackage(SourceObject->GetOutermost(), ObjectsInPackage);
		for (UObject* ObjectInPackage : ObjectsInPackage)
		{
			if (ShouldTrackClassUsage(ObjectInPackage->GetClass()))
			{
				ClassPaths.Add(ObjectInPackage->GetClass()->GetClassPathName().ToString());
			}
		}
		ClassPaths.Sort([](const FString& A, const FString& B) { return A < B; });
		ClassUsageList = FString::Join(ClassPaths, TEXT("\n"));
	}
	else
	{
		ClassUsageList = "";
	}

	return UObject::FAssetRegistryTag(ClassUsageListTagName, ClassUsageList, FAssetRegistryTag::TT_Hidden);
}

bool UNiagaraEditorSettings::IsAllowedAssetObjectByClassUsageInternal(const UObject& AssetObject, TSet<const UObject*>& CheckedAssetObjects) const
{
	CheckedAssetObjects.Add(&AssetObject);

	TArray<UObject*> ObjectsInPackage;
	GetObjectsWithPackage(AssetObject.GetOutermost(), ObjectsInPackage);
	for (UObject* ObjectInPackage : ObjectsInPackage)
	{
		if (ShouldTrackClassUsage(ObjectInPackage->GetClass()) && IsReferenceableClass(ObjectInPackage->GetClass()) == false)
		{
			if (GbLogFoundButNotAllowedAssets)
			{
				UE_LOGF(LogNiagaraEditor, Log, "Asset %ls is not allowed due to object %ls with class %ls which is not allowed in this editor context.",
					*AssetObject.GetPathName(), *ObjectInPackage->GetPathName(), *ObjectInPackage->GetClass()->GetClassPathName().ToString());
			}
			return false;
		}

		const UNiagaraNode* NiagaraNode = Cast<UNiagaraNode>(ObjectInPackage);
		if (NiagaraNode != nullptr &&
			NiagaraNode->GetReferencedAsset() != nullptr &&
			OnShouldFilterAssetByClassUsage.Execute(FTopLevelAssetPath(NiagaraNode->GetReferencedAsset()->GetPathName())) &&
			CheckedAssetObjects.Contains(NiagaraNode->GetReferencedAsset()) == false)
		{
			if (IsAllowedAssetObjectByClassUsageInternal(*NiagaraNode->GetReferencedAsset(), CheckedAssetObjects) == false)
			{
				UE_LOGF(LogNiagaraEditor, Log, "Asset %ls is not allowed due to referenced asset %ls which is not allowed in this editor context.",
					*AssetObject.GetPathName(), *NiagaraNode->GetReferencedAsset()->GetPathName());
				return false;
			}
		}
	}
	return true;
}

EGraphZoomLimitHandling UNiagaraEditorSettings::GetScriptEditorZoomLimitHandling() const
{
	return ScriptGraphZoomLimitHandling;
}

bool UNiagaraEditorSettings::IsCurveInputSnapEnabled() const
{
	return CurveEditorSharedSettings.bCurveInputSnapEnabled;
}

bool UNiagaraEditorSettings::IsCurveOutputSnapEnabled() const
{
	return CurveEditorSharedSettings.bCurveOutputSnapEnabled;
}

void UNiagaraEditorSettings::SetCurveInputSnapEnabled(bool bEnabled)
{
	CurveEditorSharedSettings.bCurveInputSnapEnabled = bEnabled;
}

void UNiagaraEditorSettings::SetCurveOutputSnapEnabled(bool bEnabled)
{
	CurveEditorSharedSettings.bCurveOutputSnapEnabled = bEnabled;
}

bool UNiagaraEditorSettings::IsAllowedObjectByClassUsageInternal(const UObject& InObject, TSet<const UObject*>& CheckedObjects) const
{
	CheckedObjects.Add(&InObject);

	TArray<UObject*> ObjectsWithOuter;
	GetObjectsWithOuter(&InObject, ObjectsWithOuter, EGetObjectsFlags::IncludeNestedObjects);
	
	for (const UObject* ObjectInPackage : ObjectsWithOuter)
	{
		if (ShouldTrackClassUsage(ObjectInPackage->GetClass()) && IsReferenceableClass(ObjectInPackage->GetClass()) == false)
		{
			if (GbLogFoundButNotAllowedAssets)
			{
				UE_LOGF(LogNiagaraEditor, Log, "Asset %ls is not allowed due to object %ls with class %ls which is not allowed in this editor context.",
					*InObject.GetPackage()->GetPathName(), *ObjectInPackage->GetPathName(), *ObjectInPackage->GetClass()->GetClassPathName().ToString());
			}
			return false;
		}

		const UNiagaraNode* NiagaraNode = Cast<UNiagaraNode>(ObjectInPackage);
		if (NiagaraNode != nullptr &&
			NiagaraNode->GetReferencedAsset() != nullptr &&
			OnShouldFilterAssetByClassUsage.Execute(FTopLevelAssetPath(NiagaraNode->GetReferencedAsset()->GetPathName())) &&
			CheckedObjects.Contains(NiagaraNode->GetReferencedAsset()) == false)
		{
			if (IsAllowedAssetObjectByClassUsageInternal(*NiagaraNode->GetReferencedAsset(), CheckedObjects) == false)
			{
				UE_LOGF(LogNiagaraEditor, Log, "Asset %ls is not allowed due to referenced asset %ls which is not allowed in this editor context.",
					*InObject.GetPackage()->GetPathName(), *NiagaraNode->GetReferencedAsset()->GetPathName());
				return false;
			}
		}
	}
	return true;
}

bool UNiagaraEditorSettings::IsAllowedAssetByClassUsage(const FAssetData& InAssetData) const
{
	if (OnShouldFilterAssetByClassUsage.IsBound() == false ||
		OnIsClassVisibleDelegate.IsBound() == false ||
		OnShouldFilterAssetByClassUsage.Execute(FTopLevelAssetPath(InAssetData.GetObjectPathString())) == false)
	{
		return true;
	}

	bool bClassDataFound = false;
	bool bInvalidClassFound = false;
	if (InAssetData.IsAssetLoaded() && InAssetData.GetAsset() != nullptr)
	{
		TSet<const UObject*> CheckedAssetObjects;
		bClassDataFound = true;
		bInvalidClassFound = IsAllowedAssetObjectByClassUsageInternal(*InAssetData.GetAsset(), CheckedAssetObjects) == false;
	}
	else
	{
		FString ClassUsageList;
		if (InAssetData.GetTagValue(ClassUsageListTagName, ClassUsageList) && ClassUsageList.Len() != 0)
		{
			bClassDataFound = true;
			TArray<FString> ClassUsageListLines;
			ClassUsageList.ParseIntoArrayLines(ClassUsageListLines);
			for (const FString& ClassUsageListLine : ClassUsageListLines)
			{
				FTopLevelAssetPath ClassPath(ClassUsageListLine);
				UClass* UsedClass = FindObject<UClass>(ClassPath);
				if (UsedClass == nullptr || (ShouldTrackClassUsage(UsedClass) && IsVisibleClass(UsedClass) == false))
				{
					bInvalidClassFound = true;
					if (GbLogFoundButNotAllowedAssets)
					{
						if (UsedClass == nullptr)
						{
							UE_LOGF(LogNiagaraEditor, Log, "Asset %ls is not allowed due to class %ls which is was not found.",
								*InAssetData.GetFullName(), *ClassUsageListLine);
						}
						else
						{
							UE_LOGF(LogNiagaraEditor, Log, "Asset %ls is not allowed due to class %ls which is not allowed in this editor context.",
								*InAssetData.GetFullName(), *UsedClass->GetPathName());
						}
					}
					else
					{
						break;
					}
				}
			}
		}
	}
	return bClassDataFound && bInvalidClassFound == false;
}

bool UNiagaraEditorSettings::IsAllowedAssetObjectByClassUsage(const UObject& InAssetObject) const
{
	TSet<const UObject*> CheckedAssetObjects;
	if (OnIsClassVisibleDelegate.IsBound() && InAssetObject.IsAsset() == false)
	{
		return IsAllowedObjectByClassUsageInternal(InAssetObject, CheckedAssetObjects);
	}
	
	if (OnShouldFilterAssetByClassUsage.IsBound() == false ||
		OnIsClassVisibleDelegate.IsBound() == false ||
		OnShouldFilterAssetByClassUsage.Execute(FTopLevelAssetPath(InAssetObject.GetPathName())) == false)
	{
		return true;
	}

	return IsAllowedAssetObjectByClassUsageInternal(InAssetObject, CheckedAssetObjects);
}

bool UNiagaraEditorSettings::IsAllowedAssetInNiagaraAssetBrowser(const FAssetData& InAssetData) const
{
	if(OnShouldFilterAssetInNiagaraAssetBrowser.IsBound())
	{
		return OnShouldFilterAssetInNiagaraAssetBrowser.Execute(FTopLevelAssetPath(InAssetData.GetObjectPathString())) == false;
	}

	return true;
}

bool UNiagaraEditorSettings::GetUpdateStackValuesOnCommitOnly() const
{
	return bUpdateStackValuesOnCommitOnly;
}

#undef LOCTEXT_NAMESPACE

