// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorSimCacheUtils.h"
#include "NiagaraSimCache.h"

#include "ContentBrowserMenuContexts.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraSimCacheJson.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Toolkits/NiagaraSimCacheToolkit.h"
#include "ToolMenus.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "HAL/FileManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogNiagaraSimCache, Log, All);

#define LOCTEXT_NAMESPACE "NiagaraEditorSimCacheUtils"

namespace FNiagaraEditorSimCacheUtils
{

template<typename TArrayType>
void ExportToDiskInternal(TArrayType CachesToExport)
{
	FString ExportFolder;
		
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const bool bFolderPicked = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("ExportSimCache", "Pick SimCache Export Folder" ).ToString(),
			*(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT)),
			ExportFolder);

		if (!bFolderPicked)
		{
			return;
		}
		FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_EXPORT, ExportFolder); 
	}
		
	bool bWarning = false;
	IFileManager& FileManager = IFileManager::Get();
	for (const UNiagaraSimCache* Cache : CachesToExport)
	{
		if (Cache == nullptr)
		{
			continue;
		}

		FString CacheRootFolder = FPaths::Combine(ExportFolder, FPaths::MakeValidFileName(Cache->GetName(), '_'));
		if (FileManager.DirectoryExists(*CacheRootFolder))
		{
			if (!FileManager.DeleteDirectory(*CacheRootFolder, true, true))
			{
				UE_LOGF(LogNiagaraSimCache, Warning, "Unable to delete existing folder %ls", *CacheRootFolder);
				bWarning = true;
				continue;
			}
		}
		if (!FileManager.MakeDirectory(*CacheRootFolder, true))
		{
			UE_LOGF(LogNiagaraSimCache, Warning, "Unable to create folder %ls", *CacheRootFolder);
			bWarning = true;
			continue;
		}
		if (!FNiagaraSimCacheJson::DumpToFile(*Cache, CacheRootFolder, FNiagaraSimCacheJson::EExportType::SeparateEachFrame))
		{
			bWarning = true;
		}
	}
		
	FNotificationInfo Info(LOCTEXT("ExportToDisk_DoneInfo", "Export completed."));
	Info.ExpireDuration = 4.0f;
	if (bWarning)
	{
		Info.Text = LOCTEXT("ExportData_DoneWarn", "Export completed with warnings.\nPlease check the log.");
		Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
	}
	FSlateNotificationManager::Get().AddNotification(Info);
}

} //namespace FNiagaraEditorSimCacheUtils


void FNiagaraEditorSimCacheUtils::ExportToDisk(TArrayView<UNiagaraSimCache*> CachesToExport)
{
	ExportToDiskInternal(CachesToExport);
}

void FNiagaraEditorSimCacheUtils::ExportToDisk(const UNiagaraSimCache* CacheToExport)
{
	ExportToDiskInternal(MakeArrayView(&CacheToExport, 1));
}

#undef LOCTEXT_NAMESPACE
