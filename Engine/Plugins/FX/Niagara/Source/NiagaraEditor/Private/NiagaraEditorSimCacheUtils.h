// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UNiagaraSimCache;

namespace FNiagaraEditorSimCacheUtils
{
	extern void ExportToDisk(TArrayView<UNiagaraSimCache*> CachesToExport);
	extern void ExportToDisk(const UNiagaraSimCache* CacheToExport);
}
