// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SegmentedControlStyle.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"

TSharedPtr<FNiagaraEditorStyle> FNiagaraEditorStyle::NiagaraEditorStyle = nullptr;

void FNiagaraEditorStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FNiagaraEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

void FNiagaraEditorStyle::Shutdown()
{
	Unregister();
	NiagaraEditorStyle.Reset();
}

const FVector2D Icon8x8(8.0f, 8.0f);
const FVector2D Icon12x12(12.0f, 12.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon32x32(32.0f, 32.0f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon64x64(64.0f, 64.0f);

void FNiagaraEditorStyle::InitStats()
{
	const FTextBlockStyle CategoryText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DetailsView.CategoryTextStyle");
	const FSlateColor SelectionColor = FAppStyle::GetSlateColor("SelectionColor");
	const FSlateColor SelectionColor_Pressed = FAppStyle::GetSlateColor("SelectionColor_Pressed");
	
	Set("NiagaraEditor.StatsText", CategoryText);
}

void FNiagaraEditorStyle::InitAssetPicker()
{
	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");

	FTextBlockStyle AssetPickerBoldAssetNameText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FLinearColor::White)
		.SetFont(DEFAULT_FONT("Bold", 9));

	Set("NiagaraEditor.AssetPickerBoldAssetNameText", AssetPickerBoldAssetNameText);

	FTextBlockStyle AssetPickerAssetNameText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FLinearColor::White)
		.SetFont(DEFAULT_FONT("Regular", 9));

	Set("NiagaraEditor.AssetPickerAssetNameText", AssetPickerAssetNameText);

	FTextBlockStyle AssetPickerAssetCategoryText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 11));

	Set("NiagaraEditor.AssetPickerAssetCategoryText", AssetPickerAssetCategoryText);


	FTextBlockStyle AssetPickerAssetSubcategoryText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 10));

	Set("NiagaraEditor.AssetPickerAssetSubcategoryText", AssetPickerAssetSubcategoryText);

	// New Asset Dialog
	FTextBlockStyle NewAssetDialogOptionText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 11));

	Set("NiagaraEditor.NewAssetDialog.OptionText", NewAssetDialogOptionText);

	FTextBlockStyle NewAssetDialogHeaderText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FLinearColor::White)
		.SetFont(DEFAULT_FONT("Bold", 10));

	Set("NiagaraEditor.NewAssetDialog.HeaderText", NewAssetDialogHeaderText);

	FTextBlockStyle NewAssetDialogSubHeaderText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FLinearColor::White)
		.SetFont(DEFAULT_FONT("Bold", 11));

	Set("NiagaraEditor.NewAssetDialog.SubHeaderText", NewAssetDialogSubHeaderText);

	Set("NiagaraEditor.NewAssetDialog.AddButton", FButtonStyle()
		.SetNormal(CORE_BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, FLinearColor(0, 0, 0, .25f)))
		.SetHovered(CORE_BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, FAppStyle::GetSlateColor("SelectionColor")))
		.SetPressed(CORE_BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, FAppStyle::GetSlateColor("SelectionColor_Pressed")))
	);

	Set("NiagaraEditor.NewAssetDialog.SubBorderColor", FLinearColor(FColor(48, 48, 48)));
	Set("NiagaraEditor.NewAssetDialog.ActiveOptionBorderColor", FLinearColor(FColor(96, 96, 96)));

	Set("NiagaraEditor.NewAssetDialog.SubBorder", new CORE_BOX_BRUSH("Common/GroupBorderLight", FMargin(4.0f / 16.0f)));


}

void FNiagaraEditorStyle::InitActionMenu()
{
	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Action Menu
	FTextBlockStyle ActionMenuHeadingText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetFont(DEFAULT_FONT("Bold", 10));

	FTextBlockStyle ActionMenuActionText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetFont(DEFAULT_FONT("Regular", 9));

	FTextBlockStyle ActionMenuSourceText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetFont(DEFAULT_FONT("Regular", 7));

	FTextBlockStyle ActionMenuFilterText = FTextBlockStyle(NormalText)
        .SetColorAndOpacity(FSlateColor::UseForeground())
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetShadowOffset(FVector2D(1.f, 1.f))
        .SetFont(DEFAULT_FONT("Bold", 9));
        
	FTextBlockStyle TemplateTabText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetFont(DEFAULT_FONT("Bold", 11));

	const FCheckBoxStyle NiagaraGraphActionMenuFilterCheckBox = FCheckBoxStyle()
            .SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
            .SetUncheckedImage( FSlateNoResource() )
            .SetUncheckedHoveredImage(CORE_BOX_BRUSH("Common/RoundedSelection_16x", 4.0f/16.0f, FLinearColor(0.7f, 0.7f, 0.7f) ))
            .SetUncheckedPressedImage(CORE_BOX_BRUSH("Common/RoundedSelection_16x", 4.0f/16.0f, FLinearColor(0.8f, 0.8f, 0.8f) ))
            .SetCheckedImage(CORE_BOX_BRUSH("Common/RoundedSelection_16x",  4.0f/16.0f, FLinearColor(0.9f, 0.9f, 0.9f) ))
            .SetCheckedHoveredImage(CORE_BOX_BRUSH("Common/RoundedSelection_16x",  4.0f/16.0f, FLinearColor(1.f, 1.f, 1.f) ))
            .SetCheckedPressedImage(CORE_BOX_BRUSH("Common/RoundedSelection_16x",  4.0f/16.0f, FLinearColor(1.f, 1.f, 1.f) ));

	const FTableRowStyle ActionMenuRowStyle = FTableRowStyle()
            .SetEvenRowBackgroundBrush(FSlateNoResource())
            .SetOddRowBackgroundBrush(FSlateNoResource())
			.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f)))
			.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f)))
            .SetSelectorFocusedBrush(CORE_BORDER_BRUSH("Common/Selector", FMargin(4.f / 16.f), FStarshipCoreStyle::GetCoreStyle().GetSlateColor("SelectorColor")))
            .SetActiveBrush(FSlateColorBrush(FStyleColors::Select))
            .SetActiveHoveredBrush(FSlateColorBrush(FStyleColors::Select))
            .SetInactiveBrush(FSlateColorBrush(FStyleColors::SelectInactive))
            .SetInactiveHoveredBrush(FSlateColorBrush(FStyleColors::SelectInactive))
            .SetActiveHighlightedBrush(FSlateColorBrush(FStyleColors::PrimaryHover))
            .SetInactiveHighlightedBrush(FSlateColorBrush(FStyleColors::SelectParent))
			.SetTextColor(FStyleColors::Foreground)
			.SetSelectedTextColor(FStyleColors::ForegroundInverted)

            .SetDropIndicator_Above(CORE_BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0, 0), FStarshipCoreStyle::GetCoreStyle().GetSlateColor("SelectorColor")))
            .SetDropIndicator_Onto(CORE_BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), FStarshipCoreStyle::GetCoreStyle().GetSlateColor("SelectorColor")))
            .SetDropIndicator_Below(CORE_BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), FStarshipCoreStyle::GetCoreStyle().GetSlateColor("SelectorColor")));
	
	Set("ActionMenu.Row", ActionMenuRowStyle);
	
	Set("ActionMenu.HeadingTextBlock", ActionMenuHeadingText);

	Set("ActionMenu.ActionTextBlock", ActionMenuActionText);

	Set("GraphActionMenu.Background", new FSlateRoundedBoxBrush(FStyleColors::Background, 3.f));
	
	Set("GraphActionMenu.ActionSourceTextBlock", ActionMenuSourceText);

	Set("GraphActionMenu.ActionFilterTextBlock", ActionMenuFilterText);

	Set("GraphActionMenu.TemplateTabTextBlock", TemplateTabText);
	
	Set("GraphActionMenu.FilterCheckBox", NiagaraGraphActionMenuFilterCheckBox);

	Set("GraphActionMenu.FilterCheckBox.Border", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 3.f));
	
}

void FNiagaraEditorStyle::InitEmitterHeader()
{
	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");
	
	// Emitter Header
	FTextBlockStyle StackHeaderText = FTextBlockStyle(NormalText);
	StackHeaderText.SetFont(DEFAULT_FONT("Regular", 11))
		.SetColorAndOpacity(FSlateColor(EStyleColor::White));

	Set("NiagaraEditor.HeadingTextBlock", StackHeaderText);

	FTextBlockStyle StackHeaderTextSubdued = FTextBlockStyle(NormalText);
	StackHeaderTextSubdued.SetFont(DEFAULT_FONT("Regular", 11))
		.SetColorAndOpacity(FStyleColors::Foreground);

	Set("NiagaraEditor.HeadingTextBlockSubdued", StackHeaderTextSubdued);

	FTextBlockStyle TabText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 12))
		.SetShadowOffset(FVector2D(0, 1))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f));
	
	Set("NiagaraEditor.AttributeSpreadsheetTabText", TabText);

	FTextBlockStyle SubduedHeadingText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 14))
		.SetColorAndOpacity(FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)));
	
	Set("NiagaraEditor.SubduedHeadingTextBox", SubduedHeadingText);

	// Details
	FTextBlockStyle DetailsHeadingText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 9));

	Set("NiagaraEditor.DetailsHeadingText", DetailsHeadingText);
}

void FNiagaraEditorStyle::InitParameters()
{
	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FEditableTextBoxStyle NormalEditableTextBox = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");

	// Parameters
	
	FSlateFontInfo NormalFont = FAppStyle::Get().GetFontStyle(TEXT("PropertyWindow.NormalFont"));
	FTextBlockStyle ParameterText = FTextBlockStyle(NormalText)
		.SetFont(NormalFont);

	Set("NiagaraEditor.ParameterText", ParameterText);

	FEditableTextBoxStyle ParameterEditableText = FEditableTextBoxStyle(NormalEditableTextBox)
		.SetFont(NormalFont)
		.SetForegroundColor(FStyleColors::Black);

	FInlineEditableTextBlockStyle ParameterEditableTextBox = FInlineEditableTextBlockStyle()
		.SetEditableTextBoxStyle(ParameterEditableText)
		.SetTextStyle(ParameterText);
	Set("NiagaraEditor.ParameterInlineEditableText", ParameterEditableTextBox);


	Set("NiagaraEditor.ParameterName.NamespaceBorder", new BOX_BRUSH("Icons/NamespaceBorder", FMargin(4.0f / 16.0f)));

	Set("NiagaraEditor.ParameterName.NamespaceText", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 8))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.9f))
		.SetShadowOffset(FVector2D(1, 1))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.7f)));

	Set("NiagaraEditor.ParameterName.NamespaceTextDark", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 8))
		.SetColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.9f))
		.SetShadowOffset(FVector2D(1, 1))
		.SetShadowColorAndOpacity(FLinearColor(1.0, 1.0, 1.0, 0.25f)));

	Set("NiagaraEditor.ParameterName.TypeText", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 8))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.5f)));

	FTextBlockStyle InlineEditableTextBlockReadOnly = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 9))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f, 0.5));

	FEditableTextBoxStyle InlineEditableTextBlockEditable = FEditableTextBoxStyle()
		.SetFont(DEFAULT_FONT("Regular", 9))
		.SetForegroundColor(FStyleColors::Foreground)
		.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
		.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Hover, CoreStyleConstants::InputFocusThickness))
		.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Primary, CoreStyleConstants::InputFocusThickness))
		.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
		.SetScrollBarStyle(FAppStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar"));
	
	FInlineEditableTextBlockStyle InlineEditableTextBlockStyle = FInlineEditableTextBlockStyle()
		.SetTextStyle(InlineEditableTextBlockReadOnly)
		.SetEditableTextBoxStyle(InlineEditableTextBlockEditable);
	Set("NiagaraEditor.Graph.Node.InlineEditablePinName", InlineEditableTextBlockStyle);
	
	Set("NiagaraEditor.StaticInputIcon", new IMAGE_BRUSH_SVG("Icons/StaticInput", Icon12x12));
	Set("NiagaraEditor.StaticIcon", new IMAGE_BRUSH("Icons/staticpill_16x", Icon16x16));
	Set("NiagaraEditor.LinkedSharedIcon", new IMAGE_BRUSH_SVG("Icons/LinkedShared", Icon16x16));
	Set("NiagaraEditor.Pins.StaticConnected", new IMAGE_BRUSH("Icons/StaticPin_Connected", Icon16x16));
	Set("NiagaraEditor.Pins.StaticDisconnected", new IMAGE_BRUSH("Icons/StaticPin_Disconnected", Icon16x16));
	Set("NiagaraEditor.Pins.StaticConnectedHovered", new IMAGE_BRUSH("Icons/StaticPin_Connected", Icon16x16, FLinearColor(0.8f, 0.8f, 0.8f)));
	Set("NiagaraEditor.Pins.StaticDisconnectedHovered", new IMAGE_BRUSH("Icons/StaticPin_Disconnected", Icon16x16, FLinearColor(0.8f, 0.8f, 0.8f)));

	Set("NiagaraEditor.Verse.Private",	new IMAGE_BRUSH_SVG("Icons/Verse_Icon", Icon20x20, FLinearColor(0.2f, 0.2f, 0.2f)));
	Set("NiagaraEditor.Verse.Public",	new IMAGE_BRUSH_SVG("Icons/Verse_Icon", Icon20x20, FLinearColor(1.0f, 1.0f, 1.0f)));
}

void FNiagaraEditorStyle::InitParameterMapView()
{
	// Parameter Map View
	Set("NiagaraEditor.Stack.DepressedHighlightedButtonBrush", new CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FStyleColors::PrimaryPress));
	Set("NiagaraEditor.Stack.FlatButtonColor", FLinearColor(FColor(205, 205, 205)));

	//Parameters panel
	Set("NiagaraEditor.Parameters.ReadColor", FLinearColor(FColor(133, 108, 176)));
	Set("NiagaraEditor.Parameters.WriteColor", FLinearColor(FColor(96, 184, 125)));
	const FTableRowStyle TreeViewStyle = FAppStyle::GetWidgetStyle<FTableRowStyle>("DetailsView.TreeView.TableRow");
	FTableRowStyle ParameterPanelRowStyle = FTableRowStyle(TreeViewStyle)
		.SetTextColor(FLinearColor::White)
		.SetSelectedTextColor(FLinearColor::White);
	Set("NiagaraEditor.Parameters.TableRow", ParameterPanelRowStyle);
	
	const FTextBlockStyle CategoryTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("DetailsView.CategoryTextStyle");
	FTextBlockStyle ParameterSectionStyle = FTextBlockStyle(CategoryTextStyle)
		.SetColorAndOpacity(FLinearColor::White);
	Set("NiagaraEditor.Parameters.HeaderText", ParameterSectionStyle);
}

void FNiagaraEditorStyle::InitCodeView()
{
	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Code View
	Set("NiagaraEditor.CodeView.ToggledHide", FLinearColor(FColor(24, 160, 251)));
	Set("NiagaraEditor.CodeView.Checkbox.Text", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 12))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.9f))
		.SetShadowOffset(FVector2D(1, 1))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

	constexpr int32 LogFontSize = 9;
	FSlateFontInfo LogFont = DEFAULT_FONT("Mono", LogFontSize);
	FTextBlockStyle NormalLogText = FTextBlockStyle(NormalText)
		.SetFont(LogFont)
		.SetColorAndOpacity(FLinearColor(FColor(0xffffffff)))
		.SetSelectedBackgroundColor(FLinearColor(FColor(0xff666666)));
	Set("NiagaraEditor.CodeView.Hlsl.Normal", NormalLogText);


	const FTextBlockStyle ErrorText = FTextBlockStyle(NormalText)
		.SetUnderlineBrush(IMAGE_BRUSH("Icons/White", Icon8x8, FLinearColor::Red, ESlateBrushTileType::Both))
		.SetColorAndOpacity(FLinearColor::Red);
		
	Set("TextEditor.NormalText", NormalText);

	constexpr int32 HlslFontSize = 9;
	FSlateFontInfo HlslFont = DEFAULT_FONT("Mono", HlslFontSize);
	FTextBlockStyle NormalHlslText = FTextBlockStyle(NormalText)
		.SetFont(HlslFont);
	const FTextBlockStyle HlslErrorText = FTextBlockStyle(NormalHlslText)
		.SetUnderlineBrush(IMAGE_BRUSH("Icons/White", Icon8x8, FLinearColor::Red, ESlateBrushTileType::Both))
		.SetColorAndOpacity(FLinearColor::Red);
	
	Set("SyntaxHighlight.HLSL.Normal", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(189, 183, 107))));
	Set("SyntaxHighlight.HLSL.Operator", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(220, 220, 220))));
	Set("SyntaxHighlight.HLSL.Keyword", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(86, 156, 214))));
	Set("SyntaxHighlight.HLSL.String", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(214, 157, 133))));
	Set("SyntaxHighlight.HLSL.Number", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(181, 206, 168))));
	Set("SyntaxHighlight.HLSL.Comment", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(87, 166, 74))));
	Set("SyntaxHighlight.HLSL.PreProcessorKeyword", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(188, 98, 171))));

	Set("SyntaxHighlight.HLSL.Error", HlslErrorText); 
	
	Set("SyntaxHighlight.Python.Normal", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(189, 183, 107))));
	Set("SyntaxHighlight.Python.Operator", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(180, 180, 180))));
	Set("SyntaxHighlight.Python.Keyword", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(86, 156, 214))));
	Set("SyntaxHighlight.Python.String", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(214, 157, 133))));
	Set("SyntaxHighlight.Python.Number", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(181, 206, 168))));
	Set("SyntaxHighlight.Python.Comment", FTextBlockStyle(NormalHlslText).SetColorAndOpacity(FLinearColor(FColor(87, 166, 74))));
}

void FNiagaraEditorStyle::InitSelectedEmitter()
{
	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Selected Emitter
	FSlateFontInfo SelectedEmitterUnsupportedSelectionFont = DEFAULT_FONT("Regular", 10);
	FTextBlockStyle SelectedEmitterUnsupportedSelectionText = FTextBlockStyle(NormalText)
		.SetFont(SelectedEmitterUnsupportedSelectionFont)
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
	Set("NiagaraEditor.SelectedEmitter.UnsupportedSelectionText", SelectedEmitterUnsupportedSelectionText);
}

void FNiagaraEditorStyle::InitToolbarIcons()
{
	Set("NiagaraEditor.ToggleOriginAxis", new IMAGE_BRUSH_SVG("Icons/OriginAxis_16", Icon16x16));
	Set("NiagaraEditor.ApplyScratchPadChanges", new IMAGE_BRUSH_SVG("Icons/Commands/ApplyScratch", Icon40x40));
	Set("NiagaraEditor.OverviewNode.IsolatedColor", FLinearColor::Yellow);
	Set("NiagaraEditor.OverviewNode.NotIsolatedColor", FLinearColor::Transparent);
}

void FNiagaraEditorStyle::InitIcons()
{
	// Icons
	Set("NiagaraEditor.Isolate", new IMAGE_BRUSH_SVG("Icons/Commands/IsolateIcon", Icon16x16));
	Set("NiagaraEditor.Module.Pin.TypeSelector", new IMAGE_BRUSH("Icons/Scratch", Icon16x16, FLinearColor::Gray));
	Set("NiagaraEditor.Module.AddPin", new IMAGE_BRUSH("Icons/PlusSymbol_12x", Icon12x12, FLinearColor::Gray));
	Set("NiagaraEditor.Module.RemovePin", new IMAGE_BRUSH("Icons/MinusSymbol_12x", Icon12x12, FLinearColor::Gray));
	Set("NiagaraEditor.Module.DynamicInput", new IMAGE_BRUSH_SVG("Icons/DynamicInput", Icon16x16));
	Set("NiagaraEditor.Module.TypeIconPill", new IMAGE_BRUSH_SVG("Icons/TypeIconPill", FVector2D(5.0f, 16.0f)));

	FLinearColor EmitterIconColor(FColor(126, 87, 67));
	FLinearColor ParticleIconColor(FColor(87, 107, 61));
	FLinearColor RendererIconColor(FColor(134, 80, 80));
	Set("NiagaraEditor.Emitter.SpawnIcon", new IMAGE_BRUSH("Icons/Spawn", Icon12x12, EmitterIconColor));
	Set("NiagaraEditor.Emitter.UpdateIcon", new IMAGE_BRUSH("Icons/Update", Icon12x12, EmitterIconColor));
	Set("NiagaraEditor.Particle.SpawnIcon", new IMAGE_BRUSH("Icons/Spawn", Icon12x12, ParticleIconColor));
	Set("NiagaraEditor.Particle.UpdateIcon", new IMAGE_BRUSH("Icons/Update", Icon12x12, ParticleIconColor));
	Set("NiagaraEditor.EventIcon", new IMAGE_BRUSH("Icons/Event", Icon12x12, ParticleIconColor));
	Set("NiagaraEditor.SimulationStageIcon", new IMAGE_BRUSH("Icons/SimulationStage", Icon12x12, ParticleIconColor));
	Set("NiagaraEditor.RenderIcon", new IMAGE_BRUSH("Icons/Render", Icon12x12, RendererIconColor));
	Set("NiagaraEditor.Stateless.SpawnIcon", new IMAGE_BRUSH("Icons/Spawn", Icon12x12, FLinearColor::White));
	Set("NiagaraEditor.Stateless.UpdateIcon", new IMAGE_BRUSH("Icons/Update", Icon12x12, FLinearColor::White));
	Set("NiagaraEditor.Stateless.RenderIcon", new IMAGE_BRUSH("Icons/Render", Icon12x12, FLinearColor::White));
	
	FSlateBrush* SlateBrush = new FSlateBrush();
	*SlateBrush = FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").BackgroundImageNormal;
	SlateBrush->TintColor = FLinearColor(0.01f, 0.01f, 0.01f);
	Set("NiagaraEditor.Module.InputTypeBorder", SlateBrush);

	Set("NiagaraEditor.DistributionEditor.Array", new IMAGE_BRUSH("Icons/DistributionEditor/Array", Icon20x20));
	Set("NiagaraEditor.DistributionEditor.Binding", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/Binding", Icon20x20));
	Set("NiagaraEditor.DistributionEditor.UniformConstant", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/UniformConstant", Icon20x20));
	Set("NiagaraEditor.DistributionEditor.NonUniformConstant", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/NonUniformConstant", Icon20x20));
	Set("NiagaraEditor.DistributionEditor.ColorConstant", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/ColorConstant", Icon20x20));

	Set("NiagaraEditor.DistributionEditor.UniformRange", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/UniformRange", Icon20x20));
	Set("NiagaraEditor.DistributionEditor.NonUniformRange", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/NonUniformRange", Icon20x20));
	Set("NiagaraEditor.DistributionEditor.ColorRange", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/ColorRange", Icon20x20));

	Set("NiagaraEditor.DistributionEditor.UniformCurve", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/UniformCurve", Icon20x20));
	Set("NiagaraEditor.DistributionEditor.NonUniformCurve", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/NonUniformCurve", Icon20x20));
	Set("NiagaraEditor.DistributionEditor.ColorGradient", new IMAGE_BRUSH("Icons/DistributionEditor/ColorGradient", Icon20x20));

	Set("NiagaraEditor.DistributionEditor.CurveKeyHandle", new IMAGE_BRUSH_SVG("Icons/DistributionEditor/CurveKeyHandle", Icon20x20));
}

void FNiagaraEditorStyle::InitTextStyles()
{
	const FTextBlockStyle NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	FSlateFontInfo SummaryViewInputOverrideFontInfo = DEFAULT_FONT("Italic", 9);
	FTextBlockStyle SummaryViewInputOverrideTextBlockStyle = FTextBlockStyle(NormalText)
		.SetFont(SummaryViewInputOverrideFontInfo);
	Set("NiagaraEditor.HierarchyEditor.SummaryView.ModuleInputNameOverride", SummaryViewInputOverrideTextBlockStyle);
}

void FNiagaraEditorStyle::InitOverview()
{
	// Overview debug icons
	Set("NiagaraEditor.Overview.DebugActive", new IMAGE_BRUSH("Icons/OverviewDebugActive", Icon16x16));
	Set("NiagaraEditor.Overview.DebugInactive", new IMAGE_BRUSH("Icons/OverviewDebugInactive", Icon16x16));
	Set("NiagaraEditor.Overview.HasMaterialParameterBindings", new IMAGE_BRUSH("Icons/HasMaterialParameterBindings", Icon16x16));
}

void FNiagaraEditorStyle::InitViewportStyle()
{
	FTextBlockStyle NormalTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");
	FTextBlockStyle CompileOverlayText = FTextBlockStyle(NormalTextStyle)
		.SetFontSize(18);
	Set("NiagaraEditor.Viewport.CompileOverlay", CompileOverlayText);
}

void FNiagaraEditorStyle::InitEmitterDetails()
{
	// Emitter details customization
	Set("NiagaraEditor.MaterialWarningBorder", new CORE_BOX_BRUSH("Common/GroupBorderLight", FMargin(4.0f / 16.0f)));
}

void FNiagaraEditorStyle::InitAssetColors()
{
	// Asset colors
	Set("NiagaraEditor.AssetColors.System", FLinearColor(FColor(1, 202, 252)));
	Set("NiagaraEditor.AssetColors.Emitter", FLinearColor(FColor(241, 99, 6)));
	Set("NiagaraEditor.AssetColors.Script", FLinearColor(1.0f, 1.0f, 0.0f));
	Set("NiagaraEditor.AssetColors.ParameterCollection", FLinearColor(1.0f, 1.0f, 0.3f));
	Set("NiagaraEditor.AssetColors.ParameterCollectionInstance", FLinearColor(1.0f, 1.0f, 0.7f));
	Set("NiagaraEditor.AssetColors.ParameterDefinitions", FLinearColor(0.57f, 0.82f, 0.06f));
	Set("NiagaraEditor.AssetColors.EffectType", FLinearColor(1.f, 1.f, 1.f));
	Set("NiagaraEditor.AssetColors.SimCache", FLinearColor(0.9f, 0.3f, 0.25f));
	Set("NiagaraEditor.AssetColors.StatelessEmitterTemplate", FLinearColor(0.9f, 0.3f, 0.25f));
	Set("NiagaraEditor.AssetColors.ValidationRuleSet", FLinearColor(0.25f, 0.25f, 0.25f));
	Set("NiagaraEditor.AssetColors.DataChannelDefinitions", FLinearColor(0.5f, 0.0f, 1.0f));
	Set("NiagaraEditor.AssetColors.StatelessConversionData", FLinearColor(1.0f, 1.0f, 1.0f));
}

void FNiagaraEditorStyle::InitThumbnails()
{
	// Script factory thumbnails
	Set("NiagaraEditor.Thumbnails.DynamicInputs", new IMAGE_BRUSH("Icons/NiagaraScriptDynamicInputs_64x", Icon64x64));
	Set("NiagaraEditor.Thumbnails.Functions", new IMAGE_BRUSH("Icons/NiagaraScriptFunction_64x", Icon64x64));
	Set("NiagaraEditor.Thumbnails.Modules", new IMAGE_BRUSH("Icons/NiagaraScriptModules_64x", Icon64x64));
}

void FNiagaraEditorStyle::InitClassIcon()
{
	Set("ClassThumbnail.NiagaraEmitter", new IMAGE_BRUSH_SVG("Icons/Classes/NiagaraEmitter_64", Icon64x64));
	Set("ClassThumbnail.NiagaraSystem", new IMAGE_BRUSH_SVG("Icons/Classes/NiagaraSystem_64", Icon64x64));

	Set("ClassIcon.NiagaraEmitter", new IMAGE_BRUSH_SVG("Icons/Classes/NiagaraEmitter_16", Icon16x16));
	Set("ClassIcon.NiagaraSystem", new IMAGE_BRUSH_SVG("Icons/Classes/NiagaraSystem_16", Icon16x16));
	Set("ClassIcon.NiagaraActor", new IMAGE_BRUSH_SVG("Icons/Classes/NiagaraSystem_16", Icon16x16));
	// Renderer class icons
	Set("ClassIcon.NiagaraSpriteRendererProperties", new IMAGE_BRUSH("Icons/Classes/renderer_sprite", Icon16x16));
	Set("ClassIcon.NiagaraMeshRendererProperties", new IMAGE_BRUSH("Icons/Classes/renderer_mesh", Icon16x16));
	Set("ClassIcon.NiagaraRibbonRendererProperties", new IMAGE_BRUSH("Icons/Classes/renderer_ribbon", Icon16x16));
	Set("ClassIcon.NiagaraLightRendererProperties", new IMAGE_BRUSH("Icons/Classes/renderer_light", Icon16x16));
	Set("ClassIcon.NiagaraRendererProperties", new IMAGE_BRUSH("Icons/Classes/renderer_default", Icon16x16));
}

void FNiagaraEditorStyle::InitStackIcons()
{
	//GPU/CPU icons
	Set("NiagaraEditor.Stack.GPUIcon", new IMAGE_BRUSH("Icons/Simulate_GPU_x40", Icon16x16));
	Set("NiagaraEditor.Stack.CPUIcon", new IMAGE_BRUSH("Icons/Simulate_CPU_x40", Icon16x16));
	Set("NiagaraEditor.Stack.LWEIcon", new IMAGE_BRUSH("Icons/Simulate_LW_x40", Icon16x16));
}

void FNiagaraEditorStyle::InitStackWidgets()
{
	FTextBlockStyle NormalText = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	FEditableTextBoxStyle DefaultTextBoxStyle = FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
	const FEditableTextBoxStyle EditableTextBoxStyle = DefaultTextBoxStyle
			.SetTextStyle(NormalText.SetFont(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont")))
			.SetPadding(0)
			.SetBackgroundImageNormal( FSlateNoResource() )
			.SetBackgroundImageHovered( FSlateNoResource() )
			.SetBackgroundImageFocused( FSlateNoResource() )
			.SetBackgroundImageReadOnly( FSlateNoResource() );
	
	Set("NiagaraEditor.Stack.NumericDropdownInput", EditableTextBoxStyle);
	Set("NiagaraEditor.Stack.IntegerAsEnum", NormalText.SetFont(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont")));

	// Notes
	FInlineEditableTextBlockStyle DefaultInlineEditableTextBlockStyle = FAppStyle::GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle");
	
	FTextBlockStyle NoteHeaderStyle;
	
	NoteHeaderStyle.SetFont(DEFAULT_FONT("Regular", 11.f));
	NoteHeaderStyle.SetColorAndOpacity(FSlateColor(FStyleColors::White));
	FInlineEditableTextBlockStyle NoteHeaderInlineEditableStyle;
	NoteHeaderInlineEditableStyle.SetTextStyle(NoteHeaderStyle);
	NoteHeaderInlineEditableStyle.SetEditableTextBoxStyle(DefaultInlineEditableTextBlockStyle.EditableTextBoxStyle);

	Set("NiagaraEditor.Stack.Note.HeaderText", NoteHeaderStyle);
	Set("NiagaraEditor.Stack.Note.HeaderEditableText", NoteHeaderInlineEditableStyle);

	FSegmentedControlStyle SegmentedButtonStyle = FCoreStyle::Get().GetWidgetStyle<FSegmentedControlStyle>("SegmentedControl");
	FCheckBoxStyle SegmentedControlStyle = FCoreStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DetailsView.SectionButton");
	SegmentedButtonStyle
	.SetControlStyle(SegmentedControlStyle)
	.SetFirstControlStyle(SegmentedControlStyle)
	.SetLastControlStyle(SegmentedControlStyle)
	.SetBackgroundBrush(FSlateNoResource());
	
	Set("NiagaraEditor.Stack.SegmentedControl", SegmentedButtonStyle);
}

void FNiagaraEditorStyle::InitNiagaraSequence()
{
	// Niagara sequence
	Set("NiagaraEditor.NiagaraSequence.DefaultTrackColor", FLinearColor(0, .25f, 0));
}

void FNiagaraEditorStyle::InitPlatformSet()
{
	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");

	// Niagara platform set customization
	Set("NiagaraEditor.PlatformSet.DropdownButton", new CORE_IMAGE_BRUSH("Common/ComboArrow", Icon8x8));

	Set("NiagaraEditor.PlatformSet.ButtonText", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 10))
		.SetColorAndOpacity(FLinearColor(0.72f, 0.72f, 0.72f))
		.SetHighlightColor(FLinearColor(1, 1, 1)));

	// Separator in the action menus
	Set( "MenuSeparator", new CORE_BOX_BRUSH( "Common/Separator", 1/4.0f, FLinearColor(1,1,1,0.2f) ) );
	
	const FString SmallRoundedButtonStart(TEXT("Common/SmallRoundedButtonLeft"));
	const FString SmallRoundedButtonMiddle(TEXT("Common/SmallRoundedButtonCentre"));
	const FString SmallRoundedButtonEnd(TEXT("Common/SmallRoundedButtonRight"));

	Set("NiagaraEditor.Module.Pin.TypeSelector.Button", FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetPressed(CORE_BOX_BRUSH("Common/Button_Pressed", 8.0f / 32.0f, FStyleColors::PrimaryPress))
		.SetHovered(CORE_BOX_BRUSH("Common/Button_Hovered", 8.0f / 32.0f, FStyleColors::PrimaryHover))
		.SetNormalPadding(FMargin(0, 0, 0, 0))
		.SetPressedPadding(FMargin(0, 0, 0, 0)));
	{
		const FLinearColor NormalColor(0.15f, 0.15f, 0.15f, 1);

		Set("NiagaraEditor.PlatformSet.StartButton", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(CORE_BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), NormalColor))
			.SetUncheckedPressedImage(CORE_BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), FStyleColors::PrimaryPress))
			.SetUncheckedHoveredImage(CORE_BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), FStyleColors::PrimaryHover))
			.SetCheckedHoveredImage(CORE_BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), FStyleColors::SelectHover))
			.SetCheckedPressedImage(CORE_BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), FStyleColors::Select))
			.SetCheckedImage(CORE_BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), FStyleColors::Select)));

		Set("NiagaraEditor.PlatformSet.MiddleButton", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(CORE_BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), NormalColor))
			.SetUncheckedPressedImage(CORE_BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), FStyleColors::PrimaryPress))
			.SetUncheckedHoveredImage(CORE_BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), FStyleColors::PrimaryHover))
			.SetCheckedHoveredImage(CORE_BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), FStyleColors::SelectHover))
			.SetCheckedPressedImage(CORE_BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), FStyleColors::Select))
			.SetCheckedImage(CORE_BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), FStyleColors::Select)));

		Set("NiagaraEditor.PlatformSet.EndButton", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(CORE_BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), NormalColor))
			.SetUncheckedPressedImage(CORE_BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), FStyleColors::SelectHover))
			.SetUncheckedHoveredImage(CORE_BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), FStyleColors::SelectHover))
			.SetCheckedHoveredImage(CORE_BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), FStyleColors::SelectHover))
			.SetCheckedPressedImage(CORE_BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), FStyleColors::Select))
			.SetCheckedImage(CORE_BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), FStyleColors::Select)));
	}

	Set("NiagaraEditor.PlatformSet.Include", new CORE_IMAGE_BRUSH("Icons/PlusSymbol_12x", Icon12x12));
	Set("NiagaraEditor.PlatformSet.Exclude", new CORE_IMAGE_BRUSH("Icons/MinusSymbol_12x", Icon12x12));
	Set("NiagaraEditor.PlatformSet.Remove", new CORE_IMAGE_BRUSH("Icons/Cross_12x", Icon12x12));

	const FSlateColor SelectionColor_Inactive = FAppStyle::GetSlateColor("SelectionColor_Inactive");

	Set("NiagaraEditor.PlatformSet.TreeView", FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateNoResource())
		.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetOddRowBackgroundBrush(FSlateNoResource())
		.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FStyleColors::Select))
		.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, FStyleColors::Select))
		.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive)));

}

void FNiagaraEditorStyle::InitDropTarget()
{
	Set("NiagaraEditor.DropTarget.BackgroundColor", FLinearColor(1.0f, 1.0f, 1.0f, 0.25f));
	Set("NiagaraEditor.DropTarget.BackgroundColorHover", FLinearColor(1.0f, 1.0f, 1.0f, 0.1f));
}

void FNiagaraEditorStyle::InitScriptGraph()
{
	Set("NiagaraEditor.ScriptGraph.SearchBorderColor", FLinearColor(.1f, .1f, .1f, 1.f));
	Set("NiagaraEditor.ScriptGraph.AffectedAssetsWarningColor", FLinearColor(FColor(255, 184, 0)));

	FSearchBoxStyle SearchBoxStyle = FCoreStyle::Get().GetWidgetStyle<FSearchBoxStyle>("SearchBox");
	SearchBoxStyle.SetLeftAlignGlassImageAndClearButton(true);
	SearchBoxStyle.SetLeftAlignSearchResultButtons(false);
	Set("NiagaraEditor.ScriptGraph.SearchBoxStyle", SearchBoxStyle);
}

void FNiagaraEditorStyle::InitDebuggerStyle()
{
	const FVector2D Icon24x24(24.0f, 24.0f);
	
	Set("NiagaraEditor.Debugger.PlayIcon", new IMAGE_BRUSH("Icons/Debugger/Play", Icon24x24));
	Set("NiagaraEditor.Debugger.SpeedIcon", new IMAGE_BRUSH("Icons/Debugger/Speed", Icon24x24));
	Set("NiagaraEditor.Debugger.PauseIcon", new IMAGE_BRUSH("Icons/Debugger/Pause", Icon24x24));
	Set("NiagaraEditor.Debugger.LoopIcon", new IMAGE_BRUSH("Icons/Debugger/Loop", Icon24x24));
	Set("NiagaraEditor.Debugger.StepIcon", new IMAGE_BRUSH("Icons/Debugger/Step", Icon24x24));

	Set("NiagaraEditor.Debugger.Outliner.Capture", new IMAGE_BRUSH("Icons/Debugger/Capture", Icon24x24));
	Set("NiagaraEditor.Debugger.Outliner.Filter", new IMAGE_BRUSH("Icons/Debugger/Filter_24x", Icon24x24));

	const FSlateColor SelectionColor = FAppStyle::GetSlateColor("SelectionColor");
	const FSlateColor SelectionColor_Pressed = FAppStyle::GetSlateColor("SelectionColor_Pressed");

	FButtonStyle OutlinerToolBarButton = FButtonStyle()
		.SetNormal(CORE_BOX_BRUSH("Common/FlatButton", FMargin(4 / 16.0f), FLinearColor(0.0f, 0.0f, 0.0f, 0.0f)))
		.SetHovered(CORE_BOX_BRUSH("Common/FlatButton", FMargin(4 / 16.0f), SelectionColor))
		.SetPressed(CORE_BOX_BRUSH("Common/FlatButton", FMargin(4 / 16.0f), SelectionColor_Pressed))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0));
	Set("NiagaraEditor.Debugger.Outliner.Toolbar", OutlinerToolBarButton);

	FTextBlockStyle NormalText = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	FEditableTextBoxStyle DefaultTextBoxStyle = FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
	const FEditableTextBoxStyle EditableTextBoxStyle = DefaultTextBoxStyle
			.SetTextStyle(NormalText.SetFont(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont")))
			.SetPadding(0)
			.SetBackgroundImageNormal( FSlateNoResource() )
			.SetBackgroundImageHovered( FSlateNoResource() )
			.SetBackgroundImageFocused( FSlateNoResource() )
			.SetBackgroundImageReadOnly( FSlateNoResource() );
	
	Set("NiagaraEditor.Debugger.SuggestionDropdownInput", EditableTextBoxStyle);
}

void FNiagaraEditorStyle::InitBakerStyle()
{
	Set("NiagaraEditor.BakerIcon", new IMAGE_BRUSH("Icons/Baker/BakerIcon", Icon40x40));
	Set("NiagaraEditor.BakerSettings", new IMAGE_BRUSH("Icons/Baker/BakerSettings", Icon40x40));
}

void FNiagaraEditorStyle::InitCommonColors()
{
	Set("NiagaraEditor.CommonColors.System", FLinearColor(FColor(1, 202, 252)));
	Set("NiagaraEditor.CommonColors.Emitter", FLinearColor(FColor(241, 99, 6)));
	Set("NiagaraEditor.CommonColors.Particle", FLinearColor(FColor(131, 218, 9)));
}

void FNiagaraEditorStyle::InitTabIcons()
{
	Set("Tab.Curves", new IMAGE_BRUSH_SVG("Icons/Tabs/Curves", Icon20x20));
	Set("Tab.GeneratedCode", new IMAGE_BRUSH_SVG("Icons/Tabs/GeneratedCode", Icon20x20));
	Set("Tab.Log", new IMAGE_BRUSH_SVG("Icons/Tabs/Log", Icon20x20));
	Set("Tab.Debugger", new IMAGE_BRUSH_SVG("Icons/Tabs/NiagaraDebugger", Icon20x20));
	Set("Tab.Parameters", new IMAGE_BRUSH_SVG("Icons/Tabs/Parameters", Icon20x20));
	Set("Tab.ScratchPad", new IMAGE_BRUSH_SVG("Icons/Tabs/ScratchPad", Icon20x20));
	Set("Tab.ScriptStats", new IMAGE_BRUSH_SVG("Icons/Tabs/ScriptStats", Icon20x20));
	Set("Tab.Settings", new IMAGE_BRUSH_SVG("Icons/Tabs/Settings", Icon20x20));
	Set("Tab.Spreadsheet", new IMAGE_BRUSH_SVG("Icons/Tabs/Spreadsheet", Icon20x20));
	Set("Tab.SystemOverview", new IMAGE_BRUSH_SVG("Icons/Tabs/SystemOverview", Icon20x20));
	Set("Tab.Timeline", new IMAGE_BRUSH_SVG("Icons/Tabs/Timeline", Icon20x20));
	Set("Tab.UserParameters", new IMAGE_BRUSH_SVG("Icons/Tabs/UserParameters", Icon20x20));
	Set("Tab.UserParameterHierarchy", new IMAGE_BRUSH_SVG("Icons/Tabs/UserParameterHierarchy", Icon20x20));
	Set("Tab.Viewport", new IMAGE_BRUSH_SVG("Icons/Tabs/Viewport", Icon20x20));
	Set("Tab.VisualEffects", new IMAGE_BRUSH_SVG("Icons/Tabs/VisualEffects", Icon20x20));
}

void FNiagaraEditorStyle::InitOutlinerStyle()
{
	const FSlateColor SelectionColor = FAppStyle::GetSlateColor("SelectionColor");
	const FSlateColor SelectionColor_Inactive = FAppStyle::GetSlateColor("SelectionColor_Inactive");

	Set("NiagaraEditor.Outliner.WorldItem", FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateNoResource())
		.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetOddRowBackgroundBrush(FSlateNoResource())
		.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive)));


	const FLinearColor SystemColor = GetColor("NiagaraEditor.AssetColors.System") * 0.6f;
	const FLinearColor SystemColorEven = SystemColor * 0.85f;
	const FLinearColor SystemColorOdd = SystemColor * 0.7f;
	Set("NiagaraEditor.Outliner.SystemItem", FTableRowStyle()
		.SetEvenRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemColorEven))
		.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemColor))
		.SetOddRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemColorOdd))
		.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemColor))
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive)));


	const FLinearColor SystemInstanceColor = GetColor("NiagaraEditor.CommonColors.System") * 0.6f;
	const FLinearColor SystemInstanceColorEven = SystemInstanceColor * 0.85f;
	const FLinearColor SystemInstanceColorOdd = SystemInstanceColor * 0.7f;
	Set("NiagaraEditor.Outliner.ComponentItem", FTableRowStyle()
		.SetEvenRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemInstanceColorEven))
		.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemInstanceColor))
		.SetOddRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemInstanceColorOdd))
		.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemInstanceColor))
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive)));

	const FLinearColor EmitterInstanceColor = GetColor("NiagaraEditor.CommonColors.Emitter") * 0.6f;
	const FLinearColor EmitterInstanceColorEven = EmitterInstanceColor * 0.85f;
	const FLinearColor EmitterInstanceColorOdd = EmitterInstanceColor * 0.7f;
	Set("NiagaraEditor.Outliner.EmitterItem", FTableRowStyle()
		.SetEvenRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, EmitterInstanceColorEven))
		.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, EmitterInstanceColor))
		.SetOddRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, EmitterInstanceColorOdd))
		.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, EmitterInstanceColor))
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive)));
}

void FNiagaraEditorStyle::InitScalabilityColors()
{
	Set("NiagaraEditor.Scalability.Included", FLinearColor(0.12f, 0.62f, 0.85f, 1));
	Set("NiagaraEditor.Scalability.Excluded", FLinearColor(0.15f, 0.15f, 0.15f, 1));

	Set("NiagaraEditor.Scalability.System.Feature.Source.EffectType", FLinearColor(1.f, 1.f, 1.f, 1));
	Set("NiagaraEditor.Scalability.System.Feature.Source.Override", FLinearColor(0.8f, 0.8f, 0.15f, 1));
	Set("NiagaraEditor.Scalability.System.Feature.Active", FLinearColor(1.f, 1.f, 1.f, 1));
	Set("NiagaraEditor.Scalability.System.Feature.Inactive", FLinearColor(0.15f, 0.15f, 0.15f, 1));

	Set("NiagaraEditor.SystemOverview.ExcludedFromScalability", FLinearColor(0.7f, 0.1f, 0.1f, 0.5f));
	Set("NiagaraEditor.SystemOverview.ExcludedFromScalability.NodeBody", new FSlateRoundedBoxBrush(GetColor("NiagaraEditor.SystemOverview.ExcludedFromScalability"), FVector4(5, 5, 5, 5)));
	Set("NiagaraEditor.SystemOverview.ExcludedFromScalability.RendererItem", new FSlateRoundedBoxBrush(GetColor("NiagaraEditor.SystemOverview.ExcludedFromScalability"), 5.f));
	Set("NiagaraEditor.SystemOverview.ExcludedFromScalability.EmitterTrack", new FSlateRoundedBoxBrush(GetColor("NiagaraEditor.SystemOverview.ExcludedFromScalability"), 1.f));
}

void FNiagaraEditorStyle::InitScalabilityIcons()
{
	Set("NiagaraEditor.Scalability", new IMAGE_BRUSH_SVG("Icons/Scalability/Scalability", Icon20x20));
	Set("NiagaraEditor.Scalability.System.Feature.Override", new IMAGE_BRUSH_SVG("Icons/Scalability/Override", Icon20x20));
	Set("NiagaraEditor.Scalability.Preview.ResetPlatform", new IMAGE_BRUSH("Icons/Scalability/ResetPreviewPlatform", Icon20x20));
}

void FNiagaraEditorStyle::InitSimCacheEditor()
{
	const FSlateColor SelectionColor = FAppStyle::GetSlateColor("SelectionColor");
	const FSlateColor SelectionColor_Inactive = FAppStyle::GetSlateColor("SelectionColor_Inactive");
	const FSlateColor DefaultForeground = FAppStyle::GetSlateColor("DefaultForeground");
	const FLinearColor SystemBackground = FLinearColor(0.05f, 0.05f, 0.05f);

	Set("NiagaraEditor.SimCache.SystemItem", FTableRowStyle()
		.SetEvenRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemBackground))
		.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetOddRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemBackground))
		.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetSelectorFocusedBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemBackground))
		.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive)));

	FTableRowStyle EmitterStyle = FTableRowStyle()
		.SetEvenRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemBackground))
		.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetOddRowBackgroundBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemBackground))
		.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetSelectorFocusedBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SystemBackground))
		.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive));
	Set("NiagaraEditor.SimCache.EmitterItem", EmitterStyle);

	Set("NiagaraEditor.SimCache.ComponentItem", FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateNoResource())
		.SetEvenRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetOddRowBackgroundBrush(FSlateNoResource())
		.SetOddRowBackgroundHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetActiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetInactiveBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		.SetInactiveHoveredBrush(CORE_IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive)));

	FTableRowStyle DataInterfaceStyle = EmitterStyle;
	Set("NiagaraEditor.SimCache.DataInterfaceItem", DataInterfaceStyle);

	FTableRowStyle DebugDataStyle = EmitterStyle;
	Set("NiagaraEditor.SimCache.DebugData", DebugDataStyle);

	const FLinearColor NormalColor(0.15f, 0.15f, 0.15f, 1);

	Set ("NiagaraEditor.SimCache.FilterToggleStyle", FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage( FSlateRoundedBoxBrush( NormalColor, 2.0f) )
		.SetUncheckedHoveredImage(FSlateRoundedBoxBrush( FStyleColors::PrimaryHover, 2.0f))
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush( FStyleColors::PrimaryPress, 2.0f))
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Select, 2.0f))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Select, 2.0f))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Select, 2.0f))
		);

	Set("NiagaraEditor.SimCache.CaptureSingleIcon", new IMAGE_BRUSH_SVG("Icons/AttributeSpreadsheet/NiagaraAttrSpreadsheetCaptureSingle", Icon20x20));
	Set("NiagaraEditor.SimCache.CaptureMultiIcon", new IMAGE_BRUSH_SVG("Icons/AttributeSpreadsheet/NiagaraAttrSpreadsheetCaptureMulti", Icon20x20));
}

FNiagaraEditorStyle::FNiagaraEditorStyle() : FSlateStyleSet("NiagaraEditorStyle")
{
	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("FX/Niagara/Content/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	InitStats();
	InitAssetPicker();
	InitActionMenu();
	InitEmitterHeader();
	InitParameters();
	InitParameterMapView();
	InitCodeView();
	InitSelectedEmitter();
	InitToolbarIcons();
	InitTabIcons();
	InitIcons();
	InitTextStyles();
	InitOverview();
	InitEmitterDetails();
	InitAssetColors();
	InitThumbnails();
	InitClassIcon();
	InitStackIcons();
	InitStackWidgets();
	InitNiagaraSequence();
	InitPlatformSet();
	InitDropTarget();
	InitScriptGraph();
	InitDebuggerStyle();
	InitBakerStyle();
	InitCommonColors();
	InitOutlinerStyle();
	InitScalabilityColors();
	InitScalabilityIcons();
	InitViewportStyle();
	InitScratchStyle();
	InitSimCacheEditor();
}

void FNiagaraEditorStyle::InitScratchStyle()
{
	const FTextBlockStyle NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	FSlateFontInfo ScratchPadEditorHeaderFont = DEFAULT_FONT("Bold", 11);
	FTextBlockStyle ScratchPadEditorHeaderText = FTextBlockStyle(NormalText)
		.SetFont(ScratchPadEditorHeaderFont);
	Set("NiagaraEditor.ScratchPad.EditorHeaderText", ScratchPadEditorHeaderText);

	FSlateFontInfo ScratchPadSubSectionHeaderFont = DEFAULT_FONT("Bold", 9);
	FTextBlockStyle ScratchPadSubSectionHeaderText = FTextBlockStyle(NormalText)
		.SetFont(ScratchPadSubSectionHeaderFont);
	Set("NiagaraEditor.ScratchPad.SubSectionHeaderText", ScratchPadSubSectionHeaderText);
}

void FNiagaraEditorStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const FNiagaraEditorStyle& FNiagaraEditorStyle::Get()
{
	if(!NiagaraEditorStyle.IsValid())
	{
		NiagaraEditorStyle = MakeShareable(new FNiagaraEditorStyle());
	}
	
	return *NiagaraEditorStyle;
}

void FNiagaraEditorStyle::ReinitializeStyle()
{
	Unregister();
	NiagaraEditorStyle.Reset();
	Register();	
}
