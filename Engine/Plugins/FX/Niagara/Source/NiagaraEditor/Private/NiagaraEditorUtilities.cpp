// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorUtilities.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "ContentBrowserModule.h"
#include "EdGraphSchema_Niagara.h"
#include "GeneralProjectSettings.h"
#include "NiagaraCompileHashVisitor.h"
#include "Styling/AppStyle.h"
#include "IContentBrowserSingleton.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraComponent.h"
#include "NiagaraConstants.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraDataInterface.h"
#include "IPythonScriptPlugin.h"
#include "NiagaraAnalytics.h"
#include "NiagaraClipboard.h"
#include "NiagaraConvertInPlaceUtilityBase.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraGraph.h"
#include "NiagaraEditorModule.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraOverviewNode.h"
#include "NiagaraParameterMapHistory.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraRecentAndFavoritesManager.h"
#include "NiagaraScript.h"
#include "NiagaraScriptRuntimeData.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "NiagaraScriptMergeManager.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSettings.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraStackEditorData.h"
#include "UObject/UObjectIterator.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraParameterDefinitionsSubscriberViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraScratchPadUtilities.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "ViewModels/Stack/NiagaraStackNote.h"
#include "Widgets/SNiagaraParameterName.h"

#include "Logging/StructuredLog.h"
#include "EdGraph/EdGraphPin.h"
#include "Editor/EditorEngine.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Logging/StructuredLog.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraVersionUpgradeBase.h"

#include "Toolkits/NiagaraSystemToolkit.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "UpgradeNiagaraScriptResults.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeExit.h"
#include "Styling/CoreStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/PropertyIterator.h"
#include "UObject/StructOnScope.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "UObject/TextProperty.h"
#include "Widgets/SNiagaraPinTypeSelector.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/NiagaraSystemScriptViewModel.h"
#include "Widgets/SDataHierarchyEditor.h"
#include "ViewModels/HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "FNiagaraEditorUtilities"

static int GNiagaraDeletePythonFilesOnError = 1;
static FAutoConsoleVariableRef CVarDeletePythonFilesOnError(
	TEXT("fx.Niagara.DeletePythonFilesOnError"),
	GNiagaraDeletePythonFilesOnError,
	TEXT("This determines whether we keep the intermediate python used by module/emitter versioning around when they were executed and resulted in an error."),
	ECVF_Default
);

const FText FNiagaraEditorSharedTexts::DebugDrawUIActionBaseText = LOCTEXT("EnableDebugDrawCheckBoxToolTip", "Enable or disable debug drawing for this item.");

TSet<FName> FNiagaraEditorUtilities::GetSystemConstantNames()
{
	TSet<FName> SystemConstantNames;
	for (const FNiagaraVariable& SystemConstant : FNiagaraConstants::GetEngineConstants())
	{
		SystemConstantNames.Add(SystemConstant.GetName());
	}
	return SystemConstantNames;
}

void FNiagaraEditorUtilities::GetTypeDefaultValue(const FNiagaraTypeDefinition& Type, TArray<uint8>& DefaultData)
{
	if (const UScriptStruct* ScriptStruct = Type.GetScriptStruct())
	{
		FNiagaraVariable DefaultVariable(Type, NAME_None);
		ResetVariableToDefaultValue(DefaultVariable);

		DefaultData.SetNumUninitialized(Type.GetSize());
		DefaultVariable.CopyTo(DefaultData.GetData());
	}
}

void FNiagaraEditorUtilities::ResetVariableToDefaultValue(FNiagaraVariable& Variable)
{
	if (const UScriptStruct* ScriptStruct = Variable.GetType().GetScriptStruct())
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(Variable.GetType());
		if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanProvideDefaultValue())
		{
			TypeEditorUtilities->UpdateVariableWithDefaultValue(Variable);
		}
		else
		{
			Variable.AllocateData();
			ScriptStruct->InitializeDefaultValue(Variable.GetData());
		}
	}
}

void FNiagaraEditorUtilities::InitializeParameterInputNode(UNiagaraNodeInput& InputNode, const FNiagaraTypeDefinition& Type, const UNiagaraGraph* InGraph, FName InputName)
{
	InputNode.Usage = ENiagaraInputNodeUsage::Parameter;
	InputNode.bCanRenameNode = true;
	InputName = UNiagaraNodeInput::GenerateUniqueName(InGraph, InputName, ENiagaraInputNodeUsage::Parameter);
	InputNode.Input.SetName(InputName);
	InputNode.Input.SetType(Type);
	if (Type != FNiagaraTypeDefinition::GetParameterMapDef())
	{
		// when adding things like data interfaces as module inputs, making them required will just generate weird compile errors
		// because "bRequired" prevents using the default value, but their pins are not exposed to the stack (as they are not part of the parameter map)
		InputNode.ExposureOptions.bRequired = false;
	}
	if (InGraph) // Only compute sort priority if a graph was passed in, similar to the way that GenrateUniqueName works above.
	{
		InputNode.CallSortPriority = UNiagaraNodeInput::GenerateNewSortPriority(InGraph, InputName, ENiagaraInputNodeUsage::Parameter);
	}
	if (Type.GetScriptStruct() != nullptr)
	{
		ResetVariableToDefaultValue(InputNode.Input);
		if (InputNode.GetDataInterface() != nullptr)
		{
			InputNode.SetDataInterface(nullptr);
		}
		if (InputNode.GetObjectAsset() != nullptr)
		{
			InputNode.SetObjectAsset(nullptr);
		}
	}
	else if(Type.IsDataInterface())
	{
		InputNode.Input.AllocateData(); // Frees previously used memory if we're switching from a struct to a class type.
		InputNode.SetDataInterface(NewObject<UNiagaraDataInterface>(&InputNode, Type.GetClass(), NAME_None, UNiagaraDataInterface::BuildObjectFlagsForOwner(&InputNode, RF_Transactional)));
	}
	else if (Type.IsUObject())
	{
		InputNode.Input.AllocateData(); // Frees previously used memory if we're switching from a struct to a class type.
		InputNode.SetObjectAsset(nullptr);
	}
}

void FNiagaraEditorUtilities::GetParameterVariablesFromSystem(UNiagaraSystem& System, TArray<FNiagaraVariable>& ParameterVariables,
	FNiagaraEditorUtilities::FGetParameterVariablesFromSystemOptions Options)
{
	UNiagaraScript* SystemScript = System.GetSystemSpawnScript();
	if (SystemScript != nullptr)
	{
		UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(SystemScript->GetLatestSource());
		if (ScriptSource != nullptr)
		{
			UNiagaraGraph* SystemGraph = ScriptSource->NodeGraph;
			if (SystemGraph != nullptr)
			{
				UNiagaraGraph::FFindInputNodeOptions FindOptions;
				FindOptions.bIncludeAttributes = false;
				FindOptions.bIncludeSystemConstants = false;
				FindOptions.bIncludeTranslatorConstants = false;
				FindOptions.bFilterDuplicates = true;

				TArray<UNiagaraNodeInput*> InputNodes;
				SystemGraph->FindInputNodes(InputNodes, FindOptions);
				for (UNiagaraNodeInput* InputNode : InputNodes)
				{
					bool bIsStructParameter = InputNode->Input.GetType().GetScriptStruct() != nullptr;
					bool bIsDataInterfaceParameter = InputNode->Input.GetType().GetClass() != nullptr;
					if ((bIsStructParameter && Options.bIncludeStructParameters) || (bIsDataInterfaceParameter && Options.bIncludeDataInterfaceParameters))
					{
						ParameterVariables.Add(InputNode->Input);
					}
				}
			}
		}
	}
}

// TODO: This is overly complicated.
void FNiagaraEditorUtilities::FixUpPastedNodes(UEdGraph* Graph, TSet<UEdGraphNode*> PastedNodes)
{
	// Collect existing inputs.
	TArray<UNiagaraNodeInput*> CurrentInputs;
	Graph->GetNodesOfClass<UNiagaraNodeInput>(CurrentInputs);
	TSet<FNiagaraVariable> ExistingInputs;
	TMap<FNiagaraVariable, UNiagaraNodeInput*> ExistingNodes;
	int32 HighestSortOrder = -1; // Set to -1 initially, so that in the event of no nodes, we still get zero.
	for (UNiagaraNodeInput* CurrentInput : CurrentInputs)
	{
		if (PastedNodes.Contains(CurrentInput) == false && CurrentInput->Usage == ENiagaraInputNodeUsage::Parameter)
		{
			ExistingInputs.Add(CurrentInput->Input);
			ExistingNodes.Add(CurrentInput->Input) = CurrentInput;
			if (CurrentInput->CallSortPriority > HighestSortOrder)
			{
				HighestSortOrder = CurrentInput->CallSortPriority;
			}
		}
	}

	// Collate pasted inputs nodes by their input for further processing.
	TMap<FNiagaraVariable, TArray<UNiagaraNodeInput*>> InputToPastedInputNodes;
	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		UNiagaraNodeInput* PastedInputNode = Cast<UNiagaraNodeInput>(PastedNode);
		if (PastedInputNode != nullptr && PastedInputNode->Usage == ENiagaraInputNodeUsage::Parameter && ExistingInputs.Contains(PastedInputNode->Input) == false)
		{
			TArray<UNiagaraNodeInput*>* NodesForInput = InputToPastedInputNodes.Find(PastedInputNode->Input);
			if (NodesForInput == nullptr)
			{
				NodesForInput = &InputToPastedInputNodes.Add(PastedInputNode->Input);
			}
			NodesForInput->Add(PastedInputNode);
		}
	}

	// Fix up the nodes based on their relationship to the existing inputs.
	for (auto PastedPairIterator = InputToPastedInputNodes.CreateIterator(); PastedPairIterator; ++PastedPairIterator)
	{
		FNiagaraVariable PastedInput = PastedPairIterator.Key();
		TArray<UNiagaraNodeInput*>& PastedNodesForInput = PastedPairIterator.Value();

		// Try to find an existing input which matches the pasted input by both name and type so that the pasted nodes
		// can be assigned the same id and value, to facilitate pasting multiple times from the same source graph.
		FNiagaraVariable* MatchingInputByNameAndType = nullptr;
		UNiagaraNodeInput* MatchingNode = nullptr;
		for (FNiagaraVariable& ExistingInput : ExistingInputs)
		{
			if (PastedInput.GetName() == ExistingInput.GetName() && PastedInput.GetType() == ExistingInput.GetType())
			{
				MatchingInputByNameAndType = &ExistingInput;
				UNiagaraNodeInput** FoundNode = ExistingNodes.Find(ExistingInput);
				if (FoundNode != nullptr)
				{
					MatchingNode = *FoundNode;
				}
				break;
			}
		}

		if (MatchingInputByNameAndType != nullptr && MatchingNode != nullptr)
		{
			// Update the id and value on the matching pasted nodes.
			for (UNiagaraNodeInput* PastedNodeForInput : PastedNodesForInput)
			{
				if (nullptr == PastedNodeForInput)
				{
					continue;
				}
				PastedNodeForInput->CallSortPriority = MatchingNode->CallSortPriority;
				PastedNodeForInput->ExposureOptions = MatchingNode->ExposureOptions;
				PastedNodeForInput->Input.AllocateData();
				PastedNodeForInput->Input.SetData(MatchingInputByNameAndType->GetData());
			}
		}
		else
		{
			// Check for duplicate names
			TSet<FName> ExistingNames;
			for (FNiagaraVariable& ExistingInput : ExistingInputs)
			{
				ExistingNames.Add(ExistingInput.GetName());
			}
			if (ExistingNames.Contains(PastedInput.GetName()))
			{
				FName UniqueName = FNiagaraUtilities::GetUniqueName(PastedInput.GetName(), ExistingNames.Union(FNiagaraEditorUtilities::GetSystemConstantNames()));
				for (UNiagaraNodeInput* PastedNodeForInput : PastedNodesForInput)
				{
					PastedNodeForInput->Input.SetName(UniqueName);
				}
			}

			// Assign the pasted inputs the same new id and add them to the end of the parameters list.
			int32 NewSortOrder = ++HighestSortOrder;
			for (UNiagaraNodeInput* PastedNodeForInput : PastedNodesForInput)
			{
				PastedNodeForInput->CallSortPriority = NewSortOrder;
			}
		}
	}

	// Fix up pasted function call nodes
	TArray<UNiagaraNodeFunctionCall*> FunctionCallNodes;
	Graph->GetNodesOfClass<UNiagaraNodeFunctionCall>(FunctionCallNodes);
	TSet<FName> ExistingNames;
	for (UNiagaraNodeFunctionCall* FunctionCallNode : FunctionCallNodes)
	{
		if (PastedNodes.Contains(FunctionCallNode) == false)
		{
			ExistingNames.Add(*FunctionCallNode->GetFunctionName());
		}
	}

	TMap<FName, FName> OldFunctionToNewFunctionNameMap;
	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		UNiagaraNodeFunctionCall* PastedFunctionCallNode = Cast<UNiagaraNodeFunctionCall>(PastedNode);
		if (PastedFunctionCallNode != nullptr)
		{
			if (ExistingNames.Contains(*PastedFunctionCallNode->GetFunctionName()))
			{
				FName FunctionCallName = *PastedFunctionCallNode->GetFunctionName();
				FName UniqueFunctionCallName = FNiagaraUtilities::GetUniqueName(FunctionCallName, ExistingNames);
				PastedFunctionCallNode->SuggestName(UniqueFunctionCallName.ToString());
				FName ActualPastedFunctionCallName = *PastedFunctionCallNode->GetFunctionName();
				ExistingNames.Add(ActualPastedFunctionCallName);
				OldFunctionToNewFunctionNameMap.Add(FunctionCallName, ActualPastedFunctionCallName);
			}
			UNiagaraGraph* NiagaraGraph = CastChecked<UNiagaraGraph>(Graph);
			for (const FNiagaraPropagatedVariable& PropagatedVariable : PastedFunctionCallNode->PropagatedStaticSwitchParameters)
			{
				NiagaraGraph->AddParameter(PropagatedVariable.ToVariable(), true);
			}
		}
	}

	FPinCollectorArray InputPins;
	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		UNiagaraNodeParameterMapSet* ParameterMapSetNode = Cast<UNiagaraNodeParameterMapSet>(PastedNode);
		if (ParameterMapSetNode != nullptr)
		{
			InputPins.Reset();
			ParameterMapSetNode->GetInputPins(InputPins);
			for (UEdGraphPin* InputPin : InputPins)
			{
				FNiagaraParameterHandle InputHandle(InputPin->PinName);
				if (OldFunctionToNewFunctionNameMap.Contains(InputHandle.GetNamespace()))
				{
					// Rename any inputs pins on parameter map sets who's function calls were renamed.
					InputPin->PinName = FNiagaraParameterHandle(OldFunctionToNewFunctionNameMap[InputHandle.GetNamespace()], InputHandle.GetName()).GetParameterHandleString();
				}
			}
		}
	}
}

void FNiagaraEditorUtilities::WriteTextFileToDisk(FString SaveDirectory, FString FileName, FString TextToSave, bool bAllowOverwriting)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	// CreateDirectoryTree returns true if the destination
	// directory existed prior to call or has been created
	// during the call.
	if (PlatformFile.CreateDirectoryTree(*SaveDirectory))
	{
		// Get absolute file path
		FString AbsoluteFilePath = SaveDirectory + "/" + FileName;

		// Allow overwriting or file doesn't already exist
		if (bAllowOverwriting || !PlatformFile.FileExists(*AbsoluteFilePath))
		{
			if (FFileHelper::SaveStringToFile(TextToSave, *AbsoluteFilePath))
			{
				UE_LOGF(LogNiagaraEditor, Log, "Wrote file to %ls", *AbsoluteFilePath);
				return;
			}

		}
	}
}


bool FNiagaraEditorUtilities::PODPropertyAppendCompileHash(const void* Container, FProperty* Property, FStringView PropertyName, FNiagaraCompileHashVisitor* InVisitor) 
{
	if (Property->IsA(FFloatProperty::StaticClass()))
	{
		FFloatProperty* CastProp = CastFieldChecked<FFloatProperty>(Property);
		float Value = CastProp->GetPropertyValue_InContainer(Container, 0);
		InVisitor->UpdatePOD(PropertyName.GetData(), Value);
		return true;
	}
	else if (Property->IsA(FIntProperty::StaticClass()))
	{
		FIntProperty* CastProp = CastFieldChecked<FIntProperty>(Property);
		int32 Value = CastProp->GetPropertyValue_InContainer(Container, 0);
		InVisitor->UpdatePOD(PropertyName.GetData(), Value);
		return true;
	}
	else if (Property->IsA(FInt16Property::StaticClass()))
	{
		FInt16Property* CastProp = CastFieldChecked<FInt16Property>(Property);
		int16 Value = CastProp->GetPropertyValue_InContainer(Container, 0);
		InVisitor->UpdatePOD(PropertyName.GetData(), Value);
		return true;
	}
	else if (Property->IsA(FUInt32Property::StaticClass()))
	{
		FUInt32Property* CastProp = CastFieldChecked<FUInt32Property>(Property);
		uint32 Value = CastProp->GetPropertyValue_InContainer(Container, 0);
		InVisitor->UpdatePOD(PropertyName.GetData(), Value);
		return true;
	}
	else if (Property->IsA(FUInt16Property::StaticClass()))
	{
		FUInt16Property* CastProp = CastFieldChecked<FUInt16Property>(Property);
		uint16 Value = CastProp->GetPropertyValue_InContainer(Container, 0);
		InVisitor->UpdatePOD(PropertyName.GetData(), Value);
		return true;
	}
	else if (Property->IsA(FByteProperty::StaticClass()))
	{
		FByteProperty* CastProp = CastFieldChecked<FByteProperty>(Property);
		uint8 Value = CastProp->GetPropertyValue_InContainer(Container, 0);
		InVisitor->UpdatePOD(PropertyName.GetData(), Value);
		return true;
	}
	else if (Property->IsA(FBoolProperty::StaticClass()))
	{
		FBoolProperty* CastProp = CastFieldChecked<FBoolProperty>(Property);
		bool Value = CastProp->GetPropertyValue_InContainer(Container, 0);
		InVisitor->UpdatePOD(PropertyName.GetData(), Value);
		return true;
	}
	else if (Property->IsA(FNameProperty::StaticClass()))
	{
		FNameProperty* CastProp = CastFieldChecked<FNameProperty>(Property);
		FName Value = CastProp->GetPropertyValue_InContainer(Container);
		InVisitor->UpdateName(PropertyName.GetData(), Value);
		return true;
	}
	else if (Property->IsA(FStrProperty::StaticClass()))
	{
		FStrProperty* CastProp = CastFieldChecked<FStrProperty>(Property);
		FString Value = CastProp->GetPropertyValue_InContainer(Container);
		InVisitor->UpdateString(PropertyName.GetData(), Value);
		return true;
	}
	return false;
}

bool FNiagaraEditorUtilities::NestedPropertiesAppendCompileHash(const void* Container, const UStruct* Struct, EFieldIteratorFlags::SuperClassFlags IteratorFlags, FStringView BaseName, FNiagaraCompileHashVisitor* InVisitor) 
{
	// We special case FNiagaraTypeDefinitions here because they need to write out a lot more than just their standalone uproperties.
	if (Struct == FNiagaraTypeDefinition::StaticStruct())
	{
		if (FNiagaraTypeDefinition* TypeDef = (FNiagaraTypeDefinition*)Container)
		{
			TypeDef->AppendCompileHash(InVisitor);
			return true;
		}
	}
	else if (Struct == FNiagaraTypeDefinitionHandle::StaticStruct())
	{
		if (FNiagaraTypeDefinitionHandle* TypeDef = (FNiagaraTypeDefinitionHandle*)Container)
		{
			TypeDef->AppendCompileHash(InVisitor);
			return true;
		}
	}

	TFieldIterator<FProperty> PropertyCountIt(Struct, IteratorFlags, EFieldIteratorFlags::ExcludeDeprecated);
	int32 NumProperties = 0;
	for (; PropertyCountIt; ++PropertyCountIt)
	{
		NumProperties++;
	}

	TStringBuilder<128> PathName;
	TStringBuilder<128> PropertyName;

	for (TFieldIterator<FProperty> PropertyIt(Struct, IteratorFlags, EFieldIteratorFlags::ExcludeDeprecated); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;

		static FName SkipMeta = TEXT("SkipForCompileHash");
		if (Property->HasMetaData(SkipMeta))
		{
			continue;
		}

		PropertyName = BaseName;
		if (NumProperties > 1)
		{
			PropertyName << TCHAR('.');
			Property->GetFName().AppendString(PropertyName);
		}

		if (PODPropertyAppendCompileHash(Container, Property, *PropertyName, InVisitor))
		{
			continue;
		}

		if (Property->IsA(FStructProperty::StaticClass()))
		{
			FStructProperty* StructProp = CastFieldChecked<FStructProperty>(Property);
			const void* StructContainer = Property->ContainerPtrToValuePtr<uint8>(Container);
			NestedPropertiesAppendCompileHash(StructContainer, StructProp->Struct, EFieldIteratorFlags::IncludeSuper, *PropertyName, InVisitor);
		}
		else if (Property->IsA(FEnumProperty::StaticClass()))
		{
			FEnumProperty* CastProp = CastFieldChecked<FEnumProperty>(Property);
			const void* EnumContainer = Property->ContainerPtrToValuePtr<uint8>(Container);
			if (!PODPropertyAppendCompileHash(EnumContainer, CastProp->GetUnderlyingProperty(), *PropertyName, InVisitor))
			{
				check(false);
				return false;
			}
		}
		else if (Property->IsA(FObjectProperty::StaticClass()))
		{
			FObjectProperty* CastProp = CastFieldChecked<FObjectProperty>(Property);
			UObject* Obj = CastProp->GetObjectPropertyValue_InContainer(Container);

			static FName HashObjectPathMeta = TEXT("CompileHashObjectPath");
			if (Property->HasMetaData(HashObjectPathMeta))
			{
				FSoftObjectPath SoftPath(Obj);
				InVisitor->UpdateString(*PropertyName, SoftPath.GetAssetPath().ToString());
			}
			else
			{
				if (Obj != nullptr)
				{
					// We just do name here as sometimes things will be in a transient package or something tricky.
					// Because we do nested id's for each called graph, it should work out in the end to have a different
					// value in the compile array if the scripts are the same name but different locations.
					InVisitor->UpdateString(*PropertyName, Obj->GetName());
				}
				else
				{
					InVisitor->UpdateString(*PropertyName, TEXT("nullptr"));
				}
			}
		}
		else if (Property->IsA(FMapProperty::StaticClass()))
		{
			FMapProperty* CastProp = CastFieldChecked<FMapProperty>(Property);
			FScriptMapHelper MapHelper(CastProp, CastProp->ContainerPtrToValuePtr<void>(Container));
			InVisitor->UpdatePOD(*PropertyName, MapHelper.Num());
			if (MapHelper.GetKeyProperty())
			{
				PathName.Reset();
				MapHelper.GetKeyProperty()->GetPathName(nullptr, PathName);
				InVisitor->UpdateString(TEXT("KeyPathname"), PathName);
				PathName.Reset();
				MapHelper.GetValueProperty()->GetPathName(nullptr, PathName);
				InVisitor->UpdateString(TEXT("ValuePathname"), PathName);

				// We currently only support maps with keys of FNames. Anything else should generate a warning.
				if (MapHelper.GetKeyProperty()->GetClass() == FNameProperty::StaticClass())
				{
					// To be safe, let's gather up all the keys and sort them lexicographically so that this is stable across application runs.
					TArray<FName> Names;
					Names.Reserve(MapHelper.Num());
					for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
					{
						const FName* KeyPtr = (FName*)MapHelper.GetKeyPtr(It);
						// Iterator guarantees a valid FName key
						Names.Add(*KeyPtr);
					}
					// Sort stably over runs
					Names.Sort(FNameLexicalLess());

					// Now hash out the values directly.
					// We support map values of POD types or map values of structs with POD types internally. Anything else we should generate a warning on.
					if (MapHelper.GetValueProperty()->IsA(FStructProperty::StaticClass()))
					{
						FStructProperty* StructProp = CastFieldChecked<FStructProperty>(MapHelper.GetValueProperty());

						for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
						{
							InVisitor->UpdateString(*FString::Printf(TEXT("Key[%d]"), It.GetLogicalIndex()), Names[It.GetLogicalIndex()].ToString());
							if (!NestedPropertiesAppendCompileHash(MapHelper.GetValuePtr(It), StructProp->Struct, EFieldIteratorFlags::IncludeSuper, FString::Printf(TEXT("Value[%d]"), It.GetLogicalIndex()), InVisitor))
							{
								UE_LOGF(LogNiagaraEditor, Warning, "Skipping %ls because it is a map value property of unsupported underlying type, please add \"meta = (SkipForCompileHash=\"true\")\" to avoid this warning in the future or handle it yourself in NestedPropertiesAppendCompileHash!", *Property->GetName());
							}
						}
					}
					else
					{
						for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
						{
							InVisitor->UpdateString(*FString::Printf(TEXT("Key[%d]"), It.GetLogicalIndex()), Names[It.GetLogicalIndex()].ToString());
							if (!PODPropertyAppendCompileHash(MapHelper.GetPairPtr(It), MapHelper.GetValueProperty(), FString::Printf(TEXT("Value[%d]"), It.GetLogicalIndex()), InVisitor))
							{
								UE_LOGF(LogNiagaraEditor, Warning, "Skipping %ls because it is a map value property of unsupported underlying type, please add \"meta = (SkipForCompileHash=\"true\")\" to avoid this warning in the future or handle it yourself in PODPropertyAppendCompileHash!", *Property->GetName());
							}
						}
					}
				}
				else
				{
					UE_LOGF(LogNiagaraEditor, Warning, "Skipping %ls because it is a map property of unsupported key type, please add \"meta = (SkipForCompileHash=\"true\")\" to avoid this warning in the future or handle it yourself in NestedPropertiesAppendCompileHash!", *Property->GetName());
				}
			}
		}
		else if (Property->IsA(FArrayProperty::StaticClass()))
		{
			FArrayProperty* CastProp = CastFieldChecked<FArrayProperty>(Property);

			FScriptArrayHelper ArrayHelper(CastProp, CastProp->ContainerPtrToValuePtr<void>(Container));
			InVisitor->UpdatePOD(*PropertyName, ArrayHelper.Num());
			PathName.Reset();
			CastProp->Inner->GetPathName(nullptr, PathName);
			InVisitor->UpdateString(TEXT("InnerPathname"), PathName);

			// We support arrays of POD types or arrays of structs with POD types internally. Anything else we should generate a warning on.
			if (CastProp->Inner->IsA(FStructProperty::StaticClass()))
			{
				FStructProperty* StructProp = CastFieldChecked<FStructProperty>(CastProp->Inner);

				for (int32 ArrayIdx = 0; ArrayIdx < ArrayHelper.Num(); ArrayIdx++)
				{
					if (!NestedPropertiesAppendCompileHash(ArrayHelper.GetRawPtr(ArrayIdx), StructProp->Struct, EFieldIteratorFlags::IncludeSuper, *PropertyName, InVisitor))
					{
						UE_LOGF(LogNiagaraEditor, Warning, "Skipping %ls because it is an array property of unsupported underlying type, please add \"meta = (SkipForCompileHash=\"true\")\" to avoid this warning in the future or handle it yourself in NestedPropertiesAppendCompileHash!", *Property->GetName());
					}
				}
			}
			else
			{
				for (int32 ArrayIdx = 0; ArrayIdx < ArrayHelper.Num(); ArrayIdx++)
				{
					if (!PODPropertyAppendCompileHash(ArrayHelper.GetRawPtr(ArrayIdx), CastProp->Inner, *PropertyName, InVisitor))
					{
						UE_LOGF(LogNiagaraEditor, Warning, "Skipping %ls because it is an array property of unsupported underlying type, please add \"meta = (SkipForCompileHash=\"true\")\" to avoid this warning in the future or handle it yourself in NestedPropertiesAppendCompileHash!", *Property->GetName());
					}
				}
			}
		}
		else if (Property->IsA(FTextProperty::StaticClass()))
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Skipping %ls because it is a UText property, please add \"meta = (SkipForCompileHash=\"true\")\" to avoid this warning in the future or handle it yourself in NestedPropertiesAppendCompileHash!", *Property->GetName());
			return true;
		}
		else
		{
			check(false);
			return false;
		}
	}
	return true;
}

void FNiagaraEditorUtilities::GatherChangeIds(UNiagaraEmitter& Emitter, TMap<FGuid, FGuid>& ChangeIds, const FString& InDebugName, bool bWriteToLogDir)
{
	FString ExportText;
	ChangeIds.Empty();
	TArray<UNiagaraGraph*> Graphs;
	TArray<UNiagaraScript*> Scripts;
	for (FNiagaraAssetVersion& Version : Emitter.GetAllAvailableVersions())
	{
		Emitter.GetEmitterData(Version.VersionGuid)->GetScripts(Scripts);
	}

	// First gather all the graphs used by this emitter..
	for (UNiagaraScript* Script : Scripts)
	{
		if (Script != nullptr && Script->GetLatestSource() != nullptr)
		{
			UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(Script->GetLatestSource());
			if (ScriptSource != nullptr)
			{
				Graphs.AddUnique(ScriptSource->NodeGraph);
			}

			if (bWriteToLogDir)
			{
				FNiagaraVMExecutableDataId Id;
				Script->ComputeVMCompilationId(Id, FGuid());
				FString KeyString;
				Id.AppendKeyString(KeyString);

				UEnum* FoundEnum = StaticEnum<ENiagaraScriptUsage>();

				FString ResultsEnum = TEXT("??");
				if (FoundEnum)
				{
					ResultsEnum = FoundEnum->GetNameStringByValue((int64)Script->Usage);
				}

				ExportText += FString::Printf(TEXT("Usage: %s CompileKey: %s\n"), *ResultsEnum, *KeyString );
			}
		}
	}

	
	// Now gather all the node change id's within these graphs.
	for (UNiagaraGraph* Graph : Graphs)
	{
		TArray<UNiagaraNode*> Nodes;
		Graph->GetNodesOfClass(Nodes);

		for (UNiagaraNode* Node : Nodes)
		{
			ChangeIds.Add(Node->NodeGuid, Node->GetChangeId());

			if (bWriteToLogDir)
			{
				ExportText += FString::Printf(TEXT("%40s    guid: %25s    changeId: %25s\n"), *Node->GetName(), *Node->NodeGuid.ToString(), *Node->GetChangeId().ToString());

			}
		}
	}

	if (bWriteToLogDir)
	{
		WriteTextFileToDisk(FPaths::ProjectLogDir(), InDebugName + TEXT(".txt"), ExportText, true);
	}
	
}

void FNiagaraEditorUtilities::GatherChangeIds(UNiagaraGraph& Graph, TMap<FGuid, FGuid>& ChangeIds, const FString& InDebugName, bool bWriteToLogDir)
{
	ChangeIds.Empty();
	TArray<UNiagaraGraph*> Graphs;
	
	FString ExportText;
	// Now gather all the node change id's within these graphs.
	{
		TArray<UNiagaraNode*> Nodes;
		Graph.GetNodesOfClass(Nodes);

		for (UNiagaraNode* Node : Nodes)
		{
			ChangeIds.Add(Node->NodeGuid, Node->GetChangeId());
			if (bWriteToLogDir)
			{
				ExportText += FString::Printf(TEXT("%40s    guid: %25s    changeId: %25s\n"), *Node->GetName(), *Node->NodeGuid.ToString(), *Node->GetChangeId().ToString());
			}
		}
	}

	if (bWriteToLogDir)
	{
		FNiagaraEditorUtilities::WriteTextFileToDisk(FPaths::ProjectLogDir(), InDebugName + TEXT(".txt"), ExportText, true);
	}
}

FText FNiagaraEditorUtilities::StatusToText(ENiagaraScriptCompileStatus Status)
{
	switch (Status)
	{
	default:
	case ENiagaraScriptCompileStatus::NCS_Unknown:
		return LOCTEXT("Recompile_Status", "Unknown status; should recompile");
	case ENiagaraScriptCompileStatus::NCS_Dirty:
		return LOCTEXT("Dirty_Status", "Dirty; needs to be recompiled");
	case ENiagaraScriptCompileStatus::NCS_Error:
		return LOCTEXT("CompileError_Status", "There was an error during compilation, see the log for details");
	case ENiagaraScriptCompileStatus::NCS_UpToDate:
		return LOCTEXT("GoodToGo_Status", "Good to go");
	case ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings:
		return LOCTEXT("GoodToGoWarning_Status", "There was a warning during compilation, see the log for details");
	}
}

ENiagaraScriptCompileStatus FNiagaraEditorUtilities::UnionCompileStatus(const ENiagaraScriptCompileStatus& StatusA, const ENiagaraScriptCompileStatus& StatusB)
{
	if (StatusA != StatusB)
	{
		if (StatusA == ENiagaraScriptCompileStatus::NCS_Unknown || StatusB == ENiagaraScriptCompileStatus::NCS_Unknown)
		{
			return ENiagaraScriptCompileStatus::NCS_Unknown;
		}
		else if (StatusA >= ENiagaraScriptCompileStatus::NCS_MAX || StatusB >= ENiagaraScriptCompileStatus::NCS_MAX)
		{
			return ENiagaraScriptCompileStatus::NCS_MAX;
		}
		else if (StatusA == ENiagaraScriptCompileStatus::NCS_Dirty || StatusB == ENiagaraScriptCompileStatus::NCS_Dirty)
		{
			return ENiagaraScriptCompileStatus::NCS_Dirty;
		}
		else if (StatusA == ENiagaraScriptCompileStatus::NCS_Error || StatusB == ENiagaraScriptCompileStatus::NCS_Error)
		{
			return ENiagaraScriptCompileStatus::NCS_Error;
		}
		else if (StatusA == ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings || StatusB == ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings)
		{
			return ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings;
		}
		else if (StatusA == ENiagaraScriptCompileStatus::NCS_BeingCreated || StatusB == ENiagaraScriptCompileStatus::NCS_BeingCreated)
		{
			return ENiagaraScriptCompileStatus::NCS_BeingCreated;
		}
		else if (StatusA == ENiagaraScriptCompileStatus::NCS_UpToDate || StatusB == ENiagaraScriptCompileStatus::NCS_UpToDate)
		{
			return ENiagaraScriptCompileStatus::NCS_UpToDate;
		}
		else
		{
			return ENiagaraScriptCompileStatus::NCS_Unknown;
		}
	}
	else
	{
		return StatusA;
	}
}

bool FNiagaraEditorUtilities::DataMatches(const FNiagaraVariable& Variable, const FStructOnScope& StructOnScope)
{
	if (Variable.GetType().GetScriptStruct() != StructOnScope.GetStruct() ||
		Variable.IsDataAllocated() == false)
	{
		return false;
	}

	return FMemory::Memcmp(Variable.GetData(), StructOnScope.GetStructMemory(), Variable.GetSizeInBytes()) == 0;
}

bool FNiagaraEditorUtilities::DataMatches(const FNiagaraVariable& VariableA, const FNiagaraVariable& VariableB)
{
	if (VariableA.GetType() != VariableB.GetType())
	{
		return false;
	}

	if (VariableA.IsDataAllocated() != VariableB.IsDataAllocated())
	{
		return false;
	}

	if (VariableA.IsDataAllocated())
	{
		return FMemory::Memcmp(VariableA.GetData(), VariableB.GetData(), VariableA.GetSizeInBytes()) == 0;
	}

	return true;
}

bool FNiagaraEditorUtilities::DataMatches(const FStructOnScope& StructOnScopeA, const FStructOnScope& StructOnScopeB)
{
	if (StructOnScopeA.GetStruct() != StructOnScopeB.GetStruct())
	{
		return false;
	}

	return FMemory::Memcmp(StructOnScopeA.GetStructMemory(), StructOnScopeB.GetStructMemory(), StructOnScopeA.GetStruct()->GetStructureSize()) == 0;
}

void FNiagaraEditorUtilities::CopyDataTo(FStructOnScope& DestinationStructOnScope, const FStructOnScope& SourceStructOnScope, bool bCheckTypes)
{
	checkf(DestinationStructOnScope.GetStruct()->GetStructureSize() == SourceStructOnScope.GetStruct()->GetStructureSize() &&
		(bCheckTypes == false || DestinationStructOnScope.GetStruct() == SourceStructOnScope.GetStruct()),
		TEXT("Can not copy data from one struct to another if their size is different or if the type is different and type checking is enabled."));
	FMemory::Memcpy(DestinationStructOnScope.GetStructMemory(), SourceStructOnScope.GetStructMemory(), SourceStructOnScope.GetStruct()->GetStructureSize());
}

TSharedPtr<SWidget> FNiagaraEditorUtilities::CreateInlineErrorText(TAttribute<FText> ErrorMessage, TAttribute<FText> ErrorTooltip)
{
	TSharedPtr<SHorizontalBox> ErrorInternalBox = SNew(SHorizontalBox);
		ErrorInternalBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(ErrorMessage)
			];

		return SNew(SHorizontalBox)
			.ToolTipText(ErrorTooltip)
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Error"))
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					ErrorInternalBox.ToSharedRef()
				];
}

void FNiagaraEditorUtilities::CompileExistingEmitters(const TArray<FVersionedNiagaraEmitter>& AffectedEmitters)
{
	TArray<TSharedPtr<FNiagaraSystemViewModel>> ExistingSystemViewModels;

	{
		FNiagaraSystemUpdateContext UpdateCtx;

		TSet<FVersionedNiagaraEmitter> CompiledEmitters;
		for (const FVersionedNiagaraEmitter& VersionedEmitter : AffectedEmitters)
		{
			// If we've already compiled this emitter, or it's invalid skip it.
			if (VersionedEmitter.Emitter == nullptr || CompiledEmitters.Contains(VersionedEmitter) || !IsValidChecked(VersionedEmitter.Emitter) || VersionedEmitter.Emitter->IsUnreachable())
			{
				continue;
			}

			// We only need to compile emitters referenced directly as instances by systems since emitters can now only be used in the context 
			// of a system.
			for (TObjectIterator<UNiagaraSystem> SystemIterator; SystemIterator; ++SystemIterator)
			{
				if (SystemIterator->ReferencesInstanceEmitter(VersionedEmitter))
				{
					SystemIterator->RequestCompile(false, &UpdateCtx);

					FNiagaraSystemViewModel::GetAllViewModelsForObject(*SystemIterator, ExistingSystemViewModels);

					for (const FNiagaraEmitterHandle& EmitterHandle : SystemIterator->GetEmitterHandles())
					{
						CompiledEmitters.Add(EmitterHandle.GetInstance());
					}
				}
			}
		}
	}
}

bool FNiagaraEditorUtilities::TryGetEventDisplayName(UNiagaraEmitter* Emitter, FGuid EventUsageId, FText& OutEventDisplayName)
{
	if (Emitter != nullptr)
	{
		Emitter->CheckVersionDataAvailable();
		for (const FNiagaraEventScriptProperties& EventScriptProperties : Emitter->GetLatestEmitterData()->GetEventHandlers())
		{
			if (EventScriptProperties.Script->GetUsageId() == EventUsageId)
			{
				OutEventDisplayName = FText::FromName(EventScriptProperties.SourceEventName);
				return true;
			}
		}
	}
	return false;
}

bool FNiagaraEditorUtilities::IsCompilableAssetClass(UClass* AssetClass)
{
	static const TSet<UClass*> CompilableClasses = { UNiagaraScript::StaticClass(), UNiagaraEmitter::StaticClass(), UNiagaraSystem::StaticClass() };
	return CompilableClasses.Contains(AssetClass);
}

FText FNiagaraEditorUtilities::GetVariableTypeCategory(const FNiagaraVariable& Variable)
{
	return GetTypeDefinitionCategory(Variable.GetType());
}

FText FNiagaraEditorUtilities::GetTypeDefinitionCategory(const FNiagaraTypeDefinition& TypeDefinition)
{
	FText Category = FText::GetEmpty();
	if (TypeDefinition.IsDataInterface())
	{
		Category = LOCTEXT("NiagaraParameterMenuGroupDI", "Data Interface");
	}
	else if (TypeDefinition.IsEnum())
	{
		Category = LOCTEXT("NiagaraParameterMenuGroupEnum", "Enum");
	}
	else if (TypeDefinition.IsUObject())
	{
		Category = LOCTEXT("NiagaraParameterMenuGroupObject", "Object");
	}
	else if (TypeDefinition.GetNameText().ToString().Contains("event"))
	{
		Category = LOCTEXT("NiagaraParameterMenuGroupEventType", "Event");
	}
	else
	{
		// add common types like bool, vector, etc into a category of their own so they appear first in the search
		Category = LOCTEXT("NiagaraParameterMenuGroupCommon", "Common");
	}
	return Category;
}

bool FNiagaraEditorUtilities::AreTypesAssignable(const FNiagaraTypeDefinition& FromType, const FNiagaraTypeDefinition& ToType)
{
	return FNiagaraUtilities::AreTypesAssignable(FromType, ToType);
}

void FNiagaraEditorUtilities::MarkDependentCompilableAssetsDirty(TArray<UObject*> InObjects)
{
	TArray<FAssetData> AssetsToCheck;
	for (UObject* InObject : InObjects)
	{	
		AssetsToCheck.Add(FAssetData(InObject));
	}
	MarkDependentCompilableAssetsDirty(AssetsToCheck);
}

void FNiagaraEditorUtilities::MarkDependentCompilableAssetsDirty(const TArray<FAssetData>& InAssetsToCheck)
{
	const FText LoadAndMarkDirtyDisplayName = NSLOCTEXT("NiagaraEditor", "MarkDependentAssetsDirtySlowTask", "Loading and marking dependent assets dirty.");
	GWarn->BeginSlowTask(LoadAndMarkDirtyDisplayName, true, true);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetIdentifier> ReferenceNames;

	TArray<FAssetData> AssetsToLoadAndMarkDirty;
	TArray<FAssetData> AssetsToCheck = InAssetsToCheck;

	while (AssetsToCheck.Num() > 0)
	{
		FAssetData AssetToCheck = AssetsToCheck[0];
		AssetsToCheck.RemoveAtSwap(0);
		bool bMustBeCompilableAsset = !InAssetsToCheck.Contains(AssetToCheck);//Don't require the original assets to be compilable.
		if (!bMustBeCompilableAsset || IsCompilableAssetClass(AssetToCheck.GetClass()))
		{
			if (AssetsToLoadAndMarkDirty.Contains(AssetToCheck) == false)
			{
				AssetsToLoadAndMarkDirty.Add(AssetToCheck);
				TArray<FName> Referencers;
				AssetRegistryModule.Get().GetReferencers(AssetToCheck.PackageName, Referencers);
				for (FName& Referencer : Referencers)
				{
					AssetRegistryModule.Get().GetAssetsByPackageName(Referencer, AssetsToCheck);
				}
			}
		}
	}

	int32 ItemIndex = 0;
	for (FAssetData AssetDataToLoadAndMarkDirty : AssetsToLoadAndMarkDirty)
	{	
		if (GWarn->ReceivedUserCancel())
		{
			break;
		}
		GWarn->StatusUpdate(ItemIndex++, AssetsToLoadAndMarkDirty.Num(), LoadAndMarkDirtyDisplayName);
		UObject* AssetToMarkDirty = AssetDataToLoadAndMarkDirty.GetAsset();
		AssetToMarkDirty->Modify(true);
	}

	GWarn->EndSlowTask();
}

template<typename Action>
void TraverseGraphFromOutputDepthFirst(const UEdGraphSchema_Niagara* Schema, UNiagaraNode* Node, Action& VisitAction)
{
	UNiagaraGraph* Graph = Node->GetNiagaraGraph();
	TArray<UNiagaraNode*> Nodes;
	Graph->BuildTraversal(Nodes, Node);
	for (UNiagaraNode* GraphNode : Nodes)
	{
		VisitAction(Schema, GraphNode);
	}
}

void FixUpNumericPinsVisitor(const UEdGraphSchema_Niagara* Schema, UNiagaraNode* Node)
{
	Node->ResolveNumerics(Schema, true, nullptr);
}

void FNiagaraEditorUtilities::FixUpNumericPins(const UEdGraphSchema_Niagara* Schema, UNiagaraNode* Node)
{
	if (ensureMsgf(Node->GetOutermost() == GetTransientPackage(), TEXT("Can not fix up numerics on non-transient node %s"), *Node->GetPathName()) == false)
	{
		return;
	}
	auto FixUpVisitor = [&](const UEdGraphSchema_Niagara* LSchema, UNiagaraNode* LNode) { FixUpNumericPinsVisitor(LSchema, LNode); };
	TraverseGraphFromOutputDepthFirst(Schema, Node, FixUpVisitor);
}

void FNiagaraEditorUtilities::SetStaticSwitchConstants(UNiagaraGraph* Graph, TArrayView<UEdGraphPin* const> CallInputs, const FCompileConstantResolver& ConstantResolver)
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	for (const TObjectPtr<UEdGraphNode>& Node : Graph->Nodes)
	{
		// if there is a static switch node its value must be set by the caller
		UNiagaraNodeStaticSwitch* SwitchNode = Cast<UNiagaraNodeStaticSwitch>(Node);
		if (SwitchNode)
		{
			if (SwitchNode->IsSetByCompiler())
			{
				SwitchNode->SetSwitchValue(ConstantResolver);
			}
			else if (SwitchNode->IsSetByPin())
			{
				SwitchNode->SetSwitchValue(ConstantResolver);
			}
			else
			{
				FEdGraphPinType VarType = Schema->TypeDefinitionToPinType(SwitchNode->GetInputType());
				SwitchNode->ClearSwitchValue();
				for (UEdGraphPin* InputPin : CallInputs)
				{
					if (InputPin->GetFName().IsEqual(SwitchNode->InputParameterName) && InputPin->PinType == VarType)
					{
						int32 SwitchValue = 0;
						if (ResolveConstantValue(InputPin, SwitchValue))
						{
							SwitchNode->SetSwitchValue(SwitchValue);
							break;
						}
					}
				}
			}
		}

		// if there is a function node, it might have delegated some of the static switch values inside its script graph
		// to be set by the next higher caller instead of directly by the user
		UNiagaraNodeFunctionCall* FunctionNode = Cast<UNiagaraNodeFunctionCall>(Node);
		if (FunctionNode)
		{
			FunctionNode->DebugState = FunctionNode->bInheritDebugStatus? ConstantResolver.CalculateDebugState() : ENiagaraFunctionDebugState::NoDebug;

			if (FunctionNode->PropagatedStaticSwitchParameters.Num() > 0)
			{
				for (const FNiagaraPropagatedVariable& SwitchValue : FunctionNode->PropagatedStaticSwitchParameters)
				{
					UEdGraphPin* ValuePin = FunctionNode->FindPin(SwitchValue.SwitchParameter.GetName(), EGPD_Input);
					if (!ValuePin)
					{
						continue;
					}
					ValuePin->DefaultValue = FString();
					FName PinName = SwitchValue.ToVariable().GetName();
					for (UEdGraphPin* InputPin : CallInputs)
					{
						if (InputPin->GetFName().IsEqual(PinName) && InputPin->PinType == ValuePin->PinType)
						{
							ValuePin->DefaultValue = InputPin->DefaultValue;
							break;
						}
					}
				}
			}
		}
	}
}

bool FNiagaraEditorUtilities::ResolveConstantValue(UEdGraphPin* Pin, int32& Value)
{
	if (Pin->LinkedTo.Num() > 0)
	{
		return false;
	}
	
	const FEdGraphPinType& PinType = Pin->PinType;
	if (PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType && PinType.PinSubCategoryObject.IsValid())
	{
		FString PinTypeName = PinType.PinSubCategoryObject->GetName();
		if (PinTypeName.Equals(FString(TEXT("NiagaraBool"))))
		{
			Value = Pin->DefaultValue.Equals(FString(TEXT("true"))) ? 1 : 0;
			return true;
		}
		else if (PinTypeName.Equals(FString(TEXT("NiagaraInt32"))))
		{
			Value = FCString::Atoi(*Pin->DefaultValue);
			return true;
		}
	}
	else if (PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum && PinType.PinSubCategoryObject.IsValid())
	{
		UEnum* Enum = Cast<UEnum>(PinType.PinSubCategoryObject);
		FString FullName = Enum->GenerateFullEnumName(*Pin->DefaultValue);
		Value = Enum->GetIndexByName(FName(*FullName));
		return Value != INDEX_NONE;
	}
	return false;
}

TSharedPtr<FStructOnScope> FNiagaraEditorUtilities::StaticSwitchDefaultIntToStructOnScope(int32 InStaticSwitchDefaultValue, FNiagaraTypeDefinition InSwitchType)
{
	if (InSwitchType.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		checkf(FNiagaraBool::StaticStruct()->GetStructureSize() == InSwitchType.GetSize(), TEXT("Value to type def size mismatch."));

		FNiagaraBool BoolValue;
		BoolValue.SetValue(InStaticSwitchDefaultValue != 0);

		TSharedPtr<FStructOnScope> StructValue = MakeShared<FStructOnScope>(InSwitchType.GetStruct());
		FMemory::Memcpy(StructValue->GetStructMemory(), &BoolValue, InSwitchType.GetSize());

		return StructValue;
	}
	else if (InSwitchType.IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || InSwitchType.IsEnum())
	{
		checkf(FNiagaraInt32::StaticStruct()->GetStructureSize() == InSwitchType.GetSize(), TEXT("Value to type def size mismatch."));

		FNiagaraInt32 IntValue;
		IntValue.Value = InStaticSwitchDefaultValue;

		TSharedPtr<FStructOnScope> StructValue = MakeShared<FStructOnScope>(InSwitchType.GetStruct());
		FMemory::Memcpy(StructValue->GetStructMemory(), &IntValue, InSwitchType.GetSize());

		return StructValue;
	}

	return TSharedPtr<FStructOnScope>();
}

/* Go through the graph and attempt to auto-detect the type of any numeric pins by working back from the leaves of the graph. Only change the types of pins, not FNiagaraVariables.*/
void PreprocessGraph(const UEdGraphSchema_Niagara* Schema, UNiagaraGraph* Graph, UNiagaraNodeOutput* OutputNode)
{
	check(OutputNode);
	{
		FNiagaraEditorUtilities::FixUpNumericPins(Schema, OutputNode);
	}
}

/* Go through the graph and force any input nodes with Numeric types to a hard-coded type of float. This will allow modules and functions to compile properly.*/
void PreProcessGraphForInputNumerics(const UEdGraphSchema_Niagara* Schema, UNiagaraGraph* Graph, TArray<FNiagaraVariable>& OutChangedNumericParams)
{
	// Visit all input nodes
	TArray<UNiagaraNodeInput*> InputNodes;
	Graph->FindInputNodes(InputNodes);
	FPinCollectorArray OutputPins;
	for (UNiagaraNodeInput* InputNode : InputNodes)
	{
		// See if any of the output pins are of Numeric type. If so, force to floats.
		OutputPins.Reset();
		InputNode->GetOutputPins(OutputPins);
		for (UEdGraphPin* OutputPin : OutputPins)
		{
			FNiagaraTypeDefinition OutputPinType = Schema->PinToTypeDefinition(OutputPin);
			if (OutputPinType == FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				OutputPin->PinType = Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetFloatDef());
			}
		}

		// Record that we touched this variable for later cleanup and make sure that the 
		// variable's type now matches the pin.
		if (InputNode->Input.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
		{
			OutChangedNumericParams.Add(InputNode->Input);
			InputNode->Input.SetType(FNiagaraTypeDefinition::GetFloatDef());
		}
	}
}

/* Should be called after all pins have been successfully auto-detected for type. This goes through and synchronizes any Numeric FNiagaraVarible outputs with the deduced pin type. This will allow modules and functions to compile properly.*/
void PreProcessGraphForAttributeNumerics(const UEdGraphSchema_Niagara* Schema, UNiagaraGraph* Graph, UNiagaraNodeOutput* OutputNode, TArray<FNiagaraVariable>& OutChangedNumericParams)
{
	// Visit the output node
	if (OutputNode != nullptr)
	{
		// For each pin, make sure that if it has a valid type, but the associated variable is still Numeric,
		// force the variable to match the pin's new type. Record that we touched this variable for later cleanup.
		FPinCollectorArray InputPins;
		OutputNode->GetInputPins(InputPins);
		check(OutputNode->Outputs.Num() == InputPins.Num());
		for (int32 i = 0; i < InputPins.Num(); i++)
		{
			FNiagaraVariable& Param = OutputNode->Outputs[i];
			UEdGraphPin* InputPin = InputPins[i];

			FNiagaraTypeDefinition InputPinType = Schema->PinToTypeDefinition(InputPin);
			if (Param.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef() &&
				InputPinType != FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				OutChangedNumericParams.Add(Param);
				Param.SetType(InputPinType);
			}
		}
	}
}

void FNiagaraEditorUtilities::ResolveNumerics(UNiagaraGraph* SourceGraph, bool bForceParametersToResolveNumerics, TArray<FNiagaraVariable>& ChangedNumericParams)
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(SourceGraph->GetSchema());

	// In the case of functions or modules, we may not have enough information at this time to fully resolve the type. In that case,
	// we circumvent the resulting errors by forcing a type. This gives the user an appropriate level of type checking. We will, however need to clean this up in
	// the parameters that we output.
	//bool bForceParametersToResolveNumerics = InScript->IsStandaloneScript();
	if (bForceParametersToResolveNumerics)
	{
		PreProcessGraphForInputNumerics(Schema, SourceGraph, ChangedNumericParams);
	}

	// Auto-deduce the input types for numerics in the graph and overwrite the types on the pins. If PreProcessGraphForInputNumerics occurred, then
	// we will have pre-populated the inputs with valid types.
	TArray<UNiagaraNodeOutput*> OutputNodes;
	SourceGraph->FindOutputNodes(OutputNodes);

	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		PreprocessGraph(Schema, SourceGraph, OutputNode);

		// Now that we've auto-deduced the types, we need to handle any lingering Numerics in the Output's FNiagaraVariable outputs. 
		// We use the pin's deduced type to temporarily overwrite the variable's type.
		if (bForceParametersToResolveNumerics)
		{
			PreProcessGraphForAttributeNumerics(Schema, SourceGraph, OutputNode, ChangedNumericParams);
		}
	}
}

void FNiagaraEditorUtilities::PreprocessFunctionGraph(const UEdGraphSchema_Niagara* Schema, UNiagaraGraph* Graph, TArrayView<UEdGraphPin* const> CallInputs, TArrayView<UEdGraphPin* const> CallOutputs,
	ENiagaraScriptUsage ScriptUsage, const FCompileConstantResolver& ConstantResolver)
{
	if (ensureMsgf(Graph->GetOutermost() == GetTransientPackage(), TEXT("Can not preprocess non-transient function graph %s"), *Graph->GetPathName()) == false)
	{
		return;
	}

	// Change any numeric inputs or outputs to match the types from the call node.
	TArray<UNiagaraNodeInput*> InputNodes;

	// Only handle nodes connected to the correct output node in the event of multiple output nodes in the graph.
	UNiagaraGraph::FFindInputNodeOptions Options;
	Options.bFilterByScriptUsage = true;
	Options.TargetScriptUsage = ScriptUsage;

	Graph->FindInputNodes(InputNodes, Options);

	FPinCollectorArray OutputPins;
	for (UNiagaraNodeInput* InputNode : InputNodes)
	{
		FNiagaraVariable& Input = InputNode->Input;
		if (Input.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
		{
			UEdGraphPin* const* MatchingPin = CallInputs.FindByPredicate([&](const UEdGraphPin* Pin) { return (Pin->PinName == Input.GetName()); });

			if (MatchingPin != nullptr)
			{
				FNiagaraTypeDefinition PinType = Schema->PinToTypeDefinition(*MatchingPin);
				Input.SetType(PinType);
				OutputPins.Reset();
				InputNode->GetOutputPins(OutputPins);
				check(OutputPins.Num() == 1);
				OutputPins[0]->PinType = (*MatchingPin)->PinType;
			}
		}
	}

	UNiagaraNodeOutput* OutputNode = Graph->FindOutputNode(ScriptUsage);
	check(OutputNode);

	FPinCollectorArray InputPins;
	OutputNode->GetInputPins(InputPins);

	for (FNiagaraVariable& Output : OutputNode->Outputs)
	{
		if (Output.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
		{
			UEdGraphPin* const* MatchingPin = CallOutputs.FindByPredicate([&](const UEdGraphPin* Pin) { return (Pin->PinName == Output.GetName()); });

			if (MatchingPin != nullptr)
			{
				FNiagaraTypeDefinition PinType = Schema->PinToTypeDefinition(*MatchingPin);
				Output.SetType(PinType);
			}
		}
	}
	
	FNiagaraEditorUtilities::FixUpNumericPins(Schema, OutputNode);
	FNiagaraEditorUtilities::SetStaticSwitchConstants(Graph, CallInputs, ConstantResolver);
}

void FNiagaraEditorUtilities::GetFilteredScriptAssets(FGetFilteredScriptAssetsOptions InFilter, TArray<FAssetData>& OutFilteredScriptAssets)
{
	FNiagaraEditorModule::Get().PreloadSelectablePluginAssetsByClass(UNiagaraScript::StaticClass());

	FARFilter ScriptFilter;
	ScriptFilter.ClassPaths.Add(UNiagaraScript::StaticClass()->GetClassPathName());

	const UEnum* NiagaraScriptUsageEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Niagara.ENiagaraScriptUsage"), EFindObjectFlags::ExactClass);
	const FString QualifiedScriptUsageString = NiagaraScriptUsageEnum->GetNameStringByValue(static_cast<uint8>(InFilter.ScriptUsageToInclude));
	int32 LastColonIndex;
	QualifiedScriptUsageString.FindLastChar(TEXT(':'), LastColonIndex);
	const FString UnqualifiedScriptUsageString = QualifiedScriptUsageString.RightChop(LastColonIndex + 1);
	ScriptFilter.TagsAndValues.Add(GET_MEMBER_NAME_CHECKED(UNiagaraScript, Usage), UnqualifiedScriptUsageString);

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> FilteredScriptAssets;
	AssetRegistryModule.Get().GetAssets(ScriptFilter, FilteredScriptAssets);

	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	for (int i = 0; i < FilteredScriptAssets.Num(); ++i)
	{
		if(AssetToolsModule.Get().IsAssetVisible(FilteredScriptAssets[i], true) == false)
		{
			continue;
		}
			
		// Get the custom version the asset was saved with so it can be used below.
		int32 NiagaraVersion = INDEX_NONE;
		FilteredScriptAssets[i].GetTagValue(UNiagaraScript::NiagaraCustomVersionTagName, NiagaraVersion);
		
		// Check if the script is deprecated
		if (InFilter.bIncludeDeprecatedScripts == false)
		{
			bool bScriptIsDeprecated = false;
			bool bFoundDeprecatedTag = FilteredScriptAssets[i].GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, bDeprecated), bScriptIsDeprecated);
			if (bFoundDeprecatedTag == false)
			{
				if (FilteredScriptAssets[i].IsAssetLoaded())
				{
					UNiagaraScript* Script = static_cast<UNiagaraScript*>(FilteredScriptAssets[i].GetAsset());
					if (Script != nullptr && Script->GetLatestScriptData())
					{
						bScriptIsDeprecated = Script->GetLatestScriptData()->bDeprecated;
					}
				}
			}
			if (bScriptIsDeprecated)
			{
				continue;
			}
		}

		// Check if usage bitmask matches
		if (InFilter.TargetUsageToMatch.IsSet())
		{
			int32 BitfieldValue = 0;
			FString BitfieldTagValue;
			if (NiagaraVersion == INDEX_NONE || NiagaraVersion < FNiagaraCustomVersion::AddSimulationStageUsageEnum)
			{
				// If there is no custom version, or it's less than the simulation stage enum fix up, we need to load the 
				// asset to get the correct bitmask since the shader stage enum broke the old ones.
				UNiagaraScript* AssetScript = Cast<UNiagaraScript>(FilteredScriptAssets[i].GetAsset());
				if (AssetScript != nullptr && AssetScript->GetLatestScriptData())
				{
					BitfieldValue = AssetScript->GetLatestScriptData()->ModuleUsageBitmask;
				}
			}
			else
			{
				// Otherwise the asset is new enough to have a valid bitmask.
				BitfieldTagValue = FilteredScriptAssets[i].GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, ModuleUsageBitmask));
				BitfieldValue = FCString::Atoi(*BitfieldTagValue);
			}

			int32 TargetBit = (BitfieldValue >> (int32)InFilter.TargetUsageToMatch.GetValue()) & 1;
			if (TargetBit != 1)
			{
				continue;
			}
		}

		// Check script visibility
		ENiagaraScriptLibraryVisibility ScriptVisibility = GetScriptAssetVisibility(FilteredScriptAssets[i]);
		if (ScriptVisibility == ENiagaraScriptLibraryVisibility::Hidden || (InFilter.bIncludeNonLibraryScripts == false && ScriptVisibility != ENiagaraScriptLibraryVisibility::Library))
		{
			continue;
		}

		// Check suggested state
		bool bSuggested = false;
		const bool bFoundSuggested = FilteredScriptAssets[i].GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, bSuggested), bSuggested);
		if(bFoundSuggested)
		{
			if(InFilter.SuggestedFiltering == FGetFilteredScriptAssetsOptions::OnlySuggested && !bSuggested)
			{
				continue;
			}
			else if(InFilter.SuggestedFiltering == FGetFilteredScriptAssetsOptions::NoSuggested && bSuggested)
			{
				continue;
			}
		}

		// Filter by allowed class usage.
		if (NiagaraEditorSettings->IsAllowedAssetByClassUsage(FilteredScriptAssets[i]) == false)
		{
			continue;
		}
		
		OutFilteredScriptAssets.Add(FilteredScriptAssets[i]);
	}
}

UNiagaraNodeOutput* FNiagaraEditorUtilities::GetScriptOutputNode(UNiagaraScript& Script)
{
	UNiagaraScriptSource* Source = CastChecked<UNiagaraScriptSource>(Script.GetLatestSource());
	return Source->NodeGraph->FindEquivalentOutputNode(Script.GetUsage(), Script.GetUsageId());
}

UNiagaraScript* FNiagaraEditorUtilities::GetScriptFromSystem(UNiagaraSystem& System, FGuid EmitterHandleId, ENiagaraScriptUsage Usage, FGuid UsageId)
{
	if (UNiagaraScript::IsEquivalentUsage(Usage, ENiagaraScriptUsage::SystemSpawnScript))
	{
		return System.GetSystemSpawnScript();
	}
	else if (UNiagaraScript::IsEquivalentUsage(Usage, ENiagaraScriptUsage::SystemUpdateScript))
	{
		return System.GetSystemUpdateScript();
	}
	else if (EmitterHandleId.IsValid())
	{
		const FNiagaraEmitterHandle* ScriptEmitterHandle = System.GetEmitterHandles().FindByPredicate(
			[EmitterHandleId](const FNiagaraEmitterHandle& EmitterHandle) { return EmitterHandle.GetId() == EmitterHandleId; });
		if (ScriptEmitterHandle != nullptr)
		{
			FVersionedNiagaraEmitterData* EmitterData = ScriptEmitterHandle->GetEmitterData();
			if (UNiagaraScript::IsEquivalentUsage(Usage, ENiagaraScriptUsage::EmitterSpawnScript))
			{
				return EmitterData->EmitterSpawnScriptProps.Script;
			}
			else if (UNiagaraScript::IsEquivalentUsage(Usage, ENiagaraScriptUsage::EmitterUpdateScript))
			{
				return EmitterData->EmitterUpdateScriptProps.Script;
			}
			else if (UNiagaraScript::IsEquivalentUsage(Usage, ENiagaraScriptUsage::ParticleSpawnScript))
			{
				return EmitterData->SpawnScriptProps.Script;
			}
			else if (UNiagaraScript::IsEquivalentUsage(Usage, ENiagaraScriptUsage::ParticleUpdateScript))
			{
				return EmitterData->UpdateScriptProps.Script;
			}
			else if (UNiagaraScript::IsEquivalentUsage(Usage, ENiagaraScriptUsage::ParticleEventScript))
			{
				for (const FNiagaraEventScriptProperties& EventScriptProperties : EmitterData->GetEventHandlers())
				{
					if (EventScriptProperties.Script->GetUsageId() == UsageId)
					{
						return EventScriptProperties.Script;
					}
				}
			}
			else if (UNiagaraScript::IsEquivalentUsage(Usage, ENiagaraScriptUsage::ParticleSimulationStageScript))
			{
				for (const UNiagaraSimulationStageBase* SimulationStage : EmitterData->GetSimulationStages())
				{
					if (SimulationStage && SimulationStage->Script && SimulationStage->Script->GetUsageId() == UsageId)
					{
						return SimulationStage->Script;
					}
				}
			}
		}
	}
	return nullptr;
}

const FNiagaraEmitterHandle* FNiagaraEditorUtilities::GetEmitterHandleForEmitter(UNiagaraSystem& System, const FVersionedNiagaraEmitter& Emitter)
{
	return System.GetEmitterHandles().FindByPredicate(
		[&Emitter](const FNiagaraEmitterHandle& EmitterHandle) { return EmitterHandle.GetInstance() == Emitter; });
}

ENiagaraScriptLibraryVisibility FNiagaraEditorUtilities::GetScriptAssetVisibility(const FAssetData& ScriptAssetData)
{
	FString Value;
	bool bIsLibraryTagFound = ScriptAssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, LibraryVisibility), Value);

	ENiagaraScriptLibraryVisibility ScriptVisibility = ENiagaraScriptLibraryVisibility::Invalid;
	if (bIsLibraryTagFound == false)
	{
		if (ScriptAssetData.IsAssetLoaded())
		{
			UNiagaraScript* Script = Cast<UNiagaraScript>(ScriptAssetData.GetAsset());
			if (Script != nullptr)
			{
				ScriptVisibility = Script->GetLatestScriptData()->LibraryVisibility;
			}
		}
	}
	else
	{
		UEnum* VisibilityEnum = StaticEnum<ENiagaraScriptLibraryVisibility>();
		int32 Index = VisibilityEnum->GetIndexByNameString(Value);
		ScriptVisibility = (ENiagaraScriptLibraryVisibility) VisibilityEnum->GetValueByIndex(Index == INDEX_NONE ? 0 : Index);
	}
	
	if (ScriptVisibility == ENiagaraScriptLibraryVisibility::Invalid)
	{
		// Check the deprecated tag value as a fallback. If even that property cannot be found the asset must be pretty old and should just be exposed as that was the default in the beginning.
		bool bIsExposed = false;
		bIsLibraryTagFound = ScriptAssetData.GetTagValue(FName("bExposeToLibrary"), bIsExposed);
		ScriptVisibility = !bIsLibraryTagFound || bIsExposed ? ENiagaraScriptLibraryVisibility::Library : ENiagaraScriptLibraryVisibility::Unexposed;
	}
	return ScriptVisibility;
}

bool FNiagaraEditorUtilities::GetIsInheritableFromAssetRegistryTags(const FAssetData& AssetData, bool& bUseInheritance)
{
	if(AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(UNiagaraEmitter, bIsInheritable), bUseInheritance))
	{
		return true;		
	}
	else
	{
		ENiagaraScriptTemplateSpecification DeprecatedTemplateTag;
		if(GetTemplateSpecificationFromTag(AssetData, DeprecatedTemplateTag))
		{
			bUseInheritance = DeprecatedTemplateTag == ENiagaraScriptTemplateSpecification::None;
			return true;
		}
	}

	return false;
}

// this function is used as an overload inside FAssetData.GetTagValue for the ENiagaraScriptTemplateSpecification enum
void LexFromString(ENiagaraScriptTemplateSpecification& OutValue, const TCHAR* Buffer)
{
	OutValue = (ENiagaraScriptTemplateSpecification) StaticEnum<ENiagaraScriptTemplateSpecification>()->GetValueByName(FName(Buffer));
}

bool FNiagaraEditorUtilities::GetTemplateSpecificationFromTag(const FAssetData& Data, ENiagaraScriptTemplateSpecification& OutTemplateSpecification)
{
	ENiagaraScriptTemplateSpecification TemplateSpecification = ENiagaraScriptTemplateSpecification::None;
	bool bTemplateEnumTagFound = Data.GetTagValue(FName("TemplateSpecification"), TemplateSpecification);

	if(bTemplateEnumTagFound)
	{
		OutTemplateSpecification = TemplateSpecification;
		return true;
	}
	else
	{
		bool bIsTemplateAsset = false;
		bool bDeprecatedTemplateTagFound = Data.GetTagValue(FName(TEXT("bIsTemplateAsset")), bIsTemplateAsset);

		if(bDeprecatedTemplateTagFound)
		{
			bIsTemplateAsset ? OutTemplateSpecification = ENiagaraScriptTemplateSpecification::Template : OutTemplateSpecification = ENiagaraScriptTemplateSpecification::None;
			return true;
		}		
	}

	return false;
}

bool FNiagaraEditorUtilities::IsScriptAssetInLibrary(const FAssetData& ScriptAssetData)
{
	return GetScriptAssetVisibility(ScriptAssetData) == ENiagaraScriptLibraryVisibility::Library;
}

bool FNiagaraEditorUtilities::IsEnginePluginAsset(const FTopLevelAssetPath& InTopLevelAssetPath)
{
	FString PackagePathLocal = "";
	return FPackageName::TryConvertGameRelativePackagePathToLocalPath(InTopLevelAssetPath.ToString(), PackagePathLocal) &&
		FPaths::IsUnderDirectory(PackagePathLocal, FPaths::EnginePluginsDir() + "FX/Niagara");
}

int32 FNiagaraEditorUtilities::GetWeightForItem(const TSharedPtr<FNiagaraMenuAction_Generic>& InCurrentAction, FName FavoriteActionsProfile, const TArray<FString>& InFilterTerms)
{
	// The overall 'weight'
	int32 TotalWeight = 0;

	// Some simple weight figures to help find the most appropriate match
	const float KeywordWeight = 3.f;
	const float DisplayNameWeight = 30.f;
	const float CategoryWeight = 40.f;
	const int32 ConsecutiveMatchExponent = 3;
	const int32 WholeMatchExponent = 4;

	const float WordContainsTermBonusWeightMultiplier = 0.5f;
	const float StartsWithBonusWeightMultiplier = 10.f;
	const float EqualsBonusWeightMultiplier = 50.f;
	const float ShorterWeight = 3.f;

	FString FilterText;

	for(const FString& FilterTerm : InFilterTerms)
	{
		FilterText += FilterTerm + TEXT(" ");
	}

	FilterText = FilterText.TrimStartAndEnd();

	// the 'looser' this is set the larger the score
	enum EWordMatchStyle
	{
		Contains,
		StartsWith,
		IsEqual
	};
	// Helper array
	struct FArrayWithWeight
	{
		FArrayWithWeight(const TArray< FString >* InArray, int32 InWeight, int32 InFirstTermMultiplier = 1, EWordMatchStyle InConsecutiveMatchStyle = StartsWith)
			: Array(InArray)
			, Weight(InWeight)
			, FirstTermMultiplier(InFirstTermMultiplier)
			, ConsecutiveMatchStyle(InConsecutiveMatchStyle)
		{
		}

		const TArray< FString >* Array;
		int32 Weight;
		int32 FirstTermMultiplier;
		EWordMatchStyle ConsecutiveMatchStyle;
	};

	// Setup an array of arrays so we can do a weighted search			
	TArray<FArrayWithWeight> WeightedArrayList;

	// DisplayName
	TArray<FString> DisplayNameArray;
	InCurrentAction->DisplayName.ToString().ParseIntoArray(DisplayNameArray, TEXT(" "), true);
	WeightedArrayList.Add(FArrayWithWeight(&DisplayNameArray, DisplayNameWeight, 10, StartsWith));

	// Alternate Name
	TArray<FString> AlternameNameArray;
	if(InCurrentAction->AlternateSearchName.IsSet())
	{
		InCurrentAction->AlternateSearchName.GetValue().ToString().ParseIntoArray(AlternameNameArray, TEXT(" "), true);
		WeightedArrayList.Add(FArrayWithWeight(&AlternameNameArray, DisplayNameWeight, 10, StartsWith));
	}

	// Keywords
	TArray<FString> KeywordsArray;
	InCurrentAction->Keywords.ToString().ParseIntoArray(KeywordsArray, TEXT(" "), true);
	WeightedArrayList.Add(FArrayWithWeight(&KeywordsArray, KeywordWeight));

	// Category + DisplayName. Used so that exact hits can still be used for the display name alone, but category + displayname still results in a combo hit.
	TArray<FString> CategoryDisplayNameArray = InCurrentAction->Categories;
	CategoryDisplayNameArray.Append(DisplayNameArray);
    WeightedArrayList.Add(FArrayWithWeight(&CategoryDisplayNameArray, CategoryWeight, 20, StartsWith));
	
	// Now iterate through all the filter terms and calculate a 'weight' using the values and multipliers
	const FString* FilterTerm = nullptr;

	// Now check the weighted lists	(We could further improve the hit weight by checking consecutive word matches)
	for (int32 KeywordArrayIndex = 0; KeywordArrayIndex < WeightedArrayList.Num(); ++KeywordArrayIndex)
	{
		const TArray<FString>& KeywordArray = *WeightedArrayList[KeywordArrayIndex].Array;
		float WeightPerList = 0.0f;
		float KeywordArrayWeight = WeightedArrayList[KeywordArrayIndex].Weight;
		int32 FirstTermMultiplier = WeightedArrayList[KeywordArrayIndex].FirstTermMultiplier;
		EWordMatchStyle WordMatchStyle = WeightedArrayList[KeywordArrayIndex].ConsecutiveMatchStyle;

		FString FullKeyword;

		for (const FString& Keyword : KeywordArray)
		{
			FullKeyword += Keyword + TEXT(" ");
		}

		FullKeyword = FullKeyword.TrimStartAndEnd();
		
		int32 ConsecutiveWordMatchCount = 0;
		// Count of how many words in this keyword array contain a filter(letter) that the user has typed in
		int32 WordMatchCount = 0;
		int32 LastMatchingIndex = INDEX_NONE;

		auto CalculateConsecutive = [&](int32 FilterIndex)
		{
			int32 PreviousLastMatchingIndex = LastMatchingIndex;
			LastMatchingIndex = FilterIndex;

			// this calculation is not quite correct in case there are multiple sets of consecutive hits
			if(LastMatchingIndex - PreviousLastMatchingIndex == 1)
			{
				ConsecutiveWordMatchCount++;
			}			
		};

		// The number of characters in this keyword array
		int32 KeywordArrayCharLength = 0;
		
		// Loop through every word that the user could be looking for
		for (int32 FilterIndex = 0; FilterIndex < InFilterTerms.Num(); ++FilterIndex)
		{
			// we exclude any word from appearing more than once
			TSet<FString> UsedWords;
			
			FilterTerm = &InFilterTerms[FilterIndex];

			float MaxWeightOfKeywords = (float)INDEX_NONE;
			EWordMatchStyle MaxFoundMatchStyle = EWordMatchStyle::Contains;

			auto CalculateSupersedingMatch = [&](float IterativeWeight, EWordMatchStyle IterativeFoundWordMatchStyle)
			{
				// for a superseding match, the found match style needs to at least be the word match style of the array
				if(IterativeFoundWordMatchStyle >= WordMatchStyle)
				{
					// we only accept the new match if its iterative weight is actually higher
					if(IterativeWeight >= MaxWeightOfKeywords)
					{
						MaxWeightOfKeywords = IterativeWeight;
						MaxFoundMatchStyle = IterativeFoundWordMatchStyle;
					}
				}
			};
			
			for (int32 KeywordIndex = 0; KeywordIndex < KeywordArray.Num(); ++KeywordIndex)
			{
				// Keep track of how long all the words in the array are

				const FString& Keyword = KeywordArray[KeywordIndex];
				KeywordArrayCharLength += Keyword.Len();
				if(UsedWords.Contains(Keyword))
				{
					continue;
				}

				UsedWords.Add(Keyword);
				
				if (Keyword.Contains(*FilterTerm, ESearchCase::IgnoreCase))
				{
					CalculateSupersedingMatch(KeywordArrayWeight * WordContainsTermBonusWeightMultiplier, Contains);
					
					// If the word starts with the term, give it a little extra boost of weight
					if (Keyword.StartsWith(*FilterTerm, ESearchCase::IgnoreCase))
					{					
						CalculateSupersedingMatch(KeywordArrayWeight * StartsWithBonusWeightMultiplier, StartsWith);						
					}
					
					if(Keyword.Equals(*FilterTerm, ESearchCase::IgnoreCase))
					{									
						CalculateSupersedingMatch(KeywordArrayWeight * EqualsBonusWeightMultiplier, IsEqual);
					}
				}
			}

			if(MaxWeightOfKeywords != (float)INDEX_NONE)
			{
				// if the match style that applies is 'stricter' than the required match style of the array, we consider it a hit
				if(MaxFoundMatchStyle >= WordMatchStyle)
				{
					WordMatchCount++;
					CalculateConsecutive(FilterIndex);
					WeightPerList += MaxWeightOfKeywords;
				}
			}

			// the first filter term can have special meaning, for example as a category. "Make" is a category, so if the user types "Make" we want to favor it
			if(FilterIndex == 0)
			{
				WeightPerList *= FirstTermMultiplier;
			}
		}

		if(WeightPerList > 0)
		{
			if (KeywordArrayCharLength > 0)
			{			
				// The longer the match is, the more points it loses
				float ShortWeight = KeywordArrayCharLength * ShorterWeight;
				WeightPerList -= ShortWeight;
				// WeightPerList could be negative if the shortweight subtraction is executed
				WeightPerList = FMath::Max(WeightPerList, 0.f);
			}

			WeightPerList += FMath::Pow(WordMatchCount * KeywordArrayWeight, 2);

			WeightPerList += FMath::Pow(ConsecutiveWordMatchCount * KeywordArrayWeight, ConsecutiveMatchExponent);

			if(FilterText.Equals(FullKeyword, ESearchCase::IgnoreCase))
			{
				WeightPerList += FMath::Pow(KeywordArrayWeight, WholeMatchExponent);
			}

			WeightPerList *= InCurrentAction->SearchWeightMultiplier;

			
			TotalWeight += WeightPerList;
		}
	}

	// NPC Variables are rarely wanted so we halve their weight here
	if(InCurrentAction->GetParameterVariable().IsSet())
	{
		if(InCurrentAction->GetParameterVariable().GetValue().IsInNameSpace(FNiagaraConstants::ParameterCollectionNamespaceString))
		{
			TotalWeight *= 0.5f;
		}
	}

	// favorite actions get favored massively
	if(InCurrentAction->FavoritesActionData.IsSet())
	{
		FNiagaraFavoritesActionData ActionData = InCurrentAction->FavoritesActionData.GetValue();
		if(UNiagaraFavoriteActionsConfig::Get()->GetActionsProfile(FavoriteActionsProfile).IsFavorite(ActionData))
		{
			TotalWeight += (int32)FMath::Pow(10.0f, 4.0f);
		}
	}
	
	return TotalWeight > 0.f ? TotalWeight : INDEX_NONE;
}

bool FNiagaraEditorUtilities::DoesItemMatchFilterText(const FText& FilterText, const TSharedPtr<FNiagaraMenuAction_Generic>& Item)
{
	TArray<FString> FilterTerms;
	FilterText.ToString().ParseIntoArray(FilterTerms, TEXT(" "), true);

	int32 DisplayNameMatchCount = 0;
	FString DisplayNameWithoutSpaces = Item->DisplayName.ToString().Replace(TEXT(" "), TEXT(""));
	for(int32 FilterIndex = 0; FilterIndex < FilterTerms.Num(); FilterIndex++)
	{
		FString FilterTerm = FilterTerms[FilterIndex];
		
		if(Item->DisplayName.ToString().Contains(FilterTerm))
		{
			DisplayNameMatchCount++;
		}
	}

	// we also want to include items that would match the filter text if the spaces are removed.
	// i.e. typing "oneminus" should also allow the action "one minus"
	if(DisplayNameWithoutSpaces.Contains(FilterText.ToString()))
	{
		return true;
	}

	if(DisplayNameMatchCount >= FilterTerms.Num() / 2.f)
	{
		return true;
	}

	TArray<FString> KeywordArray;
	Item->Keywords.ToString().ParseIntoArray(KeywordArray, TEXT(" "));
	for(int32 FilterIndex = 0; FilterIndex < FilterTerms.Num(); FilterIndex++)
	{
		FString FilterTerm = FilterTerms[FilterIndex];

		for(const FString& Keyword : KeywordArray)
		{
			if(Keyword.Contains(FilterTerm))
			{
				return true;
			}
		}		
	}
	
	for(const FString& Category : Item->Categories)
	{
		if(Category.Contains(FilterText.ToString()))
		{
			return true;
		}
	}	

	return false;
}

TTuple<EScriptSource, FText> FNiagaraEditorUtilities::GetScriptSource(const FAssetData& ScriptAssetData)
{
	FString PackagePathLocal;
	FPackageName::TryConvertGameRelativePackagePathToLocalPath(ScriptAssetData.PackagePath.ToString(), PackagePathLocal);

	const auto ResolvePluginName =
		[PackagePathLocal]() -> TOptional<FText>
		{
			int32 ContentFoundIndex = PackagePathLocal.Find(TEXT("/Content"));
			if (ContentFoundIndex != INDEX_NONE)
			{
				const FString LeftPart = PackagePathLocal.Left(ContentFoundIndex);
				const bool bFound = LeftPart.FindLastChar('/', ContentFoundIndex);

				if (bFound)
				{
					const FString PluginName = LeftPart.RightChop(ContentFoundIndex + 1);

					return FText::FromString(PluginName);
				}
			}
			return {};
		};

	if(FPaths::IsUnderDirectory(PackagePathLocal, FPaths::EnginePluginsDir() / TEXT("FX")))
	{
		TOptional<FText> PluginName = ResolvePluginName();
		if (PluginName.IsSet())
		{
			return TTuple<EScriptSource, FText>(EScriptSource::Niagara, PluginName.GetValue());
		}
	}
	
	if(FPaths::IsUnderDirectory(PackagePathLocal, FPaths::EnginePluginsDir()) || FPaths::IsUnderDirectory(PackagePathLocal, FPaths::ProjectPluginsDir()))
	{
		TOptional<FText> PluginName = ResolvePluginName();
		if (PluginName.IsSet())
		{
			return TTuple<EScriptSource, FText>(EScriptSource::Plugins, PluginName.GetValue());
		}
	}

	if (FPaths::IsRestrictedPath(PackagePathLocal))
	{
		const TArray<FString> PluginSourcePaths = { FPaths::ProjectDir(), FPaths::EngineDir() };
		for (const FString& RestrictedPath : FPaths::GetRestrictedFolderNames())
		{
			for (const FString& PluginSourcePath : PluginSourcePaths)
			{
				const FString& RestrictedPluginPath = FPaths::Combine(PluginSourcePath, TEXT("Restricted/"), RestrictedPath, TEXT("Plugins/"));
				if (FPaths::IsUnderDirectory(PackagePathLocal, RestrictedPluginPath))
				{
					TOptional<FText> PluginName = ResolvePluginName();
					if (PluginName.IsSet())
					{
						return TTuple<EScriptSource, FText>(EScriptSource::Plugins, PluginName.GetValue());
					}
				}
			}
		}
	}

	if(FPaths::IsUnderDirectory(PackagePathLocal, FPaths::GameDevelopersDir()))
	{
		return TTuple<EScriptSource, FText>(EScriptSource::Developer, FText::FromString("Developer"));
	}
	
	if(FPaths::IsUnderDirectory(PackagePathLocal, FPaths::ProjectContentDir()))
	{
		return TTuple<EScriptSource, FText>(EScriptSource::Game, FText::FromString("Game"));
	}		

	return TTuple<EScriptSource, FText>(EScriptSource::Unknown, FText::FromString(""));
}

FLinearColor FNiagaraEditorUtilities::GetScriptSourceColor(EScriptSource ScriptData)
{
	return GetDefault<UNiagaraEditorSettings>()->GetSourceColor(ScriptData);	
}

NIAGARAEDITOR_API FText FNiagaraEditorUtilities::FormatScriptName(FName Name, bool bIsInLibrary)
{
	return FText::FromString(FName::NameToDisplayString(Name.ToString(), false) + (bIsInLibrary ? TEXT("") : TEXT("*")));
}

FText FNiagaraEditorUtilities::FormatScriptDescription(FText Description, const FSoftObjectPath& Path, bool bIsInLibrary)
{
	FText LibrarySuffix = !bIsInLibrary
		? LOCTEXT("LibrarySuffix", "\n* Script is not exposed to the library.")
		: FText();

	return Description.IsEmptyOrWhitespace()
		? FText::Format(LOCTEXT("ScriptAssetDescriptionFormatPathOnly", "Path: {0}{1}"), FText::FromString(Path.ToString()), LibrarySuffix)
		: FText::Format(LOCTEXT("ScriptAssetDescriptionFormat", "{1}\nPath: {0}{2}"), FText::FromString(Path.ToString()), Description, LibrarySuffix);
}

FText FNiagaraEditorUtilities::FormatVariableDescription(FText Description, FText Name, FText Type)
{
	if (Description.IsEmptyOrWhitespace() == false)
	{
		return FText::Format(LOCTEXT("VariableDescriptionFormat", "{0}\n\nName: {1}\n\nType: {2}"), Description, Name, Type);
	}

	return FText::Format(LOCTEXT("VariableDescriptionFormat_NoDesc", "Name: {0}\n\nType: {1}"), Name, Type);
}

void FNiagaraEditorUtilities::ResetSystemsThatReferenceSystemViewModel(const FNiagaraSystemViewModel& ReferencedSystemViewModel)
{
	checkf(&ReferencedSystemViewModel, TEXT("ResetSystemsThatReferenceSystemViewModel() called on destroyed SystemViewModel."));
	TArray<TSharedPtr<FNiagaraSystemViewModel>> ComponentSystemViewModels;
	TArray<UNiagaraComponent*> ReferencingComponents = GetComponentsThatReferenceSystemViewModel(ReferencedSystemViewModel);
	for (auto Component : ReferencingComponents)
	{
		ComponentSystemViewModels.Reset();
		FNiagaraSystemViewModel::GetAllViewModelsForObject(Component->GetAsset(), ComponentSystemViewModels);
		if (ComponentSystemViewModels.Num() > 0)
		{
			//The component has a viewmodel, call ResetSystem() on the viewmodel 
			for (auto SystemViewModel : ComponentSystemViewModels)
			{
				if (SystemViewModel.IsValid() && SystemViewModel.Get() != &ReferencedSystemViewModel)
				{
					SystemViewModel->ResetSystem(FNiagaraSystemViewModel::ETimeResetMode::AllowResetTime, FNiagaraSystemViewModel::EMultiResetMode::ResetThisInstance, FNiagaraSystemViewModel::EReinitMode::ResetSystem);
				}
			}
		}
		else
		{
			//The component does not have a viewmodel, call ResetSystem() on the component
			Component->ResetSystem();
		}
	}
}

TArray<UNiagaraComponent*> FNiagaraEditorUtilities::GetComponentsThatReferenceSystem(const UNiagaraSystem& ReferencedSystem)
{
	check(&ReferencedSystem);
	TArray<UNiagaraComponent*> ReferencingComponents;
	for (TObjectIterator<UNiagaraComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		UNiagaraComponent* Component = *ComponentIt;
		if (Component && Component->GetAsset())
		{
			if (Component->GetAsset() == &ReferencedSystem)
			{
				ReferencingComponents.Add(Component);
			}
		}
	}
	return ReferencingComponents;
}

TArray<UNiagaraComponent*> FNiagaraEditorUtilities::GetComponentsThatReferenceSystemViewModel(const FNiagaraSystemViewModel& ReferencedSystemViewModel)
{
	check(&ReferencedSystemViewModel);
	TArray<UNiagaraComponent*> ReferencingComponents;
	for (TObjectIterator<UNiagaraComponent> ComponentIt; ComponentIt; ++ComponentIt)
	{
		UNiagaraComponent* Component = *ComponentIt;
		if (Component && Component->GetAsset())
		{
			for (const auto& EmitterHandle : ReferencedSystemViewModel.GetSystem().GetEmitterHandles())
			{
				FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
				if (EmitterData && Component->GetAsset()->UsesEmitter(EmitterData->GetParent()))
				{
					ReferencingComponents.Add(Component);
				}
			}
		}
	}
	return ReferencingComponents;
}

const FGuid FNiagaraEditorUtilities::AddEmitterToSystem(UNiagaraSystem& InSystem, UNiagaraEmitter& InEmitterToAdd, FGuid EmitterVersion, bool bCreateCopy)
{
	FNiagaraEditorModule::Get().GetRecentsManager()->EmitterUsed(InEmitterToAdd);

	// Kill all system instances before modifying the emitter handle list to prevent accessing deleted data.
	KillSystemInstances(InSystem);

	TSet<FName> EmitterHandleNames;
	for (const FNiagaraEmitterHandle& EmitterHandle : InSystem.GetEmitterHandles())
	{
		EmitterHandleNames.Add(EmitterHandle.GetName());
	}

	UNiagaraSystemEditorData* SystemEditorData = CastChecked<UNiagaraSystemEditorData>(InSystem.GetEditorData(), ECastCheckedType::NullChecked);
	FNiagaraEmitterHandle EmitterHandle;
	if (SystemEditorData->GetOwningSystemIsPlaceholder() == false)
	{
		// FNiagaraUtilities::GetUniqueName will produce a unique emitter name (i.e. Empty -> Empty001 -> Empty002, etc)
		// but it will not produce a unique number so if that UObject is already taken by another object type we need to find a unique number
		// The number is not important to the stack / emitter name as it's not displayed but is important to avoid colliding between UNiagaraEmitter and UNiagaraStatelessEmitter types
		FName EmitterName = FNiagaraUtilities::GetUniqueName(InEmitterToAdd.GetFName(), EmitterHandleNames);
		while (FindObjectFast<UObject>(&InSystem, EmitterName))
		{
			EmitterName.SetNumber(EmitterName.GetNumber() + 1);
		}

		InSystem.Modify();
		EmitterHandle = InSystem.AddEmitterHandle(InEmitterToAdd, EmitterName, EmitterVersion);
	}
	else if (bCreateCopy)
	{
		// When editing an emitter asset we add the emitter as a duplicate so that the parent emitter is duplicated, but it's parent emitter
		// information is maintained.
		checkf(InSystem.GetNumEmitters() == 0, TEXT("Can not add multiple emitters to a system being edited in emitter asset mode."));
		FNiagaraEmitterHandle TemporaryEmitterHandle(InEmitterToAdd, EmitterVersion);
		EmitterHandle = InSystem.DuplicateEmitterHandle(TemporaryEmitterHandle, *InEmitterToAdd.GetUniqueEmitterName());
	}
	else
	{
		EmitterHandle = FNiagaraEmitterHandle(InEmitterToAdd, EmitterVersion);
		InSystem.AddEmitterHandleDirect(EmitterHandle);		
	}

	// We only want to use the default view state when actually adding an emitter to a system asset, not when opening up an emitter which adds said emitter to a transient system
	if (SystemEditorData->GetOwningSystemIsPlaceholder() == false)
	{
		FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
		UNiagaraEmitterEditorData* EmitterEditorData = EmitterData ? Cast<UNiagaraEmitterEditorData>(EmitterData->GetEditorData()) : nullptr;
		if (EmitterEditorData)
		{
			EmitterEditorData->SetShowSummaryView(InEmitterToAdd.GetEmitterData(EmitterVersion)->AddEmitterDefaultViewState == ENiagaraEmitterDefaultSummaryState::Summary ? true : false);
		}
	}
	
	FNiagaraStackGraphUtilities::RebuildEmitterNodes(InSystem);
	SystemEditorData->SynchronizeOverviewGraphWithSystem(InSystem);

	return EmitterHandle.GetId();
}

void FNiagaraEditorUtilities::RemoveEmittersFromSystemByEmitterHandleId(UNiagaraSystem& InSystem, TSet<FGuid> EmitterHandleIdsToDelete)
{
	// Kill all system instances before modifying the emitter handle list to prevent accessing deleted data.
	KillSystemInstances(InSystem);

	const FScopedTransaction DeleteTransaction(EmitterHandleIdsToDelete.Num() == 1
		? LOCTEXT("DeleteEmitter", "Delete emitter")
		: LOCTEXT("DeleteEmitters", "Delete emitters"));

	InSystem.Modify();
	InSystem.RemoveEmitterHandlesById(EmitterHandleIdsToDelete);

	FNiagaraScriptMergeManager::Get()->ClearMergeAdapterCache();
	FNiagaraStackGraphUtilities::RebuildEmitterNodes(InSystem);
	UNiagaraSystemEditorData* SystemEditorData = CastChecked<UNiagaraSystemEditorData>(InSystem.GetEditorData(), ECastCheckedType::NullChecked);
	SystemEditorData->SynchronizeOverviewGraphWithSystem(InSystem);
}

void FNiagaraEditorUtilities::KillSystemInstances(const UNiagaraSystem& System)
{
	TArray<UNiagaraComponent*> ReferencingComponents = FNiagaraEditorUtilities::GetComponentsThatReferenceSystem(System);
	for (auto Component : ReferencingComponents)
	{
		Component->DestroyInstance();
	}
}

bool FNiagaraEditorUtilities::VerifyNameChangeForInputOrOutputNode(const UNiagaraNode& NodeBeingChanged, FName OldName, FString NewNameString, FText& OutErrorMessage)
{
	if (NewNameString.Len() >= FNiagaraConstants::MaxParameterLength)
	{
		OutErrorMessage = FText::FormatOrdered(LOCTEXT("InputOutputNodeNameLengthExceeded", "Name cannot exceed {0} characters."), FNiagaraConstants::MaxParameterLength);
		return false;
	}

	FName NewName = *NewNameString;

	if (NewName == NAME_None)
	{
		OutErrorMessage = LOCTEXT("InputOutputNodeNameEmptyNameError", "Name can not be empty.");
		return false;
	}

	if (GetSystemConstantNames().Contains(NewName))
	{
		OutErrorMessage = LOCTEXT("SystemConstantNameError", "Name can not be the same as a system constant");
	}

	if (NodeBeingChanged.IsA<UNiagaraNodeInput>())
	{
		TArray<UNiagaraNodeInput*> InputNodes;
		NodeBeingChanged.GetGraph()->GetNodesOfClass<UNiagaraNodeInput>(InputNodes);
		for (UNiagaraNodeInput* InputNode : InputNodes)
		{
			if (InputNode->Input.GetName() != OldName && InputNode->Input.GetName() == NewName)
			{
				OutErrorMessage = LOCTEXT("DuplicateInputNameError", "Name can not match an existing input name.");
				return false;
			}
		}
	}

	if (NodeBeingChanged.IsA<UNiagaraNodeOutput>())
	{
		const UNiagaraNodeOutput* OutputNodeBeingChanged = CastChecked<const UNiagaraNodeOutput>(&NodeBeingChanged);
		for (const FNiagaraVariable& Output : OutputNodeBeingChanged->GetOutputs())
		{
			if (Output.GetName() != OldName && Output.GetName() == NewName)
			{
				OutErrorMessage = LOCTEXT("DuplicateOutputNameError", "Name can not match an existing output name.");
				return false;
			}
		}
	}

	return true;
}

bool FNiagaraEditorUtilities::AddParameter(FNiagaraVariable& NewParameterVariable, FNiagaraParameterStore& TargetParameterStore, UObject& ParameterStoreOwner, UNiagaraStackEditorData* StackEditorData)
{
	FScopedTransaction AddTransaction(LOCTEXT("AddParameter", "Add Parameter"));
	ParameterStoreOwner.Modify();

	TSet<FName> ExistingParameterStoreNames;
	TArray<FNiagaraVariable> ParameterStoreVariables;
	TargetParameterStore.GetParameters(ParameterStoreVariables);
	for (const FNiagaraVariable& Var : ParameterStoreVariables)
	{
		ExistingParameterStoreNames.Add(Var.GetName());
	}

	FNiagaraEditorUtilities::ResetVariableToDefaultValue(NewParameterVariable);
	NewParameterVariable.SetName(FNiagaraUtilities::GetUniqueName(NewParameterVariable.GetName(), ExistingParameterStoreNames));

	bool bSuccess = TargetParameterStore.AddParameter(NewParameterVariable);
	if (bSuccess && StackEditorData != nullptr)
	{
		StackEditorData->SetStackEntryIsRenamePending(NewParameterVariable.GetName().ToString(), true);
	}
	return bSuccess;
}

TArray<FNiagaraVariable> FNiagaraEditorUtilities::UserParameters::GetReferencedUserParametersFromEmitter(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TArray<FNiagaraVariable> UserParameters;
	
	TSharedRef<FNiagaraScriptGraphViewModel> GraphViewModel = EmitterViewModel->GetSharedScriptViewModel()->GetGraphViewModel();

	TArray<UNiagaraNodeParameterMapGet*> MapGetNodes;
	GraphViewModel->GetGraph()->GetNodesOfClass<UNiagaraNodeParameterMapGet>(MapGetNodes);

	// first we check for direct references on parameter map get nodes
	for(UNiagaraNodeParameterMapGet* MapGetNode : MapGetNodes)
	{
		TArray<UEdGraphPin*> OutputPins;
		MapGetNode->GetOutputPins(OutputPins);

		for(UEdGraphPin* Pin : OutputPins)
		{
			if(!Pin->bOrphanedPin && Pin->PinType.PinSubCategory == UNiagaraNodeParameterMapBase::ParameterPinSubCategory)
			{
				FNiagaraVariable Variable = UEdGraphSchema_Niagara::PinToNiagaraVariable(Pin);
				FNiagaraParameterHandle Handle(Variable.GetName());
				if(Handle.IsUserHandle())
				{
					UserParameters.AddUnique(Variable);
				}
			}			
		}
	}

	TArray<UNiagaraNodeInput*> InputNodes;
	GraphViewModel->GetGraph()->GetNodesOfClass<UNiagaraNodeInput>(InputNodes);

	// then we check for DI user parameter binding properties
	for(UNiagaraNodeInput* Input : InputNodes)
	{
		if(Input->GetDataInterface() != nullptr)
		{
			for(TFieldIterator<FStructProperty> It(Input->GetDataInterface()->GetClass()); It; ++It)
			{
				if((*It)->Struct == StaticStruct<FNiagaraUserParameterBinding>())
				{
					FNiagaraUserParameterBinding ParameterBinding;
					(*It)->GetValue_InContainer(Input->GetDataInterface(), &ParameterBinding);

					if(ParameterBinding.Parameter.IsValid())
					{
						FNiagaraParameterHandle ReferencedParameter(ParameterBinding.Parameter.GetName());
						if(ReferencedParameter.IsUserHandle())
						{
							UserParameters.AddUnique(ParameterBinding.Parameter);
						}
					}
				}
			}
		}
	}

	// then we check for user parameter bindings on renderers
	//-TODO:Stateless
	if (FVersionedNiagaraEmitterData* EmitterData = EmitterViewModel->GetEmitter().GetEmitterData())
	{
		const TArray<UNiagaraRendererProperties*>& Renderers = EmitterData->GetRenderers();
		for(UNiagaraRendererProperties* Renderer : Renderers)
		{
			for(TFieldIterator<FStructProperty> It(Renderer->GetClass()); It; ++It)
			{
				if((*It)->Struct == StaticStruct<FNiagaraUserParameterBinding>())
				{
					FNiagaraUserParameterBinding ParameterBinding;
					(*It)->GetValue_InContainer(Renderer, &ParameterBinding);

					if(ParameterBinding.Parameter.IsValid())
					{
						FNiagaraParameterHandle ReferencedParameter(ParameterBinding.Parameter.GetName());
						if(ReferencedParameter.IsUserHandle())
						{
							UserParameters.AddUnique(ParameterBinding.Parameter);
						}
					}
				}
			}
		}
	}

	return UserParameters;
}

TArray<UNiagaraNodeParameterMapGet*> FNiagaraEditorUtilities::UserParameters::GetParameterMapGetNodesWithUserParameter(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable UserParameter, const FNiagaraActionScopeConfig& Config)
{
	if(Config.SpecificEmitterGuid.IsSet())
	{
		ensureMsgf(Config.bIncludeEmitters, TEXT("If you are searching for a specific emitter, bIncludeEmitters needs to be true"));
	}
	
	TArray<UNiagaraNodeParameterMapGet*> Result;

	TArray<UEdGraph*> GraphsToInclude;
	if(Config.bIncludeSystem)
	{
		GraphsToInclude.Add(SystemViewModel->GetSystemScriptViewModel()->GetGraphViewModel()->GetGraph());
	}

	if(Config.bIncludeEmitters)
	{
		for(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
		{
			if(Config.SpecificEmitterGuid.IsSet() && EmitterHandleViewModel->GetId() != Config.SpecificEmitterGuid.GetValue())
			{
				continue;
			}
			
			GraphsToInclude.Add(EmitterHandleViewModel->GetEmitterViewModel()->GetSharedScriptViewModel()->GetGraphViewModel()->GetGraph());
		}
	}

	for(UEdGraph* Graph : GraphsToInclude)
	{
		TArray<UNiagaraNodeParameterMapGet*> MapGetNodes;
		Graph->GetNodesOfClass<UNiagaraNodeParameterMapGet>(MapGetNodes);

		for(UNiagaraNodeParameterMapGet* MapGetNode : MapGetNodes)
		{
			TArray<UEdGraphPin*> OutputPins;
			MapGetNode->GetOutputPins(OutputPins);

			for(UEdGraphPin* Pin : OutputPins)
			{
				if(!Pin->bOrphanedPin && Pin->PinType.PinSubCategory == UNiagaraNodeParameterMapBase::ParameterPinSubCategory)
				{
					FNiagaraVariable Variable = UEdGraphSchema_Niagara::PinToNiagaraVariable(Pin);
					FNiagaraParameterHandle Handle(Variable.GetName());
					if(Handle.IsUserHandle() && Variable == UserParameter)
					{
						Result.AddUnique(MapGetNode);
						break;
					}
				}			
			}
		}
	}
	
	return Result;
}

TArray<FNiagaraEditorUtilities::UserParameters::FUserParameterBindingInfo> FNiagaraEditorUtilities::UserParameters::GetUserParameterBindingsForUserParameter(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable UserParameter, const FNiagaraActionScopeConfig& Config)
{
	if(Config.SpecificEmitterGuid.IsSet())
	{
		ensureMsgf(Config.bIncludeEmitters, TEXT("If you are searching for a specific emitter, bIncludeEmitters needs to be true"));
	}
	
	TArray<FUserParameterBindingInfo> Bindings;

	if(Config.bIncludeSystem)
	{
		TSharedRef<FNiagaraScriptGraphViewModel> GraphViewModel = SystemViewModel->GetSystemScriptViewModel()->GetGraphViewModel();

		TArray<UNiagaraNodeInput*> InputNodes;
		GraphViewModel->GetGraph()->GetNodesOfClass<UNiagaraNodeInput>(InputNodes);

		for(UNiagaraNodeInput* Input : InputNodes)
		{
			if(Input->GetDataInterface() != nullptr)
			{
				for(TFieldIterator<FStructProperty> It(Input->GetDataInterface()->GetClass()); It; ++It)
				{
					if((*It)->Struct == StaticStruct<FNiagaraUserParameterBinding>())
					{					
						FNiagaraUserParameterBinding* ParameterBinding = (FNiagaraUserParameterBinding*) (*It)->ContainerPtrToValuePtr<void>(Input->GetDataInterface(), 0);

						if(ParameterBinding->Parameter.IsValid())
						{
							FNiagaraParameterHandle ReferencedParameter(ParameterBinding->Parameter.GetName());
							if(ReferencedParameter.IsUserHandle() && ParameterBinding->Parameter == UserParameter)
							{
								Bindings.Emplace(ParameterBinding, Input->GetDataInterface());
							}
						}
					}
				}
			}
		}
	}

	if(Config.bIncludeEmitters)
	{
		for(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
		{
			if(Config.SpecificEmitterGuid.IsSet() && EmitterHandleViewModel->GetId() != Config.SpecificEmitterGuid.GetValue())
			{
				continue;
			}
			
			// First we gather the bindings on renderers of an emitter
			//-TODO:Stateless
			if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData())
			{
				const TArray<UNiagaraRendererProperties*>& Renderers = EmitterData->GetRenderers();
				for(UNiagaraRendererProperties* Renderer : Renderers)
				{
					for(TFieldIterator<FStructProperty> It(Renderer->GetClass()); It; ++It)
					{
						if((*It)->Struct == StaticStruct<FNiagaraUserParameterBinding>())
						{				
							FNiagaraUserParameterBinding* ParameterBinding = (FNiagaraUserParameterBinding*) (*It)->ContainerPtrToValuePtr<void>(Renderer, 0);
						
							if(ParameterBinding->Parameter.IsValid())
							{
								FNiagaraParameterHandle ReferencedParameter(ParameterBinding->Parameter.GetName());
								if(ReferencedParameter.IsUserHandle() && ParameterBinding->Parameter == UserParameter)
								{
									Bindings.Emplace(ParameterBinding, Renderer);
								}
							}
						}
					}
				}
			}

			// then we get the bindings on DIs which live on input nodes in the emitter graph
			TSharedRef<FNiagaraScriptGraphViewModel> GraphViewModel = EmitterHandleViewModel->GetEmitterViewModel()->GetSharedScriptViewModel()->GetGraphViewModel();

			TArray<UNiagaraNodeInput*> InputNodes;
			GraphViewModel->GetGraph()->GetNodesOfClass<UNiagaraNodeInput>(InputNodes);

			for(UNiagaraNodeInput* Input : InputNodes)
			{
				if(Input->GetDataInterface() != nullptr)
				{
					for(TFieldIterator<FStructProperty> It(Input->GetDataInterface()->GetClass()); It; ++It)
					{
						if((*It)->Struct == StaticStruct<FNiagaraUserParameterBinding>())
						{					
							FNiagaraUserParameterBinding* ParameterBinding = (FNiagaraUserParameterBinding*) (*It)->ContainerPtrToValuePtr<void>(Input->GetDataInterface(), 0);

							if(ParameterBinding->Parameter.IsValid())
							{
								FNiagaraParameterHandle ReferencedParameter(ParameterBinding->Parameter.GetName());
								if(ReferencedParameter.IsUserHandle() && ParameterBinding->Parameter == UserParameter)
								{
									Bindings.Emplace(ParameterBinding, Input->GetDataInterface());
								}
							}
						}
					}
				}
			}
		}
	}
	
	return Bindings;
}

TArray<UNiagaraStackFunctionInput*> FNiagaraEditorUtilities::UserParameters::GetStackFunctionInputsWithLinkedParameter(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable Parameter, const FNiagaraActionScopeConfig& Config)
{
	if(Config.SpecificEmitterGuid.IsSet())
	{
		ensureMsgf(Config.bIncludeEmitters, TEXT("If you are searching for a specific emitter, bIncludeEmitters needs to be true"));
	}
	
	TArray<UNiagaraStackFunctionInput*> Result;

	if(Config.bIncludeSystem)
	{
		SystemViewModel->GetSystemStackViewModel()->GetRootEntry()->GetUnfilteredChildrenOfType(Result, true);
	}
	
	if(Config.bIncludeEmitters)
	{
		//-TODO:Stateless
		for(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
		{
			if(Config.SpecificEmitterGuid.IsSet() && EmitterHandleViewModel->GetId() != Config.SpecificEmitterGuid.GetValue())
			{
				continue;
			}

			EmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry()->GetUnfilteredChildrenOfType(Result, true);
		}
	}

	return Result;
}

FNiagaraEditorUtilities::FNiagaraActionScopeConfig::FNiagaraActionScopeConfig(FGuid InSpecificEmitterGuid)
{
	ensureMsgf(InSpecificEmitterGuid.IsValid(), TEXT("Invalid emitter ID passed into ctor"));
	
	bIncludeSystem = false;
	bIncludeEmitters = true;
	SpecificEmitterGuid = InSpecificEmitterGuid;
}

TArray<FNiagaraEditorUtilities::FVariableAttributeBindingInfo> FNiagaraEditorUtilities::GetVariableAttributeBindingsForParameter(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable Parameter, const FNiagaraActionScopeConfig& Config)
{
	if(Config.SpecificEmitterGuid.IsSet())
	{
		ensureMsgf(Config.bIncludeEmitters, TEXT("If you are searching for a specific emitter, bIncludeEmitters needs to be true"));
	}
	
	TArray<FVariableAttributeBindingInfo> Result;

	if(Config.bIncludeSystem)
	{
		// No attribute bindings in systems at this point
	}
	if(Config.bIncludeEmitters)
	{
		//-TODO:Stateless
		for(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
		{
			if(Config.SpecificEmitterGuid.IsSet() && EmitterHandleViewModel->GetId() != Config.SpecificEmitterGuid.GetValue())
			{
				continue;
			}
			
			if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData())
			{
				const TArray<UNiagaraRendererProperties*>& Renderers = EmitterData->GetRenderers();
				for(UNiagaraRendererProperties* Renderer : Renderers)
				{
					for(TFieldIterator<FStructProperty> It(Renderer->GetClass()); It; ++It)
					{
						if((*It)->Struct == StaticStruct<FNiagaraVariableAttributeBinding>())
						{				
							FNiagaraVariableAttributeBinding* ParameterBinding = (FNiagaraVariableAttributeBinding*) (*It)->ContainerPtrToValuePtr<void>(Renderer, 0);
						
							if(ParameterBinding->GetName().IsEqual(Parameter.GetName()))
							{
								FNiagaraVariableAttributeBinding* DefaultParameterBinding = (FNiagaraVariableAttributeBinding*) (*It)->ContainerPtrToValuePtr<void>(Renderer->GetClass()->GetDefaultObject(), 0);
								Result.Emplace(ParameterBinding, DefaultParameterBinding, Renderer->GetCurrentSourceMode(), Renderer, EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter());
							}
						}
					}
				}
			}
		}
	}

	return Result;
}

TObjectPtr<UNiagaraScriptVariable> FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(const FNiagaraVariable& UserParameter, TSharedPtr<FNiagaraSystemViewModel> SystemViewModel)
{
	return Cast<UNiagaraSystemEditorData>(SystemViewModel->GetSystem().GetEditorData())->FindOrAddUserScriptVariable(UserParameter, SystemViewModel->GetSystem());
}

TObjectPtr<UNiagaraScriptVariable> FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(const FNiagaraVariable& UserParameter, UNiagaraSystem& System)
{
	return Cast<UNiagaraSystemEditorData>(System.GetEditorData())->FindOrAddUserScriptVariable(UserParameter, System);
}

const UNiagaraScriptVariable* FNiagaraEditorUtilities::UserParameters::FindScriptVariableForUserParameter(const FGuid& UserParameterGuid, const UNiagaraSystem& System)
{
	return Cast<UNiagaraSystemEditorData>(System.GetEditorData())->FindUserScriptVariable(UserParameterGuid);
}

 const UNiagaraScriptVariable* FNiagaraEditorUtilities::UserParameters::FindScriptVariableForUserParameter(const FNiagaraVariable& UserParameter, const UNiagaraSystem& System)
 {
	return Cast<UNiagaraSystemEditorData>(System.GetEditorData())->FindUserScriptVariable(UserParameter, System);
 }

void FNiagaraEditorUtilities::UserParameters::ReplaceUserParameterReferences(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable OldUserParameter, FNiagaraVariable NewUserParameter, const FNiagaraActionScopeConfig& Config)
{
	using namespace FNiagaraEditorUtilities::UserParameters;
	
	for(const FUserParameterBindingInfo& UserParameterBindingInfo : GetUserParameterBindingsForUserParameter(SystemViewModel, OldUserParameter, Config))
	{
		if(UserParameterBindingInfo.Owner)
		{
			UserParameterBindingInfo.Owner->Modify();
		}
		UserParameterBindingInfo.Binding->Parameter = NewUserParameter;
	}

	for(const FVariableAttributeBindingInfo& AttributeBindingInfo : GetVariableAttributeBindingsForParameter(SystemViewModel, OldUserParameter, Config))
	{
		if(AttributeBindingInfo.Owner)
		{
			AttributeBindingInfo.Owner->Modify();
		}
		AttributeBindingInfo.Binding->SetValue(NewUserParameter.GetName(), AttributeBindingInfo.OwningEmitter.ToBase(), AttributeBindingInfo.SourceDataMode);
	}
	
	for(UNiagaraNodeParameterMapGet* MapGet : GetParameterMapGetNodesWithUserParameter(SystemViewModel, OldUserParameter, Config))
	{
		TArray<UEdGraphPin*> OutputPins;
		MapGet->GetOutputPins(OutputPins);

		for(UEdGraphPin* GraphPin : OutputPins)
		{
			if(GraphPin->PinType.PinSubCategory == UNiagaraNodeParameterMapBase::ParameterPinSubCategory && GraphPin->bOrphanedPin == false)
			{
				FNiagaraVariable RepresentingVariable = UEdGraphSchema_Niagara::PinToNiagaraVariable(GraphPin);

				if(RepresentingVariable == OldUserParameter)
				{
					GraphPin->Modify();
					GraphPin->PinName = NewUserParameter.GetName();
				}
			}
		}
	}
}

void FNiagaraEditorUtilities::UserParameters::DeleteUserParameterReferences(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable UserParameterToDelete, const FNiagaraActionScopeConfig& Config)
{
	using namespace FNiagaraEditorUtilities::UserParameters;
	
	for(const FUserParameterBindingInfo& UserParameterBindingInfo : GetUserParameterBindingsForUserParameter(SystemViewModel, UserParameterToDelete, Config))
	{
		if(UserParameterBindingInfo.Owner)
		{
			UserParameterBindingInfo.Owner->Modify();
		}
		UserParameterBindingInfo.Binding->Parameter = FNiagaraVariable(UserParameterBindingInfo.Binding->Parameter.GetType(), NAME_None);
	}

	for(const auto& AttributeBindingInfo : GetVariableAttributeBindingsForParameter(SystemViewModel, UserParameterToDelete, Config))
	{
		if(AttributeBindingInfo.Owner)
		{
			AttributeBindingInfo.Owner->Modify();
		}

		if(AttributeBindingInfo.OwningEmitter.Emitter != nullptr)
		{
			AttributeBindingInfo.Binding->ResetToDefault(*AttributeBindingInfo.DefaultBinding, AttributeBindingInfo.OwningEmitter.ToBase(), AttributeBindingInfo.SourceDataMode);
		}
	}

	for(UNiagaraStackFunctionInput* LinkedStackFunctionInput : GetStackFunctionInputsWithLinkedParameter(SystemViewModel, UserParameterToDelete, Config))
	{
		if(LinkedStackFunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Linked && LinkedStackFunctionInput->GetLinkedParameterValue() == UserParameterToDelete)
		{
			LinkedStackFunctionInput->Reset();
				
			if(LinkedStackFunctionInput->GetInputType().IsDataInterface())
			{
				UNiagaraDataInterface* CurrentUserDataInterface = SystemViewModel->GetSystem().GetExposedParameters().GetDataInterface(UserParameterToDelete);
				if(CurrentUserDataInterface->GetClass() == LinkedStackFunctionInput->GetInputType().GetClass())
				{
					LinkedStackFunctionInput->SetDataInterfaceValue(CurrentUserDataInterface->GetClass());
				}
			}
			else if (LinkedStackFunctionInput->GetInputType().IsUObject() == false)
			{
				const UScriptStruct* LocalValueStruct = LinkedStackFunctionInput->GetInputType().GetScriptStruct();
				if (LocalValueStruct != nullptr)
				{
					TSharedRef<FStructOnScope> LocalValue = MakeShared<FStructOnScope>(LocalValueStruct);
						
					const uint8* Memory = SystemViewModel->GetSystem().GetExposedParameters().GetParameterData(UserParameterToDelete);
					if(Memory)
					{
						FMemory::Memcpy(LocalValue->GetStructMemory(), Memory, LocalValueStruct->GetStructureSize());
						LinkedStackFunctionInput->SetLocalValue(LocalValue);
					}
				}
			}
		}
	}
}

void FNiagaraEditorUtilities::ShowParentEmitterInContentBrowser(TSharedRef<FNiagaraEmitterViewModel> Emitter)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.Get().SyncBrowserToAssets(TArray<FAssetData> { FAssetData(Emitter->GetParentEmitter().Emitter) });
}

void FNiagaraEditorUtilities::OpenParentEmitterForEdit(TSharedRef<FNiagaraEmitterViewModel> Emitter)
{
	if (UNiagaraEmitter* ParentEmitter = Emitter->GetParentEmitter().Emitter)
	{
		ParentEmitter->VersionToOpenInEditor = Emitter->GetParentEmitter().Version;
		if (GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ParentEmitter))
		{
			if (FNiagaraSystemToolkit* EditorInstance = static_cast<FNiagaraSystemToolkit*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(ParentEmitter, true)))
			{
				EditorInstance->SwitchToVersion(ParentEmitter->VersionToOpenInEditor);
			}			
		}
	}
}

ECheckBoxState FNiagaraEditorUtilities::GetSelectedEmittersEnabledCheckState(TSharedRef<FNiagaraSystemViewModel> SystemViewModel)
{
	bool bFirst = true;
	ECheckBoxState CurrentState = ECheckBoxState::Undetermined;

	const TArray<FGuid>& SelectedHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandle : SystemViewModel->GetEmitterHandleViewModels())
	{
		if (SelectedHandleIds.Contains(EmitterHandle->GetId()))
		{
			ECheckBoxState EmitterState = EmitterHandle->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			if (bFirst)
			{
				CurrentState = EmitterState;
				bFirst = false;
				continue;
			}

			if (CurrentState != EmitterState)
			{
				return ECheckBoxState::Undetermined;
			}
		}
	}

	return CurrentState;
}

void FNiagaraEditorUtilities::ToggleSelectedEmittersEnabled(TSharedRef<FNiagaraSystemViewModel> SystemViewModel)
{
	bool bEnabled = GetSelectedEmittersEnabledCheckState(SystemViewModel) != ECheckBoxState::Checked;

	const TArray<FGuid>& SelectedHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	for (const FGuid& HandleId : SelectedHandleIds)
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = SystemViewModel->GetEmitterHandleViewModelById(HandleId);
		if (EmitterHandleViewModel.IsValid())
		{
			EmitterHandleViewModel->SetIsEnabled(bEnabled);
		}
	}
}

ECheckBoxState FNiagaraEditorUtilities::GetSelectedEmittersIsolatedCheckState(TSharedRef<FNiagaraSystemViewModel> SystemViewModel)
{
	bool bFirst = true;
	ECheckBoxState CurrentState = ECheckBoxState::Undetermined;

	const TArray<FGuid>& SelectedHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandle : SystemViewModel->GetEmitterHandleViewModels())
	{
		if (SelectedHandleIds.Contains(EmitterHandle->GetId()))
		{
			ECheckBoxState EmitterState = EmitterHandle->GetIsIsolated() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			if (bFirst)
			{
				CurrentState = EmitterState;
				bFirst = false;
				continue;
			}

			if (CurrentState != EmitterState)
			{
				return ECheckBoxState::Undetermined;
			}
		}
	}

	return CurrentState;
}

void FNiagaraEditorUtilities::ToggleSelectedEmittersIsolated(TSharedRef<FNiagaraSystemViewModel> SystemViewModel)
{
	bool bIsolated = GetSelectedEmittersIsolatedCheckState(SystemViewModel) != ECheckBoxState::Checked;

	TArray<FGuid> EmittersToIsolate;
	for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandle : SystemViewModel->GetEmitterHandleViewModels())
	{
		if (EmitterHandle->GetIsIsolated())
		{
			EmittersToIsolate.Add(EmitterHandle->GetId());
		}
	}

	const TArray<FGuid>& SelectedHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	for (const FGuid& HandleId : SelectedHandleIds)
	{
		if (bIsolated)
		{
			EmittersToIsolate.Add(HandleId);
		}
		else
		{
			EmittersToIsolate.Remove(HandleId);
		}
	}

	SystemViewModel->IsolateEmitters(EmittersToIsolate);
}

void FNiagaraEditorUtilities::CreateAssetFromEmitter(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	TSharedRef<FNiagaraSystemViewModel> SystemViewModel = EmitterHandleViewModel->GetOwningSystemViewModel();
	if (SystemViewModel->GetEditMode() != ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		return;
	}

	UNiagaraEmitter* EmitterToCopy = EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter().Emitter;
	const FString PackagePath = FPackageName::GetLongPackagePath(EmitterToCopy->GetOutermost()->GetName());
	const FName EmitterName = EmitterToCopy->GetFName();
	
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	
	// First duplicate the asset so that fixes can be made on the duplicate without modifying the system and before it's saved.
	UNiagaraEmitter* DuplicateEmitter = CastChecked<UNiagaraEmitter>(StaticDuplicateObject(EmitterToCopy, GetTransientPackage()));
	DuplicateEmitter->AddToRoot(); // needed because the asset save dialog can call garbage collection
	for (const FNiagaraAssetVersion& Version : DuplicateEmitter->GetAllAvailableVersions())
	{
		FNiagaraScratchPadUtilities::FixExternalScratchPadScriptsForEmitter(SystemViewModel->GetSystem(), FVersionedNiagaraEmitter(DuplicateEmitter, Version.VersionGuid));
	}

	// Save the duplicated emitter.
	UNiagaraEmitter* CreatedAsset = Cast<UNiagaraEmitter>(AssetToolsModule.Get().DuplicateAssetWithDialogAndTitle(EmitterName.GetPlainNameString(), PackagePath, DuplicateEmitter, LOCTEXT("CreateEmitterAssetDialogTitle", "Create Emitter As")));
	DuplicateEmitter->RemoveFromRoot();
	if (CreatedAsset != nullptr)
	{
		CreatedAsset->SetUniqueEmitterName(CreatedAsset->GetName());
		CreatedAsset->bIsInheritable = true;

		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CreatedAsset);

		// find the existing overview node to store the position
		UEdGraph* OverviewGraph = SystemViewModel->GetOverviewGraphViewModel()->GetGraph();
		
		TArray<UNiagaraOverviewNode*> OverviewNodes;
		OverviewGraph->GetNodesOfClass<UNiagaraOverviewNode>(OverviewNodes);

		const UNiagaraOverviewNode* CurrentNode = *OverviewNodes.FindByPredicate(
			[Guid = EmitterHandleViewModel->GetId()](const UNiagaraOverviewNode* Node)
			{
				return Node->GetEmitterHandleGuid() == Guid;
			});

		int32 CurrentX = CurrentNode->NodePosX;
		int32 CurrentY = CurrentNode->NodePosY;
		FString CurrentComment = CurrentNode->NodeComment;
		bool bCommentBubbleVisible = CurrentNode->bCommentBubbleVisible;
		bool bCommentBubblePinned = CurrentNode->bCommentBubblePinned;

		CurrentNode = nullptr;

		FScopedTransaction ScopedTransaction(LOCTEXT("CreateAssetFromEmitter", "Create asset from emitter"));
		SystemViewModel->GetSystem().Modify();

		// Replace existing emitter
		SystemViewModel->DeleteEmitters(TSet<FGuid> { EmitterHandleViewModel->GetId() });
		TSharedPtr<FNiagaraEmitterHandleViewModel> NewEmitterHandleViewModel = SystemViewModel->AddEmitter(*CreatedAsset, CreatedAsset->GetExposedVersion().VersionGuid);

		NewEmitterHandleViewModel->SetName(EmitterName);

		OverviewGraph->GetNodesOfClass<UNiagaraOverviewNode>(OverviewNodes);

		UNiagaraOverviewNode* NewNode = *OverviewNodes.FindByPredicate(
			[Guid = NewEmitterHandleViewModel->GetId()](const UNiagaraOverviewNode* Node)
			{
				return Node->GetEmitterHandleGuid() == Guid;
			});

		NewNode->NodePosX = CurrentX;
		NewNode->NodePosY = CurrentY;
		NewNode->NodeComment = CurrentComment;
		NewNode->bCommentBubbleVisible = bCommentBubbleVisible;
		NewNode->bCommentBubblePinned = bCommentBubblePinned;

		// the duplicate action will select the duplicate emitter, adding a reference to it. We deselect here as it's a transient emitter to be garbage collected.
		GEditor->GetSelectedObjects()->Deselect(DuplicateEmitter);
	}
}

bool FNiagaraEditorUtilities::AddEmitterContextMenuActions(FMenuBuilder& MenuBuilder, const TSharedPtr<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModelPtr)
{
	if (EmitterHandleViewModelPtr.IsValid())
	{
		TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelPtr.ToSharedRef();
		TSharedRef<FNiagaraSystemViewModel> OwningSystemViewModel = EmitterHandleViewModel->GetOwningSystemViewModel();

		bool bSingleSelection = OwningSystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds().Num() == 1;
		TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel = EmitterHandleViewModel->GetEmitterViewModel();
		MenuBuilder.BeginSection("EmitterActions", LOCTEXT("EmitterActions", "Emitter Actions"));
		{
		
			if (OwningSystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("ToggleEmittersEnabled", "Enabled"),
					LOCTEXT("ToggleEmittersEnabledToolTip", "Toggle whether or not the selected emitters are enabled."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FNiagaraEditorUtilities::ToggleSelectedEmittersEnabled, OwningSystemViewModel),
						FCanExecuteAction(),
						FGetActionCheckState::CreateStatic(&FNiagaraEditorUtilities::GetSelectedEmittersEnabledCheckState, OwningSystemViewModel)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("ToggleEmittersIsolated", "Isolated"),
					LOCTEXT("ToggleEmittersIsolatedToolTip", "Toggle whether or not the selected emitters are isolated."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FNiagaraEditorUtilities::ToggleSelectedEmittersIsolated, OwningSystemViewModel),
						FCanExecuteAction(),
						FGetActionCheckState::CreateStatic(&FNiagaraEditorUtilities::GetSelectedEmittersIsolatedCheckState, OwningSystemViewModel)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CreateAssetFromThisEmitter", "Create Asset From This"),
				LOCTEXT("CreateAssetFromThisEmitterToolTip", "Create an emitter asset from this emitter."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&FNiagaraEditorUtilities::CreateAssetFromEmitter, EmitterHandleViewModel),
					FCanExecuteAction::CreateLambda(
						[bSingleSelection, EmitterHandleViewModel]()
						{
							return bSingleSelection && EmitterHandleViewModel->GetOwningSystemEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset;
						}
					)
				)
			);
		}

		MenuBuilder.EndSection();
		MenuBuilder.BeginSection("EmitterActions", LOCTEXT("ParentActions", "Parent Actions"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("UpdateParentEmitter", "Set New Parent Emitter"),
				LOCTEXT("UpdateParentEmitterToolTip", "Change or add a parent emitter."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(EmitterViewModel, &FNiagaraEmitterViewModel::CreateNewParentWindow, EmitterHandleViewModel),
					FCanExecuteAction::CreateLambda(
						[bSingleSelection]()
						{
							return bSingleSelection;
						}
					)
				)
			);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveParentEmitter", "Remove Parent Emitter"),
			LOCTEXT("RemoveParentEmitterToolTip", "Remove this emitter's parent, preventing inheritance of any further changes."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(EmitterViewModel, &FNiagaraEmitterViewModel::RemoveParentEmitter),
				FCanExecuteAction::CreateLambda(
					[bSingleSelection, bHasParent = EmitterViewModel->HasParentEmitter()]()
					{
						return bSingleSelection && bHasParent;
					}
				)
			)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenParentEmitterForEdit", "Open Parent For Edit"),
			LOCTEXT("OpenParentEmitterForEditToolTip", "Open and Focus Parent Emitter."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FNiagaraEditorUtilities::OpenParentEmitterForEdit, EmitterViewModel),
				FCanExecuteAction::CreateLambda(
					[bSingleSelection, bHasParent = EmitterViewModel->HasParentEmitter()]()
		{
			return bSingleSelection && bHasParent;
		}
		)
			)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowParentEmitterInContentBrowser", "Show Parent in Content Browser"),
			LOCTEXT("ShowParentEmitterInContentBrowserToolTip", "Show the selected emitter's parent emitter in the Content Browser."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&FNiagaraEditorUtilities::ShowParentEmitterInContentBrowser, EmitterViewModel),
				FCanExecuteAction::CreateLambda(
					[bSingleSelection, bHasParent = EmitterViewModel->HasParentEmitter()]()
					{
						return bSingleSelection && bHasParent;
					}
				)
			)
		);

	
		}
		MenuBuilder.EndSection();

		return true;
	}

	return false;
}

void FNiagaraEditorUtilities::WarnWithToastAndLog(FText WarningMessage)
{
	FNotificationInfo WarningNotification(WarningMessage);
	WarningNotification.ExpireDuration = 5.0f;
	WarningNotification.bFireAndForget = true;
	WarningNotification.bUseLargeFont = false;
	WarningNotification.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
	FSlateNotificationManager::Get().AddNotification(WarningNotification);
	UE_LOGF(LogNiagaraEditor, Warning, "%ls", *WarningMessage.ToString());
}

void FNiagaraEditorUtilities::InfoWithToastAndLog(FText InfoMessage, float ToastDuration)
{
	FNotificationInfo WarningNotification(InfoMessage);
	WarningNotification.ExpireDuration = ToastDuration;
	WarningNotification.bFireAndForget = true;
	WarningNotification.bUseLargeFont = false;
	WarningNotification.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Note"));
	FSlateNotificationManager::Get().AddNotification(WarningNotification);
	UE_LOGF(LogNiagaraEditor, Log, "%ls", *InfoMessage.ToString());
}

FName FNiagaraEditorUtilities::GetUniqueObjectName(UObject* Outer, UClass* ObjectClass, const FString& CandidateName)
{
	if (StaticFindObject(ObjectClass, Outer, *CandidateName) == nullptr)
	{
		return *CandidateName;
	}
	else
	{
		FString BaseCandidateName;
		int32 LastUnderscoreIndex;
		int32 NameIndex = 0;
		if (CandidateName.FindLastChar('_', LastUnderscoreIndex) && LexTryParseString(NameIndex, *CandidateName.RightChop(LastUnderscoreIndex + 1)))
		{
			BaseCandidateName = CandidateName.Left(LastUnderscoreIndex);
			NameIndex++;
		}
		else
		{
			BaseCandidateName = CandidateName;
			NameIndex = 1;
		}

		FString UniqueCandidateName = FString::Printf(TEXT("%s_%02i"), *BaseCandidateName, NameIndex);
		while (StaticFindObject(ObjectClass, Outer, *UniqueCandidateName) != nullptr)
		{
			NameIndex++;
			UniqueCandidateName = FString::Printf(TEXT("%s_%02i"), *BaseCandidateName, NameIndex);
		}
		return *UniqueCandidateName;
	}
}

TArray<FName> FNiagaraEditorUtilities::DecomposeVariableNamespace(const FName& InVarNameToken, FName& OutName)
{
	int32 DotIndex;
	TArray<FName> OutNamespaces;
	FString VarNameString = InVarNameToken.ToString();
	while (VarNameString.FindChar(TEXT('.'), DotIndex))
	{
		OutNamespaces.Add(FName(*VarNameString.Left(DotIndex)));
		VarNameString = *VarNameString.RightChop(DotIndex + 1);
	}
	OutName = FName(*VarNameString);
	return OutNamespaces;
}

void  FNiagaraEditorUtilities::RecomposeVariableNamespace(const FName& InVarNameToken, const TArray<FName>& InParentNamespaces, FName& OutName)
{
	FString VarNameString;
	for (FName Name : InParentNamespaces)
	{
		VarNameString += Name.ToString() + TEXT(".");
	}
	VarNameString += InVarNameToken.ToString();
	OutName = FName(*VarNameString);
}

FString FNiagaraEditorUtilities::GetNamespacelessVariableNameString(const FName& InVarName)
{
	int32 DotIndex;
	FString VarNameString = InVarName.ToString();
	if (VarNameString.FindLastChar(TEXT('.'), DotIndex))
	{
		return VarNameString.RightChop(DotIndex + 1);
	}
	// No dot index, must be a namespaceless variable name (e.g. static switch name) just return the name to string.
	return VarNameString;
}

void FNiagaraEditorUtilities::GetReferencingFunctionCallNodes(UNiagaraScript* Script, TArray<UNiagaraNodeFunctionCall*>& OutReferencingFunctionCallNodes)
{
	for (TObjectIterator<UNiagaraNodeFunctionCall> It; It; ++It)
	{
		UNiagaraNodeFunctionCall* FunctionCallNode = *It;
		if (FunctionCallNode->FunctionScript == Script)
		{
			OutReferencingFunctionCallNodes.Add(FunctionCallNode);
		}
	}
}

bool FNiagaraEditorUtilities::GetVariableSortPriority(const FName& VarNameA, const FName& VarNameB)
{
	const FNiagaraNamespaceMetadata& NamespaceMetaDataA = GetNamespaceMetaDataForVariableName(VarNameA);
	if (NamespaceMetaDataA.IsValid() == false)
	{
		return false;
	}

	const FNiagaraNamespaceMetadata& NamespaceMetaDataB = GetNamespaceMetaDataForVariableName(VarNameB);
	const int32 NamespaceAPriority = GetNamespaceMetaDataSortPriority(NamespaceMetaDataA, NamespaceMetaDataB);
	if (NamespaceAPriority == 0)
	{
		return VarNameA.LexicalLess(VarNameB);
	}
	return NamespaceAPriority > 0;
}

int32 FNiagaraEditorUtilities::GetNamespaceMetaDataSortPriority(const FNiagaraNamespaceMetadata& A, const FNiagaraNamespaceMetadata& B)
{
	if (A.IsValid() == false)
	{
		return false;
	}
	else if (B.IsValid() == false)
	{
		return true;
	}

	const int32 ANum = A.Namespaces.Num();
	const int32 BNum = B.Namespaces.Num();
	for (int32 i = 0; i < FMath::Min(ANum, BNum); ++i)
	{
		const int32 ANamespacePriority = GetNamespaceSortPriority(A.Namespaces[i]);
		const int32 BNamespacePriority = GetNamespaceSortPriority(B.Namespaces[i]);
		if (ANamespacePriority != BNamespacePriority)
			return ANamespacePriority < BNamespacePriority ? 1 : -1;
	}
	if (ANum == BNum)
		return 0;

	return ANum < BNum ? 1 : -1;
}

int32 FNiagaraEditorUtilities::GetNamespaceSortPriority(const FName& Namespace)
{
	if (Namespace == FNiagaraConstants::UserNamespace)
		return 0;
	else if (Namespace == FNiagaraConstants::ModuleNamespace)
		return 1;
	else if (Namespace == FNiagaraConstants::StaticSwitchNamespace)
		return 2;
	else if (Namespace == FNiagaraConstants::DataInstanceNamespace)
		return 3;
	else if (Namespace == FNiagaraConstants::OutputNamespace)
		return 4;
	else if (Namespace == FNiagaraConstants::EngineNamespace)
		return 5;
	else if (Namespace == FNiagaraConstants::ParameterCollectionNamespace)
		return 6;
	else if (Namespace == FNiagaraConstants::SystemNamespace)
		return 7;
	else if (Namespace == FNiagaraConstants::EmitterNamespace)
		return 8;
	else if (Namespace == FNiagaraConstants::ParticleAttributeNamespace)
		return 9;
	else if (Namespace == FNiagaraConstants::TransientNamespace)
		return 10;

	return 11;
}

const FNiagaraNamespaceMetadata FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(const FName& VarName)
{
	const FNiagaraParameterHandle VarHandle = FNiagaraParameterHandle(VarName);
	const TArray<FName> VarHandleNameParts = VarHandle.GetHandleParts();
	return GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(VarHandleNameParts);
}

const FNiagaraNamespaceMetadata FNiagaraEditorUtilities::GetNamespaceMetaDataForId(const FGuid& NamespaceId)
{
	return GetDefault<UNiagaraEditorSettings>()->GetMetaDataForId(NamespaceId);
}

const FGuid& FNiagaraEditorUtilities::GetNamespaceIdForUsage(ENiagaraScriptUsage Usage)
{
	return GetDefault<UNiagaraEditorSettings>()->GetIdForUsage(Usage);
}

TArray<UNiagaraParameterDefinitions*> FNiagaraEditorUtilities::GetAllParameterDefinitions()
{
	TArray<UNiagaraParameterDefinitions*> OutParameterDefinitions;

	TArray<FAssetData> ParameterDefinitionsAssetData;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.GetRegistry().GetAssetsByClass(UNiagaraParameterDefinitions::StaticClass()->GetClassPathName(), ParameterDefinitionsAssetData);
	for (const FAssetData& ParameterDefinitionsAssetDatum : ParameterDefinitionsAssetData)
	{
		UNiagaraParameterDefinitions* ParameterDefinitions = Cast<UNiagaraParameterDefinitions>(ParameterDefinitionsAssetDatum.GetAsset());
		if (ParameterDefinitions == nullptr)
		{
			ensureMsgf(false, TEXT("Failed to load parameter definition from asset registry!"));
			continue;
		}
		OutParameterDefinitions.Add(ParameterDefinitions);
	}
	return OutParameterDefinitions;
}

bool FNiagaraEditorUtilities::GetAvailableParameterDefinitions(const TArray<FString>& ExternalPackagePaths, TArray<FAssetData>& OutParameterDefinitionsAssetData)
{
	//@todo(ng) consider linking to out of package libraries if necessary for use with content plugins

	// Gather asset registry filter args
	FARFilter ARFilter;
	ARFilter.ClassPaths.Add(UNiagaraParameterDefinitions::StaticClass()->GetClassPathName());
	ARFilter.bRecursivePaths = true;
	ARFilter.bIncludeOnlyOnDiskAssets = true;

	// Always get parameter libraries from the Niagara plugin content directory
	TSharedPtr<IPlugin> NiagaraPlugin = IPluginManager::Get().FindPlugin("Niagara");
	checkf(NiagaraPlugin, TEXT("Failed to find the Niagara plugin! Did we rename it!?"));
	FString NiagaraPluginContentMountPoint = NiagaraPlugin->GetMountedAssetPath();
	NiagaraPluginContentMountPoint.RemoveFromEnd(TEXT("/"), ESearchCase::CaseSensitive);
	ARFilter.PackagePaths.Add(*NiagaraPluginContentMountPoint);

	// Always get parameter libraries in the project content as well
	ARFilter.PackagePaths.Add(TEXT("/Game"));

	// Add external package paths
	ARFilter.PackagePaths.Append(ExternalPackagePaths);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	return AssetRegistryModule.GetRegistry().GetAssets(ARFilter, OutParameterDefinitionsAssetData);
}

void FNiagaraEditorUtilities::GetAvailableParameterCollections(TArray<UNiagaraParameterCollection*>& OutParameterCollections)
{
	const TArray<TWeakObjectPtr<UNiagaraParameterCollection>>& CachedParameterCollectionAssets = FNiagaraEditorModule::Get().GetCachedParameterCollectionAssets();

	for (const TWeakObjectPtr<UNiagaraParameterCollection>& CachedParameterCollectionAsset : CachedParameterCollectionAssets)
	{
		if (UNiagaraParameterCollection* ParameterCollection = CachedParameterCollectionAsset.Get())
		{
			OutParameterCollections.Add(ParameterCollection);
		}
	}
}

void FNiagaraEditorUtilities::GetAvailableParameterDefinitions(TArray<UNiagaraParameterDefinitions*>& OutParameterDefinitions)
{
	const TArray<TWeakObjectPtr<UNiagaraParameterDefinitions>>& CachedParameterDefinitionAssets = FNiagaraEditorModule::Get().GetCachedParameterDefinitionsAssets();

	for (const TWeakObjectPtr<UNiagaraParameterDefinitions>& CachedParameterDefinitionAsset : CachedParameterDefinitionAssets)
	{
		if (UNiagaraParameterDefinitions* ParameterDefinition = CachedParameterDefinitionAsset.Get())
		{
			OutParameterDefinitions.Add(ParameterDefinition);
		}
	}
}

TSharedPtr<INiagaraParameterDefinitionsSubscriberViewModel> FNiagaraEditorUtilities::GetOwningLibrarySubscriberViewModelForGraph(const UNiagaraGraph* Graph)
{
	if (UNiagaraScript* BaseScript = Graph->GetTypedOuter<UNiagaraScript>())
	{
		TSharedPtr<FNiagaraScriptViewModel> BaseScriptViewModel = TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>::GetExistingViewModelForObject(BaseScript);
		if (ensureMsgf(BaseScriptViewModel.IsValid(), TEXT("Failed to find valid script view model for outer script of variable being customized!")))
		{
			return BaseScriptViewModel;
		}
	}
	else if (UNiagaraEmitter* BaseEmitter = Graph->GetTypedOuter<UNiagaraEmitter>())
	{
		TSharedPtr<FNiagaraEmitterViewModel> BaseEmitterViewModel = TNiagaraViewModelManager<UNiagaraEmitter, FNiagaraEmitterViewModel>::GetExistingViewModelForObject(BaseEmitter);
		if (ensureMsgf(BaseEmitterViewModel.IsValid(), TEXT("Failed to find valid emitter view model for outer emitter of variable being customized!")))
		{
			return BaseEmitterViewModel;
		}
	}
	else if (UNiagaraSystem* BaseSystem = Graph->GetTypedOuter<UNiagaraSystem>())
	{
		TSharedPtr<FNiagaraSystemViewModel> BaseSystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(BaseSystem);
		if (ensureMsgf(BaseSystemViewModel.IsValid(), TEXT("Failed to find valid script view model for outer script of variable being customized!")))
		{
			return BaseSystemViewModel;
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Tried to find valid outer to Graph but could not find outered Script, Emitter or System!"));
	}
	return TSharedPtr<INiagaraParameterDefinitionsSubscriberViewModel>();
}

TArray<UNiagaraParameterDefinitions*> FNiagaraEditorUtilities::DowncastParameterDefinitionsBaseArray(const TArray<UNiagaraParameterDefinitionsBase*> BaseArray)
{
	TArray<UNiagaraParameterDefinitions*> OutArray;
	OutArray.AddUninitialized(BaseArray.Num());
	for (int32 i = 0; i < BaseArray.Num(); ++i)
	{
		OutArray[i] = Cast<UNiagaraParameterDefinitions>(BaseArray[i]);
	}
	return OutArray;
}

void FNiagaraEditorUtilities::RefreshAllScriptsFromExternalChanges(FRefreshAllScriptsFromExternalChangesArgs Args)
{
	UNiagaraScript* OriginatingScript = Args.OriginatingScript;
	UNiagaraGraph* OriginatingGraph = Args.OriginatingGraph;
	UNiagaraParameterDefinitions* OriginatingParameterDefinitions = Args.OriginatingParameterDefinitions;
	bool bMatchOriginatingScript = OriginatingScript != nullptr;
	bool bMatchOriginatingGraph = OriginatingGraph != nullptr;
	bool bMatchOriginatingParameterDefinitions = OriginatingParameterDefinitions != nullptr;
	TArray<UNiagaraScript*> AffectedScripts;

	for (TObjectIterator<UNiagaraScript> It; It; ++It)
	{
		if (*It == OriginatingScript || !IsValidChecked(*It))
		{
			continue;
		}

		// First see if it is directly called, as this will force a need to refresh from external changes...
		UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(It->GetLatestSource());
		if (!Source || !Source->NodeGraph)
		{
			continue;
		}

		bool bMatchedOriginatingParameterDefinitions = false;
		if (bMatchOriginatingParameterDefinitions)
		{
			// Check if the script itself is subscribed to the originating parameter definitions, and if so, mark it as having matched.
			bool bSkipSystemEmitterCheck = false;
			if (const FVersionedNiagaraScriptData* VersionedScriptData = It->GetLatestScriptData())
			{
				FVersionedNiagaraScript TempVersionedScript = FVersionedNiagaraScript(*It, VersionedScriptData->Version.VersionGuid);
				if (TempVersionedScript.GetIsSubscribedToParameterDefinitions(OriginatingParameterDefinitions))
				{
					bSkipSystemEmitterCheck = true;
					TempVersionedScript.SynchronizeWithParameterDefinitions();
				}
			}

			// If the script itself is not subscribed to the originating parameter definitions, check if the script is a system/emitter/particle script,
			// and if so, check if the outer Emitter or System is subscribed to the originating parameter definitions. If so, mark the match.
			if (bSkipSystemEmitterCheck == false)
			{
				UNiagaraScript* Script = *It;
				if (Script->IsParticleScript() || Script->IsEmitterSpawnScript() || Script->IsEmitterUpdateScript())
				{
					UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Script->GetTypedOuter<UNiagaraEmitter>());
					if (Emitter && Emitter->GetIsSubscribedToParameterDefinitions(OriginatingParameterDefinitions))
					{
						Emitter->SynchronizeWithParameterDefinitions();
						bMatchedOriginatingParameterDefinitions = true;
					}
				}
				else if (Script->IsSystemSpawnScript() || Script->IsSystemUpdateScript())
				{
					UNiagaraSystem* System = Cast<UNiagaraSystem>(Script->GetTypedOuter<UNiagaraSystem>());
					if(System && System->GetIsSubscribedToParameterDefinitions(OriginatingParameterDefinitions))
					{
						System->SynchronizeWithParameterDefinitions();
						bMatchedOriginatingParameterDefinitions = true;
					}
				}
			}
		}

		// Iterate over each node in the script
		TArray<UNiagaraNode*> NiagaraNodes;
		Source->NodeGraph->GetNodesOfClass<UNiagaraNode>(NiagaraNodes);
		bool bRefreshed = false;
		for (UNiagaraNode* NiagaraNode : NiagaraNodes)
		{
			// If the node references the originating script, refresh. Also, refresh if the originating parameter definitions were matched prior.
			UObject* ReferencedAsset = NiagaraNode->GetReferencedAsset();
			if ((bMatchOriginatingScript && ReferencedAsset == OriginatingScript) ||
				(bMatchedOriginatingParameterDefinitions))
			{
				NiagaraNode->RefreshFromExternalChanges();
				bRefreshed = true;
			}
			// If originating parameter definitions were not subscribed to the script or its potential outer emitter or system, check each node for a 
			// function call node and check whether the function call associated script is subscribed to the originating definition. If so, refresh.
			else if (bMatchOriginatingParameterDefinitions && ReferencedAsset != nullptr && NiagaraNode->IsA<UNiagaraNodeFunctionCall>())
			{
				UNiagaraScript* FunctionCallScript = CastChecked<UNiagaraScript>(ReferencedAsset);
				FVersionedNiagaraScript VersionedNiagaraScriptAdapter = FVersionedNiagaraScript(FunctionCallScript, FunctionCallScript->GetExposedVersion().VersionGuid);
				if (VersionedNiagaraScriptAdapter.GetIsSubscribedToParameterDefinitions(OriginatingParameterDefinitions))
				{
					NiagaraNode->RefreshFromExternalChanges();
					bRefreshed = true;
				}
			}
		}

		if (bRefreshed)
		{
			//Source->NodeGraph->NotifyGraphNeedsRecompile();
			AffectedScripts.AddUnique(*It);
		}
		else
		{
			// Now check to see if our graph is anywhere in the dependency chain for a given graph. If it is, 
			// then it will need to be recompiled against the latest version.
			TArray<const UNiagaraGraph*> ReferencedGraphs;
			Source->NodeGraph->GetAllReferencedGraphs(ReferencedGraphs);
			for (const UNiagaraGraph* Graph : ReferencedGraphs)
			{
				if (bMatchOriginatingGraph && Graph == OriginatingGraph)
				{
					//Source->NodeGraph->NotifyGraphNeedsRecompile();
					AffectedScripts.AddUnique(*It);
					break;
				}
			}
		}
	}

	// Now determine if any of these scripts were in Emitters. If so, those emitters should be compiled together. If not, go ahead and compile individually.
	// Use the existing view models if they exist,as they are already wired into the correct UI.
	TArray<FVersionedNiagaraEmitter> AffectedEmitters;
	for (UNiagaraScript* Script : AffectedScripts)
	{
		if (Script->IsParticleScript() || Script->IsEmitterSpawnScript() || Script->IsEmitterUpdateScript())
		{
			FVersionedNiagaraEmitter Outer = Script->GetOuterEmitter();
			if (Outer.Emitter)
			{
				AffectedEmitters.AddUnique(Outer);
			}
		}
		else if (Script->IsSystemSpawnScript() || Script->IsSystemUpdateScript())
		{
			if (UNiagaraSystem* System = Cast<UNiagaraSystem>(Script->GetTypedOuter<UNiagaraSystem>()))
			{
				for (int32 i = 0; i < System->GetNumEmitters(); i++)
				{
					FVersionedNiagaraEmitter VersionedEmitter = System->GetEmitterHandle(i).GetInstance();
					AffectedEmitters.AddUnique(VersionedEmitter);
					//TODO (mg) is the parent compile necessary?
					//AffectedEmitters.AddUnique(VersionedEmitter.Emitter->GetParent());
				}
			}
		}
		else
		{
			TSharedPtr<FNiagaraScriptViewModel> AffectedScriptViewModel = FNiagaraScriptViewModel::GetExistingViewModelForObject(Script);
			if (!AffectedScriptViewModel.IsValid())
			{
				bool bIsForDataProcessingOnly = true;
				AffectedScriptViewModel = MakeShareable(new FNiagaraScriptViewModel(FText::FromString(Script->GetName()), ENiagaraParameterEditMode::EditValueOnly, bIsForDataProcessingOnly));
				AffectedScriptViewModel->SetScript(FVersionedNiagaraScript(Script));
			}
			AffectedScriptViewModel->CompileStandaloneScript();
		}
	}

	FNiagaraEditorUtilities::CompileExistingEmitters(AffectedEmitters);
}

TArray<UNiagaraPythonScriptModuleInput*> GetFunctionCallInputs(UNiagaraClipboardContent* ClipboardContent)
{
	TArray<UNiagaraPythonScriptModuleInput*> ScriptInputs;

	TArray<const UNiagaraClipboardFunctionInput*> FunctionInputsWorkingSet = ClipboardContent->FunctionInputs;
	for(auto It = FunctionInputsWorkingSet.CreateIterator(); It; ++It)
	{
		UNiagaraPythonScriptModuleInput* ScriptInput = NewObject<UNiagaraPythonScriptModuleInput>();
		ScriptInput->Input = *It;
		ScriptInputs.Add(ScriptInput);

		FunctionInputsWorkingSet.Append((*It)->ChildrenInputs);
		It.RemoveCurrent();
	}
	
	return ScriptInputs;
}

TArray<UNiagaraPythonScriptModuleInput*> GetFunctionCallInputs(const FNiagaraScriptVersionUpgradeContext& UpgradeContext)
{
	UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
	UpgradeContext.CreateClipboardCallback(ClipboardContent);
	return GetFunctionCallInputs(ClipboardContent);
}

void AddStackWarning(const FText& ToAdd, bool& bLoggedWarning, FText& OutWarnings)
{
	if (!bLoggedWarning)
	{
		OutWarnings = LOCTEXT("NewWarningHeader", "Python conversion script:\n");
	}
	bLoggedWarning = true;
	OutWarnings = FText::Format(FText::FromString("{0}  * {1}\n"), OutWarnings, ToAdd);
}

void AddStackWarning(const FNiagaraAssetVersion& FromVersion, const FNiagaraAssetVersion& ToVersion, const FString& ToAdd, bool& bLoggedWarning, FString& OutWarnings)
{
	if (!bLoggedWarning)
	{
		OutWarnings.Appendf(TEXT("%i.%i -> %i.%i:\n"), FromVersion.MajorVersion, FromVersion.MinorVersion, ToVersion.MajorVersion, ToVersion.MinorVersion);
	}
	bLoggedWarning = true;
	OutWarnings.Appendf(TEXT("  * %s\n"), *ToAdd);
}

const FString PythonUpgradeScriptStub = TEXT(
	"import sys\n"
	"import unreal as ue\n"
	"upgrade_context = ue.load_object(None, '{0}')\n"
	"### User Upgrade Script ###\n"
	"{1}\n"
	"### End User Script ###\n"
	"upgrade_context.cancelled_by_python_error = False\n");

UNiagaraClipboardContent* FNiagaraEditorUtilities::RunPythonConversionScript(FVersionedNiagaraScriptData& NewScriptVersionData, UNiagaraClipboardContent* NewScriptInputs, FVersionedNiagaraScriptData& OldScriptVersionData, UNiagaraClipboardContent* OldScriptInputs, FText& OutWarnings)
{
	UUpgradeNiagaraScriptResults* Results = NewObject<UUpgradeNiagaraScriptResults>();
	
	FString PythonScript;
	if (OldScriptVersionData.ConversionScriptExecution == ENiagaraPythonUpdateScriptReference::DirectTextEntry)
	{
		PythonScript = OldScriptVersionData.PythonConversionScript;
	}
	else if (OldScriptVersionData.ConversionScriptExecution == ENiagaraPythonUpdateScriptReference::ScriptAsset && !OldScriptVersionData.ConversionScriptAsset.FilePath.IsEmpty())
	{
		FFileHelper::LoadFileToString(PythonScript, *OldScriptVersionData.ConversionScriptAsset.FilePath);
	}

	if (PythonScript.IsEmpty() || NewScriptInputs == nullptr ||OldScriptInputs == nullptr)
	{
		return nullptr;
	}

	// set up script context
	bool bLoggedWarning = false;
	Results->OldInputs = GetFunctionCallInputs(OldScriptInputs);
	Results->NewInputs = GetFunctionCallInputs(NewScriptInputs);
	Results->Init();
	
	// save python script to a temp file to execute
	FString TempScriptFile = FPaths::CreateTempFilename(*FPaths::ProjectIntermediateDir(), TEXT("ScriptConversion-"), TEXT(".py"));
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	ON_SCOPE_EXIT
	{
		// Delete temp script file
		if (GNiagaraDeletePythonFilesOnError || !Results->bCancelledByPythonError)
		{
			PlatformFile.DeleteFile(*TempScriptFile);
		}
	};
	if (!FFileHelper::SaveStringToFile(FString::Format(*PythonUpgradeScriptStub, {Results->GetPathName(), PythonScript}), *TempScriptFile))
	{
		UE_LOGF(LogNiagaraEditor, Error, "Unable to save python script to file %ls", *TempScriptFile);
		AddStackWarning(LOCTEXT("CreatePythonError", "Cannot create python script file!"), bLoggedWarning, OutWarnings);
		return nullptr;
	}

	Results->bCancelledByPythonError = true;
	FPythonCommandEx PythonCommand = FPythonCommandEx();
	PythonCommand.ExecutionMode = EPythonCommandExecutionMode::ExecuteFile;
	PythonCommand.Command = TempScriptFile;

	// execute python script
	IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand);

	UNiagaraClipboardContent* ResultClipboard = UNiagaraClipboardContent::Create();
	if (Results->bCancelledByPythonError)
	{
		UE_LOGF(LogNiagaraEditor, Error, "%ls\n\nPython script:\n%ls\nTo keep the intermediate script around, set fx.Niagara.DeletePythonFilesOnError to 0.", *PythonCommand.CommandResult, *PythonCommand.Command);
		AddStackWarning(LOCTEXT("PythonExecutionError", "Python script ended with error, check the log!"), bLoggedWarning, OutWarnings);
	}
	else
	{
		for (UNiagaraPythonScriptModuleInput* ModuleInput : Results->NewInputs)
		{
			ResultClipboard->FunctionInputs.Add(ModuleInput->Input);	
		}
	}
	if (PythonCommand.LogOutput.Num() > 0)
	{
		for (FPythonLogOutputEntry& Entry : PythonCommand.LogOutput)
		{
			AddStackWarning(FText::FromString(Entry.Output), bLoggedWarning, OutWarnings);
		}
	}
	if (ResultClipboard->FunctionInputs.Num() > 0)
	{
		return ResultClipboard;
	}
	return nullptr;
}

void FNiagaraEditorUtilities::RunPythonUpgradeScripts(UNiagaraNodeFunctionCall* SourceNode,	const TArray<FVersionedNiagaraScriptData*>& UpgradeVersionData, const FNiagaraScriptVersionUpgradeContext& UpgradeContext, FString& OutWarnings)
{
	UUpgradeNiagaraScriptResults* Results = NewObject<UUpgradeNiagaraScriptResults>();
	FGuid SavedVersion = SourceNode->SelectedScriptVersion;
	
	for (int i = 1; i < UpgradeVersionData.Num(); i++)
	{
		FVersionedNiagaraScriptData* PreviousData = UpgradeVersionData[i - 1];
		FVersionedNiagaraScriptData* NewData = UpgradeVersionData[i];
		if (NewData == nullptr || PreviousData == nullptr)
		{
			continue;
		}

		FString PythonScript;
		if (NewData->UpdateScriptExecution == ENiagaraPythonUpdateScriptReference::DirectTextEntry)
		{
			PythonScript = NewData->PythonUpdateScript;
		}
		else if (NewData->UpdateScriptExecution == ENiagaraPythonUpdateScriptReference::ScriptAsset && !NewData->ScriptAsset.FilePath.IsEmpty())
		{
			FFileHelper::LoadFileToString(PythonScript, *NewData->ScriptAsset.FilePath);
		}

		UNiagaraVersionUpgradeBase* VersionUpdateUtility = NewData->UpdateUtilityClass.GetDefaultObject();
		if (PythonScript.IsEmpty() && VersionUpdateUtility == nullptr)
		{
			continue;
		}

		bool bLoggedWarning = false;
		bool bIsPythonAvailable = IPythonScriptPlugin::Get()->IsPythonAvailable() && !PythonScript.IsEmpty();
		if (!bIsPythonAvailable && VersionUpdateUtility == nullptr)
		{
			UE_LOGF(LogNiagaraEditor, Log, "Python is not available update script skipped");
			AddStackWarning(PreviousData->Version, NewData->Version, "Python is not available, upgrade script ignored!", bLoggedWarning, OutWarnings);
			continue;
		}

		// set up script context
		if (SourceNode->SelectedScriptVersion != PreviousData->Version.VersionGuid)
		{
			SourceNode->SelectedScriptVersion = PreviousData->Version.VersionGuid;
			SourceNode->RefreshFromExternalChanges();
		}
		Results->OldInputs = GetFunctionCallInputs(UpgradeContext);
		SourceNode->SelectedScriptVersion = NewData->Version.VersionGuid;
		SourceNode->RefreshFromExternalChanges();
		Results->NewInputs = GetFunctionCallInputs(UpgradeContext);
		Results->Init();

		if (bIsPythonAvailable)
		{
			// save python script to a temp file to execute
			FString TempScriptFile = FPaths::CreateTempFilename(*FPaths::ProjectIntermediateDir(), TEXT("VersionUpgrade-"), TEXT(".py"));
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			ON_SCOPE_EXIT
			{
				// Delete temp script file
				if (GNiagaraDeletePythonFilesOnError || !Results->bCancelledByPythonError)
				{
					PlatformFile.DeleteFile(*TempScriptFile);
				}
			};
			if (!FFileHelper::SaveStringToFile(FString::Format(*PythonUpgradeScriptStub, {Results->GetPathName(), PythonScript}), *TempScriptFile))
			{
				UE_LOGF(LogNiagaraEditor, Error, "Unable to save python script to file %ls", *TempScriptFile);
				AddStackWarning(PreviousData->Version, NewData->Version, "Cannot create python script file!", bLoggedWarning, OutWarnings);
				continue;
			}

			Results->bCancelledByPythonError = true;
			FPythonCommandEx PythonCommand = FPythonCommandEx();
			PythonCommand.ExecutionMode = EPythonCommandExecutionMode::ExecuteFile;
			PythonCommand.Command = TempScriptFile;

			// execute python script
			IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand);

			if (Results->bCancelledByPythonError)
			{
				UE_LOGF(LogNiagaraEditor, Error, "%ls\n\nPython script:\n%ls\nTo keep the intermediate script around, set fx.Niagara.DeletePythonFilesOnError to 0.", *PythonCommand.CommandResult, *PythonCommand.Command);
				AddStackWarning(PreviousData->Version, NewData->Version, "Python script ended with error!", bLoggedWarning, OutWarnings);
			}
			else
			{
				UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
				for (UNiagaraPythonScriptModuleInput* ModuleInput : Results->NewInputs)
				{
					ClipboardContent->FunctionInputs.Add(ModuleInput->Input);	
				}
				if (ClipboardContent->FunctionInputs.Num() > 0)
				{
					FText Warnings;
					UpgradeContext.ApplyClipboardCallback(ClipboardContent, Warnings);
					if (!Warnings.IsEmpty())
					{
						AddStackWarning(PreviousData->Version, NewData->Version, Warnings.ToString(), bLoggedWarning, OutWarnings);
					}
				}
			}
			if (PythonCommand.LogOutput.Num() > 0)
			{
				for (FPythonLogOutputEntry& Entry : PythonCommand.LogOutput)
				{
					AddStackWarning(PreviousData->Version, NewData->Version, Entry.Output, bLoggedWarning, OutWarnings);
				}
			}
		}
		else
		{
			NiagaraVersionUpgrade::ScriptInputs OldInputs;
			OldInputs.Version = PreviousData->Version;
			OldInputs.FunctionInputs = Results->CopyOldInputs();
			
			NiagaraVersionUpgrade::ScriptInputs NewInputs;
			NewInputs.Version = NewData->Version;
			NewInputs.FunctionInputs = Results->CopyNewInputs();
			
			FText OutMessage;
			if (VersionUpdateUtility->ConvertScript(OldInputs, NewInputs, OutMessage))
			{
				UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
				for (UNiagaraClipboardFunctionInput* ModuleInput : NewInputs.FunctionInputs)
				{
					ClipboardContent->FunctionInputs.Add(ModuleInput);
				}
				if (ClipboardContent->FunctionInputs.Num() > 0)
				{
					FText Warnings;
					UpgradeContext.ApplyClipboardCallback(ClipboardContent, Warnings);
					if (!Warnings.IsEmpty())
					{
						AddStackWarning(PreviousData->Version, NewData->Version, Warnings.ToString(), bLoggedWarning, OutWarnings);
					}
				}
			}
			else
			{
				UE_LOGFMT(LogNiagaraEditor, Error, "Script upgrade ended with error! {ErrorMessage}", OutMessage.ToString());
				AddStackWarning(PreviousData->Version, NewData->Version, OutMessage.ToString(), bLoggedWarning, OutWarnings);
			}
		}
	}
	SourceNode->SelectedScriptVersion = SavedVersion;
}

void FNiagaraEditorUtilities::RunPythonUpgradeScripts(UUpgradeNiagaraEmitterContext* UpgradeContext)
{
	TArray<FVersionedNiagaraEmitterData*> UpgradeData = UpgradeContext->GetUpgradeData();
	if (UpgradeData.Num() <= 1)
	{
		// no script executions found, so we're done here
		return;
	}

	FString Warnings;
	for (int32 i = 1; i < UpgradeData.Num(); i++)
	{
		FVersionedNiagaraEmitterData* PreviousData = UpgradeData[i - 1];
		FVersionedNiagaraEmitterData* NewData = UpgradeData[i];
		
		FString PythonScript;
		if (PreviousData == nullptr || NewData == nullptr || NewData->UpdateScriptExecution == ENiagaraPythonUpdateScriptReference::None)
		{
			continue;
		}
		
		if (NewData->UpdateScriptExecution == ENiagaraPythonUpdateScriptReference::DirectTextEntry)
		{
			PythonScript = NewData->PythonUpdateScript;
		}
		else if (NewData->UpdateScriptExecution == ENiagaraPythonUpdateScriptReference::ScriptAsset && !NewData->ScriptAsset.FilePath.IsEmpty())
		{
			FFileHelper::LoadFileToString(PythonScript, *NewData->ScriptAsset.FilePath);
		}

		if (PythonScript.IsEmpty())
		{
			continue;
		}

		bool bLoggedWarning = false;
		if (!IPythonScriptPlugin::Get()->IsPythonAvailable())
		{
			UE_LOGF(LogNiagaraEditor, Log, "Python is not available update script skipped");
			AddStackWarning(PreviousData->Version, NewData->Version, "Python is not available, upgrade script ignored!", bLoggedWarning, Warnings);
			continue;
		}

		// save python script to a temp file to execute
		FString TempScriptFile = FPaths::CreateTempFilename(*FPaths::ProjectIntermediateDir(), TEXT("VersionUpgrade-"), TEXT(".py"));
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		ON_SCOPE_EXIT
		{
			// Delete temp script file
			if (GNiagaraDeletePythonFilesOnError || !UpgradeContext->bCancelledByPythonError)
			{
				PlatformFile.DeleteFile(*TempScriptFile);
			}
		};
		if (!FFileHelper::SaveStringToFile(FString::Format(*PythonUpgradeScriptStub, {UpgradeContext->GetPathName(), PythonScript}), *TempScriptFile))
		{
			UE_LOGF(LogNiagaraEditor, Error, "Unable to save python script to file %ls", *TempScriptFile);
			AddStackWarning(PreviousData->Version, NewData->Version, "Cannot create python script file!", bLoggedWarning, Warnings);
			continue;
		}

		UpgradeContext->bCancelledByPythonError = true;
		FPythonCommandEx PythonCommand = FPythonCommandEx();
		PythonCommand.ExecutionMode = EPythonCommandExecutionMode::ExecuteFile;
		PythonCommand.Command = TempScriptFile;

		// execute python script
		IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand);

		if (UpgradeContext->bCancelledByPythonError)
		{
			UE_LOGF(LogNiagaraEditor, Error, "%ls\n\nPython script:\n%ls\nTo keep the intermediate script around, set fx.Niagara.DeletePythonFilesOnError to 0.", *PythonCommand.CommandResult, *PythonCommand.Command);
			AddStackWarning(PreviousData->Version, NewData->Version, "Python script ended with error!", bLoggedWarning, Warnings);
		}
		else
		{
			//TODO: apply changes?
		}
		
		if (PythonCommand.LogOutput.Num() > 0)
		{
			for (FPythonLogOutputEntry& Entry : PythonCommand.LogOutput)
			{
				AddStackWarning(PreviousData->Version, NewData->Version, Entry.Output, bLoggedWarning, Warnings);
			}
		}
	}

	//TODO: add warnings to stack
}

TSharedPtr<FNiagaraSystemViewModel> CreateSystemViewModelForUpgrade(UNiagaraSystem* System)
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = MakeShared<FNiagaraSystemViewModel>();
	FNiagaraSystemViewModelOptions SystemOptions;
	SystemOptions.bCanModifyEmittersFromTimeline = false;
	SystemOptions.bCanSimulate = false;
	SystemOptions.bCanAutoCompile = false;
	SystemOptions.bIsForDataProcessingOnly = true;
	SystemOptions.MessageLogGuid = System->GetAssetGuid();
	SystemViewModel->Initialize(*System, SystemOptions);
	return SystemViewModel;
}

void FNiagaraEditorUtilities::SwitchParentEmitterVersion(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel, TSharedRef<FNiagaraSystemViewModel> SystemViewModel, const FGuid& NewVersionGuid)
{
	// if we want to run the python upgrade script, we make a snapshot of the existing emitter viewmodel
	UNiagaraPythonEmitter* OldPythonEmitter = NewObject<UNiagaraPythonEmitter>(GetTransientPackage());
	FVersionedNiagaraEmitterData* EmitterData = EmitterViewModel->GetEmitter().GetEmitterData();
	ensureMsgf(EmitterData, TEXT("EmitterData missing when switching emitter version"));
	if (EmitterData == nullptr)
	{
		return;
	}

	FVersionedNiagaraEmitter Parent = EmitterData->GetParent();
	bool bRunPython = Parent.GetEmitterData() && Parent.Emitter->IsVersioningEnabled();
	int32 ViewModelIndex = 0;
	TSharedPtr<FNiagaraSystemViewModel> OldSystemViewModel;
	if (bRunPython)
	{
		UNiagaraSystem* ExistingSystem = CastChecked<UNiagaraSystem>(StaticDuplicateObject(&SystemViewModel->GetSystem(), GetTransientPackage()));
		OldSystemViewModel = CreateSystemViewModelForUpgrade(ExistingSystem);

		TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> EmitterHandleViewModels = SystemViewModel->GetEmitterHandleViewModels();
		for (ViewModelIndex = 0; ViewModelIndex < EmitterHandleViewModels.Num(); ViewModelIndex++)
		{
			if (FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModels[ViewModelIndex]->GetEmitterHandle())
			{
				if (EmitterHandle->GetInstance() == EmitterViewModel->GetEmitter())
				{
					OldPythonEmitter->Init(OldSystemViewModel->GetEmitterHandleViewModels()[ViewModelIndex]);
					break;
				}
			}
		}
	}
	
	FScopedTransaction Transaction(LOCTEXT("ChangeEmitterVersion", "Change Parent Emitter Version"));

	// change the parent, merge the changes and reset the stack
	EmitterViewModel->PreviousEmitterVersion = EmitterViewModel->GetParentEmitter().Version;
	UNiagaraEmitter* Emitter = EmitterViewModel->GetEmitter().Emitter;
	Emitter->ChangeParentVersion(NewVersionGuid, EmitterViewModel->GetEmitter().Version);

	SystemViewModel->GetSystem().KillAllActiveCompilations();
	SystemViewModel->RefreshAll();
	SystemViewModel->GetSelectionViewModel()->EmptySelection();
	SystemViewModel->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectDeferred(Emitter);

	// optionally run python scripts
	if (bRunPython)
	{
		UNiagaraPythonEmitter* NewPythonEmitter = NewObject<UNiagaraPythonEmitter>(GetTransientPackage());
		NewPythonEmitter->Init(SystemViewModel->GetEmitterHandleViewModels()[ViewModelIndex]);
		UUpgradeNiagaraEmitterContext* UpgradeContext = NewObject<UUpgradeNiagaraEmitterContext>(GetTransientPackage());
		UpgradeContext->Init(OldPythonEmitter, NewPythonEmitter);
		
		RunPythonUpgradeScripts(UpgradeContext);

		// purge temp objects
		OldSystemViewModel->GetSystem().MarkAsGarbage();
		OldSystemViewModel->Cleanup();
		OldSystemViewModel.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	// analytics
	TArray<FAnalyticsEventAttribute> Attributes;
	if (NiagaraAnalytics::IsPluginAsset(Parent.Emitter))
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AssetName"), Parent.Emitter->GetPackage()->GetName()));
	}
	NiagaraAnalytics::RecordEvent(TEXT("Versioning.EmitterVersionChanged"), Attributes);
}

bool FNiagaraParameterUtilities::DoesParameterNameMatchSearchText(FName ParameterName, const FString& SearchTextString)
{
	FNiagaraParameterHandle ParameterHandle(ParameterName);
	TArray<FName> HandleParts = ParameterHandle.GetHandleParts();
	FNiagaraNamespaceMetadata NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(HandleParts);
	if (NamespaceMetadata.IsValid())
	{
		// If it's a registered namespace, check the display name of the namespace.
		if (NamespaceMetadata.DisplayName.ToString().Contains(SearchTextString))
		{
			return true;
		}

		// Check the namespace modifiers if it has them
		for(int HandlePartIndex = NamespaceMetadata.Namespaces.Num(); HandlePartIndex < HandleParts.Num() - 2; HandlePartIndex++)
		{
			FNiagaraNamespaceMetadata NamespaceModifierMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaceModifier(HandleParts[HandlePartIndex]);
			if (NamespaceModifierMetadata.IsValid())
			{
				// Check first by modifier metadata display name.
				if (NamespaceModifierMetadata.DisplayName.ToString().Contains(SearchTextString))
				{
					return true;
				}
			}
			else
			{
				// Otherwise just check the string.
				if (HandleParts[HandlePartIndex].ToString().Contains(SearchTextString))
				{
					return true;
				}
			}
		}

		// Last check the variable name.
		if (HandleParts.Last().ToString().Contains(SearchTextString))
		{
			return true;
		}
	}
	else if (HandleParts.ContainsByPredicate([&SearchTextString](FName NamePart) { return NamePart.ToString().Contains(SearchTextString); }))
	{
		// Otherwise if it's not in a valid namespace, just check all name parts.
		return true;
	}
	return false;
}

FText FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(FName ParameterName)
{
	FNiagaraParameterHandle ParameterHandle(ParameterName);
	TArray<FName> HandleParts = ParameterHandle.GetHandleParts();

	if (HandleParts.Num() == 0)
	{
		return FText();
	}
	else if (HandleParts.Num() == 1)
	{
		return FText::FromName(HandleParts[0]);
	}

	FText NamespaceFormat = LOCTEXT("NamespaceFormat", "({0})");
	TArray<FText> ParameterNameTextParts;
	FNiagaraNamespaceMetadata NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(HandleParts);
	if (NamespaceMetadata.IsValid() && NamespaceMetadata.DisplayName.IsEmptyOrWhitespace() == false)
	{
		ParameterNameTextParts.Add(FText::Format(NamespaceFormat, NamespaceMetadata.DisplayName.ToUpper()));
		HandleParts.RemoveAt(0, NamespaceMetadata.Namespaces.Num());
	}
	else
	{
		ParameterNameTextParts.Add(FText::FromName(HandleParts[0]).ToUpper());
		HandleParts.RemoveAt(0);
	}

	for(int32 HandlePartIndex = 0; HandlePartIndex < HandleParts.Num() - 1; HandlePartIndex++)
	{
		FNiagaraNamespaceMetadata NamespaceModifierMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaceModifier(HandleParts[HandlePartIndex]);
		if (NamespaceModifierMetadata.IsValid() && NamespaceModifierMetadata.DisplayName.IsEmptyOrWhitespace() == false)
		{
			ParameterNameTextParts.Add(FText::Format(NamespaceFormat, NamespaceModifierMetadata.DisplayName.ToUpper()));
		}
		else
		{
			ParameterNameTextParts.Add(FText::Format(NamespaceFormat, FText::FromName(HandleParts[0]).ToUpper()));
		}
	}

	if (HandleParts.Num() == 0)
	{
		ParameterNameTextParts.Add(FText::FromName(NAME_None));
	}
	else
	{
		ParameterNameTextParts.Add(FText::FromName(HandleParts.Last()));
	}

	return FText::Join(FText::FromString(TEXT(" ")), ParameterNameTextParts);
}

FName NamePartsToName(const TArray<FName>& NameParts)
{
	TArray<FString> NamePartStrings;
	for (FName NamePart : NameParts)
	{
		NamePartStrings.Add(NamePart.ToString());
	}
	return *FString::Join(NamePartStrings, TEXT("."));
}

bool FNiagaraParameterUtilities::GetNamespaceEditData(
	FName InParameterName, 
	FNiagaraParameterHandle& OutParameterHandle,
	FNiagaraNamespaceMetadata& OutNamespaceMetadata,
	FText& OutErrorMessage)
{
	OutParameterHandle = FNiagaraParameterHandle(InParameterName);
	TArray<FName> NameParts = OutParameterHandle.GetHandleParts();
	OutNamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(NameParts);
	if (OutNamespaceMetadata.IsValid() == false)
	{
		OutErrorMessage = LOCTEXT("NoMetadataForNamespace", "This parameter doesn't support editing.");
		return false;
	}

	return true;
}

bool FNiagaraParameterUtilities::GetNamespaceModifierEditData(
	FName InParameterName,
	FNiagaraParameterHandle& OutParameterHandle,
	FNiagaraNamespaceMetadata& OutNamespaceMetadata,
	FText& OutErrorMessage)
{
	if (GetNamespaceEditData(InParameterName, OutParameterHandle, OutNamespaceMetadata, OutErrorMessage))
	{
		if (OutNamespaceMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier))
		{
			OutErrorMessage = LOCTEXT("NotSupportedForThisNamespace", "This parameter doesn't support namespace modifiers.");
			return false;
		}
		return true;
	}
	else
	{
		return false;
	}
}

void FNiagaraParameterUtilities::GetChangeNamespaceMenuData(FName InParameterName, EParameterContext InParameterContext, TArray<FChangeNamespaceMenuData>& OutChangeNamespaceMenuData)
{
	TArray<FNiagaraNamespaceMetadata> NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetAllNamespaceMetadata();
	NamespaceMetadata.Sort([](const FNiagaraNamespaceMetadata& MetadataA, const FNiagaraNamespaceMetadata& MetadataB)
		{ return MetadataA.SortId < MetadataB.SortId; });

	FNiagaraParameterHandle ParameterHandle(InParameterName);
	FNiagaraNamespaceMetadata CurrentMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(ParameterHandle.GetHandleParts());
	for (const FNiagaraNamespaceMetadata& Metadata : NamespaceMetadata)
	{
		if (Metadata.IsValid() == false ||
			Metadata == CurrentMetadata ||
			Metadata.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace) ||
			(InParameterContext == EParameterContext::Script && Metadata.Options.Contains(ENiagaraNamespaceMetadataOptions::HideInScript)) ||
			(InParameterContext == EParameterContext::System && Metadata.Options.Contains(ENiagaraNamespaceMetadataOptions::HideInSystem)) ||
			(InParameterContext == EParameterContext::Definitions && Metadata.Options.Contains(ENiagaraNamespaceMetadataOptions::HideInDefinitions)))
		{
			continue;
		}

		FChangeNamespaceMenuData& MenuData = OutChangeNamespaceMenuData.AddDefaulted_GetRef();
		MenuData.Metadata = Metadata;

		FText CanChangeMessage;
		MenuData.bCanChange = FNiagaraParameterUtilities::TestCanChangeNamespaceWithMessage(InParameterName, Metadata, CanChangeMessage);

		if (MenuData.bCanChange)
		{
			MenuData.CanChangeToolTip = FText::Format(LOCTEXT("ChangeNamespaceToolTipFormat", "{0}\n\nDescription:\n{1}"), CanChangeMessage, Metadata.Description);
		}
		else
		{
			MenuData.CanChangeToolTip = CanChangeMessage;
		}

		FString NamespaceNameString = Metadata.Namespaces[0].ToString();
		for (int32 i = 1; i < Metadata.Namespaces.Num(); i++)
		{
			NamespaceNameString += TEXT(".") + Metadata.Namespaces[i].ToString();
		}

		MenuData.NamespaceParameterName = *NamespaceNameString;
	}
}

TSharedRef<SWidget> FNiagaraParameterUtilities::CreateNamespaceMenuItemWidget(FName Namespace, FText ToolTipText)
{
	return SNew(SBox)
		.Padding(FMargin(7, 2, 7, 2))
		[
			SNew(SNiagaraParameterName)
			.ParameterName(Namespace)
			.IsReadOnly(true)
			.SingleNameDisplayMode(SNiagaraParameterName::ESingleNameDisplayMode::Namespace)
			.ToolTipText(ToolTipText)
		];
}

bool FNiagaraParameterUtilities::TestCanChangeNamespaceWithMessage(FName ParameterName, const FNiagaraNamespaceMetadata& NewNamespaceMetadata, FText& OutMessage)
{
	if (NewNamespaceMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace))
	{
		OutMessage = LOCTEXT("NewNamespaceNotSupported", "The new namespace does not support editing so it can not be assigned.");
		return false;
	}

	FNiagaraParameterHandle ParameterHandle;
	FNiagaraNamespaceMetadata NamespaceMetadata;
	if (GetNamespaceEditData(ParameterName, ParameterHandle, NamespaceMetadata, OutMessage))
	{
		if (NamespaceMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace))
		{
			OutMessage = LOCTEXT("ParameterDoesntSupportChangingNamespace", "This parameter doesn't support changing its namespace.");
			return false;
		}
		else
		{
			OutMessage = FText::Format(LOCTEXT("ChagneNamespaceFormat", "Change this parameters namespace to {0}"), NewNamespaceMetadata.DisplayName);
			return true;
		}
	}
	return false;
}

FName FNiagaraParameterUtilities::ChangeNamespace(FName ParameterName, const FNiagaraNamespaceMetadata& NewNamespaceMetadata)
{
	FNiagaraParameterHandle ParameterHandle;
	FNiagaraNamespaceMetadata NamespaceMetadata;
	FText Unused;
	if (NewNamespaceMetadata.IsValid() &&
		NewNamespaceMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingNamespace) == false &&
		GetNamespaceEditData(ParameterName, ParameterHandle, NamespaceMetadata, Unused))
	{
		TArray<FName> NameParts = ParameterHandle.GetHandleParts();
		NameParts.RemoveAt(0, NamespaceMetadata.Namespaces.Num());
		int32 NumberOfNamespaceModifiers = NameParts.Num() - 1;
		if (NumberOfNamespaceModifiers > 0)
		{
			bool bNewNamespaceCanHaveNamespaceModifier =
				NewNamespaceMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier) == false;
			if (bNewNamespaceCanHaveNamespaceModifier == false)
			{
				// Remove all modifiers.
				NameParts.RemoveAt(0, NumberOfNamespaceModifiers);
			}
			else 
			{
				// Remove all but the last modifier.
				NameParts.RemoveAt(0, NumberOfNamespaceModifiers - 1);
			}
		}
		if (NewNamespaceMetadata.RequiredNamespaceModifier != NAME_None &&
			(NameParts.Num() == 1 || (NameParts.Num() > 1 && NameParts[0] != NewNamespaceMetadata.RequiredNamespaceModifier)))
		{
			NameParts.Insert(NewNamespaceMetadata.RequiredNamespaceModifier, 0);
		}
		NameParts.Insert(NewNamespaceMetadata.Namespaces, 0);
		return NamePartsToName(NameParts);
	}
	return NAME_None;
}

int32 FNiagaraParameterUtilities::GetNumberOfNamePartsBeforeEditableModifier(const FNiagaraNamespaceMetadata& NamespaceMetadata)
{
	if (NamespaceMetadata.IsValid() && NamespaceMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingNamespaceModifier) == false)
	{
		// If the namespace has a required modifier then we can add and edit a modifier after the required one.
		return NamespaceMetadata.RequiredNamespaceModifier != NAME_None ? NamespaceMetadata.Namespaces.Num() + 1 : NamespaceMetadata.Namespaces.Num();
	}
	else
	{
		return INDEX_NONE;
	}
}

void FNiagaraParameterUtilities::GetOptionalNamespaceModifiers(FName ParameterName, EParameterContext InParameterContext, TArray<FName>& OutOptionalNamespaceModifiers)
{
	FNiagaraParameterHandle ParameterHandle;
	FNiagaraNamespaceMetadata NamespaceMetadata;
	FText Unused;
	if (GetNamespaceModifierEditData(ParameterName, ParameterHandle, NamespaceMetadata, Unused))
	{
		for (FName OptionalNamespaceModifier : NamespaceMetadata.OptionalNamespaceModifiers)
		{
			FNiagaraNamespaceMetadata NamespaceModifierMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaceModifier(OptionalNamespaceModifier);
			if (NamespaceModifierMetadata.IsValid())
			{
				bool bShouldHideInContext =
					(InParameterContext == EParameterContext::Script && NamespaceModifierMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::HideInScript)) ||
					(InParameterContext == EParameterContext::System && NamespaceModifierMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::HideInSystem));

				if(bShouldHideInContext == false)
				{
					OutOptionalNamespaceModifiers.Add(OptionalNamespaceModifier);
				}
			}
		}
	}
}

FName FNiagaraParameterUtilities::GetEditableNamespaceModifierForParameter(FName InParameterName)
{
	FNiagaraParameterHandle ParameterHandle;
	FNiagaraNamespaceMetadata NamespaceMetadata;
	FText Unused;
	if (GetNamespaceModifierEditData(InParameterName, ParameterHandle, NamespaceMetadata, Unused))
	{
		TArray<FName> NameParts = ParameterHandle.GetHandleParts();
		int32 NumberOfNamePartsBeforeEditableModifier = GetNumberOfNamePartsBeforeEditableModifier(NamespaceMetadata);
		if (NameParts.Num() - NumberOfNamePartsBeforeEditableModifier == 2)
		{
			return NameParts[NumberOfNamePartsBeforeEditableModifier];
		}
	}
	return NAME_None;
}

bool FNiagaraParameterUtilities::TestCanSetSpecificNamespaceModifierWithMessage(FName InParameterName, FName InNamespaceModifier, FText& OutMessage)
{
	FNiagaraParameterHandle ParameterHandle;
	FNiagaraNamespaceMetadata NamespaceMetadata;
	if (GetNamespaceModifierEditData(InParameterName, ParameterHandle, NamespaceMetadata, OutMessage))
	{
		FNiagaraNamespaceMetadata ModifierMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaceModifier(InNamespaceModifier);
		if (ModifierMetadata.IsValid() && ModifierMetadata.Description.IsEmptyOrWhitespace() == false)
		{
			OutMessage = FText::Format(LOCTEXT("SetNamespaceModifierWithDescriptionFormat", "Set the namespace modifier for this parameter to {0}.\n\nDescription: {1}"),
				FText::FromName(InNamespaceModifier), ModifierMetadata.Description);
		}
		else
		{
			if(InNamespaceModifier == NAME_None)
			{ 
				OutMessage = LOCTEXT("ClearNamespaceModifier", "Clears the namespace modifier on this parameter.");
			}
			else 
			{
				OutMessage = FText::Format(LOCTEXT("SetNamespaceModifierFormat", "Set the namespace modifier for this parameter to {0}."), FText::FromName(InNamespaceModifier));
			}
		}
		return true;
	}
	return false;
}

FName FNiagaraParameterUtilities::SetSpecificNamespaceModifier(FName InParameterName, FName InNamespaceModifier)
{
	FNiagaraParameterHandle ParameterHandle;
	FNiagaraNamespaceMetadata NamespaceMetadata;
	FText Unused;
	if (GetNamespaceModifierEditData(InParameterName, ParameterHandle, NamespaceMetadata, Unused))
	{
		TArray<FName> NameParts = ParameterHandle.GetHandleParts();
		int32 NumberOfNamePartsBeforeEditableModifier = GetNumberOfNamePartsBeforeEditableModifier(NamespaceMetadata);
		if (NumberOfNamePartsBeforeEditableModifier != INDEX_NONE)
		{
			if (NameParts.Num() == NumberOfNamePartsBeforeEditableModifier + 1)
			{
				// Doesn't have a modifier.  Insert the supplied modifier if it's not none
				if (InNamespaceModifier != NAME_None)
				{
					NameParts.Insert(InNamespaceModifier, NumberOfNamePartsBeforeEditableModifier);
				}
				return NamePartsToName(NameParts);
			}
			else if (NameParts.Num() == NumberOfNamePartsBeforeEditableModifier + 2)
			{
				// Already has a namespace modifier.
				if (InNamespaceModifier == NAME_None || InNamespaceModifier.ToString().IsEmpty())
				{
					// If the name was none or it was empty, remove the modifier.
					NameParts.RemoveAt(NumberOfNamePartsBeforeEditableModifier);
				}
				else
				{
					NameParts[NumberOfNamePartsBeforeEditableModifier] = InNamespaceModifier;
				}
				return NamePartsToName(NameParts);
			}
		}
	}
	return NAME_None;
}

bool FNiagaraParameterUtilities::TestCanSetCustomNamespaceModifierWithMessage(FName InParameterName, FText& OutMessage)
{
	FNiagaraParameterHandle ParameterHandle;
	FNiagaraNamespaceMetadata NamespaceMetadata;
	if (GetNamespaceModifierEditData(InParameterName, ParameterHandle, NamespaceMetadata, OutMessage))
	{
		FName CurrentModifier = GetEditableNamespaceModifierForParameter(InParameterName);
		if (CurrentModifier == NAME_None)
		{
			OutMessage = LOCTEXT("SetCustomNamespaceModifier", "Add a new custom modifier to this parameter.");
		}
		else
		{
			OutMessage = LOCTEXT("EditCurrentNamespaceModifier", "Edit the current custom modifier on this parameter.");
		}
		return true;
	}
	return false;
}

FName FNiagaraParameterUtilities::SetCustomNamespaceModifier(FName InParameterName)
{
	TSet<FName> CurrentParameterNames;
	return SetCustomNamespaceModifier(InParameterName, CurrentParameterNames);
}

FName FNiagaraParameterUtilities::SetCustomNamespaceModifier(FName InParameterName, TSet<FName>& CurrentParameterNames)
{
	FName CurrentModifier = GetEditableNamespaceModifierForParameter(InParameterName);
	if (CurrentModifier == NAME_None)
	{
		FName ParameterNameWithCustomModifier = SetSpecificNamespaceModifier(InParameterName, "Custom");
		int32 CustomIndex = 1;
		while(CurrentParameterNames.Contains(ParameterNameWithCustomModifier))
		{
			ParameterNameWithCustomModifier = SetSpecificNamespaceModifier(InParameterName, *FString::Printf(TEXT("Custom%02i"), CustomIndex));
			CustomIndex++;
		}
		return ParameterNameWithCustomModifier;
	}
	else
	{
		return InParameterName;
	}
}

bool FNiagaraParameterUtilities::TestCanRenameWithMessage(FName ParameterName, FText& OutMessage)
{
	FNiagaraParameterHandle ParameterHandle;
	FNiagaraNamespaceMetadata NamespaceMetadata;
	if (GetNamespaceEditData(ParameterName, ParameterHandle, NamespaceMetadata, OutMessage))
	{
		if (NamespaceMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingName))
		{
			OutMessage = LOCTEXT("ParameterDoesntSupportChangingName", "This parameter doesn't support changing its name.");
			return false;
		}
		else
		{
			OutMessage = LOCTEXT("ChangeParameterName", "Edit this parameter's name.");
			return true;
		}
	}
	return false;
}

TSharedRef<SWidget> FNiagaraParameterUtilities::GetParameterWidget(FNiagaraVariable Variable, bool bAddTypeIcon, bool bShowValue)
{
	if(Variable.IsValid() == false)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("InvalidParameterWidgetName", "Invalid Parameter"))
			.ToolTipText(LOCTEXT("InvalidParameterWidgetTooltip", "This parameter is invalid. If there is a source, check where it might be coming from."));
	}
	
	FNiagaraTypeDefinition Type = Variable.GetType();
	const FLinearColor TypeColor = UEdGraphSchema_Niagara::GetTypeColor(Type);
	TSharedPtr<INiagaraEditorTypeUtilities> TypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(Type);

	TSharedPtr<SHorizontalBox> ParameterContainer;
	
	SHorizontalBox::FSlot* TypeIconSlot;
	TSharedRef<SVerticalBox> ParameterWidget = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(ParameterContainer, SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Expose(TypeIconSlot)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(3.f)
			[
				SNew(SNiagaraParameterNameTextBlock)
				.IsReadOnly(true)
				.ParameterText(FText::FromName(Variable.GetName()))
			]
		];

	if(bAddTypeIcon)
	{
		FText			   IconToolTip = FText::GetEmpty();
		FSlateBrush const* IconBrush = FAppStyle::Get().GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
		FSlateColor        IconColor = FSlateColor(TypeColor);
		FString			   IconDocLink, IconDocExcerpt;
		FSlateBrush const* SecondaryIconBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
		FSlateColor        SecondaryIconColor = IconColor;
		
		TypeIconSlot->AttachWidget(
			SNew(SNiagaraIconWidget)
			   .IconToolTip(IconToolTip)
			   .IconBrush(IconBrush)
			   .IconColor(IconColor)
			   .DocLink(IconDocLink)
			   .DocExcerpt(IconDocExcerpt)
			   .SecondaryIconBrush(SecondaryIconBrush) 
			   .SecondaryIconColor(SecondaryIconColor));
	}
	
	if(bShowValue && Variable.IsDataAllocated())
	{
		const FText ValueText = TypeUtilities->GetStackDisplayText(Variable);

		ParameterWidget->AddSlot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(3.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ParameterTooltipText", "Value: "))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(3.f)
			[
				SNew(STextBlock)
				.Text(ValueText)
			]
		];
	}

	return ParameterWidget;
}

TSharedRef<SWidget> FNiagaraParameterUtilities::GetParameterWidget(FNiagaraVariable Variable, FNiagaraVariableMetaData Metadata, FNiagaraParameterWidgetOptions Options)
{
	TSharedRef<SHorizontalBox> IconBox = SNew(SHorizontalBox);

	if(Options.bShowEditConditionIcon && !Metadata.EditCondition.InputName.IsNone())
	{
		TArray<FString> TargetValues = Metadata.EditCondition.TargetValues;

		FString TooltipString = LOCTEXT("ParameterEditConditionWidgetTooltip", "This parameter is editable in the details panel when {0} is set to: ").ToString();

		if(TargetValues.Num() == 0)
		{
			TooltipString.Append("true");
		}
		else
		{
			if(TargetValues.Num() > 1)
			{
				TooltipString.Append("\n");
			}
			
			for(int32 TargetValueIndex = 0; TargetValueIndex < TargetValues.Num(); TargetValueIndex++)
			{
				TooltipString.Append(TargetValues[TargetValueIndex]);
				
				if(TargetValueIndex < TargetValues.Num() - 2)
				{
					TooltipString.Append("or\n");
				}
			}
		}
		
		FText TooltipText = FText::FormatOrdered(FText::FromString(TooltipString), FText::FromName(Metadata.EditCondition.InputName));
		
		IconBox->AddSlot()
		.Padding(2.f)
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Edit"))
			.ToolTipText(TooltipText)
		];
	}

	if(Options.bShowVisibilityConditionIcon && !Metadata.VisibleCondition.InputName.IsNone())
	{
		TArray<FString> TargetValues = Metadata.VisibleCondition.TargetValues;

		FString TooltipString = LOCTEXT("ParameterVisibleConditionWidgetTooltip", "This parameter becomes visible in the details panel when {0} is set to: ").ToString();

		if(TargetValues.Num() == 0)
		{
			TooltipString.Append("true");
		}
		else
		{
			if(TargetValues.Num() > 1)
			{
				TooltipString.Append("\n");
			}
			
			for(int32 TargetValueIndex = 0; TargetValueIndex < TargetValues.Num(); TargetValueIndex++)
			{
				TooltipString.Append(TargetValues[TargetValueIndex]);
				
				if(TargetValueIndex < TargetValues.Num() - 2)
				{
					TooltipString.Append("or\n");
				}
			}
		}
		
		FText TooltipText = FText::FormatOrdered(FText::FromString(TooltipString), FText::FromName(Metadata.VisibleCondition.InputName));
		
		IconBox->AddSlot()
		.Padding(2.f)
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Visible"))
			.ToolTipText(TooltipText)
		];
	}

	if(Options.bShowAdvanced && Metadata.bAdvancedDisplay)
	{		
		FText TooltipText = LOCTEXT("ParameterIsAdvancedWidgetTooltip", "This parameter is 'advanced' and will only show up in the details panel when 'Show Advanced' is activated.");
		
		IconBox->AddSlot()
		.Padding(2.f)
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("DetailsView.PulldownArrow.Down"))
			.ToolTipText(TooltipText)
		];
	}

	TSharedPtr<STextBlock> NameOverride;
	if(Options.NameOverride.IsSet())
	{
		NameOverride = SNew(STextBlock)
		.Text(Options.NameOverride.GetValue())
		.TextStyle(&FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.HierarchyEditor.SummaryView.ModuleInputNameOverride"));

		if(Options.NameOverrideVisibility.IsSet())
		{
			NameOverride->SetVisibility(Options.NameOverrideVisibility.GetValue());
		}

		if(Options.NameOverrideTooltip.IsSet())
		{
			NameOverride->SetToolTipText(Options.NameOverrideTooltip.GetValue());
		}
	}
	
	TSharedRef<SHorizontalBox> ResultWidget = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		FNiagaraParameterUtilities::GetParameterWidget(Variable, Options.bAddTypeIcon, Options.bShowValue)       	
	];

	if(NameOverride.IsValid())
	{
		ResultWidget->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f, 0.f)
		[
			NameOverride.ToSharedRef()
		];
	}

	if(IconBox->NumSlots() > 0)
	{
		ResultWidget->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			IconBox
		];
	}

	return ResultWidget;
}

TSharedRef<SToolTip> FNiagaraParameterUtilities::GetTooltipWidget(FNiagaraVariable Variable, bool bShowValue, TSharedPtr<SWidget> AdditionalVerticalWidget)
{
	FNiagaraTypeDefinition Type = Variable.GetType();
	const FLinearColor TypeColor = UEdGraphSchema_Niagara::GetTypeColor(Type);
	TSharedPtr<INiagaraEditorTypeUtilities> TypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(Type);

	FText			   IconToolTip = FText::GetEmpty();
	FSlateBrush const* IconBrush = FAppStyle::Get().GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
	FSlateColor        IconColor = FSlateColor(TypeColor);
	FString			   IconDocLink, IconDocExcerpt;
	FSlateBrush const* SecondaryIconBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateColor        SecondaryIconColor = IconColor;
	
	TSharedRef<SWidget> TooltipContent = GetParameterWidget(Variable, true, bShowValue);

	TSharedPtr<SVerticalBox> InnerContainer;
	// we construct a tooltip widget that shows the parameter the value is associated with
	TSharedRef<SToolTip> TooltipWidget = SNew(SToolTip)
	.Content()
	[
		SAssignNew(InnerContainer, SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			TooltipContent
		]
	];

	// if we specified an additional widget, add it here
	if(AdditionalVerticalWidget.IsValid())
	{
		InnerContainer->AddSlot()
		.AutoHeight()
		[
			AdditionalVerticalWidget.ToSharedRef()
		];
	}

	return TooltipWidget;
}

float FNiagaraEditorUtilities::GetScalabilityTintAlpha(FNiagaraEmitterHandle* EmitterHandle)
{
	return EmitterHandle->GetIsEnabled() ? 1 : 0.5f;
}

int FNiagaraEditorUtilities::GetReferencedAssetCount(const FAssetData& SourceAsset,	TFunction<ETrackAssetResult(const FAssetData&)> Predicate)
{
	if (!SourceAsset.IsValid())
	{
		return 0;
	}
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	int32 SearchLimit = GetDefault<UNiagaraEditorSettings>()->GetAssetStatsSearchLimit();
	int32 Count = 0;
	
	TSet<FName> SeenObjects;
	SeenObjects.Add(SourceAsset.PackageName);
	TArray<FName> AssetsToCheck;
	AssetRegistry.GetReferencers(SourceAsset.PackageName, AssetsToCheck);
	while (AssetsToCheck.Num() > 0)
	{
		FName AssetPath = AssetsToCheck[0];
		AssetsToCheck.RemoveAtSwap(0);
		if (SeenObjects.Contains(AssetPath))
		{
			// prevent asset loops
			continue;
		}
		SeenObjects.Add(AssetPath);

		if ((Count + AssetsToCheck.Num()) > SearchLimit)
		{
			// if we are over the search limit then just tally up the remaining references for a rough estimate
			Count++;
		}
		else
		{
			// we should only ever get one asset per package, but the api wants to return a list
			TArray<FAssetData> OutAssetData;
			AssetRegistry.GetAssetsByPackageName(AssetPath, OutAssetData, true);
			for (const FAssetData& AssetToCheck : OutAssetData)
			{
				ETrackAssetResult Result = Predicate(AssetToCheck);
				if (Result == ETrackAssetResult::Count)
				{
					Count++;
				}
				else if (Result == ETrackAssetResult::CountRecursive)
				{
					Count++;
					AssetRegistry.GetReferencers(AssetToCheck.PackageName, AssetsToCheck);
				}
				else
				{
					ensure(Result == ETrackAssetResult::Ignore);
				}
			}
		}
	}
	return FMath::Max(Count, 0);
}

void FNiagaraEditorUtilities::GetAllowedUserVariableTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes)
{
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	for (const FNiagaraTypeDefinition& RegisteredUserVariableType : FNiagaraTypeRegistry::GetRegisteredUserVariableTypes())
	{
		if (NiagaraEditorSettings->IsVisibleTypeDefinition(RegisteredUserVariableType))
		{
			OutAllowedTypes.Add(RegisteredUserVariableType);
		}
	}
}

void FNiagaraEditorUtilities::GetAllowedSystemVariableTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes)
{
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	for (const FNiagaraTypeDefinition& RegisteredSystemVariableType : FNiagaraTypeRegistry::GetRegisteredSystemVariableTypes())
	{
		if (NiagaraEditorSettings->IsVisibleTypeDefinition(RegisteredSystemVariableType))
		{
			OutAllowedTypes.Add(RegisteredSystemVariableType);
		}
	}
}

void FNiagaraEditorUtilities::GetAllowedEmitterVariableTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes)
{
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	for (const FNiagaraTypeDefinition& RegisteredEmitterVariableType : FNiagaraTypeRegistry::GetRegisteredEmitterVariableTypes())
	{
		if (NiagaraEditorSettings->IsVisibleTypeDefinition(RegisteredEmitterVariableType))
		{
			OutAllowedTypes.Add(RegisteredEmitterVariableType);
		}
	}
}

void FNiagaraEditorUtilities::GetAllowedParticleVariableTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes)
{
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	for (const FNiagaraTypeDefinition& RegisteredParticleVariableType : FNiagaraTypeRegistry::GetRegisteredParticleVariableTypes())
	{
		if (NiagaraEditorSettings->IsVisibleTypeDefinition(RegisteredParticleVariableType))
		{
			OutAllowedTypes.Add(RegisteredParticleVariableType);
		}
	}
}

void FNiagaraEditorUtilities::GetAllowedParameterTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes)
{
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	for (const FNiagaraTypeDefinition& RegisteredParameterType : FNiagaraTypeRegistry::GetRegisteredParameterTypes())
	{
		if (NiagaraEditorSettings->IsVisibleTypeDefinition(RegisteredParameterType))
		{
			OutAllowedTypes.Add(RegisteredParameterType);
		}
	}
}

void FNiagaraEditorUtilities::GetAllowedPayloadTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes)
{
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	for (const FNiagaraTypeDefinition& RegisteredPayloadType : FNiagaraTypeRegistry::GetRegisteredPayloadTypes())
	{
		if (NiagaraEditorSettings->IsVisibleTypeDefinition(RegisteredPayloadType))
		{
			OutAllowedTypes.Add(RegisteredPayloadType);
		}
	}
}

bool FNiagaraEditorUtilities::IsEnumIndexVisible(const UEnum* Enum, int32 Index)
{
	return FNiagaraEnumIndexVisibilityCache::GetVisibility(Enum, Index);
}

bool FNiagaraEditorUtilities::IsClassVisibileInGlobalFilter(UClass* Class)
{
	FClassViewerModule* ClassViewerModule = FModuleManager::GetModulePtr<FClassViewerModule>("ClassViewer");
	const TSharedPtr<IClassViewerFilter> GlobalClassFilter = ClassViewerModule ? ClassViewerModule->GetGlobalClassViewerFilter() : TSharedPtr<IClassViewerFilter>();
	if (GlobalClassFilter.IsValid())
	{
		return GlobalClassFilter->IsClassAllowed(FClassViewerInitializationOptions(), Class, ClassViewerModule->CreateFilterFuncs());
	}
	return true;
}

void FNiagaraEditorUtilities::GetScriptMessageStores(UNiagaraScript* InScript, TArray<FNiagaraMessageSourceAndStore>& OutNiagaraMessageStores)
{
	UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(InScript->GetLatestSource());
	if (Source != nullptr && Source->NodeGraph != nullptr)
	{
		TArray<UNiagaraNodeFunctionCall*> FunctionCallNodes;
		Source->NodeGraph->GetNodesOfClass<UNiagaraNodeFunctionCall>(FunctionCallNodes);
		for (UNiagaraNodeFunctionCall* FunctionCallNode : FunctionCallNodes)
		{
			OutNiagaraMessageStores.Add(FNiagaraMessageSourceAndStore(*FunctionCallNode, FunctionCallNode->GetMessageStore()));
		}
	}
}

bool FNiagaraEditorUtilities::IsEditorDataInterfaceInstance(const UNiagaraDataInterface* DataInterface)
{
	return DataInterface->GetOuter()->IsA<UNiagaraNodeInput>();
}

UNiagaraDataInterface* FNiagaraEditorUtilities::GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(const UNiagaraSystem& OwningSystem, UNiagaraDataInterface& EditorDataInterfaceInstance)
{
	UNiagaraNodeInput* OuterInputNode = EditorDataInterfaceInstance.GetTypedOuter<UNiagaraNodeInput>();
	if (OuterInputNode != nullptr)
	{
		// If the data interface's owning node has been removed from it's graph then it's not valid so early out here.
		bool bIsValidInputNode = OuterInputNode->GetGraph()->Nodes.Contains(OuterInputNode);
		if (bIsValidInputNode == false)
		{
			return nullptr;
		}

		// If the data interface was owned by an input node, then we need to try to update the compiled version.
		const FNiagaraEmitterHandle* EmitterHandle;
		TArray<const UNiagaraScript*> CompiledScripts;
		
		FNiagaraStackGraphUtilities::GetEmitterHandleAndCompiledScriptsForStackNode(OwningSystem, *OuterInputNode, EmitterHandle, CompiledScripts);
		if (ensureMsgf(CompiledScripts.Num() > 0, TEXT("Could not find compiled scripts for data interface input node.")))
		{
			for (const UNiagaraScript* CompiledScript : CompiledScripts)
			{
				FString EmitterName = EmitterHandle != nullptr ? EmitterHandle->GetUniqueInstanceName() : FString();
				bool bIsParameterMapDataInterface = false;
				FName DataInterfaceName = FNiagaraHlslTranslator::GetDataInterfaceName(OuterInputNode->Input.GetName(), EmitterName, bIsParameterMapDataInterface);
				
				const FNiagaraEmitterHandle* EmitterHandleForScript = UNiagaraScript::IsSystemScript(CompiledScript->GetUsage()) ? nullptr : EmitterHandle;
				const FNiagaraScriptRuntimeData* ScriptRuntimeData = OwningSystem.GetScriptRuntimeData(EmitterHandleForScript, *CompiledScript).Get();
				if (ScriptRuntimeData != nullptr)
				{
					for (const FNiagaraScriptResolvedDataInterfaceInfo& ResolvedDI : ScriptRuntimeData->GetResolvedDataInterfaces())
					{
						if (ResolvedDI.Name == DataInterfaceName)
						{
							return ResolvedDI.ResolvedDataInterface;
						}
					}
				}
			}
		}
	}
	UNiagaraNodeFunctionCall* OuterFunctionCall = EditorDataInterfaceInstance.GetTypedOuter<UNiagaraNodeFunctionCall>();
	if (OuterFunctionCall != nullptr)
	{
		// Placeholder data interfaces are outered to the function call defines their input.

	}
	return nullptr;
}

EAxisList::Type FNiagaraEditorUtilities::VectorComponentToAxis(int32 NumComponents, int32 ComponentIndex)
{
	if (NumComponents == 3)
	{
		switch (ComponentIndex)
		{
		case 0:
			return EAxisList::X;
		case 1:
			return EAxisList::Y;
		case 2:
			return EAxisList::Z;
		}
	}
	return EAxisList::None;
}

TOptional<FSoftObjectPath> FNiagaraEditorUtilities::Preview::GetPreviewMovieObjectPath(const FAssetData& AssetData)
{
	FString PreviewMoviePathString;
	bool bHasMovieSequence = AssetData.GetTagValue("PreviewMoviePath", PreviewMoviePathString);
	FSoftObjectPath PreviewMoviePath(PreviewMoviePathString);

	if(bHasMovieSequence && PreviewMoviePath.IsAsset())
	{
		return PreviewMoviePath;
	}

	return TOptional<FSoftObjectPath>();
}

TSharedRef<SToolTip> FNiagaraEditorUtilities::Tooltips::CreateTooltip(FText Description, TOptional<FSoftObjectPath> PreviewMoviePath, const FCreateTooltipArguments& CreateTooltipArguments)
{
	if(CreateTooltipArguments.bTryCreatePreviewMovie == true && CreateTooltipArguments.PreviewMovieOwner.IsValid() == false)
	{
		UE_LOGFMT(LogNiagaraEditor, Error, "Preview Movie Tooltip was requested but no owner was specified.");
	}
	
	SHorizontalBox::FSlot* MoviePreviewSlot;
	SHorizontalBox::FSlot* DescriptionSlot;
	
	TSharedRef<SHorizontalBox> TooltipContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot().Expose(MoviePreviewSlot)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		+ SHorizontalBox::Slot().Expose(DescriptionSlot)
		.Padding(4.f, 2.f);

	TSharedPtr<SBox> PreviewMovieContainer = nullptr;
	if(CreateTooltipArguments.bTryCreatePreviewMovie && CreateTooltipArguments.PreviewMovieOwner.IsValid() && PreviewMoviePath.IsSet() && PreviewMoviePath->IsAsset())
	{
		PreviewMovieContainer = SNew(SBox)
			.Visibility_Lambda([]()
			{
				return GetDefault<UNiagaraEditorSettings>()->bDisplayPreviewMoviesInTooltips ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.WidthOverride_Lambda([]()
			{
				return GetDefault<UNiagaraEditorSettings>()->TooltipPreviewMovieSize.X;
			})
			.HeightOverride_Lambda([]()
			{
				return GetDefault<UNiagaraEditorSettings>()->TooltipPreviewMovieSize.Y;
			})
			[
				SNew(SImage)
				// We only have one preview movie brush & one media player per previewmovieviewmodel.
				.Image(CreateTooltipArguments.PreviewMovieOwner->GetMoviePreviewImageBrush())
			];
	}

	if(PreviewMovieContainer.IsValid())
	{
		MoviePreviewSlot->AttachWidget(PreviewMovieContainer.ToSharedRef());
	}

	TSharedRef<STextBlock> DescriptionText = SNew(STextBlock)
		.Text(Description)
		.WrapTextAt(SToolTip::GetToolTipWrapWidth() - 500.f)
		.Font(FCoreStyle::Get().GetFontStyle("ToolTip.Font"))
		.ColorAndOpacity(FLinearColor::Black);
	
	DescriptionSlot->AttachWidget(DescriptionText);
	
	return SNew(SToolTip)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.BrightBackground"))
		[
			TooltipContent
		];
}

TSharedRef<SToolTip> FNiagaraEditorUtilities::Tooltips::CreateTooltipForNiagaraAction(TSharedRef<FNiagaraMenuAction_Generic> Action, const FCreateTooltipArguments& CreateTooltipArguments)
{	
	return CreateTooltip(Action->ToolTip, Action->PreviewMoviePath, CreateTooltipArguments);
}

TSharedRef<SToolTip> FNiagaraEditorUtilities::Tooltips::CreateTooltipForScript(UNiagaraScript& Script, FGuid VersionGuid, const FCreateTooltipArguments& CreateTooltipArguments)
{
	FVersionedNiagaraScriptData* ScriptData = nullptr;
	if(VersionGuid.IsValid())
	{
		ScriptData = Script.GetScriptData(VersionGuid);
	}
	else
	{
		ScriptData = Script.GetLatestScriptData(); 
	}

	TOptional<FSoftObjectPath> PreviewMoviePath;
	if(CreateTooltipArguments.bTryCreatePreviewMovie)
	{
		PreviewMoviePath = FNiagaraEditorUtilities::Preview::GetPreviewMovieObjectPath(FAssetData(&Script));
	}

	return CreateTooltip(ScriptData->Description, PreviewMoviePath, CreateTooltipArguments);
}

namespace FNiagaraEditorUtilities::UserParameters::Internal
{
	TSharedRef<FNiagaraSystemViewModel> CreateDataProcessingOnlyViewModel(UNiagaraSystem& InSystem)
	{
		TSharedRef<FNiagaraSystemViewModel> SystemViewModel = MakeShared<FNiagaraSystemViewModel>();
		FNiagaraSystemViewModelOptions Options;
		Options.bCanAutoCompile = false;
		Options.bCanModifyEmittersFromTimeline = false;
		Options.bCanSimulate = false;
		Options.bCompileForEdit = false;
		Options.bIsForDataProcessingOnly = true;
		Options.EditMode = ENiagaraSystemViewModelEditMode::SystemAsset;
		SystemViewModel->Initialize(InSystem, Options);
		return SystemViewModel;
	}
}

void FNiagaraEditorUtilities::UserParameters::AddUserParameterToSystem(UNiagaraSystem& System, const FNiagaraVariableBase& UserParameter)
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(&System);
	if (SystemViewModel.IsValid() == false)
	{
		SystemViewModel = Internal::CreateDataProcessingOnlyViewModel(System);
	}
	SystemViewModel->AddParameter(UserParameter);
}

void FNiagaraEditorUtilities::UserParameters::RemoveUserParameterFromSystem(UNiagaraSystem& System, const FNiagaraVariableBase& UserParameter)
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(&System);
	if (SystemViewModel.IsValid() == false)
	{
		SystemViewModel = Internal::CreateDataProcessingOnlyViewModel(System);
	}
	SystemViewModel->DeleteParameter(UserParameter);
}

void FNiagaraEditorUtilities::UserParameters::RenameUserParameterForSystem(UNiagaraSystem& System, const FNiagaraVariableBase& OldParameter, const FNiagaraVariableBase& NewParameter)
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(&System);
	if (SystemViewModel.IsValid() == false)
	{
		SystemViewModel = Internal::CreateDataProcessingOnlyViewModel(System);
	}
	SystemViewModel->RenameParameter(OldParameter, NewParameter.GetName(), ENiagaraGetGraphParameterReferencesMode::AllGraphs);
}

void FNiagaraEditorUtilities::UserParameters::GetUserParameterDefaultForSystem(const UNiagaraSystem& System, const FNiagaraVariableWithOffset& UserParameterWithOffset, FNiagaraVariant& OutDefault)
{
	if (UserParameterWithOffset.IsDataInterface())
	{
		OutDefault.SetDataInterface(System.GetExposedParameters().GetDataInterface(UserParameterWithOffset));
	}
	else if (UserParameterWithOffset.IsUObject())
	{
		OutDefault.SetUObject(System.GetExposedParameters().GetUObject(UserParameterWithOffset));
	}
	else
	{
		OutDefault.SetBytes(System.GetExposedParameters().GetParameterData(UserParameterWithOffset), UserParameterWithOffset.GetType().GetSize());
	}
}

TOptional<FNiagaraVariant> FNiagaraEditorUtilities::UserParameters::FindUserParameterDefaultForSystem(const UNiagaraSystem& System, const FNiagaraVariableBase& UserParameter)
{
	int32 ParameterIndex = System.GetExposedParameters().IndexOf(UserParameter);
	if (ParameterIndex != INDEX_NONE)
	{
		FNiagaraVariableWithOffset UserParameterWithOffset = FNiagaraVariableWithOffset(UserParameter, ParameterIndex, FNiagaraLwcStructConverter());
		FNiagaraVariant Default;
		GetUserParameterDefaultForSystem(System, UserParameterWithOffset, Default);
		return Default;
	}
	return TOptional<FNiagaraVariant>();
}

void FNiagaraEditorUtilities::UserParameters::SetUserParameterDefaultForSystem(UNiagaraSystem& System, const FNiagaraVariableBase& UserParameter, const FNiagaraVariant& Default)
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(&System);
	if (SystemViewModel.IsValid() == false)
	{
		SystemViewModel = Internal::CreateDataProcessingOnlyViewModel(System);
	}
	SystemViewModel->SetParameterDefaultValue(UserParameter, Default);
}

FNiagaraVariable FNiagaraEditorUtilities::UserParameters::DuplicateUserParameter(FNiagaraVariable ParameterToDuplicate, UNiagaraSystem& System)
{
	FScopedTransaction AddTransaction(LOCTEXT("DuplicateUserParameter", "Duplicate User Parameter"));
	System.Modify();

	FNiagaraUserRedirectionParameterStore::MakeUserVariable(ParameterToDuplicate);

	TSet<FName> ExistingParameterStoreNames;
	TArray<FNiagaraVariable> ParameterStoreVariables;
	System.GetExposedParameters().GetParameters(ParameterStoreVariables);
	for (const FNiagaraVariable& Var : ParameterStoreVariables)
	{
		ExistingParameterStoreNames.Add(Var.GetName());
	}

	FNiagaraVariable DuplicatedParameter = ParameterToDuplicate;
	DuplicatedParameter.SetName(FNiagaraUtilities::GetUniqueName(ParameterToDuplicate.GetName(), ExistingParameterStoreNames));

	bool bSuccess = System.GetExposedParameters().AddParameter(DuplicatedParameter);

	if(bSuccess)
	{
		System.GetExposedParameters().CopyParameterData(System.GetExposedParameters(), ParameterToDuplicate, DuplicatedParameter);
		TObjectPtr<UNiagaraScriptVariable> ScriptVariableToDuplicate = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(ParameterToDuplicate, System);
		TObjectPtr<UNiagaraScriptVariable> DuplicatedScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(DuplicatedParameter, System);
		
		for(TFieldIterator<FProperty> It(UNiagaraScriptVariable::StaticClass()); It; ++It)
		{
			uint8* Source = It->ContainerPtrToValuePtr<uint8>(ScriptVariableToDuplicate);
			uint8* Target = It->ContainerPtrToValuePtr<uint8>(DuplicatedScriptVariable);
			It->CopyCompleteValue(Target, Source);
		}

		DuplicatedScriptVariable->Metadata.CreateNewGuid();
		DuplicatedScriptVariable->Variable.SetName(DuplicatedParameter.GetName());
		
		System.GetExposedParameters().TriggerOnLayoutChanged();
		return DuplicatedParameter;
	}
	
	return FNiagaraVariable();
}

TSharedRef<SWidget> FNiagaraEditorUtilities::HierarchyEditor::Scripts::GenerateRowContentForScriptParameterHierarchyEditor(TSharedRef<FHierarchyElementViewModel> HierarchyItem)
{
	if(HierarchyItem->GetDataMutable()->IsA<UHierarchyCategory>())
	{
		TSharedRef<FHierarchyCategoryViewModel> TreeViewCategory = StaticCastSharedRef<FHierarchyCategoryViewModel>(HierarchyItem);
		return SNew(SHierarchyElement, TreeViewCategory)
			.Style(&FDataHierarchyEditorStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("HierarchyEditor.Category"));
	}
	else if(const UNiagaraHierarchyScriptParameter* ScriptParameter = Cast<UNiagaraHierarchyScriptParameter>(HierarchyItem->GetData()))
	{
		if(ScriptParameter->GetScriptVariable() == nullptr)
		{
			return SNew(STextBlock).Text(LOCTEXT("InvalidHierarchyScriptParameter", "Invalid Script Hierarchy Parameter"));
		}
		
		FNiagaraParameterUtilities::FNiagaraParameterWidgetOptions WidgetOptions;
		WidgetOptions.bAddTypeIcon = true;
		WidgetOptions.bShowEditConditionIcon = true;
		WidgetOptions.bShowVisibilityConditionIcon = true;
		WidgetOptions.bShowAdvanced = true;
		TSharedRef<SWidget> ParameterWidget = FNiagaraParameterUtilities::GetParameterWidget(ScriptParameter->GetVariable().Get(FNiagaraVariable()), ScriptParameter->GetScriptVariable()->Metadata, WidgetOptions);
		const UNiagaraScriptVariable* ScriptVariable = ScriptParameter->GetScriptVariable();

		ParameterWidget->SetToolTipText(TAttribute<FText>::CreateLambda([ScriptVariable]()
		{
			return ScriptVariable->Metadata.Description;
		}));
		
		return ParameterWidget;
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SToolTip> FNiagaraEditorUtilities::Tooltips::CreateStackNoteTooltip(UNiagaraStackNote& StackNote)
{
	if(StackNote.GetTargetStackNoteData().IsSet() == false)
	{
		return SNew(SToolTip);
	}

	TSharedRef<SVerticalBox> TooltipContent = SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding(3.f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SBox)
			.HeightOverride(16.f)
			.WidthOverride(16.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Comment"))
				.ColorAndOpacity(StackNote.GetColor())
			]
		]
		+ SHorizontalBox::Slot()
		.Padding(2.f)
		[
			SNew(STextBlock)
			.Text(StackNote.GetTargetStackNoteData().GetValue().MessageHeader)
			.TextStyle(&FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.Stack.Note.HeaderText"))
			.AutoWrapText(true)
		]
	];

	if(StackNote.GetTargetStackNoteData().GetValue().Message.IsEmpty() == false)
	{
		TooltipContent->AddSlot()
		.AutoHeight()
		.Padding(5.f)
		[
			SNew(STextBlock)
			.Text(StackNote.GetTargetStackNoteData().GetValue().Message)
			.AutoWrapText(true)
		];
	}
	
	return SNew(SToolTip)
	.Content()
	[
		SNew(SBox)
		.MaxDesiredWidth(500.f)
		[
			TooltipContent
		]
	];
}

FText FNiagaraEditorUtilities::Tooltips::GetMinimalEmitterCreationTooltip()
{
	FText EmitterDescription = LOCTEXT("TrueEmptyEmitterTooltip", "Add a completely empty emitter.");
	FSoftObjectPath DefaultEmptyEmitter = GetDefault<UNiagaraEditorSettings>()->DefaultEmptyEmitter;
	if(DefaultEmptyEmitter.IsValid() && DefaultEmptyEmitter.IsAsset())
	{
		if(UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(DefaultEmptyEmitter.TryLoad()))
		{
			EmitterDescription =  Emitter->TemplateAssetDescription;
		}
	}

	FText TooltipText = LOCTEXT("AddMinimalEmitterTooltip", "Adds a minimal emitter as specified in the Niagara Settings");
	if(EmitterDescription.IsEmpty())
	{
		return TooltipText;
	}
			
	return FText::FormatOrdered( FText::AsCultureInvariant("{0}\n\n{1}."), TooltipText, EmitterDescription);
}

TMap<FGuid, TArray<FNiagaraVariableBase>> FNiagaraEditorUtilities::Scripts::Validation::ValidateScriptVariableIds(UNiagaraScript* Script, FGuid VersionGuid)
{
	bool bReturnResults = false;
	
	TMap<FGuid, TArray<FNiagaraVariableBase>> Results;

	if(!VersionGuid.IsValid())
	{
		VersionGuid = Script->GetExposedVersion().VersionGuid;
	}
	
	if(UNiagaraScriptSourceBase* SourceBase = Script->GetSource(VersionGuid))
	{
		UNiagaraScriptSource* Source = CastChecked<UNiagaraScriptSource>(SourceBase);
		
		for(const auto& ScriptVariableEntry : Source->NodeGraph->GetAllMetaData())
		{
			Results.FindOrAdd(ScriptVariableEntry.Value->Metadata.GetVariableGuid()).Add(ScriptVariableEntry.Key);
		}

		for(const auto& Result : Results)
		{
			if(Result.Value.Num() > 1)
			{
				bReturnResults = true;
				break;
			}
		}

		if(bReturnResults)
		{
			UE_LOGF(LogNiagaraEditor, Error, "---- Duplicate variable guids found in script %ls, Version %ls ---- :", *Script->GetName(), *VersionGuid.ToString());
			for(const auto& Result : Results)
			{
				if(Result.Value.Num() > 1)
				{
					FString ResultString("Duplicate guids found in: ");
					for(int32 StringConstructorIndex = 0; StringConstructorIndex < Result.Value.Num(); StringConstructorIndex++)
					{
						ResultString.Appendf(TEXT("%s, "), *Result.Value[StringConstructorIndex].GetName().ToString());
					}
			
					UE_LOGF(LogNiagaraEditor, Error, "%ls", *ResultString);
				}
			}
	
			return Results;
		}
	}

	UE_LOGF(LogNiagaraEditor, Log, "---- No duplicate variable guids in script %ls, version %ls ----", *Script->GetName(), *VersionGuid.ToString());
	return {};
}

TMap<FNiagaraVariableBase, FGuid> FNiagaraEditorUtilities::Scripts::Validation::FixupDuplicateScriptVariableGuids(UNiagaraScript* Script)
{
	/** We keep track of already reassigned guids here to reuse them if the same parameter appears in another version. */
	TMap<FNiagaraVariableBase, FGuid> OldGuids;
	TMap<FNiagaraVariableBase, FGuid> RemappedGuids;
	
	for(FNiagaraAssetVersion& AssetVersion : Script->GetAllAvailableVersions())
	{
		UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(Script->GetSource(AssetVersion.VersionGuid));
		TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>& VariableMap = ScriptSource->NodeGraph->GetAllMetaData();
		
		TMap<FGuid, TArray<FNiagaraVariableBase>> Results = Validation::ValidateScriptVariableIds(Script, AssetVersion.VersionGuid);
		for(const auto& Result : Results)
		{
			// we only want to fix any guids if we have more than 1 variable with the same guid
			if(Result.Value.Num() > 1)
			{
				// if a parameter has already been given a new guid before (in a different version), we reuse the same new guid
				for(const FNiagaraVariableBase& VariableWithDuplicateGuid : Result.Value)
				{
					if(RemappedGuids.Contains(VariableWithDuplicateGuid))
					{
						FGuid NewGuid = RemappedGuids[VariableWithDuplicateGuid];
						VariableMap[VariableWithDuplicateGuid]->Metadata.SetVariableGuid(NewGuid);
					}
					else
					{
						Script->Modify(true);
						FGuid NewGuid = FGuid::NewGuid();
						OldGuids.Add(VariableWithDuplicateGuid, VariableMap[VariableWithDuplicateGuid]->Metadata.GetVariableGuid());
						VariableMap[VariableWithDuplicateGuid]->Metadata.SetVariableGuid(NewGuid);
						RemappedGuids.Add(VariableWithDuplicateGuid, NewGuid);
					}
				}
			}
		}
	}

	if(RemappedGuids.Num() > 0)
	{
		UE_LOGF(LogNiagaraEditor, Log, "----------- Variables with new guids in script %ls ---------", *Script->GetName());
	}
	
	for(const auto& RemappedGuid : RemappedGuids)
	{
		UE_LOGF(LogNiagaraEditor, Log, "Variable: %ls. Guid: %ls -> %ls", *RemappedGuid.Key.GetName().ToString(), *OldGuids[RemappedGuid.Key].ToString(), *RemappedGuids[RemappedGuid.Key].ToString())
	}

	return RemappedGuids;
}

void FNiagaraParameterUtilities::FilterToRelevantStaticVariables(TConstArrayView<FNiagaraVariable> InVars, TArray<FNiagaraVariable>& OutVars, FName InOldEmitterAlias, FName InNewEmitterAlias, bool bFilterByEmitterAliasAndConvertToUnaliased)
{
	FNiagaraAliasContext RenameContext(FNiagaraAliasContext::ERapidIterationParameterMode::StaticVariables);
	RenameContext.ChangeEmitterName(InOldEmitterAlias.ToString(), InNewEmitterAlias.ToString());

	for (const FNiagaraVariable& InVar : InVars)
	{
		if (InVar.GetType().IsStatic())
		{
			if (bFilterByEmitterAliasAndConvertToUnaliased)
			{
				FNiagaraVariable NewVar = FNiagaraUtilities::ResolveAliases(InVar, RenameContext);

				if (NewVar.GetName() != InVar.GetName())
				{
					NewVar.SetData(InVar.GetData());
					OutVars.AddUnique(NewVar);
				}
				else
				{
					OutVars.AddUnique(InVar);
				}
			}
			else
			{
				OutVars.AddUnique(InVar);
			}
		}
	}
}

TArray<const UNiagaraScriptVariable*> FNiagaraParameterDefinitionsUtilities::FindReservedParametersByName(const FName ParameterName)
{
	TArray<const UNiagaraScriptVariable*> OutScriptVars;
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	const TArray<TWeakObjectPtr<UNiagaraParameterDefinitions>>& CachedParameterDefinitionsAssets = NiagaraEditorModule.GetCachedParameterDefinitionsAssets();
	for (const TWeakObjectPtr<UNiagaraParameterDefinitions>& CachedParameterDefinitionsAsset : CachedParameterDefinitionsAssets)
	{
		if (!CachedParameterDefinitionsAsset.IsValid())
		{
			continue;
		}
		for (const UNiagaraScriptVariable* ScriptVar : CachedParameterDefinitionsAsset->GetParametersConst())
		{
			if (ScriptVar->Variable.GetName() == ParameterName)
			{
				OutScriptVars.Add(ScriptVar);
			}
		}
	}
	return OutScriptVars;
}

int32 FNiagaraParameterDefinitionsUtilities::GetNumParametersReservedForName(const FName ParameterName)
{
	return FindReservedParametersByName(ParameterName).Num();
}

EParameterDefinitionMatchState FNiagaraParameterDefinitionsUtilities::GetDefinitionMatchStateForParameter(const FNiagaraVariableBase& Parameter)
{
	const TArray<const UNiagaraScriptVariable*> ReservedScriptVarsForName = FindReservedParametersByName(Parameter.GetName());
	if (ReservedScriptVarsForName.Num() == 0)
	{
		return EParameterDefinitionMatchState::NoMatchingDefinitions;
	}
	else if (ReservedScriptVarsForName.Num() == 1)
	{
		if (ReservedScriptVarsForName[0]->Variable.GetType() == Parameter.GetType())
		{
			return EParameterDefinitionMatchState::MatchingOneDefinition;
		}
		return EParameterDefinitionMatchState::MatchingDefinitionNameButNotType;
	}
	else
	{
		return EParameterDefinitionMatchState::MatchingMoreThanOneDefinition;
	}
}

void FNiagaraParameterDefinitionsUtilities::TrySubscribeScriptVarToDefinitionByName(UNiagaraScriptVariable* ScriptVar, INiagaraParameterDefinitionsSubscriberViewModel* OwningDefinitionSubscriberViewModel)
{
	const FName& ScriptVarName = ScriptVar->Variable.GetName();
	const TArray<const UNiagaraScriptVariable*> ReservedParametersForName = FNiagaraParameterDefinitionsUtilities::FindReservedParametersByName(ScriptVarName);

	if (ReservedParametersForName.Num() == 0)
	{
		// Do not call SetParameterIsSubscribedToLibrary() unless ScriptVarToModify is already marked as SubscribedToParameterDefinitions.
		if (ScriptVar->GetIsSubscribedToParameterDefinitions())
		{
			//SetParameterIsSubscribedToLibrary(ScriptVar, false);
			OwningDefinitionSubscriberViewModel->SetParameterIsSubscribedToDefinitions(ScriptVar->Metadata.GetVariableGuid(), false);
		}
	}
	else if (ReservedParametersForName.Num() == 1)
	{
		const UNiagaraScriptVariable* ReservedParameterDefinition = (ReservedParametersForName)[0];
		if (ReservedParameterDefinition->Variable.GetType() != ScriptVar->Variable.GetType())
		{
			const FText TypeMismatchWarningTemplate = LOCTEXT("RenameParameter_DefinitionTypeMismatch", "Renamed parameter \"{0}\" with type {1}. Type does not match existing parameter definition \"{0}\" with type {2} from {3}! ");
			FText TypeMismatchWarning = FText::Format(
				TypeMismatchWarningTemplate,
				FText::FromName(ScriptVarName),
				ScriptVar->Variable.GetType().GetNameText(),
				ReservedParameterDefinition->Variable.GetType().GetNameText(),
				FText::FromString(ReservedParameterDefinition->GetTypedOuter<UNiagaraParameterDefinitions>()->GetName()));
			FNiagaraEditorUtilities::WarnWithToastAndLog(TypeMismatchWarning);

			// Do not call SetParameterIsSubscribedToLibrary() unless ScriptVarToModify is already marked as SubscribedToParameterDefinitions.
			if (ScriptVar->GetIsSubscribedToParameterDefinitions())
			{
				OwningDefinitionSubscriberViewModel->SetParameterIsSubscribedToDefinitions(ScriptVar->Metadata.GetVariableGuid(), false);
			}
		}
		else
		{
			// NOTE: It is possible we are calling SetParameterIsSubscribedToLibrary() on a UNiagaraScriptVariable that is already marked as SubscribedToParameterDefinitions;
			// This is intended as SetParameterIsSubscribedToLibrary() will register a new link to a different parameter definition.
			OwningDefinitionSubscriberViewModel->SetParameterIsSubscribedToDefinitions(ScriptVar->Metadata.GetVariableGuid(), true);
		}
	}
}

TMap<FNiagaraEnumIndexVisibilityCache::FEnumIndexPair, bool> FNiagaraEnumIndexVisibilityCache::Cache;
FCriticalSection FNiagaraEnumIndexVisibilityCache::CacheLock;

bool FNiagaraEnumIndexVisibilityCache::GetVisibility(const UEnum* InEnum, int32 InIndex)
{
	FEnumIndexPair Key(InEnum, InIndex);
	FScopeLock Lock(&CacheLock);
	bool* bCachedIsVisible = Cache.Find(Key);
	if (bCachedIsVisible == nullptr)
	{
		bCachedIsVisible = &Cache.Add(Key);
		*bCachedIsVisible = InEnum->HasMetaData(TEXT("Hidden"), InIndex) == false && InEnum->HasMetaData(TEXT("Spacer"), InIndex) == false;
	}
	return *bCachedIsVisible;
}

#undef LOCTEXT_NAMESPACE
