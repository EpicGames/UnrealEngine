// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEffectTypeFactoryNew.h"
#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "NiagaraSettings.h"
#include "NiagaraEffectType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEffectTypeFactoryNew)

#define LOCTEXT_NAMESPACE "NiagaraEffectTypeFactory"

UNiagaraEffectTypeFactoryNew::UNiagaraEffectTypeFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UNiagaraEffectType::StaticClass();
	bCreateNew = false;
	bEditAfterNew = true;
	bCreateNew = true;
}

UObject* UNiagaraEffectTypeFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UNiagaraEffectType::StaticClass()));

	UNiagaraEffectType* NewEffectType = NewObject<UNiagaraEffectType>(InParent, Class, Name, Flags | RF_Transactional);

	return NewEffectType;
}

#undef LOCTEXT_NAMESPACE
