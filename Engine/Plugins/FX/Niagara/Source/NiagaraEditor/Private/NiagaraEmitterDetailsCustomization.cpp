// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterDetailsCustomization.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "NiagaraEmitter.h"
#include "NiagaraEditorModule.h"
#include "NiagaraSystem.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/TNiagaraViewModelManager.h"

TSharedRef<IDetailCustomization> FNiagaraEmitterDetails::MakeInstance()
{
	return MakeShared<FNiagaraEmitterDetails>();
}

void CustomizeEmitterData(IDetailLayoutBuilder& InDetailLayout, UNiagaraEmitter* EmitterBeingCustomized, TMap<FName, TSet<FProperty*>> CategoryPropertyMap)
{
	TArray<TSharedPtr<FNiagaraEmitterViewModel>> ExistingViewModels;
	TNiagaraViewModelManager<UNiagaraEmitter, FNiagaraEmitterViewModel>::GetAllViewModelsForObject(EmitterBeingCustomized, ExistingViewModels);
	FVersionedNiagaraEmitter VersionedNiagaraEmitter = ExistingViewModels[0]->GetEmitter();
	FVersionedNiagaraEmitterData* EmitterData = VersionedNiagaraEmitter.GetEmitterData();
	UNiagaraSystem* NiagaraSystem = EmitterBeingCustomized->GetTypedOuter<UNiagaraSystem>();
	TSharedPtr<FStructOnScope> StructData = MakeShareable(new FStructOnScope(FVersionedNiagaraEmitterData::StaticStruct(), reinterpret_cast<uint8*>(EmitterData)));
	for (const auto& Entry : CategoryPropertyMap)
	{
		IDetailCategoryBuilder& CategoryBuilder = InDetailLayout.EditCategory(Entry.Key);
		TArray<IDetailPropertyRow*> OutRows;
			
		// We want only one structonscope, otherwise edit conditions don't work correctly. Unfortuately this means we need to add each category separately and hide properties not assigned to the category.
		TArray<TSharedPtr<IPropertyHandle>> Handles = CategoryBuilder.AddAllExternalStructureProperties(StructData.ToSharedRef(), EPropertyLocation::Default, &OutRows);
			
		for (IDetailPropertyRow* PropertyRow : OutRows)
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow->GetPropertyHandle();
			FProperty* Property = PropertyHandle->GetProperty();
			if (!Entry.Value.Contains(Property))
			{
				PropertyHandle->MarkHiddenByCustomization();
				PropertyRow->Visibility(EVisibility::Collapsed);
				continue;
			}

			if (NiagaraSystem && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, CalculateBoundsMode) || Property->GetFName() == GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, FixedBounds)) )
			{
				PropertyRow->IsEnabled(TAttribute<bool>::Create([VersionedNiagaraEmitter, WeakNiagaraSystem=TWeakObjectPtr<UNiagaraSystem>(NiagaraSystem)]() { return !WeakNiagaraSystem.IsValid() || WeakNiagaraSystem.Get()->bFixedBounds == false; }));
			}

			const auto& PostEditChangeLambda = [VersionedNiagaraEmitter, Property]
			{
				FPropertyChangedEvent ChangeEvent(Property);
				VersionedNiagaraEmitter.Emitter->PostEditChangeVersionedProperty(ChangeEvent, VersionedNiagaraEmitter.Version);
			};
			const auto& PreEditChangeLambda = [VersionedNiagaraEmitter, Property]
			{
				VersionedNiagaraEmitter.Emitter->PreEditChange(Property);
			};

			PropertyRow->GetPropertyHandle()->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda(PostEditChangeLambda));
			PropertyRow->GetPropertyHandle()->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(PostEditChangeLambda));
			PropertyRow->GetPropertyHandle()->SetOnPropertyValuePreChange(FSimpleDelegate::CreateLambda(PreEditChangeLambda));
			PropertyRow->GetPropertyHandle()->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateLambda(PreEditChangeLambda));
		}
	}

	// only show some categories for standalone emitter assets 
	if (NiagaraSystem && NiagaraSystem->GetOuter() != GetTransientPackage())
	{
		IDetailCategoryBuilder& ScriptCategory = InDetailLayout.EditCategory(FName("Script"));
		ScriptCategory.SetCategoryVisibility(false);
		IDetailCategoryBuilder& AssetCategory = InDetailLayout.EditCategory(FName("Asset Options"));
		AssetCategory.SetCategoryVisibility(false);
		
	}
}

void FNiagaraEmitterDetails::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	InDetailLayout.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	
	if (UNiagaraEmitter* EmitterBeingCustomized = CastChecked<UNiagaraEmitter>(ObjectsBeingCustomized[0]))
	{
		TMap<FName, TSet<FProperty*>> CategoryPropertyMap;
		for (FProperty* ChildProperty : TFieldRange<FProperty>(FVersionedNiagaraEmitterData::StaticStruct()))
		{
			if (ChildProperty->HasAllPropertyFlags(CPF_Edit))
			{
				FName Category = FName(ChildProperty->GetMetaData(TEXT("Category")));
				
				// we display the scalability category within scalability mode, which is why we hide it here
				if (Category != FName("Scalability"))
				{
					CategoryPropertyMap.FindOrAdd(Category).Add(ChildProperty);
				}
			}
		}

		CustomizeEmitterData(InDetailLayout, EmitterBeingCustomized, CategoryPropertyMap);
	}
}

TSharedRef<IDetailCustomization> FNiagaraEmitterScalabilityDetails::MakeInstance()
{
	return MakeShared<FNiagaraEmitterScalabilityDetails>();
}

void FNiagaraEmitterScalabilityDetails::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	InDetailLayout.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	
	if (UNiagaraEmitter* EmitterBeingCustomized = CastChecked<UNiagaraEmitter>(ObjectsBeingCustomized[0]))
	{		
		TMap<FName, TSet<FProperty*>> CategoryPropertyMap;
		for (FProperty* ChildProperty : TFieldRange<FProperty>(FVersionedNiagaraEmitterData::StaticStruct()))
		{
			if (ChildProperty->HasAllPropertyFlags(CPF_Edit) && ChildProperty->HasMetaData(TEXT("DisplayInScalabilityContext")))
			{
				FName Category = FName(ChildProperty->GetMetaData(TEXT("Category")));
				CategoryPropertyMap.FindOrAdd(Category).Add(ChildProperty);
			}
		}

		CustomizeEmitterData(InDetailLayout, EmitterBeingCustomized, CategoryPropertyMap);
	}
}
