// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;

/*-----------------------------------------------------------------------------
FNiagaraEmitterDetails
-----------------------------------------------------------------------------*/

class FNiagaraEmitterDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout) override;
};

// customization for scalability mode
class FNiagaraEmitterScalabilityDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayout) override;
};