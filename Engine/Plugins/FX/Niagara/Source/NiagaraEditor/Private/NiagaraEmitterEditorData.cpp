// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterEditorData.h"

#include "NiagaraNodeFunctionCall.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "EdGraph/EdGraph.h"
#include "NiagaraStackEditorData.h"
#include "ScopedTransaction.h"
#include "NiagaraNodeAssignment.h"
#include "Rendering/Texture2DResource.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEmitterEditorData)

#define LOCTEXT_NAMESPACE "NiagaraEmitterEditorData"

const FName UNiagaraEmitterEditorData::PrivateMemberNames::SummarySections = GET_MEMBER_NAME_CHECKED(UNiagaraEmitterEditorData, SummarySections_DEPRECATED);

UNiagaraEmitterEditorData::UNiagaraEmitterEditorData(const FObjectInitializer& ObjectInitializer)
{
	StackEditorData = ObjectInitializer.CreateDefaultSubobject<UNiagaraStackEditorData>(this, TEXT("StackEditorData"));
	SummaryViewRoot = CreateDefaultSubobject<UHierarchyRoot>(TEXT("SummaryViewRoot"));
	
	if (StackEditorData != nullptr)
	{
		StackEditorData->OnPersistentDataChanged().AddUObject(this, &UNiagaraEmitterEditorData::StackEditorDataChanged);
	}
	
	PlaybackRangeMin = 0;
	PlaybackRangeMax = 10;
}

void UNiagaraEmitterEditorData::PostLoad_TransferSummaryDataToNewFormat()
{
	// ATTENTION
	// It is important that any element that is created during post load has a stable identifier.
	// This is _required_ as the merge process or just loading multiple times without saving will end up in calling this multiple times,
	// creating new objects that supposed to represent the same object, but with different IDs.
	// Summary View 1.0 only supported Module Inputs, Categories & Sections
	// - Module inputs have a stable ID by nature (function call node ID + module input ID)
	// - Sections are not stable by default (random guid when created). Here we add a stable identity using the section identifier.
	// - Categories by default are unstable as well. So we need to find a stable ID
	// ---> This is only required for post load, not in new user-created items, as those IDs will then remain stable across merges.
	
	if(SummarySections_DEPRECATED.Num() == 0 && SummaryViewFunctionInputMetadata_DEPRECATED.Num() == 0)
	{
		return;
	}
	
	// since we lack graph context during post load, we do this workaround to find the emitter graph
	UNiagaraEmitter* Emitter = GetTypedOuter<UNiagaraEmitter>();
	TArray<FNiagaraAssetVersion> AssetVersions = Emitter->GetAllAvailableVersions();
	UNiagaraGraph* EmitterGraph = nullptr;
	for(FNiagaraAssetVersion& AssetVersion : AssetVersions)
	{
		FVersionedNiagaraEmitterData* EmitterData = Emitter->GetEmitterData(AssetVersion.VersionGuid);
		if(EmitterData->GetEditorData() == this)
		{
			EmitterGraph = Cast<UNiagaraScriptSource>(EmitterData->GraphSource)->NodeGraph;
		}
	}

	auto GenerateStableIdentityForCategories = [](UHierarchyCategory* Category) -> TArray<FName>
	{
		TArray<FName> CategoryNameChain;
		
		const UHierarchyCategory* OuterCategory = Category;

		CategoryNameChain.Add(Category->GetCategoryName());

		for (; OuterCategory != nullptr; OuterCategory = OuterCategory->GetTypedOuter<UHierarchyCategory>() )
		{
			if(OuterCategory != nullptr)
			{
				CategoryNameChain.Add(OuterCategory->GetCategoryName());
			}
		}
	
		return CategoryNameChain;
	};
	
	TArray<UNiagaraNodeFunctionCall*> ModuleNodes = FNiagaraStackGraphUtilities::GetAllModuleNodes(EmitterGraph);
	
	TMap<FName, UHierarchySection*> Sections;
	TMap<FName, UHierarchyCategory*> IdentityCategoryMap;
	TMap<FName, FName> SectionCategoryMapping;
	if(SummarySections_DEPRECATED.Num() > 0)
	{
		for(FNiagaraStackSection& StackSection : SummarySections_DEPRECATED)
		{
			FText SectionDisplayName = StackSection.SectionDisplayName.IsEmptyOrWhitespace() ? FText::FromName(StackSection.SectionIdentifier) : StackSection.SectionDisplayName;
			UHierarchySection* Section = NewObject<UHierarchySection>(SummaryViewRoot, UHierarchySection::StaticClass());
			Section->SetSectionNameAsText(SectionDisplayName);
			
			FHierarchyElementIdentity SectionIdentity;
			SectionIdentity.Names.Add(StackSection.SectionIdentifier);
			Section->SetIdentity(SectionIdentity);
			
			SummaryViewRoot->GetSectionDataMutable().Insert(Section, 0);
			
			Sections.Add(StackSection.SectionIdentifier, Section);
			
			for(const FText& CategoryIdentity : StackSection.Categories)
			{
				FName CategoryName = FName(CategoryIdentity.ToString());
				UHierarchyCategory* HierarchyCategory = SummaryViewRoot->AddChild<UHierarchyCategory>();
				HierarchyCategory->SetCategoryName(CategoryName);
				
				FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchyCategory->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
				SectionAssociation->Section = Section;
				
				IdentityCategoryMap.Add(CategoryName, HierarchyCategory);
				SectionCategoryMapping.Add(CategoryName, Section->GetSectionName());
			}
		}

		SummarySections_DEPRECATED.Empty();
	}

	if(SummaryViewFunctionInputMetadata_DEPRECATED.Num() > 0)
	{
		// we have a custom sort order cached explicitly for summary view. This takes precedence.
		TMap<UHierarchyElement*, int32> SummarySortOrder;
		// the original metadata defined in the script asset for a given input.
		TMap<FHierarchyElementIdentity, FNiagaraVariableMetaData> AssetVariableMetadata;

		// since the order in the new summary view data is completely explicit (or managed implicitly by certain classes)
		// we have to sort inputs during transfer first by asset order, then summary sort order.
		for(auto It = SummaryViewFunctionInputMetadata_DEPRECATED.CreateConstIterator(); It; ++It)
		{
			// The input identity we are constructing from the function call node guid & the variable guid
			FHierarchyElementIdentity ParentInputIdentity;
			TArray<FHierarchyElementIdentity> ChildInputIdentities;

			FName OriginalCategoryName;

			UNiagaraNodeFunctionCall** MatchingFunctionCallNode = ModuleNodes.FindByPredicate([&](UNiagaraNodeFunctionCall* Candidate)
				{
					return It.Key().GetFunctionGuid() == Candidate->NodeGuid;
				});

			TObjectPtr<UNiagaraScriptVariable> ScriptVariable = nullptr;
			// there should typically be a node found here. In case it's not, it seems the node was deleted but the summary entry kept. We skip this one.
			if(MatchingFunctionCallNode != nullptr)
			{
				UNiagaraNodeFunctionCall* FunctionCall = *MatchingFunctionCallNode;
				ParentInputIdentity.Guids.Add(FunctionCall->NodeGuid);
				
				if(UNiagaraGraph* Graph = FunctionCall->GetCalledGraph())
				{
					bool bIsModuleInput = It.Key().GetInputGuid().IsValid();			

					// for module inputs we look up the variable guid in the asset graph
					if(bIsModuleInput)
					{
						TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptVariables;
						Graph->GetAllMetaData().GenerateValueArray(ScriptVariables);
						TObjectPtr<UNiagaraScriptVariable>* MatchingScriptVariable = ScriptVariables.FindByPredicate([&](TObjectPtr<UNiagaraScriptVariable> Candidate)
						{
							return Candidate->Metadata.GetVariableGuid() == It.Key().GetInputGuid();
						});

						// we should find a matching script variable. If there is none, the variable was probably deleted but the summary entry kept. We skip this one.
						if(MatchingScriptVariable != nullptr)
						{
							ScriptVariable = *MatchingScriptVariable;
							OriginalCategoryName = FName(ScriptVariable->Metadata.GetCategoryName_DEPRECATED().ToString());
							ParentInputIdentity.Guids.Add(ScriptVariable->Metadata.GetVariableGuid());
							AssetVariableMetadata.Add(ParentInputIdentity, ScriptVariable->Metadata);

							// now that we found the matching parent input, we look for child inputs
							for(UNiagaraScriptVariable* CandidateChildScriptVariable : ScriptVariables)
							{
								if(CandidateChildScriptVariable != nullptr && !CandidateChildScriptVariable->Metadata.GetParentAttribute_DEPRECATED().IsNone())
								{
									if(CandidateChildScriptVariable->Metadata.GetParentAttribute_DEPRECATED().IsEqual(ScriptVariable->Variable.GetName()))
									{
										FHierarchyElementIdentity ChildIdentity;
										ChildIdentity.Guids.Add(FunctionCall->NodeGuid);
										ChildIdentity.Guids.Add(CandidateChildScriptVariable->Metadata.GetVariableGuid());
										ChildInputIdentities.Add(ChildIdentity);
										AssetVariableMetadata.Add(ChildIdentity, CandidateChildScriptVariable->Metadata);
									}
								}
							}
						}
						else
						{
							continue;
						}
						
					}
					// for assignment inputs we have to match via name as the data doesn't contain a stable guid
					else
					{
						TArray<FNiagaraVariable> Variables;
						Graph->GetAllVariables(Variables);
						
						FNiagaraVariableBase* MatchingVariable = Variables.FindByPredicate([InputName = It.Key().GetInputName()](const FNiagaraVariable& Candidate)
						{
							return Candidate.GetName() == InputName;
						});

						// if we didn't find a matching variable we assume the summary entry is out of date and we skip it
						if(MatchingVariable != nullptr)
						{
							ParentInputIdentity.Names.Add(It.Key().GetInputName());
							OriginalCategoryName = FName(TEXT("Uncategorized"));
							// we add new empty metadata here as assignment node targets don't have metadata. This will assume a base default sort order of 0.
							AssetVariableMetadata.Add(ParentInputIdentity, FNiagaraVariableMetaData());
						}
						else
						{
							continue;
						}
					}					
				}				
			}
			else
			{
				continue;
			}

			bool bIsCategorySpecified = !It.Value().Category.IsNone();
			FName CategoryName;
			if(bIsCategorySpecified)
			{
				CategoryName = It.Value().Category;
			}
			// if the category is not specified, we use the default category that is defined in the graph
			else
			{
				CategoryName = OriginalCategoryName;
			}

			UHierarchyCategory* Category = nullptr;
			if(IdentityCategoryMap.Contains(CategoryName))
			{
				Category = IdentityCategoryMap[CategoryName];
			}
			else
			{
				Category = SummaryViewRoot->AddChild<UHierarchyCategory>();
				Category->SetCategoryName(CategoryName);
				IdentityCategoryMap.Add(Category->GetCategoryName(), Category);
			}

			if(ParentInputIdentity.Guids.Num() == 2)
			{
				UNiagaraHierarchyModuleInput* SummaryModuleInput = Category->AddChild<UNiagaraHierarchyModuleInput>();		
				SummaryModuleInput->SetIdentity(ParentInputIdentity);
				
				if(!It.Value().DisplayName.IsNone() && ScriptVariable != nullptr)
				{
					// We use FText::Format here because otherwise loading the editor data during a cook will result in a warning (FText should not be initialized from strings during a cook).
					FText NameOverride = FText::Format(LOCTEXT("NameOverrideFormat", "{0}"), FText::FromName(It.Value().DisplayName));
					SummaryModuleInput->SetDisplayNameOverride(NameOverride);
				}
				SummarySortOrder.Add(SummaryModuleInput, It.Value().SortIndex);

				for(const FHierarchyElementIdentity& ChildInputIdentity : ChildInputIdentities)
				{
					UNiagaraHierarchyModuleInput* SummaryModuleChildInput = SummaryModuleInput->AddChild<UNiagaraHierarchyModuleInput>();		
					SummaryModuleChildInput->SetIdentity(ChildInputIdentity);
					SummarySortOrder.Add(SummaryModuleChildInput, INDEX_NONE);
				}
			}
			else
			{
				UNiagaraHierarchyAssignmentInput* SummaryAssignmentInput = Category->AddChild<UNiagaraHierarchyAssignmentInput>();		
				SummaryAssignmentInput->SetIdentity(ParentInputIdentity);
				SummarySortOrder.Add(SummaryAssignmentInput, It.Value().SortIndex);
			}
		}

		// now that we have created all inputs, categories and sections, we fix up the order
		TArray<UHierarchyCategory*> TopLevelCategories;
		SummaryViewRoot->GetChildrenOfType(TopLevelCategories, false);

		// we sort by summary sort order if specified, and use the asset order as fallback.
		// advanced items always get put at the bottom
		auto SortInputsPredicate = ([&](UHierarchyElement& ItemA, UHierarchyElement& ItemB)
		{
			// while these should generally be valid, it's possible due to inheritance that a user created fluid system is executing this code despite the fluid plugin being disabled.
			// in that case, asset variable metadata might not contain these entries.
			if(AssetVariableMetadata.Contains(ItemA.GetPersistentIdentity()) == false || AssetVariableMetadata.Contains(ItemB.GetPersistentIdentity()) == false)
			{
				return false;
			}
			
			const FNiagaraVariableMetaData& MetaDataA = AssetVariableMetadata[ItemA.GetPersistentIdentity()];
			const FNiagaraVariableMetaData& MetaDataB = AssetVariableMetadata[ItemB.GetPersistentIdentity()];

			if(MetaDataA.bAdvancedDisplay && !MetaDataB.bAdvancedDisplay)
			{
				return false;
			}
			else if(!MetaDataA.bAdvancedDisplay && MetaDataB.bAdvancedDisplay)
			{
				return true;
			}
			
			int32 SortOrderA = SummarySortOrder[&ItemA];
			if(SortOrderA == INDEX_NONE)
			{
				SortOrderA = MetaDataA.GetEditorSortPriority_DEPRECATED();
			}

			int32 SortOrderB = SummarySortOrder[&ItemB];
			if(SortOrderB == INDEX_NONE)
			{
				SortOrderB = MetaDataB.GetEditorSortPriority_DEPRECATED();
			}
				
			return SortOrderA < SortOrderB;
		});

		TMap<UHierarchyElement*, int32> CategoryMinimumInputSortOrderMap;
		for(UHierarchyCategory* Category : TopLevelCategories)
		{
			Category->SortChildren(SortInputsPredicate, false);

			TArray<UNiagaraHierarchyModuleInput*> Inputs;
			Category->GetChildrenOfType<UNiagaraHierarchyModuleInput>(Inputs, false);

			// inputs can have child inputs one layer deep, so we sort them too
			for(UNiagaraHierarchyModuleInput* Input : Inputs)
			{
				Input->SortChildren(SortInputsPredicate, false);
			}

			if(Category->GetChildren().Num() > 0)
			{
				// since we have sorted the children by sort priority, child 0 has the lowest sort priority number (= highest priority)
				UHierarchyElement* MinimumItem = Category->GetChildren()[0];
				int32 MinimumCategorySortOrder = SummarySortOrder[MinimumItem];
				if(MinimumCategorySortOrder == INDEX_NONE)
				{
					if(AssetVariableMetadata.Contains(MinimumItem->GetPersistentIdentity()))
					{
						MinimumCategorySortOrder = AssetVariableMetadata[MinimumItem->GetPersistentIdentity()].GetEditorSortPriority_DEPRECATED();
					}
				}
				
				CategoryMinimumInputSortOrderMap.Add(Category, MinimumCategorySortOrder);
			}
			else
			{
				CategoryMinimumInputSortOrderMap.Add(Category, 0);
			}
		}

		auto SortCategoriesPredicate = ([&](UHierarchyElement& ItemA, UHierarchyElement& ItemB)
		{
			return CategoryMinimumInputSortOrderMap[&ItemA] < CategoryMinimumInputSortOrderMap[&ItemB];
		});

		SummaryViewRoot->SortChildren(SortCategoriesPredicate, false);

		// we now perform identity fixup for categories using the chain of category names.
		// we need to do this so the generated identity is stable instead of random.
		TArray<UHierarchyCategory*> AllCategories;
		SummaryViewRoot->GetChildrenOfType(AllCategories, true);
		
		for(UHierarchyCategory* Category : AllCategories)
		{
			FHierarchyElementIdentity StableIdentity;
			StableIdentity.Names = GenerateStableIdentityForCategories(Category);
			Category->SetIdentity(StableIdentity);
		}

		SummaryViewFunctionInputMetadata_DEPRECATED.Empty();
	}
}

void UNiagaraEmitterEditorData::PostLoad_TransferEmitterThumbnailImage(UObject* Owner)
{
	UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Owner);
	
	if(Emitter->ThumbnailImage != nullptr && EmitterThumbnail == nullptr)
	{
		SetThumbnail(Emitter->ThumbnailImage);
	}
}

void UNiagaraEmitterEditorData::PostLoad_TransferModuleStackNotesToNewFormat(UObject* Owner)
{
	// since we lack graph context during post load, we do this workaround to find the emitter graph
	UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Owner);
	TArray<FNiagaraAssetVersion> AssetVersions = Emitter->GetAllAvailableVersions();
	UNiagaraGraph* EmitterGraph = nullptr;
	for (FNiagaraAssetVersion& AssetVersion : AssetVersions)
	{
		FVersionedNiagaraEmitterData* EmitterData = Emitter->GetEmitterData(AssetVersion.VersionGuid);
		if(EmitterData->GetEditorData() == this)
		{
			if (UNiagaraScriptSource* GraphSource = Cast<UNiagaraScriptSource>(EmitterData->GraphSource))
			{
				GraphSource->ConditionalPostLoad();
				EmitterGraph = GraphSource->NodeGraph;
			}
		}
	}

	if(EmitterGraph == nullptr)
	{
		return;
	}

	TArray<UNiagaraNodeFunctionCall*> FunctionCallNodes;
	EmitterGraph->GetNodesOfClass(FunctionCallNodes);

	for(UNiagaraNodeFunctionCall* FunctionCallNode : FunctionCallNodes)
	{
		if(FunctionCallNode->GetDeprecatedCustomNotes().Num() > 0)
		{
			StackEditorData->TransferDeprecatedStackNotes(*FunctionCallNode);
		}
	}
}

void UNiagaraEmitterEditorData::PostLoad()
{
	Super::PostLoad();
	if (StackEditorData == nullptr)
	{
		StackEditorData = NewObject<UNiagaraStackEditorData>(this, TEXT("StackEditorData"), RF_Transactional);
		StackEditorData->OnPersistentDataChanged().AddUObject(this, &UNiagaraEmitterEditorData::StackEditorDataChanged);
	}
	StackEditorData->ConditionalPostLoad();

	if(SummaryViewRoot == nullptr)
	{
		SummaryViewRoot = NewObject<UHierarchyRoot>(this, TEXT("SummaryViewRoot"), RF_Transactional);
	}

	if(UPackage* Package = GetPackage())
	{
		if(Package->HasAnyPackageFlags(PKG_Cooked))
		{
			// We remove the thumbnail for cooked emitters as it can cause issues with cooked emitter being put into uncooked systems
			EmitterThumbnail = nullptr;
		}
	}
}

void UNiagaraEmitterEditorData::PostLoadFromOwner(UObject* InOwner)
{
	PostLoad_TransferSummaryDataToNewFormat();
	PostLoad_TransferEmitterThumbnailImage(InOwner);
	PostLoad_TransferModuleStackNotesToNewFormat(InOwner);
}

#if WITH_EDITORONLY_DATA
void UNiagaraEmitterEditorData::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UEdGraph::StaticClass()));
}
#endif

UNiagaraStackEditorData& UNiagaraEmitterEditorData::GetStackEditorData() const
{
	return *StackEditorData;
}

TRange<float> UNiagaraEmitterEditorData::GetPlaybackRange() const
{
	return TRange<float>(PlaybackRangeMin, PlaybackRangeMax);
}

void UNiagaraEmitterEditorData::SetPlaybackRange(TRange<float> InPlaybackRange)
{
	PlaybackRangeMin = InPlaybackRange.GetLowerBoundValue();
	PlaybackRangeMax = InPlaybackRange.GetUpperBoundValue();

	OnPersistentDataChanged().Broadcast();
}

void UNiagaraEmitterEditorData::StackEditorDataChanged()
{
	OnPersistentDataChanged().Broadcast();
}

void UNiagaraEmitterEditorData::SetShowSummaryView(bool bInShouldShowSummaryView)
{
	bShowSummaryView = bInShouldShowSummaryView;
	
	OnPersistentDataChanged().Broadcast();
	OnSummaryViewStateChangedDelegate.Broadcast();
}

void UNiagaraEmitterEditorData::ToggleShowSummaryView()
{
	FScopedTransaction ScopedTransaction(NSLOCTEXT("NiagaraEmitter", "EmitterModuleShowSummaryChanged", "Emitter summary view enabled/disabled."));
	Modify();

	SetShowSummaryView(!bShowSummaryView);
}

FSimpleMulticastDelegate& UNiagaraEmitterEditorData::OnSummaryViewStateChanged()
{
	return OnSummaryViewStateChangedDelegate;
}

TConstArrayView<const TObjectPtr<UHierarchySection>> UNiagaraEmitterEditorData::GetSummarySections() const
{
	return SummaryViewRoot->GetSectionData();
}

#undef LOCTEXT_NAMESPACE
