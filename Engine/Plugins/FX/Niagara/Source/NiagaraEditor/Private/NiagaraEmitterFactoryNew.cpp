// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterFactoryNew.h"

#include "EditorDirectories.h"
#include "NiagaraEmitter.h"
#include "NiagaraEditorModule.h"
#include "NiagaraScriptFactoryNew.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSpriteRendererProperties.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraEditorSettings.h"
#include "AssetRegistry/AssetData.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "NiagaraConstants.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraRecentAndFavoritesManager.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/STaggedAssetBrowser.h"
#include "Widgets/STaggedAssetBrowserAssetFactoryWindow.h"
#include "Widgets/AssetBrowser/SNiagaraAddEmitterToSystemWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraEmitterFactoryNew)

#define LOCTEXT_NAMESPACE "NiagaraEmitterFactory"

UNiagaraEmitterFactoryNew::UNiagaraEmitterFactoryNew()
{
	SupportedClass = UNiagaraEmitter::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
	EmitterToCopy = nullptr;
	bUseInheritance = false;
	bAddDefaultModulesAndRenderersToEmptyEmitter = true;
	SupportedWorkflows = (uint8) (EFactoryCreateWorkflow::Default | EFactoryCreateWorkflow::Asynchronous);
}

bool UNiagaraEmitterFactoryNew::ConfigureProperties()
{
	// By default, we use the default empty emitter, or a truly empty emitter.
	// This gets potentially overridden in ConfigurePropertiesAsync
	FSoftObjectPath MinimalEmitterPath = GetDefault<UNiagaraEditorSettings>()->DefaultEmptyEmitter;
	if(MinimalEmitterPath.IsValid() && MinimalEmitterPath.IsAsset())
	{
		if(UNiagaraEmitter* MinimalEmitter = Cast<UNiagaraEmitter>(MinimalEmitterPath.TryLoad()))
		{
			EmitterToCopy = MinimalEmitter;
			bUseInheritance = MinimalEmitter->bIsInheritable;
		}
	}
	
	if(EmitterToCopy.IsExplicitlyNull())
	{
		bUseInheritance = true;
		bAddDefaultModulesAndRenderersToEmptyEmitter = false;
	}
	
	return true;
}

bool UNiagaraEmitterFactoryNew::ConfigurePropertiesAsync(FOnFactoryConfigurePropertiesAsyncComplete OnComplete,	FOnFactoryConfigurePropertiesAsyncCancelled OnCancelled)
{	
	FSoftObjectPath Path = GetDefault<UNiagaraEditorSettings>()->EmitterAssetWizardConfiguration;
    UObject* ConfigUObject = Path.TryLoad();
	if(UTaggedAssetBrowserConfiguration* ConfigurationAsset = Cast<UTaggedAssetBrowserConfiguration>(ConfigUObject))
	{
		FDefaultDetailsTabConfiguration DefaultDetailsTabConfiguration;
		DefaultDetailsTabConfiguration.bUseDefaultDetailsTab = true;
		DefaultDetailsTabConfiguration.EmptySelectionMessage = LOCTEXT("EmptyEmitterFactorySelectionMessage", "Select an emitter as a starting point for your new emitter.\n");

		STaggedAssetBrowser::FInterfaceOverrideProfiles OverrideProfiles;
		OverrideProfiles.AssetViewOptionsProfileName = FName("TaggedAssetBrowser");
		OverrideProfiles.DefaultFilterMenuExpansion = EAssetTypeCategories::FX;
		OverrideProfiles.FilterBarSaveName = FName("TaggedAssetBrowser.NiagaraEmitterFactory");

		// We create a pseudo asset based on the path we currently want to create an asset at.
		// This is then used for asset reference filtering to only show assets that this pseudo asset/path would be compatible with
		
		// @TODO (ME): This is not ideal. Ideally the factory would have access to creation path and/or property handle (if created from property asset picker -> create menu)
		// then we wouldn't need to use GetLastDirectory which is relatively unsafe
		FString PackagePath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
		FString AssetName = "TmpAsset";
		FName PackageName = FName(PackagePath + "/" + AssetName);
		FAssetData PseudoReferencingAsset(PackageName, FName(PackagePath), FName(AssetName), UObject::StaticClass()->GetClassPathName());
		
		STaggedAssetBrowser::FArguments AssetBrowserArgs;
		AssetBrowserArgs
			.AvailableClasses({ UNiagaraEmitter::StaticClass() })
			.DefaultDetailsTabConfiguration(DefaultDetailsTabConfiguration)
			.InterfaceOverrideProfiles(OverrideProfiles)
			.RecentAndFavoritesList(FNiagaraEditorModule::Get().GetRecentsManager()->GetRecentEmitterAndSystemsList())
			.OnAdditionalShouldFilterAsset_UObject(this, &UNiagaraEmitterFactoryNew::OnAdditionalShouldFilterAsset)
			.OnGetExtraFrontendFilters_UObject(this, &UNiagaraEmitterFactoryNew::OnGetExtraFrontendFilters)
			.OnExtendAddFilterMenu_UObject(this, &UNiagaraEmitterFactoryNew::OnExtendAddFilterMenu)
			.AdditionalReferencingAssets({ PseudoReferencingAsset });

		SWindow::FArguments WindowArgs;
		WindowArgs.Title(LOCTEXT("EmitterAssetBrowserWindowTitle", "Create Niagara Emitter - Select an emitter as a base"));
		WindowArgs.SupportsMaximize(false);
		WindowArgs.SupportsMinimize(false);
		WindowArgs.ClientSize(FVector2D(1400, 750));
		WindowArgs.SizingRule(ESizingRule::UserSized);
		
		STaggedAssetBrowserWindow::FArguments AssetBrowserWindowArgs;
		AssetBrowserWindowArgs.AssetBrowserArgs(AssetBrowserArgs);
		AssetBrowserWindowArgs.WindowArgs(WindowArgs);

		STaggedAssetBrowserAssetFactoryWindow::FAsyncFactoryArguments AsyncArguments;
		AsyncArguments.Factory.Reset(this);
		AsyncArguments.OnAssetsActivated = FOnAssetsActivated::CreateUObject(this, &UNiagaraEmitterFactoryNew::OnAssetsActivated);
		AsyncArguments.OnFactoryConfigurePropertiesComplete = OnComplete;
		AsyncArguments.OnFactoryConfigurePropertiesCancelled = OnCancelled;
		
		TSharedRef<STaggedAssetBrowserAssetFactoryWindow> CreateAssetBrowserWindow = SNew(STaggedAssetBrowserAssetFactoryWindow, *ConfigurationAsset, *UNiagaraEmitter::StaticClass())
			.AssetBrowserWindowArgs(AssetBrowserWindowArgs)
			.AsyncFactoryArguments(AsyncArguments);
		
		FSlateApplication::Get().AddWindow(CreateAssetBrowserWindow, true);
		
		return true;
	}

	// If no browser configuration was found, we call OnComplete immediately, creating our asset
	OnComplete.ExecuteIfBound(this);
	return true;
}

UNiagaraNodeFunctionCall* AddModuleFromAssetPath(FString AssetPath, UNiagaraNodeOutput& TargetOutputNode)
{
	FSoftObjectPath AssetRef(AssetPath);
	UNiagaraScript* AssetScript = Cast<UNiagaraScript>(AssetRef.TryLoad());
	FAssetData ScriptAssetData(AssetScript);
	if (ScriptAssetData.IsValid())
	{
		return FNiagaraStackGraphUtilities::AddScriptModuleToStack(ScriptAssetData, TargetOutputNode);
	}
	else
	{
		UE_LOGF(LogNiagaraEditor, Error, "Failed to create default modules for emitter.  Missing %ls", *AssetRef.ToString());
		return nullptr;
	}
}

template<typename ValueType>
void SetRapidIterationParameter(FString UniqueEmitterName, UNiagaraScript& TargetScript, UNiagaraNodeFunctionCall& TargetFunctionCallNode,
	FName InputName, FNiagaraTypeDefinition InputType, ValueType Value)
{
	static_assert(!TIsUECoreVariant<ValueType, double>::Value, "Double core variant. Must be float type!");
	FNiagaraParameterHandle InputHandle = FNiagaraParameterHandle::CreateModuleParameterHandle(InputName);
	FNiagaraParameterHandle AliasedInputHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(InputHandle, &TargetFunctionCallNode);
	FNiagaraVariable RapidIterationParameter = FNiagaraStackGraphUtilities::CreateRapidIterationParameter(UniqueEmitterName, TargetScript.GetUsage(),
		AliasedInputHandle.GetParameterHandleString(), InputType);
	RapidIterationParameter.SetValue(Value);
	bool bAddParameterIfMissing = true;
	TargetScript.RapidIterationParameters.SetParameterData(RapidIterationParameter.GetData(), RapidIterationParameter, bAddParameterIfMissing);
}

UObject* UNiagaraEmitterFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UNiagaraEmitter::StaticClass()));

	const UNiagaraEditorSettings* Settings = GetDefault<UNiagaraEditorSettings>();
	check(Settings);

	UNiagaraEmitter* NewEmitter;

	if (EmitterToCopy.IsValid())
	{
		if (bUseInheritance)
		{
			NewEmitter = UNiagaraEmitter::CreateWithParentAndOwner(FVersionedNiagaraEmitter(EmitterToCopy.Get(), EmitterToCopy->GetExposedVersion().VersionGuid), InParent, Name, Flags);
		}
		else
		{
			NewEmitter = Cast<UNiagaraEmitter>(StaticDuplicateObject(EmitterToCopy.Get(), InParent, Name, Flags, Class));
			NewEmitter->SetUniqueEmitterName(Name.GetPlainNameString());
			NewEmitter->DisableVersioning(EmitterToCopy->GetExposedVersion().VersionGuid);
		}

		NewEmitter->bIsInheritable = true;
		NewEmitter->TemplateAssetDescription = FText();
		NewEmitter->Category = FText();
	}
	else
	{
		// Create an empty emitter, source, and graph.
		NewEmitter = NewObject<UNiagaraEmitter>(InParent, Class, Name, Flags | RF_Transactional);
		InitializeEmitter(NewEmitter, bAddDefaultModulesAndRenderersToEmptyEmitter);
	}
	
	NewEmitter->ForEachVersionData([](FVersionedNiagaraEmitterData& EmitterVersionData)
	{
		UNiagaraEmitterEditorData* EmitterEditorData = CastChecked<UNiagaraEmitterEditorData>(EmitterVersionData.GetEditorData());
		EmitterEditorData->SetShowSummaryView(EmitterVersionData.AddEmitterDefaultViewState == ENiagaraEmitterDefaultSummaryState::Summary ? true : false);
	});

	FNiagaraEditorModule::Get().GetRecentsManager()->EmitterUsed(*NewEmitter);
	return NewEmitter;
}

void UNiagaraEmitterFactoryNew::InitializeEmitter(UNiagaraEmitter* NewEmitter, bool bAddDefaultModulesAndRenderers)
{
	{
		NewEmitter->CheckVersionDataAvailable();
		FVersionedNiagaraEmitterData* EmitterData = NewEmitter->GetLatestEmitterData();
		EmitterData->SimTarget = ENiagaraSimTarget::CPUSim;

		UNiagaraScriptSource* Source = NewObject<UNiagaraScriptSource>(NewEmitter, NAME_None, RF_Transactional);
		UNiagaraGraph* CreatedGraph = NewObject<UNiagaraGraph>(Source, NAME_None, RF_Transactional);
		Source->NodeGraph = CreatedGraph;

		// Fix up source pointers.
		EmitterData->GraphSource = Source;
		EmitterData->SpawnScriptProps.Script->SetLatestSource(Source);
		EmitterData->UpdateScriptProps.Script->SetLatestSource(Source);
		EmitterData->EmitterSpawnScriptProps.Script->SetLatestSource(Source);
		EmitterData->EmitterUpdateScriptProps.Script->SetLatestSource(Source);
		EmitterData->GetGPUComputeScript()->SetLatestSource(Source);

		// Initialize the scripts for output.
		UNiagaraNodeOutput* EmitterSpawnOutputNode = FNiagaraStackGraphUtilities::ResetGraphForOutput(*Source->NodeGraph, ENiagaraScriptUsage::EmitterSpawnScript, EmitterData->EmitterSpawnScriptProps.Script->GetUsageId());
		UNiagaraNodeOutput* EmitterUpdateOutputNode = FNiagaraStackGraphUtilities::ResetGraphForOutput(*Source->NodeGraph, ENiagaraScriptUsage::EmitterUpdateScript, EmitterData->EmitterUpdateScriptProps.Script->GetUsageId());
		UNiagaraNodeOutput* ParticleSpawnOutputNode = FNiagaraStackGraphUtilities::ResetGraphForOutput(*Source->NodeGraph, ENiagaraScriptUsage::ParticleSpawnScript, EmitterData->SpawnScriptProps.Script->GetUsageId());
		UNiagaraNodeOutput* ParticleUpdateOutputNode = FNiagaraStackGraphUtilities::ResetGraphForOutput(*Source->NodeGraph, ENiagaraScriptUsage::ParticleUpdateScript, EmitterData->UpdateScriptProps.Script->GetUsageId());

		checkf(EmitterSpawnOutputNode != nullptr && EmitterUpdateOutputNode != nullptr && ParticleSpawnOutputNode != nullptr && ParticleUpdateOutputNode != nullptr,
			TEXT("Failed to create output nodes for emitter scripts."));

		if (bAddDefaultModulesAndRenderers)
		{
			NewEmitter->AddRenderer(NewObject<UNiagaraSpriteRendererProperties>(NewEmitter, "Renderer"), EmitterData->Version.VersionGuid);

			AddModuleFromAssetPath(TEXT("/Niagara/Modules/Emitter/EmitterState.EmitterState"), *EmitterUpdateOutputNode);
			
			UNiagaraNodeFunctionCall* SpawnRateNode = AddModuleFromAssetPath(TEXT("/Niagara/Modules/Emitter/SpawnRate.SpawnRate"), *EmitterUpdateOutputNode);
			if (SpawnRateNode != nullptr)
			{
				SetRapidIterationParameter(NewEmitter->GetUniqueEmitterName(), *EmitterData->EmitterUpdateScriptProps.Script, *SpawnRateNode,
					"SpawnRate", FNiagaraTypeDefinition::GetFloatDef(), 10.0f);
			}

			AddModuleFromAssetPath(TEXT("/Niagara/Modules/Spawn/Location/SystemLocation.SystemLocation"), *ParticleSpawnOutputNode);

			UNiagaraNodeFunctionCall* AddVelocityNode = AddModuleFromAssetPath(TEXT("/Niagara/Modules/Spawn/Velocity/AddVelocity.AddVelocity"), *ParticleSpawnOutputNode);
			if (AddVelocityNode != nullptr)
			{
				SetRapidIterationParameter(NewEmitter->GetUniqueEmitterName(), *EmitterData->SpawnScriptProps.Script, *AddVelocityNode,
					"Velocity", FNiagaraTypeDefinition::GetVec3Def(), FVector3f(0.0f, 0.0f, 100.0f));
			}

			TArray<FNiagaraVariable> Vars =
			{
				SYS_PARAM_PARTICLES_SPRITE_SIZE,
				SYS_PARAM_PARTICLES_SPRITE_ROTATION,
				SYS_PARAM_PARTICLES_LIFETIME
			};

			TArray<FString> Defaults = 
			{
				FNiagaraConstants::GetAttributeDefaultValue(SYS_PARAM_PARTICLES_SPRITE_SIZE),
				FNiagaraConstants::GetAttributeDefaultValue(SYS_PARAM_PARTICLES_SPRITE_ROTATION),
				FNiagaraConstants::GetAttributeDefaultValue(SYS_PARAM_PARTICLES_LIFETIME)
			};

			FNiagaraStackGraphUtilities::AddParameterModuleToStack(Vars, *ParticleSpawnOutputNode, INDEX_NONE, Defaults);

			AddModuleFromAssetPath(TEXT("/Niagara/Modules/Update/Lifetime/UpdateAge.UpdateAge"), *ParticleUpdateOutputNode);
			AddModuleFromAssetPath(TEXT("/Niagara/Modules/Update/Color/Color.Color"), *ParticleUpdateOutputNode);
			AddModuleFromAssetPath(TEXT("/Niagara/Modules/Solvers/SolveForcesAndVelocity.SolveForcesAndVelocity"), *ParticleUpdateOutputNode);
		}

		FNiagaraStackGraphUtilities::RelayoutGraph(*Source->NodeGraph);
		EmitterData->InterpolatedSpawnMode = ENiagaraInterpolatedSpawnMode::Interpolation;
		EmitterData->bDeterminism = false; // NOTE: Default to non-determinism
		EmitterData->SpawnScriptProps.Script->SetUsage(ENiagaraScriptUsage::ParticleSpawnScriptInterpolated);
	}
}

void UNiagaraEmitterFactoryNew::OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type Method)
{
	TArray<FAssetData> AssetDataCpy = AssetData;
	
	// Give the minimal emitter a chance to be selected if the user chose 'Minimal Emitter'
	if(AssetDataCpy.Num() == 0)
	{
		FSoftObjectPath MinimalEmitterPath = GetDefault<UNiagaraEditorSettings>()->DefaultEmptyEmitter;
		if(MinimalEmitterPath.IsValid() && MinimalEmitterPath.IsAsset())
		{
			if(UObject* MinimalEmitter = MinimalEmitterPath.TryLoad())
			{
				AssetDataCpy.Add(FAssetData(MinimalEmitter));
			}
		}
	}

	if(AssetDataCpy.Num() == 1)
	{
		FAssetData SelectedAsset = AssetDataCpy[0];
		ensure(SelectedAsset.GetClass() == UNiagaraEmitter::StaticClass());
		
		UNiagaraEmitter* EmitterAsset = Cast<UNiagaraEmitter>(SelectedAsset.GetAsset());
			
		EmitterToCopy = EmitterAsset;
		bUseInheritance = EmitterAsset->bIsInheritable;
	}
	else
	{
		// If we haven't selected an emitter asset, nor had a valid default empty emitter, we add a truly empty emitter
		EmitterToCopy = nullptr;
		bUseInheritance = true;
		bAddDefaultModulesAndRenderersToEmptyEmitter = false;
	}
}

bool UNiagaraEmitterFactoryNew::OnAdditionalShouldFilterAsset(const FAssetData& AssetData) const
{
	const UNiagaraEditorSettings* Settings = GetDefault<UNiagaraEditorSettings>();
	if(Settings->IsAllowedAssetInNiagaraAssetBrowser(AssetData) == false)
	{
		return true;
	}
	
	if(Settings->IsAllowedAssetByClassUsage(AssetData) == false)
	{
		return true;
	}

	return false;
}

void UNiagaraEmitterFactoryNew::OnExtendAddFilterMenu(UToolMenu* ToolMenu) const
{
	for (FToolMenuSection& Section : ToolMenu->Sections)
	{
		Section.Blocks.RemoveAll([](FToolMenuEntry& ToolMenuEntry)
		{
			return ToolMenuEntry.Name == FName("Common");
		});
	}
	
	TArray<FName> SectionsToKeep { FName("FilterBarAdvanced"), FName("FilterBarResetFilters"), FName("FX Filters"), FName("AssetFilterBarFilterAdvancedAsset"), FName("Niagara Filters"), FName("Niagara Tags") };
	ToolMenu->Sections.RemoveAll([&SectionsToKeep](FToolMenuSection& ToolMenuSection)
	{
		return SectionsToKeep.Contains(ToolMenuSection.Name) == false;
	});
}

TArray<TSharedRef<FFrontendFilter>> UNiagaraEmitterFactoryNew::OnGetExtraFrontendFilters() const
{
	TArray<TSharedRef<FFrontendFilter>> Result;
	
	TSharedRef<FFrontendFilterCategory> NiagaraAdditionalFiltersCategory = MakeShared<FFrontendFilterCategory>(LOCTEXT("NiagaraPropertyFilterCategoryLabel", "Niagara Filters"), LOCTEXT("NiagaraAdditionalFiltersTooltip", "Additional filters for filtering Niagara assets"));
	
	Result.Add(MakeShared<FFrontendFilter_NiagaraEmitterInheritance>(true, NiagaraAdditionalFiltersCategory));
	Result.Add(MakeShared<FFrontendFilter_NiagaraEmitterInheritance>(false, NiagaraAdditionalFiltersCategory));
	
	Result.Add(MakeShared<FFrontendFilter_NiagaraSystemEffectType>(true, NiagaraAdditionalFiltersCategory));
	Result.Add(MakeShared<FFrontendFilter_NiagaraSystemEffectType>(false, NiagaraAdditionalFiltersCategory));

	return Result;
}

#undef LOCTEXT_NAMESPACE

