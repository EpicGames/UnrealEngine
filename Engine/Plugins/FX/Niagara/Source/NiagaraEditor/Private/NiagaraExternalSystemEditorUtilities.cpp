// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraExternalSystemEditorUtilities.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessEmitterData.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraScriptMergeManager.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

#include "ScopedTransaction.h"

#include "AssetBuildUtilities/NiagaraAssetBuilder.h"
#include "NiagaraStackQuery.h"
#include "NiagaraStackEntryEnumerable.h"
#include "NiagaraShared.h"
#include "PropertyHandle.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackPropertyRow.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"
#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"

#include "NiagaraNodeOutput.h"
#include "NiagaraNodeAssignment.h"
#include "EdGraphSchema_Niagara.h"

#include "NiagaraEmitter.h"

#include "JsonSchema/JsonSchemaGenerator.h"
#include "JsonObjectConverter.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"

#include "UObject/PropertyAccessUtil.h"

#include "NiagaraEditorSettings.h"
#include "NiagaraScriptSource.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeParameterMapSet.h"

#include "NiagaraEmitterFactoryNew.h"

#define LOCTEXT_NAMESPACE "FNiagaraExternalSystemEditorUtilities"

//////////////////////////////////////////////////////////////////////////
// Helper functions for property access and JSON conversion

namespace
{
	/// Checks if a property should be included in JSON conversion
	/// Includes: CPF_Edit, CPF_BlueprintVisible, CPF_BlueprintAssignable, or "EDAEdit" metadata
	/// Excludes: CPF_Deprecated
	static bool ShouldExportProperty(const FProperty* Property)
	{
		if (Property->HasAnyPropertyFlags(CPF_Deprecated))
		{
			return false;
		}

		return Property->HasAnyPropertyFlags(CPF_Edit | CPF_BlueprintVisible | CPF_BlueprintAssignable) ||
		       Property->HasMetaData("EDAEdit");
	}

	/// Converts a struct to JSON, filtering properties via ShouldExportProperty
	static TSharedPtr<FJsonObject> StructToFilteredJsonObject(const UStruct* StructDefinition, const void* Struct)
	{
		TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

		for (TFieldIterator<FProperty> It(StructDefinition); It; ++It)
		{
			FProperty* Property = *It;
			if (ShouldExportProperty(Property))
			{
				const void* PropertyValue = Property->ContainerPtrToValuePtr<void>(Struct);
				TSharedPtr<FJsonValue> JsonValue = FJsonObjectConverter::UPropertyToJsonValue(
					Property,
					PropertyValue,
					0,
					0);
				if (JsonValue.IsValid())
				{
					JsonObject->SetField(Property->GetName(), JsonValue);
				}
			}
		}

		return JsonObject;
	}

	/// Gets all editable property names from a struct (excludes deprecated)
	static TArray<FName> GetAllPropertyNames(const UStruct* Struct)
	{
		TArray<FName> PropertyNames;

		//TODO: Ideally we could using the ToolsetLibrary here but that currently would require a bunch of hoop jumping to properly get the set of Names for accessible properties.
		EPropertyFlags CheckFlags = CPF_BlueprintVisible | CPF_BlueprintAssignable;
		if (GEditor)
		{
			CheckFlags |= CPF_Edit;
		}

		for (TFieldIterator<FProperty> It(Struct); It; ++It)
		{
			const FProperty* Property = *It;
			if (Property->HasAnyPropertyFlags(CheckFlags) && !Property->HasAnyPropertyFlags(CPF_Deprecated))
			{
				PropertyNames.Add(Property->GetFName());
			}
		}
		return PropertyNames;
	}

	/// Gets all editable properties from a UObject as JSON string
	static FString GetAllObjectProperties(const UObject* Object)
	{
		if (!Object)
		{
			return FString();
		}
		TArray<FName> PropertyNames = GetAllPropertyNames(Object->GetClass());
		return UNiagaraExternalEditUtilities::GetPropertyProvider().GetObjectProperties(Object, PropertyNames);
	}

	/// Converts a stack item reference to a readable path for error messages
	/// Example: "System(MySystem) / Emitter(Smoke) / Module(SpawnRate) / Input(Count)"
	static FString GetStackItemPath(const FNiagaraExt_StackItemReference& Ref)
	{
		TStringBuilder<512> PathBuilder;
		bool bNeedsDelimiter = false;

		auto AppendPart = [&PathBuilder, &bNeedsDelimiter](const TCHAR* Part)
		{
			if (bNeedsDelimiter)
			{
				PathBuilder << TEXT(" / ");
			}
			PathBuilder << Part;
			bNeedsDelimiter = true;
		};

		if (Ref.System)
		{
			PathBuilder << TEXT("System(") << Ref.System->GetName() << TEXT(")");
			bNeedsDelimiter = true;
		}

		if (Ref.EmitterName != NAME_None)
		{
			AppendPart(*FString::Printf(TEXT("Emitter(%s)"), *Ref.EmitterName.ToString()));
		}

		if (Ref.ScriptName != NAME_None)
		{
			AppendPart(*FString::Printf(TEXT("Script(%s)"), *Ref.ScriptName.ToString()));
		}

		if (Ref.ModuleName != NAME_None)
		{
			AppendPart(*FString::Printf(TEXT("Module(%s)"), *Ref.ModuleName.ToString()));
		}

		if (Ref.RendererIndex != INDEX_NONE)
		{
			AppendPart(*FString::Printf(TEXT("Renderer[%d]"), Ref.RendererIndex));
		}

		if (Ref.InputNameStack.Num() > 0)
		{
			TStringBuilder<256> InputPath;
			for (int32 i = 0; i < Ref.InputNameStack.Num(); ++i)
			{
				if (i > 0)
				{
					InputPath << TEXT(".");
				}
				InputPath << Ref.InputNameStack[i];
			}
			AppendPart(*FString::Printf(TEXT("Input(%s)"), InputPath.ToString()));
		}

		return PathBuilder.Len() > 0 ? PathBuilder.ToString() : TEXT("<empty reference>");
	}

	/// Sets properties on a UObject from JSON string, adding errors to OutErrors if failed.
	/// On rejection, points the caller at the object's property schema for self-discovery.
	///
	/// UE-379338 will replace this JSON-blob entry point (and the SetSystemData / SetRendererData
	/// / DI property paths that wrap it) with typed toolset input driven by per-class schemas.
	/// Once that lands, the schema-pointer hint emitted below becomes unreachable and should be
	/// removed along with the FString PropertyValues plumbing.
	static bool SetAllObjectProperties(UObject* Object, const FString& PropertiesJson, TArray<FText>& OutErrors)
	{
		if (!Object || PropertiesJson.IsEmpty())
		{
			return false;
		}

		bool bSuccess = UNiagaraExternalEditUtilities::GetPropertyProvider().SetObjectProperties(Object, PropertiesJson);
		if (!bSuccess)
		{
			OutErrors.Add(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "Property_Set_Failed", "Failed to set properties on {0} '{1}'. Check that property names and types are correct. Inspect the property schema for this class to discover valid property names."),
				FText::FromString(Object->GetClass()->GetName()),
				FText::FromString(Object->GetName())));
		}
		return bSuccess;
	}

	/// Gets combined EmitterHandle and EmitterData properties as JSON
	/// Merges: Handle properties (Name, bIsEnabled) + EmitterData struct properties
	static FString GetEmitterHandleAndDataProperties(UNiagaraSystem* System, int32 EmitterIndex)
	{
		if (!System || !System->GetEmitterHandles().IsValidIndex(EmitterIndex))
		{
			return FString();
		}

		TSharedPtr<FJsonObject> CombinedJson = MakeShared<FJsonObject>();

		const FNiagaraEmitterHandle& Handle = System->GetEmitterHandles()[EmitterIndex];
		CombinedJson->SetStringField(TEXT("Name"), Handle.GetName().ToString());
		CombinedJson->SetBoolField(TEXT("bIsEnabled"), Handle.GetIsEnabled());
		CombinedJson->SetStringField(TEXT("Id"), Handle.GetId().ToString(EGuidFormats::DigitsWithHyphens));
		CombinedJson->SetStringField(TEXT("IdName"), Handle.GetIdName().ToString());

		FNiagaraEmitterHandle& MutableHandle = const_cast<FNiagaraEmitterHandle&>(Handle);
		if (FVersionedNiagaraEmitterData* EmitterData = MutableHandle.GetEmitterData())
		{
			TSharedPtr<FJsonObject> EmitterDataJson = StructToFilteredJsonObject(
				FVersionedNiagaraEmitterData::StaticStruct(),
				EmitterData);

			if (EmitterDataJson.IsValid())
			{
				for (const auto& Pair : EmitterDataJson->Values)
				{
					CombinedJson->SetField(Pair.Key, Pair.Value);
				}
			}
		}

		FString OutString;
		TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&OutString);
		FJsonSerializer::Serialize(CombinedJson.ToSharedRef(), JsonWriter);
		return OutString;
	}

	/// Sets combined EmitterHandle and EmitterData properties from JSON
	/// Handles both EmitterHandle properties (Name, bIsEnabled) and EmitterData struct properties
	static bool SetEmitterHandleAndDataProperties(UNiagaraSystem* System, int32 EmitterIndex, const FString& PropertiesJson, TArray<FText>& OutErrors)
	{
		if (!System || !System->GetEmitterHandles().IsValidIndex(EmitterIndex) || PropertiesJson.IsEmpty())
		{
			return false;
		}

		const FNiagaraEmitterHandle& HandleRef = System->GetEmitterHandles()[EmitterIndex];
		FNiagaraEmitterHandle& Handle = const_cast<FNiagaraEmitterHandle&>(HandleRef);

		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(PropertiesJson);
		TSharedPtr<FJsonObject> JsonObject;
		if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid())
		{
			OutErrors.Add(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "Emitter_Property_InvalidJson", "Failed to parse JSON for emitter '{0}' properties. Check JSON syntax."),
				FText::FromName(Handle.GetName())));
			return false;
		}
		bool bSuccess = true;

		if (JsonObject->HasField(TEXT("Name")))
		{
			FString NameStr;
			if (JsonObject->TryGetStringField(TEXT("Name"), NameStr))
			{
				Handle.SetName(FName(*NameStr), *System);
			}
		}

		if (JsonObject->HasField(TEXT("bIsEnabled")))
		{
			bool bEnabled;
			if (JsonObject->TryGetBoolField(TEXT("bIsEnabled"), bEnabled))
			{
				Handle.SetIsEnabled(bEnabled, *System, true);
			}
		}

		if (FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData())
		{
			TSharedPtr<FJsonObject> EmitterDataJson = MakeShared<FJsonObject>(*JsonObject);
			EmitterDataJson->RemoveField(TEXT("Name"));
			EmitterDataJson->RemoveField(TEXT("bIsEnabled"));
			EmitterDataJson->RemoveField(TEXT("Id"));
			EmitterDataJson->RemoveField(TEXT("IdName"));

			if (!FJsonObjectConverter::JsonObjectToUStruct(EmitterDataJson.ToSharedRef(),
				FVersionedNiagaraEmitterData::StaticStruct(),
				EmitterData,
				0,
				0))
			{
				OutErrors.Add(FText::Format(
					NSLOCTEXT("NiagaraExternalEdit", "Emitter_Data_Set_Failed", "Failed to set emitter data properties on emitter '{0}'. Verify property names and types match FVersionedNiagaraEmitterData."),
					FText::FromName(Handle.GetName())));
				bSuccess = false;
			}
		}

		return bSuccess;
	}
}

//////////////////////////////////////////////////////////////////////////
// Default property provider, used when no override is installed.

namespace
{
	// Emits a "properties" sub-object. bUserVisibleOnly true restricts to editor-exposed properties;
	// false keeps every non-deprecated one.
	static FString DefaultListStructProperties(const UStruct* Struct, bool bUserVisibleOnly)
	{
		if (Struct == nullptr)
		{
			return FString();
		}

		FJsonSchemaPropertyFilter PropertyFilter;
		PropertyFilter.SkipFlags |= CPF_Deprecated;

		FJsonSchemaPropertyFilter::CustomCallback Cb;
		Cb.BindLambda([bUserVisibleOnly](const FProperty* Property, const FString& ParameterDefaultString, const TSharedRef<FJsonObject>& OutputSchema)
		{
			// Returning true skips the property.
			return bUserVisibleOnly ? !ShouldExportProperty(Property) : false;
		});
		PropertyFilter.CustomCb = &Cb;

		const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::UStructToJsonSchemaObject(Struct, PropertyFilter);
		if (!Schema.IsValid())
		{
			return FString();
		}

		const TSharedPtr<FJsonObject>* Properties = nullptr;
		if (!Schema->TryGetObjectField(TEXT("properties"), Properties) || Properties == nullptr)
		{
			return FString();
		}

		FString OutString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutString);
		FJsonSerializer::Serialize(Properties->ToSharedRef(), Writer);
		return OutString;
	}

	// No access-permission check: a property is included if it exists and converts to JSON. A provider
	// override may apply stricter filtering.
	static FString DefaultGetObjectProperties(const UObject* Object, const TArray<FName>& PropertyNames)
	{
		if (Object == nullptr)
		{
			return FString();
		}

		const TSharedRef<FJsonObject> PropertyData = MakeShared<FJsonObject>();
		for (const FName& PropertyName : PropertyNames)
		{
			FProperty* Property = PropertyAccessUtil::FindPropertyByName(PropertyName, Object->GetClass());
			if (Property == nullptr)
			{
				continue;
			}

			const TSharedPtr<FJsonValue> JsonValue = FJsonObjectConverter::UPropertyToJsonValue(Property, Property->ContainerPtrToValuePtr<void>(Object), 0, 0);
			if (JsonValue.IsValid())
			{
				PropertyData->SetField(PropertyName.ToString(), JsonValue);
			}
		}

		FString OutString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutString);
		FJsonSerializer::Serialize(PropertyData, Writer);
		return OutString;
	}

	// Direct field import, not transactional: sibling fields that resolve are applied even when others fail.
	static bool DefaultSetObjectProperties(UObject* Object, const FString& PropertiesJson)
	{
		if (Object == nullptr || PropertiesJson.IsEmpty())
		{
			return false;
		}

		const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PropertiesJson);
		TSharedPtr<FJsonObject> JsonObject;
		if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
		{
			return false;
		}

		return FJsonObjectConverter::JsonObjectToUStruct(JsonObject.ToSharedRef(), Object->GetClass(), Object, 0, 0);
	}

	FNiagaraExternalEditPropertyProvider MakeDefaultPropertyProvider()
	{
		FNiagaraExternalEditPropertyProvider Provider;
		Provider.ListStructProperties = &DefaultListStructProperties;
		Provider.GetObjectProperties  = &DefaultGetObjectProperties;
		Provider.SetObjectProperties  = &DefaultSetObjectProperties;
		return Provider;
	}

	// Process-wide active provider, lazily initialised to the default so calls work before any override is
	// installed. Mutated only at module startup/shutdown.
	FNiagaraExternalEditPropertyProvider& ActivePropertyProvider()
	{
		static FNiagaraExternalEditPropertyProvider Provider = MakeDefaultPropertyProvider();
		return Provider;
	}
}

void UNiagaraExternalEditUtilities::SetPropertyProvider(const FNiagaraExternalEditPropertyProvider& Provider)
{
	FNiagaraExternalEditPropertyProvider& Active = ActivePropertyProvider();
	Active = Provider;

	// Any callable the caller leaves unbound falls back to the default, so the active provider is always
	// fully bound and the call sites can never invoke an unset (crashing) TFunction.
	const FNiagaraExternalEditPropertyProvider Default = MakeDefaultPropertyProvider();
	if (!Active.ListStructProperties)
	{
		Active.ListStructProperties = Default.ListStructProperties;
	}
	if (!Active.GetObjectProperties)
	{
		Active.GetObjectProperties = Default.GetObjectProperties;
	}
	if (!Active.SetObjectProperties)
	{
		Active.SetObjectProperties = Default.SetObjectProperties;
	}
}

void UNiagaraExternalEditUtilities::ResetPropertyProvider()
{
	ActivePropertyProvider() = MakeDefaultPropertyProvider();
}

const FNiagaraExternalEditPropertyProvider& UNiagaraExternalEditUtilities::GetPropertyProvider()
{
	return ActivePropertyProvider();
}

//////////////////////////////////////////////////////////////////////////


bool ScriptUsageFromName(FName ScriptUsageName, ENiagaraScriptUsage& OutUsage)
{
	const UEnum* UsageEnum = StaticEnum<ENiagaraScriptUsage>();
	check(UsageEnum);

	int64 Value = UsageEnum->GetValueByName(ScriptUsageName);
	if (Value == INDEX_NONE)
	{
		return false;
	}

	OutUsage = (ENiagaraScriptUsage)Value;
	return true;
}

bool NameFromScriptUsage(ENiagaraScriptUsage Usage, FName& OutScriptName)
{
	const UEnum* UsageEnum = StaticEnum<ENiagaraScriptUsage>();
	check(UsageEnum);

	FName Name = UsageEnum->GetNameByValue((int64)Usage);
	if (Name == NAME_None)
	{
		return false;
	}

	OutScriptName = Name;
	return true;
}

struct FForEachFunctionInputOptions
{
	// Recurse into the children of a UNiagaraStackFunctionInput after Func is called on it.
	bool bRecurseIntoInputs = false;

	// When bRecurseIntoInputs is true, controls whether to descend into the children of a
	// Dynamic-mode input (the dynamic-input chain). Set false to flatten only static-switch /
	// conditional sub-inputs and stop at chain boundaries (chains are walked via GetDynamicInputChain).
	bool bRecurseIntoChainChildren = true;

	// Descend through non-input container entries (categories, groups).
	bool bRecurseIntoCategories = true;
};

// Walk every UNiagaraStackFunctionInput child of Entry, calling Func per input. The toolset
// uses unfiltered children — every input the stack knows about, regardless of which UI filters
// would drop it (static-switch-gated, VisibleCondition / EditCondition false, advanced display).
// Per-input visibility and editability are reported via FNiagaraExt_StackInputTopology::bIsVisible /
// bIsEditable; mutation endpoints enforce !bIsEditable as an error. Inline edit conditions have no
// UNiagaraStackFunctionInput of their own; the topology / value / schema endpoints synthesise them
// via TryEmitInlineEditConditionBefore inside this walk.
void ForEachFunctionInput(UNiagaraStackEntry* Entry, TFunction<bool(UNiagaraStackFunctionInput*)> Func, const FForEachFunctionInputOptions& Options = {})
{
	TArray<UNiagaraStackEntry*> Children;
	Entry->GetUnfilteredChildren(Children);

	for (UNiagaraStackEntry* ChildItem : Children)
	{
		if (UNiagaraStackFunctionInput* StackFunctionInput = Cast<UNiagaraStackFunctionInput>(ChildItem))
		{
			bool bContinue = Func(StackFunctionInput);
			if (Options.bRecurseIntoInputs)
			{
				const bool bIsChainParent = (StackFunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic);
				const bool bDescend = Options.bRecurseIntoChainChildren || !bIsChainParent;
				if (bDescend)
				{
					ForEachFunctionInput(ChildItem, Func, Options);
				}
			}
			if (!bContinue)
			{
				return;
			}
		}
		else if (Options.bRecurseIntoCategories)
		{
			ForEachFunctionInput(ChildItem, Func, Options);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

namespace
{
	// Builds a FNiagaraSystemViewModel for external edit work. bForDataProcessingOnly selects
	// the lightweight VM (the default for bulk edit paths); set it false for stack-issue reads,
	// which require the full VM because data-only VMs clear their issue arrays.
	TSharedPtr<FNiagaraSystemViewModel> CreateSystemViewModel(UNiagaraSystem& System, bool bForDataProcessingOnly)
	{
		TSharedPtr<FNiagaraSystemViewModel> VM = MakeShared<FNiagaraSystemViewModel>();
		FNiagaraSystemViewModelOptions Options;
		Options.bCanModifyEmittersFromTimeline = false;
		// We don't want to compile for edit mode. This also bypasses an ensure in which we hit OnSystemCompiled with a partially destructed VM.
		Options.bCompileForEdit                = false;
		Options.bCanSimulate                   = false;
		Options.bCanAutoCompile                = false;
		Options.bIsForDataProcessingOnly       = bForDataProcessingOnly;
		Options.MessageLogGuid                 = System.GetAssetGuid();
		VM->Initialize(System, Options);
		return VM;
	}

	// Flat-search a stack entry's flattened input tree (recurses into static-switch /
	// conditional sub-inputs, stops at dynamic-input chain boundaries). Returns the first
	// match by name, or nullptr if none found.
	UNiagaraStackFunctionInput* FindInputByName(UNiagaraStackEntry* SearchRoot, FName Name)
	{
		UNiagaraStackFunctionInput* Found = nullptr;
		ForEachFunctionInput(SearchRoot, [&](UNiagaraStackFunctionInput* Input)
			{
				if (Input->GetInputParameterHandle().GetName() == Name)
				{
					Found = Input;
					return false; // stop — match
				}
				return true;
			},
			FForEachFunctionInputOptions{ .bRecurseIntoInputs = true, .bRecurseIntoChainChildren = false });
		return Found;
	}

	// An inline edit condition has no UNiagaraStackFunctionInput of its own. HostInput is any
	// gated input that names this variable as its EditCondition; its public
	// GetEditConditionEnabled / SetEditConditionEnabled go through the same binder the editor's
	// inline checkbox uses.
	struct FInlineEditConditionInfo
	{
		UNiagaraStackFunctionInput* HostInput = nullptr;
		FName                       Name;
		FNiagaraTypeDefinition      Type;
	};

	// Per-input check; the emit and resolve helpers below are thin wrappers around this.
	bool TryGetInlineEditConditionInfo(UNiagaraStackFunctionInput* Input, FInlineEditConditionInfo& OutInfo)
	{
		if (!Input->GetShowEditConditionInline())
		{
			return false;
		}

		const TOptional<FNiagaraVariable> Var = Input->GetEditConditionVariable();
		if (!Var.IsSet())
		{
			return false;
		}

		OutInfo.HostInput = Input;
		OutInfo.Name      = Var->GetName();
		OutInfo.Type      = Var->GetType();
		return true;
	}

	// Dedup wrapper for the per-input walk: returns true the first time we see each distinct
	// edit condition. One edit condition can gate several siblings, but the value lives on the
	// script once, so emit a single entry.
	bool TryEmitInlineEditConditionBefore(UNiagaraStackFunctionInput* Input, TSet<FName>& EmittedEditConditions, FInlineEditConditionInfo& OutInfo)
	{
		if (!TryGetInlineEditConditionInfo(Input, OutInfo))
		{
			return false;
		}

		bool bAlreadyEmitted = false;
		EmittedEditConditions.Add(OutInfo.Name, &bAlreadyEmitted);
		return !bAlreadyEmitted;
	}

	// The caller's leaf names the edit-condition variable, not any UNiagaraStackFunctionInput,
	// so we scan inputs to find the host whose EditCondition matches. The engine has no reverse
	// index (edit-condition-name → host). Single-leaf only; chain-nested paths fall through.
	// Silent on miss so the standard GetInput "Input not found" error still fires.
	bool TryResolveInlineEditCondition(const FNiagaraExt_StackItemReference& Ref, FNiagaraExternalEditContext& Context, FInlineEditConditionInfo& OutInfo)
	{
		if (Ref.InputNameStack.Num() != 1)
		{
			return false;
		}

		UNiagaraStackModuleItem* Module = Ref.GetModule(Context);
		if (Module == nullptr)
		{
			return false;
		}

		const FName LeafName = Ref.InputNameStack[0];
		bool bFound = false;
		ForEachFunctionInput(Module, [&](UNiagaraStackFunctionInput* Input)
			{
				FInlineEditConditionInfo Info;
				if (TryGetInlineEditConditionInfo(Input, Info) && Info.Name == LeafName)
				{
					OutInfo = Info;
					bFound = true;
					return false; // stop on match
				}
				return true;
			},
			FForEachFunctionInputOptions{ .bRecurseIntoInputs = true, .bRecurseIntoChainChildren = false });
		return bFound;
	}

	// Visibility tracks the host (hidden / VisibleCondition). When the host isn't rendered,
	// neither is its inline checkbox. Editability follows visibility: flipping a visible
	// edit condition is how the user re-enables gated inputs, so it's always clickable.
	void PopulateInlineEditConditionTopology(const FInlineEditConditionInfo& Info, FNiagaraExt_StackInputTopology& OutTopology)
	{
		OutTopology.Name = Info.Name;
		OutTopology.Type = Info.Type;

		const bool bHostHidden      = Info.HostInput->GetIsHidden();
		const bool bHostVisCondPass = Info.HostInput->GetHasVisibleCondition() ? Info.HostInput->GetVisibleConditionEnabled() : true;
		OutTopology.bIsVisible      = !bHostHidden && bHostVisCondPass;
		OutTopology.bIsEditable     = OutTopology.bIsVisible;
		OutTopology.bIsDynamic      = false;
		OutTopology.bIsStaticSwitch = Info.Type.IsStatic();
	}

	void PopulateInlineEditConditionValue(const FInlineEditConditionInfo& Info, FNiagaraExt_StackInputValue& OutValue)
	{
		FNiagaraBool& BoolEntry = OutValue.InitializeAs<FNiagaraBool>();
		BoolEntry.SetValue(Info.HostInput->GetEditConditionEnabled());
	}

	// Schema for the synthesised toggle. Name and type come from the host's
	// GetEditConditionVariable(); the toggle has no stack input, so Category and MetaData are
	// left at their struct defaults. Resolving the toggle's metadata via UNiagaraGraph::GetMetaData
	// on the host's owning script graph would be a future improvement.
	void PopulateInlineEditConditionSchema(const FInlineEditConditionInfo& Info, FNiagaraExt_StackInputSchema& OutSchema)
	{
		OutSchema.Name                 = Info.Name;
		OutSchema.Type                 = Info.Type;
		OutSchema.bSupportsExpressions = false;
	}
}

FNiagaraExternalEditContext::FNiagaraExternalEditContext(UNiagaraSystem* System)
{
	if (System)
	{
		SystemViewModel = CreateSystemViewModel(*System, /*bForDataProcessingOnly=*/true);
	}
}

FNiagaraExternalEditContext::FNiagaraExternalEditContext(const FNiagaraExt_StackItemReference& ItemRef)
{
	UNiagaraSystem* System = ItemRef.GetSystem(*this);
	if (System)
	{
		SystemViewModel = CreateSystemViewModel(*System, /*bForDataProcessingOnly=*/true);
	}
}

bool FNiagaraExternalEditContext::CheckSystem(const UNiagaraSystem* System)
{
	if (!System)
	{
		Error(NSLOCTEXT("NiagaraExternalEdit", "System_Null", "System is null. A valid UNiagaraSystem is required for this operation."));
		return false;
	}
	return true;
}

bool FNiagaraExternalEditContext::CheckScriptAsset(const UNiagaraScript* ScriptAsset)
{
	if (!ScriptAsset)
	{
		Error(NSLOCTEXT("NiagaraExternalEdit", "Script_Asset_Null", "Script asset is null. A valid UNiagaraScript is required for this operation."));
		return false;
	}
	return true;
}


TSharedPtr<FNiagaraSystemViewModel> FNiagaraExternalEditContext::GetSystemViewModel()
{
	if (SystemViewModel.IsValid() == false)
	{
		Errors.Add(NSLOCTEXT("NiagaraExternalEdit", "SystemViewModel_Invalid", "System view model is invalid. The system may have failed to initialize properly."));
		return nullptr;
	}

	return SystemViewModel;
}

TSharedPtr<FNiagaraSystemViewModel> FNiagaraExternalEditContext::GetDiagnosticsSystemViewModel()
{
	if (!SystemViewModel.IsValid())
	{
		Errors.Add(NSLOCTEXT("NiagaraExternalEdit", "SystemViewModel_Invalid", "System view model is invalid. The system may have failed to initialize properly."));
		return nullptr;
	}

	if (!DiagnosticsSystemViewModel.IsValid())
	{
		DiagnosticsSystemViewModel = CreateSystemViewModel(SystemViewModel->GetSystem(), /*bForDataProcessingOnly=*/false);
	}

	return DiagnosticsSystemViewModel;
}

UNiagaraSystem* FNiagaraExternalEditContext::GetSystem()
{
	if (SystemViewModel)
	{
		UNiagaraSystem* Ret = &SystemViewModel->GetSystem();
		CheckSystem(Ret);
		return Ret;
	}
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraExt_StackItemReference::SetEmitter(FNiagaraEmitterHandle* Emitter)
{
	check(Emitter);
	EmitterName = Emitter->GetName();
	// Don't cache emitter pointer - it points into TArray and can become invalid
}

void FNiagaraExt_StackItemReference::SetEmitter(FName InEmitterName)
{
	EmitterName = InEmitterName;
	CachedScript = nullptr;
	CachedModule = nullptr;
	CachedInput = nullptr;
}

void FNiagaraExt_StackItemReference::SetScript(UNiagaraStackScriptItemGroup* Script)
{
	check(Script);
	CachedScript = Script;
	NameFromScriptUsage(Script->GetScriptUsage(), ScriptName);
	CachedModule = nullptr;
	CachedInput = nullptr;
}

void FNiagaraExt_StackItemReference::SetScript(FName InScriptName)
{
	ScriptName = InScriptName;
	CachedScript = nullptr;
	CachedModule = nullptr;
	CachedInput = nullptr;
}

void FNiagaraExt_StackItemReference::SetModule(UNiagaraStackModuleItem* Module)
{
	CachedModule = Module;
	ModuleName = *Module->GetModuleNode().GetFunctionName();
	CachedInput = nullptr;
}

void FNiagaraExt_StackItemReference::SetModule(FName InModuleName)
{
	ModuleName = InModuleName;
	CachedModule = nullptr;
	CachedInput = nullptr;
}

void FNiagaraExt_StackItemReference::SetInput(UNiagaraStackFunctionInput* Input)
{
	check(Input);
	InputNameStack.Reset();
	InputNameStack.Add(Input->GetInputParameterHandle().GetName());
	CachedInput = Input;
}

void FNiagaraExt_StackItemReference::SetInput(FName Name)
{
	InputNameStack.Reset();
	InputNameStack.Add(Name);
	CachedInput = nullptr;
}

void FNiagaraExt_StackItemReference::PushInput(UNiagaraStackFunctionInput* Input)
{
	CachedInput = Input;
	InputNameStack.Add(Input->GetInputParameterHandle().GetName());
}

void FNiagaraExt_StackItemReference::PushInput(FName Name)
{
	InputNameStack.Add(Name);
	CachedInput = nullptr;
}

void FNiagaraExt_StackItemReference::SetRenderer(int32 InRendererIndex)
{
	RendererIndex = InRendererIndex;
}

UNiagaraSystem* FNiagaraExt_StackItemReference::GetSystem(FNiagaraExternalEditContext& Context)const
{
	if(!System)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "StackRef_System_Missing", "System is missing in stack reference: {0}"),
			FText::FromString(GetStackItemPath(*this))));
	}
	return System;
}

FNiagaraEmitterHandle* FNiagaraExt_StackItemReference::GetEmitter(FNiagaraExternalEditContext& Context, bool bRequired)const
{
	// Always search for emitter - no caching to avoid dangling pointer issues
	if(!GetSystem(Context))
	{
		return nullptr;
	}

	if (EmitterName != NAME_None)
	{
		for (FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
		{
			if (EmitterHandle.GetName() == EmitterName)
			{
				return &EmitterHandle;
			}
		}

		// Emitter not found - build error message
		TArray<FString> AvailableEmitters;
		for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
		{
			AvailableEmitters.Add(Handle.GetName().ToString());
		}
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "StackRef_Emitter_NotFound", "Emitter '{0}' not found in system '{1}'. Available emitters: [{2}]"),
			FText::FromName(EmitterName),
			FText::FromString(System->GetName()),
			FText::FromString(FString::Join(AvailableEmitters, TEXT(", ")))));
	}
	else if(bRequired)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "StackRef_Emitter_NotSpecified", "Emitter name not specified in stack reference: {0}"),
			FText::FromString(GetStackItemPath(*this))));
	}

	return nullptr;
}

UNiagaraStackScriptItemGroup* FNiagaraExt_StackItemReference::GetScript(FNiagaraExternalEditContext& Context, bool bRequired)const
{
	// Check if cached script is still valid and hasn't been finalized (which happens when the owning context is destroyed)
	if(CachedScript.IsValid() && !CachedScript->IsFinalized())
	{
		return CachedScript.Get();
	}
	CachedScript = nullptr;

	if (ScriptName != NAME_None)
	{
		ENiagaraScriptUsage Usage;
		if (!ScriptUsageFromName(ScriptName, Usage))
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "StackRef_Script_InvalidName", "Invalid script name '{0}' in stack reference: {1}. Must be a valid ENiagaraScriptUsage (e.g., SystemSpawnScript, EmitterSpawnScript, ParticleUpdateScript)."),
				FText::FromName(ScriptName),
				FText::FromString(GetStackItemPath(*this))));
			return nullptr;
		}

		// Check if this is an emitter script or system script
		const bool bIsEmitterScript = (EmitterName != NAME_None);

		TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = Context.GetSystemViewModel();
		if(ensure(SystemViewModel))
		{
			FNiagaraStackScriptItemGroupQuery ScriptGroupQuery = bIsEmitterScript ?
				FNiagaraStackRootQuery::EmitterStackRootEntry(SystemViewModel.ToSharedRef(), EmitterName).FindScriptGroup(Usage, FGuid()) :
				FNiagaraStackRootQuery::SystemStackRootEntry(SystemViewModel.ToSharedRef()).FindScriptGroup(Usage, FGuid());

			CachedScript = ScriptGroupQuery.GetEntry();
			if (!CachedScript.IsValid())
			{
				Context.Error(FText::Format(
					NSLOCTEXT("NiagaraExternalEdit", "StackRef_Script_NotFound", "Script '{0}' not found in stack reference: {1}"),
					FText::FromName(ScriptName),
					FText::FromString(GetStackItemPath(*this))));
			}
		}
	}
	else if(bRequired)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "StackRef_Script_NotSpecified", "Script name not specified in stack reference: {0}"),
			FText::FromString(GetStackItemPath(*this))));
	}
	return CachedScript.Get();
}

UNiagaraStackModuleItem* FNiagaraExt_StackItemReference::GetModule(FNiagaraExternalEditContext& Context, bool bRequired)const
{
	// Check if cached module is still valid and hasn't been finalized (which happens when the owning context is destroyed)
	if(CachedModule.IsValid() && !CachedModule->IsFinalized())
	{
		return CachedModule.Get();
	}
	CachedModule = nullptr;

	if (ModuleName != NAME_None)
	{
		UNiagaraStackScriptItemGroup* Script = GetScript(Context);
		if (Script)
		{
			TNiagaraStackEntryEnumerable<UNiagaraStackScriptItemGroup>(*Script)
				.Children().OfType<UNiagaraStackModuleItem>()
				.Where([&](UNiagaraStackModuleItem* ModuleItem)
					{
						if (ModuleItem->GetModuleNode().GetFunctionName() == ModuleName)
						{
							CachedModule = ModuleItem;
							return true;
						}

						return false;
					});
		}

		if (!CachedModule.IsValid())
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "StackRef_Module_NotFound", "Module '{0}' not found in stack reference: {1}. Verify module exists in the specified script."),
				FText::FromName(ModuleName),
				FText::FromString(GetStackItemPath(*this))));
		}
	}
	else if(bRequired)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "StackRef_Module_NotSpecified", "Module name not specified in stack reference: {0}"),
			FText::FromString(GetStackItemPath(*this))));
	}
	return CachedModule.Get();
}

UNiagaraStackFunctionInput* FNiagaraExt_StackItemReference::GetInput(FNiagaraExternalEditContext& Context, bool bRequired) const
{
	if (CachedInput.IsValid())
	{
		return CachedInput.Get();
	}

	if (InputNameStack.IsEmpty())
	{
		if (bRequired)
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "StackRef_Input_NotSpecified",
				          "Input name not specified in stack reference: {0}"),
				FText::FromString(GetStackItemPath(*this))));
		}
		return nullptr;
	}

	UNiagaraStackModuleItem* Module = GetModule(Context, /*bRequired=*/true);
	if (Module == nullptr)
	{
		return nullptr;
	}

	// Walk the path: name 0 resolves against the module's flattened input tree; names 1..N
	// drill the dynamic-input chain — each step requires the previous step to be Dynamic-mode.
	UNiagaraStackEntry* SearchRoot = Module;
	UNiagaraStackFunctionInput* Found = nullptr;
	for (int32 i = 0; i < InputNameStack.Num(); ++i)
	{
		if (i > 0 && Found->GetValueMode() != UNiagaraStackFunctionInput::EValueMode::Dynamic)
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "StackRef_Input_NotDynamic",
				          "Input '{0}' is not a dynamic input, but more inputs are specified in the path. Only Dynamic-mode inputs expose a chain. Stack reference: {1}"),
				FText::FromName(Found->GetInputParameterHandle().GetName()),
				FText::FromString(GetStackItemPath(*this))));
			return nullptr;
		}

		Found = FindInputByName(SearchRoot, InputNameStack[i]);
		if (Found == nullptr)
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "StackRef_Input_NotFound",
				          "Input '{0}' not found in stack reference: {1}"),
				FText::FromName(InputNameStack[i]),
				FText::FromString(GetStackItemPath(*this))));
			return nullptr;
		}

		SearchRoot = Found;
	}

	CachedInput = Found;
	return Found;
}

UNiagaraRendererProperties* FNiagaraExt_StackItemReference::GetRenderer(FNiagaraExternalEditContext& Context, bool bRequired)const
{
	// Check if cached renderer is still valid
	if(RendererIndex != INDEX_NONE)
	{
		if (FNiagaraEmitterHandle* Emitter = GetEmitter(Context))
		{
			if (FVersionedNiagaraEmitterData* EmitterData = Emitter->GetEmitterData())
			{
				if (EmitterData->GetRenderers().IsValidIndex(RendererIndex))
				{
					return EmitterData->GetRenderer(RendererIndex);
				}
				else if (RendererIndex != INDEX_NONE)
				{
					const int32 RendererCount = EmitterData->GetRenderers().Num();
					Context.Error(FText::Format(
						NSLOCTEXT("NiagaraExternalEdit", "StackRef_Renderer_IndexOutOfBounds", "Renderer index {0} is out of bounds for emitter '{1}' (has {2} renderers). Stack reference: {3}"),
						RendererIndex,
						FText::FromName(Emitter->GetName()),
						RendererCount,
						FText::FromString(GetStackItemPath(*this))));
				}
			}
			//TODO: Stateless.
		}
	}
	else if(bRequired)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "StackRef_Renderer_NotSpecified", "Renderer index not specified in stack reference: {0}"),
			FText::FromString(GetStackItemPath(*this))));
	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////////

UNiagaraSystem* UNiagaraExternalEditUtilities::CreateNiagaraSystem(const FString& AssetName, const FString& AssetPath, UNiagaraSystem* TemplateSystem, FNiagaraExternalEditContext& Context)
{
	// Validate asset name
	if (AssetName.IsEmpty())
	{
		Context.Error(NSLOCTEXT("NiagaraExternalEdit", "CreateSystem_EmptyName", "Cannot create Niagara system: asset name is empty."));
		return nullptr;
	}

	// Validate asset path
	if (AssetPath.IsEmpty())
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "CreateSystem_EmptyPath", "Cannot create Niagara system '{0}': asset path is empty."),
			FText::FromString(AssetName)));
		return nullptr;
	}

	// Create system builder
	FNiagaraSystemAssetBuilder NiagaraSystemAssetBuilder = FNiagaraSystemAssetBuilder(AssetPath, AssetName);

	// If a template is provided, use it as a base. Otherwise, an empty system will be created.
	if (TemplateSystem)
	{
		NiagaraSystemAssetBuilder.WithSystemToCopy(TemplateSystem);
	}

	// Build the system
	FString ErrorStr;
	UNiagaraSystem* System = NiagaraSystemAssetBuilder.BuildSystem(ErrorStr);

	if (ErrorStr.IsEmpty() == false)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "CreateSystem_BuildFailed", "Failed to create system '{0}' at path '{1}': {2}"),
			FText::FromString(AssetName),
			FText::FromString(AssetPath),
			FText::FromString(ErrorStr)));
	}

	return System;
}

void UNiagaraExternalEditUtilities::GetAvailableDynamicInputs(const FNiagaraTypeDefinition& TypeDef, TArray<UNiagaraScript*>& OutDynamicInputScripts, FNiagaraExternalEditContext& Context)
{
	TArray<FAssetData> DynamicInputAssets;
	FNiagaraEditorUtilities::FGetFilteredScriptAssetsOptions DynamicInputScriptFilterOptions;
	DynamicInputScriptFilterOptions.ScriptUsageToInclude = ENiagaraScriptUsage::DynamicInput;
	DynamicInputScriptFilterOptions.bIncludeNonLibraryScripts = false;
	FNiagaraEditorUtilities::GetFilteredScriptAssets(DynamicInputScriptFilterOptions, DynamicInputAssets);

	FPinCollectorArray InputPins;
	TArray<UNiagaraNodeOutput*> OutputNodes;
	auto MatchesInputType = [&](UNiagaraScript* Script)
		{
			UNiagaraScriptSource* DynamicInputScriptSource = Cast<UNiagaraScriptSource>(Script->GetLatestSource());
			if (!DynamicInputScriptSource || !DynamicInputScriptSource->NodeGraph)
			{
				return false;
			}
			OutputNodes.Reset();
			DynamicInputScriptSource->NodeGraph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);
			if (OutputNodes.Num() == 1)
			{
				InputPins.Reset();
				OutputNodes[0]->GetInputPins(InputPins);
				if (InputPins.Num() == 1)
				{
					const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
					FNiagaraTypeDefinition PinType = NiagaraSchema->PinToTypeDefinition(InputPins[0]);
					return FNiagaraEditorUtilities::AreTypesAssignable(PinType, TypeDef);
				}
			}
			return false;
		};

	for (const FAssetData& DynamicInputAsset : DynamicInputAssets)
	{
		UNiagaraScript* DynamicInputScript = Cast<UNiagaraScript>(DynamicInputAsset.GetAsset());
		if (DynamicInputScript != nullptr)
		{
			if (MatchesInputType(DynamicInputScript))
			{
				OutDynamicInputScripts.Add(DynamicInputScript);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
//Schema Layer

void UNiagaraExternalEditUtilities::GetSystemSchema(FNiagaraExt_SystemSchema& OutSchema)
{
	OutSchema.PropertySchema = UNiagaraExternalEditUtilities::GetPropertyProvider().ListStructProperties(UNiagaraSystem::StaticClass(), true);
}

void UNiagaraExternalEditUtilities::GetEmitterSchema(FNiagaraExt_EmitterSchema& OutSchema)
{
	// Combine EmitterHandle properties and EmitterData properties into one schema
	TSharedPtr<FJsonObject> CombinedSchema = MakeShared<FJsonObject>();

	// Helper to filter properties for schema generation
	FJsonSchemaPropertyFilter PropertyFilter;
	FJsonSchemaPropertyFilter::CustomCallback Cb;
	Cb.BindLambda([](const FProperty* Property, const FString& ParameterDefaultString, const TSharedRef<FJsonObject>& OutputSchema)
	{
		return !ShouldExportProperty(Property); // Return true to skip
	});
	PropertyFilter.SkipFlags |= CPF_Deprecated;
	PropertyFilter.CustomCb = &Cb;

	// Get EmitterHandle struct schema (Name, bIsEnabled, Id, IdName)
	TSharedPtr<FJsonObject> HandleSchema = FJsonSchemaGenerator::UStructToJsonSchemaObject(
		FNiagaraEmitterHandle::StaticStruct(), PropertyFilter);

	// Get EmitterData struct schema
	TSharedPtr<FJsonObject> EmitterDataSchema = FJsonSchemaGenerator::UStructToJsonSchemaObject(
		FVersionedNiagaraEmitterData::StaticStruct(), PropertyFilter);

	// Merge both schemas
	if (HandleSchema.IsValid())
	{
		const TSharedPtr<FJsonObject>* HandleProperties = nullptr;
		if (HandleSchema->TryGetObjectField(TEXT("properties"), HandleProperties) && HandleProperties)
		{
			for (const auto& Pair : (*HandleProperties)->Values)
			{
				CombinedSchema->SetField(Pair.Key, Pair.Value);
			}
		}
	}

	if (EmitterDataSchema.IsValid())
	{
		const TSharedPtr<FJsonObject>* EmitterDataProperties = nullptr;
		if (EmitterDataSchema->TryGetObjectField(TEXT("properties"), EmitterDataProperties) && EmitterDataProperties)
		{
			for (const auto& Pair : (*EmitterDataProperties)->Values)
			{
				CombinedSchema->SetField(Pair.Key, Pair.Value);
			}
		}
	}

	// Convert to JSON string
	FString OutString;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&OutString);
	FJsonSerializer::Serialize(CombinedSchema.ToSharedRef(), JsonWriter);
	OutSchema.PropertySchema = OutString;
}

void UNiagaraExternalEditUtilities::GetRendererSchema(TSubclassOf<UNiagaraRendererProperties> RendererClass, FNiagaraExt_RendererSchema& OutSchema)
{
	OutSchema.PropertySchema = UNiagaraExternalEditUtilities::GetPropertyProvider().ListStructProperties(RendererClass, true);
	OutSchema.RendererClass = RendererClass;
}
void UNiagaraExternalEditUtilities::GetDataInterfaceSchema(TSubclassOf<UNiagaraDataInterface> DataInterfaceClass, FNiagaraExt_DataInterfaceSchema& OutSchema)
{
	OutSchema.PropertySchema = UNiagaraExternalEditUtilities::GetPropertyProvider().ListStructProperties(DataInterfaceClass, true);
	OutSchema.DataInterfaceClass = DataInterfaceClass;
}

void UNiagaraExternalEditUtilities::GetStackInputSchema(const FNiagaraExt_StackItemReference& InputReference, FNiagaraExt_StackInputSchema& OutSchema, FNiagaraExternalEditContext& Context)
{
	// Inline-edit-condition path: no stack-input object behind the name, synthesise from the host.
	FInlineEditConditionInfo EditConditionInfo;
	if (TryResolveInlineEditCondition(InputReference, Context, EditConditionInfo))
	{
		PopulateInlineEditConditionSchema(EditConditionInfo, OutSchema);
		return;
	}

	if (UNiagaraStackFunctionInput* Input = InputReference.GetInput(Context))
	{
		OutSchema.bSupportsExpressions = Input->SupportsCustomExpressions();
		//OutSchema.Category = TODO: Need to gather the category from the stack or asset data.
		OutSchema.Name = Input->GetInputParameterHandle().GetName();
		OutSchema.Type = Input->GetInputType();

		TOptional<FNiagaraVariableMetaData> MetaData = Input->GetInputMetaData();
		if (MetaData.IsSet())
		{
			OutSchema.MetaData = MetaData.GetValue();
		}
	}
}

void UNiagaraExternalEditUtilities::GetModuleSchema(const FNiagaraExt_StackItemReference& ModuleReference, FNiagaraExt_ModuleSchema& OutSchema, FNiagaraExternalEditContext& Context)
{
	if (UNiagaraStackModuleItem* Module = ModuleReference.GetModule(Context))
	{
		OutSchema.Asset = Module->GetModuleNode().FunctionScript;

		// Flatten static-switch / conditional sub-inputs into the schema; chain depth is
		// accessed via GetDynamicInputSchema. Inline-edit-condition entries are interleaved
		// just before the first input they gate, mirroring GetModuleTopology.
		TSet<FName> EmittedEditConditions;
		ForEachFunctionInput(Module, [&](UNiagaraStackFunctionInput* Input)
			{
				FInlineEditConditionInfo EditConditionInfo;
				if (TryEmitInlineEditConditionBefore(Input, EmittedEditConditions, EditConditionInfo))
				{
					PopulateInlineEditConditionSchema(EditConditionInfo, OutSchema.Inputs.AddDefaulted_GetRef());
				}

				FNiagaraExt_StackItemReference InputRef = ModuleReference;
				InputRef.SetInput(Input);
				GetStackInputSchema(InputRef, OutSchema.Inputs.AddDefaulted_GetRef(), Context);
				return true; // keep iterating siblings
			},
			FForEachFunctionInputOptions{ .bRecurseIntoInputs = true, .bRecurseIntoChainChildren = false });

		//TODO: Need to add output nodes to the schema/topology so that the LLM Can better understand the flow of data between modules.
// 		const UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(Module->GetModuleNode());
// 		TArray<FNiagaraVariable> OutputVariables;
// 		TArray<FNiagaraVariable> OutputVariablesWithOriginalAliasesIntact;
// 		FCompileConstantResolver ConstantResolver = FCompileConstantResolver();
// 		if (OutputNode)
// 		{
// 			FNiagaraEmitterHandle* Emitter = ModuleReference.GetEmitter(Context);
// 			if (Emitter && Emitter->GetEmitterData())
// 			{
// 				ConstantResolver = FCompileConstantResolver(*Emitter->GetEmitterData(), OutputNode->GetUsage(), ENiagaraFunctionDebugState::NoDebug);
// 			}
// 			else
// 			{
// 				ConstantResolver = FCompileConstantResolver(ModuleReference.GetSystem(), OutputNode->GetUsage(), ENiagaraFunctionDebugState::NoDebug);
// 			}
// 		}
// 		FNiagaraStackGraphUtilities::GetStackFunctionOutputVariables(Module->GetModuleNode(), ConstantResolver, OutputVariables, OutputVariablesWithOriginalAliasesIntact);
// 
// 		for (FNiagaraVariable& Var : OutputVariables)
// 		{
// 			OutSchema.Outputs.Emplace(Var.GetName(), Var.GetType());
// 		}
	}
}

void UNiagaraExternalEditUtilities::GetDynamicInputSchema(const FNiagaraExt_StackItemReference& DynamicInputReference, FNiagaraExt_DynamicInputSchema& OutSchema, FNiagaraExternalEditContext& Context)
{
	if (UNiagaraStackFunctionInput* Input = DynamicInputReference.GetInput(Context))
	{
		ensure(Input->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic);

		OutSchema.Asset = Input->GetDynamicInputNode()->FunctionScript;

		// Flatten this chain level's static-switch / conditional sub-inputs into the schema.
		// Stop at the next Dynamic-mode boundary; deeper chain levels are walked separately
		// via GetDynamicInputChain. Inline-edit-condition entries are interleaved just before
		// the first input they gate, mirroring GetModuleSchema.
		TSet<FName> EmittedEditConditions;
		ForEachFunctionInput(Input, [&](UNiagaraStackFunctionInput* Input)
			{
				FInlineEditConditionInfo EditConditionInfo;
				if (TryEmitInlineEditConditionBefore(Input, EmittedEditConditions, EditConditionInfo))
				{
					PopulateInlineEditConditionSchema(EditConditionInfo, OutSchema.Inputs.AddDefaulted_GetRef());
				}

				FNiagaraExt_StackItemReference InputRef = DynamicInputReference;
				InputRef.PushInput(Input);
				GetStackInputSchema(InputRef, OutSchema.Inputs.AddDefaulted_GetRef(), Context);
				return true; // keep iterating siblings
			},
			FForEachFunctionInputOptions{ .bRecurseIntoInputs = true, .bRecurseIntoChainChildren = false });
	}
}

void UNiagaraExternalEditUtilities::GetModuleSchema(const UNiagaraScript* ModuleAsset, FNiagaraExt_ModuleSchema& OutSchema, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckScriptAsset(ModuleAsset)) return;

	OutSchema.Asset = ModuleAsset;

	TArray<FNiagaraParameterMapHistory> Histories;

	//TODO: This is all very incomplete. A cobbled together collection from stack graph utilities. Improve or ideally move to stack graph utilities or new stack adapters.
	//TODO: Doesn't consider static variables and likely a lot of other things.
	//TODO: Versioning?
	if (const UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(ModuleAsset->GetLatestSource()))
	{
		const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();
		bool bSupportsExpressions = EditorSettings->IsVisibleClass(UNiagaraNodeCustomHlsl::StaticClass());

		if (UNiagaraGraph* Graph = Source->NodeGraph)
		{
			if (UNiagaraExternalEditUtilities::BuildParameterMapHistoriesFromScript(ModuleAsset, Histories, Context))
			{
				if (Histories.Num() == 1)
				{
					FNiagaraParameterMapHistory& History = Histories[0];
					TArray<const UEdGraphPin*> InputPins;
					FNiagaraStackGraphUtilities::ExtractInputPinsFromHistory(History, Graph, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly, InputPins);

					for (const UEdGraphPin* InputPin : InputPins)
					{
						FNiagaraVariableBase Var = UEdGraphSchema_Niagara::PinToNiagaraVariable(InputPin);
						FNiagaraExt_StackInputSchema& NewInputSchema = OutSchema.Inputs.AddDefaulted_GetRef();
						
						//Strip the module namespace so we just get the raw name of the input.
						FString NameStr = Var.GetName().ToString();
						NameStr.RemoveFromStart(PARAM_MAP_MODULE_STR);
						
						NewInputSchema.Name = *NameStr;
						NewInputSchema.Type = Var.GetType();

						NewInputSchema.bSupportsExpressions = bSupportsExpressions;

						TOptional<FNiagaraVariableMetaData> MetaData = Graph->GetMetaData(Var);
						if (MetaData.IsSet())
						{
							NewInputSchema.MetaData = MetaData.GetValue();
							//NewInputSchema.Category = MetaData->CategoryName_DEPRECATED. Need to get this from somewhere else...
						}
						else
						{
							//NewInputSchema.Category = FText::GetEmpty();
						}
					}

					for (int32 i = 0; i < History.Variables.Num(); i++)
					{
						bool bHasParameterMapSetWrite = false;
						for (const FNiagaraParameterMapHistory::FModuleScopedPin& WritePin : History.PerVariableWriteHistory[i])
						{
							if (WritePin.Pin != nullptr && WritePin.Pin->GetOwningNode() != nullptr && WritePin.Pin->GetOwningNode()->IsA<UNiagaraNodeParameterMapSet>())
							{
								bHasParameterMapSetWrite = true;
								break;
							}
						}

						if (bHasParameterMapSetWrite)
						{
							FNiagaraVariable& Variable = History.Variables[i];
							FNiagaraVariable& VariableWithOriginalAliasIntact = History.VariablesWithOriginalAliasesIntact[i];
							FNiagaraExt_Variable& NewOutputVar = OutSchema.Outputs.AddDefaulted_GetRef();
							NewOutputVar.Name = Variable.GetName();
							NewOutputVar.Type = Variable.GetType();
						}
					}
				}
				else
				{
					Context.Error(FText::Format(
						NSLOCTEXT("NiagaraExternalEdit", "Module_Graph_Malformed", "Module '{0}' has a malformed node graph. Expected 1 parameter history but found {1}."),
						FText::FromString(ModuleAsset->GetName()),
						Histories.Num()));
				}
			}
		}
	}
}

void UNiagaraExternalEditUtilities::GetDynamicInputSchema(const UNiagaraScript* ModuleAsset, FNiagaraExt_DynamicInputSchema& OutSchema, FNiagaraExternalEditContext& Context)
{
	GetModuleSchema(ModuleAsset, OutSchema, Context);
}

bool UNiagaraExternalEditUtilities::BuildParameterMapHistoriesFromScript(const UNiagaraScript* Script, TArray<FNiagaraParameterMapHistory>& OutHistories, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckScriptAsset(Script)) return false;

	const UNiagaraScriptSourceBase* Source = Script->GetLatestSource();
	if (!Source)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Script_Source_Missing", "Script '{0}' has no source. The script may be corrupted or not yet compiled."),
			FText::FromString(Script->GetName())));
		return false;
	}

	const UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(Source);
	if (!ScriptSource || !ScriptSource->NodeGraph)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Script_Graph_Missing", "Script '{0}' has no node graph. The script may be corrupted or in an invalid state."),
			FText::FromString(Script->GetName())));
		return false;
	}

	UNiagaraGraph* Graph = ScriptSource->NodeGraph;

	// Create parameter map history builder
	FNiagaraParameterMapHistoryBuilder Builder;
	Builder.SetIgnoreDisabled(false);

	// Find output nodes
	TArray<UNiagaraNodeOutput*> OutputNodes;
	Graph->FindOutputNodes(OutputNodes);

	if (OutputNodes.Num() == 0)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Script_OutputNodes_Missing", "Script '{0}' graph has no output nodes. The script may be corrupted or incomplete."),
			FText::FromString(Script->GetName())));
		return false;
	}

	// Build parameter maps for each output node
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		Builder.BeginUsage(OutputNode->GetUsage(), FName());
		Builder.EnableScriptAllowList(true, OutputNode->GetUsage());
		Builder.BuildParameterMaps(OutputNode, true);
		Builder.EndUsage();
	}

	OutHistories = MoveTemp(Builder.Histories);
	return OutHistories.Num() > 0;
}

//////////////////////////////////////////////////////////////////////////
// Topology - Describes the current topology of niagara Systems, Emitters,
// Modules and their inputs. Each endpoint is topology-only (no resolved values)
// and always walks its full subtree unconditionally.

namespace
{
	// Fills the lightweight metadata fields shared by both the Summary and Topology tiers.
	// RendererClasses is the de-duplicated set of renderer types on this emitter.
	void FillEmitterMetadata(const FNiagaraEmitterHandle& EmitterHandle, FName& OutEmitterName, bool& OutEnabled,
							 ENiagaraSimTarget& OutSimTarget, TArray<TSubclassOf<UNiagaraRendererProperties>>& OutRendererClasses)
	{
		OutEmitterName = EmitterHandle.GetName();
		OutEnabled = EmitterHandle.GetIsEnabled();

		if (const FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData())
		{
			OutSimTarget = EmitterData->SimTarget;

			const TArray<UNiagaraRendererProperties*>& Renderers = EmitterData->GetRenderers();
			OutRendererClasses.Reset();
			for (UNiagaraRendererProperties* Renderer : Renderers)
	{
				OutRendererClasses.AddUnique(Renderer->GetClass());
			}
	}
	}
}

// --- Summary tier -----------------------------------------------------------

void UNiagaraExternalEditUtilities::GetSystemSummary(UNiagaraSystem* System, FNiagaraExt_SystemSummary& OutSummary, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckSystem(System)) return;

	OutSummary.SystemName = System->GetFName();

	const FNiagaraUserRedirectionParameterStore& UserParamStore = System->GetExposedParameters();
	TArray<FNiagaraVariable> UserVariables;
	UserParamStore.GetUserParameters(UserVariables);

	OutSummary.UserVariables.Reset(UserVariables.Num());
	for (FNiagaraVariable Var : UserVariables)
	{
		UserParamStore.RedirectUserVariable(Var);

		FNiagaraExt_UserVariable& NewVarInfo = OutSummary.UserVariables.AddDefaulted_GetRef();
		NewVarInfo.Name = Var.GetName();
		NewVarInfo.Type = Var.GetType();

		FNiagaraVariant DefaultValue;
		UserParamStore.GetParameter_InternalUseOnly(Var, DefaultValue);
		NewVarInfo.DefaultValue.Set(Var.GetType(), DefaultValue);

		if (UNiagaraScriptVariable* ScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(Var, *System))
		{
			NewVarInfo.Description = ScriptVariable->Metadata.Description;
		}
	}

	OutSummary.Emitters.Reset(System->GetEmitterHandles().Num());
	for (const FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
	{
		FNiagaraExt_EmitterSummary& NewSummary = OutSummary.Emitters.AddDefaulted_GetRef();
		FillEmitterMetadata(EmitterHandle, NewSummary.EmitterName, NewSummary.bEnabled, NewSummary.SimTarget, NewSummary.RendererClasses);
	}
}

void UNiagaraExternalEditUtilities::GetEmitterSummary(const FNiagaraExt_StackItemReference& EmitterRef, FNiagaraExt_EmitterSummary& OutSummary, FNiagaraExternalEditContext& Context)
{
	const FNiagaraEmitterHandle* Emitter = EmitterRef.GetEmitter(Context);
	if (Emitter == nullptr)
	{
		return;
	}
	FillEmitterMetadata(*Emitter, OutSummary.EmitterName, OutSummary.bEnabled, OutSummary.SimTarget, OutSummary.RendererClasses);
}

// --- Topology tier ----------------------------------------------------------

void UNiagaraExternalEditUtilities::GetEmitterTopology(const FNiagaraExt_StackItemReference& EmitterRef, FNiagaraExt_EmitterTopology& OutTopology, FNiagaraExternalEditContext& Context)
{
	const FNiagaraEmitterHandle* Emitter = EmitterRef.GetEmitter(Context);
	if (Emitter == nullptr)
	{
		return;
	}

	// Always populate metadata.
	FillEmitterMetadata(*Emitter, OutTopology.EmitterName, OutTopology.bEnabled, OutTopology.SimTarget, OutTopology.RendererClasses);

	// Always walk script stacks and build renderer references.
	if (FVersionedNiagaraEmitterData* EmitterData = Emitter->GetEmitterData())
	{
		UNiagaraSystem* System = EmitterRef.GetSystem(Context);
		GetScriptStackTopology(FNiagaraExt_StackItemReference(System, OutTopology.EmitterName, TEXT("EmitterSpawnScript")), OutTopology.EmitterSpawnScript, Context);
		GetScriptStackTopology(FNiagaraExt_StackItemReference(System, OutTopology.EmitterName, TEXT("EmitterUpdateScript")), OutTopology.EmitterUpdateScript, Context);
		GetScriptStackTopology(FNiagaraExt_StackItemReference(System, OutTopology.EmitterName, TEXT("ParticleSpawnScript")), OutTopology.ParticleSpawnScript, Context);
		GetScriptStackTopology(FNiagaraExt_StackItemReference(System, OutTopology.EmitterName, TEXT("ParticleUpdateScript")), OutTopology.ParticleUpdateScript, Context);

		const TArray<UNiagaraRendererProperties*>& Renderers = EmitterData->GetRenderers();
		OutTopology.Renderers.Reset(Renderers.Num());
		for (int32 i = 0; i < Renderers.Num(); ++i)
		{
			UNiagaraRendererProperties* Renderer = Renderers[i];
			FNiagaraExt_RendererRef& RendererRef = OutTopology.Renderers.AddDefaulted_GetRef();
			RendererRef.RendererIndex = i;
			RendererRef.RendererClass = Renderer->GetClass();
		}
	}
}

void UNiagaraExternalEditUtilities::GetScriptStackTopology(const FNiagaraExt_StackItemReference& ScriptRef, FNiagaraExt_ScriptStackTopology& OutTopology, FNiagaraExternalEditContext& Context)
{
	if (UNiagaraStackScriptItemGroup* Script = ScriptRef.GetScript(Context))
	{
		OutTopology.ScriptName = ScriptRef.ScriptName;

		// Always walk modules unconditionally.
		for (UNiagaraStackModuleItem* ModuleItem :
		TNiagaraStackEntryEnumerable<UNiagaraStackScriptItemGroup>(*Script)
				.Children().OfType<UNiagaraStackModuleItem>())
				{
					FNiagaraExt_ModuleTopology& ModuleTopology = OutTopology.Modules.AddDefaulted_GetRef();

					FNiagaraExt_StackItemReference ModuleRef(ScriptRef);
					ModuleRef.SetModule(ModuleItem);
					GetModuleTopology(ModuleRef, ModuleTopology, Context);
		}
	}
}

void UNiagaraExternalEditUtilities::GetModuleTopology(const FNiagaraExt_StackItemReference& ModuleRef, FNiagaraExt_ModuleTopology& OutTopology, FNiagaraExternalEditContext& Context)
{
	if (UNiagaraStackModuleItem* Module = ModuleRef.GetModule(Context))
	{
		OutTopology.ModuleName = *Module->GetModuleNode().GetFunctionName();
		OutTopology.Enabled = Module->GetIsEnabled();
		OutTopology.ModuleScript = Module->GetModuleNode().FunctionScript;
		OutTopology.bIsSetParametersModule = (Cast<UNiagaraNodeAssignment>(&Module->GetModuleNode()) != nullptr);

		// Flatten static-switch / conditional sub-inputs into the same Inputs array as their
		// parent (script-level inputs share one flat namespace per module). Stop at Dynamic-
		// mode boundaries; those chains are accessed via GetDynamicInputChain. Inline edit
		// conditions are synthesised once, just before the first input they gate.
		TSet<FName> EmittedEditConditions;
		ForEachFunctionInput(Module, [&](UNiagaraStackFunctionInput* Input)
			{
				FInlineEditConditionInfo EditConditionInfo;
				if (TryEmitInlineEditConditionBefore(Input, EmittedEditConditions, EditConditionInfo))
				{
					PopulateInlineEditConditionTopology(EditConditionInfo, OutTopology.Inputs.AddDefaulted_GetRef());
				}

				FNiagaraExt_StackInputTopology& NewInputTopology = OutTopology.Inputs.AddDefaulted_GetRef();
				FNiagaraExt_StackItemReference InputRef(ModuleRef);
				InputRef.SetInput(Input);
				GetStackInputTopology(InputRef, NewInputTopology, Context);
				return true; // keep iterating siblings
			},
			FForEachFunctionInputOptions{ .bRecurseIntoInputs = true, .bRecurseIntoChainChildren = false });
	}
}

void UNiagaraExternalEditUtilities::GetStackInputTopology(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExt_StackInputTopology& OutTopology, FNiagaraExternalEditContext& Context)
{
	// Try inline-edit-condition resolution first; GetInput would log a misleading "Input not
	// found" since the name has no UNiagaraStackFunctionInput. Falls through on miss.
	FInlineEditConditionInfo EditConditionInfo;
	if (TryResolveInlineEditCondition(StackInputRef, Context, EditConditionInfo))
	{
		PopulateInlineEditConditionTopology(EditConditionInfo, OutTopology);
		return;
	}

	if (UNiagaraStackFunctionInput* Input = StackInputRef.GetInput(Context))
	{
		OutTopology.Name = Input->GetInputParameterHandle().GetName();
		OutTopology.Type = Input->GetInputType();

		// Compose visibility/editability from the three gating axes (static-switch / conditional
		// hiding, VisibleCondition, EditCondition). Hidden or VisibleCondition-false inputs are
		// neither visible nor editable; EditCondition-false inputs are visible but not editable.
		const bool bHidden       = Input->GetIsHidden();
		const bool bVisCondPass  = Input->GetHasVisibleCondition() ? Input->GetVisibleConditionEnabled() : true;
		const bool bEditCondPass = Input->GetHasEditCondition()    ? Input->GetEditConditionEnabled()    : true;
		OutTopology.bIsVisible      = !bHidden && bVisCondPass;
		OutTopology.bIsEditable     =  OutTopology.bIsVisible && bEditCondPass;
		OutTopology.bIsDynamic      = (Input->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic);
		OutTopology.bIsStaticSwitch = Input->IsStaticParameter();
		// No value payload — call GetStackInputData for values, or GetDynamicInputChain for the full chain.
	}
}

// --- Data tier (scope-level) ------------------------------------------------

void UNiagaraExternalEditUtilities::WalkScriptStackForInputValues(const FNiagaraExt_StackItemReference& ScriptRef, TArray<FNiagaraExt_ModuleInputValues>& OutValues, FNiagaraExternalEditContext& Context)
{
	if (UNiagaraStackScriptItemGroup* Script = ScriptRef.GetScript(Context))
	{
		for (UNiagaraStackModuleItem* ModuleItem :
			TNiagaraStackEntryEnumerable<UNiagaraStackScriptItemGroup>(*Script)
				.Children().OfType<UNiagaraStackModuleItem>())
		{
			FNiagaraExt_StackItemReference ModuleRef(ScriptRef);
			ModuleRef.SetModule(ModuleItem);

			FNiagaraExt_ModuleInputValues& ModuleValues = OutValues.AddDefaulted_GetRef();
			GetModuleInputValues(ModuleRef, ModuleValues, Context);
		}
	}
}

void UNiagaraExternalEditUtilities::GetEmitterInputValues(const FNiagaraExt_StackItemReference& EmitterRef, TArray<FNiagaraExt_ModuleInputValues>& OutValues, FNiagaraExternalEditContext& Context)
{
	const FNiagaraEmitterHandle* Emitter = EmitterRef.GetEmitter(Context);
	if (Emitter == nullptr)
	{
		return;
	}

	FName EmitterName = Emitter->GetName();
	UNiagaraSystem* System = EmitterRef.GetSystem(Context);

	WalkScriptStackForInputValues(FNiagaraExt_StackItemReference(System, EmitterName, TEXT("EmitterSpawnScript")), OutValues, Context);
	WalkScriptStackForInputValues(FNiagaraExt_StackItemReference(System, EmitterName, TEXT("EmitterUpdateScript")), OutValues, Context);
	WalkScriptStackForInputValues(FNiagaraExt_StackItemReference(System, EmitterName, TEXT("ParticleSpawnScript")), OutValues, Context);
	WalkScriptStackForInputValues(FNiagaraExt_StackItemReference(System, EmitterName, TEXT("ParticleUpdateScript")), OutValues, Context);
}

void UNiagaraExternalEditUtilities::GetScriptStackInputValues(const FNiagaraExt_StackItemReference& ScriptRef, TArray<FNiagaraExt_ModuleInputValues>& OutValues, FNiagaraExternalEditContext& Context)
{
	WalkScriptStackForInputValues(ScriptRef, OutValues, Context);
}

void UNiagaraExternalEditUtilities::GetModuleInputValues(const FNiagaraExt_StackItemReference& ModuleRef, FNiagaraExt_ModuleInputValues& OutValues, FNiagaraExternalEditContext& Context)
{
	if (UNiagaraStackModuleItem* Module = ModuleRef.GetModule(Context))
	{
		OutValues.ModuleName = *Module->GetModuleNode().GetFunctionName();

		// Mirror FNiagaraExt_ModuleTopology::Inputs: same names, same order, same edit-
		// condition interleaving (see GetModuleTopology).
		TSet<FName> EmittedEditConditions;
		ForEachFunctionInput(Module, [&](UNiagaraStackFunctionInput* Input)
			{
				FInlineEditConditionInfo EditConditionInfo;
				if (TryEmitInlineEditConditionBefore(Input, EmittedEditConditions, EditConditionInfo))
				{
					FNiagaraExt_StackInputValueEntry& EditConditionEntry = OutValues.Inputs.AddDefaulted_GetRef();
					EditConditionEntry.Name = EditConditionInfo.Name;
					PopulateInlineEditConditionValue(EditConditionInfo, EditConditionEntry.Value);
				}

				FNiagaraExt_StackItemReference InputRef(ModuleRef);
				InputRef.SetInput(Input);

				FNiagaraExt_StackInputValueEntry& Entry = OutValues.Inputs.AddDefaulted_GetRef();
				Entry.Name = Input->GetInputParameterHandle().GetName();
				Entry.Value.InitFromStackInput(InputRef, Context);
				return true; // keep iterating siblings
			},
			FForEachFunctionInputOptions{ .bRecurseIntoInputs = true, .bRecurseIntoChainChildren = false });
	}
}

// --- Dynamic-input chain endpoint -------------------------------------------

void UNiagaraExternalEditUtilities::WalkDynamicInputChain(UNiagaraStackFunctionInput* Input, const FNiagaraExt_StackItemReference& InputRef, FNiagaraExt_DynamicInputChainRef& OutChainRef, FNiagaraExternalEditContext& Context)
{
	FNiagaraExt_DynamicInputChain& OutChain = OutChainRef.GetMutable();
	OutChain.Name = Input->GetInputParameterHandle().GetName();
	OutChain.Type = Input->GetInputType();

	// Compose visibility/editability the same way as GetStackInputTopology.
	const bool bHidden       = Input->GetIsHidden();
	const bool bVisCondPass  = Input->GetHasVisibleCondition() ? Input->GetVisibleConditionEnabled() : true;
	const bool bEditCondPass = Input->GetHasEditCondition()    ? Input->GetEditConditionEnabled()    : true;
	OutChain.bIsVisible      = !bHidden && bVisCondPass;
	OutChain.bIsEditable     =  OutChain.bIsVisible && bEditCondPass;
	OutChain.bIsStaticSwitch =  Input->IsStaticParameter();

	OutChain.Value.InitFromStackInput(InputRef, Context);

	// Only Dynamic-mode inputs expose a chain. Within this level, flatten static-switch /
	// conditional sub-inputs (mirrors GetDynamicInputSchema's shape) and stop at the next
	// Dynamic-mode boundary; deeper levels are walked by the recursive call below.
	if (Input->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic)
	{
		ForEachFunctionInput(Input, [&](UNiagaraStackFunctionInput* DynamicInput)
			{
				if (DynamicInput)
				{
					FNiagaraExt_StackItemReference DynamicInputRef = InputRef;
					DynamicInputRef.PushInput(DynamicInput);
					FNiagaraExt_DynamicInputChainRef& Nested = OutChain.Inputs.AddDefaulted_GetRef();
					WalkDynamicInputChain(DynamicInput, DynamicInputRef, Nested, Context);
				}
				return true; // keep iterating siblings
			},
			FForEachFunctionInputOptions{ .bRecurseIntoInputs = true, .bRecurseIntoChainChildren = false });
	}
}

void UNiagaraExternalEditUtilities::GetDynamicInputChain(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExt_DynamicInputChainRef& OutChain, FNiagaraExternalEditContext& Context)
{
	UNiagaraStackFunctionInput* Input = StackInputRef.GetInput(Context);
	if (Input == nullptr)
	{
		return;
	}

	if (Input->GetValueMode() != UNiagaraStackFunctionInput::EValueMode::Dynamic)
	{
		Context.Error(NSLOCTEXT("NiagaraExternalEdit", "DynamicChain_NotDynamic",
			"GetDynamicInputChain: the starting input's value mode is not Dynamic. Only Dynamic-mode inputs have a chain to walk. Static-switch / conditional sub-inputs are flattened into their parent module's Inputs array — call GetModuleTopology and address sub-inputs by leaf name."));
		return;
	}

	WalkDynamicInputChain(Input, StackInputRef, OutChain, Context);
}

// --- Dependency endpoint ----------------------------------------------------

void UNiagaraExternalEditUtilities::GetSystemDependencies(UNiagaraSystem* System, FNiagaraExt_SystemDependencies& OutDependencies, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckSystem(System)) return;

	// Walk every emitter, collecting used assets as side-effects of InitFromStackInput.
	for (FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
	{
		if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData())
		{
			// Collect renderer classes.
			for (UNiagaraRendererProperties* Renderer : EmitterData->GetRenderers())
			{
				Context.UsedRenderers.Add(Renderer->GetClass());
			}

			FName EmitterName = EmitterHandle.GetName();
			auto WalkScriptForDeps = [ & ](FName ScriptName)
			{
				FNiagaraExt_StackItemReference ScriptRef(System, EmitterName, ScriptName);
				if (UNiagaraStackScriptItemGroup* Script = ScriptRef.GetScript(Context))
				{
					for (UNiagaraStackModuleItem* ModuleItem :
						TNiagaraStackEntryEnumerable<UNiagaraStackScriptItemGroup>(*Script)
							.Children().OfType<UNiagaraStackModuleItem>())
					{
						// Collect module asset.
						Context.UsedModules.Add(ModuleItem->GetModuleNode().FunctionScript);

						FNiagaraExt_StackItemReference ModuleRef(ScriptRef);
						ModuleRef.SetModule(ModuleItem);

						// Walk inputs — InitFromStackInput populates UsedDataInterfaces/UsedDynamicInputs as side-effects.
						// bRecurseIntoInputs=true required: nested DIs inside dynamic inputs are only
						// discovered by visiting every descendant stack function input.
						ForEachFunctionInput(ModuleItem, [&](UNiagaraStackFunctionInput* Input)
							{
								FNiagaraExt_StackItemReference InputRef(ModuleRef);
								InputRef.SetInput(Input);
								FNiagaraExt_StackInputValue Scratch;
								Scratch.InitFromStackInput(InputRef, Context);
								return true; // keep iterating siblings
							}, FForEachFunctionInputOptions{ .bRecurseIntoInputs = true });
					}
				}
			};

			WalkScriptForDeps(TEXT("EmitterSpawnScript"));
			WalkScriptForDeps(TEXT("EmitterUpdateScript"));
			WalkScriptForDeps(TEXT("ParticleSpawnScript"));
			WalkScriptForDeps(TEXT("ParticleUpdateScript"));
		}
	}

	// Also walk system-level scripts.
	auto WalkSystemScriptForDeps = [ & ](FName ScriptName)
	{
		FNiagaraExt_StackItemReference ScriptRef(System, NAME_None, ScriptName);
		if (UNiagaraStackScriptItemGroup* Script = ScriptRef.GetScript(Context))
		{
			for (UNiagaraStackModuleItem* ModuleItem :
				TNiagaraStackEntryEnumerable<UNiagaraStackScriptItemGroup>(*Script)
					.Children().OfType<UNiagaraStackModuleItem>())
			{
				Context.UsedModules.Add(ModuleItem->GetModuleNode().FunctionScript);

				FNiagaraExt_StackItemReference ModuleRef(ScriptRef);
				ModuleRef.SetModule(ModuleItem);

				ForEachFunctionInput(ModuleItem, [&](UNiagaraStackFunctionInput* Input)
					{
						FNiagaraExt_StackItemReference InputRef(ModuleRef);
						InputRef.SetInput(Input);
						FNiagaraExt_StackInputValue Scratch;
						Scratch.InitFromStackInput(InputRef, Context);
						return true; // keep iterating siblings
					}, FForEachFunctionInputOptions{ .bRecurseIntoInputs = true });
			}
		}
	};
	WalkSystemScriptForDeps(TEXT("SystemSpawnScript"));
	WalkSystemScriptForDeps(TEXT("SystemUpdateScript"));

	// Copy accumulated sets into the output struct.
	OutDependencies.UsedRenderers = Context.UsedRenderers.Array();
	OutDependencies.UsedDataInterfaces = Context.UsedDataInterfaces.Array();
	OutDependencies.UsedModules = Context.UsedModules.Array();
	OutDependencies.UsedDynamicInputs = Context.UsedDynamicInputs.Array();
}

//////////////////////////////////////////////////////////////////////////
//Data Layer - Access actual data values of Systems, Emitters and module inputs.

void UNiagaraExternalEditUtilities::GetUserVariables(UNiagaraSystem* System, FNiagaraExt_UserVariables& OutVariables, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckSystem(System)) return;

	const FNiagaraUserRedirectionParameterStore& UserParamStore = System->GetExposedParameters();
	TArray<FNiagaraVariable> UserVariables;
	UserParamStore.GetUserParameters(UserVariables);

	OutVariables.UserVariables.Reset(UserVariables.Num());
	for (FNiagaraVariable Var : UserVariables)
	{
		// GetUserParameters returns short redirect keys without the "User." namespace prefix.
		// RedirectUserVariable resolves to the canonical full-name variable so names round-trip correctly.
		UserParamStore.RedirectUserVariable(Var);

		FNiagaraExt_UserVariable& NewVarData = OutVariables.UserVariables.AddDefaulted_GetRef();
		NewVarData.Name = Var.GetName();
		NewVarData.Type = Var.GetType();

		FNiagaraVariant DefaultValue;
		UserParamStore.GetParameter_InternalUseOnly(Var, DefaultValue);

		NewVarData.DefaultValue.Set(Var.GetType(), DefaultValue);

		if (UNiagaraScriptVariable* ScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(Var, *System))
		{
			NewVarData.Description = ScriptVariable->Metadata.Description;
		}
	}
}

void UNiagaraExternalEditUtilities::GetSystemData(UNiagaraSystem* System, FNiagaraExt_SystemData& OutData, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckSystem(System)) return;

	OutData.PropertyValues = GetAllObjectProperties(System);
}

void UNiagaraExternalEditUtilities::GetEmitterData(const FNiagaraExt_StackItemReference& EmitterRef, FNiagaraExt_EmitterData& OutData, FNiagaraExternalEditContext& Context)
{
	if (FNiagaraEmitterHandle* Emitter = EmitterRef.GetEmitter(Context))
	{
		GetEmitterDataInternal(EmitterRef.GetSystem(Context), *Emitter, OutData, Context);
	}
}

void UNiagaraExternalEditUtilities::GetEmitterDataInternal(UNiagaraSystem* System, FNiagaraEmitterHandle& Emitter, FNiagaraExt_EmitterData& OutData, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckSystem(System)) return;

	// Find the emitter index
	int32 EmitterIndex = System->GetEmitterHandles().IndexOfByPredicate([&](const FNiagaraEmitterHandle& Handle) {
		return Handle.GetId() == Emitter.GetId();
	});

	if (EmitterIndex != INDEX_NONE)
	{
		OutData.PropertyValues = GetEmitterHandleAndDataProperties(System, EmitterIndex);
	}
	else
	{
		// Build list of available emitters for better error message
		TArray<FString> AvailableEmitters;
		for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
		{
			AvailableEmitters.Add(Handle.GetName().ToString());
		}
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Emitter_Data_Get_NotFound", "Emitter with ID '{0}' not found in system '{1}'. Available emitters: [{2}]"),
			FText::FromString(Emitter.GetId().ToString(EGuidFormats::DigitsWithHyphens)),
			FText::FromString(System->GetName()),
			FText::FromString(FString::Join(AvailableEmitters, TEXT(", ")))));
	}
}

void UNiagaraExternalEditUtilities::GetRendererData(const FNiagaraExt_StackItemReference& RendererRef, FNiagaraExt_RendererData& OutData, FNiagaraExternalEditContext& Context)
{
	UNiagaraRendererProperties* Renderer = RendererRef.GetRenderer(Context, true);
	FNiagaraEmitterHandle* Emitter = RendererRef.GetEmitter(Context, true);
	if (Emitter && Renderer)
	{
		GetRendererDataInternal(*Emitter, Renderer, OutData, Context);
	}
}

void UNiagaraExternalEditUtilities::GetRendererDataInternal(FNiagaraEmitterHandle& Emitter, UNiagaraRendererProperties* Renderer, FNiagaraExt_RendererData& OutData, FNiagaraExternalEditContext& Context)
{
	OutData.PropertyValues = GetAllObjectProperties(Renderer);
}

void UNiagaraExternalEditUtilities::GetStackInputData(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExt_StackInputValue& OutData, FNiagaraExternalEditContext& Context)
{
	// Inline-edit-condition path: no stack-input object behind the name, read via the host.
	FInlineEditConditionInfo EditConditionInfo;
	if (TryResolveInlineEditCondition(StackInputRef, Context, EditConditionInfo))
	{
		PopulateInlineEditConditionValue(EditConditionInfo, OutData);
		return;
	}

	OutData.InitFromStackInput(StackInputRef, Context);
}

void UNiagaraExternalEditUtilities::AddUserVariable(UNiagaraSystem* System, const FNiagaraExt_UserVariable& Variable, FNiagaraExternalEditContext& Context)
{
	if(!Context.CheckSystem(System))
	{
		return;
	}

	if (!Variable.DefaultValue.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ExternalAddUserVariable", "External System Add User Variable"));
	System->Modify();

	//Use this possibly? There is no corresponding remove and it doens't appear to do much beyond add, set default and being a rename.
	//FNiagaraEditorUtilities::AddParameter(NewParameter, SystemViewModel.Pin()->GetSystem().GetExposedParameters(), SystemViewModel.Pin()->GetSystem(), nullptr);

	FNiagaraVariableBase Var(Variable.Type, Variable.Name);
	FNiagaraVariant DefaultValue;
	Variable.DefaultValue.Get(DefaultValue, Context);
	System->GetExposedParameters().SetParameter_InternalUseOnly(Var, DefaultValue, true);
	if (UNiagaraScriptVariable* ScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(Var, *System))
	{
		ScriptVariable->Metadata.Description = Variable.Description;
	}
	FPropertyChangedEvent PropertyChangedEvent(nullptr, EPropertyChangeType::ValueSet);
	System->PostEditChangeProperty(PropertyChangedEvent);
}

void UNiagaraExternalEditUtilities::RemoveUserVariable(UNiagaraSystem* System, const FNiagaraExt_Variable& Variable, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckSystem(System)) return;

	const FScopedTransaction Transaction(LOCTEXT("ExternalRemoveUserVariable", "External System Remove User Variable"));
	System->Modify();

	FNiagaraTypeDefinition TypeDef;
	TypeDef = Variable.Type;

	bool bRemoved = System->GetExposedParameters().RemoveParameter(FNiagaraVariableBase(TypeDef, Variable.Name));

	if (bRemoved)
	{
		FPropertyChangedEvent PropertyChangedEvent(nullptr, EPropertyChangeType::ValueSet);
		System->PostEditChangeProperty(PropertyChangedEvent);
	}
	else
	{
		// Build list of available user variables for better error message
		TArray<FString> AvailableVariables;
		for (const FNiagaraVariableWithOffset& Param : System->GetExposedParameters().ReadParameterVariables())
		{
			AvailableVariables.Add(Param.GetName().ToString());
		}
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "UserVariable_Remove_NotFound", "User variable '{0}' not found in system '{1}'. Available variables: [{2}]"),
			FText::FromName(Variable.Name),
			FText::FromString(System->GetName()),
			FText::FromString(FString::Join(AvailableVariables, TEXT(", ")))));
	}
}

void UNiagaraExternalEditUtilities::AddEmitter(UNiagaraEmitter* TemplateEmitter, FName EmitterName, FNiagaraExt_EmitterTopology& OutTopology, FNiagaraExternalEditContext& Context)
{
	UNiagaraSystem* System = Context.GetSystem();
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = Context.GetSystemViewModel();
	if (System == nullptr || SystemViewModel == nullptr)
	{
		return;
	}

	if (TemplateEmitter == nullptr)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Emitter_Add_TemplateNull", "Cannot add emitter '{0}' to system '{1}': template emitter is null."),
			FText::FromName(EmitterName),
			FText::FromString(System->GetName())));
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ExternalAddEmitter", "External System Add Emitter"));
	System->Modify();

	FGuid NewEmitterId;
	if (TemplateEmitter)
	{
		NewEmitterId = FNiagaraEditorUtilities::AddEmitterToSystem(*System, *TemplateEmitter, TemplateEmitter->GetExposedVersion().VersionGuid, true);
	}
	//TODO: Something janky with adding an empty/default emitter. Need to investigate. For now forceing a template above.
// 	else
// 	{
// 		UNiagaraEmitter* DefaultEmitter = NewObject<UNiagaraEmitter>(System, UNiagaraEmitter::StaticClass(), EmitterName, RF_Transactional);
// 		UNiagaraEmitterFactoryNew::InitializeEmitter(DefaultEmitter, true);
// 		System->AddEmitterHandle(*DefaultEmitter, EmitterName, DefaultEmitter->GetExposedVersion().VersionGuid);
// 	}

	SystemViewModel->RefreshAll();

	FNiagaraEmitterHandle* EmitterHandle = System->GetEmitterHandles().FindByPredicate([&NewEmitterId](FNiagaraEmitterHandle& Handle) { return Handle.GetId() == NewEmitterId; });
	if (EmitterHandle)
	{
		EmitterHandle->SetName(EmitterName, *System);

		FNiagaraExt_StackItemReference EmitterRef(System);
		EmitterRef.SetEmitter(EmitterHandle);
		GetEmitterTopology(EmitterRef, OutTopology, Context);

		//Try to place emitter overview node in a sensible location.
		TArray<UNiagaraOverviewNode*> OverviewNodes;
		UNiagaraSystemEditorData* SystemEditorData = CastChecked<UNiagaraSystemEditorData>(System->GetEditorData(), ECastCheckedType::NullChecked);
		SystemEditorData->GetSystemOverviewGraph()->GetNodesOfClass<UNiagaraOverviewNode>(OverviewNodes);

		UNiagaraOverviewNode* RightMostNode = nullptr;
		UNiagaraOverviewNode* NewNode = nullptr;
		for (UNiagaraOverviewNode* OverviewNode : OverviewNodes)
		{
			if (OverviewNode->GetEmitterHandleGuid() == NewEmitterId)
			{
				NewNode = OverviewNode;
				continue;
			}
			if (RightMostNode == nullptr)
			{
				RightMostNode = OverviewNode;
				continue;
			}
			if (OverviewNode->NodePosX > RightMostNode->NodePosX)
			{
				RightMostNode = OverviewNode;
			}
		}
		if (NewNode)
		{
			FVector2D RightMostPos(0.0f);
			if (RightMostNode)
			{
				RightMostPos = FVector2D(RightMostNode->NodePosX, RightMostNode->NodePosY);
			}
			NewNode->NodePosX = RightMostPos.X + 300;
			NewNode->NodePosY = RightMostPos.Y;
		}

		System->OnSystemPostEditChange().Broadcast(System);
	}
	else
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Emitter_Add_Failed", "Failed to add emitter '{0}' from template '{1}' to system '{2}'. The emitter handle was not created."),
			FText::FromName(EmitterName),
			FText::FromString(TemplateEmitter ? TemplateEmitter->GetName() : TEXT("No Template Emitter")),
			FText::FromString(System->GetName())));
	}
}

void UNiagaraExternalEditUtilities::RemoveEmitter(const FNiagaraExt_StackItemReference& EmitterToRemove, FNiagaraExternalEditContext& Context)
{
	UNiagaraSystem* System = Context.GetSystem();
	if (!Context.CheckSystem(System)) return;

	if (FNiagaraEmitterHandle* EmitterHandle = EmitterToRemove.GetEmitter(Context))
	{
		TSet<FGuid> EmitterIdsToRemove;
		EmitterIdsToRemove.Add(EmitterHandle->GetId());
		FNiagaraEditorUtilities::KillSystemInstances(*System);

		const FScopedTransaction Transaction(LOCTEXT("ExternalRemoveEmitter", "External System Remove Emitter"));
		System->Modify();

		System->RemoveEmitterHandlesById(EmitterIdsToRemove);

		FNiagaraScriptMergeManager::Get()->ClearMergeAdapterCache();
		FNiagaraStackGraphUtilities::RebuildEmitterNodes(*System);
		UNiagaraSystemEditorData* SystemEditorData = CastChecked<UNiagaraSystemEditorData>(System->GetEditorData(), ECastCheckedType::NullChecked);
		SystemEditorData->SynchronizeOverviewGraphWithSystem(*System);

		System->OnSystemPostEditChange().Broadcast(System);
		return;
	}

	TArray<FString> AvailableEmitters;
	for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
	{
		AvailableEmitters.Add(Handle.GetName().ToString());
	}

	Context.Error(FText::Format(
		NSLOCTEXT("NiagaraExternalEdit", "Emitter_Remove_NotFound", "Cannot remove emitter '{0}' from system '{1}': emitter not found. Available emitters: [{2}]"),
		FText::FromName(EmitterToRemove.EmitterName),
		FText::FromString(System->GetName()),
		FText::FromString(FString::Join(AvailableEmitters, TEXT(", ")))));
}

void UNiagaraExternalEditUtilities::AddRenderer(const FNiagaraExt_StackItemReference& NewRendererLocation, const TSubclassOf<UNiagaraRendererProperties> RendererClass, FNiagaraExt_RendererRef& OutRef, FNiagaraExternalEditContext& Context)
{
	// Validate renderer class
	if (!RendererClass.Get())
	{
		Context.Error(NSLOCTEXT("NiagaraExternalEdit", "Renderer_Add_InvalidClass", "Cannot add renderer: renderer class is invalid or null."));
		return;
	}

	// Get the emitter handle (error already emitted by GetEmitter if not found)
	FNiagaraEmitterHandle* EmitterHandle = NewRendererLocation.GetEmitter(Context);
	if (!EmitterHandle)
	{
		return;
	}

	// Validate emitter instance
	UNiagaraEmitter* Emitter = EmitterHandle->GetInstance().Emitter;
	if (!Emitter)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Renderer_Add_InvalidEmitter", "Cannot add renderer to emitter '{0}': emitter instance is invalid."),
			FText::FromName(EmitterHandle->GetName())));
		return;
	}

	// Validate emitter data
	FVersionedNiagaraEmitterData* EmitterData = EmitterHandle->GetEmitterData();
	if (!EmitterData)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Renderer_Add_InvalidEmitterData", "Cannot add renderer to emitter '{0}': emitter data is invalid."),
			FText::FromName(EmitterHandle->GetName())));
		return;
	}

	// Create and add the new renderer
	int32 NewRendererIndex = EmitterData->GetRenderers().Num();
	UNiagaraRendererProperties* NewRenderer = NewObject<UNiagaraRendererProperties>(Emitter, RendererClass.Get());
	Emitter->AddRenderer(NewRenderer, EmitterData->Version.VersionGuid);

	// Move to specified position if requested
	if (NewRendererLocation.RendererIndex != INDEX_NONE)
	{
		NewRendererIndex = NewRendererLocation.RendererIndex;
		Emitter->MoveRenderer(NewRenderer, NewRendererIndex, EmitterData->Version.VersionGuid);
	}

	// Populate the out-ref so callers don't need a follow-up GetEmitterTopology.
	OutRef.RendererIndex = NewRendererIndex;
	OutRef.RendererClass = RendererClass;
}

void UNiagaraExternalEditUtilities::RemoveRenderer(const FNiagaraExt_StackItemReference& RendererToRemove, FNiagaraExternalEditContext& Context)
{
	// Get the emitter handle (error already emitted by GetEmitter if not found)
	FNiagaraEmitterHandle* EmitterHandle = RendererToRemove.GetEmitter(Context);
	if (!EmitterHandle)
	{
		return;
	}

	// Validate emitter instance
	UNiagaraEmitter* Emitter = EmitterHandle->GetInstance().Emitter;
	if (!Emitter)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Renderer_Remove_InvalidEmitter", "Cannot remove renderer from emitter '{0}': emitter instance is invalid."),
			FText::FromName(EmitterHandle->GetName())));
		return;
	}

	// Get the renderer (error already emitted by GetRenderer if not found)
	UNiagaraRendererProperties* Renderer = RendererToRemove.GetRenderer(Context);
	if (!Renderer)
	{
		return;
	}

	// Validate emitter data
	FVersionedNiagaraEmitterData* EmitterData = EmitterHandle->GetEmitterData();
	if (!EmitterData)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Renderer_Remove_InvalidEmitterData", "Cannot remove renderer from emitter '{0}': emitter data is invalid."),
			FText::FromName(EmitterHandle->GetName())));
		return;
	}

	// Remove the renderer
	Emitter->RemoveRenderer(Renderer, EmitterData->Version.VersionGuid);
}

void UNiagaraExternalEditUtilities::AddModule(const FNiagaraExt_StackItemReference& NewModuleLocationRef, const UNiagaraScript* ModuleAsset, FNiagaraExt_ModuleTopology& OutTopology, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckScriptAsset(ModuleAsset)) return;

	UNiagaraSystem* System = NewModuleLocationRef.GetSystem(Context);
	UNiagaraStackScriptItemGroup* ScriptItem = NewModuleLocationRef.GetScript(Context);

	if (System == nullptr || ScriptItem == nullptr)
	{
		return;
	}

	UNiagaraStackModuleItem* PreviousModule = NewModuleLocationRef.GetModule(Context, /*bRequired=*/false);
	UNiagaraNodeOutput* TargetOutputNode = ScriptItem->GetScriptOutputNode();
	int32 TargetIndex = INDEX_NONE;
	if (PreviousModule)
	{
		//TargetOutputNode = PreviousModule->GetOutputNode();
		TargetIndex = PreviousModule->GetModuleIndex() + 1;
	}

	if (TargetOutputNode)
	{
		const FScopedTransaction Transaction(LOCTEXT("ExternalAddModule", "External System Add Module"));
		System->Modify();

		FNiagaraStackGraphUtilities::FAddScriptModuleToStackArgs Args(ModuleAsset, *TargetOutputNode);
		Args.TargetIndex = TargetIndex;
		Args.VersionGuid = ModuleAsset->GetLatestScriptData()->Version.VersionGuid;//TODO: allow old versions?
		if (UNiagaraNodeFunctionCall* Func = FNiagaraStackGraphUtilities::AddScriptModuleToStack(Args))
		{
			ScriptItem->RefreshChildren();

			FPropertyChangedEvent PropertyChangedEvent(nullptr, EPropertyChangeType::ValueSet);
			System->PostEditChangeProperty(PropertyChangedEvent);

			TNiagaraStackQueryResult<UNiagaraStackModuleItem> AddModuleResult = FNiagaraStackScriptItemGroupQuery(*ScriptItem).FindModuleItem(Func->GetFunctionName()).GetResult();
			if (AddModuleResult.IsValid())
			{
				FNiagaraExt_StackItemReference NewModuleReference = NewModuleLocationRef;
				NewModuleReference.SetModule(AddModuleResult.StackEntry);
				GetModuleTopology(NewModuleReference, OutTopology, Context);
			}
		}
		else
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "Module_Add_Failed", "Failed to add module '{0}' to stack reference: {1}. The module asset may be corrupt or invalid."),
				FText::FromString(ModuleAsset->GetName()),
				FText::FromString(GetStackItemPath(NewModuleLocationRef))));
		}
	}
	else
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "Module_Add_NoOutputNode", "Failed to add module '{0}': script output node not found in stack reference: {1}"),
			FText::FromString(ModuleAsset->GetName()),
			FText::FromString(GetStackItemPath(NewModuleLocationRef))));
	}
}

namespace
{
	/**
	 * Convert a typed FNiagaraExt_VariableValue default into the legacy pin-default FString
	 * form expected by FNiagaraStackGraphUtilities::AddParameterModuleToStack /
	 * UNiagaraNodeAssignment::AddParameter. An invalid FNiagaraExt_VariableValue is treated
	 * as "no default" -- returns an empty string with no error, matching the original
	 * FString-DefaultValue behaviour where empty meant unset.
	 *
	 * Returns true on success (including the no-default case). Returns false and pushes
	 * an entry onto Context.Errors when a value was supplied but could not be coerced.
	 */
	static bool ResolveSetParameterPinDefaultString(
		const FNiagaraExt_SetParameterEntry& Entry,
		FNiagaraExternalEditContext& Context,
		FString& OutPinDefault)
	{
		OutPinDefault.Reset();

		if (!Entry.DefaultValue.IsValid())
		{
			return true; // no default supplied -- pin default stays empty
		}

		FNiagaraVariableBase Var(Entry.Variable.Type, Entry.Variable.Name);
		FNiagaraVariant Variant;
		Entry.DefaultValue.Get(Variant, Context);

		// Reject supplied defaults whose VariableValue branch doesn't match the parameter's declared type.
		// TryGetPinDefaultValueFromNiagaraVariant silently substitutes type-zero when the variant's bytes
		// don't match the variable's type and returns true, which would land us a misleading "default 0"
		// for a clearly malformed input. Catch the mismatch up front instead.
		if (!Variant.IsValid(Entry.Variable.Type))
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "SetParameter_DefaultTypeMismatch",
					"DefaultValue branch does not match parameter '{0}' (declared type '{1}'). "
					"Pick the FNiagaraExt_VariableValue branch that matches the parameter's type."),
				FText::FromName(Entry.Variable.Name),
				FText::FromString(Entry.Variable.Type.GetName())));
			return false;
		}

		const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
		if (Schema && Schema->TryGetPinDefaultValueFromNiagaraVariant(Var, Variant, OutPinDefault))
		{
			return true;
		}

		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "SetParameter_DefaultCoerceFailed",
				"Could not convert the supplied default value for parameter '{0}' (type '{1}') into a pin-default string."),
			FText::FromName(Entry.Variable.Name),
			FText::FromString(Entry.Variable.Type.GetName())));
		return false;
	}
}

void UNiagaraExternalEditUtilities::AddSetParametersModule(const FNiagaraExt_StackItemReference& NewModuleLocationRef, const TArray<FNiagaraExt_SetParameterEntry>& Parameters, FNiagaraExt_ModuleTopology& OutTopology, FNiagaraExternalEditContext& Context)
{
	if (Parameters.IsEmpty())
	{
		Context.Error(NSLOCTEXT("NiagaraExternalEdit", "SetParameters_NoParams", "AddSetParametersModule requires at least one parameter entry."));
		return;
	}

	TSet<FName> SeenNames;
	SeenNames.Reserve(Parameters.Num());
	for (const FNiagaraExt_SetParameterEntry& Entry : Parameters)
	{
		bool bAlreadyPresent = false;
		SeenNames.Add(Entry.Variable.Name, &bAlreadyPresent);
		if (bAlreadyPresent)
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "SetParameters_DuplicateName",
					"AddSetParametersModule: parameter name '{0}' appears more than once in the Parameters array."),
				FText::FromName(Entry.Variable.Name)));
			return;
		}
	}

	UNiagaraSystem* System = NewModuleLocationRef.GetSystem(Context);
	UNiagaraStackScriptItemGroup* ScriptItem = NewModuleLocationRef.GetScript(Context);

	if (System == nullptr || ScriptItem == nullptr)
	{
		return;
	}

	// PreviousModule and ScriptItem both derive from NewModuleLocationRef, so they belong to the same script.
	// TargetIndex == INDEX_NONE means "append" -- ConnectModuleNode handles this case explicitly.
	UNiagaraStackModuleItem* PreviousModule = NewModuleLocationRef.GetModule(Context, /*bRequired=*/false);
	UNiagaraNodeOutput* TargetOutputNode = ScriptItem->GetScriptOutputNode();
	int32 TargetIndex = INDEX_NONE;
	if (PreviousModule)
	{
		TargetIndex = PreviousModule->GetModuleIndex() + 1;
	}

	if (!TargetOutputNode)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "SetParameters_NoOutputNode", "Failed to add SetParameters module: script output node not found in stack reference: {0}"),
			FText::FromString(GetStackItemPath(NewModuleLocationRef))));
		return;
	}

	TArray<FNiagaraVariable> ParameterVariables;
	TArray<FString> DefaultValues;
	ParameterVariables.Reserve(Parameters.Num());
	DefaultValues.Reserve(Parameters.Num());
	for (const FNiagaraExt_SetParameterEntry& Entry : Parameters)
	{
		FString PinDefault;
		if (!ResolveSetParameterPinDefaultString(Entry, Context, PinDefault))
		{
			// Coercion error already pushed onto Context.Errors. Bail before opening a transaction
			// so we don't leave a half-built SetParameters module behind.
			return;
		}
		ParameterVariables.Emplace(Entry.Variable.Type, Entry.Variable.Name);
		DefaultValues.Add(MoveTemp(PinDefault));
	}

	FScopedTransaction Transaction(LOCTEXT("ExternalAddSetParametersModule", "External System Add Set Parameters Module"));
	System->Modify();

	if (UNiagaraNodeAssignment* AssignmentNode = FNiagaraStackGraphUtilities::AddParameterModuleToStack(ParameterVariables, *TargetOutputNode, TargetIndex, DefaultValues))
	{
		ScriptItem->RefreshChildren();

		FPropertyChangedEvent PropertyChangedEvent(nullptr, EPropertyChangeType::ValueSet);
		System->PostEditChangeProperty(PropertyChangedEvent);

		TNiagaraStackQueryResult<UNiagaraStackModuleItem> AddModuleResult = FNiagaraStackScriptItemGroupQuery(*ScriptItem).FindModuleItem(AssignmentNode->GetFunctionName()).GetResult();
		if (AddModuleResult.IsValid())
		{
			FNiagaraExt_StackItemReference NewModuleReference = NewModuleLocationRef;
			NewModuleReference.SetModule(AddModuleResult.StackEntry);
			GetModuleTopology(NewModuleReference, OutTopology, Context);
		}
		else
		{
			// Could not locate the newly-created module in the refreshed stack. Cancel the transaction
			// so the orphan module and associated graph mutations are rolled back on scope exit, leaving
			// the system in its pre-call state. OutTopology remains empty.
			Transaction.Cancel();
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "SetParameters_FindFailed",
					"SetParameters module '{0}' was created in the graph but could not be located in the "
					"refreshed stack. The add was rolled back (internal error, not actionable by caller)."),
				FText::FromString(AssignmentNode->GetFunctionName())));
		}
	}
	else
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "SetParameters_AddFailed", "Failed to add SetParameters module to stack reference: {0}"),
			FText::FromString(GetStackItemPath(NewModuleLocationRef))));
	}
}

void UNiagaraExternalEditUtilities::AddSetParameterEntry(const FNiagaraExt_StackItemReference& ModuleRef, const FNiagaraExt_SetParameterEntry& Entry, FNiagaraExt_ModuleTopology& OutTopology, FNiagaraExternalEditContext& Context)
{
	UNiagaraSystem* System = ModuleRef.GetSystem(Context);
	UNiagaraStackScriptItemGroup* ScriptItem = ModuleRef.GetScript(Context);
	UNiagaraStackModuleItem* Module = ModuleRef.GetModule(Context);

	if (System == nullptr || ScriptItem == nullptr || Module == nullptr)
	{
		return;
	}

	UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(&Module->GetModuleNode());
	if (AssignmentNode == nullptr)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "AddSetParameterEntry_NotAssignmentNode", "Module '{0}' is not a SetParameters module."),
			FText::FromName(ModuleRef.ModuleName)));
		return;
	}

	if (AssignmentNode->FindAssignmentTarget(Entry.Variable.Name) != INDEX_NONE)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "AddSetParameterEntry_DuplicateName",
				"Parameter '{0}' already exists on SetParameters module '{1}'. Remove it first if you need to change its type or default value."),
			FText::FromName(Entry.Variable.Name),
			FText::FromName(ModuleRef.ModuleName)));
		return;
	}

	FString PinDefault;
	if (!ResolveSetParameterPinDefaultString(Entry, Context, PinDefault))
	{
		// Coercion error already pushed onto Context.Errors. Bail before opening a transaction.
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ExternalAddSetParameterEntry", "External System Add Set Parameter Entry"));
	System->Modify();
	// AddParameter opens its own nested FScopedTransaction and calls Modify() on the assignment node,
	// FunctionScript, the script source, the source graph, and every node in that graph before mutating
	// anything. Nested transactions merge into this outer one, so undo correctly restores all objects.
	AssignmentNode->AddParameter(FNiagaraVariable(Entry.Variable.Type, Entry.Variable.Name), PinDefault);

	ScriptItem->RefreshChildren();

	FPropertyChangedEvent PropertyChangedEvent(nullptr, EPropertyChangeType::ValueSet);
	System->PostEditChangeProperty(PropertyChangedEvent);

	// Re-resolve the module after RefreshChildren. AddParameter can potentially modify the module function making the ModuleName in ModuleRef stale.
	TNiagaraStackQueryResult<UNiagaraStackModuleItem> UpdatedModuleResult = FNiagaraStackScriptItemGroupQuery(*ScriptItem).FindModuleItem(AssignmentNode->GetFunctionName()).GetResult();

	if (UpdatedModuleResult.IsValid())
	{
		FNiagaraExt_StackItemReference UpdatedRef = ModuleRef;
		UpdatedRef.SetModule(UpdatedModuleResult.StackEntry);
		GetModuleTopology(UpdatedRef, OutTopology, Context);
	}
	else
	{
		// Could not locate the module in the refreshed stack after AddParameter. Cancel the transaction
		// so the parameter-add and associated graph mutations are rolled back on scope exit, leaving
		// the module in its pre-call state. OutTopology remains empty.
		Transaction.Cancel();
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "AddSetParameterEntry_FindFailed",
				"Parameter was added to the graph but the SetParameters module '{0}' could not be located in the "
				"refreshed stack. The add was rolled back (internal error, not actionable by caller)."),
			FText::FromString(AssignmentNode->GetFunctionName())));
	}
}

void UNiagaraExternalEditUtilities::RemoveSetParameterEntry(const FNiagaraExt_StackItemReference& ModuleRef, FName ParameterName, FNiagaraExternalEditContext& Context)
{
	UNiagaraSystem* System = ModuleRef.GetSystem(Context);
	UNiagaraStackScriptItemGroup* ScriptItem = ModuleRef.GetScript(Context);
	UNiagaraStackModuleItem* Module = ModuleRef.GetModule(Context);

	if (System == nullptr || ScriptItem == nullptr || Module == nullptr)
	{
		return;
	}

	UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(&Module->GetModuleNode());
	if (AssignmentNode == nullptr)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "RemoveSetParameterEntry_NotAssignmentNode", "Module '{0}' is not a SetParameters module."),
			FText::FromName(ModuleRef.ModuleName)));
		return;
	}

	const int32 TargetIndex = AssignmentNode->FindAssignmentTarget(ParameterName);
	if (TargetIndex == INDEX_NONE)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "RemoveSetParameterEntry_NotFound", "Parameter '{0}' not found in SetParameters module '{1}'."),
			FText::FromName(ParameterName),
			FText::FromName(ModuleRef.ModuleName)));
		return;
	}

	const FNiagaraVariable VarToRemove = AssignmentNode->GetAssignmentTarget(TargetIndex);

	const FScopedTransaction Transaction(LOCTEXT("ExternalRemoveSetParameterEntry", "External System Remove Set Parameter Entry"));
	System->Modify();
	// RemoveParameter opens its own nested FScopedTransaction and calls Modify() on the assignment node,
	// FunctionScript, the script source, the source graph, and every node in that graph before mutating
	// anything. Nested transactions merge into this outer one, so undo correctly restores all objects.
	AssignmentNode->RemoveParameter(VarToRemove);

	ScriptItem->RefreshChildren();
	FPropertyChangedEvent PropertyChangedEvent(nullptr, EPropertyChangeType::ValueSet);
	System->PostEditChangeProperty(PropertyChangedEvent);
}

void UNiagaraExternalEditUtilities::RemoveModule(const FNiagaraExt_StackItemReference& ModuleToRemove, FNiagaraExternalEditContext& Context)
{
	UNiagaraSystem* System = ModuleToRemove.GetSystem(Context);
	if (!Context.CheckSystem(System)) return;

	FNiagaraEmitterHandle* EmitterHandle = ModuleToRemove.GetEmitter(Context, false);
	UNiagaraStackModuleItem* ModuleItemToRemove = ModuleToRemove.GetModule(Context);

	if (System == nullptr || ModuleItemToRemove == nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ExternalRemoveModule", "External System Remove Module"));
	System->Modify();
	FGuid EmitterId = EmitterHandle ? EmitterHandle->GetId() : FGuid();
	TArray<TWeakObjectPtr<UNiagaraNodeInput>> RemovedInputNodes;
	FNiagaraStackGraphUtilities::RemoveModuleFromStack(*System, EmitterId, ModuleItemToRemove->GetModuleNode(), RemovedInputNodes);
	ModuleToRemove.GetScript(Context)->RefreshChildren();
	FPropertyChangedEvent PropertyChangedEvent(nullptr, EPropertyChangeType::ValueSet);
	System->PostEditChangeProperty(PropertyChangedEvent);
}

void UNiagaraExternalEditUtilities::SetModuleEnabled(const FNiagaraExt_StackItemReference& ModuleRef, bool bEnabled, FNiagaraExternalEditContext& Context)
{
	UNiagaraSystem* System = ModuleRef.GetSystem(Context);
	UNiagaraStackModuleItem* ModuleItem = ModuleRef.GetModule(Context);
	if (System == nullptr || ModuleItem == nullptr)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ExternalSetModuleEnabled", "External System Set Module Enabled"));
	System->Modify();
	ModuleItem->SetIsEnabled(bEnabled);
}

void UNiagaraExternalEditUtilities::SetSystemData(UNiagaraSystem* System, const FNiagaraExt_SystemData& InData, FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckSystem(System)) return;
	SetAllObjectProperties(System, InData.PropertyValues, Context.Errors);
}

void UNiagaraExternalEditUtilities::SetEmitterData(const FNiagaraExt_StackItemReference& EmitterRef, const FNiagaraExt_EmitterData& InData, FNiagaraExternalEditContext& Context)
{
	if (FNiagaraEmitterHandle* Emitter = EmitterRef.GetEmitter(Context))
	{
		UNiagaraSystem* System = EmitterRef.GetSystem(Context);
		if (!System) return;

		// Find the emitter index
		int32 EmitterIndex = System->GetEmitterHandles().IndexOfByPredicate([&](const FNiagaraEmitterHandle& Handle) {
			return Handle.GetId() == Emitter->GetId();
		});

		if (EmitterIndex != INDEX_NONE)
		{
			SetEmitterHandleAndDataProperties(System, EmitterIndex, InData.PropertyValues, Context.Errors);
		}
		else
		{
			// Build list of available emitters for better error message
			TArray<FString> AvailableEmitters;
			for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
			{
				AvailableEmitters.Add(Handle.GetName().ToString());
			}
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "Emitter_Data_Set_NotFound", "Cannot set emitter data: emitter not found in system '{0}'. Stack reference: {1}. Available emitters: [{2}]"),
				FText::FromString(System->GetName()),
				FText::FromString(GetStackItemPath(EmitterRef)),
				FText::FromString(FString::Join(AvailableEmitters, TEXT(", ")))));
		}
	}
}

void UNiagaraExternalEditUtilities::SetRendererData(const FNiagaraExt_StackItemReference& RendererRef, const FNiagaraExt_RendererData& InData, FNiagaraExternalEditContext& Context)
{
	if (UNiagaraRendererProperties* RendererProps = RendererRef.GetRenderer(Context))
	{
		SetAllObjectProperties(RendererProps, InData.PropertyValues, Context.Errors);
	}
}

void UNiagaraExternalEditUtilities::SetStackInputData(const FNiagaraExt_StackItemReference& StackInputRef, const FNiagaraExt_StackInputValue& InData, FNiagaraExternalEditContext& Context)
{
	// Inline-edit-condition writes route through the host's SetEditConditionEnabled (same
	// binder as the editor's inline checkbox). The 3-axis editability check below doesn't
	// apply: these gate other inputs rather than being gated, so only host visibility
	// matters; enforced inside the branch.
	FInlineEditConditionInfo EditConditionInfo;
	if (TryResolveInlineEditCondition(StackInputRef, Context, EditConditionInfo))
	{
		const bool bHostHidden      = EditConditionInfo.HostInput->GetIsHidden();
		const bool bHostVisCondPass = EditConditionInfo.HostInput->GetHasVisibleCondition() ? EditConditionInfo.HostInput->GetVisibleConditionEnabled() : true;
		if (bHostHidden || !bHostVisCondPass)
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "SetStackInputData_InlineEditConditionHostHidden",
				          "Refusing to set inline edit condition '{0}': the inputs it gates are not currently active (hidden by static-switch / conditional logic). Flip the upstream switch first."),
				FText::FromName(EditConditionInfo.Name)));
			return;
		}

		const FNiagaraBool* BoolPtr = InData.GetPtr<FNiagaraBool>();
		if (BoolPtr == nullptr)
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "SetStackInputData_InlineEditConditionNotBool",
				          "Type mismatch for inline edit condition '{0}': value must be a NiagaraBool."),
				FText::FromName(EditConditionInfo.Name)));
			return;
		}

		const FScopedTransaction Transaction(LOCTEXT("ExternalSetInlineEditCondition", "External System Set Inline Edit Condition"));
		EditConditionInfo.HostInput->SetEditConditionEnabled(BoolPtr->GetValue());
		return;
	}

	// Refuse to mutate non-editable inputs. The three gating axes mirror GetStackInputTopology's
	// read path — keep in sync. Each rejection points the caller at the gating input to flip first.
	UNiagaraStackFunctionInput* Input = StackInputRef.GetInput(Context);
	if (Input == nullptr)
	{
		// GetInput already logged the failure; don't fall through to InitStackInputFromValue
		// because it would re-resolve and log the same error a second time.
		return;
	}

	const bool bHidden       = Input->GetIsHidden();
	const bool bVisCondPass  = Input->GetHasVisibleCondition() ? Input->GetVisibleConditionEnabled() : true;
	const bool bEditCondPass = Input->GetHasEditCondition()    ? Input->GetEditConditionEnabled()    : true;
	if (bHidden)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "SetStackInputData_Hidden",
			          "Refusing to set input '{0}': input is hidden by static-switch / conditional logic and is not currently part of the executing graph. Adjust the parent switch first."),
			FText::FromName(Input->GetInputParameterHandle().GetName())));
		return;
	}
	if (!bVisCondPass)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "SetStackInputData_VisCondFalse",
			          "Refusing to set input '{0}': its VisibleCondition currently evaluates false. The input is not visible in the editor — flip the gating input first."),
			FText::FromName(Input->GetInputParameterHandle().GetName())));
		return;
	}
	if (!bEditCondPass)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "SetStackInputData_EditCondFalse",
			          "Refusing to set input '{0}': its EditCondition currently evaluates false. The input is grayed out in the editor — flip the gating toggle first."),
			FText::FromName(Input->GetInputParameterHandle().GetName())));
		return;
	}

	InData.InitStackInputFromValue(StackInputRef, Context);
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraExt_VariableValue::Set(const FNiagaraTypeDefinition& TypeDef, const FNiagaraVariant& Variant)
{
	if (Variant.IsValid(TypeDef))
	{
		if (UEnum* Enum = TypeDef.GetEnum())
		{
			FNiagaraExt_VariableValue_Enum& EnumVal = InitializeAs<FNiagaraExt_VariableValue_Enum>();
			int32 EnumValue = Variant.GetBytesValue<int32>(TypeDef);
			EnumVal.Enum = Enum;
			if (Enum)
			{
				EnumVal.EnumName = Enum->GetNameByValue(EnumValue);
				EnumVal.DisplayName = Enum->GetDisplayNameTextByValue(EnumValue);
			}
		}
		else if (TypeDef.IsDataInterface())
		{
			FNiagaraExt_VariableValue_DataInterface& DIValue = InitializeAs<FNiagaraExt_VariableValue_DataInterface>();
			DIValue.DataInterfaceClass = TypeDef.GetClass();
			DIValue.DataInterface = Variant.GetMode() == ENiagaraVariantMode::DataInterface ? Cast<UNiagaraDataInterface>(Variant.GetDataInterface()) : nullptr;
		}
		else if (TypeDef.IsUObject())
		{
			FNiagaraExt_VariableValue_Object& ObjValue = InitializeAs<FNiagaraExt_VariableValue_Object>();
			ObjValue.ObjectClass = TypeDef.GetClass();
			ObjValue.Object = Variant.GetMode() == ENiagaraVariantMode::Object ? Variant.GetUObject() : nullptr;
		}
		else if (UScriptStruct* InnerScriptStruct = TypeDef.GetScriptStruct())
		{
			InitializeAs(InnerScriptStruct, Variant.GetMode() == ENiagaraVariantMode::Bytes ? Variant.GetBytes() : nullptr);
		}
	}
}

void FNiagaraExt_VariableValue::Get(FNiagaraVariant& Variant, FNiagaraExternalEditContext& Context)const
{
	if(!IsValid())
	{
		Context.Error(NSLOCTEXT("NiagaraExternalEdit", "VariableValue_Get_Invalid", "Cannot get variable value: data is invalid or uninitialized."));
		return;
	}

	if (const FNiagaraExt_VariableValue_Enum* EnumValue = GetPtr<FNiagaraExt_VariableValue_Enum>())
	{
		if (UEnum* Enum = EnumValue->Enum)
		{
			int32 NewEnumValue = Enum->GetValueByName(EnumValue->EnumName);
			Variant.SetBytes((uint8*)&NewEnumValue, sizeof(int32));
		}
	}
	else if (const FNiagaraExt_VariableValue_DataInterface* DIValue = GetPtr<FNiagaraExt_VariableValue_DataInterface>())
	{
		Variant.SetDataInterface(DIValue->DataInterface);
	}
	else if (const FNiagaraExt_VariableValue_Object* ObjValue = GetPtr<FNiagaraExt_VariableValue_Object>())
	{
		Variant.SetUObject(ObjValue->Object);
	}
	else if (IsValid())
	{
		Variant.SetBytes(GetMemory(), GetScriptStruct()->GetStructureSize());
	}
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraExt_StackInputValue::InitFromStackInput(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExternalEditContext& Context)
{
	UNiagaraStackFunctionInput* Input = StackInputRef.GetInput(Context);
	UNiagaraSystem* System = StackInputRef.GetSystem(Context);

	if(System == nullptr || Input == nullptr)
	{
		return;
	}

	const FNiagaraTypeDefinition& InputType = Input->GetInputType();

	switch (Input->GetValueMode())
	{
	case UNiagaraStackFunctionInput::EValueMode::Local:
	{
		TSharedPtr<const FStructOnScope> LocalData = Input->GetLocalValueStruct();
		const UScriptStruct* InScriptStruct = LocalData.IsValid() ? Cast<const UScriptStruct>(LocalData->GetStruct()) : nullptr;
		if (LocalData.IsValid() && InScriptStruct && LocalData->IsValid())
		{
			if (UEnum* Enum = InputType.GetEnum())
			{
				FNiagaraExt_StackInputData_Enum& EnumVal = InitializeAs<FNiagaraExt_StackInputData_Enum>();
				EnumVal.Enum = Enum;
				if (Enum && ensure(InScriptStruct == FNiagaraInt32::StaticStruct()))
				{
					int32 EnumValue = *(int32*)LocalData->GetStructMemory();
					EnumVal.EnumName = Enum->GetNameByValue(EnumValue);
					EnumVal.DisplayName = Enum->GetDisplayNameTextByValue(EnumValue);
				}
			}
			else
			{
				InitializeAs(InScriptStruct, LocalData->GetStructMemory());
			}
		}
		else
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "StackInput_Get_LocalDataInvalid", "Local data for input '{0}' is invalid or struct type could not be determined. Stack reference: {1}"),
				FText::FromName(Input->GetInputParameterHandle().GetName()),
				FText::FromString(GetStackItemPath(StackInputRef))));
		}
	}
	break;
	case UNiagaraStackFunctionInput::EValueMode::Linked:
	{
		FNiagaraExt_StackInputData_Linked& Value = InitializeAs<FNiagaraExt_StackInputData_Linked>();
		Value.LinkedVariable.Name = Input->GetLinkedParameterValue().GetName();
		Value.LinkedVariable.Type = Input->GetLinkedParameterValue().GetType();
	}
	break;
	case UNiagaraStackFunctionInput::EValueMode::Expression:
	{
		FNiagaraExt_StackInputData_HlslExpression& Value = InitializeAs<FNiagaraExt_StackInputData_HlslExpression>();
		Value.HlslExpression = Input->GetCustomExpressionText().ToString();
	}
	break;
	case UNiagaraStackFunctionInput::EValueMode::Data:
	{
		if (UNiagaraDataInterface* DI = Input->GetDataValueObject())
		{
			FNiagaraExt_StackInputData_DataInterface& Value = InitializeAs<FNiagaraExt_StackInputData_DataInterface>();
			Value.PropertyValues = GetAllObjectProperties(DI);
			Context.UsedDataInterfaces.Add(DI->GetClass());
		}
	}
	break;
	case UNiagaraStackFunctionInput::EValueMode::Dynamic:
	{
		FNiagaraExt_StackInputData_DynamicInput& Value = InitializeAs<FNiagaraExt_StackInputData_DynamicInput>();
		Value.DynamicInputAsset = Input->GetDynamicInputNode()->FunctionScript;
		Context.UsedDynamicInputs.Add(Value.DynamicInputAsset);
	}
	break;
	default:
		InitializeAs<FNiagaraExt_StackInputData_Unsupported>();
	}
}

void FNiagaraExt_StackInputValue::InitStackInputFromValue(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExternalEditContext& Context)const
{
	UNiagaraStackFunctionInput* Input = StackInputRef.GetInput(Context);
	UNiagaraSystem* System = StackInputRef.GetSystem(Context);

	if(System == nullptr || Input == nullptr)
	{
		return;
	}
	
	const FNiagaraTypeDefinition& InputType = Input->GetInputType();
	UScriptStruct* InputTypeStruct = InputType.GetScriptStruct();
	
	if (const FNiagaraExt_StackInputData_Enum* EnumValue = GetPtr<FNiagaraExt_StackInputData_Enum>())
	{
		if(UEnum* Enum = EnumValue->Enum)
		{			
			int32 NewEnumValue = Enum->GetValueByName(EnumValue->EnumName);
			TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FNiagaraInt32::StaticStruct());
			FMemory::Memcpy(StructOnScope->GetStructMemory(), &NewEnumValue, sizeof(FNiagaraInt32));
			Input->SetLocalValue(StructOnScope.ToSharedRef());
		}
	}
	else if (const FNiagaraExt_StackInputData_Linked* LinkedValue = GetPtr<FNiagaraExt_StackInputData_Linked>())
	{
		Input->SetLinkedParameterValue(FNiagaraVariableBase(LinkedValue->LinkedVariable.Type, LinkedValue->LinkedVariable.Name));
	}
	else if (const FNiagaraExt_StackInputData_HlslExpression* HlslValue = GetPtr<FNiagaraExt_StackInputData_HlslExpression>())
	{
		if (Input->SupportsCustomExpressions())
		{
			Input->SetCustomExpression(HlslValue->HlslExpression);
		}
		else
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "StackInput_Set_CustomExpressionNotSupported", "Input '{0}' does not support custom HLSL expressions. Stack reference: {1}"),
				FText::FromName(Input->GetInputParameterHandle().GetName()),
				FText::FromString(GetStackItemPath(StackInputRef))));
		}
	}
	else if (const FNiagaraExt_StackInputData_DataInterface* DIValue = GetPtr<FNiagaraExt_StackInputData_DataInterface>())
	{
		UClass* DIClass = Input->GetInputType().GetClass();
		if(DIClass && DIClass->IsChildOf<UNiagaraDataInterface>())
		{
			Input->SetDataInterfaceValueExternal(DIClass, [&](UNiagaraDataInterface* DI)
				{
					if (DI)
					{
						if (!SetAllObjectProperties(DI, DIValue->PropertyValues, Context.Errors))
						{
							Context.Error(FText::Format(
								NSLOCTEXT("NiagaraExternalEdit", "StackInput_Set_DataInterface_PropertiesFailed", "Failed to set data interface properties for input '{0}'. Stack reference: {1}"),
								FText::FromName(Input->GetInputParameterHandle().GetName()),
								FText::FromString(GetStackItemPath(StackInputRef))));
						}
					}
					else
					{
						Context.Error(FText::Format(
							NSLOCTEXT("NiagaraExternalEdit", "StackInput_Set_DataInterface_CreateFailed", "Failed to create data interface for input '{0}'. Stack reference: {1}"),
							FText::FromName(Input->GetInputParameterHandle().GetName()),
							FText::FromString(GetStackItemPath(StackInputRef))));
					}
				});
		}
		else
		{
			Context.Error(FText::Format(
				NSLOCTEXT("NiagaraExternalEdit", "StackInput_Set_NotDataInterface", "Input '{0}' is not a data interface type (actual type: {1}). Stack reference: {2}"),
				FText::FromName(Input->GetInputParameterHandle().GetName()),
				FText::FromString(InputType.GetName()),
				FText::FromString(GetStackItemPath(StackInputRef))));
		}
	}
	else if (const FNiagaraExt_StackInputData_DynamicInput* DynamicValue = GetPtr<FNiagaraExt_StackInputData_DynamicInput>())
	{
		Input->SetDynamicInput(DynamicValue->DynamicInputAsset);
	}
	else if(IsValid())
	{
		FNiagaraTypeDefinition IncommingTypeDef(const_cast<UScriptStruct*>(ScriptStruct.Get()), FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Allow);//Ugh. Todo. Propagate constness into the NiagaraTypeDefinition Struct.
		if (InputType != IncommingTypeDef)
		{
			//Allow int32 inputs for enums.
			bool bAllowDifferentType = InputType.IsEnum() && IncommingTypeDef == FNiagaraTypeDefinition::GetIntDef();
			if (!bAllowDifferentType)
			{
				Context.Error(FText::Format(
					NSLOCTEXT("NiagaraExternalEdit", "StackInput_Set_TypeMismatch", "Type mismatch for input '{0}': expected '{1}' but got '{2}'. Stack reference: {3}"),
					FText::FromName(Input->GetInputParameterHandle().GetName()),
					FText::FromString(InputType.GetName()),
					FText::FromString(IncommingTypeDef.GetName()),
					FText::FromString(GetStackItemPath(StackInputRef))));
				return;
			}
		}

		TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(ScriptStruct);
		FMemory::Memcpy(StructOnScope->GetStructMemory(), GetMemory(), ScriptStruct->GetStructureSize());
		Input->SetLocalValue(StructOnScope.ToSharedRef());
	}
	else
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "StackInput_Set_InvalidData", "Cannot set input '{0}': input data is invalid or uninitialized. Stack reference: {1}"),
			FText::FromName(Input->GetInputParameterHandle().GetName()),
			FText::FromString(GetStackItemPath(StackInputRef))));
	}
}



//////////////////////////////////////////////////////////////////////////

FNiagaraExt_DynamicInputChainRef::FNiagaraExt_DynamicInputChainRef()
{
	InitializeAs<FNiagaraExt_DynamicInputChain>();
}

const FNiagaraExt_DynamicInputChain& FNiagaraExt_DynamicInputChainRef::Get() const
{
	return FInstancedStruct::Get<FNiagaraExt_DynamicInputChain>();
}

FNiagaraExt_DynamicInputChain& FNiagaraExt_DynamicInputChainRef::GetMutable()
{
	return FInstancedStruct::GetMutable<FNiagaraExt_DynamicInputChain>();
}

//////////////////////////////////////////////////////////////////////////
// Diagnostics Layer - Compile state and stack issues.

namespace
{
	ENiagaraExt_ScriptCompileStatus MapScriptCompileStatus(ENiagaraScriptCompileStatus Status)
	{
		switch (Status)
		{
		case ENiagaraScriptCompileStatus::NCS_Unknown:               return ENiagaraExt_ScriptCompileStatus::Unknown;
		case ENiagaraScriptCompileStatus::NCS_Dirty:                 return ENiagaraExt_ScriptCompileStatus::Dirty;
		case ENiagaraScriptCompileStatus::NCS_Error:                 return ENiagaraExt_ScriptCompileStatus::Error;
		case ENiagaraScriptCompileStatus::NCS_UpToDate:              return ENiagaraExt_ScriptCompileStatus::UpToDate;
		case ENiagaraScriptCompileStatus::NCS_BeingCreated:          return ENiagaraExt_ScriptCompileStatus::BeingCreated;
		case ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings:  return ENiagaraExt_ScriptCompileStatus::UpToDateWithWarnings;
		case ENiagaraScriptCompileStatus::NCS_ComputeUpToDateWithWarnings: return ENiagaraExt_ScriptCompileStatus::ComputeUpToDateWithWarnings;
		default:                                                     return ENiagaraExt_ScriptCompileStatus::Unknown;
		}
	}

	ENiagaraExt_CompileEventSeverity MapCompileEventSeverity(FNiagaraCompileEventSeverity Severity)
	{
		switch (Severity)
		{
		case FNiagaraCompileEventSeverity::Log:     return ENiagaraExt_CompileEventSeverity::Log;
		case FNiagaraCompileEventSeverity::Display: return ENiagaraExt_CompileEventSeverity::Display;
		case FNiagaraCompileEventSeverity::Warning: return ENiagaraExt_CompileEventSeverity::Warning;
		case FNiagaraCompileEventSeverity::Error:   return ENiagaraExt_CompileEventSeverity::Error;
		default:                                    return ENiagaraExt_CompileEventSeverity::Log;
		}
	}

	ENiagaraExt_StackIssueSeverity MapStackIssueSeverity(EStackIssueSeverity Severity)
	{
		switch (Severity)
		{
		case EStackIssueSeverity::Error:   return ENiagaraExt_StackIssueSeverity::Error;
		case EStackIssueSeverity::Warning: return ENiagaraExt_StackIssueSeverity::Warning;
		case EStackIssueSeverity::Info:    return ENiagaraExt_StackIssueSeverity::Info;
		default:                           return ENiagaraExt_StackIssueSeverity::None;
		}
	}

	ENiagaraExt_StackIssueFixStyle MapStackIssueFixStyle(UNiagaraStackEntry::EStackIssueFixStyle Style)
	{
		switch (Style)
		{
		case UNiagaraStackEntry::EStackIssueFixStyle::Fix:  return ENiagaraExt_StackIssueFixStyle::Fix;
		case UNiagaraStackEntry::EStackIssueFixStyle::Link: return ENiagaraExt_StackIssueFixStyle::Link;
		default:                                            return ENiagaraExt_StackIssueFixStyle::Link;
		}
	}

	FNiagaraExt_ScriptCompileInfo BuildScriptCompileInfo(UNiagaraScript* Script, FName EmitterName)
	{
		FNiagaraExt_ScriptCompileInfo Info;
		Info.EmitterName = EmitterName;
		// NameFromScriptUsage leaves ScriptName at NAME_None on unknown usages; that is the
		// intended fallback (the field's default) and keeps this helper consistent with the
		// stack-walk's usage of NameFromScriptUsage in WalkEntryForIssues.
		NameFromScriptUsage(Script->GetUsage(), Info.ScriptName);

		const FNiagaraVMExecutableData& VMData = Script->GetVMExecutableData();
		Info.LastCompileStatus = MapScriptCompileStatus(VMData.LastCompileStatus);

#if WITH_EDITORONLY_DATA
		Info.ErrorSummary = VMData.ErrorMsg;
		Info.CompileEvents.Reserve(VMData.LastCompileEvents.Num());
		for (const FNiagaraCompileEvent& SrcEvent : VMData.LastCompileEvents)
		{
			FNiagaraExt_CompileEvent& DstEvent = Info.CompileEvents.AddDefaulted_GetRef();
			DstEvent.Severity              = MapCompileEventSeverity(SrcEvent.Severity);
			DstEvent.Message               = SrcEvent.Message;
			DstEvent.ShortDescription      = SrcEvent.ShortDescription;
			DstEvent.NodeGuid              = SrcEvent.NodeGuid;
			DstEvent.PinGuid               = SrcEvent.PinGuid;
			DstEvent.bFromScriptDependency = (SrcEvent.Source == FNiagaraCompileEventSource::ScriptDependency);
		}
#endif

		return Info;
	}

} // anonymous namespace

void UNiagaraExternalEditUtilities::EnumerateSystemScripts(
	UNiagaraSystem* System,
	TArray<FNiagaraExt_ScriptCompileInfo>& OutScripts)
{
	// System-level scripts first.
	if (UNiagaraScript* SpawnScript = System->GetSystemSpawnScript())
	{
		OutScripts.Add(BuildScriptCompileInfo(SpawnScript, NAME_None));
	}
	if (UNiagaraScript* UpdateScript = System->GetSystemUpdateScript())
	{
		OutScripts.Add(BuildScriptCompileInfo(UpdateScript, NAME_None));
	}

	// Then each emitter's scripts, in handle order.
	//
	// Note: EmitterSpawnScript and EmitterUpdateScript are intentionally filtered out. Their
	// bodies are inlined into the system's SystemSpawnScript / SystemUpdateScript during
	// compile, and they don't carry their own compile status — LastCompileStatus is always
	// Unknown and LastCompileEvents is always empty, which is just noise in the output.
	// Any real compile errors from their contents surface on the system-level scripts above.
	for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
	{
		const FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData();
		if (!EmitterData)
		{
			continue;
		}

		TArray<UNiagaraScript*> EmitterScripts;
		EmitterData->GetScripts(EmitterScripts, /*bCompilableOnly=*/false, /*bEnabledOnly=*/false);

		for (UNiagaraScript* Script : EmitterScripts)
		{
			if (!Script)
			{
				continue;
			}

			const ENiagaraScriptUsage Usage = Script->GetUsage();
			if (Usage == ENiagaraScriptUsage::EmitterSpawnScript
				|| Usage == ENiagaraScriptUsage::EmitterUpdateScript)
			{
				continue;
			}

			OutScripts.Add(BuildScriptCompileInfo(Script, Handle.GetName()));
		}
	}
}

namespace
{
	// Severity rank for FNiagaraExt_SystemCompileState::AggregateStatus derivation.
	// Higher rank == more attention-worthy. Max across per-script statuses wins.
	int32 ScriptCompileStatusRank(ENiagaraExt_ScriptCompileStatus Status)
	{
		switch (Status)
		{
		case ENiagaraExt_ScriptCompileStatus::Error:                       return 5;
		case ENiagaraExt_ScriptCompileStatus::Dirty:                       return 4;
		case ENiagaraExt_ScriptCompileStatus::UpToDateWithWarnings:        return 3;
		case ENiagaraExt_ScriptCompileStatus::ComputeUpToDateWithWarnings: return 3;
		case ENiagaraExt_ScriptCompileStatus::UpToDate:                    return 2;
		case ENiagaraExt_ScriptCompileStatus::BeingCreated:                return 1;
		case ENiagaraExt_ScriptCompileStatus::Unknown:                     return 0;
		default:                                                           return 0;
		}
	}
}

void UNiagaraExternalEditUtilities::GetSystemCompileState(
	UNiagaraSystem* System,
	FNiagaraExt_SystemCompileState& OutState,
	FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckSystem(System)) return;

	EnumerateSystemScripts(System, OutState.Scripts);

	// bIsCompiling never blocks — it reports whether a compile is active or queued.
	OutState.bIsCompiling = System->HasActiveCompilations()
		|| System->HasOutstandingCompilationRequests(/*bIncludingGPUShaders=*/false);

	// Stale when compiling or when any script is dirty.
	OutState.bIsStale = OutState.bIsCompiling;

	// Derive AggregateStatus from per-script statuses (max severity wins). The view model's
	// LatestCompileStatusCache only populates when a compile was triggered through it; this
	// utility creates a fresh view model for inspection and never triggers one, so reading
	// from the VM produces Unknown on every clean system. Deriving here keeps AggregateStatus
	// consistent with bIsStale / bHasErrors / bHasWarnings — all derived from OutState.Scripts.
	ENiagaraExt_ScriptCompileStatus Aggregate = ENiagaraExt_ScriptCompileStatus::Unknown;

	for (const FNiagaraExt_ScriptCompileInfo& ScriptInfo : OutState.Scripts)
	{
		if (ScriptInfo.LastCompileStatus == ENiagaraExt_ScriptCompileStatus::Dirty)
		{
			OutState.bIsStale = true;
		}
		if (ScriptInfo.LastCompileStatus == ENiagaraExt_ScriptCompileStatus::Error)
		{
			OutState.bHasErrors = true;
		}
		for (const FNiagaraExt_CompileEvent& Event : ScriptInfo.CompileEvents)
		{
			if (Event.Severity == ENiagaraExt_CompileEventSeverity::Error)
			{
				OutState.bHasErrors = true;
			}
			else if (Event.Severity == ENiagaraExt_CompileEventSeverity::Warning)
			{
				OutState.bHasWarnings = true;
			}
		}

		if (ScriptCompileStatusRank(ScriptInfo.LastCompileStatus) > ScriptCompileStatusRank(Aggregate))
		{
			Aggregate = ScriptInfo.LastCompileStatus;
	}
	}

	OutState.AggregateStatus = Aggregate;
}

namespace
{
	// State carried down the stack walk so each issue can be populated with its location.
	struct FStackIssueWalkState
	{
		TArray<FString> DisplayPath;
		FName CurrentEmitterName = NAME_None;
		FName CurrentScriptName = NAME_None;
		FName CurrentModuleName = NAME_None;
		int32 CurrentRendererIndex = INDEX_NONE;
		TArray<FName> CurrentInputNameStack;
		TArray<FString> DismissedIds;
	};

	// Find the index of a UNiagaraStackRendererItem's renderer within its owning emitter's renderer list.
	// Returns INDEX_NONE if the owning emitter can't be located or the renderer isn't in the list.
	int32 FindRendererIndexForStackItem(UNiagaraSystem* System, FName EmitterName, UNiagaraStackRendererItem* RendererItem)
	{
		if (!System || !RendererItem || EmitterName == NAME_None)
		{
			return INDEX_NONE;
		}

		UNiagaraRendererProperties* Props = RendererItem->GetRendererProperties();
		if (!Props)
		{
			return INDEX_NONE;
		}

		for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
		{
			if (Handle.GetName() != EmitterName)
			{
				continue;
			}
			if (const FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData())
			{
				return EmitterData->GetRenderers().IndexOfByKey(Props);
			}
			break;
		}
		return INDEX_NONE;
	}

	// Recursive helper: DFS over all unfiltered children, appending issues in document order.
	void WalkEntryForIssues(
		UNiagaraStackEntry* Entry,
		UNiagaraSystem* System,
		FStackIssueWalkState& State,
		FNiagaraExt_StackIssues& OutIssues)
	{
		if (!Entry) return;

		// Update the location context when we enter a script group, module item, renderer item, or
		// function input. Saved values are restored on unwind so the state tracks our current
		// position in the tree.
		const FName SavedScriptName    = State.CurrentScriptName;
		const FName SavedModuleName    = State.CurrentModuleName;
		const int32 SavedRendererIndex = State.CurrentRendererIndex;
		bool bPushedInputName = false;

		if (UNiagaraStackScriptItemGroup* ScriptGroup = Cast<UNiagaraStackScriptItemGroup>(Entry))
		{
			FName ScriptName;
			if (NameFromScriptUsage(ScriptGroup->GetScriptUsage(), ScriptName))
			{
				State.CurrentScriptName = ScriptName;
				State.CurrentModuleName = NAME_None;
			}
		}
		else if (UNiagaraStackModuleItem* ModuleItem = Cast<UNiagaraStackModuleItem>(Entry))
		{
			State.CurrentModuleName = *ModuleItem->GetModuleNode().GetFunctionName();
		}
		else if (UNiagaraStackRendererItem* RendererItem = Cast<UNiagaraStackRendererItem>(Entry))
		{
			State.CurrentRendererIndex = FindRendererIndexForStackItem(System, State.CurrentEmitterName, RendererItem);
		}
		else if (UNiagaraStackFunctionInput* InputItem = Cast<UNiagaraStackFunctionInput>(Entry))
		{
			// Function inputs nest — a dynamic input exposes its own children that are also
			// UNiagaraStackFunctionInput entries. Push onto the stack so issues on any depth
			// carry the full parameter handle chain.
			State.CurrentInputNameStack.Add(InputItem->GetInputParameterHandle().GetName());
			bPushedInputName = true;
		}

		State.DisplayPath.Add(Entry->GetDisplayName().ToString());

		for (const UNiagaraStackEntry::FStackIssue& Issue : Entry->GetIssues())
		{
			FNiagaraExt_StackIssue& OutIssue = OutIssues.Issues.AddDefaulted_GetRef();
			OutIssue.IssueId          = Issue.GetUniqueIdentifier();
			OutIssue.Severity         = MapStackIssueSeverity(Issue.GetSeverity());
			OutIssue.ShortDescription = Issue.GetShortDescription().ToString();
			OutIssue.LongDescription  = Issue.GetLongDescription().ToString();
			OutIssue.bCanBeDismissed  = Issue.GetCanBeDismissed();
			OutIssue.bIsDismissed     = State.DismissedIds.Contains(OutIssue.IssueId);
			OutIssue.StackDisplayPath = FString::Join(State.DisplayPath, TEXT("/"));

			FNiagaraExt_StackItemReference Location(
				System,
				State.CurrentEmitterName,
				State.CurrentScriptName,
				State.CurrentModuleName);
			Location.RendererIndex   = State.CurrentRendererIndex;
			Location.InputNameStack  = State.CurrentInputNameStack;
			OutIssue.Location        = MoveTemp(Location);

			for (const UNiagaraStackEntry::FStackIssueFix& Fix : Issue.GetFixes())
			{
				FNiagaraExt_StackIssueFix& OutFix = OutIssue.Fixes.AddDefaulted_GetRef();
				OutFix.FixId       = Fix.GetUniqueIdentifier();
				OutFix.Description = Fix.GetDescription().ToString();
				OutFix.Style       = MapStackIssueFixStyle(Fix.GetStyle());
			}

			switch (OutIssue.Severity)
			{
			case ENiagaraExt_StackIssueSeverity::Error:   ++OutIssues.NumErrors;   break;
			case ENiagaraExt_StackIssueSeverity::Warning: ++OutIssues.NumWarnings; break;
			case ENiagaraExt_StackIssueSeverity::Info:    ++OutIssues.NumInfos;    break;
			default: break;
			}
		}

		TArray<UNiagaraStackEntry*> Children;
		Entry->GetUnfilteredChildren(Children);
		for (UNiagaraStackEntry* Child : Children)
		{
			WalkEntryForIssues(Child, System, State, OutIssues);
		}

		State.DisplayPath.RemoveAt(State.DisplayPath.Num() - 1);
		State.CurrentScriptName    = SavedScriptName;
		State.CurrentModuleName    = SavedModuleName;
		State.CurrentRendererIndex = SavedRendererIndex;
		if (bPushedInputName)
		{
			State.CurrentInputNameStack.Pop(EAllowShrinking::No);
		}
	}
} // anonymous namespace

void UNiagaraExternalEditUtilities::CollectStackIssues(
	FNiagaraSystemViewModel& SystemVM,
	UNiagaraSystem* System,
	FNiagaraExt_StackIssues& OutIssues)
{
	if (UNiagaraStackViewModel* SysStack = SystemVM.GetSystemStackViewModel())
	{
		if (UNiagaraStackEntry* Root = SysStack->GetRootEntry())
		{
			FStackIssueWalkState State;
			State.DismissedIds = Root->GetStackEditorData().GetDismissedStackIssueIds();
			WalkEntryForIssues(Root, System, State, OutIssues);
		}
	}

	for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterVM : SystemVM.GetEmitterHandleViewModels())
	{
		if (UNiagaraStackViewModel* EmitterStack = EmitterVM->GetEmitterStackViewModel())
		{
			if (UNiagaraStackEntry* Root = EmitterStack->GetRootEntry())
			{
				FStackIssueWalkState State;
				State.CurrentEmitterName = EmitterVM->GetName();
				State.DismissedIds = Root->GetStackEditorData().GetDismissedStackIssueIds();
				WalkEntryForIssues(Root, System, State, OutIssues);
			}
		}
	}
}

void UNiagaraExternalEditUtilities::GetStackIssues(
	UNiagaraSystem* System,
	FNiagaraExt_StackIssues& OutIssues,
	FNiagaraExternalEditContext& Context)
{
	if (!Context.CheckSystem(System)) return;

	// Stack issues depend on fresh compile output; a snapshot taken mid-compile would mix
	// pre-compile issues with post-compile state. Refuse rather than return something misleading.
	if (System->HasActiveCompilations()
		|| System->HasOutstandingCompilationRequests(/*bIncludingGPUShaders=*/false))
	{
		Context.Error(NSLOCTEXT("NiagaraExternalEdit", "GetStackIssues_CompileInFlight",
			"Cannot collect stack issues while a compile is in flight. Wait for compilation "
			"to complete (UNiagaraSystem::OnSystemCompiled) before calling."));
		return;
	}

	// Diagnostics need a non-data-only VM (see GetDiagnosticsSystemViewModel for why).
	// Stack entries clear their issue arrays under a data-only VM, so we'd read empty results.
	// Errors already appended to Context.Errors by the getter if the VM isn't available.
	TSharedPtr<FNiagaraSystemViewModel> SystemVM = Context.GetDiagnosticsSystemViewModel();
	if (!SystemVM.IsValid()) return;

	CollectStackIssues(*SystemVM, System, OutIssues);
}

namespace
{
	// Result of a DFS lookup for a (IssueId, FixId) pair across all stack roots.
	struct FMatchResult
	{
		UNiagaraStackEntry* Entry = nullptr;
		int32 IssueIndex = INDEX_NONE;
		int32 FixIndex   = INDEX_NONE;
		bool  bCollision = false;
	};

	// Populates OutResult with the first (Entry, IssueIndex, FixIndex) triple whose identifiers
	// match; sets bCollision if a second match is found.
	void FindIssueFixMatch(
		UNiagaraStackEntry* Entry,
		const FString& IssueId,
		const FString& FixId,
		FMatchResult& OutResult)
	{
		if (!Entry) return;

		const TArray<UNiagaraStackEntry::FStackIssue>& EntryIssues = Entry->GetIssues();
		for (int32 i = 0; i < EntryIssues.Num(); ++i)
		{
			const UNiagaraStackEntry::FStackIssue& Issue = EntryIssues[i];
			if (Issue.GetUniqueIdentifier() != IssueId) continue;

			const TArray<UNiagaraStackEntry::FStackIssueFix>& Fixes = Issue.GetFixes();
			for (int32 j = 0; j < Fixes.Num(); ++j)
			{
				if (Fixes[j].GetUniqueIdentifier() != FixId) continue;

				if (OutResult.Entry == nullptr)
				{
					OutResult.Entry      = Entry;
					OutResult.IssueIndex = i;
					OutResult.FixIndex   = j;
				}
				else
				{
					OutResult.bCollision = true;
				}
			}
		}

		TArray<UNiagaraStackEntry*> Children;
		Entry->GetUnfilteredChildren(Children);
		for (UNiagaraStackEntry* Child : Children)
		{
			FindIssueFixMatch(Child, IssueId, FixId, OutResult);
		}
	}
} // anonymous namespace

void UNiagaraExternalEditUtilities::ApplyStackIssueFix(
	UNiagaraSystem* System,
	const FString& IssueId,
	const FString& FixId,
	FNiagaraExt_ApplyStackIssueFixResult& OutResult,
	FNiagaraExternalEditContext& Context)
{
	OutResult = FNiagaraExt_ApplyStackIssueFixResult{};

	if (!Context.CheckSystem(System)) return;

	// The IssueId/FixId pair must refer to the current state of the system. Refuse to operate
	// mid-compile so we never apply a fix against a stale issue that has already been resolved
	// or invalidated by an in-flight compile.
	if (System->HasActiveCompilations()
		|| System->HasOutstandingCompilationRequests(/*bIncludingGPUShaders=*/false))
	{
		Context.Error(NSLOCTEXT("NiagaraExternalEdit", "ApplyFix_CompileInFlight",
			"Cannot apply a stack issue fix while a compile is in flight. Wait for compilation "
			"to complete (UNiagaraSystem::OnSystemCompiled) before calling."));
		return;
	}

	// Diagnostics need a non-data-only VM. Errors already appended to Context.Errors by the
	// getter if the VM isn't available.
	TSharedPtr<FNiagaraSystemViewModel> SystemVM = Context.GetDiagnosticsSystemViewModel();
	if (!SystemVM.IsValid()) return;

	// Re-walk to locate the live issue/fix pair. Never use cached pointers — we need to
	// work with the live UNiagaraStackEntry objects, not the plain-value copies.
	FMatchResult Match;
	if (UNiagaraStackViewModel* SysStack = SystemVM->GetSystemStackViewModel())
	{
		FindIssueFixMatch(SysStack->GetRootEntry(), IssueId, FixId, Match);
	}
	for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterVM : SystemVM->GetEmitterHandleViewModels())
	{
		if (UNiagaraStackViewModel* EmitterStack = EmitterVM->GetEmitterStackViewModel())
		{
			FindIssueFixMatch(EmitterStack->GetRootEntry(), IssueId, FixId, Match);
		}
	}

	if (Match.bCollision)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "ApplyFix_Collision",
				"IssueId '{0}' with FixId '{1}' matched more than one stack entry. Refusing to apply — "
				"call GetStackIssues to refresh and disambiguate before retrying."),
			FText::FromString(IssueId),
			FText::FromString(FixId)));
		return;
	}

	if (Match.Entry == nullptr)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "ApplyFix_IssueNotFound",
				"No stack issue with IssueId '{0}' was found. Use GetStackIssues to obtain current IssueIds."),
			FText::FromString(IssueId)));
		return;
	}

	if (Match.FixIndex == INDEX_NONE)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "ApplyFix_FixNotFound",
				"Issue '{0}' was found but it has no fix with FixId '{1}'. Use GetStackIssues to obtain current FixIds."),
			FText::FromString(IssueId),
			FText::FromString(FixId)));
		return;
	}

	// Re-read the fix from the live entry (indexes are still valid — we haven't mutated yet).
	const UNiagaraStackEntry::FStackIssueFix& MatchedFix =
		Match.Entry->GetIssues()[Match.IssueIndex].GetFixes()[Match.FixIndex];

	if (MatchedFix.GetStyle() == UNiagaraStackEntry::EStackIssueFixStyle::Link)
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "ApplyFix_LinkStyle",
				"Fix '{0}' (on issue '{1}') is a Link-style fix. Link-style fixes are navigation hints for "
				"humans and cannot be triggered programmatically. Only Fix-style fixes can be applied."),
			FText::FromString(FixId),
			FText::FromString(IssueId)));
		return;
	}

	if (!MatchedFix.GetFixDelegate().IsBound())
	{
		Context.Error(FText::Format(
			NSLOCTEXT("NiagaraExternalEdit", "ApplyFix_Unbound",
				"Fix '{0}' has an unbound delegate and cannot be executed. This may indicate the fix "
				"is only valid while the Niagara System editor is open."),
			FText::FromString(FixId)));
		return;
	}

	// Snapshot the description before execute — the fix may finalize the entry it came from.
	const FString FixDescription = MatchedFix.GetDescription().ToString();

	// Execute the fix inside a scoped transaction so it is undoable.
	{
		FScopedTransaction Transaction(LOCTEXT("ApplyNiagaraStackIssueFix", "Apply Niagara stack issue fix"));
		MatchedFix.GetFixDelegate().ExecuteIfBound();
	}

	OutResult.bApplied = true;
	OutResult.AppliedFixDescription = FixDescription;
}

#undef LOCTEXT_NAMESPACE

