// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGraph.h"

#include "Algo/RemoveIf.h"
#include "Algo/StableSort.h"
#include "EdGraphSchema_Niagara.h"
#include "GraphEditAction.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraCommon.h"
#include "NiagaraComponent.h"
#include "NiagaraConstants.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraModule.h"
#include "NiagaraNode.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeReroute.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSettings.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"
#include "String/ParseTokens.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "DataHierarchyViewModelBase.h"
#include "ViewModels/HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraGraph)


DECLARE_CYCLE_STAT(TEXT("NiagaraEditor - Graph - PostLoad"), STAT_NiagaraEditor_Graph_PostLoad, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("NiagaraEditor - Graph - FindInputNodes"), STAT_NiagaraEditor_Graph_FindInputNodes, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("NiagaraEditor - Graph - FindInputNodes_NotFilterUsage"), STAT_NiagaraEditor_Graph_FindInputNodes_NotFilterUsage, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("NiagaraEditor - Graph - FindInputNodes_FilterUsage"), STAT_NiagaraEditor_Graph_FindInputNodes_FilterUsage, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("NiagaraEditor - Graph - FindInputNodes_FilterDupes"), STAT_NiagaraEditor_Graph_FindInputNodes_FilterDupes, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("NiagaraEditor - Graph - FindInputNodes_FindInputNodes_Sort"), STAT_NiagaraEditor_Graph_FindInputNodes_Sort, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("NiagaraEditor - Graph - FindOutputNode"), STAT_NiagaraEditor_Graph_FindOutputNode, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("NiagaraEditor - Graph - BuildTraversal"), STAT_NiagaraEditor_Graph_BuildTraversal, STATGROUP_NiagaraEditor);

#define NIAGARA_SCOPE_CYCLE_COUNTER(x) //SCOPE_CYCLE_COUNTER(x)

bool bWriteToLog = false;

#define LOCTEXT_NAMESPACE "NiagaraGraph"

int32 GNiagaraUseGraphHash = 1;
static FAutoConsoleVariableRef CVarNiagaraUseGraphHash(
	TEXT("fx.UseNewGraphHash"),
	GNiagaraUseGraphHash,
	TEXT("If > 0 a hash of the graph node state will be used, otherwise will use the older code path. \n"),
	ECVF_Default
);

namespace NiagaraGraphImpl
{

void FindReferencedVariables(const UNiagaraGraph* Graph, TConstArrayView<UNiagaraNode*> Nodes, TSet<FNiagaraVariable>& ReferencedVariables)
{
	// Check all pins on all nodes in the graph to find parameter pins which may have been missed in the parameter map traversal.  This
	// can happen for nodes which are not fully connected
	for (const UNiagaraNode* Node : Nodes)
	{
		if (const UNiagaraNodeStaticSwitch* SwitchNode = Cast<const UNiagaraNodeStaticSwitch>(Node))
		{
			if (!SwitchNode->IsSetByCompiler() && !SwitchNode->IsSetByPin())
			{
				ReferencedVariables.Add(FNiagaraVariable(SwitchNode->GetInputType(), SwitchNode->InputParameterName));
			}
		}
		else if (const UNiagaraNodeFunctionCall* FunctionNode = Cast<const UNiagaraNodeFunctionCall>(Node))
		{
			for (const FNiagaraPropagatedVariable& Propagated : FunctionNode->PropagatedStaticSwitchParameters)
			{
				ReferencedVariables.Add(Propagated.ToVariable());
			}
		}
		else if (const UNiagaraNodeParameterMapBase* ParameterMapNode = Cast<const UNiagaraNodeParameterMapBase>(Node))
		{
			const bool IsMapGet = Node->IsA<UNiagaraNodeParameterMapGet>();
			const bool IsMapSet = Node->IsA<UNiagaraNodeParameterMapSet>();

			for (UEdGraphPin* Pin : Node->Pins)
			{
				// we only consider parameter pins, inputs for MapSets and outputs for MapGets
				if (Pin->PinType.PinSubCategory == UNiagaraNodeParameterMapBase::ParameterPinSubCategory)
				{
					const bool bRelevantPin = ((Pin->Direction == EEdGraphPinDirection::EGPD_Input) && IsMapSet)
						|| ((Pin->Direction == EEdGraphPinDirection::EGPD_Output) && IsMapGet);

					if (!bRelevantPin)
					{
						continue;
					}

					ReferencedVariables.Add(UEdGraphSchema_Niagara::PinToNiagaraVariable(Pin, false));
				}
			}
		}
	}

	auto GetBindingScriptVariable = [&](const FNiagaraVariable& Variable) -> UNiagaraScriptVariable*
	{
		if (UNiagaraScriptVariable* ScriptVariable = Graph->GetScriptVariable(Variable))
		{
			if (ScriptVariable->DefaultMode == ENiagaraDefaultMode::Binding && ScriptVariable->DefaultBinding.IsValid())
			{
				return ScriptVariable;
			}
		}

		return nullptr;
	};

	TArray<UNiagaraScriptVariable*> VariablesToProcess;
	for (const FNiagaraVariable& ReferencedVariable : ReferencedVariables)
	{
		if (UNiagaraScriptVariable* ScriptVariable = GetBindingScriptVariable(ReferencedVariable))
		{
			VariablesToProcess.Add(ScriptVariable);
		}
	}

	while (!VariablesToProcess.IsEmpty())
	{
		UNiagaraScriptVariable* Variable = VariablesToProcess.Pop(EAllowShrinking::No);

		FNiagaraTypeDefinition LinkedType = Variable->Variable.GetType();
		FName BindingName = Variable->DefaultBinding.GetName();
		if (FNiagaraConstants::GetOldPositionTypeVariables().Contains(FNiagaraVariable(LinkedType, BindingName)))
		{
			// it is not uncommon that old assets have vector inputs that default bind to what is now a position type. If we detect that, we change the type to prevent a compiler error.
			LinkedType = FNiagaraTypeDefinition::GetPositionDef();
		}

		const FNiagaraVariable BoundVariable(LinkedType, BindingName);

		bool bAlreadyInSet = false;
		ReferencedVariables.Add(BoundVariable, &bAlreadyInSet);
		if (!bAlreadyInSet)
		{
			if (UNiagaraScriptVariable* ScriptVariable = GetBindingScriptVariable(BoundVariable))
			{
				VariablesToProcess.Add(ScriptVariable);
			}
		}
	}
}

}



FNiagaraGraphParameterReferenceCollection::FNiagaraGraphParameterReferenceCollection(bool bInCreated)
	: bCreatedByUser(bInCreated)
{
}

bool FNiagaraGraphParameterReferenceCollection::WasCreatedByUser() const
{
	return bCreatedByUser;
}

FNiagaraGraphScriptUsageInfo::FNiagaraGraphScriptUsageInfo() : UsageType(ENiagaraScriptUsage::Function)
{
}

void FNiagaraGraphScriptUsageInfo::PostLoad(UObject* Owner)
{
	const int32 NiagaraVer = Owner->GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraVer < FNiagaraCustomVersion::UseHashesToIdentifyCompileStateOfTopLevelScripts)
	{
		if (BaseId.IsValid() == false && GeneratedCompileId_DEPRECATED.IsValid())
		{
			// When loading old data use the last generated compile id as the base id to prevent recompiles on load for existing scripts.
			BaseId = GeneratedCompileId_DEPRECATED;
		}

		if (CompileHash.IsValid() == false && DataHash_DEPRECATED.Num() == FNiagaraCompileHash::HashSize)
		{
			CompileHash = FNiagaraCompileHash(DataHash_DEPRECATED);
			CompileHashFromGraph = FNiagaraCompileHash(DataHash_DEPRECATED);
		}
	}	
}

bool FNiagaraGraphScriptUsageInfo::IsValid() const
{
	for (int32 i = 0; i < Traversal.Num(); i++)
	{
		if (Traversal[i] == nullptr)
			return false;
	}

	return true;
}

void FNiagaraGraphFunctionAliasContext::AppendPermutationString(FStringBuilderBase& StringBuilder) const
{
	TArray<FName, TInlineAllocator<16>> SortedKeys;
	PermutationIdentifiers.GenerateKeyArray(SortedKeys);
	SortedKeys.StableSort(FNameLexicalLess());

	for (const FName& Key : SortedKeys)
	{
		StringBuilder.AppendChar(TEXT(','));
		Key.AppendString(StringBuilder);
		StringBuilder.AppendChar(TEXT('='));
		StringBuilder.Append(PermutationIdentifiers[Key]);
	}
}

void FNiagaraGraphFunctionAliasContext::AddPermutationIdentifier(const FName& Name, FStringView Value)
{
	FString* ExistingIdentifier = PermutationIdentifiers.Find(Name);
	if (ExistingIdentifier)
	{
		check(*ExistingIdentifier == Value);
	}
	else
	{
		PermutationIdentifiers.Emplace(Name, Value);
	}
}

void FNiagaraGraphFunctionAliasContext::AddPendingStaticSwitchVariable(const FNiagaraVariable& Variable, uint8 SwitchType)
{
	FStaticSwitchVariable SwitchVariable;
	SwitchVariable.Variable = Variable;
	SwitchVariable.SwitchType = SwitchType;

	CompilerStaticSwitchVariables.AddUnique(SwitchVariable);
}

FDelegateHandle UNiagaraGraph::AddOnGraphNeedsRecompileHandler(const FOnGraphChanged::FDelegate& InHandler)
{
	return OnGraphNeedsRecompile.Add(InHandler);
}

void UNiagaraGraph::RemoveOnGraphNeedsRecompileHandler(FDelegateHandle Handle)
{
	OnGraphNeedsRecompile.Remove(Handle);
}

void UNiagaraGraph::NotifyGraphChanged(const FEdGraphEditAction& InAction)
{
	InvalidateCachedParameterData();
	if ((InAction.Action & GRAPHACTION_AddNode) != 0 || (InAction.Action & GRAPHACTION_RemoveNode) != 0 ||
		(InAction.Action & GRAPHACTION_GenericNeedsRecompile) != 0)
	{
		MarkGraphRequiresSynchronization(TEXT("Graph Changed"));
	}
	if ((InAction.Action & GRAPHACTION_GenericNeedsRecompile) != 0)
	{
		OnGraphNeedsRecompile.Broadcast(InAction);
		return;
	}
	Super::NotifyGraphChanged(InAction);
}

void UNiagaraGraph::NotifyGraphChanged()
{
	Super::NotifyGraphChanged();
	InvalidateCachedParameterData();
	InvalidateNumericCache();
}

void UNiagaraGraph::PostLoad_LWCFixup(int32 NiagaraVersion)
{
	// TODO - put behind a CustomVersion
	TArray<UNiagaraNodeParameterMapBase*> NiagaraParameterNodes;
	GetNodesOfClass<UNiagaraNodeParameterMapBase>(NiagaraParameterNodes);

	if (NiagaraParameterNodes.IsEmpty())
	{
		return;
	}
	bool GraphChanged = false;

	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	TArray<FNiagaraVariable> OldTypes = FNiagaraConstants::GetOldPositionTypeVariables();

	const FEdGraphPinType PositionPinType = NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetPositionDef());

	for (UNiagaraNodeParameterMapBase* NiagaraNode : NiagaraParameterNodes)
	{
		for (UEdGraphPin* Pin : NiagaraNode->Pins)
		{
			if (NiagaraNode->IsAddPin(Pin))
			{
				continue;
			}
			FNiagaraVariable Variable = FNiagaraVariable(NiagaraSchema->PinToTypeDefinition(Pin), Pin->PinName);
			for (const FNiagaraVariable& OldVarType : OldTypes)
			{
				if (Variable == OldVarType)
				{
					Pin->PinType = PositionPinType;
					Pin->PinType.PinSubCategory = UNiagaraNodeParameterMapBase::ParameterPinSubCategory;
					NiagaraNode->MarkNodeRequiresSynchronization(__FUNCTION__, false);
					GraphChanged = true;
					break;
				}
			}

			// fix default pin types if necessary
			// NOTE - that this code is impacting more than just LWC related nodes, it is touching all MapGet nodes
			//		- Further by assigning the Pin->PinType to the DefaultPin->PinType it is going to assign
			//			UNiagaraNodeParameterMapBase::ParameterPinSubCategory to the SubCategory of the input (default) pin
			//			which currently is reserved for output pins.  This seems to be an unnecessary change and does have
			//			an impact when dealing with RefreshParameterReferences' AddParameterReference lambda where the subcategory
			//			is evaluated.  This results in DefaultValues not necessarily getting into the compile hash (if the default pin
			//			is the first reference added, then it will be encountered and rejected in AppendCompileHash and then the
			//			default value won't be handled
			if (Pin->Direction == EGPD_Output)
			{
				if (UNiagaraNodeParameterMapGet* GetNode = Cast<UNiagaraNodeParameterMapGet>(NiagaraNode))
				{
					UEdGraphPin* DefaultPin = GetNode->GetDefaultPin(Pin);
					if (DefaultPin && DefaultPin->PinType != Pin->PinType)
					{
						DefaultPin->PinType = Pin->PinType;
						NiagaraNode->MarkNodeRequiresSynchronization(__FUNCTION__, false);
						GraphChanged = true;
					}
				}
			}
		}
	}

	TArray<UNiagaraScriptVariable*> ChangedScriptVariables;
	for (FNiagaraVariable OldVarType : OldTypes)
	{
		TObjectPtr<UNiagaraScriptVariable>* ScriptVariablePtr = VariableToScriptVariable.Find(OldVarType);
		if (ScriptVariablePtr != nullptr && *ScriptVariablePtr)
		{
			const FNiagaraVariable NewVarType = FNiagaraVariable(FNiagaraTypeDefinition::GetPositionDef(), OldVarType.GetName());

			UNiagaraScriptVariable* ScriptVar = ScriptVariablePtr->Get();

			if (ScriptVar->DefaultMode == ENiagaraDefaultMode::Value)
			{
				ChangedScriptVariables.Add(ScriptVar);
			}

			ScriptVar->Variable.SetType(FNiagaraTypeDefinition::GetPositionDef());
			VariableToScriptVariable.Remove(OldVarType);

			VariableToScriptVariable.Add(NewVarType, TObjectPtr<UNiagaraScriptVariable>(ScriptVar));
			GraphChanged = true;
		}
	}

	if (!ChangedScriptVariables.IsEmpty())
	{
		TSet<UNiagaraNodeParameterMapGet*> MapGetNodes;
		for (UNiagaraScriptVariable* ChangedScriptVariable : ChangedScriptVariables)
		{
			FNiagaraVariable& Variable = ChangedScriptVariable->Variable;

			FNiagaraEditorModule& EditorModule = FNiagaraEditorModule::Get();
			auto TypeUtilityValue = EditorModule.GetTypeUtilities(Variable.GetType());

			if (TypeUtilityValue.IsValid() && TypeUtilityValue->CanHandlePinDefaults())
			{
				TArray<UEdGraphPin*> Pins = FindParameterMapDefaultValuePins(Variable.GetName());
				for (UEdGraphPin* Pin : Pins)
				{
					check(ChangedScriptVariable->DefaultMode == ENiagaraDefaultMode::Value && !Variable.GetType().IsDataInterface() && !Variable.GetType().IsUObject());

					Variable.AllocateData();
					FString NewDefaultValue = TypeUtilityValue->GetPinDefaultStringFromValue(Variable);
					NiagaraSchema->TrySetDefaultValue(*Pin, NewDefaultValue, false);

					MapGetNodes.Add(CastChecked<UNiagaraNodeParameterMapGet>(Pin->GetOwningNode()));
				}
			}
		}

		if (!MapGetNodes.IsEmpty())
		{
			GraphChanged = true;
			for (UNiagaraNodeParameterMapGet* MapGet : MapGetNodes)
			{
				MapGet->SynchronizeDefaultPins();
			}
		}
	}

	if (GraphChanged)
	{
		Modify();
		ChangeId = FGuid::NewGuid();
	}
}

// while it would be great to only run this fixup once, it's not clear how we are missing 'local' niagara script variables from our map
// and so we'll keep this running all the time.  It just running through nodes rather than a parmaeter map traversal is still a huge
// improvement, but we need to get to the bottom of where the missing script variables are coming from.
void UNiagaraGraph::PostLoad_ManageScriptVariables(int32 NiagaraVersion)
{
	// we only want to manage script variables for modules/functions.  Top level scripts system/emitter/particle scripts don't
	// need to manage their parameters via script variables
	bool bHasStandaloneScriptUsage = false;

	{
		TArray<UNiagaraNodeOutput*> OutputNodes;
		FindOutputNodes(OutputNodes);

		auto ScriptUsagePredicate = [](const UNiagaraNodeOutput* OutputNode) -> bool
		{
			const ENiagaraScriptUsage Usage = OutputNode->GetUsage();

			return (Usage == ENiagaraScriptUsage::Module ||
				Usage == ENiagaraScriptUsage::DynamicInput ||
				Usage == ENiagaraScriptUsage::Function);
		};

		bHasStandaloneScriptUsage = OutputNodes.ContainsByPredicate(ScriptUsagePredicate);
	}

	TSet<FNiagaraVariable> ReferencedVariables;
	if (bHasStandaloneScriptUsage)
	{
		TArray<UNiagaraNode*> NiagaraNodes;
		NiagaraNodes.Reserve(Nodes.Num());
		for (TObjectPtr<UEdGraphNode>& Node : Nodes)
		{
			if (Node)
			{
				if (UNiagaraNode* NiagaraNode = Cast<UNiagaraNode>(Node.Get()))
				{
					NiagaraNodes.Add(NiagaraNode);
				}
			}
		}
		NiagaraGraphImpl::FindReferencedVariables(this, NiagaraNodes, ReferencedVariables);
	}

	// clear out any script variables that are not in the set of referenced variables above
	for (FScriptVariableMap::TIterator ScriptVariableIt = VariableToScriptVariable.CreateIterator(); ScriptVariableIt; ++ScriptVariableIt)
	{
		if (!ReferencedVariables.Contains(ScriptVariableIt.Key()))
		{
			ScriptVariableIt.RemoveCurrent();
		}
	}

	// and also ensure that we're not actually missing any script variables
	for (const FNiagaraVariable& Variable : ReferencedVariables)
	{
		if (!VariableToScriptVariable.Contains(Variable))
		{
			CreateScriptVariableInternal(Variable, FNiagaraVariableMetaData(), false);
		}
	}
}

UNiagaraGraph::UNiagaraGraph()
	: bNeedNumericCacheRebuilt(true)
	, bIsRenamingParameter(false)
	, bParameterReferenceRefreshPending(true)
{
	Schema = UEdGraphSchema_Niagara::StaticClass();
	ChangeId = FGuid::NewGuid();
	
	ParameterHierarchyRoot = CreateDefaultSubobject<UHierarchyRoot>("ParameterHierarchyRoot");
}

void UNiagaraGraph::PostLoad()
{
	LLM_SCOPE(ELLMTag::Niagara);
	check(!bIsForCompilationOnly);

	Super::PostLoad();

	//////////////////////////////////////////////////////////////////////////
	/// GLOBAL CLEANUP/INVALIDATION
	///		-has no dependence on other chunks
	///		-can add/modify UNiagaraScriptVariable
	//////////////////////////////////////////////////////////////////////////

	if (GIsEditor)
	{
		SetFlags(RF_Transactional);
	}

	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

	// If this is from a prior version, enforce a valid Change Id!
	if (ChangeId.IsValid() == false)
	{
		MarkGraphRequiresSynchronization(TEXT("Graph change id was invalid"));
	}

	// Since we added the ReferenceHashFromGraph we need to keep it updated but the id won't be out of sync, so we 
	// just do a quick check to see if all are missing then force it.
	{
		int32 NumMissing = 0;
		for (int32 i = 0; i < CachedUsageInfo.Num(); i++)
		{
			if (false == CachedUsageInfo[i].ReferenceHashFromGraph.IsValid())
			{
				NumMissing++;
			}
		}

		if (NumMissing == CachedUsageInfo.Num())
		{
			CachedUsageInfo.Empty();
			LastBuiltTraversalDataChangeId = FGuid();
		}
	}

	// There was a bug where the graph traversal could have nullptr entries in it, which causes crashes later. Detect these and just clear out the cache.
	{
		bool bAnyInvalid = false;
		for (FNiagaraGraphScriptUsageInfo& CachedUsageInfoItem : CachedUsageInfo)
		{
			CachedUsageInfoItem.PostLoad(this);
			if (!CachedUsageInfoItem.IsValid())
			{
				bAnyInvalid = true;
			}
		}
		if (bAnyInvalid)
		{
			CachedUsageInfo.Empty();
			LastBuiltTraversalDataChangeId = FGuid();
		}
	}

	// Migrate input condition metadata
	if (NiagaraVer < FNiagaraCustomVersion::MetaDataAndParametersUpdate)
	{
		int NumMigrated = 0;
		// If the version of the asset is older than FNiagaraCustomVersion::MetaDataAndParametersUpdate 
		// we need to migrate the old metadata by looping through VariableToMetaData_DEPRECATED
		// and create new entries in VariableToScriptVariable
		for (auto& Pair : VariableToMetaData_DEPRECATED)
		{
			FNiagaraVariable Var(Pair.Key.GetType(), Pair.Key.GetName());
			SetMetaData(Var, Pair.Value);
			NumMigrated++;
		}
		VariableToMetaData_DEPRECATED.Empty();
	}

	// TODO - put behind a CustomVersion
	if (!VariableToScriptVariable.IsEmpty())
	{
		const TArray<FNiagaraVariable> StaticSwitchInputs = FindStaticSwitchInputs();

		for (auto It = VariableToScriptVariable.CreateIterator(); It; ++It)
		{
			FNiagaraVariable Var = It.Key();
			TObjectPtr<UNiagaraScriptVariable>& ScriptVar = It.Value();

			if (ScriptVar == nullptr)
			{
				ScriptVar = NewObject<UNiagaraScriptVariable>(const_cast<UNiagaraGraph*>(this));
				ScriptVar->Init(Var, FNiagaraVariableMetaData());
				UE_LOGF(LogNiagaraEditor, Display, "Fixed null UNiagaraScriptVariable | variable %ls | asset path %ls", *Var.GetName().ToString(), *GetPathName());
			}
			else
			{
				if (ScriptVar->GetName().Contains(TEXT(".")) || ScriptVar->GetName().Contains(TEXT(" ")))
				{
					ScriptVar->Rename(*MakeUniqueObjectName(this, UNiagaraScriptVariable::StaticClass(), NAME_None).ToString(), nullptr, REN_NonTransactional);
				}

				// Conditional postload all ScriptVars to ensure static switch default values are allocated as these are required when postloading all graph nodes later.
				ScriptVar->ConditionalPostLoad();
			}
			ScriptVar->SetIsStaticSwitch(StaticSwitchInputs.Contains(Var));
		}
	}

	if (NiagaraVer < FNiagaraCustomVersion::StaticSwitchFunctionPinsUsePersistentGuids)
	{
		// Make sure that the variable persistent ids are unique, otherwise pin allocation for static switches will be inconsistent.
		TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptVariables;
		VariableToScriptVariable.GenerateValueArray(ScriptVariables);
		TMap<FGuid, UNiagaraScriptVariable*> VariableGuidToScriptVariable;
		for (TObjectPtr<UNiagaraScriptVariable>& ScriptVariable : ScriptVariables)
		{
			if (ScriptVariable != nullptr)
			{
				UNiagaraScriptVariable** MatchingScriptVariablePtr = VariableGuidToScriptVariable.Find(ScriptVariable->Metadata.GetVariableGuid());
				if (MatchingScriptVariablePtr == nullptr)
				{
					VariableGuidToScriptVariable.Add(ScriptVariable->Metadata.GetVariableGuid(), ScriptVariable);
				}
				else
				{
					UNiagaraScriptVariable* MatchingScriptVariable = *MatchingScriptVariablePtr;
					if (ScriptVariable->GetIsSubscribedToParameterDefinitions() && MatchingScriptVariable->GetIsSubscribedToParameterDefinitions())
					{
						// Both of the script variables with duplicate ids are controlled by parameter definitions so issue a warning because neither will be updated.
						UE_LOGF(LogNiagaraEditor, Log, "Duplicate ids found for script variables which are both subscribed to parameter definitions.\nScript Variable 1 Name: %ls Type: %ls Path: %ls\nScript Variable 2 Name: %ls Type: %ls Path: %ls",
							*MatchingScriptVariable->Variable.GetName().ToString(), *MatchingScriptVariable->Variable.GetType().GetName(), *MatchingScriptVariable->GetPathName(),
							*ScriptVariable->Variable.GetName().ToString(), *ScriptVariable->Variable.GetType().GetName(), *ScriptVariable->GetPathName());
					}
					else
					{
						// Remove the duplicated entry and regenerate the ids for entries not subscribed to parameter definitions.
						VariableGuidToScriptVariable.Remove(ScriptVariable->Metadata.GetVariableGuid());
						if (ScriptVariable->GetIsSubscribedToParameterDefinitions() == false)
						{
							ScriptVariable->Metadata.SetVariableGuid(UNiagaraScriptVariable::GenerateStableGuid(ScriptVariable));
							if (VariableGuidToScriptVariable.Contains(ScriptVariable->Metadata.GetVariableGuid()))
							{
								UE_LOGF(LogNiagaraEditor, Log, "Could not generate a stable unique variable guid for script variable. Name: %ls Type: %ls Path: %ls",
									*ScriptVariable->Variable.GetName().ToString(), *ScriptVariable->Variable.GetType().GetName(), *ScriptVariable->GetPathName());
							}
							else
							{
								VariableGuidToScriptVariable.Add(ScriptVariable->Metadata.GetVariableGuid(), ScriptVariable);
							}
						}
						if (MatchingScriptVariable->GetIsSubscribedToParameterDefinitions() == false)
						{
							MatchingScriptVariable->Metadata.SetVariableGuid(UNiagaraScriptVariable::GenerateStableGuid(MatchingScriptVariable));
							if (VariableGuidToScriptVariable.Contains(MatchingScriptVariable->Metadata.GetVariableGuid()))
							{
								UE_LOGF(LogNiagaraEditor, Log, "Could not generate a stable unique variable guid for script variable. Name: %ls Type: %ls Path: %ls",
									*MatchingScriptVariable->Variable.GetName().ToString(), *MatchingScriptVariable->Variable.GetType().GetName(), *MatchingScriptVariable->GetPathName());
							}
							else
							{
								VariableGuidToScriptVariable.Add(MatchingScriptVariable->Metadata.GetVariableGuid(), MatchingScriptVariable);
							}
						}
					}
				}
			}
		}
	}

	if (NiagaraVer < FNiagaraCustomVersion::MoveCommonInputMetadataToProperties)
	{
		auto MigrateInputCondition = [](TMap<FName, FString>& PropertyMetaData, const FName& InputConditionKey, FNiagaraInputConditionMetadata& InOutInputCondition)
		{
			FString* InputCondition = PropertyMetaData.Find(InputConditionKey);
			if (InputCondition != nullptr)
			{
				FString InputName;
				FString TargetValue;
				int32 EqualsIndex = InputCondition->Find("=");
				if (EqualsIndex == INDEX_NONE)
				{
					InOutInputCondition.InputName = **InputCondition;
				}
				else
				{
					InOutInputCondition.InputName = *InputCondition->Left(EqualsIndex);
					InOutInputCondition.TargetValues.Add(InputCondition->RightChop(EqualsIndex + 1));
				}
				PropertyMetaData.Remove(InputConditionKey);
			}
		};

		for (auto& VariableToScriptVariableItem : VariableToScriptVariable)
		{
			TObjectPtr<UNiagaraScriptVariable>& MetaData = VariableToScriptVariableItem.Value;

			// Migrate advanced display.
			if (MetaData->Metadata.PropertyMetaData.Contains("AdvancedDisplay"))
			{
				MetaData->Metadata.bAdvancedDisplay = true;
				MetaData->Metadata.PropertyMetaData.Remove("AdvancedDisplay");
			}

			// Migrate inline edit condition toggle
			if (MetaData->Metadata.PropertyMetaData.Contains("InlineEditConditionToggle"))
			{
				MetaData->Metadata.bInlineEditConditionToggle = true;
				MetaData->Metadata.PropertyMetaData.Remove("InlineEditConditionToggle");
			}

			// Migrate edit and visible conditions
			MigrateInputCondition(MetaData->Metadata.PropertyMetaData, TEXT("EditCondition"), MetaData->Metadata.EditCondition);
			MigrateInputCondition(MetaData->Metadata.PropertyMetaData, TEXT("VisibleCondition"), MetaData->Metadata.VisibleCondition);
		}
	}

	if (NiagaraVer < FNiagaraCustomVersion::StandardizeParameterNames)
	{
		StandardizeParameterNames();
	}

	//////////////////////////////////////////////////////////////////////////
	/// NODE VALIDATION
	//////////////////////////////////////////////////////////////////////////


	{
		TArray<UNiagaraNodeInput*> InputNodes;

		for (UEdGraphNode* Node : Nodes)
		{
			if (ensureMsgf(Node, TEXT("Unexpected null node in niagara graph %s"), *GetPathName()))
			{
				Node->ConditionalPostLoad();
			}
		}
		
		for (UEdGraphNode* Node : Nodes)
		{
			if (UNiagaraNode* NiagaraNode = Cast<UNiagaraNode>(Node))
			{
				// Assume that all externally referenced assets have changed, so update to match. They will return true if they have changed.
				if (UObject* ReferencedAsset = NiagaraNode->GetReferencedAsset())
				{
					ReferencedAsset->ConditionalPostLoad();
					NiagaraNode->RefreshFromExternalChanges();
				}
				
				if (UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(NiagaraNode))
				{
					InputNodes.Add(InputNode);
				}
			}
		}

		// In the past, we didn't bother setting the CallSortPriority and just used lexicographic ordering.
		// In the event that we have multiple non-matching nodes with a zero call sort priority, this will
		// give every node a unique order value.
		if (!InputNodes.IsEmpty())
		{
			bool bPrioritiesAllZeroes = true;
			TArray<FName> UniqueNames;

			for (UNiagaraNodeInput* InputNode : InputNodes)
			{
				if (InputNode->CallSortPriority != 0)
				{
					bPrioritiesAllZeroes = false;
				}

				if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter)
				{
					UniqueNames.AddUnique(InputNode->Input.GetName());
				}

				if (InputNode->Usage == ENiagaraInputNodeUsage::SystemConstant)
				{
					InputNode->Input = FNiagaraConstants::UpdateEngineConstant(InputNode->Input);
				}
			}

			if (bPrioritiesAllZeroes && UniqueNames.Num() > 1)
			{
				// Just do the lexicographic sort and assign the call order to their ordered index value.
				UniqueNames.Sort(FNameLexicalLess());
				for (UNiagaraNodeInput* InputNode : InputNodes)
				{
					if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter)
					{
						int32 FoundIndex = UniqueNames.Find(InputNode->Input.GetName());
						check(FoundIndex != INDEX_NONE);
						InputNode->CallSortPriority = FoundIndex;
					}
				}
			}
		}
	}

	// Fix LWC position type mapping; before LWC, all position were FVector3f, but now that they are a separate type, we
	// need to upgrade the pins and metadata to match
	PostLoad_LWCFixup(NiagaraVer);

	PostLoad_ManageScriptVariables(NiagaraVer);

	// Fix inconsistencies in the default value declaration between graph and metadata
	ValidateDefaultPins();

	// this will ensure that the parameter maps have been refreshed and our NiagaraScriptVariable are up to date.
	// There's an argument to be made that we should just call RefreshParameterReferences directly and avoid explicitly
	// generating the compile ids but there's still code out there that is expecting to have fully generated compile ids
	// available on load.
	RebuildCachedCompileIds();

	// we want to make sure that nothing above resulted in the reference map from being initialized as it shouldn't be necessary to 
	// do such heavy lifting during PostLoad()
	ensureMsgf(bParameterReferenceRefreshPending,
		TEXT("ParameterReferenceMap was built during PostLoad of %s - for load time reasons this should be cleaned up."),
		*GetFullName());
}

#if WITH_EDITORONLY_DATA
void UNiagaraGraph::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraScriptVariable::StaticClass()));
}
#endif

void UNiagaraGraph::ChangeParameterType(const TArray<FNiagaraVariable>& ParametersToChange, const FNiagaraTypeDefinition& NewType, bool bAllowOrphanedPins)
{
	check(!bIsForCompilationOnly);
	Modify();

	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	TArray<UNiagaraNodeParameterMapBase*> NiagaraParameterNodes;
	GetNodesOfClass<UNiagaraNodeParameterMapBase>(NiagaraParameterNodes);

	// we will encounter pins that we've already changed, so we cache their previous state here to carry them over to potential new orphaned pins
	TMap<UEdGraphPin*, FNiagaraTypeDefinition> OldTypesCache;
	TMap<UEdGraphPin*, FString> OldDefaultValuesCache;
	TMap<UEdGraphPin*, FString> OldAutogeneratedDefaultValuesCache;
	
	// we keep track of pins that will (but haven't yet) change type to keep connections intact that will have pins of the same type after the op is done
	TSet<UEdGraphPin*> ChangingPins;
	// a subset of changing pins. This only includes parameter pins (i.e. outputs on map gets & inputs on map sets)
	TSet<UEdGraphPin*> ChangingParameterPins;
	TSet<UEdGraphPin*> ChangingDefaultPins;
	// since default pins are named 'None' but, if orphaned, can exist independently from their outputs, we cache the parameter pin's name here to override the new orphaned pin's name
	TMap<UEdGraphPin*, FName> OrphanedDefaultPinNameOverrides;
	
	auto GatherChangingPins = [&](UNiagaraNodeParameterMapBase* Node)
	{
		for(UEdGraphPin* Pin : Node->Pins)
		{
			if(Node->IsAddPin(Pin) || Pin->bOrphanedPin)
			{
				continue;
			}

			// we add pins that represent the parameter here
			FNiagaraVariable Variable(UEdGraphSchema_Niagara::PinToTypeDefinition(Pin), Pin->PinName);			
			if(ParametersToChange.Contains(Variable) && Pin->PinType.PinSubCategory == UNiagaraNodeParameterMapBase::ParameterPinSubCategory)
			{
				ChangingPins.Add(Pin);
				ChangingParameterPins.Add(Pin);
				OldTypesCache.Add(Pin, Variable.GetType());
				OldDefaultValuesCache.Add(Pin, Pin->DefaultValue);
				OldAutogeneratedDefaultValuesCache.Add(Pin, Pin->AutogeneratedDefaultValue);
				
				// parameter map get nodes might have additional default pins that we can't match via name
				if(UNiagaraNodeParameterMapGet* MapGet = Cast<UNiagaraNodeParameterMapGet>(Node))
				{
					if(UEdGraphPin* DefaultPin = MapGet->GetDefaultPin(Pin))
					{
						ChangingPins.Add(DefaultPin);
						ChangingDefaultPins.Add(DefaultPin);
						OrphanedDefaultPinNameOverrides.Add(DefaultPin, Pin->PinName);
						OldTypesCache.Add(DefaultPin, Variable.GetType());
						OldDefaultValuesCache.Add(DefaultPin, DefaultPin->DefaultValue);
						OldAutogeneratedDefaultValuesCache.Add(DefaultPin, DefaultPin->AutogeneratedDefaultValue);
					}
				}
			}
		}
	};
	
	for (UNiagaraNodeParameterMapBase* NiagaraNode : NiagaraParameterNodes)
	{
		GatherChangingPins(NiagaraNode);
	}
	
	auto TryCreateOrphanedPin = [&](UNiagaraNode* Node, FName OrphanedPinName, UEdGraphPin* OriginalPin)
	{
		if(OriginalPin->LinkedTo.Num() > 0 || (OriginalPin->Direction == EGPD_Input && OriginalPin->LinkedTo.Num() == 0 && OriginalPin->bDefaultValueIsIgnored == false && OldDefaultValuesCache[OriginalPin] != OldAutogeneratedDefaultValuesCache[OriginalPin]))
		{
			// We check pin connections for type to determine if we want to keep a connection or orphan it
			TSet<UEdGraphPin*> PinConnectionsToKeep;
			for (UEdGraphPin* ConnectedPin : OriginalPin->LinkedTo)
			{
				// we might have already visited and changed the connected pin, so we test against its old type cache if available
				FNiagaraTypeDefinition ConnectedPinType = UEdGraphSchema_Niagara::PinToTypeDefinition(ConnectedPin);
				FNiagaraVariable ConnectedVariable(OldTypesCache.Contains(ConnectedPin) ? OldTypesCache[ConnectedPin] : ConnectedPinType, ConnectedPin->PinName);

				/* We keep a connection in the following cases:
				// - Default case: if the connected pin already has the type we are changing to
				// - If the connected pin will have a matching type after type conversion is done
				// - If the connected pin is a numeric pin and the new type will be compatible with a numeric pin
				// - If we change types from vec -> position for easier fixup */
				if(ConnectedPinType == NewType 
					|| (ConnectedPin->PinType.PinSubCategory == UNiagaraNodeParameterMapBase::ParameterPinSubCategory && ChangingPins.Contains(ConnectedPin))
					|| (ConnectedPinType == FNiagaraTypeDefinition::GetGenericNumericDef() && FNiagaraTypeDefinition::GetNumericTypes().Contains(NewType))
					|| (GetDefault<UNiagaraSettings>()->bEnforceStrictStackTypes == false && ConnectedPinType == FNiagaraTypeDefinition::GetVec3Def() && NewType == FNiagaraTypeDefinition::GetPositionDef()))
				{
					PinConnectionsToKeep.Add(ConnectedPin);
				}						
			}

			// in case we are keeping all connections, all previously existing connections are parameter pins of the same type or will be of the same type after this operation is done
			// we also check for default values cache as by this point we have already changed the default values of the original pin
			if((PinConnectionsToKeep.Num() != OriginalPin->LinkedTo.Num()) || (OldDefaultValuesCache[OriginalPin] != OldAutogeneratedDefaultValuesCache[OriginalPin]))
			{
				ensure(OldTypesCache.Contains(OriginalPin) && OldDefaultValuesCache.Contains(OriginalPin));
				
				UEdGraphPin* NewOrphanedPin = Node->CreatePin(OriginalPin->Direction, OriginalPin->PinType.PinCategory, OrphanedPinName);
				NewOrphanedPin->PinType = UEdGraphSchema_Niagara::TypeDefinitionToPinType(OldTypesCache[OriginalPin]);
				NewOrphanedPin->DefaultValue = OldDefaultValuesCache[OriginalPin];
				NewOrphanedPin->PersistentGuid = FGuid::NewGuid();
				NewOrphanedPin->bNotConnectable = true;
				NewOrphanedPin->bOrphanedPin = true;
			
				TSet<UEdGraphPin*> PinsToDisconnect;
				for (UEdGraphPin* ConnectedPin : OriginalPin->LinkedTo)
				{
					ConnectedPin->Modify();
				
					// wire up the new orphaned pin with previous connections if we aren't keeping the connection
					if(!PinConnectionsToKeep.Contains(ConnectedPin))
					{
						NewOrphanedPin->LinkedTo.Add(ConnectedPin);
						ConnectedPin->LinkedTo.Add(NewOrphanedPin);

						PinsToDisconnect.Add(ConnectedPin);
					}
				}

				// disconnect the previous connections from the original pin
				for (UEdGraphPin* PinToDisconnect : PinsToDisconnect)
				{
					PinToDisconnect->LinkedTo.Remove(OriginalPin);
					OriginalPin->LinkedTo.Remove(PinToDisconnect);
				}

				return NewOrphanedPin;
			}
		}
		
		return static_cast<UEdGraphPin*>(nullptr);
	};
		
	for (UNiagaraNodeParameterMapBase* NiagaraNode : NiagaraParameterNodes)
	{
		TArray<UEdGraphPin*> NewOrphanedPins;

		bool bNodeChanged = false;
		TMap<UEdGraphPin*, FName> ValidOrphanedPinCandidates;
		// we go through all changing pins to change their type, and if applicable, attempt to create an orphaned pin
		for (UEdGraphPin* Pin : NiagaraNode->Pins)
		{			
			if (ChangingPins.Contains(Pin))
			{					
				Pin->Modify();
				Pin->PinType = NiagaraSchema->TypeDefinitionToPinType(NewType);
				// we need to restore the parameter sub category here, but only for parameter pins
				if(ChangingParameterPins.Contains(Pin))
				{
					Pin->PinType.PinSubCategory = UNiagaraNodeParameterMapBase::ParameterPinSubCategory;
				}
				
				if(bAllowOrphanedPins)
				{
					// the name for most parameter pins will remain the same as before, but not for default pins, which will adapt the parameter's name
					FName OrphanedPinName = OrphanedDefaultPinNameOverrides.Contains(Pin) ? OrphanedDefaultPinNameOverrides[Pin] : Pin->PinName; 
					ValidOrphanedPinCandidates.Add(Pin, OrphanedPinName);
				}

				// since we have now determined if we want to create an orphaned pin based on default values, we can safely reset them.
				Pin->ResetDefaultValue();
				UNiagaraNode::SetPinDefaultToTypeDefaultIfUnset(Pin);

				bNodeChanged = true;
			}
		}

		TMap<UEdGraphPin*, UEdGraphPin*> OriginalToOrphanedMap;
		for(auto& OrphanedPinCandidate : ValidOrphanedPinCandidates)
		{
			UEdGraphPin* NewOrphanedPin = TryCreateOrphanedPin(NiagaraNode, OrphanedPinCandidate.Value, OrphanedPinCandidate.Key);
			if(NewOrphanedPin != nullptr)
			{
				OriginalToOrphanedMap.Add(OrphanedPinCandidate.Key, NewOrphanedPin);
				NewOrphanedPins.Add(NewOrphanedPin);
			}
		}

		// we now match up the new orphaned default & output pins on map get nodes and add them to the node's map
		TSet<UEdGraphPin*> HandledGuidsOfOrphanedPins;
		for(auto& Pin : NiagaraNode->Pins)
		{	
			if(OriginalToOrphanedMap.Contains(Pin) && !HandledGuidsOfOrphanedPins.Contains(Pin))
			{
				UEdGraphPin* OrphanedPin = OriginalToOrphanedMap[Pin];
				OrphanedPin->PersistentGuid = FGuid::NewGuid();
				HandledGuidsOfOrphanedPins.Add(Pin);
				if(UNiagaraNodeParameterMapGet* MapGet = Cast<UNiagaraNodeParameterMapGet>(NiagaraNode))
				{
					if(UEdGraphPin* DefaultPin = MapGet->GetDefaultPin(Pin))
					{
						if(OriginalToOrphanedMap.Contains(DefaultPin))
						{
							if(UEdGraphPin* NewOrphanedDefaultPin = OriginalToOrphanedMap[DefaultPin])
							{
								NewOrphanedDefaultPin->PersistentGuid = FGuid::NewGuid();
								HandledGuidsOfOrphanedPins.Add(DefaultPin);
								MapGet->AddOrphanedPinPairGuids(OrphanedPin, NewOrphanedDefaultPin);
							}
						}
					}
				}
			}
		}		

		if(bNodeChanged)
		{
			NiagaraNode->MarkNodeRequiresSynchronization(__FUNCTION__, true);
		}
	}

	for(const FNiagaraVariable& CurrentParameter : ParametersToChange)
	{		
		TObjectPtr<UNiagaraScriptVariable>* ScriptVariablePtr = VariableToScriptVariable.Find(CurrentParameter);
		if (ScriptVariablePtr != nullptr && *ScriptVariablePtr)
		{
			UNiagaraScriptVariable& ScriptVar = *ScriptVariablePtr->Get();
			ScriptVar.Modify();
			ScriptVar.Variable.SetType(NewType);
			VariableToScriptVariable.Remove(CurrentParameter);

			FNiagaraVariable NewVarType(NewType, CurrentParameter.GetName());
			VariableToScriptVariable.Add(NewVarType, TObjectPtr<UNiagaraScriptVariable>(&ScriptVar));
			ScriptVariableChanged(NewVarType);
		}

		// Also fix stale parameter reference map entries. This should usually happen in RefreshParameterReferences, but since we just changed VariableToScriptVariable we need to fix that as well. 
		FNiagaraGraphParameterReferenceCollection* ReferenceCollection = ParameterToReferencesMap.Find(CurrentParameter);
		if (ReferenceCollection != nullptr)
		{
			if (ReferenceCollection->ParameterReferences.Num() > 0)
			{
				FNiagaraVariable NewVarType(NewType, CurrentParameter.GetName());
				FNiagaraGraphParameterReferenceCollection NewReferenceCollection = *ReferenceCollection;
				ParameterToReferencesMap.Add(NewVarType, NewReferenceCollection);
			}
			ParameterToReferencesMap.Remove(CurrentParameter);
		}
	}
	
	InvalidateCachedParameterData();
	OnParametersChangedDelegate.Broadcast(TInstancedStruct<FNiagaraParametersChangedData>::Make());
}

void UNiagaraGraph::ReplaceScriptReferences(UNiagaraScript* OldScript, UNiagaraScript* NewScript)
{
	bool bChanged = false;
	for (UEdGraphNode* Node : Nodes)
	{
		UNiagaraNodeFunctionCall* FunctionNode = Cast<UNiagaraNodeFunctionCall>(Node);
		if (FunctionNode && FunctionNode->FunctionScript == OldScript)
		{
			FunctionNode->Modify();
			FunctionNode->FunctionScript = NewScript;
			bChanged = true;
		}
	}
	if (bChanged)
	{
		MarkGraphRequiresSynchronization(TEXT("Script references updated"));
	}
}

UNiagaraGraph::FOnParametersChanged& UNiagaraGraph::OnParametersChanged()
{
	return OnParametersChangedDelegate;
}

TArray<UEdGraphPin*> UNiagaraGraph::FindParameterMapDefaultValuePins(const FName VariableName) const
{
	TArray<UEdGraphPin*> OutDefaultPins;
	
	TArray<UNiagaraNodeParameterMapGet*> MapGetNodes;
	GetNodesOfClass<UNiagaraNodeParameterMapGet>(MapGetNodes);

	for(UNiagaraNodeParameterMapGet* MapGet : MapGetNodes)
	{
		FPinCollectorArray OutputPins;
		MapGet->GetOutputPins(OutputPins);
		for (UEdGraphPin* OutputPin : OutputPins)
		{
			if (VariableName != OutputPin->PinName || OutputPin->PinType.PinSubCategory != UNiagaraNodeParameterMapBase::ParameterPinSubCategory)
			{
				continue;
			}
			
			if (UEdGraphPin* Pin = MapGet->GetDefaultPin(OutputPin))
			{
				check(Pin->Direction == EEdGraphPinDirection::EGPD_Input);
				OutDefaultPins.AddUnique(Pin);
			}
		}
	}
	
	return OutDefaultPins;
}

// Impacts of this function include changing the default value of UNiagaraScriptVariable::Variable & DefaultMode (note that this won't impact the
// key in the VariableToScriptVariable map.  Additionally properties on the pins can also be changed.
void UNiagaraGraph::ValidateDefaultPins()
{
	check(!bIsForCompilationOnly);

	FNiagaraEditorModule& EditorModule = FNiagaraEditorModule::Get();

	if (!VariableToScriptVariable.IsEmpty())
	{
		TArray<UNiagaraNodeParameterMapGet*> TraversedMapGetNodes;

		{
			TArray<UNiagaraNodeOutput*> OutputNodes;
			GetNodesOfClass(OutputNodes);

			TArray<UNiagaraNode*> NodesTraversed;
			for (UNiagaraNodeOutput* OutputNode : OutputNodes)
			{
				NodesTraversed.Reset();
				BuildTraversal(NodesTraversed, OutputNode);

				for (UNiagaraNode* NiagaraNode : NodesTraversed)
				{
					if (UNiagaraNodeParameterMapGet* MapGetNode = Cast<UNiagaraNodeParameterMapGet>(NiagaraNode))
					{
						TraversedMapGetNodes.AddUnique(MapGetNode);
					}
				}
			}
		}

		for (const FScriptVariableMap::ElementType& MetaData : VariableToScriptVariable)
		{
			const FNiagaraVariable& Variable = MetaData.Key;
			UNiagaraScriptVariable* ScriptVariable = MetaData.Value;
			if (!ScriptVariable || ScriptVariable->DefaultMode == ENiagaraDefaultMode::Custom || ScriptVariable->DefaultMode == ENiagaraDefaultMode::FailIfPreviouslyNotSet)
			{
				// If the user selected custom mode or if previously unset they can basically do whatever they want
				continue;
			}
			if (ScriptVariable->GetIsStaticSwitch())
			{
				// We ignore static switch variables as they handle default values differently
				continue;
			}

			// Determine if the values set in the graph (which are used for the compilation) are consistent and if necessary update the metadata value
			FPinCollectorArray DefaultPins;

			{
				for (UNiagaraNodeParameterMapGet* MapGetNode : TraversedMapGetNodes)
				{
					FPinCollectorArray OutputPins;
					MapGetNode->GetOutputPins(OutputPins);
					for (UEdGraphPin* OutputPin : OutputPins)
					{
						if (Variable.GetName() != OutputPin->PinName || OutputPin->PinType.PinSubCategory != UNiagaraNodeParameterMapBase::ParameterPinSubCategory)
						{
							continue;
						}
						if (UEdGraphPin* Pin = MapGetNode->GetDefaultPin(OutputPin))
						{
							check(Pin->Direction == EEdGraphPinDirection::EGPD_Input);
							DefaultPins.AddUnique(Pin);
						}
					}
				}
			}
			bool IsCustom = false;
			bool IsConsistent = true;
			bool DefaultInitialized = false;
			FNiagaraVariable DefaultData;
			auto TypeUtilityValue = EditorModule.GetTypeUtilities(Variable.GetType());
			for (UEdGraphPin* Pin : DefaultPins)
			{
				if (Pin->LinkedTo.Num() > 0)
				{
					IsCustom = true;
				}
				else if (!Pin->bHidden && TypeUtilityValue)
				{
					FNiagaraVariable ComparisonData = Variable;
					TypeUtilityValue->SetValueFromPinDefaultString(Pin->DefaultValue, ComparisonData);
					if (!DefaultInitialized)
					{
						DefaultData = ComparisonData;
						DefaultInitialized = true;
					}
					else if (!ComparisonData.HoldsSameData(DefaultData))
					{
						IsConsistent = false;
					}
				}
			}

			if (!IsCustom && !IsConsistent)
			{
				UE_LOGF(LogNiagaraEditor, Log, "Niagara graph %ls: The default value declaration for the variable '%ls' is not consistent between the graph and the metadata.\n  Either change the default mode to 'custom' or check the input pins in the parameter map get node in the graph.",
					*GetFullName(), *Variable.GetName().ToString());
				continue;
			}
			if (IsCustom)
			{
				ScriptVariable->DefaultMode = ENiagaraDefaultMode::Custom;
			}
			else
			{
				for (UEdGraphPin* Pin : DefaultPins)
				{
					Pin->bNotConnectable = true;
					Pin->bDefaultValueIsReadOnly = true;
					if (ScriptVariable->DefaultMode == ENiagaraDefaultMode::Binding || ScriptVariable->DefaultMode == ENiagaraDefaultMode::FailIfPreviouslyNotSet)
					{
						Pin->bHidden = true;
					}

				}
				if (DefaultInitialized)
				{
					// set default value from pins
					ScriptVariable->Variable = DefaultData;
				}
			}
		}
	}
}

FName UNiagaraGraph::StandardizeName(FName Name, ENiagaraScriptUsage Usage, bool bIsGet, bool bIsSet)
{
	bool bIsScriptUsage = 
		Usage == ENiagaraScriptUsage::Module || 
		Usage == ENiagaraScriptUsage::DynamicInput || 
		Usage == ENiagaraScriptUsage::Function;

	TArray<FStringView, TInlineAllocator<16>> NamePartStrings;
	TStringBuilder<128> NameAsString;
	Name.AppendString(NameAsString);
	UE::String::ParseTokens(NameAsString, TEXT("."), [&NamePartStrings](FStringView Token) { NamePartStrings.Add(Token); });

	TArray<FName, TInlineAllocator<16>> NameParts;
	for (FStringView NamePartString : NamePartStrings)
	{
		NameParts.Emplace(NamePartString);
	}
	if (NameParts.Num() == 0)
	{
		NameParts.Add(NAME_None);
	}

	TOptional<FName> Namespace;
	if (NameParts[0] == FNiagaraConstants::EngineNamespace ||
		NameParts[0] == FNiagaraConstants::ParameterCollectionNamespace ||
		NameParts[0] == FNiagaraConstants::UserNamespace ||
		NameParts[0] == FNiagaraConstants::ArrayNamespace ||
		NameParts[0] == FNiagaraConstants::DataInstanceNamespace)
	{
		// Don't modify engine, parameter collections, or user parameters since they're defined externally and won't have fixed up
		// names.  Also ignore the array and DataInstance namespaces because they're special.
		return Name;
	}
	else if (NameParts[0] == FNiagaraConstants::LocalNamespace)
	{
		// Local can only be used in the context of a single module so force it to have a secondary module namespace.
		static const FName LocalNamespace = *(FNiagaraConstants::LocalNamespace.ToString() + TEXT(".") + FNiagaraConstants::ModuleNamespace.ToString());
		Namespace = LocalNamespace;
		NameParts.RemoveAt(0);
	}
	else if (NameParts[0] == FNiagaraConstants::OutputNamespace)
	{
		static const FName OutputScriptNamespace = *(FNiagaraConstants::OutputNamespace.ToString() + TEXT(".") + FNiagaraConstants::ModuleNamespace.ToString());
		static const FName OutputUnknownNamespace = *(FNiagaraConstants::OutputNamespace.ToString() + TEXT(".") + FName(NAME_None).ToString());
		if (bIsScriptUsage)
		{
			if (bIsSet || NameParts.Num() <= 2)
			{
				// In a script outputs must always be written to the module namespace, and if the previous output didn't specify module
				// or a sub-namespace assume it was reading from the module namespace.
				Namespace = OutputScriptNamespace;
				NameParts.RemoveAt(0);
			}
			else
			{
				// When reading they can be from the module namespace, or a more specific namespace from a different module, we also need
				// to handle the case where they are reading from the output of a nested module, so allow an additional namespace too.
				if (NameParts.Num() > 3)
				{
					Namespace = *(NameParts[0].ToString() + TEXT(".") + NameParts[1].ToString() + TEXT(".") + NameParts[2].ToString());
					NameParts.RemoveAt(0, 3);
				}
				else
				{
					Namespace = *(NameParts[0].ToString() + TEXT(".") + NameParts[1].ToString());
					NameParts.RemoveAt(0, 2);
				}
			}
		}
		else
		{
			// The only valid usage for output parameters in system and emitter graphs is reading them from an aliased parameter of the form
			// Output.ModuleName.ValueName.  If there are not enough name parts put in 'none' for the module name.
			if (NameParts.Num() > 2)
			{
				Namespace = *(NameParts[0].ToString() + TEXT(".") + NameParts[1].ToString());
				NameParts.RemoveAt(0, 2);
			}
			else
			{
				Namespace = OutputUnknownNamespace;
				NameParts.RemoveAt(0);
			}
		}
	}
	else if (
		NameParts[0] == FNiagaraConstants::ModuleNamespace ||
		NameParts[0] == FNiagaraConstants::SystemNamespace ||
		NameParts[0] == FNiagaraConstants::EmitterNamespace ||
		NameParts[0] == FNiagaraConstants::ParticleAttributeNamespace)
	{
		if (NameParts.Num() == 2)
		{
			// Standard module input or dataset attribute.
			Namespace = NameParts[0];
			NameParts.RemoveAt(0);
		}
		else
		{
			// If there are more than 2 name parts, allow the first 2 for a namespace and sub-namespace.
			// Sub-namespaces are used for module specific dataset attributes, or for configuring
			// inputs for nested modules.
			Namespace = *(NameParts[0].ToString() + TEXT(".") + NameParts[1].ToString());
			NameParts.RemoveAt(0, 2);
		}
	}
	else if (NameParts[0] == FNiagaraConstants::TransientNamespace)
	{
		// Transient names have a single namespace.
		Namespace = FNiagaraConstants::TransientNamespace;
		NameParts.RemoveAt(0);
	}
	else
	{
		// Namespace was unknown.
		if (bIsScriptUsage && NameParts.Contains(FNiagaraConstants::ModuleNamespace))
		{
			// If we're in a script check for a misplaced module namespace and if it has one, force it to be a 
			// module output to help with fixing up usages.
			static const FName OutputScriptNamespace = *(FNiagaraConstants::OutputNamespace.ToString() + TEXT(".") + FNiagaraConstants::ModuleNamespace.ToString());
			Namespace = OutputScriptNamespace;
		}
		else if(bIsScriptUsage || bIsGet)
		{
			// If it's in a get node or it's in a script, force it into the transient namespace.
			Namespace = FNiagaraConstants::TransientNamespace;
		}
		else
		{
			// Otherwise it's a set node in a system or emitter script so it must be used to configure a module input.  For this situation
			// we have 2 cases, regular modules and set variables nodes.
			if (NameParts[0].ToString().StartsWith(TRANSLATOR_SET_VARIABLES_STR))
			{
				// For a set variables node we need to strip the module name and then fully standardize the remainder of the
				// name using the settings for a map set in a module.  We do this because the format of the input parameter will be
				// Module.NamespaceName.ValueName and the NamespaceName.ValueName portion will have been standardized as part of the
				// UNiagaraNodeAssignment post load.
				Namespace = NameParts[0];
				NameParts.RemoveAt(0);

				FString AssignmentTargetString = NameParts[0].ToString();
				for (int32 i = 1; i < NameParts.Num(); i++)
				{
					AssignmentTargetString += TEXT(".") + NameParts[i].ToString();
				}
				FName StandardizedAssignmentTargetName = StandardizeName(*AssignmentTargetString, ENiagaraScriptUsage::Module, false, true);

				NameParts.Empty();
				NameParts.Add(StandardizedAssignmentTargetName);
			}
			else
			{
				// For standard inputs we need to replace the module name with the module namespace and then standardize it, and then
				// remove the module namespace and use that as the name, and the module name as the namespace.
				Namespace = NameParts[0];
				NameParts.RemoveAt(0);

				FString ModuleInputString = FNiagaraConstants::ModuleNamespace.ToString();
				for (int32 i = 0; i < NameParts.Num(); i++)
				{
					ModuleInputString += TEXT(".") + NameParts[i].ToString();
				}
				FName StandardizedModuleInput = StandardizeName(*ModuleInputString, ENiagaraScriptUsage::Module, true, false);
				FString StandardizedModuleInputString = StandardizedModuleInput.ToString();
				StandardizedModuleInputString.RemoveFromStart(FNiagaraConstants::ModuleNamespace.ToString() + TEXT("."));

				NameParts.Empty();
				NameParts.Add(*StandardizedModuleInputString);
			}
		}
	}

	checkf(Namespace.IsSet(), TEXT("No namespace picked in %s."), *Name.ToString());

	NameParts.Remove(FNiagaraConstants::ModuleNamespace);
	if (NameParts.Num() == 0)
	{
		NameParts.Add(NAME_None);
	}

	// Form the name by concatenating the remaining parts of the name.
	FString ParameterName;
	if (NameParts.Num() == 1)
	{
		ParameterName = NameParts[0].ToString();
	}
	else
	{
		TArray<FString> RemainingNamePartStrings;
		for (FName NamePart : NameParts)
		{
			RemainingNamePartStrings.Add(NamePart.ToString());
		}
		ParameterName = FString::Join(RemainingNamePartStrings, TEXT(""));
	}

	// Last, combine it with the namespace(s) chosen above.
	return *FString::Printf(TEXT("%s.%s"), *Namespace.GetValue().ToString(), *ParameterName);
}

void UNiagaraGraph::StandardizeParameterNames()
{
	check(!bIsForCompilationOnly);

	TMap<FName, FName> OldNameToStandardizedNameMap;
	bool bAnyNamesModified = false;
	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	
	TArray<UNiagaraNodeOutput*> OutputNodes;
	GetNodesOfClass(OutputNodes);

	auto HandleParameterMapNode = [&bAnyNamesModified](const UEdGraphSchema_Niagara* NiagaraSchema, UNiagaraNodeParameterMapBase* Node, EEdGraphPinDirection PinDirection, ENiagaraScriptUsage Usage, bool bIsGet, bool bIsSet, TMap<FName, FName>& OldNameToStandardizedNameMap)
	{
		for (UEdGraphPin* Pin : Node->Pins)
		{
			if (Pin->Direction == PinDirection)
			{
				FNiagaraTypeDefinition PinType = NiagaraSchema->PinToTypeDefinition(Pin);
				if (PinType.IsValid() && PinType != FNiagaraTypeDefinition::GetParameterMapDef())
				{
					FName* NewNamePtr = OldNameToStandardizedNameMap.Find(Pin->PinName);
					FName NewName;
					if (NewNamePtr != nullptr)
					{
						NewName = *NewNamePtr;
					}
					else
					{
						NewName = StandardizeName(Pin->PinName, Usage, bIsGet, bIsSet);
						OldNameToStandardizedNameMap.Add(Pin->PinName, NewName);
					}

					if (Pin->PinName != NewName)
					{
						bAnyNamesModified = true;
					}
					Pin->PinName = NewName;
					Pin->PinFriendlyName = FText::AsCultureInvariant(NewName.ToString());
				}
			}
		}
	};

	TSet<UNiagaraNode*> AllTraversedNodes;
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		TArray<UNiagaraNode*> TraversedNodes;
		BuildTraversal(TraversedNodes, OutputNode, false);
		AllTraversedNodes.Append(TraversedNodes);

		for (UNiagaraNode* TraversedNode : TraversedNodes)
		{
			TraversedNode->ConditionalPostLoad();
			if (TraversedNode->IsA<UNiagaraNodeParameterMapGet>())
			{
				HandleParameterMapNode(NiagaraSchema, CastChecked<UNiagaraNodeParameterMapBase>(TraversedNode), EGPD_Output, OutputNode->GetUsage(), true, false, OldNameToStandardizedNameMap);
			}
			else if (TraversedNode->IsA<UNiagaraNodeParameterMapSet>())
			{
				HandleParameterMapNode(NiagaraSchema, CastChecked<UNiagaraNodeParameterMapBase>(TraversedNode), EGPD_Input, OutputNode->GetUsage(), false, true, OldNameToStandardizedNameMap);
			}
		}
	}

	for (UEdGraphNode* Node : Nodes)
	{
		UNiagaraNodeParameterMapBase* ParameterMapNode = Cast<UNiagaraNodeParameterMapBase>(Node);
		if (ParameterMapNode != nullptr && AllTraversedNodes.Contains(ParameterMapNode) == false)
		{
			if (ParameterMapNode->IsA<UNiagaraNodeParameterMapGet>())
			{
				HandleParameterMapNode(NiagaraSchema, ParameterMapNode, EGPD_Output, ENiagaraScriptUsage::Module, true, false, OldNameToStandardizedNameMap);
			}
			else if (ParameterMapNode->IsA<UNiagaraNodeParameterMapSet>())
			{
				HandleParameterMapNode(NiagaraSchema, ParameterMapNode, EGPD_Input, ENiagaraScriptUsage::Module, false, true, OldNameToStandardizedNameMap);
			}
		}
	}

	// Since we'll be modifying the keys, make a copy of the map and then clear the original so it cal be 
	// repopulated.
	FScriptVariableMap OldVariableToScriptVariable = VariableToScriptVariable;
	VariableToScriptVariable.Empty();
	for (TPair<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>> VariableScriptVariablePair : OldVariableToScriptVariable)
	{
		FNiagaraVariable Variable = VariableScriptVariablePair.Key;
		UNiagaraScriptVariable* ScriptVariable = VariableScriptVariablePair.Value;
		ScriptVariable->ConditionalPostLoad();
		
		if (ScriptVariable->GetIsStaticSwitch() == false)
		{
			// We ignore static switches here because they're not in the parameter and so they don't need
			// their parameter names to be fixed up.
			FName* NewNamePtr = OldNameToStandardizedNameMap.Find(Variable.GetName());
			FName NewName;
			if (NewNamePtr != nullptr)
			{
				NewName = *NewNamePtr;
			}
			else
			{
				FName OldName = Variable.GetName();
				NewName = StandardizeName(OldName, ENiagaraScriptUsage::Module, false, false);
				OldNameToStandardizedNameMap.Add(OldName, NewName);
			}

			if (Variable.GetName() != NewName)
			{
				bAnyNamesModified = true;
			}
			Variable.SetName(NewName);
			ScriptVariable->Variable.SetName(NewName);
		}

		VariableToScriptVariable.Add(Variable, ScriptVariable);
	}

	if (bAnyNamesModified)
	{
		MarkGraphRequiresSynchronization(TEXT("Standardized parameter names"));
	}
}

bool UNiagaraGraph::VariableLess(const FNiagaraVariable& Lhs, const FNiagaraVariable& Rhs)
{
	return Lhs.GetName().LexicalLess(Rhs.GetName());
}

TOptional<FNiagaraScriptVariableData> UNiagaraGraph::GetScriptVariableData(const FNiagaraVariable& Variable) const
{
	// for now this is only supported on the compilation copy
	if (bIsForCompilationOnly)
	{
		const int32 ScriptVariableIndex = Algo::BinarySearchBy(CompilationScriptVariables, Variable, &FNiagaraScriptVariableData::Variable, VariableLess);
		if (CompilationScriptVariables.IsValidIndex(ScriptVariableIndex))
		{
			return CompilationScriptVariables[ScriptVariableIndex];
		}
	}

	return TOptional<FNiagaraScriptVariableData>();
}

void UNiagaraGraph::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// if this is a compilation copy we want to avoid sending any notifications (as we're not really tied to anything anymore).
	// one area that this can happen is through a Reload of the source system while a compilation is in flight.
	if (bIsForCompilationOnly)
	{
		return;
	}

	NotifyGraphChanged();
	RefreshParameterReferences();
}

void UNiagaraGraph::BeginDestroy()
{
	Super::BeginDestroy();
	ReleaseCompilationCopy();
}

class UNiagaraScriptSource* UNiagaraGraph::GetSource() const
{
	return CastChecked<UNiagaraScriptSource>(GetOuter());
}

FNiagaraCompileHash UNiagaraGraph::GetCompileReferencedDataHash(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const
{
	for (int32 i = 0; i < CachedUsageInfo.Num(); i++)
	{
		if (UNiagaraScript::IsEquivalentUsage(CachedUsageInfo[i].UsageType, InUsage) && CachedUsageInfo[i].UsageId == InUsageId)
		{		
			return CachedUsageInfo[i].ReferenceHashFromGraph;
		}
	}
	return FNiagaraCompileHash();
}

FNiagaraCompileHash UNiagaraGraph::GetCompileDataHash(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const
{
	for (int32 i = 0; i < CachedUsageInfo.Num(); i++)
	{
		if (UNiagaraScript::IsEquivalentUsage(CachedUsageInfo[i].UsageType, InUsage) && CachedUsageInfo[i].UsageId == InUsageId)
		{
			if (GNiagaraUseGraphHash <= 0)
			{
				return CachedUsageInfo[i].CompileHash;
			}
			else
			{
				return CachedUsageInfo[i].CompileHashFromGraph;
			}
		}
	}
	return FNiagaraCompileHash();
}

FGuid UNiagaraGraph::GetBaseId(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const
{
	for (int32 i = 0; i < CachedUsageInfo.Num(); i++)
	{
		if (UNiagaraScript::IsEquivalentUsage(CachedUsageInfo[i].UsageType, InUsage) && CachedUsageInfo[i].UsageId == InUsageId)
		{
			return CachedUsageInfo[i].BaseId;
		}
	}
	return FGuid();
}

void UNiagaraGraph::ForceBaseId(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, const FGuid InForcedBaseId)
{
	FNiagaraGraphScriptUsageInfo* MatchingCachedUsageInfo = CachedUsageInfo.FindByPredicate([InUsage, InUsageId](const FNiagaraGraphScriptUsageInfo& CachedUsageInfoItem)
	{ 
		return CachedUsageInfoItem.UsageType == InUsage && CachedUsageInfoItem.UsageId == InUsageId; 
	});

	if (MatchingCachedUsageInfo == nullptr)
	{
		MatchingCachedUsageInfo = &CachedUsageInfo.AddDefaulted_GetRef();
		MatchingCachedUsageInfo->UsageType = InUsage;
		MatchingCachedUsageInfo->UsageId = InUsageId;
	}
	MatchingCachedUsageInfo->BaseId = InForcedBaseId;
}

UEdGraphPin* UNiagaraGraph::FindParameterMapDefaultValuePin(const FName VariableName, ENiagaraScriptUsage InUsage, ENiagaraScriptUsage InParentUsage) const
{
	TArray<UEdGraphPin*> DefaultInputPin = { nullptr };

	MultiFindParameterMapDefaultValuePins({ VariableName }, InUsage, InParentUsage, { DefaultInputPin });

	return DefaultInputPin[0];
}

void UNiagaraGraph::MultiFindParameterMapDefaultValuePins(TConstArrayView<FName> VariableNames, ENiagaraScriptUsage InUsage, ENiagaraScriptUsage InParentUsage, TArrayView<UEdGraphPin*> DefaultPins) const
{
	TArray<UEdGraphPin*> MatchingDefaultPins;

	TArray<UNiagaraNode*> NodesTraversed;
	BuildTraversal(NodesTraversed, InUsage, FGuid(), true);

	FPinCollectorArray OutputPins;

	const int32 VariableNameCount = VariableNames.Num();
	int32 DefaultPinsFound = 0;

	for (UNiagaraNode* Node : NodesTraversed)
	{
		if (UNiagaraNodeParameterMapGet* GetNode = Cast<UNiagaraNodeParameterMapGet>(Node))
		{
			OutputPins.Reset();
			GetNode->GetOutputPins(OutputPins);
			for (UEdGraphPin* OutputPin : OutputPins)
			{
				int32 NameIndex = VariableNames.IndexOfByKey(OutputPin->PinName);
				if ((NameIndex != INDEX_NONE) && (DefaultPins[NameIndex] == nullptr))
				{
					UEdGraphPin* Pin = GetNode->GetDefaultPin(OutputPin);
					if (Pin)
					{
						++DefaultPinsFound;
						DefaultPins[NameIndex] = Pin;

						if (DefaultPinsFound == VariableNameCount)
						{
							break;
						}
					}
				}
			}
		}
	}

	auto TraceInputPin = [&](UEdGraphPin* Pin)
	{
		if (Pin && Pin->LinkedTo.Num() != 0 && Pin->LinkedTo[0] != nullptr)
		{
			UNiagaraNode* Owner = Cast<UNiagaraNode>(Pin->LinkedTo[0]->GetOwningNode());
			UEdGraphPin* PreviousInput = Pin;
			int32 NumIters = 0;
			while (Owner)
			{
				// Check to see if there are any reroute or choose by usage nodes involved in this..
				UEdGraphPin* InputPin = Owner->GetPassThroughPin(PreviousInput->LinkedTo[0], InParentUsage);
				if (InputPin == nullptr)
				{
					return PreviousInput;
				}
				else if (InputPin->LinkedTo.Num() == 0)
				{
					return InputPin;
				}

				check(InputPin->LinkedTo[0] != nullptr);
				Owner = Cast<UNiagaraNode>(InputPin->LinkedTo[0]->GetOwningNode());
				PreviousInput = InputPin;
				++NumIters;
				check(NumIters < Nodes.Num()); // If you hit this assert then we have a cycle in our graph somewhere.
			}
		}

		return Pin;
	};

	if (DefaultPinsFound > 0)
	{
		for (UEdGraphPin*& DefaultInputPin : DefaultPins)
		{
			DefaultInputPin = TraceInputPin(DefaultInputPin);
		}
	}
}

void UNiagaraGraph::FindOutputNodes(TArray<UNiagaraNodeOutput*>& OutputNodes) const
{
	for (UEdGraphNode* Node : Nodes)
	{
		if (UNiagaraNodeOutput* OutNode = Cast<UNiagaraNodeOutput>(Node))
		{
			OutputNodes.Add(OutNode);
		}
	}
}


void UNiagaraGraph::FindOutputNodes(ENiagaraScriptUsage TargetUsageType, TArray<UNiagaraNodeOutput*>& OutputNodes) const
{
	TArray<UNiagaraNodeOutput*> NodesFound;
	for (UEdGraphNode* Node : Nodes)
	{
		UNiagaraNodeOutput* OutNode = Cast<UNiagaraNodeOutput>(Node);
		if (OutNode && OutNode->GetUsage() == TargetUsageType)
		{
			NodesFound.Add(OutNode);
		}
	}

	OutputNodes = NodesFound;
}

void UNiagaraGraph::FindEquivalentOutputNodes(ENiagaraScriptUsage TargetUsageType, TArray<UNiagaraNodeOutput*>& OutputNodes) const
{
	TArray<UNiagaraNodeOutput*> NodesFound;
	for (UEdGraphNode* Node : Nodes)
	{
		UNiagaraNodeOutput* OutNode = Cast<UNiagaraNodeOutput>(Node);
		if (OutNode && UNiagaraScript::IsEquivalentUsage(OutNode->GetUsage(), TargetUsageType))
		{
			NodesFound.Add(OutNode);
		}
	}

	OutputNodes = NodesFound;
}

void BuildCompleteTraversal(UEdGraphNode* CurrentNode, TArray<UEdGraphNode*>& AllNodes)
{
	if (CurrentNode == nullptr || AllNodes.Contains(CurrentNode))
	{
		return;
	}

	AllNodes.Add(CurrentNode);

	for (UEdGraphPin* Pin : CurrentNode->GetAllPins())
	{
		if(Pin->Direction == EGPD_Input)
		{
			for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
			{
				BuildCompleteTraversal(LinkedPin->GetOwningNode(), AllNodes);
			}
		}
	}
}


UNiagaraGraph* UNiagaraGraph::CreateCompilationCopy(const TArray<ENiagaraScriptUsage>& CompileUsages)
{
	check(!bIsForCompilationOnly);
	UNiagaraGraph* Result = NewObject<UNiagaraGraph>();
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

	// create a shallow copy
	for (TFieldIterator<FProperty> PropertyIt(GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if (*Property->GetNameCPP() == GET_MEMBER_NAME_CHECKED(UEdGraph, Nodes))
		{
			// The nodes will be handled separately below.
			continue;
		}
		const uint8* SourceAddr = Property->ContainerPtrToValuePtr<uint8>(this);
		uint8* DestinationAddr = Property->ContainerPtrToValuePtr<uint8>(Result);

		Property->CopyCompleteValue(DestinationAddr, SourceAddr);
	}
	Result->bIsForCompilationOnly = true;

	Result->CompilationScriptVariables.Reserve(Result->VariableToScriptVariable.Num());
	for (const auto& It : Result->VariableToScriptVariable)
	{
		if (const UNiagaraScriptVariable* ScriptVariable = It.Value)
		{
			FNiagaraScriptVariableData& VariableData = Result->CompilationScriptVariables.AddDefaulted_GetRef();
			VariableData.InitFrom(*ScriptVariable, false);
		}
	}
	Algo::StableSortBy(Result->CompilationScriptVariables, &FNiagaraScriptVariableData::Variable, VariableLess);

	Result->VariableToScriptVariable.Empty();

	// Only add references to nodes which match a compile usage.
	TMap<UEdGraphNode*, UEdGraphNode*> DuplicationMapping;
	TArray<UNiagaraNodeOutput*> OutputNodes;
	GetNodesOfClass(OutputNodes);
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		if(CompileUsages.ContainsByPredicate([OutputNode](ENiagaraScriptUsage CompileUsage) { return UNiagaraScript::IsEquivalentUsage(CompileUsage, OutputNode->GetUsage()); }))
		{
			TArray<UEdGraphNode*> TraversedNodes;
			BuildCompleteTraversal(OutputNode, TraversedNodes);
			Result->Nodes.Append(TraversedNodes);
		}
	}

	TSet<const UEdGraphNode*> RerouteNodesToFixup;
	TArray<UEdGraphNode*> NewNodes;
	for (UEdGraphNode* Node : Result->Nodes)
	{
		if (const UNiagaraNodeReroute* RerouteNode = Cast<const UNiagaraNodeReroute>(Node))
		{
			RerouteNodesToFixup.Add(RerouteNode);
			continue;
		}

		Node->ClearFlags(RF_Transactional);
		UEdGraphNode* DupNode = NewNodes.Add_GetRef(DuplicateObject(Node, Result));
		Node->SetFlags(RF_Transactional);
		DuplicationMapping.Add(Node, DupNode);
	}

	// Detect any invalidations that might have snuck in on the source graph!
	for (UEdGraphNode* Node : Result->Nodes)
	{
		for (UEdGraphPin* Pin : Node->Pins)
		{
			for (int j = 0; j < Pin->LinkedTo.Num(); j++)
			{
				if (Pin->LinkedTo[j])
				{
					UEdGraphNode* NodeB = Pin->LinkedTo[j]->GetOwningNode();
					if (Node && NodeB && Node->GetGraph() != NodeB->GetGraph())
					{
						UE_LOGF(LogNiagaraEditor, Error, "%ls", *NodeB->GetPathName());
						const FText WarningTemplate = LOCTEXT("DuplicationErrorToast", "Duplication Error! {0} was touched when cloning for compilation. Please report this issue to the Niagara team! ");
						FText MismatchWarning = FText::Format(
							WarningTemplate,
							FText::FromString(NodeB->GetPathName()));
						FNiagaraEditorUtilities::WarnWithToastAndLog(MismatchWarning);
					}
				}
			}
		}
	}

	for (UEdGraphNode* Node : NewNodes)
	{
		// fix up linked pins
		for (UEdGraphPin* Pin : Node->Pins)
		{
			ensure(Pin->GetOwningNode()->GetGraph() == Result);
			for (int i = 0; i < Pin->LinkedTo.Num(); i++)
			{
				UEdGraphPin* LinkedPin = Pin->LinkedTo[i];
				UEdGraphNode* OwningNode = LinkedPin->GetOwningNode();

				// if the owning node is already in the right graph then no need to adjust it (this can
				// happen if the pin connection is added as a part of removing the reroute nodes)
				if (OwningNode->GetGraph() == Result)
				{
					continue;
				}

				UEdGraphNode** NewNodePtr = DuplicationMapping.Find(OwningNode);
				bool bAddReciprocalLink = false;

				if (NewNodePtr == nullptr)
				{
					// check to see if we're dealing with an input pin that is coming from
					// a reroute node that was trimmed above
					if (Pin->Direction == EGPD_Input && RerouteNodesToFixup.Contains(OwningNode))
					{
						check(LinkedPin->Direction == EGPD_Output);
						LinkedPin = CastChecked<UNiagaraNodeReroute>(OwningNode)->GetTracedOutputPin(LinkedPin, false);
						// reroute nodes can return nullptr for the traced output if they are not connected to anything
						if (LinkedPin != nullptr)
						{
							OwningNode = LinkedPin->GetOwningNode();
							NewNodePtr = DuplicationMapping.Find(OwningNode);
							bAddReciprocalLink = true;
						}
					}
				}

				if (NewNodePtr == nullptr)
				{
					// If output pins were connected to another node which wasn't encountered in traversal then it's possible it won't be found here.
					// In that case set the linked pin to nullptr to be removed later.
					Pin->LinkedTo[i] = nullptr;
				}
				else
				{
					UEdGraphPin** NodePin = (*NewNodePtr)->Pins.FindByPredicate([LinkedPin](UEdGraphPin* Pin) { return Pin->PinId == LinkedPin->PinId; });
					check(NodePin);
					Pin->LinkedTo[i] = *NodePin;

					if (bAddReciprocalLink)
					{
						(*NodePin)->LinkedTo.AddUnique(Pin);
					}
				}
			}
			// Remove any null linked pins.
			Pin->LinkedTo.RemoveAll([](const UEdGraphPin* Pin) { return Pin == nullptr; });
		}
	}
	Result->Nodes = NewNodes;

	// probably not necessary for compilation, but remove references to the original graph
	Result->ParameterToReferencesMap.Empty();

	// by setting this flag to false we make sure that if something attempts to refresh the reference map that we'll
	// hit the ensure at the top of RefreshParameterReferences()
	Result->bParameterReferenceRefreshPending = true;

	return Result;
}

void UNiagaraGraph::ReleaseCompilationCopy()
{
	if (!bIsForCompilationOnly)
	{
		return;
	}
	MarkAsGarbage();
}

UNiagaraNodeOutput* UNiagaraGraph::FindOutputNode(ENiagaraScriptUsage TargetUsageType, FGuid TargetUsageId) const
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Graph_FindOutputNode);
	for (UEdGraphNode* Node : Nodes)
	{
		if (UNiagaraNodeOutput* OutNode = Cast<UNiagaraNodeOutput>(Node))
		{
			if (OutNode->GetUsage() == TargetUsageType && OutNode->GetUsageId() == TargetUsageId)
			{
				return OutNode;
			}
		}
	}
	return nullptr;
}

UNiagaraNodeOutput* UNiagaraGraph::FindEquivalentOutputNode(ENiagaraScriptUsage TargetUsageType, FGuid TargetUsageId) const
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Graph_FindOutputNode);
	for (UEdGraphNode* Node : Nodes)
	{
		if (UNiagaraNodeOutput* OutNode = Cast<UNiagaraNodeOutput>(Node))
		{
			if (UNiagaraScript::IsEquivalentUsage(OutNode->GetUsage(), TargetUsageType) && OutNode->GetUsageId() == TargetUsageId)
			{
				return OutNode;
			}
		}
	}
	return nullptr;
}


void BuildTraversalHelper(TArray<class UNiagaraNode*>& OutNodesTraversed, UNiagaraNode* CurrentNode, bool bEvaluateStaticSwitches)
{
	auto VisitPin = [](const UEdGraphPin* Pin, TArray<class UNiagaraNode*>& OutNodesTraversed, bool bEvaluateStaticSwitches) {
		if (Pin->Direction == EEdGraphPinDirection::EGPD_Input && Pin->LinkedTo.Num() > 0)
		{
			for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
			{
				bool bFilterForCompilation = true;
				TArray<const UNiagaraNode*> NodesVisitedDuringTrace;

				UEdGraphPin* TracedPin = bEvaluateStaticSwitches ? UNiagaraNode::TraceOutputPin(LinkedPin, bFilterForCompilation,  &NodesVisitedDuringTrace) : LinkedPin;
				if (TracedPin != nullptr)
				{
					// Deal with static switches driven by a pin...
					for (const UNiagaraNode* VisitedNode : NodesVisitedDuringTrace)
					{
						if (OutNodesTraversed.Contains(VisitedNode))
						{
							continue;
						}
						BuildTraversalHelper(OutNodesTraversed, (UNiagaraNode * )VisitedNode, bEvaluateStaticSwitches);
					}

					UNiagaraNode* Node = Cast<UNiagaraNode>(TracedPin->GetOwningNode());
					if (OutNodesTraversed.Contains(Node))
					{
						continue;
					}
					BuildTraversalHelper(OutNodesTraversed, Node, bEvaluateStaticSwitches);
				}
			}
		}
	};


	if (CurrentNode == nullptr)
	{
		return;
	}

	if (bEvaluateStaticSwitches)
	{
		UNiagaraNodeStaticSwitch* StaticSwitch = Cast< UNiagaraNodeStaticSwitch>(CurrentNode);
		if (StaticSwitch && StaticSwitch->IsSetByPin())
		{
			// The selector pin is never traversed directly below by the TracedPin route, so we explicitly visit it here.
			UEdGraphPin* Pin = StaticSwitch->GetSelectorPin();
			if (Pin)
				VisitPin(Pin, OutNodesTraversed, bEvaluateStaticSwitches);
		}
		else if (StaticSwitch)
		{
			return;
		}
	}

	for (UEdGraphPin* Pin : CurrentNode->GetAllPins())
	{
		VisitPin(Pin, OutNodesTraversed, bEvaluateStaticSwitches);
	}

	OutNodesTraversed.Add(CurrentNode);
}

void UNiagaraGraph::BuildTraversal(TArray<class UNiagaraNode*>& OutNodesTraversed, ENiagaraScriptUsage TargetUsage, FGuid TargetUsageId, bool bEvaluateStaticSwitches) const
{
	UNiagaraNodeOutput* Output = FindOutputNode(TargetUsage, TargetUsageId);
	if (Output)
	{
		NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Graph_BuildTraversal);

		BuildTraversalHelper(OutNodesTraversed, Output, bEvaluateStaticSwitches);
	}
}

void UNiagaraGraph::BuildTraversal(TArray<class UNiagaraNode*>& OutNodesTraversed, UNiagaraNode* FinalNode, bool bEvaluateStaticSwitches)
{
	if (FinalNode)
	{
		NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Graph_BuildTraversal);

		BuildTraversalHelper(OutNodesTraversed, FinalNode, bEvaluateStaticSwitches);
	}
}


void UNiagaraGraph::FindInputNodes(TArray<UNiagaraNodeInput*>& OutInputNodes, UNiagaraGraph::FFindInputNodeOptions Options) const
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Graph_FindInputNodes);
	TArray<UNiagaraNodeInput*> InputNodes;

	if (!Options.bFilterByScriptUsage)
	{
		NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Graph_FindInputNodes_NotFilterUsage);

		for (UEdGraphNode* Node : Nodes)
		{
			UNiagaraNodeInput* NiagaraInputNode = Cast<UNiagaraNodeInput>(Node);
			if (NiagaraInputNode != nullptr &&
				((NiagaraInputNode->Usage == ENiagaraInputNodeUsage::Parameter && Options.bIncludeParameters) ||
				(NiagaraInputNode->Usage == ENiagaraInputNodeUsage::Attribute && Options.bIncludeAttributes) ||
				(NiagaraInputNode->Usage == ENiagaraInputNodeUsage::SystemConstant && Options.bIncludeSystemConstants) || 
				(NiagaraInputNode->Usage == ENiagaraInputNodeUsage::TranslatorConstant && Options.bIncludeTranslatorConstants)))
			{
				InputNodes.Add(NiagaraInputNode);
			}
		}
	}
	else
	{
		NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Graph_FindInputNodes_FilterUsage);

		TArray<class UNiagaraNode*> Traversal;
		BuildTraversal(Traversal, Options.TargetScriptUsage, Options.TargetScriptUsageId);
		for (UNiagaraNode* Node : Traversal)
		{
			UNiagaraNodeInput* NiagaraInputNode = Cast<UNiagaraNodeInput>(Node);
			if (NiagaraInputNode != nullptr &&
				((NiagaraInputNode->Usage == ENiagaraInputNodeUsage::Parameter && Options.bIncludeParameters) ||
					(NiagaraInputNode->Usage == ENiagaraInputNodeUsage::Attribute && Options.bIncludeAttributes) ||
					(NiagaraInputNode->Usage == ENiagaraInputNodeUsage::SystemConstant && Options.bIncludeSystemConstants)))
			{
				InputNodes.Add(NiagaraInputNode);
			}
		}
	}

	if (Options.bFilterDuplicates)
	{
		NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Graph_FindInputNodes_FilterDupes);

		for (UNiagaraNodeInput* InputNode : InputNodes)
		{
			auto NodeMatches = [=](UNiagaraNodeInput* UniqueInputNode)
			{
				if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter)
				{
					return UniqueInputNode->Input.IsEquivalent(InputNode->Input, false);
				}
				else
				{
					return UniqueInputNode->Input.IsEquivalent(InputNode->Input);
				}
			};

			if (OutInputNodes.ContainsByPredicate(NodeMatches) == false)
			{
				OutInputNodes.Add(InputNode);
			}
		}
	}
	else
	{
		OutInputNodes.Append(InputNodes);
	}

	if (Options.bSort)
	{
		NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Graph_FindInputNodes_Sort);

		UNiagaraNodeInput::SortNodes(OutInputNodes);
	}
}

TArray<FNiagaraVariable> UNiagaraGraph::FindStaticSwitchInputs(bool bReachableOnly, const TArray<FNiagaraVariable>& InStaticVars) const
{
	TArray<UEdGraphNode*> NodesToProcess = bReachableOnly ? FindReachableNodes(InStaticVars) : Nodes;

	TArray<FNiagaraVariable> Result;
	for (UEdGraphNode* Node : NodesToProcess)
	{
		UNiagaraNodeStaticSwitch* SwitchNode = Cast<UNiagaraNodeStaticSwitch>(Node);
		if (SwitchNode && !SwitchNode->IsSetByCompiler() && !SwitchNode->IsSetByPin())
		{
			FNiagaraVariable Variable(SwitchNode->GetInputType(), SwitchNode->InputParameterName);
			Result.AddUnique(Variable);
		}

		if (UNiagaraNodeFunctionCall* FunctionNode = Cast<UNiagaraNodeFunctionCall>(Node))
		{
			for (const FNiagaraPropagatedVariable& Propagated : FunctionNode->PropagatedStaticSwitchParameters)
			{
				Result.AddUnique(Propagated.ToVariable());
			}			
		}
	}
	Algo::SortBy(Result, &FNiagaraVariable::GetName, FNameLexicalLess());
	return Result;
}

TArray<UEdGraphNode*> UNiagaraGraph::FindReachableNodes(const TArray<FNiagaraVariable>& InStaticVars) const
{
	
	TArray<UEdGraphNode*> ResultNodes;
	FNiagaraParameterMapHistoryBuilder Builder;
	Builder.RegisterExternalStaticVariables(InStaticVars);
	Builder.SetIgnoreDisabled(false);

	TArray<UNiagaraNodeOutput*> OutNodes;
	FindOutputNodes(OutNodes);
	ResultNodes.Append(OutNodes);
	for (UNiagaraNodeOutput* OutNode : OutNodes)
	{
		Builder.BuildParameterMaps(OutNode, true);
		{
			TArray<const UNiagaraNode*> VisitedNodes;
			Builder.GetContextuallyVisitedNodes(VisitedNodes);
			for (const UNiagaraNode* Node : VisitedNodes)
			{
				if (Node->GetOuter() == this)
					ResultNodes.AddUnique((UNiagaraNode*)Node);
			}
		}
	}

	

	return ResultNodes;
}

void UNiagaraGraph::GetParameters(TArray<FNiagaraVariable>& Inputs, TArray<FNiagaraVariable>& Outputs)const
{
	Inputs.Empty();
	Outputs.Empty();

	TArray<UNiagaraNodeInput*> InputsNodes;
	FFindInputNodeOptions Options;
	Options.bSort = true;
	FindInputNodes(InputsNodes, Options);
	for (UNiagaraNodeInput* Input : InputsNodes)
	{
		Inputs.Add(Input->Input);
	}

	TArray<UNiagaraNodeOutput*> OutputNodes;
	FindOutputNodes(OutputNodes);
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		for (FNiagaraVariable& Var : OutputNode->Outputs)
		{
			Outputs.AddUnique(Var);
		}
	}

	//Do we need to sort outputs?
	//Should leave them as they're defined in the output node?
// 	auto SortVars = [](const FNiagaraVariable& A, const FNiagaraVariable& B)
// 	{
// 		//Case insensitive lexicographical comparisons of names for sorting.
// 		return A.GetName().ToString() < B.GetName().ToString();
// 	};
// 	Outputs.Sort(SortVars);
}

void UNiagaraGraph::GetAllInputScriptVariables(TArray<UNiagaraScriptVariable*>& OutScriptVariables) const
{
	TArray<UNiagaraScriptVariable*> AllInputScriptVariables;

	for(const auto& VariablePair : GetAllMetaData())
	{
		if(VariablePair.Key.IsInNameSpace(FNiagaraConstants::ModuleNamespace) || VariablePair.Value->GetIsStaticSwitch())
		{
			AllInputScriptVariables.Add(VariablePair.Value);
		}
	}

	OutScriptVariables.Append(AllInputScriptVariables);
}

const UNiagaraGraph::FScriptVariableMap& UNiagaraGraph::GetAllMetaData() const
{
	check(!bIsForCompilationOnly);

	return VariableToScriptVariable;
}

UNiagaraGraph::FScriptVariableMap& UNiagaraGraph::GetAllMetaData()
{
	check(!bIsForCompilationOnly);

	return VariableToScriptVariable;
}

void UNiagaraGraph::ConditionalRefreshParameterReferences()
{
	if (bParameterReferenceRefreshPending)
	{
		RefreshParameterReferences();
	}
}

const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& UNiagaraGraph::GetParameterReferenceMap() const
{
	if (bParameterReferenceRefreshPending)
	{
		RefreshParameterReferences();
	}
	return ParameterToReferencesMap;
}

UNiagaraScriptVariable* UNiagaraGraph::GetScriptVariable(FNiagaraVariable Parameter) const
{
	check(!bIsForCompilationOnly);

	if (const TObjectPtr<UNiagaraScriptVariable>* FoundScriptVariable = VariableToScriptVariable.Find(Parameter))
	{
		return *FoundScriptVariable;
	}
	return nullptr;
}

UNiagaraScriptVariable* UNiagaraGraph::GetScriptVariable(FName ParameterName) const
{
	check(!bIsForCompilationOnly);

	for (auto& VariableToScriptVariableItem : VariableToScriptVariable)
	{
		if (VariableToScriptVariableItem.Key.GetName() == ParameterName)
		{
			return VariableToScriptVariableItem.Value;
		}
	}
	return nullptr;
}

UNiagaraScriptVariable* UNiagaraGraph::GetScriptVariable(FGuid VariableGuid) const
{
	check(!bIsForCompilationOnly);

	for (auto& VariableToScriptVariableItem : VariableToScriptVariable)
	{
		if(VariableToScriptVariableItem.Value->Metadata.GetVariableGuid() == VariableGuid)
		{
			return VariableToScriptVariableItem.Value;
		}
	}
	
	return nullptr;
}

TArray<UNiagaraScriptVariable*> UNiagaraGraph::GetChildScriptVariablesForInput_Deprecated(FGuid VariableGuid) const
{
	check(!bIsForCompilationOnly);

	TArray<UNiagaraScriptVariable*> ChildrenVariables;
	
	TOptional<FNiagaraVariable> FoundVariable;
	for (auto& VariableToScriptVariableItem : VariableToScriptVariable)
	{
		if(VariableToScriptVariableItem.Value->Metadata.GetVariableGuid() == VariableGuid)
		{
			FoundVariable = VariableToScriptVariableItem.Key;
		}
	}

	if(FoundVariable.IsSet())
	{
		for (auto& VariableToScriptVariableItem : VariableToScriptVariable)
		{
			if(VariableToScriptVariableItem.Value->Metadata.GetParentAttribute_DEPRECATED() == FoundVariable->GetName())
			{
				ChildrenVariables.Add(VariableToScriptVariableItem.Value);
			}
		}
	}

	ChildrenVariables.Sort([&](const UNiagaraScriptVariable& VariableA, const UNiagaraScriptVariable& VariableB)
	{
		return VariableA.Metadata.GetEditorSortPriority_DEPRECATED() < VariableB.Metadata.GetEditorSortPriority_DEPRECATED();
	});
	
	return ChildrenVariables;
}

TArray<FGuid> UNiagaraGraph::GetChildScriptVariableGuidsForInput(FGuid VariableGuid) const
{
	check(!bIsForCompilationOnly);

	TArray<FGuid> ChildrenVariableGuids;
	
	TOptional<FNiagaraVariable> FoundVariable;
	for (auto& VariableToScriptVariableItem : VariableToScriptVariable)
	{
		if(VariableToScriptVariableItem.Value->Metadata.GetVariableGuid() == VariableGuid)
		{
			FoundVariable = VariableToScriptVariableItem.Key;
		}
	}

	if(FoundVariable.IsSet())
	{	
		TArray<FNiagaraVariable> Variables;
		GetAllVariables(Variables);
	
		for(FNiagaraVariable& Variable : Variables)
		{
			TOptional<FNiagaraVariableMetaData> MetaData = GetMetaData(Variable);

			if(MetaData.IsSet())
			{
				FHierarchyElementIdentity Identity({MetaData.GetValue().GetVariableGuid()}, {});
				if(UHierarchyElement* FoundHierarchyParameter = ParameterHierarchyRoot->FindChildWithIdentity(Identity, true))
				{
					TArray<UNiagaraHierarchyScriptParameter*> ChildrenScriptParameters;
					FoundHierarchyParameter->GetChildrenOfType(ChildrenScriptParameters, false);

					for(UNiagaraHierarchyScriptParameter* ChildrenScriptParameter : ChildrenScriptParameters)
					{
						ChildrenVariableGuids.Add(ChildrenScriptParameter->GetScriptVariable()->Metadata.GetVariableGuid());
					}
				}
			}
			else
			{
				UE_LOGF(LogNiagaraEditor, Log, "Couldn't find metadata for variable");
			}
		}
	}
	
	return ChildrenVariableGuids;
}

UNiagaraScriptVariable* UNiagaraGraph::AddParameter(const FNiagaraVariable& Parameter, bool bIsStaticSwitch /*= false*/)
{
	check(!bIsForCompilationOnly);

	// Delay the NotifyGraphChanged() call until the static switch flag is set on the UNiagaraScriptVariable so that ParameterPanel displays correctly.
	const bool bNotifyChanged = false;
	UNiagaraScriptVariable* NewScriptVar = AddParameter(Parameter, FNiagaraVariableMetaData(), bIsStaticSwitch, bNotifyChanged);
	NotifyGraphChanged();
	return NewScriptVar;
}

UNiagaraScriptVariable* UNiagaraGraph::AddParameter(const FNiagaraVariable& Parameter, const FNiagaraVariableMetaData& ParameterMetaData, bool bIsStaticSwitch, bool bNotifyChanged)
{
	check(!bIsForCompilationOnly);

	FNiagaraGraphParameterReferenceCollection* FoundParameterReferenceCollection = ParameterToReferencesMap.Find(Parameter);
	if (!FoundParameterReferenceCollection)
	{
		const bool bCreatedByUser = !bIsStaticSwitch;
		ParameterToReferencesMap.Emplace(Parameter, {bCreatedByUser});
	}

	TObjectPtr<UNiagaraScriptVariable>* FoundScriptVariable = VariableToScriptVariable.Find(Parameter);
	if (!FoundScriptVariable)
	{
		Modify();

		UNiagaraScriptVariable* NewScriptVariable = CreateScriptVariableInternal(Parameter, ParameterMetaData, bIsStaticSwitch);

		if (bNotifyChanged)
		{
			NotifyGraphChanged();
		}
		return NewScriptVariable;
	}

	return *FoundScriptVariable;
}

UNiagaraScriptVariable* UNiagaraGraph::AddParameter(const UNiagaraScriptVariable* InScriptVar)
{
	check(!bIsForCompilationOnly);

	TObjectPtr<UNiagaraScriptVariable>* FoundScriptVariable = VariableToScriptVariable.Find(InScriptVar->Variable);
	if (!FoundScriptVariable)
	{
		Modify();
		UNiagaraScriptVariable* NewScriptVariable = CastChecked<UNiagaraScriptVariable>(StaticDuplicateObject(InScriptVar, this, FName()));
		NewScriptVariable->SetFlags(RF_Transactional);
		// If the incoming script variable is linked to a parameter definition, do not make a new ID.
		// The parameter ID is associated with the linked definition.
		// Vice-Versa if the new parameter is not linked to a parameter definition, create a new ID so that it is distinct for this graph.
		if (NewScriptVariable->GetIsSubscribedToParameterDefinitions() == false)
		{
			NewScriptVariable->Metadata.CreateNewGuid();
		}
 		ParameterToReferencesMap.Emplace(NewScriptVariable->Variable, {true /*bInCreated*/});
		VariableToScriptVariable.Add(NewScriptVariable->Variable, NewScriptVariable);
		TInstancedStruct<FNiagaraParametersChangedData> ChangeData = TInstancedStruct<FNiagaraParameterCreatedData>::Make(NewScriptVariable);
		OnParametersChangedDelegate.Broadcast(ChangeData);
		NotifyGraphChanged();
		return NewScriptVariable;
	}
	ensureMsgf(false, TEXT("Tried to add parameter that already existed! Parameter: %s"), *InScriptVar->Variable.GetName().ToString());
	return *FoundScriptVariable;
}

UNiagaraScriptVariable* UNiagaraGraph::CreateScriptVariableInternal(const FNiagaraVariable& Parameter, const FNiagaraVariableMetaData& ParameterMetaData, bool bIsStaticSwitch)
{
	UNiagaraScriptVariable* NewScriptVariable = NewObject<UNiagaraScriptVariable>(this, FName(), RF_Transactional);
	NewScriptVariable->Init(Parameter, ParameterMetaData);
	NewScriptVariable->SetIsStaticSwitch(bIsStaticSwitch);

	FNiagaraEditorUtilities::ResetVariableToDefaultValue(NewScriptVariable->Variable);

	// Inputs in graphs (module namespace) are intrinsically written to, so set the default mode to value instead of fail if not set.
	if (NewScriptVariable->Variable.IsInNameSpace(FNiagaraConstants::ModuleNamespace))
	{
		NewScriptVariable->DefaultMode = ENiagaraDefaultMode::Value;
	}
	else
	{
		NewScriptVariable->DefaultMode = ENiagaraDefaultMode::FailIfPreviouslyNotSet;
	}

	VariableToScriptVariable.Add(Parameter, NewScriptVariable);
	TInstancedStruct<FNiagaraParametersChangedData> ChangeData = TInstancedStruct<FNiagaraParameterCreatedData>::Make(NewScriptVariable);
	OnParametersChangedDelegate.Broadcast(ChangeData);

	return NewScriptVariable;
}

FName UNiagaraGraph::MakeUniqueParameterName(const FName& InName)
{
	TArray<TWeakObjectPtr<UNiagaraGraph>> Graphs;
	Graphs.Emplace(this);
	return MakeUniqueParameterNameAcrossGraphs(InName, Graphs);
}

 FName UNiagaraGraph::MakeUniqueParameterNameAcrossGraphs(const FName& InName, TArray<TWeakObjectPtr<UNiagaraGraph>>& InGraphs)
{
	 TSet<FName> Names;
	 for (TWeakObjectPtr<UNiagaraGraph> Graph : InGraphs)
	 {
		 if (Graph.IsValid())
		 {
			 for (const auto& ParameterElement : Graph->GetParameterReferenceMap())
			 {
				 Names.Add(ParameterElement.Key.GetName());
			 }
		 }
	 }

	 return FNiagaraUtilities::GetUniqueName(InName, Names);
}

void UNiagaraGraph::AddParameterReference(const FNiagaraVariable& Parameter, FNiagaraGraphParameterReference& NewParameterReference)
{
	FNiagaraGraphParameterReferenceCollection* FoundParameterReferenceCollection = ParameterToReferencesMap.Find(Parameter);
	if (ensureMsgf(FoundParameterReferenceCollection != nullptr, TEXT("Failed to find parameter reference collection when adding graph parameter reference!")))
	{
		FoundParameterReferenceCollection->ParameterReferences.Add(NewParameterReference);
	}
}

void UNiagaraGraph::RemoveParameter(const FNiagaraVariable& Parameter, bool bAllowDeleteStaticSwitch /*= false*/)
{
	check(!bIsForCompilationOnly);

	// make sure that we have collected parameter references already
	ConditionalRefreshParameterReferences();

	FNiagaraGraphParameterReferenceCollection* ReferenceCollection = ParameterToReferencesMap.Find(Parameter);
	if (ReferenceCollection)
	{
		for (int32 Index = 0; Index < ReferenceCollection->ParameterReferences.Num(); Index++)
		{
			const FNiagaraGraphParameterReference& Reference = ReferenceCollection->ParameterReferences[Index];
			UNiagaraNode* Node = Cast<UNiagaraNode>(Reference.Value.Get());
			if (Node && Node->GetGraph() == this)
			{
				if (Node->IsA(UNiagaraNodeStaticSwitch::StaticClass()) && bAllowDeleteStaticSwitch == false)
				{
					// Static switch parameters are automatically populated from the graph nodes and cannot be manually deleted
					NotifyGraphChanged();
					return;
				}
				UEdGraphPin* Pin = Node->GetPinByPersistentGuid(Reference.Key);
				if (Pin)
				{
					Node->RemovePin(Pin);
				}
			}
		}

		// Remove it from the reference collection directly because it might have been user added and
		// these aren't removed when the cached data is rebuilt.
		ParameterToReferencesMap.Remove(Parameter);
		NotifyGraphChanged();
	}

	TArray<FNiagaraVariable> VarsToRemove;
	for (auto It : VariableToScriptVariable)
	{
		if (It.Key == Parameter)
		{
			VarsToRemove.Add(It.Key);
		}
	}
	for (FNiagaraVariable& Var : VarsToRemove)
	{
		VariableToScriptVariable.Remove(Var);
		OnParametersChangedDelegate.Broadcast(TOptional<TInstancedStruct<FNiagaraParametersChangedData>>());
	}
}

void CopyScriptVariableDataForRename(const UNiagaraScriptVariable& OldScriptVariable, UNiagaraScriptVariable& NewScriptVariable)
{
	NewScriptVariable.Variable = OldScriptVariable.Variable;
	NewScriptVariable.DefaultMode = OldScriptVariable.DefaultMode;
	NewScriptVariable.DefaultBinding = OldScriptVariable.DefaultBinding;
	if(OldScriptVariable.GetDefaultValueData() != nullptr)
	{
		NewScriptVariable.SetDefaultValueData(OldScriptVariable.GetDefaultValueData());
	}
	NewScriptVariable.Metadata = OldScriptVariable.Metadata;
}

bool UNiagaraGraph::RenameParameterFromPin(const FNiagaraVariable& Parameter, FName NewName, UEdGraphPin* InPin)
{
	check(!bIsForCompilationOnly);

	auto TrySubscribeScriptVarToDefinitionByName = [this](UNiagaraScriptVariable* TargetScriptVar) {
		UNiagaraScript* OuterScript = GetTypedOuter<UNiagaraScript>();
		bool bForDataProcessingOnly = true;
		TSharedPtr<FNiagaraScriptViewModel> ScriptViewModel = MakeShared<FNiagaraScriptViewModel>(FText(), ENiagaraParameterEditMode::EditAll, bForDataProcessingOnly);
		ScriptViewModel->SetScript(OuterScript);
		FNiagaraParameterDefinitionsUtilities::TrySubscribeScriptVarToDefinitionByName(TargetScriptVar, ScriptViewModel.Get());
	};

	FNiagaraVariable NewVariable(Parameter.GetType(), NewName);
	UNiagaraScriptVariable* ExistingVariable = GetScriptVariable(NewVariable);
	if (Parameter.GetName() == NewName || (ExistingVariable != nullptr && ExistingVariable->Variable.GetType() == NewVariable.GetType()))
	{
		return true;
	}

	Modify();

	// make sure that we have collected parameter references already
	ConditionalRefreshParameterReferences();

	if (FNiagaraGraphParameterReferenceCollection* ReferenceCollection = ParameterToReferencesMap.Find(Parameter))
	{
		FNiagaraGraphParameterReferenceCollection NewReferences = *ReferenceCollection;
		if (NewReferences.ParameterReferences.Num() == 1 && NewReferences.ParameterReferences[0].Key == InPin->PersistentGuid)
		{
			bool bRenameRequestedFromStaticSwitch = false;
			bool bMerged = false;

			if (RenameParameter(Parameter, NewName, bRenameRequestedFromStaticSwitch, &bMerged))
			{
				if (TObjectPtr<UNiagaraScriptVariable> const* RenamedScriptVarPtr = VariableToScriptVariable.Find(FNiagaraVariable(Parameter.GetType(), NewName)))
				{
					TrySubscribeScriptVarToDefinitionByName(*RenamedScriptVarPtr);
				}

				// Rename all the bindings that point to the old parameter 
				for (auto It : VariableToScriptVariable)
				{
					UNiagaraScriptVariable* Variable = It.Value;
					if (Variable && Variable->DefaultBinding.GetName() == Parameter.GetName())
					{
						Variable->DefaultBinding.SetName(NewName);
					}
				}
				
				if (!bMerged)
				{
					FNiagaraEditorUtilities::InfoWithToastAndLog(FText::Format(
						LOCTEXT("RenamedVarInGraphForAll", "\"{0}\" has been fully renamed as it was only used on this node."), 
						FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(NewName)));
				}
				return true;
			}
			return false;
		}
	}

	FNiagaraVariable NewParameter = Parameter;
	NewParameter.SetName(NewName);

	TObjectPtr<UNiagaraScriptVariable>* FoundOldScriptVariablePtr = VariableToScriptVariable.Find(Parameter);
	TObjectPtr<UNiagaraScriptVariable>* FoundNewScriptVariablePtr = VariableToScriptVariable.Find(NewParameter);

	bool bMerged = false;
	const bool bOldScriptVariableIsStaticSwitch = FoundOldScriptVariablePtr ? (*FoundOldScriptVariablePtr)->GetIsStaticSwitch() : false;
	const FNiagaraVariableMetaData OldMetaData = FoundOldScriptVariablePtr ? (*FoundOldScriptVariablePtr)->Metadata : FNiagaraVariableMetaData();

	if (bIsRenamingParameter)
	{
		return false;
	}

	//Set metadata on the new parameter and put the new parameter into VariableToScriptVariable
	if (FoundOldScriptVariablePtr && !FoundNewScriptVariablePtr)
	{
		// Only create a new variable if needed.
		UNiagaraScriptVariable* FoundOldScriptVariable = *FoundOldScriptVariablePtr;
		UNiagaraScriptVariable* NewScriptVariable = CastChecked<UNiagaraScriptVariable>(StaticDuplicateObject(FoundOldScriptVariable, this, FName()));
		NewScriptVariable->SetFlags(RF_Transactional);
		CopyScriptVariableDataForRename(*FoundOldScriptVariable, *NewScriptVariable);
		NewScriptVariable->Variable.SetName(NewName);
		NewScriptVariable->Metadata.CreateNewGuid();
		NewScriptVariable->SetIsSubscribedToParameterDefinitions(false);
		VariableToScriptVariable.Add(NewParameter, NewScriptVariable);
		TrySubscribeScriptVarToDefinitionByName(NewScriptVariable);

		const FNiagaraGraphParameterReferenceCollection* ReferenceCollection = GetParameterReferenceMap().Find(Parameter);
		if (ReferenceCollection && ReferenceCollection->ParameterReferences.Num() < 1)
		{
			VariableToScriptVariable.Remove(Parameter);
		}
	}

	if (FoundNewScriptVariablePtr)
	{
		bMerged = true;
		FNiagaraEditorUtilities::InfoWithToastAndLog(FText::Format(
			LOCTEXT("MergedVar", "\"{0}\" has been merged with parameter \"{1}\".\nAll of \"{1}\"'s meta-data will be used, overwriting \"{0}\"'s meta-data."),
			FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(Parameter.GetName()),
			FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(NewName)));
	}
	RefreshParameterReferences();

	if (!bMerged)
	{
		FNiagaraEditorUtilities::InfoWithToastAndLog(FText::Format(
			LOCTEXT("RenamedVarInGraphForNode", "\"{0}\" has been duplicated as \"{1}\" as it is used in multiple locations.\nPlease edit the Parameters Panel version to change in all locations."),
			FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(Parameter.GetName()), 
			FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(NewName)));
	}

	return false;
}

void UNiagaraGraph::FixupReferenceCollectionsPostRename(const FNiagaraVariable& OrigVariable, const FNiagaraVariable& DestVariable, bool bParameterMerged, bool bSuppressEvents)
{
	// Fixup reference collection and pin names
	FNiagaraGraphParameterReferenceCollection NewReferences;
	if (ParameterToReferencesMap.RemoveAndCopyValue(OrigVariable, NewReferences))
	{
		if (!NewReferences.ParameterReferences.IsEmpty())
		{
			const FText NewNameText = FText::FromName(DestVariable.GetName());
			for (FNiagaraGraphParameterReference& Reference : NewReferences.ParameterReferences)
			{
				UNiagaraNode* Node = Cast<UNiagaraNode>(Reference.Value.Get());
				if (Node && Node->GetGraph() == this)
				{
					Node->Modify();
					if (UEdGraphPin* Pin = Node->GetPinByPersistentGuid(Reference.Key))
					{
						Pin->Modify();
						Node->CommitEditablePinName(NewNameText, Pin, bSuppressEvents);
					}
				}
			}
		}

		if (!bParameterMerged)
		{
			ParameterToReferencesMap.Add(DestVariable, NewReferences);
		}
		else
		{
			RefreshParameterReferences();
		}
	}
}

bool UNiagaraGraph::RenameParameter(const FNiagaraVariable& Parameter, FName NewName, bool bRenameRequestedFromStaticSwitch, bool* bOutMerged, bool bSuppressEvents)
{
	check(!bIsForCompilationOnly);

	// Initialize the merger state if requested
	if (bOutMerged)
		*bOutMerged = false;

	if (Parameter.GetName() == NewName)
		return true;

	// Block rename when already renaming. This prevents recursion when CommitEditablePinName is called on referenced nodes. 
	if (bIsRenamingParameter)
	{
		return false;
	}
	bIsRenamingParameter = true;

	bool bParameterMerged = false;

	Modify();
	// Create the new parameter
	FNiagaraVariable NewParameter = Parameter;
	NewParameter.SetName(NewName);

	TObjectPtr<UNiagaraScriptVariable>* OldScriptVariablePtr = VariableToScriptVariable.Find(Parameter);
	TObjectPtr<UNiagaraScriptVariable> OldScriptVariable = OldScriptVariablePtr ? *OldScriptVariablePtr : nullptr;
	FNiagaraVariableMetaData OldMetaData;
	OldMetaData.CreateNewGuid();
	if (OldScriptVariable != nullptr)
	{
		if (!bRenameRequestedFromStaticSwitch && OldScriptVariable->GetIsStaticSwitch())
		{
			// We current disallow renaming static switch variables in the Parameters panel. 
			bIsRenamingParameter = false;
			return false;
		}
		OldMetaData = OldScriptVariable->Metadata;
	}
		
	TObjectPtr<UNiagaraScriptVariable>* NewScriptVariablePtr = VariableToScriptVariable.Find(NewParameter);

	// Swap metadata to the new parameter; put the new parameter into VariableToScriptVariable
	if (OldScriptVariable != nullptr)
	{
		Modify();
		// Rename all the bindings that point to the old parameter 
		for (auto It : VariableToScriptVariable)
		{
			UNiagaraScriptVariable* Variable = It.Value;
			if (Variable && Variable->DefaultBinding.GetName() == Parameter.GetName())
			{
				Variable->DefaultBinding.SetName(NewParameter.GetName());
			}
		}

		// Only create a new variable if needed.
		if (NewScriptVariablePtr == nullptr)
		{
			// Replace the script variable data
			UNiagaraScriptVariable* NewScriptVariable = CastChecked<UNiagaraScriptVariable>(StaticDuplicateObject(OldScriptVariable, this, FName()));
			NewScriptVariable->SetFlags(RF_Transactional);
			CopyScriptVariableDataForRename(*OldScriptVariable, *NewScriptVariable);
			NewScriptVariable->Metadata.CreateNewGuid();
			NewScriptVariable->Variable.SetName(NewName);
			VariableToScriptVariable.Add(NewParameter, NewScriptVariable);
		}

		if (!bRenameRequestedFromStaticSwitch)
		{
			// Static switches take care to remove the last existing parameter themselves, we don't want to remove the parameter if there are still switches with the name around 
			VariableToScriptVariable.Remove(Parameter);
		}
	}

	// Either set the new meta-data or use the existing meta-data.
	if (NewScriptVariablePtr == nullptr)
	{
		SetMetaData(NewParameter, OldMetaData);
	}
	else
	{
		bParameterMerged = true;
		FName NewParamName = NewName;
		FNiagaraEditorUtilities::DecomposeVariableNamespace(NewName, NewParamName);
		FName OldParamName = Parameter.GetName();
		FNiagaraEditorUtilities::DecomposeVariableNamespace(Parameter.GetName(), OldParamName);
		FNiagaraEditorUtilities::InfoWithToastAndLog(FText::Format(
			LOCTEXT("MergedVar", "\"{0}\" has been merged with parameter \"{1}\".\nAll of \"{1}\"'s meta-data will be used, overwriting \"{0}\"'s meta-data."),
			FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(Parameter.GetName()),
			FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(NewName)));
	}

	// make sure that we have collected parameter references already
	ConditionalRefreshParameterReferences();

	FixupReferenceCollectionsPostRename(Parameter, NewParameter, bParameterMerged, bSuppressEvents);

	bIsRenamingParameter = false;

	NotifyGraphChanged();

	if (bOutMerged)
	{
		*bOutMerged = bParameterMerged;
	}

	TInstancedStruct<FNiagaraParametersChangedData> RenameData = TInstancedStruct<FNiagaraParameterRenamedData>::Make();
	FNiagaraParameterRenamedData& Data = RenameData.GetMutable<FNiagaraParameterRenamedData>();
	Data.OldScriptVariable = OldScriptVariable;
	Data.NewScriptVariable = VariableToScriptVariable[NewParameter];
	OnParametersChangedDelegate.Broadcast(RenameData);

	return true;
}

bool UNiagaraGraph::RenameStaticSwitch(UNiagaraNodeStaticSwitch* SwitchNode, FName NewParameterName)
{
	if (!SwitchNode || SwitchNode->InputParameterName == NewParameterName)
	{
		return false;
	}

	Modify();
	bIsRenamingParameter = true;
	ON_SCOPE_EXIT{ bIsRenamingParameter = false; };

	const FName OrigParameterName = SwitchNode->InputParameterName;

	const FNiagaraVariableBase OrigVariable(SwitchNode->GetInputType(), OrigParameterName);
	const FNiagaraVariableBase DestVariable(SwitchNode->GetInputType(), NewParameterName);

	// see if the old/new names map to existing UNiagaraScriptVariables
	UNiagaraScriptVariable* OrigScriptVariable = VariableToScriptVariable.FindRef(OrigVariable);
	UNiagaraScriptVariable* DestScriptVariable = VariableToScriptVariable.FindRef(DestVariable);

	// assign the new name to the node
	SwitchNode->InputParameterName = NewParameterName;

	// if we're just merging to an existing variable then we don't need to create a new one
	if (DestScriptVariable)
	{
		FNiagaraEditorUtilities::InfoWithToastAndLog(FText::Format(
			LOCTEXT("MergedVar", "\"{0}\" has been merged with parameter \"{1}\".\nAll of \"{1}\"'s meta-data will be used, overwriting \"{0}\"'s meta-data."),
			FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(OrigParameterName),
			FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(NewParameterName)));
	}
	else
	{
		TObjectPtr<UNiagaraScriptVariable>& CreatedParameter = VariableToScriptVariable.Add(DestVariable);
		
		if (OrigScriptVariable)
		{
			CreatedParameter = DuplicateObject<UNiagaraScriptVariable>(OrigScriptVariable, this);
		}
		else
		{
			CreatedParameter = NewObject<UNiagaraScriptVariable>(this);
		}
		
		CreatedParameter->SetFlags(RF_Transactional);
		CreatedParameter->Metadata.CreateNewGuid();
		CreatedParameter->Variable = DestVariable;
	}

	TInstancedStruct<FNiagaraParametersChangedData> RenameData = TInstancedStruct<FNiagaraParameterRenamedData>::Make();
	FNiagaraParameterRenamedData& Data = RenameData.GetMutable<FNiagaraParameterRenamedData>();
	Data.OldScriptVariable = OrigScriptVariable;
	Data.NewScriptVariable = VariableToScriptVariable[DestVariable];
	OnParametersChangedDelegate.Broadcast(RenameData);
	
	// now see if we need to remove any script variables that are now obsolete because they have been renamed
	if (OrigScriptVariable)
	{
		const TArray<FNiagaraVariable> StaticSwitchInputs = FindStaticSwitchInputs(false /*bReachableOnly*/);

		if (!StaticSwitchInputs.Contains(OrigVariable))
		{
			RemoveParameter(OrigVariable, true);
		}
	}

	// Fixup reference collection and pin names
	FixupReferenceCollectionsPostRename(OrigVariable, DestVariable, DestScriptVariable != nullptr /* bIsMerging */, false /* bSuppressEvents */);
	
	NotifyGraphChanged();

	// force the graph to refresh the metadata
	GetParameterReferenceMap();

	return true;
}

void UNiagaraGraph::ScriptVariableChanged(FNiagaraVariable Variable)
{
	check(!bIsForCompilationOnly);

	UNiagaraScriptVariable* ScriptVariable = GetScriptVariable(Variable);
	if (!ScriptVariable)
	{
		return;
	}

	if(ScriptVariable->GetIsStaticSwitch())
	{
		TArray<UNiagaraNodeStaticSwitch*> StaticSwitchNodes;
		GetNodesOfClass<UNiagaraNodeStaticSwitch>(StaticSwitchNodes);

		for(UNiagaraNodeStaticSwitch* StaticSwitchNode : StaticSwitchNodes)
		{
			if(StaticSwitchNode->InputParameterName == ScriptVariable->Variable.GetName())
			{
				StaticSwitchNode->AttemptUpdatePins();
			}
		}
		
		return;
	}

	TSet<UNiagaraNodeParameterMapGet*> MapGetNodes;
	TArray<UEdGraphPin*> Pins = FindParameterMapDefaultValuePins(Variable.GetName());
	for (UEdGraphPin* Pin : Pins)
	{
		if(UNiagaraNodeParameterMapGet* MapGetNode = Cast<UNiagaraNodeParameterMapGet>(Pin->GetOwningNode()))
		{
			MapGetNodes.Add(MapGetNode);
		}
		
		if (ScriptVariable->DefaultMode == ENiagaraDefaultMode::Value && !Variable.GetType().IsDataInterface() && !Variable.GetType().IsUObject())
		{
			FNiagaraEditorModule& EditorModule = FNiagaraEditorModule::Get();
			auto TypeUtilityValue = EditorModule.GetTypeUtilities(Variable.GetType());
			if (TypeUtilityValue.IsValid() && TypeUtilityValue->CanHandlePinDefaults())
			{
				if (!Variable.IsDataAllocated())
				{
					Variable.AllocateData();
				}
				FString NewDefaultValue = TypeUtilityValue->GetPinDefaultStringFromValue(Variable);
				GetDefault<UEdGraphSchema_Niagara>()->TrySetDefaultValue(*Pin, NewDefaultValue, true);
			}
		}
	}

	for(UNiagaraNodeParameterMapGet* MapGet : MapGetNodes)
	{
		MapGet->SynchronizeDefaultPins();
	}

	if(GIsTransacting == false)
	{
		ValidateDefaultPins();
	}
	
	NotifyGraphChanged();
}

FVersionedNiagaraEmitter UNiagaraGraph::GetOwningEmitter() const
{
	UNiagaraEmitter* OwningEmitter = GetTypedOuter<UNiagaraEmitter>();
	if (OwningEmitter)
	{
		for (FNiagaraAssetVersion Version : OwningEmitter->GetAllAvailableVersions())
		{
			FVersionedNiagaraEmitterData* EmitterData = OwningEmitter->GetEmitterData(Version.VersionGuid);
			if (EmitterData->GraphSource)
			{
				UNiagaraGraph* EmitterGraph = Cast<UNiagaraScriptSource>(EmitterData->GraphSource)->NodeGraph;
				if (this == EmitterGraph)
				{
					return FVersionedNiagaraEmitter(OwningEmitter, Version.VersionGuid);
				}
			}
		}
	}
	return FVersionedNiagaraEmitter();
}

bool UNiagaraGraph::SynchronizeScriptVariable(const UNiagaraScriptVariable* SourceScriptVar, UNiagaraScriptVariable* DestScriptVar /*= nullptr*/, bool bIgnoreChangeId /*= false*/)
{
	check(!bIsForCompilationOnly);

	if (DestScriptVar == nullptr)
	{
		const FGuid& SourceScriptVarId = SourceScriptVar->Metadata.GetVariableGuid();
		TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptVariables;
		VariableToScriptVariable.GenerateValueArray(ScriptVariables);
		TObjectPtr<UNiagaraScriptVariable>* ScriptVarPtr = ScriptVariables.FindByPredicate([&SourceScriptVarId](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Metadata.GetVariableGuid() == SourceScriptVarId; });
		if(ScriptVarPtr == nullptr)
		{
			// Failed to find a DestScriptVar with an Id matching that of SourceScriptVar.
			return false;
		}
		DestScriptVar = *ScriptVarPtr;
	}
	
	// Only synchronize if the dest script var change id is out of sync from the source script var change id.
	if (bIgnoreChangeId || (DestScriptVar->GetChangeId() != SourceScriptVar->GetChangeId()) )
	{
		// UNiagaraScriptVariable Properties
		// Only notify that the scriptvariable change needs to be synchronized in the graph if a default value change occurs: if we are only synchronizing metadata changes, do not dirty the graph.
		bool bRequiresSync = false;
		if(DestScriptVar->GetIsOverridingParameterDefinitionsDefaultValue() == false && UNiagaraScriptVariable::DefaultsAreEquivalent(SourceScriptVar, DestScriptVar) == false)
		{ 
			DestScriptVar->DefaultMode = SourceScriptVar->DefaultMode;
			DestScriptVar->DefaultBinding = SourceScriptVar->DefaultBinding;
			if (SourceScriptVar->GetDefaultValueData() != nullptr)
			{
				DestScriptVar->SetDefaultValueData(SourceScriptVar->GetDefaultValueData());
			}
			bRequiresSync = true;
		}
		DestScriptVar->SetChangeId(SourceScriptVar->GetChangeId());

		// FNiagaraVariable Properties
		DestScriptVar->Variable.SetData(SourceScriptVar->Variable.GetData());
		DestScriptVar->Variable.SetType(SourceScriptVar->Variable.GetType());

		// FNiagaraVariableMetadata
		DestScriptVar->Metadata.Description = SourceScriptVar->Metadata.Description;

		// Call rename parameter as we need to synchronize the parameter name to all pins.
		if (DestScriptVar->Variable.GetName() != SourceScriptVar->Variable.GetName())
		{
			bool bRenameRequestedFromStaticSwitch = false;
			bool* bMerged = nullptr;
			bool bSuppressEvents = true;
			RenameParameter(DestScriptVar->Variable, SourceScriptVar->Variable.GetName(), bRenameRequestedFromStaticSwitch, bMerged, bSuppressEvents);
		}

		// Notify the script variable has changed to propagate the default value to the graph node.
		if (bRequiresSync)
		{ 
			ScriptVariableChanged(DestScriptVar->Variable);
		}

		return bRequiresSync;
	}
	return false;
}

bool UNiagaraGraph::SynchronizeParameterDefinitionsScriptVariableRemoved(const FGuid RemovedScriptVarId)
{
	check(!bIsForCompilationOnly);

	TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptVariables;
	VariableToScriptVariable.GenerateValueArray(ScriptVariables);
	for (UNiagaraScriptVariable* ScriptVar : ScriptVariables)
	{
		if (ScriptVar->Metadata.GetVariableGuid() == RemovedScriptVarId)
		{
			ScriptVar->SetIsSubscribedToParameterDefinitions(false);
			MarkGraphRequiresSynchronization(TEXT("Graph Parameter Unlinked From Definition."));
			return true;
		}
	}
	return false;
}

void UNiagaraGraph::SynchronizeParametersWithParameterDefinitions(
	const TArray<UNiagaraParameterDefinitions*> TargetDefinitions,
	const TArray<UNiagaraParameterDefinitions*> AllDefinitions,
	const TSet<FGuid>& AllDefinitionsParameterIds,
	INiagaraParameterDefinitionsSubscriber* Subscriber,
	FSynchronizeWithParameterDefinitionsArgs Args)
{
	check(!bIsForCompilationOnly);

	bool bMarkRequiresSync = false;
	TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptVariables;
	TArray<UNiagaraScriptVariable*> TargetScriptVariables;
	VariableToScriptVariable.GenerateValueArray(ScriptVariables);

	// Filter script variables that will be synchronized if specific script variable ids are specified.
	if (Args.SpecificDestScriptVarIds.Num() > 0)
	{
		TargetScriptVariables = ScriptVariables.FilterByPredicate([&Args](const UNiagaraScriptVariable* DestScriptVar){ return Args.SpecificDestScriptVarIds.Contains(DestScriptVar->Metadata.GetVariableGuid()); });
	}
	else
	{
		TargetScriptVariables = ScriptVariables;
	}

	// Get all script variables from target definitions.
	TArray<const UNiagaraScriptVariable*> TargetLibraryScriptVariables;
	for (const UNiagaraParameterDefinitions* TargetParameterDefinitionsItr : TargetDefinitions)
	{	
		TargetLibraryScriptVariables.Append(TargetParameterDefinitionsItr->GetParametersConst());
	}

	auto GetTargetDefinitionScriptVarWithSameId = [&TargetLibraryScriptVariables](const UNiagaraScriptVariable* GraphScriptVar)->const UNiagaraScriptVariable* {
		const FGuid& GraphScriptVarId = GraphScriptVar->Metadata.GetVariableGuid();
		if (const UNiagaraScriptVariable* const* FoundLibraryScriptVarPtr = TargetLibraryScriptVariables.FindByPredicate([GraphScriptVarId](const UNiagaraScriptVariable* LibraryScriptVar) { return LibraryScriptVar->Metadata.GetVariableGuid() == GraphScriptVarId; }))
		{
			return *FoundLibraryScriptVarPtr;
		}
		return nullptr;
	};

	// If subscribing all name match parameters; 
	// If a destination parameter has the same name as a source parameter, create a subscription for the source parameter definition. 
	// Retain the destination parameter default value if it does not match the source parameters.
	if (Args.bSubscribeAllNameMatchParameters)
	{
		// Get all script variables from all definitions.
		TArray<const UNiagaraScriptVariable*> AllDefinitionsScriptVariables;
		for (const UNiagaraParameterDefinitions* AllDefinitionsItr : AllDefinitions)
		{
			AllDefinitionsScriptVariables.Append(AllDefinitionsItr->GetParametersConst());
		}

		auto GetDefinitionScriptVarWithSameNameAndType = [&AllDefinitionsScriptVariables](const UNiagaraScriptVariable* GraphScriptVar)->const UNiagaraScriptVariable* {
			if (const UNiagaraScriptVariable* const* FoundLibraryScriptVarPtr = AllDefinitionsScriptVariables.FindByPredicate([&GraphScriptVar](const UNiagaraScriptVariable* LibraryScriptVar) { return LibraryScriptVar->Variable == GraphScriptVar->Variable; }))
			{
				return *FoundLibraryScriptVarPtr;
			}
			return nullptr;
		};

		for (UNiagaraScriptVariable* TargetScriptVar : TargetScriptVariables)
		{
			// Skip parameters that are already subscribed.
			if (TargetScriptVar->GetIsSubscribedToParameterDefinitions())
			{
				continue;
			}
			else if (const UNiagaraScriptVariable* LibraryScriptVar = GetDefinitionScriptVarWithSameNameAndType(TargetScriptVar))
			{
				// Add the found definition script var as a target script var so that it can be synchronized with later.
				TargetLibraryScriptVariables.Add(LibraryScriptVar);

				const bool bDoNotAssetIfAlreadySubscribed = true;
				Subscriber->SubscribeToParameterDefinitions(CastChecked<UNiagaraParameterDefinitions>(LibraryScriptVar->GetOuter()), bDoNotAssetIfAlreadySubscribed);
				TargetScriptVar->SetIsSubscribedToParameterDefinitions(true);
				TargetScriptVar->Metadata.SetVariableGuid(LibraryScriptVar->Metadata.GetVariableGuid());
				if (UNiagaraScriptVariable::DefaultsAreEquivalent(TargetScriptVar, LibraryScriptVar) == false)
				{
					// Preserve the TargetScriptVars default value if it is not equivalent to prevent breaking changes from subscribing new parameters.
					TargetScriptVar->SetIsOverridingParameterDefinitionsDefaultValue(true);
				}
				SynchronizeScriptVariable(LibraryScriptVar, TargetScriptVar);
			}
		}
	}

	for(UNiagaraScriptVariable* TargetScriptVar : TargetScriptVariables)
	{
		if (TargetScriptVar->GetIsSubscribedToParameterDefinitions())
		{
			if (const UNiagaraScriptVariable* TargetLibraryScriptVar = GetTargetDefinitionScriptVarWithSameId(TargetScriptVar))
			{
				bMarkRequiresSync |= SynchronizeScriptVariable(TargetLibraryScriptVar, TargetScriptVar, Args.bForceSynchronizeParameters);
			}
			else if(AllDefinitionsParameterIds.Contains(TargetScriptVar->Metadata.GetVariableGuid()) == false)
			{ 
				// ScriptVar is marked as being sourced from a parameter definitions but no matching library script variables were found, break the link to the parameter definitions for ScriptVar.
				TargetScriptVar->SetIsSubscribedToParameterDefinitions(false);
				bMarkRequiresSync = true;
			}
		}
	}

	if(bMarkRequiresSync)
	{ 
		NotifyGraphNeedsRecompile();
	}
}

void UNiagaraGraph::RenameAssignmentAndSetNodePins(const FName OldName, const FName NewName)
{
	TArray<UNiagaraNodeParameterMapGet*> MapGetNodes;
	GetNodesOfClass<UNiagaraNodeParameterMapGet>(MapGetNodes);
	TArray<UNiagaraNodeAssignment*> AssignmentNodes;
	GetNodesOfClass<UNiagaraNodeAssignment>(AssignmentNodes);

	for (UNiagaraNodeParameterMapGet* MapGetNode : MapGetNodes)
	{
		TArray<UEdGraphPin*> OutputPins;
		MapGetNode->GetOutputPins(OutputPins);
		for (UEdGraphPin* OutputPin : OutputPins)
		{
			if (OutputPin->PinName == OldName)
			{
				MapGetNode->SetPinName(OutputPin, NewName);
			}
		}
	}

	for (UNiagaraNodeAssignment* AssignmentNode : AssignmentNodes)
	{
		bool bMustRefresh = AssignmentNode->RenameAssignmentTarget(OldName, NewName);
		AssignmentNode->RefreshFromExternalChanges();
	}
}

int32 UNiagaraGraph::GetOutputNodeVariableIndex(const FNiagaraVariable& Variable)const
{
	TArray<FNiagaraVariable> Variables;
	GetOutputNodeVariables(Variables);
	return Variables.Find(Variable);
}

void UNiagaraGraph::GetOutputNodeVariables(TArray< FNiagaraVariable >& OutVariables)const
{
	TArray<UNiagaraNodeOutput*> OutputNodes;
	FindOutputNodes(OutputNodes);
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		for (FNiagaraVariable& Var : OutputNode->Outputs)
		{
			OutVariables.AddUnique(Var);
		}
	}
}

void UNiagaraGraph::GetOutputNodeVariables(ENiagaraScriptUsage InScriptUsage, TArray< FNiagaraVariable >& OutVariables)const
{
	TArray<UNiagaraNodeOutput*> OutputNodes;
	FindOutputNodes(InScriptUsage, OutputNodes);
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		for (FNiagaraVariable& Var : OutputNode->Outputs)
		{
			OutVariables.AddUnique(Var);
		}
	}
}

bool UNiagaraGraph::HasParameterMapParameters()const
{
	TArray<FNiagaraVariable> Inputs;
	TArray<FNiagaraVariable> Outputs;

	GetParameters(Inputs, Outputs);

	for (FNiagaraVariable& Var : Inputs)
	{
		if (Var.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			return true;
		}
	}
	for (FNiagaraVariable& Var : Outputs)
	{
		if (Var.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			return true;
		}
	}

	return false;
}

bool UNiagaraGraph::GetPropertyMetadata(FName PropertyName, FString& OutValue) const
{
	for (const FNiagaraScriptVariableData& ScriptVariableData : CompilationScriptVariables)
	{
		if (const FString* FoundMatch = ScriptVariableData.Metadata.PropertyMetaData.Find(PropertyName))
		{
			OutValue = *FoundMatch;
			return true;
		}
	}

	for (const FScriptVariableMap::ElementType& Iter : VariableToScriptVariable)
	{
		// TODO: This should never be null, but somehow it is in some assets so guard this to prevent crashes
		// until we have better repro steps.
		if (const UNiagaraScriptVariable* ScriptVariable = Iter.Value)
		{
			if (const FString* PropertyValue = ScriptVariable->Metadata.PropertyMetaData.Find(PropertyName))
			{
				OutValue = *PropertyValue;
				return true;
			}
		}
	}

	return false;
}

bool UNiagaraGraph::HasNumericParameters()const
{
	TArray<FNiagaraVariable> Inputs;
	TArray<FNiagaraVariable> Outputs;
	
	GetParameters(Inputs, Outputs);
	
	for (FNiagaraVariable& Var : Inputs)
	{
		if (Var.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
		{
			return true;
		}
	}
	for (FNiagaraVariable& Var : Outputs)
	{
		if (Var.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
		{
			return true;
		}
	}

	return false;
}

void UNiagaraGraph::NotifyGraphNeedsRecompile()
{
	FEdGraphEditAction Action;
	Action.Action = (EEdGraphActionType)GRAPHACTION_GenericNeedsRecompile;
	NotifyGraphChanged(Action);
}


void UNiagaraGraph::NotifyGraphDataInterfaceChanged()
{
	OnDataInterfaceChangedDelegate.Broadcast();
}

FNiagaraTypeDefinition UNiagaraGraph::GetCachedNumericConversion(const class UEdGraphPin* InPin)
{
	if (bNeedNumericCacheRebuilt)
	{
		RebuildNumericCache();
	}

	FNiagaraTypeDefinition ReturnDef;
	if (InPin && InPin->PinId.IsValid())
	{
		FNiagaraTypeDefinition* FoundDef = CachedNumericConversions.Find(TPair<FGuid, UEdGraphNode*>(InPin->PinId, InPin->GetOwningNode()));
		if (FoundDef)
		{
			ReturnDef = *FoundDef;
		}
	}
	return ReturnDef;
}

bool UNiagaraGraph::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor, const TArray<UNiagaraNode*>& InTraversal) const
{
	check(!bIsForCompilationOnly);

#if WITH_EDITORONLY_DATA
	if (FNiagaraCompileHashVisitorDebugInfo* DebugInfo = InVisitor->AddDebugInfo())
	{
		DebugInfo->Object = FString::Printf(TEXT("Class: \"%s\"  Name: \"%s\""), *this->GetClass()->GetName(), *this->GetName());
	}
#endif
	InVisitor->UpdateString(TEXT("ForceRebuildId"), ForceRebuildId.ToString());

	TSet<FNiagaraVariable> ReferencedVariables;

	NiagaraGraphImpl::FindReferencedVariables(this, InTraversal, ReferencedVariables);

	TArray<UNiagaraScriptVariable*> SortedScriptVariables;
	SortedScriptVariables.Reserve(ReferencedVariables.Num());
	for (const FNiagaraVariable& ReferencedVariable : ReferencedVariables)
	{
		if (UNiagaraScriptVariable* ScriptVariable = VariableToScriptVariable.FindRef(ReferencedVariable))
		{
			SortedScriptVariables.Add(ScriptVariable);
		}
	}

	Algo::StableSortBy(SortedScriptVariables,
		[](UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Variable.GetName(); },
		FNameLexicalLess());

	for (UNiagaraScriptVariable* ScriptVariable : SortedScriptVariables)
	{
#if WITH_EDITORONLY_DATA
		if (FNiagaraCompileHashVisitorDebugInfo* DebugInfo = InVisitor->AddDebugInfo())
		{
			DebugInfo->Object = FString::Printf(TEXT("Class: \"%s\"  Name: \"%s\""), *ScriptVariable->GetClass()->GetName(), *ScriptVariable->Variable.GetName().ToString());
		}
#endif
		verify(ScriptVariable->AppendCompileHash(InVisitor));
	}

	// Write all the values of the nodes to the visitor as they could influence compilation.
	for (const UNiagaraNode* Node : InTraversal)
	{
#if WITH_EDITORONLY_DATA
		if (FNiagaraCompileHashVisitorDebugInfo* DebugInfo = InVisitor->AddDebugInfo())
		{
			DebugInfo->Object = FString::Printf(TEXT("Class: \"%s\" Title: \"%s\" Name: \"%s\" Guid: %s"), *Node->GetClass()->GetName(), *Node->GetNodeTitle(ENodeTitleType::EditableTitle).ToString(), *Node->GetName(), *LexToString(Node->NodeGuid));
		}
#endif
		verify(Node->AppendCompileHash(InVisitor));
	}

#if WITH_EDITORONLY_DATA
	// Optionally log out the information for debugging.
	if (FNiagaraCompileHashVisitor::LogCompileIdGeneration == 2 && InTraversal.Num() > 0)
	{
		FString RelativePath;
		UObject* Package = GetOutermost();
		if (Package != nullptr)
		{
			RelativePath += Package->GetName() + TEXT("/");
		}


		UObject* Parent = GetOuter();
		while (Parent != Package)
		{
			bool bSkipName = false;
			if (Parent->IsA<UNiagaraGraph>()) // Removing common clutter
				bSkipName = true;
			else if (Parent->IsA<UNiagaraScriptSourceBase>()) // Removing common clutter
				bSkipName = true;

			if (!bSkipName)
				RelativePath = RelativePath + Parent->GetName() + TEXT("/");
			Parent = Parent->GetOuter();
		}

	
		FString ObjName = GetName();
		FString DumpDebugInfoPath = FPaths::ProjectSavedDir() + TEXT("NiagaraHashes/") + RelativePath ;
		FPaths::NormalizeDirectoryName(DumpDebugInfoPath);
		DumpDebugInfoPath.ReplaceInline(TEXT("<"), TEXT("("));
		DumpDebugInfoPath.ReplaceInline(TEXT(">"), TEXT(")"));
		DumpDebugInfoPath.ReplaceInline(TEXT("::"), TEXT("=="));
		DumpDebugInfoPath.ReplaceInline(TEXT("|"), TEXT("_"));
		DumpDebugInfoPath.ReplaceInline(TEXT("*"), TEXT("-"));
		DumpDebugInfoPath.ReplaceInline(TEXT("?"), TEXT("!"));
		DumpDebugInfoPath.ReplaceInline(TEXT("\""), TEXT("\'"));


		if (!IFileManager::Get().DirectoryExists(*DumpDebugInfoPath))
		{
			if (!IFileManager::Get().MakeDirectory(*DumpDebugInfoPath, true))
				UE_LOGF(LogNiagaraEditor, Warning, "Failed to create directory for debug info '%ls'", *DumpDebugInfoPath);
		}
		FString ExportText = FString::Printf(TEXT("UNiagaraGraph::AppendCompileHash %s %s\n===========================\n"), *GetFullName(), *InTraversal[InTraversal.Num()- 1]->GetNodeTitle(ENodeTitleType::ListView).ToString());
		for (int32 i = 0; i < InVisitor->Values.Num(); i++)
		{
			ExportText += FString::Printf(TEXT("Object[%d]: %s\n"), i, *InVisitor->Values[i].Object);
			ensure(InVisitor->Values[i].PropertyKeys.Num() == InVisitor->Values[i].PropertyValues.Num());
			for (int32 j = 0; j < InVisitor->Values[i].PropertyKeys.Num(); j++)
			{
				ExportText += FString::Printf(TEXT("\tProperty[%d]: %s = %s\n"), j, *InVisitor->Values[i].PropertyKeys[j], *InVisitor->Values[i].PropertyValues[j]);
			}
		}

		FNiagaraEditorUtilities::WriteTextFileToDisk(DumpDebugInfoPath, ObjName + TEXT(".txt"), ExportText, true);
	}
#endif
	return true;
}

void DiffProperties(const FNiagaraCompileHashVisitorDebugInfo& A, const FNiagaraCompileHashVisitorDebugInfo& B)
{
	if (A.PropertyKeys.Num() != B.PropertyKeys.Num())
	{
		UE_LOGF(LogNiagaraEditor, Log, "Hash Difference: Property Count Mismatch %d vs %d on %ls", A.PropertyKeys.Num(), B.PropertyKeys.Num(), *A.Object);
	}
	else
	{
		bool bFoundMatch = false;
		for (int32 i = 0; i < A.PropertyKeys.Num(); i++)
		{
			if (A.PropertyKeys[i] == B.PropertyKeys[i])
			{
				bFoundMatch = true;
				if (A.PropertyValues[i] != B.PropertyValues[i])
				{
					UE_LOGF(LogNiagaraEditor, Log, "Hash Difference: Property Value Mismatch %ls vs %ls on property %ls of %ls", *A.PropertyValues[i], *B.PropertyValues[i], *A.PropertyKeys[i], *A.Object);
				}
			}
		}
		ensure(bFoundMatch);
	}
}


void UNiagaraGraph::ConditionalRebuildCompileIdCache()
{
	if (!bHasValidLastBuiltScriptVersionId)
	{
		bHasValidLastBuiltScriptVersionId = true;
		LastBuiltScriptVersionId = FGuid();
	}
}

void UNiagaraGraph::RebuildCachedCompileIds()
{
	check(!bIsForCompilationOnly);

	const FGuid CurrentScriptVersionId = FNiagaraCustomVersion::GetLatestScriptCompileVersion();


	// If the graph hasn't changed since last rebuild, then do nothing.
	const bool bForceRebuild = (bHasValidLastBuiltScriptVersionId && (LastBuiltScriptVersionId != CurrentScriptVersionId));
	
	if (!bForceRebuild && ChangeId == LastBuiltTraversalDataChangeId && LastBuiltTraversalDataChangeId.IsValid())
	{
		return;
	}

	if (!AllowShaderCompiling())
	{
		return;
	}

	// First find all the output nodes
	TArray<UNiagaraNodeOutput*> NiagaraOutputNodes;
	GetNodesOfClass<UNiagaraNodeOutput>(NiagaraOutputNodes);

	// Now build the new cache..
	TArray<FNiagaraGraphScriptUsageInfo> NewUsageCache;
	NewUsageCache.AddDefaulted(NiagaraOutputNodes.Num());

	bool bNeedsAnyNewCompileIds = false;

	FNiagaraGraphScriptUsageInfo* ParticleSpawnUsageInfo = nullptr;
	FNiagaraGraphScriptUsageInfo* ParticleUpdateUsageInfo = nullptr;

	for (int32 i = 0; i < NiagaraOutputNodes.Num(); i++)
	{
		UNiagaraNodeOutput* OutputNode = NiagaraOutputNodes[i];
		NewUsageCache[i].UsageType = OutputNode->GetUsage();
		NewUsageCache[i].UsageId = OutputNode->GetUsageId();

		BuildTraversal(MutableView(NewUsageCache[i].Traversal), OutputNode);

		int32 FoundMatchIdx = INDEX_NONE;
		for (int32 j = 0; j < CachedUsageInfo.Num(); j++)
		{
			if (UNiagaraScript::IsEquivalentUsage(CachedUsageInfo[j].UsageType, NewUsageCache[i].UsageType) && CachedUsageInfo[j].UsageId == NewUsageCache[i].UsageId)
			{
				FoundMatchIdx = j;
				break;
			}
		}

		if (FoundMatchIdx == INDEX_NONE || CachedUsageInfo[FoundMatchIdx].BaseId.IsValid() == false)
		{
			NewUsageCache[i].BaseId = FGuid::NewGuid();
		}
		else
		{
			//Copy the old base id if available and valid.
			NewUsageCache[i].BaseId = CachedUsageInfo[FoundMatchIdx].BaseId;
		}

		// Now compare the change id's of all the nodes in the traversal by hashing them up and comparing the hash
		// now with the hash from previous runs.
		FSHA1 GraphHashState;
		FNiagaraCompileHashVisitor Visitor(GraphHashState);
		AppendCompileHash(&Visitor, NewUsageCache[i].Traversal);
		GraphHashState.Final();

		FSHA1 HashState;
		FSHA1 HashStateStatics;
		for (UNiagaraNode* Node : NewUsageCache[i].Traversal)
		{
			Node->UpdateCompileHashForNode(HashState);

			UNiagaraNodeFunctionCall* CallNode = Cast<UNiagaraNodeFunctionCall>(Node);
			if (CallNode)
			{
				CallNode->UpdateReferencedStaticsHashForNode(HashStateStatics);
			}

		}
		HashState.Final();
		HashStateStatics.Final();

		// We can't store in a FShaHash struct directly because you can't FProperty it. Using a standin of the same size.
		{
			TStaticArray<uint8, FSHA1::DigestSize> DataHash;
			HashState.GetHash(DataHash.GetData());
		
			NewUsageCache[i].CompileHash = FNiagaraCompileHash(DataHash.GetData(), DataHash.Num());
		}

		{
			TStaticArray<uint8, FSHA1::DigestSize> DataHashStatics;
			HashStateStatics.GetHash(DataHashStatics.GetData());
			NewUsageCache[i].ReferenceHashFromGraph = FNiagaraCompileHash(DataHashStatics.GetData(), DataHashStatics.Num());
		}

		{
			// We can't store in a FShaHash struct directly because you can't UProperty it. Using a standin of the same size.
			TStaticArray<uint8, FSHA1::DigestSize> DataHash;
			GraphHashState.GetHash(DataHash.GetData());
			NewUsageCache[i].CompileHashFromGraph = FNiagaraCompileHash(DataHash.GetData(), DataHash.Num());
			NewUsageCache[i].CompileLastObjects = Visitor.Values;	

#if WITH_EDITORONLY_DATA
			// Log out all the entries that differ!
			if (FNiagaraCompileHashVisitor::LogCompileIdGeneration != 0 && FoundMatchIdx != -1 && NewUsageCache[i].CompileHashFromGraph != CachedUsageInfo[FoundMatchIdx].CompileHashFromGraph)
			{
				TArray<FNiagaraCompileHashVisitorDebugInfo> OldDebugValues;
				OldDebugValues = CachedUsageInfo[FoundMatchIdx].CompileLastObjects;

				TArray<bool> bFoundIndices; // Record if we ever found these.
				bFoundIndices.AddZeroed(OldDebugValues.Num());

				for (int32 ObjIdx = 0; ObjIdx < NewUsageCache[i].CompileLastObjects.Num(); ObjIdx++)
				{
					bool bFound = false;
					for (int32 OldObjIdx = 0; OldObjIdx < OldDebugValues.Num(); OldObjIdx++)
					{
						if (OldDebugValues[OldObjIdx].Object == NewUsageCache[i].CompileLastObjects[ObjIdx].Object)
						{
							bFound = true; // Record that we found a match for this object
							bFoundIndices[OldObjIdx] = true; // Record that we found an overall match
							DiffProperties(OldDebugValues[OldObjIdx], NewUsageCache[i].CompileLastObjects[ObjIdx]);
						}
					}

					if (!bFound)
					{
						UE_LOGF(LogNiagaraEditor, Log, "Hash Difference: New Object: %ls", *NewUsageCache[i].CompileLastObjects[ObjIdx].Object);
					}
				}


				for (int32 ObjIdx = 0; ObjIdx < OldDebugValues.Num(); ObjIdx++)
				{
					if (bFoundIndices[ObjIdx] == false)
					{
						UE_LOGF(LogNiagaraEditor, Log, "Hash Difference: Removed Object: %ls", *OldDebugValues[ObjIdx].Object);
					}					
				}
			}
#endif
		}

		if (UNiagaraScript::IsEquivalentUsage(NewUsageCache[i].UsageType, ENiagaraScriptUsage::ParticleSpawnScript) && NewUsageCache[i].UsageId == FGuid())
		{
			ParticleSpawnUsageInfo = &NewUsageCache[i];
		}

		if (UNiagaraScript::IsEquivalentUsage(NewUsageCache[i].UsageType, ENiagaraScriptUsage::ParticleUpdateScript) && NewUsageCache[i].UsageId == FGuid())
		{
			ParticleUpdateUsageInfo = &NewUsageCache[i];
		}
	}

	if (ParticleSpawnUsageInfo != nullptr && ParticleUpdateUsageInfo != nullptr)
	{
		// If we have info for both spawn and update generate the gpu version too.
		FNiagaraGraphScriptUsageInfo GpuUsageInfo;
		GpuUsageInfo.UsageType = ENiagaraScriptUsage::ParticleGPUComputeScript;
		GpuUsageInfo.UsageId = FGuid();

		FNiagaraGraphScriptUsageInfo* OldGpuInfo = CachedUsageInfo.FindByPredicate(
			[](const FNiagaraGraphScriptUsageInfo& OldInfo) { return OldInfo.UsageType == ENiagaraScriptUsage::ParticleGPUComputeScript && OldInfo.UsageId == FGuid(); });
		if (OldGpuInfo == nullptr || OldGpuInfo->BaseId.IsValid() == false)
		{
			GpuUsageInfo.BaseId = FGuid::NewGuid();
		}
		else
		{
			// Copy the old base id if available
			GpuUsageInfo.BaseId = OldGpuInfo->BaseId;
		}

		// The GPU script has no graph representation, but we still need to fill in the hash, because it's used in the shader map ID.
		// Just copy the hash from the spawn script.
		GpuUsageInfo.CompileHash = ParticleSpawnUsageInfo->CompileHash;
		GpuUsageInfo.CompileHashFromGraph = ParticleSpawnUsageInfo->CompileHashFromGraph;

		NewUsageCache.Add(GpuUsageInfo);
	}

	// Debug logic, usually disabled at top of file.
	if (bNeedsAnyNewCompileIds && bWriteToLog)
	{
		TMap<FGuid, FGuid> ComputeChangeIds;
		FNiagaraEditorUtilities::GatherChangeIds(*this, ComputeChangeIds, GetName());
	}

	// Now update the cache with the newly computed results.
	CachedUsageInfo = NewUsageCache;
	LastBuiltTraversalDataChangeId = ChangeId;

	LastBuiltScriptVersionId = CurrentScriptVersionId;
	bHasValidLastBuiltScriptVersionId = true;

	RebuildNumericCache();
}

void UNiagaraGraph::CopyCachedReferencesMap(UNiagaraGraph* TargetGraph)
{
	// make sure that we have collected parameter references already
	ConditionalRefreshParameterReferences();

	TargetGraph->ParameterToReferencesMap = ParameterToReferencesMap;
}

const class UEdGraphSchema_Niagara* UNiagaraGraph::GetNiagaraSchema() const
{
	return Cast<UEdGraphSchema_Niagara>(GetSchema());
}

void UNiagaraGraph::RebuildNumericCache()
{
	CachedNumericConversions.Empty();
	TMap<UNiagaraNode*, bool> VisitedNodes;
	for (UEdGraphNode* Node : Nodes)
	{
		ResolveNumerics(VisitedNodes, Node);
	}
	bNeedNumericCacheRebuilt = false;
}

void UNiagaraGraph::InvalidateNumericCache()
{
	bNeedNumericCacheRebuilt = true; 
	CachedNumericConversions.Empty();
}

void UNiagaraGraph::UpdateFunctionAliasByContext(FNiagaraGraphFunctionAliasContext& FunctionAliasContext) const
{
	for (const UEdGraphNode* Node : Nodes)
	{
		if (const UNiagaraNode* NiagaraNode = Cast<const UNiagaraNode>(Node))
		{
			NiagaraNode->UpdateFunctionAliasForContext(FunctionAliasContext);
		}
	}
}

void UNiagaraGraph::ResolveNumerics(TMap<UNiagaraNode*, bool>& VisitedNodes, UEdGraphNode* Node)
{
	UNiagaraNode* NiagaraNode = Cast<UNiagaraNode>(Node);
	if (NiagaraNode)
	{
		FPinCollectorArray InputPins;
		NiagaraNode->GetInputPins(InputPins);
		for (int32 i = 0; i < InputPins.Num(); i++)
		{
			if (InputPins[i])
			{
				for (int32 j = 0; j < InputPins[i]->LinkedTo.Num(); j++)
				{
					if (InputPins[i]->LinkedTo[j])
					{
						UNiagaraNode* FoundNode = Cast<UNiagaraNode>(InputPins[i]->LinkedTo[j]->GetOwningNode());
						if (!FoundNode || VisitedNodes.Contains(FoundNode))
						{
							continue;
						}
						VisitedNodes.Add(FoundNode, true);
						ResolveNumerics(VisitedNodes, FoundNode);
					}
				}
			}
		}

		NiagaraNode->ResolveNumerics(GetNiagaraSchema(), false, &CachedNumericConversions);
		
	}
}

void UNiagaraGraph::ForceGraphToRecompileOnNextCheck()
{
	Modify();
	CachedUsageInfo.Empty();
	ForceRebuildId = FGuid::NewGuid();
	MarkGraphRequiresSynchronization(__FUNCTION__);
}

void UNiagaraGraph::GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector)
{
	RebuildCachedCompileIds();
	
	for (int32 i = 0; i < CachedUsageInfo.Num(); i++)
	{
		// First add our direct dependency chain...
		if (UNiagaraScript::IsEquivalentUsage(CachedUsageInfo[i].UsageType, InUsage) && CachedUsageInfo[i].UsageId == InUsageId)
		{
			for (UNiagaraNode* Node : CachedUsageInfo[i].Traversal)
			{
				Node->GatherExternalDependencyData(InUsage, InUsageId, HashCollector);
			}
		}
		// Now add any other dependency chains that we might have...
		else if (UNiagaraScript::IsUsageDependentOn(InUsage, CachedUsageInfo[i].UsageType))
		{
			if (GNiagaraUseGraphHash == 1)
			{
				HashCollector.AddHash(CachedUsageInfo[i].CompileHashFromGraph, CachedUsageInfo[i].Traversal.Last()->GetPathName());
			}
			else
			{
				HashCollector.AddHash(CachedUsageInfo[i].CompileHash, CachedUsageInfo[i].Traversal.Last()->GetPathName());
			}

			for (UNiagaraNode* Node : CachedUsageInfo[i].Traversal)
			{
				Node->GatherExternalDependencyData(InUsage, InUsageId, HashCollector);
			}
		}
	}
}


void UNiagaraGraph::GetAllReferencedGraphs(TArray<const UNiagaraGraph*>& Graphs) const
{
	Graphs.AddUnique(this);
	TArray<UNiagaraNodeFunctionCall*> FunctionCallNodes;
	GetNodesOfClass(FunctionCallNodes);
	for (UNiagaraNodeFunctionCall* FunctionCallNode : FunctionCallNodes)
	{
		UNiagaraGraph* FunctionGraph = FunctionCallNode->GetCalledGraph();
		if (FunctionGraph != nullptr)
		{
			if (!Graphs.Contains(FunctionGraph))
			{
				FunctionGraph->GetAllReferencedGraphs(Graphs);
			}
		}
	}
}

/** Determine if another item has been synchronized with this graph.*/
bool UNiagaraGraph::IsOtherSynchronized(const FGuid& InChangeId) const
{
	if (ChangeId.IsValid() && ChangeId == InChangeId)
	{
		return true;
	}
	return false;
}

/** Identify that this graph has undergone changes that will require synchronization with a compiled script.*/
void UNiagaraGraph::MarkGraphRequiresSynchronization(FString Reason)
{
	Modify();
	ChangeId = FGuid::NewGuid();
	if (GEnableVerboseNiagaraChangeIdLogging)
	{
		UE_LOGF(LogNiagaraEditor, Verbose, "Graph %ls was marked requires synchronization.  Reason: %ls", *GetPathName(), *Reason);
	}
}

TOptional<FNiagaraVariableMetaData> UNiagaraGraph::GetMetaData(const FNiagaraVariable& InVar) const
{
	if (TOptional<FNiagaraScriptVariableData> VariableData = GetScriptVariableData(InVar))
	{
		return VariableData->Metadata;
	}

	if (const UNiagaraScriptVariable* ScriptVariable = VariableToScriptVariable.FindRef(InVar))
	{
		return ScriptVariable->Metadata;
	}
	return TOptional<FNiagaraVariableMetaData>();
}

void UNiagaraGraph::SetMetaData(const FNiagaraVariable& InVar, const FNiagaraVariableMetaData& InMetaData)
{
	check(!bIsForCompilationOnly);

	if (TObjectPtr<UNiagaraScriptVariable>* FoundMetaData = VariableToScriptVariable.Find(InVar))
	{
		if (*FoundMetaData)
		{
			// Replace the old metadata..
			UNiagaraScriptVariable* ScriptVariable = (*FoundMetaData);
			ScriptVariable->Modify();
			ScriptVariable->Metadata = InMetaData;
			if (!ScriptVariable->Metadata.GetVariableGuid().IsValid())
			{
				ScriptVariable->Metadata.CreateNewGuid();
			}
		} 
	}
	else 
	{
		Modify();
		TObjectPtr<UNiagaraScriptVariable>& NewScriptVariable = VariableToScriptVariable.Add(InVar, NewObject<UNiagaraScriptVariable>(this, FName(), RF_Transactional));
		NewScriptVariable->Init(InVar, InMetaData);
		NewScriptVariable->SetIsStaticSwitch(FindStaticSwitchInputs().Contains(InVar));
	}
}

UNiagaraGraph::FOnDataInterfaceChanged& UNiagaraGraph::OnDataInterfaceChanged()
{
	return OnDataInterfaceChangedDelegate;
}

UNiagaraGraph::FOnSubObjectSelectionChanged& UNiagaraGraph::OnSubObjectSelectionChanged()
{
	return OnSelectedSubObjectChanged;
}

void UNiagaraGraph::RefreshParameterReferences() const
{
	ensureMsgf(!bIsForCompilationOnly, TEXT("RefreshParameterReferences() shouldn't be called on a graph duplicated for compilation - %s"), *GetFullName());

	// A set of variables to track which parameters are used so that unused parameters can be removed after the reference tracking.
	TSet<FNiagaraVariable> CandidateUnreferencedParametersToRemove;

	// The set of pins which has already been handled by add parameters.
	TSet<const UEdGraphPin*> HandledParameterMapPins;

	// Purge existing parameter references and collect candidate unreferenced parameters.
	for (auto& ParameterToReferences : ParameterToReferencesMap)
	{
		ParameterToReferences.Value.ParameterReferences.Empty();
		if (ParameterToReferences.Value.WasCreatedByUser() == false)
		{
			// Collect all parameters not created for the user so that they can be removed later if no references are found for them.
			CandidateUnreferencedParametersToRemove.Add(ParameterToReferences.Key);
		}
	}

	auto AddParameterReference = [&](const FNiagaraVariable& Parameter, const UEdGraphPin* Pin)
	{
		FNiagaraGraphParameterReferenceCollection& ReferenceCollection =
			ParameterToReferencesMap.FindOrAdd(Parameter, FNiagaraGraphParameterReferenceCollection(false /*bInCreated*/));

		ReferenceCollection.ParameterReferences.AddUnique(FNiagaraGraphParameterReference(Pin->PersistentGuid, Cast<UNiagaraNode>(Pin->GetOwningNode())));

		// If we're adding a parameter reference then it needs to be removed from the list of candidate variables to remove since it's been referenced.
		CandidateUnreferencedParametersToRemove.Remove(Parameter);
	};

	auto AddStaticParameterReference = [&](const FNiagaraVariable& Variable, UNiagaraNode* Node)
	{
		FNiagaraGraphParameterReferenceCollection& ReferenceCollection =
			ParameterToReferencesMap.FindOrAdd(Variable, FNiagaraGraphParameterReferenceCollection(false /*bInCreated*/));

		ReferenceCollection.ParameterReferences.AddUnique(FNiagaraGraphParameterReference(Node->NodeGuid, Node));
		CandidateUnreferencedParametersToRemove.Remove(Variable);
	};

	auto AddBindingParameterReference = [&](const FNiagaraVariable& Variable)
	{
		// We add an empty reference collection only when no other references exist.
		// We cannot add an actual reference since those require both a guid and a node,
		// but neither of these exist for direct bindings.
		FNiagaraGraphParameterReferenceCollection& ReferenceCollection =
			ParameterToReferencesMap.FindOrAdd(Variable, FNiagaraGraphParameterReferenceCollection(true /*bInCreated*/));
	};

	// Add parameter references from parameter map traversals.
	const TArray<FNiagaraParameterMapHistory> Histories = UNiagaraNodeParameterMapBase::GetParameterMaps(this);
	for (const FNiagaraParameterMapHistory& History : Histories)
	{
		for (int32 Index = 0; Index < History.VariablesWithOriginalAliasesIntact.Num(); Index++)
		{
			const FNiagaraVariable& Parameter = History.VariablesWithOriginalAliasesIntact[Index];
			for (const FNiagaraParameterMapHistory::FModuleScopedPin& WriteEvent : History.PerVariableWriteHistory[Index])
			{
				if (WriteEvent.Pin->PinType.PinSubCategory == UNiagaraNodeParameterMapBase::ParameterPinSubCategory)
				{
					bool bAddReference = true;

					// we exclude default pins from adding references as they shouldn't count. Originally they weren't marked as parameter pins so this wasn't necessary.
					if(WriteEvent.Pin->Direction == EGPD_Input && WriteEvent.Pin->GetOwningNode()->IsA<UNiagaraNodeParameterMapGet>())
					{
						bAddReference = false;
					}

					if(bAddReference)
					{
						AddParameterReference(Parameter, WriteEvent.Pin);
					}
					
					HandledParameterMapPins.Add(WriteEvent.Pin);
				}
			}

			for (const FNiagaraParameterMapHistory::FReadHistory& ReadHistory : History.PerVariableReadHistory[Index])
			{
				if (ReadHistory.ReadPin.Pin->PinType.PinSubCategory == UNiagaraNodeParameterMapBase::ParameterPinSubCategory)
				{
					AddParameterReference(Parameter, ReadHistory.ReadPin.Pin);					
					HandledParameterMapPins.Add(ReadHistory.ReadPin.Pin);
				}
			}
		}
	}

	// Check all pins on all nodes in the graph to find parameter pins which may have been missed in the parameter map traversal.  This
	// can happen for nodes which are not fully connected and therefore don't show up in the traversal.
	const UEdGraphSchema_Niagara* NiagaraSchema = GetNiagaraSchema();
	for (UEdGraphNode* Node : Nodes)
	{
		if (UNiagaraNodeStaticSwitch* SwitchNode = Cast<UNiagaraNodeStaticSwitch>(Node))
		{
			if (!SwitchNode->IsSetByCompiler() && !SwitchNode->IsSetByPin())
			{
				FNiagaraVariable Variable(SwitchNode->GetInputType(), SwitchNode->InputParameterName);
				AddStaticParameterReference(Variable, SwitchNode);
			}
		}
		else if (UNiagaraNodeFunctionCall* FunctionNode = Cast<UNiagaraNodeFunctionCall>(Node))
		{
			for (const FNiagaraPropagatedVariable& Propagated : FunctionNode->PropagatedStaticSwitchParameters)
			{
				AddStaticParameterReference(Propagated.ToVariable(), FunctionNode);
			}
		}
		else if (UNiagaraNodeParameterMapBase* ParameterMapNode = Cast<UNiagaraNodeParameterMapBase>(Node))
		{
			for (UEdGraphPin* Pin : Node->Pins)
			{
				// we only consider parameter pins, inputs for MapSets and outputs for MapGets
				if (Pin->PinType.PinSubCategory == UNiagaraNodeParameterMapBase::ParameterPinSubCategory)
				{
					const bool bRelevantPin = ((Pin->Direction == EEdGraphPinDirection::EGPD_Input) && Node->IsA<UNiagaraNodeParameterMapSet>())
						|| ((Pin->Direction == EEdGraphPinDirection::EGPD_Output) && Node->IsA<UNiagaraNodeParameterMapGet>());

					if (!bRelevantPin)
					{
						continue;
					}

					bool bAlreadyHandledPin = false;
					HandledParameterMapPins.FindOrAdd(Pin, &bAlreadyHandledPin);

					if (!bAlreadyHandledPin)
					{
						const FNiagaraVariable Parameter = NiagaraSchema->PinToNiagaraVariable(Pin, false);
						AddParameterReference(Parameter, Pin);
					}
				}
			}
		}
	}

	// Add reference to all variables that are default bound to
	for (auto It = VariableToScriptVariable.CreateConstIterator(); It; ++It)
	{
		UNiagaraScriptVariable* Variable = It.Value();
		if (!Variable || (Variable->DefaultMode != ENiagaraDefaultMode::Binding || !Variable->DefaultBinding.IsValid()))
		{
			continue;
		}

		FNiagaraTypeDefinition LinkedType = Variable->Variable.GetType();
		FName BindingName = Variable->DefaultBinding.GetName();
		if (FNiagaraConstants::GetOldPositionTypeVariables().Contains(FNiagaraVariable(LinkedType, BindingName)))
		{
			// it is not uncommon that old assets have vector inputs that default bind to what is now a position type. If we detect that, we change the type to prevent a compiler error.
			LinkedType = FNiagaraTypeDefinition::GetPositionDef();
		}
		AddBindingParameterReference(FNiagaraVariable(LinkedType, BindingName));
	}

	// If there were any previous parameters which didn't have any references added, remove them here.
	for (const FNiagaraVariable& UnreferencedParameterToRemove : CandidateUnreferencedParametersToRemove)
	{
		ParameterToReferencesMap.Remove(UnreferencedParameterToRemove);
	}

	bParameterReferenceRefreshPending = false;
}

void UNiagaraGraph::InvalidateCachedParameterData()
{
	bParameterReferenceRefreshPending = true;
}

const TMap<FNiagaraVariable, FInputPinsAndOutputPins> UNiagaraGraph::CollectVarsToInOutPinsMap() const
{
	const UEdGraphSchema_Niagara* NiagaraSchema = GetNiagaraSchema();
	TMap<FNiagaraVariable, FInputPinsAndOutputPins> VarToPinsMap;

	// Collect all input and output nodes to inspect the pins and infer usages of variables they reference.
	TArray<UNiagaraNodeParameterMapSet*> MapSetNodes;
	TArray<UNiagaraNodeParameterMapGet*> MapGetNodes;
	GetNodesOfClass<UNiagaraNodeParameterMapSet>(MapSetNodes);
	GetNodesOfClass<UNiagaraNodeParameterMapGet>(MapGetNodes);

	FPinCollectorArray MapGetOutputPins;
	for (UNiagaraNodeParameterMapGet* MapGetNode : MapGetNodes)
	{
		MapGetOutputPins.Reset();
		MapGetNode->GetOutputPins(MapGetOutputPins);
		for (UEdGraphPin* Pin : MapGetOutputPins)
		{
			if (Pin->PinName == UNiagaraNodeParameterMapBase::SourcePinName || Pin->PinName == UNiagaraNodeParameterMapBase::AddPinName)
			{
				continue;
			}

			FNiagaraVariable Var = FNiagaraVariable(NiagaraSchema->PinToTypeDefinition(Pin), Pin->PinName);
			FInputPinsAndOutputPins& InOutPins = VarToPinsMap.FindOrAdd(Var);
			InOutPins.OutputPins.Add(Pin);
		}
	}

	FPinCollectorArray MapSetInputPins;
	for (UNiagaraNodeParameterMapSet* MapSetNode : MapSetNodes)
	{
		MapSetInputPins.Reset();
		MapSetNode->GetInputPins(MapSetInputPins);
		for (UEdGraphPin* Pin : MapSetInputPins)
		{
			if (Pin->PinName == UNiagaraNodeParameterMapBase::SourcePinName || Pin->PinName == UNiagaraNodeParameterMapBase::AddPinName)
			{
				continue;
			}

			FNiagaraVariable Var = FNiagaraVariable(NiagaraSchema->PinToTypeDefinition(Pin), Pin->PinName);
			FInputPinsAndOutputPins& InOutPins = VarToPinsMap.FindOrAdd(Var);
			InOutPins.InputPins.Add(Pin);
		}
	}
	return VarToPinsMap;
}

void UNiagaraGraph::GetAllScriptVariableGuids(TArray<FGuid>& VariableGuids) const
{
	if (bIsForCompilationOnly)
	{
		VariableGuids.Reserve(CompilationScriptVariables.Num());
		Algo::Transform(CompilationScriptVariables, VariableGuids, [&](const FNiagaraScriptVariableData& VariableData) { return VariableData.Metadata.GetVariableGuid(); });
		return;
	}

	for (auto It = VariableToScriptVariable.CreateConstIterator(); It; ++It)
	{
		VariableGuids.Add(It.Value()->Metadata.GetVariableGuid());
	}
}

void UNiagaraGraph::GetAllVariables(TArray<FNiagaraVariable>& Variables) const
{
	if (bIsForCompilationOnly)
	{
		Variables.Reserve(CompilationScriptVariables.Num());
		Algo::Transform(CompilationScriptVariables, Variables, [&](const FNiagaraScriptVariableData& VariableData) { return VariableData.Variable; });
		return;
	}

	VariableToScriptVariable.GetKeys(Variables);
}

bool UNiagaraGraph::HasVariable(const FNiagaraVariable& Variable) const
{
	if (TOptional<FNiagaraScriptVariableData> VariableData = GetScriptVariableData(Variable))
	{
		return true;
	}

	return VariableToScriptVariable.Contains(Variable);
}

TOptional<bool> UNiagaraGraph::IsStaticSwitch(const FNiagaraVariable& Variable) const
{
	if (TOptional<FNiagaraScriptVariableData> VariableData = GetScriptVariableData(Variable))
	{
		return VariableData->GetIsStaticSwitch();
	}

	if (const UNiagaraScriptVariable* FoundScriptVariable = VariableToScriptVariable.FindRef(Variable))
	{
		return FoundScriptVariable->GetIsStaticSwitch();
	}

	return TOptional<bool>();
}

TOptional<int> UNiagaraGraph::GetStaticSwitchDefaultValue(const FNiagaraVariable& Variable) const
{
	if (TOptional<FNiagaraScriptVariableData> VariableData = GetScriptVariableData(Variable))
	{
		return VariableData->GetStaticSwitchDefaultValue();
	}

	if (const UNiagaraScriptVariable* FoundScriptVariable = VariableToScriptVariable.FindRef(Variable))
	{
		return FoundScriptVariable->GetStaticSwitchDefaultValue();
	}

	return TOptional<int32>();
}

TOptional<ENiagaraDefaultMode> UNiagaraGraph::GetDefaultMode(const FNiagaraVariable& Variable, FNiagaraScriptVariableBinding* Binding) const
{
	if (TOptional<FNiagaraScriptVariableData> VariableData = GetScriptVariableData(Variable))
	{
		if (VariableData->DefaultMode == ENiagaraDefaultMode::Binding && VariableData->DefaultBinding.IsValid())
		{
			if (Binding)
			{
				*Binding = VariableData->DefaultBinding;
			}
			return { ENiagaraDefaultMode::Binding };
		}
		else
		{
			return VariableData->DefaultMode;
		}
	}

	if (const UNiagaraScriptVariable* FoundScriptVariable = VariableToScriptVariable.FindRef(Variable))
	{
		if (FoundScriptVariable->DefaultMode == ENiagaraDefaultMode::Binding && FoundScriptVariable->DefaultBinding.IsValid())
		{
			if (Binding)
			{
				*Binding = FoundScriptVariable->DefaultBinding;
			}
			return {ENiagaraDefaultMode::Binding};
		}
		else
		{
			return FoundScriptVariable->DefaultMode;
		}
	}

	return TOptional<ENiagaraDefaultMode>();
}

TOptional<FGuid> UNiagaraGraph::GetScriptVariableGuid(const FNiagaraVariable& Variable) const
{
	if (TOptional<FNiagaraScriptVariableData> VariableData = GetScriptVariableData(Variable))
	{
		return VariableData->Metadata.GetVariableGuid();
	}

	if (const UNiagaraScriptVariable* FoundScriptVariable = VariableToScriptVariable.FindRef(Variable))
	{
		return FoundScriptVariable->Metadata.GetVariableGuid();
	}

	return TOptional<FGuid>();
}

TOptional<FNiagaraVariable> UNiagaraGraph::GetVariable(const FNiagaraVariable& Variable) const
{
	if (TOptional<FNiagaraScriptVariableData> VariableData = GetScriptVariableData(Variable))
	{
		return VariableData->Variable;
	}

	if (const UNiagaraScriptVariable* FoundScriptVariable = VariableToScriptVariable.FindRef(Variable))
	{
		return FoundScriptVariable->Variable;
	}

	return TOptional<FNiagaraVariable>();
}

void UNiagaraGraph::SetIsStaticSwitch(const FNiagaraVariable& Variable, bool InValue)
{
	check(!bIsForCompilationOnly);

	if (TObjectPtr<UNiagaraScriptVariable>* FoundScriptVariable = VariableToScriptVariable.Find(Variable))
	{
		(*FoundScriptVariable)->SetIsStaticSwitch(InValue);
	}
}

bool UNiagaraGraph::ReferencesStaticVariable(FNiagaraStaticVariableSearchContext& SearchContext) const
{
	FNiagaraStaticVariableSearchContext::FGraphKey GraphKey(this, ChangeId);
	if (const bool* bHasReferencePtr = SearchContext.CachedResults.Find(GraphKey))
	{
		return *bHasReferencePtr;
	}

	for (const auto& ScriptVariableEntry : VariableToScriptVariable)
	{
		if (ScriptVariableEntry.Key.GetType().IsStatic())
		{
			SearchContext.CachedResults.Add(GraphKey, true);
			return true;
		}
	}

	if (SearchContext.bIncludeReferencedGraphs)
	{
		for (const UEdGraphNode* GraphNode : Nodes)
		{
			if (const UNiagaraNodeFunctionCall* FunctionCall = Cast<const UNiagaraNodeFunctionCall>(GraphNode))
			{
				if (const UNiagaraGraph* CalledGraph = FunctionCall->GetCalledGraph())
				{
					if (CalledGraph->ReferencesStaticVariable(SearchContext))
					{
						SearchContext.CachedResults.Add(GraphKey, true);
						return true;
					}
				}
			}
		}
	}

	SearchContext.CachedResults.Add(GraphKey, false);
	return false;
}

/** Migrates the deprecated data from the provided OwnerData and populates the ParameterHierarchyRoot */
void UNiagaraGraph::MigrateParameterScriptDataToHierarchyRoot(FVersionedNiagaraScriptData& OwnerData)
{
	UHierarchyRoot* ParameterRoot = GetScriptParameterHierarchyRoot();

	TMap<FName, UHierarchySection*> HierarchySections;
	TMap<FName, UHierarchyCategory*> IdentityCategoryMap;
	TMap<FName, FName> SectionCategoryMapping;
		
	// First we transfer all sections and create contained categories
	for(const FNiagaraStackSection& StackSection : OwnerData.InputSections_DEPRECATED)
	{
		FText SectionDisplayName = StackSection.SectionDisplayName.IsEmptyOrWhitespace() ? FText::FromName(StackSection.SectionIdentifier) : StackSection.SectionDisplayName;
		UHierarchySection* HierarchySection = NewObject<UHierarchySection>(ParameterRoot, UHierarchySection::StaticClass());
		HierarchySection->SetSectionNameAsText(SectionDisplayName);
		HierarchySection->SetTooltip(StackSection.Tooltip);

		FHierarchyElementIdentity SectionIdentity;
		SectionIdentity.Names.Add(StackSection.SectionIdentifier);
		HierarchySection->SetIdentity(SectionIdentity);
			
		ParameterRoot->GetSectionDataMutable().Insert(HierarchySection, 0);
		
		HierarchySections.Add(StackSection.SectionIdentifier, HierarchySection);

		for(const FText& CategoryIdentity : StackSection.Categories)
		{
			// We skip categories that don't have a proper identifier setup
			if(CategoryIdentity.IsEmptyOrWhitespace())
			{
				continue;
			}
				
			FName CategoryName = FName(CategoryIdentity.ToString());
			UNiagaraHierarchyScriptCategory* HierarchyCategory = ParameterRoot->AddChild<UNiagaraHierarchyScriptCategory>();
			HierarchyCategory->SetCategoryName(CategoryName);
			
			FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchyCategory->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
			SectionAssociation->Section = HierarchySection;
			
			IdentityCategoryMap.Add(CategoryName, HierarchyCategory);
			SectionCategoryMapping.Add(CategoryName, HierarchySection->GetSectionName());
		}
	}

	OwnerData.InputSections_DEPRECATED.Empty();
		
	TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>> ScriptVariablesMap = GetAllMetaData();

	TArray<UNiagaraScriptVariable*> ParentLevelInputs;
	TMap<UNiagaraScriptVariable*, TArray<UNiagaraScriptVariable*>> ParentChildInputMap;
	// We determine all parent & child level module inputs & static switches
	for(const auto& ScriptVariablePair : ScriptVariablesMap)
	{
		if(ScriptVariablePair.Key.IsInNameSpace(FNiagaraConstants::ModuleNamespace) == false && ScriptVariablePair.Value->GetIsStaticSwitch() == false)
		{
			continue;
		}
			
		UNiagaraScriptVariable* ScriptVariable = ScriptVariablePair.Value;

		if(ScriptVariable->Metadata.GetParentAttribute_DEPRECATED().IsNone())
		{
			ParentLevelInputs.Add(ScriptVariable);
			ParentChildInputMap.Add(ScriptVariable, GetChildScriptVariablesForInput_Deprecated(ScriptVariable->Metadata.GetVariableGuid()));
		}
	}

	// Now we create the actual hierarchy elements
	for(const auto& ParentChildPair : ParentChildInputMap)
	{
		// By default, we add to the root directly, unless a category was specified
		UHierarchyElement* Owner = ParameterRoot;
			
		FText SpecifiedCategoryName = ParentChildPair.Key->Metadata.GetCategoryName_DEPRECATED();
		if(SpecifiedCategoryName.IsEmptyOrWhitespace() == false)
		{
			// Categories might already exist due to section info
			if(IdentityCategoryMap.Contains(FName(SpecifiedCategoryName.ToString())))
			{
				Owner = IdentityCategoryMap[FName(SpecifiedCategoryName.ToString())];
			}
			// But if not we create a new category
			else
			{
				UNiagaraHierarchyScriptCategory* Category = ParameterRoot->AddChild<UNiagaraHierarchyScriptCategory>();
				Category->SetCategoryName(FName(SpecifiedCategoryName.ToString()));
				IdentityCategoryMap.Add(Category->GetCategoryName(), Category);

				Owner = Category;
			}
		}

		// We add the parent input to either the root, or a given category
		UNiagaraHierarchyScriptParameter* ParentHierarchyInput = Owner->AddChild<UNiagaraHierarchyScriptParameter>();
		ParentHierarchyInput->Initialize(*ParentChildPair.Key);

		// Children inputs can be added directly to the parent input and are already sorted
		for(UNiagaraScriptVariable* ChildInput : ParentChildPair.Value)
		{
			UNiagaraHierarchyScriptParameter* ChildHierarchyInput = ParentHierarchyInput->AddChild<UNiagaraHierarchyScriptParameter>();
			ChildHierarchyInput->Initialize(*ChildInput);
		}
	}

	// now that we have created all inputs, categories and sections, we fix up the order
	TArray<UHierarchyCategory*> TopLevelCategories;
	ParameterRoot->GetChildrenOfType(TopLevelCategories, false);
		
	// inputs are generally sorted by editor sort priority, but within a given context (within a category etc.)
	// child inputs also already are sorted when constructed
	auto SortInputsPredicate = ([&](UHierarchyElement& ItemA, UHierarchyElement& ItemB)
	{
		UNiagaraHierarchyScriptParameter* ScriptParameterA = Cast<UNiagaraHierarchyScriptParameter>(&ItemA);
		UNiagaraHierarchyScriptParameter* ScriptParameterB = Cast<UNiagaraHierarchyScriptParameter>(&ItemB);
		
		if(ScriptParameterA == nullptr || ScriptParameterB == nullptr)
		{
			return false;
		}
		
		UNiagaraScriptVariable* ScriptVariableA = ScriptParameterA->GetScriptVariable();
		UNiagaraScriptVariable* ScriptVariableB = ScriptParameterB->GetScriptVariable();

		if(ScriptVariableA == nullptr || ScriptVariableB == nullptr)
		{
			return false;
		}

		const FNiagaraVariableMetaData& MetaDataA = ScriptVariableA->Metadata;
		const FNiagaraVariableMetaData& MetaDataB = ScriptVariableB->Metadata;

		if(MetaDataA.bAdvancedDisplay && !MetaDataB.bAdvancedDisplay)
		{
			return false;
		}
		else if(!MetaDataA.bAdvancedDisplay && MetaDataB.bAdvancedDisplay)
		{
			return true;
		}
			
		if(MetaDataA.GetEditorSortPriority_DEPRECATED() != MetaDataB.GetEditorSortPriority_DEPRECATED())
		{
			return MetaDataA.GetEditorSortPriority_DEPRECATED() < MetaDataB.GetEditorSortPriority_DEPRECATED();
		}
			
		return ItemA.ToString().Compare(ItemB.ToString(), ESearchCase::Type::IgnoreCase) < 0;
	});

	// We now sort inputs, and keep track of the minimum sort order per category
	// This is because the stack determined category order by input sort priority (the category with the parameter of the lowest priority value would come first)
	TMap<UHierarchyElement*, int32> CategoryMinimumInputSortOrderMap;
	for(UHierarchyCategory* Category : TopLevelCategories)
	{
		Category->SortChildren(SortInputsPredicate, false);
		if(Category->GetChildren().Num() > 0)
		{
			// since we have sorted the children by sort priority, child 0 has the lowest sort priority number (= highest priority)
			UNiagaraHierarchyScriptParameter* MinimumItem = CastChecked<UNiagaraHierarchyScriptParameter>(Category->GetChildren()[0]);
			int32 MinimumCategorySortOrder = MinimumItem->GetScriptVariable()->Metadata.GetEditorSortPriority_DEPRECATED();
			
			CategoryMinimumInputSortOrderMap.Add(Category, MinimumCategorySortOrder);
		}
		else
		{
			CategoryMinimumInputSortOrderMap.Add(Category, 0);
		}
	}

	auto SortRootChildrenPredicate = ([&](UHierarchyElement& ItemA, UHierarchyElement& ItemB)
	{
		if(ItemA.IsA<UHierarchyCategory>() && ItemB.IsA<UHierarchyCategory>())
		{
			if(CategoryMinimumInputSortOrderMap[&ItemA] != CategoryMinimumInputSortOrderMap[&ItemB])
			{
				return CategoryMinimumInputSortOrderMap[&ItemA] < CategoryMinimumInputSortOrderMap[&ItemB];
			}

			return ItemA.ToString().Compare(ItemB.ToString(), ESearchCase::Type::IgnoreCase) < 0;
		}

		// Categories always come after items to ensure consistency with old behavior
		if(ItemA.IsA<UHierarchyCategory>() && ItemB.IsA<UHierarchyItem>())
		{
			return false;
		}
			
		if(ItemB.IsA<UHierarchyCategory>() && ItemA.IsA<UHierarchyItem>())
		{
			return true;
		}
			
		if(ItemA.IsA<UNiagaraHierarchyScriptParameter>() && ItemB.IsA<UNiagaraHierarchyScriptParameter>())
		{
			UNiagaraHierarchyScriptParameter* ScriptParameterA = Cast<UNiagaraHierarchyScriptParameter>(&ItemA);
			UNiagaraHierarchyScriptParameter* ScriptParameterB = Cast<UNiagaraHierarchyScriptParameter>(&ItemB);
			
			if(ScriptParameterA == nullptr || ScriptParameterB == nullptr)
			{
				return false;
			}
			
			UNiagaraScriptVariable* ScriptVariableA = ScriptParameterA->GetScriptVariable();
			UNiagaraScriptVariable* ScriptVariableB = ScriptParameterB->GetScriptVariable();

			if(ScriptVariableA == nullptr || ScriptVariableB == nullptr)
			{
				return false;
			}
			
			const FNiagaraVariableMetaData& MetaDataA = ScriptVariableA->Metadata;
			const FNiagaraVariableMetaData& MetaDataB = ScriptVariableB->Metadata;

			if(MetaDataA.bAdvancedDisplay && !MetaDataB.bAdvancedDisplay)
			{
				return false;
			}
			else if(!MetaDataA.bAdvancedDisplay && MetaDataB.bAdvancedDisplay)
			{
				return true;
			}
			
			if(MetaDataA.GetEditorSortPriority_DEPRECATED() != MetaDataB.GetEditorSortPriority_DEPRECATED())
			{
				return MetaDataA.GetEditorSortPriority_DEPRECATED() < MetaDataB.GetEditorSortPriority_DEPRECATED();
			}
			
			return ItemA.ToString().Compare(ItemB.ToString(), ESearchCase::Type::IgnoreCase) < 0;
		}
			
		return false;
	});

	ParameterRoot->SortChildren(SortRootChildrenPredicate, false);
}

#undef NIAGARA_SCOPE_CYCLE_COUNTER
#undef LOCTEXT_NAMESPACE
