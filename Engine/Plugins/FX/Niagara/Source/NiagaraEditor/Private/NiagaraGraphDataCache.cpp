// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGraphDataCache.h"

#include "EdGraphSchema_Niagara.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeAssignment.h"

static int32 GNiagaraGraphDataCacheSize = 16384;
static FAutoConsoleVariableRef CVarNiagaraGraphDataCacheSize(
	TEXT("fx.Niagara.GraphDataCacheSize"),
	GNiagaraGraphDataCacheSize,
	TEXT("Maximum number of elements to store within the GraphDataCache."),
	ECVF_ReadOnly
);

static bool GNiagaraGraphDataCacheValidation = false;
static FAutoConsoleVariableRef CVarNiagaraGraphDataCacheValidation(
	TEXT("fx.Niagara.GraphDataCacheValidation"),
	GNiagaraGraphDataCacheValidation,
	TEXT("If true will perform validation on retrieving data from the data FNiagaraGraphDataCache."),
	ECVF_Default
);

FNiagaraGraphDataCache::FNiagaraGraphDataCache()
	: StackFunctionInputPinCache(GNiagaraGraphDataCacheSize)
{
}

void FNiagaraGraphDataCache::GetStackFunctionInputsInternal(
	const UNiagaraNodeFunctionCall& FunctionCallNode,
	const UNiagaraGraph* CalledGraph,
	TConstArrayView<FNiagaraVariable> StaticVars,
	TArray<FNiagaraVariable>& OutInputVariables,
	FCompileConstantResolver ConstantResolver,
	FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options,
	bool bIgnoreDisabled,
	bool bFilterForCompilation)
{
	// we don't currently support hashing the internal details of the translator within FStackFunctionInputPinKey so in the case
	// we've been supplied a translator we'll just fall back to not using the cache
	TArray<const UEdGraphPin*> InputPins;
	if (const FNiagaraHlslTranslator* Translator = ConstantResolver.GetTranslator())
	{
		FNiagaraStackGraphUtilities::GetStackFunctionInputPinsWithoutCache(
			FunctionCallNode,
			StaticVars,
			InputPins,
			ConstantResolver,
			Options,
			bIgnoreDisabled,
			bFilterForCompilation);

		for (const UEdGraphPin* InputPin : InputPins)
		{
			OutInputVariables.Add(UEdGraphSchema_Niagara::PinToNiagaraVariable(InputPin));
		}

		return;
	}

	bool bUseCache = true;
	if(FunctionCallNode.IsA<UNiagaraNodeAssignment>())
	{
		bUseCache = false;
	}

	FStackFunctionInputPinKey CacheKey(FunctionCallNode, CalledGraph, StaticVars, ConstantResolver, Options, bIgnoreDisabled, bFilterForCompilation);

	bool bFoundInCache = false;
	if(bUseCache)
	{
		if (const FStackFunctionInputPinValue* CachedValue = StackFunctionInputPinCache.FindAndTouch(CacheKey))
		{
			bFoundInCache = true;
			OutInputVariables = CachedValue->InputVariables;
		}
	}

	if(bUseCache == false || bFoundInCache == false)
	{
		FNiagaraStackGraphUtilities::GetStackFunctionInputPinsWithoutCache(
			FunctionCallNode,
			StaticVars,
			InputPins,
			ConstantResolver,
			Options,
			bIgnoreDisabled,
			bFilterForCompilation);

		for(const UEdGraphPin* InputPin : InputPins)
		{
			OutInputVariables.Add(UEdGraphSchema_Niagara::PinToNiagaraVariable(InputPin));
		}

		if(bUseCache)
		{
			FStackFunctionInputPinValue NewValue;
			NewValue.InputVariables = OutInputVariables;
			StackFunctionInputPinCache.Add(CacheKey, NewValue);
		}
	}

	if (GNiagaraGraphDataCacheValidation)
	{
		TArray<const UEdGraphPin*> ValidationPins;
		FNiagaraStackGraphUtilities::GetStackFunctionInputPinsWithoutCache(
			FunctionCallNode,
			StaticVars,
			ValidationPins,
			ConstantResolver,
			Options,
			bIgnoreDisabled,
			bFilterForCompilation);

		TArray<FNiagaraVariable> ValidationVariables;
		for (const UEdGraphPin* ValidationPin : ValidationPins)
		{
			ValidationVariables.Add(UEdGraphSchema_Niagara::PinToNiagaraVariable(ValidationPin));
		}

		if (!ensureMsgf(ValidationVariables == OutInputVariables, TEXT("FNiagaraGraphDataCache failed to accurately collect StackFunctionInputPins for Function %s"), *FunctionCallNode.GetPathName()))
		{
			OutInputVariables = ValidationVariables;
		}
	}
}

void FNiagaraGraphDataCache::GetStackFunctionInputs(const UNiagaraNodeFunctionCall& FunctionCallNode, TConstArrayView<FNiagaraVariable> StaticVars, TArray<FNiagaraVariable>& OutInputVariables, FCompileConstantResolver ConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled)
{
	if (const UNiagaraGraph* CalledGraph = FunctionCallNode.GetCalledGraph())
	{
		GetStackFunctionInputsInternal(FunctionCallNode, CalledGraph, StaticVars, OutInputVariables, ConstantResolver, Options, bIgnoreDisabled, false /*bFilterForCompilation*/);
	}
}

void FNiagaraGraphDataCache::GetStackFunctionInputs(const UNiagaraNodeFunctionCall& FunctionCallNode, TConstArrayView<FNiagaraVariable> StaticVars, TArray<FNiagaraVariable>& OutInputVariables, TSet<FNiagaraVariable>& OutHiddenVariables, FCompileConstantResolver ConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled)
{
	if (const UNiagaraGraph* CalledGraph = FunctionCallNode.GetCalledGraph())
	{
		TArray<FNiagaraVariable> FilteredVariables;

		GetStackFunctionInputsInternal(FunctionCallNode, CalledGraph, StaticVars, OutInputVariables, ConstantResolver, Options, bIgnoreDisabled, false /*bFilterForCompilation*/);
		GetStackFunctionInputsInternal(FunctionCallNode, CalledGraph, StaticVars, FilteredVariables, ConstantResolver, Options, bIgnoreDisabled, true /*bFilterForCompilation*/);

		for (const FNiagaraVariable& InputVariable : OutInputVariables)
		{
			if (FilteredVariables.Contains(InputVariable) == false)
			{
				OutHiddenVariables.Add(InputVariable);
			}
		}
	}
}

void FNiagaraGraphDataCache::InvalidateCache()
{
	StackFunctionInputPinCache.Empty(GNiagaraGraphDataCacheSize);
}

FNiagaraGraphDataCache::FStackFunctionInputPinKey::FStackFunctionInputPinKey(const UNiagaraNodeFunctionCall& FunctionCallNode, const UNiagaraGraph* CalledGraph, TConstArrayView<FNiagaraVariable> StaticVariables, const FCompileConstantResolver& InConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions InOptions, bool bInIgnoreDisabled, bool bInFilterForCompilation)
{
	if (ensure(CalledGraph))
	{
		FunctionCallGraphKey = FObjectKey(CalledGraph);
		FunctionCallGraphChangeId = CalledGraph->GetChangeID();
		ContextStaticVariables = StaticVariables;
	}

	InputPinOption = (uint8)InOptions;
	bIgnoreDisabled = bInIgnoreDisabled;
	bFilterForCompilation = bInFilterForCompilation;
	ConstantResolverHash = InConstantResolver.BuildTypeHash();

	const FString& FunctionName = FunctionCallNode.GetFunctionName();

	Hash = GetTypeHash(FunctionCallGraphKey);
	Hash = HashCombine(Hash, GetTypeHash(FunctionCallGraphChangeId));

	for (const FNiagaraVariable& StaticVariable : ContextStaticVariables)
	{
		Hash = HashCombine(Hash, GetTypeHash(StaticVariable));
		const uint8* StaticVariableData = StaticVariable.GetData();
		const int32 ByteCount = StaticVariable.GetAllocatedSizeInBytes();
		for (int32 ByteIt = 0; ByteIt < ByteCount; ++ByteIt)
		{
			Hash = HashCombine(Hash, GetTypeHash(StaticVariableData[ByteIt]));
		}

	}

	/* We double-hash specifically anything that matches the namespace. However we don't hash the name, just the type
	  and the value. This way we can still re-use across different instances of the same graph.*/
	for (const FNiagaraVariable& StaticVariable : ContextStaticVariables)
	{
		if (StaticVariable.IsInNameSpace(FunctionName))
		{
			Hash = HashCombine(Hash, GetTypeHash(StaticVariable));
			const uint8* StaticVariableData = StaticVariable.GetData();
			const int32 ByteCount = StaticVariable.GetAllocatedSizeInBytes();
			for (int32 ByteIt = 0; ByteIt < ByteCount; ++ByteIt)
			{
				Hash = HashCombine(Hash, GetTypeHash(StaticVariableData[ByteIt]));
			}
		}
	}
	Hash = HashCombine(Hash, ConstantResolverHash);
	Hash = HashCombine(Hash, InputPinOption);
	Hash = HashCombine(Hash, bIgnoreDisabled);
	Hash = HashCombine(Hash, bFilterForCompilation);

	// the default values of the input pins can be relevant when it comes to static switch values defined within the graph
	FPinCollectorArray InputPins;
	FunctionCallNode.GetInputPins(InputPins);
	for (const UEdGraphPin* InputPin : InputPins)
	{
		Hash = HashCombine(Hash, GetTypeHash(InputPin->PinName));
		Hash = HashCombine(Hash, GetTypeHash(InputPin->DefaultValue));
	}
}

bool FNiagaraGraphDataCache::FStackFunctionInputPinKey::operator==(const FStackFunctionInputPinKey& Other) const
{
	return FunctionCallGraphKey == Other.FunctionCallGraphKey
		&& FunctionCallGraphChangeId == Other.FunctionCallGraphChangeId
		&& ContextStaticVariables == Other.ContextStaticVariables
		&& ConstantResolverHash == Other.ConstantResolverHash
		&& InputPinOption == Other.InputPinOption
		&& bIgnoreDisabled == Other.bIgnoreDisabled
		&& bFilterForCompilation == Other.bFilterForCompilation;
}
