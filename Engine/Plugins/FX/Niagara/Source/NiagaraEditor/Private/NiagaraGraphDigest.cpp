// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraGraphDigest.h"

#include "Containers/MapBuilder.h"

#include "Internationalization/Internationalization.h"

#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraCompilationBridge.h"
#include "NiagaraCompilationPrivate.h"
#include "NiagaraDigestDatabase.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeConvert.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraNodeIf.h"
#include "NiagaraNodeOp.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeOutputTag.h"
#include "NiagaraNodeParameterMapFor.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeReadDataSet.h"
#include "NiagaraNodeReroute.h"
#include "NiagaraNodeSelect.h"
#include "NiagaraNodeSimTargetSelector.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraNodeWriteDataSet.h"
#include "NiagaraTraversalStateContext.h"
#include "NiagaraScriptSource.h"

#include "Algo/RemoveIf.h"

#define LOCTEXT_NAMESPACE "NiagaraCompiler"

#define NIAGARA_GRAPH_DIGEST_NODE_TYPE(name) NIAGARA_GRAPH_DIGEST_NODE_IMPLEMENT_BODY(name)

NIAGARA_GRAPH_DIGEST_NODE_TYPE_LIST;

#undef NIAGARA_GRAPH_DIGEST_NODE_TYPE

struct FNiagaraCompilationGraphCreateContext
{
	FNiagaraCompilationGraphCreateContext(FNiagaraCompilationGraphDigested& InParentGraph, TArray<const FNiagaraCompilationGraphDigested*>& InDigestedChildGraphs, const FNiagaraGraphChangeIdBuilder& InChangeIdBuilder)
		: ParentGraph(InParentGraph)
		, DigestedChildGraphs(InDigestedChildGraphs)
		, ChangeIdBuilder(InChangeIdBuilder)
	{
	}

	FNiagaraCompilationGraphDigested& ParentGraph;
	TArray<const FNiagaraCompilationGraphDigested*>& DigestedChildGraphs;
	const FNiagaraGraphChangeIdBuilder& ChangeIdBuilder;
};

struct FNiagaraCompilationGraphDuplicateContext
{
	FNiagaraCompilationGraphDuplicateContext(FNiagaraCompilationGraph& InTargetGraph, const FNiagaraCompilationGraph& InSourceGraph, const TArray<ENiagaraScriptUsage>& InScriptUsages, const FNiagaraCompilationCopyData* InCopyCompilationData, const FNiagaraCompilationBranchMap& InBranches)
		: TargetGraph(InTargetGraph)
		, SourceGraph(InSourceGraph)
		, ScriptUsages(InScriptUsages)
		, CopyCompilationData(InCopyCompilationData)
		, Branches(InBranches)
	{
	}

	FNiagaraCompilationGraph& TargetGraph;
	const FNiagaraCompilationGraph& SourceGraph;
	const TArray<ENiagaraScriptUsage>& ScriptUsages;
	const FNiagaraCompilationCopyData* CopyCompilationData;
	const FNiagaraCompilationBranchMap& Branches;

	TMap<const FNiagaraCompilationNode*, FNiagaraCompilationNode*> DuplicatedNodeMap;
	TArray<FNiagaraCompilationNodeFunctionCall*> FunctionsRequiresGraph;
};

struct FNiagaraCompilationGraphInstanceContext
{
	FNiagaraCompilationGraphInstanceContext(const FNiagaraFixedConstantResolver& InConstantResolver, const FNiagaraCompilationGraph* InCompilationGraph, const FNiagaraPrecompileData* InPrecompileData)
		: ConstantResolver(InConstantResolver)
		, PrecompileData(InPrecompileData)
	{
		bDisableDebugSwitches = InPrecompileData ? InPrecompileData->bDisableDebugSwitches : false;
		TraversalContext.BeginContext(InCompilationGraph, ConstantResolver);
	}
	FNiagaraCompilationGraphInstanceContext() = delete;

	const FNiagaraFixedConstantResolver& ConstantResolver;
	const FNiagaraPrecompileData* PrecompileData;
	FNiagaraTraversalStateContext TraversalContext;
	FGraphTraversalHandle GraphTraversalHandle;
	TArray<const FNiagaraCompilationNodeFunctionCall*> FunctionStack;
	bool bForceNumericResolution = false;
	bool bDisableDebugSwitches = false;

	void EnterFunction(const FNiagaraCompilationNodeFunctionCall* InCallingNode)
	{
		TraversalContext.PushFunction(InCallingNode, ConstantResolver);
		GraphTraversalHandle.PushNode(InCallingNode);
		FunctionStack.Push(InCallingNode);
	}

	void LeaveFunction(const FNiagaraCompilationNodeFunctionCall* FunctionBeingLeft)
	{
		const FNiagaraCompilationNodeFunctionCall* CurrentFunction = FunctionStack.Top();
		check(CurrentFunction == FunctionBeingLeft);

		TraversalContext.PopFunction(CurrentFunction);
		GraphTraversalHandle.PopNode();
		FunctionStack.Pop();
	}

	const FNiagaraCompilationNodeFunctionCall* GetCurrentFunctionNode() const
	{
		return FunctionStack.Top();
	}
};

namespace NiagaraCompilationImpl
{

DECLARE_DELEGATE_RetVal_TwoParams(TUniquePtr<FNiagaraCompilationNode>, FCreateNodeDelegate, const UEdGraphNode*, FNiagaraCompilationGraphCreateContext&);
DECLARE_DELEGATE_RetVal_TwoParams(TUniquePtr<FNiagaraCompilationNode>, FDuplicateNodeDelegate, const FNiagaraCompilationNode*, FNiagaraCompilationGraphDuplicateContext&);

struct FNodeHelper
{
	FNodeHelper(FNiagaraCompilationNode::ENodeType InNodeType)
		: NodeType(InNodeType)
	{}

	FNodeHelper() = delete;

	FCreateNodeDelegate CreateNodeDelegate;
	FDuplicateNodeDelegate DuplicateNodeDelegate;
	const FNiagaraCompilationNode::ENodeType NodeType;
};

template<typename NodeClass, typename CompilationNodeClass>
FNodeHelper CreateNodeHelper(FNiagaraCompilationNode::ENodeType NodeType)
{
	FNodeHelper Helper(NodeType);
	Helper.CreateNodeDelegate.BindLambda
	(
		[](const UEdGraphNode* SourceNode, FNiagaraCompilationGraphCreateContext& NodeContext) -> TUniquePtr<FNiagaraCompilationNode>
		{
			if (const NodeClass* TypedSourceNode = static_cast<const NodeClass*>(SourceNode))
			{
				return MakeUnique<CompilationNodeClass>(TypedSourceNode, NodeContext);
			}

			return TUniquePtr<FNiagaraCompilationNode>();
		}
	);

	Helper.DuplicateNodeDelegate.BindLambda
	(
		[](const FNiagaraCompilationNode* SourceNode, FNiagaraCompilationGraphDuplicateContext& NodeContext) -> TUniquePtr<FNiagaraCompilationNode>
		{
			if (const CompilationNodeClass* TypedSourceNode = static_cast<const CompilationNodeClass*>(SourceNode))
			{
				return MakeUnique<CompilationNodeClass>(*TypedSourceNode, NodeContext);
			}
			return TUniquePtr<FNiagaraCompilationNode>();
		}
	);

	return Helper;
}

using FCreateNodeMap = TMap<UClass*, FNodeHelper>;

#define MAKE_COMPILATION_ENUM_NAME(name) FNiagaraCompilationNode::ENodeType::name
#define MAKE_SOURCE_NODE_CLASS_NAME(name) UNiagaraNode##name
#define MAKE_COMPILATION_CLASS_NAME(name) FNiagaraCompilationNode##name

#define NIAGARA_GRAPH_DIGEST_NODE_TYPE(name) \
	.Add(MAKE_SOURCE_NODE_CLASS_NAME(name)::StaticClass(), CreateNodeHelper<MAKE_SOURCE_NODE_CLASS_NAME(name), MAKE_COMPILATION_CLASS_NAME(name)>(MAKE_COMPILATION_ENUM_NAME(name)))

const FCreateNodeMap& GetCreateNodeMap()
{
	static FCreateNodeMap NodeMap;
	static FRWLock MapLock;

	{
		FReadScopeLock Lock(MapLock);

		if (!NodeMap.IsEmpty())
		{
			return NodeMap;
		}
	}

	{
		FWriteScopeLock Lock(MapLock);

		if (NodeMap.IsEmpty())
		{
			NodeMap = TMapBuilder<UClass*, FNodeHelper>()
				NIAGARA_GRAPH_DIGEST_NODE_TYPE_LIST;
		}
	}

	return NodeMap;
}

#undef NIAGARA_GRAPH_DIGEST_NODE_TYPE


TUniquePtr<FNiagaraCompilationNode> CreateNode(const UEdGraphNode* SourceNode, FNiagaraCompilationGraphCreateContext& NodeContext)
{
	if (const FNodeHelper* NodeHelper = GetCreateNodeMap().Find(SourceNode->GetClass()))
	{
		if (NodeHelper->CreateNodeDelegate.IsBound())
		{
			return NodeHelper->CreateNodeDelegate.Execute(SourceNode, NodeContext);
		}
	}

	return nullptr;
}

TUniquePtr<FNiagaraCompilationNode> DuplicateNode(const FNiagaraCompilationNode* SourceNode, FNiagaraCompilationGraphDuplicateContext& NodeContext)
{
	if (const FNodeHelper* NodeHelper = GetCreateNodeMap().Find(SourceNode->GetSourceNodeClass()))
	{
		if (NodeHelper->DuplicateNodeDelegate.IsBound())
		{
			return NodeHelper->DuplicateNodeDelegate.Execute(SourceNode, NodeContext);
		}
	}

	return nullptr;
}

const UEdGraphPin* FindInputExecutionPin(const UNiagaraNode* Node)
{
	if (Node)
	{
		for (const UEdGraphPin* Pin : Node->Pins)
		{
			if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
			{
				if (Node->IsParameterMapPin(Pin))
				{
					return Pin;
				}
			}
		}
	}
	return nullptr;
}

const UEdGraphPin* TraceThroughIgnoredNodes(const UEdGraphPin* Pin)
{
	check(Pin);
	check(Pin->Direction == EEdGraphPinDirection::EGPD_Output);

	while (Pin)
	{
		const UEdGraphNode* OwningNode = Pin->GetOwningNode();
		const UEdGraphPin* InputPin = nullptr;
		if (!OwningNode->IsNodeEnabled())
		{
			if (const UNiagaraNode* NiagaraNode = Cast<const UNiagaraNode>(OwningNode))
			{
				if (NiagaraNode->IsParameterMapPin(Pin))
				{
					InputPin = FindInputExecutionPin(NiagaraNode);
				}
				else
				{
					return nullptr;
				}
			}
		}
		else if (const UNiagaraNodeReroute* RerouteNode = Cast<const UNiagaraNodeReroute>(OwningNode))
		{
			InputPin = RerouteNode->GetInputPin(0);
		}
		else
		{
			break;
		}

		Pin = (InputPin && !InputPin->LinkedTo.IsEmpty()) ? InputPin->LinkedTo[0] : nullptr;
	}

	return Pin;
}

const FNiagaraCompilationOutputPin* TraceOutputPin(TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationDigestBridge>& Builder, const FNiagaraCompilationOutputPin* Pin, bool bFilterForCompilation)
{
	if (!bFilterForCompilation)
	{
		return Pin;
	}

	// we ignore reroute nodes because those have already been removed from the compilation copies
	// what is left is static switches
	if (const FNiagaraCompilationNodeStaticSwitch* StaticSwitchNode = Pin->OwningNode->AsType<FNiagaraCompilationNodeStaticSwitch>())
	{
		return StaticSwitchNode->TraceOutputPin(Builder, Pin, bFilterForCompilation);
	}

	return Pin;
}

TArray<const FNiagaraCompilationNode*> CollectConnectedNodes(TConstArrayView<const FNiagaraCompilationNode*> RootNodes, int32 MaxNodeCount, const FNiagaraCompilationBranchMap& BranchMap)
{
	TSet<const FNiagaraCompilationNode*> VisitedNodeSet;
	VisitedNodeSet.Reserve(MaxNodeCount);
	VisitedNodeSet.Append(RootNodes);

	TArray<const FNiagaraCompilationNode*> NodesToProcess;
	NodesToProcess.Reserve(MaxNodeCount);
	NodesToProcess.Append(RootNodes);

	while (!NodesToProcess.IsEmpty())
	{
		const FNiagaraCompilationNode* CurrentNode = NodesToProcess.Pop();
		for (const FNiagaraCompilationInputPin& InputPin : CurrentNode->InputPins)
		{
			if (const FNiagaraCompilationOutputPin* OutputPin = InputPin.TraceBranchMap(BranchMap)->LinkedTo)
			{
				bool bAlreadyAdded;
				VisitedNodeSet.Add(OutputPin->OwningNode, &bAlreadyAdded);
				if (!bAlreadyAdded)
				{
					NodesToProcess.Add(OutputPin->OwningNode);
				}
			}
		}
	}

	return VisitedNodeSet.Array();
}

TArray<const FNiagaraCompilationNode*> TopologicalSort(TConstArrayView<const FNiagaraCompilationNode*> Nodes, const FNiagaraCompilationBranchMap& BranchMap)
{
	TSet<const FNiagaraCompilationNode*> VisitedNodeSet;
	TArray<const FNiagaraCompilationNode*> SortedNodes;
	SortedNodes.Reserve(Nodes.Num());

	using FProcessingNode = TTuple<const FNiagaraCompilationNode*, bool /*IsParent*/>;
	TArray<FProcessingNode> NodesToProcess;

	for (const FNiagaraCompilationNode* Node : Nodes)
	{
		if (!VisitedNodeSet.Contains(Node))
		{
			NodesToProcess.Emplace(Node, false);
		}

		while (!NodesToProcess.IsEmpty())
		{
			FProcessingNode CurrentNode = NodesToProcess.Pop();
			if (CurrentNode.Value)
			{
				SortedNodes.Add(CurrentNode.Key);
			}
			else if (!VisitedNodeSet.Contains(CurrentNode.Key))
			{
				VisitedNodeSet.Add(CurrentNode.Key);
				NodesToProcess.Emplace(CurrentNode.Key, true);
				for (const FNiagaraCompilationInputPin& InputPin : CurrentNode.Key->InputPins)
				{
					if (const FNiagaraCompilationOutputPin* OutputPin = InputPin.TraceBranchMap(BranchMap)->LinkedTo)
					{
						const FNiagaraCompilationNode* TargetNode = OutputPin->OwningNode;
						if (!VisitedNodeSet.Contains(TargetNode))
						{
							NodesToProcess.Emplace(TargetNode, false);
						}
					}
				}
			}
		}
	}

	return MoveTemp(SortedNodes);
}

template<typename PinType>
PinType* FindPinByName(TArrayView<PinType> Pins, FName Name)
{
	return Pins.FindByPredicate([Name](const PinType& Pin) -> bool
	{
		return Pin.Variable.GetName() == Name;
	});
}

template<typename PinType>
int32 GetPinIndexById(TConstArrayView<PinType> Pins, const FGuid& PinId)
{
	return Pins.IndexOfByPredicate([&PinId](const FNiagaraCompilationPin& Pin) -> bool
	{
		return Pin.UniquePinId == PinId;
	});
}

template<typename PinType>
int32 GetPinIndexByPersistentId(TConstArrayView<PinType> Pins, const FGuid& PersistentId)
{
	return Pins.IndexOfByPredicate([&PersistentId](const FNiagaraCompilationPin& Pin) -> bool
	{
		return Pin.PersistentGuid == PersistentId;
	});
}

bool GetStaticSwitchValueFromPin(FNiagaraCompilationGraphInstanceContext& Context, const FNiagaraCompilationOutputPin& OutputPin, int32& StaticSwitchValue)
{
	if (!Context.PrecompileData)
	{
		return false;
	}

	bool bResultFound = false;

	FGraphTraversalHandle GraphTraversalHandle(Context.GraphTraversalHandle);
	GraphTraversalHandle.PushPin(&OutputPin);

	// find it in the compiled data
	if (const FString* StaticValue = Context.PrecompileData->PinToConstantValues.Find(GraphTraversalHandle))
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

		FNiagaraVariable VarWithValue(OutputPin.Variable);
		FNiagaraVariable SearchVar(OutputPin.Variable.GetType(), FName(*StaticValue));
		int32 StaticVarSearchIdx = Context.PrecompileData->StaticVariables.Find(SearchVar);

		if (StaticVarSearchIdx == INDEX_NONE)
		{
			TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(OutputPin.Variable.GetType());
			if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanHandlePinDefaults())
			{
				TypeEditorUtilities->SetValueFromPinDefaultString(*StaticValue, VarWithValue);
			}
		}
		else
		{
			VarWithValue = Context.PrecompileData->StaticVariables[StaticVarSearchIdx];
		}

		if (VarWithValue.IsDataAllocated())
		{
			if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
			{
				StaticSwitchValue = VarWithValue.GetValue<bool>();
				bResultFound = true;
			}
			else if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || VarWithValue.GetType().IsEnum())
			{
				StaticSwitchValue = VarWithValue.GetValue<int32>();
				bResultFound = true;
			}
			else
			{
				check(false);
			}
		}
	}

	return bResultFound;
}

}; // NiagaraCompilationImpl

void FNiagaraGraphChangeIdBuilder::ParseReferencedGraphs(const UNiagaraGraph* Graph)
{
	TSet<const UNiagaraGraph*> CurrentGraphChain;
	RecursiveBuildGraphChangeId(Graph, CurrentGraphChain);
}

FGuid FNiagaraGraphChangeIdBuilder::FindChangeId(const UNiagaraGraph* Graph) const
{
	const FGuid* ChangeId = ChangeIdMap.Find(Graph);
	if (ensure(ChangeId))
	{
		return *ChangeId;
	}

	return FGuid();
}

FGuid FNiagaraGraphChangeIdBuilder::RecursiveBuildGraphChangeId(const UNiagaraGraph* Graph, TSet<const UNiagaraGraph*>& CurrentGraphChain)
{
	if (!Graph)
	{
		return FGuid();
	}

	// see if the graph has already been processed
	if (const FGuid* ExistingChangeId = ChangeIdMap.Find(Graph))
	{
		return *ExistingChangeId;
	}

	FGuid ChangeId = Graph->GetChangeID();

	// check for a cycle in the graph chain we're currently processing
	const bool bCycleInGraphChain = CurrentGraphChain.Contains(Graph);
	if (bCycleInGraphChain)
	{
		ensure(false);
		return ChangeId;
	}

	CurrentGraphChain.Add(Graph);

	for (const UEdGraphNode* Node : Graph->Nodes)
	{
		const UNiagaraGraph* SubGraph = nullptr;

		if (const UNiagaraNodeFunctionCall* FunctionCallNode = Cast<const UNiagaraNodeFunctionCall>(Node))
		{
			SubGraph = FunctionCallNode->GetCalledGraph();
		}
		else if (const UNiagaraNodeEmitter* EmitterNode = Cast<const UNiagaraNodeEmitter>(Node))
		{
			SubGraph = EmitterNode->GetCalledGraph();
		}

		if (SubGraph)
		{
			ChangeId = FGuid::Combine(ChangeId, SubGraph->GetChangeID());
			ChangeId = FGuid::Combine(ChangeId, RecursiveBuildGraphChangeId(SubGraph, CurrentGraphChain));
		}
	}

	ChangeIdMap.Add(Graph, ChangeId);

	return ChangeId;
}

void FNiagaraCompilationGraphDigested::Digest(const UNiagaraGraph* InGraph, const FNiagaraGraphChangeIdBuilder& ChangeIdBuilder)
{
	using namespace NiagaraCompilationImpl;

	TArray<UClass*> IgnoredClassTypes =
	{
		UNiagaraNodeReroute::StaticClass(),
		UEdGraphNode_Comment::StaticClass(),
	};

	SourceGraph = InGraph;
	Nodes.Reserve(InGraph->Nodes.Num());

	TMap<const UEdGraphNode*, int32> NodeIndexMap;

	FNiagaraCompilationGraphCreateContext NodeContext(*this, ChildGraphs, ChangeIdBuilder);

	for (const UEdGraphNode* SourceNode : InGraph->Nodes)
	{
		if (!SourceNode->IsNodeEnabled())
		{
			continue;
		}

		if (TUniquePtr<FNiagaraCompilationNode> CompilationNode = CreateNode(SourceNode, NodeContext))
		{
			NodeIndexMap.Add(SourceNode, Nodes.Num());

			Nodes.Emplace(MoveTemp(CompilationNode));
		}
		else
		{
			check(IgnoredClassTypes.Contains(SourceNode->GetClass()));
		}
	}

	// after all the nodes & pins have been created we can go in and make the connections between the nodes
	for (TUniquePtr<FNiagaraCompilationNode>& CompilationNode : Nodes)
	{
		for (FNiagaraCompilationInputPin& InputPin : CompilationNode->InputPins)
		{
			const UEdGraphPin* SourceInputPin = CompilationNode->SourceNode->Pins[InputPin.SourcePinIndex];
			const UEdGraphPin* SourceLinkedPin = nullptr;

			// apparently some content exists where we'll have multiple copies of a connection added.  Find the first non-null LinkedPin
			for (const UEdGraphPin* CurrentLinkedPin : SourceInputPin->LinkedTo)
			{
				if (SourceLinkedPin == nullptr)
				{
					SourceLinkedPin = CurrentLinkedPin;
					break;
				}
			}

			if (SourceLinkedPin)
			{
				if (const UEdGraphPin* LinkedPin = TraceThroughIgnoredNodes(SourceLinkedPin))
				{
					const UEdGraphNode* LinkedSourceNode = LinkedPin->GetOwningNode();
					const int32* LinkedNodeIndexPtr = NodeIndexMap.Find(LinkedSourceNode);

					// There are situations where a pin is connected to a node that doesn't exist within the UNiagaraGraph::Nodes array.
					// This corruption seems connected to auto-generated input nodes for function calls.  For now we're going to ignore those
					// connections
					const int32 LinkedNodeIndex = (LinkedNodeIndexPtr && Nodes.IsValidIndex(*LinkedNodeIndexPtr)) ? *LinkedNodeIndexPtr : INDEX_NONE;

					if (LinkedNodeIndex != INDEX_NONE)
					{
						TUniquePtr<FNiagaraCompilationNode>& LinkedNode = Nodes[LinkedNodeIndex];
						const int32 LinkedSourcePinIndex = LinkedNode->SourceNode->Pins.IndexOfByKey(LinkedPin);
						if (LinkedSourcePinIndex != INDEX_NONE)
						{
							FNiagaraCompilationOutputPin* LinkedOutputPin = LinkedNode->OutputPins.FindByPredicate([LinkedSourcePinIndex](const FNiagaraCompilationOutputPin& OutputPin)
							{
								return OutputPin.SourcePinIndex == LinkedSourcePinIndex;
							});

							InputPin.LinkedTo = LinkedOutputPin;

							// and create the reverse link
							if (LinkedOutputPin)
							{
								LinkedOutputPin->LinkedTo.AddUnique(&InputPin);
							}
						}
					}
				}
			}
		}
	}

	UPackage* TransientPackage = GetTransientPackage();

	// todo - why do we need a duplicate of the CDO?
	auto RegisterTransientCDO = [this](UPackage* Owner, UClass* DataInterfaceClass) -> void
	{
		if (!CachedDataInterfaceCDODuplicates.Contains(DataInterfaceClass))
		{
			UNiagaraDataInterface* DataInterfaceCDO = DataInterfaceClass->GetDefaultObject<UNiagaraDataInterface>();
			UNiagaraDataInterface* DuplicatedCDO = DuplicateObject<UNiagaraDataInterface>(DataInterfaceCDO, Owner);

			CachedDataInterfaceCDODuplicates.Add(DataInterfaceClass, DuplicatedCDO);
		}
	};

	using FScriptMap = TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>;
	const FScriptMap& GraphScriptVariables = InGraph->GetAllMetaData();
	ScriptVariableData.Reserve(GraphScriptVariables.Num());

	for (const FScriptMap::ElementType& GraphScriptVariable : GraphScriptVariables)
	{
		if (GraphScriptVariable.Value)
		{
			ScriptVariableData.AddDefaulted_GetRef().InitFrom(*GraphScriptVariable.Value, false);

			const FNiagaraVariable& Variable = GraphScriptVariable.Key;
			const FNiagaraTypeDefinition& VariableType = Variable.GetType();
			if (VariableType.IsDataInterface())
			{
				RegisterTransientCDO(TransientPackage, VariableType.GetClass());
			}

			if (VariableType.IsStatic())
			{
				bContainsStaticVariables = true;
			}
		}
	}

	// additionally we're going to go through the mapgets in the node and see if we might have missed any
	// datainterface variables.  There are cases in top level graphs where seemingly orphaned mapget nodes
	// can reference a data interface variable but it isn't represented in the script variables or in input
	// nodes.
	for (TUniquePtr<FNiagaraCompilationNode>& CompilationNode : Nodes)
	{
		if (const FNiagaraCompilationNodeParameterMapGet* MapGet = CompilationNode->AsType<FNiagaraCompilationNodeParameterMapGet>())
		{
			for (const FNiagaraCompilationOutputPin& OutputPin : MapGet->OutputPins)
			{
				if (OutputPin.Variable.IsDataInterface())
				{
					RegisterTransientCDO(TransientPackage, OutputPin.Variable.GetType().GetClass());
				}
			}
		}
	}

	if (const UNiagaraScript* SourceScript = InGraph->GetTypedOuter<const UNiagaraScript>())
	{
		SourceScriptName = SourceScript->GetName();
		SourceScriptFullName = SourceScript->GetFullName();
	}

	// go through all the data interfaces that have been registered and make sure we populate CachedDataInterfaceCDODuplicates
	// along with moving things over to transient duplicates
	for (const FDataInterfaceDuplicateMap::ElementType& DataInterfaceIt : CachedDataInterfaceDuplicates)
	{
		if (UNiagaraDataInterface* DataInterface = DataInterfaceIt.Value.Get())
		{
			if (UClass* DataInterfaceClass = DataInterface->GetClass())
			{
				RegisterTransientCDO(TransientPackage, DataInterfaceClass);
			}
		}
	}
}

UNiagaraDataInterface* FNiagaraCompilationGraphDigested::DigestDataInterface(UNiagaraDataInterface* SourceDataInterface)
{
	check(IsInGameThread());
	TObjectPtr<UNiagaraDataInterface>& ExistingDuplicate = CachedDataInterfaceDuplicates.FindOrAdd(SourceDataInterface);

	if (!ExistingDuplicate)
	{
		ExistingDuplicate = DuplicateObject<UNiagaraDataInterface>(SourceDataInterface, GetTransientPackage());
	}

	return ExistingDuplicate;
}

void FNiagaraCompilationGraphDigested::RegisterObjectAsset(FName VariableName, UObject* SourceObjectAsset)
{
	const UObject* ExistingObjectAsset = CachedNamedObjectAssets.FindRef(VariableName);
	if (ExistingObjectAsset)
	{
		check(ExistingObjectAsset == SourceObjectAsset);
	}
	else
	{
		CachedNamedObjectAssets.Add(VariableName, SourceObjectAsset);
	}
}

void FNiagaraCompilationGraphDigested::CollectReferencedDataInterfaceCDO(FDataInterfaceCDOMap& Interfaces) const
{
	// add our collected DIs then iterate over all the child graphs
	for (FDataInterfaceCDOMap::TConstIterator It = CachedDataInterfaceCDODuplicates.CreateConstIterator(); It; ++It)
	{
		Interfaces.FindOrAdd(It.Key(), It.Value());
	}

	for (const FNiagaraCompilationGraphDigested* ChildGraph : ChildGraphs)
	{
		ChildGraph->CollectReferencedDataInterfaceCDO(Interfaces);
	}
}

void FNiagaraCompilationGraph::FindOutputNodes(TArray<const FNiagaraCompilationNodeOutput*>& OutputNodes) const
{
	OutputNodes.Reserve(OutputNodeIndices.Num());
	for (int32 OutputNodeIndex : OutputNodeIndices)
	{
		if (const FNiagaraCompilationNodeOutput* OutputNode = Nodes[OutputNodeIndex]->AsType<FNiagaraCompilationNodeOutput>())
		{
			OutputNodes.Add(OutputNode);
		}
	}
}

void FNiagaraCompilationGraph::FindOutputNodes(ENiagaraScriptUsage TargetUsageType, TArray<const FNiagaraCompilationNodeOutput*>& OutputNodes) const
{
	OutputNodes.Reserve(OutputNodeIndices.Num());
	for (int32 OutputNodeIndex : OutputNodeIndices)
	{
		if (const FNiagaraCompilationNodeOutput* OutputNode = Nodes[OutputNodeIndex]->AsType<FNiagaraCompilationNodeOutput>())
		{
			if (OutputNode->Usage == TargetUsageType)
			{
				OutputNodes.Add(OutputNode);
			}
		}
	}
}

const FNiagaraCompilationNodeOutput* FNiagaraCompilationGraph::FindOutputNode(ENiagaraScriptUsage Usage, const FGuid& UsageId) const
{
	for (int32 OutputNodeIndex : OutputNodeIndices)
	{
		const FNiagaraCompilationNodeOutput& OutputNode = Nodes[OutputNodeIndex]->AsTypeRef<FNiagaraCompilationNodeOutput>();
		if (OutputNode.Usage == Usage && OutputNode.UsageId == UsageId)
		{
			return &OutputNode;
		}
	}

	return nullptr;
}

const FNiagaraCompilationNodeOutput* FNiagaraCompilationGraph::FindEquivalentOutputNode(ENiagaraScriptUsage Usage, const FGuid& UsageId) const
{
	for (int32 OutputNodeIndex : OutputNodeIndices)
	{
		const FNiagaraCompilationNodeOutput& OutputNode = Nodes[OutputNodeIndex]->AsTypeRef<FNiagaraCompilationNodeOutput>();
		if (UNiagaraScript::IsEquivalentUsage(OutputNode.Usage, Usage) && OutputNode.UsageId == UsageId)
		{
			return &OutputNode;
		}
	}

	return nullptr;
}

void FNiagaraCompilationGraph::FindInputNodes(TArray<const FNiagaraCompilationNodeInput*>& OutInputNodes, UNiagaraGraph::FFindInputNodeOptions Options) const
{
	using namespace NiagaraCompilationImpl;

	TArray<const FNiagaraCompilationNodeInput*> InputNodes;

	if (!Options.bFilterByScriptUsage)
	{
		for (int32 InputNodeIndex : InputNodeIndices)
		{
			const FNiagaraCompilationNodeInput* InputNode = Nodes[InputNodeIndex]->AsType<FNiagaraCompilationNodeInput>();
			if (ensure(InputNode))
			{
				if ((InputNode->Usage == ENiagaraInputNodeUsage::Parameter && Options.bIncludeParameters) ||
					(InputNode->Usage == ENiagaraInputNodeUsage::Attribute && Options.bIncludeAttributes) ||
					(InputNode->Usage == ENiagaraInputNodeUsage::SystemConstant && Options.bIncludeSystemConstants) ||
					(InputNode->Usage == ENiagaraInputNodeUsage::TranslatorConstant && Options.bIncludeTranslatorConstants))
				{
					InputNodes.Add(InputNode);
				}
			}
		}
	}
	else
	{
		const TArray<const FNiagaraCompilationNode*> OutputNodes = FindOutputNodesByUsage({Options.TargetScriptUsage});
		TArray<const FNiagaraCompilationNode*> ConnectedNodes = CollectConnectedNodes(OutputNodes, Nodes.Num(), FNiagaraCompilationBranchMap());

		for (int32 InputNodeIndex : InputNodeIndices)
		{
			const FNiagaraCompilationNodeInput* InputNode = Nodes[InputNodeIndex]->AsType<FNiagaraCompilationNodeInput>();
			if (ensure(InputNode))
			{
				if ((InputNode->Usage == ENiagaraInputNodeUsage::Parameter && Options.bIncludeParameters) ||
					(InputNode->Usage == ENiagaraInputNodeUsage::Attribute && Options.bIncludeAttributes) ||
					(InputNode->Usage == ENiagaraInputNodeUsage::SystemConstant && Options.bIncludeSystemConstants))
				{
					if (ConnectedNodes.Contains(InputNode))
					{
						InputNodes.Add(InputNode);
					}
				}
			}
		}
	}

	if (Options.bFilterDuplicates)
	{
		for (const FNiagaraCompilationNodeInput* InputNode : InputNodes)
		{
			auto NodeMatches = [=](const FNiagaraCompilationNodeInput* UniqueInputNode)
			{
				if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter)
				{
					return UniqueInputNode->InputVariable.IsEquivalent(InputNode->InputVariable, false);
				}
				else
				{
					return UniqueInputNode->InputVariable.IsEquivalent(InputNode->InputVariable);
				}
			};

			if (OutInputNodes.ContainsByPredicate(NodeMatches) == false)
			{
				OutInputNodes.Add(InputNode);
			}
		}
	}
	else
	{
		OutInputNodes.Append(InputNodes);
	}

	if (Options.bSort)
	{
		Algo::Sort(OutInputNodes, [](const FNiagaraCompilationNodeInput* Lhs, const FNiagaraCompilationNodeInput* Rhs) -> bool
		{
			if (Lhs->CallSortPriority < Rhs->CallSortPriority)
			{
				return true;
			}
			else if (Lhs->CallSortPriority > Rhs->CallSortPriority)
			{
				return false;
			}

			//If equal priority, sort lexicographically.
			return Lhs->InputVariable.GetName().LexicalLess(Rhs->InputVariable.GetName());
		});
	}
}

void FNiagaraCompilationGraph::UpdateFunctionAliasByContext(FNiagaraGraphFunctionAliasContext& FunctionAliasContext) const
{
	for (const TUniquePtr<FNiagaraCompilationNode>& Node : Nodes)
	{
		if (const FNiagaraCompilationNode* NiagaraNode = Node.Get())
		{
			NiagaraNode->UpdateFunctionAliasForContext(FunctionAliasContext);
		}
	}
}

bool FNiagaraCompilationGraph::HasParametersOfType(const FNiagaraTypeDefinition& Type) const
{
	for (int32 InputNodeIndex : InputNodeIndices)
	{
		const FNiagaraCompilationNodeInput& InputNode = Nodes[InputNodeIndex]->AsTypeRef<FNiagaraCompilationNodeInput>();
		if (InputNode.InputVariable.GetType() == Type)
		{
			return true;
		}
	}

	for (int32 OutputNodeIndex : OutputNodeIndices)
	{
		const FNiagaraCompilationNodeOutput& OutputNode = Nodes[OutputNodeIndex]->AsTypeRef<FNiagaraCompilationNodeOutput>();
		for (const FNiagaraVariable& OutputVariable : OutputNode.Outputs)
		{
			if (OutputVariable.GetType() == Type)
			{
				return true;
			}
		}
	}

	return false;
}

const FNiagaraScriptVariableData* FNiagaraCompilationGraph::GetScriptVariableData(const FNiagaraVariableBase& Variable) const
{
	return ScriptVariableData.FindByPredicate([&Variable](const FNiagaraScriptVariableData& VariableData) -> bool
	{
		return Variable.IsEquivalent(VariableData.Variable, false /*bAllowAssignableTypes*/);
	});
}

TOptional<ENiagaraDefaultMode> FNiagaraCompilationGraph::GetDefaultMode(const FNiagaraVariableBase& Variable, FNiagaraScriptVariableBinding& Binding) const
{
	if (const FNiagaraScriptVariableData* ScriptVariable = GetScriptVariableData(Variable))
	{
		Binding = ScriptVariable->DefaultBinding;
		return ScriptVariable->DefaultMode;
	}

	return TOptional<ENiagaraDefaultMode>();
}

TOptional<FNiagaraVariableMetaData> FNiagaraCompilationGraph::GetMetaData(const FNiagaraVariableBase& Variable) const
{
	if (const FNiagaraScriptVariableData* ScriptVariable = GetScriptVariableData(Variable))
	{
		return ScriptVariable->Metadata;
	}

	return TOptional<FNiagaraVariableMetaData>();
}

TArray<const FNiagaraCompilationNode*> FNiagaraCompilationGraph::FindOutputNodesByUsage(TConstArrayView<ENiagaraScriptUsage> Usages) const
{
	TArray<const FNiagaraCompilationNode*> OutputNodesByUsage;

	for (int32 OutputNodeIndex : OutputNodeIndices)
	{
		const FNiagaraCompilationNodeOutput& OutputNode = Nodes[OutputNodeIndex]->AsTypeRef<FNiagaraCompilationNodeOutput>();

		if (Usages.ContainsByPredicate([&OutputNode](ENiagaraScriptUsage CompileUsage) { return UNiagaraScript::IsEquivalentUsage(CompileUsage, OutputNode.Usage); }))
		{
			OutputNodesByUsage.Add(&OutputNode);
		}
	}

	return OutputNodesByUsage;
}

TSharedPtr<FNiagaraCompilationGraphInstanced, ESPMode::ThreadSafe> FNiagaraCompilationGraphDigested::InstantiateSubGraph(
	const TArray<ENiagaraScriptUsage>& Usages,
	const FNiagaraCompilationCopyData* CopyCompilationData,
	const FNiagaraCompilationBranchMap& Branches,
	TArray<FNiagaraCompilationNodeFunctionCall*>& PendingInstantiations) const
{
	using namespace NiagaraCompilationImpl;

	TSharedPtr<FNiagaraCompilationGraphInstanced, ESPMode::ThreadSafe> SubGraph = MakeShared<FNiagaraCompilationGraphInstanced, ESPMode::ThreadSafe>();
	SubGraph->InstantiationSourceGraph = AsShared().ToSharedPtr();

	TArray<const FNiagaraCompilationNode*> OutputNodesByUsage = FindOutputNodesByUsage(Usages);
	const int32 InstantiatedOutputNodeCount = OutputNodesByUsage.Num();

	TArray<const FNiagaraCompilationNode*> SortedNodes = TopologicalSort(CollectConnectedNodes(OutputNodesByUsage, Nodes.Num(), Branches), Branches);

	FNiagaraCompilationGraphDuplicateContext DuplicateNodeContext(*SubGraph, *this, Usages, CopyCompilationData, Branches);

	const int32 SubGraphNodeCount = SortedNodes.Num();

	SubGraph->Nodes.Reserve(SubGraphNodeCount);
	SubGraph->OutputNodeIndices.Reserve(InstantiatedOutputNodeCount);
	SubGraph->InputNodeIndices.Reserve(InputNodeIndices.Num());

	for (const FNiagaraCompilationNode* SourceNode : SortedNodes)
	{
		TUniquePtr<FNiagaraCompilationNode> TargetNode = DuplicateNode(SourceNode, DuplicateNodeContext);

		DuplicateNodeContext.DuplicatedNodeMap.Add(SourceNode, TargetNode.Get());

		SubGraph->Nodes.Emplace(MoveTemp(TargetNode));
	}

	// copy over all the properties for the subgraph (this will contain data that could have been culled based on the
	// output nodes that we're filtering by above
	SubGraph->VariableBinding = VariableBinding;
	SubGraph->ScriptVariableData = ScriptVariableData;
	SubGraph->SourceScriptName = SourceScriptName;
	SubGraph->SourceScriptFullName = SourceScriptFullName;
	SubGraph->bContainsStaticVariables = bContainsStaticVariables;

	while (!DuplicateNodeContext.FunctionsRequiresGraph.IsEmpty())
	{
		PendingInstantiations.Add(DuplicateNodeContext.FunctionsRequiresGraph.Pop());
	}

	// the StaticSwitchInputs will be missing the results from static switch nodes if they are being
	// bypassed in the compilation copy.  So we just copy the missing variables from the source graph
	for (const FNiagaraVariableBase& SwitchInput : StaticSwitchInputs)
	{
		SubGraph->StaticSwitchInputs.AddUnique(SwitchInput);
	}

	return SubGraph;
}

void FNiagaraCompilationGraphInstanced::ValidateRefinement() const
{
	// validate that now that we've refined the graph we have no more generic numerics and also ensure
	// that there are no more static switches connected
	for (const TUniquePtr<FNiagaraCompilationNode>& Node : Nodes)
	{
		switch (Node->NodeType)
		{
			case FNiagaraCompilationNode::ENodeType::StaticSwitch:
			{
				// only switches handled by the compiler are allowed to make it to this stage
				const bool bSetByCompiler = Node->AsType<FNiagaraCompilationNodeStaticSwitch>()->bSetByCompiler;
				
				if (!bSetByCompiler)
				{
					UE_LOGF(LogNiagaraEditor, Warning, "Encountered StaticSwitch that should have been resolved.  Graph will likely fail to compile.  Task: %ls Node %ls",
						*SourceScriptFullName, *Node->NodeName);
				}
			}
			break;

			// it's possible that we could eliminate these during instantiation as well, but currently
			// the information used is stored on the translator (because we can use the same
			// instantiated graphs for different usages)
			//case FNiagaraCompilationNode::ENodeType::UsageSelector:
			//case FNiagaraCompilationNode::ENodeType::SimTargetSelector:
			//	check(false);
			//break;
		}

		for (const FNiagaraCompilationInputPin& InputPin : Node->InputPins)
		{
			ensureMsgf(InputPin.Variable.GetType() != FNiagaraTypeDefinition::GetGenericNumericDef() || !InputPin.LinkedTo,
				TEXT("Failed during ValidateRefinement for compilation task - %s.  Connected InputPin[%s.%s] is still a generic."),
				*SourceScriptFullName, *InputPin.OwningNode->NodeName, *InputPin.PinName.ToString());
		}

		for (const FNiagaraCompilationOutputPin& OutputPin : Node->OutputPins)
		{
			ensureMsgf(OutputPin.Variable.GetType() != FNiagaraTypeDefinition::GetGenericNumericDef() || OutputPin.LinkedTo.IsEmpty(),
				TEXT("Failed during ValidateRefinement for compilation task - %s.  Connected OutputPin[%s.%s] is still a generic."),
				*SourceScriptFullName, *OutputPin.OwningNode->NodeName, *OutputPin.PinName.ToString());
		}
	}
}

void FNiagaraCompilationGraphInstanced::Refine(FNiagaraCompilationGraphInstanceContext& InstantiationContext, const FNiagaraCompilationNodeFunctionCall* CallingNode)
{
	if (CallingNode)
	{
		PatchGenericNumericsFromCaller(InstantiationContext);
	}

	ResolveNumerics(InstantiationContext);

	if (CallingNode)
	{
		// go through the nodes and apply any changes necessary based on the CallingNode
		for (TUniquePtr<FNiagaraCompilationNode>& CompilationNode : Nodes)
		{
			if (FNiagaraCompilationNodeFunctionCall* FunctionCallNode = CompilationNode->AsType<FNiagaraCompilationNodeFunctionCall>())
			{
				InheritDebugState(InstantiationContext, *FunctionCallNode);
				PropagateDefaultValues(InstantiationContext, *FunctionCallNode);
			}
		}
	}

	StripUnconnectedPins(InstantiationContext);

	// validate that now that we've refined the graph we have no more generic numerics and also ensure
	// that there are no more static switches connected
	ValidateRefinement();
}

void FNiagaraCompilationGraphInstanced::PatchGenericNumericsFromCaller(FNiagaraCompilationGraphInstanceContext& Context)
{
	static const FNiagaraTypeDefinition& GenericTypeDef = FNiagaraTypeDefinition::GetGenericNumericDef();
	const FNiagaraCompilationNodeFunctionCall* CallingNode = Context.GetCurrentFunctionNode();

	for (int32 InputNodeIndex : InputNodeIndices)
	{
		FNiagaraCompilationNodeInput& InputNode = Nodes[InputNodeIndex]->AsTypeRef<FNiagaraCompilationNodeInput>();

		if (InputNode.InputVariable.GetType() == GenericTypeDef)
		{
			const FNiagaraCompilationInputPin* CallerInputPin = CallingNode->InputPins.FindByPredicate([&InputNode](const FNiagaraCompilationInputPin& InputPin) -> bool
			{
				return InputPin.PinName == InputNode.InputVariable.GetName();
			});

			if (CallerInputPin)
			{
				InputNode.InputVariable.SetType(CallerInputPin->Variable.GetType());
				InputNode.OutputPins[0].Variable.SetType(CallerInputPin->Variable.GetType());
			}
		}
	}

	for (int32 OutputNodeIndex : OutputNodeIndices)
	{
		FNiagaraCompilationNodeOutput& OutputNode = Nodes[OutputNodeIndex]->AsTypeRef<FNiagaraCompilationNodeOutput>();

		const int32 InputPinCount = OutputNode.InputPins.Num();
		for (int32 InputPinIt = 0; InputPinIt < InputPinCount; ++InputPinIt)
		{
			FNiagaraVariable& OutputVariable = OutputNode.Outputs[InputPinIt];

			if (OutputVariable.GetType() == GenericTypeDef)
			{
				const FNiagaraCompilationOutputPin* CallerOutputPin = CallingNode->OutputPins.FindByPredicate([&OutputVariable](const FNiagaraCompilationOutputPin& OutputPin) -> bool
				{
					return OutputPin.PinName == OutputVariable.GetName();
				});

				if (CallerOutputPin)
				{
					OutputVariable.SetType(CallerOutputPin->Variable.GetType());
					OutputNode.InputPins[InputPinIt].Variable.SetType(CallerOutputPin->Variable.GetType());
				}
			}
		}
	}
}

void FNiagaraCompilationGraphInstanced::ResolveNumerics(FNiagaraCompilationGraphInstanceContext& Context)
{
	const FNiagaraTypeDefinition& GenericTypeDef = FNiagaraTypeDefinition::GetGenericNumericDef();
	const FNiagaraTypeDefinition& PlaceholderTypeDef = FNiagaraTypeDefinition::GetFloatDef();

	// if we're forcing generics to be converted we apply that to the inputs first
	if (Context.bForceNumericResolution)
	{
		for (int32 InputNodeIndex : InputNodeIndices)
		{
			FNiagaraCompilationNodeInput& InputNode = Nodes[InputNodeIndex]->AsTypeRef<FNiagaraCompilationNodeInput>();

			for (FNiagaraCompilationOutputPin& OutputPin : InputNode.OutputPins)
			{
				if (OutputPin.Variable.GetType() == GenericTypeDef)
				{
					OutputPin.Variable.SetType(PlaceholderTypeDef);
				}
			}

			if (InputNode.InputVariable.GetType() == GenericTypeDef)
			{
				InputNode.InputVariable.SetType(PlaceholderTypeDef);
			}
		}
	}

	// process the graph
	for (TUniquePtr<FNiagaraCompilationNode>& Node : Nodes)
	{
		Node->ResolveNumerics();
	}

	// if we're forcing generics to be converted we finish off by making sure the outputs are converted
	if (Context.bForceNumericResolution)
	{
		for (int32 OutputNodeIndex : OutputNodeIndices)
		{
			FNiagaraCompilationNodeOutput& OutputNode = Nodes[OutputNodeIndex]->AsTypeRef<FNiagaraCompilationNodeOutput>();

			const int32 InputPinCount = OutputNode.InputPins.Num();
			for (int32 InputPinIt = 0; InputPinIt < InputPinCount; ++InputPinIt)
			{
				if ((OutputNode.InputPins[InputPinIt].Variable.GetType() != GenericTypeDef)
					&& (OutputNode.Outputs[InputPinIt].GetType() == GenericTypeDef))
				{
					OutputNode.Outputs[InputPinIt].SetType(PlaceholderTypeDef);
				}
			}
		}
	}
}

void FNiagaraCompilationGraphInstanced::StripUnconnectedPins(FNiagaraCompilationGraphInstanceContext& Context)
{
	// through instantiation some pins may become disconnected (for example because of static switches being evaluated).  For some nodes,
	// like MapGet, this can be problematic because the culled paths may lead to some pins being in a weird state.
	
	// If the output of a MapGet is unconnected, then it's default input pin can also be disconnected.
	TArray<FNiagaraCompilationNodeParameterMapGet*, TInlineAllocator<32>> MapGetNodes;

	for (TUniquePtr<FNiagaraCompilationNode>& Node : Nodes)
	{
		if (FNiagaraCompilationNodeParameterMapGet* MapGetNode = Node->AsType<FNiagaraCompilationNodeParameterMapGet>())
		{
			MapGetNodes.Add(MapGetNode);
		}
	}

	bool bVisitMapGetNodes = !MapGetNodes.IsEmpty();

	while (bVisitMapGetNodes)
	{
		bVisitMapGetNodes = false;

		for (FNiagaraCompilationNodeParameterMapGet* MapGetNode : MapGetNodes)
		{
			const int32 OutputPinCount = MapGetNode->OutputPins.Num();
			for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
			{
				const FNiagaraCompilationOutputPin& OutputPin = MapGetNode->OutputPins[OutputPinIt];
				if (OutputPin.LinkedTo.IsEmpty())
				{
					if (MapGetNode->DefaultInputPinIndices.IsValidIndex(OutputPinIt))
					{
						const int32 InputPinIndex = MapGetNode->DefaultInputPinIndices[OutputPinIt];
						if (MapGetNode->InputPins.IsValidIndex(InputPinIndex))
						{
							// if we do change something in the graph then revisit our map nodes to see if there
							// may be more connections that can be removed
							if (MapGetNode->InputPins[InputPinIndex].Disconnect())
							{
								bVisitMapGetNodes = true;
							}
						}
					}
				}
			}
		}
	}
}

void FNiagaraCompilationGraphInstanced::InheritDebugState(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationNodeFunctionCall& FunctionCallNode)
{
	FunctionCallNode.DebugState = FunctionCallNode.bInheritDebugState ? Context.ConstantResolver.GetDebugState() : FunctionCallNode.DebugState;
}

void FNiagaraCompilationGraphInstanced::PropagateDefaultValues(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationNodeFunctionCall& FunctionCallNode)
{
	using namespace NiagaraCompilationImpl;

	const FNiagaraCompilationNodeFunctionCall* CallingNode = Context.GetCurrentFunctionNode();

	for (const FNiagaraCompilationNodeFunctionCall::FTaggedVariable& PropagatedVariable : FunctionCallNode.PropagatedStaticSwitchParameters)
	{
		const FName FunctionPinName = PropagatedVariable.Key.GetName();
		const FName CallerPinName = PropagatedVariable.Value;

		FNiagaraCompilationInputPin* FunctionInputPin = FindPinByName<FNiagaraCompilationInputPin>(FunctionCallNode.InputPins, FunctionPinName);

		if (FunctionInputPin)
		{
			// reset the default value, to be replaced by the calling function's input, if one is found
			FunctionInputPin->DefaultValue = FString();

			if (const FNiagaraCompilationInputPin* CallerInputPin = FindPinByName<const FNiagaraCompilationInputPin>(CallingNode->InputPins, CallerPinName))
			{
				FunctionInputPin->DefaultValue = CallerInputPin->DefaultValue;
			}
		}
	}
}

TSharedPtr<FNiagaraCompilationGraphInstanced, ESPMode::ThreadSafe> FNiagaraCompilationGraphDigested::Instantiate(const FNiagaraPrecompileData* PrecompileData, const FNiagaraCompilationCopyData* CopyCompilationData, const TArray<ENiagaraScriptUsage>& Usages, const FNiagaraFixedConstantResolver& ConstantResolver) const
{
	// in order to manage the traversal state context and to keep a lit on the potential for crazy recursion we're
	// going to track the dependent graphs that require instantiation in this quasi tree structure
	struct FChildrenStackEntry
	{
		FChildrenStackEntry(int32 InParentFunctionIndex)
			: ParentFunctionIndex(InParentFunctionIndex)
		{
		}

		const int32 ParentFunctionIndex;
		TArray<FNiagaraCompilationNodeFunctionCall*> Functions;
		int32 CurrentFunctionIndex = 0;
	};

	TArray<FChildrenStackEntry> FunctionsToInstantiate;
	TSharedPtr<FNiagaraCompilationGraphInstanced, ESPMode::ThreadSafe> InstantiatedGraph = InstantiateSubGraph(Usages, CopyCompilationData, FNiagaraCompilationBranchMap(), FunctionsToInstantiate.Emplace_GetRef(INDEX_NONE).Functions);

	// initialize the traversal context with the data that was pulled from the parameter map history done during the
	// precompile
	FNiagaraCompilationGraphInstanceContext InstantiationContext(ConstantResolver, InstantiatedGraph.Get(), PrecompileData);

	if (ensure(InstantiatedGraph))
	{
		InstantiatedGraph->ResolveNumerics(InstantiationContext);
	}

	int32 TotalGraphCount = 1;
	int32 TotalNodeCount = Nodes.Num();
	int32 TotalCulledNodeCount = 0;

	int32 CurrentFunctionSetIndex = 0;
	while (CurrentFunctionSetIndex != INDEX_NONE)
	{
		FChildrenStackEntry& CurrentStack = FunctionsToInstantiate[CurrentFunctionSetIndex];
		if (CurrentStack.Functions.IsValidIndex(CurrentStack.CurrentFunctionIndex))
		{
			const int32 NextFunctionSetIndex = FunctionsToInstantiate.Num();

			FNiagaraCompilationNodeFunctionCall* FunctionToInstantiate = CurrentStack.Functions[CurrentStack.CurrentFunctionIndex];
			FChildrenStackEntry& ChildStack = FunctionsToInstantiate.Emplace_GetRef(CurrentFunctionSetIndex);

			InstantiationContext.EnterFunction(FunctionToInstantiate);

			FNiagaraCompilationBranchMap Branches;
			EvaluateStaticBranches(InstantiationContext, Branches);

			FNiagaraCompilationGraphDigested* CalledDigestedGraph = FunctionToInstantiate->CalledGraph->AsDigested();
			if (ensure(CalledDigestedGraph))
			{
				TSharedPtr<FNiagaraCompilationGraphInstanced, ESPMode::ThreadSafe> CalledInstantiatedGraph =
					CalledDigestedGraph->InstantiateSubGraph({ FunctionToInstantiate->CalledScriptUsage }, CopyCompilationData, Branches, ChildStack.Functions);

				CalledInstantiatedGraph->Refine(InstantiationContext, FunctionToInstantiate);

				++TotalGraphCount;
				TotalNodeCount += CalledInstantiatedGraph->Nodes.Num();
				TotalCulledNodeCount += CalledDigestedGraph->Nodes.Num() - CalledInstantiatedGraph->Nodes.Num();

				// now replace the called graph with the instantiated version
				FunctionToInstantiate->CalledGraph = CalledInstantiatedGraph;
			}

			CurrentFunctionSetIndex = NextFunctionSetIndex;
		}
		else
		{
			CurrentFunctionSetIndex = CurrentStack.ParentFunctionIndex;
			if (CurrentFunctionSetIndex != INDEX_NONE)
			{
				if (ensure(!InstantiationContext.FunctionStack.IsEmpty()))
				{
					// get the parent graph from the InstanatiationContext so that we can aggregate any relevant
					// information up the chain
					const int32 ParentFunctionIndex = InstantiationContext.FunctionStack.Num() - 2;
					const int32 ChildFunctionIndex = InstantiationContext.FunctionStack.Num() - 1;
					ensure(InstantiationContext.FunctionStack.IsValidIndex(ChildFunctionIndex));

					FNiagaraCompilationGraphInstanced* ParentGraph = InstantiationContext.FunctionStack.IsValidIndex(ParentFunctionIndex)
						? InstantiationContext.FunctionStack[ParentFunctionIndex]->CalledGraph->AsInstanced()
						: InstantiatedGraph.Get();

					FNiagaraCompilationGraphInstanced* ChildGraph = InstantiationContext.FunctionStack[ChildFunctionIndex]->CalledGraph->AsInstanced();
					
					if (ensure(ParentGraph && ChildGraph))
					{
						ParentGraph->AggregateChildGraph(ChildGraph);
					}
				}

				FChildrenStackEntry& ParentEntry = FunctionsToInstantiate[CurrentFunctionSetIndex];
				InstantiationContext.LeaveFunction(ParentEntry.Functions[ParentEntry.CurrentFunctionIndex]);

				++ParentEntry.CurrentFunctionIndex;
			}
		}
	}

	return InstantiatedGraph;
}

void FNiagaraCompilationGraph::EvaluateStaticBranches(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationBranchMap& Branches) const
{
	// build a list of connected nodes in topological order for the current graph

	// start at the back (outputs) and work to the front, evaluating each node
		// for the following node types we need to evaluate their state for the instantiated graph:
		//	static switches
		//	UsageSelectors (explicitly, not all UsageSelectors, because that also includes Select nodes)
		//	SimTarget

	TSet<const FNiagaraCompilationNode*> VisitedNodes;
	VisitedNodes.Reserve(Nodes.Num());

	const FNiagaraCompilationNodeFunctionCall* CallingNode = Context.GetCurrentFunctionNode();

	TArray<const FNiagaraCompilationNode*> NodesToProcess = CallingNode->CalledGraph->GetOutputNodes();
	while (!NodesToProcess.IsEmpty())
	{
		const FNiagaraCompilationNode* NodeToProcess = NodesToProcess.Pop();

		TArray<const FNiagaraCompilationInputPin*> ValidInputPins = NodeToProcess->EvaluateBranches(Context, Branches);

		for (const FNiagaraCompilationInputPin* InputPin : ValidInputPins)
		{
			if (InputPin->LinkedTo)
			{
				bool bAlreadyVisited;
				VisitedNodes.Add(InputPin->LinkedTo->OwningNode, &bAlreadyVisited);
				if (!bAlreadyVisited)
				{
					NodesToProcess.Push(InputPin->LinkedTo->OwningNode);
				}
			}
		}
	}
}

void FNiagaraCompilationGraph::NodeTraversal(
	bool bRecursive,
	bool bOrdered,
	TFunctionRef<bool(const FNiagaraCompilationNodeOutput&)> RootNodeFilter,
	TFunctionRef<bool(const FNiagaraCompilationNode&)> NodeOperation) const
{
	using namespace NiagaraCompilationImpl;

	for (int32 OutputNodeIndex : OutputNodeIndices)
	{
		const FNiagaraCompilationNodeOutput* OutputNode = Nodes[OutputNodeIndex]->AsType<FNiagaraCompilationNodeOutput>();
		if (RootNodeFilter(*OutputNode))
		{
			FNiagaraCompilationBranchMap BranchMap;
			TArray<const FNiagaraCompilationNode*> NodesToProcess = CollectConnectedNodes( { OutputNode }, Nodes.Num(), BranchMap );

			if (bOrdered)
			{
				NodesToProcess = TopologicalSort(NodesToProcess, BranchMap);
			}

			for (const FNiagaraCompilationNode* NodeToProcess : NodesToProcess)
			{
				if (!NodeOperation(*NodeToProcess))
				{
					break;
				}

				if (bRecursive)
				{
					if (const FNiagaraCompilationNodeFunctionCall* FunctionCallNode = NodeToProcess->AsType<FNiagaraCompilationNodeFunctionCall>())
					{
						if (const FNiagaraCompilationGraph* ChildGraph = FunctionCallNode->CalledGraph.Get())
						{
							auto ChildGraphRootNodeFilter = [FunctionCallNode](const FNiagaraCompilationNodeOutput& OutputNode) -> bool
							{
								return (OutputNode.Usage == FunctionCallNode->CalledScriptUsage);
							};

							ChildGraph->NodeTraversal(bRecursive, bOrdered, ChildGraphRootNodeFilter, NodeOperation);
						}
					}
				}
			}
		}
	}
}

void FNiagaraCompilationGraphInstanced::AggregateChildGraph(const FNiagaraCompilationGraphInstanced* ChildGraph)
{
	if (ChildGraph->bContainsStaticVariables)
	{
		bContainsStaticVariables = true;
	}
}

void FNiagaraCompilationGraph::CollectReachableNodes(const FNiagaraCompilationNodeOutput* OutputNode, TArray<const FNiagaraCompilationNode*>& ReachableNodes) const
{
	ReachableNodes = NiagaraCompilationImpl::CollectConnectedNodes( {OutputNode}, Nodes.Num(), FNiagaraCompilationBranchMap() );
}

void FNiagaraCompilationGraph::BuildTraversal(const FNiagaraCompilationNode* RootNode, TArray<const FNiagaraCompilationNode*>& OrderedNodes) const
{
	BuildTraversal(RootNode, FNiagaraCompilationBranchMap(), OrderedNodes);
}

void FNiagaraCompilationGraph::BuildTraversal(const FNiagaraCompilationNode* RootNode, const FNiagaraCompilationBranchMap& Branches, TArray<const FNiagaraCompilationNode*>& OrderedNodes) const
{
	using namespace NiagaraCompilationImpl;
	OrderedNodes = TopologicalSort(CollectConnectedNodes({ RootNode }, Nodes.Num(), Branches), Branches);
}

void FNiagaraCompilationGraphDigested::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(CachedDataInterfaceDuplicates);
	Collector.AddReferencedObjects(CachedDataInterfaceCDODuplicates);
	Collector.AddReferencedObjects(CachedNamedObjectAssets);
}

FString FNiagaraCompilationGraphDigested::GetReferencerName() const
{
	return TEXT("FNiagaraCompilationGraphDigested");
}

TArray<const FNiagaraCompilationNode*> FNiagaraCompilationGraph::GetOutputNodes() const
{
	TArray<const FNiagaraCompilationNode*> OutputNodes;
	OutputNodes.Reserve(OutputNodeIndices.Num());
	Algo::Transform(OutputNodeIndices, OutputNodes, [this](int32 OutputNodeIndex) -> const FNiagaraCompilationNode*
	{
		return Nodes[OutputNodeIndex].Get();
	});

	return OutputNodes;
}

FNiagaraCompilationPin::FNiagaraCompilationPin(const UEdGraphPin* InPin)
	: Direction(InPin->Direction)
{
	PinName = InPin->PinName;
	DefaultValue = InPin->DefaultValue;
	PersistentGuid = InPin->PersistentGuid;
	PinType = InPin->PinType;
	UniquePinId = InPin->PinId;
	bHidden = InPin->bHidden;
}

FNiagaraCompilationPin::FNiagaraCompilationPin(const FNiagaraCompilationPin& SourcePin, const FNiagaraCompilationNode* InOwningNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: Variable(SourcePin.Variable)
	, SourcePinIndex(SourcePin.SourcePinIndex)
	, PinName(SourcePin.PinName)
	, DefaultValue(SourcePin.DefaultValue)
	, PersistentGuid(SourcePin.PersistentGuid)
	, PinType(SourcePin.PinType)
	, UniquePinId(SourcePin.UniquePinId)
	, bHidden(SourcePin.bHidden)
	, OwningNode(InOwningNode)
	, Direction(SourcePin.Direction)
{
}

UEdGraphPin* FNiagaraCompilationPin::GetSourcePin() const
{
	return OwningNode->SourceNode->Pins[SourcePinIndex];
}

bool FNiagaraCompilationPin::IsWildcard() const
{
	return PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType && PinType.PinSubCategoryObject == FNiagaraTypeDefinition::GetWildcardStruct();
}

FNiagaraCompilationInputPin::FNiagaraCompilationInputPin(const UEdGraphPin* InPin)
	: FNiagaraCompilationPin(InPin)
{
	check(InPin->Direction == EEdGraphPinDirection::EGPD_Input);
	bDefaultValueIsIgnored = InPin->bDefaultValueIsIgnored;

	bool bForceValue = false;

	if (!bDefaultValueIsIgnored)
	{
		if ((InPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType)
			|| (InPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType)
			|| (InPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum)
			|| (InPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum))
		{
			bForceValue = InPin->LinkedTo.IsEmpty();
		}
	}

	FNameBuilder PinNameBuilder(PinName);
	const ENiagaraStructConversion StructConversion = PinNameBuilder.ToView().StartsWith(PARAM_MAP_USER_STR)
		? ENiagaraStructConversion::UserFacing
		: ENiagaraStructConversion::Simulation;

	Variable = UEdGraphSchema_Niagara::PinToNiagaraVariable(InPin, bForceValue, StructConversion);
}

FNiagaraCompilationInputPin::FNiagaraCompilationInputPin(const FNiagaraCompilationInputPin& SourceInputPin, const FNiagaraCompilationNode* InOwningNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationPin(SourceInputPin, InOwningNode, Context)
	, LinkedTo(nullptr)
	, bDefaultValueIsIgnored(SourceInputPin.bDefaultValueIsIgnored)
{
	const FNiagaraCompilationInputPin* TracedInputPin = SourceInputPin.TraceBranchMap(Context.Branches);

	if (const FNiagaraCompilationOutputPin* OutputPin = TracedInputPin->LinkedTo)
	{
		const FNiagaraCompilationNode* SourceNode = OutputPin->OwningNode;
		if (ensure(SourceNode))
		{
			FNiagaraCompilationNode* ConnectedNode = Context.DuplicatedNodeMap.FindRef(SourceNode);
			if (ensure(ConnectedNode))
			{
				// find the pin index in the array
				const int32 OutputPinIndex = UE_PTRDIFF_TO_INT32(OutputPin - SourceNode->OutputPins.GetData());
				ensure(SourceNode->OutputPins.IsValidIndex(OutputPinIndex));

				LinkedTo = &ConnectedNode->OutputPins[OutputPinIndex];
				ConnectedNode->OutputPins[OutputPinIndex].LinkedTo.Add(this);
			}
		}
	}
	// if we're not connected to anything we need to also check to see if that's because our link has
	// been terminated by a static branch.  If so we may need to update our default value based on the contents
	// of the static switch
	else if (TracedInputPin != &SourceInputPin)
	{
		const FNiagaraTypeDefinition& GenericNumericDef = FNiagaraTypeDefinition::GetGenericNumericDef();

		if (Variable.GetType() == GenericNumericDef)
		{
			const FNiagaraTypeDefinition& SourceTypeDef = TracedInputPin->Variable.GetType();
			if (SourceTypeDef != GenericNumericDef)
			{
				PinType = TracedInputPin->PinType;
				Variable.SetType(SourceTypeDef);
			}
		}

		if (ensure(FNiagaraTypeDefinition::TypesAreAssignable(Variable.GetType(), TracedInputPin->Variable.GetType())))
		{
			if (TracedInputPin->Variable.IsDataAllocated())
			{
				Variable.AllocateData();
				Variable.SetData(TracedInputPin->Variable.GetData());
			}
		}
	}
}

const FNiagaraCompilationInputPin* FNiagaraCompilationInputPin::TraceBranchMap(const FNiagaraCompilationBranchMap& BranchMap) const
{
	const FNiagaraCompilationInputPin* CurrentInputPin = this;

	while (CurrentInputPin->LinkedTo)
	{
		if (const FNiagaraCompilationInputPin* NextInputPin = BranchMap.FindRef(CurrentInputPin->LinkedTo))
		{
			CurrentInputPin = NextInputPin;
		}
		else
		{
			break;
		}
	}

	return CurrentInputPin;
}

bool FNiagaraCompilationInputPin::Disconnect()
{
	if (LinkedTo)
	{
		FNiagaraCompilationOutputPin* MutableOutputPin = const_cast<FNiagaraCompilationOutputPin*>(LinkedTo);
		LinkedTo = nullptr;

		return MutableOutputPin->LinkedTo.RemoveSingle(this) > 0;
	}

	return false;
}

FNiagaraCompilationOutputPin::FNiagaraCompilationOutputPin(const UEdGraphPin* InPin)
	: FNiagaraCompilationPin(InPin)
{
	check(InPin->Direction == EEdGraphPinDirection::EGPD_Output);

	FNameBuilder PinNameBuilder(InPin->PinName);
	const ENiagaraStructConversion StructConversion = PinNameBuilder.ToView().StartsWith(PARAM_MAP_USER_STR)
		? ENiagaraStructConversion::UserFacing
		: ENiagaraStructConversion::Simulation;

	Variable = UEdGraphSchema_Niagara::PinToNiagaraVariable(InPin, false, StructConversion);
}

FNiagaraCompilationOutputPin::FNiagaraCompilationOutputPin(const FNiagaraCompilationOutputPin& SourceOutputPin, const FNiagaraCompilationNode* InOwningNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationPin(SourceOutputPin, InOwningNode, Context)
{
	// OutputPins are updated for duplication by the input pins, but we can reserve space for the pins
	LinkedTo.Reserve(SourceOutputPin.LinkedTo.Num());
}


FNiagaraCompilationNode::FNiagaraCompilationNode(ENodeType InNodeType, const UEdGraphNode* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: NodeType(InNodeType)
{
	auto IsAddPin = [](const UEdGraphPin* Pin) -> bool
	{
		return Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryMisc &&
			Pin->PinType.PinSubCategory == UNiagaraNodeWithDynamicPins::AddPinSubCategory;
	};

	if (InNode)
	{
		NodeName = InNode->GetName();
		FullName = InNode->GetFullName();
		FullTitle = InNode->GetNodeTitle(ENodeTitleType::FullTitle).ToString();
		NodeGuid = InNode->NodeGuid;
		NodeEnabled = InNode->IsNodeEnabled();

		SourceNode = InNode;
		OwningGraph = &Context.ParentGraph;
		const UEdGraph* SourceOwningGraph = Context.ParentGraph.SourceGraph.Get();

		const int32 PinCount = SourceNode->Pins.Num();

		for (int32 PinIt = 0; PinIt < PinCount; ++PinIt)
		{
			const UEdGraphPin* Pin = SourceNode->Pins[PinIt];

			// skip add & orphaned pins
			if (IsAddPin(Pin) || Pin->bOrphanedPin)
			{
				continue;
			}

			if (Pin->bOrphanedPin)
			{
				// todo error reporting
				UE_LOGF(LogNiagaraEditor, Warning, "Node pin is no longer valid.  This pin must be disconnected or reset to default so it can be removed.");
				continue;
			}

			// in the case of the pin being connected to a node that is not in the graph's Node array we just skip the pin
			bool bConnectedToInvalidNode = false;
			for (const UEdGraphPin* LinkedPin : Pin->LinkedTo)
			{
				if (!SourceOwningGraph->Nodes.Contains(LinkedPin->GetOwningNode()))
				{
					bConnectedToInvalidNode = true;
					break;
				}
			}

			if (bConnectedToInvalidNode)
			{
				continue;
			}

			FNiagaraCompilationPin* CompilationPin = nullptr;

			if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
			{
				CompilationPin = &InputPins.Emplace_GetRef(Pin);
			}
			else
			{
				CompilationPin = &OutputPins.Emplace_GetRef(Pin);
			}
			CompilationPin->SourcePinIndex = PinIt;
			CompilationPin->OwningNode = this;
		}

		if (const UNiagaraNode* NiagaraNode = Cast<const UNiagaraNode>(InNode))
		{
			NumericSelectionMode = NiagaraNode->GetNumericOutputTypeSelectionMode();
		}
		else
		{
			NumericSelectionMode = ENiagaraNumericOutputTypeSelectionMode::None;
		}
	}
}

FNiagaraCompilationNode::FNiagaraCompilationNode(const FNiagaraCompilationNode& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: NodeType(InNode.NodeType)
	, NodeName(InNode.NodeName)
	, FullName(InNode.FullName)
	, FullTitle(InNode.FullTitle)
	, NodeGuid(InNode.NodeGuid)
	, NodeEnabled(InNode.NodeEnabled)
	, NumericSelectionMode(InNode.NumericSelectionMode)
	, OwningGraph(&Context.TargetGraph)
	, SourceNode(InNode.SourceNode)
{
	InputPins.Reserve(InNode.InputPins.Num());
	for (const FNiagaraCompilationInputPin& SourceInputPin : InNode.InputPins)
	{
		InputPins.Emplace(SourceInputPin, this, Context);
	}

	OutputPins.Reserve(InNode.OutputPins.Num());
	for (const FNiagaraCompilationOutputPin& SourceOutputPin : InNode.OutputPins)
	{
		OutputPins.Emplace(SourceOutputPin, this, Context);
	}
}

void FNiagaraCompilationNode::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	if (bRecursive)
	{
		Builder.VisitInputPins(this, bFilterForCompilation);
	}
}

void FNiagaraCompilationNode::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	Translator->Error(FText::FromString("Unimplemented Node!"), this, nullptr);
}

void FNiagaraCompilationNode::ResolveNumericPins(TConstArrayView<int32> InputPinIndices, TConstArrayView<int32> OutputPinIndices)
{
	for (int32 InputPinIndex : InputPinIndices)
	{
		FNiagaraCompilationInputPin& InputPin = InputPins[InputPinIndex];

		if (InputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType ||
			InputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType)
		{
			// If the input pin is the generic numeric type set it to the type of the linked output pin which should have been processed already.
			if (InputPin.Variable.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef() && InputPin.LinkedTo)
			{
				const FNiagaraCompilationOutputPin* LinkedOutputPin = InputPin.LinkedTo;
				const FNiagaraTypeDefinition& LinkedOutputPinType = LinkedOutputPin->Variable.GetType();
				if (LinkedOutputPinType.IsValid())
				{
					// Only update the input pin type if the linked pin type is valid.
					InputPin.PinType = LinkedOutputPin->PinType;
					InputPin.Variable.SetType(LinkedOutputPinType);
				}
			}
		}
	}

	if (NumericSelectionMode == ENiagaraNumericOutputTypeSelectionMode::None)
	{
		return;
	}

	const FNiagaraTypeDefinition& GenericTypeDef = FNiagaraTypeDefinition::GetGenericNumericDef();

	TArray<FNiagaraCompilationOutputPin*, TInlineAllocator<16>> PinsToResolve;
	for (int32 OutputPinIndex : OutputPinIndices)
	{
		if (OutputPins[OutputPinIndex].Variable.GetType() == GenericTypeDef)
		{
			PinsToResolve.Add(&OutputPins[OutputPinIndex]);
		}
	}

	if (!PinsToResolve.IsEmpty())
	{
		TArray<FNiagaraTypeDefinition, TInlineAllocator<16>> ConcreteInputTypes;
		for (int32 InputPinIndex : InputPinIndices)
		{
			if (InputPins[InputPinIndex].Variable.GetType() != GenericTypeDef)
			{
				ConcreteInputTypes.Add(InputPins[InputPinIndex].Variable.GetType());
			}
		}

		if (!ConcreteInputTypes.IsEmpty())
		{
			FNiagaraTypeDefinition ResolvedType = GenericTypeDef;

			if (NumericSelectionMode == ENiagaraNumericOutputTypeSelectionMode::Custom)
			{
				ResolvedType = ResolveCustomNumericType(ConcreteInputTypes);
			}
			else
			{
				ResolvedType = FNiagaraTypeDefinition::GetNumericOutputType(ConcreteInputTypes, NumericSelectionMode);
			}

			if (ResolvedType != GenericTypeDef)
			{
				for (FNiagaraCompilationOutputPin* PinToResolve : PinsToResolve)
				{
					PinToResolve->Variable.SetType(ResolvedType);
				}
			}
		}
	}
}

void FNiagaraCompilationNode::ResolveNumerics()
{
	TArray<int32, TInlineAllocator<16>> InputPinIndices;
	TArray<int32, TInlineAllocator<16>> OutputPinIndices;

	const int32 InputPinCount = InputPins.Num();
	const int32 OutputPinCount = OutputPins.Num();

	InputPinIndices.Reserve(InputPinCount);
	OutputPinIndices.Reserve(OutputPinCount);

	for (int32 InputPinIt = 0; InputPinIt < InputPinCount; ++InputPinIt)
	{
		InputPinIndices.Add(InputPinIt);
	}

	for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
	{
		OutputPinIndices.Add(OutputPinIt);
	}

	ResolveNumericPins(InputPinIndices, OutputPinIndices);
}

FNiagaraTypeDefinition FNiagaraCompilationNode::ResolveCustomNumericType(TConstArrayView<FNiagaraTypeDefinition> ConcreteInputTypes) const
{
	checkf(false, TEXT("Not implemented for node type"));
	return FNiagaraTypeDefinition::GetFloatDef();
}

bool FNiagaraCompilationNode::CompileInputPins(FTranslator* Translator, TArray<int32>& OutInputResults) const
{
	bool bError = false;

	OutInputResults.Reserve(InputPins.Num());
	for (const FNiagaraCompilationInputPin& InputPin : InputPins)
	{
		int32 Result = Translator->CompileInputPin(&InputPin);
		if (Result == INDEX_NONE)
		{
			bError = true;
			Translator->Error(FText::Format(LOCTEXT("CompileInputPinErrorFormat", "Error compiling Pin"), FText::FromName(InputPin.PinName)), this, &InputPin);
		}

		OutInputResults.Add(Result);
	}

	return bError;
}

bool FNiagaraCompilationNode::ConditionalRouteParameterMapAroundMe(FParameterMapHistoryBuilder& Builder) const
{
	if (!NodeEnabled && Builder.GetIgnoreDisabled())
	{
		RouteParameterMapAroundMe(Builder);
		return true;
	}

	return false;
}

void FNiagaraCompilationNode::RouteParameterMapAroundMe(FParameterMapHistoryBuilder& Builder) const
{
	const FNiagaraTypeDefinition ParameterMapDef = FNiagaraTypeDefinition::GetParameterMapDef();

	auto FindParameterMapDef = [&ParameterMapDef](const FNiagaraCompilationPin& Pin) -> bool
	{
		return Pin.Variable.GetType() == ParameterMapDef;
	};

	const FNiagaraCompilationInputPin* InputPin = InputPins.FindByPredicate(FindParameterMapDef);
	const FNiagaraCompilationOutputPin* OutputPin = OutputPins.FindByPredicate(FindParameterMapDef);

	if (InputPin && OutputPin && InputPin->LinkedTo)
	{
		const int32 PMIdx = Builder.TraceParameterMapOutputPin(InputPin->LinkedTo);
		Builder.RegisterParameterMapPin(PMIdx, OutputPin);
	}
}

void FNiagaraCompilationNode::RegisterPassthroughPin(FParameterMapHistoryBuilder& Builder, const FNiagaraCompilationInputPin* InputPin, const FNiagaraCompilationOutputPin* OutputPin, bool bFilterForCompilation, bool bVisitInputPin) const
{
	if (bVisitInputPin)
	{
		Builder.VisitInputPin(InputPin, bFilterForCompilation);
	}

	FNiagaraTypeDefinition InDef = InputPin->Variable.GetType();
	FNiagaraTypeDefinition OutDef = OutputPin->Variable.GetType();

	if (InDef == FNiagaraTypeDefinition::GetParameterMapDef() && OutDef == FNiagaraTypeDefinition::GetParameterMapDef() && InputPin->LinkedTo)
	{
		int32 PMIdx = Builder.TraceParameterMapOutputPin(InputPin->LinkedTo);
		Builder.RegisterParameterMapPin(PMIdx, OutputPin);
	}
	else if (InDef.IsStatic() && InputPin->LinkedTo)
	{
		int32 ConstantIdx = Builder.GetConstantFromOutputPin(InputPin->LinkedTo);
		Builder.RegisterConstantPin(ConstantIdx, InputPin);

		if (OutDef == InDef)
		{
			Builder.RegisterConstantPin(ConstantIdx, OutputPin);
		}
	}
	else if (InDef.IsStatic() && !InputPin->LinkedTo)
	{
		FString CachedDefaultValue;
		if (!Builder.TraversalStateContext->GetFunctionDefaultValue(NodeGuid, InputPin->PinName, CachedDefaultValue))
		{
			CachedDefaultValue = InputPin->DefaultValue;
		}

		int32 ConstantIdx = Builder.AddOrGetConstantFromValue(CachedDefaultValue);
		Builder.RegisterConstantPin(ConstantIdx, InputPin);
		if (OutDef == InDef)
		{
			Builder.RegisterConstantPin(ConstantIdx, OutputPin);
		}
	}
}

FString FNiagaraCompilationNode::GetTypeName() const
{
	check(false);
	return TEXT("<Unknown>");
}

int32 FNiagaraCompilationNode::GetInputPinIndexById(const FGuid& InId) const
{
	return NiagaraCompilationImpl::GetPinIndexById<FNiagaraCompilationInputPin>(InputPins, InId);
}

int32 FNiagaraCompilationNode::GetInputPinIndexByPersistentId(const FGuid& InId) const
{
	return NiagaraCompilationImpl::GetPinIndexByPersistentId<FNiagaraCompilationInputPin>(InputPins, InId);
}

int32 FNiagaraCompilationNode::GetOutputPinIndexById(const FGuid& InId) const
{
	return NiagaraCompilationImpl::GetPinIndexById<FNiagaraCompilationOutputPin>(OutputPins, InId);
}

const FNiagaraCompilationInputPin* FNiagaraCompilationNode::GetInputExecPin() const
{
	for (const FNiagaraCompilationInputPin& InputPin : InputPins)
	{
		if (InputPin.Variable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			return &InputPin;
		}
	}

	return nullptr;
}

const FNiagaraCompilationOutputPin* FNiagaraCompilationNode::GetOutputExecPin() const
{
	for (const FNiagaraCompilationOutputPin& OutputPin : OutputPins)
	{
		if (OutputPin.Variable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			return &OutputPin;
		}
	}

	return nullptr;
}

TArray<const FNiagaraCompilationInputPin*> FNiagaraCompilationNode::EvaluateBranches(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationBranchMap& Branches) const
{
	TArray<const FNiagaraCompilationInputPin*> ValidPins;
	ValidPins.Reserve(InputPins.Num());
	Algo::Transform(InputPins, ValidPins, [this](const FNiagaraCompilationInputPin& InputPin) -> const FNiagaraCompilationInputPin*
	{
		return &InputPin;
	});

	return ValidPins;
}

FNiagaraCompilationNodeAssignment::FNiagaraCompilationNodeAssignment(const UNiagaraNodeAssignment* InNode, FNiagaraCompilationGraphCreateContext& Context)
: FNiagaraCompilationNodeFunctionCall(InNode, Context, ENodeType::Assignment)
{
}

FNiagaraCompilationNodeAssignment::FNiagaraCompilationNodeAssignment(const FNiagaraCompilationNodeAssignment& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
: FNiagaraCompilationNodeFunctionCall(InNode, Context)
{

}

FNiagaraCompilationNodeEmitter::FNiagaraCompilationNodeEmitter(const UNiagaraNodeEmitter* InNode, FNiagaraCompilationGraphCreateContext& Context)
: FNiagaraCompilationNode(ENodeType::Emitter, InNode, Context)
{
	EmitterID = InNode->GetEmitterID();
	EmitterHandleID = InNode->GetEmitterHandleId();
	EmitterUniqueName = InNode->GetEmitterUniqueName();

	if (const UNiagaraGraph* DependentGraph = InNode->GetCalledGraph())
	{
		CalledGraph = FNiagaraDigestDatabase::Get().CreateGraphDigest(DependentGraph, Context.ChangeIdBuilder);
		if (const FNiagaraCompilationGraphDigested* DigestedGraph = CalledGraph->AsDigested())
		{
			Context.DigestedChildGraphs.AddUnique(DigestedGraph);
		}
	}

	Usage = InNode->GetUsage();
	EmitterName = InNode->GetName();
	EmitterPathName = InNode->GetPathName();
	EmitterHandleIdString = EmitterHandleID.ToString(EGuidFormats::Digits);
}


FNiagaraCompilationNodeEmitter::FNiagaraCompilationNodeEmitter(const FNiagaraCompilationNodeEmitter& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, EmitterID(InNode.EmitterID)
	, EmitterHandleID(InNode.EmitterHandleID)
	, EmitterUniqueName(InNode.EmitterUniqueName)
	, EmitterName(InNode.EmitterName)
	, EmitterPathName(InNode.EmitterPathName)
	, EmitterHandleIdString(InNode.EmitterHandleIdString)
	, Usage(InNode.Usage)
{
	// we need to replace the CalledGraph here with the graph that has been instantiated already
	const FNiagaraCompilationCopyData::FSharedCompilationCopy* EmitterCopy = Context.CopyCompilationData->EmitterData.FindByPredicate([this](const FNiagaraCompilationCopyData::FSharedCompilationCopy& EmitterCopy) -> bool
	{
		return EmitterCopy->EmitterUniqueName == EmitterUniqueName;
	});

	// because toggling the enabled state of an emitter doesn't necessarily change the emitter node in the graph
	// and so the EmitterCopy may be null in this case.  If that's true, we'll mark the node as being disabled.
	if (EmitterCopy)
	{
		CalledGraph = (*EmitterCopy)->InstantiatedGraph;
	}
	else
	{
		NodeEnabled = false;
	}
}

void FNiagaraCompilationNodeEmitter::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	if (Builder.ExclusiveEmitterHandle.IsSet() && Builder.ExclusiveEmitterHandle != EmitterHandleID)
	{
		RouteParameterMapAroundMe(Builder);
		return;
	}

	const FNiagaraFixedConstantResolver* ChildConstantResolver = Builder.ConstantResolver->FindChildResolver(EmitterHandleID);
	if (!ChildConstantResolver)
	{
		// if no child resolver was found for the specified emitter, that means that the emitter is likely not enabled and so we can proceed without
		// processing it
		RouteParameterMapAroundMe(Builder);
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	if (InputPins[0].LinkedTo)
	{
		if (bRecursive)
		{
			ParamMapIdx = Builder.TraceParameterMapOutputPin(NiagaraCompilationImpl::TraceOutputPin(Builder, InputPins[0].LinkedTo, true));
		}
		else
		{
			ParamMapIdx = Builder.CreateParameterMap();
		}
	}

	if (CalledGraph && ParamMapIdx != INDEX_NONE && Builder.bShouldBuildSubHistories)
	{
		Builder.TraversalStateContext->PushEmitter(this);
		Builder.EnterEmitter(EmitterUniqueName, CalledGraph.Get(), this);

		const TArray<ENiagaraScriptUsage> Usages =
		{
			ENiagaraScriptUsage::EmitterSpawnScript,
			ENiagaraScriptUsage::EmitterUpdateScript,
			ENiagaraScriptUsage::ParticleSpawnScript,
			ENiagaraScriptUsage::ParticleSpawnScriptInterpolated,
			ENiagaraScriptUsage::ParticleUpdateScript,
			ENiagaraScriptUsage::ParticleEventScript,
			ENiagaraScriptUsage::ParticleSimulationStageScript
		};

		uint32 NodeIdx = Builder.BeginNodeVisitation(ParamMapIdx, this);
		for (ENiagaraScriptUsage OutputNodeUsage : Usages)
		{
			TArray<const FNiagaraCompilationNodeOutput*> OutputNodes;
			CalledGraph->FindOutputNodes(OutputNodeUsage, OutputNodes);

			// Build up a new parameter map history with all the child graph nodes..
			FParameterMapHistoryBuilder ChildBuilder;
			*ChildBuilder.ConstantResolver = *ChildConstantResolver;
			ChildBuilder.RegisterEncounterableVariables(Builder.GetEncounterableVariables());
			ChildBuilder.EnableScriptAllowList(true, Usage);

			TArray<FNiagaraVariable> LocalStaticVars;
			FNiagaraParameterUtilities::FilterToRelevantStaticVariables(Builder.StaticVariables, LocalStaticVars, *EmitterUniqueName, TEXT("Emitter"), true);
			ChildBuilder.RegisterExternalStaticVariables(LocalStaticVars);
			ChildBuilder.AvailableCollections->EditCollections() = Builder.AvailableCollections->ReadCollections();

			FString LocalEmitterName = TEXT("Emitter");
			ChildBuilder.EnterEmitter(LocalEmitterName, CalledGraph.Get(), this);
			for (const FNiagaraCompilationNodeOutput* OutputNode : OutputNodes)
			{
				ChildBuilder.BuildParameterMaps(OutputNode, true);
			}
			ChildBuilder.ExitEmitter(LocalEmitterName, this);

			FNiagaraAliasContext ResolveAliasesContext(OutputNodeUsage);
			ResolveAliasesContext.ChangeEmitterToEmitterName(EmitterUniqueName);
			for (FParameterMapHistory& History : ChildBuilder.Histories)
			{
				Builder.Histories[ParamMapIdx].MapPinHistory.Append(History.MapPinHistory);
				for (int32 SrcVarIdx = 0; SrcVarIdx < History.Variables.Num(); SrcVarIdx++)
				{
					FNiagaraVariable& Var = History.Variables[SrcVarIdx];
					Var = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);

					int32 ExistingIdx = Builder.Histories[ParamMapIdx].FindVariable(Var.GetName(), Var.GetType());
					if (ExistingIdx == INDEX_NONE)
					{
						ExistingIdx = Builder.AddVariableToHistory(Builder.Histories[ParamMapIdx], Var, History.VariablesWithOriginalAliasesIntact[SrcVarIdx], nullptr);
					}
					ensure(ExistingIdx < Builder.Histories[ParamMapIdx].PerVariableWarnings.Num());
					ensure(ExistingIdx < Builder.Histories[ParamMapIdx].PerVariableReadHistory.Num());
					ensure(ExistingIdx < Builder.Histories[ParamMapIdx].PerVariableWriteHistory.Num());
					Builder.Histories[ParamMapIdx].PerVariableReadHistory[ExistingIdx].Append(History.PerVariableReadHistory[SrcVarIdx]);
					Builder.Histories[ParamMapIdx].PerVariableWriteHistory[ExistingIdx].Append(History.PerVariableWriteHistory[SrcVarIdx]);
					Builder.Histories[ParamMapIdx].PerVariableWarnings[ExistingIdx].Append(History.PerVariableWarnings[SrcVarIdx]);
					for (int32 PerConstantIdx = 0; PerConstantIdx < History.PerVariableConstantValue[SrcVarIdx].Num(); PerConstantIdx++)
					{
						const FString& ConstantStr = History.PerVariableConstantValue[SrcVarIdx][PerConstantIdx];
						Builder.Histories[ParamMapIdx].PerVariableConstantValue[ExistingIdx].AddUnique(ConstantStr);
					}
				}
				Builder.Histories[ParamMapIdx].EncounteredParameterCollections.Append(History.EncounteredParameterCollections);
				Builder.Histories[ParamMapIdx].PinToConstantValues.Append(History.PinToConstantValues);
			}

			ResolveAliasesContext.ChangeRapidIterationParameterMode(FNiagaraAliasContext::ERapidIterationParameterMode::StaticVariables);

			// We only want to push out appropriately scoped static variables that should be in the system builder, not in-betweens like "Module.MyInputVar"
			// or per-particle or others vars. Really only want Emitter or System parameters here.
			for (int32 StaticVarIdx = 0; StaticVarIdx < ChildBuilder.StaticVariables.Num(); StaticVarIdx++)
			{
				const FNiagaraVariable& ChildStaticVar = ChildBuilder.StaticVariables[StaticVarIdx];
				if (ChildBuilder.StaticVariableExportable[StaticVarIdx])
				{
					// Should match logic in FNiagaraParameterMapHistoryBuilder::RegisterConstantVariableWrite
					FNiagaraVariable ResolvedStaticVar = FNiagaraUtilities::ResolveAliases(ChildStaticVar, ResolveAliasesContext);

					// Index of uses == operator, which only checks name and type. This will allow us to detect instances of the duplicate
					// data down the line.
					int32 FoundStaticVarIdx = Builder.StaticVariables.Find(ResolvedStaticVar);

					if (FoundStaticVarIdx == INDEX_NONE) // Didn't find it, so add it.
					{
						Builder.StaticVariables.Add(ResolvedStaticVar);
						Builder.StaticVariableExportable.Emplace(true);
					}
					else if (false == Builder.StaticVariables[FoundStaticVarIdx].HoldsSameData(ResolvedStaticVar))
					{
						Builder.StaticVariables.Add(ResolvedStaticVar);// Add as a duplicate here. We will filter out later
						Builder.StaticVariableExportable.Emplace(true);
					}

					ensure(Builder.StaticVariables.Num() == Builder.StaticVariableExportable.Num());
				}
			}
		}

		Builder.EndNodeVisitation(ParamMapIdx, NodeIdx);
		Builder.ExitEmitter(EmitterUniqueName, this);
		Builder.TraversalStateContext->PopEmitter(this);
	}

	for (const FNiagaraCompilationOutputPin& Pin : OutputPins)
	{
		if (Pin.Variable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			Builder.RegisterParameterMapPin(ParamMapIdx, &Pin);
		}
	}
}

void FNiagaraCompilationNodeEmitter::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	check(Outputs.Num() == 0);

	// First compile fully down the hierarchy for our predecessors..
	TArray<const FNiagaraCompilationNodeInput*> InputNodes;
	UNiagaraGraph::FFindInputNodeOptions Options;
	Options.bSort = true;
	Options.bFilterDuplicates = true;
	Options.bFilterByScriptUsage = true;
	Options.TargetScriptUsage = Translator->GetTargetUsage() == ENiagaraScriptUsage::SystemSpawnScript ? ENiagaraScriptUsage::EmitterSpawnScript : ENiagaraScriptUsage::EmitterUpdateScript;

	if (CalledGraph && NodeEnabled) // Called graph may be null on an disabled emitter
	{
		CalledGraph->FindInputNodes(InputNodes, Options);
	}

	TArray<int32> CompileInputs;

	if (InputPins.Num() > 1)
	{
		Translator->Error(LOCTEXT("TooManyOutputPinsError", "Too many input pins on node."), this, nullptr);
		return;
	}

	int32 InputPinCompiled = Translator->CompileInputPin(&InputPins[0]);
	if (!NodeEnabled)
	{
		// Do the minimal amount of work necessary if we are disabled.
		CompileInputs.Reserve(1);
		CompileInputs.Add(InputPinCompiled);
		Translator->Emitter(this, CompileInputs, Outputs);
		return;
	}

	if (InputNodes.Num() <= 0)
	{
		Translator->Error(LOCTEXT("InputNodesNotFound", "Input nodes on called graph not found"), this, nullptr);
		return;
	}

	CompileInputs.Reserve(InputNodes.Num());

	bool bError = false;
	const FNiagaraVariable InputMapVariable = FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("InputMap"));
	for (const FNiagaraCompilationNodeInput* EmitterInputNode : InputNodes)
	{
		if (EmitterInputNode->InputVariable.IsEquivalent(InputMapVariable))
		{
			CompileInputs.Add(InputPinCompiled);
		}
		else
		{
			CompileInputs.Add(INDEX_NONE);
		}
	}

	if (!bError)
	{
		Translator->Emitter(this, CompileInputs, Outputs);
	}
}

FNiagaraCompilationNodeFunctionCall::FNiagaraCompilationNodeFunctionCall(const UNiagaraNodeFunctionCall* InNode, FNiagaraCompilationGraphCreateContext& Context, ENodeType InNodeType)
: FNiagaraCompilationNode(InNodeType, InNode, Context)
, CalledGraph(nullptr)
{
	if (const UNiagaraGraph* DependentGraph = InNode->GetCalledGraph())
	{
		CalledGraph = FNiagaraDigestDatabase::Get().CreateGraphDigest(DependentGraph, Context.ChangeIdBuilder);
		if (const FNiagaraCompilationGraphDigested* DigestedGraph = CalledGraph->AsDigested())
		{
			Context.DigestedChildGraphs.AddUnique(DigestedGraph);
		}
	}

	// Top level functions (functions invoked in the root graph) will use the serialized DebugState value, all others will
	// use the value cached during traversal based on the bInheritDebugState flag and the system's current value.  With
	// NoDebug being used when things are not inherited.  This seems like a bug and top level functions should probably also
	// be using NoDebug in the case of !bInheritDebugStatus
	DebugState = InNode->DebugState;

	FunctionName = InNode->GetFunctionName();

	// translating the hlsl uses the FunctionScriptName as the function name.  In the original implementation
	// this results in function names being NiagaraScript_34_Func_(), which isn't great.  This attempts to
	// preserve the actual function name but we need to be able to disambiguate between potential conflicts.
	// For now this just includes the version of the graph...outside of that things can get a bit dicey (do we
	// really want to be able to support having the same name used places?)
	if (UNiagaraScript* FunctionScript = InNode->FunctionScript)
	{
		FunctionScriptName = FunctionScript->GetName();
		if (InNode->SelectedScriptVersion.IsValid() && FunctionScript->IsVersioningEnabled())
		{
			if (const FNiagaraAssetVersion* ScriptVersion = FunctionScript->FindVersionData(InNode->SelectedScriptVersion))
			{
				FunctionScriptName.Appendf(TEXT("_v%d_%d"), ScriptVersion->MajorVersion, ScriptVersion->MinorVersion);
			}
		}
	}

	Signature = InNode->Signature;
	bInheritDebugState = InNode->bInheritDebugStatus;
	PropagatedStaticSwitchParameters.Reserve(InNode->PropagatedStaticSwitchParameters.Num());
	for (const FNiagaraPropagatedVariable& PropagatedVariable : InNode->PropagatedStaticSwitchParameters)
	{
		const FName PropagatedName = PropagatedVariable.PropagatedName.IsEmpty()
			? PropagatedVariable.SwitchParameter.GetName()
			: *PropagatedVariable.PropagatedName;

		const FTaggedVariable& TaggedVariable = PropagatedStaticSwitchParameters.Emplace_GetRef(
			PropagatedVariable.SwitchParameter, PropagatedName);

		Context.ParentGraph.StaticSwitchInputs.AddUnique(FNiagaraVariable(TaggedVariable.Key.GetType(), TaggedVariable.Value));
	}
	CalledScriptUsage = InNode->GetCalledUsage();

	if (!CalledGraph && Signature.IsValid())
	{
		Signature.FunctionSpecifiers = InNode->FunctionSpecifiers;
	}
	bValidateDataInterfaces = InNode->GetValidateDataInterfaces();
}

FNiagaraCompilationNodeFunctionCall::FNiagaraCompilationNodeFunctionCall(const FNiagaraCompilationNodeFunctionCall& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, CalledGraph(InNode.CalledGraph)
	, DebugState(InNode.DebugState)
	, FunctionName(InNode.FunctionName)
	, FunctionScriptName(InNode.FunctionScriptName)
	, Signature(InNode.Signature)
	, CalledScriptUsage(InNode.CalledScriptUsage)
	, PropagatedStaticSwitchParameters(InNode.PropagatedStaticSwitchParameters)
	, bInheritDebugState(InNode.bInheritDebugState)
	, bValidateDataInterfaces(InNode.bValidateDataInterfaces)
{
	if (CalledGraph)
	{
		Context.FunctionsRequiresGraph.Add(this);
	}

	for (const FTaggedVariable& PropagatedVariable : InNode.PropagatedStaticSwitchParameters)
	{
		Context.TargetGraph.StaticSwitchInputs.AddUnique(FNiagaraVariable(PropagatedVariable.Key.GetType(), PropagatedVariable.Value));
	}
}

void FNiagaraCompilationNodeFunctionCall::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	// when dealing with function scripts it's not sufficient to assume that InputPin[0] is the ParamMap pin, and so we
	// need to search through the inputs to find the right one
	const int32 ParamMapPinIndex = InputPins.IndexOfByPredicate([](const FNiagaraCompilationInputPin& InputPin) -> bool
	{
		return InputPin.Variable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef();
	});
	const bool bHasParamMapPin = ParamMapPinIndex != INDEX_NONE;
	const FNiagaraCompilationInputPin* ParamMapPin = bHasParamMapPin ? &InputPins[ParamMapPinIndex] : nullptr;
	if (bHasParamMapPin && !ParamMapPin->LinkedTo)
	{
		// Looks like this function call is not yet hooked up. Skip it to prevent cascading errors in the compilation
		return;
	}

	FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	const bool ValidScriptAndGraph = CalledGraph.IsValid();

	if (!ValidScriptAndGraph)
	{
		if (!Signature.IsValid() || Signature.bRequiresExecPin)
		{
			RouteParameterMapAroundMe(Builder);
		}
		return;
	}

	const FNiagaraCompilationNodeOutput* OutputNode = CalledGraph->FindOutputNode(ENiagaraScriptUsage::Function, FGuid());
	if (OutputNode == nullptr)
	{
		OutputNode = CalledGraph->FindOutputNode(ENiagaraScriptUsage::Module, FGuid());
	}
	if (OutputNode == nullptr)
	{
		OutputNode = CalledGraph->FindOutputNode(ENiagaraScriptUsage::DynamicInput, FGuid());
	}

	// Because the pin traversals above the the Super::BuildParameterMapHistory don't traverse into the input pins if they aren't linked.
	// This leaves static variables that aren't linked to anything basically ignored. The code below fills in that gap.
	// We don't do it in the base code because many input pins are traversed only if the owning node wants them to be.
	// OutputNode is another example of manually registering the input pins.
	for (const FNiagaraCompilationInputPin& InputPin : InputPins)
	{
		// Make sure to register pin constants
		if (!InputPin.LinkedTo && InputPin.Variable.GetType().IsStatic() && InputPin.DefaultValue.Len() != 0)
		{
			FString CachedPinDefaultValue;
			if (!Builder.TraversalStateContext->GetFunctionDefaultValue(NodeGuid, InputPin.PinName, CachedPinDefaultValue))
			{
				CachedPinDefaultValue = InputPin.DefaultValue;
			}
			Builder.RegisterConstantFromInputPin(&InputPin, CachedPinDefaultValue);
		}
	}

	ENiagaraFunctionDebugState CachedDebugState;
	if (!Builder.TraversalStateContext->GetFunctionDebugState(NodeGuid, CachedDebugState))
	{
		CachedDebugState = DebugState;
	}

	FNiagaraFixedConstantResolver FunctionResolver = Builder.ConstantResolver->WithDebugState(CachedDebugState);
	if (Builder.HasCurrentUsageContext())
	{
		// if we traverse a full emitter graph the usage might change during the traversal, so we need to update the constant resolver
		FunctionResolver = FunctionResolver.WithUsage(Builder.GetCurrentUsageContext());
	}

	int32 ParamMapIdx = INDEX_NONE;
	uint32 NodeIdx = INDEX_NONE;

	if (bHasParamMapPin && bRecursive)
	{
		ParamMapIdx = Builder.TraceParameterMapOutputPin(ParamMapPin->LinkedTo);
	}

	Builder.TraversalStateContext->PushFunction(this, FunctionResolver);
	Builder.EnterFunction(FunctionName, CalledGraph.Get(), this);
	if (ParamMapIdx != INDEX_NONE)
	{
		NodeIdx = Builder.BeginNodeVisitation(ParamMapIdx, this);
	}

	// check if we should be recursing deeper into the graph
	const bool DoDepthTraversal = (Builder.MaxGraphDepthTraversal == INDEX_NONE || Builder.CurrentGraphDepth < Builder.MaxGraphDepthTraversal);

	if (DoDepthTraversal)
	{

		++Builder.CurrentGraphDepth;

		const bool bChildRecursive = true;
		OutputNode->BuildParameterMapHistory(Builder, bChildRecursive, bFilterForCompilation);

		--Builder.CurrentGraphDepth;
	}

	// Since we're about to lose the pin calling context, we finish up the function call parameter map pin wiring
	// here when we have the calling context and the child context still available to us...
	TArray<TPair<const FNiagaraCompilationOutputPin*, int32>, TInlineAllocator<16> > MatchedPairs;
	TArray<TPair<const FNiagaraCompilationOutputPin*, int32>, TInlineAllocator<16> > MatchedConstants;
	TArray<bool, TInlineAllocator<16> > OutputMatched;

	const int32 OutputPinCount = OutputPins.Num();

	OutputMatched.AddDefaulted(OutputPinCount);

	// Find the matches of names and types of the sub-graph output pins and this function call nodes' outputs.
	for (const FNiagaraCompilationInputPin& ChildOutputNodeInputPin : OutputNode->InputPins)
	{
		const FNiagaraVariable& VarChild = ChildOutputNodeInputPin.Variable;

		if (ChildOutputNodeInputPin.LinkedTo && VarChild.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
			{
				const FNiagaraCompilationOutputPin& OutputPin = OutputPins[OutputPinIt];
				if (OutputPin.Variable.IsEquivalent(VarChild))
				{
					MatchedPairs.Emplace(&OutputPin, Builder.TraceParameterMapOutputPin(ChildOutputNodeInputPin.LinkedTo));
					OutputMatched[OutputPinIt] = true;
				}
			}
		}
		else if (VarChild.GetType().IsStatic())
		{
			for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
			{
				const FNiagaraCompilationOutputPin& OutputPin = OutputPins[OutputPinIt];
				if (!OutputMatched[OutputPinIt] && OutputPin.Variable.IsEquivalent(VarChild))
				{
					MatchedConstants.Emplace(&OutputPin, Builder.GetConstantFromInputPin(&ChildOutputNodeInputPin));
					OutputMatched[OutputPinIt] = true;
				}
			}
		}
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		Builder.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}

	Builder.ExitFunction(this);
	Builder.TraversalStateContext->PopFunction(this);

	if (DoDepthTraversal)
	{
		for (int32 i = 0; i < MatchedPairs.Num(); i++)
		{
			Builder.RegisterParameterMapPin(MatchedPairs[i].Value, MatchedPairs[i].Key);
		}

		for (int32 i = 0; i < MatchedConstants.Num(); i++)
		{
			Builder.RegisterConstantPin(MatchedConstants[i].Value, MatchedConstants[i].Key);
		}
	}
	else
	{
		for (const FNiagaraCompilationOutputPin& OutputPin : OutputPins)
		{
			if (OutputPin.Variable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				Builder.RegisterParameterMapPin(ParamMapIdx, &OutputPin);
				break;
			}
		}
	}
}

TSet<FName> FNiagaraCompilationNodeFunctionCall::GetUnusedFunctionInputPins() const
{
	if (!CalledGraph
		|| CalledScriptUsage != ENiagaraScriptUsage::Module
		|| InputPins.IsEmpty()
		|| InputPins[0].LinkedTo == nullptr)
	{
		return TSet<FName>();
	}

	// Find the start node for the traversal
	const FNiagaraCompilationNodeOutput* OutputNode = CalledGraph->FindOutputNode(ENiagaraScriptUsage::Module, FGuid());
	if (OutputNode == nullptr)
	{
		return TSet<FName>();
	}

	TStringBuilder<512> FunctionNameWithDelimiter;
	FunctionNameWithDelimiter.Append(FunctionName);
	FunctionNameWithDelimiter.Append(TEXT("."));

	const TCHAR* ModuleWithDelimiter = TEXT("Module.");

	// Get the used function parameters from the parameter map set node linked to the function's input pin.
	// Note that this is only valid for module scripts, not function scripts.
	TArray<const FNiagaraCompilationInputPin*> ResultPins;
	const FNiagaraCompilationInputPin& ParameterMapInputPin = InputPins[0];

	if (ParameterMapInputPin.LinkedTo->OwningNode->NodeType == ENodeType::ParameterMapSet)
	{
		for (const FNiagaraCompilationInputPin& ParamMapNodeInputPin : ParameterMapInputPin.LinkedTo->OwningNode->InputPins)
		{
			FNameBuilder PinNameBuilder(ParamMapNodeInputPin.PinName);
			if (PinNameBuilder.ToView().StartsWith(FunctionNameWithDelimiter))
			{
				ResultPins.Add(&ParamMapNodeInputPin);
			}
		}
	}

	if (ResultPins.Num() == 0)
	{
		return TSet<FName>();
	}

	// Find reachable nodes
	TArray<const FNiagaraCompilationNode*> ReachableNodes;
	CalledGraph->CollectReachableNodes(OutputNode, ReachableNodes);

	for (const FNiagaraCompilationNode* ReachableNode : ReachableNodes)
	{
		if (ReachableNode->NodeType == ENodeType::ParameterMapGet)
		{
			for (const FNiagaraCompilationOutputPin& ParamMapNodeOutputPin : ReachableNode->OutputPins)
			{
				if (ParamMapNodeOutputPin.LinkedTo.Num() == 0)
				{
					continue;
				}

				FNameBuilder PinNameBuilder(ParamMapNodeOutputPin.PinName);
				if (!PinNameBuilder.ToView().StartsWith(ModuleWithDelimiter))
				{
					continue;
				}

				TStringBuilder<512> ResolvedNamespacePinName;
				ResolvedNamespacePinName.Append(FunctionNameWithDelimiter);
				ResolvedNamespacePinName.Append(PinNameBuilder.ToView().RightChop(FCString::Strlen(ModuleWithDelimiter)));

				const FNiagaraVariable VariableToFind(ParamMapNodeOutputPin.Variable.GetType(), FName(ResolvedNamespacePinName));

				ResultPins.SetNum(Algo::RemoveIf(ResultPins, [&VariableToFind](const FNiagaraCompilationInputPin* ResultPin) -> bool
				{
					return ResultPin->Variable == VariableToFind;
				}));

				/*
				for (TArray<const FNiagaraCompilationInputPin*>::TIterator It(ResultPins); It; ++It)
				{
					const FNiagaraCompilationInputPin* ResultPin = *It;
					if (ResultPin->PinType == ParamMapNodeOutputPin.PinType
						&& ResultPin->PinName == FName(ResolvedNamespacePinName))
					{
						It.RemoveCurrentSwap();
					}
				}
				*/
			}
		}
	}

	TSet<FName> UnusedPinNames;
	UnusedPinNames.Reserve(ResultPins.Num());
	for (const FNiagaraCompilationInputPin* InputPin : ResultPins)
	{
		UnusedPinNames.Add(InputPin->PinName);
	}

	return UnusedPinNames;
}

void FNiagaraCompilationNodeFunctionCall::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	TArray<int32> Inputs;

	bool bError = false;

	if (CalledGraph)
	{
		TArray<const FNiagaraCompilationNodeInput*> FunctionInputNodes;
		UNiagaraGraph::FFindInputNodeOptions Options;
		Options.bSort = true;
		Options.bFilterDuplicates = true;
		CalledGraph->FindInputNodes(FunctionInputNodes, Options);

		// We check which module inputs are not used so we can later remove them from the compilation of the
		// parameter map that sets the input values for our function. This is mainly done to prevent data interfaces being
		// initialized as parameter when they are not used in the function or module.
		TSet<FName> HiddenPinNames = GetUnusedFunctionInputPins();
		Translator->EnterFunctionCallNode(HiddenPinNames);

		for (const FNiagaraCompilationNodeInput* FunctionInputNode : FunctionInputNodes)
		{
			const FNiagaraTypeDefinition& InputNodeType = FunctionInputNode->InputVariable.GetType();

			TOptional<FNiagaraTypeDefinition> SWCType;
			if (FNiagaraTypeHelper::IsLWCType(InputNodeType))
			{
				SWCType = FNiagaraTypeHelper::GetSWCType(InputNodeType);
			}

			//Finds the matching Pin in the caller.
			auto MatchInputNodePredicate = [&FunctionInputNode, &SWCType](const FNiagaraCompilationInputPin& InputPin) -> bool
			{
				if (InputPin.Variable.GetName() != FunctionInputNode->InputVariable.GetName())
				{
					return false;
				}

				if (InputPin.Variable.IsEquivalent(FunctionInputNode->InputVariable))
				{
					return true;
				}

				// the last thing we need to worry about is when types differ because of LWC vs SWC concerns
				if (SWCType.IsSet())
				{
					return InputPin.Variable.GetType() == *SWCType;
				}

				return false;
			};

			const FNiagaraCompilationInputPin* CallerPin = InputPins.FindByPredicate(MatchInputNodePredicate);

			if (!CallerPin)
			{
				if (FunctionInputNode->bExposed)
				{
					//Couldn't find the matching pin for an exposed input. Probably a stale function call node that needs to be refreshed.
					Translator->Error(LOCTEXT("StaleFunctionCallError", "Function call is stale and needs to be refreshed."), this, nullptr);
					bError = true;
				}
				else if (FunctionInputNode->bRequired)
				{
					// Not exposed, but required. This means we should just add as a constant.
					Inputs.Add(Translator->GetConstant(FunctionInputNode->InputVariable));
					continue;
				}


				Inputs.Add(INDEX_NONE);
				continue;
			}

			const FNiagaraCompilationOutputPin* CallerLinkedTo = CallerPin->LinkedTo;
			FNiagaraVariable PinVar = CallerPin->Variable;
			if (!CallerLinkedTo)
			{
				//if (Translator->CanReadAttributes())
				{
					//Try to auto bind if we're not linked to by the caller.
					FNiagaraVariable AutoBoundVar;
					ENiagaraInputNodeUsage AutBoundUsage = ENiagaraInputNodeUsage::Undefined;
					if (FindAutoBoundInput(FunctionInputNode, CallerPin, AutoBoundVar, AutBoundUsage))
					{
						check(false);
					}
				}
			}

			if (CallerLinkedTo)
			{
				//Param is provided by the caller. Typical case.
				Inputs.Add(Translator->CompileOutputPin(CallerLinkedTo));
				continue;
			}
			else
			{
				if (FunctionInputNode->bRequired && FunctionInputNode->bExposed)
				{
					if (CallerPin->bDefaultValueIsIgnored)
					{
						//This pin can't use a default and it is required so flag an error.
						Translator->Error(FText::Format(LOCTEXT("RequiredInputUnboundErrorFmt", "Required input {0} was not bound and could not be automatically bound."), FText::FromName(CallerPin->PinName)),
							this, CallerPin);
						bError = true;
						//We weren't linked to anything and we couldn't auto bind so tell the compiler this input isn't provided and it should use it's local default.
						Inputs.Add(INDEX_NONE);
					}
					else
					{
						//We also compile the pin anyway if it is required as we'll be attempting to use it's inline default.
						Inputs.Add(Translator->CompileInputPin(CallerPin));
					}
				}
				else
				{
					//We optional, weren't linked to anything and we couldn't auto bind so tell the compiler this input isn't provided and it should use it's local default.
					Inputs.Add(INDEX_NONE);
				}
			}
		}

		Translator->ExitFunctionCallNode();
	}
	else if (Signature.IsValid())
	{
		if (Signature.Inputs.Num() > 0)
		{
			if (bValidateDataInterfaces && Signature.Inputs[0].GetType().IsDataInterface())
			{
				UClass* DIClass = Signature.Inputs[0].GetType().GetClass();
				if (UNiagaraDataInterface* DataInterfaceCDO = Cast<UNiagaraDataInterface>(DIClass->GetDefaultObject()))
				{
					TArray<FText> ValidationErrors;
					DataInterfaceCDO->ValidateFunction(Signature, ValidationErrors);

					bError = ValidationErrors.Num() > 0;

					for (FText& ValidationError : ValidationErrors)
					{
						Translator->Error(ValidationError, this, nullptr);
					}

					if (bError)
					{
						return;
					}
				}
			}
		}
		Translator->EnterFunctionCallNode(TSet<FName>());
		bError = CompileInputPins(Translator, Inputs);
		Translator->ExitFunctionCallNode();
	}
	else
	{
		Translator->Error(FText::Format(LOCTEXT("UnknownFunction", "Unknown Function Call! Missing Script or Data Interface Signature. Stack Name: {0}"), FText::FromString(FunctionName)), this, nullptr);
		bError = true;
	}

	if (!bError)
	{
		Translator->FunctionCall(this, Inputs, Outputs);
	}
}

const FNiagaraCompilationInputPin* FNiagaraCompilationNodeFunctionCall::FindStaticSwitchInputPin(FName VariableName) const
{
	if (CalledGraph)
	{
		for (const FNiagaraVariableBase& StaticSwitchInput : CalledGraph->StaticSwitchInputs)
		{
			if (StaticSwitchInput.GetName() == VariableName)
			{
				for (const FNiagaraCompilationInputPin& InputPin : InputPins)
				{
					if (InputPin.PinName == VariableName)
					{
						return &InputPin;
					}
				}
			}
		}
	}

	return nullptr;
}

bool FNiagaraCompilationNodeFunctionCall::FindAutoBoundInput(const FNiagaraCompilationNodeInput* InputNode, const FNiagaraCompilationInputPin* PinToAutoBind, FNiagaraVariable& OutFoundVar, ENiagaraInputNodeUsage& OutNodeUsage) const
{
	if (!PinToAutoBind || PinToAutoBind->LinkedTo || !(InputNode->bExposed && InputNode->bCanAutoBind))
	{
		return false;
	}
	ensureMsgf(InputNode->bExposed == true, TEXT("AutoBind inputs should be exposed for Function(%s) Pin(%s)"), *FunctionName, *PinToAutoBind->PinName.ToString());

	FNiagaraVariable PinVar = PinToAutoBind->Variable;

	//See if we can auto bind this pin to something in the caller script.
	const FNiagaraCompilationGraph* CallerGraph = OwningGraph;
	check(CallerGraph);
	const FNiagaraCompilationNodeOutput* CallerOutputNodeSpawn = CallerGraph->FindOutputNode(ENiagaraScriptUsage::ParticleSpawnScript, FGuid());
	const FNiagaraCompilationNodeOutput* CallerOutputNodeUpdate = CallerGraph->FindOutputNode(ENiagaraScriptUsage::ParticleUpdateScript, FGuid());

	//First, let's see if we're an attribute of this emitter. Only valid if we're a module call off the primary script.
	if (CallerOutputNodeSpawn || CallerOutputNodeUpdate)
	{
		const FNiagaraCompilationNodeOutput* CallerOutputNode = CallerOutputNodeSpawn != nullptr ? CallerOutputNodeSpawn : CallerOutputNodeUpdate;
		check(CallerOutputNode);
		{
			const FNiagaraVariable* AttrVarPtr = CallerOutputNode->Outputs.FindByPredicate([&PinVar](const FNiagaraVariable& Attr) -> bool
			{
				return PinVar.IsEquivalent(Attr);
			});

			if (AttrVarPtr)
			{
				OutFoundVar = *AttrVarPtr;
				OutNodeUsage = ENiagaraInputNodeUsage::Attribute;
				return true;
			}
		}
	}

	//Next, lets see if we are a system constant.
	//Do we need a smarter (possibly contextual) handling of system constants?
	const TArray<FNiagaraVariable>& SysConstants = FNiagaraConstants::GetEngineConstants();
	if (SysConstants.Contains(PinVar))
	{
		OutFoundVar = PinVar;
		OutNodeUsage = ENiagaraInputNodeUsage::SystemConstant;
		return true;
	}

	//Unable to auto bind.
	return false;
}

const FNiagaraCompilationNodeParameterMapSet* FNiagaraCompilationNodeFunctionCall::GetOverrideNode() const
{
	if (const FNiagaraCompilationInputPin* ExecPin = GetInputExecPin())
	{
		if (const FNiagaraCompilationOutputPin* SourcePin = ExecPin->LinkedTo)
		{
			return SourcePin->OwningNode->AsType<FNiagaraCompilationNodeParameterMapSet>();
		}
	}

	return nullptr;
}

bool FNiagaraCompilationNodeFunctionCall::HasOverridePin(const FNiagaraParameterHandle& ParameterHandle) const
{
	// static switch
	if (FindStaticSwitchInputPin(ParameterHandle.GetName()))
	{
		return true;
	}

	// override node - find the ParamMapSet preceding this function call
	if (const FNiagaraCompilationNodeParameterMapSet* OverrideNode = GetOverrideNode())
	{
		const FName ParameterHandleName = ParameterHandle.GetParameterHandleString();
		return OverrideNode->InputPins.ContainsByPredicate([ParameterHandleName](const FNiagaraCompilationInputPin& InputPin) -> bool
		{
			return InputPin.PinName == ParameterHandleName;
		});
	}

	return false;
}

void FNiagaraCompilationNodeFunctionCall::MultiFindParameterMapDefaultValues(ENiagaraScriptUsage ParentScriptUsage, const FNiagaraFixedConstantResolver& ConstantResolver, TArrayView<FNiagaraVariable> Variables) const
{
	if (!Variables.IsEmpty() && CalledGraph.IsValid())
	{
		FNiagaraCompilationBranchMap Branches;

		FNiagaraCompilationGraphInstanceContext DummyContext(ConstantResolver, nullptr, nullptr);
		DummyContext.EnterFunction(this);
		CalledGraph->EvaluateStaticBranches(DummyContext, Branches);
	
		TArray<const FNiagaraCompilationNodeOutput*> OutputNodes;
		CalledGraph->FindOutputNodes(CalledScriptUsage, OutputNodes);

		for (const FNiagaraCompilationNodeOutput* OutputNode : OutputNodes)
		{
			TArray<const FNiagaraCompilationNode*> ReachableNodes;
			CalledGraph->BuildTraversal(OutputNode, Branches, ReachableNodes);

			for (const FNiagaraCompilationNode* ReachableNode : ReachableNodes)
			{
				if (const FNiagaraCompilationNodeParameterMapGet* MapGetNode = ReachableNode->AsType<FNiagaraCompilationNodeParameterMapGet>())
				{
					for (FNiagaraVariable& Variable : Variables)
					{
						// only worry about the ones we haven't processed yet
						if (!Variable.IsDataAllocated())
						{
							int32 OutputPinIt = MapGetNode->OutputPins.IndexOfByPredicate([Variable](const FNiagaraCompilationOutputPin& OutputPin) -> bool
							{
								return OutputPin.PinName == Variable.GetName();
							});

							if (MapGetNode->DefaultInputPinIndices.IsValidIndex(OutputPinIt))
							{
								const int32 DefaultPinIt = MapGetNode->DefaultInputPinIndices[OutputPinIt];
								if (MapGetNode->InputPins.IsValidIndex(DefaultPinIt))
								{
									const FNiagaraCompilationInputPin& DefaultPin = MapGetNode->InputPins[DefaultPinIt];

									if (!DefaultPin.LinkedTo)
									{
										Variable.SetData(DefaultPin.Variable.GetData());
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

struct FNiagaraCompilationNodeCustomHlsl::FIndexedTokenLess
{
	FIndexedTokenLess(TConstArrayView<FString> InStringsView)
	: StringsView(InStringsView)
	{}

	bool operator()(int32 LhsIndex, int32 RhsIndex) const
	{
		return StringsView[LhsIndex].Compare(StringsView[RhsIndex]) < 0;
	}

	TConstArrayView<FString> StringsView;
};

struct FNiagaraCompilationNodeCustomHlsl::FStringViewLess
{
	bool operator()(FStringView LhsValue, FStringView RhsValue) const
	{
		return LhsValue.Compare(RhsValue) < 0;
	}
};

FNiagaraCompilationNodeCustomHlsl::FNiagaraCompilationNodeCustomHlsl(const UNiagaraNodeCustomHlsl* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNodeFunctionCall(InNode, Context, ENodeType::CustomHlsl)
{
	CustomScriptUsage = InNode->ScriptUsage;
	Signature = InNode->Signature;
	CustomHlsl = InNode->GetCustomHlsl();
	TArray<FStringView> TokenViews;
	InNode->GetTokens(TokenViews, false, false);

	const int32 TokenCount = TokenViews.Num();
	Tokens.Reserve(TokenCount);
	SortedTokenIndices.Reserve(TokenCount);

	int32 TokenIt = 0;
	for (const FStringView View : TokenViews)
	{
		Tokens.Push(FString(View));
		SortedTokenIndices.Add(TokenIt++);
	}
	Algo::StableSort(SortedTokenIndices, FIndexedTokenLess(Tokens));

	InNode->GetIncludeFilePaths(CustomIncludePaths);

	bCallsImpureFunctions = InNode->CallsImpureDataInterfaceFunctions();
}

FNiagaraCompilationNodeCustomHlsl::FNiagaraCompilationNodeCustomHlsl(const FNiagaraCompilationNodeCustomHlsl& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNodeFunctionCall(InNode, Context)
	, CustomScriptUsage(InNode.CustomScriptUsage)
	, Signature(InNode.Signature)
	, CustomHlsl(InNode.CustomHlsl)
	, Tokens(InNode.Tokens)
	, SortedTokenIndices(InNode.SortedTokenIndices)
	, CustomIncludePaths(InNode.CustomIncludePaths)
	, bCallsImpureFunctions(InNode.bCallsImpureFunctions)
{

}

void FNiagaraCompilationNodeCustomHlsl::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	FNiagaraCompilationNodeFunctionCall::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	// This only works currently if the input pins are in the same order as the signature pins.
	if (InputPins.Num() == Signature.Inputs.Num() && OutputPins.Num() == Signature.Outputs.Num())
	{
		TArray<FNiagaraVariable> LocalVars;
		bool bHasParamMapInput = false;
		bool bHasParamMapOutput = false;
		for (int32 i = 0; i < InputPins.Num(); i++)
		{
			FNiagaraVariable Input = Signature.Inputs[i];
			if (Input.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				bHasParamMapInput = true;
				if (InputPins[i].LinkedTo)
				{
					ParamMapIdx = Builder.TraceParameterMapOutputPin(InputPins[i].LinkedTo);
				}
			}
			else
			{
				LocalVars.Add(Input);
			}
		}

		for (int32 i = 0; i < OutputPins.Num(); i++)
		{
			FNiagaraVariable Output = Signature.Outputs[i];
			if (Output.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				bHasParamMapOutput = true;
				Builder.RegisterParameterMapPin(ParamMapIdx, &OutputPins[i]);
			}
			else
			{
				LocalVars.Add(Output);
			}
		}

		TArray<FString> PossibleNamespaces;
		FNiagaraParameterUtilities::GetValidNamespacesForReading(Builder.GetBaseUsageContext(), 0, PossibleNamespaces);

		if ((bHasParamMapOutput || bHasParamMapInput) && ParamMapIdx != INDEX_NONE)
		{
			for (int32 i = 0; i < Tokens.Num(); i++)
			{
				bool bFoundLocal = false;
				if (INDEX_NONE != FNiagaraVariable::SearchArrayForPartialNameMatch(LocalVars, *Tokens[i]))
				{
					bFoundLocal = true;
				}

				if (!bFoundLocal && Tokens[i].Contains(TEXT("."))) // Only check tokens with namespaces in them..
				{
					for (const FString& ValidNamespace : PossibleNamespaces)
					{
						// There is one possible path here, one where we're using the namespace as-is from the valid list.
						if (Tokens[i].StartsWith(ValidNamespace, ESearchCase::CaseSensitive))
						{
							Builder.HandleExternalVariableRead(ParamMapIdx, *Tokens[i]);
						}
					}
				}
			}
		}
	}
}

bool FNiagaraCompilationNodeCustomHlsl::TokensContainsName(FName InName) const
{
	FNameBuilder NameBuilder(InName);
	FStringView NameBuilderView = NameBuilder.ToView();

	int32 SearchStartIndex = Algo::LowerBoundBy(SortedTokenIndices, NameBuilderView, [this](int32 TokenIndex) -> FStringView
	{
		return Tokens[TokenIndex];
	}, FStringViewLess());

	return Tokens.IsValidIndex(SearchStartIndex) && Tokens[SortedTokenIndices[SearchStartIndex]].StartsWith(NameBuilderView);
}

FNiagaraCompilationNodeIf::FNiagaraCompilationNodeIf(const UNiagaraNodeIf* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::If, InNode, Context)
{
	const int32 PinsPerSelection = InNode->PathAssociatedPinGuids.Num();

	OutputVariables = InNode->OutputVars;
	ConditionalPinIndex = GetInputPinIndexByPersistentId(InNode->ConditionPinGuid);
	FalseInputPinIndices.Reserve(PinsPerSelection);
	TrueInputPinIndices.Reserve(PinsPerSelection);

	for (const FPinGuidsForPath& PinGuids : InNode->PathAssociatedPinGuids)
	{
		FalseInputPinIndices.Add(GetInputPinIndexByPersistentId(PinGuids.InputFalsePinGuid));
		TrueInputPinIndices.Add(GetInputPinIndexByPersistentId(PinGuids.InputTruePinGuid));
	}
}


FNiagaraCompilationNodeIf::FNiagaraCompilationNodeIf(const FNiagaraCompilationNodeIf& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, OutputVariables(InNode.OutputVariables)
	, ConditionalPinIndex(InNode.ConditionalPinIndex)
	, FalseInputPinIndices(InNode.FalseInputPinIndices)
	, TrueInputPinIndices(InNode.TrueInputPinIndices)
{

}

void FNiagaraCompilationNodeIf::ResolveNumerics()
{
	const int32 InputPinCount = InputPins.Num();
	const int32 OutputPinCount = OutputPins.Num();

	const int32 InputPinOffset = 1; // offset because of the condition pin

	for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
	{
		ResolveNumericPins({InputPinOffset + OutputPinIt, InputPinOffset + OutputPinIt + OutputPinCount}, {OutputPinIt});
	}
}

void FNiagaraCompilationNodeIf::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	int32 Condition = Translator->CompileInputPin(&InputPins[ConditionalPinIndex]);

	TArray<int32> PathTrue;
	PathTrue.Reserve(TrueInputPinIndices.Num());
	for (int32 PathTrueIndex : TrueInputPinIndices)
	{
		PathTrue.Add(Translator->CompileInputPin(&InputPins[PathTrueIndex]));
	}

	TArray<int32> PathFalse;
	PathFalse.Reserve(FalseInputPinIndices.Num());
	for (int32 PathFalseIndex : FalseInputPinIndices)
	{
		PathFalse.Add(Translator->CompileInputPin(&InputPins[PathFalseIndex]));
	}

	Translator->If(this, OutputVariables, Condition, PathTrue, PathFalse, Outputs);
}

FNiagaraCompilationNodeInput::FNiagaraCompilationNodeInput(const UNiagaraNodeInput* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::Input, InNode, Context)
{
	Context.ParentGraph.InputNodeIndices.Add(Context.ParentGraph.Nodes.Num());

	InputVariable = InNode->Input;
	Usage = InNode->Usage;
	CallSortPriority = InNode->CallSortPriority;

	if (InputVariable.IsDataInterface())
	{
		UNiagaraDataInterface* SourceDataInterface = InNode->GetDataInterface();

		DataInterfaceName = InputVariable.GetName();
		check(SourceDataInterface);
		SourceDataInterface->GetEmitterReferencesByName(DataInterfaceEmitterReferences);
		DuplicatedDataInterface = Context.ParentGraph.DigestDataInterface(SourceDataInterface);
	}
	else if (InputVariable.IsUObject())
	{
		UObject* SourceObjectAsset = InNode->GetObjectAsset();
		ObjectAssetName = InputVariable.GetName();
		Context.ParentGraph.RegisterObjectAsset(ObjectAssetName, SourceObjectAsset);
		ObjectAssetPath = FSoftObjectPath(SourceObjectAsset);
	}

	bExposed = InNode->IsExposed();
	bRequired = InNode->ExposureOptions.bRequired;
	bCanAutoBind = InNode->ExposureOptions.bCanAutoBind;
}


FNiagaraCompilationNodeInput::FNiagaraCompilationNodeInput(const FNiagaraCompilationNodeInput& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, InputVariable(InNode.InputVariable)
	, Usage(InNode.Usage)
	, DataInterfaceEmitterReferences(InNode.DataInterfaceEmitterReferences)
	, CallSortPriority(InNode.CallSortPriority)
	, DataInterfaceName(InNode.DataInterfaceName)
	, ObjectAssetName(InNode.ObjectAssetName)
	, bRequired(InNode.bRequired)
	, bExposed(InNode.bExposed)
	, bCanAutoBind(InNode.bCanAutoBind)
	, DuplicatedDataInterface(InNode.DuplicatedDataInterface)
	, ObjectAssetPath(InNode.ObjectAssetPath)
{
	Context.TargetGraph.InputNodeIndices.Add(Context.TargetGraph.Nodes.Num());
}

void FNiagaraCompilationNodeInput::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	if (InputVariable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
	{
		int32 ParamMapIdx = Builder.FindMatchingParameterMapFromContextInputs(InputVariable);

		if (ParamMapIdx == INDEX_NONE && Usage != ENiagaraInputNodeUsage::TranslatorConstant)
		{
			ParamMapIdx = Builder.CreateParameterMap();
		}
		else if (ParamMapIdx == INDEX_NONE && Builder.Histories.Num() != 0)
		{
			ParamMapIdx = 0;
		}

		if (ParamMapIdx != INDEX_NONE)
		{
			uint32 NodeIdx = Builder.BeginNodeVisitation(ParamMapIdx, this);
			Builder.EndNodeVisitation(ParamMapIdx, NodeIdx);

			Builder.RegisterParameterMapPin(ParamMapIdx, &OutputPins[0]);
		}
	}
	else if (InputVariable.GetType().IsStatic())
	{
		int32 ConstantIdx = Builder.FindMatchingStaticFromContextInputs(InputVariable);

		if (ConstantIdx != INDEX_NONE)
		{
			Builder.RegisterConstantPin(ConstantIdx, &OutputPins[0]);
		}
	}
}

void FNiagaraCompilationNodeInput::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	if (!NodeEnabled)
	{
		Outputs.Add(INDEX_NONE);
		return;
	}

	if (InputVariable.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
	{
		Outputs.Add(INDEX_NONE);
		Translator->Error(LOCTEXT("InvalidPinType", "Numeric types should be able to be inferred from use by this phase of compilation."), this, nullptr);
		return;
	}

	int32 FunctionParam = INDEX_NONE;
	if (bExposed && Translator->GetFunctionParameter(InputVariable, FunctionParam))
	{
		//If we're in a function and this parameter hasn't been provided, compile the local default.
		if (FunctionParam == INDEX_NONE)
		{
			int32 Default = InputPins.Num() > 0 ? Translator->CompileInputPin(&InputPins[0]) : INDEX_NONE;
			if (Default == INDEX_NONE)
			{
				//We failed to compile the default pin so just use the value of the input.
				if (Usage == ENiagaraInputNodeUsage::Parameter)
				{
					if (!DataInterfaceName.IsNone())
					{
						check(DuplicatedDataInterface);
						check(InputVariable.IsDataInterface());
						Outputs.Add(Translator->RegisterDataInterface(InputVariable, DuplicatedDataInterface, false, false));
						return;
					}
					else if (!ObjectAssetName.IsNone())
					{
						check(InputVariable.IsUObject());
						Outputs.Add(Translator->RegisterUObjectPath(InputVariable, ObjectAssetPath, false));
						return;
					}
				}

				Default = Translator->GetConstant(InputVariable);
			}
			Outputs.Add(Default);
			return;
		}
	}

	switch (Usage)
	{
	case ENiagaraInputNodeUsage::Parameter:
		if (!DataInterfaceName.IsNone())
		{
			check(DuplicatedDataInterface);
			check(InputVariable.IsDataInterface());
			Outputs.Add(Translator->RegisterDataInterface(InputVariable, DuplicatedDataInterface, false, false));
			break;
		}
		else if (!ObjectAssetName.IsNone())
		{
			check(InputVariable.IsUObject());
			Outputs.Add(Translator->RegisterUObjectPath(InputVariable, ObjectAssetPath, false));
			break;
		}
		else
		{
			Outputs.Add(Translator->GetParameter(InputVariable));
			break;
		}
	case ENiagaraInputNodeUsage::SystemConstant:
		Outputs.Add(Translator->GetParameter(InputVariable)); break;
	case ENiagaraInputNodeUsage::Attribute:
		Outputs.Add(Translator->GetAttribute(InputVariable)); break;
	case ENiagaraInputNodeUsage::TranslatorConstant:
		Outputs.Add(Translator->GetParameter(InputVariable)); break;
	case ENiagaraInputNodeUsage::RapidIterationParameter:
		Outputs.Add(Translator->GetRapidIterationParameter(InputVariable)); break;
	default:
		check(false);
	}
}

void FNiagaraCompilationNodeInput::UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const
{
	if (Usage == ENiagaraInputNodeUsage::TranslatorConstant && InputVariable == TRANSLATOR_PARAM_CALL_ID)
	{
		// The call ID should be unique for each translated node as it is used by the seeded random functions.
		// We don't want it to be shared across the spawn and update script, so functions including it will have the usage added to their name.
		static const FName NAME_ScriptUsage = TEXT("ScriptUsage");
		InOutFunctionAliasContext.AddPermutationIdentifier(NAME_ScriptUsage, FString::FormatAsNumber(static_cast<uint8>(InOutFunctionAliasContext.ScriptUsage)));
	}
}


FNiagaraCompilationNodeOp::FNiagaraCompilationNodeOp(const UNiagaraNodeOp* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::Op, InNode, Context)
{
	OpName = InNode->OpName;
}


FNiagaraCompilationNodeOp::FNiagaraCompilationNodeOp(const FNiagaraCompilationNodeOp& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, OpName(InNode.OpName)
{

}

FNiagaraTypeDefinition FNiagaraCompilationNodeOp::ResolveCustomNumericType(TConstArrayView<FNiagaraTypeDefinition> ConcreteInputTypes) const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (OpInfo && OpInfo->CustomNumericResolveFunction.IsBound())
	{
		return OpInfo->CustomNumericResolveFunction.Execute(ConcreteInputTypes);
	}

	return FNiagaraTypeDefinition::GetGenericNumericDef();
}

void FNiagaraCompilationNodeOp::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (OpInfo && OpInfo->StaticVariableResolveFunction.IsBound())
	{
		const FNiagaraCompilationOutputPin* OutputPin = OutputPins.IsEmpty() ? nullptr : &OutputPins[0];
		const bool bAllPinsStatic = !InputPins.ContainsByPredicate([](const FNiagaraCompilationInputPin& InputPin) -> bool
		{
			return !InputPin.Variable.GetType().IsStatic();
		});

		if (bAllPinsStatic)
		{
			TArray<int32> Vars;

			for (const FNiagaraCompilationInputPin& InputPin : InputPins)
			{
				int32 Value = 0;
				Builder.SetConstantByStaticVariable(Value, &InputPin);
				Vars.Add(Value);
			}

			if (Vars.Num() > 0)
			{
				int32 Result = OpInfo->StaticVariableResolveFunction.Execute(Vars);
				int32 ConstantIdx = Builder.AddOrGetConstantFromValue(FString::FromInt(Result));

				if (UNiagaraScript::LogCompileStaticVars > 0)
				{
					UE_LOGF(LogNiagaraEditor, Log, "Inputs Static Node Op: %ls", *FullTitle);
					for (int32 i = 0; i < Vars.Num(); i++)
					{
						UE_LOGF(LogNiagaraEditor, Log, "[%d] %d", i, Vars[i]);
					}
					UE_LOGF(LogNiagaraEditor, Log, "Result: %d", Result);
				}

				Builder.RegisterConstantPin(ConstantIdx, OutputPin);
			}
		}
	}
}

void FNiagaraCompilationNodeOp::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (!OpInfo)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("OpName"), FText::FromName(OpName));
		FText Format = LOCTEXT("Unknown opcode", "Unknown opcode on {OpName} node.");
		Translator->Error(FText::Format(Format, Args), this, nullptr);
		return;
	}

	const int32 InputCount = OpInfo->bSupportsAddedInputs ? InputPins.Num() : OpInfo->Inputs.Num();
	const int32 OutputCount = OpInfo->Outputs.Num();

	TArray<int32> Inputs;
	bool bError = false;
	for (int32 InputPinIt = 0; InputPinIt < InputCount; ++InputPinIt)
	{
		const FNiagaraCompilationInputPin& InputPin = InputPins[InputPinIt];
		int32 CompiledInput = Translator->CompileInputPin(&InputPin);
		if (CompiledInput == INDEX_NONE)
		{
			bError = true;
			FFormatNamedArguments Args;
			Args.Add(TEXT("OpName"), FText::FromString(FullTitle));
			FText Format = LOCTEXT("InputErrorFormat", "Error compiling input on {OpName} node.");
			Translator->Error(FText::Format(Format, Args), this, &InputPin);
		}
		else if (InputPinIt < OpInfo->Inputs.Num() && OpInfo->Inputs[InputPinIt].DataType == FNiagaraTypeDefinition::GetGenericNumericDef())
		{
			// Some nodes disallow integer or floating numeric input pins, so we guard against them here. 
			// This will catch both implicitly and explicitly set pin types.
			// Currently this is for the Random Float/Integer and Seeded Random Float/Integer ops, but might be useful for others in the future. 
			FNiagaraTypeDefinition TypeDef = InputPin.Variable.GetType();
			if (TypeDef.IsFloatPrimitive() && !OpInfo->bNumericsCanBeFloats)
			{
				bError = true;
				FFormatNamedArguments Args;
				Args.Add(TEXT("OpName"), FText::FromString(FullTitle));
				FText Format = LOCTEXT("InputTypeErrorFormatFloat", "The {OpName} node cannot have float based numeric input pins.");
				Translator->Error(FText::Format(Format, Args), this, &InputPin);
			}
			else if (!TypeDef.IsFloatPrimitive() && !OpInfo->bNumericsCanBeIntegers)
			{
				bError = true;
				FFormatNamedArguments Args;
				Args.Add(TEXT("OpName"), FText::FromString(FullTitle));
				FText Format = LOCTEXT("InputTypeErrorFormatInt", "The {OpName} node cannot have integer based numeric input pins.");
				Translator->Error(FText::Format(Format, Args), this, &InputPin);
			}
		}
		Inputs.Add(CompiledInput);
	}

	Translator->Operation(this, Inputs, Outputs);
}


FNiagaraCompilationNodeOutput::FNiagaraCompilationNodeOutput(const UNiagaraNodeOutput* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::Output, InNode, Context)
{
	Context.ParentGraph.OutputNodeIndices.Add(Context.ParentGraph.Nodes.Num());

	Usage = InNode->GetUsage();
	UsageId = InNode->GetUsageId();
	StackContextOverrideName = InNode->GetStackContextOverride();
	Outputs = InNode->GetOutputs();
}

FNiagaraCompilationNodeOutput::FNiagaraCompilationNodeOutput(const FNiagaraCompilationNodeOutput& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, Usage(InNode.Usage)
	, UsageId(InNode.UsageId)
	, StackContextOverrideName(InNode.StackContextOverrideName)
	, Outputs(InNode.Outputs)
{
	Context.TargetGraph.OutputNodeIndices.Add(Context.TargetGraph.Nodes.Num());
}

void FNiagaraCompilationNodeOutput::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	static const FNiagaraTypeDefinition ParameterMapType = FNiagaraTypeDefinition::GetParameterMapDef();

	FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	for (const FNiagaraCompilationInputPin& InputPin : InputPins)
	{
		if (InputPin.Variable.GetType().IsStatic())
		{
			Builder.RegisterConstantFromInputPin(&InputPin, InputPin.DefaultValue);
		}
		else if (InputPin.Variable.GetType() == ParameterMapType)
		{
			if (InputPin.LinkedTo)
			{
				int32 ParamMapIdx = Builder.TraceParameterMapOutputPin(InputPin.LinkedTo);
				if (UNiagaraScript::LogCompileStaticVars > 0)
				{
					UE_LOGF(LogNiagaraEditor, Log, "Build Parameter Map History: NiagaraCompilationNodeOutput %ls PMapIdx: %d", *FullTitle, ParamMapIdx);
				}
				Builder.RegisterParameterMapPin(ParamMapIdx, &InputPin);
			}
		}
	}
}

void FNiagaraCompilationNodeOutput::Compile(FTranslator* Translator, TArray<int32>& OutputTokens) const
{
	TArray<int32> Results;
	bool bError = CompileInputPins(Translator, Results);
	if (!bError)
	{
		Translator->Output(this, Results);
	}
}



FNiagaraCompilationNodeOutputTag::FNiagaraCompilationNodeOutputTag(const UNiagaraNodeOutputTag* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::OutputTag, InNode, Context)
{
	bEmitMessageOnFailure = InNode->bEmitMessageOnFailure;
	bEditorOnly = InNode->bEditorOnly;
	FailureSeverity = InNode->FailureSeverity;
}

FNiagaraCompilationNodeOutputTag::FNiagaraCompilationNodeOutputTag(const FNiagaraCompilationNodeOutputTag& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, bEmitMessageOnFailure(InNode.bEmitMessageOnFailure)
	, bEditorOnly(InNode.bEditorOnly)
	, FailureSeverity(InNode.FailureSeverity)
{

}

void FNiagaraCompilationNodeOutputTag::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	uint32 NodeIdx = INDEX_NONE;

	const int32 InputPinCount = InputPins.Num();
	for (int32 InputPinIt = 0; InputPinIt < InputPinCount; ++InputPinIt)
	{
		const FNiagaraCompilationInputPin* InputPin = &InputPins[InputPinIt];

		FNiagaraTypeDefinition VarTypeDef = InputPin->Variable.GetType();
		if (InputPinIt == 0 && InputPin != nullptr && VarTypeDef == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			const FNiagaraCompilationOutputPin* PriorParamPin = nullptr;
			if (InputPin->LinkedTo)
			{
				PriorParamPin = InputPin->LinkedTo;
			}

			// Now plow into our ancestor node
			if (PriorParamPin)
			{
				ParamMapIdx = Builder.TraceParameterMapOutputPin(PriorParamPin);
				NodeIdx = Builder.BeginNodeVisitation(ParamMapIdx, this);
			}
		}
		else if (InputPinIt > 0 && InputPin != nullptr && ParamMapIdx != INDEX_NONE)
		{
			Builder.HandleVariableWrite(ParamMapIdx, InputPin);
		}
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		Builder.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}

	Builder.RegisterParameterMapPin(ParamMapIdx, &OutputPins[0]);
}

void FNiagaraCompilationNodeOutputTag::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	for (const FNiagaraCompilationInputPin& InputPin : InputPins)
	{
		if (NodeEnabled == false && InputPin.Variable.GetType() != FNiagaraTypeDefinition::GetParameterMapDef())
		{
			continue;
		}
		int32 CompiledInput = Translator->CompileInputPin(&InputPin);

		if (InputPin.Variable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			Outputs.Add(CompiledInput);
		}
		else
		{
			Translator->WriteCompilerTag(CompiledInput, &InputPin, bEditorOnly, bEmitMessageOnFailure, FailureSeverity);
		}
	}

	ensure(Outputs.Num() == 1);
}



FNiagaraCompilationNodeConvert::FNiagaraCompilationNodeConvert(const UNiagaraNodeConvert* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::Convert, InNode, Context)
{
	const TArray<FNiagaraConvertConnection>& SourceConnections = InNode->GetConnections();
	Connections.Reserve(SourceConnections.Num());
	Algo::Transform(SourceConnections, Connections, [](const FNiagaraConvertConnection& SourceConnection) -> FCachedConnection
	{
		FCachedConnection Connection;
		Connection.SourcePinId = SourceConnection.SourcePinId;
		Connection.SourcePath = SourceConnection.SourcePath;
		Connection.DestinationPinId = SourceConnection.DestinationPinId;
		Connection.DestinationPath = SourceConnection.DestinationPath;

		return Connection;
	});
}

FNiagaraCompilationNodeConvert::FNiagaraCompilationNodeConvert(const FNiagaraCompilationNodeConvert& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, Connections(InNode.Connections)
{
}

void FNiagaraCompilationNodeConvert::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	TArray<int32, TInlineAllocator<16>> CompileInputs;
	CompileInputs.Reserve(InputPins.Num());
	for (const FNiagaraCompilationInputPin& InputPin : InputPins)
	{
		if (InputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType ||
			InputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum ||
			InputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType ||
			InputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum)
		{
			int32 CompiledInput = Translator->CompileInputPin(&InputPin);
			if (CompiledInput == INDEX_NONE)
			{
				Translator->Error(LOCTEXT("ConvertInputError", "Error compiling input for convert node."), this, &InputPin);
			}
			CompileInputs.Add(CompiledInput);
		}
	}

	Translator->Convert(this, CompileInputs, Outputs);
}

FNiagaraCompilationNodeParameterMapGet::FNiagaraCompilationNodeParameterMapGet(const UNiagaraNodeParameterMapGet* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::ParameterMapGet, InNode, Context)
{
	DefaultInputPinIndices.Reserve(OutputPins.Num());

	for (const FNiagaraCompilationOutputPin& OutputPin : OutputPins)
	{
		int32 InputPinIndex = INDEX_NONE;
		if (const UEdGraphPin* DefaultPin = InNode->GetDefaultPin(InNode->Pins[OutputPin.SourcePinIndex]))
		{
			const int32 NodeInputPinIndex = InNode->Pins.IndexOfByKey(DefaultPin);
			InputPinIndex = InputPins.IndexOfByPredicate([&NodeInputPinIndex](const FNiagaraCompilationInputPin& InputPin) -> bool
			{
				return InputPin.SourcePinIndex == NodeInputPinIndex;
			});
		}
		DefaultInputPinIndices.Add(InputPinIndex);
	}
}

FNiagaraCompilationNodeParameterMapGet::FNiagaraCompilationNodeParameterMapGet(const FNiagaraCompilationNodeParameterMapGet& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, DefaultInputPinIndices(InNode.DefaultInputPinIndices)
{

}

void FNiagaraCompilationNodeParameterMapGet::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	if (bRecursive)
	{
		Builder.VisitInputPin(&InputPins[0], bFilterForCompilation);
	}

	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	if (InputPins[0].LinkedTo)
	{
		ParamMapIdx = Builder.TraceParameterMapOutputPin((InputPins[0].LinkedTo));
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		uint32 NodeIdx = Builder.BeginNodeVisitation(ParamMapIdx, this);
		const int32 OutputPinCount = OutputPins.Num();
		check(DefaultInputPinIndices.Num() == OutputPinCount);

		for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
		{
			const FNiagaraCompilationOutputPin& OutputPin = OutputPins[OutputPinIt];
			const bool HasDefault = DefaultInputPinIndices[OutputPinIt] != INDEX_NONE;

			bool bUsedDefaults = false;
			if (bRecursive && HasDefault)
			{
				const FNiagaraCompilationInputPin* DefaultPin = &InputPins[DefaultInputPinIndices[OutputPinIt]];

				Builder.HandleVariableRead(ParamMapIdx, &OutputPin, true, DefaultPin, bFilterForCompilation, bUsedDefaults);
			}
			else
			{
				Builder.HandleVariableRead(ParamMapIdx, &OutputPin, true, nullptr, bFilterForCompilation, bUsedDefaults);
			}
		}
		Builder.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}
}

void FNiagaraCompilationNodeParameterMapGet::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	// First compile fully down the hierarchy for our predecessors..
	const int32 InputPinCount = InputPins.Num();

	if (InputPinCount)
	{
		// Initialize the outputs to invalid values.
		check(Outputs.Num() == 0);
		Outputs.Init(INDEX_NONE, OutputPins.Num());

		TArray<int32, TInlineAllocator<16>> CompileInputs;
		CompileInputs.Init(INDEX_NONE, InputPinCount);

		const FNiagaraCompilationInputPin& ParamMapInputPin = InputPins[0];
		if (ParamMapInputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType ||
			ParamMapInputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType ||
			ParamMapInputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum ||
			ParamMapInputPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum)
		{
			int32 CompiledInput = Translator->CompileInputPin(&ParamMapInputPin);
			if (CompiledInput == INDEX_NONE)
			{
				Translator->Error(LOCTEXT("MapGetInputError", "Error compiling input for param map get node."), this, &ParamMapInputPin);
			}

			CompileInputs[0] = CompiledInput;
		}

		if (ParamMapInputPin.LinkedTo)
		{
			Translator->ParameterMapGet(this, CompileInputs, Outputs);
		}
	}
}

FNiagaraCompilationNodeParameterMapFor::FNiagaraCompilationNodeParameterMapFor(const UNiagaraNodeParameterMapFor* InNode, FNiagaraCompilationGraphCreateContext& Context, ENodeType InNodeType)
	: FNiagaraCompilationNodeParameterMapSet(InNode, Context, InNodeType)
{
}

FNiagaraCompilationNodeParameterMapFor::FNiagaraCompilationNodeParameterMapFor(const FNiagaraCompilationNodeParameterMapFor& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNodeParameterMapSet(InNode, Context)
{

}

void FNiagaraCompilationNodeParameterMapFor::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	if (Translator->GetSimulationTarget() == ENiagaraSimTarget::GPUComputeSim)
	{
		const int32 IterationCount = Translator->CompileInputPin(&InputPins[1]);
		Translator->ParameterMapForBegin(this, IterationCount);

		FNiagaraCompilationNodeParameterMapSet::Compile(Translator, Outputs);

		Translator->ParameterMapForEnd(this);
	}
	else
	{
		FNiagaraCompilationNodeParameterMapSet::Compile(Translator, Outputs);
		//Translator->Message(FNiagaraCompileEventSeverity::Log,LOCTEXT("UnsupportedParamMapFor", "Parameter map for is not yet supported on cpu."), this, nullptr);
	}
}

void FNiagaraCompilationNodeParameterMapFor::CollectInputPinsToCompile(FTranslator* Translator, FInputPinCollection& ActiveInputPins) const
{
	FNiagaraCompilationNodeParameterMapSet::CollectInputPinsToCompile(Translator, ActiveInputPins);

	// we need to cull the iteration pin from the list of pins to compile as it's already going to be compiled during the MapFor::Compile
	ActiveInputPins.Remove(&InputPins[1]);
}

FNiagaraCompilationNodeParameterMapForWithContinue::FNiagaraCompilationNodeParameterMapForWithContinue(const UNiagaraNodeParameterMapForWithContinue* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNodeParameterMapFor(InNode, Context, ENodeType::ParameterMapForWithContinue)
{
}

FNiagaraCompilationNodeParameterMapForWithContinue::FNiagaraCompilationNodeParameterMapForWithContinue(const FNiagaraCompilationNodeParameterMapForWithContinue& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNodeParameterMapFor(InNode, Context)
{

}

void FNiagaraCompilationNodeParameterMapForWithContinue::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	if (Translator->GetSimulationTarget() == ENiagaraSimTarget::GPUComputeSim)
	{
		const int32 IterationCount = Translator->CompileInputPin(&InputPins[1]);

		Translator->ParameterMapForBegin(this, IterationCount);

		const int32 IterationEnabledChunk = Translator->CompileInputPin(&InputPins[2]);
		Translator->ParameterMapForContinue(this, IterationEnabledChunk);

		FNiagaraCompilationNodeParameterMapSet::Compile(Translator, Outputs);

		Translator->ParameterMapForEnd(this);
	}
	else
	{
		FNiagaraCompilationNodeParameterMapSet::Compile(Translator, Outputs);
		//Translator->Message(FNiagaraCompileEventSeverity::Log,LOCTEXT("UnsupportedParamMapFor", "Parameter map for is not yet supported on cpu."), this, nullptr);
	}
}

void FNiagaraCompilationNodeParameterMapForWithContinue::CollectInputPinsToCompile(FTranslator* Translator, FInputPinCollection& ActiveInputPins) const
{
	FNiagaraCompilationNodeParameterMapFor::CollectInputPinsToCompile(Translator, ActiveInputPins);

	// we also need to cull out the iteration enabled pin
	ActiveInputPins.Remove(&InputPins[2]);
}

FNiagaraCompilationNodeParameterMapForIndex::FNiagaraCompilationNodeParameterMapForIndex(const UNiagaraNodeParameterMapForIndex* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::ParameterMapForIndex, InNode, Context)
{
}

FNiagaraCompilationNodeParameterMapForIndex::FNiagaraCompilationNodeParameterMapForIndex(const FNiagaraCompilationNodeParameterMapForIndex& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
{

}

void FNiagaraCompilationNodeParameterMapForIndex::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	if (Translator)
	{
		if (Translator->GetSimulationTarget() == ENiagaraSimTarget::GPUComputeSim)
		{
			Outputs.Add(Translator->ParameterMapForInnerIndex());
		}
		else
		{
			FNiagaraVariable Constant(FNiagaraTypeDefinition::GetIntDef(), TEXT("Constant"));
			Constant.SetValue(0);
			Outputs.Add(Translator->GetConstant(Constant));
		}
	}
}

FNiagaraCompilationNodeParameterMapSet::FNiagaraCompilationNodeParameterMapSet(const UNiagaraNodeParameterMapSet* InNode, FNiagaraCompilationGraphCreateContext& Context, ENodeType InNodeType)
	: FNiagaraCompilationNode(InNodeType, InNode, Context)
{
}

FNiagaraCompilationNodeParameterMapSet::FNiagaraCompilationNodeParameterMapSet(const FNiagaraCompilationNodeParameterMapSet& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
{

}

void FNiagaraCompilationNodeParameterMapSet::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	const FString SourceFullName = SourceNode->GetPathName();
	FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	uint32 NodeIdx = INDEX_NONE;

	const int32 InputPinCount = InputPins.Num();
	for (int32 InputPinIt = 0; InputPinIt < InputPinCount; ++InputPinIt)
	{
		const FNiagaraCompilationInputPin& InputPin = InputPins[InputPinIt];

		FNiagaraTypeDefinition VarTypeDef = InputPin.Variable.GetType();
		if (InputPinIt == 0 && VarTypeDef == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			const FNiagaraCompilationOutputPin* PriorParamPin = nullptr;
			if (InputPin.LinkedTo)
			{
				PriorParamPin = InputPin.LinkedTo;
			}

			// Now plow into our ancestor node
			if (PriorParamPin)
			{
				ParamMapIdx = Builder.TraceParameterMapOutputPin(PriorParamPin);
				NodeIdx = Builder.BeginNodeVisitation(ParamMapIdx, this);
			}
		}
		else if (InputPinIt > 0 && ParamMapIdx != INDEX_NONE)
		{
			Builder.HandleVariableWrite(ParamMapIdx, &InputPin);
		}
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		Builder.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}

	Builder.RegisterParameterMapPin(ParamMapIdx, &OutputPins[0]);
}

void FNiagaraCompilationNodeParameterMapSet::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	// Initialize the outputs to invalid values.
	check(Outputs.Num() == 0);
	Outputs.Init(INDEX_NONE, OutputPins.Num());

	FInputPinCollection ActiveInputPins;
	CollectInputPinsToCompile(Translator, ActiveInputPins);

	TArray<FTranslator::FCompiledPin, TInlineAllocator<16>> CompileInputs;
	CompileInputs.Reserve(ActiveInputPins.Num());

	// update the translator with the culled function calls before compiling any further
	for (const FNiagaraCompilationInputPin* InputPin : ActiveInputPins)
	{
		if (NodeEnabled || InputPin->Variable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			int32 CompiledInput = Translator->CompileInputPin(InputPin);
			if (CompiledInput == INDEX_NONE)
			{
				Translator->Error(LOCTEXT("MapSetInputError", "Error compiling input for set node."), this, InputPin);
			}
			CompileInputs.Emplace(CompiledInput, InputPin);
		}
	}

	if (ActiveInputPins.Num() && ActiveInputPins[0] && ActiveInputPins[0]->LinkedTo)
	{
		Translator->ParameterMapSet(this, CompileInputs, Outputs);
	}
}

void FNiagaraCompilationNodeParameterMapSet::CollectInputPinsToCompile(FTranslator* Translator, FInputPinCollection& ActiveInputPins) const
{
	ActiveInputPins.Reserve(InputPins.Num());
	// do a first pass over all of the pins so that we can properly cull out input pins and
	// propagate the disabled pins up the chain
	for (const FNiagaraCompilationInputPin& InputPin : InputPins)
	{
		if (Translator->IsFunctionVariableCulledFromCompilation(InputPin.PinName))
		{
			Translator->CullMapSetInputPin(&InputPin);
		}
		else
		{
			ActiveInputPins.Add(&InputPin);
		}
	}
}

FNiagaraCompilationNodeReadDataSet::FNiagaraCompilationNodeReadDataSet(const UNiagaraNodeReadDataSet* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::ReadDataSet, InNode, Context)
{
	DataSet = InNode->DataSet;
	DataSetVariables = InNode->Variables;
}

FNiagaraCompilationNodeReadDataSet::FNiagaraCompilationNodeReadDataSet(const FNiagaraCompilationNodeReadDataSet& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, DataSet(InNode.DataSet)
	, DataSetVariables(InNode.DataSetVariables)
{

}

void FNiagaraCompilationNodeReadDataSet::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	if (InputPins[0].LinkedTo)
	{
		ParamMapIdx = Builder.TraceParameterMapOutputPin(NiagaraCompilationImpl::TraceOutputPin(Builder, InputPins[0].LinkedTo, bFilterForCompilation));
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		uint32 NodeIdx = Builder.BeginNodeVisitation(ParamMapIdx, this);
		Builder.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}

	Builder.RegisterParameterMapPin(ParamMapIdx, &OutputPins[0]);
}

void FNiagaraCompilationNodeReadDataSet::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	TArray<int32> Inputs;
	CompileInputPins(Translator, Inputs);

	Translator->ReadDataSet(DataSet, DataSetVariables, ENiagaraDataSetAccessMode::AppendConsume, Inputs[0], Outputs);
}

FNiagaraCompilationNodeStaticSwitch::FNiagaraCompilationNodeStaticSwitch(const UNiagaraNodeStaticSwitch* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNodeUsageSelector(InNode, Context, ENodeType::StaticSwitch)
{
	bSetByCompiler = InNode->IsSetByCompiler();
	bSetByPin = InNode->IsSetByPin();
	bDebugStateSwitch = InNode->IsDebugSwitch();
	SwitchType = InNode->SwitchTypeData.SwitchType;
	SwitchBranchCount = InNode->GetOptionValues().Num();
	InputParameterName = InNode->InputParameterName;
	SwitchConstant = InNode->SwitchTypeData.SwitchConstant;
	InputType = InNode->GetInputType();

	SelectorPinIndex = INDEX_NONE;
	if (const UEdGraphPin* SelectorPin = InNode->GetSelectorPin())
	{
		SelectorPinIndex = InputPins.IndexOfByPredicate([&SelectorPin, &InNode](const FNiagaraCompilationInputPin& InputPin) -> bool
		{
			return InNode->Pins[InputPin.SourcePinIndex] == SelectorPin;
		});
	}

	if (!bSetByCompiler && !bSetByPin)
	{
		Context.ParentGraph.StaticSwitchInputs.AddUnique(FNiagaraVariable(InputType, InputParameterName));
	}
}

FNiagaraCompilationNodeStaticSwitch::FNiagaraCompilationNodeStaticSwitch(const FNiagaraCompilationNodeStaticSwitch& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNodeUsageSelector(InNode, Context)
	, bSetByCompiler(InNode.bSetByCompiler)
	, bSetByPin(InNode.bSetByPin)
	, SwitchType(InNode.SwitchType)
	, InputType(InNode.InputType)
	, SwitchBranchCount(InNode.SwitchBranchCount)
	, SelectorPinIndex(InNode.SelectorPinIndex)
	, InputParameterName(InNode.InputParameterName)
	, SwitchConstant(InNode.SwitchConstant)
{
	if (!bSetByCompiler && !bSetByPin)
	{
		Context.TargetGraph.StaticSwitchInputs.AddUnique(FNiagaraVariable(InputType, InputParameterName));
	}
}

int32 FNiagaraCompilationNodeStaticSwitch::GetBaseInputChannel(int32 SelectorValue) const
{
	const int32 InputChannelCount = bSetByPin ? InputPins.Num() - 1 : InputPins.Num();
	int32 BaseInputPinIndex = INDEX_NONE;
	switch (SwitchType)
	{
	case ENiagaraStaticSwitchType::Bool:
		BaseInputPinIndex = SelectorValue ? 0 : (InputChannelCount / 2);
		break;

	case ENiagaraStaticSwitchType::Enum:
	case ENiagaraStaticSwitchType::Integer:
		BaseInputPinIndex = FMath::Clamp(SelectorValue, 0, SwitchBranchCount - 1) * InputChannelCount / SwitchBranchCount;
		break;
	}

	return BaseInputPinIndex;
}

void FNiagaraCompilationNodeStaticSwitch::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	if (!bFilterForCompilation)
	{
		FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);
		//FNiagaraCompilationNodeUsageSelector::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

		// seems odd for the !bFilterForCompilation condition, but the original code registers a passthrough
		// for a single parameter map input/output pair, if one exists
		const FNiagaraTypeDefinition ParameterMapDef = FNiagaraTypeDefinition::GetParameterMapDef();

		auto IsParameterMapPin = [&ParameterMapDef](const FNiagaraCompilationPin& Pin) -> bool
		{
			return Pin.Variable.GetType() == ParameterMapDef;
		};

		const FNiagaraCompilationOutputPin* OutputParameterMapPin = OutputPins.FindByPredicate(IsParameterMapPin);
		const FNiagaraCompilationInputPin* InputParameterMapPin = InputPins.FindByPredicate(IsParameterMapPin);

		if (OutputParameterMapPin && InputParameterMapPin)
		{
			const bool bVisitInputPin = false;
			RegisterPassthroughPin(Builder, InputParameterMapPin, OutputParameterMapPin, bFilterForCompilation, bVisitInputPin);
		}

		return;
	}

	int32 SelectorValue = INDEX_NONE;
	Builder.TraversalStateContext->GetStaticSwitchValue(NodeGuid, SelectorValue);

	if (bSetByPin)
	{
		const FNiagaraCompilationInputPin* SelectorPin = &InputPins[SelectorPinIndex];
		Builder.VisitInputPin(SelectorPin, bFilterForCompilation);
		Builder.SetConstantByStaticVariable(SelectorValue, SelectorPin);

		if (UNiagaraScript::LogCompileStaticVars != 0)
		{
			UE_LOGF(LogNiagaraEditor, Log, "Static Switch Value: %d  Name: \"%ls\"  SwitchInputParameterName:\"%ls\"", SelectorValue, *FullTitle, *InputParameterName.ToString());
		}
	}

	if (ensure(SelectorValue != INDEX_NONE))
	{
		const int32 OutputPinCount = OutputPins.Num();
		for (int OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
		{
			const FNiagaraCompilationOutputPin& OutputPin = OutputPins[OutputPinIt];
			const int32 BaseInputChannel = GetBaseInputChannel(SelectorValue);
			if (BaseInputChannel != INDEX_NONE)
			{
				const FNiagaraCompilationInputPin* InputPin = &InputPins[BaseInputChannel + OutputPinIt];
				const bool bVisitInputPin = true;
				RegisterPassthroughPin(Builder, InputPin, &OutputPin, bFilterForCompilation, bVisitInputPin);
			}
		}
	}
}
void FNiagaraCompilationNodeStaticSwitch::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	int32 SelectorValue = INDEX_NONE;

	if (!bSetByCompiler)
	{
		// all static switches should have already been resolved.  If they haven't it's likely because the graph
		// has been setup incorrectly.  Possibly a selector pin is being used but it's associated static variable
		// isn't being defined.
		if (InputPins.IsValidIndex(SelectorPinIndex))
		{
			Translator->Error(LOCTEXT("CouldNotResolveStaticVarByPin", "Could not resolve static variable through pin."), this, &InputPins[SelectorPinIndex]);
		}

		// default to the first entry
		SelectorValue = 0;
	}

	const int32 OutputPinCount = OutputPins.Num();
	Outputs.Init(INDEX_NONE, OutputPinCount);

	if (bSetByCompiler)
	{
		FNiagaraFixedConstantResolver ConstantResolver(Translator);

		if (const FNiagaraVariable* Found = FNiagaraConstants::FindStaticSwitchConstant(SwitchConstant))
		{
			FNiagaraVariable Constant = *Found;
			if (ConstantResolver.ResolveConstant(Constant))
			{
				if (SwitchType == ENiagaraStaticSwitchType::Bool)
				{
					SelectorValue = Constant.GetValue<bool>();
				}
				else if (SwitchType == ENiagaraStaticSwitchType::Integer || SwitchType == ENiagaraStaticSwitchType::Enum)
				{
					SelectorValue = Constant.GetValue<int32>();
				}
			}
		}
	}

	if (ensure(SelectorValue != INDEX_NONE))
	{
		const int32 BaseInputChannel = GetBaseInputChannel(SelectorValue);

		for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
		{
			Outputs[OutputPinIt] = Translator->CompileInputPin(&InputPins[BaseInputChannel + OutputPinIt]);
		}
	}
}

void FNiagaraCompilationNodeStaticSwitch::ResolveNumerics()
{
	const int32 InputPinCount = InputPins.Num();
	const int32 OutputPinCount = OutputPins.Num();

	for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
	{
		// Fix up numeric input pins and keep track of numeric types to decide the output type.
		TArray<int32, TInlineAllocator<16>> InputPinIndices;

		InputPinIndices.Reserve(SwitchBranchCount);

		for (int32 BranchIt = 0; BranchIt < SwitchBranchCount; ++BranchIt)
		{
			InputPinIndices.Add(OutputPinIt + OutputPinCount * BranchIt);
		}

		ResolveNumericPins(InputPinIndices, { OutputPinIt });
	}
}

void FNiagaraCompilationNodeStaticSwitch::UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const
{
	if (bSetByCompiler)
	{
		if (const FNiagaraVariable* Found = FNiagaraConstants::FindStaticSwitchConstant(SwitchConstant))
		{
			InOutFunctionAliasContext.AddPendingStaticSwitchVariable(*Found, static_cast<uint8>(SwitchType));
		}
	}
}

bool FNiagaraCompilationNodeStaticSwitch::ResolveConstantValue(const FNiagaraCompilationInputPin& Pin, int32& Value)
{
	if (Pin.LinkedTo)
	{
		return false;
	}

	const FEdGraphPinType& PinType = Pin.PinType;
	if (PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType && PinType.PinSubCategoryObject.IsValid())
	{
		FString PinTypeName = PinType.PinSubCategoryObject->GetName();
		if (PinTypeName.Equals(FString(TEXT("NiagaraBool"))))
		{
			Value = Pin.DefaultValue.Equals(FString(TEXT("true"))) ? 1 : 0;
			return true;
		}
		else if (PinTypeName.Equals(FString(TEXT("NiagaraInt32"))))
		{
			Value = FCString::Atoi(*Pin.DefaultValue);
			return true;
		}
	}
	else if (PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum && PinType.PinSubCategoryObject.IsValid())
	{
		UEnum* Enum = Cast<UEnum>(PinType.PinSubCategoryObject);
		FString FullName = Enum->GenerateFullEnumName(*Pin.DefaultValue);
		Value = Enum->GetIndexByName(FName(*FullName));
		return Value != INDEX_NONE;
	}
	return false;
}

const FNiagaraCompilationOutputPin* FNiagaraCompilationNodeStaticSwitch::TraceOutputPin(FParameterMapHistoryBuilder& Builder, const FNiagaraCompilationOutputPin* Pin, bool bFilterForCompilation) const
{
	int32 OutputIndex = UE_PTRDIFF_TO_INT32(Pin - OutputPins.GetData());
	if (OutputPins.IsValidIndex(OutputIndex))
	{
		int32 SelectorValue;
		if (!ensure(Builder.TraversalStateContext->GetStaticSwitchValue(NodeGuid, SelectorValue)))
		{
			SelectorValue = 0;
		}

		const int32 BaseInputChannel = GetBaseInputChannel(SelectorValue);

		if (BaseInputChannel != INDEX_NONE)
		{
			if (ensure(InputPins.IsValidIndex(BaseInputChannel + OutputIndex)))
			{
				const FNiagaraCompilationInputPin& InputPin = InputPins[BaseInputChannel + OutputIndex];
				if (InputPin.LinkedTo)
				{
					return NiagaraCompilationImpl::TraceOutputPin(Builder, InputPin.LinkedTo, bFilterForCompilation);
				}
			}
		}
	}

	return Pin;
}

TArray<const FNiagaraCompilationInputPin*> FNiagaraCompilationNodeStaticSwitch::EvaluateBranches(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationBranchMap& Branches) const
{
	if (bSetByCompiler && !bDebugStateSwitch)
	{
		return FNiagaraCompilationNode::EvaluateBranches(Context, Branches);
	}

	int32 SwitchValue = INDEX_NONE;
	bool IsValueSet = false;

	if (bDebugStateSwitch)
	{
		ENiagaraFunctionDebugState FunctionDebugState = ENiagaraFunctionDebugState::NoDebug;
		if (!Context.bDisableDebugSwitches)
		{
			Context.TraversalContext.GetCurrentDebugState(FunctionDebugState);
		}

		IsValueSet = true;
		SwitchValue = static_cast<int32>(FunctionDebugState);
	}
	else if (bSetByPin)
	{
		auto EvaluateVariableSwitchValue = [this](const FNiagaraVariable& ConstantVariable, int32& OutSwitchValue) -> bool
		{
			if (SwitchType == ENiagaraStaticSwitchType::Bool)
			{
				OutSwitchValue = ConstantVariable.GetValue<bool>();
				return true;
			}
			else if (SwitchType == ENiagaraStaticSwitchType::Integer || SwitchType == ENiagaraStaticSwitchType::Enum)
			{
				OutSwitchValue = ConstantVariable.GetValue<int32>();
				return true;
			}

			return false;
		};

		const FNiagaraVariable* Found = FNiagaraConstants::FindStaticSwitchConstant(SwitchConstant);
		FNiagaraVariable Constant = Found ? *Found : FNiagaraVariable();
		if (Found && Context.ConstantResolver.ResolveConstant(Constant))
		{
			IsValueSet = EvaluateVariableSwitchValue(Constant, SwitchValue);
		}

		// if we're set by pin then we can go through the previously completed parameter map traversal history
		// to handle the selector pin
		if (!IsValueSet && bSetByPin)
		{
			const FNiagaraCompilationInputPin& SelectorPin = InputPins[SelectorPinIndex];
			if (SelectorPin.LinkedTo)
			{
				if (ensure(SelectorPin.Variable.GetType().IsStatic()))
				{
					// failing here isn't the end of the world.  We verify things in ValidateRefinement(), but
					// not getting a static value switch here may just mean that we're evaluating a static switch
					// that isn't in the traversal (a subsequent static switch cuts off this branch of the graph).
					// By having the static switch handling done differently we incorporate them into the duplication
					// process so that we never even create them.
					IsValueSet = NiagaraCompilationImpl::GetStaticSwitchValueFromPin(Context, *SelectorPin.LinkedTo, SwitchValue);
				}
			}
			// pick the static switch value based on the default value of the pin since it's not connected to anything
			else
			{
				FNiagaraVariable DefaultVariable(SelectorPin.Variable);
				if (!DefaultVariable.IsDataAllocated())
				{
					FNiagaraEditorUtilities::ResetVariableToDefaultValue(DefaultVariable);
				}

				IsValueSet = EvaluateVariableSwitchValue(DefaultVariable, SwitchValue);
			}
		}
	}
	else
	{
		// check the calling function
		for (const FNiagaraCompilationInputPin& InputPin : Context.GetCurrentFunctionNode()->InputPins)
		{
			if (InputPin.PinName.IsEqual(InputParameterName) && InputPin.Variable.GetType() == InputType)
			{
				if (FNiagaraCompilationNodeStaticSwitch::ResolveConstantValue(InputPin, SwitchValue))
				{
					IsValueSet = true;
				}
			}
		}
	}

	if (IsValueSet && SwitchValue != INDEX_NONE)
	{
		TArray<const FNiagaraCompilationInputPin*> ValidInputPins;
		ValidInputPins.Reserve(OutputPins.Num());

		if (SwitchValue != INDEX_NONE)
		{
			const int32 BaseInputIndex = GetBaseInputChannel(SwitchValue);
			int32 OutputPinCount = OutputPins.Num();
			for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
			{
				const FNiagaraCompilationInputPin& SwitchInputPin = InputPins[BaseInputIndex + OutputPinIt];
				const FNiagaraCompilationOutputPin& SwitchOutputPin = OutputPins[OutputPinIt];

				ValidInputPins.Add(&SwitchInputPin);
				Branches.Add(&SwitchOutputPin, &SwitchInputPin);
			}
		}

		return ValidInputPins;
	}

	return FNiagaraCompilationNode::EvaluateBranches(Context, Branches);
}

FNiagaraCompilationNodeSimTargetSelector::FNiagaraCompilationNodeSimTargetSelector(const UNiagaraNodeSimTargetSelector* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNodeUsageSelector(InNode, Context, ENodeType::SimTargetSelector)
{
}

FNiagaraCompilationNodeSimTargetSelector::FNiagaraCompilationNodeSimTargetSelector(const FNiagaraCompilationNodeSimTargetSelector& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNodeUsageSelector(InNode, Context)
{

}

void FNiagaraCompilationNodeSimTargetSelector::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	//ENiagaraSimTarget SimulationTarget = Translator->GetSimulationTarget();
	bool bCPUSim = Translator->IsCompileOptionDefined(*FNiagaraCompileOptions::CpuScriptDefine);
	bool bGPUSim = Translator->IsCompileOptionDefined(*FNiagaraCompileOptions::GpuScriptDefine);

	if (Translator->GetTargetUsage() >= ENiagaraScriptUsage::Function && Translator->GetTargetUsage() <= ENiagaraScriptUsage::DynamicInput)
	{
		// Functions through Dynamic inputs are missing the context, so just use CPU by default.
		bCPUSim = true;
	}

	int32 VarIdx;
	if (bCPUSim/*SimulationTarget == ENiagaraSimTarget::CPUSim*/)
	{
		VarIdx = 0;
	}
	else if (bGPUSim/*SimulationTarget == ENiagaraSimTarget::GPUComputeSim*/)
	{
		VarIdx = InputPins.Num() / 2;
	}
	else
	{
		Translator->Error(LOCTEXT("InvalidSimTarget", "Unknown simulation target"), this, nullptr);
		return;
	}

	Outputs.SetNumUninitialized(OutputPins.Num());
	for (int32 i = 0; i < OutputVars.Num(); i++)
	{
		Outputs[i] = Translator->CompileInputPin(&InputPins[VarIdx + i]);
	}
}

FNiagaraCompilationNodeUsageSelector::FNiagaraCompilationNodeUsageSelector(const UNiagaraNodeUsageSelector* InNode, FNiagaraCompilationGraphCreateContext& Context, ENodeType InNodeType)
	: FNiagaraCompilationNode(InNodeType, InNode, Context)
{
	OutputVars = InNode->OutputVars;
	OutputVarGuids = InNode->OutputVarGuids;
}

FNiagaraCompilationNodeUsageSelector::FNiagaraCompilationNodeUsageSelector(const FNiagaraCompilationNodeUsageSelector& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, OutputVars(InNode.OutputVars)
	, OutputVarGuids(InNode.OutputVarGuids)
{
}

void FNiagaraCompilationNodeUsageSelector::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	ENiagaraScriptUsage BaseUsage = Builder.GetBaseUsageContext();
	ENiagaraScriptUsage CurrentUsage = Builder.GetCurrentUsageContext();

	check(OutputPins.Num() == OutputVars.Num());

	ENiagaraScriptGroup UsageGroup = ENiagaraScriptGroup::Max;
	if (UNiagaraScript::ConvertUsageToGroup(CurrentUsage, UsageGroup))
	{
		if (bRecursive)
		{
			int32 VarIdx = 0;
			for (int64 i = 0; i < (int64)ENiagaraScriptGroup::Max; i++)
			{
				if ((int64)UsageGroup == i)
				{
					break;
				}

				VarIdx += OutputVars.Num();
			}

			for (int32 i = 0; i < OutputVars.Num(); i++)
			{
				//Builder.VisitInputPin(InputPins[VarIdx + i], this, bFilterForCompilation);
				RegisterPassthroughPin(Builder, &InputPins[VarIdx + i], &OutputPins[i], bFilterForCompilation, true);
			}
		}
	}
}

void FNiagaraCompilationNodeUsageSelector::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	ENiagaraScriptUsage CurrentUsage = Translator->GetCurrentUsage();
	ENiagaraScriptGroup UsageGroup = ENiagaraScriptGroup::Max;
	if (UNiagaraScript::ConvertUsageToGroup(CurrentUsage, UsageGroup))
	{
		int32 VarIdx = 0;
		for (int64 i = 0; i < (int64)ENiagaraScriptGroup::Max; i++)
		{
			if ((int64)UsageGroup == i)
			{
				break;
			}

			VarIdx += OutputVars.Num();
		}

		Outputs.SetNumUninitialized(OutputPins.Num());
		for (int32 i = 0; i < OutputVars.Num(); i++)
		{
			Outputs[i] = Translator->CompileInputPin(&InputPins[VarIdx + i]);
		}
	}
	else
	{
		Translator->Error(LOCTEXT("InvalidUsage", "Invalid script usage"), this, nullptr);
	}
}

void FNiagaraCompilationNodeUsageSelector::UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const
{
	static const FName NAME_Usage = TEXT("Usage");
	switch (InOutFunctionAliasContext.CompileUsage)
	{
		case ENiagaraScriptUsage::SystemSpawnScript:
		case ENiagaraScriptUsage::SystemUpdateScript:
			InOutFunctionAliasContext.AddPermutationIdentifier(NAME_Usage, TEXT("System"));
			break;
		case ENiagaraScriptUsage::EmitterSpawnScript:
		case ENiagaraScriptUsage::EmitterUpdateScript:
			InOutFunctionAliasContext.AddPermutationIdentifier(NAME_Usage, TEXT("Emitter"));
			break;
		case ENiagaraScriptUsage::ParticleSpawnScript:
		case ENiagaraScriptUsage::ParticleUpdateScript:
		case ENiagaraScriptUsage::ParticleEventScript:
		case ENiagaraScriptUsage::ParticleSimulationStageScript:
		case ENiagaraScriptUsage::ParticleGPUComputeScript:
			InOutFunctionAliasContext.AddPermutationIdentifier(NAME_Usage, TEXT("Particle"));
			break;
	}
}

const FNiagaraCompilationOutputPin* FNiagaraCompilationNodeUsageSelector::FindOutputPin(const FNiagaraVariable& Variable) const
{
	const int32 OutputIndex = OutputVars.IndexOfByPredicate([&Variable](const FNiagaraVariable& OutputVariable) -> bool
	{
		return OutputVariable == Variable;
	});

	if (OutputVarGuids.IsValidIndex(OutputIndex))
	{
		const FGuid& OutputVarGuid = OutputVarGuids[OutputIndex];

		return OutputPins.FindByPredicate([&OutputVarGuid](const FNiagaraCompilationOutputPin& OutputPin) -> bool
		{
			return OutputPin.PersistentGuid == OutputVarGuid;
		});
	}

	return nullptr;
}

FNiagaraCompilationNodeSelect::FNiagaraCompilationNodeSelect(const UNiagaraNodeSelect* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNodeUsageSelector(InNode, Context, ENodeType::Select)
	, SelectorPinType(InNode->SelectorPinType)
{
	SelectorPinIndex = GetInputPinIndexByPersistentId(InNode->SelectorPinGuid);
	NumOptionsPerVariable = InNode->GetOptionValues().Num();
	SelectorValues = InNode->GetOptionValues();
}

FNiagaraCompilationNodeSelect::FNiagaraCompilationNodeSelect(const FNiagaraCompilationNodeSelect& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNodeUsageSelector(InNode, Context)
	, SelectorPinType(InNode.SelectorPinType)
	, SelectorPinIndex(InNode.SelectorPinIndex)
	, NumOptionsPerVariable(InNode.NumOptionsPerVariable)
	, SelectorValues(InNode.SelectorValues)
{

}

void FNiagaraCompilationNodeSelect::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	const FNiagaraCompilationInputPin& SelectorPin = InputPins[SelectorPinIndex];
	int32 Selection = Translator->CompileInputPin(&SelectorPin);

	const int32 OutputVarCount = OutputVars.Num();
	TArray<int32> ConnectedOutputVarIndices;
	TArray<FNiagaraVariable> ConnectedOutputVars;
	ConnectedOutputVarIndices.Reserve(OutputVarCount);
	ConnectedOutputVars.Reserve(OutputVarCount);
	for (int32 OutputVarIndex = 0; OutputVarIndex < OutputVarCount; ++OutputVarIndex)
	{
		const FNiagaraVariable& OutputVar = OutputVars[OutputVarIndex];
		const FNiagaraCompilationOutputPin* OutputPin = FindOutputPin(OutputVar);
		if (OutputPin && !OutputPin->LinkedTo.IsEmpty())
		{
			ConnectedOutputVarIndices.Add(OutputVarIndex);
			ConnectedOutputVars.Add(OutputVar);
		}
	}

	// a map from selector value to compiled option pins (i.e.: for selector value 0 all pins that should be case "if 0" get their compiled index added under key 0)
	TMap<int32, TArray<int32>> OptionValues;
	OptionValues.Reserve(NumOptionsPerVariable);

	for (const FNiagaraVariable& OutputVar : ConnectedOutputVars)
	{
		const FNiagaraCompilationOutputPin* PinByVariable = FindOutputPin(OutputVar);
		if (ensure(PinByVariable))
		{
			if (OutputVar.GetType() != PinByVariable->Variable.GetType())
			{
				Translator->Error(FText::Format(LOCTEXT("PinTypeOutputVarTypeMismatch", "Internal output variable type {0} does not match pin type {1}. Please refresh node."),
					FText::FromString(OutputVar.GetType().GetName()), FText::FromString(PinByVariable->Variable.GetType().GetName())),
					this, PinByVariable);
			}
			else if (PinByVariable->IsWildcard())
			{
				Translator->Error(FText::Format(LOCTEXT("PinTypeOutputWildcard", "Wildcard types are intended to be placeholders and replaced by real types.  Please resolve type for {0}."),
					FText::FromString(OutputVar.GetName().ToString())),
					this, PinByVariable);
			}
		}
	}

	int32 HandledInputCount = 0;
	const int32 InputPinCount = InputPins.Num();
	for (int32 InputPinIt = 0; InputPinIt < InputPinCount; ++InputPinIt)
	{
		const FNiagaraCompilationInputPin& InputPin = InputPins[InputPinIt];

		// skip the selector pin as it was already compiled above
		if (&InputPin == &SelectorPin)
		{
			continue;
		}

		const int32 OutputVarIndex = HandledInputCount % OutputVarCount;
		const int32 SelectorIndex = HandledInputCount / OutputVarCount;

		++HandledInputCount;
		
		// also skip if the output isn't connected
		if (!ConnectedOutputVarIndices.Contains(OutputVarIndex))
		{
			continue;
		}

		int32 CompiledInput = Translator->CompileInputPin(&InputPin);
		const int32 SelectorValue = SelectorValues[SelectorIndex];
		OptionValues.FindOrAdd(SelectorValue).Add(CompiledInput);
	}

	TArray<int32> SelectOutputs;
	Translator->Select(this, Selection, ConnectedOutputVars, OptionValues, SelectOutputs);

	// remap the compiled outputs based on list of connected outputs
	Outputs.Init(INDEX_NONE, OutputVarCount);
	for (int32 ConnectedVarIndex = 0; ConnectedVarIndex < ConnectedOutputVars.Num(); ++ConnectedVarIndex)
	{
		if (SelectOutputs.IsValidIndex(ConnectedVarIndex))
		{
			Outputs[ConnectedOutputVarIndices[ConnectedVarIndex]] = SelectOutputs[ConnectedVarIndex];
		}
	}
}

void FNiagaraCompilationNodeSelect::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	// note that we bypass the UsageSelector implementation and go straight to the Node
	FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);
}

FNiagaraCompilationNodeWriteDataSet::FNiagaraCompilationNodeWriteDataSet(const UNiagaraNodeWriteDataSet* InNode, FNiagaraCompilationGraphCreateContext& Context)
	: FNiagaraCompilationNode(ENodeType::WriteDataSet, InNode, Context)
{
	EventName = InNode->EventName;
	DataSet = InNode->DataSet;
	DataSetVariables = InNode->Variables;
}

FNiagaraCompilationNodeWriteDataSet::FNiagaraCompilationNodeWriteDataSet(const FNiagaraCompilationNodeWriteDataSet& InNode, FNiagaraCompilationGraphDuplicateContext& Context)
	: FNiagaraCompilationNode(InNode, Context)
	, EventName(InNode.EventName)
	, DataSet(InNode.DataSet)
	, DataSetVariables(InNode.DataSetVariables)
{

}

void FNiagaraCompilationNodeWriteDataSet::BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const
{
	FNiagaraCompilationNode::BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	if (ConditionalRouteParameterMapAroundMe(Builder))
	{
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	if (InputPins[0].LinkedTo)
	{
		ParamMapIdx = Builder.TraceParameterMapOutputPin(NiagaraCompilationImpl::TraceOutputPin(Builder, InputPins[0].LinkedTo, bFilterForCompilation));
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		uint32 NodeIdx = Builder.BeginNodeVisitation(ParamMapIdx, this);
		Builder.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}

	Builder.RegisterDataSetWrite(ParamMapIdx, DataSet);

	Builder.RegisterParameterMapPin(ParamMapIdx, &OutputPins[0]);
}

void FNiagaraCompilationNodeWriteDataSet::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	bool bError = false;

	//TODO implement writing to data sets in hlsl compiler and vm.

	TArray<int32> Inputs;
	CompileInputPins(Translator, Inputs);

	check(!EventName.IsNone());

	FNiagaraDataSetID AlteredDataSet = DataSet;
	AlteredDataSet.Name = EventName.IsNone() ? DataSet.Name : EventName;
	Translator->WriteDataSet(AlteredDataSet, DataSetVariables, ENiagaraDataSetAccessMode::AppendConsume, Inputs, Outputs);
}

#undef LOCTEXT_NAMESPACE
