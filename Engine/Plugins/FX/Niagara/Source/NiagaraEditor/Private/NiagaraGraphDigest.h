// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraGraph.h"
#include "NiagaraGraphDigestTypes.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraTypes.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptVariable.h"

class UEdGraphNode;
class FNiagaraFixedConstantResolver;
class FNiagaraCompilationGraph;
class FNiagaraCompilationGraphDigested;
class FNiagaraCompilationGraphInstanced;
class FNiagaraCompilationNode;
class FNiagaraPrecompileData;
class FNiagaraCompilationCopyData;
class FNiagaraParameterHandle;

enum class ENiagaraStaticSwitchType : uint8;

#define NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(name) \
	public: \
		using SourceClass = UNiagaraNode##name; \
	public: \
		virtual UClass* GetSourceNodeClass() const override; \
		static UClass* GetStaticSourceNodeClass(); 

#define NIAGARA_GRAPH_DIGEST_NODE_IMPLEMENT_BODY(name) \
	UClass* FNiagaraCompilationNode##name::GetStaticSourceNodeClass() \
	{ \
		return SourceClass::StaticClass(); \
	} \
	UClass* FNiagaraCompilationNode##name::GetSourceNodeClass() const \
	{ \
		return SourceClass::StaticClass(); \
	}

class FNiagaraCompilationInputPin;
class FNiagaraCompilationOutputPin;
struct FNiagaraCompilationGraphCreateContext;
struct FNiagaraCompilationGraphDuplicateContext;
struct FNiagaraCompilationGraphInstanceContext;
struct FNiagaraDigestHelper;
template<typename T> class TNiagaraHlslTranslator;

using FNiagaraCompilationBranchMap = TMap<const FNiagaraCompilationOutputPin*, const FNiagaraCompilationInputPin*>;

class FNiagaraCompilationPin
{
protected:
	UE_NONCOPYABLE(FNiagaraCompilationPin);
	virtual ~FNiagaraCompilationPin() = default;

	FNiagaraCompilationPin() = delete;
	FNiagaraCompilationPin(const UEdGraphPin* InPin);
	FNiagaraCompilationPin(const FNiagaraCompilationPin& SourcePin, const FNiagaraCompilationNode* InOwningNode, FNiagaraCompilationGraphDuplicateContext& Context);

public:
	using FLinkedPinView = TArrayView<const FNiagaraCompilationPin* const>;
	virtual FLinkedPinView GetLinkedPins() const = 0;

	UEdGraphPin* GetSourcePin() const;

	bool IsWildcard() const;

	FNiagaraVariable Variable;
	int32 SourcePinIndex = INDEX_NONE;
	FName PinName;
	FString DefaultValue;
	FGuid PersistentGuid;
	FEdGraphPinType PinType;
	FGuid UniquePinId;
	bool bHidden;

	const FNiagaraCompilationNode* OwningNode = nullptr;
	const EEdGraphPinDirection Direction;
};

class FNiagaraCompilationInputPin : public FNiagaraCompilationPin
{
public:
	UE_NONCOPYABLE(FNiagaraCompilationInputPin);
	
	FNiagaraCompilationInputPin() = delete;
	FNiagaraCompilationInputPin(const UEdGraphPin* InPin);
	FNiagaraCompilationInputPin(const FNiagaraCompilationInputPin& SourceInputPin, const FNiagaraCompilationNode* InOwningNode, FNiagaraCompilationGraphDuplicateContext& Context);

	bool Disconnect();

	virtual FLinkedPinView GetLinkedPins() const override
	{
		return MakeArrayView(reinterpret_cast<const FNiagaraCompilationPin* const*>(&LinkedTo), 1);
	}

	const FNiagaraCompilationInputPin* TraceBranchMap(const FNiagaraCompilationBranchMap& BranchMap) const;

	const FNiagaraCompilationOutputPin* LinkedTo = nullptr;
	bool bDefaultValueIsIgnored = false;
};

class FNiagaraCompilationOutputPin : public FNiagaraCompilationPin
{
public:
	UE_NONCOPYABLE(FNiagaraCompilationOutputPin);

	FNiagaraCompilationOutputPin() = delete;
	FNiagaraCompilationOutputPin(const UEdGraphPin* InPin);
	FNiagaraCompilationOutputPin(const FNiagaraCompilationOutputPin& SourceOutputPin, const FNiagaraCompilationNode* InOwningNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual FLinkedPinView GetLinkedPins() const override
	{
		return MakeArrayView(reinterpret_cast<const FNiagaraCompilationPin* const*>(LinkedTo.GetData()), LinkedTo.Num());
	}

	TArray<const FNiagaraCompilationInputPin*> LinkedTo;
};

class INiagaraCompilationFeedbackInterface
{
public:
	virtual void Message(FNiagaraCompileEventSeverity Severity, FText MessageText, const FNiagaraCompilationNode* Node, const FNiagaraCompilationPin* Pin, FString ShortDescription = FString(), bool bDismissable = false) {};
	virtual void Error(FText ErrorText, const FNiagaraCompilationNode* Node, const FNiagaraCompilationPin* Pin, FString ShortDescription = FString(), bool bDismissable = false) {};
	virtual void Warning(FText WarningText, const FNiagaraCompilationNode* Node, const FNiagaraCompilationPin* Pin, FString ShortDescription = FString(), bool bDismissable = false) {};
};

class FNiagaraCompilationNode
{
public:
#define NIAGARA_GRAPH_DIGEST_NODE_TYPE(name) name,
	enum class ENodeType : int32
	{
		NIAGARA_GRAPH_DIGEST_NODE_TYPE_LIST
		NodeTypeCount
	};
#undef NIAGARA_GRAPH_DIGEST_NODE_TYPE

	using SourceClass = UEdGraphNode;
	using FTranslator = TNiagaraHlslTranslator<FNiagaraCompilationDigestBridge>;
	using FParameterMapHistoryBuilder = TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationDigestBridge>;
	using FParameterMapHistory = TNiagaraParameterMapHistory<FNiagaraCompilationDigestBridge>;

	UE_NONCOPYABLE(FNiagaraCompilationNode);
	FNiagaraCompilationNode(ENodeType NodeType, const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNode(const FNiagaraCompilationNode& SourceNode, FNiagaraCompilationGraphDuplicateContext& Context);
	FNiagaraCompilationNode() = delete;
	virtual ~FNiagaraCompilationNode() = default;

	bool IsChildOf(UClass* NodeClass) const;
	virtual UClass* GetSourceNodeClass() const = 0;

	template<typename CompilationNodeType>
	const CompilationNodeType* AsType() const
	{
		if (GetSourceNodeClass()->IsChildOf(CompilationNodeType::GetStaticSourceNodeClass()))
		{
			return static_cast<const CompilationNodeType*>(this);
		}
		return nullptr;
	}

	template<typename CompilationNodeType>
	CompilationNodeType* AsType()
	{
		if (GetSourceNodeClass()->IsChildOf(CompilationNodeType::GetStaticSourceNodeClass()))
		{
			return static_cast<CompilationNodeType*>(this);
		}
		return nullptr;
	}

	template<typename CompilationNodeType>
	const CompilationNodeType& AsTypeRef() const
	{
		const CompilationNodeType* TypePtr = AsType<CompilationNodeType>();
		check(TypePtr);
		return *TypePtr;
	}

	template<typename CompilationNodeType>
	CompilationNodeType& AsTypeRef()
	{
		CompilationNodeType* TypePtr = AsType<CompilationNodeType>();
		check(TypePtr);
		return *TypePtr;
	}

	TArray<FNiagaraCompilationInputPin> InputPins;
	TArray<FNiagaraCompilationOutputPin> OutputPins;

	const ENodeType NodeType;
	FString NodeName;
	FString FullName;
	FString FullTitle;
	FGuid NodeGuid;
	bool NodeEnabled = false;
	ENiagaraNumericOutputTypeSelectionMode NumericSelectionMode;
	const FNiagaraCompilationGraph* OwningGraph = nullptr;

	TWeakObjectPtr<const UEdGraphNode> SourceNode;

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;
	virtual void ResolveNumerics();
	virtual void UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const {};

	bool ConditionalRouteParameterMapAroundMe(FParameterMapHistoryBuilder& Builder) const;
	void RouteParameterMapAroundMe(FParameterMapHistoryBuilder& Builder) const;
	void RegisterPassthroughPin(FParameterMapHistoryBuilder& Builder, const FNiagaraCompilationInputPin* InputPin, const FNiagaraCompilationOutputPin* OutputPin, bool bFilterForCompilation, bool bVisitInputPin) const;

	FString GetTypeName() const;

	int32 GetInputPinIndexById(const FGuid& InId) const;
	int32 GetInputPinIndexByPersistentId(const FGuid& InId) const;
	int32 GetOutputPinIndexById(const FGuid& InId) const;

	const FNiagaraCompilationInputPin* GetInputExecPin() const;
	const FNiagaraCompilationOutputPin* GetOutputExecPin() const;

	virtual TArray<const FNiagaraCompilationInputPin*> EvaluateBranches(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationBranchMap& Branches) const;

protected:
	void ResolveNumericPins(TConstArrayView<int32> InputPinIndices, TConstArrayView<int32> OutputPinIndices);
	virtual FNiagaraTypeDefinition ResolveCustomNumericType(TConstArrayView<FNiagaraTypeDefinition> ConcreteInputTypes) const;

	bool CompileInputPins(FTranslator* Translator, TArray<int32>& OutInputResults) const;
};

// Helper class which can recursively generate the effective ChangeIds for a graph and all graphs that it
// references.  With our current setup for digested graph, this new ChangeId reflects the fact that if a
// graph references a changed graph (through function call or emitter node) then we'll need to re-digest
// both caller and callee
class FNiagaraGraphChangeIdBuilder
{
public:
	void ParseReferencedGraphs(const UNiagaraGraph* Graph);
	FGuid FindChangeId(const UNiagaraGraph* Graph) const;

protected:
	FGuid RecursiveBuildGraphChangeId(const UNiagaraGraph* Graph, TSet<const UNiagaraGraph*>& CurrentGraphChain);

	TMap<const UNiagaraGraph*, FGuid> ChangeIdMap;
};

class FNiagaraCompilationGraph
{
public:
	virtual ~FNiagaraCompilationGraph() = default;

	using FSharedPtr = TSharedPtr<FNiagaraCompilationGraph, ESPMode::ThreadSafe>;

	bool IsValid() const { return true; } // todo - when creation is offloaded to a task we'll need this to synchronize completion

	TArray<TUniquePtr<FNiagaraCompilationNode>> Nodes;
	TArray<int32> InputNodeIndices;
	TArray<int32> OutputNodeIndices;
	FNiagaraScriptVariableBinding VariableBinding;
	TArray<FNiagaraScriptVariableData> ScriptVariableData;
	TArray<FNiagaraVariableBase> StaticSwitchInputs;

	// true if the graph contains a static variable or one of it's recursively referenced graphs contains one
	bool bContainsStaticVariables = false;
	FString SourceScriptName;
	FString SourceScriptFullName;

	void FindOutputNodes(TArray<const FNiagaraCompilationNodeOutput*>& OutputNodes) const;
	void FindOutputNodes(ENiagaraScriptUsage TargetUsageType, TArray<const FNiagaraCompilationNodeOutput*>& OutputNodes) const;
	const FNiagaraCompilationNodeOutput* FindOutputNode(ENiagaraScriptUsage Usage, const FGuid& UsageId) const;
	const FNiagaraCompilationNodeOutput* FindEquivalentOutputNode(ENiagaraScriptUsage Usage, const FGuid& UsageId) const;
	TArray<const FNiagaraCompilationNode*> FindOutputNodesByUsage(TConstArrayView<ENiagaraScriptUsage> Usages) const;

	void FindInputNodes(TArray<const FNiagaraCompilationNodeInput*>& OutInputNodes, UNiagaraGraph::FFindInputNodeOptions Options = UNiagaraGraph::FFindInputNodeOptions()) const;
	/** If this graph is the source of a function call, it can add a string to the function name to discern it from different
	  * function calls to the same graph. For example, if the graph contains static switches and two functions call it with
	  * different switch parameters, the final function names in the hlsl must be different.
	  */
	void UpdateFunctionAliasByContext(FNiagaraGraphFunctionAliasContext& FunctionAliasContext) const;

	bool HasParametersOfType(const FNiagaraTypeDefinition& Type) const;

	TOptional<ENiagaraDefaultMode> GetDefaultMode(const FNiagaraVariableBase&, FNiagaraScriptVariableBinding& Binding) const;
	TOptional<FNiagaraVariableMetaData> GetMetaData(const FNiagaraVariableBase&) const;

	void CollectReachableNodes(const FNiagaraCompilationNodeOutput* OutputNode, TArray<const FNiagaraCompilationNode*>& ReachableNodes) const;
	void BuildTraversal(const FNiagaraCompilationNode* RootNode, TArray<const FNiagaraCompilationNode*>& OrderedNodes) const;
	void BuildTraversal(const FNiagaraCompilationNode* RootNode, const FNiagaraCompilationBranchMap& Branches, TArray<const FNiagaraCompilationNode*>& OrderedNodes) const;

	void EvaluateStaticBranches(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationBranchMap& Branches) const;

	void NodeTraversal(
		bool bRecursive,
		bool bOrdered,
		TFunctionRef<bool(const FNiagaraCompilationNodeOutput&)> RootNodeFilter,
		TFunctionRef<bool(const FNiagaraCompilationNode&)> NodeOperation) const;

	virtual FNiagaraCompilationGraphDigested* AsDigested() { return nullptr; }
	virtual FNiagaraCompilationGraphInstanced* AsInstanced() { return nullptr; }

protected:
	const FNiagaraScriptVariableData* GetScriptVariableData(const FNiagaraVariableBase&) const;

	TArray<const FNiagaraCompilationNode*> GetOutputNodes() const;
};

// Digested version of the compilation graph.  Created on the game thread from a UNiagaraGraph
// this will act as the source for FNiagaraCompilationGraphInstanced that will be crated in async
// tasks.  Is also responsible for securing references to UObjects over it's lifetime as an FGCObject
class FNiagaraCompilationGraphDigested : public TSharedFromThis<FNiagaraCompilationGraphDigested, ESPMode::ThreadSafe>
										, public FNiagaraCompilationGraph
										, FGCObject
{
public:
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	TSharedPtr<FNiagaraCompilationGraphInstanced, ESPMode::ThreadSafe> Instantiate(const FNiagaraPrecompileData* PrecompileData, const FNiagaraCompilationCopyData* CopyCompilationData, const TArray<ENiagaraScriptUsage>& Usages, const FNiagaraFixedConstantResolver& ConstantResolver) const;

	void Digest(const UNiagaraGraph* InGraph, const FNiagaraGraphChangeIdBuilder& Digester);

	UNiagaraDataInterface* DigestDataInterface(UNiagaraDataInterface* SourceDataInterface);
	void RegisterObjectAsset(FName VariableName, UObject* SourceObjectAsset);

	virtual FNiagaraCompilationGraphDigested* AsDigested() override { return this; }

	using FDataInterfaceCDOMap = TMap<TObjectPtr<UClass>, TObjectPtr<UNiagaraDataInterface>>;
	void CollectReferencedDataInterfaceCDO(FDataInterfaceCDOMap& Interfaces) const;

	using FDataInterfaceDuplicateMap = TMap<TObjectKey<UNiagaraDataInterface>, TObjectPtr<UNiagaraDataInterface>>;
	FDataInterfaceDuplicateMap CachedDataInterfaceDuplicates;
	FDataInterfaceCDOMap CachedDataInterfaceCDODuplicates;
	TMap<FName, TObjectPtr<UObject>> CachedNamedObjectAssets;

	TWeakObjectPtr<const UNiagaraGraph> SourceGraph;

protected:
	TSharedPtr<FNiagaraCompilationGraphInstanced, ESPMode::ThreadSafe> InstantiateSubGraph(
		const TArray<ENiagaraScriptUsage>& Usages,
		const FNiagaraCompilationCopyData* CopyCompilationData,
		const FNiagaraCompilationBranchMap& Branches,
		TArray<FNiagaraCompilationNodeFunctionCall*>& PendingInstantiations) const;

	TArray<const FNiagaraCompilationGraphDigested*> ChildGraphs;
};

// Instanced version of the compilation graph.  Created from a FNiagaraCompilationGraphDigested
// this will act as the representation of the UNiagaraGraph that is actually being translated and
// compiled (i.e. will have nodes stripped out if static branches aren't taken).
class FNiagaraCompilationGraphInstanced : public FNiagaraCompilationGraph
{
public:
	void AggregateChildGraph(const FNiagaraCompilationGraphInstanced* ChildGraph);

	void ResolveNumerics(FNiagaraCompilationGraphInstanceContext& Context);
	void Refine(FNiagaraCompilationGraphInstanceContext& Context, const FNiagaraCompilationNodeFunctionCall* CallingNode);

	virtual FNiagaraCompilationGraphInstanced* AsInstanced() { return this; }

	TSharedPtr<const FNiagaraCompilationGraph> InstantiationSourceGraph;

protected:
	void ValidateRefinement() const;
	void PatchGenericNumericsFromCaller(FNiagaraCompilationGraphInstanceContext& Context);
	void InheritDebugState(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationNodeFunctionCall& FunctionCallNode);
	void PropagateDefaultValues(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationNodeFunctionCall& FunctionCallNode);
	void StripUnconnectedPins(FNiagaraCompilationGraphInstanceContext& Context);
};

class FNiagaraCompilationScript
{
public:
	TUniquePtr<FNiagaraCompilationGraph> CompilationGraph;
	TWeakObjectPtr<const UNiagaraScript> SourceScript;
};

class FNiagaraCompilationNodeEmitter : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(Emitter);

	FNiagaraCompilationNodeEmitter(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeEmitter(const FNiagaraCompilationNodeEmitter& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	FNiagaraEmitterID EmitterID;
	FGuid EmitterHandleID;
	FString EmitterUniqueName;
	FString EmitterName;
	FString EmitterPathName;
	FString EmitterHandleIdString;
	FNiagaraCompilationGraph::FSharedPtr CalledGraph;
	ENiagaraScriptUsage Usage;
};

class FNiagaraCompilationNodeFunctionCall : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(FunctionCall);

	FNiagaraCompilationNodeFunctionCall(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context, ENodeType InNodeType = ENodeType::FunctionCall);
	FNiagaraCompilationNodeFunctionCall(const FNiagaraCompilationNodeFunctionCall& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	const FNiagaraCompilationInputPin* FindStaticSwitchInputPin(FName VariableName) const;
	TSet<FName> GetUnusedFunctionInputPins() const;
	bool FindAutoBoundInput(const FNiagaraCompilationNodeInput* InputNode, const FNiagaraCompilationInputPin* PinToAutoBind, FNiagaraVariable& OutFoundVar, ENiagaraInputNodeUsage& OutNodeUsage) const;
	bool GetParameterMapDefaultValue(const FNiagaraFixedConstantResolver& ConstantResolver, FNiagaraVariable& DefaultValue) const;
	bool HasOverridePin(const FNiagaraParameterHandle& ParameterHandle) const;
	const FNiagaraCompilationNodeParameterMapSet* GetOverrideNode() const;

	void MultiFindParameterMapDefaultValues(ENiagaraScriptUsage ScriptUsage, const FNiagaraFixedConstantResolver& ConstantResolver, TArrayView<FNiagaraVariable> Variables) const;

	FNiagaraCompilationGraph::FSharedPtr CalledGraph;
	ENiagaraFunctionDebugState DebugState;
	FString FunctionName;
	FString FunctionScriptName;
	FNiagaraFunctionSignature Signature;
	ENiagaraScriptUsage CalledScriptUsage;

	using FTaggedVariable = TTuple<FNiagaraVariable, FName>;
	TArray<FTaggedVariable> PropagatedStaticSwitchParameters;
	bool bInheritDebugState;

	bool bValidateDataInterfaces;
};

class FNiagaraCompilationNodeAssignment : public FNiagaraCompilationNodeFunctionCall
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(Assignment);

	FNiagaraCompilationNodeAssignment(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeAssignment(const FNiagaraCompilationNodeAssignment& InNode, FNiagaraCompilationGraphDuplicateContext& Context);
};

class FNiagaraCompilationNodeCustomHlsl : public FNiagaraCompilationNodeFunctionCall
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(CustomHlsl);

	FNiagaraCompilationNodeCustomHlsl(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeCustomHlsl(const FNiagaraCompilationNodeCustomHlsl& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;

	bool TokensContainsName(FName InName) const;

	ENiagaraScriptUsage CustomScriptUsage;
	FNiagaraFunctionSignature Signature;
	FString CustomHlsl;
	TArray<FString> Tokens;
	TArray<int32> SortedTokenIndices;
	TArray<FNiagaraCustomHlslInclude> CustomIncludePaths;
	bool bCallsImpureFunctions = false;

private:
	struct FIndexedTokenLess;
	struct FStringViewLess;
};

class FNiagaraCompilationNodeIf : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(If);

	FNiagaraCompilationNodeIf(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeIf(const FNiagaraCompilationNodeIf& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	virtual void ResolveNumerics() override;

	TArray<FNiagaraVariable> OutputVariables;
	int32 ConditionalPinIndex;
	TArray<int32> FalseInputPinIndices;
	TArray<int32> TrueInputPinIndices;
};

class FNiagaraCompilationNodeInput : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(Input);

	FNiagaraCompilationNodeInput(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeInput(const FNiagaraCompilationNodeInput& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;
	virtual void UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const override;

	FNiagaraVariable InputVariable;
	ENiagaraInputNodeUsage Usage;
	TArray<FString> DataInterfaceEmitterReferences;
	int32 CallSortPriority;
	FName DataInterfaceName = NAME_None;
	FName ObjectAssetName = NAME_None;
	bool bRequired;
	bool bExposed;
	bool bCanAutoBind;

	TObjectPtr<UNiagaraDataInterface> DuplicatedDataInterface;
	FSoftObjectPath ObjectAssetPath;
};

class FNiagaraCompilationNodeOp : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(Op);

	FNiagaraCompilationNodeOp(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeOp(const FNiagaraCompilationNodeOp& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	FName OpName;

protected:
	virtual FNiagaraTypeDefinition ResolveCustomNumericType(TConstArrayView<FNiagaraTypeDefinition> ConcreteInputTypes) const override;
};

class FNiagaraCompilationNodeOutput : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(Output);

	FNiagaraCompilationNodeOutput(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeOutput(const FNiagaraCompilationNodeOutput& SourceNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>&) const;

	ENiagaraScriptUsage Usage;
	FGuid UsageId;
	TOptional<FName> StackContextOverrideName;
	TArray<FNiagaraVariable> Outputs;
};

class FNiagaraCompilationNodeOutputTag : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(OutputTag);

	FNiagaraCompilationNodeOutputTag(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeOutputTag(const FNiagaraCompilationNodeOutputTag& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	bool bEmitMessageOnFailure;
	bool bEditorOnly;
	FNiagaraCompileEventSeverity FailureSeverity;

};

struct FNiagaraCompilationCachedConnection
{
	FGuid SourcePinId;
	FGuid DestinationPinId;
	TArray<FName> SourcePath;
	TArray<FName> DestinationPath;
};

class FNiagaraCompilationNodeConvert : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(Convert);

	FNiagaraCompilationNodeConvert(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeConvert(const FNiagaraCompilationNodeConvert& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	using FCachedConnection = FNiagaraCompilationCachedConnection;
	TArray<FCachedConnection> Connections;
};

class FNiagaraCompilationNodeParameterMapGet : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(ParameterMapGet);

	FNiagaraCompilationNodeParameterMapGet(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeParameterMapGet(const FNiagaraCompilationNodeParameterMapGet& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	TArray<int32> DefaultInputPinIndices;
};

class FNiagaraCompilationNodeParameterMapSet : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(ParameterMapSet);

	FNiagaraCompilationNodeParameterMapSet(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context, ENodeType InNodeType = ENodeType::ParameterMapSet);
	FNiagaraCompilationNodeParameterMapSet(const FNiagaraCompilationNodeParameterMapSet& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

protected:
	using FInputPinCollection = TArray<const FNiagaraCompilationInputPin*, TInlineAllocator<16>>;
	virtual void CollectInputPinsToCompile(FTranslator* Translator, FInputPinCollection& ActiveInputPins) const;
};

class FNiagaraCompilationNodeParameterMapFor : public FNiagaraCompilationNodeParameterMapSet
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(ParameterMapFor);

	FNiagaraCompilationNodeParameterMapFor(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context, ENodeType InNodeType = ENodeType::ParameterMapFor);
	FNiagaraCompilationNodeParameterMapFor(const FNiagaraCompilationNodeParameterMapFor& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

protected:
	virtual void CollectInputPinsToCompile(FTranslator* Translator, FInputPinCollection& ActiveInputPins) const override;
};

class FNiagaraCompilationNodeParameterMapForWithContinue : public FNiagaraCompilationNodeParameterMapFor
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(ParameterMapForWithContinue);

	FNiagaraCompilationNodeParameterMapForWithContinue(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeParameterMapForWithContinue(const FNiagaraCompilationNodeParameterMapForWithContinue& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

protected:
	virtual void CollectInputPinsToCompile(FTranslator* Translator, FInputPinCollection& ActiveInputPins) const override;
};

class FNiagaraCompilationNodeParameterMapForIndex : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(ParameterMapForIndex);

	FNiagaraCompilationNodeParameterMapForIndex(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeParameterMapForIndex(const FNiagaraCompilationNodeParameterMapForIndex& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;
};

class FNiagaraCompilationNodeReadDataSet : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(ReadDataSet);

	FNiagaraCompilationNodeReadDataSet(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeReadDataSet(const FNiagaraCompilationNodeReadDataSet& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	FNiagaraDataSetID DataSet;
	TArray<FNiagaraVariable> DataSetVariables;
};

class FNiagaraCompilationNodeUsageSelector : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(UsageSelector);

	FNiagaraCompilationNodeUsageSelector(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context, ENodeType InNodeType = ENodeType::UsageSelector);
	FNiagaraCompilationNodeUsageSelector(const FNiagaraCompilationNodeUsageSelector& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;
	virtual void UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const override;

	const FNiagaraCompilationOutputPin* FindOutputPin(const FNiagaraVariable& Variable) const;

	TArray<FNiagaraVariable> OutputVars;
	TArray<FGuid> OutputVarGuids;
};

class FNiagaraCompilationNodeSelect : public FNiagaraCompilationNodeUsageSelector
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(Select);

	FNiagaraCompilationNodeSelect(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeSelect(const FNiagaraCompilationNodeSelect& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	FNiagaraTypeDefinition SelectorPinType;
	int32 SelectorPinIndex;
	int32 NumOptionsPerVariable;
	TArray<int32> SelectorValues;
};

class FNiagaraCompilationNodeStaticSwitch : public FNiagaraCompilationNodeUsageSelector
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(StaticSwitch);

	FNiagaraCompilationNodeStaticSwitch(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeStaticSwitch(const FNiagaraCompilationNodeStaticSwitch& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;
	virtual void ResolveNumerics() override;
	virtual void UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const override;
	virtual TArray<const FNiagaraCompilationInputPin*> EvaluateBranches(FNiagaraCompilationGraphInstanceContext& Context, FNiagaraCompilationBranchMap& Branches) const override;

	static bool ResolveConstantValue(const FNiagaraCompilationInputPin& Pin, int32& Value);

	const FNiagaraCompilationOutputPin* TraceOutputPin(FParameterMapHistoryBuilder& Builder, const FNiagaraCompilationOutputPin* OutputPin, bool bFilterForCompilation) const;
	int32 GetBaseInputChannel(int32 SelectorValue) const;

	bool bSetByCompiler;
	bool bSetByPin;
	bool bDebugStateSwitch;
	ENiagaraStaticSwitchType SwitchType;
	FNiagaraTypeDefinition InputType;
	int32 SwitchBranchCount;
	int32 SelectorPinIndex;
	FName InputParameterName;
	FName SwitchConstant;
};

class FNiagaraCompilationNodeSimTargetSelector : public FNiagaraCompilationNodeUsageSelector
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(SimTargetSelector);

	FNiagaraCompilationNodeSimTargetSelector(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeSimTargetSelector(const FNiagaraCompilationNodeSimTargetSelector& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;
};

class FNiagaraCompilationNodeWriteDataSet : public FNiagaraCompilationNode
{
public:
	NIAGARA_GRAPH_DIGEST_NODE_GENERATE_BODY(WriteDataSet);

	FNiagaraCompilationNodeWriteDataSet(const SourceClass* InNode, FNiagaraCompilationGraphCreateContext& Context);
	FNiagaraCompilationNodeWriteDataSet(const FNiagaraCompilationNodeWriteDataSet& InNode, FNiagaraCompilationGraphDuplicateContext& Context);

	virtual void BuildParameterMapHistory(FParameterMapHistoryBuilder& Builder, bool bRecursive, bool bFilterForCompilation) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	FName EventName;
	FNiagaraDataSetID DataSet;
	TArray<FNiagaraVariable> DataSetVariables;
};
