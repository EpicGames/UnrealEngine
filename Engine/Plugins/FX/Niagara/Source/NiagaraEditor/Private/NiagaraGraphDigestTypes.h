// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define NIAGARA_GRAPH_DIGEST_NODE_TYPE_LIST \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(Assignment) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(Emitter) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(FunctionCall) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(If) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(CustomHlsl) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(Input) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(Op) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(Output) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(OutputTag) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(Convert) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(ParameterMapGet) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(ParameterMapFor) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(ParameterMapForWithContinue) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(ParameterMapForIndex) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(ParameterMapSet) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(ReadDataSet) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(Select) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(StaticSwitch) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(SimTargetSelector) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(UsageSelector) \
	NIAGARA_GRAPH_DIGEST_NODE_TYPE(WriteDataSet)

// forward declares
#define NIAGARA_GRAPH_DIGEST_NODE_TYPE(name) class UNiagaraNode##name;
NIAGARA_GRAPH_DIGEST_NODE_TYPE_LIST
#undef NIAGARA_GRAPH_DIGEST_NODE_TYPE

#define NIAGARA_GRAPH_DIGEST_NODE_TYPE(name) class FNiagaraCompilationNode##name;
NIAGARA_GRAPH_DIGEST_NODE_TYPE_LIST
#undef NIAGARA_GRAPH_DIGEST_NODE_TYPE

class FNiagaraCompilationInputPin;
class FNiagaraCompilationNode;
class FNiagaraCompilationOutputPin;
class FNiagaraCompilationPin;

