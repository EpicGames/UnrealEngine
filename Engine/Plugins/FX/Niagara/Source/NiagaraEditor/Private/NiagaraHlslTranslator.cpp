// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraHlslTranslator.h"

#include "Algo/Contains.h"
#include "EdGraphSchema_Niagara.h"
#include "INiagaraEditorTypeUtilities.h"
#include "Modules/ModuleManager.h"
#include "NiagaraAttributeTrimmer.h"
#include "NiagaraCommon.h"
#include "NiagaraCompilationPrivate.h"
#include "NiagaraConstants.h"
#include "NiagaraDataInterface.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorTickables.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraFunctionLibrary.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraGraph.h"
#include "NiagaraGraphDigest.h"
#include "NiagaraNode.h"
#include "NiagaraNodeConvert.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeIf.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeParameterMapFor.h"
#include "NiagaraNodeSelect.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraNodeOp.h"
#include "NiagaraSettings.h"
#include "NiagaraShared.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraTrace.h"
#include "NiagaraTraversalStateContext.h"
#include "ShaderCore.h"
#include "String/ParseTokens.h"
#include "Misc/FileHelper.h"
#include "Misc/CoreMiscDefines.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraShader.h"
#include "HAL/Platform.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraHlslTranslator)

#define LOCTEXT_NAMESPACE "NiagaraCompiler"

DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - Translate"), STAT_NiagaraEditor_HlslTranslator_Translate, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - BuildParameterMapHlslDefinitions"), STAT_NiagaraEditor_HlslTranslator_BuildParameterMapHlslDefinitions, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - Emitter"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_Emitter, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - MapGet"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_MapGet, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - FunctionCall"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_FunctionCall, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - FunctionCallCloneGraphNumeric"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_FunctionCallCloneGraphNumeric, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - FunctionCallCloneGraphNonNumeric"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_FunctionCallCloneGraphNonNumeric, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - RegisterFunctionCall"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - GenerateFunctionCall"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_GenerateFunctionCall, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - CustomHLSL"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_CustomHLSL, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - FuncBody"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_FuncBody, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - Output"), STAT_NiagaraEditor_HlslTranslator_Output, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - MapSet"), STAT_NiagaraEditor_HlslTranslator_MapSet, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - MapForBegin"), STAT_NiagaraEditor_HlslTranslator_MapForBegin, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - MapForEnd"), STAT_NiagaraEditor_HlslTranslator_MapForEnd, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - Operation"), STAT_NiagaraEditor_HlslTranslator_Operation, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - If"), STAT_NiagaraEditor_HlslTranslator_If, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - Select"), STAT_NiagaraEditor_HlslTranslator_Select, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - CompilePin"), STAT_NiagaraEditor_HlslTranslator_CompilePin, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - CompileOutputPin"), STAT_NiagaraEditor_HlslTranslator_CompileOutputPin, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - GetParameter"), STAT_NiagaraEditor_HlslTranslator_GetParameter, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - GenerateFunctionSignature"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_GenerateFunctionSignature, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - RegisterFunctionCall_Source"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall_Source, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - RegisterFunctionCall_Compile"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall_Compile, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - RegisterFunctionCall_Signature"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall_Signature, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - RegisterFunctionCall_FunctionDefStr"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall_FunctionDefStr, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - GenerateFunctionSignature_UniqueDueToMaps"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_GenerateFunctionSignature_UniqueDueToMaps, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - GenerateFunctionSignature_Outputs"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_GenerateFunctionSignature_Outputs, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - GenerateFunctionSignature_Inputs"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_GenerateFunctionSignature_Inputs, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - HlslTranslator - GenerateFunctionSignature_FindInputNodes"), STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_GenerateFunctionSignature_FindInputNodes, STATGROUP_NiagaraEditor);

#define NIAGARA_SCOPE_CYCLE_COUNTER(x) //SCOPE_CYCLE_COUNTER(x)

static int GNiagaraCompilationUseNewMissingDefaultTraversal = 1;
static FAutoConsoleVariableRef CVarNiagaraCompilationUseNewMissingDefaultTraversal(
	TEXT("fx.Niagara.Compilation.UseNewMissingDefaultTraversal"),
	GNiagaraCompilationUseNewMissingDefaultTraversal,
	TEXT("Whether or not to use the new missing default traversal when compiling scripts."),
	ECVF_Default
);

// not pretty. TODO: refactor
// this will be called via a delegate from UNiagaraScript's cache for cook function,
// because editor tickables aren't ticked during cooking
void FNiagaraShaderQueueTickable::ProcessQueue()
{
	static bool bReentrantProcessingGuard = false;

	// The current implementation of the compilation queue will drain the contents during one invocation of ProcessQueue.  If processing
	// the results of one of the shader script indirectly results in another invocation of ProcessQueue we'll run into trouble because
	// the contents of the queue will be double processed.  While we could move the queue to a local variable to process to prevent this
	// double processing we'll still run into trouble because the second scope could be expecting a result, which won't be available till
	// the earlier scope is complete.  Needs a better system...
	checkf(bReentrantProcessingGuard == false, TEXT("FNiagaraShaderQueueTickable::ProcessQueue() is not re-entrant!  Only a single scope can process results!"));

	TGuardValue<bool> GuardProcessQueue(bReentrantProcessingGuard, true);

	check(IsInGameThread());

	for (FNiagaraCompilationQueue::NiagaraCompilationQueueItem &Item : FNiagaraCompilationQueue::Get()->GetQueue())
	{
		FNiagaraShaderScript* ShaderScript = Item.Script;
		TRefCountPtr<FNiagaraShaderMap>NewShaderMap = Item.ShaderMap;

		if (ShaderScript == nullptr) // This script has been removed from the pending queue post submission... just skip it.
		{
			FNiagaraShaderMap::RemovePendingMap(NewShaderMap);
			NewShaderMap->SetCompiledSuccessfully(false);
			UE_LOGF(LogNiagaraEditor, Log, "GPU shader compile skipped. Id %d", NewShaderMap->GetCompilingId());
			continue;
		}
		UNiagaraScript* CompilableScript = CastChecked<UNiagaraScript>(ShaderScript->GetBaseVMScript());

		// For now System scripts don't generate HLSL and go through a special pass...
		// [OP] thinking they'll likely never run on GPU anyways
		if (CompilableScript->IsValidLowLevel() == false || CompilableScript->CanBeRunOnGpu() == false || !CompilableScript->GetVMExecutableData().IsValid() ||
			CompilableScript->GetVMExecutableData().LastHlslTranslationGPU.Len() == 0)
		{
			NewShaderMap->SetCompiledSuccessfully(false);
			FNiagaraShaderMap::RemovePendingMap(NewShaderMap);
			ShaderScript->RemoveOutstandingCompileId(NewShaderMap->GetCompilingId());
			UE_LOGF(LogNiagaraEditor, Log, "GPU shader compile skipped. Id %d", NewShaderMap->GetCompilingId());
			continue;
		}

		ShaderScript->BuildScriptParametersMetadata(CompilableScript->GetVMExecutableData().ShaderScriptParametersMetadata);
		ShaderScript->SetSourceName(TEXT("NiagaraComputeShader"));
		UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(CompilableScript->GetOuter());
		if (Emitter && Emitter->GetUniqueEmitterName().Len() > 0)
		{
			ShaderScript->SetSourceName(Emitter->GetUniqueEmitterName());
		}
		ShaderScript->SetHlslOutput(CompilableScript->GetVMExecutableData().LastHlslTranslationGPU);

		{
			// Create a shader compiler environment for the script that will be shared by all jobs from this script
			TRefCountPtr<FSharedShaderCompilerEnvironment> CompilerEnvironment = new FSharedShaderCompilerEnvironment();

			// Shaders are created in-sync in the postload when running the automated tests.
			const bool bSynchronousCompile = GIsAutomationTesting;

			// Compile the shaders for the script.
			NewShaderMap->Compile(ShaderScript, Item.ShaderMapId, CompilerEnvironment, Item.Platform, bSynchronousCompile, Item.bApply);
		}
	}

	FNiagaraCompilationQueue::Get()->GetQueue().Empty();
}

void FNiagaraShaderQueueTickable::Tick(float DeltaSeconds)
{
	ProcessQueue();
}

namespace NiagaraHlslTranslatorImpl
{

// helper function to evaluate whether the given character is an ascii alpha/number
bool IsAsciiAlnum(TCHAR ch)
{
	return ((ch >= TCHAR('0') && ch <= TCHAR('9')) ||
		(ch >= TCHAR('A') && ch <= TCHAR('Z')) ||
		(ch >= TCHAR('a') && ch <= TCHAR('z')));
}

// helper function to append an integer to a string builder (mimics FString::AppendInt)
void AppendInt(FStringBuilderBase& OutBuilder, TCHAR ch)
{
	const TCHAR* DigitToChar = TEXT("0123456789");
	constexpr int32 MaxDigitCount = 16;
	TCHAR TempDigits[MaxDigitCount];
	int32 CurDigitIndex = MaxDigitCount;

	uint32 chAsUint = static_cast<uint32>(ch);

	do
	{
		TempDigits[--CurDigitIndex] = DigitToChar[chAsUint % 10];
		chAsUint /= 10;
	} while (chAsUint);

	OutBuilder.Append(TempDigits + CurDigitIndex, MaxDigitCount - CurDigitIndex);
}

// helper function to mimic FString::ReplaceInline but uses StringBuilders/Views
void ReplaceInline(FStringBuilderBase& StringBuilder, int32 SearchStartIndex, FStringView SearchString, FStringView ReplaceString)
{
	const int32 SearchStringLen = SearchString.Len();
	const int32 ReplaceStringLen = ReplaceString.Len();

	while (SearchStartIndex < StringBuilder.Len())
	{
		SearchStartIndex = StringBuilder.ToView().Find(SearchString, SearchStartIndex);
		if (SearchStartIndex != INDEX_NONE)
		{
			StringBuilder.ReplaceAt(SearchStartIndex, SearchStringLen, ReplaceString);
			SearchStartIndex += ReplaceStringLen;
		}
		else
		{
			break;
		}
	}
}

void AppendConvertedToAscii(FStringView InNameView, FStringBuilderBase& OutBuilder)
{
	for (TCHAR ch : InNameView)
	{
		if (IsAsciiAlnum(ch) || FChar::IsUnderscore(ch))
		{
			// Do nothing.. these are valid chars..
			OutBuilder << ch;
		}
		// skip whitespace characters
		else if (!FChar::IsWhitespace(ch))
		{
			// Need to replace the bad characters..
			OutBuilder << TEXT("ASC");

			// replacement for FString::AppendInt
			AppendInt(OutBuilder, ch);
		}
	}
}

void AppendSanitizedSymbolString(FStringView SymbolName, bool bCollapseNamespaces, const TMap<FString, FString>& ReplacementMap, FStringBuilderBase& StringBuilder)
{
	if (SymbolName.Len() == 0)
	{
		return;
	}

	// Split up into individual namespaces...
	TArray<FStringView, TInlineAllocator<16>> NamePartStrings;
	UE::String::ParseTokens(SymbolName, TEXT("."), [&NamePartStrings](FStringView Token) { NamePartStrings.Add(Token); });

	bool bIsFirstSplit = true;
	const int32 OutputStartIndex = StringBuilder.Len();

	// Rules for variable namespaces..
	for (FStringView SplitName : NamePartStrings)
	{
		SplitName = SplitName.TrimStartAndEnd();

		if (!bIsFirstSplit)
		{
			StringBuilder << (bCollapseNamespaces ? TEXT("_") : TEXT("."));
		}

		if (SplitName[0] >= TCHAR('0') && SplitName[0] <= TCHAR('9')) // Cannot start with a numeric digit
		{
			StringBuilder << TEXT("INTEGER_");
		}

		if (const FString* FoundReplacementStr = ReplacementMap.FindByHash(GetTypeHash(SplitName), SplitName)) // Look for the string in the keyword protections array.
		{
			StringBuilder << *FoundReplacementStr;
		}
		else
		{
			AppendConvertedToAscii(SplitName, StringBuilder);
		}

		bIsFirstSplit = false;
	}

	ReplaceInline(StringBuilder, OutputStartIndex, TEXT("__"), TEXT("ASC95ASC95"));
}

void AppendSanitizedSymbolName(FName SymbolName, bool bCollapseNamespaces, const TMap<FString, FString>& ReplacementMap, FStringBuilderBase& StringBuilder)
{
	FNameBuilder NameBuilder(SymbolName);
	AppendSanitizedSymbolString(NameBuilder.ToView(), bCollapseNamespaces, ReplacementMap, StringBuilder);
}

void AppendFunctionSignatureSymbol(const FNiagaraFunctionSignature& Sig, FStringBuilderBase& SigStr)
{
	Sig.Name.AppendString(SigStr);
	if (!Sig.OwnerName.IsNone() && Sig.OwnerName.IsValid())
	{
		SigStr << TEXT("_");
		SigStr << Sig.OwnerName.ToString().Replace(TEXT("."), TEXT("_"));
	}
	else
	{
		SigStr << TEXT("_Func");
	}

	if (Sig.bRequiresExecPin)
	{
		SigStr << TEXT("_UEImpureCall"); // Let the cross compiler know that we intend to keep this.
	}

	for (const TTuple<FName, FName>& Specifier : Sig.FunctionSpecifiers)
	{
		SigStr << TEXT("_");
		Specifier.Key.AppendString(SigStr);

		const int32 SpecifierValueStartIndex = SigStr.Len();
		Specifier.Value.AppendString(SigStr);
		ReplaceInline(SigStr, SpecifierValueStartIndex, TEXT("."), TEXT("_"));
	}

	auto AddVarsToSig = [&SigStr](TConstArrayView<FNiagaraVariableBase> Vars)
	{
		for (const FNiagaraVariableBase& Var : Vars)
		{
			SigStr << TEXT("_");
			Var.GetName().AppendString(SigStr);
			Var.GetType().GetFName().AppendString(SigStr);
		}
	};

	TArray<FNiagaraVariableBase> VariadicParams;
	VariadicParams.Reserve(Sig.NumOptionalInputs() + Sig.NumOptionalOutputs());
	Sig.GetVariadicInputs(VariadicParams);
	Sig.GetVariadicOutputs(VariadicParams);
	AddVarsToSig(VariadicParams);
}

void AppendStructHlslTypeName(const FNiagaraTypeDefinition& Type, FStringBuilderBase& StringBuilder)
{
	check(FNiagaraTypeHelper::IsLWCType(Type) == false);

	if (Type.IsValid() == false)
	{
		StringBuilder.Append(TEXT("undefined"));
	}
	else if (Type == FNiagaraTypeDefinition::GetFloatDef())
	{
		StringBuilder.Append(TEXT("float"));
	}
	else if (Type == FNiagaraTypeDefinition::GetVec2Def())
	{
		StringBuilder.Append(TEXT("float2"));
	}
	else if (Type == FNiagaraTypeDefinition::GetVec3Def() || Type == FNiagaraTypeDefinition::GetPositionDef())
	{
		StringBuilder.Append(TEXT("float3"));
	}
	else if (Type == FNiagaraTypeDefinition::GetVec4Def() || Type == FNiagaraTypeDefinition::GetColorDef() || Type == FNiagaraTypeDefinition::GetQuatDef())
	{
		StringBuilder.Append(TEXT("float4"));
	}
	else if (Type == FNiagaraTypeDefinition::GetMatrix4Def())
	{
		StringBuilder.Append(TEXT("float4x4"));
	}
	else if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || Type.GetEnum() != nullptr)
	{
		StringBuilder.Append(TEXT("int"));
	}
	else if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		StringBuilder.Append(TEXT("bool"));
	}
	else if (Type == FNiagaraTypeDefinition::GetParameterMapDef())
	{
		StringBuilder.Append(TEXT("FParamMap0"));
	}
	else
	{
		Type.GetFName().AppendString(StringBuilder);
	}
}

FNiagaraVariable ConvertToSimulationVariable(const FNiagaraVariable& Param)
{
	if (FNiagaraTypeHelper::IsLWCType(Param.GetType()))
	{
		UScriptStruct* Struct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(Param.GetType().GetScriptStruct(), ENiagaraStructConversion::Simulation);
		FNiagaraVariable SimParam(FNiagaraTypeDefinition(Struct), Param.GetName());
		if (Param.IsDataAllocated())
		{
			SimParam.AllocateData();
			FNiagaraTypeRegistry::GetStructConverter(Param.GetType()).ConvertDataToSimulation(SimParam.GetData(), Param.GetData());
		}
		return SimParam;
	}
	return Param;
}

// Build the specifiers + inputs to the function, i.e. <Key=Value>(in int In_Value, out int Out_Value)
void AppendFunctionHlslParametersString(const FNiagaraFunctionSignature& FunctionSignature, const TMap<FString, FString>& ReplacementMap, FStringBuilderBase& StringBuilder)
{
	// Build specifiers
	if (FunctionSignature.FunctionSpecifiers.Num())
	{
		bool bNeedsComma = false;
		StringBuilder.AppendChar(TEXT('<'));
		for (auto SpecifierIt = FunctionSignature.FunctionSpecifiers.CreateConstIterator(); SpecifierIt; ++SpecifierIt)
		{
			if (bNeedsComma)
			{
				StringBuilder.Append(TEXT(", "));
			}
			bNeedsComma = true;

			SpecifierIt.Key().AppendString(StringBuilder);
			StringBuilder.AppendChar(TEXT('='));
			StringBuilder.AppendChar(TEXT('"'));
			if (SpecifierIt.Value().IsNone())
			{
				StringBuilder.Append(TEXT("Value"));
			}
			else
			{
				SpecifierIt.Value().AppendString(StringBuilder);
			}
			StringBuilder.AppendChar(TEXT('"'));
		}
		StringBuilder.AppendChar(TEXT('>'));
	}

	// Build function parameters
	{
		StringBuilder.Append(TEXT("("));
		bool bNeedsComma = false;

		// Inputs
		for (int i = 1; i < FunctionSignature.Inputs.Num(); ++i)
		{
			FNiagaraVariable InputVar = ConvertToSimulationVariable(FunctionSignature.Inputs[i]);
			if (bNeedsComma)
			{
				StringBuilder.Append(TEXT(", "));
			}
			bNeedsComma = true;

			StringBuilder.Append(TEXT("in "));
			AppendStructHlslTypeName(InputVar.GetType(), StringBuilder);
			StringBuilder.Append(TEXT(" In_"));
			AppendSanitizedSymbolName(InputVar.GetName(), false /*bCollapse*/, ReplacementMap, StringBuilder);
		}

		// Outputs
		for (int i = 0; i < FunctionSignature.Outputs.Num(); ++i)
		{
			FNiagaraVariable OutputVar = ConvertToSimulationVariable(FunctionSignature.Outputs[i]);
			if (bNeedsComma)
			{
				StringBuilder.Append(TEXT(", "));
			}
			bNeedsComma = true;

			StringBuilder.Append(TEXT("out "));
			AppendStructHlslTypeName(OutputVar.GetType(), StringBuilder);
			StringBuilder.Append(TEXT(" Out_"));
			AppendSanitizedSymbolName(OutputVar.GetName(), false /*bCollapse*/, ReplacementMap, StringBuilder);
		}
		StringBuilder.Append(TEXT(")"));
	}
}

void AppendFunctionHlslPrototypeString(FStringView InVariableName, const FNiagaraFunctionSignature& FunctionSignature, const TMap<FString, FString>& ReplacementMap, FStringBuilderBase& StringBuilder)
{
	if (FunctionSignature.bMemberFunction)
	{
		StringBuilder.Append(InVariableName);
		StringBuilder.Append(TEXT("."));
		AppendSanitizedSymbolName(FunctionSignature.Name, false /*bCollapse*/, ReplacementMap, StringBuilder);

		AppendFunctionHlslParametersString(FunctionSignature, ReplacementMap, StringBuilder);
		StringBuilder.Append(TEXT(";"));
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("None member functions not supported currently"));
	}
}

};

ENiagaraScriptCompileStatus FNiagaraTranslateResults::TranslateResultsToSummary(const FNiagaraTranslateResults* TranslateResults)
{
	ENiagaraScriptCompileStatus SummaryStatus = ENiagaraScriptCompileStatus::NCS_Unknown;
	if (TranslateResults != nullptr)
	{
		if (TranslateResults->NumErrors > 0)
		{
			SummaryStatus = ENiagaraScriptCompileStatus::NCS_Error;
		}
		else
		{
			if (TranslateResults->bHLSLGenSucceeded)
			{
				if (TranslateResults->NumWarnings)
				{
					SummaryStatus = ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings;
				}
				else
				{
					SummaryStatus = ENiagaraScriptCompileStatus::NCS_UpToDate;
				}
			}
		}
	}
	return SummaryStatus;
}

FNiagaraVariable ConvertToSimulationVariable(const FNiagaraVariable& Param)
{
	if (FNiagaraTypeHelper::IsLWCType(Param.GetType()))
	{
		UScriptStruct* Struct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(Param.GetType().GetScriptStruct(), ENiagaraStructConversion::Simulation);
		FNiagaraVariable SimParam(FNiagaraTypeDefinition(Struct), Param.GetName());
		if (Param.IsDataAllocated())
		{
			SimParam.AllocateData();
			FNiagaraTypeRegistry::GetStructConverter(Param.GetType()).ConvertDataToSimulation(SimParam.GetData(), Param.GetData());
		}
		return SimParam;
	}
	return Param;
}

// helper struct to provide an RAII interface for handling permutations scoping.  We either implement preprocessor directives
// for creating different permutations, or if the Translator doesn't support it, then we fall back to static branches where
// possible (this is not viable for declarations in the code)
enum class EPermutationScopeContext
{
	Declaration,
	Expression
};

template<EPermutationScopeContext Scope>
struct TSimStagePermutationContext
{
	TSimStagePermutationContext(FString& InHlslOutput)
		: HlslOutput(InHlslOutput)
		, Enabled(false)
		, HasBranch(false)
	{}

	TSimStagePermutationContext(const FNiagaraHlslTranslator& Translator, const FNiagaraHlslTranslationStage& TranslationStage, FString& InHlslOutput)
		: HlslOutput(InHlslOutput)
		, Enabled(false)
		, HasBranch(false)
	{
		AddBranch(Translator, TranslationStage);
	}

	TSimStagePermutationContext(const FNiagaraHlslTranslator& Translator, TConstArrayView<FNiagaraHlslTranslationStage> TranslationStages, TConstArrayView<int32> StageIndices, FString& InHlslOutput)
		: HlslOutput(InHlslOutput)
		, Enabled(false)
		, HasBranch(false)
	{
		AddBranchInternal(Translator, TranslationStages, StageIndices);
	}

	~TSimStagePermutationContext()
	{
		Release();
	}

	void AddBranch(const FNiagaraHlslTranslator& Translator, const FNiagaraHlslTranslationStage& TranslationStage)
	{
		// vs2017 generates a link error; this works around it...some how
		const FNiagaraHlslTranslationStage& LocalTranslation = TranslationStage;

		AddBranchInternal(Translator, MakeArrayView(&LocalTranslation, 1), MakeArrayView({ 0 }));
	}
	
	void Release()
	{
		if (Enabled)
		{
			HlslOutput.Appendf(TEXT("#endif // %s\n"), *TranslationStageName);
			Enabled = false;
		}
	}

	static bool SupportsBranching(const FNiagaraHlslTranslator& Translator)
	{
		return Translator.GetSimulationTarget() == ENiagaraSimTarget::GPUComputeSim;
	}


private:
	FString BuildConditionString(TConstArrayView<FNiagaraHlslTranslationStage> TranslationStages, TConstArrayView<int32> StageIndices)
	{
		FString ConditionString;

		const int32 StageIndexCount = StageIndices.Num();
		for (int32 i=0; i < StageIndexCount; ++i)
		{
			const int32 StageIndex = StageIndices[i];
			if (i)
			{
				ConditionString.Append(TEXT(" || "));
			}
			ConditionString.Appendf(TEXT("(SimulationStageIndex == %d)"), TranslationStages[StageIndex].SimulationStageIndex);
		}

		return ConditionString;
	}

	void AddBranchInternal(const FNiagaraHlslTranslator& Translator, TConstArrayView<FNiagaraHlslTranslationStage> TranslationStages, TConstArrayView<int32> StageIndices)
	{
		if (SupportsBranching(Translator) && StageIndices.Num())
		{
			Enabled = true;
				
			const FString PreviousTranslationStageName = TranslationStageName;
			TranslationStageName = StageIndices.Num() > 1 ? TEXT("Multiple stages") : TranslationStages[StageIndices[0]].PassNamespace;

			FString ConditionString = BuildConditionString(TranslationStages, StageIndices);

			if (HasBranch)
			{
				HlslOutput.Appendf(TEXT(
					"#elif (%s) // %s\n"),
					*ConditionString,
					*TranslationStageName);
			}
			else
			{
				HlslOutput.Appendf(TEXT("#if (%s) // %s\n"),
					*ConditionString,
					*TranslationStageName);
			}

			HasBranch = true;
		}
	}

	FString& HlslOutput;
	FString TranslationStageName;
	bool Enabled;
	bool HasBranch;
};

typedef TSimStagePermutationContext<EPermutationScopeContext::Declaration> FDeclarationPermutationContext;
typedef TSimStagePermutationContext<EPermutationScopeContext::Expression> FExpressionPermutationContext;

// utility function to replace a namespace (potentially internal within a fully qualified name) with another.
// As an example MyParticlesValue.Particles.CurrentValue with Source <Particles> and Replace <Array> will
// target the first 'Particles' namespace and ignore the leading namespace qualifier that included 'Particles'
static void ReplaceNamespaceInline(FString& FullName, FStringView Source, FStringView Replace)
{
	const int32 SourceLength = Source.Len();
	int32 SourceIdx = FullName.Find(Source.GetData());
	while (SourceIdx != INDEX_NONE)
	{
		if ((SourceIdx == 0 || (SourceIdx > 0 && FullName[SourceIdx - 1] == '.')))
		{
			FullName.RemoveAt(SourceIdx, SourceLength);
			FullName.InsertAt(SourceIdx, Replace.GetData());
			break;
		}
		SourceIdx = FullName.Find(Source.GetData(), ESearchCase::IgnoreCase, ESearchDir::FromStart, SourceIdx + SourceLength);
	}
}

FString FNiagaraHlslTranslator::GetCode(int32 ChunkIdx)
{
	FNiagaraCodeChunk& Chunk = CodeChunks[ChunkIdx];
	return GetCode(Chunk);
}

FString FNiagaraHlslTranslator::GetCode(FNiagaraCodeChunk& Chunk)
{
	TArray<FStringFormatArg> Args;
	for (int32 i = 0; i < Chunk.SourceChunks.Num(); ++i)
	{
		Args.Add(GetCodeAsSource(Chunk.SourceChunks[i]));
	}
	FString DefinitionString = FString::Format(*Chunk.Definition, Args);

	FString FinalString;

	if (Chunk.Mode == ENiagaraCodeChunkMode::Body)
	{
		FinalString += TEXT("\t");
	}

	if (Chunk.SymbolName.IsEmpty())
	{
		check(!DefinitionString.IsEmpty());
		FinalString += DefinitionString + (Chunk.bIsTerminated ? TEXT(";\n") : TEXT("\n"));
	}
	else
	{
		if (DefinitionString.IsEmpty())
		{
			if (!Chunk.bDecl)//Otherwise, we're doing nothing here.
			{
				Warning(LOCTEXT("MissingDeclForChunk", "Missing definition string."));
			}

			FinalString += GetStructHlslTypeName(Chunk.Type) + TEXT(" ") + Chunk.SymbolName + TEXT(";\n");
		}
		else
		{
			if (Chunk.bDecl)
			{
				FinalString += GetStructHlslTypeName(Chunk.Type) + TEXT(" ") + Chunk.SymbolName + TEXT(" = ") + DefinitionString + TEXT(";\n");
			}
			else
			{
				FinalString += Chunk.SymbolName + TEXT(" = ") + DefinitionString + TEXT(";\n");
			}
		}
	}
	return FinalString;
}

FString FNiagaraHlslTranslator::GetCodeAsSource(int32 ChunkIdx)
{
	if (ChunkIdx >= 0 && ChunkIdx < CodeChunks.Num())
	{
		FNiagaraCodeChunk& Chunk = CodeChunks[ChunkIdx];
		return Chunk.SymbolName + Chunk.ComponentMask;
	}
	return "Undefined";
}

// specialization because validation may be different between graph implementations
template<>
bool TNiagaraHlslTranslator<FNiagaraCompilationDigestBridge>::ValidateTypePins(const FNode* NodeToValidate)
{
	bool bPinsAreValid = true;
	for (const FPin* Pin : GetPins(NodeToValidate))
	{
		if (Pin->PinType.PinCategory.IsNone())
		{
			Error(LOCTEXT("InvalidPinTypeError", "Node pin has an undefined type."), NodeToValidate, Pin);
			bPinsAreValid = false;
		}
		else if (Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType || Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType)
		{
			FNiagaraTypeDefinition Type = Pin->Variable.GetType();
			if (Type.IsValid() == false)
			{
				Error(LOCTEXT("InvalidPinTypeError", "Node pin has an undefined type."), NodeToValidate, Pin);
				bPinsAreValid = false;
			}
			else if (Type == FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				Error(LOCTEXT("NumericPinError", "A numeric pin was not resolved to a known type.  Numeric pins must be connected or must be converted to an explicitly typed pin in order to compile."), NodeToValidate, Pin);
				bPinsAreValid = false;
			}
		}

		// disabling this check because orphaned pins won't make it this far for the digested graphs
		//if (Pin->bOrphanedPin)
		//{
		//	Warning(LOCTEXT("OrphanedPinError", "Node pin is no longer valid.  This pin must be disconnected or reset to default so it can be removed."), NodeToValidate, Pin);
		//}
	}
	return bPinsAreValid;
}


template<>
bool TNiagaraHlslTranslator<FNiagaraCompilationGraphBridge>::ValidateTypePins(const FNode* NodeToValidate)
{
	bool bPinsAreValid = true;
	for (const FPin* Pin : GetPins(NodeToValidate))
	{
		if (Pin->PinType.PinCategory.IsNone())
		{
			Error(LOCTEXT("InvalidPinTypeError", "Node pin has an undefined type."), NodeToValidate, Pin);
			bPinsAreValid = false;
		}
		else if (Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType || Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType)
		{
			FNiagaraTypeDefinition Type = UEdGraphSchema_Niagara::PinToTypeDefinition(Pin, ENiagaraStructConversion::Simulation);
			if (Type.IsValid() == false)
			{
				Error(LOCTEXT("InvalidPinTypeError", "Node pin has an undefined type."), NodeToValidate, Pin);
				bPinsAreValid = false;
			}
			else if (Type == FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				Error(LOCTEXT("NumericPinError", "A numeric pin was not resolved to a known type.  Numeric pins must be connected or must be converted to an explicitly typed pin in order to compile."), NodeToValidate, Pin);
				bPinsAreValid = false;
			}
		}

		if (Pin->bOrphanedPin)
		{
			Warning(LOCTEXT("OrphanedPinError", "Node pin is no longer valid.  This pin must be disconnected or reset to default so it can be removed."), NodeToValidate, Pin);
		}
	}
	return bPinsAreValid;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::GenerateFunctionSignature(ENiagaraScriptUsage ScriptUsage, FStringView InName, FName InFullName, FStringView InFunctionNameSuffix, const FGraph* FuncGraph, TConstArrayView<const FInputPin*> CallInputs, TArray<int32>& Inputs,
	FString& FunctionComment, FNiagaraFunctionSignature& OutSig) const
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_GenerateFunctionSignature);

	TArray<const FInputNode*, TInlineAllocator<16>> InputNodes;
	{
		NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_GenerateFunctionSignature_FindInputNodes);
		UNiagaraGraph::FFindInputNodeOptions Options;
		Options.bSort = true;
		Options.bFilterDuplicates = true;
		Options.bIncludeTranslatorConstants = false;
		// If we're compiling the emitter function we need to filter to the correct usage so that we only get inputs associated with the emitter call, but if we're compiling any other kind of function call we need all inputs
		// since the function call nodes themselves will have been generated with pins for all inputs and since we match the input nodes here to the inputs passed in by index, the two collections must match otherwise we fail
		// to compile a graph that would otherwise work correctly.
		Options.bFilterByScriptUsage = ScriptUsage == ENiagaraScriptUsage::EmitterSpawnScript || ScriptUsage == ENiagaraScriptUsage::EmitterUpdateScript;
		Options.TargetScriptUsage = ScriptUsage;
		InputNodes = GraphBridge::GetGraphInputNodes(FuncGraph, Options);

		if (Inputs.Num() != InputNodes.Num())
		{
			const_cast<TNiagaraHlslTranslator<GraphBridge>*>(this)->Error(FText::Format(LOCTEXT("GenerateFunctionSignatureFail", "Generating function signature for {0} failed.  The function call is providing a different number of inputs than the function graph supplies."),
				FText::FromString(InFullName.ToString())));
			return;
		}
	}

	TArray<FNiagaraVariable, TInlineAllocator<16>> ConstantLiteralInputVars;
	TArray<FNiagaraVariable, TInlineAllocator<16>> InputVars;
	TArray<FNiagaraVariableBase, TInlineAllocator<16>> OutputVars;

	bool bHasDIParameters = false;
	{
		const int32 InputCount = Inputs.Num();

		ConstantLiteralInputVars.Reserve(InputCount);
		InputVars.Reserve(InputCount);

		for (int32 InputIt = 0; InputIt < InputCount; ++InputIt)
		{
			if (Inputs[InputIt] == INDEX_NONE)
			{
				continue;
			}

			FNiagaraVariable InputVar = GraphBridge::GetInputVariable(InputNodes[InputIt]);
			if (GetLiteralConstantVariable(InputVar))
			{
				ConstantLiteralInputVars.Emplace(InputVar);
				Inputs[InputIt] = INDEX_NONE;
			}
			else
			{
				if (!bHasDIParameters && InputVar.GetType().IsDataInterface())
				{
					bHasDIParameters = true;
				}
				InputVars.Emplace(InputVar);
			}
		}
	}

	Inputs.Remove(INDEX_NONE);

	{
		const TArray<FNiagaraVariable> OutputNodeVariables = GraphBridge::GetGraphOutputNodeVariables(FuncGraph, ScriptUsage);
		OutputVars.Reserve(OutputNodeVariables.Num());
		Algo::Transform(OutputNodeVariables, OutputVars, [](const FNiagaraVariable& Var) -> FNiagaraVariableBase { return Var; });
	}

	const FString* ModuleAliasStr = ActiveHistoryForFunctionCalls.GetModuleAlias();
	const FString* EmitterAliasStr = ActiveHistoryForFunctionCalls.GetEmitterAlias();

	// For now, we want each module call to be unique due to parameter maps and aliasing causing different variables
	// to be written within each call.
	const bool bHasParameterMapParameters = GraphBridge::GraphHasParametersOfType(FuncGraph, FNiagaraTypeDefinition::GetParameterMapDef());
	const bool bScriptUsageRequiresUnique = ScriptUsage == ENiagaraScriptUsage::Module
		|| ScriptUsage == ENiagaraScriptUsage::DynamicInput
		|| ScriptUsage == ENiagaraScriptUsage::EmitterSpawnScript
		|| ScriptUsage == ENiagaraScriptUsage::EmitterUpdateScript;
	const bool bParametersRequiresUnique = bHasParameterMapParameters || bHasDIParameters;
	const bool bHasUniqueNaming = (ModuleAliasStr != nullptr || EmitterAliasStr != nullptr);

	if (bHasUniqueNaming && (bScriptUsageRequiresUnique || bParametersRequiresUnique))
	{
		FNameBuilder SignatureName;
		if (ModuleAliasStr != nullptr)
		{
			SignatureName << BuildSanitizedSymbolString(*ModuleAliasStr);
		}
		if (EmitterAliasStr != nullptr)
		{
			if (ModuleAliasStr)
			{
				SignatureName << TEXT('_');
			}
			SignatureName << BuildSanitizedSymbolString(*EmitterAliasStr);
		}
		if (InFunctionNameSuffix.IsEmpty() == false)
		{
			SignatureName << TEXT('_');
			SignatureName << InFunctionNameSuffix;
		}
		NiagaraHlslTranslatorImpl::ReplaceInline(SignatureName, 0, TEXT("."), TEXT("_"));
		OutSig = FNiagaraFunctionSignature(*SignatureName, InputVars, OutputVars, true, false);
	}
	else
	{
		// compose a string containing all of the decorations to the function and boil it down into a Guid that we can append to the function name
		FGuid PermutationGuid;
		{
			FNiagaraGraphFunctionAliasContext FunctionAliasContext;
			FunctionAliasContext.CompileUsage = GetCurrentUsage();
			FunctionAliasContext.ScriptUsage = TranslationStages[ActiveStageIdx].ScriptUsage;

			for (const FNiagaraVariableBase& StaticSwitchInput : GraphBridge::GraphGetStaticSwitchInputs(FuncGraph))
			{
				const FInputPin* const* FoundInputPinPtr = CallInputs.FindByPredicate([PinName = StaticSwitchInput.GetName()](const FInputPin* Pin) -> bool
				{
					return Pin && PinName.IsEqual(Pin->PinName);
				});

				if (FoundInputPinPtr)
				{
					const FInputPin& InputPin = **FoundInputPinPtr;
					FunctionAliasContext.AddPermutationIdentifier(InputPin.PinName, InputPin.DefaultValue);
				}
			}

			FuncGraph->UpdateFunctionAliasByContext(FunctionAliasContext);

			FNameBuilder FunctionAliasSuffix;
			ResolveGraphFunctionAlias(FunctionAliasContext, FunctionAliasSuffix);

			for (const FNiagaraVariable& LiteralConstant : ConstantLiteralInputVars)
			{
				checkf(LiteralConstant.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()), TEXT("Only boolean types are currently supported for literal constants."));
				FunctionAliasSuffix << TEXT(',');
				LiteralConstant.GetName().AppendString(FunctionAliasSuffix);
				FunctionAliasSuffix << TEXT('_');
				FunctionAliasSuffix << (LiteralConstant.GetValue<bool>() ? TEXT("true") : TEXT("false"));
			}

			if (!InputVars.IsEmpty())
			{
				FunctionAliasSuffix << TEXT(",Input - [");
				for (const FNiagaraVariableBase& InputVar : InputVars)
				{
					InputVar.GetType().GetFName().AppendString(FunctionAliasSuffix);
					FunctionAliasSuffix << TEXT(" ");
					InputVar.GetName().AppendString(FunctionAliasSuffix);
					FunctionAliasSuffix << TEXT(";");
				}
				FunctionAliasSuffix << TEXT("]");
			}

			if (!OutputVars.IsEmpty())
			{
				FunctionAliasSuffix << TEXT(",Output - [");
				for (const FNiagaraVariableBase& OutputVar : OutputVars)
				{
					OutputVar.GetType().GetFName().AppendString(FunctionAliasSuffix);
					FunctionAliasSuffix << TEXT(" ");
					OutputVar.GetName().AppendString(FunctionAliasSuffix);
					FunctionAliasSuffix << TEXT(";");
				}
				FunctionAliasSuffix << TEXT("]");
			}

			if (FunctionAliasSuffix.Len() > 0)
			{
				PermutationGuid = FGuid::NewDeterministicGuid(FunctionAliasSuffix.ToView());
				FunctionComment = FString(InName, FunctionAliasSuffix.Len());
				FunctionComment.Append(FunctionAliasSuffix.ToView());
			}
		}

		FNameBuilder NameBuilder;
		NameBuilder.Append(InName);

		if (PermutationGuid.IsValid())
		{
			NameBuilder << TEXT('_');
			PermutationGuid.AppendString(NameBuilder, EGuidFormats::Base36Encoded);
		}

		OutSig = FNiagaraFunctionSignature(*NameBuilder, InputVars, OutputVars, true, false);
	}

	// if we are splitting up our functions then we need to mark which stage this function signature is associated 
	// with so that if we encounter a function implementation for another stage that it will also be added
	if (OutSig.bRequiresContext && FDeclarationPermutationContext::SupportsBranching(*this))
	{
		OutSig.ContextStageIndex = TranslationStages[ActiveStageIdx].SimulationStageIndex;
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ResolveGraphFunctionAlias(const FNiagaraGraphFunctionAliasContext& FunctionAliasContext, FStringBuilderBase& FunctionAliasSuffix) const
{
	FunctionAliasContext.AppendPermutationString(FunctionAliasSuffix);

	TConstArrayView<FNiagaraGraphFunctionAliasContext::FStaticSwitchVariable> PendingStaticSwitchVariables = FunctionAliasContext.ReadPendingStaticSwitchVariables();
	if (!PendingStaticSwitchVariables.IsEmpty())
	{
		FNiagaraFixedConstantResolver ConstantResolver(this);

		for (FNiagaraGraphFunctionAliasContext::FStaticSwitchVariable CompilerVariable : PendingStaticSwitchVariables)
		{
			int32 SwitchValue = INDEX_NONE;

			if (ConstantResolver.ResolveConstant(CompilerVariable.Variable))
			{
				ENiagaraStaticSwitchType SwitchType = static_cast<ENiagaraStaticSwitchType>(CompilerVariable.SwitchType);
				if (SwitchType == ENiagaraStaticSwitchType::Bool)
				{
					SwitchValue = CompilerVariable.Variable.GetValue<bool>() ? 1 : 0;
				}
				else if (SwitchType == ENiagaraStaticSwitchType::Integer || SwitchType == ENiagaraStaticSwitchType::Enum)
				{
					SwitchValue = CompilerVariable.Variable.GetValue<int32>();
				}
			}

			if (SwitchValue != INDEX_NONE)
			{
				FunctionAliasSuffix.AppendChar(TEXT(','));
				CompilerVariable.Variable.GetName().AppendString(FunctionAliasSuffix);
				FunctionAliasSuffix.AppendChar(TEXT('='));
				FunctionAliasSuffix.Append(FString::FormatAsNumber(SwitchValue));
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FNiagaraHlslTranslator::FNiagaraHlslTranslator()
	: ReadIdx(0)
	, WriteIdx(0)
	, CompilationTarget(ENiagaraSimTarget::CPUSim)
	, CurrentBodyChunkMode(ENiagaraCodeChunkMode::Body)
	, ActiveStageIdx(-1)
	, bInitializedDefaults(false)
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	bEnforceStrictTypesValidations = Settings->bEnforceStrictStackTypes;

	const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();
	KeywordReplacementMap = EditorSettings->GetHLSLKeywordReplacementsMap();
}


FString FNiagaraHlslTranslator::GetFunctionDefinitions()
{
	FString FwdDeclString;
	FString DefinitionsString;

	// add includes from custom hlsl nodes
	for (const FNiagaraCustomHlslInclude& Include : FunctionIncludeFilePaths)
	{
		DefinitionsString += GetFunctionIncludeStatement(Include);
	}

	for (const auto& FuncPair : Functions)
	{
		FString Sig = GetFunctionSignature(FuncPair.Key);
		FwdDeclString += Sig + TEXT(";\n");
		if (!FuncPair.Value.Body.IsEmpty())
		{
			FDeclarationPermutationContext PermutationContext(*this, TranslationStages, FuncPair.Value.StageIndices, DefinitionsString);
			if (!FuncPair.Value.Comments.IsEmpty())
			{
				TArray<FStringView, TInlineAllocator<16>> CommentLines;
				UE::String::ParseTokens(FuncPair.Value.Comments, TEXT(","), [&CommentLines](FStringView Token) { CommentLines.Add(Token); });

				for (FStringView CommentLine : CommentLines)
				{
					DefinitionsString += TEXT("// ");
					DefinitionsString += CommentLine;
					DefinitionsString += TEXT("\n");
				}
			}
			DefinitionsString += Sig + TEXT("\n{\n") + FuncPair.Value.Body + TEXT("}\n\n");
		}
		// Don't do anything if the value is empty on the function pair, as this is indicative of 
		// data interface functions that should be defined differently.
	}

	// Check to see if we have interpolated spawn enabled, for the GPU we need to look for the additional defines
	bool bHasInterpolatedSpawn = CompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated;
	if ( CompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleGPUComputeScript )
	{
		bHasInterpolatedSpawn = CompileOptions.AdditionalDefines.Contains(TEXT("InterpolatedSpawn"));
	}

	//Add a few hard coded helper functions in.
	FwdDeclString += TEXT("float GetSpawnInterpolation();");
	//Add helper function to get the interpolation factor.
	if ( bHasInterpolatedSpawn )
	{
		DefinitionsString += TEXT("float GetSpawnInterpolation()\n{\n");
		DefinitionsString += TEXT("\treturn HackSpawnInterp;\n");
		DefinitionsString += TEXT("}\n\n");
	}
	else
	{
		DefinitionsString += TEXT("float GetSpawnInterpolation()\n{\n");
		DefinitionsString += TEXT("\treturn 1.0f;");
		DefinitionsString += TEXT("}\n\n");
	}

	return FwdDeclString + TEXT("\n") + DefinitionsString;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::BuildMissingDefaults()
{
	AddBodyComment(TEXT("// Begin HandleMissingDefaultValues"));
	if (TranslationStages[ActiveStageIdx].ShouldDoSpawnOnlyLogic() || TranslationStages[ActiveStageIdx].bShouldUpdateInitialAttributeValues)
	{
		// First go through all the variables that we did not write the defaults for yet. For spawn scripts, this usually
		// means variables that reference other variables but are not themselves used within spawn.
		for (FNiagaraVariable& Var : DeferredVariablesMissingDefault)
		{
			const FInputPin* DefaultPin = UniqueVarToDefaultPin.FindChecked(Var);
			bool bWriteToParamMapEntries = UniqueVarToWriteToParamMap.FindChecked(Var);
			int32 OutputChunkId = INDEX_NONE;

			TOptional<ENiagaraDefaultMode> DefaultMode;
			FNiagaraScriptVariableBinding DefaultBinding;

			const FNode* DefaultPinNode = nullptr;

			if (DefaultPin) 
			{
				DefaultPinNode = GraphBridge::GetOwningNode(DefaultPin);
				if (const FGraph* DefaultPinGraph = GraphBridge::GetOwningGraph(DefaultPinNode))
				{
					const FNiagaraVariableBase* AliasedVar = UniqueVarToAliasedVar.Find(Var);
					DefaultMode = GraphBridge::GetGraphDefaultMode(DefaultPinGraph, AliasedVar ? *AliasedVar : FNiagaraVariableBase(Var), DefaultBinding);
				}
			}

			bool bIsKnownMissingDefault = ParamMapHistories[ActiveStageIdx].Variables.Contains(Var);
			if (bIsKnownMissingDefault ||
				GNiagaraCompilationUseNewMissingDefaultTraversal == 0)
			{
				HandleParameterRead(ActiveStageIdx, Var, DefaultPin, DefaultPinNode, OutputChunkId, DefaultMode, DefaultBinding, !bWriteToParamMapEntries, true);
			}
			else
			{
				// Missing defaults that are unknown to the current parameter map need to be handled differently since their parameter history is unrelated to the
				// current stage.  Calling parameter map read in this case can result in calling compile on a pin with mismatched parameter map history and graph
				// pins which can result in compile errors or incorrect code generation.  If the unknown default can not be resolved we use the struct default.
				int32 MissingDefaultIdx = HandleReadUnknownMissingDefault(Var, DefaultPin);
				if (MissingDefaultIdx == INDEX_NONE)
				{
					FNiagaraVariable DefaultVar = Var;
					FNiagaraEditorUtilities::ResetVariableToDefaultValue(DefaultVar);
					MissingDefaultIdx = GetConstant(DefaultVar);
				}
				FString ParameterMapInstanceName = GetParameterMapInstanceName(ActiveStageIdx);
				AddBodyChunk(ParameterMapInstanceName + TEXT(".") + BuildSanitizedSymbolName(Var.GetName()), TEXT("{0}"), Var.GetType(), MissingDefaultIdx, false);
			}
		}

		DeferredVariablesMissingDefault.Empty();

		if (TranslationStages[ActiveStageIdx].bShouldUpdateInitialAttributeValues)
		{
			// Now go through and initialize any "Particles.Initial." variables
			for (FNiagaraVariable& Var : InitialNamespaceVariablesMissingDefault)
			{
				if (FNiagaraParameterUtilities::IsInitialValue(Var))
				{
					FNiagaraVariable SourceForInitialValue = FNiagaraParameterUtilities::GetSourceForInitialValue(Var);
					FString ParameterMapInstanceName = GetParameterMapInstanceName(0);
					FString Value = FString::Printf(TEXT("%s.%s = %s.%s;\n"), *ParameterMapInstanceName, *BuildSanitizedSymbolName(Var.GetName()),
						*ParameterMapInstanceName, *BuildSanitizedSymbolName(SourceForInitialValue.GetName()));
					AddBodyChunk(Value);
					continue;
				}
			}
			InitialNamespaceVariablesMissingDefault.Empty();
		}

	}
	AddBodyComment(TEXT("// End HandleMissingDefaultValues\n\n"));
}

template<typename GraphBridge>
int32 TNiagaraHlslTranslator<GraphBridge>::HandleReadUnknownMissingDefault(const FNiagaraVariable& DefaultVar, const FInputPin* DefaultPin)
{
	// Validate inputs
	if (DefaultPin == nullptr)
	{
		return INDEX_NONE;
	}
	const FNode* DefaultPinNode = GraphBridge::GetOwningNode(DefaultPin);
	if (DefaultPinNode == nullptr)
	{
		return INDEX_NONE;
	}
	const FGraph* DefaultPinGraph = GraphBridge::GetOwningGraph(DefaultPinNode);
	if (DefaultPinGraph == nullptr)
	{
		return INDEX_NONE;
	}

	// Get the default mode
	const FNiagaraVariableBase* AliasedVar = UniqueVarToAliasedVar.Find(DefaultVar);
	FNiagaraScriptVariableBinding DefaultBinding;
	TOptional<ENiagaraDefaultMode> DefaultMode = GraphBridge::GetGraphDefaultMode(DefaultPinGraph, AliasedVar ? *AliasedVar : FNiagaraVariableBase(DefaultVar), DefaultBinding);

	int32 DefaultChunkIndex = INDEX_NONE;
	if (DefaultMode == ENiagaraDefaultMode::Binding && DefaultBinding.IsValid())
	{
		// Bound defaults that can be handled are either local to the script being translated which will be in the UniqueVarToChunk map, or
		// are in an external constant namespace which can be handled safely with parameter map read.
		FNiagaraVariable BoundVar = FNiagaraVariable(DefaultVar.GetType(), DefaultBinding.GetName());
		int32* ExistingDefault = UniqueVarToChunk.Find(BoundVar);
		if (ExistingDefault != nullptr)
		{
			DefaultChunkIndex = *ExistingDefault;
		}
		else if (FNiagaraParameterUtilities::IsExternalConstantNamespace(BoundVar, CompileOptions.TargetUsage, CompileOptions.GetTargetUsageBitmask()))
		{
			HandleParameterRead(ActiveStageIdx, BoundVar, nullptr, nullptr, DefaultChunkIndex);
		}
	}
	else if (DefaultMode == ENiagaraDefaultMode::Custom)
	{
		// For custom defaults we only support linked parameter map gets.  First we handle the linked variable as if it was a
		// default binding, and if that doesn't work we recursively evaluate the linked default pin if it exists.
		const FOutputPin* LinkedOutputPin = GraphBridge::GetLinkedOutputPin(DefaultPin);
		if (LinkedOutputPin != nullptr)
		{
			const FNode* LinkedNode = GraphBridge::GetOwningNode(LinkedOutputPin);
			if (LinkedNode != nullptr)
			{
				const FParamMapGetNode* LinkedGetNode = GraphBridge::AsParamMapGetNode(LinkedNode);
				if (LinkedGetNode != nullptr)
				{
					FNiagaraVariable LinkedVar = GraphBridge::GetPinVariable(LinkedOutputPin, true, ENiagaraStructConversion::Simulation);
					int32* ExistingDefault = UniqueVarToChunk.Find(LinkedVar);
					if (ExistingDefault != nullptr)
					{
						DefaultChunkIndex = *ExistingDefault;
					}
					else if (FNiagaraParameterUtilities::IsExternalConstantNamespace(LinkedVar, CompileOptions.TargetUsage, CompileOptions.GetTargetUsageBitmask()))
					{
						HandleParameterRead(ActiveStageIdx, LinkedVar, nullptr, nullptr, DefaultChunkIndex);
					}
					else
					{
						const FInputPin* LinkedDefaultPin = GraphBridge::GetDefaultPin(LinkedGetNode, LinkedOutputPin);
						if (LinkedDefaultPin != nullptr)
						{
							DefaultChunkIndex = HandleReadUnknownMissingDefault(LinkedVar, LinkedDefaultPin);
						}
					}
				}
			}
		}
	}

	if (DefaultChunkIndex == INDEX_NONE)
	{
		// If a default was not found through a binding or through custom traversal, check if the pin itself has a default.
		FNiagaraVariable PinVariable = GraphBridge::GetPinVariable(DefaultPin, true, ENiagaraStructConversion::Simulation);
		DefaultChunkIndex = PinVariable.IsDataAllocated() ? GetConstant(PinVariable) : INDEX_NONE;
	}

	return DefaultChunkIndex;
}

template<typename GraphBridge>
FString TNiagaraHlslTranslator<GraphBridge>::BuildParameterMapHlslDefinitions(TArray<FNiagaraVariable>& PrimaryDataSetOutputEntries)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_BuildParameterMapHlslDefinitions);
	FString HlslOutputString;

	// Determine the unique parameter map structs...
	TArray<const FPin*> UniqueParamMapStartingPins;
	for (int32 ParamMapIdx = 0; ParamMapIdx < ParamMapHistories.Num(); ParamMapIdx++)
	{
		const FPin* OriginalPin = ParamMapHistories[ParamMapIdx].GetOriginalPin();
		UniqueParamMapStartingPins.AddUnique(OriginalPin);
	}


	TArray<FNiagaraVariable> UniqueVariables;

	// Add in currently defined system vars.
	for (const auto& SystemVarPair : ParamMapDefinedSystemVars)
	{
		const auto& Var = SystemVarPair.Value.Variable;
		if (Var.GetType().GetClass() != nullptr)
		{
			continue;
		}

		// ignore those that are rapid iteration parameters as those will be read in directly from the cbuffer
		if (FNiagaraParameterUtilities::IsRapidIterationParameter(Var))
		{
			continue;
		}

		UniqueVariables.AddUnique(Var);
	}

	// Add in currently defined emitter vars.
	TArray<FNiagaraVariable> ValueArray;
	ParamMapDefinedEmitterParameterToNamespaceVars.GenerateValueArray(ValueArray);
	for (FNiagaraVariable& Var : ValueArray)
	{
		if (Var.GetType().GetClass() != nullptr)
		{
			continue;
		}

		UniqueVariables.AddUnique(Var);
	}

	// Add in currently defined attribute vars.
	TArray<FVarAndDefaultSource> VarAndDefaultSourceArray;
	ParamMapDefinedAttributesToNamespaceVars.GenerateValueArray(VarAndDefaultSourceArray);
	for (FVarAndDefaultSource& VarAndDefaultSource : VarAndDefaultSourceArray)
	{
		if (VarAndDefaultSource.Variable.GetType().GetClass() != nullptr)
		{
			continue;
		}

		UniqueVariables.AddUnique(VarAndDefaultSource.Variable);
	}

	// Add in any bulk usage vars.
	for (FNiagaraVariable& Var : ExternalVariablesForBulkUsage)
	{
		if (Var.GetType().GetClass() != nullptr)
		{
			continue;
		}

		UniqueVariables.AddUnique(Var);
	}

	// Add in any interpolated spawn variables
	for (FNiagaraVariable& Var : InterpSpawnVariables)
	{
		if (Var.GetType().GetClass() != nullptr)
		{
			continue;
		}

		UniqueVariables.AddUnique(Var);
	}

	//Add any special case engine provided values in the param maps. These are not uniforms but engine provided things like Engine.ExecIndex and Engine.Emitter.ID.
 	for (FNiagaraVariable& Var : ParamMapDefinedEngineVars)
 	{
 		if (Var.GetType().GetClass() != nullptr)
 		{
 			continue;
 		}
 	
 		UniqueVariables.AddUnique(Var);
 	}

	bool bIsSpawnScript = IsSpawnScript();

	// For now we only care about attributes from the other output parameter map histories.
	for (int32 ParamMapIdx = 0; ParamMapIdx < OtherOutputParamMapHistories.Num(); ParamMapIdx++)
	{
		TArray<FNiagaraVariable> Vars = OtherOutputParamMapHistories[ParamMapIdx].Variables;
		for (const FNiagaraVariableBase& Var : CompileOptions.AdditionalVariables)
		{
			bool bFoundSource = false;
			if (FNiagaraParameterUtilities::IsPreviousValue(Var))
			{
				FNiagaraVariable Source = FNiagaraParameterUtilities::GetSourceForPreviousValue(FNiagaraVariable(Var));
				const FNiagaraTypeDefinition& SourceType = Source.GetType();

				for (int32 ParamMapIdxTest = 0; !bFoundSource && ParamMapIdxTest < OtherOutputParamMapHistories.Num(); ParamMapIdxTest++)
				{				
					const TArray<FNiagaraVariable>& OtherVariables = OtherOutputParamMapHistories[ParamMapIdxTest].Variables;
					const int32 OtherVarIndex = OtherVariables.IndexOfByPredicate([&](const FNiagaraVariable& OtherVar)
					{
						return OtherVar.GetName() == Source.GetName();
					});

					if (OtherVarIndex != INDEX_NONE)
					{
						const FNiagaraTypeDefinition& OtherType = OtherVariables[OtherVarIndex].GetType();

						if (OtherType == SourceType)
						{
							bFoundSource = true;
						}
						// special case handling for when we have an implicit Position type for the source
						// and we've found a Vector
						else if (SourceType == FNiagaraTypeDefinition::GetPositionDef()
							&& OtherType == FNiagaraTypeDefinition::GetVec3Def())
						{
							bFoundSource = true;
						}
					}
				}
			}

			if (bFoundSource)
			{
				Vars.AddUnique(FNiagaraVariable(Var.GetType(), Var.GetName()));
			}
		}

		for (int32 VarIdx = 0; VarIdx < Vars.Num(); VarIdx++)
		{
			FNiagaraVariable& Var = Vars[VarIdx];
			if (OtherOutputParamMapHistories[ParamMapIdx].IsPrimaryDataSetOutput(Var, CompileOptions.TargetUsage))
			{
				int32 PreviousMax = UniqueVariables.Num();
				if (UniqueVariables.AddUnique(Var) == PreviousMax) // i.e. we didn't find it previously, so we added to the end.
				{
					if (bIsSpawnScript)
					{
						if (!AddStructToDefinitionSet(Var.GetType()))
						{
							Error(FText::Format(LOCTEXT("ParameterMapTypeError", "Cannot handle type {0}! Variable: {1}"), Var.GetType().GetNameText(), FText::FromName(Var.GetName())), nullptr, nullptr);
						}
					}
				}
			}
		}
	}

	// Add the attribute indices to the list of unique variables
	TArray<FString> RegisterNames;
	for (int32 UniqueVarIdx = 0; UniqueVarIdx < UniqueVariables.Num(); UniqueVarIdx++)
	{
		const FNiagaraVariable& NiagaraVariable = UniqueVariables[UniqueVarIdx];
		if (FNiagaraParameterUtilities::IsAttribute(NiagaraVariable))
		{
			FString VariableName = BuildSanitizedSymbolName(NiagaraVariable.GetName());
			ReplaceNamespaceInline(VariableName, PARAM_MAP_ATTRIBUTE_STR, PARAM_MAP_INDICES_STR);
			RegisterNames.Add(VariableName);
		}
	}
	for (const FString& RegisterName : RegisterNames)
	{
		FNiagaraVariable NiagaraVariable = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), *RegisterName);
		UniqueVariables.AddUnique(NiagaraVariable);
	}


	TMap<FString, TArray<TPair<FString, FString>> > ParamStructNameToMembers;
	TArray<FString> ParamStructNames;

	for (int32 UniqueVarIdx = 0; UniqueVarIdx < UniqueVariables.Num(); UniqueVarIdx++)
	{
		int UniqueParamMapIdx = 0;
		FNiagaraVariable Variable = ConvertToSimulationVariable(UniqueVariables[UniqueVarIdx]);

		if (!AddStructToDefinitionSet(Variable.GetType()))
		{
			Error(FText::Format(LOCTEXT("ParameterMapTypeError", "Cannot handle type {0}! Variable: {1}"), Variable.GetType().GetNameText(), FText::FromName(Variable.GetName())), nullptr, nullptr);
		}

		// In order 
		for (int32 ParamMapIdx = 0; ParamMapIdx < OtherOutputParamMapHistories.Num(); ParamMapIdx++)
		{
			if (OtherOutputParamMapHistories[ParamMapIdx].IsPrimaryDataSetOutput(Variable, CompileOptions.TargetUsage))
			{
				PrimaryDataSetOutputEntries.AddUnique(Variable);
				break;
			}
		}

		TArray<FString> StructNameArray;
		FString SanitizedVarName = BuildSanitizedSymbolName(Variable.GetName());
		int32 NumFound = SanitizedVarName.ParseIntoArray(StructNameArray, TEXT("."));
		if (NumFound == 1) // Meaning no split above
		{
			Error(FText::Format(LOCTEXT("OnlyOneNamespaceEntry", "Only one namespace entry found for: {0}"), FText::FromString(SanitizedVarName)), nullptr, nullptr);
		}
		else if (NumFound > 1)
		{
			while (StructNameArray.Num())
			{
				FString FinalName = StructNameArray[StructNameArray.Num() - 1];
				StructNameArray.RemoveAt(StructNameArray.Num() - 1);
				FString StructType = FString::Printf(TEXT("FParamMap%d_%s"), UniqueParamMapIdx, *FString::Join(StructNameArray, TEXT("_")));
				if (StructNameArray.Num() == 0)
				{
					StructType = FString::Printf(TEXT("FParamMap%d"), UniqueParamMapIdx);
				}

				FString TypeName = GetStructHlslTypeName(Variable.GetType());
				FString VarName = BuildSanitizedSymbolString(FinalName);
				if (NumFound > StructNameArray.Num() + 1 && StructNameArray.Num() != 0)
				{
					TypeName = FString::Printf(TEXT("FParamMap%d_%s_%s"), UniqueParamMapIdx, *FString::Join(StructNameArray, TEXT("_")), *VarName);
				}
				else if (StructNameArray.Num() == 0)
				{
					TypeName = FString::Printf(TEXT("FParamMap%d_%s"), UniqueParamMapIdx, *VarName);
				}
				TPair<FString, FString> Pair(TypeName, VarName);
				ParamStructNameToMembers.FindOrAdd(StructType).AddUnique(Pair);
				ParamStructNames.AddUnique(StructType);
			}
		}
	}

	// Build up the sub-structs..
	ParamStructNames.Sort();
	FString StructDefString = "";
	for (int32 i = ParamStructNames.Num() - 1; i >= 0; i--)
	{
		const FString& StructName = ParamStructNames[i];
		StructDefString += FString::Printf(TEXT("struct %s\n{\n"), *StructName);
		TArray<TPair<FString, FString>> StructMembers = ParamStructNameToMembers[StructName];
		auto SortMembers = [](const TPair<FString, FString>& A, const TPair<FString, FString>& B)
		{
			return A.Value < B.Value;
		};
		StructMembers.Sort(SortMembers);
		for (const TPair<FString, FString>& Line : StructMembers)
		{
			StructDefString += TEXT("\t") + Line.Key + TEXT(" ") + Line.Value + TEXT(";\n");;
		}
		StructDefString += TEXT("};\n\n");
	}

	HlslOutputString += StructDefString;

	return HlslOutputString;
}


bool FNiagaraHlslTranslator::ShouldConsiderTargetParameterMap(ENiagaraScriptUsage InUsage) const
{
	ENiagaraScriptUsage TargetUsage = GetTargetUsage();
	if (TargetUsage >= ENiagaraScriptUsage::ParticleSpawnScript && TargetUsage <= ENiagaraScriptUsage::ParticleEventScript)
	{
		return InUsage >= ENiagaraScriptUsage::ParticleSpawnScript && InUsage <= ENiagaraScriptUsage::ParticleSimulationStageScript;
	}
	else if (TargetUsage == ENiagaraScriptUsage::SystemSpawnScript)
	{
		if (InUsage == ENiagaraScriptUsage::SystemUpdateScript)
		{
			return true;
		}
		else if (TargetUsage == InUsage)
		{
			return true;
		}
	}
	else if (TargetUsage == InUsage)
	{
		return true;
	}

	return false;
}

void FNiagaraHlslTranslator::HandleNamespacedExternalVariablesToDataSetRead(TArray<FNiagaraVariable>& InDataSetVars, FString InNamespaceStr)
{
	for (const FNiagaraVariable& Var : ExternalVariablesForBulkUsage)
	{
		if (FNiagaraParameterUtilities::IsInNamespace(Var, InNamespaceStr))
		{
			InDataSetVars.Add(Var);
		}
	}
}

bool FNiagaraHlslTranslator::IsVariableInUniformBuffer(const FNiagaraVariable& Variable) const
{
	static FNiagaraVariable GpuExcludeVariables[] =
	{
		// Variables that must be calcualted on the GPU
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(),   TEXT("Engine.ExecutionCount")),
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(),   TEXT("Engine_ExecutionCount")),

		// Spawn variables
		FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Emitter_SpawnInterval")),
		FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Emitter.SpawnInterval")),
		FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Emitter_InterpSpawnStartDt")),
		FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Emitter.InterpSpawnStartDt")),
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(),   TEXT("Emitter_SpawnGroup")),
		FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(),   TEXT("Emitter.SpawnGroup")),
	};

	if (CompilationTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		for ( const FNiagaraVariable& ExcludeVar : GpuExcludeVariables)
		{
			if ( Variable == ExcludeVar )
			{
				return false;
			}
		}
	}
	return true;
}

template<typename T>
void FNiagaraHlslTranslator::BuildConstantBuffer(ENiagaraCodeChunkMode ChunkMode)
{
	for (const FNiagaraVariable& Variable : T::GetVariables())
	{
		const FString SymbolName = BuildSanitizedSymbolName(Variable.GetName(), true);
		AddChunkToConstantBuffer(SymbolName, Variable, ChunkMode);
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::RecordParamMapDefinedAttributeToNamespaceVar(const FNiagaraVariable& VarToRecord, const FPin* VarAssociatedDefaultPin)
{
	bool bDefaultPinExplicit = true;
	if (VarAssociatedDefaultPin == nullptr || VarAssociatedDefaultPin->bHidden)
	{
		bDefaultPinExplicit = false;
	}

	if (FVarAndDefaultSource* VarAndDefaultSourcePtr = ParamMapDefinedAttributesToNamespaceVars.Find(VarToRecord.GetName()))
	{
		VarAndDefaultSourcePtr->bDefaultExplicit |= bDefaultPinExplicit;
	}
	else
	{
		FVarAndDefaultSource VarAndDefaultSource;
		VarAndDefaultSource.Variable = VarToRecord;
		VarAndDefaultSource.bDefaultExplicit = bDefaultPinExplicit;
		ParamMapDefinedAttributesToNamespaceVars.Add(VarToRecord.GetName(), VarAndDefaultSource);
	}
}

static void ConvertFloatToHalf(const FNiagaraCompileOptions& InCompileOptions, TArray<FNiagaraVariable>& Attributes)
{
	// for now we're going to only process particle scripts as we don't currently support attributes
	// being read transparently from the parameter store (which would be done for system/emitter scripts)
	if (!UNiagaraScript::IsParticleScript(InCompileOptions.TargetUsage))
	{
		return;
	}

	if (InCompileOptions.AdditionalDefines.Contains(TEXT("CompressAttributes")))
	{
		static FNiagaraTypeDefinition ConvertMapping[][2] =
		{
			{ FNiagaraTypeDefinition::GetFloatDef(), FNiagaraTypeDefinition::GetHalfDef() },
			{ FNiagaraTypeDefinition::GetVec2Def(), FNiagaraTypeDefinition::GetHalfVec2Def() },
			{ FNiagaraTypeDefinition::GetVec3Def(), FNiagaraTypeDefinition::GetHalfVec3Def() },
			{ FNiagaraTypeDefinition::GetVec4Def(), FNiagaraTypeDefinition::GetHalfVec4Def() },
			{ FNiagaraTypeDefinition::GetColorDef(), FNiagaraTypeDefinition::GetHalfVec4Def() },
			{ FNiagaraTypeDefinition::GetQuatDef(), FNiagaraTypeDefinition::GetHalfVec4Def() },
		};

		TArray<FNiagaraVariable> ConvertExceptions =
		{
			SYS_PARAM_ENGINE_INV_DELTA_TIME,
			SYS_PARAM_ENGINE_TIME,
			SYS_PARAM_ENGINE_REAL_TIME,
			SYS_PARAM_ENGINE_SYSTEM_AGE,
			SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS_ALIVE,
			SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS,
			SYS_PARAM_ENGINE_SYSTEM_RANDOM_SEED,
			SYS_PARAM_ENGINE_SYSTEM_NUM_TIME_STEPS,
			SYS_PARAM_ENGINE_SYSTEM_TIME_STEP_FRACTION,
			SYS_PARAM_ENGINE_SYSTEM_NUM_PARTICLES,
			SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS,
			SYS_PARAM_ENGINE_NUM_SYSTEM_INSTANCES,
			SYS_PARAM_ENGINE_EMITTER_NUM_PARTICLES,
			SYS_PARAM_ENGINE_EMITTER_TOTAL_SPAWNED_PARTICLES,
			SYS_PARAM_PARTICLES_UNIQUE_ID,
			SYS_PARAM_PARTICLES_ID,
			SYS_PARAM_EMITTER_AGE,
			SYS_PARAM_EMITTER_RANDOM_SEED,
			SYS_PARAM_ENGINE_EMITTER_INSTANCE_SEED,
			SYS_PARAM_PARTICLES_LIFETIME,
		};

		for (FNiagaraVariable& Attribute : Attributes)
		{
			// check if the variable matches an exception that we don't want to convert
			if (FNiagaraVariable::SearchArrayForPartialNameMatch(ConvertExceptions, Attribute.GetName()) != INDEX_NONE)
			{
				continue;
			}

			// also we'll check if the current attribute is a previous version of an exception because we wouldn't want
			// those to be mismatched
			if (FNiagaraParameterUtilities::IsPreviousValue(Attribute))
			{
				FNiagaraVariable SrcAttribute = FNiagaraParameterUtilities::GetSourceForPreviousValue(Attribute);

				if (FNiagaraVariable::SearchArrayForPartialNameMatch(ConvertExceptions, SrcAttribute.GetName()) != INDEX_NONE)
				{
					continue;
				}
			}

			for (int32 ConvertIt = 0; ConvertIt < UE_ARRAY_COUNT(ConvertMapping); ++ConvertIt)
			{
				if (Attribute.GetType() == ConvertMapping[ConvertIt][0])
				{
					Attribute.SetType(ConvertMapping[ConvertIt][1]);
					break;
				}
			}
		}
	}
}

// Collapses the set of attributes based on if they would overlap (i.e. attributes share the same name and either have
// the same type or an equivalent type, like position and vector, (for now)).
static void CollapseAttributes(const FNiagaraCompileOptions& InCompileOptions, TArray<FNiagaraVariable>& SortedAttributes)
{
	const FNiagaraTypeDefinition& Vec3Type = FNiagaraTypeDefinition::GetVec3Def();
	const FNiagaraTypeDefinition& PosType = FNiagaraTypeDefinition::GetPositionDef();

	auto CanCollapseAttributeTypes = [&](const FNiagaraTypeDefinition& Lhs, const FNiagaraTypeDefinition& Rhs) -> bool
	{
		if (Lhs == Rhs)
		{
			return true;
		}

		// for now we're only going to consider the implications of Vec3 vs Position types
		if ((Lhs == Vec3Type && Rhs == PosType)
			|| (Lhs == PosType && Rhs == Vec3Type))
		{
			return true;
		}

		return false;
	};

	for (int32 AttrIt = 0; AttrIt < SortedAttributes.Num(); ++AttrIt)
	{
		const FNiagaraVariable& CurrentVariable = SortedAttributes[AttrIt];
		const FNiagaraTypeDefinition& CurrentVariableType = CurrentVariable.GetType();

		// look at all attributes sharing the name and remove all that have the same or equivalent type
		int32 NextAttrIt = AttrIt + 1;
		while (SortedAttributes.IsValidIndex(NextAttrIt) && SortedAttributes[NextAttrIt].GetName() == CurrentVariable.GetName())
		{
			const FNiagaraVariable& NextVariable = SortedAttributes[NextAttrIt];

			if (CanCollapseAttributeTypes(CurrentVariableType, NextVariable.GetType()))
			{
				SortedAttributes.RemoveAt(NextAttrIt, EAllowShrinking::No);
			}
			else
			{
				++NextAttrIt;
			}
		}
	}
}

template<typename GraphBridge>
FNiagaraTranslateResults TNiagaraHlslTranslator<GraphBridge>::Translate(const FNiagaraCompileOptions& InCompileOptions, const FHlslNiagaraTranslatorOptions& InTranslateOptions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NiagaraHlslTranslate);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*InCompileOptions.GetPathName(), NiagaraChannel);

	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_Translate);
	check(CompileData);
	check(CompileDuplicateData);

	FNiagaraTypeRegistryTLSProxy TypeRegistryProxy;

	CompileOptions = InCompileOptions;
	TranslationOptions = InTranslateOptions;
	CompilationTarget = TranslationOptions.SimTarget;
	TranslateResults.bHLSLGenSucceeded = false;
	TranslateResults.OutputHLSL = "";
	HlslOutput.Reserve(1024 * 1024); // reserve some space to prevent reallocations during translation

	const FGraph* SourceGraph = GraphBridge::GetGraph(CompileDuplicateData);

	if (!SourceGraph)
	{
		Error(LOCTEXT("GetGraphFail", "Cannot find graph node!"), nullptr, nullptr);
		return TranslateResults;
	}

	if (GraphBridge::IsGraphEmpty(SourceGraph))
	{
		if (UNiagaraScript::IsSystemScript(CompileOptions.TargetUsage))
		{
			Error(LOCTEXT("GetNoNodeSystemFail", "Graph contains no nodes! Please add an emitter."), nullptr, nullptr);
		}
		else
		{
			Error(LOCTEXT("GetNoNodeFail", "Graph contains no nodes! Please add an output node."), nullptr, nullptr);
		}
		return TranslateResults;
	}

	const bool bRequiresPersistentIDs = CompileOptions.AdditionalDefines.Contains(TEXT("RequiresPersistentIDs"));

	TranslationStages.Empty();
	ActiveStageIdx = 0;

	const bool bHasInterpolatedSpawn = CompileOptions.AdditionalDefines.Contains(TEXT("InterpolatedSpawn"));
	ParamMapHistories.Empty();
	ParamMapSetVariablesToChunks.Empty();

	OtherOutputParamMapHistories = CompileDuplicateData->GetPrecomputedHistories();

	// Make the sanitized variable version of this list.
	OtherOutputParamMapHistoriesSanitizedVariables.AddDefaulted(OtherOutputParamMapHistories.Num());
	for (int32 i = 0; i < OtherOutputParamMapHistories.Num(); i++)
	{
		OtherOutputParamMapHistoriesSanitizedVariables[i].Reserve(OtherOutputParamMapHistories[i].Variables.Num());
		for (const FNiagaraVariable& Var : OtherOutputParamMapHistories[i].Variables)
		{
			OtherOutputParamMapHistoriesSanitizedVariables[i].Emplace(Var.GetType(), *BuildSanitizedSymbolName(Var.GetName()));
		}
	}

	const bool bCPUSim = CompileOptions.IsCpuScript();
	const bool bGPUSim = CompileOptions.IsGpuScript();

	if (CompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleEventScript && bGPUSim)
	{
		Error(LOCTEXT("CannotUseEventsWithGPU", "GPU Events scripts are currently unsupported. Consider using the Particle Attribute Reader instead!"), nullptr, nullptr);
		return TranslateResults;
	}

	switch (CompileOptions.TargetUsage)
	{
	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
	{
		FNiagaraHlslTranslationStage& SpawnStage = TranslationStages.Emplace_GetRef(CompileOptions.TargetUsage, CompileOptions.TargetUsageId);
		SpawnStage.PassNamespace = TEXT("MapSpawn");
		SpawnStage.ChunkModeIndex = ENiagaraCodeChunkMode::SpawnBody;
		SpawnStage.SimulationStageIndex = 0;
		SpawnStage.NumIterations = 1;
		SpawnStage.bWritesParticles = true;
		SpawnStage.bShouldUpdateInitialAttributeValues = true;
		OutputNodes.Add(SourceGraph->FindEquivalentOutputNode(ENiagaraScriptUsage::ParticleSpawnScript, SpawnStage.UsageId));

		FNiagaraHlslTranslationStage& UpdateStage = TranslationStages.Emplace_GetRef(ENiagaraScriptUsage::ParticleUpdateScript, FGuid());
		UpdateStage.PassNamespace = TEXT("MapUpdate");
		UpdateStage.ChunkModeIndex = ENiagaraCodeChunkMode::UpdateBody;
		UpdateStage.InterpolatedSpawnMode = CompileOptions.AdditionalDefines.Contains(TEXT("RunUpdateWithoutInterpolation")) ? ENiagaraInterpolatedSpawnMode::RunUpdateScript : ENiagaraInterpolatedSpawnMode::Interpolation;
		UpdateStage.SimulationStageIndex = 0;
		UpdateStage.NumIterations = 1;
		UpdateStage.bWritesParticles = true;
		OutputNodes.Add(SourceGraph->FindEquivalentOutputNode(UpdateStage.ScriptUsage, UpdateStage.UsageId));

		ParamMapHistories.AddDefaulted(2);
		ParamMapSetVariablesToChunks.AddDefaulted(2);
		ParamMapHistoriesSourceInOtherHistories.AddDefaulted(2);
	}	break;
	case ENiagaraScriptUsage::ParticleGPUComputeScript:
	{
		FNiagaraHlslTranslationStage& SpawnStage = TranslationStages.Emplace_GetRef((bHasInterpolatedSpawn ? ENiagaraScriptUsage::ParticleSpawnScriptInterpolated : ENiagaraScriptUsage::ParticleSpawnScript), FGuid());
		SpawnStage.PassNamespace = TEXT("MapSpawn");
		SpawnStage.ChunkModeIndex = ENiagaraCodeChunkMode::SpawnBody;
		SpawnStage.SimulationStageIndex = 0;
		SpawnStage.NumIterations = 1;
		SpawnStage.bWritesParticles = true;
		SpawnStage.bShouldUpdateInitialAttributeValues = true;
		OutputNodes.Add(SourceGraph->FindEquivalentOutputNode(ENiagaraScriptUsage::ParticleSpawnScript, TranslationStages[0].UsageId));

		FNiagaraHlslTranslationStage& UpdateStage = TranslationStages.Emplace_GetRef(ENiagaraScriptUsage::ParticleUpdateScript, FGuid());
		UpdateStage.PassNamespace = TEXT("MapUpdate");
		UpdateStage.ChunkModeIndex = ENiagaraCodeChunkMode::UpdateBody;
		UpdateStage.InterpolatedSpawnMode = ENiagaraInterpolatedSpawnMode::NoInterpolation;
		if (CompileOptions.AdditionalDefines.Contains(TEXT("InterpolatedSpawn")))
		{
			UpdateStage.InterpolatedSpawnMode = CompileOptions.AdditionalDefines.Contains(TEXT("RunUpdateWithoutInterpolation")) ? ENiagaraInterpolatedSpawnMode::RunUpdateScript : ENiagaraInterpolatedSpawnMode::Interpolation;
		}
		UpdateStage.SimulationStageIndex = 0;
		UpdateStage.NumIterations = 1;
		UpdateStage.bWritesParticles = true;
		OutputNodes.Add(SourceGraph->FindEquivalentOutputNode(TranslationStages[1].ScriptUsage, TranslationStages[1].UsageId));

		ParamMapHistories.AddDefaulted(2);
		ParamMapHistoriesSourceInOtherHistories.AddDefaulted(2);
		ParamMapSetVariablesToChunks.AddDefaulted(2);

		// Add the spawn / update stage
		{
			FSimulationStageMetaData& SimulationStageMetaData = CompilationOutput.ScriptData.SimulationStageMetaData.AddDefaulted_GetRef();
			SimulationStageMetaData.SimulationStageName = UNiagaraSimulationStageBase::ParticleSpawnUpdateName;
			SimulationStageMetaData.NumIterations = 1;
			SimulationStageMetaData.bWritesParticles = true;
			SimulationStageMetaData.bPartialParticleUpdate = false;
			SimulationStageMetaData.GpuDispatchType = ENiagaraGpuDispatchType::OneD;
			SimulationStageMetaData.GpuDirectDispatchElementType = ENiagaraDirectDispatchElementType::NumThreads;
			SimulationStageMetaData.GpuDispatchNumThreads = FNiagaraShader::GetDefaultThreadGroupSize(ENiagaraGpuDispatchType::OneD);
		}

		{
			int32 SourceSimStageIndex = 0;
			
			// OutputNode order in traversal doesn't necessarily match the stack ordering. Use the GUID order to define the actual stages.
			TArray<const FOutputNode*> FoundOutputNodes;
			TArray<int32> FoundStageHistories;

			for (const auto& CompileSimStageData : CompileData->CompileSimStageData )
			{
				const FGuid& StageGuid = CompileSimStageData.StageGuid;
				for (int32 FoundHistoryIdx = 0; FoundHistoryIdx < OtherOutputParamMapHistories.Num(); FoundHistoryIdx++)
				{
					FParamMapHistory& FoundHistory = OtherOutputParamMapHistories[FoundHistoryIdx];
					const FOutputNode* HistoryOutputNode = FoundHistory.GetFinalOutputNode();
					if (HistoryOutputNode && GraphBridge::GetOutputNodeUsageId(HistoryOutputNode) == StageGuid)
					{
						FoundOutputNodes.Add(HistoryOutputNode);
						FoundStageHistories.Add(FoundHistoryIdx);
						break;
					}
				}
			}

			// Now iterate the nodes in the order we found them.
			for (int32 FoundIdx = 0; FoundIdx < FoundOutputNodes.Num(); FoundIdx++)
			{
				const FOutputNode* HistoryOutputNode = FoundOutputNodes[FoundIdx];
				const FParamMapHistory& FoundHistory = OtherOutputParamMapHistories[FoundStageHistories[FoundIdx]];

				if (HistoryOutputNode && GraphBridge::GetOutputNodeScriptType(HistoryOutputNode) == ENiagaraScriptUsage::ParticleSimulationStageScript)
				{
					const auto& CompileSimStageData = CompileData->CompileSimStageData[SourceSimStageIndex];

					FNameBuilder StageNameBuilder;
					StageNameBuilder.AppendChar(TEXT('_'));
					AppendSanitizedFunctionNameSuffix(FNameBuilder(CompileSimStageData.StageName).ToView(), StageNameBuilder);
					
					const int32 TranslationStageIndex = TranslationStages.Num();
					FNiagaraHlslTranslationStage& SimStage = TranslationStages.Emplace_GetRef(GraphBridge::GetOutputNodeScriptType(HistoryOutputNode), GraphBridge::GetOutputNodeScriptTypeId(HistoryOutputNode));

					const int32 DestSimStageIndex = CompilationOutput.ScriptData.SimulationStageMetaData.Num();
					SimStage.PassNamespace = FString::Printf(TEXT("MapSimStage%d%s"), DestSimStageIndex, StageNameBuilder.ToString());
					SimStage.ChunkModeIndex = (ENiagaraCodeChunkMode)(((int32)ENiagaraCodeChunkMode::SimulationStageBody) + (TranslationStageIndex - 2));
					if (SimStage.ChunkModeIndex >= ENiagaraCodeChunkMode::SimulationStageBodyMax)
					{
						Error(FText::Format(LOCTEXT("TooManySimulationStages", "Cannot support more than %d simulation stages when adding %d!"),
							FText::AsNumber((int32)ENiagaraCodeChunkMode::SimulationStageBodyMax - (int32)ENiagaraCodeChunkMode::SimulationStageBody),
							FText::AsNumber((int32)SimStage.ChunkModeIndex)), nullptr, nullptr);
					}
					SimStage.InterpolatedSpawnMode = ENiagaraInterpolatedSpawnMode::NoInterpolation;
					SimStage.bCopyPreviousParams = false;
					SimStage.SimulationStageIndex = DestSimStageIndex;					
					SimStage.EnabledBinding = CompileSimStageData.EnabledBinding;
					SimStage.ElementCount = CompileSimStageData.ElementCount;
					SimStage.ElementCountXBinding = CompileSimStageData.ElementCountXBinding;
					SimStage.ElementCountYBinding = CompileSimStageData.ElementCountYBinding;
					SimStage.ElementCountZBinding = CompileSimStageData.ElementCountZBinding;
					SimStage.NumIterations = CompileSimStageData.NumIterations;
					SimStage.ExecuteBehavior = CompileSimStageData.ExecuteBehavior;
					SimStage.bPartialParticleUpdate = CompileSimStageData.PartialParticleUpdate;
					SimStage.IterationSourceType = CompileSimStageData.IterationSourceType;
					SimStage.IterationDataInterface = CompileSimStageData.IterationDataInterface;
					SimStage.IterationDirectBinding = CompileSimStageData.IterationDirectBinding;
					SimStage.NumIterationsBinding = CompileSimStageData.NumIterationsBinding;
					SimStage.bParticleIterationStateEnabled = CompileSimStageData.bParticleIterationStateEnabled;
					SimStage.ParticleIterationStateBinding = CompileSimStageData.ParticleIterationStateBinding;
					SimStage.ParticleIterationStateRange = CompileSimStageData.ParticleIterationStateRange;
					SimStage.bGpuDispatchForceLinear = CompileSimStageData.bGpuDispatchForceLinear;
					SimStage.DirectDispatchType = CompileSimStageData.DirectDispatchType;
					SimStage.DirectDispatchElementType = CompileSimStageData.DirectDispatchElementType;
					SimStage.bOverrideGpuDispatchNumThreads = CompileSimStageData.bOverrideGpuDispatchNumThreads;
					SimStage.OverrideGpuDispatchNumThreads = CompileSimStageData.OverrideGpuDispatchNumThreads;

					if (CompileSimStageData.bOverrideGpuDispatchNumThreads)
					{
						SimStage.OverrideGpuDispatchNumThreads.X = CompileData->GetStaticVariableValue(CompileSimStageData.OverrideGpuDispatchNumThreadsXBinding, SimStage.OverrideGpuDispatchNumThreads.X);
						SimStage.OverrideGpuDispatchNumThreads.Y = CompileData->GetStaticVariableValue(CompileSimStageData.OverrideGpuDispatchNumThreadsYBinding, SimStage.OverrideGpuDispatchNumThreads.Y);
						SimStage.OverrideGpuDispatchNumThreads.Z = CompileData->GetStaticVariableValue(CompileSimStageData.OverrideGpuDispatchNumThreadsZBinding, SimStage.OverrideGpuDispatchNumThreads.Z);
					}

					const FOutputNode* OutputNode = SourceGraph->FindEquivalentOutputNode(SimStage.ScriptUsage, SimStage.UsageId);
					ensure(OutputNode == HistoryOutputNode);
					OutputNodes.Add(OutputNode);

					ParamMapHistories.AddDefaulted(1);
					ParamMapHistoriesSourceInOtherHistories.AddDefaulted(1);

					// If we allow partial writes we need to ensure that we are not reading from our own buffer, we ask our data interfaces if this is true or not
					if (SimStage.bPartialParticleUpdate)
					{
						for (const typename FPrecompileData::FCompileDataInterfaceData& DataInterfaceData : (*CompileData->SharedCompileDataInterfaceData.Get()))
						{
							if(DataInterfaceData.ReadsEmitterParticleData.Contains(CompileData->EmitterUniqueName))
							{
								IConsoleVariable* CVarIgnoreUnsafeReads = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.Niagara.ParticleRead.IgnoreUnsafeReads"));
								const bool bIgnoredUnsafeReads = CVarIgnoreUnsafeReads ? CVarIgnoreUnsafeReads->GetBool() : false;
								if (bIgnoredUnsafeReads && CompileSimStageData.bParticleIterationStateEnabled)
								{
									Warning(LOCTEXT("ReadsSelfEmitterDataIterationStage", "Reading own emitter data using iteration state can cause a race."), nullptr, nullptr);
								}
								else
								{
									SimStage.bPartialParticleUpdate = false;
								}
								break;
							}
						}
					}

					// See if we write any "particle" attributes
					for (int32 iVar = 0; iVar < FoundHistory.VariableMetaData.Num(); ++iVar)
					{
						// Particle attribute?
						if (!FNiagaraParameterUtilities::IsAttribute(FoundHistory.Variables[iVar]))
						{
							continue;
						}

						// Is this an output?
						const bool bIsOutput = FoundHistory.PerVariableWriteHistory[iVar].ContainsByPredicate([](const FModuleScopedPin& InPin) -> bool
						{
							return GraphBridge::AsParamMapSetNode(GraphBridge::GetOwningNode(InPin.Pin)) != nullptr;
						});

						if (!bIsOutput)
						{
							continue;
						}

						//-TODO: Temporarily skip the IGNORE variable, this needs to be cleaned up
						static const FName Name_IGNORE("IGNORE");
						FName ParameterName;
						FNiagaraEditorUtilities::DecomposeVariableNamespace(FoundHistory.Variables[iVar].GetName(), ParameterName);
						if (ParameterName == Name_IGNORE)
						{
							continue;
						}

						// We write particle attributes at this stage, store list off so we can potentially selectivly write them later
						SimStage.bWritesParticles = true;
						SimStage.SetParticleAttributes.Add(FoundHistory.Variables[iVar]);
					}

					// If we don't write particles then disable particle updates, it's meaningless and produces different HLSL since we would use a RW buffer not plain old Input
					SimStage.bPartialParticleUpdate &= SimStage.bWritesParticles;
					
					// Set up the compile output for the shader stages so that we can properly execute at runtime.
					FSimulationStageMetaData& SimulationStageMetaData = CompilationOutput.ScriptData.SimulationStageMetaData.AddDefaulted_GetRef();
					SimulationStageMetaData.SimulationStageName = CompileData->CompileSimStageData[SourceSimStageIndex].StageName;
					SimulationStageMetaData.EnabledBinding = CompileData->CompileSimStageData[SourceSimStageIndex].EnabledBinding;
					SimulationStageMetaData.ElementCount = CompileData->CompileSimStageData[SourceSimStageIndex].ElementCount;
					SimulationStageMetaData.ElementCountXBinding = CompileData->CompileSimStageData[SourceSimStageIndex].ElementCountXBinding;
					SimulationStageMetaData.ElementCountYBinding = CompileData->CompileSimStageData[SourceSimStageIndex].ElementCountYBinding;
					SimulationStageMetaData.ElementCountZBinding = CompileData->CompileSimStageData[SourceSimStageIndex].ElementCountZBinding;
					SimulationStageMetaData.ExecuteBehavior = SimStage.ExecuteBehavior;
					SimulationStageMetaData.IterationSourceType = CompileData->CompileSimStageData[SourceSimStageIndex].IterationSourceType;
					SimulationStageMetaData.IterationDataInterface = CompileData->CompileSimStageData[SourceSimStageIndex].IterationDataInterface;
					SimulationStageMetaData.IterationDirectBinding = CompileData->CompileSimStageData[SourceSimStageIndex].IterationDirectBinding;
					SimulationStageMetaData.NumIterationsBinding = CompileData->CompileSimStageData[SourceSimStageIndex].NumIterationsBinding;
					SimulationStageMetaData.NumIterations = SimStage.NumIterations;
					SimulationStageMetaData.bWritesParticles = SimStage.bWritesParticles;
					SimulationStageMetaData.bPartialParticleUpdate = SimStage.bPartialParticleUpdate;
					SimulationStageMetaData.bParticleIterationStateEnabled = SimStage.bParticleIterationStateEnabled;
					SimulationStageMetaData.ParticleIterationStateBinding = SimStage.ParticleIterationStateBinding;
					SimulationStageMetaData.ParticleIterationStateRange = SimStage.ParticleIterationStateRange;

					// Determine dispatch information from iteration source (if we have one)
					SimulationStageMetaData.GpuDispatchType = ENiagaraGpuDispatchType::OneD;
					SimulationStageMetaData.GpuDispatchNumThreads = FNiagaraShader::GetDefaultThreadGroupSize(ENiagaraGpuDispatchType::OneD);
					SimulationStageMetaData.bGpuIndirectDispatch = false;
					if ( SimulationStageMetaData.IterationSourceType == ENiagaraIterationSource::DataInterface )
					{
						if (SimulationStageMetaData.IterationDataInterface.IsNone())
						{
							Error(LOCTEXT("NoIterationDataInterface", "An data interface must be bound to a data interface iteration stage"), nullptr, nullptr);
							return TranslateResults;
						}

						if (const FNiagaraVariable* IterationSourceVar = CompileData->EncounteredVariables.FindByPredicate([&](const FNiagaraVariable& VarInfo) { return VarInfo.GetName() == SimulationStageMetaData.IterationDataInterface; }))
						{
							if ( UNiagaraDataInterface* IterationSourceCDO = CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(IterationSourceVar->GetType().GetClass()) )
							{
								SimulationStageMetaData.GpuDispatchType = SimStage.bGpuDispatchForceLinear ? ENiagaraGpuDispatchType::OneD : IterationSourceCDO->GetGpuDispatchType();
								SimulationStageMetaData.bGpuIndirectDispatch = IterationSourceCDO->GetGpuUseIndirectDispatch();

								if (SimStage.bOverrideGpuDispatchNumThreads)
								{
									SimulationStageMetaData.GpuDispatchNumThreads = SimStage.OverrideGpuDispatchNumThreads;
								}
								else
								{
									SimulationStageMetaData.GpuDispatchNumThreads = (SimulationStageMetaData.GpuDispatchType == ENiagaraGpuDispatchType::Custom) ? IterationSourceCDO->GetGpuDispatchNumThreads() : FNiagaraShader::GetDefaultThreadGroupSize(SimulationStageMetaData.GpuDispatchType);
								}
							}
						}
					}
					else if (SimulationStageMetaData.IterationSourceType == ENiagaraIterationSource::DirectSet)
					{
						SimulationStageMetaData.GpuDispatchType = SimStage.DirectDispatchType;
						SimulationStageMetaData.GpuDirectDispatchElementType = SimStage.DirectDispatchElementType;
						if (SimStage.bOverrideGpuDispatchNumThreads)
						{
							SimulationStageMetaData.GpuDispatchNumThreads = SimStage.OverrideGpuDispatchNumThreads;
						}
						else
						{
							SimulationStageMetaData.GpuDispatchNumThreads = FNiagaraShader::GetDefaultThreadGroupSize(SimStage.DirectDispatchType);
						}
					}

					// Increment source stage index
					++SourceSimStageIndex;

					// Other outputs are written to as appropriate data interfaces are found. See HandleDataInterfaceCall for details
					ParamMapSetVariablesToChunks.AddDefaulted(1);
				}
			}
		}
	}	break;
	default:
	{
		FNiagaraHlslTranslationStage& DefaultStage = TranslationStages.Emplace_GetRef(CompileOptions.TargetUsage, CompileOptions.TargetUsageId);
		DefaultStage.PassNamespace = TEXT("Map");
		DefaultStage.ChunkModeIndex = ENiagaraCodeChunkMode::Body;
		DefaultStage.SimulationStageIndex = 0;
		DefaultStage.NumIterations = 1;
		DefaultStage.bWritesParticles = true;
		DefaultStage.bShouldUpdateInitialAttributeValues = TranslationStages[0].ShouldDoSpawnOnlyLogic() || (IsEventSpawnScript() && CompileOptions.AdditionalDefines.Contains(FNiagaraCompileOptions::EventSpawnInitialAttribWritesDefine));

		OutputNodes.Add(SourceGraph->FindEquivalentOutputNode(TranslationStages[0].ScriptUsage, TranslationStages[0].UsageId));

		if (CompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			for (int32 StageIdx = 0; StageIdx < CompileData->CompileSimStageData.Num(); StageIdx++)
			{
				const FGuid& StageGuid = CompileData->CompileSimStageData[StageIdx].StageGuid;
				if (StageGuid == CompileOptions.TargetUsageId && CompileData->CompileSimStageData.IsValidIndex(StageIdx))
				{
					DefaultStage.IterationSourceType = CompileData->CompileSimStageData[StageIdx].IterationSourceType;
					DefaultStage.IterationDataInterface = CompileData->CompileSimStageData[StageIdx].IterationDataInterface;
					DefaultStage.IterationDirectBinding = CompileData->CompileSimStageData[StageIdx].IterationDirectBinding;
				}
			}
		}
		ParamMapHistories.AddDefaulted(1);
		ParamMapHistoriesSourceInOtherHistories.AddDefaulted(1);
		ParamMapSetVariablesToChunks.AddDefaulted(1);
	}	break;

	}

	for (int32 i = 0; i < TranslationStages.Num(); i++)
	{
		const FOutputNode* OutputNode = OutputNodes[i];

		if (OutputNode == nullptr)
		{
			Error(FText::Format(LOCTEXT("GetOutputNodeFail", "Cannot find output node of type {0}!"), FText::AsNumber((int32)TranslationStages[i].ScriptUsage)), nullptr, nullptr);
			return TranslateResults;
		}
		
		ValidateTypePins(OutputNode);
		{
			bool bHasAnyConnections = false;
			for (const FInputPin* InputPin : GraphBridge::GetInputPins(OutputNode))
			{
				if (GraphBridge::GetLinkedOutputPin(InputPin) != nullptr)
				{
					bHasAnyConnections = true;
					break;
				}
			}
			if (!bHasAnyConnections)
			{
				Error(FText::Format(LOCTEXT("GetOutputNodeConnectivityFail", "Cannot find any connections to output node of type {0}!"), FText::AsNumber((int32)TranslationStages[i].ScriptUsage)), nullptr, nullptr);
				return TranslateResults;
			}
		}
	}

	PerStageMainPreSimulateChunks.SetNum(TranslationStages.Num());

	ParamMapDefinedEngineVars.Emplace(SYS_PARAM_ENGINE_EXEC_INDEX);
	ParamMapDefinedEngineVars.Emplace(SYS_PARAM_ENGINE_EMITTER_ID);

	// Get all the parameter map histories traced to this graph from output nodes. We'll revisit this shortly in order to build out just the ones we care about for this translation.
	if (ParamMapHistories.Num() == 1 && OtherOutputParamMapHistories.Num() == 1 && (CompileOptions.TargetUsage == ENiagaraScriptUsage::Function || CompileOptions.TargetUsage == ENiagaraScriptUsage::DynamicInput))
	{
		ParamMapHistories[0] = (OtherOutputParamMapHistories[0]);
		ParamMapHistoriesSourceInOtherHistories[0] = 0;

		TArray<int32> Entries;
		Entries.AddZeroed(OtherOutputParamMapHistories[0].Variables.Num());
		for (int32 i = 0; i < Entries.Num(); i++)
		{
			Entries[i] = INDEX_NONE;
		}
		ParamMapSetVariablesToChunks[0] = (Entries);
	}
	else
	{
		const bool UsesInterpolation = RequiresInterpolation();

		if (UsesInterpolation)
		{
			InterpSpawnVariables.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Interpolation.InterpSpawn_Index"));
			InterpSpawnVariables.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation.InterpSpawn_SpawnTime"));
			InterpSpawnVariables.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation.InterpSpawn_UpdateTime"));
			InterpSpawnVariables.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation.InterpSpawn_InvSpawnTime"));
			InterpSpawnVariables.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation.InterpSpawn_InvUpdateTime"));
			InterpSpawnVariables.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation.SpawnInterp"));
			InterpSpawnVariables.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation.Emitter_SpawnInterval"));
			InterpSpawnVariables.Emplace(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Interpolation.Emitter_InterpSpawnStartDt"));
			InterpSpawnVariables.Emplace(FNiagaraTypeDefinition::GetIntDef(), TEXT("Interpolation.Emitter_SpawnGroup"));
		}
		
		for (int32 HistoryIdx = 0; HistoryIdx < OtherOutputParamMapHistories.Num(); HistoryIdx++)
		{
			FParamMapHistory& FoundHistory = OtherOutputParamMapHistories[HistoryIdx];

			const FOutputNode* HistoryOutputNode = FoundHistory.GetFinalOutputNode();
			if (HistoryOutputNode != nullptr && !ShouldConsiderTargetParameterMap(GraphBridge::GetOutputNodeScriptType(HistoryOutputNode)))
			{
				continue;
			}

			// Now see if we want to use any of these specifically..
			for (int32 ParamMapIdx = 0; ParamMapIdx < TranslationStages.Num(); ParamMapIdx++)
			{
				const FOutputNode* TargetOutputNode = OutputNodes[ParamMapIdx];
				if (FoundHistory.GetFinalOutputNode() == TargetOutputNode)
				{
					if (bRequiresPersistentIDs)
					{
						//TODO: Setup alias for current level to decouple from "Particles". Would we ever want emitter or system persistent IDs?
						FNiagaraVariable Var = FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particles.ID"));
						FoundHistory.AddVariable(Var, Var, NAME_None, nullptr);
					}
					{
						// NOTE(mv): This will explicitly expose Particles.UniqueID to the HLSL code regardless of whether it is exposed in a script or not. 
						//           This is necessary as the script needs to know about it even when no scripts reference it. 
						FNiagaraVariable Var = FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particles.UniqueID"));
						FoundHistory.AddVariable(Var, Var, NAME_None, nullptr);
					}

					if (UsesInterpolation)
					{
						for (const FNiagaraVariable& Var : InterpSpawnVariables)
						{
							FoundHistory.AddVariable(Var, Var, NAME_None, nullptr);
						}
					}

					ParamMapHistories[ParamMapIdx] = (FoundHistory);
					ParamMapHistoriesSourceInOtherHistories[ParamMapIdx] = HistoryIdx;

					TArray<int32> Entries;
					Entries.AddZeroed(FoundHistory.Variables.Num());
					for (int32 i = 0; i < Entries.Num(); i++)
					{
						Entries[i] = INDEX_NONE;
					}
					ParamMapSetVariablesToChunks[ParamMapIdx] = (Entries);
				}
			}
		}
	}

	CompilationOutput.ScriptData.ParameterCollectionPaths.Empty();
	for (FParamMapHistory& History : ParamMapHistories)
	{
		GraphBridge::AddCollectionPaths(History, CompilationOutput.ScriptData.ParameterCollectionPaths);
	}
	ENiagaraScriptUsage Usage = CompileOptions.TargetUsage;
	if (Usage != ENiagaraScriptUsage::SystemSpawnScript && Usage != ENiagaraScriptUsage::SystemUpdateScript && Usage != ENiagaraScriptUsage::Module && Usage != ENiagaraScriptUsage::DynamicInput)
	{
		ValidateParticleIDUsage();
	}

	BuildConstantBuffer<FNiagaraGlobalParameters>(ENiagaraCodeChunkMode::GlobalConstant);
	// only use the SystemConstantBuffer if we are doing particle scripts (for system scripts the data should come from
	// the datasets)
	if (!IsBulkSystemScript())
	{
		BuildConstantBuffer<FNiagaraSystemParameters>(ENiagaraCodeChunkMode::SystemConstant);
		BuildConstantBuffer<FNiagaraOwnerParameters>(ENiagaraCodeChunkMode::OwnerConstant);
		BuildConstantBuffer<FNiagaraEmitterParameters>(ENiagaraCodeChunkMode::EmitterConstant);
	}

	//Create main scope pin cache.
	PinToCodeChunks.AddDefaulted(1);

	ActiveHistoryForFunctionCalls.BeginTranslation(GetUniqueEmitterName());

	CompilationOutput.ScriptData.StatScopes.Empty();
	EnterStatsScope(FNiagaraStatScope(*CompileOptions.GetName(), *CompileOptions.GetName()));

	TArray<int32> OutputChunks;

	bool bInterpolateParams = false;
	FString StageSetupAndTeardownHLSL;

	if (TranslationStages.Num() > 1)
	{
		for (int32 i = 0; i < TranslationStages.Num(); i++)
		{
			ActiveStageIdx = i;
			CurrentBodyChunkMode = TranslationStages[i].ChunkModeIndex;
			if (TranslationStages[i].ShouldDoSpawnOnlyLogic())
				bInitializedDefaults = false;

			if (UNiagaraScript::IsParticleSpawnScript(TranslationStages[i].ScriptUsage))
			{
				AddBodyComment(bHasInterpolatedSpawn ? TEXT("//Begin Interpolated Spawn Script!") : TEXT("//Begin Spawn Script!"));
				CurrentParamMapIndices.Empty();
				CurrentParamMapIndices.Add(0);
				ActiveHistoryForFunctionCalls.BeginUsage(TranslationStages[i].ScriptUsage);
				OutputNodes[i]->Compile(this, OutputChunks);
				ActiveHistoryForFunctionCalls.EndUsage();
				InstanceWrite = FDataSetAccessInfo(); // Reset after building the output..
				AddBodyComment(TEXT("//End Spawn Script!\n\n"));

				AddBodyComment(TEXT("//Handle resetting previous values at the end of spawn so that they match outputs! (Needed for motion blur/etc)"));
				AddBodyChunk(TEXT("HandlePreviousValuesForSpawn(Context);"));
				
				BuildMissingDefaults();
			}

			if (TranslationStages[i].InterpolatedSpawnMode != ENiagaraInterpolatedSpawnMode::NoInterpolation)
			{
				bInterpolateParams = true;
			}

			if (UNiagaraScript::IsParticleUpdateScript(TranslationStages[i].ScriptUsage))
			{
				AddBodyComment(TEXT("//Begin Update Script!"));

				// We reset the counter for deterministic randoms to get parity between the standalone update script
				// and the update script part in the interpolated spawn script
				AddBodyChunk(TEXT("RandomCounterDeterministic = 0;"));
				
				//Now we compile the update script (with partial dt) and read from the temp values written above.
				CurrentParamMapIndices.Empty();
				CurrentParamMapIndices.Add(1);
				ActiveHistoryForFunctionCalls.BeginUsage(TranslationStages[i].ScriptUsage);
				OutputNodes[i]->Compile(this, OutputChunks);
				ActiveHistoryForFunctionCalls.EndUsage();
				AddBodyComment(TEXT("//End Update Script!\n\n"));
			}
			else if (TranslationStages[i].ScriptUsage == ENiagaraScriptUsage::ParticleSimulationStageScript)
			{
				AddBodyComment(FString::Printf(TEXT("//Begin Stage Script: %s!"), *TranslationStages[i].PassNamespace));
				//Now we compile the simulation stage and read from the temp values written above.
				CurrentParamMapIndices.Empty();
				CurrentParamMapIndices.Add(i);
				PinToCodeChunks.Empty();
				PinToCodeChunks.AddDefaulted(1);				
				FName IterSource = TranslationStages[i].GetIterationDataInterface();
				ActiveHistoryForFunctionCalls.BeginUsage(TranslationStages[i].ScriptUsage, IterSource);
				OutputNodes[i]->Compile(this, OutputChunks);
				HandleSimStageSetupAndTeardown(i, StageSetupAndTeardownHLSL);
				ActiveHistoryForFunctionCalls.EndUsage();
				AddBodyComment(FString::Printf(TEXT("//End Simulation Stage Script: %s\n\n"), *TranslationStages[i].PassNamespace));
			}
		}
		CurrentBodyChunkMode = ENiagaraCodeChunkMode::Body;
	}
	else if (TranslationStages.Num() == 1)
	{
		CurrentBodyChunkMode = TranslationStages[0].ChunkModeIndex;
		ActiveStageIdx = 0;
		check(CompileOptions.TargetUsage == TranslationStages[0].ScriptUsage);
		CurrentParamMapIndices.Empty();
		CurrentParamMapIndices.Add(0); 
		if (TranslationStages[0].ShouldDoSpawnOnlyLogic())
			bInitializedDefaults = false;

		FName IterSource = TranslationStages[0].GetIterationDataInterface();
		ActiveHistoryForFunctionCalls.BeginUsage(TranslationStages[0].ScriptUsage, IterSource);
		OutputNodes[0]->Compile(this, OutputChunks);
		ActiveHistoryForFunctionCalls.EndUsage();

		bool bIsEventSpawn = IsEventSpawnScript();
		if (IsSpawnScript() || TranslationStages[0].bShouldUpdateInitialAttributeValues)
		{
			// be sure to reset the usage as it might have changed during the above Compile and BuildMissingDefaults does a traversal
			ActiveHistoryForFunctionCalls.BeginUsage(TranslationStages[0].ScriptUsage, IterSource);
			BuildMissingDefaults();
			ActiveHistoryForFunctionCalls.EndUsage();
		}

		if (UNiagaraScript::IsParticleEventScript(TranslationStages[0].ScriptUsage))
		{
			if (CompileOptions.AdditionalDefines.Contains(FNiagaraCompileOptions::EventSpawnDefine))
			{
				AddBodyComment(TEXT("//Handle resetting previous values at the end of spawn so that they match outputs! (Needed for motion blur/etc)"));
				AddBodyChunk(TEXT("HandlePreviousValuesForSpawn(Context);"));
			}
		}
	}
	else
	{
		Error(LOCTEXT("NoTranslationStages", "Cannot find any translation stages!"), nullptr, nullptr);
		return TranslateResults;
	}

	CurrentParamMapIndices.Empty();
	ExitStatsScope();

	ActiveHistoryForFunctionCalls.EndTranslation(GetUniqueEmitterName());

	TranslateResults.bHLSLGenSucceeded = TranslateResults.NumErrors == 0;

	//If we're compiling a function then we have all we need already, we don't want to actually generate shader/vm code.
	if (FunctionCtx())
	{
		return TranslateResults;
	}

	//Now evaluate all the code chunks to generate the shader code.
	//FString HlslOutput;
	if (TranslateResults.bHLSLGenSucceeded)
	{
		//TODO: Declare all used structures up here too.
		CompilationOutput.ScriptData.ReadDataSets.Empty();
		CompilationOutput.ScriptData.WriteDataSets.Empty();

		//Generate function definitions
		FString FunctionDefinitionString = GetFunctionDefinitions();
		FunctionDefinitionString += TEXT("\n");
		{
			if (TranslationStages.Num() > 1 && RequiresInterpolation())
			{
				int32 OutputIdx = 0;
				//ensure the interpolated spawn constants are part of the parameter set.
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_ENGINE_TIME, nullptr, 0, OutputIdx, nullptr);
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_ENGINE_WORLD_DELTA_TIME, nullptr, 0, OutputIdx, nullptr);
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_ENGINE_DELTA_TIME, nullptr, 0, OutputIdx, nullptr);
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_ENGINE_INV_DELTA_TIME, nullptr, 0, OutputIdx, nullptr);
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_ENGINE_EXEC_COUNT, nullptr, 0, OutputIdx, nullptr);
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_EMITTER_SPAWNRATE, nullptr, 0, OutputIdx, nullptr);
				if (CompilationTarget != ENiagaraSimTarget::GPUComputeSim)
				{
					ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_EMITTER_SPAWN_INTERVAL, nullptr, 0, OutputIdx, nullptr);
					ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_EMITTER_INTERP_SPAWN_START_DT, nullptr, 0, OutputIdx, nullptr);
					ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_EMITTER_SPAWN_GROUP, nullptr, 0, OutputIdx, nullptr);
				}
			}

			if (TranslationStages.Num() > 0)
			{
				int32 OutputIdx = 0;
				// NOTE(mv): This will explicitly expose Engine.Emitter.TotalSpawnedParticles to the HLSL code regardless of whether it is exposed in a script or not. 
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_ENGINE_EMITTER_TOTAL_SPAWNED_PARTICLES, nullptr, 0, OutputIdx, nullptr);
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_EMITTER_RANDOM_SEED, nullptr, 0, OutputIdx, nullptr);
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_ENGINE_EMITTER_INSTANCE_SEED, nullptr, 0, OutputIdx, nullptr);
				ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_ENGINE_SYSTEM_RANDOM_SEED, nullptr, 0, OutputIdx, nullptr);
			}
		}

		PackRegisteredUniformChunk(CompilationOutput.ScriptData.Parameters);

		// Generate the Parameter Map HLSL definitions. We don't add to the final HLSL output here. We just build up the strings and tables
		// that are needed later.
		TArray<FNiagaraVariable> PrimaryDataSetOutputEntries;
		FString ParameterMapDefinitionStr = BuildParameterMapHlslDefinitions(PrimaryDataSetOutputEntries);

		// Traverse all the used data interfaces and generate any used structs
		// This step is for template files that might use structs which need defining via the translator
		{
			TSet<FName> InterfaceClasses;
			for (int32 i = 0; i < CompilationOutput.ScriptData.DataInterfaceInfo.Num(); i++)
			{
				FNiagaraScriptDataInterfaceCompileInfo& Info = CompilationOutput.ScriptData.DataInterfaceInfo[i];
				UNiagaraDataInterface* CDO = CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(Info.Type.GetClass());
				check(CDO != nullptr);
				if (CDO && CDO->CanExecuteOnTarget(ENiagaraSimTarget::GPUComputeSim))
				{
					if (!InterfaceClasses.Contains(Info.Type.GetFName()))
					{
						TArray<FNiagaraFunctionSignature> FunctionSigs;
						CDO->GetFunctionSignatures(FunctionSigs);
						for (const FNiagaraFunctionSignature& FunctionSig : FunctionSigs)
						{
							if (FunctionSig.bSupportsGPU == false)
							{
								continue;
							}

							for (const FNiagaraVariable& Input : FunctionSig.Inputs)
							{
								AddStructToDefinitionSet(Input.GetType());
							}
							for (const FNiagaraVariableBase& Output : FunctionSig.Outputs)
							{
								AddStructToDefinitionSet(Output.GetType());
							}
						}
					}
				}
			}
		}

		// Ensure some structures are always added as we use them in custom HLSL / data interface templates
		// Remove some structures which we define inside NiagaraEmitterInstanceShader.usf as we want a common set of functions
		// Ensure we always add structures that are fundamental to custom HLSL or data interface templates shader files
		StructsToDefine.AddUnique(FNiagaraTypeDefinition::GetIDDef());
		StructsToDefine.Remove(FNiagaraTypeDefinition::GetRandInfoDef());

		for (const FNiagaraTypeDefinition& Type : StructsToDefine)
		{
			FText ErrorMessage;
			HlslOutput += BuildHLSLStructDecl(Type, ErrorMessage, CompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleGPUComputeScript);
			if (ErrorMessage.IsEmpty() == false)
			{
				Error(ErrorMessage, nullptr, nullptr);
			}
		}

		const TPair<ENiagaraCodeChunkMode, FString> ChunkModeInfos[] =
		{
			MakeTuple(ENiagaraCodeChunkMode::GlobalConstant,	TEXT("FNiagaraGlobalParameters")),
			MakeTuple(ENiagaraCodeChunkMode::SystemConstant,	TEXT("FNiagaraSystemParameters")),
			MakeTuple(ENiagaraCodeChunkMode::OwnerConstant,		TEXT("FNiagaraOwnerParameters")),
			MakeTuple(ENiagaraCodeChunkMode::EmitterConstant,	TEXT("FNiagaraEmitterParameters")),
			MakeTuple(ENiagaraCodeChunkMode::Uniform,			TEXT("FNiagaraExternalParameters")),
		};

		const TCHAR* InterpPrefix[] = { TEXT(""), INTERPOLATED_PARAMETER_PREFIX };

		// GPU simulation we prefer loose bindings, all but the external cbuffer are loose.  External contains structures which we
		// don't understand when generating parameters so for the moment we can't pack it in easily.
		// This is two separate loops as the VVM assumes cbuffer order is AllCurrent -> AllPrevious, whereas GPU we bind in order to minimize parameter copies
		if (TranslationOptions.SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			for (const TPair<ENiagaraCodeChunkMode, FString>& ChunkModeInfo : ChunkModeInfos)
			{
				for (int32 InterpIt = 0; InterpIt < (bInterpolateParams ? 2 : 1); ++InterpIt)
				{
					const bool bIsCBuffer = ChunkModeInfo.Key == ENiagaraCodeChunkMode::Uniform;
					if (bIsCBuffer)
					{
						HlslOutput.Appendf(TEXT("cbuffer %s%s\n{\n"), InterpPrefix[InterpIt], *ChunkModeInfo.Value);
					}

					for (int32 ChunkOffset : ChunksByMode[int(ChunkModeInfo.Key)])
					{
						FNiagaraVariable BufferVariable(CodeChunks[ChunkOffset].Type, FName(*CodeChunks[ChunkOffset].SymbolName));
						if (IsVariableInUniformBuffer(BufferVariable))
						{
							FNiagaraCodeChunk Chunk = CodeChunks[ChunkOffset];
							Chunk.SymbolName = InterpPrefix[InterpIt] + Chunk.SymbolName;

							HlslOutput += TEXT("\t") + GetCode(Chunk);
						}
					}

					if (bIsCBuffer)
					{
						HlslOutput += TEXT("}\n\n");
					}
				}
			}
		}
		else
		{
			for (int32 InterpIt = 0; InterpIt < (bInterpolateParams ? 2 : 1); ++InterpIt)
			{
				for (const TPair<ENiagaraCodeChunkMode, FString>& ChunkModeInfo : ChunkModeInfos)
				{
					HlslOutput.Appendf(TEXT("cbuffer %s%s\n{\n"), InterpPrefix[InterpIt], *ChunkModeInfo.Value);
					for (int32 ChunkOffset : ChunksByMode[int(ChunkModeInfo.Key)])
					{
						FNiagaraVariable BufferVariable(CodeChunks[ChunkOffset].Type, FName(*CodeChunks[ChunkOffset].SymbolName));
						if (IsVariableInUniformBuffer(BufferVariable))
						{
							FNiagaraCodeChunk Chunk = CodeChunks[ChunkOffset];
							Chunk.SymbolName = InterpPrefix[InterpIt] + Chunk.SymbolName;

							HlslOutput += TEXT("\t") + GetCode(Chunk);
						}
					}
					HlslOutput += TEXT("}\n\n");
				}
			}
		}

		WriteDataSetStructDeclarations(DataSetReadInfo[0], true, HlslOutput);
		WriteDataSetStructDeclarations(DataSetWriteInfo[0], false, HlslOutput);

		//Map of all variables accessed by all datasets.
		TArray<TArray<FNiagaraVariable>> DataSetVariables;

		TMap<FNiagaraDataSetID, int32> DataSetReads;
		TMap<FNiagaraDataSetID, int32> DataSetWrites;

		const FNiagaraDataSetID InstanceDataSetID = GetInstanceDataSetID();

		const int32 InstanceReadVarsIndex = DataSetVariables.AddDefaulted();
		const int32 InstanceWriteVarsIndex = DataSetVariables.AddDefaulted();

		DataSetReads.Add(InstanceDataSetID, InstanceReadVarsIndex);
		DataSetWrites.Add(InstanceDataSetID, InstanceWriteVarsIndex);

		if (IsBulkSystemScript())
		{
			// We have two sets of data that can change independently.. The engine data set are variables
			// that are essentially set once per system. The constants are rapid iteration variables
			// that exist per emitter and change infrequently. Since they are so different, putting
			// them in two distinct read data sets seems warranted.
			const FNiagaraDataSetID SystemEngineDataSetID = GetSystemEngineDataSetID();

			const int32 SystemEngineReadVarsIndex = DataSetVariables.Num();
			DataSetReads.Add(SystemEngineDataSetID, SystemEngineReadVarsIndex);
			TArray<FNiagaraVariable>& SystemEngineReadVars = DataSetVariables.AddDefaulted_GetRef();

			HandleNamespacedExternalVariablesToDataSetRead(SystemEngineReadVars, TEXT("Engine"));
			HandleNamespacedExternalVariablesToDataSetRead(SystemEngineReadVars, TEXT("User"));

			// We sort the variables so that they end up in the same ordering between Spawn & Update...
			Algo::SortBy(SystemEngineReadVars, &FNiagaraVariable::GetName, FNameLexicalLess());

			{
				FNiagaraParameters ExternalParams;
				ExternalParams.Parameters = DataSetVariables[SystemEngineReadVarsIndex];
				CompilationOutput.ScriptData.DataSetToParameters.Add(GetSystemEngineDataSetID().Name, ExternalParams);
			}
		}

		// Now we pull in the HLSL generated above by building the parameter map definitions..
		HlslOutput += ParameterMapDefinitionStr;

		// Gather up all the unique Attribute variables that we generated.
		TArray<FNiagaraVariable> BasicAttributes;
		for (FNiagaraVariable& Var : InstanceRead.Variables)
		{
			if (Var.GetType().GetClass() != nullptr || Var.GetType().IsStatic())
			{
				continue;
			}
			BasicAttributes.AddUnique(Var);
		}
		for (FNiagaraVariable& Var : InstanceWrite.Variables)
		{
			if (Var.GetType().GetClass() != nullptr || Var.GetType().IsStatic())
			{
				continue;
			}
			else if (Var.GetType() != FNiagaraTypeDefinition::GetParameterMapDef())
			{
				BasicAttributes.AddUnique(Var);
			}
			else
			{
				for (FNiagaraVariable& ParamMapVar : PrimaryDataSetOutputEntries)
				{
					BasicAttributes.AddUnique(ParamMapVar);
				}
			}
		}

		TrimAttributes(CompileOptions, BasicAttributes);

		// We sort the variables so that they end up in the same ordering between Spawn & Update...
		Algo::Sort(BasicAttributes, [](const FNiagaraVariable& Lhs, const FNiagaraVariable& Rhs)
		{
			const int32 NameDiff = Lhs.GetName().Compare(Rhs.GetName());
			if (NameDiff)
			{
				return NameDiff < 0;
			}
			return Lhs.GetType().GetFName().Compare(Rhs.GetType().GetFName()) < 0;
		});

		ConvertFloatToHalf(CompileOptions, BasicAttributes);
		CollapseAttributes(CompileOptions, BasicAttributes);

		DataSetVariables[InstanceReadVarsIndex] = BasicAttributes;
		DataSetVariables[InstanceWriteVarsIndex] = BasicAttributes;

		//Define the simulation context. Which is a helper struct containing all the input, result and intermediate data needed for a single simulation.
		//Allows us to reuse the same simulate function but provide different wrappers for final IO between GPU and CPU sims.
		{
			HlslOutput += TEXT("struct FSimulationContext") TEXT("\n{\n");

			// We need to reserve a place in the simulation context for the base Parameter Map.
			if (PrimaryDataSetOutputEntries.Num() != 0 || ParamMapDefinedSystemVars.Num() != 0 || ParamMapDefinedEmitterParameterToNamespaceVars.Num() != 0 || (ParamMapSetVariablesToChunks.Num() != 0 && ParamMapSetVariablesToChunks[0].Num() > 0))
			{
				for (int32 i = 0; i < TranslationStages.Num(); i++)
				{
					FDeclarationPermutationContext PermutationContext(*this, TranslationStages[i], HlslOutput);
					HlslOutput += TEXT("\tFParamMap0 ") + TranslationStages[i].PassNamespace + TEXT(";\n");
				}
			}

			WriteDataSetContextVars(DataSetReadInfo[0], true, HlslOutput);
			WriteDataSetContextVars(DataSetWriteInfo[0], false, HlslOutput);


			HlslOutput += TEXT("};\n\n");
		}
		
		HlslOutput += TEXT("static float HackSpawnInterp = 1.0;\n");

		HlslOutput += FunctionDefinitionString;

		TArray<int32> WriteConditionVars;

		// copy the accessed data sets over to the script, so we can grab them during sim
		for (TPair <FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>> InfoPair : DataSetReadInfo[0])
		{
			CompilationOutput.ScriptData.ReadDataSets.Add(InfoPair.Key);
		}

		for (TPair <FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>> InfoPair : DataSetWriteInfo[0])
		{
			FNiagaraDataSetProperties SetProps;
			SetProps.ID = InfoPair.Key;
			for (TPair <int32, FDataSetAccessInfo> IndexPair : InfoPair.Value)
			{
				//-TODO:FNiagaraVariableBase the source can be FNiagaraVariableBase also
				SetProps.Variables.Empty();
				FNiagaraVariable::ConvertToBaseArray(IndexPair.Value.Variables, SetProps.Variables);
			}

			CompilationOutput.ScriptData.WriteDataSets.Add(SetProps);

			int32* ConditionalWriteChunkIdxPtr = DataSetWriteConditionalInfo[0].Find(InfoPair.Key);
			if (ConditionalWriteChunkIdxPtr == nullptr)
			{
				WriteConditionVars.Add(INDEX_NONE);
			}
			else
			{
				WriteConditionVars.Add(*ConditionalWriteChunkIdxPtr);
			}
		}

		DefineInterpolatedParametersFunction(HlslOutput);
		DefinePreviousParametersFunction(HlslOutput, DataSetVariables, DataSetReads, DataSetWrites);

		// define functions for reading and writing all secondary data sets
		DefineDataSetReadFunction(HlslOutput, CompilationOutput.ScriptData.ReadDataSets);
		DefineDataSetWriteFunction(HlslOutput, CompilationOutput.ScriptData.WriteDataSets, WriteConditionVars);

		//Define the shared per instance simulation function
		// for interpolated scripts AND GPU sim, define spawn and sim in separate functions
		if (TranslationStages.Num() > 1)
		{
			for (int32 StageIdx = 0; StageIdx < TranslationStages.Num(); StageIdx++)
			{
				FDeclarationPermutationContext PermutationContext(*this, TranslationStages[StageIdx], HlslOutput);

				HlslOutput += TEXT("void Simulate") + TranslationStages[StageIdx].PassNamespace + TEXT("(inout FSimulationContext Context)\n{\n");
				int32 ChunkMode = (int32)TranslationStages[StageIdx].ChunkModeIndex;
				for (int32 i = 0; i < ChunksByMode[ChunkMode].Num(); ++i)
				{
					HlslOutput += TEXT("\t") + GetCode(ChunksByMode[ChunkMode][i]);
				}
				HlslOutput += TEXT("}\n");
			}
		}
		else
		{
			HlslOutput += TEXT("void Simulate(inout FSimulationContext Context)\n{\n");
			for (int32 i = 0; i < ChunksByMode[(int32)ENiagaraCodeChunkMode::Body].Num(); ++i)
			{
				HlslOutput += GetCode(ChunksByMode[(int32)ENiagaraCodeChunkMode::Body][i]);
			}
			HlslOutput += TEXT("}\n");
		}

		if (TranslationOptions.SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			FString DataInterfaceHLSL;
			DefineDataInterfaceHLSL(DataInterfaceHLSL);
			HlslOutput += DataInterfaceHLSL;

			DefineExternalFunctionsHLSL(HlslOutput);

			HlslOutput += StageSetupAndTeardownHLSL;
		}

		//And finally, define the actual main function that handles the reading and writing of data and calls the shared per instance simulate function.
		//TODO: Different wrappers for GPU and CPU sims. 
		if (TranslationOptions.SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			DefineMainGPUFunctions(DataSetVariables, DataSetReads, DataSetWrites);
		}
		else
		{
			DefineMain(HlslOutput, DataSetVariables, DataSetReads, DataSetWrites);
		}

		//Get full list of instance data accessed by the script as the VM binding assumes same for input and output.
		for (FNiagaraVariable& Var : DataSetVariables[InstanceReadVarsIndex])
		{
			if (FNiagaraParameterUtilities::IsAttribute(Var))
			{
				FNiagaraVariable BasicAttribVar = FNiagaraParameterUtilities::ResolveAsBasicAttribute(Var, false);
				CompilationOutput.ScriptData.Attributes.AddUnique(BasicAttribVar);
			}
			else
			{
				CompilationOutput.ScriptData.Attributes.AddUnique(Var);
			}
		}

		// Log out all the information we've built thus far for assistance debugging
		FString Preamble = TEXT("// Shader generated by Niagara HLSL Translator\n\n");
		
		const UEnum* ExecuteBehaviorEnum = StaticEnum<ENiagaraSimStageExecuteBehavior>();
		for (int32 i = 0; i < CompilationOutput.ScriptData.SimulationStageMetaData.Num(); i++)
		{
			const FSimulationStageMetaData& SimStageMetaData = CompilationOutput.ScriptData.SimulationStageMetaData[i];
			Preamble.Appendf(TEXT("// SimStage[%d] = %s\n"), i, *SimStageMetaData.SimulationStageName.ToString());
			Preamble.Appendf(TEXT("//\tNumIterations = %d\n"), SimStageMetaData.NumIterations);
			Preamble.Appendf(TEXT("//\tExecuteBehavior = %s\n"), *ExecuteBehaviorEnum->GetNameStringByValue((int64)SimStageMetaData.ExecuteBehavior));
			Preamble.Appendf(TEXT("//\tWritesParticles = %s\n"), SimStageMetaData.bWritesParticles ? TEXT("True") : TEXT("False"));
			Preamble.Appendf(TEXT("//\tPartialParticleUpdate = %s\n"), SimStageMetaData.bPartialParticleUpdate ? TEXT("True") : TEXT("False"));

			if ( SimStageMetaData.bParticleIterationStateEnabled )
			{
				Preamble.Appendf(TEXT("//\tParticleIterationStage = Attribute(%s) Range(%d ... %d)\n"), *SimStageMetaData.ParticleIterationStateBinding.ToString(), SimStageMetaData.ParticleIterationStateRange.X, SimStageMetaData.ParticleIterationStateRange.Y);
			}

			for (const FName& Dest : SimStageMetaData.OutputDestinations)
			{
				Preamble.Appendf(TEXT("//\tOutputs to: \"%s\"\n"), *Dest.ToString());
			}

			for (const FName& Dest : SimStageMetaData.InputDataInterfaces)
			{
				Preamble.Appendf(TEXT("//\tReads from: \"%s\"\n"), *Dest.ToString());
			}
		}

		// Display the computed compile tags in the source hlsl to make checking easier.
		if (TranslateResults.CompileTags.Num() != 0)
		{
			Preamble.Appendf(TEXT("// Compile Tags: \n"));

			for (const FNiagaraCompilerTag& Tag : TranslateResults.CompileTags)
			{
				Preamble.Appendf(TEXT("//\tVariable: \"%s\" StringValue: \"%s\" \n"), *Tag.Variable.ToString(),  *Tag.StringValue);
			}
		}
		if (TranslateResults.CompileTags.Num() != 0)
		{
			Preamble.Appendf(TEXT("// Compile Tags Editor Only: \n"));

			for (const FNiagaraCompilerTag& Tag : TranslateResults.CompileTagsEditorOnly)
			{
				Preamble.Appendf(TEXT("//\tVariable: \"%s\" StringValue: \"%s\" \n"), *Tag.Variable.ToString(), *Tag.StringValue);
			}
		}

		// Display the computed compile tags in the source hlsl to make checking easier.
		if (CompileData->StaticVariables.Num() != 0)
		{
			Preamble.Appendf(TEXT("\n// Compile Data> Static Variables Input: \n"));

			for (const FNiagaraVariable& StaticVar: CompileData->StaticVariables)
			{
				Preamble.Appendf(TEXT("//\tVariable: %s \n"), *StaticVar.ToString());
			}

		}

		if (CompileData->PinToConstantValues.Num() != 0)
		{
			Preamble.Appendf(TEXT("\n// Compile Data> PinToConstantValues Input: \n"));

			for (auto Iter : CompileData->PinToConstantValues)
			{
				Preamble.Appendf(TEXT("//\tPin: %s Value: %s\n"), *Iter.Key.ToString(), *Iter.Value);
			}
		}

		if (CompilationOutput.ScriptData.StaticVariablesWritten.Num() != 0)
		{
			Preamble.Appendf(TEXT("\n// Static Variables Written: \n"));

			for (const FNiagaraVariable& StaticVar : CompilationOutput.ScriptData.StaticVariablesWritten)
			{
				Preamble.Appendf(TEXT("//\tVariable: %s \n"), *StaticVar.ToString());
			}

		}

		HlslOutput = Preamble + TEXT("\n\n") +  HlslOutput;

		// We may have created some transient data interfaces. This cleans up the ones that we created.
		CompilationOutput.ScriptData.ShaderScriptParametersMetadata = ShaderScriptParametersMetadata;

		if (InstanceRead.Variables.Num() == 1 && InstanceRead.Variables[0].GetName() == TEXT("Particles.UniqueID")) {
			// NOTE(mv): Explicitly allow reading from Particles.UniqueID, as it is an engine managed variable and 
			//           is written to before Simulate() in the SpawnScript...
			// TODO(mv): Also allow Particles.ID for the same reasons?
			CompilationOutput.ScriptData.bReadsAttributeData = false;
		} else {
			CompilationOutput.ScriptData.bReadsAttributeData = InstanceRead.Variables.Num() != 0;
		}
		TranslateResults.OutputHLSL = HlslOutput;

	}

	return TranslateResults;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::HandleSimStageSetupAndTeardown(int32 InWhichStage, FString& OutHlsl)
{
	FNiagaraHlslTranslationStage& TranslationStage = TranslationStages[InWhichStage];
	const FName IterationDataInterface = TranslationStage.GetIterationDataInterface();

	// If we're particles then do nothing different..
	if (IterationDataInterface == NAME_None)
		return;

	FExpressionPermutationContext PermutationContext(OutHlsl);
	PermutationContext.AddBranch(*this, TranslationStage);

	// Ok, we're iterating over a known iteration source. Let's find it in the parameter map history so we know type/etc.
	FNiagaraVariable IterationSourceVar;
	const FNiagaraVariable* FoundVar = CompileData->EncounteredVariables.FindByPredicate([&](const FNiagaraVariable& VarInfo) { return VarInfo.GetName() == IterationDataInterface; });
	if (FoundVar != nullptr)
	{
		IterationSourceVar = *FoundVar;
	}
	
	if (!IterationSourceVar.IsValid())
	{
		Error(FText::Format(LOCTEXT("CannotFindIterationSourceInParamMap", "Variable {0} missing in graphs referenced during compile!"), FText::FromName(IterationDataInterface)), nullptr, nullptr);
		return;
	}

	UNiagaraDataInterface* CDO = CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(IterationSourceVar.GetType().GetClass());
	if (!CDO || !IterationSourceVar.GetType().GetClass())
	{
		Error(FText::Format(LOCTEXT("CannotFindIterationSourceCDOInParamMap", "Variable {0}'s cached CDO for class was missing during compile!"), FText::FromName(IterationDataInterface)), nullptr, nullptr);
		return;
	}

	// Now take a look at any of the variables that were actually written to / read from in this stage.
	TArray<FNiagaraVariable> ReadVars;
	TArray<FNiagaraVariable> WriteVars;
	TArray<FNiagaraVariable> AllVars;
	for (int32 ParamHistoryIdx = 0; ParamHistoryIdx < ParamMapHistories.Num(); ParamHistoryIdx++)
	{
		if (InWhichStage != ParamHistoryIdx && TranslationStage.ShouldDoSpawnOnlyLogic() == false)
			continue;

		for (int32 i = 0; i < ParamMapHistories[ParamHistoryIdx].Variables.Num(); i++)
		{
			const FNiagaraVariable& Var = ParamMapHistories[ParamHistoryIdx].Variables[i];

			if (Var.IsInNameSpace(IterationDataInterface))
			{
				if (ParamMapHistories[ParamHistoryIdx].PerVariableReadHistory[i].Num() > 0 && !ReadVars.Contains(Var))
					ReadVars.Emplace(Var);
				if (ParamMapHistories[ParamHistoryIdx].PerVariableWriteHistory[i].Num() > 0 && !WriteVars.Contains(Var))
					WriteVars.Emplace(Var);
			}
		}
	}

	for (int32 ParamHistoryIdx = 0; ParamHistoryIdx < ParamMapHistories.Num(); ParamHistoryIdx++)
	{		
		for (int32 i = 0; i < ParamMapHistories[ParamHistoryIdx].Variables.Num(); i++)
		{
			const FNiagaraVariable& Var = ParamMapHistories[ParamHistoryIdx].Variables[i];

			if (Var.IsInNameSpace(IterationDataInterface))
			{
				if (!AllVars.Contains(Var))
					AllVars.Emplace(Var);
			}
		}
	}

	// Find the data interface in the table. Note that this may not be found because we don't actually call any functions on the data interface yet.
	int32 DataInterfaceOwnerIndex = INDEX_NONE;
	for (int32 i = 0; i < CompilationOutput.ScriptData.DataInterfaceInfo.Num(); i++)
	{
		FNiagaraScriptDataInterfaceCompileInfo& Info = CompilationOutput.ScriptData.DataInterfaceInfo[i];
		if (IterationDataInterface == Info.Name)
		{
			DataInterfaceOwnerIndex = i;
			break;
		}
	}

	// RIght now we need to know if anyone wrote to the IterationSource this stage. That can be one of two ways:
	// 1) Someone wrote to StackContext.XXXX
	// 2) Someone called a function that was marked to write 
	const int32 SourceSimStage = TranslationStage.SimulationStageIndex;
	ensure(CompilationOutput.ScriptData.SimulationStageMetaData.IsValidIndex(SourceSimStage));
	bool bWroteToIterationSource = CompilationOutput.ScriptData.SimulationStageMetaData[SourceSimStage].OutputDestinations.Contains(TranslationStage.GetIterationDataInterface());
	if (WriteVars.Num() > 0)
		bWroteToIterationSource = true;

	// Now decide if we need to put in the pre/post
	if (CDO && CDO->CanExecuteOnTarget(ENiagaraSimTarget::GPUComputeSim))
	{
		bool bNeedsDIOwner = false;
		bool bNeedsSetupAndTeardown = false;
		bool bNeedsAttributeWrite = false;
		bool bNeedsAttributeRead = false;

		// Put in the general pre/post if we wrote to the IterationSoruce at all
		if (CDO->SupportsSetupAndTeardownHLSL() && bWroteToIterationSource)
		{
			bNeedsDIOwner = true;
			bNeedsSetupAndTeardown = true;
		}

		// Handle reading/writing to the StackContext. namespace
		if (CDO->SupportsIterationSourceNamespaceAttributesHLSL())
		{
			if (ReadVars.Num() > 0)
			{
				bNeedsDIOwner = true;
				bNeedsAttributeRead = true;
			}
			if (WriteVars.Num() > 0)
			{
				bNeedsDIOwner = true;
				bNeedsAttributeWrite = true;
			}
		}
		
		// If it wasn't previously added, let's go ahead and do so. Maybe they are solely using the StackContext namespace.
		if (DataInterfaceOwnerIndex == INDEX_NONE && bNeedsDIOwner)
		{
			DataInterfaceOwnerIndex = RegisterDataInterface(IterationSourceVar, CDO, true, true);
		}

		// If we haven't created it by now, bail out. 
		if (DataInterfaceOwnerIndex == INDEX_NONE && bNeedsDIOwner)
		{
			Error(FText::Format(LOCTEXT("CannotRegisterDataInterface", "Variable {0}'s cannot register as a data interface!"), FText::FromName(IterationDataInterface)), nullptr, nullptr);
			return;
		}

		// It is an invalid state to use the IterationSource and StackContext namespace without implementing SupportsIterationSourceNamespaceAttributesHLSL
		if (ReadVars.Num() > 0 && !bNeedsAttributeRead)
		{
			Error(FText::Format(LOCTEXT("CannotUseContextRead", "Variable {0} cannot be used in conjunction with StackContext namespace variable reads! It must implement SupportsIterationSourceNamespaceAttributesHLSL."), FText::FromName(IterationDataInterface)), nullptr, nullptr);
			return;
		}

		if (WriteVars.Num() > 0 && !bNeedsAttributeWrite)
		{
			Error(FText::Format(LOCTEXT("CannotUseContextWrite", "Variable {0} cannot be used in conjunction with StackContext namespace variable writes! It must implement SupportsIterationSourceNamespaceAttributesHLSL."), FText::FromName(IterationDataInterface)), nullptr, nullptr);
			return;
		}

		if (!bNeedsSetupAndTeardown && !bNeedsAttributeRead && !bNeedsAttributeWrite)
		{
			return;
		}

		// Convert to a FNiagaraDataInterfaceGPUParamInfo to keep the API simple and consistent
		TArray<FNiagaraFunctionSignature> GeneratedFunctionSignatures;
		FNiagaraDataInterfaceGPUParamInfo DIInstanceInfo;
		ConvertCompileInfoToParamInfo(CompilationOutput.ScriptData.DataInterfaceInfo[DataInterfaceOwnerIndex], DIInstanceInfo, GeneratedFunctionSignatures);

		// This next part might be a big confusing, but because DataInterfaces are in non-editor code, it makes it impossible for them to return graphs or other
		// structures. We want them to feel free to invoke their own fucntions and not have to do a lot of extra wranging, so we treat them like a custom hlsl node.
		// The follow section will set up the necessary infrastructure to "Act" like a custom hlsl node for the translator.
		TranslationStage.CustomReadFunction = FString::Printf(TEXT("SetupFromIterationSource_%s"), *TranslationStage.PassNamespace);
		TranslationStage.CustomWriteFunction = FString::Printf(TEXT("TeardownFromIterationSource_%s"), *TranslationStage.PassNamespace);
		FString SetupFunctionHLSL;
		FString TeardownFunctionHLSL;
		FNiagaraFunctionSignature Sig;
		Sig.Name = *TranslationStage.CustomReadFunction;
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("Map")));
		Sig.Inputs.Add(FNiagaraVariable(FNiagaraTypeDefinition(IterationSourceVar.GetType().GetClass()), TEXT("TargetDataInterface")));
		Sig.Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("Map")));


		TArray<int32> Inputs;
		Inputs.Emplace(InWhichStage); // The parameter map input index above
		Inputs.Emplace(DataInterfaceOwnerIndex); // the data interface index above
		FString SetupBody;
		FString TeardownBody;

		TArray<FText> GeneratedErrors;
		bool bPartialWrites = false;
		
		const bool bSpawnOnly = TranslationStage.ExecuteBehavior == ENiagaraSimStageExecuteBehavior::OnSimulationReset;
		if (bNeedsSetupAndTeardown)
		{
			FString SetupGeneratedHLSL;
			if (CDO->GenerateSetupHLSL(DIInstanceInfo, MakeArrayView(Sig.Inputs), bSpawnOnly, bPartialWrites, GeneratedErrors, SetupGeneratedHLSL) && SetupGeneratedHLSL.Len() > 0)
			{
				Sig.Name = *FString::Printf(TEXT("SetupFromIterationSource_%s_GeneratedSetup"), *TranslationStage.PassNamespace);
				FNiagaraFunctionSignature SignatureOut = Sig;
				FString SetupOutHLSL;
				ProcessCustomHlsl(SetupGeneratedHLSL, TranslationStage.ScriptUsage, Sig, Inputs, nullptr, SetupOutHLSL, SignatureOut);
				SetupFunctionHLSL += GetFunctionSignature(SignatureOut) + TEXT("\n{\n") + SetupOutHLSL + TEXT("\n}\n");
				SetupBody += FString::Printf(TEXT("\n\t%s(Context);\n"), *BuildFunctionSignatureString(SignatureOut));
			}

			FString TeardownGeneratedHLSL;
			if (CDO->GenerateTeardownHLSL(DIInstanceInfo, MakeArrayView(Sig.Inputs), bSpawnOnly, bPartialWrites, GeneratedErrors, TeardownGeneratedHLSL) && TeardownGeneratedHLSL.Len() > 0)
			{
				Sig.Name = *FString::Printf(TEXT("TeardownFromIterationSource_%s_GeneratedTeardown"), *TranslationStage.PassNamespace);
				FNiagaraFunctionSignature SignatureOut = Sig;
				FString TeardownOutHLSL;
				ProcessCustomHlsl(TeardownGeneratedHLSL, TranslationStage.ScriptUsage, Sig, Inputs, nullptr, TeardownOutHLSL, SignatureOut);
				TeardownFunctionHLSL += GetFunctionSignature(SignatureOut) + TEXT("\n{\n") + TeardownOutHLSL + TEXT("\n}\n");
				TeardownBody += FString::Printf(TEXT("\n\t%s(Context);\n"), *BuildFunctionSignatureString(SignatureOut));
			}

		}


		if (bNeedsAttributeRead)
		{
			FString AttributeReadGeneratedHLSL;
			TArray<FString> AttributeHLSLNames;
			for (const FNiagaraVariable& Var : ReadVars)
			{
				AttributeHLSLNames.Emplace(TEXT("Map.") + BuildSanitizedSymbolName(Var.GetName()));
			}

			if (CDO->GenerateIterationSourceNamespaceReadAttributesHLSL(DIInstanceInfo,  IterationSourceVar, MakeArrayView(Sig.Inputs), MakeArrayView(ReadVars), MakeArrayView(AttributeHLSLNames), bSpawnOnly, bSpawnOnly, bPartialWrites, GeneratedErrors, AttributeReadGeneratedHLSL) && AttributeReadGeneratedHLSL.Len() > 0)
			{
				Sig.Name = *FString::Printf(TEXT("SetupFromIterationSource_%s_GeneratedReadAttributes"), *TranslationStage.PassNamespace);
				FNiagaraFunctionSignature SignatureOut = Sig;
				FString AttributeReadOutHLSL;
				ProcessCustomHlsl(AttributeReadGeneratedHLSL, TranslationStage.ScriptUsage, Sig, Inputs, nullptr, AttributeReadOutHLSL, SignatureOut);
				SetupFunctionHLSL += GetFunctionSignature(SignatureOut) + TEXT("\n{\n") + AttributeReadOutHLSL + TEXT("\n}\n");
				SetupBody += FString::Printf(TEXT("\n\t%s(Context);\n"), *BuildFunctionSignatureString(SignatureOut));

			}
		}

		if (bNeedsAttributeWrite)
		{
			FString AttributeWriteGeneratedHLSL;
			TArray<FString> AttributeHLSLNames;
			for (const FNiagaraVariable& Var : WriteVars)
			{
				AttributeHLSLNames.Emplace(TEXT("Map.") + BuildSanitizedSymbolName(Var.GetName()));
			}

			if (CDO->GenerateIterationSourceNamespaceWriteAttributesHLSL(DIInstanceInfo, IterationSourceVar, MakeArrayView(Sig.Inputs), MakeArrayView(WriteVars), MakeArrayView(AttributeHLSLNames), MakeArrayView(AllVars), bSpawnOnly, bPartialWrites, GeneratedErrors, AttributeWriteGeneratedHLSL) && AttributeWriteGeneratedHLSL.Len() > 0)
			{
				Sig.Name = *FString::Printf(TEXT("TeardownFromIterationSource_%s_GeneratedWriteAttributes"), *TranslationStage.PassNamespace);
				FNiagaraFunctionSignature SignatureOut = Sig;
				FString AttributeWriteOutHLSL;
				ProcessCustomHlsl(AttributeWriteGeneratedHLSL, TranslationStage.ScriptUsage, Sig, Inputs, nullptr, AttributeWriteOutHLSL, SignatureOut);
				TeardownFunctionHLSL += GetFunctionSignature(SignatureOut) + TEXT("\n{\n") + AttributeWriteOutHLSL + TEXT("\n}\n");
				TeardownBody += FString::Printf(TEXT("\n\t%s(Context);\n"), *BuildFunctionSignatureString(SignatureOut));
			}
		}

		for (const FText& ErrorText : GeneratedErrors)
		{
			Error(ErrorText, nullptr, nullptr);
		}

		SetupFunctionHLSL += FString::Printf(TEXT("void %s(inout FSimulationContext Context)\n{\n"), *TranslationStage.CustomReadFunction);
		SetupFunctionHLSL += SetupBody;
		SetupFunctionHLSL += TEXT("\n}\n");
		TeardownFunctionHLSL += FString::Printf(TEXT("void %s(inout FSimulationContext Context)\n{\n"), *TranslationStage.CustomWriteFunction);
		TeardownFunctionHLSL += TeardownBody;
		TeardownFunctionHLSL += TEXT("\n}\n");
		OutHlsl += SetupFunctionHLSL + TEXT("\n\n") + TeardownFunctionHLSL + TEXT("\n\n");
	}
}


void FNiagaraHlslTranslator::GatherVariableForDataSetAccess(const FNiagaraVariable& Var, FString Format, int32& IntCounter, int32 &FloatCounter, int32& HalfCounter, int32 DataSetIndex, FString InstanceIdxSymbol, FString &HlslOutputString, bool bWriteHLSL)
{
	TArray<FString> Components;
	UScriptStruct* Struct = Var.GetType().GetScriptStruct();
	if (!Struct)
	{
		Error(FText::Format(LOCTEXT("BadStructDef", "Variable {0} missing struct definition."), FText::FromName(Var.GetName())));
		return;
	}

	TArray<ENiagaraBaseTypes> Types;
	GatherComponentsForDataSetAccess(Struct, TEXT(""), false, Components, Types);

	//Add floats and then ints to hlsl
	TArray<FStringFormatArg> FormatArgs;
	FormatArgs.Empty(5);
	FormatArgs.Add(TEXT(""));//We'll set the var name below.
	FormatArgs.Add(TEXT(""));//We'll set the type name below.
	// none for the output op (data set comes from acquireindex op)
	if (DataSetIndex != INDEX_NONE)
	{
		FormatArgs.Add(DataSetIndex);
	}
	const int32 RegIdx = FormatArgs.Add(0);
	if (!InstanceIdxSymbol.IsEmpty())
	{
		FormatArgs.Add(InstanceIdxSymbol);
	}
	const int32 DefaultIdx = FormatArgs.Add(0);

	check(Components.Num() == Types.Num());
	for (int32 CompIdx = 0; CompIdx < Components.Num(); ++CompIdx)
	{
		if (Types[CompIdx] == ENiagaraBaseTypes::Float)
		{
			FormatArgs[1] = TEXT("Float");
			FormatArgs[DefaultIdx] = TEXT("0.0f");
			FormatArgs[RegIdx] = FloatCounter++;
		}
		else if (Types[CompIdx] == ENiagaraBaseTypes::Half)
		{
			FormatArgs[1] = TEXT("Half");
			FormatArgs[DefaultIdx] = TEXT("0.0f");
			FormatArgs[RegIdx] = HalfCounter++;
		}
		else if (Types[CompIdx] == ENiagaraBaseTypes::Int32)
		{
			FormatArgs[1] = TEXT("Int");
			FormatArgs[DefaultIdx] = TEXT("0");
			FormatArgs[RegIdx] = IntCounter++;
		}
		else
		{
			check(Types[CompIdx] == ENiagaraBaseTypes::Bool);
			FormatArgs[1] = TEXT("Bool");
			FormatArgs[DefaultIdx] = TEXT("false");
			FormatArgs[RegIdx] = IntCounter++;
		}
		FormatArgs[0] = Components[CompIdx];
		if (bWriteHLSL)
		{
			HlslOutputString += FString::Format(*Format, FormatArgs);
		}
	}
}

void FNiagaraHlslTranslator::GatherComponentsForDataSetAccess(UScriptStruct* Struct, FString VariableSymbol, bool bMatrixRoot, TArray<FString>& Components, TArray<ENiagaraBaseTypes>& Types)
{
	bool bIsVector = IsHlslBuiltinVector(FNiagaraTypeDefinition(Struct));
	bool bIsScalar = FNiagaraTypeDefinition::IsScalarDefinition(Struct);
	bool bIsMatrix = FNiagaraTypeDefinition(Struct) == FNiagaraTypeDefinition::GetMatrix4Def();
	if (bIsMatrix)
	{
		bMatrixRoot = true;
	}

	//Bools are an awkward special case. TODO: make neater.
	if (FNiagaraTypeDefinition(Struct) == FNiagaraTypeDefinition::GetBoolDef())
	{
		Types.Add(ENiagaraBaseTypes::Bool);
		Components.Add(VariableSymbol);
		return;
	}
	else if (FNiagaraTypeDefinition(Struct) == FNiagaraTypeDefinition::GetHalfDef())
	{
		Types.Add(ENiagaraBaseTypes::Half);
		Components.Add(VariableSymbol);
		return;
	}

	for (TFieldIterator<const FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;

		if (const FStructProperty* StructProp = CastField<const FStructProperty>(Property))
		{
			UScriptStruct* NiagaraStruct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::Simulation);
			if (bMatrixRoot && FNiagaraTypeDefinition(NiagaraStruct) == FNiagaraTypeDefinition::GetFloatDef())
			{
				GatherComponentsForDataSetAccess(NiagaraStruct, VariableSymbol + ComputeMatrixColumnAccess(Property->GetName()), bMatrixRoot, Components, Types);
			}
			else if (bMatrixRoot &&  FNiagaraTypeDefinition(NiagaraStruct) == FNiagaraTypeDefinition::GetVec4Def())
			{
				GatherComponentsForDataSetAccess(NiagaraStruct, VariableSymbol + ComputeMatrixRowAccess(Property->GetName()), bMatrixRoot, Components, Types);
			}
			else
			{
				GatherComponentsForDataSetAccess(NiagaraStruct, VariableSymbol + TEXT(".") + Property->GetName(), bMatrixRoot, Components, Types);
			}
		}
		else
		{
			FString VarName = VariableSymbol;
			if (bMatrixRoot)
			{
				if (bIsVector && Property->IsA(FFloatProperty::StaticClass())) // Parent is a vector type, we are a float type
				{
					VarName += ComputeMatrixColumnAccess(Property->GetName());
				}
			}
			else if (!bIsScalar)
			{
				VarName += TEXT(".");
				VarName += bIsVector ? Property->GetName().ToLower() : Property->GetName();
			}

			if (Property->IsA(FFloatProperty::StaticClass()))
			{
				Types.Add(ENiagaraBaseTypes::Float);
				Components.Add(VarName);
			}
			else if (Property->IsA(FIntProperty::StaticClass()))
			{
				Types.Add(ENiagaraBaseTypes::Int32);
				Components.Add(VarName);
			}
			else if (Property->IsA(FBoolProperty::StaticClass()))
			{
				Types.Add(ENiagaraBaseTypes::Bool);
				Components.Add(VarName);
			}
			else if (Property->IsA(FUInt16Property::StaticClass()))
			{
				Types.Add(ENiagaraBaseTypes::Half);
				Components.Add(VarName);
			}
		}
	}
}

void FNiagaraHlslTranslator::DefinePreviousParametersFunction(FString& HlslOutputString, TArray<TArray<FNiagaraVariable>>& DataSetVariables, TMap<FNiagaraDataSetID, int32>& DataSetReads, TMap<FNiagaraDataSetID, int32>& DataSetWrites)
{
	HlslOutputString +=
		TEXT("#if (SimulationStageIndex == 0) // MapSpawn\n")
		TEXT("void HandlePreviousValuesForSpawn(inout FSimulationContext Context)\n{\n");

	const bool WriteFunctionInternals = UNiagaraScript::IsParticleSpawnScript(CompileOptions.TargetUsage)
		|| UNiagaraScript::IsGPUScript(CompileOptions.TargetUsage)
		|| (UNiagaraScript::IsParticleEventScript(CompileOptions.TargetUsage)
			&& CompileOptions.AdditionalDefines.Contains(FNiagaraCompileOptions::EventSpawnDefine));

	if (WriteFunctionInternals)
	{
		TArray<FNiagaraDataSetID> ReadDataSetIDs;
		TArray<FNiagaraDataSetID> WriteDataSetIDs;

		DataSetReads.GetKeys(ReadDataSetIDs);
		DataSetWrites.GetKeys(WriteDataSetIDs);

		for (int32 DataSetIndex = 0; DataSetIndex < DataSetWrites.Num(); ++DataSetIndex)
		{
			const FNiagaraDataSetID DataSetID = ReadDataSetIDs[DataSetIndex];
			const TArray<FNiagaraVariable>& NiagaraVariables = DataSetVariables[DataSetWrites[DataSetID]];
			for (const FNiagaraVariable& Var : NiagaraVariables)
			{
				if (FNiagaraParameterUtilities::IsPreviousValue(Var))
				{
					FString CurMap = TranslationStages[0].PassNamespace;
					FString Value = FString::Printf(TEXT("Context.%s.%s = Context.%s.%s;\n"), *CurMap, *BuildSanitizedSymbolName(Var.GetName()),
						*CurMap, *BuildSanitizedSymbolName(FNiagaraParameterUtilities::GetSourceForPreviousValue(Var).GetName()));
					HlslOutputString += Value + TEXT("\n");
				}
			}
		}
	}
	HlslOutputString += TEXT("}\n#endif\n\n");
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::DefineInterpolatedParametersFunction(FString &HlslOutputString)
{
	for (int32 i = 0; i < TranslationStages.Num(); i++)
	{
		if (TranslationStages[i].InterpolatedSpawnMode == ENiagaraInterpolatedSpawnMode::NoInterpolation)
		{
			continue;
		}

		FString Emitter_InterpSpawnStartDt = BuildSanitizedSymbolName(ActiveHistoryForFunctionCalls.ResolveAliases(SYS_PARAM_EMITTER_INTERP_SPAWN_START_DT).GetName());
		Emitter_InterpSpawnStartDt = Emitter_InterpSpawnStartDt.Replace(TEXT("."), TEXT("_"));//TODO: This should be rolled into GetSanitisedSymbolName but currently some usages require the '.' intact. Fix those up!.
		FString Emitter_SpawnInterval = BuildSanitizedSymbolName(ActiveHistoryForFunctionCalls.ResolveAliases(SYS_PARAM_EMITTER_SPAWN_INTERVAL).GetName());
		Emitter_SpawnInterval = Emitter_SpawnInterval.Replace(TEXT("."), TEXT("_"));//TODO: This should be rolled into GetSanitisedSymbolName but currently some usages require the '.' intact. Fix those up!.

		HlslOutputString += TEXT("void InterpolateParameters(inout FSimulationContext Context)\n{\n");

		FString PrevMap = TranslationStages[i - 1].PassNamespace;
		FString CurMap = TranslationStages[i].PassNamespace;
		{
			FExpressionPermutationContext PermutationContext(*this, TranslationStages[i], HlslOutputString);

			if (TranslationStages[i].InterpolatedSpawnMode == ENiagaraInterpolatedSpawnMode::RunUpdateScript)
			{
				HlslOutputString += TEXT("\tint InterpSpawn_Index = 0;\n");
				HlslOutputString += TEXT("\tfloat InterpSpawn_SpawnTime = Engine_DeltaTime - ") + Emitter_InterpSpawnStartDt + TEXT(";\n");
			}
			else if (TranslationStages[i].InterpolatedSpawnMode == ENiagaraInterpolatedSpawnMode::Interpolation)
			{
				HlslOutputString += TEXT("\tint InterpSpawn_Index = ExecIndex();\n");
				HlslOutputString += TEXT("\tfloat InterpSpawn_SpawnTime = ") + Emitter_InterpSpawnStartDt + TEXT(" + (") + Emitter_SpawnInterval + TEXT(" * InterpSpawn_Index);\n");
			}
			else
			{
				Error(FText::Format(LOCTEXT("UnsupportedInterpSpawn", "Unknown interpolated spawn value {0}"), FText::AsNumber(static_cast<int32>(TranslationStages[i].InterpolatedSpawnMode))));
			}
			HlslOutputString += TEXT("\tfloat InterpSpawn_UpdateTime = Engine_DeltaTime - InterpSpawn_SpawnTime;\n");
			HlslOutputString += TEXT("\tfloat InterpSpawn_InvSpawnTime = 1.0 / InterpSpawn_SpawnTime;\n");
			HlslOutputString += TEXT("\tfloat InterpSpawn_InvUpdateTime = 1.0 / InterpSpawn_UpdateTime;\n");
			HlslOutputString += TEXT("\tfloat SpawnInterp = InterpSpawn_SpawnTime * Engine_InverseDeltaTime ;\n");
			HlslOutputString += TEXT("\tHackSpawnInterp = SpawnInterp;\n");

			HlslOutputString += TEXT("\tContext.") + PrevMap + TEXT(".Interpolation.InterpSpawn_Index = InterpSpawn_Index;\n");
			HlslOutputString += TEXT("\tContext.") + PrevMap + TEXT(".Interpolation.InterpSpawn_SpawnTime = InterpSpawn_SpawnTime;\n");
			HlslOutputString += TEXT("\tContext.") + PrevMap + TEXT(".Interpolation.InterpSpawn_UpdateTime = InterpSpawn_UpdateTime;\n");
			HlslOutputString += TEXT("\tContext.") + PrevMap + TEXT(".Interpolation.InterpSpawn_InvSpawnTime = InterpSpawn_InvSpawnTime;\n");
			HlslOutputString += TEXT("\tContext.") + PrevMap + TEXT(".Interpolation.InterpSpawn_InvUpdateTime = InterpSpawn_InvUpdateTime;\n");
			HlslOutputString += TEXT("\tContext.") + PrevMap + TEXT(".Interpolation.SpawnInterp = SpawnInterp;\n");
			HlslOutputString += TEXT("\tContext.") + PrevMap + TEXT(".Interpolation.Emitter_SpawnInterval = Emitter_SpawnInterval;\n");
			HlslOutputString += TEXT("\tContext.") + PrevMap + TEXT(".Interpolation.Emitter_InterpSpawnStartDt = Emitter_InterpSpawnStartDt;\n");
			HlslOutputString += TEXT("\tContext.") + PrevMap + TEXT(".Interpolation.Emitter_SpawnGroup = Emitter_SpawnGroup;\n");

			int32 ModesToInterpolate[] =
			{
				static_cast<int32>(ENiagaraCodeChunkMode::GlobalConstant),
				static_cast<int32>(ENiagaraCodeChunkMode::SystemConstant),
				static_cast<int32>(ENiagaraCodeChunkMode::OwnerConstant),
				static_cast<int32>(ENiagaraCodeChunkMode::EmitterConstant),
				static_cast<int32>(ENiagaraCodeChunkMode::Uniform)
			};

			for (int32 ChunkMode : ModesToInterpolate)
			{
				for (int32 UniformIdx = 0; UniformIdx < ChunksByMode[ChunkMode].Num(); ++UniformIdx)
				{
					int32 ChunkIdx = ChunksByMode[ChunkMode][UniformIdx];
					if (ChunkIdx != INDEX_NONE)
					{
						const FNiagaraVariable* FoundNamespacedVar = nullptr;

						for (const auto& SystemVarPair : ParamMapDefinedSystemVars)
						{
							if (SystemVarPair.Value.ChunkIndex == ChunkIdx)
							{
								FoundNamespacedVar = &SystemVarPair.Value.Variable;
								break;
							}
						}

						if (FoundNamespacedVar != nullptr)
						{
							FNiagaraCodeChunk& Chunk = CodeChunks[ChunkIdx];
							if (ShouldInterpolateParameter(*FoundNamespacedVar))
							{
								
								HlslOutputString += GetInterpolateHlsl(*FoundNamespacedVar, PrevMap, Chunk);
							}
							else
							{
								// For now, we do nothing for non-floating point variables..
							}
						}
					}
				}
			}
			HlslOutputString += TEXT("\tContext.") + CurMap + TEXT(".Engine.DeltaTime = InterpSpawn_UpdateTime;\n");
			HlslOutputString += TEXT("\tContext.") + CurMap + TEXT(".Engine.InverseDeltaTime = InterpSpawn_InvUpdateTime;\n");
		}

		HlslOutputString += TEXT("}\n\n");
	}
}

void FNiagaraHlslTranslator::DefineDataSetReadFunction(FString &HlslOutputString, TArray<FNiagaraDataSetID> &ReadDataSets)
{
	if (UNiagaraScript::IsParticleEventScript(CompileOptions.TargetUsage) && CompilationTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		HlslOutputString += TEXT("void ReadDataSets(inout FSimulationContext Context, int SetInstanceIndex)\n{\n");
	}
	else
	{
		HlslOutputString += TEXT("void ReadDataSets(inout FSimulationContext Context)\n{\n");
	}

	// We shouldn't read anything in a Spawn Script!
	if (UNiagaraScript::IsParticleSpawnScript(CompileOptions.TargetUsage) || UNiagaraScript::IsGPUScript(CompileOptions.TargetUsage))
	{
		HlslOutputString += TEXT("}\n\n");
		return;
	}

	for (TPair<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>>& DataSetInfoPair : DataSetReadInfo[0])
	{
		FNiagaraDataSetID DataSet = DataSetInfoPair.Key;
		int32 OffsetCounterInt = 0;
		int32 OffsetCounterFloat = 0;
		int32 OffsetCounterHalf = 0;
		int32 DataSetIndex = 1;
		for (TPair<int32, FDataSetAccessInfo>& IndexInfoPair : DataSetInfoPair.Value)
		{
			FString Symbol = FString("\tContext.") + DataSet.Name.ToString() + "Read.";  // TODO: HACK - need to get the real symbol name here
			FString SetIdx = FString::FromInt(DataSetIndex);
			FString DataSetComponentBufferSize = "DSComponentBufferSizeRead{1}" + SetIdx;
			if (CompilationTarget == ENiagaraSimTarget::GPUComputeSim)
			{
				for (FNiagaraVariable &Var : IndexInfoPair.Value.Variables)
				{
					// TODO: temp = should really generate output functions for each set
					FString Fmt = Symbol + Var.GetName().ToString() + FString(TEXT("{0} = ReadDataSet{1}")) + SetIdx + TEXT("[{2}*") + DataSetComponentBufferSize + " + SetInstanceIndex];\n";
					GatherVariableForDataSetAccess(Var, Fmt, OffsetCounterInt, OffsetCounterFloat, OffsetCounterHalf, -1, TEXT(""), HlslOutputString);
				}
			}
			else
			{
				for (FNiagaraVariable &Var : IndexInfoPair.Value.Variables)
				{
					// TODO: currently always emitting a non-advancing read, needs to be changed for some of the use cases
					FString Fmt = TEXT("\tContext.") + DataSet.Name.ToString() + "Read." + Var.GetName().ToString() + TEXT("{0} = InputDataNoadvance{1}({2}, {3});\n");
					GatherVariableForDataSetAccess(Var, Fmt, OffsetCounterInt, OffsetCounterFloat, OffsetCounterHalf, DataSetIndex, TEXT(""), HlslOutputString);
				}
			}
		}
	}

	HlslOutputString += TEXT("}\n\n");
}


void FNiagaraHlslTranslator::DefineDataSetWriteFunction(FString &HlslOutputString, TArray<FNiagaraDataSetProperties> &WriteDataSets, TArray<int32>& WriteConditionVarIndices)
{
	HlslOutputString += TEXT("void WriteDataSets(inout FSimulationContext Context)\n{\n");

	int32 DataSetIndex = 1;
	for (TPair<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>>& DataSetInfoPair : DataSetWriteInfo[0])
	{
		FNiagaraDataSetID DataSet = DataSetInfoPair.Key;
		int32 OffsetCounter = 0;

		HlslOutputString += "\t{\n";
		HlslOutputString += "\tint TmpWriteIndex;\n";
		int32* ConditionalWriteIdxPtr = DataSetWriteConditionalInfo[0].Find(DataSet);
		if (nullptr == ConditionalWriteIdxPtr || INDEX_NONE == *ConditionalWriteIdxPtr)
		{
			HlslOutputString += "\tbool bValid = true;\n";
		}
		else
		{
			HlslOutputString += "\tbool bValid = " + FString("Context.") + DataSet.Name.ToString() + "Write_Valid;\n";
		}
		int32 WriteOffsetInt = 0;
		int32 WriteOffsetFloat = 0;
		int32 WriteOffsetHalf = 0;

		// grab the current ouput index; currently pass true, but should use an arbitrary bool to determine whether write should happen or not

		HlslOutputString += "\tTmpWriteIndex = AcquireIndex(";
		HlslOutputString += FString::FromInt(DataSetIndex);
		HlslOutputString += ", bValid);\n";

		HlslOutputString += CompilationTarget == ENiagaraSimTarget::GPUComputeSim ? "\tif(TmpWriteIndex>=0)\n\t{\n" : "";

		for (TPair<int32, FDataSetAccessInfo>& IndexInfoPair : DataSetInfoPair.Value)
		{
			FString Symbol = FString("Context.") + DataSet.Name.ToString() + "Write";  // TODO: HACK - need to get the real symbol name here
			if (CompilationTarget == ENiagaraSimTarget::GPUComputeSim)
			{
				FString SetIdx = FString::FromInt(DataSetIndex);
				FString DataSetComponentBufferSize = "DSComponentBufferSizeWrite{1}" + SetIdx;
				for (FNiagaraVariable &Var : IndexInfoPair.Value.Variables)
				{
					// TODO: temp = should really generate output functions for each set
					FString Fmt = FString(TEXT("\t\tRWWriteDataSet{1}")) + SetIdx + TEXT("[{2}*") + DataSetComponentBufferSize + TEXT(" + {3}] = ") + Symbol + TEXT(".") + Var.GetName().ToString() + TEXT("{0};\n");
					GatherVariableForDataSetAccess(Var, Fmt, WriteOffsetInt, WriteOffsetFloat, WriteOffsetHalf, -1, TEXT("TmpWriteIndex"), HlslOutputString);
				}
			}
			else
			{
				for (FNiagaraVariable &Var : IndexInfoPair.Value.Variables)
				{
					// TODO: data set index is always 1; need to increase each set
					FString Fmt = TEXT("\t\tOutputData{1}(") + FString::FromInt(DataSetIndex) + (", {2}, {3}, ") + Symbol + "." + Var.GetName().ToString() + TEXT("{0});\n");
					GatherVariableForDataSetAccess(Var, Fmt, WriteOffsetInt, WriteOffsetFloat, WriteOffsetHalf, -1, TEXT("TmpWriteIndex"), HlslOutputString);
				}
			}
		}

		HlslOutputString += CompilationTarget == ENiagaraSimTarget::GPUComputeSim ? "\t}\n" : "";
		DataSetIndex++;
		HlslOutputString += "\t}\n";
	}

	HlslOutput += TEXT("}\n\n");
}

void FNiagaraHlslTranslator::ConvertCompileInfoToParamInfo(const FNiagaraScriptDataInterfaceCompileInfo& Info, FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo, TArray<FNiagaraFunctionSignature>& GeneratedFunctionSignatures)
{
	FString OwnerIDString = Info.Name.ToString();
	FString SanitizedOwnerIDString = BuildSanitizedSymbolString(OwnerIDString.Replace(TEXT("."), TEXT("_")));

	DIInstanceInfo.DataInterfaceHLSLSymbol = SanitizedOwnerIDString;
	DIInstanceInfo.DIClassName = Info.Type.GetClass()->GetName();

	// Build a list of function instances that will be generated for this DI.
	bool bHasWriteFunctions = false;
	TSet<FNiagaraFunctionSignature> SeenFunctions;
	DIInstanceInfo.GeneratedFunctions.Reserve(Info.RegisteredFunctions.Num());
	GeneratedFunctionSignatures.Reserve(Info.RegisteredFunctions.Num());
	for (const FNiagaraFunctionSignature& OriginalSig : Info.RegisteredFunctions)
	{
		bool bAlreadySeen = false;
		SeenFunctions.FindOrAdd(OriginalSig, &bAlreadySeen);
		if (bAlreadySeen)
		{
			continue;
		}

		if (!OriginalSig.bSupportsGPU)
		{
			Error(FText::Format(LOCTEXT("GPUDataInterfaceFunctionNotSupported", "DataInterface {0} function {1} cannot run on the GPU."), FText::FromName(Info.Type.GetFName()), FText::FromName(OriginalSig.Name)));
			continue;
		}
		if (OriginalSig.bWriteFunction)
		{
			bHasWriteFunctions = true;
		}

		// make a copy so we can modify the owner id and get the correct hlsl signature
		FNiagaraFunctionSignature Sig = OriginalSig;
		Sig.OwnerName = Info.Name;

		FNiagaraDataInterfaceGeneratedFunction& DIFunc = DIInstanceInfo.GeneratedFunctions.AddDefaulted_GetRef();
		DIFunc.DefinitionName = Sig.Name;
		DIFunc.InstanceName = BuildFunctionSignatureString(Sig);
		DIFunc.MiscUsageBitMask = Sig.MiscUsageBitMask;
		DIFunc.Specifiers.Empty(Sig.FunctionSpecifiers.Num());
		for (const TTuple<FName, FName>& Specifier : Sig.FunctionSpecifiers)
		{
			DIFunc.Specifiers.Add(Specifier);
		}

		auto AddVarsToGPUDIFuncInfo = [](const TArray<FNiagaraVariableBase>& InVars, TArray<FNiagaraVariableCommonReference>& OutVarRefs)
		{
			OutVarRefs.Reset(InVars.Num());
			for(const FNiagaraVariableBase& Var : InVars)
			{
				auto& NewVarRef = OutVarRefs.AddDefaulted_GetRef();
				NewVarRef.Name = Var.GetName();
				NewVarRef.UnderlyingType = Var.GetType().ClassStructOrEnum;
			}
		};

		TArray<FNiagaraVariableBase> TempVars;
		//Write out our variadic parameters to allow proper binding for VM external functions.
		Sig.GetVariadicInputs(TempVars);
		AddVarsToGPUDIFuncInfo(TempVars, DIFunc.VariadicInputs);
		Sig.GetVariadicOutputs(TempVars);
		AddVarsToGPUDIFuncInfo(TempVars, DIFunc.VariadicOutputs);

		//Also output the actual signature to help with some hlsl generation.
		GeneratedFunctionSignatures.Add(Sig);
	}
}

void FNiagaraHlslTranslator::DefineDataInterfaceHLSL(FString& InHlslOutput)
{
	FString InterfaceCommonHLSL;
	FString InterfaceUniformHLSL;
	FString InterfaceFunctionHLSL;
	TSet<FName> InterfaceClasses;

	for (int32 i = 0; i < CompilationOutput.ScriptData.DataInterfaceInfo.Num(); i++)
	{
		FNiagaraScriptDataInterfaceCompileInfo& Info = CompilationOutput.ScriptData.DataInterfaceInfo[i];

		UNiagaraDataInterface* CDO = GetDataInterfaceCDO(Info.Type.GetClass());
		check(CDO != nullptr);
		if (CDO && CDO->CanExecuteOnTarget(ENiagaraSimTarget::GPUComputeSim))
		{
			if ( !InterfaceClasses.Contains(Info.Type.GetFName()) )
			{
				CDO->GetCommonHLSL(InterfaceCommonHLSL);
				InterfaceClasses.Add(Info.Type.GetFName());
			}

			TArray<FNiagaraFunctionSignature> GeneratedFunctionSignatures;
			FNiagaraDataInterfaceGPUParamInfo& DIInstanceInfo = ShaderScriptParametersMetadata.DataInterfaceParamInfo.AddDefaulted_GetRef();
			ConvertCompileInfoToParamInfo(Info, DIInstanceInfo,GeneratedFunctionSignatures);

			FNiagaraDataInterfaceHlslGenerationContext DIHlslGenContext(DIInstanceInfo, GeneratedFunctionSignatures);
			DIHlslGenContext.GetSanitizedFunctionParametersDelegate.BindStatic(FNiagaraHlslTranslator::GenerateFunctionParametersHlsl);
			DIHlslGenContext.GetFunctionSignatureSymbolDelegate.BindStatic(&FNiagaraHlslTranslator::GetFunctionSignatureSymbol);
			DIHlslGenContext.GetStructHlslTypeNameDelegate.BindStatic(&FNiagaraHlslTranslator::GetStructHlslTypeName);
			DIHlslGenContext.GetPropertyHlslTypeNameDelegate.BindStatic(&FNiagaraHlslTranslator::GetPropertyHlslTypeName);
			DIHlslGenContext.GetSanitizedSymbolNameDelegate.BindStatic(&FNiagaraHlslTranslator::GetSanitizedSymbolName);
			DIHlslGenContext.GetHlslDefaultForTypeDelegate.BindStatic(&FNiagaraHlslTranslator::GetHlslDefaultForType);
			CDO->GetParameterDefinitionHLSL(DIHlslGenContext, InterfaceUniformHLSL);

			// Ask the DI to generate HLSL.
			TArray<FNiagaraDataInterfaceGeneratedFunction> PreviousHits;
			for (int FunctionInstanceIndex = 0; FunctionInstanceIndex < DIInstanceInfo.GeneratedFunctions.Num(); ++FunctionInstanceIndex)
			{
				const FNiagaraDataInterfaceGeneratedFunction& DIFunc = DIInstanceInfo.GeneratedFunctions[FunctionInstanceIndex];
				ensure(!PreviousHits.Contains(DIFunc));

				DIHlslGenContext.FunctionInstanceIndex = FunctionInstanceIndex;
				const bool HlslOK = CDO->GetFunctionHLSL(DIHlslGenContext, InterfaceFunctionHLSL);
				if (!HlslOK)
				{
					Error(FText::Format(LOCTEXT("GPUDataInterfaceFunctionNotImplemented", "DataInterface {0} function {1} is not implemented for GPU."), FText::FromName(Info.Type.GetFName()), FText::FromName(DIFunc.DefinitionName)));
				}
				else
				{
					PreviousHits.Add(DIFunc);
				}
			}
		}
		else
		{
			Error(FText::Format(LOCTEXT("NonGPUDataInterfaceError", "DataInterface {0} ({1}) cannot run on the GPU."), FText::FromName(Info.Name), FText::FromString(CDO ? CDO->GetClass()->GetName() : TEXT(""))));
		}
	}
	InHlslOutput += InterfaceCommonHLSL + InterfaceUniformHLSL + InterfaceFunctionHLSL;
}

void FNiagaraHlslTranslator::DefineExternalFunctionsHLSL(FString &InHlslOutput)
{
	for (FNiagaraFunctionSignature& FunctionSig : CompilationOutput.ScriptData.AdditionalExternalFunctions )
	{
		if ( UNiagaraFunctionLibrary::DefineFunctionHLSL(FunctionSig, InHlslOutput) == false )
		{
			Error(FText::Format(LOCTEXT("ExternFunctionMissingHLSL", "ExternalFunction {0} does not have a HLSL implementation for the GPU."), FText::FromName(FunctionSig.Name)));
		}
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::DefineMainGPUFunctions(
	const TArray<TArray<FNiagaraVariable>>& DataSetVariables,
	const TMap<FNiagaraDataSetID, int32>& DataSetReads,
	const TMap<FNiagaraDataSetID, int32>& DataSetWrites)
{
	TArray<FNiagaraDataSetID> ReadDataSetIDs;
	TArray<FNiagaraDataSetID> WriteDataSetIDs;
	
	DataSetReads.GetKeys(ReadDataSetIDs);
	DataSetWrites.GetKeys(WriteDataSetIDs);

	FNiagaraEmitterID EmitterID = GetEmitterID();

	// Whether Alive is used and must be set at each run
	bool bUsesAlive = false;
	TArray<FName> DataSetNames;
	for (const FNiagaraDataSetID& ReadId : ReadDataSetIDs)
	{
		DataSetNames.AddUnique(ReadId.Name);
	}
	for (const FNiagaraDataSetID& WriteId : WriteDataSetIDs)
	{
		DataSetNames.AddUnique(WriteId.Name);
	}
	for (int32 i = 0; i < ParamMapHistories.Num(); i++)
	{
		for (const FName& DataSetName : DataSetNames)
		{
			if (ParamMapHistories[i].FindVariable(*(DataSetName.ToString() + TEXT(".Alive")), FNiagaraTypeDefinition::GetBoolDef()) != INDEX_NONE)
			{
				bUsesAlive = true;
				TranslationStages[i].bUsesAlive = true;
				break;
			}
		}
	}

	const bool bRequiresPersistentIDs = CompileOptions.AdditionalDefines.Contains(TEXT("RequiresPersistentIDs"));

	// A list of constant to reset after Emitter_SpawnGroup gets modified by GetEmitterSpawnInfoForParticle()
	TArray<FString> EmitterSpawnGroupReinit;

	///////////////////////
	// InitConstants()
	HlslOutput += TEXT("void InitConstants(inout FSimulationContext Context)\n{\n");
	{
		// Fill in the defaults for parameters.
		const int32 StageCount = PerStageMainPreSimulateChunks.Num();
		for (int32 StageIdx = 0; StageIdx < StageCount; ++StageIdx)
		{
			const TArray<FString>& MainPreSimulateChunks = PerStageMainPreSimulateChunks[StageIdx];

			if (MainPreSimulateChunks.Num())
			{
				FExpressionPermutationContext PermutationContext(*this, TranslationStages[StageIdx], HlslOutput);

				for (const FString& InitChunk : MainPreSimulateChunks)
				{
					HlslOutput += TEXT("\t") + InitChunk + TEXT("\n");

					if (InitChunk.Contains(TEXT("Emitter_SpawnGroup;")))
					{
						EmitterSpawnGroupReinit.Add(InitChunk);
					}
				}
			}
		}
	}
	HlslOutput += TEXT("}\n\n");

	///////////////////////
	// InitSpawnVariables()
	HlslOutput += TEXT("void InitSpawnVariables(inout FSimulationContext Context)\n{\n");
	{
		FExpressionPermutationContext PermutationContext(HlslOutput);
		
		if (TranslationStages.Num() > 1)
		{
			PermutationContext.AddBranch(*this, TranslationStages[0]);
		}

		// Reset constant that have been modified by GetEmitterSpawnInfoForParticle()
		if (EmitterSpawnGroupReinit.Num())
		{
			for (const FString& ReinitChunk : EmitterSpawnGroupReinit)
			{
				HlslOutput += TEXT("\t") + ReinitChunk + TEXT("\n");
			}
			HlslOutput += TEXT("\n");
		}

		FString ContextName = TEXT("\tContext.Map.");
		if (TranslationStages.Num() > 1) // First context 0 is "MapSpawn"
		{
			ContextName = FString::Printf(TEXT("\tContext.%s."), *TranslationStages[0].PassNamespace);
		}

		//The VM register binding assumes the same inputs as outputs which is obviously not always the case.
		for (int32 DataSetIndex = 0, IntCounter = 0, FloatCounter = 0, HalfCounter = 0; DataSetIndex < DataSetReads.Num(); ++DataSetIndex)
		{
			const FNiagaraDataSetID DataSetID = ReadDataSetIDs[DataSetIndex];
			const TArray<FNiagaraVariable>& NiagaraVariables = DataSetVariables[DataSetReads[DataSetID]];
			for (const FNiagaraVariable& Var : NiagaraVariables)
			{
				FString VarFmt = ContextName + BuildSanitizedSymbolName(Var.GetName()) + TEXT("{0} = {4};\n");
				GatherVariableForDataSetAccess(Var, VarFmt, IntCounter, FloatCounter, HalfCounter, DataSetIndex, TEXT(""), HlslOutput);
			}
		}

		if (bUsesAlive)
		{
			HlslOutput += TEXT("\n") + ContextName + TEXT("DataInstance.Alive=true;\n");
		}

		if (bRequiresPersistentIDs)
		{
			HlslOutput += TEXT("\n\tint IDIndex, IDAcquireTag;\n\tAcquireID(0, IDIndex, IDAcquireTag);\n");
			HlslOutput += ContextName + TEXT("Particles.ID.Index = IDIndex;\n");
			HlslOutput += ContextName + TEXT("Particles.ID.AcquireTag = IDAcquireTag;\n");
		}

		HlslOutput += FString::Printf(TEXT("\t%sEngine.Emitter.ID.ID = %d;\n"), *ContextName, EmitterID.ID);
	}
	HlslOutput += TEXT("}\n\n");

	////////////////////////
	// LoadUpdateVariables()
	HlslOutput += TEXT("void LoadUpdateVariables(inout FSimulationContext Context, int InstanceIdx)\n{\n");
	{
		FExpressionPermutationContext PermutationContext(HlslOutput);
		int32 StartIdx = 1;
		for (int32 i = StartIdx; i < TranslationStages.Num(); i++)
		{
			const FName IterationDataInterface = TranslationStages[i].GetIterationDataInterface();

			// No need to load particle data for stages with an iteration source, since those do not run one thread per particle.
			if (IterationDataInterface != NAME_None)
			{
				if (TranslationStages[i].CustomReadFunction.IsEmpty())
				{
					continue;
				}
			}

			PermutationContext.AddBranch(*this, TranslationStages[i]);

			if (IterationDataInterface != NAME_None)
			{
				if (!TranslationStages[i].CustomReadFunction.IsEmpty())
				{
					HlslOutput += TranslationStages[i].CustomReadFunction + TEXT("(Context);\n\n");
					continue;
				}
			}

			FString ContextName = TEXT("\t\tContext.Map.");
			if (TranslationStages.Num() > 1) // Second context is "MapUpdate"
			{
				ContextName = FString::Printf(TEXT("\t\tContext.%s."), *TranslationStages[i].PassNamespace);
			}

			TArray<FNiagaraVariable> GatheredPreviousVariables;

			for (int32 DataSetIndex = 0, IntCounter = 0, FloatCounter = 0, HalfCounter = 0; DataSetIndex < DataSetReads.Num(); ++DataSetIndex)
			{
				const FNiagaraDataSetID DataSetID = ReadDataSetIDs[DataSetIndex];
				const TArray<FNiagaraVariable>& NiagaraVariables = DataSetVariables[DataSetReads[DataSetID]];
				for (const FNiagaraVariable& Var : NiagaraVariables)
				{
					const FString VarName = ContextName + BuildSanitizedSymbolName(Var.GetName());
					FString VarFmt;
					bool bWrite = true;

					// If the NiagaraClearEachFrame value is set on the data set, we don't bother reading it in each frame as we know that it is is invalid. However,
					// this is only used for the base data set. Other reads are potentially from events and are therefore perfectly valid.
					if (DataSetIndex == 0 && Var.GetType().GetScriptStruct() != nullptr && Var.GetType().GetScriptStruct()->GetMetaData(TEXT("NiagaraClearEachFrame")).Equals(TEXT("true"), ESearchCase::IgnoreCase))
					{
						VarFmt = VarName + TEXT("{0} = {4};\n");
					}
					else if (DataSetIndex == 0 && FNiagaraParameterUtilities::IsPreviousValue(Var) && TranslationStages[i].ScriptUsage == ENiagaraScriptUsage::ParticleUpdateScript)
					{
						GatheredPreviousVariables.AddUnique(Var);
						bWrite = false; // We need to bump the read indices forwards, but not actually add the read.
					}
					else
					{
						VarFmt = VarName + TEXT("{0} = InputData{1}({2}, {3}, InstanceIdx);\n");

						if (FNiagaraParameterUtilities::IsAttribute(Var))
						{
							FString RegisterName = VarName;
							ReplaceNamespaceInline(RegisterName, PARAM_MAP_ATTRIBUTE_STR, PARAM_MAP_INDICES_STR);

							const int32 RegisterValue = Var.GetType().IsFloatPrimitive() ? FloatCounter : IntCounter;
							HlslOutput += RegisterName + FString::Printf(TEXT(" = %d;\n"), RegisterValue);
						}
					}
					GatherVariableForDataSetAccess(Var, VarFmt, IntCounter, FloatCounter, HalfCounter, DataSetIndex, TEXT(""), HlslOutput, bWrite);
				}
			}

			// Put any gathered previous variables into the list here so that we can use them by recording the last value from the parent variable on load.
			for (FNiagaraVariable VarPrevious : GatheredPreviousVariables)
			{
				FNiagaraVariable SrcVar = FNiagaraParameterUtilities::GetSourceForPreviousValue(VarPrevious);
				const FString VarName = ContextName + BuildSanitizedSymbolName(SrcVar.GetName());
				const FString VarPrevName = ContextName + BuildSanitizedSymbolName(VarPrevious.GetName());
				HlslOutput += VarPrevName + TEXT(" = ") + VarName + TEXT(";\n");
			}

			if (bUsesAlive)
			{
				HlslOutput += ContextName + TEXT("DataInstance.Alive=true;\n");
			}

			HlslOutput += FString::Printf(TEXT("\t%sEngine.Emitter.ID.ID = %d;\n"), *ContextName, EmitterID.ID);
		}
	}
	HlslOutput += TEXT("}\n\n");

	/////////////////////////////////////
	// ConditionalInterpolateParameters()
	HlslOutput += TEXT("void ConditionalInterpolateParameters(inout FSimulationContext Context)\n{\n");
	{
		if (RequiresInterpolation())
		{
			HlslOutput += TEXT("\tInterpolateParameters(Context);\n"); // Requires ExecIndex, which needs to be in a stage.
		}
	}
	HlslOutput += TEXT("}\n\n");

	//////////////////////
	// TransferAttibutes()
	HlslOutput += TEXT("void TransferAttributes(inout FSimulationContext Context)\n{\n");
	{
		FExpressionPermutationContext PermutationContext(HlslOutput);

		int32 StartIdx = 1;
		for (int32 i = StartIdx; i < TranslationStages.Num(); i++)
		{
			PermutationContext.AddBranch(*this, TranslationStages[i]);

			if (TranslationStages[i].bCopyPreviousParams)
			{
				if (ParamMapDefinedAttributesToNamespaceVars.Num() != 0)
				{
					HlslOutput += TEXT("\t\tContext.") + TranslationStages[i].PassNamespace + TEXT(".Particles = Context.") + TranslationStages[i - 1].PassNamespace + TEXT(".Particles;\n");
					if (TranslationStages[i - 1].bWritesAlive)
					{
						HlslOutput += TEXT("\t\tContext.") + TranslationStages[i].PassNamespace + TEXT(".DataInstance = Context.") + TranslationStages[i - 1].PassNamespace + TEXT(".DataInstance;\n");
					}
					else if ( TranslationStages[i].bWritesAlive )
					{
						HlslOutput += TEXT("\t\tContext.") + TranslationStages[i].PassNamespace + TEXT(".DataInstance.Alive = true;\n");
					}
				}
				
				if (i == 1 && TranslationStages[i].ScriptUsage == ENiagaraScriptUsage::ParticleUpdateScript) // The Update Phase might need previous parameters set.
				{
					// Put any gathered previous variables into the list here so that we can use them by recording the last value from the parent variable on transfer from previous stage if interpolated spawning.
					TArray<FVarAndDefaultSource> VarAndDefaultSourceArray;
					TArray<FNiagaraVariable> GatheredPreviousVariables;
					ParamMapDefinedAttributesToNamespaceVars.GenerateValueArray(VarAndDefaultSourceArray);

					for (const FVarAndDefaultSource& VarAndDefaultSource : VarAndDefaultSourceArray)
					{
						const FNiagaraVariable& Var = VarAndDefaultSource.Variable;
						if (FNiagaraParameterUtilities::IsPreviousValue(Var))
						{
							FNiagaraVariable SrcVar = FNiagaraParameterUtilities::GetSourceForPreviousValue(Var);
							const FString VarName = BuildSanitizedSymbolName(SrcVar.GetName());
							const FString VarPrevName = BuildSanitizedSymbolName(Var.GetName());
							HlslOutput += TEXT("\t\tContext.") + TranslationStages[i].PassNamespace + TEXT(".") + VarPrevName + TEXT(" = Context.") + TranslationStages[i-1].PassNamespace + TEXT(".") + VarName + TEXT(";\n");
						}
					}
				}
			}
		
		}
	}
	HlslOutput += TEXT("}\n\n");
	
	/////////////////////////
	// StoreUpdateVariables()
	HlslOutput += TEXT("void StoreUpdateVariables(in FSimulationContext Context, bool bIsValidInstance)\n{\n");
	{
		FExpressionPermutationContext PermutationContext(HlslOutput);

		int32 StartIdx = 1;
		for (int32 i = StartIdx; i < TranslationStages.Num(); i++)
		{
			const FName IterationDataInterface = TranslationStages[i].GetIterationDataInterface();

			// No need to store particle data for stages with an iteration source, since those do not run one thread per particle.
			if (IterationDataInterface != NAME_None)
			{
				if (TranslationStages[i].CustomWriteFunction.IsEmpty())
				{
					continue;
				}
			}
			// If we do not write particle data or kill particles we can avoid the write altogether which will allow us to also cull attribute reads to the ones that are only 'required'
			else if (!TranslationStages[i].bWritesParticles)
			{
				ensure(TranslationStages[i].bWritesAlive == false);
				continue;
			}

			PermutationContext.AddBranch(*this, TranslationStages[i]);

			if (IterationDataInterface != NAME_None)
			{
				if (!TranslationStages[i].CustomWriteFunction.IsEmpty())
				{
					HlslOutput += TEXT("if ( bIsValidInstance )\n");
					HlslOutput += TEXT("{\n");
					HlslOutput += TEXT("\t") + TranslationStages[i].CustomWriteFunction + TEXT("(Context);\n\n");
					HlslOutput += TEXT("}\n");
					continue;
				}
			}

			bool bWriteInstanceCount = !TranslationStages[i].bPartialParticleUpdate;
			if (TranslationStages[i].bWritesAlive || (i == 1 && TranslationStages[0].bWritesAlive))
			{
				// This stage kills particles, so we must skip the dead ones when writing out the data.
				// It's also possible that this is the update phase, and it doesn't kill particles, but the spawn phase does. It would be nice
				// if we could only do this for newly spawned particles, but unfortunately that would mean placing thread sync operations
				// under dynamic flow control, which is not allowed. Therefore, we must always use the more expensive path when the spawn phase
				// can kill particles.
				bWriteInstanceCount = false;
				HlslOutput += TEXT("\t\tconst bool bValid = bIsValidInstance && Context.") + TranslationStages[i].PassNamespace + TEXT(".DataInstance.Alive;\n");
				HlslOutput += TEXT("\t\tconst int WriteIndex = OutputIndex(0, true, bValid);\n");
			}
			else
			{
				// The stage doesn't kill particles, we can take the simpler path which doesn't need to manage the particle count.
				HlslOutput += TEXT("\t\tconst bool bValid = bIsValidInstance;\n");
				HlslOutput += TEXT("\t\tconst int WriteIndex = OutputIndex(0, false, bValid);\n");
			}

			FString ContextName = TEXT("Context.Map.");
			if (TranslationStages.Num() > 1) // Last context is "MapUpdate"
			{
				ContextName = FString::Printf(TEXT("Context.%s."), *TranslationStages[i].PassNamespace);
			}

			if (bRequiresPersistentIDs && !TranslationStages[i].bPartialParticleUpdate)
			{
				HlslOutput += FString::Printf(TEXT("\t\tUpdateID(0, bValid ? %sParticles.ID.Index : -1, WriteIndex);\n"), *ContextName);
			}

			HlslOutput += TEXT("\t\tif (bValid)\n\t\t{\n");

			for (int32 DataSetIndex = 0, IntCounter = 0, FloatCounter = 0, HalfCounter = 0; DataSetIndex < DataSetWrites.Num(); ++DataSetIndex)
			{
				const FNiagaraDataSetID DataSetID = ReadDataSetIDs[DataSetIndex];
				const TArray<FNiagaraVariable>& NiagaraVariables = DataSetVariables[DataSetWrites[DataSetID]];
				for (const FNiagaraVariable& Var : NiagaraVariables)
				{
					const bool bWriteToHLSL = !TranslationStages[i].bPartialParticleUpdate || TranslationStages[i].SetParticleAttributes.Contains(Var);

					// If coming from a parameter map, use the one on the context, otherwise use the output.
					FString VarFmt = TEXT("\t\t\tOutputData{1}(0, {2}, {3}, ") + ContextName + BuildSanitizedSymbolName(Var.GetName()) + TEXT("{0});\n");
					GatherVariableForDataSetAccess(Var, VarFmt, IntCounter, FloatCounter, HalfCounter, -1, TEXT("WriteIndex"), HlslOutput, bWriteToHLSL);
				}
			}

			HlslOutput += TEXT("\t\t}\n");

			if (bWriteInstanceCount)
			{
				//-TODO: This path should be deprecated if we ever remove the ability to disable partial writes
				HlslOutput += TEXT(
					"\t\t// If a stage doesn't kill particles, StoreUpdateVariables() never calls AcquireIndex(), so the\n"
					"\t\t// count isn't updated. In that case we must manually copy the original count here.\n"
					"\t\tif (WriteInstanceCountOffset != 0xFFFFFFFF && GLinearThreadId == 0) \n"
					"\t\t{\n"
					"\t\t	RWInstanceCounts[WriteInstanceCountOffset] = GSpawnStartInstance + NumSpawnedInstances; \n"
					"\t\t}\n"
				);
			}
		}
	}
	HlslOutput += TEXT("\n}\n\n");

	/////////////////////////
	// CopyInstance()
	HlslOutput += TEXT("void CopyInstance(in int InstanceIdx)\n{\n");
	{
#if 0
		HlslOutput += TEXT("\tFSimulationContext Context = (FSimulationContext)0;\n");
		if (UNiagaraScript::IsParticleEventScript(CompileOptions.TargetUsage))
		{
			for (int32 VarArrayIdx = 0; VarArrayIdx < DataSetReads.Num(); VarArrayIdx++)
			{
				const FNiagaraDataSetID DataSetID = ReadDataSetIDs[VarArrayIdx];
				const TArray<FNiagaraVariable>& ArrayRef = DataSetVariables[DataSetReads[DataSetID]];
				DefineDataSetVariableReads(HlslOutput, DataSetID, VarArrayIdx, ArrayRef);
			}

			if (bGpuUsesAlive)
			{
				HlslOutput += TEXT("\tContext.Map.DataInstance.Alive = true;\n");
			}

			for (int32 VarArrayIdx = 0; VarArrayIdx < DataSetWrites.Num(); VarArrayIdx++)
			{
				const FNiagaraDataSetID DataSetID = WriteDataSetIDs[VarArrayIdx];
				const TArray<FNiagaraVariable>& ArrayRef = DataSetVariables[DataSetWrites[DataSetID]];
				DefineDataSetVariableWrites(HlslOutput, DataSetID, VarArrayIdx, ArrayRef);
			}
		}
#else
		HlslOutput += TEXT("\t// TODO!\n");
#endif
	}
	HlslOutput += TEXT("}\n");

	//////////////////////////////////////////////////////////////////////////
	// Generate common main body
	HlslOutput += TEXT(
			"\n\n/*\n"
			"*	CS wrapper for our generated code; calls spawn and update functions on the corresponding instances in the buffer\n"
			" */\n"
			"\n"
			"[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, THREADGROUP_SIZE_Z)]\n"
			"void SimulateMainComputeCS(\n"
			"	uint3 InDispatchThreadId	: SV_DispatchThreadID,\n"
			"	uint3 InGroupId				: SV_GroupID,\n"
			"	uint3 InGroupThreadId		: SV_GroupThreadID,\n"
			"	uint  InGroupIndex			: SV_GroupIndex)\n"
			"{\n"
			"	GDispatchThreadId	= InDispatchThreadId;\n"
			"	GGroupId			= InGroupId;\n"
			"	GGroupThreadId		= InGroupThreadId;\n"
			"	GGroupIndex			= InGroupIndex;\n"
			"	GEmitterTickCounter = EmitterTickCounter;\n"
			"	GRandomSeedOffset = 0;\n"
		);

	//////////////////////////////////////////////////////////////////////////
	// Generate each translation stages body
	for (int32 i=1; i < TranslationStages.Num(); i++)
	{
		const FNiagaraHlslTranslationStage& TranslationStage = TranslationStages[i];
		const bool bInterpolatedSpawning = CompileOptions.AdditionalDefines.Contains(TEXT("InterpolatedSpawn")) || i != 1;
		const bool bAlwaysRunUpdateScript = CompileOptions.AdditionalDefines.Contains(TEXT("GpuAlwaysRunParticleUpdateScript"));
		const bool bParticleSpawnStage = i == 1;

		HlslOutput.Appendf(
			TEXT("%s SimulationStageIndex == %d // %s\n"),
			(i == 1) ? TEXT("#if") : TEXT("#elif"),
			TranslationStage.SimulationStageIndex,
			*TranslationStage.PassNamespace
		);

		// Indirect dispatches need to setup the bounds and translation into linear
		HlslOutput += TEXT(
			"#if NIAGARA_DISPATCH_INDIRECT\n"
			"	DispatchThreadIdBounds = IndirectDispatchArgs[IndirectDispatchArgsOffset].xyz;\n"
			"	GLinearThreadId = GDispatchThreadId.x;\n"
			"	#if NIAGARA_DISPATCH_TYPE >= NIAGARA_DISPATCH_TYPE_TWO_D\n"
			"		GLinearThreadId += GDispatchThreadId.y * DispatchThreadIdBounds.x;\n"
			"	#endif\n"
			"	#if NIAGARA_DISPATCH_TYPE >= NIAGARA_DISPATCH_TYPE_THREE_D\n"
			"		GLinearThreadId += GDispatchThreadId.z * DispatchThreadIdBounds.x * DispatchThreadIdBounds.y;\n"
			"	#endif\n"
			"#else //NIAGARA_DISPATCH_INDIRECT\n"
			"	GLinearThreadId		= GDispatchThreadId.x + (GDispatchThreadId.y * DispatchThreadIdToLinear.y);\n"
			"	#if NIAGARA_DISPATCH_TYPE >= NIAGARA_DISPATCH_TYPE_THREE_D\n"
			"		GLinearThreadId += GDispatchThreadId.z * DispatchThreadIdToLinear.z;\n"
			"	#endif\n"
			"#endif //NIAGARA_DISPATCH_INDIRECT\n"
			"\n"
		);

		// Particle iteration stage
		if (TranslationStage.IterationSourceType == ENiagaraIterationSource::Particles)
		{
			// Do we have particle state iteration enable
			HlslOutput.Append(TEXT("	bool bRunSpawnUpdateLogic = true;\n"));
			if (TranslationStage.bParticleIterationStateEnabled && TranslationStage.bWritesParticles)
			{
				if (TranslationStage.bPartialParticleUpdate == false)
				{
					FName StageName;
					if (CompilationOutput.ScriptData.SimulationStageMetaData.IsValidIndex(TranslationStage.SimulationStageIndex))
					{
						StageName = CompilationOutput.ScriptData.SimulationStageMetaData[TranslationStage.SimulationStageIndex].SimulationStageName;
					}

					Error(FText::Format(LOCTEXT("ParticleIterationState_Invalid", "Simulation stage '{0}' is incompatible with particle state iteration due to killing particles or disabling particle updates."), FText::FromName(StageName)), nullptr, nullptr);
				}
				HlslOutput += TEXT(
					"	if ( ParticleIterationStateInfo.x != -1 )\n"
					"	{\n"
					"		int ParticleStateValue = InputDataInt(0, uint(ParticleIterationStateInfo.x), GLinearThreadId);\n"
					"		bRunSpawnUpdateLogic = (ParticleStateValue >= ParticleIterationStateInfo.y) && (ParticleStateValue <= ParticleIterationStateInfo.z);\n"
					"	}\n"
				);
			}

			// We combine the update & spawn scripts together on GPU so we only need to check for spawning on the first translation stage
			// Note: Depending on how spawning inside stages works we may need to enable the spawn logic for those stages *only*			
			if (bParticleSpawnStage)
			{
				HlslOutput += TEXT(
					"	if (ReadInstanceCountOffset == 0xFFFFFFFF)\n"
					"	{\n"
					"		GSpawnStartInstance = 0;\n"
					"	}\n"
					"	else\n"
					"	{\n"
					"		GSpawnStartInstance = RWInstanceCounts[ReadInstanceCountOffset];\n"
					"	}\n"
					"	const uint MaxInstances = GSpawnStartInstance + NumSpawnedInstances;\n"
					"	const bool bRunUpdateLogic = bRunSpawnUpdateLogic && GLinearThreadId < GSpawnStartInstance && GLinearThreadId < MaxInstances;\n"
					"	const bool bRunSpawnLogic = bRunSpawnUpdateLogic && GLinearThreadId >= GSpawnStartInstance && GLinearThreadId < MaxInstances;\n"
				);
			}
			else
			{
				HlslOutput += TEXT(
					"	GSpawnStartInstance = RWInstanceCounts[ReadInstanceCountOffset];\n"
					"	const bool bRunUpdateLogic = bRunSpawnUpdateLogic && GLinearThreadId < GSpawnStartInstance;\n"
					"	const bool bRunSpawnLogic = false;\n"
				);
			}
		}
		// Iteration Data Interface
		else if (TranslationStage.IterationSourceType == ENiagaraIterationSource::DataInterface)
		{
			//-TODO: We can simplify the logic here with SimulationStage_GetInstanceCount() as only things that can provide an instance count offset
			//		 can really be variable, everything else is driven from CPU code.
			HlslOutput += TEXT(
				"	const uint MaxInstances = SimulationStage_GetInstanceCount();\n"
				"	GLinearThreadId = all(GDispatchThreadId < DispatchThreadIdBounds) ? GLinearThreadId : MaxInstances;\n"
				"	GSpawnStartInstance = MaxInstances;\n"
				"	const bool bRunUpdateLogic = (GLinearThreadId < GSpawnStartInstance) && (SimStart != 1);\n"
				"	const bool bRunSpawnLogic = (GLinearThreadId < GSpawnStartInstance) && (SimStart == 1);\n"
			);
		}
		// Iteration Direct Value
		else if (TranslationStage.IterationSourceType == ENiagaraIterationSource::DirectSet)
		{
			if (TranslationStage.DirectDispatchElementType == ENiagaraDirectDispatchElementType::NumThreads)
			{
				HlslOutput += TEXT(
					"	const uint MaxInstances = SimulationStage_GetInstanceCount();\n"
					"	const bool bValidInstance = all(GDispatchThreadId < DispatchThreadIdBounds);\n"
					"	const bool bRunUpdateLogic = bValidInstance;\n"
					"	const bool bRunSpawnLogic = false;\n"
					"	GLinearThreadId = bValidInstance ? GLinearThreadId : MaxInstances;\n"
					"	GSpawnStartInstance = MaxInstances;\n"
				);
			}
			else
			{
				HlslOutput += TEXT(
					"	const uint MaxInstances = SimulationStage_GetInstanceCount();\n"
					"	const bool bValidInstance = all(GDispatchThreadId < DispatchThreadIdBounds);\n"
					"	const bool bRunUpdateLogic = true;\n"
					"	const bool bRunSpawnLogic = false;\n"
					"	GLinearThreadId = bValidInstance ? GLinearThreadId : MaxInstances;\n"
					"	GSpawnStartInstance = MaxInstances;\n"
				);
			}
		}
		else
		{
			checkf(false, TEXT("Unsupported iteration source"));
		}

		HlslOutput += TEXT(
			"	\n"
			"	const float RandomSeedInitialisation = NiagaraInternalNoise(GLinearThreadId * 16384, 0 * 8196, (bRunUpdateLogic ? 4096 : 0) + EmitterTickCounter);	// initialise the random state seed\n"
			"	\n"
			"	FSimulationContext Context = (FSimulationContext)0;\n"
		);

		// Add Update Logic
		HlslOutput.Append(TEXT("	BRANCH\n"));
		HlslOutput.Append(TEXT("	if (bRunUpdateLogic)\n"));
		HlslOutput.Append(TEXT("	{\n"));
		HlslOutput.Append(TEXT("		SetupExecIndexForGPU();\n"));
		HlslOutput.Append(TEXT("		InitConstants(Context);\n"));
		HlslOutput.Append(TEXT("		LoadUpdateVariables(Context, GLinearThreadId);\n"));
		HlslOutput.Append(TEXT("		ReadDataSets(Context);\n"));
		if (bInterpolatedSpawning == false && bAlwaysRunUpdateScript == false)
		{
			HlslOutput.Appendf(TEXT("		Simulate%s(Context);\n"), *TranslationStages[i].PassNamespace);
			HlslOutput.Append(TEXT("		WriteDataSets(Context);\n"));
		}
		HlslOutput.Append(TEXT("	}\n"));

		// Add Spawn Logic
		HlslOutput.Append(TEXT("	else if (bRunSpawnLogic)\n"));
		HlslOutput.Append(TEXT("	{\n"));
		if (TranslationStage.IterationSourceType == ENiagaraIterationSource::Particles)
		{
			HlslOutput.Append(TEXT("		SetupExecIndexAndSpawnInfoForGPU();\n"));
		}
		else
		{
			HlslOutput.Append(TEXT("		SetupExecIndexForGPU();\n"));
		}
		HlslOutput.Append(TEXT("		InitConstants(Context);\n"));
		HlslOutput.Append(TEXT("		InitSpawnVariables(Context);\n"));
		HlslOutput.Append(TEXT("		ReadDataSets(Context);\n"));
		if (bParticleSpawnStage == true)
		{
			HlslOutput.Append(TEXT("		Context.MapSpawn.Particles.UniqueID = Engine_Emitter_TotalSpawnedParticles + GLinearThreadId - GSpawnStartInstance;\n"));
			HlslOutput.Append(TEXT("		ConditionalInterpolateParameters(Context);\n"));
			HlslOutput.Append(TEXT("		SimulateMapSpawn(Context);\n"));
		}
		HlslOutput.Append(TEXT("		TransferAttributes(Context);\n"));
		if (bInterpolatedSpawning == false && bAlwaysRunUpdateScript == false)
		{
			HlslOutput.Append(TEXT("		WriteDataSets(Context);\n"));
		}
		HlslOutput.Append(TEXT("	}\n\n"));

		// Interpolated spawning must also run the update logic if we have spawned
		if (bInterpolatedSpawning == true || bAlwaysRunUpdateScript == true)
		{
			HlslOutput.Append(TEXT("	if (bRunUpdateLogic || bRunSpawnLogic)\n"));
			HlslOutput.Append(TEXT("	{\n"));
			HlslOutput.Appendf(TEXT("		Simulate%s(Context);\n"), *TranslationStages[i].PassNamespace);
			HlslOutput.Append(TEXT("		WriteDataSets(Context);\n"));
			HlslOutput.Append(TEXT("	}\n\n"));
		}

		// Store Data
		HlslOutput.Append(TEXT("	StoreUpdateVariables(Context, bRunUpdateLogic || bRunSpawnLogic);\n\n"));
	}

	// End of logic
	HlslOutput.Append(TEXT("#endif\n"));
	HlslOutput.Append(TEXT("}\n"));
}

void FNiagaraHlslTranslator::DefineMain(FString &OutHlslOutput,
	const TArray<TArray<FNiagaraVariable>>& DataSetVariables,
	const TMap<FNiagaraDataSetID, int32>& DataSetReads,
	const TMap<FNiagaraDataSetID, int32>& DataSetWrites)
{
	check(CompilationTarget != ENiagaraSimTarget::GPUComputeSim);

	OutHlslOutput += TEXT("void SimulateMain()\n{\n");
	
	EnterStatsScope(FNiagaraStatScope(*(CompileOptions.GetName() + TEXT("_Main")), TEXT("Main")), OutHlslOutput);

	OutHlslOutput += TEXT("\n\tFSimulationContext Context = (FSimulationContext)0;\n");
	TMap<FName, int32> InputRegisterAllocations;
	TMap<FName, int32> OutputRegisterAllocations;

	ReadIdx = 0;
	WriteIdx = 0;

	//TODO: Grab indices for reading data sets and do the read.
	//read input.

	TArray<FNiagaraDataSetID> ReadDataSetIDs;
	TArray<FNiagaraDataSetID> WriteDataSetIDs;

	DataSetReads.GetKeys(ReadDataSetIDs);
	DataSetWrites.GetKeys(WriteDataSetIDs);

	//The VM register binding assumes the same inputs as outputs which is obviously not always the case.
	for (int32 VarArrayIdx = 0; VarArrayIdx < DataSetReads.Num(); VarArrayIdx++)
	{
		const FNiagaraDataSetID DataSetID = ReadDataSetIDs[VarArrayIdx];
		const TArray<FNiagaraVariable>& ArrayRef = DataSetVariables[DataSetReads[DataSetID]];
		DefineDataSetVariableReads(HlslOutput, DataSetID, VarArrayIdx, ArrayRef);
	}

	bool bRequiresPersistentIDs = CompileOptions.AdditionalDefines.Contains(TEXT("RequiresPersistentIDs"));
	if (bRequiresPersistentIDs && UNiagaraScript::IsSpawnScript(CompileOptions.TargetUsage))
	{
		FString MapName = UNiagaraScript::IsInterpolatedParticleSpawnScript(CompileOptions.TargetUsage) ? TEXT("Context.MapSpawn") : TEXT("Context.Map");
		//Add code to handle persistent IDs.
		OutHlslOutput += TEXT("\tint TempIDIndex;\n\tint TempIDTag;\n");
		OutHlslOutput += TEXT("\tAcquireID(0, TempIDIndex, TempIDTag);\n");
		OutHlslOutput += FString::Printf(TEXT("\t%s.Particles.ID.Index = TempIDIndex;\n\t%s.Particles.ID.AcquireTag = TempIDTag;\n"), *MapName, *MapName);
	}

	{
		// Manually write to Particles.UniqueID on spawn, and deliberately place it at the top of SimulateMain to make sure it's initialized in the right order

		// NOTE(mv): These relies on Particles.UniqueID and Engine.Emitter.TotalSpawnedParticles both being explicitly added to the parameter histories in 
		//           FNiagaraHlslTranslator::Translate.

		// NOTE(mv): This relies on Particles.UniqueID being excluded from being default initialized. 
		//           This happens in FParamMapHistory::ShouldIgnoreVariableDefault
		if (UNiagaraScript::IsParticleSpawnScript(CompileOptions.TargetUsage))
		{
			FString MapName = UNiagaraScript::IsInterpolatedParticleSpawnScript(CompileOptions.TargetUsage) ? TEXT("Context.MapSpawn") : TEXT("Context.Map");
			OutHlslOutput += FString::Printf(TEXT("\t%s.Particles.UniqueID = Engine_Emitter_TotalSpawnedParticles + ExecIndex();\n"), *MapName);
		}
		else if (UNiagaraScript::IsGPUScript(CompileOptions.TargetUsage))
		{
			// NOTE(mv): The GPU script only have one file, so we need to make sure we only apply this in the spawn phase. 
			//           
			OutHlslOutput += TEXT("\tif (Phase == 0) \n\t{\n\t\tContext.MapSpawn.Particles.UniqueID = Engine_Emitter_TotalSpawnedParticles + ExecIndex();\n\t}\n");
		}
	}

	FNiagaraEmitterID EmitterID = GetEmitterID();

	// Fill in the defaults for parameters.
	for (const auto& PerStageChunks : PerStageMainPreSimulateChunks)
	{
		for (const auto& Chunk : PerStageChunks)
		{
			OutHlslOutput += TEXT("\t") + Chunk + TEXT("\n");
		}
	}

	// call the read data set function
	OutHlslOutput += TEXT("\tReadDataSets(Context);\n");
	for (int32 StageIdx = 0; StageIdx < TranslationStages.Num(); StageIdx++)
	{
		if (StageIdx == 0)
		{
			// Either go on to the next phase, or write to the final output context.
			if (RequiresInterpolation())
			{
				OutHlslOutput += TEXT("\tInterpolateParameters(Context);\n"); // Requires ExecIndex, which needs to be in a stage.
			}
		}

		//Set Engine provided param map values.
		OutHlslOutput += FString::Printf(TEXT("\tContext.%s.Engine.ExecIndex = ExecIndex();\n"), *TranslationStages[StageIdx].PassNamespace);
		OutHlslOutput += FString::Printf(TEXT("\tContext.%s.Engine.Emitter.ID.ID = %d;\n"), *TranslationStages[StageIdx].PassNamespace, EmitterID.ID);

		FName ScopeName(TranslationStages[StageIdx].PassNamespace + TEXT("Main"));
		EnterStatsScope(FNiagaraStatScope(*(CompileOptions.GetName() + TEXT("_") + ScopeName.ToString()), ScopeName), OutHlslOutput);
		OutHlslOutput += FString::Printf(TEXT("\tSimulate%s(Context);\n"), TranslationStages.Num() > 1 ? *TranslationStages[StageIdx].PassNamespace : TEXT(""));
		ExitStatsScope(OutHlslOutput);

		if (StageIdx + 1 < TranslationStages.Num() && TranslationStages[StageIdx + 1].bCopyPreviousParams)
		{
			OutHlslOutput += TEXT("\t//Begin Transfer of Attributes!\n");
			if (ParamMapDefinedAttributesToNamespaceVars.Num() != 0)
			{
				OutHlslOutput += TEXT("\tContext.") + TranslationStages[StageIdx + 1].PassNamespace + TEXT(".Particles = Context.") + TranslationStages[StageIdx].PassNamespace + TEXT(".Particles;\n");
				if (TranslationStages[StageIdx].bWritesAlive)
				{
					OutHlslOutput += TEXT("\t\tContext.") + TranslationStages[StageIdx + 1].PassNamespace + TEXT(".DataInstance = Context.") + TranslationStages[StageIdx].PassNamespace + TEXT(".DataInstance;\n");
				}

				if (StageIdx == 0 && UNiagaraScript::IsInterpolatedParticleSpawnScript(CompileOptions.TargetUsage)) // The Update Phase might need previous parameters set.
				{
					// Put any gathered previous variables into the list here so that we can use them by recording the last value from the parent variable on transfer from previous stage if interpolated spawning.
					TArray<FVarAndDefaultSource> VarAndDefaultSourceArray;
					TArray<FNiagaraVariable> GatheredPreviousVariables;
					ParamMapDefinedAttributesToNamespaceVars.GenerateValueArray(VarAndDefaultSourceArray);

					for (const FVarAndDefaultSource& VarAndDefaultSource : VarAndDefaultSourceArray)
					{
						const FNiagaraVariable& Var = VarAndDefaultSource.Variable;
						if (FNiagaraParameterUtilities::IsPreviousValue(Var))
						{
							FNiagaraVariable SrcVar = FNiagaraParameterUtilities::GetSourceForPreviousValue(Var);
							const FString VarName = BuildSanitizedSymbolName(SrcVar.GetName());
							const FString VarPrevName = BuildSanitizedSymbolName(Var.GetName());
							OutHlslOutput += TEXT("\t\tContext.") + TranslationStages[StageIdx + 1].PassNamespace + TEXT(".") + VarPrevName + TEXT(" = Context.") + TranslationStages[StageIdx].PassNamespace + TEXT(".")+ VarName +  TEXT(";\n");
						}
					}
				}
			
			}
			OutHlslOutput += TEXT("\t//End Transfer of Attributes!\n\n");
		}
	}

	// write secondary data sets
	OutHlslOutput += TEXT("\tWriteDataSets(Context);\n");

	//The VM register binding assumes the same inputs as outputs which is obviously not always the case.
	//We should separate inputs and outputs in the script.
	for (int32 VarArrayIdx = 0; VarArrayIdx < DataSetWrites.Num(); VarArrayIdx++)
	{
		const FNiagaraDataSetID DataSetID = WriteDataSetIDs[VarArrayIdx];
		const TArray<FNiagaraVariable> ArrayRef = DataSetVariables[DataSetWrites[DataSetID]];
		DefineDataSetVariableWrites(HlslOutput, DataSetID, VarArrayIdx, ArrayRef);
	}

	ExitStatsScope(OutHlslOutput);
	OutHlslOutput += TEXT("}\n");
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::DefineDataSetVariableWrites(FString &OutHlslOutput, const FNiagaraDataSetID& Id, int32 DataSetIndex, const TArray<FNiagaraVariable>& WriteVars)
{
	check(CompilationTarget != ENiagaraSimTarget::GPUComputeSim);

	//TODO Grab indices for data set writes (inc output) and do the write. Need to rewrite this for events interleaved..
	OutHlslOutput += "\t{\n";
	bool bUsesAlive = false;
	if (!UNiagaraScript::IsNonParticleScript(CompileOptions.TargetUsage))
	{
		FString DataSetName = Id.Name.ToString();
		bool bHasPerParticleAliveSpawn = false;
		bool bHasPerParticleAliveUpdate = false;
		bool bHasPerParticleAliveEvent = false;
		for (int32 i = 0; i < ParamMapHistories.Num(); i++)
		{
			const FOutputNode* OutputNode = ParamMapHistories[i].GetFinalOutputNode();
			if (!OutputNode)
			{
				continue;
			}

			if (INDEX_NONE == ParamMapHistories[i].FindVariable(*(DataSetName + TEXT(".Alive")), FNiagaraTypeDefinition::GetBoolDef()))
			{
				continue;
			}

			switch (GraphBridge::GetOutputNodeScriptType(OutputNode))
			{
			case ENiagaraScriptUsage::ParticleSpawnScript:
			case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
				bHasPerParticleAliveSpawn = true;
				break;
			case ENiagaraScriptUsage::ParticleUpdateScript:
				bHasPerParticleAliveUpdate = true;
				break;
			case ENiagaraScriptUsage::ParticleEventScript:
				bHasPerParticleAliveEvent = true;
				break;
			}
		}

		if ((bHasPerParticleAliveSpawn || bHasPerParticleAliveUpdate) && TranslationStages.Num() > 1)
		{
			// NOTE: TranslationStages.Num() > 1 for GPU Script or CPU Interpolated Spawn CPU scripts

			// NOTE: Context.MapSpawn is copied to Context.MapUpdate before this point in the script, so we might
			//       as well just keep it simple and check against MapUpdate only instead of redundantly branch.
			OutHlslOutput += TEXT("\tbool bValid = Context.MapUpdate.") + DataSetName + TEXT(".Alive;\n");
			bUsesAlive = true;
		}
		else if ((UNiagaraScript::IsParticleSpawnScript(CompileOptions.TargetUsage) && bHasPerParticleAliveSpawn) 
			|| (UNiagaraScript::IsParticleUpdateScript(CompileOptions.TargetUsage) && bHasPerParticleAliveUpdate)
			|| (UNiagaraScript::IsParticleEventScript(CompileOptions.TargetUsage) && bHasPerParticleAliveEvent))
		{
			// Non-interpolated CPU spawn script
			OutHlslOutput += TEXT("\tbool bValid = Context.Map.") + DataSetName + TEXT(".Alive;\n");
			bUsesAlive = true;
		}
	}

	// grab the current ouput index to write datas 
	if (bUsesAlive)
	{
		OutHlslOutput += "\tint TmpWriteIndex = OutputIndex(0, true, bValid);\n";
	}
	else
	{
		OutHlslOutput += "\tint TmpWriteIndex = OutputIndex(0, false, true);\n";
	}

	bool bRequiresPersistentIDs = CompileOptions.AdditionalDefines.Contains(TEXT("RequiresPersistentIDs"));
	if (bRequiresPersistentIDs && DataSetIndex == 0)
	{
		FString MapName = GetParameterMapInstanceName(0);
		OutHlslOutput += FString::Printf(TEXT("\tUpdateID(0, %s.Particles.ID.Index, TmpWriteIndex);\n"), *MapName);
	}

	int32 WriteOffsetInt = 0;
	int32 WriteOffsetFloat = 0;
	int32 WriteOffsetHalf = 0;
	for (const FNiagaraVariable &Var : WriteVars)
	{
		// If coming from a parameter map, use the one on the context, otherwise use the output.
		FString Fmt;
		if (TranslationStages.Num() > 1)
		{
			Fmt = TEXT("\tOutputData{1}(0, {2}, {3}, Context.") + TranslationStages[TranslationStages.Num() - 1].PassNamespace + TEXT(".") + BuildSanitizedSymbolName(Var.GetName()) + TEXT("{0});\n");
		}
		else
		{
			Fmt = TEXT("\tOutputData{1}(0, {2}, {3}, Context.Map.") + BuildSanitizedSymbolName(Var.GetName()) + TEXT("{0});\n");
		}
		GatherVariableForDataSetAccess(Var, Fmt, WriteOffsetInt, WriteOffsetFloat, WriteOffsetHalf, -1, TEXT("TmpWriteIndex"), OutHlslOutput);
	}
	OutHlslOutput += "\t}\n";
}

void FNiagaraHlslTranslator::DefineDataSetVariableReads(FString &OutHlslOutput, const FNiagaraDataSetID& Id, int32 DataSetIndex, const TArray<FNiagaraVariable>& ReadVars)
{
	check(CompilationTarget != ENiagaraSimTarget::GPUComputeSim);

	int32 ReadOffsetInt = 0;
	int32 ReadOffsetFloat = 0;
	int32 ReadOffsetHalf = 0;

	FString DataSetName = Id.Name.ToString();
	FString Fmt;

	bool bIsGPUScript = UNiagaraScript::IsGPUScript(CompileOptions.TargetUsage);
	bool bIsSpawnScript =
		UNiagaraScript::IsParticleSpawnScript(CompileOptions.TargetUsage) || UNiagaraScript::IsInterpolatedParticleSpawnScript(CompileOptions.TargetUsage) ||
		UNiagaraScript::IsEmitterSpawnScript(CompileOptions.TargetUsage) || UNiagaraScript::IsSystemSpawnScript(CompileOptions.TargetUsage);
	bool bIsUpdateScript =
		UNiagaraScript::IsParticleUpdateScript(CompileOptions.TargetUsage) || UNiagaraScript::IsEmitterUpdateScript(CompileOptions.TargetUsage) ||
		UNiagaraScript::IsSystemUpdateScript(CompileOptions.TargetUsage);
	bool bIsEventScript = UNiagaraScript::IsParticleEventScript(CompileOptions.TargetUsage);
	bool bIsSystemOrEmitterScript =
		UNiagaraScript::IsEmitterSpawnScript(CompileOptions.TargetUsage) || UNiagaraScript::IsSystemSpawnScript(CompileOptions.TargetUsage) ||
		UNiagaraScript::IsEmitterUpdateScript(CompileOptions.TargetUsage) || UNiagaraScript::IsSystemUpdateScript(CompileOptions.TargetUsage);
	bool bIsPrimaryDataSet = DataSetIndex == 0;

	// This will initialize parameters to 0 for spawning.  For the system and emitter combined spawn script we want to do this on the
	// primary data set which contains the particle data, but we do not want to do this for the secondary data set since it has
	// external user and engine parameters which must be read.
	if (bIsGPUScript || (bIsSpawnScript && (bIsPrimaryDataSet || bIsSystemOrEmitterScript == false)))
	{
		FString ContextName = TEXT("\tContext.Map.");
		if (TranslationStages.Num() > 1)
		{
			ContextName = FString::Printf(TEXT("\tContext.%s."), *TranslationStages[0].PassNamespace);
		}

		FString VarReads;

		for (const FNiagaraVariable &Var : ReadVars)
		{
			Fmt = ContextName + BuildSanitizedSymbolName(Var.GetName()) + TEXT("{0} = {4};\n");
			GatherVariableForDataSetAccess(Var, Fmt, ReadOffsetInt, ReadOffsetFloat, ReadOffsetHalf, DataSetIndex, TEXT(""), VarReads);
		}

		OutHlslOutput += VarReads;
	}

	// This will initialize parameters to their correct initial values from constants or data sets for update, and will also initialize parameters
	// for spawn if this is a combined system and emitter spawn script and we're reading from a secondary data set for engine and user parameters.
	if (bIsGPUScript || bIsEventScript || bIsUpdateScript || (bIsSpawnScript && bIsPrimaryDataSet == false && bIsSystemOrEmitterScript))
	{
		FString ContextName = TEXT("\tContext.Map.");
		if (TranslationStages.Num() > 1)
		{
			ContextName = FString::Printf(TEXT("\tContext.%s."), *TranslationStages[TranslationStages.Num() - 1].PassNamespace);
		}

		// if we're a GPU spawn script (meaning a combined spawn/update script), we need to reset register index counter
		if (UNiagaraScript::IsGPUScript(CompileOptions.TargetUsage))
		{
			ReadOffsetInt = 0;
			ReadOffsetFloat = 0;
			ReadOffsetHalf = 0;
		}

		FString VarReads;

		TArray <FNiagaraVariable> GatheredPreviousVariables;

		for (const FNiagaraVariable &Var : ReadVars)
		{
			bool bWrite = true;
			const FString VariableName = ContextName + BuildSanitizedSymbolName(Var.GetName());
			// If the NiagaraClearEachFrame value is set on the data set, we don't bother reading it in each frame as we know that it is is invalid. However,
			// this is only used for the base data set. Other reads are potentially from events and are therefore perfectly valid.
			if (DataSetIndex == 0 && Var.GetType().GetScriptStruct() != nullptr && Var.GetType().GetScriptStruct()->GetMetaData(TEXT("NiagaraClearEachFrame")).Equals(TEXT("true"), ESearchCase::IgnoreCase))
			{
				Fmt = VariableName + TEXT("{0} = {4};\n");
			}
			else if (DataSetIndex == 0 && FNiagaraParameterUtilities::IsPreviousValue(Var) && bIsUpdateScript)
			{
				GatheredPreviousVariables.AddUnique(Var);
				bWrite = false; // We need to bump the read indices forwards, but not actually add the read.
			}
			else
			{
				Fmt = VariableName + TEXT("{0} = InputData{1}({2}, {3});\n");

				if (FNiagaraParameterUtilities::IsAttribute(Var))
				{
					FString RegisterName = VariableName;
					ReplaceNamespaceInline(RegisterName, PARAM_MAP_ATTRIBUTE_STR, PARAM_MAP_INDICES_STR);

					Fmt += RegisterName + TEXT(" = {3};\n");
				}
			}
			GatherVariableForDataSetAccess(Var, Fmt, ReadOffsetInt, ReadOffsetFloat, ReadOffsetHalf, DataSetIndex, TEXT(""), VarReads, bWrite);
		}
		OutHlslOutput += VarReads;


		// Put any gathered previous variables into the list here so that we can use them by recording the last value from the parent variable on load.
		for (FNiagaraVariable VarPrevious : GatheredPreviousVariables)
		{
			FNiagaraVariable SrcVar = FNiagaraParameterUtilities::GetSourceForPreviousValue(VarPrevious);
			const FString VarName = ContextName + BuildSanitizedSymbolName(SrcVar.GetName());
			const FString VarPrevName = ContextName + BuildSanitizedSymbolName(VarPrevious.GetName());
			HlslOutput += VarPrevName + TEXT(" = ") + VarName + TEXT(";\n");
		}

	}
}

void FNiagaraHlslTranslator::WriteDataSetContextVars(TMap<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>>& DataSetAccessInfo, bool bRead, FString& OutHlslOutput)
{
	//Now the intermediate storage for the data set reads and writes.
	uint32 DataSetIndex = 0;
	for (TPair<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>>& DataSetInfoPair : DataSetAccessInfo)
	{
		FNiagaraDataSetID DataSet = DataSetInfoPair.Key;

		if (!bRead)
		{
			OutHlslOutput += TEXT("\tbool ") + DataSet.Name.ToString() + TEXT("Write_Valid; \n");
		}

		OutHlslOutput += TEXT("\tF") + DataSet.Name.ToString() + "DataSet " + DataSet.Name.ToString() + (bRead ? TEXT("Read") : TEXT("Write")) + TEXT(";\n");
	}
};


void FNiagaraHlslTranslator::WriteDataSetStructDeclarations(TMap<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>>& DataSetAccessInfo, bool bRead, FString& OutHlslOutput)
{
	uint32 DataSetIndex = 1;
	for (TPair<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>>& DataSetInfoPair : DataSetAccessInfo)
	{
		FNiagaraDataSetID DataSet = DataSetInfoPair.Key;
		FString StructName = TEXT("F") + DataSet.Name.ToString() + "DataSet";
		OutHlslOutput += TEXT("struct ") + StructName + TEXT("\n{\n");

		for (TPair<int32, FDataSetAccessInfo>& IndexInfoPair : DataSetInfoPair.Value)
		{
			for (FNiagaraVariable Var : IndexInfoPair.Value.Variables)
			{
				OutHlslOutput += TEXT("\t") + GetStructHlslTypeName(Var.GetType()) + TEXT(" ") + Var.GetName().ToString() + ";\n";
			}
		}

		OutHlslOutput += TEXT("};\n");

		// declare buffers for compute shader HLSL only; VM doesn't need htem
		// because its InputData and OutputData functions handle data set management explicitly
		//
		if (CompilationTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			FString IndexString = FString::FromInt(DataSetIndex);
			if (bRead)
			{
				OutHlslOutput += FString(TEXT("Buffer<float> ReadDataSetFloat")) + IndexString + ";\n";
				OutHlslOutput += FString(TEXT("Buffer<int> ReadDataSetInt")) + IndexString + ";\n";
				OutHlslOutput += "int DSComponentBufferSizeReadFloat" + IndexString + ";\n";
				OutHlslOutput += "int DSComponentBufferSizeReadInt" + IndexString + ";\n";
			}
			else
			{
				OutHlslOutput += FString(TEXT("RWBuffer<float> RWWriteDataSetFloat")) + IndexString + ";\n";
				OutHlslOutput += FString(TEXT("RWBuffer<int> RWWriteDataSetInt")) + IndexString + ";\n";
				OutHlslOutput += "int DSComponentBufferSizeWriteFloat" + IndexString + ";\n";
				OutHlslOutput += "int DSComponentBufferSizeWriteInt" + IndexString + ";\n";
			}
		}

		DataSetIndex++;
	}

}


//Decomposes each variable into its constituent register accesses.
void FNiagaraHlslTranslator::DecomposeVariableAccess(UStruct* Struct, bool bRead, FString IndexSymbol, FString HLSLString)
{
	FString AccessStr;

	for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;

		if (FStructProperty* StructProp = CastFieldChecked<FStructProperty>(Property))
		{
			UScriptStruct* NiagaraStruct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::Simulation);
			FNiagaraTypeDefinition PropDef(NiagaraStruct);
			if (!IsHlslBuiltinVector(PropDef))
			{
				DecomposeVariableAccess(NiagaraStruct, bRead, IndexSymbol, AccessStr);
				return;
			}
		}

		int32 Index = INDEX_NONE;
		if (bRead)
		{
			Index = ReadIdx++;
			AccessStr = TEXT("ReadInput(");
			AccessStr += FString::FromInt(ReadIdx);
			AccessStr += ");\n";
		}
		else
		{
			Index = WriteIdx++;
			AccessStr = TEXT("WriteOutput(");
			AccessStr += FString::FromInt(WriteIdx);
			AccessStr += ");\n";
		}

		HLSLString += AccessStr;

		FNiagaraTypeDefinition StructDef(Cast<UScriptStruct>(Struct));
		FString TypeName = GetStructHlslTypeName(StructDef);
	}
};

FString FNiagaraHlslTranslator::GetSanitizedSymbolName(FStringView SymbolName, bool bCollapseNamespaces)
{
	const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();

	TStringBuilder<1024> StringBuilder;
	NiagaraHlslTranslatorImpl::AppendSanitizedSymbolString(SymbolName, bCollapseNamespaces, EditorSettings->GetHLSLKeywordReplacementsMap(), StringBuilder);

	return StringBuilder.ToString();
}

FString FNiagaraHlslTranslator::BuildSanitizedSymbolString(FStringView SymbolName, bool bCollapseNamespaces) const
{
	FNameBuilder StringBuilder;
	AppendSanitizedSymbolString(StringBuilder, SymbolName, bCollapseNamespaces);

	return StringBuilder.ToString();
}

FString FNiagaraHlslTranslator::BuildSanitizedSymbolName(FName SymbolName, bool bCollapseNamespaces) const
{
	FNameBuilder NameBuilder(SymbolName);
	return BuildSanitizedSymbolString(NameBuilder.ToView(), bCollapseNamespaces);
}

void FNiagaraHlslTranslator::AppendSanitizedSymbolString(FStringBuilderBase& Builder, FStringView SymbolName, bool bCollapseNamespaces) const
{
	return NiagaraHlslTranslatorImpl::AppendSanitizedSymbolString(SymbolName, bCollapseNamespaces, KeywordReplacementMap, Builder);
}

FString FNiagaraHlslTranslator::GetSanitizedDIFunctionName(const FString& FunctionName)
{
	bool bWordStart = true;
	FString Sanitized;
	for (int i = 0; i < FunctionName.Len(); ++i)
	{
		TCHAR c = FunctionName[i];

		if (c == ' ')
		{
			bWordStart = true;
		}
		else
		{
			if (bWordStart)
			{
				c = FChar::ToUpper(c);
				bWordStart = false;
			}
			Sanitized.AppendChar(c);
		}
	}

	return Sanitized;
}

void FNiagaraHlslTranslator::AppendSanitizedFunctionNameSuffix(FStringView Name, FStringBuilderBase& StringBuilder)
{
	FNameBuilder SanitizedName;

	{ // single character replacements
		const FStringView::ElementType UnderscoreKeys[] = { TEXT('.'), TEXT('\\'), TEXT('/'), TEXT(','), TEXT('-'), TEXT(':') };
		const FStringView::ElementType SkipKeys[] = { TEXT('\t'), TEXT(' ') };

		for (FStringView::ElementType InChar : Name)
		{
			if (!Algo::Contains(SkipKeys, InChar))
			{
				if (Algo::Contains(UnderscoreKeys, InChar))
				{
					SanitizedName.AppendChar(TEXT('_'));
				}
				else
				{
					SanitizedName.AppendChar(InChar);
				}
			}
		}
	}

	{ // multi character replacements
		struct FReplacementPair
		{
			const TCHAR* Key = nullptr;
			const TCHAR* Value = nullptr;

			const int32 KeyLen = 0;
			const int32 ValueLen = 0;

			FReplacementPair(const TCHAR* InKey, const TCHAR* InValue)
			: Key(InKey)
			, Value(InValue)
			, KeyLen(FCString::Strlen(InKey))
			, ValueLen(FCString::Strlen(InValue))
			{}
		};

		const FReplacementPair MultiCharReplacements[] =
		{
			{ TEXT("__"), TEXT("ASC95ASC95") }
		};

		for (const FReplacementPair& Replacement : MultiCharReplacements)
		{
			int32 SearchPosition = 0;

			while (SearchPosition != INDEX_NONE)
			{
				SearchPosition = SanitizedName.ToView().Find(Replacement.Key, SearchPosition);
				if (SearchPosition != INDEX_NONE)
				{
					SanitizedName.ReplaceAt(SearchPosition, Replacement.KeyLen, Replacement.Value);
					SearchPosition += Replacement.ValueLen;
				}
			}
		}
	}

	NiagaraHlslTranslatorImpl::AppendConvertedToAscii(SanitizedName.ToView(), StringBuilder);
}


FString FNiagaraHlslTranslator::ConvertToAsciiString(FString Str)
{
	FNameBuilder Builder;
	NiagaraHlslTranslatorImpl::AppendConvertedToAscii(Str, Builder);

	return Builder.ToString();
}

FString FNiagaraHlslTranslator::GetUniqueSymbolName(FName BaseName)
{
	FNameBuilder BaseNameBuilder(BaseName);
	FString RetString = BuildSanitizedSymbolString(BaseNameBuilder.ToView());
	FName RetName = *RetString;
	uint32* NameCount = SymbolCounts.Find(RetName);
	if (NameCount == nullptr)
	{
		SymbolCounts.Add(RetName) = 1;
		return RetString;
	}

	if (*NameCount > 0)
	{
		RetString += LexToString(*NameCount);
	}
	++(*NameCount);
	return RetString;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::EnterFunction(const FString& Name, FNiagaraFunctionSignature& Signature, TArrayView<const int32> Inputs, const FGuid& InGuid)
{
	FunctionContextStack.Emplace(Name, Signature, Inputs, InGuid);
	TArray<FName> Entries;
	ActiveStageWriteTargets.Push(Entries);
	ActiveStageReadTargets.Push(Entries);
	//May need some more heavy and scoped symbol tracking?

	//Add new scope for pin reuse.
	PinToCodeChunks.AddDefaulted(1);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ExitFunction()
{
	FunctionContextStack.Pop();
	//May need some more heavy and scoped symbol tracking?

	//Pop pin reuse scope.
	PinToCodeChunks.Pop();

	// Accumulate the write targets.
	TArray<FName> Entries = ActiveStageWriteTargets.Pop();
	if (ActiveStageWriteTargets.Num())
	{
		for (const FName& Entry : Entries)
		{
			ActiveStageWriteTargets.Top().AddUnique(Entry);
		}
	}

	// Accumulate the read targets.
	Entries = ActiveStageReadTargets.Pop();
	if (ActiveStageReadTargets.Num())
	{
		for (const FName& Entry : Entries)
		{
			ActiveStageReadTargets.Top().AddUnique(Entry);
		}
	}
}

FString FNiagaraHlslTranslator::GeneratedConstantString(float Constant)
{
	return LexToString(Constant);
}

static int32 GbNiagaraScriptStatTracking = 1;
static FAutoConsoleVariableRef CVarNiagaraScriptStatTracking(
	TEXT("fx.NiagaraScriptStatTracking"),
	GbNiagaraScriptStatTracking,
	TEXT("If > 0 stats tracking operations will be compiled into Niagara Scripts. \n"),
	ECVF_Default
);

void FNiagaraHlslTranslator::EnterStatsScope(FNiagaraStatScope StatScope)
{
	if (GbNiagaraScriptStatTracking)
	{
		int32 ScopeIdx = CompilationOutput.ScriptData.StatScopes.AddUnique(StatScope);
		AddBodyChunk(TEXT(""), FString::Printf(TEXT("EnterStatScope(%d /**%s*/)"), ScopeIdx, *StatScope.FullName.ToString()), FNiagaraTypeDefinition::GetFloatDef(), false);
		StatScopeStack.Push(ScopeIdx);
	}
}

void FNiagaraHlslTranslator::ExitStatsScope()
{
	if (GbNiagaraScriptStatTracking)
	{
		int32 ScopeIdx = StatScopeStack.Pop();
		AddBodyChunk(TEXT(""), FString::Printf(TEXT("ExitStatScope(/**%s*/)"), *CompilationOutput.ScriptData.StatScopes[ScopeIdx].FullName.ToString()), FNiagaraTypeDefinition::GetFloatDef(), false);
	}
}

void FNiagaraHlslTranslator::EnterStatsScope(FNiagaraStatScope StatScope, FString& OutHlsl)
{
	if (GbNiagaraScriptStatTracking)
	{
		int32 ScopeIdx = CompilationOutput.ScriptData.StatScopes.AddUnique(StatScope);
		OutHlsl += FString::Printf(TEXT("EnterStatScope(%d /**%s*/);\n"), ScopeIdx, *StatScope.FullName.ToString());
		StatScopeStack.Push(ScopeIdx);
	}
}

void FNiagaraHlslTranslator::ExitStatsScope(FString& OutHlsl)
{
	if (GbNiagaraScriptStatTracking)
	{
		int32 ScopeIdx = StatScopeStack.Pop();
		OutHlsl += FString::Printf(TEXT("ExitStatScope(/**%s*/);\n"), *CompilationOutput.ScriptData.StatScopes[ScopeIdx].FullName.ToString());
	}
}

FString FNiagaraHlslTranslator::GetCallstack()
{
	FString Callstack = CompileOptions.GetName();

	for (FFunctionContext& Ctx : FunctionContextStack)
	{
		Callstack += TEXT(".") + Ctx.Name;
	}

	return Callstack;
}

TArray<FGuid> FNiagaraHlslTranslator::GetCallstackGuids()
{
	TArray<FGuid> Callstack;
	for (FFunctionContext& Ctx : FunctionContextStack)
	{
		Callstack.Add(Ctx.Id);
	}

	return Callstack;
}

FString FNiagaraHlslTranslator::GeneratedConstantString(FVector4 Constant)
{
	TArray<FStringFormatArg> Args;
	Args.Add(LexToString(Constant.X));
	Args.Add(LexToString(Constant.Y));
	Args.Add(LexToString(Constant.Z));
	Args.Add(LexToString(Constant.W));
	return FString::Format(TEXT("float4({0}, {1}, {2}, {3})"), Args);
}

int32 FNiagaraHlslTranslator::AddChunkToConstantBuffer(const FString& SymbolName, const FNiagaraVariable& InVariable, ENiagaraCodeChunkMode ChunkMode)
{
	if (const FUniformVariableInfo* UniformVariableInfo = ParamMapDefinedSystemVars.Find(InVariable.GetName()))
	{
		check(UniformVariableInfo->ChunkMode == static_cast<int32>(ChunkMode));
		check(UniformVariableInfo->Variable == InVariable);
		return UniformVariableInfo->ChunkIndex;
	}

	const FNiagaraTypeDefinition& Type = InVariable.GetType();
	const int32 ChunkModeIndex = static_cast<int32>(ChunkMode);

	check(FNiagaraTypeHelper::IsLWCType(Type) == false);
	int32 ChunkIndex = CodeChunks.AddDefaulted();
	FNiagaraCodeChunk& Chunk = CodeChunks[ChunkIndex];
	Chunk.SymbolName = BuildSanitizedSymbolString(SymbolName, true);
	Chunk.Type = Type;
	Chunk.Original = InVariable;
	Chunk.Mode = ChunkMode;

	ChunksByMode[ChunkModeIndex].Add(ChunkIndex);

	auto& SystemVar = ParamMapDefinedSystemVars.Add(InVariable.GetName());
	SystemVar.ChunkIndex = ChunkIndex;
	SystemVar.ChunkMode = ChunkModeIndex;
	SystemVar.Variable = InVariable;

	return ChunkIndex;
}

int32 FNiagaraHlslTranslator::RegisterUniformChunkToPack(const FString& SymbolName, const FNiagaraVariable& InVariable, bool AddPadding, FNiagaraParameters& Parameters, TOptional<FNiagaraVariable>& ConflictingVariable)
{
	// we must ensure that there's a one to one relationship between symbol name and parameter.  The generated VM only
	// knows about the symbols while the parameter stores knows about the parameters, if these mismatch, then we're going
	// to be incorrectly addressing the constant table
	int32 ParameterIndex = Parameters.Parameters.IndexOfByKey(InVariable);

	if (ParameterIndex == INDEX_NONE)
	{
		// add the parameter, but first evaluate whether any of the symbols for existing parameters would conflict
		const int32 ConflictIndex = Parameters.Parameters.IndexOfByPredicate([&](const FNiagaraVariable& ExistingParameter)
		{
			FNameBuilder ExistingParameterName(ExistingParameter.GetName());
			FNameBuilder SanitizedParameterName;
			AppendSanitizedSymbolString(SanitizedParameterName, ExistingParameterName);

			return ExistingParameterName.ToView().Equals(SymbolName);
		});

		if (ConflictIndex != INDEX_NONE)
		{
			ConflictingVariable = Parameters.Parameters[ConflictIndex];
			return INDEX_NONE;
		}
		else
		{
			ParameterIndex = Parameters.Parameters.Add(InVariable);
		}
	}
	
	int32 ChunkIndex = AddChunkToConstantBuffer(SymbolName, InVariable, ENiagaraCodeChunkMode::Uniform);

	if (AddPadding)
	{
		UniformParametersToPack.Add(ParameterIndex, ChunkIndex);
	}

	return ChunkIndex;
}

void FNiagaraHlslTranslator::PackRegisteredUniformChunk(FNiagaraParameters& Parameters)
{
	if (UniformParametersToPack.IsEmpty())
	{
		return;
	}

	// go through the parameters that have been registered, generate a layout that reduces waste and adjust the chunks
	// that have already been added via RegisterUniformChunkToPack()

	constexpr int32 SlotSizeInBytes = 4;
	constexpr int32 SlotsPerRow = 4;

	const int32 InitialParameterCount = Parameters.Parameters.Num();

	TArray<int32> OrderedVariableIndices;
	OrderedVariableIndices.Reserve(InitialParameterCount);

	TArray<int32> SingleIndices;
	TArray<int32> DoubleIndices;
	TArray<int32> TripleIndices;

	int32 UniquePaddingIndex = 1;

	for (int32 ParameterIt = 0; ParameterIt < InitialParameterCount; ++ParameterIt)
	{
		const FNiagaraVariable& Parameter = Parameters.Parameters[ParameterIt];
		const int32 ParameterSizeInBytes = Align(Parameter.GetSizeInBytes(), SlotSizeInBytes);
		const int32 ParameterSizeInSlots = ParameterSizeInBytes / SlotSizeInBytes;

		switch (ParameterSizeInSlots % SlotsPerRow)
		{
			case 0:			OrderedVariableIndices.Add(ParameterIt);	break;
			case 1:			SingleIndices.Add(ParameterIt);				break;
			case 2:			DoubleIndices.Add(ParameterIt);				break;
			case 3:			TripleIndices.Add(ParameterIt);				break;
		}
	}

	auto ConditionalAddSingle = [&]() -> void
	{
		static const FName PaddingName = TEXT("Internal.Padding_int32");

		if (SingleIndices.IsEmpty())
		{
			// if we're not all done adding elements, then we're going to have to add some padding
			if (!DoubleIndices.IsEmpty() || !TripleIndices.IsEmpty())
			{
				TOptional<FNiagaraVariable> ConflictingVariable;

				FName InstanceName = PaddingName;
				InstanceName.SetNumber(UniquePaddingIndex++);

				FNiagaraVariable PaddingVariable(FNiagaraTypeDefinition::GetIntDef(), InstanceName);
				FString SymbolName = BuildSanitizedSymbolName(InstanceName);

				const int32 PaddingVariableIndex = Parameters.Parameters.Num();
				RegisterUniformChunkToPack(SymbolName, PaddingVariable, true /*AddPadding*/, Parameters, ConflictingVariable);
				check(!ConflictingVariable.IsSet());

				OrderedVariableIndices.Add(PaddingVariableIndex);
			}
		}
		else
		{
			OrderedVariableIndices.Add(SingleIndices.Pop());
		}
	};

	while (!TripleIndices.IsEmpty())
	{
		OrderedVariableIndices.Add(TripleIndices.Pop());
		ConditionalAddSingle();
	}

	while (DoubleIndices.Num() > 1)
	{
		OrderedVariableIndices.Add(DoubleIndices.Pop());
		OrderedVariableIndices.Add(DoubleIndices.Pop());
	}

	if (!DoubleIndices.IsEmpty())
	{
		OrderedVariableIndices.Add(DoubleIndices.Pop());
		ConditionalAddSingle();
		ConditionalAddSingle();
	}

	OrderedVariableIndices.Append(SingleIndices);

	// now we want to reorder the chunks for the uniform expressions based on how we've rearranged the variables here
	const int32 ChunkModeToSort = static_cast<int32>(ENiagaraCodeChunkMode::Uniform);
	TArray<int32>& ChunksToSort = ChunksByMode[ChunkModeToSort];
	const int32 ChunkCount = ChunksToSort.Num();

	// note that chunks that fail IsVariableInUniformBuffer will still be in the ChunksByMode, but won't actually get written
	// out.  For now we preserve their existence in the ChunksByMode.  Make a copy of the uniform chunks so that we can find
	// all the ones that need to get added back in
	TSet<int32> OutstandingChunkIndices(ChunksToSort);
	ChunksToSort.Reset();

	for (int32 SortedParamIt : OrderedVariableIndices)
	{
		const int32 ChunkIndex = UniformParametersToPack.FindRef(SortedParamIt);
		ChunksToSort.Add(ChunkIndex);
		OutstandingChunkIndices.Remove(ChunkIndex);
	}

	// now append in the outstanding chunk indices
	for (int32 OutstandingChunkIndex : OutstandingChunkIndices)
	{
		ChunksToSort.Add(OutstandingChunkIndex);
	}

	// and finally reorder the Parameters array as well
	TArray<FNiagaraVariable> OrderedVariables;
	OrderedVariables.Reserve(OrderedVariableIndices.Num());
	for (int32 VariableIndex : OrderedVariableIndices)
	{
		OrderedVariables.Add(Parameters.Parameters[VariableIndex]);
	}

	Parameters.Parameters = MoveTemp(OrderedVariables);
}

int32 FNiagaraHlslTranslator::AddSourceChunk(FString SymbolName, const FNiagaraTypeDefinition& Type, bool bSanitize)
{
	int32 Ret = CodeChunks.IndexOfByPredicate(
		[&](const FNiagaraCodeChunk& Chunk)
	{
		return Chunk.Mode == ENiagaraCodeChunkMode::Source && Chunk.SymbolName == SymbolName && Chunk.Type == Type;
	}
	);

	if (Ret == INDEX_NONE)
	{
		check(FNiagaraTypeHelper::IsLWCType(Type) == false);
		Ret = CodeChunks.AddDefaulted();
		FNiagaraCodeChunk& Chunk = CodeChunks[Ret];
		Chunk.SymbolName = bSanitize ? BuildSanitizedSymbolString(SymbolName) : SymbolName;
		Chunk.Type = Type;
		Chunk.Mode = ENiagaraCodeChunkMode::Source;

		ChunksByMode[(int32)ENiagaraCodeChunkMode::Source].Add(Ret);
	}
	return Ret;
}


int32 FNiagaraHlslTranslator::AddBodyComment(FStringView Comment)
{
	return AddBodyChunk(TEXT(""), Comment, FNiagaraTypeDefinition::GetIntDef(), TArrayView<int32>(), false, false);
}

int32 FNiagaraHlslTranslator::AddBodyChunk(FStringView Value)
{
	return AddBodyChunk(TEXT(""), Value, FNiagaraTypeDefinition::GetIntDef(), INDEX_NONE, false, false);
}

int32 FNiagaraHlslTranslator::AddBodyChunk(FStringView SymbolName, FStringView Definition, const FNiagaraTypeDefinition& Type, TArrayView<int32> SourceChunks, bool bDecl, bool bIsTerminated)
{
	check(CurrentBodyChunkMode == ENiagaraCodeChunkMode::Body || CurrentBodyChunkMode == ENiagaraCodeChunkMode::SpawnBody || CurrentBodyChunkMode == ENiagaraCodeChunkMode::UpdateBody ||
		(CurrentBodyChunkMode >= ENiagaraCodeChunkMode::SimulationStageBody && CurrentBodyChunkMode < ENiagaraCodeChunkMode::SimulationStageBodyMax));
	check(FNiagaraTypeHelper::IsLWCType(Type) == false);
	int32 Ret = CodeChunks.AddDefaulted();
	FNiagaraCodeChunk& Chunk = CodeChunks[Ret];
	Chunk.SymbolName = BuildSanitizedSymbolString(SymbolName);
	Chunk.Definition = Definition;
	Chunk.Type = Type;
	Chunk.bDecl = bDecl;
	Chunk.bIsTerminated = bIsTerminated;
	Chunk.Mode = CurrentBodyChunkMode;
	Chunk.SourceChunks.Append(SourceChunks);

	ChunksByMode[(int32)CurrentBodyChunkMode].Add(Ret);
	return Ret;
}



int32 FNiagaraHlslTranslator::AddBodyChunk(FStringView SymbolName, FStringView Definition, const FNiagaraTypeDefinition& Type, int32 SourceChunk, bool bDecl, bool bIsTerminated)
{
	return AddBodyChunk(SymbolName, Definition, Type, MakeArrayView( &SourceChunk, 1), bDecl, bIsTerminated);
}

int32 FNiagaraHlslTranslator::AddBodyChunk(FStringView SymbolName, FStringView Definition, const FNiagaraTypeDefinition& Type, bool bDecl, bool bIsTerminated)
{
	return AddBodyChunk(SymbolName, Definition, Type, TArrayView<int32>(), bDecl, bIsTerminated);
}

bool FNiagaraHlslTranslator::ShouldInterpolateParameter(const FNiagaraVariable& Parameter)
{
	//TODO: Some data driven method of deciding what parameters to interpolate and how to do it.
	//Possibly allow definition of a dynamic input for the interpolation?
	//With defaults for various types. Matrix=none, quat=slerp, everything else = Lerp.

	//We don't want to interpolate matrices. Possibly consider moving to an FTransform like representation rather than matrices which could be interpolated?
	if (Parameter.GetType() == FNiagaraTypeDefinition::GetMatrix4Def())
	{
		return false;
	}

	if (!Parameter.GetType().IsFloatPrimitive())
	{
		return false;
	}

	if (FNiagaraParameterUtilities::IsRapidIterationParameter(Parameter))
	{
		return false;
	}

	//Skip interpolation for some system constants.
	if (Parameter == SYS_PARAM_ENGINE_WORLD_DELTA_TIME ||
		Parameter == SYS_PARAM_ENGINE_DELTA_TIME ||
		Parameter == SYS_PARAM_ENGINE_INV_DELTA_TIME ||
		Parameter == SYS_PARAM_ENGINE_EXEC_COUNT ||
		Parameter == SYS_PARAM_EMITTER_SPAWNRATE ||
		Parameter == SYS_PARAM_EMITTER_SPAWN_INTERVAL ||
		Parameter == SYS_PARAM_EMITTER_INTERP_SPAWN_START_DT ||
		Parameter == SYS_PARAM_ENGINE_EMITTER_TOTAL_SPAWNED_PARTICLES ||
		Parameter == SYS_PARAM_ENGINE_EMITTER_SPAWN_COUNT_SCALE ||
		Parameter == SYS_PARAM_EMITTER_RANDOM_SEED ||
		Parameter == SYS_PARAM_ENGINE_SYSTEM_NUM_TIME_STEPS ||
		Parameter == SYS_PARAM_ENGINE_SYSTEM_TIME_STEP_FRACTION ||
		Parameter == SYS_PARAM_ENGINE_SYSTEM_NUM_PARTICLES ||
		Parameter == SYS_PARAM_ENGINE_SYSTEM_NUM_EMITTERS ||
		Parameter == SYS_PARAM_ENGINE_EMITTER_INSTANCE_SEED ||
		Parameter == SYS_PARAM_ENGINE_SYSTEM_TICK_COUNT ||
		Parameter == SYS_PARAM_ENGINE_SYSTEM_RANDOM_SEED)
	{
		return false;
	}

	return true;
}

FString FNiagaraHlslTranslator::GetInterpolateHlsl(const FNiagaraVariable& Parameter, const FString& PrevMapName, const FNiagaraCodeChunk& Chunk) const
{
	FString FoundName = BuildSanitizedSymbolName(Parameter.GetName());
	if (Parameter.GetType() == FNiagaraTypeDefinition::GetQuatDef() && CompileOptions.AdditionalDefines.Contains(FNiagaraCompileOptions::AccurateQuatInterpolation))
	{
		return FString::Format(TEXT("\tContext.{0}.{1} = NiagaraQuatSLerp({2}{3}{4}, {3}{4}, SpawnInterp);\n"), {PrevMapName, FoundName, INTERPOLATED_PARAMETER_PREFIX, Chunk.SymbolName, Chunk.ComponentMask});
	}
	return FString::Format(TEXT("\tContext.{0}.{1} = lerp({2}{3}{4}, {3}{4}, SpawnInterp);\n"), {PrevMapName, FoundName, INTERPOLATED_PARAMETER_PREFIX, Chunk.SymbolName, Chunk.ComponentMask});
}

// specialization handling the case where the translation process treats the graph as mutable
template<>
void TNiagaraHlslTranslator<FNiagaraCompilationDigestBridge>::UpdateStaticSwitchConstants(const FPin* Pin)
{
}

template<>
void TNiagaraHlslTranslator<FNiagaraCompilationGraphBridge>::UpdateStaticSwitchConstants(const FPin* ChildPin)
{
	if (UNiagaraNodeStaticSwitch* SwitchNode = Cast<UNiagaraNodeStaticSwitch>(ChildPin->GetOwningNode()))
	{
		SwitchNode->CheckForOutdatedEnum(this);

		TArray<UNiagaraNodeStaticSwitch*> NodesToUpdate;
		NodesToUpdate.Add(SwitchNode);

		FPinCollectorArray InPins;
		for (int i = 0; i < NodesToUpdate.Num(); i++)
		{
			SwitchNode->UpdateCompilerConstantValue(this);

			// also check direct upstream static switches, because they are otherwise skipped during the compilation and
			// might be evaluated without their values set correctly.
			InPins.Reset();
			SwitchNode->GetInputPins(InPins);
			for (FPin* Pin : InPins)
			{
				if (UNiagaraNodeStaticSwitch* ConnectedNode = Cast<UNiagaraNodeStaticSwitch>(Pin->GetOwningNode()))
				{
					NodesToUpdate.AddUnique(ConnectedNode);
				}
			}
		}
	}
}

template<typename GraphBridge>
int32 TNiagaraHlslTranslator<GraphBridge>::GetRapidIterationParameter(const FNiagaraVariable& Parameter)
{
	if (!AddStructToDefinitionSet(Parameter.GetType()))
	{
		Error(FText::Format(LOCTEXT("GetRapidIterationParameterTypeFail_InvalidType", "Cannot handle type {0}! Variable: {1}"), Parameter.GetType().GetNameText(), FText::FromName(Parameter.GetName())), nullptr, nullptr);
		return INDEX_NONE;
	}

	int32 FuncParam = INDEX_NONE;
	if (GetFunctionParameter(Parameter, FuncParam))
	{
		Error(FText::Format(LOCTEXT("GetRapidIterationParameterFuncParamFail", "Variable: {0} cannot be a function parameter because it is a RapidIterationParameter type."), FText::FromName(Parameter.GetName())), nullptr, nullptr);
		return INDEX_NONE;
	}

	bool bIsCandidateForRapidIteration = false;
	if (ActiveHistoryForFunctionCalls.InTopLevelFunctionCall(CompileOptions.TargetUsage))
	{
		if (FNiagaraStackGraphUtilities::IsRapidIterationType(Parameter.GetType()))
		{
			bIsCandidateForRapidIteration = true;
		}
		else
		{
			Error(FText::Format(LOCTEXT("GetRapidIterationParameterTypeFail_UnsupportedInput", "Variable: {0} cannot be a RapidIterationParameter input node because it isn't a supported type {1}"), FText::FromName(Parameter.GetName()), Parameter.GetType().GetNameText()), nullptr, nullptr);
			return INDEX_NONE;
		}
	}
	else
	{
		Error(FText::Format(LOCTEXT("GetRapidIterationParameterInTopLevelFail", "Variable: {0} cannot be a RapidIterationParameter input node because it isn't in the top level of an emitter/system/particle graph."), FText::FromName(Parameter.GetName())), nullptr, nullptr);
		return INDEX_NONE;
	}

	FNiagaraVariable RapidIterationConstantVar = Parameter;

	int32 LastSetChunkIdx = INDEX_NONE;
	// Check to see if this is the first time we've encountered this node and it is a viable candidate for rapid iteration
	if (bIsCandidateForRapidIteration && TranslationOptions.bParameterRapidIteration)
	{

		// go ahead and make it into a constant variable..
		int32 OutputChunkId = INDEX_NONE;
		if (ParameterMapRegisterExternalConstantNamespaceVariable(Parameter, nullptr, INDEX_NONE, OutputChunkId, nullptr))
		{
			return OutputChunkId;
		}
	}
	else
	{
		int32 FoundIdx = TranslationOptions.OverrideModuleConstants.Find(RapidIterationConstantVar);
		if (FoundIdx != INDEX_NONE)
		{
			FString DebugConstantStr;
			int32 OutputChunkId = GetConstant(TranslationOptions.OverrideModuleConstants[FoundIdx], &DebugConstantStr);
			//UE_LOGF(LogNiagaraEditor, Display, "Converted parameter %ls to constant %ls for script %ls", *RapidIterationConstantVar.GetName().ToString(), *DebugConstantStr, *CompileOptions.FullName);
			return OutputChunkId;
		}
	}
	return INDEX_NONE;
}

template<typename GraphBridge>
int32 TNiagaraHlslTranslator<GraphBridge>::GetParameter(const FNiagaraVariable& Parameter)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_GetParameter);
	if (!AddStructToDefinitionSet(Parameter.GetType()))
	{
		Error(FText::Format(LOCTEXT("GetParameterFail", "Cannot handle type {0}! Variable: {1}"), Parameter.GetType().GetNameText(), FText::FromName(Parameter.GetName())), nullptr, nullptr);
	}

	if (Parameter == TRANSLATOR_PARAM_BEGIN_DEFAULTS)
	{
		if (CurrentDefaultPinTraversal.Num() != 0)
		{
			return ActiveStageIdx;
		}
		else
		{
			Error(FText::Format(LOCTEXT("InitializingDefaults", "Cannot have a {0} node if you are not tracing a default value from a Get node."), FText::FromName(Parameter.GetName())), nullptr, nullptr);
			return INDEX_NONE;
		}
	}

	if (Parameter == TRANSLATOR_PARAM_CALL_ID)
	{
		FNiagaraVariable CallIDValue = Parameter;
		int32 CallID = GetUniqueCallerID();
		CallIDValue.SetValue(CallID);
		return GetConstant(CallIDValue);
	}

	int32 FuncParam = INDEX_NONE;
	const FNiagaraVariable* FoundKnownVariable = FNiagaraConstants::GetKnownConstant(Parameter.GetName(), false);

	if (FoundKnownVariable == nullptr && GetFunctionParameter(Parameter, FuncParam))
	{
		if (FuncParam != INDEX_NONE)
		{
			if (Parameter.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				return FuncParam;
			}
			//If this is a valid function parameter, use that.
			FString SymbolName = TEXT("In_") + BuildSanitizedSymbolName(Parameter.GetName());
			return AddSourceChunk(SymbolName, ConvertToSimulationVariable(Parameter).GetType());
		}
	}

	if (FoundKnownVariable != nullptr)
	{
		FNiagaraVariable Var = *FoundKnownVariable;
		//Some special variables can be replaced directly with constants which allows for extra optimization in the compiler.
		if (GetLiteralConstantVariable(Var))
		{
			return GetConstant(Var);
		}
	}

	// We don't pass in the input node here (really there could be multiple nodes for the same parameter)
	// so we have to match up the input parameter map variable value through the pre-traversal histories 
	// so that we know which parameter map we are referencing.
	FString SymbolName = BuildSanitizedSymbolName(Parameter.GetName());
	if (Parameter.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
	{
		if (ParamMapHistories.Num() == 0)
		{
			return INDEX_NONE;
		}

		for (int32 i = 0; i < ParamMapHistories.Num(); i++)
		{
			// Double-check against the current output node we are tracing. Ignore any parameter maps
			// that don't include that node.
			if (CurrentParamMapIndices.Num() != 0 && !CurrentParamMapIndices.Contains(i))
			{
				continue;
			}

			for (int32 PinIdx = 0; PinIdx < ParamMapHistories[i].MapPinHistory.Num(); PinIdx++)
			{
				const FPin* Pin = ParamMapHistories[i].MapPinHistory[PinIdx];

				if (Pin != nullptr)
				{
					const FInputNode* InputNode = GraphBridge::AsInputNode(GraphBridge::GetOwningNode(Pin));
					if (InputNode != nullptr && GraphBridge::GetInputVariable(InputNode) == Parameter)
					{
						if (CurrentDefaultPinTraversal.Num() == 0)
						{
							if (!bInitializedDefaults)
							{
								InitializeParameterMapDefaults(i);
							}
						}

						return i;
					}
				}
			}
		}
		return INDEX_NONE;
	}

	//Not a in a function or not a valid function parameter so grab from the main uniforms.
	int32 OutputChunkIdx = INDEX_NONE;
	FNiagaraVariable OutputVariable = Parameter;
	if (FNiagaraParameterUtilities::IsInNamespace(OutputVariable, PARAM_MAP_ATTRIBUTE_STR) || FNiagaraParameterUtilities::IsExternalConstantNamespace(OutputVariable, CompileOptions.TargetUsage, CompileOptions.GetTargetUsageBitmask()))
	{
		if (!ParameterMapRegisterExternalConstantNamespaceVariable(OutputVariable, nullptr, 0, OutputChunkIdx, nullptr))
		{
			OutputChunkIdx = INDEX_NONE;
		}
	}
	else
	{
		OutputVariable = FNiagaraParameterUtilities::MoveToExternalConstantNamespaceVariable(OutputVariable, CompileOptions.TargetUsage);
		if (!ParameterMapRegisterExternalConstantNamespaceVariable(OutputVariable, nullptr, 0, OutputChunkIdx, nullptr))
		{
			OutputChunkIdx = INDEX_NONE;
		}
	}

	// We won't be able to resolve static variables that are inputs, so just let them be the defaults for support class scripts.
	if (OutputChunkIdx == INDEX_NONE && OutputVariable.GetType().IsStatic() &&
		(UNiagaraScript::IsModuleScript(CompileOptions.TargetUsage) || UNiagaraScript::IsFunctionScript(CompileOptions.TargetUsage) || UNiagaraScript::IsDynamicInputScript(CompileOptions.TargetUsage)))
	{
		OutputChunkIdx = GetConstant(OutputVariable);
	}

	if (OutputChunkIdx == INDEX_NONE)
	{
		Error(FText::Format(LOCTEXT("GetParameterFail", "Cannot handle type {0}! Variable: {1}"), Parameter.GetType().GetNameText(), FText::FromName(Parameter.GetName())), nullptr, nullptr);
	}

	return OutputChunkIdx;
}

int32 FNiagaraHlslTranslator::GetConstant(const FNiagaraVariable& Constant, FString* DebugOutputValue)
{
	if (Constant.IsDataInterface() || Constant.IsUObject())
	{
		return INDEX_NONE;
	}

	FString ConstantStr;
	FNiagaraVariable LiteralConstant = Constant;
	if (GetLiteralConstantVariable(LiteralConstant))
	{
		checkf(LiteralConstant.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()) || LiteralConstant.GetType() == FNiagaraTypeDefinition::GetVec3Def() || LiteralConstant.GetType() == FNiagaraTypeDefinition::GetPositionDef(), TEXT("Only boolean and vec3 types are currently supported for literal constants."));
		ConstantStr = GenerateConstantString(LiteralConstant);
	}
	else
	{
		ConstantStr = GenerateConstantString(Constant);
	}

	if (DebugOutputValue != nullptr)
	{
		*DebugOutputValue = ConstantStr;
	}
	if (ConstantStr.IsEmpty())
	{
		return INDEX_NONE;
	}
	
	int32 BodyChunk = AddBodyChunk(GetUniqueSymbolName(TEXT("Constant")), ConstantStr, Constant.GetType());
	if (CodeChunks.IsValidIndex(BodyChunk))
	{
		CodeChunks[BodyChunk].Original = Constant;
	}
	return BodyChunk;
}

int32 FNiagaraHlslTranslator::GetConstantDirect(float InConstantValue)
{
	FNiagaraVariable Constant(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Constant"));
	Constant.SetValue(InConstantValue);

	return GetConstant(Constant);
}

int32 FNiagaraHlslTranslator::GetConstantDirect(bool InConstantValue)
{
	FNiagaraVariable Constant(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Constant"));
	Constant.SetValue(InConstantValue);

	return GetConstant(Constant);
}

int32 FNiagaraHlslTranslator::GetConstantDirect(int InConstantValue)
{
	FNiagaraVariable Constant(FNiagaraTypeDefinition::GetIntDef(), TEXT("Constant"));
	Constant.SetValue(InConstantValue);

	return GetConstant(Constant);
}

bool FNiagaraHlslTranslator::GenerateStructInitializer(TStringBuilder<128>& InitializerString, UStruct* UserDefinedStruct, const void* StructData, int32 ByteOffset)
{
	//-TODO: Alignment Issues
	//-TODO: Also adjust automation test NiagaraTest_CustomLWCStructHlsl when this is fixed 
	//const bool bHasAlignmentDummy = CompileOptions.TargetUsage != ENiagaraScriptUsage::ParticleGPUComputeScript;

	//auto CheckAlignment =
	//	[&](FProperty* Property)
	//	{
	//		if ( bHasAlignmentDummy && (ByteOffset != Property->GetOffset_ReplaceWith_ContainerPtrToValuePtr()))
	//		{
	//			const int32 ByteDelta = Property->GetOffset_ReplaceWith_ContainerPtrToValuePtr() - ByteOffset;
	//			const int32 NumFloats = ByteDelta / sizeof(float);
	//			check(NumFloats > 0 && ByteDelta > 0);
	//			for (int32 i = 0; i < NumFloats; ++i)
	//			{
	//				InitializerString.Append(TEXT("0, "));
	//			}
	//			ByteOffset += ByteDelta;
	//		}
	//	};

	InitializerString.AppendChar(TEXT('{'));
	for (FField* ChildProperty = UserDefinedStruct->ChildProperties; ChildProperty; ChildProperty = ChildProperty->Next)
	{
		if (ChildProperty != UserDefinedStruct->ChildProperties)
		{
			InitializerString.AppendChar(TEXT(','));
		}

		if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(ChildProperty))
		{
			//CheckAlignment(FloatProperty);
			const float Value = FloatProperty->GetPropertyValue_InContainer(StructData);
			InitializerString.Appendf(TEXT("%g"), Value);
			ByteOffset += 4;
		}
		else if (FIntProperty* IntProperty = CastField<FIntProperty>(ChildProperty))
		{
			//CheckAlignment(IntProperty);
			const int32 Value = IntProperty->GetPropertyValue_InContainer(StructData);
			InitializerString.Appendf(TEXT("%d"), Value);
			ByteOffset += 4;
		}
		else if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(ChildProperty))
		{
			//CheckAlignment(BoolProperty);
			const bool bValue = BoolProperty->GetPropertyValue_InContainer(StructData);
			InitializerString.Append(bValue ? TEXT("true") : TEXT("false"));
			ByteOffset += 4;
		}
		else if (FStructProperty* StructProperty = CastField<FStructProperty>(ChildProperty))
		{
			//CheckAlignment(StructProperty);
			if ( !GenerateStructInitializer(InitializerString, StructProperty->Struct, StructProperty->ContainerPtrToValuePtr<const void>(StructData), ByteOffset) )
			{
				return false;
			}
		}
		else
		{
			Error(FText::Format(LOCTEXT("GenerateConstantStructInitializeTypeError", "Unknown type '{0}' member '{1}' in structure '{2}' when generating initializer struct."), FText::FromString(ChildProperty->GetClass()->GetName()), FText::FromString(ChildProperty->GetName()), FText::FromString(UserDefinedStruct->GetName())));
			return false;
		}
	}
	InitializerString.AppendChar(TEXT('}'));
	return true;
}

FString FNiagaraHlslTranslator::GenerateConstantString(const FNiagaraVariable& Constant)
{
	FNiagaraTypeDefinition Type = Constant.GetType();
	if (!AddStructToDefinitionSet(Type))
	{
		Error(FText::Format(LOCTEXT("GetConstantFail", "Cannot handle type {0}! Variable: {1}"), Type.GetNameText(), FText::FromName(Constant.GetName())));
	}
	FString ConstantStr = GetHlslDefaultForType(Type);

	if (Constant.IsDataAllocated())
	{
		if (Type == FNiagaraTypeDefinition::GetFloatDef())
		{
			float* ValuePtr = (float*)Constant.GetData();
			ConstantStr = FString::Printf(TEXT("%g"), *ValuePtr);
		}
		else if (Type == FNiagaraTypeDefinition::GetVec2Def())
		{
			float* ValuePtr = (float*)Constant.GetData();
			ConstantStr = FString::Printf(TEXT("float2(%g,%g)"), *ValuePtr, *(ValuePtr + 1));
		}
		else if (Type == FNiagaraTypeDefinition::GetVec3Def() || Type == FNiagaraTypeDefinition::GetPositionDef())
		{
			float* ValuePtr = (float*)Constant.GetData();
			ConstantStr = FString::Printf(TEXT("float3(%g,%g,%g)"), *ValuePtr, *(ValuePtr + 1), *(ValuePtr + 2));
		}
		else if (Type == FNiagaraTypeDefinition::GetVec4Def())
		{
			float* ValuePtr = (float*)Constant.GetData();
			ConstantStr = FString::Printf(TEXT("float4(%g,%g,%g,%g)"), *ValuePtr, *(ValuePtr + 1), *(ValuePtr + 2), *(ValuePtr + 3));
		}
		else if (Type == FNiagaraTypeDefinition::GetColorDef())
		{
			float* ValuePtr = (float*)Constant.GetData();
			ConstantStr = FString::Printf(TEXT("float4(%g,%g,%g,%g)"), *ValuePtr, *(ValuePtr + 1), *(ValuePtr + 2), *(ValuePtr + 3));
		}
		else if (Type == FNiagaraTypeDefinition::GetQuatDef())
		{
			float* ValuePtr = (float*)Constant.GetData();
			ConstantStr = FString::Printf(TEXT("float4(%g,%g,%g,%g)"), *ValuePtr, *(ValuePtr + 1), *(ValuePtr + 2), *(ValuePtr + 3));
		}
		else if (Type == FNiagaraTypeDefinition::GetIntDef() || Type.GetStruct() == FNiagaraTypeDefinition::GetIntStruct())
		{
			int32* ValuePtr = (int32*)Constant.GetData();
			ConstantStr = FString::Printf(TEXT("%d"), *ValuePtr);
		}
		else if (Type == FNiagaraTypeDefinition::GetMatrix4Def() )
		{
			float* ValuePtr = (float*)Constant.GetData();
			ConstantStr = FString::Printf(
				TEXT("float4x4(%g,%g,%g,%g, %g,%g,%g,%g, %g,%g,%g,%g, %g,%g,%g,%g)"),
				ValuePtr[ 0], ValuePtr[ 1], ValuePtr[ 2], ValuePtr[ 3],
				ValuePtr[ 4], ValuePtr[ 5], ValuePtr[ 6], ValuePtr[ 7],
				ValuePtr[ 8], ValuePtr[ 9], ValuePtr[10], ValuePtr[11],
				ValuePtr[12], ValuePtr[13], ValuePtr[14], ValuePtr[15]
			);
		}
		else if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
		{
			if (Constant.GetValue<FNiagaraBool>().IsValid() == false)
			{
				Error(FText::Format(LOCTEXT("StructContantsBoolInvalidError", "Boolean constant {0} is not set to explicit True or False. Defaulting to False."), FText::FromName(Constant.GetName())));
				ConstantStr = TEXT("false");
			}
			else
			{
				bool bValue = Constant.GetValue<FNiagaraBool>().GetValue();
				ConstantStr = bValue ? TEXT("true") : TEXT("false");
			}
		}
		else if (UStruct* UserDefinedStruct = Type.GetStruct())
		{
			TStringBuilder<128> InitializerString;
			if ( !GenerateStructInitializer(InitializerString, UserDefinedStruct, Constant.GetData()) )
			{
				Error(FText::Format(LOCTEXT("FailedToGenerateConstantInitialiezrError", "Type '{0}' constant '{1}' failed to create structure initializer. Defaulting to 0."), FText::FromString(Type.GetName()), FText::FromName(Constant.GetName())));
				return ConstantStr;
			}
			return InitializerString.ToString();
		}
		else
		{
			Warning(FText::Format(LOCTEXT("GenerateConstantUnknownTypeError", "Type '{0}' constant '{1}' is unknown.  Defaulting to 0."), FText::FromString(Type.GetName()), FText::FromName(Constant.GetName())));
			return ConstantStr;
		}
	}
	return ConstantStr;
}

bool FNiagaraHlslTranslationStage::ShouldDoSpawnOnlyLogic() const
{
	if (UNiagaraScript::IsSpawnScript(ScriptUsage))
	{
		return true;
	}
	if((ScriptUsage == ENiagaraScriptUsage::ParticleSimulationStageScript) && (ExecuteBehavior == ENiagaraSimStageExecuteBehavior::OnSimulationReset))
	{
		return true;
	}

	return false;
}

bool FNiagaraHlslTranslationStage::IsExternalConstantNamespace(const FNiagaraVariable& InVar, ENiagaraScriptUsage InTargetUsage, uint32 InTargetBitmask)
{
	if (FNiagaraParameterUtilities::IsExternalConstantNamespace(InVar, InTargetUsage, InTargetBitmask))
	{
		if (IterationSourceType == ENiagaraIterationSource::DataInterface && InVar.IsInNameSpace(IterationDataInterface))
			return false;
		else
			return true;
	}
	return false;
}

bool FNiagaraHlslTranslator::IsWriteAllowedForNamespace(const FNiagaraVariable& Var, ENiagaraScriptUsage TargetUsage, FText& ErrorMsg)
{
	if (UNiagaraScript::IsStandaloneScript(TargetUsage))
	{
		return true;
	}
	if (UNiagaraScript::IsSystemScript(TargetUsage) && (Var.IsInNameSpace(FNiagaraConstants::EmitterNamespace) || Var.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespace)))
	{
		ErrorMsg = FText::Format(LOCTEXT("WriteAllowedForNamespaceFail_System", "Cannot set variable {0} in system scripts."), FText::FromName(Var.GetName()));
		return false;
	}
	if (UNiagaraScript::IsEmitterScript(TargetUsage) && (Var.IsInNameSpace(FNiagaraConstants::SystemNamespace) || Var.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespace)))
	{
		ErrorMsg = FText::Format(LOCTEXT("WriteAllowedForNamespaceFail_Emitter", "Cannot set variable {0} in emitter scripts."), FText::FromName(Var.GetName()));
		return false;
	}
	if (UNiagaraScript::IsParticleScript(TargetUsage) && (Var.IsInNameSpace(FNiagaraConstants::SystemNamespace) || Var.IsInNameSpace(FNiagaraConstants::EmitterNamespace)))
	{
		const FNiagaraHlslTranslationStage& ActiveStage = TranslationStages[ActiveStageIdx];
		if (ActiveStage.IterationSourceType == ENiagaraIterationSource::DataInterface && Var.IsInNameSpace(ActiveStage.IterationDataInterface))
		{
			// weird special case where particle script can write to emitter data interface sub properties
			return true;
		}
		ErrorMsg = FText::Format(LOCTEXT("WriteAllowedForNamespaceFail_Particles", "Cannot set variable {0} in particle scripts."), FText::FromName(Var.GetName()));
		return false;
	}
	return true;
}

template<typename GraphBridge>
bool FNiagaraHlslTranslationStage::IsRelevantToSpawnForStage(const typename GraphBridge::FParamMapHistory& InHistory, const FNiagaraVariable& InAliasedVar, const FNiagaraVariable& InVar) const
{
	const FNiagaraVariableBase DataSetVariable = InAliasedVar.IsInNameSpace(FNiagaraConstants::StackContextNamespace) ? InVar : InAliasedVar;

	if (InHistory.IsPrimaryDataSetOutput(DataSetVariable, ScriptUsage) && (UNiagaraScript::IsSpawnScript(ScriptUsage) || bShouldUpdateInitialAttributeValues))
	{
		return true;
	}

	if ((ScriptUsage == ENiagaraScriptUsage::ParticleSimulationStageScript) && (ExecuteBehavior == ENiagaraSimStageExecuteBehavior::OnSimulationReset))
	{
		switch (IterationSourceType)
		{
			case ENiagaraIterationSource::Particles:		return InHistory.IsPrimaryDataSetOutput(DataSetVariable, ENiagaraScriptUsage::EmitterSpawnScript);
			case ENiagaraIterationSource::DataInterface:	return InVar.IsInNameSpace(IterationDataInterface) && !InVar.IsDataInterface();
			case ENiagaraIterationSource::DirectSet:		return false;
			default:										check(false);
		}
	}
	return false;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::InitializeParameterMapDefaults(int32 ParamMapHistoryIdx)
{
	bInitializedDefaults = true;
	AddBodyComment(TEXT("//Begin Initialize Parameter Map Defaults"));
	check(ParamMapHistories.Num() == TranslationStages.Num());

	UniqueVars.Empty();
	UniqueVarToDefaultPin.Empty();
	UniqueVarToWriteToParamMap.Empty();
	UniqueVarToChunk.Empty();
	UniqueVarToAliasedVar.Empty();

	FNiagaraHlslTranslationStage& ActiveStage = TranslationStages[ActiveStageIdx];
	// First pass just use the current parameter map.
	{
		const typename GraphBridge::FParamMapHistory& History = ParamMapHistories[ParamMapHistoryIdx];
		for (int32 i = 0; i < History.Variables.Num(); i++)
		{
			const FNiagaraVariable& Var = History.Variables[i];
			const FNiagaraVariable& AliasedVar = History.VariablesWithOriginalAliasesIntact[i];
			// Only add primary data set outputs at the top of the script if in a spawn script, otherwise they should be left alone.
			if (TranslationStages[ActiveStageIdx].ShouldDoSpawnOnlyLogic() || TranslationStages[ActiveStageIdx].bShouldUpdateInitialAttributeValues)
			{
				if (TranslationStages[ActiveStageIdx].template IsRelevantToSpawnForStage<GraphBridge>(History, AliasedVar, Var) &&
					!UniqueVars.Contains(Var))
				{
					UniqueVars.Add(Var);
					UniqueVarToAliasedVar.Add(Var, AliasedVar);
					const FInputPin* DefaultPin = History.GetDefaultValuePin(i);
					UniqueVarToDefaultPin.Add(Var, DefaultPin);
					UniqueVarToWriteToParamMap.Add(Var, true);
				}
			}
		}
	}

	// Only add primary data set outputs at the top of the script if in a spawn script, otherwise they should be left alone.
	// Above we added all the known from the spawn script, now let's add for all the others.
	if (TranslationStages[ActiveStageIdx].ShouldDoSpawnOnlyLogic() || TranslationStages[ActiveStageIdx].bShouldUpdateInitialAttributeValues)
	{
		// Go through all referenced parameter maps and pull in any variables that are 
		// in the primary data set output namespaces.
		for (int32 ParamMapIdx = 0; ParamMapIdx < OtherOutputParamMapHistories.Num(); ParamMapIdx++)
		{
			const typename GraphBridge::FParamMapHistory& History = OtherOutputParamMapHistories[ParamMapIdx];
			for (int32 i = 0; i < History.Variables.Num(); i++)
			{
				const FNiagaraVariable& Var = History.Variables[i];
				const FNiagaraVariable& AliasedVar = History.VariablesWithOriginalAliasesIntact[i];
				if (TranslationStages[ActiveStageIdx].template IsRelevantToSpawnForStage<GraphBridge>(History, AliasedVar, Var) &&
					!UniqueVars.Contains(Var))
				{
					UniqueVars.Add(Var);
					UniqueVarToAliasedVar.Add(Var, AliasedVar);
					const FInputPin* DefaultPin = History.GetDefaultValuePin(i);
					UniqueVarToDefaultPin.Add(Var, DefaultPin);
					UniqueVarToWriteToParamMap.Add(Var, false);
				}
			}
		}

		// Now sort them into buckets: Defined by constants (write immediately), Defined as initial values (delay to end),
		// or defined by linkage or other script (defer to end if not originating from spawn, otherwise insert before first use)
		for (FNiagaraVariable& Var : UniqueVars)
		{
			const FInputPin* DefaultPin = UniqueVarToDefaultPin.FindChecked(Var);
			bool bWriteToParamMapEntries = UniqueVarToWriteToParamMap.FindChecked(Var);
			int32 OutputChunkId = INDEX_NONE;
			
			TOptional<ENiagaraDefaultMode> DefaultMode;
			FNiagaraScriptVariableBinding DefaultBinding;
			const FNode* DefaultPinNode = nullptr;

			if (DefaultPin) 
			{
				DefaultPinNode = GraphBridge::GetOwningNode(DefaultPin);
				if (const FGraph* DefaultPinGraph = GraphBridge::GetOwningGraph(DefaultPinNode))
				{
					const FNiagaraVariableBase* AliasedVar = UniqueVarToAliasedVar.Find(Var);
					DefaultMode = GraphBridge::GetGraphDefaultMode(DefaultPinGraph, AliasedVar ? *AliasedVar : FNiagaraVariableBase(Var), DefaultBinding);
				}
			}

			// During the initial pass, only support constants for the default pin and non-bound variables
			const bool bDefaultPinConnected = GraphBridge::GetLinkedOutputPin(DefaultPin) != nullptr;

			if (!FNiagaraParameterUtilities::IsInitialValue(Var) && !bDefaultPinConnected && !(DefaultMode.IsSet() && (*DefaultMode == ENiagaraDefaultMode::Binding || *DefaultMode == ENiagaraDefaultMode::FailIfPreviouslyNotSet)))
			{
				HandleParameterRead(ParamMapHistoryIdx, Var, DefaultPin, DefaultPinNode, OutputChunkId, TOptional<ENiagaraDefaultMode>(), TOptional<FNiagaraScriptVariableBinding>(), !bWriteToParamMapEntries);
				UniqueVarToChunk.Add(Var, OutputChunkId);
			}
			else if (FNiagaraParameterUtilities::IsInitialValue(Var))
			{
				FNiagaraVariable SourceForInitialValue = FNiagaraParameterUtilities::GetSourceForInitialValue(Var);
				if (!UniqueVars.Contains(SourceForInitialValue))
				{
					//@todo(ng) disabled pending investigation UE-150159
					//Error(FText::Format(LOCTEXT("MissingInitialValueSource", "Variable {0} is used, but its source variable {1} is not set!"), FText::FromName(Var.GetName()), FText::FromName(SourceForInitialValue.GetName())), nullptr, nullptr);
				}
				InitialNamespaceVariablesMissingDefault.Add(Var);
			}
			else
			{
				DeferredVariablesMissingDefault.Add(Var);
			}
		}
	}

	AddBodyComment(TEXT("//End Initialize Parameter Map Defaults"));
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Output(const FOutputNode* OutputNode, const TArray<int32>& ComputedInputs)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_Output);

	TArray<FNiagaraVariable> Attributes;
	TArray<int32> Inputs;

	// Build up the attribute list. We don't auto-expand parameter maps here.
	const TArray<FNiagaraVariable>& Outputs = GraphBridge::GetOutputVariables(OutputNode);
	int32 NumberOfValidComputedInputs = 0;
	for (int32 ComputedInput : ComputedInputs)
	{
		if (ComputedInput != INDEX_NONE)
		{
			NumberOfValidComputedInputs++;
		}
	}
	check(NumberOfValidComputedInputs == Outputs.Num());
	for (int32 PinIdx = 0; PinIdx < Outputs.Num(); PinIdx++)
	{
		Attributes.Add(ConvertToSimulationVariable(Outputs[PinIdx]));
		Inputs.Add(ComputedInputs[PinIdx]);
	}

	if (FunctionCtx())
	{
		for (int32 i = 0; i < Attributes.Num(); ++i)
		{
			if (!AddStructToDefinitionSet(Attributes[i].GetType()))
			{
				Error(FText::Format(LOCTEXT("GetConstantFail", "Cannot handle type {0}! Variable: {1}"), Attributes[i].GetType().GetNameText(), FText::FromName(Attributes[i].GetName())));
			}

			if (Attributes[i].GetType() != FNiagaraTypeDefinition::GetParameterMapDef())
			{
				FString SymbolName = *BuildSanitizedSymbolString(TEXT("Out_") + Attributes[i].GetName().ToString());
				ENiagaraCodeChunkMode OldMode = CurrentBodyChunkMode;
				CurrentBodyChunkMode = ENiagaraCodeChunkMode::Body;
				AddBodyChunk(SymbolName, TEXT("{0}"), Attributes[i].GetType(), Inputs[i], false);
				CurrentBodyChunkMode = OldMode;
			}
		}
	}
	else
	{

		{
			check(InstanceWrite.CodeChunks.Num() == 0);//Should only hit one output node.

			FString DataSetAccessName = GetDataSetAccessSymbol(GetInstanceDataSetID(), INDEX_NONE, false);
			//First chunk for a write is always the condition pin.
			for (int32 i = 0; i < Attributes.Num(); ++i)
			{
				const FNiagaraVariable& Var = Attributes[i];

				if (!AddStructToDefinitionSet(Var.GetType()))
				{
					Error(FText::Format(LOCTEXT("GetConstantFail", "Cannot handle type {0}! Variable: {1}"), Var.GetType().GetNameText(), FText::FromName(Var.GetName())));
				}

				//DATASET TODO: add and treat input 0 as the 'valid' input for conditional write
				int32 Input = Inputs[i];


				if (Var.GetType() != FNiagaraTypeDefinition::GetParameterMapDef())
				{
					FNiagaraVariable VarNamespaced = FNiagaraParameterUtilities::BasicAttributeToNamespacedAttribute(Var);
					FString ParameterMapInstanceName = GetParameterMapInstanceName(0);
					int32 ChunkIdx = AddBodyChunk(ParameterMapInstanceName + TEXT(".") + BuildSanitizedSymbolName(VarNamespaced.GetName()), TEXT("{0}"), VarNamespaced.GetType(), Input, false);

					// Make sure that we end up in the list of Attributes that have been written to by this script.
					if (ParamMapDefinedAttributesToUniformChunks.Find(Var.GetName()) == nullptr)
					{
						ParamMapDefinedAttributesToUniformChunks.Add(Var.GetName(), Input);
						FVarAndDefaultSource VarAndDefaultSource;
						VarAndDefaultSource.Variable = VarNamespaced;
						VarAndDefaultSource.bDefaultExplicit = false;
						ParamMapDefinedAttributesToNamespaceVars.Add(Var.GetName(), VarAndDefaultSource);
					}

					InstanceWrite.Variables.AddUnique(VarNamespaced);
					InstanceWrite.CodeChunks.Add(ChunkIdx);
				}
				else
				{
					InstanceWrite.Variables.AddUnique(Var);
				}
			}
		}
	}
}

int32 FNiagaraHlslTranslator::GetAttribute(const FNiagaraVariable& Attribute)
{
	if (!AddStructToDefinitionSet(Attribute.GetType()))
	{
		Error(FText::Format(LOCTEXT("GetConstantFail", "Cannot handle type {0}! Variable: {1}"), Attribute.GetType().GetNameText(), FText::FromName(Attribute.GetName())));
	}

	if (TranslationStages.Num() > 1 && UNiagaraScript::IsParticleSpawnScript(TranslationStages[0].ScriptUsage) && (Attribute.GetName() != TEXT("Particles.UniqueID")))
	{
		if (ActiveStageIdx > 0)
		{
			//This is a special case where we allow the grabbing of attributes in the update section of an interpolated spawn script.
			//But we return the results of the previously ran spawn script.
			FString ParameterMapInstanceName = GetParameterMapInstanceName(0);

			FNiagaraVariable NamespacedVar = Attribute;
			FString SymbolName = *(ParameterMapInstanceName + TEXT(".") + BuildSanitizedSymbolName(NamespacedVar.GetName()));
			return AddSourceChunk(SymbolName, Attribute.GetType());
		}
		else
		{
			Error(LOCTEXT("AttrReadInSpawnError", "Cannot read attribute in a spawn script as it's value is not yet initialized."));
			return INDEX_NONE;
		}
	}
	else
	{
		// NOTE(mv): Explicitly allow reading from Particles.UniqueID, as it is an engine managed variable and 
		//           is written to before Simulate() in the SpawnScript...
		// TODO(mv): Also allow Particles.ID for the same reasons?
		CompilationOutput.ScriptData.DataUsage.bReadsAttributeData |= (Attribute.GetName() != TEXT("Particles.UniqueID"));

		int32 Chunk = INDEX_NONE;
		if (!ParameterMapRegisterNamespaceAttributeVariable(Attribute, 0, Chunk))
		{
			Error(FText::Format(LOCTEXT("AttrReadError", "Cannot read attribute {0} {1}."), Attribute.GetType().GetNameText(), FText::FromString(*Attribute.GetName().ToString())));
			return INDEX_NONE;
		}
		return Chunk;
	}
}

FString FNiagaraHlslTranslator::GetDataSetAccessSymbol(FNiagaraDataSetID DataSet, int32 IndexChunk, bool bRead)
{
	FString Ret = TEXT("\tContext.") + DataSet.Name.ToString() + (bRead ? TEXT("Read") : TEXT("Write"));
	/*
	FString Ret = TEXT("Context.") + DataSet.Name.ToString();
	Ret += bRead ? TEXT("Read") : TEXT("Write");
	Ret += IndexChunk != INDEX_NONE ? TEXT("_") + CodeChunks[IndexChunk].SymbolName : TEXT("");*/
	return Ret;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ParameterMapForBegin(const FParamMapForNode* ForNode, int32 IterationCount)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_MapForBegin);

	const int32 IndexChunkIndex = AddBodyChunk(GetUniqueSymbolName(TEXT("Index")), TEXT(""), FNiagaraTypeDefinition::GetIntDef(), true);
	ParameterMapForIndexStack.Push(IndexChunkIndex);

	TArray<int32> SourceChunks;
	SourceChunks.Add(IndexChunkIndex);
	SourceChunks.Add(IterationCount);

	AddBodyChunk(TEXT(""), TEXT("for({0} = 0; {0} < {1}; ++{0})\n\t{"), FNiagaraTypeDefinition::GetIntDef(), SourceChunks, false, false);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ParameterMapForContinue(const FParamMapForNode* ForNode, int32 IterationEnabled)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_MapForBegin);

	AddBodyChunk(TEXT(""), TEXT("if (!{0}) continue;"), FNiagaraTypeDefinition::GetBoolDef(), IterationEnabled, false, false);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ParameterMapForEnd(const FParamMapForNode* ForNode)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_MapForEnd);

	AddBodyChunk(TEXT(""), TEXT("}"), FNiagaraTypeDefinition::GetIntDef(), false, false);

	ParameterMapForIndexStack.Pop();
}

int32 FNiagaraHlslTranslator::ParameterMapForInnerIndex() const
{
	if (ParameterMapForIndexStack.Num())
	{
		return ParameterMapForIndexStack.Last();
	}

	return INDEX_NONE;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ParameterMapSet(const FParamMapSetNode* SetNode, TArrayView<const FCompiledPin> Inputs, TArray<int32>& Outputs)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_MapSet);

	Outputs.SetNum(1);

	FString ParameterMapInstanceName = TEXT("Context.Map");

	// There is only one output pin for a set node, the parameter map must 
	// continue to route through it.
	if (!GraphBridge::NodeIsEnabled(SetNode))
	{
		if (Inputs.Num() >= 1)
		{
			Outputs[0] = Inputs[0].CompilationIndex;
		}
		return;
	}

	int32 ParamMapHistoryIdx = INDEX_NONE;
	for (int32 i = 0; i < Inputs.Num(); i++)
	{
		int32 Input = Inputs[i].CompilationIndex;
		if (i == 0) // This is the parameter map
		{
			Outputs[0] = Inputs[0].CompilationIndex;
			ParamMapHistoryIdx = Inputs[0].CompilationIndex;
			ParameterMapInstanceName = GetParameterMapInstanceName(ParamMapHistoryIdx);

			if (ParamMapHistoryIdx == -1)
			{
				Error(LOCTEXT("NoParamMapIdxForInput", "Cannot find parameter map for input!"), SetNode, nullptr);
				Outputs[0] = INDEX_NONE;
					return;
			}
			continue;
		}
		else // These are the pins that we are setting on the parameter map.
		{
			FNiagaraVariable Var = GraphBridge::GetPinVariable(Inputs[i].Pin, false, ENiagaraStructConversion::Simulation);

			if (!AddStructToDefinitionSet(Var.GetType()))
			{
				Error(FText::Format(LOCTEXT("ParameterMapSetTypeError", "Cannot handle type {0}! Variable: {1}"), Var.GetType().GetNameText(), FText::FromName(Var.GetName())), SetNode, Inputs[i].Pin);
			}

			if (TranslationStages[ActiveStageIdx].IsExternalConstantNamespace(Var, CompileOptions.TargetUsage, CompileOptions.GetTargetUsageBitmask()))
			{
				Error(FText::Format(LOCTEXT("SetSystemConstantFail", "Cannot Set external constant, Type: {0} Variable: {1}"), Var.GetType().GetNameText(), FText::FromName(Var.GetName())), SetNode, Inputs[i].Pin);
				continue;
			}

			FText WriteErrorText;
			if (!IsWriteAllowedForNamespace(Var, ActiveHistoryForFunctionCalls.GetCurrentUsageContext(), WriteErrorText))
			{
				const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
				if (Settings->InvalidNamespaceWriteSeverity == ENiagaraCompileErrorSeverity::Error)
				{
					Error(WriteErrorText, SetNode, Inputs[i].Pin);
				}
				else if (Settings->InvalidNamespaceWriteSeverity == ENiagaraCompileErrorSeverity::Warning)
				{
					Warning(WriteErrorText, SetNode, Inputs[i].Pin);
				}
				else if (Settings->InvalidNamespaceWriteSeverity == ENiagaraCompileErrorSeverity::LogOnly)
				{
					Message(FNiagaraCompileEventSeverity::Log, WriteErrorText, SetNode, Inputs[i].Pin);
				}
				continue;
			}

			Var = ActiveHistoryForFunctionCalls.ResolveAliases(Var);
			
			const FNiagaraKnownConstantInfo ConstantInfo = FNiagaraConstants::GetKnownConstantInfo(Var.GetName(), false);
			if (ConstantInfo.ConstantVar != nullptr && ConstantInfo.ConstantVar->GetType() != Var.GetType() && ConstantInfo.ConstantType != ENiagaraKnownConstantType::Attribute)
			{
				Error(FText::Format(LOCTEXT("MismatchedConstantTypes", "Variable {0} is a system constant, but its type is different! {1} != {2}"), FText::FromName(Var.GetName()),
					ConstantInfo.ConstantVar->GetType().GetNameText(), Var.GetType().GetNameText()), SetNode, Inputs[i].Pin);
			}

			if (FNiagaraConstants::IsEngineManagedAttribute(Var))
			{
				Error(FText::Format(LOCTEXT("SettingSystemAttr", "Variable {0} is an engine managed particle attribute and cannot be set directly."), FText::FromName(Var.GetName())), SetNode, Inputs[i].Pin);
				continue;
			}


			if (ParamMapHistoryIdx < ParamMapHistories.Num())
			{
				int32 VarIdx = ParamMapHistories[ParamMapHistoryIdx].FindVariableByName(Var.GetName());
				if (VarIdx != INDEX_NONE && VarIdx < ParamMapSetVariablesToChunks[ParamMapHistoryIdx].Num())
				{
					ParamMapSetVariablesToChunks[ParamMapHistoryIdx][VarIdx] = Inputs[i].CompilationIndex;
					RecordParamMapDefinedAttributeToNamespaceVar(Var, Inputs[i].Pin);
					if (!Var.GetType().IsStatic())
					{
						// Note that data interfaces aren't ever in the primary data set even if the namespace matches.)
						if (ParamMapHistories[ParamMapHistoryIdx].IsPrimaryDataSetOutput(Var, GetTargetUsage()))
						{
							CompilationOutput.ScriptData.AttributesWritten.AddUnique(Var);
						}
						else if (ParamMapHistories[ParamMapHistoryIdx].IsVariableFromCustomIterationNamespaceOverride(Var))
						{
							CompilationOutput.ScriptData.AttributesWritten.AddUnique(Var);
						}
					}
					else 
					{
						if (Var.GetType().IsStatic())
						{	
							bool bAllSame = true;
							int32 FoundOverrideIdx = CompileData->StaticVariablesWithMultipleWrites.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
								{
									return (InObj.GetName() == Var.GetName());
								});;

							if (FoundOverrideIdx != INDEX_NONE)
							{
								bAllSame = false;
							}
							else if (FoundOverrideIdx == INDEX_NONE && FNiagaraParameterUtilities::IsAliasedEmitterParameter(Var))
							{
								FNiagaraAliasContext ResolveAliasesContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript);
								ResolveAliasesContext.ChangeEmitterToEmitterName(CompileData->GetUniqueEmitterName());
								FNiagaraVariable TestEmitterResolvedVar = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);

								FoundOverrideIdx = CompileData->StaticVariablesWithMultipleWrites.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
									{
										return (InObj.GetName() == TestEmitterResolvedVar.GetName());
									});;

								if (FoundOverrideIdx != INDEX_NONE)
								{
									bAllSame = false;
								}
							}

							if (!bAllSame)
							{
								Error(FText::Format(LOCTEXT("ParameterMapStaticMultipleWriteErrorFormat", "Static variable is not set to a consistent value. Please ensure that all values are equal.  Parameter: {0}"), FText::FromName(Var.GetName())), SetNode, Inputs[i].Pin);
							}
						}

						if (ParamMapHistories[ParamMapHistoryIdx].IsPrimaryDataSetOutput(Var, GetTargetUsage(), true, true)) // Note that data interfaces aren't ever in the primary data set even if the namespace matches.)
						{
							FString DebugStr;
							FNiagaraVariable StaticVersionOfVar = Var;
							SetConstantByStaticVariable(StaticVersionOfVar, Inputs[i].Pin, &DebugStr);
							AddBodyComment(TEXT("//SetConstantByStaticVariable \"") + DebugStr + TEXT("\""));
							CompilationOutput.ScriptData.StaticVariablesWritten.AddUnique(StaticVersionOfVar);
						}
					}
				}
			}

			if (Var.IsDataInterface())
			{
				if (CompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated && TranslationStages[ActiveStageIdx].ScriptUsage == ENiagaraScriptUsage::ParticleUpdateScript)
				{
					// We don't want to add writes for particle update data interface parameters in both interpolated spawn and update, so skip them when processing the update stage of the 
					// interpolated spawn script.  We don't skip the writes when compiling the particle update script because it's not recompiled when the interpolated spawn flag is changed
					// and this would result in missing data interfaces if interpolated spawn was turned off.
					continue;
				}

				bool bAllowDataInterfaces = true;
				if (ParamMapHistoryIdx < ParamMapHistories.Num() && ParamMapHistories[ParamMapHistoryIdx].IsPrimaryDataSetOutput(Var, CompileOptions.TargetUsage, bAllowDataInterfaces))
				{
					if (Input < 0 || Input >= CompilationOutput.ScriptData.DataInterfaceInfo.Num())
					{
						Error(FText::Format(LOCTEXT("ParameterMapDataInterfaceNotFoundErrorFormat", "Data interface could not be found for parameter map set.  Parameter: {0}"), FText::FromName(Var.GetName())), SetNode, Inputs[i].Pin);
						continue;
					}

					FName UsageName;
					if (FNiagaraParameterUtilities::IsAliasedEmitterParameter(Var))
					{
						FNiagaraVariable AliasedVar = ActiveHistoryForFunctionCalls.ResolveAliases(Var);
						UsageName = AliasedVar.GetName();
					}
					else
					{
						UsageName = Var.GetName();
					}

					FNiagaraScriptDataInterfaceCompileInfo& Info = CompilationOutput.ScriptData.DataInterfaceInfo[Input];
					if (Info.RegisteredParameterMapWrite == NAME_None)
					{
						Info.RegisteredParameterMapWrite = UsageName;
					}
					else
					{
						// This is a special case where a single data interface is written to multiple parameters.  In order to handle this correctly the data interface
						// needs it's own registration slot so that it ends up in the parameter store under the correct name so that it can be looked up correctly in various places
						// and can be bound properly to the renderer parameter stores.
						UNiagaraDataInterface* CDO = CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(const_cast<UClass*>(Var.GetType().GetClass()));
						if (CDO)
						{
							FString PlaceholderName = FString::Printf(TEXT("Placeholder.Placeholder%i"), CompilationOutput.ScriptData.DataInterfaceInfo.Num());
							FNiagaraVariable PlaceholderVar(Var.GetType(), *PlaceholderName);
							int32 PlaceholderIndex = RegisterDataInterface(PlaceholderVar, CDO, true, false);
							CompilationOutput.ScriptData.DataInterfaceInfo[PlaceholderIndex].RegisteredParameterMapRead = CompilationOutput.ScriptData.DataInterfaceInfo[Input].RegisteredParameterMapWrite;
							CompilationOutput.ScriptData.DataInterfaceInfo[PlaceholderIndex].RegisteredParameterMapWrite = UsageName;
						}
					}
				}
			}
			else if ( Var.IsUObject() )
			{
				if (CompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated && TranslationStages[ActiveStageIdx].ScriptUsage == ENiagaraScriptUsage::ParticleUpdateScript)
				{
					// We don't want to add writes for particle update data interface parameters in both interpolated spawn and update, so skip them when processing the update stage of the 
					// interpolated spawn script.  We don't skip the writes when compiling the particle update script because it's not recompiled when the interpolated spawn flag is changed
					// and this would result in missing data interfaces if interpolated spawn was turned off.
					continue;
				}

				const bool bAllowDataInterfaces = true;	// Also allows UObjects
				if (ParamMapHistoryIdx < ParamMapHistories.Num() && ParamMapHistories[ParamMapHistoryIdx].IsPrimaryDataSetOutput(Var, CompileOptions.TargetUsage, bAllowDataInterfaces))
				{
					if (Input < 0 || Input >= CompilationOutput.ScriptData.UObjectInfos.Num())
					{
						Error(FText::Format(LOCTEXT("ParameterMapUObjectNotFoundErrorFormat", "UObject could not be found for parameter map set.  Parameter: {0}"), FText::FromName(Var.GetName())), SetNode, Inputs[i].Pin);
						continue;
					}

					FName ResolvedName;
					if (FNiagaraParameterUtilities::IsAliasedEmitterParameter(Var))
					{
						ResolvedName = ActiveHistoryForFunctionCalls.ResolveAliases(Var).GetName();
					}
					else
					{
						ResolvedName = Var.GetName();
					}

					// Ensure we don't have duplicate parameter map writes for the same parameter as it would be meaningless
					for (int32 iObjectInfo=0; iObjectInfo < CompilationOutput.ScriptData.UObjectInfos.Num(); ++iObjectInfo)
					{
						if (iObjectInfo == Input)
						{
							continue;
						}
						const FNiagaraScriptUObjectCompileInfo& ObjectInfo = CompilationOutput.ScriptData.UObjectInfos[iObjectInfo];
						if (ObjectInfo.RegisteredParameterMapWrites.Contains(ResolvedName))
						{
							Error(
								FText::Format(
									LOCTEXT("UObjectVariableWroteMoreThanOnce", "Object variable '{0}' has been wrote to more than once. This is not supported."),
									FText::FromName(ResolvedName)
								),
								nullptr, nullptr
							);
							break;
						}
					}

					FNiagaraScriptUObjectCompileInfo& Info = CompilationOutput.ScriptData.UObjectInfos[Input];
					Info.RegisteredParameterMapWrites.AddUnique(ResolvedName);

					//-TODO: We don't support Var=NPC.Var currently this requires some runtime remapping of the parameter stores
					if (Info.Variable.IsInNameSpace(FNiagaraConstants::ParameterCollectionNamespaceString))
					{
						Error(
							FText::Format(
								LOCTEXT("UObjectParameterInvalidNamespaceFormat", "Variable Write '{0}' can not read NPC variable '{1}'. This is not supported currently."),
								FText::FromName(ResolvedName),
								FText::FromName(Info.Variable.GetName())
							),
							nullptr, nullptr
						);
						continue;
					}
				}
			}
			else
			{
				if (Var == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("DataInstance.Alive")))
				{
					const int32 OutputStageIndex = TranslationStages[ActiveStageIdx].SimulationStageIndex;
					if ( CompilationOutput.ScriptData.SimulationStageMetaData.IsValidIndex(OutputStageIndex) )
					{
						CompilationOutput.ScriptData.SimulationStageMetaData[OutputStageIndex].bWritesParticles = true;
						CompilationOutput.ScriptData.SimulationStageMetaData[OutputStageIndex].bPartialParticleUpdate = false;
					}

					TranslationStages[ActiveStageIdx].bWritesParticles = true;
					TranslationStages[ActiveStageIdx].bPartialParticleUpdate = false;
					TranslationStages[ActiveStageIdx].bWritesAlive = true;
				}
				AddBodyChunk(ParameterMapInstanceName + TEXT(".") + BuildSanitizedSymbolName(Var.GetName()), TEXT("{0}"), Var.GetType(), Input, false);
			}
		}
	}
}

FString FNiagaraHlslTranslator::GetUniqueEmitterName() const
{
	if (CompileOptions.TargetUsage == ENiagaraScriptUsage::SystemSpawnScript || CompileOptions.TargetUsage == ENiagaraScriptUsage::SystemUpdateScript)
	{
		return FString(GetEmitterUniqueName());
	}
	else
	{
		return TEXT("Emitter");
	}
}

bool FNiagaraHlslTranslator::IsBulkSystemScript() const
{
	return (CompileOptions.TargetUsage == ENiagaraScriptUsage::SystemSpawnScript || CompileOptions.TargetUsage == ENiagaraScriptUsage::SystemUpdateScript);
}

bool FNiagaraHlslTranslator::IsSpawnScript() const
{
	for (int32 i = 0; i < TranslationStages.Num(); i++)
	{
		if (UNiagaraScript::IsSpawnScript(TranslationStages[i].ScriptUsage))
		{
			return true;
		}
	}
	return false;
}

bool FNiagaraHlslTranslator::IsEventSpawnScript()const
{
	return UNiagaraScript::IsParticleEventScript(CompileOptions.TargetUsage) && CompileOptions.AdditionalDefines.Contains(FNiagaraCompileOptions::EventSpawnDefine);
}

bool FNiagaraHlslTranslator::RequiresInterpolation() const
{
	for (int32 i = 0; i < TranslationStages.Num(); i++)
	{
		if (TranslationStages[i].InterpolatedSpawnMode != ENiagaraInterpolatedSpawnMode::NoInterpolation)
		{
			return true;
		}
	}
	return false;
}

template<typename GraphBridge>
bool TNiagaraHlslTranslator<GraphBridge>::GetLiteralConstantVariable(FNiagaraVariable& OutVar) const
{
	if (FNiagaraParameterUtilities::IsInNamespace(OutVar, PARAM_MAP_EMITTER_STR) || FNiagaraParameterUtilities::IsInNamespace(OutVar, PARAM_MAP_SYSTEM_STR))
	{
		FNiagaraVariable ResolvedVar = ActiveHistoryForFunctionCalls.ResolveAliases(OutVar);
		if (OutVar == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.Localspace")))
		{
			bool bEmitterLocalSpace = CompileOptions.AdditionalDefines.Contains(ResolvedVar.GetName().ToString());
			OutVar.SetValue(bEmitterLocalSpace ? FNiagaraBool(true) : FNiagaraBool(false));
			return true;
		}
		if (OutVar == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.Determinism")))
		{
			bool bEmitterDeterminism = CompileOptions.AdditionalDefines.Contains(ResolvedVar.GetName().ToString());
			OutVar.SetValue(bEmitterDeterminism ? FNiagaraBool(true) : FNiagaraBool(false));
			return true;
		}
		if (OutVar == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.InterpolatedSpawn")))
		{
			bool bEmitterInterpolatedSpawn = CompileOptions.AdditionalDefines.Contains(ResolvedVar.GetName().ToString());
			OutVar.SetValue(bEmitterInterpolatedSpawn ? FNiagaraBool(true) : FNiagaraBool(false));
			return true;
		}
		if (OutVar == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.OverrideGlobalSpawnCountScale")))
		{
			bool bOverrideGlobalSpawnCountScale = CompileOptions.AdditionalDefines.Contains(ResolvedVar.GetName().ToString());
			OutVar.SetValue(bOverrideGlobalSpawnCountScale ? FNiagaraBool(true) : FNiagaraBool(false));
			return true;
		}
		if (OutVar == FNiagaraVariable(FNiagaraTypeDefinition::GetSimulationTargetEnum(), TEXT("Emitter.SimulationTarget")))
		{
			FNiagaraInt32 EnumValue;
			EnumValue.Value = CompilationTarget == ENiagaraSimTarget::GPUComputeSim || CompileOptions.AdditionalDefines.Contains(TEXT("GPUComputeSim")) ? 1 : 0;
			OutVar.SetValue(EnumValue);
			return true;
		}
	}
	else if (OutVar == FNiagaraVariable(FNiagaraTypeDefinition::GetScriptUsageEnum(), TEXT("Script.Usage")))
	{
		ENiagaraScriptUsage Usage = TranslationStages[ActiveStageIdx].ScriptUsage;
		FNiagaraInt32 EnumValue;
		EnumValue.Value = (uint8)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchUsage(Usage);
		OutVar.SetValue(EnumValue);
		return true;
	}
	else if (OutVar == FNiagaraVariable(FNiagaraTypeDefinition::GetScriptContextEnum(), TEXT("Script.Context")))
	{
		ENiagaraScriptUsage Usage = GetCurrentUsage();
		FNiagaraInt32 EnumValue;
		EnumValue.Value = (uint8)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchContext(Usage);
		OutVar.SetValue(EnumValue);
		return true;
	}
	else if (OutVar == SYS_PARAM_ENGINE_EMITTER_SIMULATION_POSITION) 
	{
		FNiagaraVariable ResolvedLocalSpaceCompileOptionVar = ActiveHistoryForFunctionCalls.ResolveAliases(SYS_PARAM_EMITTER_LOCALSPACE);
		if (CompileOptions.AdditionalDefines.Contains(ResolvedLocalSpaceCompileOptionVar.GetName().ToString()))
		{
			OutVar.SetValue(FVector3f(ForceInitToZero));
			return true;
		}
	}
	return false;
}

template<typename GraphBridge>
bool TNiagaraHlslTranslator<GraphBridge>::HandleBoundConstantVariableToDataSetRead(FNiagaraVariable InVariable, const FNode* InNode, int32 InParamMapHistoryIdx, int32& Output, const FPin* InDefaultPin)
{
	if (InVariable == SYS_PARAM_ENGINE_EMITTER_SIMULATION_POSITION)
	{
		// Simulation position is 0 for localspace emitters.
		// If we are not in localspace then this will not be a literal constant and is instead a default linked variable as handled in GenerateConstantString().
		// If we are in localspace, interpret Engine.Emitter.SimulationPosition and Engine.Owner.Position and handle via ParameterMapRegisterExternalConstantNamespaceVariable.
		FNiagaraVariable ResolvedLocalSpaceCompileOptionVar = ActiveHistoryForFunctionCalls.ResolveAliases(SYS_PARAM_EMITTER_LOCALSPACE);
		const bool bIsEmitterLocalSpaceCompileOptionSet = CompileOptions.AdditionalDefines.Contains(ResolvedLocalSpaceCompileOptionVar.GetName().ToString());

		if (bIsEmitterLocalSpaceCompileOptionSet == false)
		{
			return ParameterMapRegisterExternalConstantNamespaceVariable(SYS_PARAM_ENGINE_POSITION, InNode, InParamMapHistoryIdx, Output, InDefaultPin);
		}
	}
	return false;
}

template<typename GraphBridge>
bool TNiagaraHlslTranslator<GraphBridge>::ParameterMapRegisterExternalConstantNamespaceVariable(FNiagaraVariable InVariable, const FNode* InNodeForErrorReporting, int32 InParamMapHistoryIdx, int32& Output, const FPin* InDefaultPin)
{
	Output = INDEX_NONE;

	if (!ensure(InParamMapHistoryIdx >= 0))
	{
		return false;
	}

	const bool bIsEngineProvidedValue = ParamMapDefinedEngineVars.Contains(InVariable);//These values are provided by the engine/translator and do not need to be read as a constant

	InVariable = ActiveHistoryForFunctionCalls.ResolveAliases(InVariable);

	if (!InVariable.IsValid())
	{
		Error(FText::Format(LOCTEXT("GetSystemConstantFail", "Unknown System constant, Type: {0} Variable: {1}"), InVariable.GetType().GetNameText(), FText::FromName(InVariable.GetName())), InNodeForErrorReporting, nullptr);
		return false;
	}

	if (InVariable.GetType().IsStatic())
	{
		return false;
	}

	FString SymbolName = BuildSanitizedSymbolName(InVariable.GetName());
	FString ParameterMapInstanceName = GetParameterMapInstanceName(InParamMapHistoryIdx);

	bool bMissingParameter = false;
	FParameterCollection Collection = GraphBridge::IsParameterCollectionParameter(ParamMapHistories[InParamMapHistoryIdx], InVariable, bMissingParameter);
	if (GraphBridge::IsCollectionValid(Collection) && bMissingParameter)
	{
		Error(FText::Format(LOCTEXT("MissingNPCParameterError", "Parameter named {0} of type {1} was not found in Parameter Collection {2}"),
			FText::FromName(InVariable.GetName()), InVariable.GetType().GetNameText(), FText::FromString(GraphBridge::GetCollectionFullName(Collection))), InNodeForErrorReporting, InDefaultPin);
		return false;
	}

	const bool bIsDataInterface = InVariable.IsDataInterface();
	const bool bIsUObject = InVariable.IsUObject();
	const FString* EmitterAlias = ActiveHistoryForFunctionCalls.GetEmitterAlias();

	bool bIsPerInstanceBulkSystemParam = IsBulkSystemScript() && !bIsDataInterface && !bIsUObject && (FNiagaraParameterUtilities::IsUserParameter(InVariable) || FNiagaraParameterUtilities::IsPerInstanceEngineParameter(InVariable, EmitterAlias != nullptr ? *EmitterAlias : TEXT("Emitter")));
	const bool bIsExternalConstantParameter = FNiagaraParameterUtilities::IsRapidIterationParameter(InVariable) && !InVariable.GetType().IsStatic();

	if(!bIsEngineProvidedValue)//This is not actually a uniform so just add the code chunk to read it directly.
	{
		if (!bIsPerInstanceBulkSystemParam)
		{
			int32 UniformChunk = 0;

			if (false == ParamMapDefinedSystemVars.Contains(InVariable.GetName()))
			{
				if (bIsDataInterface)
				{
					UNiagaraDataInterface* DataInterface = nullptr;
					if (GraphBridge::IsCollectionValid(Collection))
					{
						DataInterface = GraphBridge::GetCollectionDataInterface(Collection, InVariable);
						if (DataInterface == nullptr)
						{
							Error(FText::Format(LOCTEXT("ParameterCollectionDataInterfaceNotFoundErrorFormat", "Data interface named {0} of type {1} was not found in Parameter Collection {2}"),
								FText::FromName(InVariable.GetName()), InVariable.GetType().GetNameText(), FText::FromString(GraphBridge::GetCollectionFullName(Collection))), InNodeForErrorReporting, InDefaultPin);
							return false;
						}
					}
					else
					{
						DataInterface = CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(const_cast<UClass*>(InVariable.GetType().GetClass()));
						if (DataInterface == nullptr)
						{
							Error(FText::Format(LOCTEXT("GetDuplicatedDataInterfaceCDOForClassFailed", "GetDuplicatedDataInterfaceCDOForClass failed for Variable({0}) Class({1})"), FText::FromName(InVariable.GetName()), InVariable.GetType().GetNameText()), InNodeForErrorReporting, InDefaultPin);
							return false;
						}
					}
					if (ensure(DataInterface))
					{
						Output = RegisterDataInterface(InVariable, DataInterface, true, true);
						return true;
					}
				}
				if (bIsUObject)
				{
					UObject* Object = nullptr;
					if (GraphBridge::IsCollectionValid(Collection))
					{
						Object = GraphBridge::GetCollectionUObject(Collection, InVariable);
					}
					Output = RegisterUObject(InVariable, Object, true);
					return true;
				}
				if (!InVariable.IsDataAllocated() && !InDefaultPin)
				{
					FNiagaraEditorUtilities::ResetVariableToDefaultValue(InVariable);
				}
				else if (!InVariable.IsDataAllocated())
				{
					FillVariableWithDefaultValue(InVariable, InDefaultPin);
				}

				if (InVariable.GetAllocatedSizeInBytes() != InVariable.GetSizeInBytes())
				{
					Error(FText::Format(LOCTEXT("GetParameterUnsetParam", "Variable {0} hasn't had its default value set. Required Bytes: {1} vs Allocated Bytes: {2}"), FText::FromName(InVariable.GetName()), FText::AsNumber(InVariable.GetType().GetSize()), FText::AsNumber(InVariable.GetSizeInBytes())), nullptr, nullptr);
				}

				if (IsVariableInUniformBuffer(InVariable))
				{
					TOptional<FNiagaraVariable> ConflictingVariable;

					const bool bApplyPadding = UNiagaraScript::IsGPUScript(CompileOptions.TargetUsage);
					UniformChunk = RegisterUniformChunkToPack(SymbolName, InVariable, bApplyPadding, CompilationOutput.ScriptData.Parameters, ConflictingVariable);

					if (ConflictingVariable.IsSet())
					{
						Error(FText::Format(LOCTEXT("NonUniqueSymbolNames", "Parameters ('{0}' and '{1}') found which resolve to the same HLSL symbol name '{2}'.  These should be disambiguated."),
							FText::FromName(InVariable.GetName()), FText::FromName(ConflictingVariable->GetName()), FText::FromString(SymbolName)), InNodeForErrorReporting, InDefaultPin);
					}
				}
				else
				{
					UniformChunk = AddChunkToConstantBuffer(SymbolName, InVariable, ENiagaraCodeChunkMode::Uniform);
				}
			}
			else
			{
				const auto& SystemVar = ParamMapDefinedSystemVars.FindChecked(InVariable.GetName());
				UniformChunk = SystemVar.ChunkIndex;
			}

			if (bIsExternalConstantParameter)
			{
				Output = UniformChunk;
				return true;
			}
			else
			{
				//Add this separately as the same uniform can appear in the pre sim chunks more than once in different param maps.
				PerStageMainPreSimulateChunks[ActiveStageIdx].AddUnique(FString::Printf(TEXT("%s.%s = %s;"), *ParameterMapInstanceName, *SymbolName, *GetCodeAsSource(UniformChunk)));
			}
		}
		else if (bIsPerInstanceBulkSystemParam && !ExternalVariablesForBulkUsage.Contains(InVariable))
		{
			ExternalVariablesForBulkUsage.Add(InVariable);
		}
	}

	Output = AddSourceChunk(ParameterMapInstanceName + TEXT(".") + SymbolName, InVariable.GetType());
	if (CodeChunks.IsValidIndex(Output))
	{ // Leave a breadcrumb to resolve for compile tags.
		CodeChunks[Output].Original = InVariable;
	}

	//Track a few special case reads that the system needs to know about.
	if(InVariable == SYS_PARAM_ENGINE_SYSTEM_SIGNIFICANCE_INDEX)
	{
		CompilationOutput.ScriptData.bReadsSignificanceIndex = true;
	}

	return true;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::FillVariableWithDefaultValue(FNiagaraVariable& InVariable, const FPin* InDefaultPin)
{
	FNiagaraVariable Var = GraphBridge::GetPinVariable(InDefaultPin, true, ENiagaraStructConversion::Simulation);
	FNiagaraEditorUtilities::ResetVariableToDefaultValue(InVariable);
	if (Var.IsDataAllocated() && Var.GetData() != nullptr)
	{
		InVariable.SetData(Var.GetData());
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::FillVariableWithDefaultValue(int32& OutValue, const FPin* InDefaultPin)
{
	FNiagaraVariable Var = GraphBridge::GetPinVariable(InDefaultPin, true, ENiagaraStructConversion::Simulation);
	FNiagaraVariable VarFinal = Var;
	FNiagaraEditorUtilities::ResetVariableToDefaultValue(VarFinal); // Do this to handle non-zero defaults
	if (Var.IsDataAllocated() && Var.GetData() != nullptr)
	{
		VarFinal.SetData(Var.GetData());
	}

	if (VarFinal.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		OutValue = VarFinal.GetValue<bool>();
	}
	else if (VarFinal.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || VarFinal.GetType().IsEnum())
	{
		OutValue = VarFinal.GetValue<int32>();
	}
}


template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::SetConstantByStaticVariable(int32& OutValue, const FPin* InDefaultPin, FString* OutDebugString)
{
	if (InDefaultPin == nullptr)
		return;
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	OutValue = 0;
	FNiagaraVariable Var = GraphBridge::GetPinVariable(InDefaultPin, true, ENiagaraStructConversion::UserFacing);
	FNiagaraVariable VarDefault = Var;
	FNiagaraEditorUtilities::ResetVariableToDefaultValue(VarDefault);// Do this to handle non-zero defaults
	if (VarDefault.GetType().IsStatic())
	{
		FNiagaraVariable VarWithValue = FNiagaraVariable(Var.GetType(), Var.GetName());
		FString Value;
		int32 Input = INDEX_NONE;

		const FPin* PinToTest = InDefaultPin;
		if (const FInputPin* DefaultInputPin = GraphBridge::GetPinAsInput(InDefaultPin))
		{
			if (const FOutputPin* ConnectedPin = GraphBridge::GetLinkedOutputPin(DefaultInputPin))
			{
				PinToTest = ConnectedPin;
			}
		}

		FGraphTraversalHandle PinHandle(ActiveHistoryForFunctionCalls.ActivePath);
		PinHandle.PushPin(PinToTest);

		const FString* ConstantPtr = CompileData->PinToConstantValues.Find(PinHandle);
		if (ConstantPtr != nullptr)
		{
			FNiagaraVariable SearchVar(Var.GetType(), *(*ConstantPtr));
			int32 StaticVarSearchIdx = CompileData->StaticVariables.Find(SearchVar);

			if (StaticVarSearchIdx == -1)
			{
				TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(Var.GetType());
				if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanHandlePinDefaults())
				{
					TypeEditorUtilities->SetValueFromPinDefaultString(*ConstantPtr, VarWithValue);
				}
			}
			else
			{
				VarWithValue = CompileData->StaticVariables[StaticVarSearchIdx];
			}
		}

		if (OutDebugString != nullptr)
		{
			*OutDebugString = PinHandle.ToString();
		}

		if (VarWithValue.IsDataAllocated())
		{
			if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
			{
				OutValue = VarWithValue.GetValue<bool>();
			}
			else if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || VarWithValue.GetType().IsEnum())
			{
				OutValue = VarWithValue.GetValue<int32>();
			}
		}
		else if (!UNiagaraScript::IsModuleScript(CompileOptions.TargetUsage) && !UNiagaraScript::IsFunctionScript(CompileOptions.TargetUsage) &&
			!UNiagaraScript::IsDynamicInputScript(CompileOptions.TargetUsage)) // Can't always resolve these outside the context of an emitter or system
		{
			Error(LOCTEXT("CouldNotResolveStaticVarByPin", "Could not resolve static variable through pin."), GraphBridge::GetOwningNode(InDefaultPin), InDefaultPin);
		}
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::SetConstantByStaticVariable(FNiagaraVariable& OutValue, const FPin* InDefaultPin, FString* DebugString)
{
	OutValue.AllocateData();
	int32 Constant = 0;
	SetConstantByStaticVariable(Constant, InDefaultPin, DebugString);

	if (OutValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		OutValue.SetValue<bool>(Constant != 0 ? true : false);
	}
	else if (OutValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || OutValue.GetType().IsEnum())
	{
		OutValue.SetValue<int32>(Constant);
	}
}

void FNiagaraHlslTranslator::SetConstantByStaticVariable(FNiagaraVariable& OutValue, const FNiagaraVariable& Var, FString* DebugString)
{
	OutValue = Var;
	OutValue.AllocateData();
	int32 Constant = 0;
	SetConstantByStaticVariable(Constant, Var, DebugString);

	if (Var.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		OutValue.SetValue<bool>(Constant != 0 ? true : false);
	}
	else if (Var.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || Var.GetType().IsEnum())
	{
		OutValue.SetValue<int32>(Constant);
	}
}

void FNiagaraHlslTranslator::SetConstantByStaticVariable(int32& OutValue, const FNiagaraVariable& Var, FString* DebugString)
{
	OutValue = 0;
	FNiagaraVariable VarDefault = Var;
	FNiagaraEditorUtilities::ResetVariableToDefaultValue(VarDefault);// Do this to handle non-zero defaults
	if (VarDefault.GetType().IsStatic())
	{
		if (DebugString != nullptr)
		{
			*DebugString = Var.GetName().ToString();
		}
		FNiagaraVariable VarWithValue = FNiagaraVariable(Var.GetType(), Var.GetName());
		FString Value = Var.GetName().ToString();
		int32 Input = INDEX_NONE;

		// If we found a string, we should try and map to the actual value of that variable..
		if (Value.Len() != 0)
		{
			if (!VarWithValue.IsDataAllocated())
			{
				TConstArrayView<FNiagaraVariable> StaticVariables = GetStaticVariables();
				int32 FoundOverrideIdx = StaticVariables.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
				{
					return (InObj.GetName() == *Value);
				});;

				if (FoundOverrideIdx != INDEX_NONE)
				{
					VarWithValue.SetData(StaticVariables[FoundOverrideIdx].GetData());
				}
				else if (FNiagaraParameterUtilities::IsAliasedEmitterParameter(Var))
				{
					FNiagaraAliasContext ResolveAliasesContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript);
					ResolveAliasesContext.ChangeEmitterToEmitterName(GetUniqueEmitterName());
					FNiagaraVariable TestEmitterResolvedVar = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);

					FoundOverrideIdx = StaticVariables.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
					{
						return (InObj.GetName() == TestEmitterResolvedVar.GetName());
					});;

					if (FoundOverrideIdx != INDEX_NONE)
					{
						VarWithValue.SetData(StaticVariables[FoundOverrideIdx].GetData());
					}
				}
			}
		}

		if (VarWithValue.IsDataAllocated())
		{
			if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
			{
				OutValue = VarWithValue.GetValue<bool>();
			}
			else if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || VarWithValue.GetType().IsEnum())
			{
				OutValue = VarWithValue.GetValue<int32>();
			}

			//UE_LOGF(LogNiagaraEditor, Log, "SetConstantByStaticVariable Added %ls (%d)", *Var.GetName().ToString(), OutValue);
		}
		else
		{
			Error(FText::Format(LOCTEXT("CouldNotResolveStaticVar", "Could not resolve static variable \"{0}\". Default type value used instead."),  FText::FromName(Var.GetName())));
		}
	}
}


template<typename GraphBridge>
bool TNiagaraHlslTranslator<GraphBridge>::ParameterMapRegisterUniformAttributeVariable(const FNiagaraVariable& InVariable, FNode* InNode, int32 InParamMapHistoryIdx, int32& Output)
{
	FNiagaraVariable NewVar = FNiagaraParameterUtilities::BasicAttributeToNamespacedAttribute(InVariable);
	if (NewVar.IsValid())
	{
		return ParameterMapRegisterNamespaceAttributeVariable(NewVar, InNode, InParamMapHistoryIdx, Output);
	}
	return false;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ValidateParticleIDUsage()
{
	if (CompileOptions.AdditionalDefines.Contains(TEXT("RequiresPersistentIDs")))
	{
		// persistent IDs are active and can be safely used as inputs
		return;
	}
	FName particleIDName(TEXT("Particles.ID"));
	for (FParamMapHistory& History : ParamMapHistories)
	{
		for (const FNiagaraVariable& Variable : History.Variables)
		{
			if (Variable.GetName() == particleIDName)
			{
				Error(LOCTEXT("PersistentIDActivationFail", "Before the Particles.ID parameter can be used, the 'Requires persistent IDs' option has to be activated in the emitter properties. Note that this comes with additional memory and CPU costs."), nullptr, nullptr);
			}
		}
	}
}

template<typename GraphBridge>
bool TNiagaraHlslTranslator<GraphBridge>::ParameterMapRegisterNamespaceAttributeVariable(const FNiagaraVariable& InVariable, const FNode* InNode, int32 InParamMapHistoryIdx, int32& Output)
{
	FString SymbolNameNamespaced = BuildSanitizedSymbolName(InVariable.GetName());
	FString ParameterMapInstanceName = GetParameterMapInstanceName(InParamMapHistoryIdx);
	FNiagaraVariable NamespaceVar = InVariable;

	Output = INDEX_NONE;
	FNiagaraVariable BasicVar = FNiagaraParameterUtilities::ResolveAsBasicAttribute(InVariable);
	if (BasicVar.IsValid())
	{
		if (false == ParamMapDefinedAttributesToUniformChunks.Contains(BasicVar.GetName()))
		{
			FString SymbolNameDefined = BuildSanitizedSymbolName(BasicVar.GetName());
			int32 UniformChunk = INDEX_NONE;
			int32 Idx = InstanceRead.Variables.Find(NamespaceVar);
			if (Idx != INDEX_NONE)
			{
				UniformChunk = InstanceRead.CodeChunks[Idx];
			}
			else
			{
				UniformChunk = AddSourceChunk(ParameterMapInstanceName + TEXT(".") + SymbolNameNamespaced, NamespaceVar.GetType());
				InstanceRead.CodeChunks.Add(UniformChunk);
				InstanceRead.Variables.Add(NamespaceVar);
			}

			ParamMapDefinedAttributesToUniformChunks.Add(BasicVar.GetName(), UniformChunk);
			FVarAndDefaultSource VarAndDefaultSource;
			VarAndDefaultSource.Variable = NamespaceVar;
			VarAndDefaultSource.bDefaultExplicit = true;
			ParamMapDefinedAttributesToNamespaceVars.Add(BasicVar.GetName(), VarAndDefaultSource);
		}
		Output = AddSourceChunk(ParameterMapInstanceName + TEXT(".") + SymbolNameNamespaced, NamespaceVar.GetType());
		return true;
	}

	if (Output == INDEX_NONE)
	{
		Error(FText::Format(LOCTEXT("GetEmitterUniformFail", "Unknown Emitter Uniform Variable, Type: {0} Variable: {1}"), InVariable.GetType().GetNameText(), FText::FromName(InVariable.GetName())), InNode, nullptr);
	}
	return false;
}

template<typename GraphBridge>
bool TNiagaraHlslTranslator<GraphBridge>::ParameterMapRegisterNamespaceAttributeVariable(const FNiagaraVariable& InVariable, int32 InParamMapHistoryIdx, int32& Output)
{
	return ParameterMapRegisterNamespaceAttributeVariable(InVariable, nullptr, InParamMapHistoryIdx, Output);
}

FString FNiagaraHlslTranslator::GetParameterMapInstanceName(int32 ParamMapHistoryIdx)
{
	FString ParameterMapInstanceName;

	if (TranslationStages.Num() > ActiveStageIdx)
	{
		ParameterMapInstanceName = FString::Printf(TEXT("Context.%s"), *TranslationStages[ActiveStageIdx].PassNamespace);
	}
	return ParameterMapInstanceName;
}

// simple helper template allowing us to keep Emitter() generic even if the Builder interface may diverge
template<typename GraphBridge>
class FScopedBuilderEmitter
{
public:
	FScopedBuilderEmitter(FNiagaraCompilationGraphBridge::FParamMapHistoryBuilder& InBuilder, const typename GraphBridge::FEmitterNode* InEmitterNode)
		: EmitterNode(InEmitterNode)
		, EmitterUniqueName(GraphBridge::GetEmitterUniqueName(InEmitterNode))
		, Builder(InBuilder)
	{
		Builder.EnterEmitter(EmitterUniqueName, GraphBridge::GetEmitterGraph(EmitterNode), EmitterNode);
	}

	~FScopedBuilderEmitter()
	{
		Builder.ExitEmitter(EmitterUniqueName, EmitterNode);
	}

private:
	const typename GraphBridge::FEmitterNode* EmitterNode;
	const FString EmitterUniqueName;
	typename GraphBridge::FParamMapHistoryBuilder& Builder;
};

template<>
class FScopedBuilderEmitter<FNiagaraCompilationDigestBridge>
{
public:
	FScopedBuilderEmitter(TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationDigestBridge>& InBuilder, const FNiagaraCompilationNodeEmitter* InEmitterNode)
		: EmitterNode(InEmitterNode)
		, Builder(InBuilder)
	{
		Builder.TraversalStateContext->PushEmitter(EmitterNode);
		Builder.EnterEmitter(EmitterNode->EmitterUniqueName, EmitterNode->CalledGraph.Get(), EmitterNode);
	}

	~FScopedBuilderEmitter()
	{
		Builder.ExitEmitter(EmitterNode->EmitterUniqueName, EmitterNode);
		Builder.TraversalStateContext->PopEmitter(EmitterNode);
	}

private:
	const FNiagaraCompilationNodeEmitter* EmitterNode;
	TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationDigestBridge>& Builder;
};

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Emitter(const FEmitterNode* EmitterNode, TArray<int32>& Inputs, TArray<int32>& Outputs)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_Emitter);

	// Just pass through the input parameter map pin if the node isn't enabled...
	if (!GraphBridge::NodeIsEnabled(EmitterNode))
	{
		TArray<const FOutputPin*> OutputPins = GraphBridge::GetOutputPins(EmitterNode);
		Outputs.Init(INDEX_NONE, OutputPins.Num());

		if (Inputs.Num() >= 1)
		{
			Outputs[0] = Inputs[0];
		}
		return;
	}

	FNiagaraFunctionSignature Signature;
	const FGraph* EmitterGraph = GraphBridge::GetEmitterGraph(EmitterNode);
	if (EmitterGraph == nullptr)
	{
		Error(LOCTEXT("FunctionCallNonexistantScriptSource", "Emitter call missing ScriptSource"), EmitterNode, nullptr);
		return;
	}

	// We need the generated string to generate the proper signature for now.
	FString EmitterUniqueName = GraphBridge::GetEmitterUniqueName(EmitterNode);

	ENiagaraScriptUsage ScriptUsage = GraphBridge::GetEmitterUsage(EmitterNode);
	FString Name = GraphBridge::GetEmitterName(EmitterNode);
	FString FullName = GraphBridge::GetEmitterPathName(EmitterNode);

	FName StatName = *EmitterUniqueName;
	EnterStatsScope(FNiagaraStatScope(StatName, StatName));

	TArray<const FOutputPin*> CallOutputs = GraphBridge::GetOutputPins(EmitterNode);
	TArray<const FInputPin*> CallInputs = GraphBridge::GetInputPins(EmitterNode);

	if (Inputs.Num() == 0 || GraphBridge::GetPinType(CallInputs[0], ENiagaraStructConversion::UserFacing) != FNiagaraTypeDefinition::GetParameterMapDef())
	{
		Error(LOCTEXT("EmitterMissingParamMap", "Emitter call missing ParameterMap input pin!"), EmitterNode, nullptr);
		return;
	}

	int32 ParamMapHistoryIdx = Inputs[0];
	if (ParamMapHistoryIdx == INDEX_NONE)
	{
		Error(LOCTEXT("EmitterMissingParamMapIndex", "Emitter call missing valid ParameterMap index!"), EmitterNode, nullptr);
		return;
	}

	{
		FScopedBuilderEmitter<GraphBridge> EmitterBuidlerScope(ActiveHistoryForFunctionCalls, EmitterNode);

		// Clear out the parameter map writes to emitter module parameters as they should not be shared across emitters.
		if (ParamMapHistoryIdx != -1 && ParamMapHistoryIdx < ParamMapHistories.Num())
		{
			FParamMapHistory& History = ParamMapHistories[ParamMapHistoryIdx];

			//Set some special case values that the engine provides directly
			FNiagaraVariable EmitterIDVar = ActiveHistoryForFunctionCalls.ResolveAliases(SYS_PARAM_ENGINE_EMITTER_ID);
			//Add the alias resolved var to the ParamMapDefinedEngineVars so that we can use this array to inject these values into the hlsl struct defs later.
			ParamMapDefinedEngineVars.Add(EmitterIDVar);
			History.AddVariable(EmitterIDVar, SYS_PARAM_ENGINE_EMITTER_ID, NAME_None, nullptr);
			FNiagaraEmitterID EmitterID = GraphBridge::GetEmitterID(EmitterNode);
			AddBodyChunk(TEXT(""), FString::Printf(TEXT("%s.%s.ID = %d"), *GetParameterMapInstanceName(ParamMapHistoryIdx), *BuildSanitizedSymbolName(EmitterIDVar.GetName()), EmitterID.ID), FNiagaraTypeDefinition::GetIntDef(), false);

			for (int32 i = 0; i < ParamMapHistories[ParamMapHistoryIdx].Variables.Num(); i++)
			{
				check(ParamMapHistories[ParamMapHistoryIdx].VariablesWithOriginalAliasesIntact.Num() > i);
				FNiagaraVariable Var = ParamMapHistories[ParamMapHistoryIdx].VariablesWithOriginalAliasesIntact[i];
				if (FNiagaraParameterUtilities::IsAliasedModuleParameter(Var))
				{
					ParamMapSetVariablesToChunks[ParamMapHistoryIdx][i] = INDEX_NONE;
				}
			}
		}

		// We act like a function call here as the semantics are identical.
		RegisterFunctionCall(ScriptUsage, Name, FName(FullName), EmitterNode->NodeGuid, GraphBridge::GetEmitterHandleIdString(EmitterNode), EmitterGraph, Signature, false, FString(), {}, Inputs, CallInputs, CallOutputs, Signature);
		GenerateFunctionCall(ScriptUsage, Signature, Inputs, Outputs);

		// Clear out the parameter map writes to emitter module parameters as they should not be shared across emitters.
		if (ParamMapHistoryIdx != -1 && ParamMapHistoryIdx < ParamMapHistories.Num())
		{
			for (int32 i = 0; i < ParamMapHistories[ParamMapHistoryIdx].Variables.Num(); i++)
			{
				check(ParamMapHistories[ParamMapHistoryIdx].VariablesWithOriginalAliasesIntact.Num() > i);
				FNiagaraVariable Var = ParamMapHistories[ParamMapHistoryIdx].VariablesWithOriginalAliasesIntact[i];
				if (ActiveHistoryForFunctionCalls.IsInEncounteredFunctionNamespace(Var) || FNiagaraParameterUtilities::IsAliasedModuleParameter(Var) || FNiagaraParameterUtilities::IsInNamespace(Var, PARAM_MAP_TRANSIENT_STR))
				{
					ParamMapSetVariablesToChunks[ParamMapHistoryIdx][i] = INDEX_NONE;
				}
			}
		}
	}

	ExitStatsScope();
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ParameterMapGet(const FParamMapGetNode* GetNode, TArrayView<const int32> Inputs, TArray<int32>& Outputs)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_MapGet);

	TArray<const typename GraphBridge::FOutputPin*> OutputPins = GraphBridge::GetOutputPins(GetNode);
	Outputs.Init(INDEX_NONE, OutputPins.Num());

	// Push out invalid values for all output pins if the node is disabled.
	if (!GraphBridge::NodeIsEnabled(GetNode))
	{
		return;
	}

	TArray<const typename GraphBridge::FInputPin*> InputPins = GraphBridge::GetInputPins(GetNode);
	int32 ParamMapHistoryIdx = Inputs[0];

	if (ParamMapHistoryIdx == -1)
	{
		Error(LOCTEXT("NoParamMapIdxForInput", "Cannot find parameter map for input!"), GetNode, nullptr);
		return;
	}
	else if (ParamMapHistoryIdx >= ParamMapHistories.Num())
	{
		Error(FText::Format(LOCTEXT("InvalidParamMapIdxForInput", "Invalid parameter map index for input {0} of {1}!"), ParamMapHistoryIdx, ParamMapHistories.Num()), GetNode, nullptr);
		return;
	}

	FString ParameterMapInstanceName = GetParameterMapInstanceName(ParamMapHistoryIdx);

	for (int32 i = 0; i < OutputPins.Num(); i++)
	{
		// These are the pins that we are getting off the parameter map.
		FNiagaraTypeDefinition OutputTypeDefinition = GraphBridge::GetPinType(OutputPins[i], ENiagaraStructConversion::UserFacing);
		const bool bNeedsValue =
			OutputTypeDefinition != FNiagaraTypeDefinition::GetParameterMapDef() &&
			OutputTypeDefinition.IsDataInterface() == false &&
			OutputTypeDefinition.IsUObject() == false;

		FNiagaraVariable Var = GraphBridge::GetPinVariable(OutputPins[i], bNeedsValue, ENiagaraStructConversion::Simulation);

		FNiagaraScriptVariableBinding DefaultBinding;
		TOptional<ENiagaraDefaultMode> DefaultMode = GraphBridge::GetGraphDefaultMode(GraphBridge::GetOwningGraph(GetNode), Var, DefaultBinding);
		if (Var.GetType().IsStatic())
		{
			if (FNiagaraParameterUtilities::IsExternalConstantNamespace(Var, CompileOptions.TargetUsage, CompileOptions.GetTargetUsageBitmask()))
			{
				if (DefaultMode.IsSet() && *DefaultMode == ENiagaraDefaultMode::FailIfPreviouslyNotSet && !Var.IsInNameSpace(FNiagaraConstants::UserNamespaceString))
				{
					// Register an external dependency...
					RegisterCompileDependency(Var, FText::Format(LOCTEXT("UsedBeforeSet", "Variable {0} was read before being set. It's default mode is \"Fail If Previously Not Set\", so this isn't allowed."), FText::FromName(Var.GetName())), GetNode, OutputPins[i], true, ParamMapHistoryIdx);
				}
			}
			else if (DefaultMode.IsSet() && *DefaultMode == ENiagaraDefaultMode::FailIfPreviouslyNotSet && !Var.IsInNameSpace(FNiagaraConstants::UserNamespaceString) && !Var.IsInNameSpace(FNiagaraConstants::ModuleNamespaceString))
			{
				// Check for an internal dependency
				bool bFailIfNotSet = false;
				FNiagaraVariable TestVar = ActiveHistoryForFunctionCalls.ResolveAliases(Var);
				ValidateFailIfPreviouslyNotSet(TestVar, bFailIfNotSet);
				if (bFailIfNotSet)
				{
					RegisterCompileDependency(Var, FText::Format(LOCTEXT("UsedBeforeSet", "Variable {0} was read before being set. It's default mode is \"Fail If Previously Not Set\", so this isn't allowed."), FText::FromName(Var.GetName())), GetNode, nullptr, false, ParamMapHistoryIdx);
				}
			}

			Outputs[i] = MakeStaticVariableDirect(OutputPins[i]);
		}
		else
		{
			HandleParameterRead(ParamMapHistoryIdx, Var, GraphBridge::GetDefaultPin(GetNode, OutputPins[i]), GetNode, Outputs[i], DefaultMode, DefaultBinding);
		}
	}
}

template<typename GraphBridge>
int32 TNiagaraHlslTranslator<GraphBridge>::MakeStaticVariableDirect(const FPin* InDefaultPin)
{
	int32 Constant = INDEX_NONE;
	FString DebugStr;
	SetConstantByStaticVariable(Constant, InDefaultPin, &DebugStr);

	AddBodyComment(TEXT("//SetConstantByStaticVariable \"") + DebugStr + TEXT("\""));
	return GetConstantDirect(Constant);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ValidateFailIfPreviouslyNotSet(const FNiagaraVariable& InVar, bool& bFailIfNotSet)
{
	bFailIfNotSet = false;
	FNiagaraVariable SearchVar = InVar;
	if (FNiagaraParameterUtilities::IsInitialValue(InVar))
	{
		SearchVar = FNiagaraParameterUtilities::GetSourceForInitialValue(InVar);
	}
	else if (FNiagaraParameterUtilities::IsPreviousValue(InVar))
	{
		SearchVar = FNiagaraParameterUtilities::GetSourceForPreviousValue(InVar);
	}

	FVarAndDefaultSource* ParamMapDefinedVarAndDefaultSource = ParamMapDefinedAttributesToNamespaceVars.Find(SearchVar.GetName());
	bool bSetPreviously = ParamMapDefinedVarAndDefaultSource != nullptr && ParamMapDefinedVarAndDefaultSource->bDefaultExplicit;
	for (int32 OtherParamIdx = 0; OtherParamIdx < OtherOutputParamMapHistories.Num() && !bSetPreviously; OtherParamIdx++)
	{
		// Stop if this is already in our evaluation chain. Assume only indices above us are valid sourcers for this.
		if (ParamMapHistoriesSourceInOtherHistories.Contains(OtherParamIdx))
			break;

		int32 FoundInParamIdx = OtherOutputParamMapHistories[OtherParamIdx].FindVariableByName(SearchVar.GetName());
		if (INDEX_NONE != FoundInParamIdx)
		{
			for (const typename GraphBridge::FModuleScopedPin& ScopedPin : OtherOutputParamMapHistories[OtherParamIdx].PerVariableWriteHistory[FoundInParamIdx])
			{
				if (ScopedPin.Pin->Direction == EGPD_Input && ScopedPin.Pin->bHidden == false)
				{
					bSetPreviously = true;
					break;
				}
			}
		}
	}
	if (!bSetPreviously && !UNiagaraScript::IsStandaloneScript(CompileOptions.TargetUsage))
		bFailIfNotSet = true;
}


template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::HandleParameterRead(int32 ParamMapHistoryIdx, const FNiagaraVariable& InVar, const FInputPin* DefaultPin, const FNode* ErrorNode, int32& OutputChunkId, TOptional<ENiagaraDefaultMode> DefaultMode, TOptional<FNiagaraScriptVariableBinding> DefaultBinding, bool bTreatAsUnknownParameterMap, bool bIgnoreDefaultSetFirst)
{
	FString ParameterMapInstanceName = GetParameterMapInstanceName(ParamMapHistoryIdx);
	FNiagaraVariable Var = ConvertToSimulationVariable(InVar);
	if (!AddStructToDefinitionSet(Var.GetType()))
	{
		Error(FText::Format(LOCTEXT("ParameterMapGetTypeError", "Cannot handle type {0}! Variable: {1}"), Var.GetType().GetNameText(), FText::FromName(Var.GetName())), nullptr, nullptr);
	}

	// If this is a System parameter, just wire in the system appropriate system attribute.
	FString SymbolName = BuildSanitizedSymbolName(Var.GetName());

	bool bIsPerInstanceAttribute = false;
	bool bIsCandidateForRapidIteration = false;
	const FInputPin* InputPin = DefaultPin;

	FString Namespace = FNiagaraParameterUtilities::GetNamespace(Var);
	if (!FNiagaraParameterUtilities::IsValidNamespaceForReading(CompileOptions.TargetUsage, CompileOptions.TargetUsageBitmask, Namespace))
	{
		if (UNiagaraScript::IsStandaloneScript(CompileOptions.TargetUsage) && Namespace.StartsWith(PARAM_MAP_ATTRIBUTE_STR))
		{
			Error(FText::Format(LOCTEXT("InvalidReadingNamespaceStandalone", "Variable {0} is in a namespace that isn't valid for reading. Enable at least one of the 'particle' options in the target usage bitmask of your script to access the 'Particles.' namespace."), FText::FromName(Var.GetName())), ErrorNode, nullptr);
		}
		else
		{
			Error(FText::Format(LOCTEXT("InvalidReadingNamespace", "Variable {0} is in a namespace that isn't valid for reading"), FText::FromName(Var.GetName())), ErrorNode, nullptr);
		}
		return;
	}

	//Some special variables can be replaced directly with constants which allows for extra optimization in the compiler.
	if (GetLiteralConstantVariable(Var))
	{
		OutputChunkId = GetConstant(Var);
		return;
	}
	else if (HandleBoundConstantVariableToDataSetRead(Var, ErrorNode, ParamMapHistoryIdx, OutputChunkId, DefaultPin))
	{
		return;
	}

	
	if (FNiagaraParameterUtilities::IsExternalConstantNamespace(Var, CompileOptions.TargetUsage, CompileOptions.GetTargetUsageBitmask()))
	{
		if (DefaultMode.IsSet() && *DefaultMode == ENiagaraDefaultMode::FailIfPreviouslyNotSet && !bIgnoreDefaultSetFirst)
		{
			RegisterCompileDependency(Var, FText::Format(LOCTEXT("UsedBeforeSet", "Variable {0} was read before being set. It's default mode is \"Fail If Previously Not Set\", so this isn't allowed."), FText::FromName(Var.GetName())), ErrorNode, nullptr, true, ParamMapHistoryIdx);
		}
		if (Var.GetType().IsStatic())
		{
			OutputChunkId = MakeStaticVariableDirect(DefaultPin);
			return;
		}
		else if (ParameterMapRegisterExternalConstantNamespaceVariable(Var, ErrorNode, ParamMapHistoryIdx, OutputChunkId, DefaultPin))
		{
			return;
		}			
	}
	else if (FNiagaraParameterUtilities::IsAliasedModuleParameter(Var) && ActiveHistoryForFunctionCalls.InTopLevelFunctionCall(CompileOptions.TargetUsage))
	{
		if (DefaultMode.IsSet() && *DefaultMode == ENiagaraDefaultMode::Binding && DefaultBinding.IsSet() && DefaultBinding->IsValid())
		{
			// Skip the case where the below condition is met, but it's overridden by a binding.
			bIsCandidateForRapidIteration = false;
		}
		else if (InputPin != nullptr && GraphBridge::GetLinkedOutputPin(InputPin) == nullptr && FNiagaraStackGraphUtilities::IsRapidIterationType(Var.GetType()))
		{
			bIsCandidateForRapidIteration = true;
		}
	}

	FParamMapHistory& History = ParamMapHistories[ParamMapHistoryIdx];
	bool bWasEmitterAliased = FNiagaraParameterUtilities::IsAliasedEmitterParameter(Var);
	Var = ActiveHistoryForFunctionCalls.ResolveAliases(Var);

	const FNiagaraKnownConstantInfo ConstantInfo = FNiagaraConstants::GetKnownConstantInfo(Var.GetName(), false);
	if (ConstantInfo.ConstantVar != nullptr && ConstantInfo.ConstantVar->GetType() != Var.GetType() && ConstantInfo.ConstantType != ENiagaraKnownConstantType::Attribute)
	{
		Error(FText::Format(LOCTEXT("MismatchedConstantTypes", "Variable {0} is a system constant, but its type is different! {1} != {2}"), FText::FromName(Var.GetName()),
			ConstantInfo.ConstantVar->GetType().GetNameText(), Var.GetType().GetNameText()), ErrorNode, nullptr);
	}

	if (History.IsPrimaryDataSetOutput(Var, GetTargetUsage())) // Note that data interfaces aren't ever in the primary data set even if the namespace matches.
	{
		bIsPerInstanceAttribute = true;
	}

	if (TranslationStages[ActiveStageIdx].IterationSourceType == ENiagaraIterationSource::DataInterface && TranslationStages[ActiveStageIdx].ScriptUsage == ENiagaraScriptUsage::ParticleSimulationStageScript && !bIsPerInstanceAttribute)
	{
		bIsPerInstanceAttribute = Var.IsInNameSpace(TranslationStages[ActiveStageIdx].IterationDataInterface);
	}

	// Make sure to leave IsAlive alone if copying over previous stage params.
	if (Var == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("DataInstance.Alive")) && ActiveStageIdx > 0 && TranslationStages[ActiveStageIdx - 1].bCopyPreviousParams 
		&& TranslationStages[ActiveStageIdx - 1].bWritesAlive)
	{
		bIsPerInstanceAttribute = true; 
	}


	bool bFailIfPreviouslyNotSetSentinel = false;
	bool bValidateFailIfPreviouslyNotSet = false;
	if (DefaultMode.IsSet())
	{
		bValidateFailIfPreviouslyNotSet = *DefaultMode == ENiagaraDefaultMode::FailIfPreviouslyNotSet;
	}

	if (bValidateFailIfPreviouslyNotSet)
	{
		ValidateFailIfPreviouslyNotSet(Var, bFailIfPreviouslyNotSetSentinel);
	}

	
	int32 LastSetChunkIdx = INDEX_NONE;
	if (ParamMapHistoryIdx < ParamMapHistories.Num())
	{
		// See if we've written this variable before, if so we can reuse the index
		int32 VarIdx = ParamMapHistories[ParamMapHistoryIdx].FindVariableByName(Var.GetName());
		if (VarIdx != INDEX_NONE && VarIdx < ParamMapSetVariablesToChunks[ParamMapHistoryIdx].Num())
		{
			LastSetChunkIdx = ParamMapSetVariablesToChunks[ParamMapHistoryIdx][VarIdx];
		}

		bool bIsStaticConstant = Var.GetType().IsStatic();
		if (LastSetChunkIdx == INDEX_NONE && bIsStaticConstant)
		{
			OutputChunkId = MakeStaticVariableDirect(DefaultPin);
			return;
		}

		// Check to see if this is the first time we've encountered this node and it is a viable candidate for rapid iteration
		if (LastSetChunkIdx == INDEX_NONE && bIsCandidateForRapidIteration)
		{
			FNiagaraVariable OriginalVar = Var;
			bool bVarChanged = false;
			if (!bWasEmitterAliased && ActiveHistoryForFunctionCalls.GetEmitterAlias() != nullptr)
			{
				Var = FNiagaraParameterUtilities::ConvertVariableToRapidIterationConstantName(Var, *(*ActiveHistoryForFunctionCalls.GetEmitterAlias()), GetTargetUsage());
				bVarChanged = true;
			}
			else if (UNiagaraScript::IsSystemSpawnScript(CompileOptions.TargetUsage) || UNiagaraScript::IsSystemUpdateScript(CompileOptions.TargetUsage))
			{
				Var = FNiagaraParameterUtilities::ConvertVariableToRapidIterationConstantName(Var, nullptr, GetTargetUsage());
				bVarChanged = true;
			}

			
			if (TranslationOptions.bParameterRapidIteration)
			{
				// Now try to look up with the new name.. we may have already made this an external variable before..
				if (bVarChanged)
				{
					VarIdx = ParamMapHistories[ParamMapHistoryIdx].FindVariableByName(Var.GetName());
					if (VarIdx != INDEX_NONE && VarIdx < ParamMapSetVariablesToChunks[ParamMapHistoryIdx].Num())
					{
						LastSetChunkIdx = ParamMapSetVariablesToChunks[ParamMapHistoryIdx][VarIdx];
					}
				}

				// If it isn't found yet.. go ahead and make it into a constant variable..
				if (LastSetChunkIdx == INDEX_NONE && ParameterMapRegisterExternalConstantNamespaceVariable(Var, ErrorNode, ParamMapHistoryIdx, OutputChunkId, InputPin))
				{
					LastSetChunkIdx = OutputChunkId;
					if (VarIdx != INDEX_NONE && VarIdx < ParamMapSetVariablesToChunks[ParamMapHistoryIdx].Num())
					{
						// Record that we wrote to it.
						ParamMapSetVariablesToChunks[ParamMapHistoryIdx][VarIdx] = LastSetChunkIdx;
						RecordParamMapDefinedAttributeToNamespaceVar(Var, DefaultPin);
					}
				return;
				}
			}
			else
			{		
				int32 FoundIdx = TranslationOptions.OverrideModuleConstants.Find(Var);
				if (FoundIdx == INDEX_NONE)
				{
					if (!bWasEmitterAliased && ActiveHistoryForFunctionCalls.GetEmitterAlias() != nullptr && CompileData != nullptr)
					{
						Var = FNiagaraParameterUtilities::ConvertVariableToRapidIterationConstantName(OriginalVar, *CompileData->EmitterUniqueName, GetTargetUsage());
						bVarChanged = true;
						FoundIdx = TranslationOptions.OverrideModuleConstants.Find(Var);
					}
				}
				
				if (FoundIdx != INDEX_NONE)
				{
					FString DebugConstantStr;
					OutputChunkId = GetConstant(TranslationOptions.OverrideModuleConstants[FoundIdx], &DebugConstantStr);
					UE_LOGF(LogNiagaraEditor, VeryVerbose, "Converted parameter %ls to constant %ls for script %ls", *Var.GetName().ToString(), *DebugConstantStr, *CompileOptions.FullName);
					return;
				}
				else if (InputPin != nullptr && !InputPin->bDefaultValueIsIgnored) // Use the default from the input pin because this variable was previously never encountered.
				{
					FNiagaraVariable PinVar = GraphBridge::GetPinVariable(InputPin, true, ENiagaraStructConversion::Simulation);
					FString DebugConstantStr;
					OutputChunkId = GetConstant(PinVar, &DebugConstantStr);
					UE_LOGF(LogNiagaraEditor, VeryVerbose, "Converted default value of parameter %ls to constant %ls for script %ls. Likely added since this system was last compiled.", *Var.GetName().ToString(), *DebugConstantStr, *CompileOptions.FullName);
					return;
				}
				
				Error(FText::Format(LOCTEXT("InvalidRapidIterationReplacement", "Variable {0} is a rapid iteration param, but it wasn't found in the override list to bake out!"), FText::FromName(Var.GetName())), ErrorNode, nullptr);
			}
		}

		// We have yet to write to this parameter, use the default value if specified and the parameter 
		// isn't a per-particle value.
		bool bIgnoreDefaultValue = ParamMapHistories[ParamMapHistoryIdx].ShouldIgnoreVariableDefault(Var);

		// First check to see if this is defaulted to fail if not set previously. If so, then make sure we don't suck in defaults and error out.
		if (bValidateFailIfPreviouslyNotSet && bFailIfPreviouslyNotSetSentinel && !bIgnoreDefaultSetFirst)
		{		
			RegisterCompileDependency(Var, FText::Format(LOCTEXT("UsedBeforeSet", "Variable {0} was read before being set. It's default mode is \"Fail If Previously Not Set\", so this isn't allowed."), FText::FromName(Var.GetName())), ErrorNode, nullptr, false, ParamMapHistoryIdx);
		}

		if (bIsPerInstanceAttribute)
		{
			FVarAndDefaultSource* ExistingVarAndDefaultSource = ParamMapDefinedAttributesToNamespaceVars.Find(Var.GetName());
			FNiagaraVariable* ExistingVar = ExistingVarAndDefaultSource ? &ExistingVarAndDefaultSource->Variable : nullptr;

			bool ExistsInAttribArrayAlready = ExistingVar != nullptr;
			if (ExistsInAttribArrayAlready && ExistingVar->GetType() != Var.GetType())
			{
				if ((ExistingVar->GetType() == FNiagaraTypeDefinition::GetVec3Def() && Var.GetType() == FNiagaraTypeDefinition::GetPositionDef())
					|| (ExistingVar->GetType() == FNiagaraTypeDefinition::GetPositionDef() && Var.GetType() == FNiagaraTypeDefinition::GetVec3Def()))
				{
					if (bEnforceStrictTypesValidations)
					{
						Warning(FText::Format(LOCTEXT("MismatchedPositionTypes", "Variable {0} was defined both as position and vector, please check your modules and linked values for compatibility."), FText::FromName(Var.GetName())), ErrorNode, nullptr);
					}
				}
				else
				{
					Error(FText::Format(LOCTEXT("Mismatched Types", "Variable {0} was defined earlier, but its type is different! {1} != {2}"), FText::FromName(Var.GetName()),
						ExistingVar->GetType().GetNameText(), Var.GetType().GetNameText()), ErrorNode, nullptr);
				}
			}


			if ((TranslationStages.Num() > 1 && !UNiagaraScript::IsParticleSpawnScript(TranslationStages[ActiveStageIdx].ScriptUsage) && ExistsInAttribArrayAlready) ||
				!UNiagaraScript::IsSpawnScript(TranslationStages[ActiveStageIdx].ScriptUsage))
			{
				bIgnoreDefaultValue = true;
			}
		}



		if (LastSetChunkIdx == INDEX_NONE && (UNiagaraScript::IsSpawnScript(TranslationStages[ActiveStageIdx].ScriptUsage)))
		{
			if (FNiagaraParameterUtilities::IsInitialValue(Var))
			{
				FNiagaraVariable SourceForInitialValue = FNiagaraParameterUtilities::GetSourceForInitialValue(Var);
				bool bFoundExistingSet = false;
				for (int32 OtherParamIdx = 0; OtherParamIdx < OtherOutputParamMapHistories.Num(); OtherParamIdx++)
				{
					if (INDEX_NONE != OtherOutputParamMapHistories[OtherParamIdx].FindVariableByName(SourceForInitialValue.GetName()))
					{
						bFoundExistingSet = true;
					}
				}

				if (bFoundExistingSet)
				{
					LastSetChunkIdx = AddBodyChunk(
						ParameterMapInstanceName + TEXT(".") + BuildSanitizedSymbolName(Var.GetName()),
						ParameterMapInstanceName + TEXT(".") + BuildSanitizedSymbolName(SourceForInitialValue.GetName()), Var.GetType(),
						false
					);

					RecordParamMapDefinedAttributeToNamespaceVar(Var, DefaultPin);
				}
				else
				{
					//@todo(ng) disabled pending investigation UE-150159
					//Error(FText::Format(LOCTEXT("MissingInitialValueSource", "Variable {0} is used, but its source variable {1} is not set!"), FText::FromName(Var.GetName()), FText::FromName(SourceForInitialValue.GetName())), nullptr, nullptr);
				}
			}
			else if (UniqueVars.Contains(Var) && UniqueVarToChunk.Contains(Var))
			{
				int32* FoundIdx = UniqueVarToChunk.Find(Var);
				if (FoundIdx != nullptr)
				{
					LastSetChunkIdx = *FoundIdx;
				}
			}
		}


		if (LastSetChunkIdx == INDEX_NONE && !bIgnoreDefaultValue)
		{
			if (DefaultMode.IsSet() && *DefaultMode == ENiagaraDefaultMode::Binding && DefaultBinding.IsSet() && DefaultBinding->IsValid())
			{
				FNiagaraScriptVariableBinding Bind = *DefaultBinding;

				int32 Out = INDEX_NONE;
				FNiagaraVariable BindVar = FNiagaraVariable(InVar.GetType(), Bind.GetName());
				if (FNiagaraConstants::GetOldPositionTypeVariables().Contains(BindVar))
				{
					// Old assets often have vector inputs that default bind to what is now a position type. If we detect that, we change the type to prevent a compiler error.
					BindVar.SetType(FNiagaraTypeDefinition::GetPositionDef());
				}
				HandleParameterRead(ActiveStageIdx, BindVar, nullptr, ErrorNode, Out);

				if (Out != INDEX_NONE)
				{
					LastSetChunkIdx = Out;
				}
				else
				{
					Error(FText::Format(LOCTEXT("CannotFindBinding", "The module input {0} is bound to {1}, but {1} is not yet defined. Make sure {1} is defined prior to this module call."),
						FText::FromName(Var.GetName()),
						FText::FromName(Bind.GetName())), ErrorNode, nullptr);
				}
			}
			else if (InputPin != nullptr) // Default was found, trace back its inputs.
			{
				const FOutputPin* LinkedOutputPin = GraphBridge::GetLinkedOutputPin(InputPin);

				// Check to see if there are any overrides passed in to the translator. This allows us to bake in rapid iteration variables for performance.
				if (LinkedOutputPin == nullptr && bIsCandidateForRapidIteration && !TranslationOptions.bParameterRapidIteration)
				{
					bool bVarChanged = false;
					FNiagaraVariable RapidIterationConstantVar;
					if (!bWasEmitterAliased && ActiveHistoryForFunctionCalls.GetEmitterAlias() != nullptr)
					{
						RapidIterationConstantVar = FNiagaraParameterUtilities::ConvertVariableToRapidIterationConstantName(Var, *(*ActiveHistoryForFunctionCalls.GetEmitterAlias()), GetTargetUsage());
						bVarChanged = true;
					}
					else if (UNiagaraScript::IsSystemSpawnScript(CompileOptions.TargetUsage) || UNiagaraScript::IsSystemUpdateScript(CompileOptions.TargetUsage))
					{
						RapidIterationConstantVar = FNiagaraParameterUtilities::ConvertVariableToRapidIterationConstantName(Var, nullptr, GetTargetUsage());
						bVarChanged = true;
					}

					int32 FoundIdx = TranslationOptions.OverrideModuleConstants.Find(RapidIterationConstantVar);
					if (FoundIdx != INDEX_NONE)
					{
						FString DebugConstantStr;
						OutputChunkId = GetConstant(TranslationOptions.OverrideModuleConstants[FoundIdx], &DebugConstantStr);
						UE_LOGF(LogNiagaraEditor, Display, "Converted parameter %ls to constant %ls for script %ls", *Var.GetName().ToString(), *DebugConstantStr, *CompileOptions.FullName);
						return;
					}
					else if (!InputPin->bDefaultValueIsIgnored) // Use the default from the input pin because this variable was previously never encountered.
					{
						FNiagaraVariable PinVar = GraphBridge::GetPinVariable(InputPin, true, ENiagaraStructConversion::Simulation);
						FString DebugConstantStr;
						OutputChunkId = GetConstant(PinVar, &DebugConstantStr);
						UE_LOGF(LogNiagaraEditor, Display, "Converted default value of parameter %ls to constant %ls for script %ls. Likely added since this system was last compiled.", *Var.GetName().ToString(), *DebugConstantStr, *CompileOptions.FullName);
						return;
					}

					Error(FText::Format(LOCTEXT("InvalidRapidIterationReplacement", "Variable {0} is a rapid iteration param, but it wasn't found in the override list to bake out!"), FText::FromName(Var.GetName())), ErrorNode, nullptr);
				}

				CurrentDefaultPinTraversal.Push(InputPin);
				if (LinkedOutputPin)
				{
					// Double-check to make sure that we are connected to a TRANSLATOR_PARAM_BEGIN_DEFAULTS input node rather than
					// a normal parameter-based parameter map input node to ensure that we don't get into weird traversals.
					TArray<const FNode*> Nodes;
					const FNode* LinkedOutputPinNode = GraphBridge::GetOwningNode(LinkedOutputPin);
					GraphBridge::BuildTraversal(GraphBridge::GetOwningGraph(LinkedOutputPinNode), LinkedOutputPinNode, Nodes);
					for (const FNode* Node : Nodes)
					{
						const FInputNode* InputNode = GraphBridge::AsInputNode(Node);
						if (InputNode && GraphBridge::GetInputVariable(InputNode).GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
						{
							if (InputNode->Usage != ENiagaraInputNodeUsage::TranslatorConstant)
							{
								Error(FText::Format(LOCTEXT("InvalidParamMapStartForDefaultPin", "Default found for {0}, but the parameter map source for default pins needs to be a {1} node, not a generic input node."),
									FText::FromName(Var.GetName()),
									FText::FromName(TRANSLATOR_PARAM_BEGIN_DEFAULTS.GetName())), ErrorNode, nullptr);
							}
						}
					}
				}
				LastSetChunkIdx = CompileInputPin(InputPin);
				CurrentDefaultPinTraversal.Pop();
			}
			else
			{
				LastSetChunkIdx = GetConstant(Var);
			}

			if (!Var.IsDataInterface() && !Var.IsUObject() && LastSetChunkIdx != INDEX_NONE)
			{
				if (bTreatAsUnknownParameterMap == false)
				{
					if (VarIdx != INDEX_NONE && VarIdx < ParamMapSetVariablesToChunks[ParamMapHistoryIdx].Num())
					{
						// Record that we wrote to it.
						ParamMapSetVariablesToChunks[ParamMapHistoryIdx][VarIdx] = LastSetChunkIdx;
						RecordParamMapDefinedAttributeToNamespaceVar(Var, DefaultPin);
					}
					else if (VarIdx == INDEX_NONE && UniqueVars.Contains(Var))
					{
						RecordParamMapDefinedAttributeToNamespaceVar(Var, DefaultPin);
					}
					else
					{
						Error(FText::Format(LOCTEXT("NoVarDefaultFound", "Default found for {0}, but not found in ParameterMap traversal"), FText::FromName(Var.GetName())), ErrorNode, nullptr);
					}
				}

				// Actually insert the text that sets the default value
				if (LastSetChunkIdx != INDEX_NONE)
				{
					if (Var.GetType().GetClass() == nullptr) // Only need to do this wiring for things that aren't data interfaces.
					{
						AddBodyChunk(ParameterMapInstanceName + TEXT(".") + BuildSanitizedSymbolName(Var.GetName()), TEXT("{0}"), Var.GetType(), LastSetChunkIdx, false);
					}
				}
			}

			if (LastSetChunkIdx == INDEX_NONE && VarIdx != INDEX_NONE)
			{
				if (Var.IsDataInterface() || Var.IsUObject())
				{
					// If this variable is a data interface and it's in the parameter map, but hasn't been set yet, then is is an external data interface so try to register it.
					if (ParameterMapRegisterExternalConstantNamespaceVariable(Var, ErrorNode, ParamMapHistoryIdx, OutputChunkId, DefaultPin))
					{
						return;
					}
				}
			}
		}
	}

	// If we are of a data interface, we should output the data interface registration index, otherwise output
	// the map namespace that we're writing to.
	if (Var.IsDataInterface())
	{
		// In order for a module to compile successfully, we potentially need to generate default values
		// for variables encountered without ever being set. We do this by creating an instance of the CDO.
		if (UNiagaraScript::IsStandaloneScript(CompileOptions.TargetUsage) && LastSetChunkIdx == INDEX_NONE)
		{
			UNiagaraDataInterface* DataInterface = CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(const_cast<UClass*>(Var.GetType().GetClass()));
			check(DataInterface != nullptr);
			if (DataInterface)
			{
				LastSetChunkIdx = RegisterDataInterface(Var, DataInterface, true, false);
			}
		}

		OutputChunkId = LastSetChunkIdx;
	}
	else if (Var.IsUObject())
	{
		// In order for a module to compile successfully, we potentially need to generate default values
		// for variables encountered without ever being set. We do this by creating an instance of the CDO.
		if (UNiagaraScript::IsStandaloneScript(CompileOptions.TargetUsage) && LastSetChunkIdx == INDEX_NONE)
		{
			LastSetChunkIdx = RegisterUObject(Var, nullptr, true);
		}

		OutputChunkId = LastSetChunkIdx;
	}
	else
	{
		OutputChunkId = AddSourceChunk(ParameterMapInstanceName + TEXT(".") + BuildSanitizedSymbolName(Var.GetName()), Var.GetType());
		RecordParamMapDefinedAttributeToNamespaceVar(Var, DefaultPin);
	}
}

bool FNiagaraHlslTranslator::IsCompileOptionDefined(const TCHAR* InDefineStr)
{
	return CompileOptions.AdditionalDefines.Contains(InDefineStr);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ReadDataSet(const FNiagaraDataSetID DataSet, const TArray<FNiagaraVariable>& Variables, ENiagaraDataSetAccessMode AccessMode, int32 InputChunk, TArray<int32>& Outputs)
{
	//Eventually may allow events that take in a direct index or condition but for now we don't
	int32 ParamMapHistoryIdx = InputChunk;

	if (ParamMapHistoryIdx == -1)
	{
		Error(LOCTEXT("NoParamMapIdxToReadDataSet", "Cannot find parameter map for input to ReadDataSet!"), nullptr, nullptr);
		for (int32 i = 0; i < Outputs.Num(); i++)
		{
			Outputs[i] = INDEX_NONE;
		}
		return;
	}
	else if (ParamMapHistoryIdx >= ParamMapHistories.Num())
	{
		Error(FText::Format(LOCTEXT("InvalidParamMapIdxToReadDataSet", "Invalid parameter map index for ReadDataSet input {0} of {1}!"), ParamMapHistoryIdx, ParamMapHistories.Num()), nullptr, nullptr);
		for (int32 i = 0; i < Outputs.Num(); i++)
		{
			Outputs[i] = INDEX_NONE;
		}
		return;
	}

	TMap<int32, FDataSetAccessInfo>& Reads = DataSetReadInfo[(int32)AccessMode].FindOrAdd(DataSet);

	FDataSetAccessInfo* DataSetReadForInput = Reads.Find(InputChunk);
	if (!DataSetReadForInput)
	{
		if (Reads.Num() != 0) // If it is the same event within the graph that is ok, but we don't get here unless it is new.
		{
			Error(FText::Format(LOCTEXT("TooManyDataSetReads", "Only one Event Read node per Event handler! Read data set node: \"{0}\""), FText::FromName(DataSet.Name)), nullptr, nullptr);
		}

		DataSetReadForInput = &Reads.Add(InputChunk);

		DataSetReadForInput->Variables = Variables;
		DataSetReadForInput->CodeChunks.Reserve(Variables.Num() + 1);

		FString DataSetAccessSymbol = GetDataSetAccessSymbol(DataSet, InputChunk, true);
		//Add extra output to indicate if event read is valid data.
		//DataSetReadForInput->CodeChunks.Add(AddSourceChunk(DataSetAccessSymbol + TEXT("_Valid"), FNiagaraTypeDefinition::GetIntDef()));
		for (int32 i = 0; i < Variables.Num(); ++i)
		{
			const FNiagaraVariable& Var = Variables[i];
			if (!AddStructToDefinitionSet(Var.GetType()))
			{
				Error(FText::Format(LOCTEXT("GetConstantFailTypeVar", "Cannot handle type {0}! Variable: {1}"), Var.GetType().GetNameText(), FText::FromName(Var.GetName())), nullptr, nullptr);
			}
			DataSetReadForInput->CodeChunks.Add(AddSourceChunk(DataSetAccessSymbol + TEXT(".") + BuildSanitizedSymbolName(Var.GetName()), Var.GetType()));
		}
		Outputs.Add(ParamMapHistoryIdx);
		Outputs.Append(DataSetReadForInput->CodeChunks);
	}
	else
	{
		check(Variables.Num() == DataSetReadForInput->Variables.Num());
		Outputs.Add(ParamMapHistoryIdx);
		Outputs.Append(DataSetReadForInput->CodeChunks);
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::WriteDataSet(const FNiagaraDataSetID DataSet, const TArray<FNiagaraVariable>& Variables, ENiagaraDataSetAccessMode AccessMode, const TArray<int32>& Inputs, TArray<int32>& Outputs)
{
	int32 ParamMapHistoryIdx = Inputs[0];
	int32 ConditionalChunk = Inputs[1];
	int32 InputChunk = Inputs[2];
	Outputs.SetNum(1);
	Outputs[0] = ParamMapHistoryIdx;

	if (ParamMapHistoryIdx == -1)
	{
		Error(LOCTEXT("NoParamMapIdxToWriteDataSet", "Cannot find parameter map for input to WriteDataSet!"), nullptr, nullptr);
		for (int32 i = 0; i < Outputs.Num(); i++)
		{
			Outputs[i] = INDEX_NONE;
		}
		return;
	}
	else if (ParamMapHistoryIdx >= ParamMapHistories.Num())
	{
		Error(FText::Format(LOCTEXT("InvalidParamMapIdxToWriteDataSet", "Invalid parameter map index for WriteDataSet input {0} of {1}!"), ParamMapHistoryIdx, ParamMapHistories.Num()), nullptr, nullptr);
		for (int32 i = 0; i < Outputs.Num(); i++)
		{
			Outputs[i] = INDEX_NONE;
		}
		return;
	}

	if (DataSetWriteInfo[(int32)AccessMode].Find(DataSet))
	{
		Error(LOCTEXT("WritingToSameDataSetError", "Multiple writes to the same dataset.  Only one is allowed per script stage."), nullptr, nullptr);
		return;
	}

	TMap<int32, FDataSetAccessInfo>& Writes = DataSetWriteInfo[(int32)AccessMode].FindOrAdd(DataSet);
	FDataSetAccessInfo* DataSetWriteForInput = Writes.Find(InputChunk);
	int32& DataSetWriteConditionalIndex = DataSetWriteConditionalInfo[(int32)AccessMode].FindOrAdd(DataSet);

	//We should never try to write to the exact same dataset at the same index/condition twice.
	//This is still possible but we can catch easy cases here.
	if (DataSetWriteForInput)
	{
		//TODO: improve error report.
		Error(LOCTEXT("WritingToSameDataSetForInputError", "Writing to the same dataset with the same condition/index."), NULL, NULL);
		return;
	}

	DataSetWriteConditionalIndex = ConditionalChunk;
	DataSetWriteForInput = &Writes.Add(InputChunk);

	DataSetWriteForInput->Variables = Variables;

	//FString DataSetAccessName = GetDataSetAccessSymbol(DataSet, InputChunk, false);
	FString DataSetAccessName = FString("Context.") + DataSet.Name.ToString() + "Write";	// TODO: HACK - need to get the real symbol name here

	//First chunk for a write is always the condition pin.
	//We always write the event payload into the temp storage but we can access this condition to pass to the final actual write to the buffer.

	DataSetWriteForInput->CodeChunks.Add(AddBodyChunk(DataSetAccessName + TEXT("_Valid"), TEXT("{0}"), FNiagaraTypeDefinition::GetBoolDef(), Inputs[1], false));
	for (int32 i = 0; i < Variables.Num(); ++i)
	{
		const FNiagaraVariable& Var = Variables[i];
		int32 Input = Inputs[i + 2];//input 0 is the valid input (no entry in variables array), so we need of offset all other inputs by 1.
		DataSetWriteForInput->CodeChunks.Add(AddBodyChunk(DataSetAccessName + TEXT(".") + BuildSanitizedSymbolName(Var.GetName()), TEXT("{0}"), Var.GetType(), Input, false));
	}
}

template<typename GraphBridge>
int32 TNiagaraHlslTranslator<GraphBridge>::RegisterUObject(const FNiagaraVariable& Variable, UObject* Object, bool bAddParameterMapRead)
{
	int32 ObjectIndex = INDEX_NONE;
	if (bAddParameterMapRead)
	{
		// If we are registering a read then look for an existing one by name
		ObjectIndex = CompilationOutput.ScriptData.UObjectInfos.IndexOfByPredicate([&](const FNiagaraScriptUObjectCompileInfo& ExistingInfo) { return ExistingInfo.Variable == Variable; });
	}
	// Assume all writes are unique
	//else if ( Object != nullptr )
	//{
	//	// If we are registering a write then look for an existing one by value
	//	ObjectIndex = CompilationOutput.ScriptData.UObjectInfos.IndexOfByPredicate([&](const FNiagaraScriptUObjectCompileInfo& ExistingInfo) { return ExistingInfo.Object == Object; });
	//}

	FNiagaraScriptUObjectCompileInfo* CompileInfo = nullptr;
	if (ObjectIndex == INDEX_NONE)
	{
		ObjectIndex = CompilationOutput.ScriptData.UObjectInfos.AddDefaulted();
		CompileInfo = &CompilationOutput.ScriptData.UObjectInfos[ObjectIndex];
		CompileInfo->Variable = Variable;
		CompileInfo->Object = Object;
	}
	else
	{
		CompileInfo = &CompilationOutput.ScriptData.UObjectInfos[ObjectIndex];
	}

	if (bAddParameterMapRead)
	{
		FName ResolvedName;
		if (FNiagaraParameterUtilities::IsAliasedEmitterParameter(Variable.GetName().ToString()))
		{
			ResolvedName = ActiveHistoryForFunctionCalls.ResolveAliases(Variable).GetName();
		}
		else
		{
			ResolvedName = Variable.GetName();
		}
		CompileInfo->RegisteredParameterMapRead = ResolvedName;
	}

	return ObjectIndex;
}

template<typename GraphBridge>
int32 TNiagaraHlslTranslator<GraphBridge>::RegisterUObjectPath(const FNiagaraVariable& Variable, const FSoftObjectPath& ObjectPath, bool bAddParameterMapRead)
{
	int32 ObjectIndex = INDEX_NONE;
	if (bAddParameterMapRead)
	{
		// If we are registering a read then look for an existing one by name
		ObjectIndex = CompilationOutput.ScriptData.UObjectInfos.IndexOfByPredicate([&](const FNiagaraScriptUObjectCompileInfo& ExistingInfo) { return ExistingInfo.Variable == Variable; });
	}
	// Assume all writes are unique
	//else if ( Object != nullptr )
	//{
	//	// If we are registering a write then look for an existing one by value
	//	ObjectIndex = CompilationOutput.ScriptData.UObjectInfos.IndexOfByPredicate([&](const FNiagaraScriptUObjectCompileInfo& ExistingInfo) { return ExistingInfo.Object == Object; });
	//}

	FNiagaraScriptUObjectCompileInfo* CompileInfo = nullptr;
	if (ObjectIndex == INDEX_NONE)
	{
		ObjectIndex = CompilationOutput.ScriptData.UObjectInfos.AddDefaulted();
		CompileInfo = &CompilationOutput.ScriptData.UObjectInfos[ObjectIndex];
		CompileInfo->Variable = Variable;
		CompileInfo->ObjectPath = ObjectPath;
	}
	else
	{
		CompileInfo = &CompilationOutput.ScriptData.UObjectInfos[ObjectIndex];
	}

	if (bAddParameterMapRead)
	{
		FName ResolvedName;
		if (FNiagaraParameterUtilities::IsAliasedEmitterParameter(Variable.GetName().ToString()))
		{
			ResolvedName = ActiveHistoryForFunctionCalls.ResolveAliases(Variable).GetName();
		}
		else
		{
			ResolvedName = Variable.GetName();
		}
		CompileInfo->RegisteredParameterMapRead = ResolvedName;
	}

	return ObjectIndex;
}

template<typename GraphBridge>
int32 TNiagaraHlslTranslator<GraphBridge>::RegisterDataInterface(const FNiagaraVariable& Var, const UNiagaraDataInterface* DataInterface, bool bPlaceholder, bool bAddParameterMapRead)
{
	//FString PathName = DataInterface ? *DataInterface->GetPathName() : TEXT("XX");
	//UE_LOGF(LogNiagaraEditor, Log, "RegisterDataInterface %ls %ls %ls '%ls' %ls", *Var.GetName().ToString(), *Var.GetType().GetName(), *PathName,
	//	bPlaceholder ? TEXT("bPlaceholder=true") : TEXT("bPlaceholder=false"));

	int32 FuncParam;
	if (GetFunctionParameter(Var, FuncParam))
	{
		if (FuncParam != INDEX_NONE)
		{
			//This data interface param has been overridden by the function call so use that index.	
			return FuncParam;
		}
	}

	//If we get here then this is a new data interface.
	const FString* EmitterAlias = ActiveHistoryForFunctionCalls.GetEmitterAlias();
	FName DataInterfaceName = GetDataInterfaceName(Var.GetName(), EmitterAlias != nullptr ? *EmitterAlias : FString(), bAddParameterMapRead);

	if (DataInterface && DataInterface->NeedsGPUContextInit() && CompileOptions.IsGpuScript())
	{
		CompilationOutput.ScriptData.bNeedsGPUContextInit = true;
	}

	FNiagaraVariable ReadVariable;
	if (bAddParameterMapRead)
	{
		ReadVariable = FNiagaraParameterUtilities::IsAliasedEmitterParameter(Var.GetName().ToString())
			? ActiveHistoryForFunctionCalls.ResolveAliases(Var) 
			: Var;
	}

	int32 Idx = CompilationOutput.ScriptData.DataInterfaceInfo.IndexOfByPredicate([&](const FNiagaraScriptDataInterfaceCompileInfo& OtherInfo)
	{
		return OtherInfo.Name == DataInterfaceName || (ReadVariable.IsValid() && OtherInfo.RegisteredParameterMapWrite == ReadVariable.GetName());
	});

	if (Idx == INDEX_NONE)
	{
		Idx = CompilationOutput.ScriptData.DataInterfaceInfo.AddDefaulted();
		CompilationOutput.ScriptData.DataInterfaceInfo[Idx].Name = DataInterfaceName;
		CompilationOutput.ScriptData.DataInterfaceInfo[Idx].Type = Var.GetType();
		CompilationOutput.ScriptData.DataInterfaceInfo[Idx].bIsPlaceholder = bPlaceholder;

		FString SourceEmitterName = FString();
		if (CompileData->GetUniqueEmitterName().IsEmpty() == false)
		{
			SourceEmitterName = CompileData->GetUniqueEmitterName();
		}
		else if (EmitterAlias != nullptr)
		{
			SourceEmitterName = *EmitterAlias;
		}
		CompilationOutput.ScriptData.DataInterfaceInfo[Idx].SourceEmitterName = SourceEmitterName;

		//Interface requires per instance data so add a user pointer table entry.
		if (DataInterface != nullptr && DataInterface->PerInstanceDataSize() > 0)
		{
			CompilationOutput.ScriptData.DataInterfaceInfo[Idx].UserPtrIdx = CompilationOutput.ScriptData.NumUserPtrs++;
		}

		if (bAddParameterMapRead)
		{
			CompilationOutput.ScriptData.DataInterfaceInfo[Idx].RegisteredParameterMapRead = ReadVariable.GetName();
		}
	}
	else
	{
		check(CompilationOutput.ScriptData.DataInterfaceInfo[Idx].Type == Var.GetType());
	}

	return Idx;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Operation(const FOpNode* Operation, TArray<int32>& Inputs, TArray<int32>& Outputs)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_Operation);

	// Use the pins to determine the output type here since they may have been changed due to numeric pin fix up.
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(Operation->OpName);
	TArray<const FOutputPin*> OutputPins = GraphBridge::GetOutputPins(Operation);

	TArray<const FInputPin*> InputPins = GraphBridge::GetInputPins(Operation);
	TArray<FNiagaraTypeDefinition> InputTypes;

	bool bAllPinsStatic = true;
	{
		for (int32 InputIdx = 0; InputIdx < InputPins.Num(); InputIdx++)
		{
			FNiagaraTypeDefinition InputType = GraphBridge::GetPinType(InputPins[InputIdx], ENiagaraStructConversion::UserFacing);
			InputTypes.Add(InputType);
			if (!InputType.IsStatic())
				bAllPinsStatic = false;
		}

		for (int32 OutputIdx = 0; OutputIdx < OutputPins.Num(); OutputIdx++)
		{
			FNiagaraTypeDefinition OutputType = GraphBridge::GetPinType(OutputPins[OutputIdx], ENiagaraStructConversion::UserFacing);
			if (!OutputType.IsStatic())
				bAllPinsStatic = false;
		}
	}

	FText ValidationError;
	if (bEnforceStrictTypesValidations && OpInfo && OpInfo->InputTypeValidationFunction.IsBound() && OpInfo->InputTypeValidationFunction.Execute(InputTypes, ValidationError) == false)
	{
		Warning(ValidationError, Operation, OutputPins[0], TEXT("Invalid op types"));
	}

	if (OpInfo && OpInfo->StaticVariableResolveFunction.IsBound() && bAllPinsStatic)
	{
		if (OpInfo->Outputs.Num() != 1 || OutputPins.Num() != OpInfo->Outputs.Num())
		{
			FText PinNameText = GraphBridge::GetPinFriendlyName(OutputPins[0]).IsEmpty() ? FText::FromName(OutputPins[0]->PinName) : GraphBridge::GetPinFriendlyName(OutputPins[0]);
			Error(LOCTEXT("InvalidOutputPinCount", "Only one output pin is supported for static variables"), Operation, OutputPins[0]);
			Outputs.Add(INDEX_NONE);
			return;
		}

		FNiagaraTypeDefinition OutputType = GraphBridge::GetPinType(OutputPins[0], ENiagaraStructConversion::UserFacing);
		if(!OutputType.IsStatic())
		{
			Error(LOCTEXT("InvalidOutputPinType", "Only static types are supported for this operation!"), Operation, OutputPins[0]);
			Outputs.Add(INDEX_NONE);
			return;
		}
		
		int32 NumVars = 0;
		
		for (int32 InputIdx = 0; InputIdx < InputPins.Num(); InputIdx++)
		{
			FNiagaraTypeDefinition InputType = GraphBridge::GetPinType(InputPins[InputIdx], ENiagaraStructConversion::UserFacing);
			if (!InputType.IsStatic())
			{
				Error(LOCTEXT("InvalidInputPinType", "Only static types are supported for this operation!"), Operation, InputPins[InputIdx]);
				Outputs.Add(INDEX_NONE);
				return;
			}
			NumVars++;
		}

		const FNiagaraOpInOutInfo& IOInfo = OpInfo->Outputs[0];

		if (NumVars > 0)
		{
			int32 OutputChunkId = MakeStaticVariableDirect(OutputPins[0]);
			Outputs.Add(OutputChunkId);
		}
		else
		{
			Outputs.Add(INDEX_NONE);
		}
		return;
	}

	for (int32 OutputIndex = 0; OutputIndex < OutputPins.Num(); OutputIndex++)
	{
		const FOutputPin* OutputPin = OutputPins[OutputIndex];
		FNiagaraTypeDefinition OutputType = GraphBridge::GetPinType(OutputPin, ENiagaraStructConversion::Simulation);

		if (!AddStructToDefinitionSet(OutputType))
		{
			FText PinNameText = GraphBridge::GetPinFriendlyName(OutputPin).IsEmpty() ? FText::FromName(OutputPin->PinName) : GraphBridge::GetPinFriendlyName(OutputPin);
			Error(FText::Format(LOCTEXT("GetConstantFailTypePin", "Cannot handle type {0}! Output Pin: {1}"), OutputType.GetNameText(), PinNameText), Operation, OutputPin);
		}
		if (OpInfo != nullptr)
		{
			const FNiagaraOpInOutInfo& IOInfo = OpInfo->Outputs[OutputIndex];
			FString OutputHlsl;
			if (OpInfo->bSupportsAddedInputs)
			{
				if (!OpInfo->CreateHlslForAddedInputs(Inputs.Num(), OutputHlsl))
				{
					FText PinNameText = GraphBridge::GetPinFriendlyName(OutputPin).IsEmpty() ? FText::FromName(OutputPin->PinName) : GraphBridge::GetPinFriendlyName(OutputPin);
					Error(FText::Format(LOCTEXT("AggregateInputFailTypePin", "Cannot create hlsl output for type {0}! Output Pin: {1}"), OutputType.GetNameText(), PinNameText), Operation, OutputPin);
					OutputHlsl = IOInfo.HlslSnippet;
				}
			}
			else
			{
				OutputHlsl = IOInfo.HlslSnippet;
			}
			check(!OutputHlsl.IsEmpty());

			// add type casts if necessary
			for (int i = 0; i < InputPins.Num(); i++)
			{
				const FOutputPin* LinkedPin = GraphBridge::GetLinkedOutputPin(InputPins[i]);
				if (LinkedPin && InputTypes[i] != FNiagaraTypeDefinition::GetGenericNumericDef() && InputTypes[i] != GraphBridge::GetPinType(LinkedPin, ENiagaraStructConversion::UserFacing))
				{
					FString NumberedArg(TEXT("{") + FString::FromInt(i) + TEXT("}"));
					FString TypeName = GetStructHlslTypeName(InputTypes[i]);
					FString CastArg(TEXT("((") + TypeName + TEXT(")") + NumberedArg + TEXT(")"));
					OutputHlsl.ReplaceInline(*NumberedArg, *CastArg);
				}
			}
			Outputs.Add(AddBodyChunk(GetUniqueSymbolName(IOInfo.Name), OutputHlsl, OutputType, Inputs));
		}
	}
}

// simple helper template allowing us to keep FunctionCall() generic even if the Builder interface may diverge
template<typename GraphBridge>
class FScopedBuilderFunctionCall
{
};

// specialization of this helper function intended for the NiagaraGraphInterface 
template<>
class FScopedBuilderFunctionCall<FNiagaraCompilationGraphBridge>
{
public:
	FScopedBuilderFunctionCall(TNiagaraHlslTranslator<FNiagaraCompilationGraphBridge>* InTranslator, FNiagaraCompilationGraphBridge::FParamMapHistoryBuilder& InBuilder, const FNiagaraCompilationGraphBridge::FFunctionCallNode* InFunctionCallNode)
		: FunctionCallNode(InFunctionCallNode)
		, Builder(InBuilder)
	{
		Builder.EnterFunction(FunctionCallNode->GetFunctionName(), FunctionCallNode->GetCalledGraph(), FunctionCallNode);
	}

	~FScopedBuilderFunctionCall()
	{
		Builder.ExitFunction(FunctionCallNode);
	}

private:
	const FNiagaraCompilationGraphBridge::FFunctionCallNode* FunctionCallNode;
	FNiagaraCompilationGraphBridge::FParamMapHistoryBuilder& Builder;
};

template<>
class FScopedBuilderFunctionCall<FNiagaraCompilationDigestBridge>
{
public:
	FScopedBuilderFunctionCall(TNiagaraHlslTranslator<FNiagaraCompilationDigestBridge>* InTranslator, FNiagaraCompilationDigestBridge::FParamMapHistoryBuilder& InBuilder, const FNiagaraCompilationDigestBridge::FFunctionCallNode* InFunctionCallNode)
		: FunctionCallNode(InFunctionCallNode)
		, Builder(InBuilder)
	{
		Builder.TraversalStateContext->PushFunction(FunctionCallNode, FNiagaraFixedConstantResolver(InTranslator));
		Builder.EnterFunction(FunctionCallNode->FunctionName, FunctionCallNode->CalledGraph.Get(), FunctionCallNode);
	}

	~FScopedBuilderFunctionCall()
	{
		Builder.ExitFunction(FunctionCallNode);
		Builder.TraversalStateContext->PopFunction(FunctionCallNode);
	}

private:
	const FNiagaraCompilationDigestBridge::FFunctionCallNode* FunctionCallNode;
	FNiagaraCompilationDigestBridge::FParamMapHistoryBuilder& Builder;
};

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::FunctionCall(const FFunctionCallNode* FunctionNode, TArray<int32>& Inputs, TArray<int32>& Outputs)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_FunctionCall);

	TArray<const FOutputPin*> CallOutputs = GraphBridge::GetOutputPins(FunctionNode);
	TArray<const FInputPin*> CallInputs = GraphBridge::GetInputPins(FunctionNode);

	// Validate that there are no input pins with the same name and type
	TMultiMap<FName, FEdGraphPinType> SeenPins;
	for (const FPin* Pin : CallInputs)
	{
		FEdGraphPinType* SeenType = SeenPins.FindPair(Pin->PinName, Pin->PinType);
		if (SeenType)
		{
			Error(LOCTEXT("FunctionCallDuplicateInput", "Function call has duplicated inputs. Please make sure that each function parameter is unique."), FunctionNode, Pin);
			return;
		}
		else
		{
			SeenPins.Add(Pin->PinName, Pin->PinType);
		}
	}

	// If the function call is disabled, we 
	// need to route the input parameter map pin to the output parameter map pin.
	// Any other outputs become invalid.
	if (!GraphBridge::NodeIsEnabled(FunctionNode))
	{
		int32 InputPinIdx = INDEX_NONE;

		for (int32 i = 0; i < CallInputs.Num(); i++)
		{
			const FPin* Pin = CallInputs[i];
			if (GraphBridge::GetPinType(Pin, ENiagaraStructConversion::UserFacing) == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				// Found the input pin
				InputPinIdx = Inputs[i];
				break;
			}
		}

		Outputs.SetNum(CallOutputs.Num());
		for (int32 i = 0; i < CallOutputs.Num(); i++)
		{
			Outputs[i] = INDEX_NONE;
			const FPin* Pin = CallOutputs[i];
			if (GraphBridge::GetPinType(Pin, ENiagaraStructConversion::UserFacing) == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				// Mapping the input parameter map pin to the output.
				Outputs[i] = InputPinIdx;
			}
		}
		return;
	}

	FNiagaraFunctionSignature OutputSignature;
	const FGraph* FunctionGraph = GraphBridge::GetFunctionNodeGraph(FunctionNode);
	if (FunctionGraph == nullptr && !FunctionNode->Signature.IsValid())
	{
		Error(LOCTEXT("FunctionCallNonexistantFunctionScript", "Function call missing FunctionScript and invalid signature"), FunctionNode, nullptr);
		return;
	}

	//UE_LOGF(LogNiagaraEditor, Log, "Function Call: %ls %d", *FunctionNode->GetFunctionName(), ActiveStageIdx);
	// We need the generated string to generate the proper signature for now.
	FScopedBuilderFunctionCall<GraphBridge> ScopedBuilderFunctionCall(this, ActiveHistoryForFunctionCalls, FunctionNode);

	// Check if there are static switch parameters being set directly by a set node from the stack UI.
	// This can happen if a module was changed and the original parameter was replaced by a static switch with the same name, but the emitter was not yet updated.
	const FString* ModuleAlias = ActiveHistoryForFunctionCalls.GetModuleAlias();
	if (ModuleAlias)
	{
		for (int32 i = 0; i < ParamMapHistories.Num(); i++)
		{
			for (int32 j = 0; j < ParamMapHistories[i].VariablesWithOriginalAliasesIntact.Num(); j++)
			{
				const FNiagaraVariable Var = ParamMapHistories[i].VariablesWithOriginalAliasesIntact[j];
				FString VarStr = Var.GetName().ToString();
				if (VarStr.StartsWith(*ModuleAlias))
				{
					VarStr.MidInline(ModuleAlias->Len() + 1, MAX_int32, EAllowShrinking::No);
					if (FunctionNode->FindStaticSwitchInputPin(*VarStr))
					{
						Error(FText::Format(LOCTEXT("SwitchPinFoundForSetPin", "A switch node pin exists but is being set directly using Set node! Please use the stack UI to resolve the conflict. Output Pin: {0}"), FText::FromName(Var.GetName())), FunctionNode, nullptr);
					}
				}
			}
		}
	}

	ENiagaraScriptUsage ScriptUsage = ENiagaraScriptUsage::Function;
	FString Name;
	FString FullName;
	bool bCustomHlsl = false;
	FString CustomHlsl;
	TArray<FNiagaraCustomHlslInclude> CustomHlslIncludeFilePaths;
	FNiagaraFunctionSignature Signature = FunctionNode->Signature;

	if (FunctionGraph)
	{
		ScriptUsage = GraphBridge::GetFunctionUsage(FunctionNode);
		Name = GraphBridge::GetFunctionScriptName(FunctionNode);
		FullName = GraphBridge::GetFunctionFullName(FunctionNode);
	}
	else if (Signature.bRequiresExecPin)
	{
		if (CallInputs.Num() == 0 || GraphBridge::GetPinType(CallInputs[0], ENiagaraStructConversion::UserFacing) != FNiagaraTypeDefinition::GetParameterMapDef())
		{
			Error(LOCTEXT("FunctionCallInvalidSignatureExecIn", "The first input pin must be a parameter map pin because the signature RequiresExecPin!"), FunctionNode, nullptr);
		}
		if (CallOutputs.Num() == 0 || GraphBridge::GetPinType(CallOutputs[0], ENiagaraStructConversion::UserFacing) != FNiagaraTypeDefinition::GetParameterMapDef())
		{
			Error(LOCTEXT("FunctionCallInvalidSignatureExecOut", "The first output pin must be a parameter map pin because the signature RequiresExecPin!"), FunctionNode, nullptr);
		}
	}

	if (Signature.bIsCompileTagGenerator)
	{
		if (CallInputs.Num() != Inputs.Num())
		{
			Error(LOCTEXT("FunctionCallInvalidSignatureTagGen", "Mismatch in counts between signature and actual pins on the node!"), FunctionNode, nullptr);
		}
		else
		{
			FNiagaraVariable ResolvedVariable;
			for (int32 i = 0; i < CallInputs.Num(); i++)
			{
				FNiagaraTypeDefinition TypeDef = GraphBridge::GetPinType(CallInputs[i], ENiagaraStructConversion::UserFacing);
				if (TypeDef.IsDataInterface())
				{
					UNiagaraDataInterface* CDO = CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(TypeDef.GetClass());
					if (CDO == nullptr)
					{
						// If the cdo wasn't found, the data interface was not passed through a parameter map and so it won't be bound correctly, so add a compile error
						// and invalidate the signature.
						Error(LOCTEXT("DataInterfaceNotFoundTagGen", "Data interface used, but not found in precompiled data. Please notify Niagara team of bug."), nullptr, nullptr);
						return;
					}
				
					FString Prefix;
					if (!CDO->GenerateCompilerTagPrefix(Signature, Prefix))
					{
						Error(LOCTEXT("DataInterfaceFailedTagGen", "Data interface wanted to generate compiler tag, but was unable to resolve prefix. Please notify Niagara team of bug."), nullptr, nullptr);
					}

					FNiagaraVariable Variable(TypeDef, *(TEXT("Module.") + Prefix));

					if (FNiagaraParameterUtilities::IsAliasedModuleParameter(Variable) && ActiveHistoryForFunctionCalls.InTopLevelFunctionCall(CompileOptions.TargetUsage))
					{
						ResolvedVariable = ActiveHistoryForFunctionCalls.ResolveAliases(Variable);
					}
					else
					{
						ResolvedVariable = FNiagaraVariable(TypeDef, *Prefix);
					}

					Signature.FunctionSpecifiers.Add(FName("CompilerTagKey")) =  ResolvedVariable.GetName();

					break;
				}
			}


			for (int32 i = 0; i < CallInputs.Num(); i++)
			{
				FNiagaraTypeDefinition TypeDef = GraphBridge::GetPinType(CallInputs[i], ENiagaraStructConversion::UserFacing);
				if (!(TypeDef.IsDataInterface() || TypeDef == FNiagaraTypeDefinition::GetParameterMapDef()))
				{
					WriteCompilerTag(Inputs[i], CallInputs[i], false, false, FNiagaraCompileEventSeverity::Display, ResolvedVariable.GetName().ToString());
				}
			}
		}
	}

	if (Signature.NoDefaultValueInputs.Num() > 0)
	{
		for (int32 i = 0; i < CallInputs.Num(); i++)
		{
			const FInputPin* InPin = CallInputs[i];
			FNiagaraVariableBase Var = GraphBridge::GetPinVariable(InPin, false, ENiagaraStructConversion::Simulation);
			if (Signature.NoDefaultValueInputs.Contains(Var) && GraphBridge::GetLinkedOutputPin(InPin) == nullptr)
			{
				Error(LOCTEXT("NoDefaultValueInput", "Function call input pin doesn't support default values and needs to be wired in."), FunctionNode, InPin);
			}
		}
	}

	if (const FCustomHlslNode* CustomFunctionHlsl = GraphBridge::AsCustomHlslNode(FunctionNode))
	{
		// All of the arguments here are resolved withing the HandleCustomHlsl function..
		HandleCustomHlslNode(CustomFunctionHlsl, ScriptUsage, Name, FullName, bCustomHlsl, CustomHlsl, CustomHlslIncludeFilePaths, Signature, Inputs);
	}

	RegisterFunctionCall(ScriptUsage, Name, FName(FullName), FunctionNode->NodeGuid, FString(), FunctionGraph, Signature, bCustomHlsl, CustomHlsl, CustomHlslIncludeFilePaths, Inputs, CallInputs, CallOutputs, OutputSignature);

	if (OutputSignature.IsValid() == false)
	{
		Error(LOCTEXT("FunctionCallInvalidSignature", "Could not generate a valid function signature."), FunctionNode, nullptr);
		return;
	}

	GenerateFunctionCall(ScriptUsage, OutputSignature, Inputs, Outputs);
}

void FNiagaraHlslTranslator::EnterFunctionCallNode(const TSet<FName>& UnusedInputs)
{
	FunctionNodeStack.AddDefaulted_GetRef().UnusedInputs = UnusedInputs;
}

void FNiagaraHlslTranslator::ExitFunctionCallNode()
{
	ensure(FunctionNodeStack.Num() > 0);
	FunctionNodeStack.Pop(EAllowShrinking::No);
}

bool FNiagaraHlslTranslator::IsFunctionVariableCulledFromCompilation(const FName& InputName) const
{
	if (FunctionNodeStack.Num() == 0)
	{
		return false;
	}
	
	FunctionNodeStackEntry StackEntry = FunctionNodeStack.Last();
	if (StackEntry.UnusedInputs.Contains(InputName))
	{
		return true;
	}

	FString InputNameString = InputName.ToString();
	for (const FString& CulledFunction : StackEntry.CulledFunctionNames)
	{
		// if the entire function call was culled, we don't want to compile anything related to it
		if (InputNameString.StartsWith(CulledFunction + "."))
		{
			return true;
		}
	
	}
	return false;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::CullMapSetInputPin(const FPin* InputPin)
{
	if (FunctionNodeStack.IsEmpty())
	{
		return;
	}

	const FOutputPin* LinkedPin = GraphBridge::GetLinkedOutputPin(GraphBridge::GetPinAsInput(InputPin));
	if (!LinkedPin)
	{
		return;
	}

	// when a map set input is culled that is connected to a function call node (as is the case for dynamic inputs), we also need to cull any upstream pins that set inputs for the culled function call node
	if (const FFunctionCallNode* FunctionNode = GraphBridge::AsFunctionCallNode(GraphBridge::GetOwningNode(LinkedPin)))
	{
		FString FunctionScriptName = GraphBridge::GetFunctionName(FunctionNode);
		FunctionNodeStack.Last().CulledFunctionNames.Add(FunctionScriptName);
	}
}

// From a valid list of namespaces, resolve any aliased tokens and promote namespaced variables without a main namespace to the input parameter map instance namespace
template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::FinalResolveNamespacedTokens(const FString& ParameterMapInstanceNamespace, TArray<FString>& Tokens, TArray<FString>& ValidChildNamespaces, FParamMapHistoryBuilder& Builder, TArray<FNiagaraVariable>& UniqueParameterMapEntriesAliasesIntact, TArray<FNiagaraVariable>& UniqueParameterMapEntries, int32 ParamMapHistoryIdx, const FNode* InNodeForErrorReporting)
{
	for (int32 i = 0; i < Tokens.Num(); i++)
	{
		if (Tokens[i].Contains(TEXT("."))) // Only check tokens with namespaces in them..
		{
			for (const FString& ValidNamespace : ValidChildNamespaces)
			{
				FNiagaraVariable Var;

				// There are two possible paths here, one where we're using the namespace as-is from the valid list and one where we've already
				// prepended with the main parameter map instance namespace but may not have resolved any internal aliases yet.
				if (Tokens[i].StartsWith(ValidNamespace, ESearchCase::CaseSensitive))
				{
					FNiagaraVariable TempVar(FNiagaraTypeDefinition::GetFloatDef(), *Tokens[i]); // Declare a dummy name here that we will convert aliases for and use later
					Var = Builder.ResolveAliases(TempVar);
				}
				else if (Tokens[i].StartsWith(ParameterMapInstanceNamespace + ValidNamespace, ESearchCase::CaseSensitive))
				{
					FString BaseToken = Tokens[i].Mid(ParameterMapInstanceNamespace.Len());
					FNiagaraVariable TempVar(FNiagaraTypeDefinition::GetFloatDef(), *BaseToken); // Declare a dummy name here that we will convert aliases for and use later
					Var = Builder.ResolveAliases(TempVar);
				}

				if (Var.IsValid())
				{
					if (ParamMapHistoryIdx != INDEX_NONE)
					{
						bool bAdded = false;
						for (int32 j = 0; j < OtherOutputParamMapHistories.Num(); j++)
						{
							int32 VarIdx = OtherOutputParamMapHistories[j].FindVariableByName(Var.GetName(), true);
							if (VarIdx == INDEX_NONE) // Allow for the name to already have been HLSL-ized
								VarIdx = FNiagaraVariable::SearchArrayForPartialNameMatch(OtherOutputParamMapHistoriesSanitizedVariables[j], Var.GetName());

							if (VarIdx != INDEX_NONE)
							{
								if (OtherOutputParamMapHistories[j].VariablesWithOriginalAliasesIntact[VarIdx].IsValid())
								{
									UniqueParameterMapEntriesAliasesIntact.AddUnique(OtherOutputParamMapHistories[j].VariablesWithOriginalAliasesIntact[VarIdx]);
								}
								else
								{
									UniqueParameterMapEntriesAliasesIntact.AddUnique(OtherOutputParamMapHistories[j].Variables[VarIdx]);
								}
								UniqueParameterMapEntries.AddUnique(OtherOutputParamMapHistories[j].Variables[VarIdx]);
								bAdded = true;
								break;
							}
						}
						if (!bAdded && !UNiagaraScript::IsStandaloneScript(CompileOptions.TargetUsage)) // Don't warn in modules, they don't have enough context.
						{
							Error(FText::Format(LOCTEXT("GetCustomFail1", "Cannot use variable in custom expression, it hasn't been encountered yet: {0}"), FText::FromName(Var.GetName())), InNodeForErrorReporting, nullptr);
						}

					}


					Tokens[i] = ParameterMapInstanceNamespace + BuildSanitizedSymbolName(Var.GetName());
					break;
				}
			}
		}
	}
}

static bool IsWhitespaceToken(const FString& Token)
{
	return
		Token.Len() == 0 ||
		Token[0] == TCHAR('\r') || Token[0] == TCHAR('\n') || Token[0] == TCHAR('\t') || Token[0] == TCHAR(' ') || 
		(Token.Len() >= 2 && Token[0] == TCHAR('/') && (Token[1] == TCHAR('/') || Token[1] == TCHAR('*')))
		;
}

template<typename GraphBridge>
bool TNiagaraHlslTranslator<GraphBridge>::ParseDIFunctionSpecifiers(const FNode* NodeForErrorReporting, FNiagaraFunctionSignature& Sig, TArray<FString>& Tokens, int32& TokenIdx)
{
	const int32 NumTokens = Tokens.Num();

	// Skip whitespace between the function name and the arguments or specifiers.
	while (TokenIdx < NumTokens && IsWhitespaceToken(Tokens[TokenIdx]))
	{
		++TokenIdx;
	}

	// If we don't have a specifier list start token, we don't need to do anything.
	if (TokenIdx == NumTokens || Tokens[TokenIdx] != TEXT("<"))
	{
		return true;
	}

	enum class EParserState
	{
		ExpectName,
		ExpectEquals,
		ExpectValue,
		ExpectCommaOrEnd
	};

	EParserState ParserState = EParserState::ExpectName;
	FString SpecifierName;

	// All the tokens inside the specifier list, including the angle brackets, will be replaced with empty strings,
	// because they're not valid HLSL. We just want to extract Key=Value pairs into the signature's specifier list.
	while (TokenIdx < NumTokens)
	{
		FString Token = Tokens[TokenIdx];
		Tokens[TokenIdx] = TEXT("");
		++TokenIdx;

		if (IsWhitespaceToken(Token))
		{
			continue;
		}

		if (Token[0] == TCHAR('<'))
		{
			// Nothing.
		}
		else if (Token[0] == TCHAR('>'))
		{
			if (ParserState != EParserState::ExpectCommaOrEnd)
			{
				Error(LOCTEXT("DataInterfaceFunctionCallUnexpectedEnd", "Unexpected end of specifier list."), NodeForErrorReporting, nullptr);
				return false;
			}
			break;
		}
		else if (Token[0] == TCHAR('='))
		{
			if (ParserState == EParserState::ExpectEquals)
			{
				ParserState = EParserState::ExpectValue;
			}
			else
			{
				Error(LOCTEXT("DataInterfaceFunctionCallExpectEquals", "Invalid token in specifier list, expecting '='."), NodeForErrorReporting, nullptr);
				return false;
			}
		}
		else if (Token[0] == TCHAR(','))
		{
			if (ParserState == EParserState::ExpectCommaOrEnd)
			{
				ParserState = EParserState::ExpectName;
			}
			else
			{
				Error(LOCTEXT("DataInterfaceFunctionCallExpectComma", "Invalid token in specifier list, expecting ','."), NodeForErrorReporting, nullptr);
				return false;
			}
		}
		else
		{
			if (ParserState == EParserState::ExpectName)
			{
				SpecifierName = Token;
				ParserState = EParserState::ExpectEquals;
			}
			else if (ParserState == EParserState::ExpectValue)
			{
				int32 Start = 0, ValueLen = Token.Len();
				// Remove the quotation marks if they are used.
				if (Token.Len() >= 2 && Token[0] == TCHAR('"') && Token[ValueLen - 1] == TCHAR('"'))
				{
					Start = 1;
					ValueLen -= 2;
				}
				Sig.FunctionSpecifiers.Add(FName(SpecifierName), FName(Token.Mid(Start, ValueLen)));
				ParserState = EParserState::ExpectCommaOrEnd;
			}
		}
	}

	return true;
}

template<typename GraphBridge>
bool PartialParticleUpdateHelper(TConstArrayView<FString> Tokens, const typename GraphBridge::FPrecompileData& PrecompileData)
{
	static const FString UseParticleReadTokens[] =
	{
		TEXT("InputDataFloat"),
		TEXT("InputDataInt"),
		TEXT("InputDataBool"),
		TEXT("InputDataHalf"),
	};

	for (const FString& Token : Tokens)
	{
		for (const FString& BannedToken : UseParticleReadTokens)
		{
			if (Token == BannedToken)
			{
				// Clear out the ability to use partial particle writes as we can't be sure how InputData is being used
				for (const auto& CompileStageData : PrecompileData.CompileSimStageData)
				{
					CompileStageData.PartialParticleUpdate = false;
				}

				return true;
			}
		}
	}

	return false;
}

template<>
bool PartialParticleUpdateHelper<FNiagaraCompilationDigestBridge>(TConstArrayView<FString> Tokens, const FNiagaraPrecompileData& PrecompileData)
{
	// does nothing; this is taken care of in a pre-transation step
	return false;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::ProcessCustomHlsl(const FString& InCustomHlsl, ENiagaraScriptUsage InUsage, const FNiagaraFunctionSignature& InSignature, const TArray<int32>& Inputs, const FNode* InNodeForErrorReporting, FString& OutCustomHlsl, FNiagaraFunctionSignature& OutSignature)
{
	// Split up the hlsl into constituent tokens
	TArray<FStringView> TokenViews;
	UNiagaraNodeCustomHlsl::GetTokensFromString(InCustomHlsl, TokenViews);
	TArray<FString> Tokens;
	Tokens.Reset(TokenViews.Num());
	for (const FStringView View : TokenViews)
	{
		Tokens.Push(FString(View));
	}

	// Check for any access to LWC values in the View uniform buffer, and convert to float for backwards compat
	// Newly written code can access the LWC values directly using PrimaryView.X if desired
	{
		static const TCHAR* LWCViewMembers[] =
		{
			TEXT("WorldToClip"),
			TEXT("ClipToWorld"),
			TEXT("ScreenToWorld"),
			TEXT("PrevClipToWorld"),
			TEXT("WorldCameraOrigin"),
			TEXT("WorldViewOrigin"),
			TEXT("PrevWorldCameraOrigin"),
			TEXT("PrevWorldViewOrigin"),
			TEXT("PreViewTranslation"),
			TEXT("PrevPreViewTranslation"),
		};
		static const FString ViewNamespace(TEXT("View."));

		for (FString& Token : Tokens)
		{
			if (Token.StartsWith(ViewNamespace, ESearchCase::CaseSensitive))
			{
				FString TokenMemberName = FString(Token).Mid(ViewNamespace.Len());
				FString TokenPostfix;

				int32 MemberEnd = INDEX_NONE;
				if (TokenMemberName.FindChar(TCHAR('.'), MemberEnd))
				{
					TokenPostfix = TokenMemberName.Mid(MemberEnd);
					TokenMemberName = TokenMemberName.Mid(0, MemberEnd);
				}

				for (const TCHAR* LWCMember : LWCViewMembers)
				{
					if (TokenMemberName.Equals(LWCMember, ESearchCase::CaseSensitive))
					{
						Token = FString::Printf(TEXT("DFDemote(PrimaryView.%s)"), LWCMember);
						if (TokenPostfix.Len() > 0)
						{
							Token.Append(TokenPostfix);
						}
						break;
					}
				}
			}
		}
	}

	// Look for tokens that should be replaced with a data interface or not used directly
	if (CompilationTarget != ENiagaraSimTarget::GPUComputeSim)
	{
		if (PartialParticleUpdateHelper<GraphBridge>(Tokens, *CompileData))
		{
			Warning(LOCTEXT("UseParticleReadsNotInputData", "Please convert usage of InputData methods to particle reads to avoid compatibility issues."), InNodeForErrorReporting, nullptr);
		}
	}

	int32 ParamMapHistoryIdx = INDEX_NONE;
	bool bHasParamMapOutputs = false;
	bool bHasParamMapInputs = false;

	// Resolve the names of any internal variables from the input variables.
	TArray<FNiagaraVariable> SigInputs;
	for (int32 i = 0; i < OutSignature.Inputs.Num(); i++)
	{
		FNiagaraVariable Input = OutSignature.Inputs[i];
		if (Input.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			FNameBuilder ReplaceSrc(Input.GetName());
			FString ReplaceDest = GetParameterMapInstanceName(0);
			UNiagaraNodeCustomHlsl::ReplaceExactMatchTokens(Tokens, ReplaceSrc, ReplaceDest, true);

			SigInputs.Add(Input);
			OutSignature.bRequiresContext = true;
			ParamMapHistoryIdx = Inputs[i];
			bHasParamMapInputs = true;
		}
		else if (Input.GetType().IsDataInterface())
		{
			UNiagaraDataInterface* CDO = CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(Input.GetType().GetClass());
			if (CDO == nullptr)
			{
				// If the cdo wasn't found, the data interface was not passed through a parameter map and so it won't be bound correctly, so add a compile error
				// and invalidate the signature.
				Error(FText::Format(LOCTEXT("DataInterfaceNotFoundCustomHLSL", "Data interface ({0}) used by custom hlsl, but not found in precompiled data. Please notify Niagara team of bug."),
					FText::FromName(Input.GetName())),
					InNodeForErrorReporting,
					nullptr);
				return;
			}
			int32 OwnerIdx = Inputs[i];
			if (OwnerIdx < 0 || OwnerIdx >= CompilationOutput.ScriptData.DataInterfaceInfo.Num())
			{
				Error(LOCTEXT("FunctionCallDataInterfaceMissingRegistration", "Function call signature does not match to a registered DataInterface. Valid DataInterfaces should be wired into a DataInterface function call."), InNodeForErrorReporting, nullptr);
				return;
			}

			// Go over all the supported functions in the DI and look to see if they occur in the 
			// actual custom hlsl source. If they do, then add them to the function table that we need to map.
			FNiagaraScriptDataInterfaceCompileInfo& Info = CompilationOutput.ScriptData.DataInterfaceInfo[OwnerIdx];
			TArray<FNiagaraFunctionSignature> Funcs;
			CDO->GetFunctionSignatures(Funcs);

			TArray<FString> SanitizedFunctionNames;
			SanitizedFunctionNames.Reserve(Funcs.Num());
			for (const FNiagaraFunctionSignature& FunctionSignature : Funcs)
			{
				SanitizedFunctionNames.Emplace(GetSanitizedDIFunctionName(FunctionSignature.GetNameString()));
			}

			bool bPermuteSignatureByDataInterface = false;

			const FString InputPrefix = Input.GetName().ToString() + TEXT(".");
			for (int32 TokenIndex = 0; TokenIndex < Tokens.Num();)
			{
				// If we don't start with the prefix keep looking
				if ( !Tokens[TokenIndex].StartsWith(InputPrefix, ESearchCase::CaseSensitive) )
				{
					++TokenIndex;
					continue;
				}

				// Find matching function
				FStringView FunctionName(Tokens[TokenIndex]);
				FunctionName = FunctionName.Mid(InputPrefix.Len());

				const int32 FunctionIndex = SanitizedFunctionNames.IndexOfByPredicate([FunctionName](const FString& SigName){ return FunctionName.Equals(SigName, ESearchCase::CaseSensitive) != 0; });
				if (FunctionIndex == INDEX_NONE)
				{
					Error(
						FText::Format(LOCTEXT("DataInterfaceInvalidFunctionCustomHLSL", "Data interface '{0}' does not contain function '{1}' as used in custom HLSL."), FText::FromName(Input.GetName()), FText::FromString(FunctionName.GetData())),
						InNodeForErrorReporting,
						nullptr
					);
					return;
				}

				bPermuteSignatureByDataInterface = true;

				// We can't replace the method-style call with the actual function name yet, because function specifiers
				// are part of the name, and we haven't determined them yet. Just store a pointer to the token for now.
				FString& FunctionNameToken = Tokens[TokenIndex];
				++TokenIndex;

				FNiagaraFunctionSignature Sig = Funcs[FunctionIndex];

				// Override the owner id of the signature with the actual caller.
				Sig.OwnerName = Info.Name;

				// Function specifiers can be given inside angle brackets, using this syntax:
				//
				//		DI.Function<Specifier1=Value1, Specifier2="Value 2">(Arguments);
				//
				// We need to extract the specifiers and replace any tokens inside the angle brackets with empty strings,
				// to arrive back at valid HLSL.
				if (!ParseDIFunctionSpecifiers(InNodeForErrorReporting, Sig, Tokens, TokenIndex))
				{
					return;
				}

				// Now we can build the function name and replace the method call token with the final function name.
				FunctionNameToken = BuildFunctionSignatureString(Sig);
				if (Sig.bRequiresExecPin)
				{
					Sig.Inputs.Insert(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("InExecPin")), 0);
					Sig.Outputs.Insert(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("OutExecPin")), 0);
				}
				if (Info.UserPtrIdx != INDEX_NONE && CompilationTarget != ENiagaraSimTarget::GPUComputeSim)
				{
					//This interface requires per instance data via a user ptr so place the index as the first input.
					Sig.Inputs.Insert(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("InstanceData")), 0);

					// Look for the opening parenthesis.
					while (TokenIndex < Tokens.Num() && Tokens[TokenIndex] != TEXT("("))
					{
						++TokenIndex;
					}

					if (TokenIndex < Tokens.Num())
					{
						// Skip the parenthesis.
						++TokenIndex;

						// Insert the instance index as the first argument. We don't need to do range checking because even if
						// the tokens end after the parenthesis, we'll be inserting at the end of the array.
						Tokens.Insert(LexToString(Info.UserPtrIdx), TokenIndex++);

						// We only insert a comma if there are valid parameters, i.e. non-class, non-parameter map arugments
						// The first input argument is the instance data so we don't evaluate that
						// For example a impure function may have the instance data / exec pin / data interface as inputs, then an exec pin for output, none of which will translate to real parameters
						const auto IsRealHlslArg = [](const FNiagaraTypeDefinition& Type) { return !Type.GetClass() && Type != FNiagaraTypeDefinition::GetParameterMapDef(); };

						bool bHasRemainingArgs = false;
						for (int32 j = 1; j < Sig.Inputs.Num() && !bHasRemainingArgs; ++j)
						{
							bHasRemainingArgs = IsRealHlslArg(Sig.Inputs[j].GetType());
						}
						for (int32 j = 0; j < Sig.Outputs.Num() && !bHasRemainingArgs; ++j)
						{
							bHasRemainingArgs = IsRealHlslArg(Sig.Outputs[j].GetType());
						}

						if (bHasRemainingArgs)
						{
							// If there are other arguments, insert a comma and a space. These are separators, so they need to be different tokens.
							Tokens.Insert(TEXT(","), TokenIndex++);
							Tokens.Insert(TEXT(" "), TokenIndex++);
						}
					}
				}

				Info.RegisteredFunctions.Add(Sig);
				Functions.FindOrAdd(Sig);

				HandleDataInterfaceCall(Info, Sig);
			}

			if (bPermuteSignatureByDataInterface)
			{
				OutSignature.Name = FName(OutSignature.Name.ToString() + BuildSanitizedSymbolName(Info.Name, true));
			}

			SigInputs.Add(Input);
		}
		else
		{
			FNameBuilder ReplaceSrc(Input.GetName());
			TStringBuilder<128> ReplaceDest;
			ReplaceDest.Append(TEXT("In_"));
			ReplaceDest.Append(ReplaceSrc);

			UNiagaraNodeCustomHlsl::ReplaceExactMatchTokens(Tokens, ReplaceSrc, ReplaceDest, true);
			SigInputs.Add(Input);
		}
	}
	OutSignature.Inputs = SigInputs;

	// Resolve the names of any internal variables from the output variables.
	TArray<FNiagaraVariableBase> SigOutputs;
	for (FNiagaraVariableBase& Output : OutSignature.Outputs)
	{
		if (Output.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			FNameBuilder ReplaceSrc(Output.GetName());
			FString ReplaceDest = GetParameterMapInstanceName(0);

			UNiagaraNodeCustomHlsl::ReplaceExactMatchTokens(Tokens, ReplaceSrc, ReplaceDest, true);
			SigOutputs.Add(Output);
			OutSignature.bRequiresContext = true;
			bHasParamMapOutputs = true;
		}
		else
		{
			FNameBuilder ReplaceSrc(Output.GetName());
			TStringBuilder<128> ReplaceDest;
			ReplaceDest.Append(TEXT("Out_"));
			ReplaceDest.Append(ReplaceSrc);

			UNiagaraNodeCustomHlsl::ReplaceExactMatchTokens(Tokens, ReplaceSrc, ReplaceDest, true);
			SigOutputs.Add(Output);
		}
	}

	if (bHasParamMapOutputs || bHasParamMapInputs)
	{
		// Clean up any namespaced variables in the token list if they are aliased or promote any tokens that are namespaced to the parent 
		// parameter map.
		TArray<FString> PossibleNamespaces;
		FNiagaraParameterUtilities::GetValidNamespacesForReading(CompileOptions.TargetUsage, 0, PossibleNamespaces);

		for (FParamMapHistory& History : ParamMapHistories)
		{
			for (FNiagaraVariable& Var : History.Variables)
			{
				FString Namespace = FNiagaraParameterUtilities::GetNamespace(Var);
				PossibleNamespaces.AddUnique(Namespace);
			}
		}

		TArray<FNiagaraVariable> UniqueParamMapEntries;
		TArray<FNiagaraVariable> UniqueParamMapEntriesAliasesIntact;
		FinalResolveNamespacedTokens(GetParameterMapInstanceName(0) + TEXT("."), Tokens, PossibleNamespaces, ActiveHistoryForFunctionCalls, UniqueParamMapEntriesAliasesIntact, UniqueParamMapEntries, ParamMapHistoryIdx, InNodeForErrorReporting);

		// We must register any external constant variables that we encountered.
		for (int32 VarIdx = 0; VarIdx < UniqueParamMapEntriesAliasesIntact.Num(); VarIdx++)
		{
			FNiagaraVariable VarAliased = UniqueParamMapEntriesAliasesIntact[VarIdx];
			FNiagaraVariable VarActual = UniqueParamMapEntries[VarIdx];

			if (FNiagaraParameterUtilities::IsExternalConstantNamespace(VarAliased, CompileOptions.TargetUsage, CompileOptions.GetTargetUsageBitmask()) ||
				FNiagaraParameterUtilities::IsExternalConstantNamespace(VarActual, CompileOptions.TargetUsage, CompileOptions.GetTargetUsageBitmask()))
			{
				int32 TempOutput;
				if (ParameterMapRegisterExternalConstantNamespaceVariable(VarActual, InNodeForErrorReporting, ParamMapHistoryIdx, TempOutput, nullptr))
				{
					continue;
				}
			}
		}
	}

	// Now reassemble the tokens into the final hlsl output
	OutSignature.Outputs = SigOutputs;
	OutCustomHlsl = FString::Join(Tokens, TEXT(""));

	// Dynamic inputs are assumed to be of the form 
	// "20.0f * Particles.Velocity.x + length(Particles.Velocity)", i.e. a mix of native functions, constants, operations, and variable names.
	// This needs to be modified to match the following requirements:	
	// 1) Write to the output variable of the dynamic input.
	// 2) Terminate in valid HLSL (i.e. have a ; at the end)
	// 3) Be guaranteed to write to the correct output type.
	if (InUsage == ENiagaraScriptUsage::DynamicInput)
	{
		if (InSignature.Outputs.Num() != 1)
		{
			Error(LOCTEXT("CustomHlslDynamicInputMissingOutputs", "Custom hlsl dynamic input signature should have one and only one output."), InNodeForErrorReporting, nullptr);
			return;
		}
		if (InSignature.Inputs.Num() < 1 || InSignature.Inputs[0].GetType() != FNiagaraTypeDefinition::GetParameterMapDef())
		{
			Error(LOCTEXT("CustomHlslDynamicInputMissingInputs", "Custom hlsl dynamic input signature should have at least one input (a parameter map)."), InNodeForErrorReporting, nullptr);
			return;
		}

		OutSignature.bRequiresContext = true;
		FString ReplaceSrc = InSignature.Outputs[0].GetName().ToString();
		FString ReplaceDest = TEXT("Out_") + ReplaceSrc;
		OutCustomHlsl = ReplaceDest + TEXT(" = (") + GetStructHlslTypeName(InSignature.Outputs[0].GetType()) + TEXT(")(") + OutCustomHlsl + TEXT(");\n");
	}

	OutCustomHlsl = OutCustomHlsl.Replace(TEXT("\n"), TEXT("\n\t"));
	OutCustomHlsl = TEXT("\n") + OutCustomHlsl + TEXT("\n");
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::HandleCustomHlslNode(const FCustomHlslNode* CustomFunctionHlsl, ENiagaraScriptUsage& OutScriptUsage, FString& OutName, FString& OutFullName, bool& bOutCustomHlsl, FString& OutCustomHlsl, TArray<FNiagaraCustomHlslInclude>& OutCustomHlslIncludeFilePaths,
	FNiagaraFunctionSignature& OutSignature, TArray<int32>& Inputs)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_CustomHLSL);
	if (!CustomFunctionHlsl)
	{
		return;
	}

	// Determine the important outputs
	OutScriptUsage = GraphBridge::GetCustomHlslUsage(CustomFunctionHlsl);

	FString CustomHlslSourceString = GraphBridge::GetCustomHlslString(CustomFunctionHlsl);

	// because the NodeGuid isn't necessarily sufficiently unique we're going to use a hash of the NodeGuid, the source and the inputs & outputs of the node to
	// be used as a unique identifier for the name
	FString UniqueIdentifier;

	{
		FSHA1 HashState;
		HashState.Update((const uint8*)&CustomFunctionHlsl->NodeGuid, sizeof(CustomFunctionHlsl->NodeGuid));
		HashState.UpdateWithString(*CustomHlslSourceString, CustomHlslSourceString.Len());

		TArray<const FPin*> Pins = GraphBridge::GetPins(CustomFunctionHlsl);
		for (const FPin* Pin : Pins)
		{
			FNiagaraVariable PinVariable = GraphBridge::GetPinVariable(Pin, false /*bNeedsValue*/, ENiagaraStructConversion::UserFacing);
			const uint32 PinVariableTypeHash = GetPersistentTypeHash(PinVariable);
			HashState.Update((const uint8*)&PinVariableTypeHash, sizeof(PinVariableTypeHash));
		}

		UniqueIdentifier = HashState.Finalize().ToString();
	}

	OutName = BuildSanitizedSymbolString(CustomFunctionHlsl->Signature.Name.ToString() + UniqueIdentifier);
	OutSignature = CustomFunctionHlsl->Signature;
	OutFullName = GraphBridge::GetFunctionFullName(CustomFunctionHlsl);
	OutSignature.Name = *OutName; // Force the name to be set to include the node guid for safety...
	bOutCustomHlsl = true;
	OutCustomHlsl = CustomHlslSourceString;
	OutCustomHlsl.ReplaceInline(TEXT("\r\n"), TEXT("\n"), ESearchCase::CaseSensitive);
	GraphBridge::GetCustomHlslIncludePaths(CustomFunctionHlsl, OutCustomHlslIncludeFilePaths);

	FNiagaraFunctionSignature InSignature = CustomFunctionHlsl->Signature;
	ProcessCustomHlsl(OutCustomHlsl, OutScriptUsage, InSignature, Inputs, CustomFunctionHlsl, OutCustomHlsl, OutSignature);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::HandleDataInterfaceCall(FNiagaraScriptDataInterfaceCompileInfo& Info, const FNiagaraFunctionSignature& InMatchingSignature)
{
	const bool bCPUSim = CompileOptions.IsCpuScript();
	const bool bGPUSim = CompileOptions.IsGpuScript();
	const FNode* CurNode = ActiveHistoryForFunctionCalls.GetCallingContext();
	if (bCPUSim && !InMatchingSignature.bSupportsCPU)
	{
		Error(FText::Format(LOCTEXT("FunctionCallDataInterfaceCPUMissing", "Function call \"{0}\" does not work on CPU sims."), FText::FromName(InMatchingSignature.Name)), CurNode, nullptr);
	}
	else if (bGPUSim && !InMatchingSignature.bSupportsGPU)
	{
		Error(FText::Format(LOCTEXT("FunctionCallDataInterfaceGPUMissing", "Function call \"{0}\" does not work on GPU sims."), FText::FromName(InMatchingSignature.Name)), CurNode, nullptr);
	}

	if (InMatchingSignature.ModuleUsageBitmask != 0 && !UNiagaraScript::IsSupportedUsageContextForBitmask(InMatchingSignature.ModuleUsageBitmask, TranslationStages[ActiveStageIdx].ScriptUsage))
	{
		UEnum* EnumClass = StaticEnum<ENiagaraScriptUsage>();

		FString AllowedContexts;
		TArray<ENiagaraScriptUsage> Usages = UNiagaraScript::GetSupportedUsageContextsForBitmask(InMatchingSignature.ModuleUsageBitmask);
		for (ENiagaraScriptUsage Usage : Usages)
		{
			if (AllowedContexts.Len() > 0)
			{
				AllowedContexts.Append(TEXT(", "));
			}
			check(EnumClass != nullptr);
			AllowedContexts.Append(EnumClass->GetNameByValue(static_cast<int64>(Usage)).ToString());
		}
		
		FText ThisContextText = FText::FromName(EnumClass->GetNameByValue(static_cast<int64>(TranslationStages[ActiveStageIdx].ScriptUsage)));
		Error(FText::Format(LOCTEXT("FunctionCallDataInterfaceWrongContext", "Function call \"{0}\" is not allowed for stack context {1}. Allowed: {2}"), FText::FromName(InMatchingSignature.Name), ThisContextText, FText::FromString(AllowedContexts)), CurNode, nullptr);
	}

	// register structs used by DI parameters
	for (const FNiagaraVariable& Input : InMatchingSignature.Inputs)
	{
		if (!AddStructToDefinitionSet(Input.GetType()))
		{
			Error(FText::Format(LOCTEXT("DIFunctionInputTypeError", "Cannot handle type {0} in DI function input. Function {1}, Input {2}"), Input.GetType().GetNameText(), FText::FromName(InMatchingSignature.Name), FText::FromName(Input.GetName())), CurNode, nullptr);
		}
	}
	for (const FNiagaraVariableBase& Output : InMatchingSignature.Outputs)
	{
		if (!AddStructToDefinitionSet(Output.GetType()))
		{
			Error(FText::Format(LOCTEXT("DIFunctionOutputTypeError", "Cannot handle type {0} in DI function output. Function {1}, Output {2}"), Output.GetType().GetNameText(), FText::FromName(InMatchingSignature.Name), FText::FromName(Output.GetName())), CurNode, nullptr);
		}
	}
	
	//UE_LOGF(LogNiagaraEditor, Log, "HandleDataInterfaceCall %d %ls %ls %ls", ActiveStageIdx, *InMatchingSignature.Name.ToString(), InMatchingSignature.bWriteFunction ? TEXT("true") : TEXT("False"), *Info.Name.ToString());

	if (InMatchingSignature.bWriteFunction && CompilationOutput.ScriptData.SimulationStageMetaData.Num() > 1 && TranslationStages[ActiveStageIdx].SimulationStageIndex != -1)
	{
		const int32 SourceSimStage = TranslationStages[ActiveStageIdx].SimulationStageIndex;
		CompilationOutput.ScriptData.SimulationStageMetaData[SourceSimStage].OutputDestinations.AddUnique(Info.Name);
		if (ActiveStageWriteTargets.Num() > 0)
		{
			ActiveStageWriteTargets.Top().AddUnique(Info.Name);
		}
	}
	if (InMatchingSignature.bReadFunction && CompilationOutput.ScriptData.SimulationStageMetaData.Num() > 1 && TranslationStages[ActiveStageIdx].SimulationStageIndex != -1)
	{
		const int32 SourceSimStage = TranslationStages[ActiveStageIdx].SimulationStageIndex;
		CompilationOutput.ScriptData.SimulationStageMetaData[SourceSimStage].InputDataInterfaces.AddUnique(Info.Name);
		if (ActiveStageReadTargets.Num() > 0)
		{
			ActiveStageReadTargets.Top().AddUnique(Info.Name);
		}
	}
}

template<typename GraphBridge>
bool IsVariableWriteBeforeRead(const TArray<typename GraphBridge::FParamMapHistory::FReadHistory>& ReadHistory)
{
	for (const typename GraphBridge::FParamMapHistory::FReadHistory& History : ReadHistory)
	{
		if (History.PreviousWritePin.Pin == nullptr)
		{
			return false;
		}
	}
	return true;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::RegisterFunctionCall(ENiagaraScriptUsage ScriptUsage, const FString& InName, FName InFullName, const FGuid& CallNodeId, const FString& InFunctionNameSuffix, const FGraph* SourceGraph,
                                                  const FNiagaraFunctionSignature& InSignature, bool bIsCustomHlsl, const FString& InCustomHlsl, const TArray<FNiagaraCustomHlslInclude>& InCustomHlslIncludeFilePaths, TArray<int32>& Inputs, TConstArrayView<const FInputPin*> CallInputs, TConstArrayView<const FOutputPin*> CallOutputs,
                                                  FNiagaraFunctionSignature& OutSignature)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall);

	//////////////////////////////////////////////////////////////////////////
	if (SourceGraph)
	{
		NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall_Source);

		FString FunctionComment;
		GenerateFunctionSignature(ScriptUsage, InName, InFullName, InFunctionNameSuffix, SourceGraph, CallInputs, Inputs, FunctionComment, OutSignature);

		// 		//Sort the input and outputs to match the sorted parameters. They may be different.
		// 		TArray<FNiagaraVariable> OrderedInputs;
		// 		TArray<FNiagaraVariable> OrderedOutputs;
		// 		SourceGraph->GetParameters(OrderedInputs, OrderedOutputs);
		// 		FPinCollectorArray InPins;
		// 		FunctionNode->GetInputPins(InPins);
		// 
		// 		TArray<int32> OrderedInputChunks;
		// 		OrderedInputChunks.SetNumUninitialized(Inputs.Num());
		// 		for (int32 i = 0; i < InPins.Num(); ++i)
		// 		{
		// 			FNiagaraVariable PinVar(Schema->PinToTypeDefinition(InPins[i]), *InPins[i]->PinName);
		// 			int32 InputIdx = OrderedInputs.IndexOfByKey(PinVar);
		// 			check(InputIdx != INDEX_NONE);
		// 			OrderedInputChunks[i] = Inputs[InputIdx];
		// 		}
		// 		Inputs = OrderedInputChunks;

		FNiagaraFunctionBody* FuncBody = Functions.Find(OutSignature);
		if (!FuncBody)
		{
			NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_FuncBody);

			if (OutSignature.Name == NAME_None)
			{
				const FString* ModuleAlias = ActiveHistoryForFunctionCalls.GetModuleAlias();
				Error(FText::Format(LOCTEXT("FunctionCallMissingFunction", "Function call signature does not reference a function. Top-level module: {0} Source: {1}"), ModuleAlias ? FText::FromString(*ModuleAlias) : FText::FromString(TEXT("Unknown module")), FText::FromString(CompileOptions.FullName)), nullptr, nullptr);
				return;
			}

			bool bIsModuleFunction = false;
			bool bStageMinFilter = false;
			bool bStageMaxFilter = false;
			FString MinParam;
			FString MaxParam;

			//We've not compiled this function yet so compile it now.
			EnterFunction(InName, OutSignature, Inputs, CallNodeId);

			const FOutputNode* FuncOutput = SourceGraph->FindOutputNode(ScriptUsage, FGuid());
			check(FuncOutput);

			if (ActiveHistoryForFunctionCalls.GetModuleAlias() != nullptr)
			{
				bool bIsInTopLevelFunction = ActiveHistoryForFunctionCalls.InTopLevelFunctionCall(CompileOptions.TargetUsage);

				const FInputPin* ParamMapPin = nullptr;
				for (const FInputPin* Pin : CallInputs)
				{
					if (GraphBridge::GetPinType(Pin, ENiagaraStructConversion::UserFacing) == FNiagaraTypeDefinition::GetParameterMapDef())
					{
						ParamMapPin = Pin;
						break;
					}
				}

				if (ParamMapPin != nullptr)
				{
					bIsModuleFunction = (bIsInTopLevelFunction && ParamMapPin != nullptr && UNiagaraScript::IsGPUScript(CompileOptions.TargetUsage));

					const FNode* ParamNode = GraphBridge::GetOwningNode(ParamMapPin);
					if (ParamNode)
					{
						check(ParamMapHistories.Num() == TranslationStages.Num());
						const FParamMapHistory& History = ParamMapHistories[ActiveStageIdx];
						uint32 FoundIdx = History.MapNodeVisitations.Find(ParamNode);
						if (FoundIdx != INDEX_NONE)
						{
							check((uint32)History.MapNodeVariableMetaData.Num() > FoundIdx);
							check(INDEX_NONE != History.MapNodeVariableMetaData[FoundIdx].Key);
							check(INDEX_NONE != History.MapNodeVariableMetaData[FoundIdx].Value);

							for (uint32 VarIdx = History.MapNodeVariableMetaData[FoundIdx].Key; VarIdx < History.MapNodeVariableMetaData[FoundIdx].Value; VarIdx++)
							{
								if (IsVariableWriteBeforeRead<GraphBridge>(History.PerVariableReadHistory[VarIdx]))
								{
									// We don't need to worry about defaults if the variable is written before being read or never read at all.
									continue;
								}

								const FNiagaraVariable& Var = History.Variables[VarIdx];
								const FNiagaraVariable& AliasedVar = History.VariablesWithOriginalAliasesIntact[VarIdx];
								const bool bIsAliased = Var.GetName() != AliasedVar.GetName();

								// For non aliased values we resolve the defaults once at the top level since it's impossible to know which context they were actually used in, but
								// for aliased values we check to see if they're used in the current context by resolving the alias and checking against the current resolved variable
								// name since aliased values can only be resolved for reading in the correct context.
								bool bIsValidForCurrentCallingContext = (bIsInTopLevelFunction && bIsAliased == false) || (bIsAliased && ActiveHistoryForFunctionCalls.ResolveAliases(AliasedVar).GetName() == Var.GetName());
								if (bIsValidForCurrentCallingContext && !Var.GetType().IsStatic() )
								{
									int32 LastSetChunkIdx = ParamMapSetVariablesToChunks[ActiveStageIdx][VarIdx];
									if (LastSetChunkIdx == INDEX_NONE)
									{
										const FInputPin* DefaultPin = History.GetDefaultValuePin(VarIdx);
										FNiagaraScriptVariableBinding DefaultBinding;
										TOptional<ENiagaraDefaultMode> DefaultMode = GraphBridge::GetGraphDefaultMode(SourceGraph, AliasedVar, DefaultBinding);

										// Do not error on defaults for parameter reads here; we may be entering a SetVariable function call which is setting the first default for a parameter.
										const bool bTreatAsUnknownParameterMap = false;
										const bool bIgnoreDefaultSetFirst = true;
										HandleParameterRead(ActiveStageIdx, AliasedVar, DefaultPin, ParamNode, LastSetChunkIdx, DefaultMode, DefaultBinding, bTreatAsUnknownParameterMap, bIgnoreDefaultSetFirst);
										
										// If this variable was in the pending defaults list, go ahead and remove it
										// as we added it before first use...
										if (DeferredVariablesMissingDefault.Contains(Var))
										{
											DeferredVariablesMissingDefault.Remove(Var);
											UniqueVarToChunk.Add(Var, LastSetChunkIdx);
										}
									}
								}
							}
						}
					}
				}
			}

			//Track the start of this function in the chunks so we can remove them after we grab the function hlsl.
			int32 ChunkStart = CodeChunks.Num();
			int32 ChunkStartsByMode[(int32)ENiagaraCodeChunkMode::Num];
			for (int32 i = 0; i < (int32)ENiagaraCodeChunkMode::Num; ++i)
			{
				ChunkStartsByMode[i] = ChunksByMode[i].Num();
			}

			TArray<int32> FuncOutputChunks;

			ENiagaraCodeChunkMode OldMode = CurrentBodyChunkMode;
			CurrentBodyChunkMode = ENiagaraCodeChunkMode::Body;
			{
				NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall_Compile);
				FuncOutput->Compile(this, FuncOutputChunks);
			}
			CurrentBodyChunkMode = OldMode;

			{
				NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall_FunctionDefStr);

				FNiagaraFunctionBody& FunctionBody = Functions.Add(OutSignature);

				//Grab all the body chunks for this function.
				FunctionBody.StageIndices.AddUnique(ActiveStageIdx);
				FunctionBody.Body.Reserve(256 * ChunksByMode[(int32)ENiagaraCodeChunkMode::Body].Num());
				FunctionBody.Comments = MoveTemp(FunctionComment);

				for (int32 i = ChunkStartsByMode[(int32)ENiagaraCodeChunkMode::Body]; i < ChunksByMode[(int32)ENiagaraCodeChunkMode::Body].Num(); ++i)
				{
					FunctionBody.Body += GetCode(ChunksByMode[(int32)ENiagaraCodeChunkMode::Body][i]);
				}

				//Now remove all chunks for the function again.			
				//This is super hacky. Should move chunks etc into a proper scoping system.
				const int32 UniformMode = static_cast<int32>(ENiagaraCodeChunkMode::Uniform);
				const int32 FuncUniformCount = ChunksByMode[UniformMode].Num() - ChunkStartsByMode[UniformMode];
				TArray<FNiagaraCodeChunk> FuncUniforms;
				TArray<int32> OriginalUniformChunkIndices;

				FuncUniforms.Reserve(FuncUniformCount);
				OriginalUniformChunkIndices.Reserve(FuncUniformCount);

				for (int32 i = 0; i < (int32)ENiagaraCodeChunkMode::Num; ++i)
				{
					//Keep uniform chunks.
					if (i == UniformMode)
					{
						for (int32 ChunkIdx = ChunkStartsByMode[i]; ChunkIdx < ChunksByMode[i].Num(); ++ChunkIdx)
						{
							FuncUniforms.Add(CodeChunks[ChunksByMode[i][ChunkIdx]]);
							OriginalUniformChunkIndices.Add(ChunksByMode[i][ChunkIdx]);
						}
					}

					ChunksByMode[i].RemoveAt(ChunkStartsByMode[i], ChunksByMode[i].Num() - ChunkStartsByMode[i]);
				}
				CodeChunks.RemoveAt(ChunkStart, CodeChunks.Num() - ChunkStart, EAllowShrinking::No);

				//Re-add the uniforms. Really this is horrible. Rework soon.
				for (int32 FuncUniformIt = 0; FuncUniformIt < FuncUniformCount; ++FuncUniformIt)
				{
					const FNiagaraCodeChunk& Chunk = FuncUniforms[FuncUniformIt];
					const int32 OriginalChunkIndex = OriginalUniformChunkIndices[FuncUniformIt];

					const int32 NewChunkIndex = CodeChunks.Add(Chunk);
					ChunksByMode[UniformMode].Add(NewChunkIndex);

					for (auto& SystemVarPair : ParamMapDefinedSystemVars)
					{
						if ((SystemVarPair.Value.ChunkIndex == OriginalChunkIndex)
							&& (SystemVarPair.Value.ChunkMode == UniformMode))
						{
							SystemVarPair.Value.ChunkIndex = NewChunkIndex;
						}
					}

					for (auto& PackedUniformPair : UniformParametersToPack)
					{
						if (PackedUniformPair.Value == OriginalChunkIndex)
						{
							PackedUniformPair.Value = NewChunkIndex;
						}
					}
				}

				// We don't support an empty function definition when calling a real function.
				if (FunctionBody.Body.IsEmpty())
				{
					FunctionBody.Body += TEXT("\n");
				}

				FunctionBody.StageWriteTargets = ActiveStageWriteTargets.Top();
				FunctionBody.StageReadTargets = ActiveStageReadTargets.Top();
			}

			ExitFunction();
		}
		else
		{
			FuncBody->StageIndices.AddUnique(ActiveStageIdx);

			// Just because we had a cached call, doesn't mean that we should ignore adding read or writetargets
			for (const FName& Entry : FuncBody->StageWriteTargets)
			{
				const int32 SourceSimStage = TranslationStages[ActiveStageIdx].SimulationStageIndex;
				CompilationOutput.ScriptData.SimulationStageMetaData[SourceSimStage].OutputDestinations.AddUnique(Entry);
				ActiveStageWriteTargets.Top().AddUnique(Entry);
			}

			for (const FName& Entry : FuncBody->StageReadTargets)
			{
				const int32 SourceSimStage = TranslationStages[ActiveStageIdx].SimulationStageIndex;
				CompilationOutput.ScriptData.SimulationStageMetaData[SourceSimStage].InputDataInterfaces.AddUnique(Entry);
				ActiveStageReadTargets.Top().AddUnique(Entry);
			}
		}
	}
	else
	{
		NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_RegisterFunctionCall_Signature);

		check(InSignature.IsValid());

		OutSignature = InSignature;

		//First input for these is the owner of the function.
		if (bIsCustomHlsl)
		{
			FNiagaraFunctionBody* FuncBody = Functions.Find(OutSignature);
			if (!FuncBody)
			{
				//We've not compiled this function yet so compile it now.
				EnterFunction(InName, OutSignature, Inputs, CallNodeId);

				FNiagaraFunctionBody& FunctionBody = Functions.Add(OutSignature);
				FunctionBody.Body = InCustomHlsl;
				FunctionBody.StageIndices.AddUnique(ActiveStageIdx);

				// We don't support an empty function definition when calling a real function.
				if (FunctionBody.Body.IsEmpty())
				{
					FunctionBody.Body += TEXT("\n");
				}

				FunctionBody.StageWriteTargets = ActiveStageWriteTargets.Top();
				FunctionBody.StageReadTargets = ActiveStageReadTargets.Top();

				for (const FNiagaraCustomHlslInclude& FileInclude : InCustomHlslIncludeFilePaths)
				{
					FunctionIncludeFilePaths.AddUnique(FileInclude);
				}

				ExitFunction();
			}
			else
			{
				FuncBody->StageIndices.AddUnique(ActiveStageIdx);
			}
		}
		else if (!InSignature.bMemberFunction) // Fastpath or other provided function
		{
			if (INDEX_NONE == CompilationOutput.ScriptData.AdditionalExternalFunctions.Find(OutSignature))
			{
				CompilationOutput.ScriptData.AdditionalExternalFunctions.Add(OutSignature);
			}
			Functions.FindOrAdd(OutSignature);
		}
		else
		{
			check(!Inputs.IsEmpty());

			// Usually the DataInterface is the zeroth entry in the signature inputs, unless we are using the exec pin, in which case it is at index 1.
			int32 DataInterfaceOwnerIdx = Inputs[0]; 
			if (InSignature.bRequiresExecPin)
			{
				ensure(Inputs.IsValidIndex(1));
				DataInterfaceOwnerIdx = Inputs[1]; 
			}

			if (DataInterfaceOwnerIdx < 0 || DataInterfaceOwnerIdx >= CompilationOutput.ScriptData.DataInterfaceInfo.Num())
			{
				Error(LOCTEXT("FunctionCallDataInterfaceMissingRegistration", "Function call signature does not match to a registered DataInterface. Valid DataInterfaces should be wired into a DataInterface function call."), nullptr, nullptr);
				return;
			}
			FNiagaraScriptDataInterfaceCompileInfo& Info = CompilationOutput.ScriptData.DataInterfaceInfo[DataInterfaceOwnerIdx];

			// Double-check to make sure that the signature matches those specified by the data 
			// interface. It could be that the existing node has been removed and the graph
			// needs to be refactored. If that's the case, emit an error.
			UNiagaraDataInterface* CDO = CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(Info.Type.GetClass());
			if (CDO == nullptr)
			{
				// If the cdo wasn't found, the data interface was not passed through a parameter map and so it won't be bound correctly, so add a compile error
				// and invalidate the signature.
				Error(LOCTEXT("DataInterfaceNotFoundInParameterMap", "Data interfaces can not be sampled directly, they must be passed through a parameter map to be bound correctly."), nullptr, nullptr);
				OutSignature.Name = NAME_None;
				return;
			}

			if (OutSignature.bMemberFunction)
			{
				TArray<FNiagaraFunctionSignature> DataInterfaceFunctions;
				CDO->GetFunctionSignatures(DataInterfaceFunctions);

				const int32 FoundMatch = DataInterfaceFunctions.IndexOfByPredicate([&](const FNiagaraFunctionSignature& Sig) -> bool { return Sig.EqualsIgnoringSpecifiers(OutSignature); });
				if (FoundMatch < 0)
				{
					Error(LOCTEXT("FunctionCallDataInterfaceMissing", "Function call signature does not match DataInterface possible signatures?"), nullptr, nullptr);
					return;
				}
				else
				{
					HandleDataInterfaceCall(Info, DataInterfaceFunctions[FoundMatch]);
				}

				if (DataInterfaceFunctions[FoundMatch].bRequiresExecPin)
				{
					OutSignature.Inputs.Insert(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("InExecPin")), 0);
					OutSignature.Outputs.Insert(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("OutExecPin")), 0);
				}
				if (Info.UserPtrIdx != INDEX_NONE && CompilationTarget != ENiagaraSimTarget::GPUComputeSim)
				{
					//This interface requires per instance data via a user ptr so place the index as the first input.
					Inputs.Insert(AddSourceChunk(LexToString(Info.UserPtrIdx), FNiagaraTypeDefinition::GetIntDef(), false), 0);
					OutSignature.Inputs.Insert(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("InstanceData")), 0);
				}
			}

			//Override the owner id of the signature with the actual caller.
			OutSignature.OwnerName = Info.Name;
			Info.RegisteredFunctions.Add(OutSignature);

			Functions.FindOrAdd(OutSignature);
		}

	}
}

void FNiagaraHlslTranslator::GenerateFunctionCall(ENiagaraScriptUsage ScriptUsage, FNiagaraFunctionSignature& FunctionSignature, TArrayView<const int32> Inputs, TArray<int32>& Outputs)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraHLSLTranslator_GenerateFunctionCall);

	bool bEnteredStatScope = false;   
	if (ScriptUsage == ENiagaraScriptUsage::Module)
	{
		bEnteredStatScope = true;
		EnterStatsScope(FNiagaraStatScope(*BuildFunctionSignatureString(FunctionSignature), *(FunctionSignature.GetNameString())));
	}

	TArray<FString> MissingParameters;
	int32 ParamIdx = 0;
	TArray<int32> Params;
	Params.Reserve(Inputs.Num() + Outputs.Num());
	FString DefStr = BuildFunctionSignatureString(FunctionSignature) + TEXT("(");
	for (int32 i = 0; i < FunctionSignature.Inputs.Num(); ++i)
	{
		const FNiagaraVariable& InputVar = FunctionSignature.Inputs[i];
		FNiagaraTypeDefinition Type = InputVar.GetType();
		if (Type.UnderlyingType != 0 && Type.ClassStructOrEnum == nullptr)
		{
			Error(FText::Format(LOCTEXT("InvalidTypeDefError", "Invalid data in niagara type definition, might be due to broken serialization or missing DI implementation! Variable: {0}"), FText::FromName(InputVar.GetName())));
			continue;
		}

		if (!ensure(i < Inputs.Num()))
		{
			Error(FText::Format(LOCTEXT("InvalidInputNum", "Function Input of %d is out of bounds in function signature! Variable: {0}"), 
			FText::AsNumber(i),
			FText::FromName(InputVar.GetName())));
			continue;
		}

		//We don't write class types as real params in the hlsl
		if (!Type.GetClass())
		{
			if (!AddStructToDefinitionSet(Type))
			{
				Error(FText::Format(LOCTEXT("GetConstantFailTypeVar2", "Cannot handle type {0}! Variable: {1}"), Type.GetNameText(), FText::FromName(InputVar.GetName())));
			}

			int32 Input = Inputs[i];
			bool bSkip = false;

			if (InputVar.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				Input = INDEX_NONE;
				bSkip = true;
			}

			if (!bSkip)
			{
				if (ParamIdx != 0)
				{
					DefStr += TEXT(", ");
				}

				Params.Add(Input);
				if (Input == INDEX_NONE)
				{
					MissingParameters.Add(InputVar.GetName().ToString());
				}
				else
				{
					DefStr += FString::Printf(TEXT("{%d}"), ParamIdx);
				}
				++ParamIdx;
			}
		}
	}

	for (int32 i = 0; i < FunctionSignature.Outputs.Num(); ++i)
	{
		FNiagaraVariableBase& OutVar = FunctionSignature.Outputs[i];
		FNiagaraTypeDefinition Type = ConvertToSimulationVariable(OutVar).GetType();

		//We don't write class types as real params in the hlsl
		if (!Type.GetClass())
		{
			if (!AddStructToDefinitionSet(Type))
			{
				Error(FText::Format(LOCTEXT("GetConstantFailTypeVar3", "Cannot handle type {0}! Variable: {1}"), Type.GetNameText(), FText::FromName(FunctionSignature.Outputs[i].GetName())));
			}

			int32 Output = INDEX_NONE;
			int32 ParamOutput = INDEX_NONE;
			bool bSkip = false;
			if (FunctionSignature.Outputs[i].GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				int32 FoundInputParamMapIdx = INDEX_NONE;
				for (int32 j = 0; j < FunctionSignature.Inputs.Num(); j++)
				{
					if (FunctionSignature.Inputs[j].GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
					{
						FoundInputParamMapIdx = j;
						break;
					}
				}
				if (FoundInputParamMapIdx < Inputs.Num() && FoundInputParamMapIdx != INDEX_NONE)
				{
					Output = Inputs[FoundInputParamMapIdx];
				}
				bSkip = true;
			}
			else
			{
				FString OutputStr = FString::Printf(TEXT("%sOutput_%s"), *BuildFunctionSignatureString(FunctionSignature), *OutVar.GetName().ToString());
				Output = AddBodyChunk(GetUniqueSymbolName(*OutputStr), TEXT(""), Type);
				ParamOutput = Output;
			}

			Outputs.Add(Output);

			if (!bSkip)
			{
				if (ParamIdx > 0)
				{
					DefStr += TEXT(", ");
				}

				Params.Add(ParamOutput);
				if (ParamOutput == INDEX_NONE)
				{
					MissingParameters.Add(OutVar.GetName().ToString());
				}
				else
				{
					DefStr += FString::Printf(TEXT("{%d}"), ParamIdx);
				}
				++ParamIdx;
			}
		}
	}

	if (FunctionSignature.bRequiresContext)
	{
		if (ParamIdx > 0)
		{
			DefStr += TEXT(", ");
		}
		DefStr += "Context";
	}

	DefStr += TEXT(")");

	if (MissingParameters.Num() > 0)
	{
		for (FString MissingParam : MissingParameters)
		{
			FText Fmt = LOCTEXT("ErrorCompilingParameterFmt", "Error compiling parameter {0} in function call {1}");
			FText ErrorText = FText::Format(Fmt, FText::FromString(MissingParam), FText::FromString(BuildFunctionSignatureString(FunctionSignature)));
			Error(ErrorText);
		}
		return;
	}

	AddBodyChunk(TEXT(""), DefStr, FNiagaraTypeDefinition::GetFloatDef(), Params);

	if (bEnteredStatScope)
	{
		ExitStatsScope();
	}
}

FString FNiagaraHlslTranslator::GetFunctionSignatureSymbol(const FNiagaraFunctionSignature& Sig)
{
	const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();

	TStringBuilder<1024> SigStr;
	NiagaraHlslTranslatorImpl::AppendFunctionSignatureSymbol(Sig, SigStr);

	TStringBuilder<1024> SanitizedSigStr;
	NiagaraHlslTranslatorImpl::AppendSanitizedSymbolString(SigStr, false, EditorSettings->GetHLSLKeywordReplacementsMap(), SanitizedSigStr);

	return SanitizedSigStr.ToString();
}

void FNiagaraHlslTranslator::AppendFunctionSignatureString(const FNiagaraFunctionSignature& Sig, FStringBuilderBase& StringBuilder) const
{
	TStringBuilder<1024> SigStr;
	NiagaraHlslTranslatorImpl::AppendFunctionSignatureSymbol(Sig, SigStr);

	NiagaraHlslTranslatorImpl::AppendSanitizedSymbolString(SigStr, false, KeywordReplacementMap, StringBuilder);
}

FString FNiagaraHlslTranslator::BuildFunctionSignatureString(const FNiagaraFunctionSignature& Sig) const
{
	TStringBuilder<1024> SanitizedSigStr;
	AppendFunctionSignatureString(Sig, SanitizedSigStr);

	return SanitizedSigStr.ToString();
}


FString FNiagaraHlslTranslator::BuildFunctionHlslPrototypeString(FStringView InVariableName, const FNiagaraFunctionSignature& FunctionSignature) const
{
	TStringBuilder<1024> StringBuilder;
	AppendFunctionHlslPrototypeString(InVariableName, FunctionSignature, StringBuilder);

	return StringBuilder.ToString();
}

void FNiagaraHlslTranslator::AppendFunctionHlslPrototypeString(FStringView InVariableName, const FNiagaraFunctionSignature& FunctionSignature, FStringBuilderBase& StringBuilder) const
{
	NiagaraHlslTranslatorImpl::AppendFunctionHlslPrototypeString(InVariableName, FunctionSignature, KeywordReplacementMap, StringBuilder);
}

FString FNiagaraHlslTranslator::GenerateFunctionHlslPrototype(FStringView InVariableName, const FNiagaraFunctionSignature& FunctionSignature)
{
	const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();
	TStringBuilder<1024> StringBuilder;
	NiagaraHlslTranslatorImpl::AppendFunctionHlslPrototypeString(InVariableName, FunctionSignature, EditorSettings->GetHLSLKeywordReplacementsMap(), StringBuilder);
	return StringBuilder.ToString();
}

FString FNiagaraHlslTranslator::GenerateFunctionParametersHlsl(const FNiagaraFunctionSignature& FunctionSignature)
{
	const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();
	TStringBuilder<1024> StringBuilder;
	NiagaraHlslTranslatorImpl::AppendFunctionHlslParametersString(FunctionSignature, EditorSettings->GetHLSLKeywordReplacementsMap(), StringBuilder);
	return StringBuilder.ToString();
}

FName FNiagaraHlslTranslator::GetDataInterfaceName(FName BaseName, const FString& UniqueEmitterName, bool bIsParameterMapDataInterface)
{
	if (UniqueEmitterName.IsEmpty() == false)
	{
		if (FNiagaraParameterUtilities::IsAliasedEmitterParameter(BaseName.ToString()))
		{
			return FNiagaraParameterUtilities::ResolveEmitterAlias(BaseName, UniqueEmitterName);
		}
		else if(bIsParameterMapDataInterface == false)
		{
			// Don't mangle the parameter map reads for emitter scripts because they are from the system or user parameter stores and
			// they won't bind correctly.
			return *(UniqueEmitterName + TEXT(".") + BaseName.ToString());
		}
	}
	return BaseName;
}

FString FNiagaraHlslTranslator::GetFunctionIncludeStatement(const FNiagaraCustomHlslInclude& Include)
{
	TStringBuilder<128> IncludeStatement;

	if (Include.bIsVirtual)
	{
		if ( !CheckVirtualShaderFilePath(Include.FilePath) )
		{
			Error(FText::FromString("Invalid virtual include " + Include.FilePath));
		}
		else
		{
			IncludeStatement.Appendf(TEXT("#include \"%s\"\n"), *Include.FilePath);
		}
	}
	else if (FString FileContents; FFileHelper::LoadFileToString(FileContents, *Include.FilePath))
	{
		IncludeStatement.Appendf(TEXT("\n// included from %s\n"), *Include.FilePath);
		IncludeStatement.Append(FileContents);
		IncludeStatement.Append("\n");
	}

	return IncludeStatement.ToString();
}

FString FNiagaraHlslTranslator::GetFunctionSignature(const FNiagaraFunctionSignature& Sig)
{
	TStringBuilder<1024> SigStr;
	SigStr << TEXT("void ");
	AppendFunctionSignatureString(Sig, SigStr);

	SigStr << TEXT("(");
	int32 ParamIdx = 0;
	for (int32 i = 0; i < Sig.Inputs.Num(); ++i)
	{
		const FNiagaraVariable& Input = Sig.Inputs[i];
		//We don't write class types as real params in the hlsl
		if (Input.GetType().GetClass() == nullptr)
		{
			if (Input.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				// Skip parameter maps.
			}
			else
			{
				if (ParamIdx > 0)
				{
					SigStr << TEXT(", ");
				}

				FNiagaraVariable SimInput = ConvertToSimulationVariable(Input);
				SigStr << GetStructHlslTypeName(SimInput.GetType()) + TEXT(" In_") + BuildSanitizedSymbolName(Input.GetName(), true);
				++ParamIdx;
			}
		}
	}

	for (int32 i = 0; i < Sig.Outputs.Num(); ++i)
	{
		const FNiagaraVariableBase& Output = Sig.Outputs[i];
		//We don't write class types as real params in the hlsl
		if (Output.GetType().GetClass() == nullptr)
		{
			if (Output.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				// Skip output parameter maps..
			}
			else
			{
				if (ParamIdx > 0)
				{
					SigStr << TEXT(", ");
				}

				FNiagaraVariable SimOutput = ConvertToSimulationVariable(Output);
				SigStr << TEXT("out ") + GetStructHlslTypeName(SimOutput.GetType()) + TEXT(" ") + BuildSanitizedSymbolString(TEXT("Out_") + Output.GetName().ToString());
				++ParamIdx;
			}
		}
	}
	if (Sig.bRequiresContext)
	{
		if (ParamIdx > 0)
		{
			SigStr << TEXT(", ");
		}
		SigStr << TEXT("inout FSimulationContext Context");
	}
	SigStr << TEXT(")");
	return SigStr.ToString();
}

FNiagaraTypeDefinition FNiagaraHlslTranslator::GetChildType(const FNiagaraTypeDefinition& BaseType, const FName& PropertyName)
{
	const UScriptStruct* Struct = BaseType.GetScriptStruct();
	if (Struct != nullptr)
	{
		// Dig through properties to find the matching property native type (if it exists)
		for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = *PropertyIt;
			if (Property->GetName() == PropertyName.ToString())
			{
				if (Property->IsA(FFloatProperty::StaticClass()))
				{
					return FNiagaraTypeDefinition::GetFloatDef();
				}
				else if (Property->IsA(FIntProperty::StaticClass()))
				{
					return FNiagaraTypeDefinition::GetIntDef();
				}
				else if (Property->IsA(FBoolProperty::StaticClass()))
				{
					return FNiagaraTypeDefinition::GetBoolDef();
				}
				else if (Property->IsA(FEnumProperty::StaticClass()))
				{
					const FEnumProperty* EnumProp = CastField<FEnumProperty>(Property);
					return FNiagaraTypeDefinition(EnumProp->GetEnum());
				}
				else if (Property->IsA(FByteProperty::StaticClass()))
				{
					const FByteProperty* ByteProp = CastField<FByteProperty>(Property);
					return FNiagaraTypeDefinition(ByteProp->GetIntPropertyEnum());
				}
				else if (const FStructProperty* StructProp = CastFieldChecked<const FStructProperty>(Property))
				{
					return FNiagaraTypeDefinition(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::Simulation));
				}
			}
		}
	}
	return FNiagaraTypeDefinition();
}

FString FNiagaraHlslTranslator::ComputeMatrixColumnAccess(const FString& Name)
{
	FString Value;
	int32 Column = -1;

	if (Name.Find("X", ESearchCase::IgnoreCase) != -1)
		Column = 0;
	else if (Name.Find("Y", ESearchCase::IgnoreCase) != -1)
		Column = 1;
	else if (Name.Find("Z", ESearchCase::IgnoreCase) != -1)
		Column = 2;
	else if (Name.Find("W", ESearchCase::IgnoreCase) != -1)
		Column = 3;

	if (Column != -1)
	{
		Value.Append("[");
		Value.AppendInt(Column);
		Value.Append("]");
	}
	else
	{
		Error(FText::FromString("Failed to generate type for " + Name + " up to path " + Value));
	}
	return Value;
}

FString FNiagaraHlslTranslator::ComputeMatrixRowAccess(const FString& Name)
{
	FString Value;
	int32 Row = -1;
	if (Name.Find("Row0", ESearchCase::IgnoreCase) != -1)
		Row = 0;
	else if (Name.Find("Row1", ESearchCase::IgnoreCase) != -1)
		Row = 1;
	else if (Name.Find("Row2", ESearchCase::IgnoreCase) != -1)
		Row = 2;
	else if (Name.Find("Row3", ESearchCase::IgnoreCase) != -1)
		Row = 3;

	if (Row != -1)
	{
		Value.Append("[");
		Value.AppendInt(Row);
		Value.Append("]");
	}
	else
	{
		Error(FText::FromString("Failed to generate type for " + Name + " up to path " + Value));
	}
	return Value;
}

FString FNiagaraHlslTranslator::NamePathToString(const FString& Prefix, const FNiagaraTypeDefinition& RootType, const TArray<FName>& NamePath)
{
	// We need to deal with matrix parameters differently than any other type by using array syntax.
	// As we recurse down the tree, we stay away of when we're dealing with a matrix and adjust 
	// accordingly.
	FString Value = Prefix;
	FNiagaraTypeDefinition CurrentType = RootType;
	bool bParentWasMatrix = (RootType == FNiagaraTypeDefinition::GetMatrix4Def());
	for (int32 i = 0; i < NamePath.Num(); i++)
	{
		FString Name = NamePath[i].ToString();
		CurrentType = GetChildType(CurrentType, NamePath[i]);
		// Found a matrix... brackets from here on out.
		if (CurrentType == FNiagaraTypeDefinition::GetMatrix4Def())
		{
			bParentWasMatrix = true;
			Value.Append("." + Name);
		}
		// Parent was a matrix, determine row..
		else if (bParentWasMatrix && CurrentType == FNiagaraTypeDefinition::GetVec4Def())
		{
			Value.Append(ComputeMatrixRowAccess(Name));
		}
		// Parent was a matrix, determine column...
		else if (bParentWasMatrix && CurrentType == FNiagaraTypeDefinition::GetFloatDef())
		{
			Value.Append(ComputeMatrixColumnAccess(Name));
		}
		// Handle all other valid types by just using "." 
		else if (CurrentType.IsValid())
		{
			Value.Append("." + Name);
		}
		else
		{
			Error(FText::FromString("Failed to generate type for " + Name + " up to path " + Value));
		}
	}
	return Value;
}

FString FNiagaraHlslTranslator::GenerateAssignment(const FNiagaraTypeDefinition& SrcPinType, const TArray<FName>& ConditionedSourcePath, const FNiagaraTypeDefinition& DestPinType, const TArray<FName>& ConditionedDestinationPath)
{
	FString SourceDefinition = NamePathToString("{1}", SrcPinType, ConditionedSourcePath);
	FString DestinationDefinition = NamePathToString("{0}", DestPinType, ConditionedDestinationPath);

	return DestinationDefinition + " = " + SourceDefinition;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Convert(const FConvertNode* Convert, TArrayView<const int32> Inputs, TArray<int32>& Outputs)
{
	if (ValidateTypePins(Convert) == false)
	{
		return;
	}

	TArray<const FInputPin*> InputPins = GraphBridge::GetInputPins(Convert);
	TArray<const FOutputPin*> OutputPins = GraphBridge::GetOutputPins(Convert);

	// Add input struct definitions if necessary.
	for (const FInputPin* InputPin : InputPins)
	{
		if (InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType ||
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType || 
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum || 
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum)
		{
			FNiagaraTypeDefinition Type = GraphBridge::GetPinType(InputPin, ENiagaraStructConversion::Simulation);
			if (!AddStructToDefinitionSet(Type))
			{
				Error(FText::Format(LOCTEXT("ConvertTypeError_InvalidInput", "Cannot handle input pin type {0}! Pin: {1}"), Type.GetNameText(), GraphBridge::GetPinFriendlyName(InputPin)), Convert, InputPin);
			}
		}
	}

	// Generate outputs.
	Outputs.Reserve(Outputs.Num() + OutputPins.Num() + 1);
	for (const FOutputPin* OutputPin : OutputPins)
	{
		if (OutputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType ||
			OutputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType ||
			OutputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum ||
			OutputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum)
		{
			FNiagaraTypeDefinition Type = GraphBridge::GetPinType(OutputPin, ENiagaraStructConversion::Simulation);
			if (!AddStructToDefinitionSet(Type))
			{
				Error(FText::Format(LOCTEXT("ConvertTypeError_InvalidOutput", "Cannot handle output pin type {0}! Pin: {1}"), Type.GetNameText(), GraphBridge::GetPinFriendlyName(OutputPin)), Convert, OutputPin);
			}
			
			// The convert node should already have issued errors if the connections aren't complete.
			// So we won't do anything else here.
			
			int32 OutChunk = AddBodyChunk(GetUniqueSymbolName(OutputPin->PinName), TEXT(""), Type);

			Outputs.Add(OutChunk);
		}
	}

	// Set output values based on connections.
	for (const FConvertConnection& Connection : GraphBridge::GetConvertConnections(Convert))
	{
		int32 SourceIndex = GraphBridge::GetPinIndexById(InputPins, Connection.SourcePinId);
		int32 DestinationIndex = GraphBridge::GetPinIndexById(OutputPins, Connection.DestinationPinId);
		if (Inputs.IsValidIndex(SourceIndex) && Outputs.IsValidIndex(DestinationIndex))
		{
			FNiagaraTypeDefinition SrcPinType = GraphBridge::GetPinType(InputPins[SourceIndex], ENiagaraStructConversion::Simulation);
			if (!AddStructToDefinitionSet(SrcPinType))
			{
				Error(FText::Format(LOCTEXT("ConvertTypeError_InvalidSubpinInput", "Cannot handle input subpin type {0}! Subpin: {1}"), SrcPinType.GetNameText(), GraphBridge::GetPinDisplayName(InputPins[SourceIndex])));
			}
			TArray<FName> ConditionedSourcePath = ConditionPropertyPath(SrcPinType, Connection.SourcePath);

			FNiagaraTypeDefinition DestPinType = GraphBridge::GetPinType(OutputPins[DestinationIndex], ENiagaraStructConversion::Simulation);
			if (!AddStructToDefinitionSet(DestPinType))
			{
				Error(FText::Format(LOCTEXT("ConvertTypeError_InvalidSubpinOutput", "Cannot handle output subpin type type {0}! Subpin: {1}"), DestPinType.GetNameText(), GraphBridge::GetPinDisplayName(OutputPins[SourceIndex])));
			}
			TArray<FName> ConditionedDestinationPath = ConditionPropertyPath(DestPinType, Connection.DestinationPath);

			FString ConvertDefinition = GenerateAssignment(SrcPinType, ConditionedSourcePath, DestPinType, ConditionedDestinationPath);

			TArray<int32> SourceChunks;
			SourceChunks.Add(Outputs[DestinationIndex]);
			SourceChunks.Add(Inputs[SourceIndex]);
			AddBodyChunk(TEXT(""), ConvertDefinition, FNiagaraTypeDefinition::GetFloatDef(), SourceChunks);
		}
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::If(const FIfNode* IfNode, const TArray<FNiagaraVariable>& Vars, int32 Condition, TArray<int32>& PathA, TArray<int32>& PathB, TArray<int32>& Outputs)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_If);

	int32 NumVars = Vars.Num();
	check(PathA.Num() == NumVars);
	check(PathB.Num() == NumVars);

	TArray<FString> OutSymbols;
	OutSymbols.Reserve(Vars.Num());
	int32 PinIdx = 1;
	for (const FNiagaraVariable& Var : Vars)
	{
		FNiagaraTypeDefinition Type = GraphBridge::GetPinType(GraphBridge::GetInputPin(IfNode, PinIdx), ENiagaraStructConversion::Simulation);
		if (!AddStructToDefinitionSet(Type))
		{
			FText OutErrorMessage = FText::Format(LOCTEXT("If_UnknownNumeric", "Variable in If node uses invalid type. Var: {0} Type: {1}"),
				FText::FromName(Var.GetName()), Type.GetNameText());

			Error(OutErrorMessage, IfNode, nullptr);
		}
		OutSymbols.Add(GetUniqueSymbolName(*(Var.GetName().ToString() + TEXT("_IfResult"))));
		Outputs.Add(AddBodyChunk(OutSymbols.Last(), TEXT(""), Type, true));

		++PinIdx;
	}
	AddBodyChunk(TEXT(""), TEXT("if({0})\n\t{"), FNiagaraTypeDefinition::GetFloatDef(), Condition, false, false);
	for (int32 i = 0; i < NumVars; ++i)
	{
		FNiagaraTypeDefinition OutChunkType = CodeChunks[Outputs[i]].Type;
		FNiagaraCodeChunk& BranchChunk = CodeChunks[AddBodyChunk(OutSymbols[i], TEXT("{0}"), OutChunkType, false)];
		BranchChunk.AddSourceChunk(PathA[i]);
	}
	AddBodyChunk(TEXT(""), TEXT("}\n\telse\n\t{"), FNiagaraTypeDefinition::GetFloatDef(), false, false);
	for (int32 i = 0; i < NumVars; ++i)
	{
		FNiagaraTypeDefinition OutChunkType = CodeChunks[Outputs[i]].Type;
		FNiagaraCodeChunk& BranchChunk = CodeChunks[AddBodyChunk(OutSymbols[i], TEXT("{0}"), OutChunkType, false)];
		BranchChunk.AddSourceChunk(PathB[i]);
	}
	AddBodyChunk(TEXT(""), TEXT("}"), FNiagaraTypeDefinition::GetFloatDef(), false, false);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Select(const FSelectNode* SelectNode, int32 Selector, const TArray<FNiagaraVariable>& OutputVariables, TMap<int32, TArray<int32>>& Options, TArray<int32>& Outputs)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_Select);
	
	if (Options.Num() == 0)
	{
		FText OutErrorMessage = LOCTEXT("NoOptions", "Select node has no input pins. Please select a selector type.");
		Error(OutErrorMessage, SelectNode, nullptr);
	}

	for(const FNiagaraVariable& Variable : OutputVariables)
	{
		if (!AddStructToDefinitionSet(Variable.GetType()))
		{
			FText OutErrorMessage = FText::Format(LOCTEXT("Select_UnknownNumeric", "Output type in Select node uses invalid type. Type: {0}"),
				Variable.GetType().GetNameText());

			Error(OutErrorMessage, SelectNode, GraphBridge::GetSelectOutputPin(SelectNode, Variable));
		}
	}

	FString SymbolNameSuffix = GetUniqueSymbolName(TEXT("_SelectResult"));
	TArray<FString> SymbolNames;
	if (Options.Num() > 0)
	{
		for (const FNiagaraVariable& Variable : OutputVariables)
		{
			FNiagaraTypeDefinition OutputType = FNiagaraTypeHelper::GetSWCType(Variable.GetType());
			const FNiagaraVariable DefaultVar(OutputType, Variable.GetName());
			const int32 DefaultConstant = GetConstant(DefaultVar);

			FString SymbolName = Variable.GetName().ToString() + SymbolNameSuffix;
			SymbolNames.Add(SymbolName);
			
			const int32 SymbolIndex = AddBodyChunk(SymbolName, TEXT("{0}"), OutputType, true);
			FNiagaraCodeChunk& OutputSymbolChunk = CodeChunks[SymbolIndex];
			OutputSymbolChunk.AddSourceChunk(DefaultConstant);
			
			Outputs.Add(SymbolIndex);
		}
	}

	TArray<int32> SelectorValues;
	Options.GenerateKeyArray(SelectorValues);

	const bool bIsBoolSelector = FNiagaraTypeDefinition::GetBoolDef().IsSameBaseDefinition(SelectNode->SelectorPinType);

	for (int32 SelectorValueIndex = 0; SelectorValueIndex < SelectorValues.Num(); SelectorValueIndex++)
	{
		FString Definition;
		// We default to the first value from the select operation to ensure we always set to a valid value
		// Failure to do this can result in bad / incorrect values being used in the VVM
		if (SelectorValueIndex > 0)
		{
			if (bIsBoolSelector)
			{
				Definition = SelectorValues[SelectorValueIndex] == 0 ? TEXT("if({0} == 0)\n\t{ ") : TEXT("if({0} != 0)\n\t{ ");
			}
			else
			{
				Definition = FString::Printf(TEXT("if({0} == %d)\n\t{ "), SelectorValues[SelectorValueIndex]);
			}
			TArray<int32> SourceChunks = { Selector };
			AddBodyChunk(TEXT(""), Definition, FNiagaraTypeDefinition::GetFloatDef(), SourceChunks, false, false);
		}

		int32 NaturalIndex = 0;
		for(int32 CompiledPinCodeChunk : Options[SelectorValues[SelectorValueIndex]])
		{
			FNiagaraCodeChunk& BranchChunk = CodeChunks[AddBodyChunk(SymbolNames[NaturalIndex], TEXT("{0}"), FNiagaraTypeHelper::GetSWCType(OutputVariables[NaturalIndex].GetType()), false)];
			BranchChunk.AddSourceChunk(CompiledPinCodeChunk);
			NaturalIndex++;
		}
		
		if (SelectorValueIndex > 0)
		{
			AddBodyChunk(TEXT(""), TEXT("}"), FNiagaraTypeDefinition::GetFloatDef(), false, false);
		}
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::FindConstantValue(int32 InputCompileResult, const FNiagaraTypeDefinition& TypeDef, FString& Value, FNiagaraVariable& Variable)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	{
		bool bSearch = true;
		FString SourceName;
		while (bSearch)
		{
			if (InputCompileResult != INDEX_NONE)
			{
				if (CodeChunks.IsValidIndex(InputCompileResult))
				{
					if (CodeChunks[InputCompileResult].Mode >= ENiagaraCodeChunkMode::Body && CodeChunks[InputCompileResult].Mode < ENiagaraCodeChunkMode::SimulationStageBodyMax)
					{
						if (CodeChunks[InputCompileResult].SourceChunks.Num() == 1 && CodeChunks[InputCompileResult].Definition == TEXT("{0}")) // Handle intermediate assignment
						{
							InputCompileResult = CodeChunks[InputCompileResult].SourceChunks[0]; // Follow the linkage
						}
						else if (CodeChunks[InputCompileResult].Original.IsDataAllocated()) // Handle constants
						{
							Variable.AllocateData();
							CodeChunks[InputCompileResult].Original.CopyTo(Variable.GetData());
							bSearch = false;
						}
						else if (CodeChunks[InputCompileResult].Original.IsValid()) // Handle default assignments
						{
							Value = CodeChunks[InputCompileResult].Original.GetName().ToString();
							bSearch = false;
						}
						else // Handle setting to defaults as we didn't find a match.
						{
							TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(TypeDef);
							if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanHandlePinDefaults() && CodeChunks[InputCompileResult].Definition.IsEmpty() == false)
							{
								// Note that this might fail due to string not being properly formatted for the type. If so, we just take the definition string altogether.
								bool bHasValue = TypeEditorUtilities->SetValueFromPinDefaultString(CodeChunks[InputCompileResult].Definition, Variable);
								if (bHasValue == false)
								{
									Value = CodeChunks[InputCompileResult].Definition;
								}
							}
							else
							{
								Value = CodeChunks[InputCompileResult].Definition;
							}
							bSearch = false;
						}
					}
					else  if (CodeChunks[InputCompileResult].Mode == ENiagaraCodeChunkMode::Uniform)
					{
						//UE_LOGF(LogNiagaraEditor, Log, "CompilerTag %ls looking for uniform with symbol %ls", *Variable.GetName().ToString(), *CodeChunks[InputCompileResult].SymbolName);
						FString VarSymbol;
						for (const FNiagaraVariable& UniformVar : CompilationOutput.ScriptData.Parameters.Parameters)
						{
							VarSymbol.Reset();
							VarSymbol = BuildSanitizedSymbolName(UniformVar.GetName(), true);
							if (VarSymbol == CodeChunks[InputCompileResult].SymbolName)
							{
								//UE_LOGF(LogNiagaraEditor, Log, "Match Found: %ls == %ls", *VarSymbol, *UniformVar.GetName().ToString());
								Value = UniformVar.GetName().ToString();
								break;
							}
							else
							{
								//UE_LOGF(LogNiagaraEditor, Log, "No Match: %ls != %ls", *VarSymbol, *UniformVar.GetName().ToString());
							}
						}
						check(Value.IsEmpty() == false);//Somethings wrong if we're in a uniform chunk and we can't find it's matching variable again.
						bSearch = false;
					}
					else  if (CodeChunks[InputCompileResult].Mode == ENiagaraCodeChunkMode::Source)
					{
						if (SourceName.Len() == 0)
							SourceName = CodeChunks[InputCompileResult].SymbolName;
						else
							bSearch = false; // Don't keep searching as we might be going outside a function call boundary and lose track. Just allow one hop.

						if (CodeChunks[InputCompileResult].Original.GetName().IsNone() == false)
						{
							Value = CodeChunks[InputCompileResult].Original.GetName().ToString();
							bSearch = false;
						}
						else if (CodeChunks[InputCompileResult].SourceChunks.Num() == 0) // Search through parent chunks for a name match
						{
							bool bFoundAlternate = false;

							// First see if this is output from a function call variable, if so we need to check to see when it was last written and what the chunk was that happened in.
							int32 ParamMapHistoryIdx = ActiveStageIdx; // Should match the order in Translation stages! 
							if (ParamMapHistories.IsValidIndex(ParamMapHistoryIdx))
							{
								TArray<FString> SplitName;
								CodeChunks[InputCompileResult].SymbolName.ParseIntoArray(SplitName, TEXT("."));
								FString NewName;

								if (SplitName.Num() > 2 && SplitName[0] == TEXT("Context") &&  SplitName[1] == TranslationStages[ParamMapHistoryIdx].PassNamespace)
								{
									for (int32 SplitIdx = 2; SplitIdx < SplitName.Num(); SplitIdx++)
									{
										if (NewName.Len() > 0)
											NewName += TEXT(".") + SplitName[SplitIdx];
										else
											NewName += SplitName[SplitIdx];
									}
								}

								if (NewName.Len() > 0)
								{
									int32 VarIdx = ParamMapHistories[ParamMapHistoryIdx].FindVariableByName(*NewName);
									if (VarIdx != INDEX_NONE && VarIdx < ParamMapSetVariablesToChunks[ParamMapHistoryIdx].Num())
									{
										int32 PossibleIndex = ParamMapSetVariablesToChunks[ParamMapHistoryIdx][VarIdx];
										if (PossibleIndex < InputCompileResult - 1)
										{
											InputCompileResult = PossibleIndex;
											bFoundAlternate = true;
										}
									}
								}
							}

							if (!bFoundAlternate)
							{
								for (int32 i = InputCompileResult - 1; i >= 0 && i < InputCompileResult; i--)
								{
									if (CodeChunks[i].SymbolName == SourceName)
									{
										InputCompileResult = i;
										break;
									}
									if (i == 0)
									{
										bSearch = false;
									}
								}
							}
						}
					}
					else
					{
						break;
					}
				}
				else
				{
					break;
				}
			}
			else
			{
				break;
			}
		}
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::WriteCompilerTag(int32 InputCompileResult, const FPin* Pin, bool bEditorOnly, bool bEmitMessageOnFailure, FNiagaraCompileEventSeverity FailureSeverity, const FString& Prefix)
{
	FString Value;
	FNiagaraTypeDefinition TypeDef = GraphBridge::GetPinType(Pin, ENiagaraStructConversion::UserFacing);
	FNiagaraVariable Variable(TypeDef, Prefix.Len() ? *(Prefix + TEXT(".") + Pin->PinName.ToString()) : Pin->PinName);


	//If we're in an emitter script then the tag needs to be made per emitter with EmitterName.Tag
	bool bIsSystemOrEmitterScript =
		UNiagaraScript::IsEmitterSpawnScript(CompileOptions.TargetUsage) || UNiagaraScript::IsSystemSpawnScript(CompileOptions.TargetUsage) ||
		UNiagaraScript::IsEmitterUpdateScript(CompileOptions.TargetUsage) || UNiagaraScript::IsSystemUpdateScript(CompileOptions.TargetUsage);

	if (bIsSystemOrEmitterScript)
	{
		if (const FString* EmitterAliasStr = ActiveHistoryForFunctionCalls.GetEmitterAlias())
		{
			Variable.SetName(*(*EmitterAliasStr + TEXT(".") + Variable.GetName().ToString()));
		}
	}

	FindConstantValue(InputCompileResult, TypeDef, Value, Variable);

	if (Value.Len() == 0 && Variable.IsDataAllocated() == false && bEmitMessageOnFailure)
	{
		Message(FailureSeverity, FText::FromString(TEXT("Output Compile Tag must be connected to a constant or a uniform variable to work! Ignoring the compile tag.")), GraphBridge::GetOwningNode(Pin), Pin);
	}
	else
	{
		TArray<FNiagaraCompilerTag>& CompileTags = bEditorOnly ? TranslateResults.CompileTagsEditorOnly : TranslateResults.CompileTags;

		// Always use the latest output value for the tag.
		if (FNiagaraCompilerTag* Tag = FNiagaraCompilerTag::FindTag(CompileTags, Variable))
		{
			Tag->StringValue = Value;
			Tag->Variable = Variable;
		}
		else
		{
			CompileTags.Emplace(Variable, Value);
		}
	}
}

template<typename GraphBridge>
int32 TNiagaraHlslTranslator<GraphBridge>::CompileInputPin(const FInputPin* Pin)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_CompilePin);

	check(Pin);
	if (!ensure(Pin->Direction == EGPD_Input))
	{
		return INDEX_NONE;
	}

	if (const FOutputPin* LinkedPin = GraphBridge::GetLinkedOutputPin(Pin))
	{
		FText ConnectionMessage;
		if (!GraphBridge::CanCreateConnection(LinkedPin, Pin, ConnectionMessage))
		{
			FText OutErrorMessage = FText::Format(LOCTEXT("InputConnectionDisallowed", "Input connection is not allowed! Reason: {0}"),
				ConnectionMessage);

			Error(OutErrorMessage, GraphBridge::GetOwningNode(Pin), Pin);
		}
		return CompileOutputPin(LinkedPin);
	}
	else if (!Pin->bDefaultValueIsIgnored && (Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType || Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType))
	{
		FNiagaraTypeDefinition TypeDef = GraphBridge::GetPinType(Pin, ENiagaraStructConversion::UserFacing);
		if (TypeDef == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			Error(FText::FromString(TEXT("Parameter Maps must be created via an Input Node, not the default value of a pin! Please connect to a valid input Parameter Map.")), GraphBridge::GetOwningNode(Pin), Pin);
			return INDEX_NONE;
		}

		//No connections to this input so add the default as a const expression.			
		FNiagaraVariable PinVar = GraphBridge::GetPinVariable(Pin, true, ENiagaraStructConversion::Simulation);
		return GetConstant(PinVar);
	}
	else if (!Pin->bDefaultValueIsIgnored && (Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum || Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum))
	{
		//No connections to this input so add the default as a const expression.			
		FNiagaraVariable PinVar = GraphBridge::GetPinVariable(Pin, true, ENiagaraStructConversion::Simulation);
		return GetConstant(PinVar);
	}

	return INDEX_NONE;
}

template<typename GraphBridge>
int32 TNiagaraHlslTranslator<GraphBridge>::CompileOutputPin(const FPin* InPin)
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_HlslTranslator_CompileOutputPin);

	if (InPin == nullptr)
	{
		Error(LOCTEXT("InPinNullptr", "Input pin is nullptr!"), nullptr, nullptr);
		return INDEX_NONE;
	}

	UpdateStaticSwitchConstants(InPin);

	// The incoming pin to compile may be pointing to a reroute node. If so, we just jump over it
	// to where it really came from.
	const FPin* Pin = InPin;
	if (Pin == nullptr || Pin->Direction != EGPD_Output)
	{
		Error(LOCTEXT("TraceOutputPinFailed", "Failed to trace pin to an output!"), GraphBridge::GetOwningNode(InPin), InPin);
		return INDEX_NONE;
	}

	const FNode* Node = GraphBridge::GetOwningNode(Pin);

	// if the node is disabled (which commonly happens when a module is disabled in the stack), we skip it and follow the execution pin to the next node
	if (!GraphBridge::NodeIsEnabled(Node))
	{
		TArray<const FOutputPin*> Outputs = GraphBridge::GetOutputPins(Node);
		const FPin* OutputPin = Outputs.IsEmpty() ? nullptr : Outputs[0];
		if ((OutputPin == Pin) && GraphBridge::GetPinType(Pin, ENiagaraStructConversion::UserFacing) == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			TArray<const FInputPin*> InputPins = GraphBridge::GetInputPins(Node);
			for (int32 PinIndex = 0; PinIndex < InputPins.Num(); PinIndex++)
			{
				if (const FInputPin* ExecPin = InputPins[PinIndex])
				{
					if (const FOutputPin* ConnectedPin = GraphBridge::GetLinkedOutputPin(ExecPin))
					{
						return CompileOutputPin(ConnectedPin);
					}
				}
			}
		}
		Error(LOCTEXT("TraceDisabledPinFailed", "Failed to trace output pin of disabled node to a valid input!"), Node, Pin);
		return INDEX_NONE;
	}

	int32 Ret = INDEX_NONE;
	int32* Chunk = PinToCodeChunks.Last().Find(Pin);
	if (Chunk)
	{
		Ret = *Chunk; //We've compiled this pin before. Return it's chunk.
	}
	else
	{
		//Otherwise we need to compile the node to get its output pins.
		if (ValidateTypePins(Node))
		{
			TArray<int32> Outputs;
			TArray<const FOutputPin*> OutputPins = GraphBridge::GetOutputPins(Node);
			
			Node->Compile(this, Outputs);
			// this requires the nodes to only compile their valid output pins - no orphaned pins
			if (OutputPins.Num() == Outputs.Num())
			{
				for (int32 i = 0; i < Outputs.Num(); ++i)
				{
					//Cache off the pin.
					//Can we allow the caching of local defaults in numerous function calls?
					PinToCodeChunks.Last().Add(OutputPins[i], Outputs[i]);

					if (Outputs[i] != INDEX_NONE)
					{
						//Grab the expression for the pin we're currently interested in. Otherwise we'd have to search the map for it.
						Ret = OutputPins[i] == Pin ? Outputs[i] : Ret;
					}
				}
			}
			else
			{
				Error(LOCTEXT("IncorrectNumOutputsError", "Incorrect number of outputs. Can possibly be fixed with a graph refresh."), Node, nullptr);
			}
		}
	}

	return Ret;
}

template<typename GraphBridge>
FString TNiagaraHlslTranslator<GraphBridge>::NodePinToMessage(FText MessageText, const FNode* Node, const FPin* Pin)
{
	FString NodePinStr = TEXT("");
	FString NodePinPrefix = TEXT(" - ");
	FString NodePinSuffix = TEXT("");
	if (Node)
	{
		FString NodeTitle = GraphBridge::GetNodeTitle(Node);
		if (NodeTitle.Len() > 0)
		{
			NodePinStr += TEXT("Node: ") + NodeTitle;
			NodePinSuffix = TEXT(" - ");
		}
		else
		{
			FString NodeName = GraphBridge::GetNodeName(Node);
			if (NodeName.Len() > 0)
			{
				NodePinStr += TEXT("Node: ") + NodeName;
				NodePinSuffix = TEXT(" - ");
			}
		}
	}
	if (Pin)
	{
		const FText FriendlyPinName = GraphBridge::GetPinFriendlyName(Pin);
		NodePinStr += TEXT(" Pin: ") + (FriendlyPinName.IsEmpty() ? Pin->PinName.ToString() : FriendlyPinName.ToString());
		NodePinSuffix = TEXT(" - ");
	}

	FString MessageString = FString::Printf(TEXT("%s%s%s%s"), *MessageText.ToString(), *NodePinPrefix, *NodePinStr, *NodePinSuffix);
	return MessageString;
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Message(FNiagaraCompileEventSeverity Severity, FText MessageText, const FNode* InNode, const FPin* Pin, FStringView ShortDescription)
{
	const FNode* CurContextNode = ActiveHistoryForFunctionCalls.GetCallingContext();
	const FNode* TargetNode = InNode ? InNode : CurContextNode;
 
	FString MessageString = NodePinToMessage(MessageText, TargetNode, Pin);
	TranslateResults.CompileEvents.Add(FNiagaraCompileEvent(Severity, MessageString, FString(ShortDescription), TargetNode ? TargetNode->NodeGuid : FGuid(), Pin ? Pin->PersistentGuid : FGuid(), GetCallstackGuids()));
 
	if (Severity == FNiagaraCompileEventSeverity::Error)
	{
		TranslateResults.NumErrors++;
	}
	else if (Severity == FNiagaraCompileEventSeverity::Warning)
	{
		TranslateResults.NumWarnings++;
	}
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Message(FNiagaraCompileEventSeverity Severity, FText MessageText, FStringView ShortDescription)
{
	Message(Severity, MessageText, nullptr, nullptr, ShortDescription);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Error(FText ErrorText, const FNode* InNode, const FPin* Pin, FStringView ShortDescription)
{
	Message(FNiagaraCompileEventSeverity::Error, ErrorText, InNode, Pin, ShortDescription);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Error(FText ErrorText, FStringView ShortDescription)
{
	Error(ErrorText, nullptr, nullptr, ShortDescription);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Warning(FText WarningText, const FNode* InNode, const FPin* Pin, FStringView ShortDescription)
{
	Message(FNiagaraCompileEventSeverity::Warning, WarningText, InNode, Pin, ShortDescription);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::Warning(FText WarningText, FStringView ShortDescription)
{
	Warning(WarningText, nullptr, nullptr, ShortDescription);
}

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::RegisterCompileDependency(const FNiagaraVariableBase& InVar, FText MessageText, const FNode* Node, const FPin* Pin, bool bEmitAsLinker, int32 ParamMapHistoryIdx)
{
	if (FNiagaraCVarUtilities::GetShouldEmitMessagesForFailIfNotSet() == false)
	{
		return;
	}

	if (InVar.GetType().IsDataInterface() || InVar.GetType().IsUObject() || InVar.IsInNameSpace(FNiagaraConstants::UserNamespaceString) || InVar.IsInNameSpace(FNiagaraConstants::EngineNamespaceString) || InVar.IsInNameSpace(FNiagaraConstants::ParameterCollectionNamespaceString))
	{
		return;
	}

	if (FNiagaraConstants::IsNiagaraConstant(InVar) || InVar.GetName() == TEXT("Emitter.InterpSpawnStartDt") || InVar.GetName() == TEXT("Emitter.SpawnInterval"))
	{
		return;
	}

	if (bEmitAsLinker)																					
	{			
		bool bVarFromCustomIterationNamespaceOverride = ParamMapHistories[ParamMapHistoryIdx].IsVariableFromCustomIterationNamespaceOverride(InVar);
		const FNode* CurContextNode = ActiveHistoryForFunctionCalls.GetCallingContext();
		const FNode* TargetNode = Node ? Node : CurContextNode;

		FString MessageString = NodePinToMessage(MessageText, TargetNode, Pin);
		TranslateResults.CompileDependencies.AddUnique(FNiagaraCompileDependency(InVar, MessageString, TargetNode ? TargetNode->NodeGuid : FGuid(), Pin ? Pin->PersistentGuid : FGuid(), GetCallstackGuids(), bVarFromCustomIterationNamespaceOverride));
	}
	else
	{
		const bool bDismissable = false;
		const FString ShortDescription = FString();
		Message(FNiagaraCVarUtilities::GetCompileEventSeverityForFailIfNotSet(), MessageText, Node, Pin, ShortDescription);
	}
}

bool FNiagaraHlslTranslator::GetFunctionParameter(const FNiagaraVariable& Parameter, int32& OutParam)const
{
	// Assume that it wasn't bound by default.
	OutParam = INDEX_NONE;
	if (const FFunctionContext* FunctionContext = FunctionCtx())
	{
		int32 ParamIdx = FunctionContext->Signature.Inputs.IndexOfByPredicate([&](const FNiagaraVariable& InVar) { return InVar.IsEquivalent(Parameter); });
		if (ParamIdx != INDEX_NONE)
		{
			OutParam = FunctionContext->Inputs[ParamIdx];
		}
		return true;
	}
	return false;
}

int32 FNiagaraHlslTranslator::GetUniqueCallerID()
{
	if (!TranslationStages[ActiveStageIdx].bCallIDInitialized)
	{
		// The Call ID is changed every time a compiled node requests it, but we want to randomize it a bit from the start.
		// Otherwise compilation units all start from the same ID (resulting in the same chain of generated randoms).
		TranslationStages[ActiveStageIdx].CurrentCallID = (int32)(GetTypeHash(GetEmitterUniqueName()) + (uint8)TranslationStages[ActiveStageIdx].ScriptUsage * 1024);
		TranslationStages[ActiveStageIdx].bCallIDInitialized = true;
	}
	return TranslationStages[ActiveStageIdx].CurrentCallID++;
}

bool FNiagaraHlslTranslator::CanReadAttributes()const
{
	if (UNiagaraScript::IsParticleUpdateScript(TranslationStages[ActiveStageIdx].ScriptUsage))
	{
		return true;
	}
	return false;
}

template<typename GraphBridge>
ENiagaraScriptUsage TNiagaraHlslTranslator<GraphBridge>::GetCurrentUsage() const
{
	if (UNiagaraScript::IsParticleScript(CompileOptions.TargetUsage))
	{
		return CompileOptions.TargetUsage;
	}
	else if (UNiagaraScript::IsSystemSpawnScript(CompileOptions.TargetUsage) || UNiagaraScript::IsSystemUpdateScript(CompileOptions.TargetUsage))
	{
		if (ActiveHistoryForFunctionCalls.ContextContains(ENiagaraScriptUsage::EmitterSpawnScript))
		{
			return ENiagaraScriptUsage::EmitterSpawnScript;
		}
		else if (ActiveHistoryForFunctionCalls.ContextContains(ENiagaraScriptUsage::EmitterUpdateScript))
		{
			return ENiagaraScriptUsage::EmitterUpdateScript;
		}
		return CompileOptions.TargetUsage;
	}
	else if (UNiagaraScript::IsStandaloneScript(CompileOptions.TargetUsage))
	{
		// Since we never use the results of a standalone script directly, just choose one by default.
		return ENiagaraScriptUsage::ParticleSpawnScript;
	}
	else
	{
		check(false);
		return ENiagaraScriptUsage::ParticleSpawnScript;
	}
}

ENiagaraScriptUsage FNiagaraHlslTranslator::GetTargetUsage() const
{
	if (CompileOptions.TargetUsage == ENiagaraScriptUsage::ParticleGPUComputeScript) // Act as if building spawn script.
	{
		return ENiagaraScriptUsage::ParticleSpawnScript;
	}
	if (UNiagaraScript::IsInterpolatedParticleSpawnScript(CompileOptions.TargetUsage))
	{
		return ENiagaraScriptUsage::ParticleSpawnScript;
	}
	return CompileOptions.TargetUsage;
}

FGuid FNiagaraHlslTranslator::GetTargetUsageId() const
{
	return CompileOptions.TargetUsageId;
}
//////////////////////////////////////////////////////////////////////////

template<typename GraphBridge>
void TNiagaraHlslTranslator<GraphBridge>::TrimAttributes(const FNiagaraCompileOptions& InCompileOptions, TArray<FNiagaraVariable>& Attributes)
{
	using FNiagaraAttributeTrimming = FNiagaraAttributeTrimmerHelper<GraphBridge>;

	if (!UNiagaraScript::IsParticleScript(InCompileOptions.TargetUsage))
	{
		return;
	}

	const bool SafeTrimAttributesEnabled = InCompileOptions.AdditionalDefines.Contains(TEXT("TrimAttributesSafe"));
	const bool AggressiveTrimAttributesEnabled = InCompileOptions.AdditionalDefines.Contains(TEXT("TrimAttributes"));

	if (SafeTrimAttributesEnabled || AggressiveTrimAttributesEnabled)
	{
		// validate that the attributes have unique sanitized names
		{
			bool HasOverlappingNames = false;

			TMap<FString, FNiagaraVariableBase> SanitizedNames;
			for (const FNiagaraVariableBase& Attribute : Attributes)
			{
				const FString SanitizedName = BuildSanitizedSymbolName(Attribute.GetName());
				if (const FNiagaraVariableBase* ExistingVariable = SanitizedNames.Find(SanitizedName))
				{
					HasOverlappingNames = true;
				}
				else
				{
					SanitizedNames.Add(SanitizedName, Attribute);
				}
			}

			// the trimming algorithm doesn't work when names are overlapping, so just early out of the function
			if (HasOverlappingNames)
			{
				return;
			}
		}

		const bool bRequiresPersistentIDs = InCompileOptions.AdditionalDefines.Contains(TEXT("RequiresPersistentIDs"));

		// we want to use the ParamMapHistories of both the particle update and spawn scripts because they need to
		// agree to define a unified attribute set
		TArray<const FParamMapHistory*, TInlineAllocator<2>> LocalParamHistories;
		for (const FParamMapHistory& History : OtherOutputParamMapHistories)
		{
			if (UNiagaraScript::IsParticleScript(History.OriginatingScriptUsage))
			{
				// for now we'll be disabling attribute trimming if a family of particle scripts contain generation of
				// additional dataset writes (events) as we don't have access to the connectivity of it's variables as
				// we do for the rest of the script
				if (History.AdditionalDataSetWrites.Num())
				{
					return;
				}

				LocalParamHistories.Add(&History);
			}
		}

		// check through the AdditionalDefines to see if any variables have been explicitly preserved
		TSet<FName> AttributesToPreserve;

		for (const FString& AdditionalDefine : InCompileOptions.AdditionalDefines)
		{
			const FString PreserveTag = TEXT("PreserveAttribute=");
			if (AdditionalDefine.StartsWith(PreserveTag))
			{
				AttributesToPreserve.Add(*AdditionalDefine.RightChop(PreserveTag.Len()));
			}
		}

		AttributesToPreserve.Add(SYS_PARAM_INSTANCE_ALIVE.GetName());
		AttributesToPreserve.Add(SYS_PARAM_PARTICLES_UNIQUE_ID.GetName());
		if (bRequiresPersistentIDs)
		{
			AttributesToPreserve.Add(SYS_PARAM_PARTICLES_ID.GetName());
		}

		const TArray<FNiagaraVariable> PreTrimmedAttributes = Attributes;

		if (SafeTrimAttributesEnabled)
		{
			FNiagaraAttributeTrimming::TrimAttributes_Safe(LocalParamHistories, AttributesToPreserve, Attributes);
		}
		else if (AggressiveTrimAttributesEnabled)
		{
			FNiagaraAttributeTrimming::TrimAttributes_Aggressive(CompileDuplicateData, LocalParamHistories, AttributesToPreserve, Attributes);
		}

		for (const FNiagaraVariable& Attribute : PreTrimmedAttributes)
		{
			if (!Attributes.Contains(Attribute))
			{
				TranslateResults.CompileTagsEditorOnly.Emplace(Attribute, TEXT("Trimmed"));
			}
		}
	}
}

TUniquePtr<INiagaraHlslTranslator> INiagaraHlslTranslator::CreateTranslator(const FNiagaraCompileRequestDataBase* InCompileData, const FNiagaraCompileRequestDuplicateDataBase* InDuplicateData)
{
	// translate everything
	const FNiagaraCompileRequestData* CompileRequest = static_cast<const FNiagaraCompileRequestData*>(InCompileData);
	const FNiagaraCompileRequestDuplicateData* CompileRequestDuplicate = static_cast<const FNiagaraCompileRequestDuplicateData*>(InDuplicateData);

	return MakeUnique<TNiagaraHlslTranslator<FNiagaraCompilationGraphBridge>>(CompileRequest, CompileRequestDuplicate);
}

TUniquePtr<INiagaraHlslTranslator> INiagaraHlslTranslator::CreateTranslator(const FNiagaraPrecompileData* InCompileData, const FNiagaraCompilationCopyData* InDuplicateData)
{
	return MakeUnique<TNiagaraHlslTranslator<FNiagaraCompilationDigestBridge>>(InCompileData, InDuplicateData);
}

template<typename GraphBridge>
FStringView TNiagaraHlslTranslator<GraphBridge>::GetEmitterUniqueName() const
{
	return CompileData->EmitterUniqueName;
}

template<typename GraphBridge>
FNiagaraEmitterID TNiagaraHlslTranslator<GraphBridge>::GetEmitterID() const
{
	return CompileData->EmitterID;
}

template<typename GraphBridge>
TConstArrayView<FNiagaraVariable> TNiagaraHlslTranslator<GraphBridge>::GetStaticVariables() const
{
	return CompileData->StaticVariables;
}

template<typename GraphBridge>
UNiagaraDataInterface* TNiagaraHlslTranslator<GraphBridge>::GetDataInterfaceCDO(UClass* DIClass) const
{
	return CompileDuplicateData->GetDuplicatedDataInterfaceCDOForClass(DIClass);
}

FString INiagaraHlslTranslator::GetHlslDefaultForType(const FNiagaraTypeDefinition& Type)
{
	if (Type == FNiagaraTypeDefinition::GetFloatDef())
	{
		return "(0.0)";
	}
	else if (Type == FNiagaraTypeDefinition::GetVec2Def())
	{
		return "float2(0.0, 0.0)";
	}
	else if (Type == FNiagaraTypeDefinition::GetVec3Def() || Type == FNiagaraTypeDefinition::GetPositionDef())
	{
		return "float3(0.0, 0.0, 0.0)";
	}
	else if (Type == FNiagaraTypeDefinition::GetVec4Def())
	{
		return "float4(0.0, 0.0, 0.0, 0.0)";
	}
	else if (Type == FNiagaraTypeDefinition::GetQuatDef())
	{
		return "float4(0.0, 0.0, 0.0, 1.0)";
	}
	else if (Type == FNiagaraTypeDefinition::GetColorDef())
	{
		return "float4(1.0, 1.0, 1.0, 1.0)";
	}
	else if (Type == FNiagaraTypeDefinition::GetIntDef())
	{
		return "(0)";
	}
	else if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		return "(false)";
	}
	else
	{
		return TEXT("(") + GetStructHlslTypeName(Type) + TEXT(")0");
	}
}

bool INiagaraHlslTranslator::IsBuiltInHlslType(const FNiagaraTypeDefinition& Type)
{
	return
		Type == FNiagaraTypeDefinition::GetFloatDef() ||
		Type == FNiagaraTypeDefinition::GetVec2Def() ||
		Type == FNiagaraTypeDefinition::GetVec3Def() ||
		Type == FNiagaraTypeDefinition::GetVec4Def() ||
		Type == FNiagaraTypeDefinition::GetColorDef() ||
		Type == FNiagaraTypeDefinition::GetPositionDef() ||
		Type == FNiagaraTypeDefinition::GetQuatDef() ||
		Type == FNiagaraTypeDefinition::GetMatrix4Def() ||
		Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) ||
		Type.GetStruct() == FNiagaraTypeDefinition::GetIntStruct() ||
		Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef());
}

FString INiagaraHlslTranslator::GetStructHlslTypeName(const FNiagaraTypeDefinition& Type)
{
	check(FNiagaraTypeHelper::IsLWCType(Type) == false);

	if (Type.IsValid() == false)
	{
		return "undefined";
	}
	else if (Type == FNiagaraTypeDefinition::GetFloatDef())
	{
		return "float";
	}
	else if (Type == FNiagaraTypeDefinition::GetVec2Def())
	{
		return "float2";
	}
	else if (Type == FNiagaraTypeDefinition::GetVec3Def() || Type == FNiagaraTypeDefinition::GetPositionDef())
	{
		return "float3";
	}
	else if (Type == FNiagaraTypeDefinition::GetVec4Def() || Type == FNiagaraTypeDefinition::GetColorDef() || Type == FNiagaraTypeDefinition::GetQuatDef())
	{
		return "float4";
	}
	else if (Type == FNiagaraTypeDefinition::GetMatrix4Def())
	{
		return "float4x4";
	}
	else if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || Type.GetEnum() != nullptr)
	{
		return "int";
	}
	else if (Type.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
	{
		return "bool";
	}
	else if (Type == FNiagaraTypeDefinition::GetParameterMapDef())
	{
		return "FParamMap0";
	}
	else
	{
		return Type.GetName();
	}
}

FString INiagaraHlslTranslator::GetPropertyHlslTypeName(const FProperty* Property)
{
	if (Property->IsA(FFloatProperty::StaticClass()))
	{
		return "float";
	}
	else if (Property->IsA(FIntProperty::StaticClass()))
	{
		return "int";
	}
	else if (Property->IsA(FUInt32Property::StaticClass()))
	{
		return "int";
	}
	else if (Property->IsA(FStructProperty::StaticClass()))
	{
		const FStructProperty* StructProp = CastField<const FStructProperty>(Property);
		return GetStructHlslTypeName(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::Simulation));
	}
	else if (Property->IsA(FEnumProperty::StaticClass()))
	{
		return "int";
	}
	else if (Property->IsA(FByteProperty::StaticClass()))
	{
		return "int";
	}
	else if (Property->IsA(FBoolProperty::StaticClass()))
	{
		return "bool";
	}
	else
	{
		return TEXT("");
	}
}

FString INiagaraHlslTranslator::BuildHLSLStructDecl(const FNiagaraTypeDefinition& Type, FText& OutErrorMessage, bool bGpuScript)
{
	if (!IsBuiltInHlslType(Type))
	{
		check(FNiagaraTypeHelper::IsLWCType(Type) == false);
		FString Decl = "struct " + GetStructHlslTypeName(Type) + "\n{\n";

		int32 StructSize = 0;
		for (TFieldIterator<FProperty> PropertyIt(Type.GetStruct(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;

			FString PropertyTypeName;
			int32 PropertyTypeSize;
			if (Property->IsA(FFloatProperty::StaticClass()))
			{
				PropertyTypeName = TEXT("float");
				PropertyTypeSize = 4;
			}
			else if (
				Property->IsA(FIntProperty::StaticClass()) ||
				Property->IsA(FUInt32Property::StaticClass()) ||
				Property->IsA(FEnumProperty::StaticClass()) ||
				Property->IsA(FByteProperty::StaticClass()) ||
				Property->IsA(FBoolProperty::StaticClass())
				)
			{
				PropertyTypeName = TEXT("int");
				PropertyTypeSize = 4;
			}
			else if (Property->IsA(FStructProperty::StaticClass()))
			{
				const FStructProperty* StructProp = CastField<const FStructProperty>(Property);
				const FNiagaraTypeDefinition NiagaraType(StructProp->Struct);
				PropertyTypeSize = NiagaraType.GetSize();
				PropertyTypeName = GetStructHlslTypeName(NiagaraType);
			}
			else
			{
				OutErrorMessage = FText::Format(
					LOCTEXT("UnknownPropertyTypeErrorFormat", "Failed to build hlsl struct declaration for type {0}.  Property {1} has an unsuported type {2}."),
					FText::FromString(Type.GetName()), Property->GetDisplayNameText(), FText::FromString(Property->GetClass()->GetName())
				);
				return TEXT("");
			}

			//-TODO: Alignment issues...
			//if (!bGpuScript && (StructSize != Property->GetOffset_ReplaceWith_ContainerPtrToValuePtr()))
			//{
			//	const int32 ByteDelta = Property->GetOffset_ReplaceWith_ContainerPtrToValuePtr() - StructSize;
			//	const int32 NumFloats = ByteDelta / sizeof(float);
			//	check(NumFloats > 0 && ByteDelta > 0);
			//	for ( int32 i=0; i < NumFloats; ++i )
			//	{
			//		Decl.Appendf(TEXT("\tfloat Dummy%d;\n"), DummyIndex++);
			//	}
			//	StructSize += ByteDelta;
			//}

			Decl.Appendf(TEXT("\t%s %s;\n"), *PropertyTypeName, *Property->GetName());
			StructSize += PropertyTypeSize;
		}

		//-TODO: Alignment issues...
		//if (!bGpuScript && (StructSize != Type.GetStruct()->GetStructureSize()))
		//{
		//	const int32 ByteDelta = Type.GetStruct()->GetStructureSize() - StructSize;
		//	const int32 NumFloats = ByteDelta / sizeof(float);
		//	check(NumFloats > 0 && ByteDelta > 0);
		//	for (int32 i = 0; i < NumFloats; ++i)
		//	{
		//		Decl.Appendf(TEXT("\tfloat Dummy%d;\n"), DummyIndex++);
		//	}
		//}

		Decl.Append(TEXT("};\n\n"));
		return Decl;
	}

	return TEXT("");
}

bool INiagaraHlslTranslator::IsHlslBuiltinVector(const FNiagaraTypeDefinition& Type)
{
	if ((Type == FNiagaraTypeDefinition::GetVec2Def()) ||
		(Type == FNiagaraTypeDefinition::GetVec3Def()) ||
		(Type == FNiagaraTypeDefinition::GetVec4Def()) ||
		(Type == FNiagaraTypeDefinition::GetQuatDef()) ||
		(Type == FNiagaraTypeDefinition::GetPositionDef()) ||
		(Type == FNiagaraTypeDefinition::GetColorDef()))
	{
		return true;
	}
	return false;
}

bool FNiagaraHlslTranslator::AddStructToDefinitionSet(const FNiagaraTypeDefinition& TypeDef)
{
	// First make sure that this is a type that we do need to define...
	if (IsBuiltInHlslType(TypeDef))
	{
		return true;
	}

	if (TypeDef == FNiagaraTypeDefinition::GetGenericNumericDef())
	{
		return false;
	}

	// We build these types on-the-fly.
	if (TypeDef == FNiagaraTypeDefinition::GetParameterMapDef())
	{
		return true;
	}

	if (StructsToDefine.Contains(TypeDef))
	{
		return true;
	}

	// Now make sure that we don't have any other struct types within our struct. Add them prior to the struct in question to make sure
	// that the syntax works out properly.
	if (UScriptStruct* Struct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(TypeDef.GetScriptStruct(), ENiagaraStructConversion::Simulation))
	{
		// We need to recursively dig through the struct to get at the lowest level of the input struct, which
		// could be a native type.
		for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = *PropertyIt;
			if (const FStructProperty* StructProp = CastField<const FStructProperty>(Property))
			{
				if (!AddStructToDefinitionSet(StructProp->Struct.Get()))
				{
					return false;
				}
			}
		}

		// Add the new type def
		FNiagaraTypeDefinition NewTypeDef(Struct);
		if (!StructsToDefine.Contains(NewTypeDef))
		{
			check(FNiagaraTypeHelper::IsLWCType(NewTypeDef) == false);
			StructsToDefine.Add(NewTypeDef);

			// Add the struct name to the unique symbol names to make it so that we don't declare a variable the same name as the struct type.
			GetUniqueSymbolName(*NewTypeDef.GetName());
		}
	}

	return true;
}

TArray<FName> FNiagaraHlslTranslator::ConditionPropertyPath(const FNiagaraTypeDefinition& Type, const TArray<FName>& InPath)
{
	// TODO: Build something more extensible and less hard coded for path conditioning.
	UScriptStruct* Struct = Type.GetScriptStruct();
	if (InPath.Num() == 0) // Pointing to the root
	{
		return TArray<FName>();
	}
	else if (IsHlslBuiltinVector(Type))
	{
		checkf(InPath.Num() == 1, TEXT("Invalid path for vector"));
		TArray<FName> ConditionedPath;
		ConditionedPath.Add(*InPath[0].ToString().ToLower());
		return ConditionedPath;
	}
	else if (FNiagaraTypeDefinition::IsScalarDefinition(Struct))
	{
		return TArray<FName>();
	}
	else if (Struct != nullptr)
	{
		// We need to recursively dig through the struct to get at the lowest level of the input path specified, which
		// could be a native type.
		for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = *PropertyIt;
			const FStructProperty* StructProp = CastField<const FStructProperty>(Property);
			// The names match, but even then things might not match up properly..
			if (InPath[0].ToString() == Property->GetName())
			{
				// The names match and this is a nested type, so we can keep digging...
				if (StructProp != nullptr)
				{
					// If our path continues onward, keep recursively digging. Otherwise, just return where we've gotten to so far.
					if (InPath.Num() > 1)
					{
						TArray<FName> ReturnPath;
						ReturnPath.Add(InPath[0]);
						TArray<FName> Subset = InPath;
						Subset.RemoveAt(0);
						TArray<FName> Children = ConditionPropertyPath(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::Simulation), Subset);
						for (const FName& Child : Children)
						{
							ReturnPath.Add(Child);
						}
						return ReturnPath;
					}
					else
					{
						TArray<FName> ConditionedPath;
						ConditionedPath.Add(InPath[0]);
						return ConditionedPath;
					}
				}
			}
		}
		return InPath;
	}
	return InPath;
}
//////////////////////////////////////////////////////////////////////////

// explicit template instantiation of the translators available based on the graph interfaces
template class TNiagaraHlslTranslator<FNiagaraCompilationGraphBridge>;
template class TNiagaraHlslTranslator<FNiagaraCompilationDigestBridge>;


#undef NIAGARA_SCOPE_CYCLE_COUNTER
#undef LOCTEXT_NAMESPACE
