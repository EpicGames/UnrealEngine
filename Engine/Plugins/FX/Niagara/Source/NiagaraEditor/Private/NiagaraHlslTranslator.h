// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Kismet2/CompilerResultsLog.h"
#include "INiagaraCompiler.h"
#include "NiagaraCommon.h"
#include "NiagaraCompilationBridge.h"
#include "NiagaraParameterMapHistory.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraShared.h"
#include "NiagaraTypes.h"

#include "NiagaraHlslTranslator.generated.h"

class FNiagaraCompileOptions;
class FNiagaraCompileRequestData;
class FNiagaraCompileRequestDuplicateData;
struct FNiagaraGraphHelper;
struct FNiagaraTranslatorOutput;
class UNiagaraNode;
class UNiagaraNodeFunctionCall;
class UNiagaraNodeOutput;
class UNiagaraNodeParameterMapGet;
class UNiagaraNodeParameterMapSet;
class UNiagaraScriptSource;

UENUM()
enum class ENiagaraDataSetAccessMode : uint8
{
	/** Data set reads and writes use shared counters to add and remove the end of available data. Writes are conditional and read */
	AppendConsume,
	/** Data set is accessed directly at a specific index. */
	Direct,

	Num UMETA(Hidden),
};

struct FNiagaraCustomHlslInclude
{
	bool bIsVirtual = false;
	FString FilePath;

	bool operator==(const FNiagaraCustomHlslInclude& Other) const
	{
		return bIsVirtual == Other.bIsVirtual && FilePath == Other.FilePath;
	}
};

/** Data which is generated from the hlsl by the VectorVMBackend and fed back into the */
struct FNiagaraTranslatorOutput
{
	FNiagaraVMExecutableData ScriptData;

	/** Ordered table of functions actually called by the VM script. */
	struct FCalledVMFunction
	{
		FString Name;
		TArray<bool> InputParamLocations;
		int32 NumOutputs;
		FCalledVMFunction() :NumOutputs(0) {}
	};
	TArray<FCalledVMFunction> CalledVMFunctionTable;

	FString Errors;

};

enum class ENiagaraCodeChunkMode : uint8
{
	GlobalConstant,
	SystemConstant,
	OwnerConstant,
	EmitterConstant,
	Uniform,
	Source,
	Body,
	SpawnBody,
	UpdateBody,
	InitializerBody,
	SimulationStageBody,
	SimulationStageBodyMax = SimulationStageBody + 100,
	Num,
};

FORCEINLINE uint32 GetTypeHash(const FNiagaraFunctionSignature& Sig)
{
	uint32 Hash = GetTypeHash(Sig.Name);
	for (const FNiagaraVariable& Var : Sig.Inputs)
	{
		Hash = HashCombine(Hash, GetTypeHash(Var));
	}
	for (const FNiagaraVariableBase& Var : Sig.Outputs)
	{
		Hash = HashCombine(Hash, GetTypeHash(Var));
	}
	Hash = HashCombine(Hash, GetTypeHash(Sig.OwnerName));
	Hash = HashCombine(Hash, GetTypeHash(Sig.ContextStageIndex));
	return Hash;
}

struct FNiagaraCodeChunk
{
	/** Symbol name for the chunk. Cam be empty for some types of chunk. */
	FString SymbolName;
	/** Format definition for incorporating SourceChunks into the final code for this chunk. */
	FString Definition;
	/** Original constant data*/
	FNiagaraVariable Original;
	/** The returned data type of this chunk. */
	FNiagaraTypeDefinition Type;
	/** If this chunk should declare it's symbol name. */
	bool bDecl;
	/** If the chunk is unterminated (no semicolon, because it's a scope or similar */
	bool bIsTerminated;
	/** Chunks used as input for this chunk. */
	TArray<int32> SourceChunks;
	/** Component mask for access to padded uniforms; will be empty except for float2 and float3 uniforms */
	FString ComponentMask;

	ENiagaraCodeChunkMode Mode;

	FNiagaraCodeChunk()
		: bDecl(true)
		, bIsTerminated(true)
		, ComponentMask("")
		, Mode(ENiagaraCodeChunkMode::Num)
	{
		Type = FNiagaraTypeDefinition::GetFloatDef();
	}

	void AddSourceChunk(int32 ChunkIdx)
	{
		SourceChunks.Add(ChunkIdx);
	}

	int32 GetSourceChunk(int32 i)
	{
		return SourceChunks[i];
	}

	void ReplaceSourceIndex(int32 SourceIdx, int32 NewIdx)
	{
		SourceChunks[SourceIdx] = NewIdx;
	}

	bool operator==(const FNiagaraCodeChunk& Other)
	{
		return SymbolName == Other.SymbolName &&
			Definition == Other.Definition &&
			Mode == Other.Mode &&
			Type == Other.Type &&
			bDecl == Other.bDecl &&
			Original == Other.Original &&
			SourceChunks == Other.SourceChunks;
	}
};

class FNiagaraHlslTranslationStage
{
public:
	FNiagaraHlslTranslationStage(ENiagaraScriptUsage InScriptUsage, FGuid InUsageId)
		: ScriptUsage(InScriptUsage)
		, UsageId(InUsageId)
	{
	}

	ENiagaraScriptUsage ScriptUsage;
	FGuid UsageId;
	FString PassNamespace;
	ENiagaraInterpolatedSpawnMode InterpolatedSpawnMode = ENiagaraInterpolatedSpawnMode::NoInterpolation;
	bool bCopyPreviousParams = true;
	ENiagaraCodeChunkMode ChunkModeIndex = (ENiagaraCodeChunkMode)-1;
	ENiagaraIterationSource IterationSourceType = ENiagaraIterationSource::Particles;
	FName IterationDataInterface;
	FName IterationDirectBinding;
	int32 SimulationStageIndex = -1;
	FName EnabledBinding;
	FIntVector3 ElementCount = FIntVector3::ZeroValue;
	FName ElementCountXBinding;
	FName ElementCountYBinding;
	FName ElementCountZBinding;
	int32 NumIterations = 1;
	FName NumIterationsBinding;
	ENiagaraSimStageExecuteBehavior ExecuteBehavior = ENiagaraSimStageExecuteBehavior::Always;
	bool bParticleIterationStateEnabled = false;
	FName ParticleIterationStateBinding;
	FIntPoint ParticleIterationStateRange = FIntPoint::ZeroValue;
	bool bUsesAlive = false;
	bool bWritesAlive = false;
	bool bWritesParticles = false;
	bool bPartialParticleUpdate = false;
	bool bGpuDispatchForceLinear = false;
	ENiagaraGpuDispatchType DirectDispatchType = ENiagaraGpuDispatchType::OneD;
	ENiagaraDirectDispatchElementType DirectDispatchElementType = ENiagaraDirectDispatchElementType::NumThreads;
	bool bOverrideGpuDispatchNumThreads = false;
	bool bShouldUpdateInitialAttributeValues = false;
	FIntVector OverrideGpuDispatchNumThreads = FIntVector(1, 1, 1);
	TArray<FNiagaraVariable> SetParticleAttributes;
	FString CustomReadFunction;
	FString CustomWriteFunction;
	int32 ParamMapHistoryIndex = INDEX_NONE;

	bool ShouldDoSpawnOnlyLogic() const;
	template<typename GraphBridge>
	bool IsRelevantToSpawnForStage(const typename GraphBridge::FParamMapHistory& InHistory, const FNiagaraVariable& InAliasedVar, const FNiagaraVariable& InVar) const;

	bool IsExternalConstantNamespace(const FNiagaraVariable& InVar, ENiagaraScriptUsage InTargetUsage, uint32 InTargetBitmask);
	FName GetIterationDataInterface() const { return IterationSourceType == ENiagaraIterationSource::DataInterface ? IterationDataInterface : NAME_None; }

	int32 CurrentCallID = 0;
	bool bCallIDInitialized = false;
};

struct FunctionNodeStackEntry
{
	TSet<FName> UnusedInputs;
	TSet<FString> CulledFunctionNames;
};

class FNiagaraHlslTranslator : public INiagaraHlslTranslator
{
public:

	struct FDataSetAccessInfo
	{
		//Variables accessed.
		TArray<FNiagaraVariable> Variables;
		/** Code chunks relating to this access. */
		TArray<int32> CodeChunks;
	};

	struct FNiagaraConstantBuffer
	{
		TArray<FNiagaraVariable> Variables;
		FString StructName;
	};

protected:
	FNiagaraCompileOptions CompileOptions;

	FHlslNiagaraTranslatorOptions TranslationOptions;

	/** The set of all generated code chunks for this script. */
	TArray<FNiagaraCodeChunk> CodeChunks;

	/** Array of code chunks of each different type. */
	TArray<int32> ChunksByMode[(int32)ENiagaraCodeChunkMode::Num];

	/** The combined output of the compilation of this script. This is temporary and will be reworked soon. */
	FNiagaraTranslatorOutput CompilationOutput;

	/** Message log. Automatically handles marking the NodeGraph with errors. */
	FCompilerResultsLog MessageLog;

	/** Captures information about a script compile. */
	FNiagaraTranslateResults TranslateResults;

	TMap<FName, uint32> GeneratedSymbolCounts;

	FDataSetAccessInfo InstanceRead;
	FDataSetAccessInfo InstanceWrite;

	TMap<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>> DataSetReadInfo[(int32)ENiagaraDataSetAccessMode::Num];
	TMap<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>> DataSetWriteInfo[(int32)ENiagaraDataSetAccessMode::Num];
	TMap<FNiagaraDataSetID, int32> DataSetWriteConditionalInfo[(int32)ENiagaraDataSetAccessMode::Num];

	FString GetDataSetAccessSymbol(FNiagaraDataSetID DataSet, int32 IndexChunk, bool bRead);
	FNiagaraDataSetID GetInstanceDataSetID()const { return FNiagaraDataSetID(TEXT("DataInstance"), ENiagaraDataSetType::ParticleData); }
	FNiagaraDataSetID GetSystemEngineDataSetID()const { return FNiagaraDataSetID(TEXT("Engine"), ENiagaraDataSetType::ParticleData); }
	FNiagaraDataSetID GetSystemUserDataSetID()const { return FNiagaraDataSetID(TEXT("User"), ENiagaraDataSetType::ParticleData); }
	FNiagaraDataSetID GetSystemConstantDataSetID()const { return FNiagaraDataSetID(TEXT("Constant"), ENiagaraDataSetType::ParticleData); }

	/** All functions called in the script. */
	struct FNiagaraFunctionBody
	{
		FString Body;
		FString Comments;
		TArray<int32> StageIndices;
		TArray<FName> StageWriteTargets;
		TArray<FName> StageReadTargets;
	};

	TMap<FNiagaraFunctionSignature, FNiagaraFunctionBody> Functions;
	TArray<FNiagaraCustomHlslInclude> FunctionIncludeFilePaths;
	TArray<TArray<FName>> ActiveStageWriteTargets;
	TArray<TArray<FName>> ActiveStageReadTargets;

	void GenerateFunctionCall(ENiagaraScriptUsage ScriptUsage, FNiagaraFunctionSignature& FunctionSignature, TArrayView<const int32> Inputs, TArray<int32>& Outputs);
	FString GetFunctionIncludeStatement(const FNiagaraCustomHlslInclude& Include);
	FString GetFunctionSignature(const FNiagaraFunctionSignature& Sig);

	void WriteDataSetContextVars(TMap<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>>& DataSetAccessInfo, bool bRead, FString& OutHLSLOutput);
	void WriteDataSetStructDeclarations(TMap<FNiagaraDataSetID, TMap<int32, FDataSetAccessInfo>>& DataSetAccessInfo, bool bRead, FString& OutHLSLOutput);
	void DecomposeVariableAccess(UStruct* Struct, bool bRead, FString IndexSymbol, FString HLSLString);

	FString GetUniqueSymbolName(FName BaseName);

	/** Stack of all function params. */
	struct FFunctionContext
	{
		FString Name;
		FNiagaraFunctionSignature& Signature;
		TArrayView<const int32> Inputs;
		FGuid Id;
		FFunctionContext(const FString& InName, FNiagaraFunctionSignature& InSig, TArrayView<const int32> InInputs, const FGuid& InId)
			: Name(InName)
			, Signature(InSig)
			, Inputs(InInputs)
			, Id(InId)
		{}
	};
	TArray<FFunctionContext> FunctionContextStack;
	const FFunctionContext* FunctionCtx()const { return FunctionContextStack.Num() > 0 ? &FunctionContextStack.Last() : nullptr; }
	FString GetCallstack();
	TArray<FGuid> GetCallstackGuids();

	void EnterStatsScope(FNiagaraStatScope StatScope);
	void ExitStatsScope();
	void EnterStatsScope(FNiagaraStatScope StatScope, FString& OutHlsl);
	void ExitStatsScope(FString& OutHlsl);

	FString GeneratedConstantString(float Constant);
	FString GeneratedConstantString(FVector4 Constant);

	/* Add a chunk that is not written to the source, only used as a source chunk for others. */
	int32 AddSourceChunk(FString SymbolName, const FNiagaraTypeDefinition& Type, bool bSanitize = true);

	/** Add a chunk defining a uniform value. */
	int32 AddChunkToConstantBuffer(const FString& SymbolName, const FNiagaraVariable& InVariable, ENiagaraCodeChunkMode ChunkMode);

	/** Reserves a chunk for a uniform value.  The chunk isn't resolved till the call to PackRegisteredUniformChunk. */
	int32 RegisterUniformChunkToPack(const FString& SymbolName, const FNiagaraVariable& InVariable, bool AddPadding, FNiagaraParameters& Parameters, TOptional<FNiagaraVariable>& ConflictingVariable);
	void PackRegisteredUniformChunk(FNiagaraParameters& Parameters);

	/* Add a chunk that is written to the body of the shader code. */
	int32 AddBodyChunk(FStringView SymbolName, FStringView Definition, const FNiagaraTypeDefinition& Type, TArrayView<int32> SourceChunks, bool bDecl = true, bool bIsTerminated = true);
	int32 AddBodyChunk(FStringView SymbolName, FStringView Definition, const FNiagaraTypeDefinition& Type, int32 SourceChunk, bool bDecl = true, bool bIsTerminated = true);
	int32 AddBodyChunk(FStringView SymbolName, FStringView Definition, const FNiagaraTypeDefinition& Type, bool bDecl = true, bool bIsTerminated = true);
	int32 AddBodyComment(FStringView Comment);
	int32 AddBodyChunk(FStringView Definition);

	FString GetUniqueEmitterName() const;
	void ConvertCompileInfoToParamInfo(const FNiagaraScriptDataInterfaceCompileInfo& InCompileInfo, FNiagaraDataInterfaceGPUParamInfo& OutGPUParamInfo, TArray<FNiagaraFunctionSignature>& GeneratedFunctionSignatures);

	FString GetFunctionDefinitions();

public:
	NIAGARAEDITOR_API FNiagaraHlslTranslator();
	virtual ~FNiagaraHlslTranslator() = default;

	virtual const FNiagaraTranslatorOutput& GetTranslateOutput() const override { return CompilationOutput; }
	virtual const FString& GetTranslatedHLSL() const override { return HlslOutput; }

	bool DisableDebugSwitches() const { return TranslationOptions.bDisableDebugSwitches; }

	NIAGARAEDITOR_API bool IsCompileOptionDefined(const TCHAR* InDefineStr);

	NIAGARAEDITOR_API int32 GetAttribute(const FNiagaraVariable& Attribute);

	NIAGARAEDITOR_API int32 GetConstant(const FNiagaraVariable& Constant, FString* DebugOutputValue = nullptr);

	NIAGARAEDITOR_API int32 ParameterMapForInnerIndex() const;

	NIAGARAEDITOR_API void DefinePreviousParametersFunction(FString& HlslOutput, TArray<TArray<FNiagaraVariable>>& DataSetVariables, TMap<FNiagaraDataSetID, int32>& DataSetReads, TMap<FNiagaraDataSetID, int32>& DataSetWrites);

	NIAGARAEDITOR_API void DefineDataSetReadFunction(FString& HlslOutput, TArray<FNiagaraDataSetID>& ReadDataSets);
	NIAGARAEDITOR_API void DefineDataSetWriteFunction(FString& HlslOutput, TArray<FNiagaraDataSetProperties>& WriteDataSets, TArray<int32>& WriteConditionVarIndices);
	NIAGARAEDITOR_API void DefineMain(FString& HLSLOutput, const TArray<TArray<FNiagaraVariable>>& DataSetVariables, const TMap<FNiagaraDataSetID, int32>& DataSetReads, const TMap<FNiagaraDataSetID, int32>& DataSetWrites);

	NIAGARAEDITOR_API void DefineDataSetVariableReads(FString& HLSLOutput, const FNiagaraDataSetID& Id, int32 DataSetIndex, const TArray<FNiagaraVariable>& ReadVars);
	NIAGARAEDITOR_API void DefineDataInterfaceHLSL(FString& HlslOutput);
	NIAGARAEDITOR_API void DefineExternalFunctionsHLSL(FString& HlslOutput);

	// Format string should have up to 5 entries, {{0} = Computed Variable Suffix, {1} = Float or Int, {2} = Data Set Index, {3} = Register Index, {4} Default value for that type.
	NIAGARAEDITOR_API void GatherVariableForDataSetAccess(const FNiagaraVariable& Variable, FString Format, int32& RegisterIdxInt, int32& RegisterIdxFloat, int32& RegisterIdxHalf, int32 DataSetIndex, FString InstanceIdxSymbol, FString& HlslOutput, bool bWriteHLSL = true);
	NIAGARAEDITOR_API void GatherComponentsForDataSetAccess(UScriptStruct* Struct, FString VariableSymbol, bool bMatrixRoot, TArray<FString>& Components, TArray<ENiagaraBaseTypes>& Types);

	NIAGARAEDITOR_API void EnterFunctionCallNode(const TSet<FName>& UnusedInputs);
	NIAGARAEDITOR_API void ExitFunctionCallNode();
	NIAGARAEDITOR_API bool IsFunctionVariableCulledFromCompilation(const FName& InputName) const;

	NIAGARAEDITOR_API bool GetFunctionParameter(const FNiagaraVariable& Parameter, int32& OutParam)const;
	NIAGARAEDITOR_API int32 GetUniqueCallerID();

	NIAGARAEDITOR_API bool CanReadAttributes()const;
	NIAGARAEDITOR_API ENiagaraScriptUsage GetTargetUsage() const;
	NIAGARAEDITOR_API FGuid GetTargetUsageId() const;
	ENiagaraSimTarget GetSimulationTarget() const
	{
		return CompilationTarget;
	}

	static NIAGARAEDITOR_API TArray<FName> ConditionPropertyPath(const FNiagaraTypeDefinition& Type, const TArray<FName>& InPath);
	static NIAGARAEDITOR_API FString GetSanitizedSymbolName(FStringView SymbolName, bool bCollapseNamespaces = false);
	static NIAGARAEDITOR_API FString GetSanitizedDIFunctionName(const FString& FunctionName);
	static NIAGARAEDITOR_API void AppendSanitizedFunctionNameSuffix(FStringView Name, FStringBuilderBase& StringBuilder);

	/** Replaces all non-ascii characters with a "ASCXXX" string, where XXX is their int value */
	static NIAGARAEDITOR_API FString ConvertToAsciiString(FString Name);

	NIAGARAEDITOR_API bool AddStructToDefinitionSet(const FNiagaraTypeDefinition& TypeDef);

	static NIAGARAEDITOR_API FString GetFunctionSignatureSymbol(const FNiagaraFunctionSignature& Sig);

	static NIAGARAEDITOR_API FName GetDataInterfaceName(FName BaseName, const FString& UniqueEmitterName, bool bIsParameterMapDataInterface);
	static NIAGARAEDITOR_API FString GenerateFunctionHlslPrototype(FStringView InVariableName, const FNiagaraFunctionSignature& FunctionSignature);
	static NIAGARAEDITOR_API FString GenerateFunctionParametersHlsl(const FNiagaraFunctionSignature& FunctionSignature);

	NIAGARAEDITOR_API void SetConstantByStaticVariable(int32& OutValue, const FNiagaraVariable& Var, FString* DebugString = nullptr);
	NIAGARAEDITOR_API void SetConstantByStaticVariable(FNiagaraVariable& OutValue, const FNiagaraVariable& Var, FString* DebugString = nullptr);

	NIAGARAEDITOR_API bool IsEventSpawnScript()const;

	// collection of functions that rely on the graph implementation and so must be relegated to the TNiagaraHlslTranslator
	virtual void DefineDataSetVariableWrites(FString& HlslOutput, const FNiagaraDataSetID& Id, int32 DataSetIndex, const TArray<FNiagaraVariable>& WriteVars) = 0;
	virtual void Message(FNiagaraCompileEventSeverity Severity, FText MessageText, FStringView ShortDescription = FStringView()) = 0;
	virtual void Error(FText ErrorText, FStringView ShortDescription = FStringView()) = 0;
	virtual void Warning(FText WarningText, FStringView ShortDescription = FStringView()) = 0;

	/** If OutVar can be replaced by a literal constant, it's data is initialized with the correct value and we return true. Returns false otherwise. */
	virtual bool GetLiteralConstantVariable(FNiagaraVariable& OutVar) const = 0;


protected:
	bool ShouldConsiderTargetParameterMap(ENiagaraScriptUsage InUsage) const;

	void HandleNamespacedExternalVariablesToDataSetRead(TArray<FNiagaraVariable>& InDataSetVars, FString InNamespaceStr);

	// For GPU simulations we have to special case some variables and pass them view shader parameters rather than the uniform buffer as they vary from CPU simulations
	bool IsVariableInUniformBuffer(const FNiagaraVariable& Variable) const;

	FString ComputeMatrixColumnAccess(const FString& Name);
	FString ComputeMatrixRowAccess(const FString& Name);

	// Add a raw float constant chunk
	int32 GetConstantDirect(float InValue);
	int32 GetConstantDirect(bool InValue);
	int32 GetConstantDirect(int InValue);

	FNiagaraTypeDefinition GetChildType(const FNiagaraTypeDefinition& BaseType, const FName& PropertyName);
	FString NamePathToString(const FString& Prefix, const FNiagaraTypeDefinition& RootType, const TArray<FName>& NamePath);
	FString GenerateAssignment(const FNiagaraTypeDefinition& SrcType, const TArray<FName>& SrcPath, const FNiagaraTypeDefinition& DestType, const TArray<FName>& DestPath);

	//Generates the code for the passed chunk.
	FString GetCode(FNiagaraCodeChunk& Chunk);
	FString GetCode(int32 ChunkIdx);
	//Retreives the code for this chunk being used as a source for another chunk
	FString GetCodeAsSource(int32 ChunkIdx);

	// Generate a structure initializer string
	// Returns true if we generated the structure successfully or false if we encounter something we could not handle
	bool GenerateStructInitializer(TStringBuilder<128>& InitializerString, UStruct* UserDefinedStruct, const void* StructData, int32 ByteOffset = 0);
	// Convert a variable with actual data into a constant string
	FString GenerateConstantString(const FNiagaraVariable& Constant);

	// Takes the current script state (interpolated or not) and determines the correct context variable.
	FString GetParameterMapInstanceName(int32 ParamMapHistoryIdx);

	// Register an attribute in its non namespaced form
	virtual bool ParameterMapRegisterNamespaceAttributeVariable(const FNiagaraVariable& InVariable, int32 InParamMapHistoryIdx, int32& Output) = 0;

	bool ShouldInterpolateParameter(const FNiagaraVariable& Parameter);
	FString GetInterpolateHlsl(const FNiagaraVariable& Parameter, const FString& PrevMapName, const FNiagaraCodeChunk& Chunk) const;

	bool IsBulkSystemScript() const;
	bool IsSpawnScript() const;
	bool RequiresInterpolation() const;
	bool IsWriteAllowedForNamespace(const FNiagaraVariable& Var, ENiagaraScriptUsage TargetUsage, FText& ErrorMsg);

	void AppendSanitizedSymbolString(FStringBuilderBase& Builder, FStringView SymbolName, bool bCollapseNamespaces = false) const;
	FString BuildSanitizedSymbolString(FStringView SymbolName, bool bCollapseNamespaces = false) const;
	FString BuildSanitizedSymbolName(FName SymbolName, bool bCollapseNamespaces = false) const;

	void AppendFunctionSignatureString(const FNiagaraFunctionSignature& Sig, FStringBuilderBase& Builder) const;
	FString BuildFunctionSignatureString(const FNiagaraFunctionSignature& Sig) const;

	void AppendFunctionHlslPrototypeString(FStringView InVariableName, const FNiagaraFunctionSignature& FunctionSignature, FStringBuilderBase& Builder) const;
	FString BuildFunctionHlslPrototypeString(FStringView InVariableName, const FNiagaraFunctionSignature& FunctionSignature) const;

	template<typename T>
	void BuildConstantBuffer(ENiagaraCodeChunkMode ChunkMode);

	virtual FNiagaraEmitterID GetEmitterID() const = 0;
	virtual FStringView GetEmitterUniqueName() const = 0;
	virtual TConstArrayView<FNiagaraVariable> GetStaticVariables() const = 0;
	virtual UNiagaraDataInterface* GetDataInterfaceCDO(UClass* DIClass) const = 0;

	/** Map of symbol names to count of times it's been used. Used for generating unique symbol names. */
	TMap<FName, uint32> SymbolCounts;

	//Set of non-builtin structs we have to define in hlsl.
	TArray<FNiagaraTypeDefinition> StructsToDefine;

	// Keep track of which parameter map history this came from.
	TArray<int32> ParamMapHistoriesSourceInOtherHistories;

	// All of the variables arrays in the other histories converted to sanitized HLSL format. Used in parsing custom hlsl nodes.
	TArray< TArray<FNiagaraVariable> > OtherOutputParamMapHistoriesSanitizedVariables;

	// Synced to the ParamMapHistories.
	TArray<TArray<int32>> ParamMapSetVariablesToChunks;

	// Used to keep track of contextual information about the currently compiled function node
	TArray<FunctionNodeStackEntry> FunctionNodeStack;

	// Synced to the System uniforms encountered for parameter maps thus far.
	struct FUniformVariableInfo
	{
		FNiagaraVariable Variable;
		int32 ChunkIndex;
		int32 ChunkMode;
	};

	struct FVarAndDefaultSource
	{
		FNiagaraVariable Variable;
		bool bDefaultExplicit = false; //Whether or not the default value of the variable is explicit, e.g. there is an explicit value on a pin, explicit binding, or explicit custom initialization.
	};

	TArray<FNiagaraVariable> ParamMapDefinedEngineVars; // Engine provided values that we want to be accessible via the parameter map but not to add as uniforms etc.

	TMap<int32 /*parameter index*/, int32 /*chunk index*/> UniformParametersToPack; // Map connecting the parameters that need to be packed for a uniform buffer with their registered chunk index

	TMap<FName, FUniformVariableInfo> ParamMapDefinedSystemVars; // Map from the defined constants to the uniform chunk expressing them (i.e. have we encountered before in this graph?)

	// Synced to the EmitterParameter uniforms encountered for parameter maps thus far.
	TMap<FName, int32> ParamMapDefinedEmitterParameterVarsToUniformChunks; // Map from the variable name exposed by the emitter as a parameter to the uniform chunk expressing it (i.e. have we encountered before in this graph?)
	TMap<FName, FNiagaraVariable> ParamMapDefinedEmitterParameterToNamespaceVars; // Map from defined parameter to the Namespaced variable expressing it.

	// Synced to the Attributes encountered for parameter maps thus far.
	TMap<FName, int32> ParamMapDefinedAttributesToUniformChunks; // Map from the variable name exposed as a attribute to the uniform chunk expressing it (i.e. have we encountered before in this graph?)
	TMap<FName, FVarAndDefaultSource> ParamMapDefinedAttributesToNamespaceVars; // Map from defined parameter to the Namespaced variable expressing it.

	// Synced to the external variables used when bulk compiling system scripts.
	TArray<FNiagaraVariable> ExternalVariablesForBulkUsage;

	// List of primary output variables encountered that need to be properly handled in spawn scripts.
	TArray<FNiagaraVariable> UniqueVars;

	// List of variables for interpolated spawning
	TArray<FNiagaraVariable> InterpSpawnVariables;
	
	// Map of primary output variable description to whether or not it came from this script's parameter map
	TMap<FNiagaraVariable, bool> UniqueVarToWriteToParamMap;
	
	// Map ofthe primary output variable description to the actual chunk id that wrote to it.
	TMap<FNiagaraVariable, int32> UniqueVarToChunk;

	// Map of the output variable to the found aliased variable, used to discover the default binding, etc
	TMap<FNiagaraVariableBase, FNiagaraVariableBase> UniqueVarToAliasedVar;

	TArray<int32> ParameterMapForIndexStack;

	// Strings to be inserted within the main function
	TArray<TArray<FString>> PerStageMainPreSimulateChunks;

	// read and write data set indices
	int32 ReadIdx;
	int32 WriteIdx;

	// Parameter data per data interface.
	FNiagaraShaderScriptParametersMetadata ShaderScriptParametersMetadata;
	
	/** Stack of currently tracked stats scopes. */
	TArray<int32> StatScopeStack;

	FString HlslOutput;

	ENiagaraSimTarget CompilationTarget;

	// Used to keep track of which output node we are working back from. This allows us 
	// to find the right parameter map.
	TArray<int32> CurrentParamMapIndices;

	ENiagaraCodeChunkMode CurrentBodyChunkMode;

	TArray<FNiagaraHlslTranslationStage> TranslationStages;
	int32 ActiveStageIdx;
	bool bInitializedDefaults;
	bool bEnforceStrictTypesValidations;

	// Variables that need to be initialized based on some other variable's value at the end of spawn.
	TArray<FNiagaraVariable> InitialNamespaceVariablesMissingDefault;
	// Variables that need to be initialized in the body or at the end of spawn.
	TArray<FNiagaraVariable> DeferredVariablesMissingDefault;

	// local copy of UNiagaraEditorSettings::GetHLSLKeywordReplacmentsMap() for the duration of the translation process
	TMap<FString, FString> KeywordReplacementMap;
};

// all data and function implementations that rely on specifics of the graph implementation
// are shuffled into this class to allow us to share the code through the inherited GraphBridge
template<typename GraphBridge>
class TNiagaraHlslTranslator : public FNiagaraHlslTranslator, private GraphBridge
{
	using typename GraphBridge::FGraph;
	using typename GraphBridge::FPin;
	using typename GraphBridge::FNode;
	using typename GraphBridge::FParamMapHistory;
	using typename GraphBridge::FParamMapHistoryBuilder;
	using typename GraphBridge::FPrecompileData;
	using typename GraphBridge::FCompilationCopy;
	using typename GraphBridge::FFunctionCallNode;
	using typename GraphBridge::FOpNode;
	using typename GraphBridge::FOutputNode;
	using typename GraphBridge::FIfNode;
	using typename GraphBridge::FInputNode;
	using typename GraphBridge::FSelectNode;
	using typename GraphBridge::FCustomHlslNode;
	using typename GraphBridge::FConvertNode;
	using typename GraphBridge::FParamMapForNode;
	using typename GraphBridge::FParamMapSetNode;
	using typename GraphBridge::FParamMapGetNode;
	using typename GraphBridge::FEmitterNode;
	using typename GraphBridge::FStaticSwitchNode;
	using typename GraphBridge::FInputPin;
	using typename GraphBridge::FOutputPin;
	using typename GraphBridge::FModuleScopedPin;
	using typename GraphBridge::FGraphFunctionAliasContext;
	using typename GraphBridge::FConvertConnection;
	using typename GraphBridge::FParameterCollection;

public:
	using FBridge = GraphBridge;

	TNiagaraHlslTranslator() = delete;
	TNiagaraHlslTranslator(const FPrecompileData* InPrecompileData, const FCompilationCopy* InCompilationCopy)
	: CompileData(InPrecompileData)
	, CompileDuplicateData(InCompilationCopy)
	{}

	virtual FNiagaraTranslateResults Translate(const FNiagaraCompileOptions& InCompileOptions, const FHlslNiagaraTranslatorOptions& Options) override;

	void RegisterFunctionCall(ENiagaraScriptUsage ScriptUsage, const FString& InName, FName InFullName, const FGuid& CallNodeId, const FString& InFunctionNameSuffix, const FGraph* SourceGraph, const FNiagaraFunctionSignature& InSignature, bool bIsCustomHlsl, const FString& InCustomHlsl, const TArray<FNiagaraCustomHlslInclude>& InCustomHlslIncludeFilePaths, TArray<int32>& Inputs, TConstArrayView<const FInputPin*> CallInputs, TConstArrayView<const FOutputPin*> CallOutputs,
		FNiagaraFunctionSignature& OutSignature);
	/** Compiles an output Pin on a graph node. Caches the result for any future inputs connected to it. */
	int32 CompileOutputPin(const FPin* Pin);

	void EnterFunction(const FString& Name, FNiagaraFunctionSignature& Signature, TArrayView<const int32> Inputs, const FGuid& InGuid);
	void ExitFunction();
	void HandleDataInterfaceCall(FNiagaraScriptDataInterfaceCompileInfo& Info, const FNiagaraFunctionSignature& InMatchingSignature);

	struct FCompiledPin
	{
		int32 CompilationIndex;
		const FPin* Pin;

		FCompiledPin(int32 CompilationIndex, const FPin* InPin)
		: CompilationIndex(CompilationIndex)
		, Pin(InPin)
		{}
	};

public:

	int32 CompileInputPin(const FInputPin* Pin);

	int32 RegisterUObject(const FNiagaraVariable& Variable, UObject* Object, bool bAddParameterMapRead);
	int32 RegisterUObjectPath(const FNiagaraVariable& Variable, const FSoftObjectPath& ObjectPath, bool bAddParmeterMapRead);
	int32 RegisterDataInterface(const FNiagaraVariable& Var, const UNiagaraDataInterface* DataInterface, bool bPlaceholder, bool bAddParameterMapRead);

	void Operation(const FOpNode* Operation, TArray<int32>& Inputs, TArray<int32>& Outputs);
	void Output(const FOutputNode* OutputNode, const TArray<int32>& ComputedInputs);

	int32 GetParameter(const FNiagaraVariable& Parameter);
	int32 GetRapidIterationParameter(const FNiagaraVariable& Parameter);
	void ReadDataSet(const FNiagaraDataSetID DataSet, const TArray<FNiagaraVariable>& Variable, ENiagaraDataSetAccessMode AccessMode, int32 InputChunk, TArray<int32>& Outputs);
	void WriteDataSet(const FNiagaraDataSetID DataSet, const TArray<FNiagaraVariable>& Variable, ENiagaraDataSetAccessMode AccessMode, const TArray<int32>& Inputs, TArray<int32>& Outputs);
	void ParameterMapSet(const FParamMapSetNode* SetNode, TArrayView<const FCompiledPin> Inputs, TArray<int32>& Outputs);
	void ParameterMapGet(const FParamMapGetNode* GetNode, TArrayView<const int32> Inputs, TArray<int32>& Outputs);
	void ParameterMapForBegin(const FParamMapForNode* ForNode, int32 IterationCount);
	void ParameterMapForContinue(const FParamMapForNode* ForNode, int32 IterationEnabled);
	void ParameterMapForEnd(const FParamMapForNode* ForNode);
	void Emitter(const FEmitterNode* GetNode, TArray<int32>& Inputs, TArray<int32>& Outputs);
	void FunctionCall(const FFunctionCallNode* FunctionNode, TArray<int32>& Inputs, TArray<int32>& Outputs);
	void Convert(const FConvertNode* Convert, TArrayView<const int32> Inputs, TArray<int32>& Outputs);
	void If(const FIfNode* IfNode, const TArray<FNiagaraVariable>& Vars, int32 Condition, TArray<int32>& PathA, TArray<int32>& PathB, TArray<int32>& Outputs);
	/** Options is a map from selector values to compiled pin code chunk indices */
	void Select(const FSelectNode* SelectNode, int32 Selector, const TArray<FNiagaraVariable>& OutputVariables, TMap<int32, TArray<int32>>& Options, TArray<int32>& Outputs);

	void DefineInterpolatedParametersFunction(FString& HlslOutput);
	void DefineMainGPUFunctions(const TArray<TArray<FNiagaraVariable>>& DataSetVariables, const TMap<FNiagaraDataSetID, int32>& DataSetReads, const TMap<FNiagaraDataSetID, int32>& DataSetWrites);
	virtual void DefineDataSetVariableWrites(FString& HlslOutput, const FNiagaraDataSetID& Id, int32 DataSetIndex, const TArray<FNiagaraVariable>& WriteVars) override;

	void CullMapSetInputPin(const FPin* InputPin);

	void WriteCompilerTag(int32 InputCompileResult, const FPin* Pin, bool bEditorOnly, bool bEmitMessageOnFailure, FNiagaraCompileEventSeverity FailureSeverity, const FString& Prefix = FString());

	void Message(FNiagaraCompileEventSeverity Severity, FText MessageText, const FNode* Node, const FPin* Pin, FStringView ShortDescription = FStringView());
	void Error(FText ErrorText, const FNode* Node, const FPin* Pin, FStringView ShortDescription = FStringView());
	void Warning(FText WarningText, const FNode* Node, const FPin* Pin, FStringView ShortDescription = FStringView());
	void RegisterCompileDependency(const FNiagaraVariableBase& InVar, FText ErrorText, const FNode* Node, const FPin* Pin, bool bEmitAsLinker, int32 ParamMapHistoryIdx);
	ENiagaraScriptUsage GetCurrentUsage() const;

	virtual void Message(FNiagaraCompileEventSeverity Severity, FText MessageText, FStringView ShortDescription = FStringView()) override;
	virtual void Error(FText ErrorText, FStringView ShortDescription = FStringView()) override;
	virtual void Warning(FText WarningText, FStringView ShortDescription = FStringView()) override;
	FString NodePinToMessage(FText MessageText, const FNode* Node, const FPin* Pin);

	/** If OutVar can be replaced by a literal constant, it's data is initialized with the correct value and we return true. Returns false otherwise. */
	virtual bool GetLiteralConstantVariable(FNiagaraVariable& OutVar) const override;

	/** If Var can be replaced by a another constant variable, or is a constant itself, add the appropriate body chunk and return true. */
	bool HandleBoundConstantVariableToDataSetRead(FNiagaraVariable InVariable, const FNode* InNode, int32 InParamMapHistoryIdx, int32& Output, const FPin* InDefaultPin);

	void FillVariableWithDefaultValue(FNiagaraVariable& InVar, const FPin* InDefaultPin);
	void FillVariableWithDefaultValue(int32& OutValue, const FPin* InDefaultPin);

	void SetConstantByStaticVariable(int32& OutValue, const FPin* InDefaultPin, FString* DebugString = nullptr);
	void SetConstantByStaticVariable(FNiagaraVariable& OutValue, const FPin* InDefaultPin, FString* DebugString = nullptr);

	int32 MakeStaticVariableDirect(const FPin* InDefaultPin);

	void TrimAttributes(const FNiagaraCompileOptions& InCompileOptions, TArray<FNiagaraVariable>& Attributes);

protected:
	void InitializeParameterMapDefaults(int32 ParamMapHistoryIdx);
	void HandleParameterRead(int32 ParamMapHistoryIdx, const FNiagaraVariable& Var, const FInputPin* DefaultPin, const FNode* ErrorNode, int32& OutputChunkId, TOptional<ENiagaraDefaultMode> DefaultMode = TOptional<ENiagaraDefaultMode>(), TOptional<FNiagaraScriptVariableBinding> DefaultBinding = TOptional<FNiagaraScriptVariableBinding>(), bool bTreatAsUnknownParameterMap = false, bool bIgnoreDefaultSetFirst = false);
	FString BuildParameterMapHlslDefinitions(TArray<FNiagaraVariable>& PrimaryDataSetOutputEntries);
	void BuildMissingDefaults();
	int32 HandleReadUnknownMissingDefault(const FNiagaraVariable& DefaultVar, const FInputPin* DefaultPin);
	void FinalResolveNamespacedTokens(const FString& ParameterMapInstanceNamespace, TArray<FString>& Tokens, TArray<FString>& ValidChildNamespaces, FParamMapHistoryBuilder& Builder, TArray<FNiagaraVariable>& UniqueParameterMapEntriesAliasesIntact, TArray<FNiagaraVariable>& UniqueParameterMapEntries, int32 ParamMapHistoryIdx, const FNode* InNodeForErrorReporting);
	void FindConstantValue(int32 InputCompileResult, const FNiagaraTypeDefinition& TypeDef, FString& Value, FNiagaraVariable& Variable);

	bool ParseDIFunctionSpecifiers(const FNode* NodeForErrorReporting, FNiagaraFunctionSignature& Sig, TArray<FString>& Tokens, int32& TokenIdx);
	void HandleCustomHlslNode(const FCustomHlslNode* CustomFunctionHlsl, ENiagaraScriptUsage& OutScriptUsage, FString& OutName, FString& OutFullName, bool& bOutCustomHlsl, FString& OutCustomHlsl, TArray<FNiagaraCustomHlslInclude>& OutCustomHlslIncludeFilePaths,
		FNiagaraFunctionSignature& OutSignature, TArray<int32>& Inputs);
	void ProcessCustomHlsl(const FString& InCustomHlsl, ENiagaraScriptUsage InUsage, const FNiagaraFunctionSignature& InSignature, const TArray<int32>& Inputs, const FNode* NodeForErrorReporting, FString& OutCustomHlsl, FNiagaraFunctionSignature& OutSignature);
	void HandleSimStageSetupAndTeardown(int32 InWhichStage, FString& OutHlsl);
	// Register a System/Engine/read-only variable in its namespaced form
	bool ParameterMapRegisterExternalConstantNamespaceVariable(FNiagaraVariable InVariable, const FNode* InNode, int32 InParamMapHistoryIdx, int32& Output, const FPin* InDefaultPin);

	// Register an attribute in its non namespaced form
	bool ParameterMapRegisterNamespaceAttributeVariable(const FNiagaraVariable& InVariable, const FNode* InNode, int32 InParamMapHistoryIdx, int32& Output);
	virtual bool ParameterMapRegisterNamespaceAttributeVariable(const FNiagaraVariable& InVariable, int32 InParamMapHistoryIdx, int32& Output) override;

	// Register an attribute in its namespaced form
	bool ParameterMapRegisterUniformAttributeVariable(const FNiagaraVariable& InVariable, FNode* InNode, int32 InParamMapHistoryIdx, int32& Output);

	// Checks that the Partices.ID parameter is only used if persistent IDs are active
	void ValidateParticleIDUsage();
	bool ValidateTypePins(const FNode* NodeToValidate);
	void GenerateFunctionSignature(ENiagaraScriptUsage ScriptUsage, FStringView InName, FName InFullName, FStringView InFunctionNameSuffix, const FGraph* FuncGraph, TConstArrayView<const FInputPin*> CallInputs, TArray<int32>& Inputs,
		FString& FunctionComment, FNiagaraFunctionSignature& OutSig) const;
	void ResolveGraphFunctionAlias(const FNiagaraGraphFunctionAliasContext& FunctionAliasContext, FStringBuilderBase& FunctionAliasSuffix) const;

	void ValidateFailIfPreviouslyNotSet(const FNiagaraVariable& InVar, bool& bFailIfNotSet);

	void UpdateStaticSwitchConstants(const FPin* Pin);

	virtual FStringView GetEmitterUniqueName() const override;
	virtual FNiagaraEmitterID GetEmitterID() const override;
	virtual TConstArrayView<FNiagaraVariable> GetStaticVariables() const override;
	virtual UNiagaraDataInterface* GetDataInterfaceCDO(UClass* DIClass) const override;

	// Specific method to reconcile whether the default value has been implicitly or explicitly set for a namespaced var added to the param map.
	void RecordParamMapDefinedAttributeToNamespaceVar(const FNiagaraVariable& VarToRecord, const FPin* VarAssociatedDefaultPin);

private:
	const FPrecompileData* CompileData;
	const FCompilationCopy* CompileDuplicateData;

	/**
	Map of Pins to compiled code chunks. Allows easy reuse of previously compiled pins.
	A stack so that we can track pin reuse within function calls but not have cached pins cross talk with subsequent calls to the same function.
	*/
	TArray<TMap<const FPin*, int32>> PinToCodeChunks;

	// Keep track of all the paths that the parameter maps can take through the graph.
	TArray<FParamMapHistory> ParamMapHistories;

	// Keep track of the other output nodes in the graph's histories so that we can make sure to 
	// create any variables that are needed downstream.
	TArray<FParamMapHistory> OtherOutputParamMapHistories;

	// Make sure that the function call names match up on the second traversal.
	FParamMapHistoryBuilder ActiveHistoryForFunctionCalls;

	// Map of primary ouput variable description to its default value pin
	TMap<FNiagaraVariable, const FInputPin*> UniqueVarToDefaultPin;

	TArray<const FPin*> CurrentDefaultPinTraversal;

	TArray<const FOutputNode*> OutputNodes;
};
