// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMessageManager.h"
#include "NiagaraScriptSource.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "NiagaraMessages.h"
#include "Toolkits/NiagaraScriptToolkit.h"
#include "NiagaraEditorModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "NiagaraEditorUtilities.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "NiagaraMessages.h"

#define LOCTEXT_NAMESPACE "NiagaraMessageManager"

FNiagaraMessageManager* FNiagaraMessageManager::Singleton = nullptr;
uint32 FNiagaraMessageManager::NextTopicBitflag = 1;
bool FNiagaraMessageManager::bNeedFlushMessages = false;
bool FNiagaraMessageManager::bRefreshTimeElapsed = true;

FNiagaraMessageManager::FNiagaraMessageManager()
{
}

FNiagaraMessageManager* FNiagaraMessageManager::Get()
{
	if (Singleton == nullptr)
	{
		Singleton = new FNiagaraMessageManager();
	}
	return Singleton;
}

void FNiagaraMessageManager::AddMessage(const TSharedRef<const INiagaraMessage>& InMessage, const FGuid& InMessageAssetKey)
{
	bNeedFlushMessages = true;

	FAssetMessageInfo& AssetMessageInfo = AssetToMessageInfoMap.FindOrAdd(InMessageAssetKey);
	AssetMessageInfo.Messages.Add(InMessage);
	AssetMessageInfo.bDirty = true;
	AssetMessageInfo.DirtyTopicBitfield |= InMessage->GetMessageTopicBitflag();
}

void FNiagaraMessageManager::AddMessageJob(TUniquePtr<const INiagaraMessageJob>&& InMessageJob, const FGuid& InMessageJobAssetKey)
{
	//MessageJobs.Emplace(FMessageJobAndAssetKey(InMessageJob, InMessageJobAssetKey));
	MessageJobs.Emplace(FMessageJobAndAssetKey(InMessageJobAssetKey)); //@todo(ng) touchup
	MessageJobs.Last().MessageJob = MoveTemp(InMessageJob);
}

void FNiagaraMessageManager::ClearAssetMessages(const FGuid& AssetKey)
{
	AssetToMessageInfoMap.Remove(AssetKey);
}

void FNiagaraMessageManager::ClearAssetMessagesForTopic(const FGuid& AssetKey, const FName& Topic)
{
	FAssetMessageInfo* AssetMessageInfo = AssetToMessageInfoMap.Find(AssetKey);
	if (AssetMessageInfo != nullptr)
	{
		const uint32 TopicBitflag = GetMessageTopicBitflag(Topic);
		for(int i = AssetMessageInfo->Messages.Num() - 1; i > -1; --i)
		{
			if (TopicBitflag & AssetMessageInfo->Messages[i]->GetMessageTopicBitflag())
			{
				AssetMessageInfo->DirtyTopicBitfield |= AssetMessageInfo->Messages[i]->GetMessageTopicBitflag();
				AssetMessageInfo->Messages.RemoveAt(i, EAllowShrinking::No);
				AssetMessageInfo->bDirty = true;
				bNeedFlushMessages = true;
			}
		}
	}
}

void FNiagaraMessageManager::ClearAssetMessagesForObject(const FGuid& AssetKey, const FObjectKey& ObjectKeys)
{
	FAssetMessageInfo* AssetMessageInfo = AssetToMessageInfoMap.Find(AssetKey);
	if (AssetMessageInfo != nullptr)
	{
		for (int i = AssetMessageInfo->Messages.Num() - 1; i > -1; --i)
		{
			const TArray<FObjectKey>& MessageObjectKeys = AssetMessageInfo->Messages[i]->GetAssociatedObjectKeys();
			if (MessageObjectKeys.Contains(ObjectKeys))
			{
				AssetMessageInfo->DirtyTopicBitfield |= AssetMessageInfo->Messages[i]->GetMessageTopicBitflag();
				AssetMessageInfo->Messages.RemoveAt(i, EAllowShrinking::No);
				AssetMessageInfo->bDirty = true;
				bNeedFlushMessages = true;
			}
		}
	}
}

const TOptional<const FString> FNiagaraMessageManager::GetStringForScriptUsageInStack(const ENiagaraScriptUsage InScriptUsage)
{
	switch (InScriptUsage) {
	case ENiagaraScriptUsage::ParticleSpawnScript:
		return TOptional<const FString>(FString("Particle Spawn Script"));

	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
		return TOptional<const FString>(TEXT("Particle Spawn Script Interpolated"));

	case ENiagaraScriptUsage::ParticleGPUComputeScript:
		return TOptional<const FString>(TEXT("Particle GPU Compute Script"));

	case ENiagaraScriptUsage::ParticleUpdateScript:
		return TOptional<const FString>(TEXT("Particle Update Script"));
		
	case ENiagaraScriptUsage::ParticleEventScript:
		return TOptional<const FString>(TEXT("Particle Event Script"));

	case ENiagaraScriptUsage::ParticleSimulationStageScript:
		return TOptional<const FString>(FString("Particle Simulation Stage Script"));

	case ENiagaraScriptUsage::EmitterSpawnScript:
		return TOptional<const FString>(TEXT("Emitter Spawn Script"));

	case ENiagaraScriptUsage::EmitterUpdateScript:
		return TOptional<const FString>(TEXT("Emitter Update Script"));

	case ENiagaraScriptUsage::SystemSpawnScript:
		return TOptional<const FString>(TEXT("System Spawn Script"));

	case ENiagaraScriptUsage::SystemUpdateScript:
		return TOptional<const FString>(TEXT("System Update Script"));

	//We don't expect to see these usages in the stack so do not set the toptional
	case ENiagaraScriptUsage::DynamicInput:
	case ENiagaraScriptUsage::Function:
	case ENiagaraScriptUsage::Module:
		return TOptional<const FString>();

	//unhandled cases
	default:
		ensureMsgf(false, TEXT("Tried to get script usage text for usage that is not handled!"));
		return TOptional<const FString>();
	}
}

void FNiagaraMessageManager::RegisterMessageTopic(FName TopicName)
{
	const uint32* TopicBitflag = RegisteredTopicToBitflagsMap.Find(TopicName);
	checkf(
		TopicBitflag == nullptr,
		TEXT("Tried to register topic '%s' but it has already been registered!"),
			*TopicName.ToString());

	RegisteredTopicToBitflagsMap.Add(TopicName, NextTopicBitflag);

	// binary increment the topic bitflag for the next topic to be registered.
	NextTopicBitflag <<= 1;
}

void FNiagaraMessageManager::RegisterAdditionalMessageLogTopic(FName MessageLogTopicName)
{
	AdditionalMessageLogTopics.AddUnique(MessageLogTopicName);
}

uint32 FNiagaraMessageManager::GetMessageTopicBitflag(FName TopicName)
{
	const uint32* TopicBitflag = RegisteredTopicToBitflagsMap.Find(TopicName);
	if (TopicBitflag == nullptr)
	{
		// It is possible the message topic has not been registered yet. If so, register now.
		RegisterMessageTopic(TopicName);
		return *(RegisteredTopicToBitflagsMap.Find(TopicName));
	}

	return *TopicBitflag;
}

void FNiagaraMessageManager::Tick(float DeltaSeconds)
{
	DoMessageJobsTick();
	TryFlushMessagesTick();
}

void FNiagaraMessageManager::DoMessageJobsTick()
{
	if (MessageJobs.Num() > 0)
	{
		double WorkStartTime = FPlatformTime::Seconds();
		double CurrentWorkLoopTime = WorkStartTime;

		while(MessageJobs.Num() > 0)
		{ 
			FMessageJobAndAssetKey CurrentMessageJobAndAssetKey = MessageJobs.Pop(EAllowShrinking::No);
			TSharedRef<const INiagaraMessage> GeneratedMessage = CurrentMessageJobAndAssetKey.MessageJob->GenerateNiagaraMessage();
			AddMessage(GeneratedMessage, CurrentMessageJobAndAssetKey.AssetKey);
			CurrentWorkLoopTime = FPlatformTime::Seconds();

			if (CurrentWorkLoopTime - WorkStartTime > MaxJobWorkTime)
			{
				// Max job work time has been reached, early exit so as to not stall the UI.
				return;
			}
		} 
	}
}

void FNiagaraMessageManager::TryFlushMessagesTick()
{
	if (bNeedFlushMessages && bRefreshTimeElapsed)
	{
		FlushMessages();
	}
}

void FNiagaraMessageManager::FlushMessages()
{
	bNeedFlushMessages = false;
	bRefreshTimeElapsed = false;
	FTimerDelegate SetRefreshTimerElapsedDelegate;
	SetRefreshTimerElapsedDelegate.BindStatic(&SetRefreshTimerElapsed);

	GEditor->GetTimerManager()->SetTimer(
		RefreshTimerHandle
		, SetRefreshTimerElapsedDelegate
		, RefreshHysterisisTime
		, false);

	for (auto AssetIt = AssetToMessageInfoMap.CreateIterator(); AssetIt; ++AssetIt)
	{ 
		FAssetMessageInfo& AssetMessageInfo = AssetIt.Value();
		if (AssetMessageInfo.bDirty)
		{
			for (auto RegistrationIt = AssetMessageInfo.RegistrationKey_To_RegistrationHandleMap.CreateIterator(); RegistrationIt; ++RegistrationIt)
			{
				INiagaraMessageRegistrationHandle* RegistrationHandle = RegistrationIt.Value().Get();

				// Only push messages that have a topic that is dirty AND in the registration handle topics.
				const uint32 DesiredTopicBitfield = AssetMessageInfo.DirtyTopicBitfield & RegistrationHandle->GetTopicBitfield();
				if (DesiredTopicBitfield != 0)
				{
					// Note, filtering is not using desired topic bitfield here as that requires the registered view to recycle non-dirty messages
					const TArray<TSharedRef<const INiagaraMessage>>& TopicalMessages = RegistrationHandle->FilterMessages(AssetMessageInfo.Messages, RegistrationHandle->GetTopicBitfield()); 
					RegistrationHandle->GetOnRequestRefresh().Execute(TopicalMessages);
				}
			}
		}
		AssetMessageInfo.bDirty = false;
		AssetMessageInfo.DirtyTopicBitfield = 0;
	}
}

void FNiagaraMessageManager::SetRefreshTimerElapsed()
{
	bRefreshTimeElapsed = true;
}

uint32 FNiagaraMessageManager::MakeBitfieldForMessageTopics(const FText& DebugNameText, const TArray<FName>& MessageTopics)
{
	uint32 TopicBitfield = 0;
	for (const FName& TopicName : MessageTopics)
	{
		const uint32 TopicBitflag = GetMessageTopicBitflag(TopicName);
		TopicBitfield |= TopicBitflag;
	}
	return TopicBitfield;
}

FNiagaraMessageTopicRegistrationHandle::FOnRequestRefresh& FNiagaraMessageManager::SubscribeToAssetMessagesByTopic(
	  const FText& DebugNameText
	, const FGuid& MessageAssetKey
	, const TArray<FName>& MessageTopics
	, FGuid& OutMessageManagerRegistrationKey)
{
	checkf(MessageAssetKey != FGuid(), TEXT("Tried to subscribe to an asset without a set asset key!"));
	uint32 TopicBitfield = MakeBitfieldForMessageTopics(DebugNameText, MessageTopics);

	TSharedPtr<FNiagaraMessageTopicRegistrationHandle> RegistrationHandle = MakeShared<FNiagaraMessageTopicRegistrationHandle>(TopicBitfield);
	const FGuid RegistrationKey = FGuid::NewGuid();
	OutMessageManagerRegistrationKey = RegistrationKey;
	FAssetMessageInfo& AssetMessageInfo = AssetToMessageInfoMap.FindOrAdd(MessageAssetKey);
	TSharedPtr<INiagaraMessageRegistrationHandle>& RegistrationHandleRef =
		AssetMessageInfo.RegistrationKey_To_RegistrationHandleMap.Emplace(RegistrationKey, RegistrationHandle);

	return RegistrationHandleRef->GetOnRequestRefresh();
}

FNiagaraMessageTopicRegistrationHandle::FOnRequestRefresh& FNiagaraMessageManager::SubscribeToAssetMessagesByObject(
	  const FText& DebugNameText
	, const FGuid& MessageAssetKey
	, const FObjectKey& ObjectKey
	, FGuid& OutMessageManagerRegistrationKey)
{
	checkf(MessageAssetKey != FGuid(), TEXT("Tried to subscribe to an asset without a set asset key!"));

	TSharedPtr<FNiagaraMessageObjectRegistrationHandle> RegistrationHandle = MakeShared<FNiagaraMessageObjectRegistrationHandle>(ObjectKey);
	const FGuid RegistrationKey = FGuid::NewGuid();
	OutMessageManagerRegistrationKey = RegistrationKey;
	FAssetMessageInfo& AssetMessageInfo = AssetToMessageInfoMap.FindOrAdd(MessageAssetKey);
	TSharedPtr<INiagaraMessageRegistrationHandle>& RegistrationHandleRef =
		AssetMessageInfo.RegistrationKey_To_RegistrationHandleMap.Emplace(RegistrationKey, RegistrationHandle);

	return RegistrationHandleRef->GetOnRequestRefresh();
}

void FNiagaraMessageManager::Unsubscribe(const FText& DebugNameText, const FGuid& MessageAssetKey, FGuid& MessageManagerRegistrationKey)
{
	ensureMsgf(MessageAssetKey != FGuid(), TEXT("Tried to unsubscribe from an asset without a set asset key!"));

	FAssetMessageInfo* AssetMessageInfo = AssetToMessageInfoMap.Find(MessageAssetKey);
	if (ensureMsgf(AssetMessageInfo != nullptr,
		TEXT("Tried to unbind message subscriber '%s' but failed to find the associated asset message info!"), *DebugNameText.ToString())
		)
	{
		AssetMessageInfo->RegistrationKey_To_RegistrationHandleMap.FindAndRemoveChecked(MessageManagerRegistrationKey);
	}
	MessageManagerRegistrationKey = FGuid();
}

FNiagaraCompileEventToken::FNiagaraCompileEventToken(
	  const FString& InScriptAssetPath
	, const FText& InMessage
	, const TOptional<const FGuid>& InNodeGUID
	, const TOptional<const FGuid>& InPinGUID)
	: ScriptAssetPath(InScriptAssetPath)
	, NodeGUID(InNodeGUID)
	, PinGUID(InPinGUID)
{
	if (!InMessage.IsEmpty())
	{
		CachedText = InMessage;
	}
	else
	{
		CachedText = FText::FromString(InScriptAssetPath);
	}

	MessageTokenActivated = FOnMessageTokenActivated::CreateStatic(&FNiagaraCompileEventToken::OpenScriptAssetByPathAndFocusNodeOrPinIfSet, ScriptAssetPath, NodeGUID, PinGUID);
}

void FNiagaraCompileEventToken::OpenScriptAssetByPathAndFocusNodeOrPinIfSet(
	  const TSharedRef<IMessageToken>& Token
	, FString InScriptAssetPath
	, const TOptional<const FGuid> InNodeGUID
	, const TOptional<const FGuid> InPinGUID)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(InScriptAssetPath));
	if (AssetData.IsValid())
	{
		UNiagaraScript* ScriptAsset = Cast<UNiagaraScript>(AssetData.GetAsset());
		if (ScriptAsset != nullptr && ScriptAsset->IsAsset())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ScriptAsset, EToolkitMode::Standalone);
			FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
			if (InPinGUID.IsSet())
			{
				
				TSharedRef<FNiagaraScriptGraphPinToFocusInfo> PinToFocusInfo = MakeShared<FNiagaraScriptGraphPinToFocusInfo>(InPinGUID.GetValue());
				FNiagaraScriptIDAndGraphFocusInfo PinToFocusAndScriptID = FNiagaraScriptIDAndGraphFocusInfo(ScriptAsset->GetUniqueID(), PinToFocusInfo);
				NiagaraEditorModule.GetOnScriptToolkitsShouldFocusGraphElement().Broadcast(&PinToFocusAndScriptID);
			}
			else if (InNodeGUID.IsSet())
			{
				TSharedRef<FNiagaraScriptGraphNodeToFocusInfo> NodeToFocusInfo = MakeShared<FNiagaraScriptGraphNodeToFocusInfo>(InNodeGUID.GetValue());
				FNiagaraScriptIDAndGraphFocusInfo NodeToFocusAndScriptID = FNiagaraScriptIDAndGraphFocusInfo(ScriptAsset->GetUniqueID(), NodeToFocusInfo);
				NiagaraEditorModule.GetOnScriptToolkitsShouldFocusGraphElement().Broadcast(&NodeToFocusAndScriptID);
			}
		}
		else if (ScriptAsset != nullptr)
		{
			UNiagaraSystem* System = ScriptAsset->GetTypedOuter<UNiagaraSystem>();
			UNiagaraEmitter* Emitter = ScriptAsset->GetTypedOuter<UNiagaraEmitter>();
			if (System)
			{
				if (System->IsAsset())
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(System, EToolkitMode::Standalone);
				}
				else if (Emitter && Emitter->IsAsset()) 
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Emitter, EToolkitMode::Standalone);
				}

				TArray<TSharedPtr<FNiagaraSystemViewModel>> ReferencingSystemViewModels;
				FNiagaraSystemViewModel::GetAllViewModelsForObject(System, ReferencingSystemViewModels);
				for (TSharedPtr<FNiagaraSystemViewModel> ReferencingSystemViewModel : ReferencingSystemViewModels)
				{
					TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchModuleViewModel =
						ReferencingSystemViewModel->GetScriptScratchPadViewModel()->GetViewModelForScript(ScriptAsset);
					if (ScratchModuleViewModel.IsValid())
					{
						ReferencingSystemViewModel->GetScriptScratchPadViewModel()->FocusScratchPadScriptViewModel(ScratchModuleViewModel.ToSharedRef());

						if (InPinGUID.IsSet())
						{
							TSharedRef<FNiagaraScriptGraphPinToFocusInfo> PinToFocusInfo = MakeShared<FNiagaraScriptGraphPinToFocusInfo>(InPinGUID.GetValue());
							FNiagaraScriptIDAndGraphFocusInfo PinToFocusAndScriptID = FNiagaraScriptIDAndGraphFocusInfo(ScriptAsset->GetUniqueID(), PinToFocusInfo);
							ScratchModuleViewModel->RaisePinFocusRequested(&PinToFocusAndScriptID);
						}
						else if (InNodeGUID.IsSet())
						{
							TSharedRef<FNiagaraScriptGraphNodeToFocusInfo> NodeToFocusInfo = MakeShared<FNiagaraScriptGraphNodeToFocusInfo>(InNodeGUID.GetValue());
							FNiagaraScriptIDAndGraphFocusInfo NodeToFocusAndScriptID = FNiagaraScriptIDAndGraphFocusInfo(ScriptAsset->GetUniqueID(), NodeToFocusInfo);
							ScratchModuleViewModel->RaiseNodeFocusRequested(&NodeToFocusAndScriptID);
						}
					}
				}
			}
			else if (Emitter && Emitter->IsAsset())
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Emitter, EToolkitMode::Standalone);
			}
		}
		else
		{
			FNiagaraEditorUtilities::WarnWithToastAndLog(FText::Format(
				LOCTEXT("CantNavigateWarning", "Could not navigate to script {0}\nIt was either was not a valid script, or it is not an asset which can be opened directly."),
				FText::FromString(InScriptAssetPath)));
		}
	}
}

TSharedRef<FNiagaraCompileEventToken> FNiagaraCompileEventToken::Create(
	  const FString& InScriptAssetPath
	, const FText& InMessage
	, const TOptional<const FGuid>& InNodeGUID /*= TOptional<const FGuid>() */
	, const TOptional<const FGuid>& InPinGUID /*= TOptional<const FGuid>()*/ )
{
	return MakeShareable(new FNiagaraCompileEventToken(InScriptAssetPath, InMessage, InNodeGUID, InPinGUID));
}

TArray<TSharedRef<const INiagaraMessage>> FNiagaraMessageTopicRegistrationHandle::FilterMessages(const TArray<TSharedRef<const INiagaraMessage>>& Messages, const uint32& DesiredTopicBitfield) const
{
	return Messages.FilterByPredicate([DesiredTopicBitfield](const TSharedRef<const INiagaraMessage>& Message) {return Message->GetMessageTopicBitflag() & DesiredTopicBitfield; });
}

TArray<TSharedRef<const INiagaraMessage>> FNiagaraMessageObjectRegistrationHandle::FilterMessages(const TArray<TSharedRef<const INiagaraMessage>>& Messages, const uint32& DesiredTopicBitfield) const
{
	return Messages.FilterByPredicate([this](const TSharedRef<const INiagaraMessage>& Message) {
		const TArray<FObjectKey>& MessageObjectKeys = Message->GetAssociatedObjectKeys();
		return MessageObjectKeys.Contains(ObjectKey);
		});
}

#undef LOCTEXT_NAMESPACE //NiagaraMessageManager
