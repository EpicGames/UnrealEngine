// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMessageUtilities.h"
#include "NiagaraMessages.h"
#include "CoreMinimal.h"

#define LOCTEXT_NAMESPACE "NiagaraMessageUtilities"

FText FNiagaraMessageUtilities::MakePostCompileSummaryText(const FText& CompiledObjectNameText, ENiagaraScriptCompileStatus LatestCompileStatus, const int32& WarningCount, const int32& ErrorCount)
{
	FText MessageText = FText();
	bool bHasErrors = ErrorCount > 0;
	bool bHasWarnings = WarningCount > 0;

	switch (LatestCompileStatus) {
	case ENiagaraScriptCompileStatus::NCS_Error:
		if (bHasErrors)
		{
			if (bHasWarnings)
			{
				MessageText = LOCTEXT("NiagaraCompileStatusErrorInfo_ErrorsWarnings", "{0} failed to compile with {1} {1}|plural(one=warning,other=warnings) and {2} {2}|plural(one=error,other=errors).");
				MessageText = FText::Format(MessageText, CompiledObjectNameText, FText::FromString(FString::FromInt(WarningCount)), FText::FromString(FString::FromInt(ErrorCount)));
				break;
			}
			MessageText = LOCTEXT("NiagaraCompileStatusErrorInfo_Errors", "{0} failed to compile with {1} {1}|plural(one=error,other=errors).");
			MessageText = FText::Format(MessageText, CompiledObjectNameText, FText::FromString(FString::FromInt(ErrorCount)));
		}
		else
		{
			ensureMsgf(false, TEXT("Compile status came back as NCS_Error but no Error messages were generated! Inspect this asset!"));
			MessageText = LOCTEXT("NiagaraCompileStatusErrorInfo", "{0} failed to compile.");
			MessageText = FText::Format(MessageText, CompiledObjectNameText);
		}
		break;

	case ENiagaraScriptCompileStatus::NCS_UpToDate:
		MessageText = LOCTEXT("NiagaraCompileStatusSuccessInfo", "{0} successfully compiled.");
		MessageText = FText::Format(MessageText, CompiledObjectNameText);
		break;

	case ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings:
		if (bHasWarnings)
		{
			MessageText = LOCTEXT("NiagaraCompileStatusWarningInfo_Warnings", "{0} successfully compiled with {1} {1}|plural(one=warning,other=warnings).");
			MessageText = FText::Format(MessageText, CompiledObjectNameText, FText::FromString(FString::FromInt(WarningCount)));
		}
		else
		{
			ensureMsgf(false, TEXT("Compile status came back as NCS_UpToDateWithWarnings but no Warning messages were generated! Inspect this asset!"));
			MessageText = LOCTEXT("NiagaraCompileStatusWarningInfo", "{0} successfully compiled with warnings.");
			MessageText = FText::Format(MessageText, CompiledObjectNameText);
		}
		break;

	case ENiagaraScriptCompileStatus::NCS_Unknown:
	case ENiagaraScriptCompileStatus::NCS_Dirty:
		if (bHasWarnings && bHasErrors)
		{
			MessageText = LOCTEXT("NiagaraCompileStatusUnknownInfo_ErrorsWarnings", "{0} compile status unknown with {1} {1}|plural(one=warning,other=warnings) and {2} {2}|plural(one=error,other=errors).");
			MessageText = FText::Format(MessageText, CompiledObjectNameText, FText::FromString(FString::FromInt(WarningCount)), FText::FromString(FString::FromInt(ErrorCount)));
		}
		else if (bHasErrors)
		{
			MessageText = LOCTEXT("NiagaraCompileStatusUnknownInfo_Errors", "{0} compile status unknown with {1} {1}|plural(one=error,other=errors).");
			MessageText = FText::Format(MessageText, CompiledObjectNameText, FText::FromString(FString::FromInt(ErrorCount)));
		}
		else if (bHasWarnings)
		{
			MessageText = LOCTEXT("NiagaraCompileStatusUnknownInfo_Warnings", "{0} compile status unknown with {1} {1}|plural(one=warning,other=warnings).");
			MessageText = FText::Format(MessageText, CompiledObjectNameText, FText::FromString(FString::FromInt(WarningCount)));
		}
		else
		{
			MessageText = LOCTEXT("NiagaraCompileStatusUnknownInfo", "{0} compile status unknown.");
			MessageText = FText::Format(MessageText, CompiledObjectNameText);
		}
		break;

	default:
		ensureMsgf(false, TEXT("Unexpected niagara compile status encountered!"));

	}
	return MessageText;
}

UNiagaraStackEntry::FStackIssue FNiagaraMessageUtilities::MessageToStackIssue(TSharedRef<const INiagaraMessage> InMessage, FString InStackEditorDataKey)
{
	FText ShortDescription = InMessage->GenerateMessageTitle();
	
	TSharedRef<FTokenizedMessage> TokenizedMessage = InMessage->GenerateTokenizedMessage();
	EStackIssueSeverity StackIssueSeverity;
	switch (TokenizedMessage->GetSeverity())
	{
	case EMessageSeverity::Error:
		StackIssueSeverity = EStackIssueSeverity::Error;
		break;
	case EMessageSeverity::PerformanceWarning:
	case EMessageSeverity::Warning:
		StackIssueSeverity = EStackIssueSeverity::Warning;
		break;
	case EMessageSeverity::Info:
		StackIssueSeverity = EStackIssueSeverity::Info;
		break;
	default:
		StackIssueSeverity = EStackIssueSeverity::Info;
		break;
	}

	if(ShortDescription.IsEmpty())
	{
		ShortDescription = LOCTEXT("UnspecifiedTitle", "Message");
	}

	TArray<UNiagaraStackEntry::FStackIssueFix> FixLinks;
	TArray<FText> LinkMessages;
	TArray<FSimpleDelegate> LinkNavigateActions;
	InMessage->GenerateLinks(LinkMessages, LinkNavigateActions);
	for (int32 LinkIndex = 0; LinkIndex < LinkMessages.Num(); LinkIndex++)
	{
		const FText& LinkMessage = LinkMessages[LinkIndex];
		const FSimpleDelegate& LinkNavigateAction = LinkNavigateActions[LinkIndex];
		FixLinks.Add(UNiagaraStackEntry::FStackIssueFix(
			LinkMessage,
			UNiagaraStackEntry::FStackIssueFixDelegate::CreateLambda([LinkNavigateAction]() { LinkNavigateAction.Execute(); }),
			UNiagaraStackEntry::EStackIssueFixStyle::Link));
	}

	return UNiagaraStackEntry::FStackIssue(
		StackIssueSeverity,
		ShortDescription,
		InMessage->GenerateMessageText(),
		InStackEditorDataKey,
		InMessage->GetDismissHandler().IsBound(),
		FixLinks,
		InMessage->GetDismissHandler());
}

UNiagaraStackEntry::FStackIssue FNiagaraMessageUtilities::StackMessageToStackIssue(const FNiagaraStackMessage& InMessage, FString InStackEditorDataKey, const TArray<FLinkNameAndDelegate>& Links)
{
	EStackIssueSeverity StackIssueSeverity;
	switch (InMessage.MessageSeverity)
	{
	case ENiagaraMessageSeverity::CriticalError:
	case ENiagaraMessageSeverity::Error:
		StackIssueSeverity = EStackIssueSeverity::Error;
		break;
	case ENiagaraMessageSeverity::PerformanceWarning:
	case ENiagaraMessageSeverity::Warning:
		StackIssueSeverity = EStackIssueSeverity::Warning;
		break;
	case ENiagaraMessageSeverity::Info:
		StackIssueSeverity = EStackIssueSeverity::Info;
		break;
	default:
		StackIssueSeverity = EStackIssueSeverity::Info;
		break;
	}

	TArray<UNiagaraStackEntry::FStackIssueFix> FixLinks;

	for(const FLinkNameAndDelegate& Link : Links)
	{
		const FSimpleDelegate& Delegate  = Link.LinkDelegate;
		FixLinks.Add(UNiagaraStackEntry::FStackIssueFix(
			Link.LinkNameText,
			UNiagaraStackEntry::FStackIssueFixDelegate::CreateLambda([Delegate]() { Delegate.Execute(); }),
			UNiagaraStackEntry::EStackIssueFixStyle::Link));
	}

	FText ShortDescription = InMessage.ShortDescription;

	if(ShortDescription.IsEmptyOrWhitespace())
	{
		if(StackIssueSeverity == EStackIssueSeverity::Info)
		{
			ShortDescription = LOCTEXT("EmptyShortDescriptionFromStackMessage_Info", "Information");
		}
		else if(StackIssueSeverity == EStackIssueSeverity::Warning)
		{
			ShortDescription = LOCTEXT("EmptyShortDescriptionFromStackMessage_Warning", "Warning");
		}
		else if(StackIssueSeverity == EStackIssueSeverity::Error)
		{
			ShortDescription = LOCTEXT("EmptyShortDescriptionFromStackMessage_Error", "Error");
		}
	}
	
	return UNiagaraStackEntry::FStackIssue(
		StackIssueSeverity,
		ShortDescription,
		InMessage.MessageText,
		InStackEditorDataKey,
		InMessage.bAllowDismissal,
		FixLinks);
}

FText FNiagaraMessageUtilities::GetShortDescriptionFromSeverity(EStackIssueSeverity Severity)
{
	if(Severity == EStackIssueSeverity::Error)
	{
		return LOCTEXT("CompileErrorShortDescription", "Compile Error");
	}
	else if(Severity == EStackIssueSeverity::Warning)
	{
		return LOCTEXT("CompileWarningShortDescription", "Compile Warning");			
	}
	else if(Severity == EStackIssueSeverity::Info)
	{
		return LOCTEXT("CompileNoteShortDescription", "Compile Note");
	}
	else
	{
		return LOCTEXT("SeverityNotFoundShortDescription", "Compile Error");
	}
}

#undef LOCTEXT_NAMESPACE /** NiagaraMessageUtilities */
