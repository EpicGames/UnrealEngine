// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMessages.h"

#include "NiagaraEditorUtilities.h"
#include "NiagaraMessageManager.h"
#include "NiagaraMessageUtilities.h"
#include "NiagaraScriptSource.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeEmitter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraMessages)

#define LOCTEXT_NAMESPACE "NiagaraMessages"

const FName FNiagaraMessageTopics::CompilerTopicName = "Compiler";
const FName FNiagaraMessageTopics::ObjectTopicName = "Object";

FText INiagaraMessage::GenerateMessageTitle() const
{
	return FText::GetEmpty();
}

FNiagaraMessageCompileEvent::FNiagaraMessageCompileEvent(
	const FNiagaraCompileEvent& InCompileEvent
	, TArray<FNiagaraScriptNameAndAssetPath>& InContextScriptNamesAndAssetPaths
	, TOptional<const FText>& InOwningScriptNameAndUsageText
	, TOptional<const FNiagaraScriptNameAndAssetPath>& InCompiledScriptNameAndAssetPath
	, const TArray<FObjectKey>& InAssociatedObjectKeys
	)
	: INiagaraMessage(InAssociatedObjectKeys)
	, CompileEvent(InCompileEvent)
	, ContextScriptNamesAndAssetPaths(InContextScriptNamesAndAssetPaths)
	, OwningScriptNameAndUsageText(InOwningScriptNameAndUsageText)
	, CompiledScriptNameAndAssetPath(InCompiledScriptNameAndAssetPath)
{
}

TSharedRef<FTokenizedMessage> FNiagaraMessageCompileEvent::GenerateTokenizedMessage() const
{
	EMessageSeverity::Type MessageSeverity = EMessageSeverity::Info;
	switch (CompileEvent.Severity) {
	case FNiagaraCompileEventSeverity::Error:
		MessageSeverity = EMessageSeverity::Error;
		break;
	case FNiagaraCompileEventSeverity::Warning:
		MessageSeverity = EMessageSeverity::Warning;
		break;
	case FNiagaraCompileEventSeverity::Display:
		MessageSeverity = EMessageSeverity::Info;
		break;
	// log is still treated like an info
	case FNiagaraCompileEventSeverity::Log:
		MessageSeverity = EMessageSeverity::Info;
		break;
	default:
		ensureMsgf(false, TEXT("Compile event severity type not handled!"));
		MessageSeverity = EMessageSeverity::Info;
		break;
	}
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(MessageSeverity);

	//Add message from compile event at start of message
	if (CompiledScriptNameAndAssetPath.IsSet())
	{
		//Compile event is from a script asset
		if (ContextScriptNamesAndAssetPaths.Num() == 0)
		{

			if (CompileEvent.PinGuid != FGuid())
			{
				Message->AddToken(FNiagaraCompileEventToken::Create(CompiledScriptNameAndAssetPath.GetValue().ScriptAssetPathString, FText::FromString(CompileEvent.Message), CompileEvent.NodeGuid, CompileEvent.PinGuid));
			}
			else if (CompileEvent.NodeGuid != FGuid())
			{
				Message->AddToken(FNiagaraCompileEventToken::Create(CompiledScriptNameAndAssetPath.GetValue().ScriptAssetPathString, FText::FromString(CompileEvent.Message), CompileEvent.NodeGuid));
			}
			else
			{
				Message->AddToken(FTextToken::Create(FText::FromString(CompileEvent.Message)));
			}
		}
		else
		{
			if (CompileEvent.PinGuid != FGuid())
			{
				Message->AddToken(FNiagaraCompileEventToken::Create(ContextScriptNamesAndAssetPaths.Last().ScriptAssetPathString, FText::FromString(CompileEvent.Message), CompileEvent.NodeGuid, CompileEvent.PinGuid));
			}
			else if (CompileEvent.NodeGuid != FGuid())
			{
				Message->AddToken(FNiagaraCompileEventToken::Create(ContextScriptNamesAndAssetPaths.Last().ScriptAssetPathString, FText::FromString(CompileEvent.Message), CompileEvent.NodeGuid));
			}
			else
			{
				Message->AddToken(FNiagaraCompileEventToken::Create(ContextScriptNamesAndAssetPaths.Last().ScriptAssetPathString, FText::FromString(CompileEvent.Message)));
			}
		}
	}
	else
	{
		//Compile event is from an emitter or system asset
		if (ContextScriptNamesAndAssetPaths.Num() == 0)
		{
			Message->AddToken(FTextToken::Create(FText::FromString(CompileEvent.Message)));
		}
		else if (CompileEvent.PinGuid != FGuid())
		{
			Message->AddToken(FNiagaraCompileEventToken::Create(ContextScriptNamesAndAssetPaths.Last().ScriptAssetPathString, FText::FromString(CompileEvent.Message), CompileEvent.NodeGuid, CompileEvent.PinGuid));
		}
		else if (CompileEvent.NodeGuid != FGuid())
		{
			Message->AddToken(FNiagaraCompileEventToken::Create(ContextScriptNamesAndAssetPaths.Last().ScriptAssetPathString, FText::FromString(CompileEvent.Message), CompileEvent.NodeGuid));
		}
		else
		{
			Message->AddToken(FNiagaraCompileEventToken::Create(ContextScriptNamesAndAssetPaths.Last().ScriptAssetPathString, FText::FromString(CompileEvent.Message)));
		}
	}

	//Now add the owning script name and usage if it is set
	if (OwningScriptNameAndUsageText.IsSet())
	{
		Message->AddToken(FTextToken::Create(OwningScriptNameAndUsageText.GetValue()));
	}

	//Finally add the context stack of the scripts that were passed through to get to the originating graph
	for (const FNiagaraScriptNameAndAssetPath& ScriptNameAndPath : ContextScriptNamesAndAssetPaths)
	{
		Message->AddToken(FNiagaraCompileEventToken::Create(ScriptNameAndPath.ScriptAssetPathString, FText::FromString(*ScriptNameAndPath.ScriptNameString)));
	}
	return Message;
}

FText FNiagaraMessageCompileEvent::GenerateMessageText() const
{
	if (OwningScriptNameAndUsageText.IsSet())
	{
		return FText::Format(LOCTEXT("MessageAndOwnerFormat", "{0}{1}"), FText::FromString(CompileEvent.Message), OwningScriptNameAndUsageText.GetValue());
	}
	else
	{
		return FText::FromString(CompileEvent.Message);
	}
}

FText FNiagaraMessageCompileEvent::GenerateMessageTitle() const
{
	EStackIssueSeverity StackIssueSeverity;
	switch (CompileEvent.Severity)
	{
	case FNiagaraCompileEventSeverity::Error:
		StackIssueSeverity = EStackIssueSeverity::Error;
		break;
	case FNiagaraCompileEventSeverity::Warning:
		StackIssueSeverity = EStackIssueSeverity::Warning;
		break;
	case FNiagaraCompileEventSeverity::Display:
		StackIssueSeverity = EStackIssueSeverity::Info;
		break;
	default:
		StackIssueSeverity = EStackIssueSeverity::Info;
		break;
	}

	if (!CompileEvent.ShortDescription.IsEmpty())
	{
		return FText::FromString(CompileEvent.ShortDescription);
	}
	else
	{
		return FNiagaraMessageUtilities::GetShortDescriptionFromSeverity(StackIssueSeverity);
	}
}

void FNiagaraMessageCompileEvent::GenerateLinks(TArray<FText>& OutLinkDisplayNames, TArray<FSimpleDelegate>& OutLinkNavigationActions) const
{
	// TODO: Do this without instantiating these tokens.
	if ((CompiledScriptNameAndAssetPath.IsSet() || ContextScriptNamesAndAssetPaths.Num() > 0) &&
		(CompileEvent.NodeGuid.IsValid() || CompileEvent.PinGuid.IsValid()))
	{
		FString AssetPath = CompiledScriptNameAndAssetPath.IsSet()
			? CompiledScriptNameAndAssetPath.GetValue().ScriptAssetPathString
			: ContextScriptNamesAndAssetPaths.Last().ScriptAssetPathString;

		TSharedPtr<FNiagaraCompileEventToken> NavigateToSourceToken;
		if (CompileEvent.PinGuid.IsValid())
		{
			NavigateToSourceToken = FNiagaraCompileEventToken::Create(AssetPath, FText::FromString(CompileEvent.Message), CompileEvent.NodeGuid, CompileEvent.PinGuid);
		}
		else if (CompileEvent.NodeGuid.IsValid())
		{
			NavigateToSourceToken = FNiagaraCompileEventToken::Create(AssetPath, FText::FromString(CompileEvent.Message), CompileEvent.NodeGuid);
		}

		if (NavigateToSourceToken.IsValid())
		{
			OutLinkDisplayNames.Add(LOCTEXT("NavigateToMessageSource", "Navigate to message source"));
			OutLinkNavigationActions.Add(FSimpleDelegate::CreateLambda([NavigateToSourceToken]()
				{
					NavigateToSourceToken->GetOnMessageTokenActivated().Execute(NavigateToSourceToken.ToSharedRef());
				}));
		}
	}

	for (const FNiagaraScriptNameAndAssetPath& ScriptNameAndPath : ContextScriptNamesAndAssetPaths)
	{
		OutLinkDisplayNames.Add(FText::Format(LOCTEXT("NavigateToAssetFormat", "Navigate to asset: {0}"), FText::FromString(*ScriptNameAndPath.ScriptNameString)));
		TSharedRef<FNiagaraCompileEventToken> NavigateToAssetToken = FNiagaraCompileEventToken::Create(ScriptNameAndPath.ScriptAssetPathString, FText::FromString(*ScriptNameAndPath.ScriptNameString));
		OutLinkNavigationActions.Add(FSimpleDelegate::CreateLambda([NavigateToAssetToken]()
			{
				NavigateToAssetToken->GetOnMessageTokenActivated().Execute(NavigateToAssetToken);
			}));
	}
}

FNiagaraMessageJobCompileEvent::FNiagaraMessageJobCompileEvent(
	const FNiagaraCompileEvent& InCompileEvent
	, const TWeakObjectPtr<const UNiagaraScript>& InOriginatingScriptWeakObjPtr
	, FGuid InCompiledScriptVersion
	, const TOptional<const FString>& InOwningScriptNameString
	, const TOptional<const FString>& InSourceScriptAssetPath
	)
	: CompileEvent(InCompileEvent)
	, OriginatingScriptWeakObjPtr(InOriginatingScriptWeakObjPtr)
	, CompiledScriptVersion(InCompiledScriptVersion)
	, OwningScriptNameString(InOwningScriptNameString)
	, SourceScriptAssetPath(InSourceScriptAssetPath)
{
}

TSharedRef<const INiagaraMessage> FNiagaraMessageJobCompileEvent::GenerateNiagaraMessage() const
{
	TArray<FGuid> ContextStackGuids = CompileEvent.StackGuids;

	if (OriginatingScriptWeakObjPtr.IsValid())
	{
		const UNiagaraScriptSourceBase* FunctionScriptSourceBase = OriginatingScriptWeakObjPtr->GetSource(CompiledScriptVersion);
		checkf(FunctionScriptSourceBase->IsA<UNiagaraScriptSource>(), TEXT("Script source for function call node is not assigned or is not of type UNiagaraScriptSource!"))
			const UNiagaraScriptSource* FunctionScriptSource = Cast<UNiagaraScriptSource>(FunctionScriptSourceBase);
		checkf(FunctionScriptSource, TEXT("Script source base was somehow not a derived type!"));
		const UNiagaraGraph* ScriptGraph = FunctionScriptSource->NodeGraph;
		checkf(ScriptGraph, TEXT("Function Script does not have a UNiagaraGraph!"));

		TArray<FNiagaraScriptNameAndAssetPath> ContextScriptNamesAndPaths;
		TOptional<const FNiagaraScriptNameAndAssetPath> CompiledScriptNameAndPath;
		TOptional<const FText> OwningScriptNameAndUsageText;
		TOptional<const FText> ScriptNameAndPathsGetterFailureReason;
		TOptional<const FString> OwningScriptNameStringCopy = OwningScriptNameString;
		TArray<FObjectKey> ContextObjectKeys;
		bool bSuccessfullyFoundScripts = RecursiveGetScriptNamesAndAssetPathsFromContextStack(ContextStackGuids, CompileEvent.NodeGuid, ScriptGraph, ContextScriptNamesAndPaths, OwningScriptNameStringCopy, ScriptNameAndPathsGetterFailureReason, ContextObjectKeys);

		if (bSuccessfullyFoundScripts == false)
		{
			//If we can't find the scripts in the context stack of the compile event, return a message with the error and ask for a recompile.
			return MakeShared<FNiagaraMessageText>(ScriptNameAndPathsGetterFailureReason.GetValue(), EMessageSeverity::Error, FNiagaraMessageTopics::CompilerTopicName);
		}

		const ENiagaraScriptUsage ScriptUsage = OriginatingScriptWeakObjPtr->GetUsage();

		if (OwningScriptNameStringCopy.IsSet())
		{
			FString ScriptAndUsage = OwningScriptNameStringCopy.GetValue();
			const TOptional<const FString> ScriptUsageInStackString = FNiagaraMessageManager::GetStringForScriptUsageInStack(ScriptUsage);
			if (ScriptUsageInStackString.IsSet())
			{
				ScriptAndUsage = ScriptAndUsage + FString(", ") + ScriptUsageInStackString.GetValue() + FString(", ");
			}
			OwningScriptNameAndUsageText = TOptional<const FText>(FText::FromString(ScriptAndUsage));
		}

		if (SourceScriptAssetPath.IsSet())
		{
			//If this compile event is from a script, set the compiled script name and asset path so the user can navigate to errors locally.
			CompiledScriptNameAndPath = TOptional<const FNiagaraScriptNameAndAssetPath>(FNiagaraScriptNameAndAssetPath(OriginatingScriptWeakObjPtr->GetName(), SourceScriptAssetPath.GetValue()));
		}
		return MakeShared<FNiagaraMessageCompileEvent>(CompileEvent, ContextScriptNamesAndPaths, OwningScriptNameAndUsageText, CompiledScriptNameAndPath, ContextObjectKeys);
	}
	//The originating script weak ptr is no longer valid, send an error message asking for recompile.
	FText MessageText = LOCTEXT("CompileEventMessageJobFail", "Cached info for compile event is out of date, recompile to get full info. Event: {0}");
	//Add in the message of the compile event for visibility on which compile events do not have info.
	MessageText = FText::Format(MessageText, FText::FromString(CompileEvent.Message));
	return MakeShared<FNiagaraMessageText>(MessageText, EMessageSeverity::Error, FNiagaraMessageTopics::CompilerTopicName);
}

bool FNiagaraMessageJobCompileEvent::RecursiveGetScriptNamesAndAssetPathsFromContextStack(
	TArray<FGuid>& InContextStackNodeGuids
	, FGuid NodeGuid
	, const UNiagaraGraph* InGraphToSearch
	, TArray<FNiagaraScriptNameAndAssetPath>& OutContextScriptNamesAndAssetPaths
	, TOptional<const FString>& OutEmitterName
	, TOptional<const FText>& OutFailureReason
	, TArray<FObjectKey>& OutContextNodeObjectKeys
	) const
{
	checkf(InGraphToSearch, TEXT("Failed to get a node graph to search!"));

	if (NodeGuid.IsValid())
	{
		TObjectPtr<UEdGraphNode> const* EventNodePtr = InGraphToSearch->Nodes.FindByPredicate([NodeGuid](UEdGraphNode* Node) { return Node->NodeGuid == NodeGuid; });
		if (EventNodePtr != nullptr)
		{
			OutContextNodeObjectKeys.AddUnique(FObjectKey(*EventNodePtr));
		}
	}

	if (InContextStackNodeGuids.Num() == 0)
	{
		//StackGuids arr has been cleared out which means we have walked the entire context stack.
		return true;
	}

	// Search in the current graph for a node with a GUID that matches a GUID in the list of Function Call and Emitter node GUIDs that define the context stack for a compile event
	auto FindNodeInGraphWithContextStackGuid = [&InGraphToSearch, &InContextStackNodeGuids]()->TOptional<UEdGraphNode*> {
		for (UEdGraphNode* GraphNode : InGraphToSearch->Nodes)
		{
			for (int i = 0; i < InContextStackNodeGuids.Num(); i++)
			{
				if (GraphNode->NodeGuid == InContextStackNodeGuids[i])
				{
					InContextStackNodeGuids.RemoveAt(i);
					return GraphNode;
				}
			}
		}
		return TOptional<UEdGraphNode*>();
	};

	TOptional<UEdGraphNode*> ContextNode = FindNodeInGraphWithContextStackGuid();
	if (ContextNode.IsSet())
	{
		// found a node in the current graph that has a GUID in the context list
		OutContextNodeObjectKeys.Add(FObjectKey(ContextNode.GetValue()));

		UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(ContextNode.GetValue());
		if (FunctionCallNode)
		{
			// node is a function call node, now get the Niagara Script assigned to this node, add a message token and recursively call into the graph of that script.
			UNiagaraScript* FunctionCallNodeAssignedScript = FunctionCallNode->FunctionScript;
			if (FunctionCallNodeAssignedScript == nullptr)
			{
				FText FailureReason = LOCTEXT("GenerateCompileEventMessage_FunctionCallNodeScriptNotFound", "Script for Function Call Node \"{0}\" not found!");
				OutFailureReason = FText::Format(FailureReason, FText::FromString(FunctionCallNode->GetFunctionName()));
				return false;
			}
			UNiagaraScriptSourceBase* FunctionScriptSourceBase = FunctionCallNode->GetFunctionScriptSource();
			if (FunctionScriptSourceBase == nullptr)
			{
				FText FailureReason = LOCTEXT("GenerateCompileEventMessage_FunctionCallNodeScriptSourceBaseNotFound", "Source Script for Function Call Node \"{0}\" not found!");
				OutFailureReason = FText::Format(FailureReason, FText::FromString(FunctionCallNode->GetFunctionName()));
				return false;
			}
			UNiagaraScriptSource* FunctionScriptSource = Cast<UNiagaraScriptSource>(FunctionScriptSourceBase);
			checkf(FunctionScriptSource, TEXT("Script source base was somehow not a derived type!"));

			UNiagaraGraph* FunctionScriptGraph = FunctionScriptSource->NodeGraph;
			if (FunctionScriptGraph == nullptr)
			{
				FText FailureReason = LOCTEXT("GenerateCompileEventMessage_FunctionCallNodeGraphNotFound", "Graph for Function Call Node \"{0}\" not found!");
				OutFailureReason = FText::Format(FailureReason, FText::FromString(FunctionCallNode->GetFunctionName()));
				return false;
			}
			OutContextScriptNamesAndAssetPaths.Add(FNiagaraScriptNameAndAssetPath(FunctionCallNodeAssignedScript->GetName(), FunctionCallNodeAssignedScript->GetPathName()));
			return RecursiveGetScriptNamesAndAssetPathsFromContextStack(InContextStackNodeGuids, NodeGuid, FunctionScriptGraph, OutContextScriptNamesAndAssetPaths, OutEmitterName, OutFailureReason, OutContextNodeObjectKeys);
		}

		UNiagaraNodeEmitter* EmitterNode = Cast<UNiagaraNodeEmitter>(ContextNode.GetValue());
		if (EmitterNode)
		{
			// node is an emitter node, now get the Emitter name, add a message token and recursively call into the graph of that emitter.
			UNiagaraScriptSource* EmitterScriptSource = EmitterNode->GetScriptSource();
			checkf(EmitterScriptSource, TEXT("Emitter Node does not have a Script Source!"));
			UNiagaraGraph* EmitterScriptGraph = EmitterScriptSource->NodeGraph;
			checkf(EmitterScriptGraph, TEXT("Emitter Script Source does not have a UNiagaraGraph!"));

			OutEmitterName = EmitterNode->GetEmitterUniqueName();
			return RecursiveGetScriptNamesAndAssetPathsFromContextStack(InContextStackNodeGuids, NodeGuid, EmitterScriptGraph, OutContextScriptNamesAndAssetPaths, OutEmitterName, OutFailureReason, OutContextNodeObjectKeys);
		}
		checkf(false, TEXT("Matching node is not a function call or emitter node!"));
	}
	FText FailureReason = LOCTEXT("CompileEventMessageGenerator_CouldNotFindMatchingNodeGUID", "Failed to walk the entire context stack, is this compile event out of date ? Event: '{0}'");
	OutFailureReason = FText::Format(FailureReason, FText::FromString(CompileEvent.Message));
	return false;
}

FNiagaraStackMessage::FNiagaraStackMessage()
{
}

FNiagaraStackMessage::FNiagaraStackMessage(const FText& InMessageText, const FText& InShortDescription,	ENiagaraMessageSeverity InSeverity, bool bInAllowDismissal, FGuid InGuid)
{
	MessageText = InMessageText;
	ShortDescription = InShortDescription;
	MessageSeverity = InSeverity;
	bAllowDismissal = bInAllowDismissal;
	Guid = InGuid;
}

TSharedRef<FTokenizedMessage> FNiagaraMessageText::GenerateTokenizedMessage() const
{
	return FTokenizedMessage::Create(MessageSeverity, MessageText);
}

FText FNiagaraMessageText::GenerateMessageText() const
{
	return MessageText;
}

FText FNiagaraMessageText::GenerateMessageTitle() const
{
	return ShortDescription;
}

const uint32 INiagaraMessage::GetMessageTopicBitflag() const
{
	if (MessageTopicBitflag == 0)
	{
		MessageTopicBitflag = FNiagaraMessageManager::Get()->GetMessageTopicBitflag(GetMessageTopic());
	}
	return MessageTopicBitflag;
}

void UNiagaraMessageDataText::Init(const FText& InMessageText, const ENiagaraMessageSeverity InMessageSeverity, const FName& InTopicName)
{
	MessageText = InMessageText;
	MessageSeverity = InMessageSeverity;
	TopicName = InTopicName;
}

void UNiagaraMessageDataText::Init(const FText& InMessageText, const FText& InShortDescription,	const ENiagaraMessageSeverity InMessageSeverity, const FName& InTopicName)
{
	MessageText = InMessageText;
	ShortDescription = InShortDescription;
	MessageSeverity = InMessageSeverity;
	TopicName = InTopicName;
}

TSharedRef<const INiagaraMessage> UNiagaraMessageDataText::GenerateNiagaraMessage(const FGenerateNiagaraMessageInfo& InGenerateInfo) const
{
	const TArray<FLinkNameAndDelegate>& Links = InGenerateInfo.GetLinks();
	if (Links.Num() > 0)
	{
		return MakeShared<const FNiagaraMessageTextWithLinks>(MessageText, ShortDescription, EMessageSeverity::Type(MessageSeverity), TopicName, InGenerateInfo.GetDismissHandler(), Links, InGenerateInfo.GetAssociatedObjectKeys());
	}
	return MakeShared<const FNiagaraMessageText>(MessageText, ShortDescription, EMessageSeverity::Type(MessageSeverity), TopicName, InGenerateInfo.GetDismissHandler(), InGenerateInfo.GetAssociatedObjectKeys());
}

TSharedRef<FTokenizedMessage> FNiagaraMessageTextWithLinks::GenerateTokenizedMessage() const
{
	TSharedRef<FTokenizedMessage> TokenizedMessage = FNiagaraMessageText::GenerateTokenizedMessage();
	for (const FLinkNameAndDelegate& LinkNameAndDelegate : Links)
	{
		TSharedRef<FActionToken> LinkActionToken = FActionToken::Create(LinkNameAndDelegate.LinkNameText, FText(), FOnActionTokenExecuted(LinkNameAndDelegate.LinkDelegate));
		TokenizedMessage->AddToken(LinkActionToken);
	}
	return TokenizedMessage;
}

void FNiagaraMessageTextWithLinks::GenerateLinks(TArray<FText>& OutLinkDisplayNames, TArray<FSimpleDelegate>& OutLinkNavigationActions) const
{
	for (const FLinkNameAndDelegate& LinkInfo : Links)
	{
		OutLinkDisplayNames.Add(LinkInfo.LinkNameText);
		OutLinkNavigationActions.Add(LinkInfo.LinkDelegate);
	}
}

#undef LOCTEXT_NAMESPACE //NiagaraMessages

