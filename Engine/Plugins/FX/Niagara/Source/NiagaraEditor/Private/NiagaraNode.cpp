// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNode.h"
#include "NiagaraConstants.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraScriptSource.h"

#include "EdGraphSchema_Niagara.h"
#include "GraphEditAction.h"
#include "Widgets/SNiagaraGraphNode.h"
#include "Misc/SecureHash.h"
#include "ToolMenuSection.h"
#include "ToolMenu.h"
#include "GraphEditorActions.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "ScopedTransaction.h"
#include "Misc/Guid.h"
#include "UObject/TextProperty.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNode)

#define LOCTEXT_NAMESPACE "NiagaraNode"

UNiagaraNode::UNiagaraNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNiagaraNode::PostLoad()
{
	Super::PostLoad();

	if (ChangeId.IsValid() == false)
	{
		ChangeId = FGuid::NewGuid();
	}

	if (GIsEditor && HasAllFlags(RF_Transactional) == false)
	{
		SetFlags(RF_Transactional);
	}
	
	for (UEdGraphPin* Pin : Pins)
	{
		if(Pin == nullptr)
		{
			UNiagaraGraph* OwningGraph = GetNiagaraGraph();
			FString AssetPath = OwningGraph ? OwningGraph->GetPathName() : "Undetermined";
			FString NodeName = GetNodeTitle(ENodeTitleType::FullTitle).ToString();
			UE_LOGF(LogNiagaraEditor, Log, "Pin of node %ls in asset %ls is nullptr!", *NodeName, *AssetPath);
			continue;
		}
		if(Pin->LinkedTo.Num() > 0)
		{
			for(UEdGraphPin* LinkedPin : Pin->LinkedTo)
			{
				if(LinkedPin == nullptr)
				{
					UNiagaraGraph* OwningGraph = GetNiagaraGraph();

					FString ScriptUsageName = TEXT("Unknown Usage");
					FString AssetPath = "Undetermined";
					FString PinName = Pin->PinName.ToString();
					FString NodeName = "Undetermined";

					if (UNiagaraScriptSource* ScriptSource = GetSource())
					{
						if (UNiagaraScript* Script = ScriptSource->GetTypedOuter<UNiagaraScript>())
						{
							const UEnum* UsageEnum = StaticEnum<ENiagaraScriptUsage>();
							ScriptUsageName = UsageEnum->GetDisplayNameTextByValue(int64(Script->GetUsage())).ToString();
						}
					}

					if ( OwningGraph )
					{
						AssetPath = OwningGraph->GetPathName();
					}

					FString OwningNodeDetailed;
					if (UEdGraphNode* OwningNode = Pin->GetOwningNode())
					{
						NodeName = FString::Printf(TEXT("%s:%s"), *GetNameSafe(OwningNode->GetClass()), *OwningNode->GetNodeTitle(ENodeTitleType::FullTitle).ToString());

						OwningNodeDetailed.Appendf(TEXT("\nOwningNodePath - %s"), *GetFullNameSafe(OwningNode));
						for (int32 i=0; i < OwningNode->Pins.Num(); ++i)
						{
							if (UEdGraphPin* OwningNodePin = OwningNode->Pins[i])
							{
								OwningNodeDetailed.Appendf(TEXT("\nPin[%d] - %s | %s"), i, *OwningNodePin->GetDisplayName().ToString(), *OwningNodePin->GetFName().ToString());
							}
							else
							{
								OwningNodeDetailed.Appendf(TEXT("\nPin[%d] - nullptr"), i);
							}
						}
					}

					UE_LOGF(LogNiagaraEditor, Warning, "Connected pin of Pin '%ls' in '%ls' of node '%ls' in asset '%ls' is nullptr!%ls", *PinName, *ScriptUsageName, *NodeName, *AssetPath, *OwningNodeDetailed);
				}
			}
		}
		
		const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
		check(Schema);

		UEdGraphSchema_Niagara::ConvertIllegalPinsInPlace(Pin);

		const FNiagaraTypeDefinition PinType = Schema->PinToTypeDefinition(Pin);
		if (PinType.GetEnum() == FNiagaraTypeDefinition::GetCoordinateSpaceEnum())
		{
			int32 EnumIndex = PinType.GetEnum()->GetIndexByNameString(Pin->DefaultValue, EGetByNameFlags::None);
			if (EnumIndex != INDEX_NONE )
			{
				Pin->DefaultValue = PinType.GetEnum()->GetNameStringByIndex(EnumIndex);
			}
		}
	}
}

bool UNiagaraNode::NestedPropertiesAppendCompileHash(const void* Container, const UStruct* Struct, EFieldIteratorFlags::SuperClassFlags IteratorFlags, const FString& BaseName, FNiagaraCompileHashVisitor* InVisitor) const
{
	return FNiagaraEditorUtilities::NestedPropertiesAppendCompileHash(Container, Struct, IteratorFlags, BaseName, InVisitor);
}

bool UNiagaraNode::PODPropertyAppendCompileHash(const void* Container, FProperty* Property, const FString& PropertyName, FNiagaraCompileHashVisitor* InVisitor) const
{
	return FNiagaraEditorUtilities::PODPropertyAppendCompileHash(Container, Property, PropertyName, InVisitor);
}


bool UNiagaraNode::GenerateCompileHashForClassMembers(const UClass* InClass, FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!NestedPropertiesAppendCompileHash(static_cast<const void*>(this), InClass, EFieldIteratorFlags::ExcludeSuper, InClass->GetName(), InVisitor))
	{
		return false;
	}
	return true;
}

bool UNiagaraNode::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (NiagaraNodeAppendCompileHash(InVisitor))
	{
		UClass* Class = GetClass();
		UClass* NiagaraNodeClass = UNiagaraNode::StaticClass();

		while (Class != NiagaraNodeClass && Class != nullptr)
		{
			if (!GenerateCompileHashForClassMembers(Class, InVisitor))
			{
				return false;
			}

			Class = Class->GetSuperClass();
		}
		return true;
	}
	return false;
}

bool UNiagaraNode::PinAppendCompileHash(const UEdGraphPin* InPin, FNiagaraCompileHashVisitor* InVisitor) const
{
#if WITH_EDITORONLY_DATA
	InVisitor->UpdateReference(TEXT("\tPinId"), InPin);
	InVisitor->UpdateString(TEXT("\tName"), InPin->GetName());
	InVisitor->UpdatePOD(TEXT("\tDirection"), InPin->Direction);
	
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	check(Schema);

	FNiagaraTypeDefinition PinType = Schema->PinToTypeDefinition(InPin);
	PinType.AppendCompileHash(InVisitor);


	if (InPin->LinkedTo.Num() > 0)
	{
		for (int32 i = 0; i < InPin->LinkedTo.Num(); i++)
		{
			InVisitor->UpdateReference(TEXT("\tLinkedTo"), InPin->LinkedTo[i]);
		}
	}
	else
	{
		InVisitor->UpdateString(TEXT("\tDefaultValue"), InPin->DefaultValue);
		InVisitor->UpdateString(TEXT("\tAutogeneratedDefaultValue"), InPin->AutogeneratedDefaultValue);
		bool bDefaultObj = InPin->DefaultObject != nullptr;
		InVisitor->UpdatePOD(TEXT("\tDefaultObject"), bDefaultObj);
		if (InPin->DefaultObject)
		{
			UE_LOGF(LogNiagaraEditor, Warning, "AppendCompileHash can't handle default objects properly, please investigate!");
		}
	}

	for (int32 i = 0; i < InPin->SubPins.Num(); i++)
	{
		PinAppendCompileHash(InPin->SubPins[i], InVisitor);
	}
	
	if (InPin->ParentPin != nullptr)
	{
		InVisitor->UpdateReference(TEXT("\tParentPin"), InPin->ParentPin);
	}
	return true;
#else
	return false;
#endif
}

bool UNiagaraNode::NiagaraNodeAppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
#if WITH_EDITORONLY_DATA
	UClass* Class = GetClass();
	InVisitor->UpdateString(TEXT("Classname"), Class->GetName());
	InVisitor->UpdateReference(TEXT("Node"), this);
	// DO NOT include the name in the hash as it makes copy/paste duplicates not work out b/c the names internally will be incremented by a digit. 
	// The state and iteration occurrence should be enough to uniquely identify.
	//InVisitor->UpdateString(TEXT("NodeName"), GetName()); 
	ENodeEnabledState NodeEnabledState = GetDesiredEnabledState();
	InVisitor->UpdatePOD(TEXT("EnabledState"), NodeEnabledState);

	for (const UEdGraphPin* Pin : Pins)
	{
		if (!PinAppendCompileHash(Pin, InVisitor))
		{
			return false;
		}
	}

	return true;
#else
	return false;
#endif
}

bool UNiagaraNode::SetPinDefaultToTypeDefaultIfUnset(UEdGraphPin* InPin)
{
	//UE_LOGF(LogNiagaraEditor, Warning, "SetPinDefaultToTypeDefaultIfUnset");

	if (InPin->DefaultValue.Len() != 0)
		return true;

	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraTypeDefinition NiagaraType = Schema->PinToTypeDefinition(InPin);
	bool bNeedsValue = NiagaraType.IsDataInterface() == false;
	FNiagaraVariable Var = Schema->PinToNiagaraVariable(InPin, bNeedsValue);

	FString PinDefaultValue;
	if (Schema->TryGetPinDefaultValueFromNiagaraVariable(Var, PinDefaultValue))
	{
		InPin->DefaultValue = PinDefaultValue;
		InPin->AutogeneratedDefaultValue = PinDefaultValue;
		return true;
	}

	return bNeedsValue;
}


bool UNiagaraNode::ReallocatePins(bool bMarkNeedsResynchronizeOnChange)
{
	Modify();
	
	// Move the existing pins to a saved array
	TArray<UEdGraphPin*> OldPins(Pins);
	Pins.Reset();

	// Recreate the new pins
	AllocateDefaultPins();

	// Determine if the pins are the same as they were previously...
	bool bAllSame = OldPins.Num() == Pins.Num();

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	
	// Copy the old pin data and remove it.
	for (int32 OldPinIndex = 0; OldPinIndex < OldPins.Num(); ++OldPinIndex)
	{
		UEdGraphPin* OldPin = OldPins[OldPinIndex];

		// When matching pins, use the pin id if either pin id is valid, otherwise match by name.
		// we typically don't allow orphaned pins to find a match, unless types match each other again
		auto PinMatchPredicateGuid = [&](UEdGraphPin* Pin) 
		{
			// early out conditions for when matching up is not wanted
			if (Pin->Direction != OldPin->Direction)
			{
				return false;
			}
			
			bool bPinsHaveIds = Pin->PersistentGuid.IsValid() || OldPin->PersistentGuid.IsValid();
			bool bAllowPin = OldPin->bOrphanedPin == false || OldPin->PinType == Pin->PinType;
			return bAllowPin && ((bPinsHaveIds && Pin->PersistentGuid == OldPin->PersistentGuid) || (bPinsHaveIds == false && Pin->PinName == OldPin->PinName));
		};

		auto PinMatchPredicateName = [&](UEdGraphPin* Pin)
		{
			// early out conditions for when matching up is not wanted
			if (Pin->Direction != OldPin->Direction || Pin->PinName != OldPin->PinName)
			{
				return false;
			}

			if (OldPin->PinType == Pin->PinType)
			{
				return true;
			}

			// Allow numeric pins and wildcard pins to match any type.
			FNiagaraTypeDefinition NewType = Schema->PinTypeToTypeDefinition(Pin->PinType);
			return NewType == FNiagaraTypeDefinition::GetGenericNumericDef() || NewType == FNiagaraTypeDefinition::GetWildcardDef();
		};

		OldPin->Modify();
		UEdGraphPin** MatchingNewPin = Pins.FindByPredicate(PinMatchPredicateGuid);
		if (MatchingNewPin == nullptr)
		{
			MatchingNewPin = Pins.FindByPredicate(PinMatchPredicateName);
		}

		if (MatchingNewPin)
		{
			// If the pin types don't match, CopyPersistentDataFromOldPin could very well overwrite our Matching pin with bad data.
			// Let's cache it for now and reset it after copying the other relevant data off the old pin.
			FString DefaultValue;
			FString AutogeneratedDefaultValue;
			class UObject* DefaultObject = nullptr;
			FText DefaultTextValue;
			bool bTypeMismatch = (*MatchingNewPin)->PinType != OldPin->PinType;

			FNiagaraTypeDefinition OldPinNiagaraType = Schema->PinToTypeDefinition(OldPin);
			FNiagaraTypeDefinition NewPinNiagaraType = Schema->PinToTypeDefinition(*MatchingNewPin);

			bool bRetainOldTypeDueToNumerics = OldPinNiagaraType != FNiagaraTypeDefinition::GetGenericNumericDef() && NewPinNiagaraType == FNiagaraTypeDefinition::GetGenericNumericDef();
			
			if (bTypeMismatch && !bRetainOldTypeDueToNumerics)
			{
				DefaultValue = (*MatchingNewPin)->DefaultValue;
				DefaultObject = (*MatchingNewPin)->DefaultObject;
				DefaultTextValue = (*MatchingNewPin)->DefaultTextValue;
			}
			
			// This copies the existing default values, pin linkages, advanced pin view, pin splitting, etc.
			(*MatchingNewPin)->MovePersistentDataFromOldPin(*OldPin);
			
			// Somehow this pin was considered orphaned before, but now exists, so it is orphaned no longer.
			if (OldPin->bOrphanedPin)
			{
				OldPin->bOrphanedPin = false;
				UE_LOGF(LogNiagaraEditor, Log, "Pin \"%ls\" in node \"%ls\" was orphaned, but is now matched. De-orphaning.", *OldPin->GetName(), *GetFullName());
			}

			// The prior call would have clobbered our default values, which causes a crash down the line when we attempt to compile.
			// This resets to the default values prior to copying over the persistent data.
			// @TODO Make this try to preserve as much of the old default values as possible.
			// @TODO Should we push this up to CopyPersistentDataFromOldPin globally?
			if (bTypeMismatch && !bRetainOldTypeDueToNumerics)
			{
				(*MatchingNewPin)->DefaultValue = DefaultValue;
				(*MatchingNewPin)->DefaultObject = DefaultObject;
				(*MatchingNewPin)->DefaultTextValue = DefaultTextValue;
				bAllSame = false;
			}
			else if (bTypeMismatch && bRetainOldTypeDueToNumerics)
			{
				(*MatchingNewPin)->PinType = OldPin->PinType;
			}
			
		}
		else
		{
			bAllSame = false;

			// if the pin wasn't already marked orphaned before (from type conversions etc.), check if we need to mark it as such
			if(OldPin->bOrphanedPin == false)
			{
				if ((OldPin->bDefaultValueIsIgnored == false && OldPin->DefaultValue != OldPin->AutogeneratedDefaultValue) || OldPin->LinkedTo.Num() > 0)
				{
					// If an old pin is not reused mark it as orphaned so that it's value and connections aren't lost.
					OldPin->bOrphanedPin = true;
				}
			}
		}
	}

	for (UEdGraphPin* OldPin : OldPins)
	{
		// we exclude pins from being added as orphaned pins if they were set to pending kill already
		if (OldPin->bOrphanedPin && !OldPin->IsPendingKill())
		{
			// Add orphaned pins back in at the end so that the user can fix them.
			Pins.Add(OldPin);
		}
		else
		{
			OldPin->MarkAsGarbage();
		}
	}

	OnPostSynchronizationInReallocatePins();

	//GetGraph()->NotifyGraphChanged();
	if (bMarkNeedsResynchronizeOnChange && !bAllSame)
	{
		MarkNodeRequiresSynchronization(__FUNCTION__, true);
	}
	else
	{
		// Even if we're not marking the graph as needing sychronization we still need to let other listeners,
		// such as the UI, know that the graph has changed.
		GetNiagaraGraph()->NotifyGraphNeedsRecompile();
		VisualsChangedDelegate.Broadcast(this);
	}
	return bAllSame;
}

int32 UNiagaraNode::CompileInputPin(FTranslator* Translator, UEdGraphPin* Pin) const
{
	return Translator->CompileInputPin(Pin);
}

bool UNiagaraNode::IsValidPinToCompile(UEdGraphPin* Pin) const 
{ 
	return Pin->bOrphanedPin == false;
}

bool UNiagaraNode::CompileInputPins(FTranslator* Translator, TArray<int32>& OutCompiledInputs) const
{
	bool bError = false;
	
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	for (int32 i = 0; i < InputPins.Num(); ++i)
	{
		UEdGraphPin* Pin = InputPins[i];
		if (!IsValidPinToCompile(Pin))
		{
			OutCompiledInputs.Add(INDEX_NONE);
			continue;
		}

		check(Pin->Direction == EGPD_Input);
		int32 Result = CompileInputPin(Translator, Pin);
		if (Result == INDEX_NONE)
		{
			bError = true;
			Translator->Error(FText::Format(LOCTEXT("CompileInputPinErrorFormat", "Error compiling Pin"), Pin->PinFriendlyName), this, Pin);
		}

		OutCompiledInputs.Add(Result);
	}
	return bError;
}

void UNiagaraNode::RequestNewPinType(UEdGraphPin* PinToChange, FNiagaraTypeDefinition NewType)
{
	FScopedTransaction Transaction(LOCTEXT("ChangePin", "Changed Pin Type"));
	Modify();
	PinToChange->Modify();

	if (OnNewPinTypeRequested(PinToChange, NewType))
	{
		PinTypeChanged(PinToChange);
	}
	else
	{
		Transaction.Cancel();
	}
}

void UNiagaraNode::PostPlacedNewNode()
{
	if (ChangeId.IsValid() == false)
	{
		MarkNodeRequiresSynchronization(__FUNCTION__, false); // The add will have notified us anyway
	}
}

void UNiagaraNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	Super::AutowireNewNode(FromPin);

	if (FromPin != nullptr)
	{
		const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
		check(Schema);
		
		//ENiagaraCompoundType FromType = Schema->GetPinDataType(FromPin);

		//Find first of this nodes pins with the right type and direction.
		UEdGraphPin* FirstPinOfSameType = NULL;
		EEdGraphPinDirection DesiredDirection = FromPin->Direction == EGPD_Output ? EGPD_Input : EGPD_Output;
		for (UEdGraphPin* Pin : Pins)
		{
			//ENiagaraCompoundType ToType = Schema->GetPinDataType(Pin);
			if (Pin->Direction == DesiredDirection)
			{
				const FPinConnectionResponse Response = Schema->CanCreateConnection(FromPin, Pin);
				if (Response.Response != ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW) //-V1051
				{
					FirstPinOfSameType = Pin;
					break;
				}
			}			
		}

		if (FirstPinOfSameType && GetSchema()->TryCreateConnection(FromPin, FirstPinOfSameType))
		{
			if (UEdGraphNode* FromPinNode = FromPin->GetOwningNodeUnchecked())
			{
				FromPinNode->NodeConnectionListChanged();
			}
		}
	}
}

bool UNiagaraNode::ConvertNumericPinToType(UEdGraphPin* InGraphPin, FNiagaraTypeDefinition TypeDef)
{
	int32 PinIndex = GetPinIndex(InGraphPin);
	if (PinIndex == -1)
	{
		return false;
	}

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	check(Schema);
	FEdGraphPinType PinType = Schema->TypeDefinitionToPinType(TypeDef);
	if (!Schema->PinTypesValidForNumericConversion(PinType, InGraphPin->PinType))
	{
		return false;
	}

	InGraphPin->Modify();
	InGraphPin->PinType = PinType;
	InGraphPin->ResetDefaultValue();
	PinTypeChanged(InGraphPin);
	ReallocatePins();

	return true;
}

bool UNiagaraNode::GetIsPinRenamePending(const UEdGraphPin* Pin)
{
	return PinsGuidsWithRenamePending.Contains(Pin->PersistentGuid);
}

void UNiagaraNode::SetIsPinRenamePending(const UEdGraphPin* Pin, bool bInIsRenamePending)
{
	if (bInIsRenamePending)
	{
		PinsGuidsWithRenamePending.AddUnique(Pin->PersistentGuid);
	}
	else
	{
		PinsGuidsWithRenamePending.Remove(Pin->PersistentGuid);
	}
}

bool UNiagaraNode::IsParameterMapPin(const UEdGraphPin* Pin) const
{
	return	UEdGraphSchema_Niagara::PinToTypeDefinition(Pin) == FNiagaraTypeDefinition::GetParameterMapDef();
}

TSharedPtr<SGraphNode> UNiagaraNode::CreateVisualWidget() 
{
	return SNew(SNiagaraGraphNode, this);
}


void UNiagaraNode::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	FText Text;
	if (GetTooltipTextForKnownPin(Pin, Text))
	{
		HoverTextOut = Text.ToString();
		return;
	}
	
	const UNiagaraGraph* NiagaraGraph = Cast<UNiagaraGraph>(GetGraph());
	if (NiagaraGraph)
	{
		const UEdGraphSchema_Niagara* Schema = Cast<UEdGraphSchema_Niagara>(NiagaraGraph->GetSchema());
		if (Schema)
		{
			FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(&Pin);
			if (Pin.PinToolTip.IsEmpty())
			{
				Text = FText::Format(LOCTEXT("PinHoverTooltip", "Name: \"{0}\"\nType: {1}"),
                FText::FromName(Pin.PinName),
                TypeDef.GetNameText());
			}
			else
			{
				Text = FText::Format(LOCTEXT("PinHoverTooltipFromPin", "{0}\n\nType: {1}"),
                FText::FromString(Pin.PinToolTip),
                TypeDef.GetNameText());
			}			
			HoverTextOut = Text.ToString();
		}
	}
}

void UNiagaraNode::AddWidgetsToInputBox(TSharedPtr<SVerticalBox> InputBox)
{

}

void UNiagaraNode::AddWidgetsToOutputBox(TSharedPtr<SVerticalBox> OutputBox)
{
	
}

void UNiagaraNode::GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{	
	{
		FToolMenuSection& Section = Menu->AddSection("Alignment");
		Section.AddSubMenu(
			"Alignment",
			LOCTEXT("AlignmentHeader", "Alignment"),
			FText(),
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			{
				FToolMenuSection& SubMenuSection = InMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
			}

			{
				FToolMenuSection& SubMenuSection = InMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
			}
		}));
	}
}

bool UNiagaraNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA<UEdGraphSchema_Niagara>();
}

TSharedRef<SWidget> UNiagaraNode::CreateTitleRightWidget()
{
	return SNullWidget::NullWidget;
}

void UNiagaraNode::MarkNodeRequiresSynchronization(FString Reason, bool bRaiseGraphNeedsRecompile)
{
	Modify();
	ChangeId = FGuid::NewGuid();
	//UE_LOGF(LogNiagaraEditor, Verbose, "Node %ls was marked requires synchronization.  Reason: %ls", *GetPathName(), *Reason);

	if (bRaiseGraphNeedsRecompile)
	{
		if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(GetGraph()))
		{
			Graph->NotifyGraphNeedsRecompile();
		}
	}

	VisualsChangedDelegate.Broadcast(this);
}

void UNiagaraNode::ForceChangeId(const FGuid& InId, bool bRaiseGraphNeedsRecompile)
{
	Modify();
	ChangeId = InId;

	if (bRaiseGraphNeedsRecompile)
	{
		UNiagaraGraph* Graph = GetNiagaraGraph();
		Graph->NotifyGraphNeedsRecompile();
	}
}

void UNiagaraNode::PinDefaultValueChanged(UEdGraphPin* Pin) 
{
	if (Pin->bOrphanedPin && Pin->DefaultValue == Pin->AutogeneratedDefaultValue)
	{
		RemovePin(Pin);
	}
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
	Super::PinDefaultValueChanged(Pin);
}

void UNiagaraNode::OnRenameNode(const FString& NewName)
{
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
	Super::OnRenameNode(NewName);
}

void UNiagaraNode::OnPinRemoved(UEdGraphPin* InRemovedPin)
{
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
	Super::OnPinRemoved(InRemovedPin);
}

void UNiagaraNode::NodeConnectionListChanged()
{
	// Cache the pins so that we're not modifying the collection while iterating.
	TArray<UEdGraphPin*> CurrentPins = Pins;
	for (UEdGraphPin* Pin : CurrentPins)
	{
		if (Pin->bOrphanedPin && Pin->LinkedTo.Num() == 0 && Pin->DefaultValue == Pin->AutogeneratedDefaultValue)
		{
			RemovePin(Pin);
		}
	}
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
	Super::NodeConnectionListChanged();
}


void UNiagaraNode::PinConnectionListChanged(UEdGraphPin* Pin) 
{
	if (Pin->bOrphanedPin && Pin->LinkedTo.Num() == 0)
	{
		RemovePin(Pin);
	}
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
	Super::PinConnectionListChanged(Pin);
}

void UNiagaraNode::PinTypeChanged(UEdGraphPin* Pin) 
{
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
}

const UNiagaraGraph* UNiagaraNode::GetNiagaraGraph()const
{
	return CastChecked<UNiagaraGraph>(GetGraph());
}

UNiagaraGraph* UNiagaraNode::GetNiagaraGraph()
{
	return CastChecked<UNiagaraGraph>(GetGraph());
}

UNiagaraScriptSource* UNiagaraNode::GetSource()const
{
	return GetNiagaraGraph()->GetSource();
}

void UNiagaraNode::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	Translator->Error(FText::FromString("Unimplemented Node!"), this, nullptr);
}

UEdGraphPin* UNiagaraNode::GetInputPin(int32 InputIndex) const
{
	for (int32 PinIndex = 0, FoundInputs = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Input)
		{
			if (InputIndex == FoundInputs)
			{
				return Pins[PinIndex];
			}
			else
			{
				FoundInputs++;
			}
		}
	}

	return NULL;
}

bool UNiagaraNode::CanAddToGraph(UNiagaraGraph* TargetGraph, FString& OutErrorMsg) const
{
	if (TargetGraph == nullptr)
	{
		OutErrorMsg = LOCTEXT("NiagaraNodeInvalidGraph", "Target Graph is invalid.").ToString();
		return false;
	}

	return true;
}

void UNiagaraNode::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	if (bRecursive)
	{
		OutHistory.VisitInputPins(this, bFilterForCompilation);
	}
}

UEdGraphPin* UNiagaraNode::GetOutputPin(int32 OutputIndex) const
{
	for (int32 PinIndex = 0, FoundOutputs = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Output)
		{
			if (OutputIndex == FoundOutputs)
			{
				return Pins[PinIndex];
			}
			else
			{
				FoundOutputs++;
			}
		}
	}

	return NULL;
}

UEdGraphPin* UNiagaraNode::GetPinByPersistentGuid(const FGuid& InPersistentGuid) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (InPersistentGuid == Pin->PersistentGuid)
		{
			return Pin;
		}
	}

	return nullptr;
}

void UNiagaraNode::NumericResolutionByPins(const UEdGraphSchema_Niagara* Schema, TArrayView<UEdGraphPin* const> InputPins, TArrayView<UEdGraphPin* const> OutputPins, 
	bool bFixInline, TMap<TPair<FGuid, UEdGraphNode*>, FNiagaraTypeDefinition>* PinCache)
{
	TArray<FNiagaraTypeDefinition> InputTypes;
	TArray<FNiagaraTypeDefinition> OutputTypes;

	TArray<FNiagaraTypeDefinition> NonNumericInputs;
	for (UEdGraphPin* InputPin : InputPins)
	{
		FNiagaraTypeDefinition InputPinType = Schema->PinToTypeDefinition(InputPin);
		if (InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType ||
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType)
		{

			// If the input pin is the generic numeric type set it to the type of the linked output pin which should have been processed already.
			if (InputPinType == FNiagaraTypeDefinition::GetGenericNumericDef() && InputPin->LinkedTo.Num() == 1)
			{
				UEdGraphPin* InputPinLinkedPin = InputPin->LinkedTo[0];
				FNiagaraTypeDefinition InputPinLinkedPinType = Schema->PinToTypeDefinition(InputPinLinkedPin);
				if (InputPinLinkedPinType.IsValid())
				{
					if (InputPinLinkedPinType == FNiagaraTypeDefinition::GetGenericNumericDef() && PinCache)
					{
						FNiagaraTypeDefinition* FoundDef = PinCache->Find(TPair<FGuid, UEdGraphNode*>(InputPinLinkedPin->PinId, InputPinLinkedPin->GetOwningNode()));
						if (FoundDef && FoundDef->IsValid())
						{
							InputPinLinkedPinType = *FoundDef;
						}
					}

					// Only update the input pin type if the linked pin type is valid.
					FEdGraphPinType PinType = Schema->TypeDefinitionToPinType(InputPinLinkedPinType);
					if (bFixInline)
					{
						InputPin->PinType = PinType;
					}
					InputPinType = InputPinLinkedPinType;
				}
			}

			if (InputPinType != FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				NonNumericInputs.Add(InputPinType);
			}
		}
		InputTypes.Add(InputPinType);
	}

	// Fix up numeric outputs based on the inputs.
	for (UEdGraphPin* OutputPin : OutputPins)
	{
		FNiagaraTypeDefinition OutputPinType = Schema->PinToTypeDefinition(OutputPin);
		ENiagaraNumericOutputTypeSelectionMode NumericMode = GetNumericOutputTypeSelectionMode();
		if (NonNumericInputs.Num() > 0 && NumericMode != ENiagaraNumericOutputTypeSelectionMode::None)
		{
			FNiagaraTypeDefinition OutputNumericType;
			if (NumericMode == ENiagaraNumericOutputTypeSelectionMode::Custom)
			{
				OutputNumericType = ResolveCustomNumericType(NonNumericInputs);
			}
			else
			{
				OutputNumericType = FNiagaraTypeDefinition::GetNumericOutputType(NonNumericInputs, NumericMode);
			}
			if (OutputNumericType != FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				if (OutputPinType == FNiagaraTypeDefinition::GetGenericNumericDef())
				{
					FEdGraphPinType PinType = Schema->TypeDefinitionToPinType(OutputNumericType);
					if (bFixInline)
					{
						OutputPin->PinType = PinType;
					}
					OutputPinType = OutputNumericType;
				}
			}
		}
		OutputTypes.Add(OutputPinType);
	}


	if (PinCache)
	{
		int32 j;
		for (j = 0; j < InputPins.Num(); j++)
		{
			PinCache->Add(TPair<FGuid, UEdGraphNode*>(InputPins[j]->PinId, InputPins[j]->GetOwningNode()), InputTypes[j]);
		}
		for (j = 0; j < OutputPins.Num(); j++)
		{
			PinCache->Add(TPair<FGuid, UEdGraphNode*>(OutputPins[j]->PinId, OutputPins[j]->GetOwningNode()), OutputTypes[j]);
		}
	}
}

FNiagaraTypeDefinition UNiagaraNode::ResolveCustomNumericType(const TArray<FNiagaraTypeDefinition>&) const
{
	checkf(false, TEXT("Not implemented for node type"));
	return FNiagaraTypeDefinition::GetFloatDef();
}


void UNiagaraNode::ResolveNumerics(const UEdGraphSchema_Niagara* Schema, bool bSetInline, TMap<TPair<FGuid, UEdGraphNode*>, FNiagaraTypeDefinition>* PinCache)
{
	// Fix up numeric input pins and keep track of numeric types to decide the output type.
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);
	NumericResolutionByPins(Schema, InputPins, OutputPins, bSetInline, PinCache);
}

void UNiagaraNode::GetWildcardPinHoverConnectionTextAddition(const UEdGraphPin* WildcardPin, const UEdGraphPin* OtherPin, ECanCreateConnectionResponse ConnectionResponse, FString& OutString) const
{
	// OutString can be modified depending on use case and connection response
}

ENiagaraNumericOutputTypeSelectionMode UNiagaraNode::GetNumericOutputTypeSelectionMode() const
{
	return ENiagaraNumericOutputTypeSelectionMode::None;
}

UEdGraphPin* UNiagaraNode::TraceOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bFilterForCompilation, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace)
{
	if (LocallyOwnedOutputPin == nullptr)
	{
		return nullptr;
	}

	UNiagaraNode* LinkedNode = CastChecked<UNiagaraNode>(LocallyOwnedOutputPin->GetOwningNode());
	return LinkedNode->GetTracedOutputPin(LocallyOwnedOutputPin, bFilterForCompilation, OutNodesVisitedDuringTrace);
}

UEdGraphPin* UNiagaraNode::GetTracedOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bFilterForCompilation, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace) const
{
	if (OutNodesVisitedDuringTrace != nullptr)
	{
		OutNodesVisitedDuringTrace->Add(this);
	}
	return LocallyOwnedOutputPin;
}


bool UNiagaraNode::SubstituteCompiledPin(FTranslator* Translator, UEdGraphPin** LocallyOwnedPin)
{
	return false;
}

void UNiagaraNode::RouteParameterMapAroundMe(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive) const
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());

	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);

	const UEdGraphPin* InputPin = nullptr;
	const UEdGraphPin* OutputPin = nullptr;

	for (const UEdGraphPin* Pin : InputPins)
	{
		if (Schema->PinToTypeDefinition(Pin) == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			InputPin = Pin;
			break;
		}
	}

	for (const UEdGraphPin* Pin : OutputPins)
	{
		if (Schema->PinToTypeDefinition(Pin) == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			OutputPin = Pin;
			break;
		}
	}

	if (InputPin != nullptr && OutputPin != nullptr && InputPin->LinkedTo.Num() != 0)
	{
		int32 PMIdx = OutHistory.TraceParameterMapOutputPin(InputPin->LinkedTo[0]);
		OutHistory.RegisterParameterMapPin(PMIdx, OutputPin);
	}
}

void UNiagaraNode::RegisterPassthroughPin(FNiagaraParameterMapHistoryBuilder& OutHistory, UEdGraphPin* InputPin, UEdGraphPin* OutputPin, bool bFilterForCompilation, bool bVisitInputPin) const
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	if (bVisitInputPin)
		OutHistory.VisitInputPin(InputPin, bFilterForCompilation);

	FNiagaraTypeDefinition InDef = Schema->PinToTypeDefinition(InputPin);
	FNiagaraTypeDefinition OutDef = Schema->PinToTypeDefinition(OutputPin);

	if (InputPin && InDef == FNiagaraTypeDefinition::GetParameterMapDef() && OutDef == FNiagaraTypeDefinition::GetParameterMapDef() && InputPin->LinkedTo.Num() > 0)
	{
		int32 PMIdx = OutHistory.TraceParameterMapOutputPin(InputPin->LinkedTo[0]);
		OutHistory.RegisterParameterMapPin(PMIdx, OutputPin);
	}
	else if (InputPin && InDef.IsStatic() && InputPin->LinkedTo.Num() > 0)
	{
		int32 ConstantIdx = OutHistory.GetConstantFromOutputPin(InputPin->LinkedTo[0]);
		OutHistory.RegisterConstantPin(ConstantIdx, InputPin);

		if (OutputPin && OutDef == InDef)
		{
			OutHistory.RegisterConstantPin(ConstantIdx, OutputPin);
		}
	}
	else if (InputPin && InDef.IsStatic() && InputPin->LinkedTo.Num() == 0)
	{
		int32 ConstantIdx = OutHistory.AddOrGetConstantFromValue(InputPin->DefaultValue);
		OutHistory.RegisterConstantPin(ConstantIdx, InputPin);
		if (OutputPin && OutDef == InDef)
		{
			OutHistory.RegisterConstantPin(ConstantIdx, OutputPin);
		}
	}
}

bool UNiagaraNode::GetTooltipTextForKnownPin(const UEdGraphPin& Pin, FText& OutTooltip) const
{
	const UNiagaraGraph* NiagaraGraph = Cast<UNiagaraGraph>(GetGraph());
	const UEdGraphSchema_Niagara* Schema = Cast<UEdGraphSchema_Niagara>(NiagaraGraph->GetSchema());
	if (Schema)
	{
		FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(&Pin);
		FNiagaraVariable Var(TypeDef, Pin.PinName);

		// check for known engine constants
		if (const FNiagaraVariableMetaData* VariableMetaData = FNiagaraConstants::GetConstantMetaData(Var))
		{
			FText Text = VariableMetaData->Description;
			if (Text.IsEmptyOrWhitespace() == false)
			{
				OutTooltip = FText::Format(LOCTEXT("KnownPinHoverTooltip", "{0}\nType: {1}"), Text, TypeDef.GetNameText());
				return true;
			}
		}
	}
	return false;
}

UNiagaraNode::FOnNodeVisualsChanged& UNiagaraNode::OnVisualsChanged()
{
	return VisualsChangedDelegate;
}

void UNiagaraNode::UpdateCompileHashForNode(FSHA1& HashState) const
{
	HashState.Update((const uint8*)&ChangeId, sizeof(FGuid));
}

#if WITH_EDITORONLY_DATA
void UNiagaraNode::GatherForLocalization(FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags) const
{
	// Niagara nodes only contain editor-only text data
	Super::GatherForLocalization(PropertyLocalizationDataGatherer, GatherTextFlags | EPropertyLocalizationGathererTextFlags::ForceEditorOnly);
}
#endif

void UNiagaraNode::GetCompilationInputPins(FPinCollectorArray& InputPins) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Input && !Pin->bOrphanedPin)
		{
			InputPins.Add(Pin);
		}
	}
}

void UNiagaraNode::GetCompilationOutputPins(FPinCollectorArray& OutputPins) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Output && !Pin->bOrphanedPin)
		{
			OutputPins.Add(Pin);
		}
	}
}

#undef LOCTEXT_NAMESPACE
