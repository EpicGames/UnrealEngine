// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeAssignment.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScript.h"
#include "NiagaraSystem.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraComponent.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraConstants.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "ScopedTransaction.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "NiagaraCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeAssignment)

#define LOCTEXT_NAMESPACE "NiagaraNodeAssigment"

void UNiagaraNodeAssignment::AllocateDefaultPins()
{
	GenerateScript();
	Super::AllocateDefaultPins();
}

FText UNiagaraNodeAssignment::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return Title;
}

void UNiagaraNodeAssignment::RefreshTitle()
{
	if (AssignmentTargets.Num() == 1)
	{
		Title = FText::Format(LOCTEXT("NodeTitleSingle", "Set {0}"), FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(AssignmentTargets[0].GetName()));
	}
	else if (AssignmentTargets.Num() > 1)
	{
		Title = FText::Format(LOCTEXT("NodeTitleMultiple", "Set {0} (+{1})"), FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(AssignmentTargets[0].GetName()), AssignmentTargets.Num() - 1);
	}
	else
	{
		Title = LOCTEXT("NodeTitle", "Set Parameters");
	}
}

FText UNiagaraNodeAssignment::GetTooltipText() const
{
	FText BaseText = LOCTEXT("NodeTooltipFormat", "Sets these parameters in the stack:");

	TArray<FText> TargetNames;
	TargetNames.Add(BaseText);
	for (const FNiagaraVariable& Var : AssignmentTargets)
	{
		TargetNames.Add(FText::Format(LOCTEXT("Indent", "\t{0}"), FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(Var.GetName())));
	}

	return FText::Join(FText::FromString("\n"), TargetNames);
}

TSharedRef<SWidget> UNiagaraNodeAssignment::CreateAddParameterMenu(const TSharedPtr<SComboButton>& AddButton)
{
	TSharedPtr<FNiagaraSystemToolkitParameterPanelViewModel> ParameterPanelViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemToolkitParameterPanelViewModel>::GetExistingViewModelForObject(GetTypedOuter<UNiagaraSystem>());
	if (ParameterPanelViewModel.IsValid())
	{
		return ParameterPanelViewModel->CreateAddParameterMenuForAssignmentNode(this, AddButton);
	}
	return SNullWidget::NullWidget;
}

bool UNiagaraNodeAssignment::RefreshFromExternalChanges()
{
	Modify(false);
	if (FunctionScript != nullptr)
	{
		FunctionScript->Modify(false);
	}
	FunctionScript = nullptr;
	GenerateScript();
	ReallocatePins();
	RefreshTitle();
	OnInputsChangedDelegate.Broadcast();
	return true;
}

void UNiagaraNodeAssignment::PostLoad()
{
	Super::PostLoad();

	bool NeedsScriptRefresh = false;

	// Handle the case where we moved towards an array of assignment targets...
	if (AssignmentTarget_DEPRECATED.IsValid() && AssignmentTargets.Num() == 0)
	{
		AssignmentTargets.Add(AssignmentTarget_DEPRECATED);
		AssignmentDefaultValues.Add(AssignmentDefaultValue_DEPRECATED);
		OldFunctionCallName = FunctionDisplayName;
		FunctionDisplayName.Empty();
		RefreshFromExternalChanges();

		UE_LOGF(LogNiagaraEditor, Log, "Found old Assignment Node, converting variable \"%ls\" in \"%ls\"", *AssignmentTarget_DEPRECATED.GetName().ToString(), *GetFullName());

		MarkNodeRequiresSynchronization(__FUNCTION__, true);
		
		// Deduce what rapid iteration variable we would have previously been and prepare to change
		// any instances of it.
		TMap<FNiagaraVariable, FNiagaraVariable> Converted;
		FNiagaraParameterHandle TargetHandle(AssignmentTarget_DEPRECATED.GetName());
		FString VarNamespace = TargetHandle.GetNamespace().ToString();
		
		FNiagaraVariable RemapVar = FNiagaraVariable(AssignmentTarget_DEPRECATED.GetType(), *(OldFunctionCallName + TEXT(".") + TargetHandle.GetName().ToString()));
		FNiagaraVariable NewVar = FNiagaraUtilities::ResolveAliases(RemapVar, FNiagaraAliasContext().ChangeModuleName(OldFunctionCallName, FunctionDisplayName + TEXT(".") + VarNamespace));
		Converted.Add(RemapVar, NewVar);

		bool bConvertedAnything = false;

		// Now clean up the input set node going into us...
		UEdGraphPin* Pin = GetInputPin(0);
		if (Pin != nullptr && Pin->LinkedTo.Num() == 1)
		{
			// Likely we have a set node going into us, check to see if it has any variables that need to be cleaned up.
			if (UNiagaraNodeParameterMapSet* SetNode = Cast<UNiagaraNodeParameterMapSet>(Pin->LinkedTo[0]->GetOwningNode()))
			{
				SetNode->ConditionalPostLoad();

				FPinCollectorArray InputPins;
				SetNode->GetInputPins(InputPins);

				const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
				for (UEdGraphPin* InputPin : InputPins)
				{
					FNiagaraVariable Var = NiagaraSchema->PinToNiagaraVariable(InputPin, false);
					if (Var.GetName() == RemapVar.GetName())
					{
						if (NewVar != Var)
						{
							SetNode->SetPinName(InputPin, NewVar.GetName());
							UE_LOGF(LogNiagaraEditor, Log, "Converted Set pin variable \"%ls\" to \"%ls\" in \"%ls\"", *Var.GetName().ToString(), *NewVar.GetName().ToString(), *GetFullName());
							bConvertedAnything = true;
						}
					}
				}
			}
			else if (Pin->LinkedTo[0]->GetOwningNode())
			{
				// Sometimes we don't automatically have set nodes between modules in the stack... just skip over these.
				UE_LOGF(LogNiagaraEditor, Log, "Found node \"%ls\" attached to assignment \"%ls\" variable %ls", *Pin->LinkedTo[0]->GetOwningNode()->GetFullName(), *GetFullName(), *NewVar.GetName().ToString());
			}
		}

		// Now we need to find the scripts affecting this node... we cheat and walk up our ownership hierarchy until we find a system or emitter.
		if (Converted.Num() != 0)
		{
			UNiagaraEmitter* Emitter = nullptr;
			UNiagaraSystem* System = nullptr;
			UObject* OuterObj = GetOuter();
			while (OuterObj != nullptr)
			{
				if (Emitter == nullptr)
				{
					Emitter = Cast<UNiagaraEmitter>(OuterObj);
				}
				if (System == nullptr)
				{
					System = Cast<UNiagaraSystem>(OuterObj);
				}

				OuterObj = OuterObj->GetOuter();
			}

			// Gather up the affected scripts from the relevant owner...
			TArray<UNiagaraScript*> Scripts;
			if (Emitter)
			{
				if (UNiagaraScript* OuterScript = GetTypedOuter<UNiagaraScript>())
				{
					OuterScript->GetOuterEmitter().GetEmitterData()->GetScripts(Scripts, false);
				}
				else if (UNiagaraScriptSource* OuterSource = GetTypedOuter<UNiagaraScriptSource>())
				{
					OuterSource->GetOuterEmitter().GetEmitterData()->GetScripts(Scripts, false);
				}
				else
				{
					ensureMsgf(false, TEXT("Unable to find path from emitter %s to %s"), *Emitter->GetPathName(), *this->GetPathName());
				}
			}
			if (System)
			{
				Scripts.Add(System->GetSystemSpawnScript());
				Scripts.Add(System->GetSystemUpdateScript());
			}

			for (UNiagaraScript* Script : Scripts)
			{
				check(Script);
				if (Script->HasAnyFlags(RF_NeedPostLoad))
				{
					Script->RapidIterationParameters.PostLoad(Script);
				}
				if (Script->HandleVariableRenames(Converted, Emitter ? Emitter->GetUniqueEmitterName() : FString()))
				{
					bConvertedAnything = true;
				}
			}
		}

		if (!bConvertedAnything)
		{
			UE_LOGF(LogNiagaraEditor, Log, "Found old Assignment Node, nothing was attached???? variable \"%ls\" in \"%ls\"", *AssignmentTarget_DEPRECATED.GetName().ToString(), *GetFullName());
		}
	}
	else
	{
		const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
		if (NiagaraVer < FNiagaraCustomVersion::AssignmentNodeUsesBeginDefaults)
		{
			NeedsScriptRefresh = true;
		}
		if (NiagaraVer < FNiagaraCustomVersion::AssignmentNodeHasCorrectUsageBitmask)
		{
			if (FunctionScript != nullptr)
			{
				UpdateUsageBitmaskFromOwningScript();
			}
		}
	}

	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraVer < FNiagaraCustomVersion::StandardizeParameterNames)
	{
		for (FNiagaraVariable& AssignmentTarget : AssignmentTargets)
		{
			FName CurrentName = AssignmentTarget.GetName();
			FName NewName = UNiagaraGraph::StandardizeName(CurrentName, ENiagaraScriptUsage::Module, false, true);
			AssignmentTarget.SetName(NewName);
		}
		NeedsScriptRefresh = true;
	}

	// check for outdated LWC types
	if (PostLoad_LWCFixup(NiagaraVer))
	{
		NeedsScriptRefresh = true;
	}

	if (NeedsScriptRefresh)
	{
		FunctionScript = nullptr;
		GenerateScript();
	}

	RefreshTitle();
}

bool UNiagaraNodeAssignment::PostLoad_LWCFixup(int32 NiagaraVersion)
{
	bool NeedsScriptRefresh = false;

	const TArray<FNiagaraVariable>& OldTypes = FNiagaraConstants::GetOldPositionTypeVariables();
	const FEdGraphPinType PositionPinType = UEdGraphSchema_Niagara::TypeDefinitionToPinType(FNiagaraTypeDefinition::GetPositionDef());

	for (FNiagaraVariable& AssignmentTarget : AssignmentTargets)
	{
		if (OldTypes.Contains(AssignmentTarget))
		{
			AssignmentTarget.SetType(FNiagaraTypeDefinition::GetPositionDef());

			FNameBuilder ExternalVariableNameBuilder;
			ExternalVariableNameBuilder.Append(FunctionDisplayName);
			ExternalVariableNameBuilder.Append(TEXT("."));
			AssignmentTarget.GetName().AppendString(ExternalVariableNameBuilder);

			FName ExternalVariableName(ExternalVariableNameBuilder);

			// additionally we need to look at the input to the assignment node, if it's a set node
			// then we'll need to change the type of the pin there too so that when we run a compile
			// we'll be able to connect that to the internal input node that we've created in our graph
			if (UEdGraphPin* InputPin = GetInputPin(0))
			{
				if (UEdGraphPin* ConnectedPin = InputPin->LinkedTo.IsEmpty() ? nullptr : InputPin->LinkedTo[0])
				{
					if (UNiagaraNodeParameterMapSet* SetNode = Cast<UNiagaraNodeParameterMapSet>(ConnectedPin->GetOwningNode()))
					{
						SetNode->ConditionalPostLoad();

						// find any appropriate input pins that will need to have their types changed
						FPinCollectorArray SetNodeInputPins;
						SetNode->GetInputPins(SetNodeInputPins);
						for (UEdGraphPin* SetNodeInputPin : SetNodeInputPins)
						{
							FNiagaraVariable SetNodeInputVariable = UEdGraphSchema_Niagara::PinToNiagaraVariable(SetNodeInputPin);
							if (SetNodeInputVariable.IsValid() && SetNodeInputVariable.GetName() == ExternalVariableName)
							{
								SetNodeInputPin->PinType = PositionPinType;
							}
						}
					}
				}
			}

			NeedsScriptRefresh = true;
		}
	}

	return NeedsScriptRefresh;
}

#if WITH_EDITORONLY_DATA
void UNiagaraNodeAssignment::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraGraph::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraNodeInput::StaticClass()));	
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraNodeOutput::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraNodeParameterMapGet::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraNodeParameterMapSet::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraScript::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraScriptSource::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraScriptVariable::StaticClass()));	
}
#endif

void UNiagaraNodeAssignment::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	Super::BuildParameterMapHistory(OutHistory, bRecursive, bFilterForCompilation);
}

void UNiagaraNodeAssignment::GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const
{
	// Assignment nodes own their function graphs and therefore have no external dependencies so we override the default function behavior here to avoid 
	// adding additional non-deterministic guids to the compile id generation which can invalid the DDC for compiled scripts, especially during emitter merging.
}

void UNiagaraNodeAssignment::GenerateScript()
{
	if (FunctionScript == nullptr)
	{
		FunctionScript = NewObject<UNiagaraScript>(this, FName(*(TRANSLATOR_SET_VARIABLES_UNDERSCORE_STR + NodeGuid.ToString())), RF_Transactional);
		FunctionScript->CheckVersionDataAvailable();
		FunctionScript->SetUsage(ENiagaraScriptUsage::Module);
		GetScriptData()->Description = LOCTEXT("AssignmentNodeDesc", "Sets one or more variables in the stack.");
		InitializeScript(FunctionScript);
		UpdateUsageBitmaskFromOwningScript();
		ComputeNodeName();
	}
}

void UNiagaraNodeAssignment::AddParameter(FNiagaraVariable InVar, FString InDefaultValue)
{
	const FText TransactionDesc = FText::Format(LOCTEXT("SetFunctionTransactionDesc", "Add the parameter {0}."), FText::FromName(InVar.GetName()));
	FScopedTransaction ScopedTransaction(TransactionDesc);

	// Since we blow away the graph, we need to cache *everything* we create potentially.
	Modify();
	FunctionScript->Modify();
	UNiagaraScriptSource* Source = GetFunctionScriptSource();
	Source->Modify();
	Source->NodeGraph->Modify();
	for (UEdGraphNode* Node : Source->NodeGraph->Nodes)
	{
		Node->Modify();
	}

	AddAssignmentTarget(InVar, &InDefaultValue);

	RefreshFromExternalChanges();
	MarkNodeRequiresSynchronization(__FUNCTION__, true);

	RefreshTitle();
	AssignmentTargetsChangedDelegate.Broadcast();
}

void UNiagaraNodeAssignment::RemoveParameter(const FNiagaraVariable& InVar)
{
	const FText TransactionDesc = FText::Format(LOCTEXT("RemoveFunctionTransactionDesc", "Remove the parameter {0}."), FText::FromName(InVar.GetName()));
	FScopedTransaction ScopedTransaction(TransactionDesc);

	// Since we blow away the graph, we need to cache *everything* we create potentially.
	Modify();
	FunctionScript->Modify();
	UNiagaraScriptSource* Source = GetFunctionScriptSource();
	Source->Modify();
	Source->NodeGraph->Modify();
	for (UEdGraphNode* Node : Source->NodeGraph->Nodes)
	{
		Node->Modify();
	}

	int32 Index = INDEX_NONE;
	if (AssignmentTargets.Find(InVar, Index))
	{
		AssignmentTargets.RemoveAt(Index);
		AssignmentDefaultValues.RemoveAt(Index);
	}

	RefreshFromExternalChanges();
	MarkNodeRequiresSynchronization(__FUNCTION__, true);

	RefreshTitle();
	AssignmentTargetsChangedDelegate.Broadcast();
}

void UNiagaraNodeAssignment::UpdateUsageBitmaskFromOwningScript()
{
	FunctionScript->GetScriptData(SelectedScriptVersion)->ModuleUsageBitmask = CalculateScriptUsageBitmask();
}

void UNiagaraNodeAssignment::InitializeScript(UNiagaraScript* NewScript)
{
	if (NewScript != NULL)
	{		
		UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(NewScript->GetLatestSource());

		if (nullptr == Source)
		{
			Source = NewObject<UNiagaraScriptSource>(NewScript, NAME_None, RF_Transactional);
			NewScript->SetSource(Source, SelectedScriptVersion);
		}

		UNiagaraGraph* CreatedGraph = Source->NodeGraph;
		if (nullptr == CreatedGraph)
		{
			CreatedGraph = NewObject<UNiagaraGraph>(Source, NAME_None, RF_Transactional);
			Source->NodeGraph = CreatedGraph;
		}
		CreatedGraph->Modify();
		
		TArray<UNiagaraNodeInput*> InputNodes;
		CreatedGraph->FindInputNodes(InputNodes);

		UNiagaraNodeInput* InputMapInputNode;
		UNiagaraNodeInput** InputMapInputNodePtr = InputNodes.FindByPredicate([](UNiagaraNodeInput* InputNode)
		{ 
			return InputNode->Usage == ENiagaraInputNodeUsage::Parameter && InputNode->Input.GetType() == FNiagaraTypeDefinition::GetParameterMapDef() && InputNode->Input.GetName() == "InputMap"; 
		});
		if (InputMapInputNodePtr == nullptr)
		{
			FGraphNodeCreator<UNiagaraNodeInput> InputNodeCreator(*CreatedGraph);
			InputMapInputNode = InputNodeCreator.CreateNode();
			InputMapInputNode->Input = FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("InputMap"));
			InputMapInputNode->Usage = ENiagaraInputNodeUsage::Parameter;
			InputNodeCreator.Finalize();
		}
		else
		{
			InputMapInputNode = *InputMapInputNodePtr;
		}

		UNiagaraNodeInput* BeginDefaultsInputNode;
		UNiagaraNodeInput** BeginDefaultsInputNodePtr = InputNodes.FindByPredicate([](UNiagaraNodeInput* InputNode)
		{
			return InputNode->Usage == ENiagaraInputNodeUsage::TranslatorConstant && InputNode->Input == TRANSLATOR_PARAM_BEGIN_DEFAULTS;
		});
		if (BeginDefaultsInputNodePtr == nullptr)
		{
			FGraphNodeCreator<UNiagaraNodeInput> InputNodeCreator(*CreatedGraph);
			BeginDefaultsInputNode = InputNodeCreator.CreateNode();
			BeginDefaultsInputNode->Input = TRANSLATOR_PARAM_BEGIN_DEFAULTS;
			BeginDefaultsInputNode->Usage = ENiagaraInputNodeUsage::TranslatorConstant;
			BeginDefaultsInputNode->ExposureOptions.bCanAutoBind = true;
			BeginDefaultsInputNode->ExposureOptions.bHidden = true;
			BeginDefaultsInputNode->ExposureOptions.bRequired = false;
			BeginDefaultsInputNode->ExposureOptions.bExposed = false;
			InputNodeCreator.Finalize();
		}
		else
		{
			BeginDefaultsInputNode = *BeginDefaultsInputNodePtr;
		}

		UNiagaraNodeOutput* OutputNode = CreatedGraph->FindOutputNode(ENiagaraScriptUsage::Module);
		if (OutputNode == nullptr)
		{
			FGraphNodeCreator<UNiagaraNodeOutput> OutputNodeCreator(*CreatedGraph);
			OutputNode = OutputNodeCreator.CreateNode();
			FNiagaraVariable ParamMapAttrib(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("OutputMap"));
			OutputNode->SetUsage(ENiagaraScriptUsage::Module);
			OutputNode->Outputs.Add(ParamMapAttrib);
			OutputNodeCreator.Finalize();
		}

		TArray<UNiagaraNodeParameterMapGet*> GetNodes;
		CreatedGraph->GetNodesOfClass(GetNodes);

		TArray<UNiagaraNodeParameterMapSet*> SetNodes;
		CreatedGraph->GetNodesOfClass(SetNodes);

		if (SetNodes.Num() == 0)
		{
			FGraphNodeCreator<UNiagaraNodeParameterMapSet> InputNodeCreator(*CreatedGraph);
			UNiagaraNodeParameterMapSet* InputNode = InputNodeCreator.CreateNode();
			InputNodeCreator.Finalize();
			SetNodes.Add(InputNode);

			InputMapInputNode->GetOutputPin(0)->MakeLinkTo(SetNodes[0]->GetInputPin(0));
			SetNodes[0]->GetOutputPin(0)->MakeLinkTo(OutputNode->GetInputPin(0));
		}

		// We create two get nodes. The first is for the direct values.
		// The second is in the case of referencing other parameters that we want to use as defaults.
		if (GetNodes.Num() == 0)
		{
			FGraphNodeCreator<UNiagaraNodeParameterMapGet> InputNodeCreator(*CreatedGraph);
			UNiagaraNodeParameterMapGet* InputNode = InputNodeCreator.CreateNode();
			InputNodeCreator.Finalize();
			GetNodes.Add(InputNode);

			InputMapInputNode->GetOutputPin(0)->MakeLinkTo(GetNodes[0]->GetInputPin(0));
		}
		if (GetNodes.Num() == 1)
		{
			FGraphNodeCreator<UNiagaraNodeParameterMapGet> InputNodeCreator(*CreatedGraph);
			UNiagaraNodeParameterMapGet* InputNode = InputNodeCreator.CreateNode();
			InputNodeCreator.Finalize();
			GetNodes.Add(InputNode);

			BeginDefaultsInputNode->GetOutputPin(0)->MakeLinkTo(GetNodes[1]->GetInputPin(0));
		}

		// Clean out existing pins
		while (!SetNodes[0]->IsAddPin(SetNodes[0]->GetInputPin(1)))
		{
			SetNodes[0]->RemovePin(SetNodes[0]->GetInputPin(1));
		}

		while (!GetNodes[0]->IsAddPin(GetNodes[0]->GetOutputPin(0)))
		{
			GetNodes[0]->RemovePin(GetNodes[0]->GetInputPin(0));
		}

		while (!GetNodes[1]->IsAddPin(GetNodes[1]->GetOutputPin(0)))
		{
			GetNodes[1]->RemovePin(GetNodes[1]->GetInputPin(0));
		}

		const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();

		for (int32 i = 0; i < AssignmentTargets.Num(); i++)
		{
			// Now create the proper new pins and connect them.
			FName Name = AssignmentTargets[i].GetName();
			FNiagaraTypeDefinition Type = AssignmentTargets[i].GetType();
			FString DefaultValue = AssignmentDefaultValues[i];

			if (Name != NAME_None)
			{
				FNiagaraParameterHandle TargetHandle(Name);
				UEdGraphPin* SetPin = SetNodes[0]->RequestNewTypedPin(EGPD_Input, Type, Name);
				FString ModuleVarName = FString::Printf(TEXT("Module.%s"), *TargetHandle.GetParameterHandleString().ToString());
				UEdGraphPin* GetPin = GetNodes[0]->RequestNewTypedPin(EGPD_Output, Type, *ModuleVarName);
				FNiagaraVariable TargetVar = NiagaraSchema->PinToNiagaraVariable(GetPin, false);
				GetPin->MakeLinkTo(SetPin);

				if (!DefaultValue.IsEmpty())
				{
					UEdGraphPin* DefaultInputPin = GetNodes[0]->GetDefaultPin(GetPin);

					FNiagaraVariable SeekVar = FNiagaraVariable(Type, FName(*DefaultValue));
					const FNiagaraVariable* FoundVar = FNiagaraConstants::FindEngineConstant(SeekVar);
					if (FoundVar != nullptr)
					{
						UEdGraphPin* DefaultGetPin = GetNodes[1]->RequestNewTypedPin(EGPD_Output, Type, FoundVar->GetName());
						DefaultGetPin->MakeLinkTo(DefaultInputPin);
					}
					else
					{
						DefaultInputPin->bDefaultValueIsIgnored = false;
						DefaultInputPin->DefaultValue = DefaultValue;
					}
				}

				if (FNiagaraConstants::IsNiagaraConstant(AssignmentTargets[i]))
				{
					if (const FNiagaraVariableMetaData* FoundMetaData = FNiagaraConstants::GetConstantMetaData(AssignmentTargets[i]))
					{
						FNiagaraVariableMetaData NewMetaData;
						TOptional<FNiagaraVariableMetaData> ExistingMetaData = CreatedGraph->GetMetaData(TargetVar);
						if (ExistingMetaData.IsSet())
						{
							NewMetaData = ExistingMetaData.GetValue();
						}
						NewMetaData.Description = FoundMetaData->Description;
						CreatedGraph->SetMetaData(TargetVar, NewMetaData);
					}
				}
				
				if (UNiagaraScriptVariable* ScriptVar = CreatedGraph->GetScriptVariable(TargetVar))
				{
					ScriptVar->DefaultMode = ENiagaraDefaultMode::Custom;
					CreatedGraph->ScriptVariableChanged(TargetVar);
				}
			}
		}
	}

	RefreshTitle();
}

int32 UsageToBitmask(ENiagaraScriptUsage Usage)
{
	return 1 << (int32)Usage;
}

int32 UNiagaraNodeAssignment::CalculateScriptUsageBitmask()
{
	int32 UsageBitmask = 0;
	UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*this);
	if (OutputNode != nullptr)
	{
		if (UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::SystemSpawnScript) ||
			UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::SystemUpdateScript))
		{
			UsageBitmask =
				UsageToBitmask(ENiagaraScriptUsage::SystemSpawnScript) |
				UsageToBitmask(ENiagaraScriptUsage::SystemUpdateScript);
		}
		if (UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::EmitterSpawnScript) ||
			UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::EmitterUpdateScript))
		{
			UsageBitmask =
				UsageToBitmask(ENiagaraScriptUsage::EmitterSpawnScript) |
				UsageToBitmask(ENiagaraScriptUsage::EmitterUpdateScript);
		}
		if (UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::ParticleSpawnScript) ||
			UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::ParticleUpdateScript) ||
			UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::ParticleEventScript) ||
			UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::ParticleSimulationStageScript))
		{
			UsageBitmask =
				UsageToBitmask(ENiagaraScriptUsage::ParticleSpawnScript) |
				UsageToBitmask(ENiagaraScriptUsage::ParticleUpdateScript) |
				UsageToBitmask(ENiagaraScriptUsage::ParticleEventScript) |
				UsageToBitmask(ENiagaraScriptUsage::ParticleSimulationStageScript);
		}
	}
	return UsageBitmask;
}

int32 UNiagaraNodeAssignment::FindAssignmentTarget(const FName& InName, const FNiagaraTypeDefinition& InType)
{
	for (int32 i = 0; i < AssignmentTargets.Num(); i++)
	{
		FName Name = AssignmentTargets[i].GetName();
		FNiagaraTypeDefinition Type = AssignmentTargets[i].GetType();
	
		if (InName == Name && InType == Type)
		{
			return i;
		}
	}

	return INDEX_NONE;
}

int32 UNiagaraNodeAssignment::FindAssignmentTarget(const FName& InName)
{
	for (int32 i = 0; i < AssignmentTargets.Num(); i++)
	{
		if (InName == AssignmentTargets[i].GetName())
		{
			return i;
		}
	}

	return INDEX_NONE;
}

int32 UNiagaraNodeAssignment::AddAssignmentTarget(const FNiagaraVariable& InVar, const FString* InDefaultValue)
{
	int32 IdxA = AssignmentTargets.AddDefaulted();
	int32 IdxB = AssignmentDefaultValues.AddDefaulted();
	check(IdxA == IdxB);
	SetAssignmentTarget(IdxA, InVar, InDefaultValue);
	AssignmentTargetsChangedDelegate.Broadcast();
	return IdxA;
}

bool UNiagaraNodeAssignment::SetAssignmentTarget(int32 Idx, const FNiagaraVariable& InVar, const FString* InDefaultValue)
{
	check(Idx < AssignmentTargets.Num());

	bool bRetValue = false;
	if (InVar != AssignmentTargets[Idx])
	{
		AssignmentTargets[Idx] = InVar;
		MarkNodeRequiresSynchronization(__FUNCTION__, true);
		bRetValue = true;
	}
	
	if (InDefaultValue != nullptr && AssignmentDefaultValues[Idx] != *InDefaultValue)
	{ 
		AssignmentDefaultValues[Idx] = *InDefaultValue;
		MarkNodeRequiresSynchronization(__FUNCTION__, true);
		bRetValue = true;
	}

	AssignmentTargetsChangedDelegate.Broadcast();
	return bRetValue;
}

bool UNiagaraNodeAssignment::RenameAssignmentTarget(FName OldName, FName NewName)
{
	for (FNiagaraVariable& AssignmentTarget : AssignmentTargets)
	{

		if (AssignmentTarget.GetName() == OldName)
		{
			Modify();
			if (FunctionScript != nullptr)
			{
				FunctionScript->Modify();
				GetFunctionScriptSource()->Modify();
			}

			AssignmentTarget.SetName(NewName);
			RefreshTitle();
			AssignmentTargetsChangedDelegate.Broadcast();
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

