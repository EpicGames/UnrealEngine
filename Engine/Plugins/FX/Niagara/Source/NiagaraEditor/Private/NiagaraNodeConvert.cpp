// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeConvert.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraEditorUtilities.h"
#include "Widgets/SNiagaraGraphNodeConvert.h"
#include "NiagaraHlslTranslator.h"
#include "UObject/UnrealType.h"
#include "Kismet2/StructureEditorUtils.h"

#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeConvert)

#define LOCTEXT_NAMESPACE "NiagaraNodeConvert"

struct FNiagaraConvertEntry
{
	bool bConnected = false;

	FGuid PinId;
	FName Name;
	FNiagaraTypeDefinition Type;
	TArray< FNiagaraConvertEntry> Children;
	UEdGraphPin* Pin;

	FNiagaraConvertEntry(const FGuid& InPinId, const FName& InName, const FNiagaraTypeDefinition& InType, UEdGraphPin* InPin): PinId(InPinId), Name(InName), Pin(InPin){ }

	void  ResolveConnections(const TArray<FNiagaraConvertConnection>& InConnections, TArray<FString>& OutMissingConnections, int32 ConnectionDepth = 0) 
	{
		TArray<FNiagaraConvertConnection> CandidateConnections;
		for (const FNiagaraConvertConnection& Connection : InConnections)
		{
			if (Connection.DestinationPinId == PinId)
			{
				// If connecting root to root, we are fine.
				if (ConnectionDepth == 0 && Connection.DestinationPath.Num() == 0)
				{
					bConnected = true;
					break;
				}
				else if ((Connection.DestinationPath.Num() == (ConnectionDepth+1) && Connection.DestinationPath[ConnectionDepth] == Name))
				{
					bConnected = true;
					break;
				}

				if (ConnectionDepth == 0 || (ConnectionDepth > 0 && Connection.DestinationPath.Num() > ConnectionDepth && Connection.DestinationPath[ConnectionDepth] == Name))
				{
					CandidateConnections.Add(Connection);
				}
			}
		}

		if (bConnected == true)
		{
			return;
		}

		// Now see if all children are connected and then return that you are connected.
		if (Children.Num() > 0)
		{
			if (CandidateConnections.Num() > 0)
			{
				TArray <FString> MissingConnectionsChildren;
				int32 NumConnected = 0;
				for (FNiagaraConvertEntry& Entry : Children)
				{
					Entry.ResolveConnections(CandidateConnections, MissingConnectionsChildren, 1 + ConnectionDepth);

					if (Entry.bConnected)
					{
						NumConnected++;
					}
				}

				if (NumConnected == Children.Num())
				{
					bConnected = true;
				}
				else
				{
					for (const FString& MissingConnectionStr : MissingConnectionsChildren)
					{
						OutMissingConnections.Emplace(Name.ToString() + TEXT(".") + MissingConnectionStr);
					}
				}
			}
			else
			{
				OutMissingConnections.Emplace(Name.ToString());
			}
		}
		else
		{
			OutMissingConnections.Emplace(Name.ToString());
		}
	}

	static void CreateEntries(const UEdGraphSchema_Niagara* Schema, const FGuid& InPinId, UEdGraphPin* InPin, const UScriptStruct* InStruct, TArray< FNiagaraConvertEntry>& OutEntries)
	{
		check(Schema);

		for (TFieldIterator<FProperty> PropertyIt(InStruct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = *PropertyIt;
			if (Schema->IsValidNiagaraPropertyType(Property))
			{
				FNiagaraTypeDefinition PropType = Schema->GetTypeDefForProperty(Property);

				int32 Index = OutEntries.Emplace(InPinId, Property->GetFName(), PropType, InPin);

				const FStructProperty* StructProperty = CastField<FStructProperty>(Property);

				if (StructProperty != nullptr)
				{
					CreateEntries(Schema, InPinId, InPin, FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProperty->Struct, ENiagaraStructConversion::UserFacing), OutEntries[Index].Children);
				}
			}
		}
	}
};


UNiagaraNodeConvert::UNiagaraNodeConvert() : UNiagaraNodeWithDynamicPins(), bIsWiringShown(true)
{

}

void UNiagaraNodeConvert::PostLoad()
{
	Super::PostLoad();

	if (IsLocalConstantValue())
	{
		// collapse convert nodes with simple constant values on load
		SetWiringShown(false);
	}
}

void UNiagaraNodeConvert::AllocateDefaultPins()
{
	CreateAddPin(EGPD_Input);
	CreateAddPin(EGPD_Output);
}

TSharedPtr<SGraphNode> UNiagaraNodeConvert::CreateVisualWidget()
{
	return SNew(SNiagaraGraphNodeConvert, this);
}

void UNiagaraNodeConvert::Compile(FTranslator* Translator, TArray<int32>& CompileOutputs) const
{
	FPinCollectorArray InputPins;
	GetCompilationInputPins(InputPins);

	TArray<int32, TInlineAllocator<16>> CompileInputs;
	CompileInputs.Reserve(InputPins.Num());
	for(UEdGraphPin* InputPin : InputPins)
	{
		if (InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType || 
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum || 
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType ||
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum)
		{
			int32 CompiledInput = Translator->CompileInputPin(InputPin);
			if (CompiledInput == INDEX_NONE)
			{
				Translator->Error(LOCTEXT("InputError", "Error compiling input for convert node."), this, InputPin);
			}
			CompileInputs.Add(CompiledInput);
		}
	}

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	check(Schema);

	FPinCollectorArray OutputPins;
	GetCompilationOutputPins(OutputPins);

	// Go through all the output nodes and cross-reference them with the connections list. Output errors if any connections are incomplete.
	{
		TArray< FNiagaraConvertEntry> Entries;
		for (UEdGraphPin* OutputPin : OutputPins)
		{

			FNiagaraTypeDefinition TypeDef;
			if (OutputPin && OutputPin->HasAnyConnections())
			{
				TypeDef = Schema->PinToTypeDefinition(OutputPin);
				const UScriptStruct* Struct = TypeDef.GetScriptStruct();
				if (Struct)
				{
					FNiagaraConvertEntry::CreateEntries(Schema, OutputPin->PinId, OutputPin, Struct, Entries);
				}
			}
		}

		for (FNiagaraConvertEntry& Entry : Entries)
		{
			TArray<FString> MissingConnections;
			Entry.ResolveConnections(Connections, MissingConnections);

			if (Entry.bConnected == false)
			{
				for (const FString& MissedConnection : MissingConnections)
				{
					Translator->Error(FText::Format(LOCTEXT("MissingOutputPinConnection", "Missing internal connection for output pin slot: {0}"), 
						FText::FromString(MissedConnection)), this, Entry.Pin);
				}
			}
		}
	}


	Translator->Convert(this, CompileInputs, CompileOutputs);
}

FString FNiagaraConvertConnection::ToString() const
{
	FString SrcName;
	FString DestName;
	for (const FName& Src : SourcePath)
	{
		SrcName += TEXT("/") + Src.ToString();
	}

	for (const FName& Dest : DestinationPath)
	{
		DestName += TEXT("/") + Dest.ToString();
	}

	return SrcName + TEXT(" to ") + DestName;
}

namespace FNiagaraConvertRefreshHelpers
{
	// Helper function to reduce book-keeping around searching by predicate
	UEdGraphPin* FindPinByID(FGuid SearchPinId, TArray<UEdGraphPin*> PinsToSearch)
	{
		UEdGraphPin** FoundPin = PinsToSearch.FindByPredicate([&SearchPinId](UEdGraphPin* Pin) {return Pin->PinId == SearchPinId; });
		if (FoundPin)
		{
			return *FoundPin;
		}

		return nullptr;
	}

	// Helper function that renames a pin if it matches a member in the given struct
	void RenamePinIfInStruct(UEdGraphPin* Pin, const UScriptStruct* Struct, FGuid& ConnectionGuid)
	{

		const UUserDefinedStruct* UserDefinedStruct = Cast<const UUserDefinedStruct>(Struct);

		for (FProperty* Property : TFieldRange<FProperty>(Struct, EFieldIteratorFlags::IncludeSuper))
		{

			FString PropertyName;
			FGuid PropertyGuid = FStructureEditorUtils::GetGuidForProperty(Property);

			// User defined structs will have GUIDs as part of the name, so the friendly name is needed instead
			if (UserDefinedStruct)
			{
				PropertyName = FStructureEditorUtils::GetVariableFriendlyName(UserDefinedStruct, PropertyGuid);
			}
			else
			{
				PropertyName = Property->GetName();
			}

			// Rename and store GUID
			if (Pin->GetName() == PropertyName || (ConnectionGuid == PropertyGuid && ConnectionGuid != FGuid()))
			{
				ConnectionGuid = PropertyGuid;
				Pin->PinName = FName(*PropertyName);

				break;
			}
		}
	}

	// Recurses paths of a convert connection, and renames if the path is valid to conform to changes in the underlying struct
	// Returns true if the path is valid, if a property can be found, its pointer is stored in OutProperty.
	bool RecursePaths(const UScriptStruct* ParentStruct, TArray<FName>& Paths, int32 PathIndex, FProperty*& OutProperty)
	{
		bool bResult = false;
		if (Paths.IsValidIndex(PathIndex))
		{
			FName CurrentPath = Paths[PathIndex];
			// Paths will end with "Value" for scalars, which is a special case. The path is valid, but there's nothing more to search for
			if (CurrentPath.ToString().Equals("Value"))
			{
				return true;
			}

			FGuid CurrentGUID = FStructureEditorUtils::GetGuidFromPropertyName(CurrentPath);
			FProperty* FoundProperty = nullptr;

			// Search for path by name or GUID in struct
			for (FProperty* Property : TFieldRange<FProperty>(ParentStruct))
			{
				FGuid PropertyGuid = FStructureEditorUtils::GetGuidFromPropertyName(Property->GetFName());

				if (CurrentPath == Property->GetFName() || (PropertyGuid != FGuid() && PropertyGuid == CurrentGUID))
				{
					FoundProperty = Property;
					break;
				}
			}


			if (FoundProperty)
			{
				// Path is valid up until this point
				bResult = true;
				const FStructProperty* StructProperty = CastField<const FStructProperty>(FoundProperty);

				// For nested structs update the parent struct to this property
				if (StructProperty)
				{
					ParentStruct = StructProperty->Struct;
				}

				// Continue if there are more entries
				if (Paths.IsValidIndex(PathIndex + 1))
				{
					bResult = RecursePaths(ParentStruct, Paths, PathIndex + 1, OutProperty);
				}
				if (bResult)
				{
					// If no other calls have assigned the out property, do so here
					if (!OutProperty)
					{
						OutProperty = FoundProperty;
					}

					// Fix up the name here.
					Paths[PathIndex] = FoundProperty->GetFName();
				}
			}

		}
		else if (Paths.Num() == 0)
		{
			// Top-level struct connections will have an empty paths array. 
			return true;
		}

		return bResult;
	}
}



bool UNiagaraNodeConvert::RefreshFromExternalChanges()
{
	bool bHasChanged = false;

	// Used to get struct definition
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	check(Schema);
	
	const UScriptStruct* SourceScriptStruct;
	const UScriptStruct* DestinationScriptStruct;

	// Cache connections before clean-up for comparison to check if anything changed/a recompile is necessary.
	const TArray<FNiagaraConvertConnection> OldConnections = Connections;

	// Clean up paths in each connection. Remove or mark orphaned pins, and remove connections as needed.
	for (int32 ConnectionIndex = 0; ConnectionIndex < Connections.Num(); ConnectionIndex++)
	{
		FNiagaraConvertConnection Connection = Connections[ConnectionIndex];

		UEdGraphPin* SourcePin = FNiagaraConvertRefreshHelpers::FindPinByID(Connection.SourcePinId, Pins);
		UEdGraphPin* DestinationPin = FNiagaraConvertRefreshHelpers::FindPinByID(Connection.DestinationPinId, Pins);

		// Get type definitions for the pins in this connection
		// PinToTypeDefinition handles nullptr case, so no need to null check them here
		FNiagaraTypeDefinition SourcePinTypeDef = Schema->PinToTypeDefinition(SourcePin);
		FNiagaraTypeDefinition DestinationPinTypeDef = Schema->PinToTypeDefinition(DestinationPin);

		SourceScriptStruct = SourcePinTypeDef.GetScriptStruct();
		DestinationScriptStruct = DestinationPinTypeDef.GetScriptStruct();

		bool bIsSourceScalar = FNiagaraTypeDefinition::IsScalarDefinition(SourcePinTypeDef);
		bool bIsDestinationScalar = FNiagaraTypeDefinition::IsScalarDefinition(DestinationPinTypeDef);
		
		// Rename pins and fix up stored GUID if found in struct
		// If both pins are structs, search both for the other pin, rename pin, and set GUID as necessary. 
		if (!bIsSourceScalar && !bIsDestinationScalar)
		{
			FGuid InOutSourceGuidToSearch = Connection.SourcePropertyId;
			FGuid InOutDestGuidToSearch = Connection.DestinationPropertyId;

			FNiagaraConvertRefreshHelpers::RenamePinIfInStruct(SourcePin, DestinationScriptStruct, InOutSourceGuidToSearch);
			FNiagaraConvertRefreshHelpers::RenamePinIfInStruct(DestinationPin, SourceScriptStruct, InOutDestGuidToSearch);

			if (InOutSourceGuidToSearch != FGuid())
			{
				Connection.SourcePropertyId = InOutSourceGuidToSearch;
			}

			if (InOutDestGuidToSearch != FGuid())
			{
				Connection.DestinationPropertyId = InOutDestGuidToSearch;
			}
		}
		else if (bIsSourceScalar && !bIsDestinationScalar)
		{
			FGuid InOutGuidToSearch = Connection.SourcePropertyId;

			FNiagaraConvertRefreshHelpers::RenamePinIfInStruct(SourcePin, DestinationScriptStruct, InOutGuidToSearch);

			if (InOutGuidToSearch != FGuid())
			{	
				Connection.SourcePropertyId = InOutGuidToSearch;
			}
		}
		else if (!bIsSourceScalar && bIsDestinationScalar)
		{
			FGuid InOutGuidToSearch = Connection.DestinationPropertyId;

			FNiagaraConvertRefreshHelpers::RenamePinIfInStruct(DestinationPin, SourceScriptStruct, InOutGuidToSearch);

			if (InOutGuidToSearch != FGuid())
			{
				Connection.DestinationPropertyId = InOutGuidToSearch;
			}
		}

		// Need to initialize as nullptr for RecursePaths
		FProperty* SourceProperty = nullptr;
		FProperty* DestinationProperty = nullptr;

		bool bSourceFound = FNiagaraConvertRefreshHelpers::RecursePaths(SourceScriptStruct, Connection.SourcePath, 0, SourceProperty);
		bool bDestinationFound = FNiagaraConvertRefreshHelpers::RecursePaths(DestinationScriptStruct, Connection.DestinationPath, 0, DestinationProperty);

		bool bTypesAreAssignable = false;

		// Ensure types are still assignable.
		if (bSourceFound && bDestinationFound)
		{
			// Use the property found by recursing paths if possible
			// A null Source or Destination property means the path was empty, and we should use the pin's type. 
			FNiagaraTypeDefinition SourcePropTypeDef = SourceProperty ? Schema->GetTypeDefForProperty(SourceProperty) : SourcePinTypeDef;
			FNiagaraTypeDefinition DestinationPropTypeDef = DestinationProperty ? Schema->GetTypeDefForProperty(DestinationProperty) : DestinationPinTypeDef;

			bTypesAreAssignable = FNiagaraTypeDefinition::TypesAreAssignable(SourcePropTypeDef, DestinationPropTypeDef) || FNiagaraTypeDefinition::IsLossyConversion(SourcePropTypeDef, DestinationPropTypeDef);
		}


		// Scalar pins that are not connected should be orphaned
		if (bIsSourceScalar && (!bDestinationFound || !bTypesAreAssignable))
		{
			SourcePin->bOrphanedPin = true;
		}
		// If they were orphaned, and now have a valid connection they are no longer orphan pins
		else if (SourcePin->bOrphanedPin && bDestinationFound && bTypesAreAssignable)
		{
			SourcePin->bOrphanedPin = false;
		}

		// Repeat for destination pins
		if (bIsDestinationScalar && (!bSourceFound || !bTypesAreAssignable))
		{
			DestinationPin->bOrphanedPin = true;
		}
		else if (DestinationPin->bOrphanedPin && bSourceFound && bTypesAreAssignable)
		{
			DestinationPin->bOrphanedPin = false;
		}

		// Remove invalid connections
		if (!bDestinationFound || !bSourceFound || !bTypesAreAssignable)
		{
			Connections.RemoveAt(ConnectionIndex);
			ConnectionIndex--;
		} 
		else
		{
			// write updated connection to array.
			Connections[ConnectionIndex] = Connection;
		}
	}

	// Compare new and old connections to see if data has changed.
	if (OldConnections.Num() == Connections.Num())
	{
		for (int32 ConnectionIndex = 0; ConnectionIndex < Connections.Num(); ConnectionIndex++)
		{
			FNiagaraConvertConnection OldConneciton = OldConnections[ConnectionIndex];
			FNiagaraConvertConnection NewConnection = Connections[ConnectionIndex];

			if (!(NewConnection == OldConneciton))
			{
				bHasChanged = true;
				break;
			}
		}
	}
	else
	{
		bHasChanged = true;
	}

	// There are changes to the underlying struct that may not impact the connections stored, so we have no way of testing when a visual refresh is necessary.
	// So assume the UI always needs to be refreshed if this function is called, since the user is triggering a node refresh explicitly.
	// If the connections haven't changed, just update the UI, and do not recompile. 
	// Any necessary recompile and visual update will be handled by the schema, if the data has changed (i.e. this function returns true). See: UEdGraphSchema_Niagara::RefreshNode
	if (!bHasChanged)
	{
		MarkNodeRequiresSynchronization(__FUNCTION__, false);
	}
	
	return bHasChanged;
}

void UNiagaraNodeConvert::AutowireNewNode(UEdGraphPin* FromPin)
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	check(Schema);

	FNiagaraTypeDefinition TypeDef;
	EEdGraphPinDirection Dir = FromPin ? (EEdGraphPinDirection)FromPin->Direction : EGPD_Output;
	EEdGraphPinDirection OppositeDir = FromPin && (EEdGraphPinDirection)FromPin->Direction == EGPD_Input ? EGPD_Output : EGPD_Input;

	if (AutowireSwizzle.IsEmpty())
	{
		// we only allow breaking on output pins
		if (AutowireBreakType.GetStruct() != nullptr)
		{			
			TypeDef = AutowireBreakType;
			Dir = EGPD_Output;
			OppositeDir = EGPD_Input;
		}
		// but we allow making from output puts and for input pins
		else if(AutowireMakeType.GetStruct() != nullptr)
		{
			TypeDef = AutowireMakeType;
			Dir = EGPD_Input;
			OppositeDir = EGPD_Output;
		}

		if (TypeDef.IsValid() == false)
		{
			return;
		}

		//No swizzle so we make or break the type.
		const UScriptStruct* Struct = TypeDef.GetScriptStruct();
		if (Struct)
		{
			bool bConnectionMade = false;
			UEdGraphPin* ConnectPin = RequestNewTypedPin(OppositeDir, TypeDef);
			check(ConnectPin);
			
			if(FromPin)
			{
				// FromPin and ConnectPin could have the same direction in case we have a make type and we are dragging off from i.e. float to make vector
				// if so, we won't connect that pin and instead will try to connect with the other pins below
				bool bCanConnect = GetSchema()->CanCreateConnection(FromPin, ConnectPin).Response != ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW;
				
				// if our from pin is an input pin, we make sure to break prior connections first 
				if (bCanConnect && FromPin->Direction == EGPD_Input && ConnectPin->Direction == EGPD_Output)
				{
					FromPin->BreakAllPinLinks();
				}

				if(bCanConnect)
				{
					ConnectPin->MakeLinkTo(FromPin);
					bConnectionMade = true;
				}
			}

			TArray<FName> SrcPath;
			TArray<FName> DestPath;
			//Add a corresponding pin for each property in the from Pin.
			for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				SrcPath.Empty(SrcPath.Num());
				DestPath.Empty(DestPath.Num());
				const FProperty* Property = *PropertyIt;
				if (!Schema->IsValidNiagaraPropertyType(Property))
				{
					continue;
				}
				FNiagaraTypeDefinition PropType = Schema->GetTypeDefForProperty(Property);
				UEdGraphPin* NewPin = RequestNewTypedPin(Dir, PropType, *Property->GetDisplayNameText().ToString());

				if (Dir == EGPD_Input)
				{					
					if(FromPin && FromPin->Direction == EGPD_Output && bConnectionMade == false)
					{
						FNiagaraTypeDefinition FromPinType = UEdGraphSchema_Niagara::PinToTypeDefinition(FromPin);
						FNiagaraTypeDefinition InputPinType = UEdGraphSchema_Niagara::PinToTypeDefinition(NewPin);

						if(FNiagaraTypeDefinition::TypesAreAssignable(InputPinType, FromPinType) && GetSchema()->CanCreateConnection(FromPin, NewPin).Response != ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW)
						{
							FromPin->MakeLinkTo(NewPin);
							bConnectionMade = true;
						}
					}
					
					if (FNiagaraTypeDefinition::IsScalarDefinition(PropType))
					{
						SrcPath.Add(TEXT("Value"));
					}
					DestPath.Add(*Property->GetName());
					FGuid PropertyGuid = FStructureEditorUtils::GetGuidForProperty(Property);
					Connections.Add(FNiagaraConvertConnection(NewPin->PinId, SrcPath, ConnectPin->PinId, DestPath, PropertyGuid, FGuid()));
					if (SrcPath.Num())
					{
						AddExpandedRecord(FNiagaraConvertPinRecord(NewPin->PinId, SrcPath).GetParent());
					}
					if (DestPath.Num())
					{
						AddExpandedRecord(FNiagaraConvertPinRecord(ConnectPin->PinId, DestPath).GetParent());
					}
				}
				else
				{
					SrcPath.Add(*Property->GetName());
					if (FNiagaraTypeDefinition::IsScalarDefinition(PropType))
					{
						DestPath.Add(TEXT("Value"));
					}
					FGuid PropertyGuid = FStructureEditorUtils::GetGuidForProperty(Property);
					Connections.Add(FNiagaraConvertConnection(ConnectPin->PinId, SrcPath, NewPin->PinId, DestPath, FGuid(), PropertyGuid));
					if (DestPath.Num())
					{
						AddExpandedRecord(FNiagaraConvertPinRecord(NewPin->PinId, DestPath).GetParent());
					}
					if (SrcPath.Num())
					{
						AddExpandedRecord(FNiagaraConvertPinRecord(ConnectPin->PinId, SrcPath).GetParent());
					}
				}
			}
		}
		SetWiringShown(false);
	}
	else
	{
		check(FromPin);
		check(OppositeDir == EGPD_Input);
		static FNiagaraTypeDefinition SwizTypes[4] =
		{
			FNiagaraTypeDefinition::GetFloatDef(),
			FNiagaraTypeDefinition::GetVec2Def(),
			FNiagaraTypeDefinition::GetVec3Def(),
			FNiagaraTypeDefinition::GetVec4Def()
		};
		static FName SwizComponents[4] =
		{
			TEXT("X"),
			TEXT("Y"),
			TEXT("Z"),
			TEXT("W")
		};

		// TypeDef won't be initialized for swizzles yet
		TypeDef = Schema->PinToTypeDefinition(FromPin);

		UEdGraphPin* ConnectPin = RequestNewTypedPin(OppositeDir, TypeDef);
		check(ConnectPin);
		ConnectPin->MakeLinkTo(FromPin);
		
		check(AutowireSwizzle.Len() <= 4 && AutowireSwizzle.Len() > 0);
		FNiagaraTypeDefinition SwizType = SwizTypes[AutowireSwizzle.Len() - 1];
		UEdGraphPin* NewPin = RequestNewTypedPin(EGPD_Output, SwizType, *SwizType.GetNameText().ToString());

		TArray<FName> SrcPath;
		TArray<FName> DestPath;
		for (int32 i = 0; i < AutowireSwizzle.Len(); ++i)
		{
			TCHAR Char = AutowireSwizzle[i];
			FString CharStr = FString::ConstructFromPtrSize(&Char, 1);
			SrcPath.Empty(SrcPath.Num());
			DestPath.Empty(DestPath.Num());

			SrcPath.Add(*CharStr);
			DestPath.Add(FNiagaraTypeDefinition::IsScalarDefinition(SwizType) ? TEXT("Value") : SwizComponents[i]);
			Connections.Add(FNiagaraConvertConnection(ConnectPin->PinId, SrcPath, NewPin->PinId, DestPath));
			if (DestPath.Num())
			{
				AddExpandedRecord(FNiagaraConvertPinRecord(NewPin->PinId, DestPath).GetParent());
			}

			if (SrcPath.Num())
			{
				AddExpandedRecord(FNiagaraConvertPinRecord(ConnectPin->PinId, SrcPath).GetParent());
			}
		}
	}

	MarkNodeRequiresSynchronization(__FUNCTION__, true);
	//GetGraph()->NotifyGraphChanged();
}

FText UNiagaraNodeConvert::GetNodeTitle(ENodeTitleType::Type TitleType)const
{
	if (!AutowireSwizzle.IsEmpty())
	{
		return FText::FromString(AutowireSwizzle);
	}
	else if (AutowireMakeType.IsValid())
	{
		return FText::Format(LOCTEXT("MakeTitle", "Make {0}"), AutowireMakeType.GetNameText());
	}
	else if (AutowireBreakType.IsValid())
	{
		return FText::Format(LOCTEXT("BreakTitle", "Break {0}"), AutowireBreakType.GetNameText());
	}
	else
	{
		FPinCollectorArray InPins;
		FPinCollectorArray OutPins;
		GetInputPins(InPins);
		GetOutputPins(OutPins);
		if (InPins.Num() == 2 && OutPins.Num() == 2)
		{
			//We are converting one pin type directly to another so we can have a nice name.
			const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
			check(Schema);
			FNiagaraTypeDefinition AType = Schema->PinToTypeDefinition(InPins[0]);
			FNiagaraTypeDefinition BType = Schema->PinToTypeDefinition(OutPins[0]);
			return FText::Format(LOCTEXT("SpecificConvertTitle", "{0} -> {1}"), AType.GetNameText(), BType.GetNameText());
		}
		else
		{
			return LOCTEXT("DefaultTitle", "Convert");
		}

	}
}

TArray<FNiagaraConvertConnection>& UNiagaraNodeConvert::GetConnections()
{
	return Connections;
}

const TArray<FNiagaraConvertConnection>& UNiagaraNodeConvert::GetConnections() const
{
	return Connections;
}

void UNiagaraNodeConvert::OnPinRemoved(UEdGraphPin* PinToRemove)
{
	TSet<FGuid> TypePinIds;
	for (UEdGraphPin* Pin : GetAllPins())
	{
		TypePinIds.Add(Pin->PinId);
	}

	auto RemovePredicate = [&](const FNiagaraConvertConnection& Connection)
	{
		return TypePinIds.Contains(Connection.SourcePinId) == false || TypePinIds.Contains(Connection.DestinationPinId) == false;
	};

	Connections.RemoveAll(RemovePredicate);
}

void UNiagaraNodeConvert::InitAsSwizzle(FString Swiz)
{
	AutowireSwizzle = Swiz;
}

void UNiagaraNodeConvert::InitAsMake(FNiagaraTypeDefinition Type)
{
	AutowireMakeType = Type;
}

void UNiagaraNodeConvert::InitAsBreak(FNiagaraTypeDefinition Type)
{
	AutowireBreakType = Type;
}

bool UNiagaraNodeConvert::InitConversion(UEdGraphPin* FromPin, UEdGraphPin* ToPin)
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	check(Schema);
	FNiagaraTypeDefinition FromType = Schema->PinToTypeDefinition(FromPin);
	FNiagaraTypeDefinition ToType = Schema->PinToTypeDefinition(ToPin);

	//Can only convert normal struct types.
	if (!FromType.IsValid() || !ToType.IsValid() || FromType.GetClass() || ToType.GetClass())
	{
		return false;
	}

	UEdGraphPin* ConnectFromPin = RequestNewTypedPin(EGPD_Input, FromType);
	FromPin->MakeLinkTo(ConnectFromPin);
	UEdGraphPin* ConnectToPin = RequestNewTypedPin(EGPD_Output, ToType);
	// Before we connect our new link, make sure that the old ones are gone.
	ToPin->BreakAllPinLinks();
	ToPin->MakeLinkTo(ConnectToPin);
	check(ConnectFromPin);
	check(ConnectToPin);

	TArray<FName> SrcPath;
	TArray<FName> DestPath;
	TFieldIterator<FProperty> FromPropertyIt(FromType.GetScriptStruct(), EFieldIteratorFlags::IncludeSuper);
	TFieldIterator<FProperty> ToPropertyIt(ToType.GetScriptStruct(), EFieldIteratorFlags::IncludeSuper);

	TFieldIterator<FProperty> NextFromPropertyIt(FromType.GetScriptStruct(), EFieldIteratorFlags::IncludeSuper);
	while (FromPropertyIt && ToPropertyIt)
	{
		FProperty* FromProp = *FromPropertyIt;
		FProperty* ToProp = *ToPropertyIt;
		if (NextFromPropertyIt)
		{
			++NextFromPropertyIt;
		}

		if (Schema->IsValidNiagaraPropertyType(FromProp) && Schema->IsValidNiagaraPropertyType(ToProp))
		{
			FNiagaraTypeDefinition FromPropType = Schema->GetTypeDefForProperty(FromProp);
			FNiagaraTypeDefinition ToPropType = Schema->GetTypeDefForProperty(ToProp);
			SrcPath.Empty();
			DestPath.Empty();
			if (FNiagaraTypeDefinition::TypesAreAssignable(FromPropType, ToPropType) || FNiagaraTypeDefinition::IsLossyConversion(FromPropType, ToPropType))
			{
				SrcPath.Add(*FromProp->GetName());
				DestPath.Add(*ToProp->GetName());
				Connections.Add(FNiagaraConvertConnection(ConnectFromPin->PinId, SrcPath, ConnectToPin->PinId, DestPath));

				if (SrcPath.Num())
				{
					AddExpandedRecord(FNiagaraConvertPinRecord(ConnectFromPin->PinId, SrcPath).GetParent());
				}

				if (DestPath.Num())
				{
					AddExpandedRecord(FNiagaraConvertPinRecord(ConnectToPin->PinId, DestPath).GetParent());
				}
			}
		}
			
		//If there is no next From property, just keep with the same one and set it to all future To properties.
		if (NextFromPropertyIt)
		{
			++FromPropertyIt;
		}		

		++ToPropertyIt;
	}

	return Connections.Num() > 0;
}


bool UNiagaraNodeConvert::IsWiringShown() const
{
	return bIsWiringShown;
}

void UNiagaraNodeConvert::SetWiringShown(bool bInShown)
{
	bIsWiringShown = bInShown;
	VisualsChangedDelegate.Broadcast(this);
}

bool UNiagaraNodeConvert::IsLocalConstantValue() const
{
	FPinCollectorArray Outputs;
	FPinCollectorArray Inputs;
	GetOutputPins(Outputs);
	GetInputPins(Inputs);
	if (Outputs.Num() == 2 && IsAddPin(Outputs.Last()) && Inputs.Num() >= 2 && IsAddPin(Inputs.Last()))
	{
		for (UEdGraphPin* InPin : Inputs)
		{
			if (InPin->LinkedTo.Num() != 0)
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

void UNiagaraNodeConvert::RemoveExpandedRecord(const FNiagaraConvertPinRecord& InRecord)
{
	if (HasExpandedRecord(InRecord) == true)
	{
		FScopedTransaction ConnectTransaction(NSLOCTEXT("NiagaraConvert", "ConvertNodeCollpaseTransaction", "Collapse node."));
		Modify();
		ExpandedItems.Remove(InRecord);
	}
}


bool UNiagaraNodeConvert::HasExpandedRecord(const FNiagaraConvertPinRecord& InRecord)
{
	for (const FNiagaraConvertPinRecord& ExpandedRecord : ExpandedItems)
	{
		if (ExpandedRecord.PinId == InRecord.PinId && ExpandedRecord.Path == InRecord.Path)
		{
			return true;
		}
	}
	return false;
}

void UNiagaraNodeConvert::AddExpandedRecord(const FNiagaraConvertPinRecord& InRecord)
{
	if (HasExpandedRecord(InRecord) == false)
	{
		Modify();
		FScopedTransaction ConnectTransaction(NSLOCTEXT("NiagaraConvert", "ConvertNodeExpandedTransaction", "Expand node."));
		ExpandedItems.AddUnique(InRecord);
	}
}

bool FNiagaraConvertPinRecord::operator ==(const FNiagaraConvertPinRecord & B) const
{
	return (PinId == B.PinId && Path == B.Path);
}

#undef LOCTEXT_NAMESPACE
