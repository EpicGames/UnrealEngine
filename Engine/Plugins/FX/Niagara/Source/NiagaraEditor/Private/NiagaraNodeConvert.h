// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraNodeWithDynamicPins.h"

#include "SGraphPin.h"

#include "NiagaraNodeConvert.generated.h"

class UEdGraphPin;

/** Helper struct that stores the location of a socket.*/
USTRUCT()
struct FNiagaraConvertPinRecord
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FGuid PinId;

	UPROPERTY()
	TArray<FName> Path;

	FNiagaraConvertPinRecord() {}

	FNiagaraConvertPinRecord(FGuid InGuid, const TArray<FName>& InPath) : PinId(InGuid), Path(InPath)
	{
	}

	FNiagaraConvertPinRecord GetParent() const
	{
		FNiagaraConvertPinRecord Record = *this;
		if (Record.Path.Num() > 0)
		{
			if (Record.Path[Record.Path.Num() - 1] == FName())
				Record.Path.RemoveAt(Record.Path.Num() - 1);
			if (Record.Path.Num() > 0)
				Record.Path.RemoveAt(Record.Path.Num() - 1);
		}
		return Record;
	}

	bool operator ==(const FNiagaraConvertPinRecord &) const;
};

/** Helper struct that stores a connection between two sockets.*/
USTRUCT()
struct FNiagaraConvertConnection
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FGuid SourcePinId;

	UPROPERTY()
	TArray<FName> SourcePath;

	UPROPERTY()
	FGuid DestinationPinId;

	UPROPERTY()
	TArray<FName> DestinationPath;

	// Optional Guids for storing the property's Guid within a struct to help with determining if a rename has occurred.
	UPROPERTY()
	FGuid SourcePropertyId;

	UPROPERTY()
	FGuid DestinationPropertyId;

	FNiagaraConvertConnection()
	{
	}

	FNiagaraConvertConnection(FGuid InSourcePinId, const TArray<FName>& InSourcePath, FGuid InDestinationPinId, const TArray<FName>& InDestinationPath, FGuid InSourcePropertyId = FGuid(), FGuid InDestinationPropertyId = FGuid())
		: SourcePinId(InSourcePinId)
		, SourcePath(InSourcePath)
		, DestinationPinId(InDestinationPinId)
		, DestinationPath(InDestinationPath)
		, SourcePropertyId(InSourcePropertyId)
		, DestinationPropertyId(InDestinationPropertyId)
	{
	}

	bool operator ==(const FNiagaraConvertConnection& B) const { return SourcePinId == B.SourcePinId && DestinationPinId == B.DestinationPinId && DestinationPath == B.DestinationPath && SourcePath == B.SourcePath; }

	FString ToString() const;
};

/** A node which allows the user to build a set of arbitrary output types from an arbitrary set of input types by connecting their inner components. */
UCLASS()
class UNiagaraNodeConvert : public UNiagaraNodeWithDynamicPins
{
public:
	GENERATED_BODY()

	UNiagaraNodeConvert();

	//~ Begin UObject interface
	virtual void PostLoad() override;
	//~ End UObject interface

	//~ UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual void  AutowireNewNode(UEdGraphPin* FromPin)override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	//~ UNiagaraNode interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;

	virtual bool RefreshFromExternalChanges() override;

	/** Gets the nodes inner connection. */
	TArray<FNiagaraConvertConnection>& GetConnections();
	const TArray<FNiagaraConvertConnection>& GetConnections() const;

	/** Initializes this node as a swizzle by component string. */
	void InitAsSwizzle(FString Swiz);

	/** Initializes this node as a make node based on an output type. */
	void InitAsMake(FNiagaraTypeDefinition Type);

	/** Initializes this node as a break node based on an input tye. */
	void InitAsBreak(FNiagaraTypeDefinition Type);

	/** Init as an automatic conversion between two types.*/
	bool InitConversion(UEdGraphPin* FromPin, UEdGraphPin* ToPin);

	/** Do we show the internal switchboard UI?*/
	bool IsWiringShown() const;
	
	/** Show internal switchboard UI.*/
	void SetWiringShown(bool bInShown);

	/** Remove that a socket is collapsed.*/
	void RemoveExpandedRecord(const FNiagaraConvertPinRecord& InRecord);
	/** Store that a socket is expanded.*/
	void AddExpandedRecord(const FNiagaraConvertPinRecord& InRecord);
	/** Is this socket expanded?*/
	bool HasExpandedRecord(const FNiagaraConvertPinRecord& InRecord);

	/** Returns true if the node has a constant value input and a single output */
	bool IsLocalConstantValue() const;
	
private:
	//~ EdGraphNode interface
	virtual void OnPinRemoved(UEdGraphPin* Pin) override;

	//A swizzle string set externally to instruct the autowiring code.
	UPROPERTY()
	FString AutowireSwizzle;

	//A type def used when auto wiring up the convert node.
	UPROPERTY()
	FNiagaraTypeDefinition AutowireMakeType;
	UPROPERTY()
	FNiagaraTypeDefinition AutowireBreakType;

	/** The internal connections for this node. */
	UPROPERTY()
	TArray<FNiagaraConvertConnection> Connections;

	/** Is the switcboard UI shown?*/
	UPROPERTY()
	bool bIsWiringShown;

	/** Store of all sockets that are expanded.*/
	UPROPERTY()
	TArray<FNiagaraConvertPinRecord> ExpandedItems;
};
