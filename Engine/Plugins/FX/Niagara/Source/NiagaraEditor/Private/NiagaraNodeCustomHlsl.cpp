// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeCustomHlsl.h"

#include "EdGraphSchema_Niagara.h"
#include "Misc/FileHelper.h"
#include "NiagaraGraph.h"
#include "NiagaraHlslTranslator.h"
#include "ScopedTransaction.h"
#include "Widgets/SNiagaraGraphNodeCustomHlsl.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeCustomHlsl)

#define LOCTEXT_NAMESPACE "NiagaraNodeCustomHlsl"

UNiagaraNodeCustomHlsl::UNiagaraNodeCustomHlsl(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PinPendingRename = nullptr;
	bCanRenameNode = true;
	ScriptUsage = ENiagaraScriptUsage::Function;

	Signature.Name = TEXT("Custom Hlsl");
	FunctionDisplayName = Signature.Name.ToString();

	bIsShaderCodeShown = true;
}

const FString& UNiagaraNodeCustomHlsl::GetCustomHlsl() const
{
	return CustomHlsl;
}

void UNiagaraNodeCustomHlsl::SetCustomHlsl(const FString& InCustomHlsl)
{
	Modify();
	CustomHlsl = InCustomHlsl;
	RefreshFromExternalChanges();
	if (GetOuter()->IsA<UNiagaraGraph>())
	{
		// This is needed to guard against a crash when setting this value before the node has actually been
		// added to a graph.
		MarkNodeRequiresSynchronization(__FUNCTION__, true);
	}
}

bool UNiagaraNodeCustomHlsl::IsShaderCodeShown() const
{
	return bIsShaderCodeShown;
}

void UNiagaraNodeCustomHlsl::SetShaderCodeShown(bool bInShown)
{
	if (bIsShaderCodeShown != bInShown)
	{
		Modify();
		bIsShaderCodeShown = bInShown;
		PostEditChange();
	}
}

void UNiagaraNodeCustomHlsl::GetIncludeFilePaths(TArray<FNiagaraCustomHlslInclude>& OutCustomHlslIncludeFilePaths) const
{
	for (const FString& FilePath : VirtualIncludeFilePaths)
	{
		if (!FilePath.IsEmpty())
		{			
			OutCustomHlslIncludeFilePaths.Add({true, FilePath});
		}
	}
	for (const auto& [FilePath] : AbsoluteIncludeFilePaths)
	{
		if (!FilePath.IsEmpty())
		{			
			OutCustomHlslIncludeFilePaths.Add({false, FilePath});
		}
	}
}

TSharedPtr<SGraphNode> UNiagaraNodeCustomHlsl::CreateVisualWidget()
{
	return SNew(SNiagaraGraphNodeCustomHlsl, this);
}

void UNiagaraNodeCustomHlsl::OnRenameNode(const FString& NewName)
{
	Signature.Name = *NewName;
	FunctionDisplayName = NewName;
}

FText UNiagaraNodeCustomHlsl::GetHlslText() const
{
	return FText::FromString(CustomHlsl);
}

void UNiagaraNodeCustomHlsl::OnCustomHlslTextCommitted(const FText& InText, ETextCommit::Type InType)
{
	FString NewValue = InText.ToString();
	if (!NewValue.Equals(CustomHlsl, ESearchCase::CaseSensitive))
	{
		FScopedTransaction Transaction(LOCTEXT("CustomHlslCommit", "Edited Custom Hlsl"));
		SetCustomHlsl(NewValue);
	}
}

FLinearColor UNiagaraNodeCustomHlsl::GetNodeTitleColor() const
{
	return UEdGraphSchema_Niagara::NodeTitleColor_CustomHlsl;
}

FText UNiagaraNodeCustomHlsl::GetTooltipText() const
{
	return LOCTEXT("CustomHlslTooltip", "Inserts the entered hlsl code into the translated script.");
}

bool UNiagaraNodeCustomHlsl::GetTokensFromString(const FString& InHlsl, TArray<FStringView>& OutTokens, bool IncludeComments, bool IncludeWhitespace)
{
	if (InHlsl.Len() == 0)
	{
		return false;
	}

	FString Separators = TEXT("!;/*+-=)(?:, []<>\"\t\r\n{}");
	const int32 TargetLength = InHlsl.Len();

	int32 TokenStart = 0;
	bool TokenIsWhitespace = true;

	auto AddToken = [&](bool DoAdd, FStringView TokenString)
	{
		if (DoAdd && (!TokenIsWhitespace || IncludeWhitespace))
		{
			OutTokens.Add(TokenString);
		}
		
		// reset the meta data about the token
		TokenIsWhitespace = true;
	};

	for (int32 i = 0; i < TargetLength; )
	{
		int32 Index = INDEX_NONE;

		const bool bWhitespace = FChar::IsWhitespace(InHlsl[i]);

		// Determine if we are a splitter character or a regular character.
		if (Separators.FindChar(InHlsl[i], Index) && Index != INDEX_NONE)
		{
			// Commit the current token, if any.
			if (i > TokenStart)
			{
				AddToken(true, FStringView(InHlsl).Mid(TokenStart, i - TokenStart));
			}

			if (!bWhitespace)
			{
				TokenIsWhitespace = false;
			}

			if (InHlsl[i] == '/' && (i + 1 != TargetLength) && InHlsl[i + 1] == '/')
			{
				// Single-line comment, everything up to the end of the line becomes a token (including the comment start and the newline).
				int32 FoundEndIdx = InHlsl.Find("\n", ESearchCase::CaseSensitive, ESearchDir::FromStart, i + 2);
				if (FoundEndIdx == INDEX_NONE)
				{
					FoundEndIdx = TargetLength - 1;
				}

				AddToken(IncludeComments, FStringView(InHlsl).Mid(i, FoundEndIdx - i + 1));
				i = FoundEndIdx + 1;
			}
			else if (InHlsl[i] == '/' && (i + 1 != TargetLength) && InHlsl[i + 1] == '*')
			{
				// Multi-line comment, all of it becomes a single token, including the start and end markers.
				int32 FoundEndIdx = InHlsl.Find("*/", ESearchCase::CaseSensitive, ESearchDir::FromStart, i + 2);
				if (FoundEndIdx != INDEX_NONE)
				{
					// Include both characters of the terminator.
					FoundEndIdx += 1;
				}
				else
				{
					// This is an unterminated multi-line comment, but there's nothing we can do at this point.
					FoundEndIdx = TargetLength - 1;
				}

				AddToken(IncludeComments, FStringView(InHlsl).Mid(i, FoundEndIdx - i + 1));
				i = FoundEndIdx + 1;
			}
			else if (InHlsl[i] == '"')
			{
				// Strings in HLSL, what?
				// This is an extension used to support calling DI functions which have specifiers. The syntax is:
				//		DIName.Function<Specifier1="Value 1", Specifier2="Value 2">();
				// The string is considered a single token, including the quotation marks.
				int32 FoundEndIdx = InHlsl.Find("\"", ESearchCase::CaseSensitive, ESearchDir::FromStart, i + 1);
				if (FoundEndIdx == INDEX_NONE)
				{
					// Unterminated string. A very weird compiler error will follow, but there's nothing we can do at this point.
					FoundEndIdx = TargetLength - 1;
				}

				AddToken(true, FStringView(InHlsl).Mid(i, FoundEndIdx - i + 1));
				i = FoundEndIdx + 1;

			}
			else
			{
				AddToken(true, FStringView(InHlsl).Mid(i, 1));
				i++;
			}

			// Start a new token after the separator.
			TokenStart = i;
		}
		else
		{
			if (!bWhitespace)
			{
				TokenIsWhitespace = false;
			}

			// This character is part of a token, continue scanning.
			i++;
		}
	}

	// We may need to pull in the last chars from the end.
	if (TokenStart < TargetLength)
	{
		AddToken(true, FStringView(InHlsl).Mid(TokenStart));
	}
	return true;
}

bool UNiagaraNodeCustomHlsl::GetTokens(TArray<FStringView>& OutTokens, bool IncludeComments, bool IncludeWhitespace) const
{
	return GetTokensFromString(CustomHlsl, OutTokens, IncludeComments, IncludeWhitespace);
}

void UNiagaraNodeCustomHlsl::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bRequiresRecompilation = false;

	if (PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraNodeCustomHlsl, CustomHlsl))
		{
			bRequiresRecompilation = true;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraNodeCustomHlsl, AbsoluteIncludeFilePaths))
		{
			bRequiresRecompilation = true;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraNodeCustomHlsl, VirtualIncludeFilePaths))
		{
			bRequiresRecompilation = true;
		}
	}

	if (bRequiresRecompilation)
	{
		RefreshFromExternalChanges();
		GetNiagaraGraph()->NotifyGraphNeedsRecompile();
	}
}

void UNiagaraNodeCustomHlsl::InitAsCustomHlslDynamicInput(const FNiagaraTypeDefinition& OutputType)
{
	Modify();
	ReallocatePins();
	RequestNewTypedPin(EGPD_Input, FNiagaraTypeDefinition::GetParameterMapDef(), FName("Map"));
	RequestNewTypedPin(EGPD_Output, OutputType, FName("CustomHLSLOutput"));
	ScriptUsage = ENiagaraScriptUsage::DynamicInput;
}

bool UNiagaraNodeCustomHlsl::CallsImpureDataInterfaceFunctions() const
{
	TArray<FString> ImpureFunctionNames;

	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	for (const UEdGraphPin* InputPin : InputPins)
	{
		FNiagaraTypeDefinition NiagaraType = UEdGraphSchema_Niagara::PinToTypeDefinition(InputPin, ENiagaraStructConversion::Simulation);
		if (NiagaraType.IsDataInterface())
		{
			if (UNiagaraDataInterface* DataInterfaceClass = CastChecked<UNiagaraDataInterface>(NiagaraType.GetClass()->GetDefaultObject(false)))
			{
				TArray<FNiagaraFunctionSignature> FunctionSignatures;
				DataInterfaceClass->GetFunctionSignatures(FunctionSignatures);

				for (const FNiagaraFunctionSignature& FunctionSignature : FunctionSignatures)
				{
					if (FunctionSignature.bRequiresExecPin)
					{
						TStringBuilder<256> Builder;
						InputPin->PinName.AppendString(Builder);
						Builder.AppendChar(TCHAR('.'));
						FunctionSignature.Name.AppendString(Builder);

						ImpureFunctionNames.AddUnique(Builder.ToString());
					}
				}
			}
		}
	}

	if (!ImpureFunctionNames.IsEmpty())
	{
		TArray<FStringView> CustomHlslTokens;
		GetTokensFromString(CustomHlsl, CustomHlslTokens, false, false);

		for (const FString& ImpureFunctionName : ImpureFunctionNames)
		{
			if (CustomHlslTokens.Contains(ImpureFunctionName))
			{
				return true;
			}
		}
	}

	return false;

}

bool UNiagaraNodeCustomHlsl::IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const
{
	if (GraphPinObj == PinPendingRename && ScriptUsage != ENiagaraScriptUsage::DynamicInput)
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool UNiagaraNodeCustomHlsl::IsPinNameEditable(const UEdGraphPin* GraphPinObj) const
{

	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(GraphPinObj);
	if (TypeDef.IsValid() && GraphPinObj &&  CanRenamePin(GraphPinObj) && ScriptUsage != ENiagaraScriptUsage::DynamicInput)
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool UNiagaraNodeCustomHlsl::VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj)  const
{
	// Check to see if the symbol has to be mangled to be valid hlsl. If it does, then prevent it from being 
	// valid. This helps clear up any ambiguity downstream in the translator.
	FString NewName = InName.ToString();
	FString SanitizedNewName = FNiagaraHlslTranslator::GetSanitizedSymbolName(NewName);

	if (NewName != SanitizedNewName || NewName.Len() == 0)
	{
		OutErrorMessage = FText::Format(LOCTEXT("InvalidPinName_Restricted", "Pin \"{0}\" cannot be renamed to \"{1}\". Certain words are restricted, as are spaces and special characters. Suggestion: \"{2}\""), InGraphPinObj->GetDisplayName(), InName, FText::FromString(SanitizedNewName));
		return false;
	}
	TSet<FName> Names;
	for (int32 i = 0; i < Pins.Num(); i++)
	{
		if (Pins[i] != InGraphPinObj)
			Names.Add(Pins[i]->GetFName());
	}
	if (Names.Contains(*NewName))
	{
		OutErrorMessage = FText::Format(LOCTEXT("InvalidPinName_Conflicts", "Pin \"{0}\" cannot be renamed to \"{1}\" as it conflicts with another name in use. Suggestion: \"{2}\""), InGraphPinObj->GetDisplayName(), InName, FText::FromName(FNiagaraUtilities::GetUniqueName(*SanitizedNewName, Names)));
		return false;
	}
	OutErrorMessage = FText::GetEmpty();
	return true;
}

bool UNiagaraNodeCustomHlsl::CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents)
{
	if (Pins.Contains(InGraphPinObj))
	{
		FScopedTransaction AddNewPinTransaction(LOCTEXT("Rename Pin", "Renamed pin"));
		Modify();
		InGraphPinObj->Modify();

		FString OldPinName = InGraphPinObj->PinName.ToString();
		InGraphPinObj->PinName = *InName.ToString();
		InGraphPinObj->PinFriendlyName = InName;
		if (bSuppressEvents == false)
			OnPinRenamed(InGraphPinObj, OldPinName);

		return true;
	}
	return false;
}

bool UNiagaraNodeCustomHlsl::CancelEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj)
{
	if (InGraphPinObj == PinPendingRename)
	{
		PinPendingRename = nullptr;
	}
	return true;
}


/** Called when a new typed pin is added by the user. */
void UNiagaraNodeCustomHlsl::OnNewTypedPinAdded(UEdGraphPin*& NewPin)
{
	TSet<FName> Names;
	for (int32 i = 0; i < Pins.Num(); i++)
	{
		if (Pins[i] != NewPin)
			Names.Add(Pins[i]->GetFName());
	}
	FNameBuilder OriginalPinName(NewPin->GetFName());
	const FName SanitizedName = *FNiagaraHlslTranslator::GetSanitizedSymbolName(OriginalPinName.ToView());
	FName Name = FNiagaraUtilities::GetUniqueName(SanitizedName, Names);
	NewPin->PinName = Name;
	UNiagaraNodeWithDynamicPins::OnNewTypedPinAdded(NewPin);
	RebuildSignatureFromPins();
	PinPendingRename = NewPin;
}

/** Called when a pin is renamed. */
void UNiagaraNodeCustomHlsl::OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldPinName)
{
	UNiagaraNodeWithDynamicPins::OnPinRenamed(RenamedPin, OldPinName);
	RebuildSignatureFromPins();
}

/** Removes a pin from this node with a transaction. */
void UNiagaraNodeCustomHlsl::RemoveDynamicPin(UEdGraphPin* Pin)
{
	UNiagaraNodeWithDynamicPins::RemoveDynamicPin(Pin);
	RebuildSignatureFromPins();
}

void UNiagaraNodeCustomHlsl::MoveDynamicPin(UEdGraphPin* Pin, int32 DirectionToMove)
{
	UNiagaraNodeWithDynamicPins::MoveDynamicPin(Pin, DirectionToMove);
	RebuildSignatureFromPins();
}

void UNiagaraNodeCustomHlsl::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	Super::BuildParameterMapHistory(OutHistory, bRecursive, bFilterForCompilation);
	if (!IsNodeEnabled() && OutHistory.GetIgnoreDisabled())
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	TArray<FStringView> TokenViews;
	GetTokens(TokenViews, false, false);
	TArray<FString> Tokens;
	Tokens.Reset(TokenViews.Num());
	for (const FStringView View : TokenViews)
	{
		Tokens.Push(FString(View));
	}

	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);

	int32 ParamMapIdx = INDEX_NONE;
	// This only works currently if the input pins are in the same order as the signature pins.
	if (InputPins.Num() == Signature.Inputs.Num() + 1 && OutputPins.Num() == Signature.Outputs.Num() + 1)// the add pin is extra
	{
		TArray<FNiagaraVariable> LocalVars;
		bool bHasParamMapInput = false;
		bool bHasParamMapOutput = false;
		for (int32 i = 0; i < InputPins.Num(); i++)
		{
			if (IsAddPin(InputPins[i]))
				continue;

			FNiagaraVariable Input = Signature.Inputs[i];
			if (Input.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				bHasParamMapInput = true;
				if (InputPins[i]->LinkedTo.Num() != 0)
				{
					ParamMapIdx = OutHistory.TraceParameterMapOutputPin(InputPins[i]->LinkedTo[0]);
				}
			}
			else
			{
				LocalVars.Add(Input);
			}
		}

		for (int32 i = 0; i < OutputPins.Num(); i++)
		{
			if (IsAddPin(OutputPins[i]))
				continue;

			FNiagaraVariable Output = Signature.Outputs[i];
			if (Output.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				bHasParamMapOutput = true;
				OutHistory.RegisterParameterMapPin(ParamMapIdx, OutputPins[i]);
			}
			else
			{
				LocalVars.Add(Output);
			}
		}

		TArray<FString> PossibleNamespaces;
		FNiagaraParameterUtilities::GetValidNamespacesForReading(OutHistory.GetBaseUsageContext(), 0, PossibleNamespaces);

		if ((bHasParamMapOutput || bHasParamMapInput) && ParamMapIdx != INDEX_NONE)
		{
			for (int32 i = 0; i < Tokens.Num(); i++)
			{
				bool bFoundLocal = false;
				if (INDEX_NONE != FNiagaraVariable::SearchArrayForPartialNameMatch(LocalVars, *Tokens[i]))
				{
					bFoundLocal = true;
				}

				if (!bFoundLocal && Tokens[i].Contains(TEXT("."))) // Only check tokens with namespaces in them..
				{
					for (const FString& ValidNamespace : PossibleNamespaces)
					{
						// There is one possible path here, one where we're using the namespace as-is from the valid list.
						if (Tokens[i].StartsWith(ValidNamespace, ESearchCase::CaseSensitive))
						{
							OutHistory.HandleExternalVariableRead(ParamMapIdx, *Tokens[i]);
						}
					}
				}
			}
		}
	}
}

FNiagaraCompileHash GetFileContentHash(const FString& FileContents)
{
	FSHA1 CompileHash;
	CompileHash.UpdateWithString(*FileContents, FileContents.Len());
	CompileHash.Final();

	TArray<uint8> DataHash;
	DataHash.AddUninitialized(FSHA1::DigestSize);
	CompileHash.GetHash(DataHash.GetData());

	return FNiagaraCompileHash(DataHash);
}

void UNiagaraNodeCustomHlsl::GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const
{
	for (const auto& [IncludePath] : AbsoluteIncludeFilePaths)
	{
		if (IncludePath.IsEmpty())
		{
			continue;
		}

		if (FString FileContents; FFileHelper::LoadFileToString(FileContents, *IncludePath))
		{
			HashCollector.AddHash(GetFileContentHash(FileContents), IncludePath);
		}
	}

	for (const FString& IncludePath : VirtualIncludeFilePaths)
	{
		if (IncludePath.IsEmpty())
		{
			continue;
		}
		
		FString FileContents;
		if(LoadShaderSourceFile(*IncludePath, SP_PCD3D_SM5, &FileContents, nullptr))
		{
			HashCollector.AddHash(GetFileContentHash(FileContents), IncludePath);
		}
	}
}

// Replace items in the tokens array if they start with the src string or optionally src string and a namespace delimiter
uint32 UNiagaraNodeCustomHlsl::ReplaceExactMatchTokens(TArray<FString>& Tokens, FStringView SrcString, FStringView ReplaceString, bool bAllowNamespaceSeparation)
{
	const int32 SrcLength = SrcString.Len();

	uint32 Count = 0;
	for (int32 i = 0; i < Tokens.Num(); i++)
	{
		if (FStringView(Tokens[i]).StartsWith(SrcString, ESearchCase::CaseSensitive))
		{
			const int32 TokenLength = Tokens[i].Len();

			if (TokenLength > SrcLength)
			{
				if (bAllowNamespaceSeparation && Tokens[i][SrcLength] == TCHAR('.'))
				{
					Tokens[i] = ReplaceString + Tokens[i].Mid(SrcLength);
					++Count;
				}
			}
			else
			{
				Tokens[i] = ReplaceString;
				++Count;
			}
		}
	}
	return Count;
}


bool UNiagaraNodeCustomHlsl::AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const
{
	return Super::AllowNiagaraTypeForAddPin(InType) || InType.IsDataInterface();
}

bool UNiagaraNodeCustomHlsl::AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType, EEdGraphPinDirection InDirection) const
{
	if (AllowNiagaraTypeForAddPin(InType))
	{
		if (InType.IsStatic() && InDirection == EGPD_Output)
			return false;
		else
			return true;
	}
	return false;
}

bool UNiagaraNodeCustomHlsl::ReferencesVariable(const FNiagaraVariableBase& InVar) const
{
	// for now we'll just do a text search through the non-comment code strings to see if we can find
	// the name of the provided variable
	// todo - all variable references in custom code should be explicit and typed
	TArray<FStringView> Tokens;

	GetTokens(Tokens, false, false);

	const FString VariableName = InVar.GetName().ToString();

	for (const FStringView& Token : Tokens)
	{
		if (Token.Contains(VariableName))
		{
			return true;
		}
	}

	return false;
}

void UNiagaraNodeCustomHlsl::RebuildSignatureFromPins()
{
	Modify();
	FNiagaraFunctionSignature Sig = Signature;
	Sig.Inputs.Empty();
	Sig.Outputs.Empty();

	FPinCollectorArray InputPins;
	FPinCollectorArray OutputPins;
	GetInputPins(InputPins);
	GetOutputPins(OutputPins);

	const UEdGraphSchema_Niagara* Schema = Cast<UEdGraphSchema_Niagara>(GetSchema());

	for (UEdGraphPin* Pin : InputPins)
	{
		if (IsAddPin(Pin))
		{
			continue;
		}
		Sig.Inputs.Add(Schema->PinToNiagaraVariable(Pin, true));
	}

	for (UEdGraphPin* Pin : OutputPins)
	{
		if (IsAddPin(Pin))
		{
			continue;
		}
		Sig.Outputs.Add(Schema->PinToNiagaraVariable(Pin, false));
	}

	Signature = Sig;
}

#undef LOCTEXT_NAMESPACE


