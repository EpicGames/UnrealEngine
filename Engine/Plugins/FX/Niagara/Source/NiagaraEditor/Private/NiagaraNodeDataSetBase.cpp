// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeDataSetBase.h"
#include "UObject/UnrealType.h"
#include "INiagaraCompiler.h"
#include "NiagaraEvents.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeDataSetBase)

#define LOCTEXT_NAMESPACE "UNiagaraNodeDataSetBase"

const FName UNiagaraNodeDataSetBase::ConditionVarName(TEXT("__Condition"));
const FName UNiagaraNodeDataSetBase::ParamMapInVarName(TEXT("__ParamMapIn"));
const FName UNiagaraNodeDataSetBase::ParamMapOutVarName(TEXT("__ParamMapOut"));

UNiagaraNodeDataSetBase::UNiagaraNodeDataSetBase(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer), ExternalStructAsset(nullptr)
{
}

bool UNiagaraNodeDataSetBase::InitializeFromStruct(const UStruct* PayloadStruct)
{
	if (InitializeFromStructInternal(PayloadStruct))
	{
		//ReallocatePins();
		return true;
	}
	return false;
}

bool UNiagaraNodeDataSetBase::InitializeFromStructInternal(const UStruct* PayloadStruct)
{
	if (!PayloadStruct)
	{
		return false;
	}

	ExternalStructAsset = PayloadStruct;

	return SynchronizeWithStruct();
}


void UNiagaraNodeDataSetBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		ReallocatePins();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}


FLinearColor UNiagaraNodeDataSetBase::GetNodeTitleColor() const
{
	check(DataSet.Type == ENiagaraDataSetType::Event);//Implement other datasets
	return CastChecked<UEdGraphSchema_Niagara>(GetSchema())->NodeTitleColor_Event;
}

void UNiagaraNodeDataSetBase::AddParameterMapPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	UEdGraphPin* ParamMapInPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), ParamMapInVarName, 0);
	ParamMapInPin->bDefaultValueIsIgnored = true;
	ParamMapInPin->PinFriendlyName = LOCTEXT("UNiagaraNodeWriteDataSetParamMapInPin", "ParamMap");
	UEdGraphPin* ParamMapOutPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), ParamMapOutVarName, 1);
	ParamMapOutPin->bDefaultValueIsIgnored = true;
	ParamMapOutPin->PinFriendlyName = LOCTEXT("UNiagaraNodeWriteDataSetParamMapOutPin", "ParamMap");
}

void UNiagaraNodeDataSetBase::PostLoad()
{
	Super::PostLoad();
	if (ExternalStructAsset  == nullptr)
	{
		const TArray<FNiagaraTypeDefinition>& PayloadTypes = FNiagaraTypeRegistry::GetRegisteredPayloadTypes();
		const FNiagaraTypeDefinition* PayloadType = PayloadTypes.FindByPredicate([&](const FNiagaraTypeDefinition& Type) { return Type.GetName() == DataSet.Name.ToString(); });
		if (PayloadType != nullptr && PayloadType->GetStruct() != nullptr)
		{
			ExternalStructAsset = PayloadType->GetStruct();
		}
	}

	/*if (ExternalStructAsset != nullptr)
	{
		UE_LOGF(LogNiagaraEditor, Display, "Niagara script '%ls' references struct asset '%ls'", *GetFullName(), *ExternalStructAsset->GetFullName());
	}*/

	// The struct asset can still be nullptr, so make sure to synchronize only if it's valid
	if (ExternalStructAsset != nullptr && !IsSynchronizedWithStruct(true, nullptr, true))
	{
		SynchronizeWithStruct();
		ReallocatePins();
	}

	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

	if (NiagaraVer < FNiagaraCustomVersion::AddingParamMapToDataSetBaseNode)
	{
		ReallocatePins();
	}
}

bool UNiagaraNodeDataSetBase::IsSynchronizedWithStruct(bool bIgnoreConditionVariable, FString* Issues, bool bLogIssues) const
{
	if (ExternalStructAsset != nullptr)
	{
		bool bFoundIssues = false;
		// First check to see if any variables have been added...
		{
			for (TFieldIterator<FProperty> PropertyIt(ExternalStructAsset, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				const FProperty* Property = *PropertyIt;

				if (bIgnoreConditionVariable && Property->IsA<FBoolProperty>() && Property->GetFName() == ConditionVarName)
				{
					continue;
				}

				const FNiagaraVariable* VarFound = Variables.FindByPredicate([&](const FNiagaraVariable& Var) { return Var.GetName() == Property->GetFName(); });
				if (VarFound == nullptr)
				{
					bFoundIssues = true;
					if (bLogIssues)
					{
						UE_LOGF(LogNiagaraEditor, Warning, "%ls Unable to find matching variable for struct property: '%ls' ... possible add?", *GetPathNameSafe(this), *GetPathNameSafe(Property));
					}
					if (Issues != nullptr)
					{
						Issues->Append(FString::Printf(TEXT("Unable to find matching variable for struct property: '%s' ... possible add?\n"), *Property->GetName()));
					}
				}
				else
				{
					FNiagaraTypeDefinition TypeDefFound;
					if (GetSupportedNiagaraTypeDef(Property, TypeDefFound))
					{
						if (VarFound->GetType() != TypeDefFound)
						{
							bFoundIssues = true;
							if (bLogIssues)
							{
								UE_LOGF(LogNiagaraEditor, Warning, "%ls Matching variable for struct property '%ls', but different type:  Existing Type:'%ls' vs New Type:'%ls' ... possible type change in user-defined struct?", *GetPathNameSafe(this), *GetPathNameSafe(Property), *VarFound->GetType().GetName(), *TypeDefFound.GetName());
							}
							if (Issues != nullptr)
							{
								Issues->Append(FString::Printf(TEXT("Matching variable for struct property '%s', but different type: Existing Type:'%s' vs New Type:'%s' ... possible type change in user-defined struct?\n"), *Property->GetName(), *VarFound->GetType().GetName(), *TypeDefFound.GetName()));
							}
						}
					}
					else
					{
						bFoundIssues = true;
						if (bLogIssues)
						{
							UE_LOGF(LogNiagaraEditor, Warning, "%ls Matching variable for struct property '%ls', but different type:  Existing Type:'%ls' vs New Type:'Unsupported' ... possible type change in user-defined struct?", *GetPathNameSafe(this), *GetPathNameSafe(Property), *VarFound->GetType().GetName());
						}
						if (Issues != nullptr)
						{
							Issues->Append(FString::Printf(TEXT("Matching variable for struct property '%s', but different type: Existing Type:'%s' vs New Type:'Unsupported' ... possible type change in user-defined struct?\n"), *Property->GetName(), *VarFound->GetType().GetName()));
						}
					}
				}
			}
		}

		// Now check to see if any variables have been removed..
		{
			for (const FNiagaraVariable& Var : Variables)
			{
				bool bFound = false;
				for (TFieldIterator<FProperty> PropertyIt(ExternalStructAsset, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
				{
					const FProperty* Property = *PropertyIt;
					if (Property->GetName() == Var.GetName().ToString())
					{
						bFound = true;
						break;
					}
				}

				if (!bFound)
				{
					bFoundIssues = true;
					if (bLogIssues)
					{
						UE_LOGF(LogNiagaraEditor, Warning, "%ls Unable to find matching struct property for variable: '%ls' ... possible removal?", *GetPathNameSafe(this), *Var.GetName().ToString());
					}

					if (Issues != nullptr)
					{
						Issues->Append(FString::Printf(TEXT("Unable to find matching struct property for variable: '%s' ... possible removal?\n"), *Var.GetName().ToString()));
					}
				}
			}
		}
		return !bFoundIssues;
	}
	else
	{
		if (bLogIssues)
		{
			UE_LOGF(LogNiagaraEditor, Warning, "%ls Unable to find matching Niagara Type: %ls", *GetPathNameSafe(this), *DataSet.Name.ToString());
		}

		if (Issues != nullptr)
		{
			Issues->Append(FString::Printf(TEXT("Unable to find matching Niagara Type: %s\n"), *DataSet.Name.ToString()));
		}
		return false;
	}
}

bool UNiagaraNodeDataSetBase::RefreshFromExternalChanges()
{
	if (!IsSynchronizedWithStruct(true, nullptr, false))
	{
		SynchronizeWithStruct();
		ReallocatePins();
		return true;
	}
	return false;
}

bool UNiagaraNodeDataSetBase::GetSupportedNiagaraTypeDef(const FProperty* Property, FNiagaraTypeDefinition& TypeDef) const
{
	const FStructProperty* StructProp = CastField<FStructProperty>(Property);
	if (Property->IsA(FFloatProperty::StaticClass()))
	{
		TypeDef = FNiagaraTypeDefinition::GetFloatDef();
		return true;
	}
	else if (Property->IsA(FDoubleProperty::StaticClass()))
	{
		TypeDef = FNiagaraTypeDefinition::GetFloatDef();
		return true;
	}
	else if (Property->IsA(FBoolProperty::StaticClass()))
	{
		TypeDef = FNiagaraTypeDefinition::GetBoolDef();
		return true;
	}
	else if (Property->IsA(FIntProperty::StaticClass()))
	{
		TypeDef = FNiagaraTypeDefinition::GetIntDef();
		return true;
	}
	else if (StructProp && StructProp->Struct == FNiagaraTypeDefinition::GetVec2Struct())
	{
		TypeDef = FNiagaraTypeDefinition::GetVec2Def();
		return true;
	}
	else if (StructProp && StructProp->Struct == FNiagaraTypeDefinition::GetVec3Struct())
	{
		TypeDef = FNiagaraTypeDefinition::GetVec3Def();
		return true;
	}
	else if (StructProp && StructProp->Struct == FNiagaraTypeDefinition::GetVec4Struct())
	{
		TypeDef = FNiagaraTypeDefinition::GetVec4Def();
		return true;
	}
	else if (StructProp && StructProp->Struct == FNiagaraTypeDefinition::GetColorStruct())
	{
		TypeDef = FNiagaraTypeDefinition::GetColorDef();
		return true;
	}
	else if (StructProp && StructProp->Struct == FNiagaraTypeDefinition::GetPositionStruct())
	{
		TypeDef = FNiagaraTypeDefinition::GetPositionDef();
		return true;
	}
	else if (StructProp && StructProp->Struct == FNiagaraTypeDefinition::GetQuatStruct())
	{
		TypeDef = FNiagaraTypeDefinition::GetQuatDef();
		return true;
	}
	else if (StructProp && StructProp->Struct)
	{
		TypeDef = FNiagaraTypeDefinition(FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProp->Struct, ENiagaraStructConversion::UserFacing));
		return FNiagaraTypeRegistry::GetRegisteredParameterTypes().Contains(TypeDef);
	}
	return false;
}

bool UNiagaraNodeDataSetBase::SynchronizeWithStruct()
{

	Variables.Empty();
	VariableFriendlyNames.Empty();
	DataSet = FNiagaraDataSetID();

	// TODO: need to add valid as a variable separately for now; compiler support to validate index is missing	
	//Variables.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), "Valid"));

	//UGH! Why do we have our own custom type rep again. Why aren't we just using the FPropertySystem?
	//
	// [OP] not really different from anywhere else, nodes everywhere else hold FNiagaraVariables as their outputs; 
	//  just traversing the ustruct here to build an array of those; this is temporary and should be genericised, of course
	for (TFieldIterator<FProperty> PropertyIt(ExternalStructAsset, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;
		FText DisplayNameText = Property->GetDisplayNameText();
		FString DisplayName = DisplayNameText.ToString();
		FNiagaraTypeDefinition TypeDef;

		if (GetSupportedNiagaraTypeDef(Property, TypeDef))
		{
			Variables.Add(FNiagaraVariable(TypeDef, *Property->GetName()));
			VariableFriendlyNames.Add(DisplayName);
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Could not add property %ls in struct %ls to NiagaraNodeDataSetBase, class %ls", *Property->GetName(), *ExternalStructAsset->GetName(), ExternalStructAsset->GetClass() ? *ExternalStructAsset->GetClass()->GetName() : *FString("nullptr"));
		}
	}

	DataSet.Name = *ExternalStructAsset->GetName();
	return true;
}


#undef LOCTEXT_NAMESPACE




