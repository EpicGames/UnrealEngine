// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeEmitter.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "NiagaraEditorUtilities.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraCommon.h"
#include "NiagaraScriptSource.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraHlslTranslator.h"
#include "Stats/Stats.h"
#include "NiagaraEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeEmitter)

#define LOCTEXT_NAMESPACE "NiagaraNodeEmitter"

DECLARE_CYCLE_STAT(TEXT("Niagara - Module - NiagaraNodeEmitter_Compile"), STAT_NiagaraEditor_Module_NiagaraNodeEmitter_Compile, STATGROUP_NiagaraEditor);
#define NIAGARA_SCOPE_CYCLE_COUNTER(x) //SCOPE_CYCLE_COUNTER(x)

void UNiagaraNodeEmitter::PostInitProperties()
{
	Super::PostInitProperties();
	PinPendingRename = nullptr;
	CachedGraphWeakPtr = nullptr;
	CachedScriptSourceWeakPtr = nullptr;
}

UNiagaraSystem* UNiagaraNodeEmitter::GetOwnerSystem() const
{
	return OwnerSystem;
}

void UNiagaraNodeEmitter::SetOwnerSystem(UNiagaraSystem* InOwnerSystem)
{
	OwnerSystem = InOwnerSystem;
	RefreshFromExternalChanges();
}

FGuid UNiagaraNodeEmitter::GetEmitterHandleId() const
{
	return EmitterHandleId;
}

void UNiagaraNodeEmitter::SetEmitterHandleId(FGuid InEmitterHandleId)
{
	EmitterHandleId = InEmitterHandleId;
	DisplayName = GetNameFromEmitter();
}

void UNiagaraNodeEmitter::PostLoad()
{
	Super::PostLoad();
}

bool UNiagaraNodeEmitter::IsPinNameEditable(const UEdGraphPin* GraphPinObj) const
{
	return false;
}

bool UNiagaraNodeEmitter::IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const
{
	return false;
}


bool UNiagaraNodeEmitter::VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const
{
	if (InName.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("InvalidName", "Invalid pin name");
		return false;
	}
	return true;
}

bool UNiagaraNodeEmitter::CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents)
{
	return false;
}

bool UNiagaraNodeEmitter::GenerateCompileHashForClassMembers(const UClass* InClass, FNiagaraCompileHashVisitor* InVisitor) const
{
	if (InClass == UNiagaraNodeEmitter::StaticClass())
	{
		// For emitters, we really just want the emitter name.
		FName EmitterName;
		if (OwnerSystem != nullptr && EmitterHandleId.IsValid())
		{
			for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
			{
				if (EmitterHandle.GetId() == EmitterHandleId)
				{
					EmitterName = (EmitterHandle.GetName());
					break;
				}
			}
		}
		else if (CachedUniqueName.IsValid())
		{
			EmitterName = (CachedUniqueName);
		}

		InVisitor->UpdateString(TEXT("EmitterName"), EmitterName.ToString());
		return true;
	}
	else
	{
		return Super::GenerateCompileHashForClassMembers(InClass, InVisitor);
	}
}

void UNiagaraNodeEmitter::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* NiagaraSchema = Cast<UEdGraphSchema_Niagara>(GetSchema());
	CreatePin(EGPD_Input, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("InputMap"));
	CreatePin(EGPD_Output, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("OutputMap"));
}

bool UNiagaraNodeEmitter::CanUserDeleteNode() const
{
	return false;
}

bool UNiagaraNodeEmitter::CanDuplicateNode() const
{
	return false;
}

FText UNiagaraNodeEmitter::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText UsageText;
	if (ScriptType == ENiagaraScriptUsage::EmitterSpawnScript)
	{
		UsageText = LOCTEXT("SpawnTitle", "Spawn");
	}
	else if (ScriptType == ENiagaraScriptUsage::EmitterUpdateScript)
	{
		UsageText = LOCTEXT("UpdateTitle", "Update");
	}
	else
	{
		UsageText = LOCTEXT("Unknown Title", "Unknown");
	}
	return FText::Format(LOCTEXT("EmitterNameTitle","Emitter {0} {1}"), DisplayName, UsageText);
}

FLinearColor UNiagaraNodeEmitter::GetNodeTitleColor() const
{
	return CastChecked<UEdGraphSchema_Niagara>(GetSchema())->NodeTitleColor_Attribute;
}

void UNiagaraNodeEmitter::NodeConnectionListChanged()
{
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
	//GetGraph()->NotifyGraphChanged();
}

FNiagaraEmitterID UNiagaraNodeEmitter::GetEmitterID()const
{
	if (OwnerSystem != nullptr && EmitterHandleId.IsValid())
	{
		const auto& Emitters = OwnerSystem->GetEmitterHandles();
		for (int32 i = 0; i < Emitters.Num(); ++i)
		{
			const FNiagaraEmitterHandle& EmitterHandle = Emitters[i];
			if (EmitterHandle.GetId() == EmitterHandleId)
			{
				return FNiagaraEmitterID(i);
			}
		}
	}

	return CachedEmitterID;
}

FString UNiagaraNodeEmitter::GetEmitterUniqueName() const
{
	if (OwnerSystem != nullptr && EmitterHandleId.IsValid())
	{
		for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
		{
			if (EmitterHandle.GetId() == EmitterHandleId)
			{
				return EmitterHandle.GetUniqueInstanceName();
			}
		}
	}

	return CachedUniqueName.ToString();
}

UNiagaraScriptSource* UNiagaraNodeEmitter::GetScriptSource() const
{
	// First get the emitter that we're referencing..
	FVersionedNiagaraEmitter VersionedEmitter;
	if (OwnerSystem)
	{
		for (int32 i = 0; i < OwnerSystem->GetNumEmitters(); ++i)
		{
			if (OwnerSystem->GetEmitterHandle(i).GetId() == EmitterHandleId)
			{
				VersionedEmitter = OwnerSystem->GetEmitterHandle(i).GetInstance();
			}
		}
	}

	// Now get the graph off that emitter
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	if (EmitterData && EmitterData->GraphSource)
	{
		UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(EmitterData->GraphSource);
		return Source;
	}

	return Cast<UNiagaraScriptSource>(CachedScriptSourceWeakPtr.Get());
}

UNiagaraGraph* UNiagaraNodeEmitter::GetCalledGraph() const
{
	// First get the emitter that we're referencing..
	FVersionedNiagaraEmitter VersionedEmitter;
	if (OwnerSystem)
	{
		for (int32 i = 0; i < OwnerSystem->GetNumEmitters(); ++i)
		{
			if (OwnerSystem->GetEmitterHandle(i).GetId() == EmitterHandleId)
			{
				VersionedEmitter = OwnerSystem->GetEmitterHandle(i).GetInstance();
			}
		}
	}

	// Now get the graph off that emitter
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	if (EmitterData && EmitterData->GraphSource)
	{
		UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(EmitterData->GraphSource);
		if (Source)
		{
			return Source->NodeGraph;
		}
	}

	if (UNiagaraGraph* CachedGraph = CachedGraphWeakPtr.Get())
	{
		return CachedGraph;
	}
	return nullptr;
}

bool UNiagaraNodeEmitter::RefreshFromExternalChanges()
{
	DisplayName = GetNameFromEmitter();
	ENodeEnabledState OldEnabledState = GetDesiredEnabledState();
	SyncEnabledState();
	if (OldEnabledState != GetDesiredEnabledState())
	{
		MarkNodeRequiresSynchronization(TEXT("Emitter Node Enabled Changed"), true);
	}
	return true;
}

void UNiagaraNodeEmitter::SyncEnabledState()
{
	if (OwnerSystem != nullptr && EmitterHandleId.IsValid())
	{
		for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
		{
			if (EmitterHandle.GetId() == EmitterHandleId)
			{
				if (EmitterHandle.GetIsEnabled() == false)
				{
					SetEnabledState(ENodeEnabledState::Disabled, false);
				}
				else
				{
					SetEnabledState(ENodeEnabledState::Enabled, false);
				}
			}
		}
	}
}

void UNiagaraNodeEmitter::SetCachedVariablesForCompilation(const FName& InUniqueName, FNiagaraEmitterID InEmitterID, UNiagaraGraph* InGraph, UNiagaraScriptSourceBase* InSource)
{
	CachedUniqueName = InUniqueName;
	CachedEmitterID = InEmitterID;
	CachedGraphWeakPtr = InGraph;
	CachedScriptSourceWeakPtr = InSource;
}


FText UNiagaraNodeEmitter::GetNameFromEmitter()
{
	if (OwnerSystem != nullptr && EmitterHandleId.IsValid())
	{
		for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem->GetEmitterHandles())
		{
			if (EmitterHandle.GetId() == EmitterHandleId)
			{
				return FText::AsCultureInvariant(EmitterHandle.GetName().ToString());
			}
		}
	}
	else if (CachedUniqueName.IsValid())
	{
		return FText::AsCultureInvariant(CachedUniqueName.ToString());
	}
	return FText();
}

void UNiagaraNodeEmitter::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	Super::BuildParameterMapHistory(OutHistory, bRecursive, bFilterForCompilation);

	if ((!IsNodeEnabled() && OutHistory.GetIgnoreDisabled()) || (OutHistory.ExclusiveEmitterHandle.IsSet() && (OutHistory.ExclusiveEmitterHandle != GetEmitterHandleId())))
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);
	
	int32 ParamMapIdx = INDEX_NONE;
	if (GetInputPin(0)->LinkedTo.Num() != 0)
	{
		if (bRecursive)
		{
			ParamMapIdx = OutHistory.TraceParameterMapOutputPin(UNiagaraNode::TraceOutputPin(GetInputPin(0)->LinkedTo[0]));
		}
		else
		{
			ParamMapIdx = OutHistory.CreateParameterMap();
		}
	}

	const FNiagaraEmitterHandle* EmitterHandle = nullptr;
	if (const UNiagaraSystem* System = OutHistory.ConstantResolver->GetSystem())
	{
		EmitterHandle = System->GetEmitterHandles().FindByPredicate([this](const FNiagaraEmitterHandle& Handle) -> bool
		{
			return Handle.GetId() == EmitterHandleId;
		});
	}

	const FCompileConstantResolver ChildConstantResolver = EmitterHandle
		? OutHistory.ConstantResolver->AsEmitter(EmitterHandle->GetInstance())
		: *OutHistory.ConstantResolver;

	FString EmitterUniqueName = GetEmitterUniqueName();
	UNiagaraGraph* Graph = GetCalledGraph();
	if (Graph && ParamMapIdx != INDEX_NONE && OutHistory.bShouldBuildSubHistories)
	{
		OutHistory.EnterEmitter(EmitterUniqueName, Graph, this);

		TArray<ENiagaraScriptUsage> Usages;
		Usages.Add(ENiagaraScriptUsage::EmitterSpawnScript);
		Usages.Add(ENiagaraScriptUsage::EmitterUpdateScript);
		Usages.Add(ENiagaraScriptUsage::ParticleSpawnScript);
		Usages.Add(ENiagaraScriptUsage::ParticleSpawnScriptInterpolated);
		Usages.Add(ENiagaraScriptUsage::ParticleUpdateScript);
		Usages.Add(ENiagaraScriptUsage::ParticleEventScript);
		Usages.Add(ENiagaraScriptUsage::ParticleSimulationStageScript);
	
		uint32 NodeIdx = OutHistory.BeginNodeVisitation(ParamMapIdx, this);
		for (ENiagaraScriptUsage OutputNodeUsage : Usages)
		{
			TArray<UNiagaraNodeOutput*> OutputNodes;

			Graph->FindOutputNodes(OutputNodeUsage, OutputNodes);

			// Build up a new parameter map history with all the child graph nodes..
			FNiagaraParameterMapHistoryBuilder ChildBuilder;
			*ChildBuilder.ConstantResolver = ChildConstantResolver.WithUsage(OutputNodeUsage);
			ChildBuilder.RegisterEncounterableVariables(OutHistory.GetEncounterableVariables());
			ChildBuilder.EnableScriptAllowList(true, GetUsage());

			TArray<FNiagaraVariable> LocalStaticVars;
			FNiagaraParameterUtilities::FilterToRelevantStaticVariables(OutHistory.StaticVariables, LocalStaticVars, *EmitterUniqueName, TEXT("Emitter"), true);
			ChildBuilder.RegisterExternalStaticVariables(LocalStaticVars);

			FString LocalEmitterName = TEXT("Emitter");
			ChildBuilder.EnterEmitter(LocalEmitterName, Graph, this);
			for (UNiagaraNodeOutput* OutputNode : OutputNodes)
			{
				ChildBuilder.BuildParameterMaps(OutputNode, true);
			}
			ChildBuilder.ExitEmitter(LocalEmitterName, this);
			 
			FNiagaraAliasContext ResolveAliasesContext(OutputNodeUsage);
			ResolveAliasesContext.ChangeEmitterToEmitterName(EmitterUniqueName);
			for (FNiagaraParameterMapHistory& History : ChildBuilder.Histories)
			{
				OutHistory.Histories[ParamMapIdx].MapPinHistory.Append(History.MapPinHistory);
				for (int32 SrcVarIdx = 0; SrcVarIdx < History.Variables.Num(); SrcVarIdx++)
				{
					FNiagaraVariable& Var = History.Variables[SrcVarIdx];
					Var = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);

					int32 ExistingIdx = OutHistory.Histories[ParamMapIdx].FindVariable(Var.GetName(), Var.GetType());
					if (ExistingIdx == INDEX_NONE)
					{
						ExistingIdx = OutHistory.AddVariableToHistory(OutHistory.Histories[ParamMapIdx], Var, History.VariablesWithOriginalAliasesIntact[SrcVarIdx], nullptr);
					}
					ensure(ExistingIdx < OutHistory.Histories[ParamMapIdx].PerVariableWarnings.Num());
					ensure(ExistingIdx < OutHistory.Histories[ParamMapIdx].PerVariableReadHistory.Num());
					ensure(ExistingIdx < OutHistory.Histories[ParamMapIdx].PerVariableWriteHistory.Num());
					OutHistory.Histories[ParamMapIdx].PerVariableReadHistory[ExistingIdx].Append(History.PerVariableReadHistory[SrcVarIdx]);
					OutHistory.Histories[ParamMapIdx].PerVariableWriteHistory[ExistingIdx].Append(History.PerVariableWriteHistory[SrcVarIdx]);
					OutHistory.Histories[ParamMapIdx].PerVariableWarnings[ExistingIdx].Append(History.PerVariableWarnings[SrcVarIdx]);	
					for (int32 PerConstantIdx = 0; PerConstantIdx < History.PerVariableConstantValue[SrcVarIdx].Num(); PerConstantIdx++)
					{	
						const FString& ConstantStr = History.PerVariableConstantValue[SrcVarIdx][PerConstantIdx];
						OutHistory.Histories[ParamMapIdx].PerVariableConstantValue[ExistingIdx].AddUnique(ConstantStr);	
					}
				}
				OutHistory.Histories[ParamMapIdx].EncounteredParameterCollections.Append(History.EncounteredParameterCollections);
				OutHistory.Histories[ParamMapIdx].PinToConstantValues.Append(History.PinToConstantValues);
			}

			ResolveAliasesContext.ChangeRapidIterationParameterMode(FNiagaraAliasContext::ERapidIterationParameterMode::StaticVariables);
			// We only want to push out appropriately scoped static variables that should be in the system builder, not in-betweens like "Module.MyInputVar"
			// or per-particle or others vars. Really only want Emitter or System parameters here.
			for (int32 StaticVarIdx = 0; StaticVarIdx < ChildBuilder.StaticVariables.Num(); StaticVarIdx++)
			{
				const FNiagaraVariable& ChildStaticVar = ChildBuilder.StaticVariables[StaticVarIdx];
				if (ChildBuilder.StaticVariableExportable[StaticVarIdx])
				{
					// Should match logic in FNiagaraParameterMapHistoryBuilder::RegisterConstantVariableWrite
					FNiagaraVariable ResolvedStaticVar = FNiagaraUtilities::ResolveAliases(ChildStaticVar, ResolveAliasesContext);

					// Index of uses == operator, which only checks name and type. This will allow us to detect instances of the duplicate
					// data down the line.
					int32 FoundStaticVarIdx = OutHistory.StaticVariables.Find(ResolvedStaticVar);
					
					if (FoundStaticVarIdx == INDEX_NONE) // Didn't find it, so add it.
					{
						OutHistory.StaticVariables.Add(ResolvedStaticVar);
						OutHistory.StaticVariableExportable.Emplace(true);
					}
					else if (false == OutHistory.StaticVariables[FoundStaticVarIdx].HoldsSameData(ResolvedStaticVar))
					{
						OutHistory.StaticVariables.Add(ResolvedStaticVar);// Add as a duplicate here. We will filter out later
						OutHistory.StaticVariableExportable.Emplace(true);
					}

					ensure(OutHistory.StaticVariables.Num() == OutHistory.StaticVariableExportable.Num());
				}
			}
		}

		OutHistory.EndNodeVisitation(ParamMapIdx, NodeIdx);
		OutHistory.ExitEmitter(EmitterUniqueName, this);
	}

	for (UEdGraphPin* Pin : OutputPins)
	{
		FNiagaraTypeDefinition Type = Schema->PinToTypeDefinition(Pin);
		if (Type == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			OutHistory.RegisterParameterMapPin(ParamMapIdx, Pin);
		}
	}
}

void UNiagaraNodeEmitter::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	NIAGARA_SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_Module_NiagaraNodeEmitter_Compile);
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	InputPins.RemoveAll([](UEdGraphPin* InputPin) { return (InputPin->PinType.PinCategory != UEdGraphSchema_Niagara::PinCategoryType) && (InputPin->PinType.PinCategory != UEdGraphSchema_Niagara::PinCategoryEnum); });

	FPinCollectorArray OutputPins;
	GetCompilationOutputPins(OutputPins);

	check(Outputs.Num() == 0);

	// First compile fully down the hierarchy for our predecessors..
	UNiagaraGraph* CalledGraph = GetCalledGraph();
	TArray<UNiagaraNodeInput*> InputsNodes;
	UNiagaraGraph::FFindInputNodeOptions Options;
	Options.bSort = true;
	Options.bFilterDuplicates = true;
	Options.bFilterByScriptUsage = true;
	Options.TargetScriptUsage = Translator->GetTargetUsage() == ENiagaraScriptUsage::SystemSpawnScript ? ENiagaraScriptUsage::EmitterSpawnScript : ENiagaraScriptUsage::EmitterUpdateScript;
	
	if (CalledGraph && IsNodeEnabled()) // Called graph may be null on an disabled emitter
	{
		CalledGraph->FindInputNodes(InputsNodes, Options);
	}

	TArray<int32> CompileInputs;
	
	if (InputPins.Num() > 1)
	{
		Translator->Error(LOCTEXT("TooManyOutputPinsError", "Too many input pins on node."), this, nullptr);
		return;
	}

	int32 InputPinCompiled = Translator->CompileInputPin(InputPins[0]);
	if (!IsNodeEnabled())
	{
		// Do the minimal amount of work necessary if we are disabled.
		CompileInputs.Reserve(1);
		CompileInputs.Add(InputPinCompiled);
		Translator->Emitter(this, CompileInputs, Outputs);
		return;
	}

	if (InputsNodes.Num() <= 0)
	{
		Translator->Error(LOCTEXT("InputNodesNotFound", "Input nodes on called graph not found"), this, nullptr);
		return;
	}

	CompileInputs.Reserve(InputsNodes.Num());

	bool bError = false;
	for (UNiagaraNodeInput* EmitterInputNode : InputsNodes)
	{
		if (EmitterInputNode->Input.IsEquivalent(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("InputMap"))))
		{
			CompileInputs.Add(InputPinCompiled);
		}
		else
		{
			CompileInputs.Add(INDEX_NONE);
		}
	}

	if (!bError)
	{
		Translator->Emitter(this, CompileInputs, Outputs);
	}
}

void UNiagaraNodeEmitter::GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const
{
	UNiagaraGraph* CalledGraph = GetCalledGraph();

	if (CalledGraph && IsNodeEnabled()) // Skip if disabled
	{
		CalledGraph->RebuildCachedCompileIds();
		ENiagaraScriptUsage TargetUsage = InUsage == ENiagaraScriptUsage::SystemSpawnScript ? ENiagaraScriptUsage::EmitterSpawnScript : ENiagaraScriptUsage::EmitterUpdateScript;
		FNiagaraCompileHash Hash = CalledGraph->GetCompileDataHash(TargetUsage, FGuid(0,0,0,0));
		HashCollector.AddHash(Hash, CalledGraph->GetPathName());
		CalledGraph->GatherExternalDependencyData(TargetUsage, FGuid(0, 0, 0, 0), HashCollector);
	}
}

#undef NIAGARA_SCOPE_CYCLE_COUNTER
#undef LOCTEXT_NAMESPACE // NiagaraNodeEmitter

