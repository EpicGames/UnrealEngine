// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraNodeWithDynamicPins.h"
#include "NiagaraNodeEmitter.generated.h"

class UNiagaraSystem;

/** A niagara graph node which represents an emitter and it's parameters. */
UCLASS(MinimalAPI)
class UNiagaraNodeEmitter : public UNiagaraNodeWithDynamicPins
{
public:
	GENERATED_BODY()

	/** Gets the System that owns this emitter node. */
	UNiagaraSystem* GetOwnerSystem() const;

	/** Sets the System that owns this emitter node. */
	void SetOwnerSystem(UNiagaraSystem* InOwnerSystem);

	/** Gets the id of the emitter handle which this node represents. */
	FGuid GetEmitterHandleId() const;

	/** Sets the id of the emitter handle which this node represents. */
	void SetEmitterHandleId(FGuid InEmitterHandleId);

	//~ EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual bool CanUserDeleteNode() const override;
	virtual bool CanDuplicateNode() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void NodeConnectionListChanged() override;

	//~ UNiagaraNode Interface
	virtual bool RefreshFromExternalChanges() override;

	//~ UObject Interface
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;

	virtual bool IsPinNameEditable(const UEdGraphPin* GraphPinObj) const override;
	virtual bool IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const override;
	virtual bool VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const override;
	virtual bool CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents = false)  override;

	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;

	/** Synchronize with the handle associated with this emitter for enabled/disabled state.*/
	void SyncEnabledState();

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const;

	ENiagaraScriptUsage GetUsage() const { return ScriptType; }
	void SetUsage(ENiagaraScriptUsage InUsage) { ScriptType = InUsage; }

	FNiagaraEmitterID GetEmitterID()const;
	FString GetEmitterUniqueName() const;
	UNiagaraScriptSource* GetScriptSource() const;
	UNiagaraGraph* GetCalledGraph() const;
	
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual void GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const override;

	void SetCachedVariablesForCompilation(const FName& InUniqueName, FNiagaraEmitterID InEmitterID, UNiagaraGraph* InGraph, UNiagaraScriptSourceBase* InSource);

protected:
	UEdGraphPin* PinPendingRename;

	virtual bool GenerateCompileHashForClassMembers(const UClass* InClass, FNiagaraCompileHashVisitor* InVisitor) const;
	
private:
	/** Looks up the name of the emitter and converts it to text. */
	FText GetNameFromEmitter();

private:
	

	/** The System that owns the emitter which is represented by this node. */
	UPROPERTY(meta = (SkipForCompileHash = "true"))
	TObjectPtr<UNiagaraSystem> OwnerSystem;

	/** The id of the emitter handle which points to the emitter represented by this node. */
	UPROPERTY()
	FGuid EmitterHandleId;

	/** The display name for the title bar of this node. */
	UPROPERTY(meta = (SkipForCompileHash = "true"))
	FText DisplayName;

	UPROPERTY()
	ENiagaraScriptUsage ScriptType;

	FName CachedUniqueName;
	FNiagaraEmitterID CachedEmitterID;
	TWeakObjectPtr<UNiagaraGraph> CachedGraphWeakPtr;
	TWeakObjectPtr<UNiagaraScriptSourceBase> CachedScriptSourceWeakPtr;
};