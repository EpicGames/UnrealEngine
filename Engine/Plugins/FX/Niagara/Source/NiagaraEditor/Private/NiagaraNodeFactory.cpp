// Copyright Epic Games, Inc. All Rights Reserved.
#include "NiagaraNodeFactory.h"
#include "NiagaraNodeReroute.h"
#include "SGraphNodeKnot.h"

class SNiagaraGraphNodeKnot : public SGraphNodeKnot
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphNodeKnot) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraNodeReroute* InKnot)
	{
		SGraphNodeKnot::Construct(SGraphNodeKnot::FArguments(), InKnot);
		InKnot->OnVisualsChanged().AddSP(this, &SNiagaraGraphNodeKnot::HandleNiagaraNodeChanged);
	}

private:
	void HandleNiagaraNodeChanged(UNiagaraNode* InNode)
	{

		UpdateGraphNode();
	}
};

FNiagaraNodeFactory::~FNiagaraNodeFactory()
{
}

TSharedPtr<SGraphNode> FNiagaraNodeFactory::CreateNodeWidget(UEdGraphNode* InNode)
{
	if (UNiagaraNodeReroute* RerouteNode = Cast<UNiagaraNodeReroute>(InNode))
	{
		return SNew(SNiagaraGraphNodeKnot, RerouteNode);
	}

	return FGraphNodeFactory::CreateNodeWidget(InNode);
}
