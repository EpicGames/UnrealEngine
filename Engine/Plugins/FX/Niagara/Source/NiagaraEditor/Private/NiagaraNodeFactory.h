// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NodeFactory.h"

class FNiagaraNodeFactory : public FGraphNodeFactory
{
public:
    virtual ~FNiagaraNodeFactory();
	/** Create a widget for the supplied node */
	virtual TSharedPtr<class SGraphNode> CreateNodeWidget(class UEdGraphNode* InNode) override;
};
