// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeFunctionCall.h"

#include "Editor.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraComponent.h"
#include "NiagaraConstants.h"
#include "NiagaraCustomVersion.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraMessages.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "Widgets/SNiagaraGraphNodeFunctionCallWithSpecifiers.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraSettings.h"
#include "ScopedTransaction.h"
#include "SourceCodeNavigation.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Preferences/UnrealEdOptions.h"
#include "Widgets/Input/SComboButton.h"
#include "ToolMenus.h"
#include "HAL/FileManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeFunctionCall)


#define LOCTEXT_NAMESPACE "NiagaraNodeFunctionCall"

void UNiagaraNodeFunctionCall::PostLoad()
{
	Super::PostLoad();

	if (FunctionScript)
	{
		const int32 NiagaraCustomVersion = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
		FunctionScript->ConditionalPostLoad();
		if (NiagaraCustomVersion < FNiagaraCustomVersion::ModuleVersioning)
		{
			SelectedScriptVersion = FGuid();
			InvalidScriptVersionReference = FGuid();
			PreviousScriptVersion = FGuid();
		}
		FixupFunctionScriptVersion();

		// We need to make sure that the variables that could potentially be used in AllocateDefaultPins have been properly
		// loaded. Otherwise, we could be out of date.
		UNiagaraScriptSource* Source = GetFunctionScriptSource();
		if (Source)
		{
			Source->ConditionalPostLoad();
			if (UNiagaraGraph* Graph = Source->NodeGraph)
			{
				Graph->ConditionalPostLoad();

				// Fix up autogenerated default values if necessary.
				if (NiagaraCustomVersion < FNiagaraCustomVersion::EnabledAutogeneratedDefaultValuesForFunctionCallNodes)
				{
					FPinCollectorArray InputPins;
					GetInputPins(InputPins);

					TArray<UNiagaraNodeInput*> InputNodes;
					UNiagaraGraph::FFindInputNodeOptions Options;
					Options.bSort = true;
					Options.bFilterDuplicates = true;
					Graph->FindInputNodes(InputNodes, Options);

					for (UEdGraphPin* InputPin : InputPins)
					{
						auto FindInputNodeByName = [&](UNiagaraNodeInput* InputNode) { return InputNode->Input.GetName().ToString() == InputPin->PinName.ToString(); };
						UNiagaraNodeInput** MatchingInputNodePtr = InputNodes.FindByPredicate(FindInputNodeByName);
						if (MatchingInputNodePtr != nullptr)
						{
							UNiagaraNodeInput* MatchingInputNode = *MatchingInputNodePtr;
							SetPinAutoGeneratedDefaultValue(*InputPin, *MatchingInputNode);

							// If the default value wasn't set, update it with the new autogenerated default.
							if (InputPin->DefaultValue.IsEmpty())
							{
								InputPin->DefaultValue = InputPin->AutogeneratedDefaultValue;
							}
						}
					}
				}
			}
		}

		if (NiagaraCustomVersion < FNiagaraCustomVersion::StaticSwitchFunctionPinsUsePersistentGuids)
		{
			UpdateStaticSwitchPinsWithPersistentGuids();
		}

		if (NiagaraCustomVersion < FNiagaraCustomVersion::RepopulateFunctionCallNodePinNameBindings)
		{
			FNiagaraStackGraphUtilities::PopulateFunctionCallNameBindings(*this);
		}
	}
	else
	{
		//Perform some fix up due to a bug with dynamic pins and add pins.
		//Rebuild the dynamic pins and ensure we have add pins where needed.		
		const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
		if (NiagaraVer < FNiagaraCustomVersion::DynamicPinNodeFixup && AllowDynamicPins())
		{
			bool bFoundInputAdd = false;
			bool bFoundOutputAdd = false;
			for (const UEdGraphPin* Pin : Pins)
			{
				if (IsAddPin(Pin))
				{
					if (Pin->Direction == EGPD_Input)
					{
						bFoundInputAdd = true;
					}
					else
					{
						bFoundOutputAdd = true;
					}
				}
			}

			if (!bFoundInputAdd)
			{
				CreateAddPin(EGPD_Input);
			}

			if (!bFoundOutputAdd)
			{
				CreateAddPin(EGPD_Output);
			}
		}
	}


	// Allow data interfaces an opportunity to intercept changes
	if (Signature.IsValid() && Signature.bMemberFunction)
	{
		if ((Signature.Inputs.Num() > 0) && Signature.Inputs[0].GetType().IsDataInterface())
		{
			UNiagaraDataInterface* CDO = CastChecked<UNiagaraDataInterface>(Signature.Inputs[0].GetType().GetClass()->GetDefaultObject());
			FNiagaraFunctionSignature CopyForComparison = Signature;
			if (CDO->UpgradeFunctionCall(Signature))
			{
				FunctionDisplayName.Empty();
				if (Signature != CopyForComparison)
				{
					ReallocatePins();
				}
			}
		}
	}

	UpdatePinTooltips();

	// Clean up invalid old references to propagated parameters
	CleanupPropagatedSwitchValues();
	
	if (FunctionDisplayName.IsEmpty())
	{
		ComputeNodeName();
	}

	if (MessageKeyToMessageMap_DEPRECATED.IsEmpty() == false)
	{
		for (auto KeyMessagePair : MessageKeyToMessageMap_DEPRECATED)
		{
			MessageStore.AddMessage(KeyMessagePair.Key, KeyMessagePair.Value);
		}
		MessageKeyToMessageMap_DEPRECATED.Empty();
	}
}

void UNiagaraNodeFunctionCall::UpgradeDIFunctionCalls()
{
	// We no longer use this upgrade path, but leaving here for convience in case we need to use this route again for other things
#if 0
	UClass* InterfaceClass = nullptr;
	UNiagaraDataInterface* InterfaceCDO = nullptr;
	if (Signature.IsValid() && FunctionScript == nullptr)
	{
		if (Signature.Inputs.Num() > 0)
		{
			if (Signature.Inputs[0].GetType().IsDataInterface())
			{
				InterfaceClass = Signature.Inputs[0].GetType().GetClass();
				InterfaceCDO = Cast<UNiagaraDataInterface>(InterfaceClass->GetDefaultObject());
			}
		}
	}

	FString UpgradeNote;
	if (!UpgradeNote.IsEmpty())
	{
		UE_LOGF(LogNiagaraEditor, Log, "Upgradeing Niagara Data Interface fuction call node. This may cause unnessessary recompiles. Please resave these assets if this occurs. Or use fx.UpgradeAllNiagaraAssets.");
		UE_LOGF(LogNiagaraEditor, Log, "Node: %ls", *GetFullName());
		if (InterfaceCDO)
		{
			UE_LOGF(LogNiagaraEditor, Log, "Interface: %ls", *InterfaceCDO->GetFullName());
		}
		UE_LOGF(LogNiagaraEditor, Log, "Function: %ls", *Signature.GetName());
		UE_LOGF(LogNiagaraEditor, Log, "Upgrade Note: %ls",* UpgradeNote);
	}
#endif
}

TSharedPtr<SGraphNode> UNiagaraNodeFunctionCall::CreateVisualWidget()
{
	if (!FunctionScript && FunctionSpecifiers.Num() == 0)
	{
		FunctionSpecifiers = Signature.FunctionSpecifiers;
	}
	if (FunctionSpecifiers.Num() == 0)
	{
		return Super::CreateVisualWidget();
	}
	else
	{
		return SNew(SNiagaraGraphNodeFunctionCallWithSpecifiers, this);
	}
}

void UNiagaraNodeFunctionCall::AddOrphanedStaticSwitchPinForDataRetention(FNiagaraVariableBase StaticSwitchVariable, const FString& StaticSwitchPinDefault)
{
	UEdGraphPin* NewPin = AddStaticSwitchInputPin(StaticSwitchVariable);
	NewPin->DefaultValue = StaticSwitchPinDefault;
	NewPin->bOrphanedPin = true;
}

void UNiagaraNodeFunctionCall::RemoveAllDynamicPins()
{
	FScopedTransaction RemovePinTransaction(LOCTEXT("RemoveAllDynamicPinsTransaction", "Remove all dynamic pins"));
	if(AllowDynamicPins())
	{
		FPinCollectorArray PinArray;
		GetInputPins(PinArray);

		for(UEdGraphPin* Pin : PinArray)
		{
			if(CanModifyPin(Pin) && IsBaseSignatureOfDataInterfaceFunction(Pin) == false)
			{
				RemoveDynamicPin(Pin);
			}
		}

		GetOutputPins(PinArray);

		for(UEdGraphPin* Pin : PinArray)
		{
			if(CanModifyPin(Pin) && IsBaseSignatureOfDataInterfaceFunction(Pin) == false)
			{
				RemoveDynamicPin(Pin);
			}
		}
	}
}

UClass* UNiagaraNodeFunctionCall::GetDIClass()const
{
	if(Signature.IsValid())
	{
		if (Signature.Inputs.Num() > 0)
		{
			if (Signature.Inputs[0].GetType().IsDataInterface() && GetValidateDataInterfaces())
			{
				return Signature.Inputs[0].GetType().GetClass();
			}
		}
	}
	return nullptr;	
}

void UNiagaraNodeFunctionCall::SetFunctionSpecifier(FName Key, FName Value)
{
	Modify();
	FunctionSpecifiers.FindOrAdd(Key) = Value;
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
}

UEdGraphPin* UNiagaraNodeFunctionCall::AddStaticSwitchInputPin(FNiagaraVariable Input)
{
	UNiagaraGraph* Graph = GetCalledGraph();
	const UEdGraphSchema_Niagara* Schema = Graph->GetNiagaraSchema();

	UEdGraphPin* NewPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Input.GetType()), Input.GetName());
	NewPin->bNotConnectable = true;
	NewPin->bDefaultValueIsIgnored = FindPropagatedVariable(Input) != nullptr;
	NewPin->SetSavePinIfOrphaned(true);

	FGuid PinPersistentGuid;
	TOptional<int32> SwitchDefaultValue = Graph->GetStaticSwitchDefaultValue(Input);
	TOptional<FGuid> ScriptVariableGuid = Graph->GetScriptVariableGuid(Input);
	if (SwitchDefaultValue.IsSet() && ScriptVariableGuid.IsSet())
	{
		checkf(Input.GetType().GetSize() == sizeof(FNiagaraInt32), TEXT("Incorrectly sized Input variable for UNiagaraNodeFunctionCall::AddStaticSwitchInputPin().  Size: %d (should be %zu). Type: %s. Name: %s. Node: %s. ScriptVariableGuid: %s"),
			Input.GetType().GetSize(), sizeof(FNiagaraInt32),
			*Input.GetType().GetName(),
			*Input.GetName().ToString(),
			*GetFullName(),
			*ScriptVariableGuid->ToString());
		Input.AllocateData();
		Input.SetValue<FNiagaraInt32>( {*SwitchDefaultValue} );
		PinPersistentGuid = *ScriptVariableGuid;
	}
	else
	{
		FNiagaraEditorUtilities::ResetVariableToDefaultValue(Input);
		PinPersistentGuid = FGuid();
	}
	NewPin->PersistentGuid = PinPersistentGuid;

	FString PinDefaultValue;
	if (Schema->TryGetPinDefaultValueFromNiagaraVariable(Input, PinDefaultValue))
	{
		NewPin->AutogeneratedDefaultValue = PinDefaultValue;
		NewPin->DefaultValue = PinDefaultValue;
	}

	return NewPin;
}

bool UNiagaraNodeFunctionCall::CanModifyPin(const UEdGraphPin* Pin) const
{
	if(IsAddPin(Pin))
	{
		return false;
	}
	
	if(Signature.IsValid() && (Signature.VariadicInput() || Signature.VariadicOutput()))
	{
		if(IsBaseSignatureOfDataInterfaceFunction(Pin) == false)
		{
			return true;
		}
	}

	return false;
}

void UNiagaraNodeFunctionCall::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		ReallocatePins();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);

	MarkNodeRequiresSynchronization(__FUNCTION__, true);
	//GetGraph()->NotifyGraphChanged();
}

void UNiagaraNodeFunctionCall::AllocateDefaultPins()
{
	if (FunctionScriptAssetObjectPath != NAME_None && FunctionScript == nullptr)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FAssetData ScriptAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(FunctionScriptAssetObjectPath.ToString()));
		if (ScriptAssetData.IsValid())
		{
			FunctionScript = Cast<UNiagaraScript>(ScriptAssetData.GetAsset());
			if (FunctionScript && FunctionScript->IsVersioningEnabled())
			{
				SelectedScriptVersion = FunctionScript->GetExposedVersion().VersionGuid;
			}
		}
	}

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	if (HasValidScriptAndGraph())
	{
		UNiagaraGraph* Graph = GetCalledGraph();

		//These pins must be refreshed and kept in the correct order for the function
		TArray<FNiagaraVariable> Inputs;
		TArray<FNiagaraVariable> Outputs;
		Graph->GetParameters(Inputs, Outputs);

		TArray<UNiagaraNodeInput*> InputNodes;
		UNiagaraGraph::FFindInputNodeOptions Options;
		Options.bSort = true;
		Options.bFilterDuplicates = true;
		Graph->FindInputNodes(InputNodes, Options);

		AdvancedPinDisplay = ENodeAdvancedPins::NoPins;
		for (UNiagaraNodeInput* InputNode : InputNodes)
		{
			if (InputNode->IsExposed())
			{
				UEdGraphPin* NewPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(InputNode->Input.GetType()), InputNode->Input.GetName());
					
				//An inline pin default only makes sense if we are required. 
				//Non exposed or optional inputs will used their own function input nodes defaults when not directly provided by a link.
				//Special class types cannot have an inline default.
 				NewPin->bDefaultValueIsIgnored = !(InputNode->IsRequired() && InputNode->Input.GetType().GetClass() == nullptr);

				SetPinAutoGeneratedDefaultValue(*NewPin, *InputNode);
				NewPin->DefaultValue = NewPin->AutogeneratedDefaultValue;

				//TODO: Some visual indication of Auto bound pins.
				//I tried just linking to null but
// 				FNiagaraVariable AutoBoundVar;
// 				ENiagaraInputNodeUsage AutBoundUsage = ENiagaraInputNodeUsage::Undefined;
// 				bool bCanAutoBind = FindAutoBoundInput(InputNode->AutoBindOptions, NewPin, AutoBoundVar, AutBoundUsage);
// 				if (bCanAutoBind)
// 				{
// 
// 				}

				if (InputNode->IsHidden())
				{
					NewPin->bAdvancedView = true;
					AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
				}
				else
				{
					NewPin->bAdvancedView = false;
				}
			}
		}

		TArray<FNiagaraVariable> SwitchNodeInputs = Graph->FindStaticSwitchInputs();
		for (FNiagaraVariable& Input : SwitchNodeInputs)
		{
			if (Input.IsValid())
			{
				AddStaticSwitchInputPin(Input);
			}
			else
			{
				UE_LOGF(LogNiagaraEditor, Warning, "Static switch input %ls on function call node %ls is invalid", *Input.GetName().ToString(), *GetPathName());
			}
		}

		for (FNiagaraVariable& Output : Outputs)
		{
			UEdGraphPin* NewPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(Output.GetType()), Output.GetName());
			NewPin->bDefaultValueIsIgnored = true;
		}

		// Make sure to note that we've synchronized with the external version.
		CachedChangeId = Graph->GetChangeID();
	}
	else
	{
		if (Signature.bRequiresExecPin)
		{
			UEdGraphPin* NewPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT(""));
			NewPin->bDefaultValueIsIgnored = true;
		}
		
		for (FNiagaraVariable& Input : Signature.Inputs)
		{
			UEdGraphPin* NewPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Input.GetType()), Input.GetName());
			NewPin->bDefaultValueIsIgnored = false;
			NewPin->PinToolTip = Signature.InputDescriptions.Contains(Input) ? Signature.InputDescriptions[Input].ToString() : FString();
			FString PinDefaultValue;
			if (Schema->TryGetPinDefaultValueFromNiagaraVariable(Input, PinDefaultValue))
			{
				NewPin->AutogeneratedDefaultValue = PinDefaultValue;
				NewPin->DefaultValue = PinDefaultValue;
			}
		}

		if (Signature.bRequiresExecPin)
		{
			UEdGraphPin* NewPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT(""));
			NewPin->bDefaultValueIsIgnored = true;
		}

		for (FNiagaraVariableBase& Output : Signature.Outputs)
		{
			UEdGraphPin* NewPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(Output.GetType()), Output.GetName());
			NewPin->bDefaultValueIsIgnored = true;
			NewPin->PinToolTip = Signature.OutputDescriptions.Contains(Output) ? Signature.OutputDescriptions[Output].ToString() : FString();
		}

		if (AllowDynamicPins())
		{
			if(Signature.IsValid() && (Signature.VariadicInput() || Signature.VariadicOutput()))
			{
				if(Signature.VariadicInput())
				{
					CreateAddPin(EGPD_Input);
				}
				if(Signature.VariadicOutput())
				{
					CreateAddPin(EGPD_Output);
				}
			}
			else
			{
				CreateAddPin(EGPD_Input);
				CreateAddPin(EGPD_Output);
			}
		}

		// We don't reference an external function, so set an invalid id.
		CachedChangeId = FGuid();
	}

	if (FunctionDisplayName.IsEmpty())
	{
		ComputeNodeName();
	}

	UpdateNodeErrorMessage();
}

// Returns true if this node is deprecated
bool UNiagaraNodeFunctionCall::IsDeprecated() const
{
	FVersionedNiagaraScriptData* ScriptData = GetScriptData();
	if (ScriptData)
	{
		return ScriptData->bDeprecated;
	}
	return false;
}

void UNiagaraNodeFunctionCall::UpdateInputNameBinding(const FGuid& BoundVariableGuid, const FName& BoundName)
{
	if (!BoundVariableGuid.IsValid() || BoundName.IsNone())
	{
		return;
	}
	BoundPinNames.Add(BoundVariableGuid, BoundName);
}

FText UNiagaraNodeFunctionCall::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FString DetectedName = FunctionScript ? FunctionScript->GetName() : Signature.GetNameString();
	if (DetectedName.IsEmpty())
	{
		return FText::FromString(TEXT("Missing ( Was\"") + FunctionDisplayName + TEXT("\")"));
	}
	else
	{
		return FText::FromString(FName::NameToDisplayString(FunctionDisplayName, false));
	}
}

FText UNiagaraNodeFunctionCall::GetTooltipText()const
{
	if (FunctionScript != nullptr)
	{
		return FunctionScript->GetDescription(SelectedScriptVersion);
	}
	else if (Signature.IsValid())
	{
		return Signature.Description;
	} 
	else
	{
		return LOCTEXT("NiagaraFuncCallUnknownSignatureTooltip", "Unknown function call");
	}
}

FLinearColor UNiagaraNodeFunctionCall::GetNodeTitleColor() const
{
	return UEdGraphSchema_Niagara::NodeTitleColor_FunctionCall;
}

bool UNiagaraNodeFunctionCall::CanAddToGraph(UNiagaraGraph* TargetGraph, FString& OutErrorMsg) const
{
	if (Super::CanAddToGraph(TargetGraph, OutErrorMsg) == false)
	{
		return false;
	}
	UPackage* TargetPackage = TargetGraph->GetOutermost();

	TArray<const UNiagaraGraph*> FunctionGraphs;
	UNiagaraScript* SpawningFunctionScript = FunctionScript;
	
	// We probably haven't loaded the script yet. Let's do so now so that we can trace its lineage.
	if (FunctionScriptAssetObjectPath != NAME_None && FunctionScript == nullptr)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FAssetData ScriptAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(FunctionScriptAssetObjectPath.ToString()));
		if (ScriptAssetData.IsValid())
		{
			SpawningFunctionScript = Cast<UNiagaraScript>(ScriptAssetData.GetAsset());
		}
	}

	// Now we need to get the graphs referenced by the script that we are about to spawn in.
	if (SpawningFunctionScript && SpawningFunctionScript->GetSource(SelectedScriptVersion))
	{
		UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(SpawningFunctionScript->GetSource(SelectedScriptVersion));
		if (Source)
		{
			UNiagaraGraph* FunctionGraph = Source->NodeGraph;
			if (FunctionGraph)
			{
				FunctionGraph->GetAllReferencedGraphs(FunctionGraphs);
			}
		}
	}

	// Iterate over each graph referenced by this spawning function call and see if any of them reference the graph that we are about to be spawned into. If 
	// a match is found, then adding us would introduce a cycle and we need to abort the add.
	for (const UNiagaraGraph* Graph : FunctionGraphs)
	{
		UPackage* FunctionPackage = Graph->GetOutermost();
		if (FunctionPackage != nullptr && TargetPackage != nullptr && FunctionPackage == TargetPackage)
		{
			OutErrorMsg = LOCTEXT("NiagaraFuncCallCannotAddToGraph", "Cannot add to graph because the Function Call used by this node would lead to a cycle.").ToString();
			return false;
		}
	}

	return true;
}

UNiagaraGraph* UNiagaraNodeFunctionCall::GetCalledGraph() const
{
	if (FunctionScript)
	{
		if (UNiagaraScriptSource* Source = GetFunctionScriptSource())
		{
			UNiagaraGraph* FunctionGraph = Source->NodeGraph;
			return FunctionGraph;
		}
	}

	return nullptr;
}


ENiagaraScriptUsage UNiagaraNodeFunctionCall::GetCalledUsage() const
{
	if (FunctionScript)
	{
		return FunctionScript->GetUsage();
	}
	return ENiagaraScriptUsage::Function;
}

UNiagaraScriptSource* UNiagaraNodeFunctionCall::GetFunctionScriptSource() const
{
	if (FunctionScript == nullptr)
	{
		return nullptr;
	}
	UNiagaraScriptSourceBase* SourceBase = FunctionScript->GetSource(SelectedScriptVersion);
	return SourceBase ? CastChecked<UNiagaraScriptSource>(SourceBase) : nullptr;
}

FVersionedNiagaraScriptData* UNiagaraNodeFunctionCall::GetScriptData() const
{
	return FunctionScript ? FunctionScript->GetScriptData(SelectedScriptVersion) : nullptr;
}

static FText GetFormattedDeprecationMessage(const FVersionedNiagaraScriptData* ScriptData, const FString& FunctionDisplayName)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("NodeName"), FText::FromString(FunctionDisplayName));

	if (ScriptData->DeprecationRecommendation != nullptr)
	{
		Args.Add(TEXT("Recommendation"), FText::FromString(ScriptData->DeprecationRecommendation.GetPathName()));
	}

	if (ScriptData->DeprecationMessage.IsEmptyOrWhitespace() == false)
	{
		Args.Add(TEXT("Message"), ScriptData->DeprecationMessage);
	}

	FText FormatString = LOCTEXT("DeprecationErrorFmtUnknown", "Function call \"{NodeName}\" is deprecated. No recommendation was provided.");

	if (ScriptData->DeprecationRecommendation != nullptr && ScriptData->DeprecationMessage.IsEmptyOrWhitespace() == false)
	{
		FormatString = LOCTEXT("DeprecationErrorFmtMessageAndRecommendation", "Function call \"{NodeName}\" is deprecated. Reason:\n{Message}.\nPlease use {Recommendation} instead.");
	}
	else if (ScriptData->DeprecationRecommendation != nullptr)
	{
		FormatString = LOCTEXT("DeprecationErrorFmtRecommendation", "Function call \"{NodeName}\" is deprecated. Please use {Recommendation} instead.");
	}
	else if (ScriptData->DeprecationMessage.IsEmptyOrWhitespace() == false)
	{
		FormatString = LOCTEXT("DeprecationErrorFmtMessage", "Function call \"{NodeName}\" is deprecated. Reason:\n{Message} ");
	}

	return FText::Format(FormatString, Args);
}

void UNiagaraNodeFunctionCall::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	// this is dirty, but we'll continue doing this for now because it's all going to go away with Digested graphs
	UNiagaraNodeFunctionCall* MutableThis = const_cast<UNiagaraNodeFunctionCall*>(this);

	TArray<int32> Inputs;

	bool bError = false;

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	FVersionedNiagaraScriptData* ScriptData = GetScriptData();
	if (ScriptData && HasValidScriptAndGraph())
	{
		if (InvalidScriptVersionReference.IsValid())
		{
			FText WarningMessage = FText::Format(LOCTEXT("InvalidScriptVersionReferenceMessage", "Function call \"{0}\" references an invalid script version ({1}). Compiling with the exposed version as fallback, but please check the node and set a valid version."), FText::FromString(FunctionDisplayName), FText::FromString(InvalidScriptVersionReference.ToString()));
			Translator->Warning(WarningMessage, this, nullptr);
		}
		if (ScriptData->bDeprecated && IsNodeEnabled())
		{
			FText DeprecationMessage = GetFormattedDeprecationMessage(ScriptData, FunctionDisplayName);

			Translator->Warning(DeprecationMessage, this, nullptr);
		}
		
		FPinCollectorArray CallerInputPins;
		GetInputPins(CallerInputPins);
		
		UNiagaraScriptSource* Source = GetFunctionScriptSource();
		UNiagaraGraph* FunctionGraph = Source->NodeGraph;

		TArray<UNiagaraNodeInput*> FunctionInputNodes;
		UNiagaraGraph::FFindInputNodeOptions Options;
		Options.bSort = true;
		Options.bFilterDuplicates = true;
		FunctionGraph->FindInputNodes(FunctionInputNodes, Options);

		// We check which module inputs are not used so we can later remove them from the compilation of the
		// parameter map that sets the input values for our function. This is mainly done to prevent data interfaces being
		// initialized as parameter when they are not used in the function or module.
		TSet<FName> HiddenPinNames;
		for (UEdGraphPin* Pin : FNiagaraStackGraphUtilities::GetUnusedFunctionInputPins(*this, FCompileConstantResolver(Translator, DebugState)))
		{
			HiddenPinNames.Add(Pin->PinName);
		}
		Translator->EnterFunctionCallNode(HiddenPinNames);

		for (UNiagaraNodeInput* FunctionInputNode : FunctionInputNodes)
		{
			//Finds the matching Pin in the caller.
			UEdGraphPin** PinPtr = CallerInputPins.FindByPredicate([&](UEdGraphPin* InPin) { return Schema->PinToNiagaraVariable(InPin).IsEquivalent(FunctionInputNode->Input); });
			if (!PinPtr)
			{
				if (FunctionInputNode->IsExposed())
				{
					//Couldn't find the matching pin for an exposed input. Probably a stale function call node that needs to be refreshed.
					Translator->Error(LOCTEXT("StaleFunctionCallError", "Function call is stale and needs to be refreshed."), this, nullptr);
					bError = true;
				}
				else if (FunctionInputNode->ExposureOptions.bRequired == true)
				{
					// Not exposed, but required. This means we should just add as a constant.
					Inputs.Add(Translator->GetConstant(FunctionInputNode->Input));
					continue;
				}


				Inputs.Add(INDEX_NONE);
				continue;
			}

			UEdGraphPin* CallerPin = *PinPtr;
			UEdGraphPin* CallerLinkedTo = CallerPin->LinkedTo.Num() > 0 ? UNiagaraNode::TraceOutputPin(CallerPin->LinkedTo[0]) : nullptr;			
			FNiagaraVariable PinVar = Schema->PinToNiagaraVariable(CallerPin);
			if (!CallerLinkedTo)
			{
				//if (Translator->CanReadAttributes())
				{
					//Try to auto bind if we're not linked to by the caller.
					FNiagaraVariable AutoBoundVar;
					ENiagaraInputNodeUsage AutBoundUsage = ENiagaraInputNodeUsage::Undefined;
					if (FindAutoBoundInput(FunctionInputNode, CallerPin, AutoBoundVar, AutBoundUsage))
					{
						UNiagaraGraph* CallerGraph = MutableThis->GetNiagaraGraph();

						UNiagaraNodeInput* NewNode = NewObject<UNiagaraNodeInput>(CallerGraph);
						NewNode->Input = PinVar;
						NewNode->Usage = AutBoundUsage;
						NewNode->AllocateDefaultPins();
						CallerLinkedTo = NewNode->GetOutputPin(0);
						CallerPin->BreakAllPinLinks();
						CallerPin->MakeLinkTo(CallerLinkedTo);
					}
				}
			}

			if (CallerLinkedTo)
			{
				//Param is provided by the caller. Typical case.
				Inputs.Add(Translator->CompileInputPin(CallerPin));
				continue;
			}
			else
			{
				if (FunctionInputNode->IsRequired())
				{
					if (CallerPin->bDefaultValueIsIgnored)
					{
						//This pin can't use a default and it is required so flag an error.
						Translator->Error(FText::Format(LOCTEXT("RequiredInputUnboundErrorFmt", "Required input {0} was not bound and could not be automatically bound."), CallerPin->GetDisplayName()),
							this, CallerPin);
						bError = true;
						//We weren't linked to anything and we couldn't auto bind so tell the compiler this input isn't provided and it should use it's local default.
						Inputs.Add(INDEX_NONE);
					}
					else
					{
						//We also compile the pin anyway if it is required as we'll be attempting to use it's inline default.
						Inputs.Add(Translator->CompileInputPin(CallerPin));
					}
				}
				else
				{
					//We optional, weren't linked to anything and we couldn't auto bind so tell the compiler this input isn't provided and it should use it's local default.
					Inputs.Add(INDEX_NONE);
				}
			}
		}

		FCompileConstantResolver ConstantResolver(Translator, DebugState);
		FNiagaraEditorUtilities::SetStaticSwitchConstants(GetCalledGraph(), CallerInputPins, ConstantResolver);
		Translator->ExitFunctionCallNode();
	}
	else if (MutableThis->Signature.IsValid())
	{
		UClass* DIClass = GetDIClass();
		if (DIClass)
		{
			if (UNiagaraDataInterface* DataInterfaceCDO = Cast<UNiagaraDataInterface>(DIClass->GetDefaultObject()))
			{
				TArray<FText> ValidationErrors;
				DataInterfaceCDO->ValidateFunction(Signature, ValidationErrors);

				bError = ValidationErrors.Num() > 0;

				for (FText& ValidationError : ValidationErrors)
				{
					Translator->Error(ValidationError, this, nullptr);
				}

				if (bError)
				{
					return;
				}
			}
		}
		Translator->EnterFunctionCallNode(TSet<FName>());
		MutableThis->Signature.FunctionSpecifiers = FunctionSpecifiers;
		bError = CompileInputPins(Translator, Inputs);
		Translator->ExitFunctionCallNode();
	}		
	else
	{
		Translator->Error(FText::Format(LOCTEXT("UnknownFunction", "Unknown Function Call! Missing Script or Data Interface Signature. Stack Name: {0}"), FText::FromString(GetFunctionName())), this, nullptr);
		bError = true;
	}

	if (!bError)
	{
		Translator->FunctionCall(this, Inputs, Outputs);
	}
}

UObject*  UNiagaraNodeFunctionCall::GetReferencedAsset() const
{
	if (FunctionScript && FunctionScript->GetOutermost() != GetOutermost())
	{
		return FunctionScript;
	}
	else
	{
		return nullptr;
	}
}

void UNiagaraNodeFunctionCall::OpenReferencedAsset() const
{
	if (FunctionScript && FunctionScript->GetOutermost() != GetOutermost())
	{
		if (FunctionScript->IsVersioningEnabled())
		{
			FunctionScript->VersionToOpenInEditor = SelectedScriptVersion;
		}
		#if WITH_EDITOR
		if ( GEditor )
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(FunctionScript);
		}
		#endif
	}
	else if (GUnrealEd && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		if (!Signature.SourceFile.IsEmpty() && FPaths::FileExists(Signature.SourceFile))
		{
			FSourceCodeNavigation::OpenSourceFile(Signature.SourceFile, Signature.SourceLine);
		}
		else if (Signature.Inputs.Num() > 0)
		{
			// if the signature doesn't have the source info stored we try to find the source file
			// by looking at the data interface on the input pin as a fallback
			UClass* Class = Signature.Inputs[0].GetType().GetClass();
			if (Class && Class->IsChildOf(UNiagaraDataInterface::StaticClass()))
			{
				FString ClassSourcePath;
				if(FSourceCodeNavigation::FindClassSourcePath(Class, ClassSourcePath))
				{
					const FString AbsoluteSourcePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ClassSourcePath);
					FSourceCodeNavigation::OpenSourceFile(AbsoluteSourcePath);
				}
			}
		}
	}
}

void UNiagaraNodeFunctionCall::UpdateNodeErrorMessage()
{
	FVersionedNiagaraScriptData* ScriptData = GetScriptData();
	if (ScriptData)
	{
		if (ScriptData->bDeprecated)
		{
			UEdGraphNode::bHasCompilerMessage = true;
			ErrorType = EMessageSeverity::Warning;
			
			UEdGraphNode::ErrorMsg = GetFormattedDeprecationMessage(ScriptData, FunctionDisplayName).ToString();
		}
		else if (ScriptData->bExperimental)
		{
			UEdGraphNode::bHasCompilerMessage = true;
			UEdGraphNode::ErrorType = EMessageSeverity::Info;

			if (ScriptData->ExperimentalMessage.IsEmptyOrWhitespace())
			{
				UEdGraphNode::NodeUpgradeMessage = LOCTEXT("FunctionExperimental", "This function is marked as experimental, use with care!");
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Message"), ScriptData->ExperimentalMessage);
				UEdGraphNode::NodeUpgradeMessage = FText::Format(LOCTEXT("FunctionExperimentalReason", "This function is marked as experimental, reason:\n{Message}."), Args);
			}
		}
		else
		{
			UEdGraphNode::bHasCompilerMessage = false;
			UEdGraphNode::ErrorMsg = FString();
		}
	}
	else if (Signature.IsValid())
	{
		if (Signature.bSoftDeprecatedFunction)
		{
			UEdGraphNode::bHasCompilerMessage = true;
			UEdGraphNode::ErrorType = EMessageSeverity::Info;

			UEdGraphNode::NodeUpgradeMessage = LOCTEXT("FunctionDeprecatedSoftly", "There is a newer version of this function, consider switching over to it.");
		}
		else if (Signature.bExperimental)
		{
			UEdGraphNode::bHasCompilerMessage = true;
			UEdGraphNode::ErrorType = EMessageSeverity::Info;

			if (Signature.ExperimentalMessage.IsEmptyOrWhitespace())
			{
				UEdGraphNode::NodeUpgradeMessage = LOCTEXT("FunctionExperimental", "This function is marked as experimental, use with care!");
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Message"), Signature.ExperimentalMessage);
				UEdGraphNode::NodeUpgradeMessage = FText::Format(LOCTEXT("FunctionExperimentalReason", "This function is marked as experimental, reason:\n{Message}."), Args);
			}
		}
	}
}

TArray<FGuid> UNiagaraNodeFunctionCall::GetBoundPinGuidsByName(FName InputName) const
{
	TArray<FGuid> Result;
	for (auto Entry : BoundPinNames)
	{
		if (Entry.Value == InputName)
		{
			Result.Add(Entry.Key);
		}
	}
	return Result;
}

void UNiagaraNodeFunctionCall::FixupFunctionScriptVersion()
{
	if (!FunctionScript)
	{
		return;
	}
	FunctionScript->CheckVersionDataAvailable();
	if (FunctionScript->IsVersioningEnabled())
	{
		if (!SelectedScriptVersion.IsValid())
		{
			// When we did not reference a versioned script before and do now, reference the first version (instead of the exposed version as new function calls do)
			SelectedScriptVersion = FunctionScript->GetAllAvailableVersions()[0].VersionGuid;
		}
		else if (!FunctionScript->GetAllAvailableVersions().ContainsByPredicate([this](const FNiagaraAssetVersion& Version) { return SelectedScriptVersion == Version.VersionGuid; }))
		{
			// The version we are referencing does not exist any more, maybe it was deleted.
			// So we fall back to the exposed version, but save the old guid to generate a warning.
			InvalidScriptVersionReference = SelectedScriptVersion;
			SelectedScriptVersion = FunctionScript->GetExposedVersion().VersionGuid;
		}
	}
	else
	{
		SelectedScriptVersion = FGuid();
		InvalidScriptVersionReference = FGuid();
		PreviousScriptVersion = FGuid();
	}
}

void UNiagaraNodeFunctionCall::UpdatePinTooltips()
{
	if (!Signature.IsValid())
	{
		return;
	}

	// if it's a DI function we grab the newest tooltips first
	if ((Signature.Inputs.Num() > 0) && Signature.Inputs[0].GetType().IsDataInterface())
	{
		UNiagaraDataInterface* CDO = CastChecked<UNiagaraDataInterface>(Signature.Inputs[0].GetType().GetClass()->GetDefaultObject());
		TArray<FNiagaraFunctionSignature> AllSignatures;
		CDO->GetFunctionSignatures(AllSignatures);
		for (const FNiagaraFunctionSignature& DISignature : AllSignatures)
		{
			if (Signature.Name != DISignature.Name)
			{
				continue;
			}
			Signature.Description = DISignature.Description;
			Signature.InputDescriptions = DISignature.InputDescriptions;
			Signature.OutputDescriptions = DISignature.OutputDescriptions;
		}
	}

	// update input pin tooltips
	TArray<UEdGraphPin*> InputPins;
	GetInputPins(InputPins);
	if (Signature.Inputs.Num() == InputPins.Num())
	{
		for (int i = 0; i < InputPins.Num(); i++)
		{
			FNiagaraVariable& Input = Signature.Inputs[i];
			InputPins[i]->PinToolTip = Signature.InputDescriptions.Contains(Input) ? Signature.InputDescriptions[Input].ToString() : FString();
		}
	}

	// update output pin tooltips
	TArray<UEdGraphPin*> OutputPins;
	GetOutputPins(OutputPins);
	if (Signature.Outputs.Num() == OutputPins.Num())
	{
		for (int i = 0; i < OutputPins.Num(); i++)
		{
			FNiagaraVariableBase& Output = Signature.Outputs[i];
			OutputPins[i]->PinToolTip = Signature.OutputDescriptions.Contains(Output) ? Signature.OutputDescriptions[Output].ToString() : FString();
		}
	}
}

void UNiagaraNodeFunctionCall::UpdateStaticSwitchPinsWithPersistentGuids()
{
	UNiagaraGraph* CalledGraph = GetCalledGraph();
	if (CalledGraph != nullptr)
	{
		TArray<UEdGraphPin*> InputPins;
		GetInputPins(InputPins);
		TArray<FNiagaraVariable> StaticSwitchVariables = CalledGraph->FindStaticSwitchInputs();
		for (UEdGraphPin* InputPin : InputPins)
		{
			FNiagaraVariable InputVariable = GetDefault<UEdGraphSchema_Niagara>()->PinToNiagaraVariable(InputPin);
			if (StaticSwitchVariables.Contains(InputVariable))
			{
				TOptional<FGuid> VariableGuid = CalledGraph->GetScriptVariableGuid(InputVariable);
				if (VariableGuid.IsSet())
				{
					InputPin->PersistentGuid = *VariableGuid;
				}
			}
		}
	}
}

bool UNiagaraNodeFunctionCall::RefreshFromExternalChanges()
{
	bool bReload = false;
	if (FunctionScript)
	{
		FixupFunctionScriptVersion();
		UNiagaraScriptSource* Source = GetFunctionScriptSource();
		// cooked niagara can be missing source, but we shouldn't need to reload if already cooked for editor
		if (!GetOutermost()->bIsCookedForEditor)
		{
			if (ensureMsgf(Source != nullptr, TEXT("No source found for FunctionScript %s in RefreshFromExternalChanges for %s"), *GetPathNameSafe(FunctionScript), *GetPathNameSafe(this)))
			{
				bReload = CachedChangeId != Source->NodeGraph->GetChangeID();
			}
		}
	}
	else if (Signature.IsValid())
	{
		bReload = true;		
	}

	UpdateNodeErrorMessage();

	// Go over the static switch parameters to set their propagation status on the pins
	UNiagaraGraph* CalledGraph = GetCalledGraph();
	if (CalledGraph)
	{
		CleanupPropagatedSwitchValues();
		FPinCollectorArray InputPins;
		GetInputPins(InputPins);
		for (FNiagaraVariable InputVar : CalledGraph->FindStaticSwitchInputs())
		{
			for (UEdGraphPin* Pin : InputPins)
			{
				if (InputVar.GetName().IsEqual(Pin->GetFName()))
				{
					Pin->bDefaultValueIsIgnored = FindPropagatedVariable(InputVar) != nullptr;
					break;
				}
			}
		}
	}

	if (bReload)
	{
		// TODO - Leverage code in reallocate pins to determine if any pins have changed...
		ReallocatePins(false);
		FNiagaraStackGraphUtilities::SynchronizeReferencingMapPinsWithFunctionCall(*this);
		FNiagaraStackGraphUtilities::FixDynamicInputNodeOutputPinsFromExternalChanges(*this);
		return true;
	}
	else
	{
		return false;
	}
}

void UNiagaraNodeFunctionCall::GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const
{
	if (HasValidScriptAndGraph())
	{
		// We don't know which graph type we're referencing, so we try them all... may need to replace this with something faster in the future.
		UNiagaraGraph* FunctionGraph = GetCalledGraph();
		FunctionGraph->RebuildCachedCompileIds();
		for (int32 i = (int32)ENiagaraScriptUsage::Function; i <= (int32)ENiagaraScriptUsage::DynamicInput; i++)
		{
			FGuid FoundGuid = FunctionGraph->GetBaseId((ENiagaraScriptUsage)i, FGuid(0, 0, 0, 0));
			FNiagaraCompileHash FoundCompileHash = FunctionGraph->GetCompileDataHash((ENiagaraScriptUsage)i, FGuid(0, 0, 0, 0));
			if (FoundGuid.IsValid() && FoundCompileHash.IsValid())
			{
				HashCollector.AddHash(FoundCompileHash, FunctionGraph->GetPathName());
				FunctionGraph->GatherExternalDependencyData((ENiagaraScriptUsage)i, FGuid(0, 0, 0, 0), HashCollector);
			}
		}
	}
}

void UNiagaraNodeFunctionCall::UpdateCompileHashForNode(FSHA1& HashState) const
{
	Super::UpdateCompileHashForNode(HashState);
	HashState.UpdateWithString(*GetFunctionName(), GetFunctionName().Len());
}

void UNiagaraNodeFunctionCall::UpdateReferencedStaticsHashForNode(FSHA1& HashState) const
{

	// Assume that we only care about static switch pins and their current values
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	for (int32 PinIndex = 0; PinIndex < InputPins.Num(); ++PinIndex)
	{
		if (!InputPins[PinIndex])
		{
			continue;
		}

		if (InputPins[PinIndex]->PersistentGuid.IsValid() && InputPins[PinIndex]->LinkedTo.Num() == 0 && InputPins[PinIndex]->bNotConnectable == true)
		{			
			HashState.Update((const uint8*)&InputPins[PinIndex]->PersistentGuid, sizeof(InputPins[PinIndex]->PersistentGuid));
			HashState.UpdateWithString(*InputPins[PinIndex]->DefaultValue, InputPins[PinIndex]->DefaultValue.Len());
		}
	}

}

void UNiagaraNodeFunctionCall::GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);
	if ( FunctionScript != nullptr || Signature.Inputs.Num() == 0 )
	{
		return;
	}

	if ( Signature.Inputs[0].GetType().IsDataInterface() == false )
	{
		return;
	}

	if ( UClass* DIClass = Cast<UClass>(Signature.Inputs[0].GetType().GetClass()) )
	{
		INiagaraDataInterfaceNodeActionProvider::GetNodeContextMenuActions(DIClass, Menu, Context, Signature);
	}
}

TSharedRef<SWidget> UNiagaraNodeFunctionCall::CreateTitleRightWidget()
{
	if ( FunctionScript != nullptr || Signature.Inputs.Num() == 0 )
	{
		return SNullWidget::NullWidget;
	}

	if ( Signature.Inputs[0].GetType().IsDataInterface() == false )
	{
		return SNullWidget::NullWidget;
	}

	if ( UClass* DIClass = Cast<UClass>(Signature.Inputs[0].GetType().GetClass()) )
	{
		FString MenuBaseName = "GraphEditor.Niagara.InlinePrimaryActions.";
		FString MenuName = MenuBaseName.Append(DIClass->GetName());

		if(UToolMenus::Get()->IsMenuRegistered(FName(*MenuName)) == false)
		{
			UToolMenu* InlinePrimaryMenu = UToolMenus::Get()->RegisterMenu(FName(*MenuName));
			InlinePrimaryMenu->AddDynamicSection("GetInlineNodeContextActions", FNewToolMenuDelegate::CreateLambda([DIClass, this](UToolMenu* InMenu)
			{
				// it's important to pass in the menu from the lambda parameter instead of using the original menu
				INiagaraDataInterfaceNodeActionProvider::GetInlineNodeContextMenuActions(DIClass, InMenu);
			}));
		}

		TSharedPtr<SWidget> ButtonContentWidget = nullptr;

		INiagaraDataInterfaceNodeActionProvider::FInlineMenuDisplayOptions DisplayOptions = INiagaraDataInterfaceNodeActionProvider::GetInlineMenuDisplayOptions(DIClass, this);

		if(DisplayOptions.bDisplayInline == false)
		{
			return SNullWidget::NullWidget;
		}
		
		if(DisplayOptions.DisplayBrush != nullptr)
		{
			ButtonContentWidget = SNew(SImage)
				.Image(DisplayOptions.DisplayBrush);
		}
		else
		{
			ButtonContentWidget = SNew(STextBlock)
				.Text(DisplayOptions.DisplayName.IsEmpty() ? LOCTEXT("NodeInlineOptionsMenuLabel", "Options") : DisplayOptions.DisplayName);
		}

		TSharedRef<SWidget> Result = SNew(SComboButton)
		.ComboButtonStyle(&FAppStyle::GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
		.OnGetMenuContent(FOnGetContent::CreateLambda([this, MenuName, DIClass]()
		{
			UGraphNodeContextMenuContext* ContextObject = NewObject<UGraphNodeContextMenuContext>();
			ContextObject->Init(this->GetGraph(), this, nullptr, false);
			FToolMenuContext Context(ContextObject);

			// we generate the menu instead of using FindMenu as this will cause the 'instance' of this menu to be created, with dynamic sections being generated.
			UToolMenu* GeneratedMenu = UToolMenus::Get()->GenerateMenu(FName(*MenuName), Context);
			return UToolMenus::Get()->GenerateWidget(GeneratedMenu);
		}))
		.ButtonContent()
		[
			ButtonContentWidget.ToSharedRef()
		];

		if(DisplayOptions.TooltipText.IsEmpty() == false)
		{
			Result->SetToolTipText(DisplayOptions.TooltipText);
		}
		
		return Result;
	}

	return SNullWidget::NullWidget;
}

void UNiagaraNodeFunctionCall::CollectAddPinActions(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin)const
{
	Super::CollectAddPinActions(Collector, AddPin);
	if (FunctionScript == nullptr && Signature.Inputs.Num() > 0)
	{
		UClass* DIClass = Cast<UClass>(Signature.Inputs[0].GetType().GetClass());
		if (DIClass)
		{
			INiagaraDataInterfaceNodeActionProvider::CollectAddPinActions(DIClass, Collector, AddPin);
		}
	}
}

bool UNiagaraNodeFunctionCall::HasValidScriptAndGraph() const
{
	return FunctionScript != nullptr && GetCalledGraph() != nullptr;
}

void UNiagaraNodeFunctionCall::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());

	auto FindParameterMapPinIndex = [&]()
	{
		FPinCollectorArray InputPins;
		GetInputPins(InputPins);
		for (int32 PinIndex = 0; PinIndex < InputPins.Num(); ++PinIndex)
		{
			if (Schema->PinToTypeDefinition(InputPins[PinIndex]) == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				return PinIndex;
			}
		}
		return int32(INDEX_NONE);
	};

	// when dealing with function scripts it's not sufficient to assume that InputPin[0] is the ParamMap pin, and so we
	// need to search through the inputs to find the right one
	const int32 ParamMapPinIndex = FindParameterMapPinIndex();
	const bool bHasParamMapPin = ParamMapPinIndex != INDEX_NONE;
	const UEdGraphPin* ParamMapPin = bHasParamMapPin ? GetInputPin(ParamMapPinIndex) : nullptr;
	if (bHasParamMapPin && ParamMapPin->LinkedTo.Num() == 0)
	{
		// Looks like this function call is not yet hooked up. Skip it to prevent cascading errors in the compilation
		return;
	}

	Super::BuildParameterMapHistory(OutHistory, bRecursive, bFilterForCompilation);
	if (!IsNodeEnabled() && OutHistory.GetIgnoreDisabled())
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	const bool ValidScriptAndGraph = HasValidScriptAndGraph();

	if (ValidScriptAndGraph == false && Signature.IsValid() == false)
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	if (ValidScriptAndGraph)
	{
		UNiagaraGraph* FunctionGraph = GetCalledGraph();

		UNiagaraNodeOutput* OutputNode = FunctionGraph->FindOutputNode(ENiagaraScriptUsage::Function);
		if (OutputNode == nullptr)
		{
			OutputNode = FunctionGraph->FindOutputNode(ENiagaraScriptUsage::Module);
		}
		if (OutputNode == nullptr)
		{
			OutputNode = FunctionGraph->FindOutputNode(ENiagaraScriptUsage::DynamicInput);
		}

		FPinCollectorArray InputPins;
		GetInputPins(InputPins);
		
		// Because the pin traversals above the the Super::BuildParameterMapHistory don't traverse into the input pins if they aren't linked.
		// This leaves static variables that aren't linked to anything basically ignored. The code below fills in that gap.
		// We don't do it in the base code because many input pins are traversed only if the owning node wants them to be.
		// OutputNode is another example of manually registering the input pins.
		for (UEdGraphPin* Pin : InputPins)
		{
			// Make sure to register pin constants
			if (Pin->LinkedTo.Num() == 0 && Schema->IsPinStatic(Pin) && Pin->DefaultValue.Len() != 0)
			{
				OutHistory.RegisterConstantFromInputPin(Pin, Pin->DefaultValue);
			}
		}

		FCompileConstantResolver FunctionResolver = OutHistory.ConstantResolver->WithDebugState(DebugState);
		if (OutHistory.HasCurrentUsageContext())
		{
			// if we traverse a full emitter graph the usage might change during the traversal, so we need to update the constant resolver
			FunctionResolver = FunctionResolver.WithUsage(OutHistory.GetCurrentUsageContext());
		}
		FNiagaraEditorUtilities::SetStaticSwitchConstants(FunctionGraph, InputPins, FunctionResolver);

		int32 ParamMapIdx = INDEX_NONE;
		uint32 NodeIdx = INDEX_NONE;

		if (bHasParamMapPin && bRecursive)
		{
			ParamMapIdx = OutHistory.TraceParameterMapOutputPin((ParamMapPin->LinkedTo[0]));
		}

		OutHistory.EnterFunction(GetFunctionName(), FunctionGraph, this);
		if (ParamMapIdx != INDEX_NONE)
		{
			NodeIdx = OutHistory.BeginNodeVisitation(ParamMapIdx, this);
		}

		const bool DoDepthTraversal = OutHistory.ShouldProcessDepthTraversal(FunctionGraph);

		// check if we should be recursing deeper into the graph
		if (DoDepthTraversal)
		{
			++OutHistory.CurrentGraphDepth;

			OutputNode->BuildParameterMapHistory(OutHistory, true, bFilterForCompilation);

			--OutHistory.CurrentGraphDepth;
		}

		// Since we're about to lose the pin calling context, we finish up the function call parameter map pin wiring
		// here when we have the calling context and the child context still available to us...
		FPinCollectorArray OutputPins;
		GetOutputPins(OutputPins);

		TArray<TPair<UEdGraphPin*, int32>, TInlineAllocator<16> > MatchedPairs;
		TArray<TPair<UEdGraphPin*, int32>, TInlineAllocator<16> > MatchedConstants;
		TArray<bool, TInlineAllocator<16> > OutputMatched;
		TArray<FNiagaraVariableBase, TInlineAllocator<16>> OutputVariables;

		const int32 OutputPinCount = OutputPins.Num();

		OutputMatched.AddDefaulted(OutputPinCount);
		OutputVariables.Reserve(OutputPinCount);

		for (int32 OutputIt = 0; OutputIt < OutputPinCount; ++OutputIt)
		{
			OutputVariables.Emplace(Schema->PinToNiagaraVariable(OutputPins[OutputIt]));
		}

		// Find the matches of names and types of the sub-graph output pins and this function call nodes' outputs.
		for (UEdGraphPin* ChildOutputNodePin : OutputNode->GetAllPins())
		{
			if (!ChildOutputNodePin)
			{
				continue;
			}

			const FNiagaraVariableBase VarChild = Schema->PinToNiagaraVariable(ChildOutputNodePin);

			if (ChildOutputNodePin->LinkedTo.Num() > 0 && VarChild.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				for (int32 i = 0; i < OutputPins.Num(); i++)
				{
					if (OutputVariables[i].IsEquivalent(VarChild))
					{
						TPair<UEdGraphPin*, int32>& Pair = MatchedPairs.AddZeroed_GetRef();
						Pair.Key = OutputPins[i];
						Pair.Value = OutHistory.TraceParameterMapOutputPin(ChildOutputNodePin->LinkedTo[0]);
						OutputMatched[i] = true;
					}
				}
			}
			else if (VarChild.GetType().IsStatic())
			{
				for (int32 i = 0; i < OutputPins.Num(); i++)
				{
					if (!OutputMatched[i] && OutputVariables[i].IsEquivalent(VarChild))
					{
						TPair<UEdGraphPin*, int32>& Pair = MatchedConstants.AddZeroed_GetRef();
						Pair.Key = OutputPins[i];
						Pair.Value = OutHistory.GetConstantFromInputPin(ChildOutputNodePin);
						OutputMatched[i] = true;
					}
				}
			}
		}

		if (ParamMapIdx != INDEX_NONE)
		{
			OutHistory.EndNodeVisitation(ParamMapIdx, NodeIdx);
		}

		OutHistory.ExitFunction(this);

		if (DoDepthTraversal)
		{
			for (int32 i = 0; i < MatchedPairs.Num(); i++)
			{
				OutHistory.RegisterParameterMapPin(MatchedPairs[i].Value, MatchedPairs[i].Key);
			}

			for (int32 i = 0; i < MatchedConstants.Num(); i++)
			{
				OutHistory.RegisterConstantPin(MatchedConstants[i].Value, MatchedConstants[i].Key);
			}
		}
		else
		{
			for (int32 OutputPinIt = 0; OutputPinIt < OutputPinCount; ++OutputPinIt)
			{
				if (OutputVariables[OutputPinIt].GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
				{
					OutHistory.RegisterParameterMapPin(ParamMapIdx, OutputPins[OutputPinIt]);
					break;
				}
			}
		}
	}
	else if (Signature.bRequiresExecPin)
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
	}
}

void UNiagaraNodeFunctionCall::ChangeScriptVersion(FGuid NewScriptVersion, const FNiagaraScriptVersionUpgradeContext& UpgradeContext, bool bShowNotesInStack, bool bDeferOverridePinUpdate)
{
	bool bPreviousVersionValid = !InvalidScriptVersionReference.IsValid();
	if (NewScriptVersion == SelectedScriptVersion && bPreviousVersionValid)
	{
		return;
	}
	Modify();

	PythonUpgradeScriptWarnings.Empty();
	if (bPreviousVersionValid && FunctionScript && !UpgradeContext.bSkipPythonScript)
	{
		// run python update scripts
		TArray<FVersionedNiagaraScriptData*> UpgradeVersionData;
		FVersionedNiagaraScriptData* PreviousData = FunctionScript->GetScriptData(SelectedScriptVersion);
		FVersionedNiagaraScriptData* NewData = FunctionScript->GetScriptData(NewScriptVersion);
		for (const FNiagaraAssetVersion& Version : FunctionScript->GetAllAvailableVersions())
		{
			if (PreviousData->Version <= Version && Version <= NewData->Version)
			{
				UpgradeVersionData.Add(FunctionScript->GetScriptData(Version.VersionGuid));
			}
		}
		FNiagaraEditorUtilities::RunPythonUpgradeScripts(this, UpgradeVersionData, UpgradeContext, PythonUpgradeScriptWarnings);
	}
	
	InvalidScriptVersionReference = FGuid();
	PreviousScriptVersion = bShowNotesInStack ? SelectedScriptVersion : NewScriptVersion;
	SelectedScriptVersion = NewScriptVersion;

	if (!bDeferOverridePinUpdate)
	{
		UpdateOverridePins(UpgradeContext);
	}

	MarkNodeRequiresSynchronization(__FUNCTION__, true);
}

void UNiagaraNodeFunctionCall::UpdateOverridePins(const FNiagaraScriptVersionUpgradeContext& UpgradeContext)
{
	if (FunctionScript)
	{
		// Automatically remove old inputs so it does not show a bunch of warnings to the user
		FPinCollectorArray InputPins;
		GetInputPins(InputPins);
		for (UEdGraphPin* Pin : InputPins)
		{
			if (Pin->bOrphanedPin)
			{
				RemovePin(Pin);
			}
		}
		UNiagaraNodeParameterMapSet* OverrideNode = FNiagaraStackGraphUtilities::GetStackFunctionOverrideNode(*this);
		if (OverrideNode != nullptr)
		{
			FPinCollectorArray OverridePins;
			OverrideNode->Modify();
			OverrideNode->GetInputPins(OverridePins);

			if (!OverridePins.IsEmpty())
			{
				TMap<FName, FNiagaraTypeDefinition> FunctionInputNames;
				TArray<FNiagaraVariable> ModuleInputVariables;
				FNiagaraStackGraphUtilities::GetStackFunctionInputs(*this, ModuleInputVariables, UpgradeContext.ConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly);
				for (const FNiagaraVariable& InputVariable : ModuleInputVariables)
				{
					FunctionInputNames.Add(FNiagaraParameterHandle(InputVariable.GetName()).GetName(), InputVariable.GetType());
				}

				for (UEdGraphPin* OverridePin : OverridePins)
				{
					if (!FNiagaraStackGraphUtilities::IsOverridePinForFunction(*OverridePin, *this))
					{
						continue;
					}
					FName InputName = FNiagaraParameterHandle(OverridePin->PinName).GetName();
					FNiagaraTypeDefinition* InputType = FunctionInputNames.Find(InputName);
					FNiagaraTypeDefinition OverridePinType = UEdGraphSchema_Niagara::PinTypeToTypeDefinition(OverridePin->PinType);
					if (InputType != nullptr && *InputType != OverridePinType)
					{
						// looks like the type of the module input changed - we'll change the override pin type as well
						OverridePin->Modify();
						OverridePin->PinType = UEdGraphSchema_Niagara::TypeDefinitionToPinType(*InputType);
					}
					else if (InputType == nullptr)
					{
						// the input doesn't exist any more, delete the override pin
						TArray<TWeakObjectPtr<UNiagaraDataInterface>> RemovedDataObjects;
						FNiagaraStackGraphUtilities::RemoveNodesForStackFunctionInputOverridePin(*OverridePin, RemovedDataObjects);
						OverrideNode->RemovePin(OverridePin);
					}
				}
			}
		}
	}
}

UEdGraphPin* UNiagaraNodeFunctionCall::FindParameterMapDefaultValuePin(const FName VariableName, ENiagaraScriptUsage InParentUsage, FCompileConstantResolver ConstantResolver) const
{
	if (FunctionScript)
	{
		UNiagaraScriptSource* ScriptSource = GetFunctionScriptSource();
		if (ScriptSource != nullptr && ScriptSource->NodeGraph != nullptr)
		{
			// Set the static switch values so we traverse the correct node paths
			TArray<UEdGraphPin*> InputPins;
			GetInputPins(InputPins);

			FNiagaraEditorUtilities::SetStaticSwitchConstants(ScriptSource->NodeGraph, InputPins, ConstantResolver.WithDebugState(DebugState));
			
			return ScriptSource->NodeGraph->FindParameterMapDefaultValuePin(VariableName, FunctionScript->GetUsage(), InParentUsage);
		}
	}
	return nullptr;
}

void UNiagaraNodeFunctionCall::FindParameterMapDefaultValuePins(TConstArrayView<FName> VariableNames, ENiagaraScriptUsage InParentUsage, const FCompileConstantResolver& ConstantResolver, TArrayView<UEdGraphPin*> DefaultPins)
{
	if (FunctionScript)
	{
		UNiagaraScriptSource* ScriptSource = GetFunctionScriptSource();
		if (ScriptSource != nullptr && ScriptSource->NodeGraph != nullptr)
		{
			// Set the static switch values so we traverse the correct node paths
			TArray<UEdGraphPin*> InputPins;
			GetInputPins(InputPins);

			FNiagaraEditorUtilities::SetStaticSwitchConstants(ScriptSource->NodeGraph, InputPins, ConstantResolver.WithDebugState(DebugState));

			ScriptSource->NodeGraph->MultiFindParameterMapDefaultValuePins(VariableNames, FunctionScript->GetUsage(), InParentUsage, DefaultPins);
		}
	}
}

UEdGraphPin* UNiagaraNodeFunctionCall::FindStaticSwitchInputPin(const FName& VariableName) const
{
	UNiagaraGraph* CalledGraph = GetCalledGraph();
	if (CalledGraph == nullptr)
	{
		return nullptr;
	}
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	for (FNiagaraVariable InputVar : CalledGraph->FindStaticSwitchInputs())
	{
		if (InputVar.GetName().IsEqual(VariableName))
		{			
			for (UEdGraphPin* Pin : InputPins)
			{
				if (VariableName.IsEqual(Pin->GetFName()))
				{
					return Pin;
				}
			}
		}
	}
	return nullptr;
}

bool UNiagaraNodeFunctionCall::ContainsDebugSwitch() const
{
	UNiagaraGraph* CalledGraph = GetCalledGraph();
	if (CalledGraph)
	{
		TArray<UNiagaraNodeStaticSwitch*> Switches;
		CalledGraph->GetNodesOfClass<UNiagaraNodeStaticSwitch>(Switches);
		for (const auto& Switch : Switches)
		{
			if (Switch->IsDebugSwitch())
			{
				return true;
			}
		}

		TArray<UNiagaraNodeFunctionCall*> FunctionCalls;
		CalledGraph->GetNodesOfClass<UNiagaraNodeFunctionCall>(FunctionCalls);
		for (const auto& Function : FunctionCalls)
		{
			if (Function->ContainsDebugSwitch())
			{
				return true;
			}
		}
	}
	return false;
}

void UNiagaraNodeFunctionCall::SuggestName(FString SuggestedName, bool bForceSuggestion)
{
	ComputeNodeName(SuggestedName, bForceSuggestion);
}

UNiagaraNodeFunctionCall::FOnInputsChanged& UNiagaraNodeFunctionCall::OnInputsChanged()
{
	return OnInputsChangedDelegate;
}

FNiagaraPropagatedVariable* UNiagaraNodeFunctionCall::FindPropagatedVariable(const FNiagaraVariable& Variable)
{
	for (FNiagaraPropagatedVariable& Propagated : PropagatedStaticSwitchParameters)
	{
		if (Propagated.SwitchParameter == Variable)
		{
			return &Propagated;
		}
	}
	return nullptr;
}

void UNiagaraNodeFunctionCall::RemovePropagatedVariable(const FNiagaraVariable& Variable)
{
	for (int i = 0; i < PropagatedStaticSwitchParameters.Num(); i++)
	{
		if (PropagatedStaticSwitchParameters[i].SwitchParameter == Variable)
		{
			PropagatedStaticSwitchParameters.RemoveAt(i);
			return;
		}
	}
}

ENiagaraNumericOutputTypeSelectionMode UNiagaraNodeFunctionCall::GetNumericOutputTypeSelectionMode() const
{
	if (GetScriptData())
	{
		return GetScriptData()->NumericOutputTypeSelectionMode;
	}	
	return ENiagaraNumericOutputTypeSelectionMode::None;
}

void UNiagaraNodeFunctionCall::AutowireNewNode(UEdGraphPin* FromPin)
{
	UNiagaraNode::AutowireNewNode(FromPin);
	ComputeNodeName();
}

void UNiagaraNodeFunctionCall::RefreshSignature()
{
	if (Signature.IsValid() && Signature.bMemberFunction)
	{
		if ((Signature.Inputs.Num() > 0) && Signature.Inputs[0].GetType().IsDataInterface())
		{
			UNiagaraDataInterface* CDO = CastChecked<UNiagaraDataInterface>(Signature.Inputs[0].GetType().GetClass()->GetDefaultObject());
			TArray<FNiagaraFunctionSignature> BaseDIFuncs;
			CDO->GetFunctionSignatures(BaseDIFuncs);
			if (FNiagaraFunctionSignature* BaseSig = BaseDIFuncs.FindByPredicate([&](const FNiagaraFunctionSignature& CheckSig) { return Signature.Name == CheckSig.Name; }))
			{
				//Revert signature to base and re-add dynamic pins as new inputs and outputs.
				Signature = *BaseSig;				

				FPinCollectorArray FoundPins;
				GetInputPins(FoundPins);

				for (int32 i = 0; i < FoundPins.Num(); ++i)
				{
					UEdGraphPin* InputPin = FoundPins[i];
					FNiagaraVariable InputVariable = UEdGraphSchema_Niagara::PinToNiagaraVariable(InputPin);

					if(!BaseSig->Inputs.Contains(InputVariable) && IsAddPin(InputPin) == false && IsExecPin(InputPin) == false)
					{
						Signature.AddInput(InputVariable, FText::FromString(InputPin->PinToolTip));
					}
				}

				GetOutputPins(FoundPins);

				for (int32 i = 0; i < FoundPins.Num(); ++i)
				{
					UEdGraphPin* OutputPin = FoundPins[i];
					FNiagaraVariableBase InputVariable = UEdGraphSchema_Niagara::PinToNiagaraVariable(OutputPin);

					if(!BaseSig->Outputs.Contains(InputVariable) && IsAddPin(OutputPin) == false && IsExecPin(OutputPin) == false)
					{
						Signature.AddOutput(UEdGraphSchema_Niagara::PinToNiagaraVariable(OutputPin), FText::FromString(OutputPin->PinToolTip));
					}
				}

				return;
			}
		}
	}
	Signature = FNiagaraFunctionSignature();
}

bool UNiagaraNodeFunctionCall::IsBaseSignatureOfDataInterfaceFunction(const UEdGraphPin* Pin) const
{
	if ((Signature.Inputs.Num() > 0) && Signature.Inputs[0].GetType().IsDataInterface())
	{
		UNiagaraDataInterface* CDO = CastChecked<UNiagaraDataInterface>(Signature.Inputs[0].GetType().GetClass()->GetDefaultObject());
		TArray<FNiagaraFunctionSignature> BaseDIFuncs;
		CDO->GetFunctionSignatures(BaseDIFuncs);
		if (FNiagaraFunctionSignature* BaseSig = BaseDIFuncs.FindByPredicate([&](const FNiagaraFunctionSignature& CheckSig) { return Signature.Name == CheckSig.Name; }))
		{
			FPinCollectorArray FoundPins;
			TArray<FNiagaraVariableBase> InputOrOutputVariables;

			if(BaseSig->bRequiresExecPin && IsExecPin(Pin))
			{
				return true;
			}

			if(Pin->Direction == EGPD_Input)
			{
				GetInputPins(FoundPins);
				InputOrOutputVariables = BaseSig->GetInputs();
			}
			else
			{
				GetOutputPins(FoundPins);
				InputOrOutputVariables = BaseSig->Outputs;
			}
			
			FNiagaraVariableBase Variable = UEdGraphSchema_Niagara::PinToNiagaraVariable(Pin);

			if(InputOrOutputVariables.Contains(Variable))
			{
				return true;
			}
		}
	}

	return false;
}

void UNiagaraNodeFunctionCall::ComputeNodeName(FString SuggestedName, bool bForceSuggestion)
{
	FString FunctionName = FunctionScript ? FunctionScript->GetName() : Signature.GetNameString();
	FName ProposedName;
	if (SuggestedName.IsEmpty() == false)
	{ 
		// If we have a suggested name and and either there is no function name, or it is a permutation of the function name
		// it can be used as the proposed name.
		if (bForceSuggestion || FunctionName.IsEmpty() || SuggestedName == FunctionName || (SuggestedName.StartsWith(FunctionName) && SuggestedName.RightChop(FunctionName.Len()).IsNumeric()))
		{
			ProposedName = *SuggestedName;
		}
	}

	if(ProposedName == NAME_None)
	{
		const FString CurrentName = FunctionDisplayName;
		if (FunctionName.IsEmpty() == false)
		{
			ProposedName = *FunctionName;
		}
		else
		{
			ProposedName = *CurrentName;
		}
	}

	UNiagaraGraph* Graph = GetNiagaraGraph();
	TArray<UNiagaraNodeFunctionCall*> Nodes;
	Graph->GetNodesOfClass(Nodes);

	TSet<FName> Names;
	for (UNiagaraNodeFunctionCall* Node : Nodes)
	{
		CA_ASSUME(Node != nullptr);
		if (Node != this)
		{
			Names.Add(*Node->GetFunctionName());
		}
	}

	FString NewName = FNiagaraUtilities::GetUniqueName(ProposedName, Names).ToString();
	if (!FunctionDisplayName.Equals(NewName))
	{
		FunctionDisplayName = NewName;
	}
}

void UNiagaraNodeFunctionCall::SetPinAutoGeneratedDefaultValue(UEdGraphPin& FunctionInputPin, UNiagaraNodeInput& FunctionScriptInputNode)
{
	if (FunctionInputPin.bDefaultValueIsIgnored == false)
	{
		FPinCollectorArray InputPins;
		FunctionScriptInputNode.GetInputPins(InputPins);
		if (InputPins.Num() == 1 && InputPins[0]->bDefaultValueIsIgnored == false)
		{
			// If the function graph's input node had an input pin, and that pin's default wasn't ignored, use that value. 
			FunctionInputPin.AutogeneratedDefaultValue = InputPins[0]->DefaultValue;
		}
		else
		{
			const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
			FString PinDefaultValue;
			if (Schema->TryGetPinDefaultValueFromNiagaraVariable(FunctionScriptInputNode.Input, PinDefaultValue))
			{
				FunctionInputPin.AutogeneratedDefaultValue = PinDefaultValue;
			}
		}
	}
}

void UNiagaraNodeFunctionCall::CleanupPropagatedSwitchValues()
{
	for (int i = PropagatedStaticSwitchParameters.Num() - 1; i >= 0; i--)
	{
		FNiagaraPropagatedVariable& Propagated = PropagatedStaticSwitchParameters[i];
		if (Propagated.SwitchParameter.GetName().IsNone() || !IsValidPropagatedVariable(Propagated.SwitchParameter))
		{
			PropagatedStaticSwitchParameters.RemoveAt(i);
		}
	}
}

bool UNiagaraNodeFunctionCall::IsValidPropagatedVariable(const FNiagaraVariable& Variable) const
{
	UNiagaraGraph* Graph = GetCalledGraph();
	if (!Graph)
	{
		return false;
	}
	for (const FNiagaraVariable& Var : Graph->FindStaticSwitchInputs(false))
	{
		if (Var == Variable)
		{
			return true;
		}
	}
	return false;
}

bool UNiagaraNodeFunctionCall::FindAutoBoundInput(const UNiagaraNodeInput* InputNode, const UEdGraphPin* PinToAutoBind, FNiagaraVariable& OutFoundVar, ENiagaraInputNodeUsage& OutNodeUsage) const
{
	check(InputNode);
	if (PinToAutoBind->LinkedTo.Num() > 0 || !InputNode->CanAutoBind())
	{
		return false;
	}
	ensureMsgf(InputNode->IsExposed() == true, TEXT("AutoBind inputs should be exposed for Function(%s) Pin(%s)"), *FunctionDisplayName, *PinToAutoBind->GetName());

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	FNiagaraVariable PinVar = Schema->PinToNiagaraVariable(PinToAutoBind);

	//See if we can auto bind this pin to something in the caller script.
	const UNiagaraGraph* CallerGraph = GetNiagaraGraph();
	check(CallerGraph);
	UNiagaraNodeOutput* CallerOutputNodeSpawn = CallerGraph->FindOutputNode(ENiagaraScriptUsage::ParticleSpawnScript);
	UNiagaraNodeOutput* CallerOutputNodeUpdate = CallerGraph->FindOutputNode(ENiagaraScriptUsage::ParticleUpdateScript);

	//First, let's see if we're an attribute of this emitter. Only valid if we're a module call off the primary script.
	if (CallerOutputNodeSpawn || CallerOutputNodeUpdate)
	{
		UNiagaraNodeOutput* CallerOutputNode = CallerOutputNodeSpawn != nullptr ? CallerOutputNodeSpawn : CallerOutputNodeUpdate;
		check(CallerOutputNode);
		{
			FNiagaraVariable* AttrVarPtr = CallerOutputNode->Outputs.FindByPredicate([&](const FNiagaraVariable& Attr) { return PinVar.IsEquivalent(Attr); });
			if (AttrVarPtr)
			{
				OutFoundVar = *AttrVarPtr;
				OutNodeUsage = ENiagaraInputNodeUsage::Attribute;
				return true;
			}
		}
	}
	
	//Next, lets see if we are a system constant.
	//Do we need a smarter (possibly contextual) handling of system constants?
	const TArray<FNiagaraVariable>& SysConstants = FNiagaraConstants::GetEngineConstants();
	if (SysConstants.Contains(PinVar))
	{
		OutFoundVar = PinVar;
		OutNodeUsage = ENiagaraInputNodeUsage::SystemConstant;
		return true;
	}

	//Unable to auto bind.
	return false;
}

#undef LOCTEXT_NAMESPACE

