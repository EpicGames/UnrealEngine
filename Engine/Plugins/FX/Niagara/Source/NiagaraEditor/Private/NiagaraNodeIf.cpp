// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeIf.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraHlslTranslator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeIf)

#define LOCTEXT_NAMESPACE "NiagaraNodeIf"

const FString UNiagaraNodeIf::InputTruePinSuffix(" if True");
const FString UNiagaraNodeIf::InputFalsePinSuffix(" if False");
const FName UNiagaraNodeIf::ConditionPinName("Condition");

UNiagaraNodeIf::UNiagaraNodeIf(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNiagaraNodeIf::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// @TODO why do we need to have this post-change property here at all? 
	// Doing a null check b/c otherwise if doing a Duplicate via Ctrl-W, we die inside AllocateDefaultPins due to 
	// the point where we get this call not being completely formed.
	if (PropertyChangedEvent.Property != nullptr)
	{
		ReallocatePins();
	}
}

void UNiagaraNodeIf::PostLoad()
{
	Super::PostLoad();

	if (PathAssociatedPinGuids.Num() != OutputVars.Num())
	{
		PathAssociatedPinGuids.SetNum(OutputVars.Num());
	}

	auto LoadGuid = [&](FGuid& Guid, const FString& Name, const EEdGraphPinDirection Direction, const FString* LegacyName = nullptr)
	{
		UEdGraphPin* Pin = FindPin(Name, Direction);
		if ( Pin == nullptr && LegacyName != nullptr )
		{
			Pin = FindPin(*LegacyName, Direction);
			if (Pin)
			{
				Pin->PinName = FName(*Name);
			}
		}

		if (Pin)
		{
			if (!Pin->PersistentGuid.IsValid())
			{
				Pin->PersistentGuid = FGuid::NewGuid();
			}
			Guid = Pin->PersistentGuid;
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Error, "Unable to find output pin named %ls", *Name);
		}
	};

	LoadGuid(ConditionPinGuid, ConditionPinName.ToString(), EGPD_Input);
	for (int32 i = 0; i < OutputVars.Num(); i++)
	{
		const FString VarName = OutputVars[i].GetName().ToString();
		LoadGuid(PathAssociatedPinGuids[i].OutputPinGuid, VarName, EGPD_Output);

		const FString InputTrueName = VarName + InputTruePinSuffix;
		const FString LegacyInputTrueName = VarName + TEXT(" A");
		LoadGuid(PathAssociatedPinGuids[i].InputTruePinGuid, InputTrueName, EGPD_Input, &LegacyInputTrueName);

		const FString InputFalseName = VarName + InputFalsePinSuffix;
		const FString LegacyInputFalseName = VarName + TEXT(" B");
		LoadGuid(PathAssociatedPinGuids[i].InputFalsePinGuid, InputFalseName, EGPD_Input, &LegacyInputFalseName);
	}
}

bool UNiagaraNodeIf::AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const
{
	// Explicitly allow Numeric types, and explicitly disallow ParameterMap types
	return (Super::AllowNiagaraTypeForAddPin(InType) || InType == FNiagaraTypeDefinition::GetGenericNumericDef()) && InType != FNiagaraTypeDefinition::GetParameterMapDef();
}

void UNiagaraNodeIf::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	//Add the condition pin.
	UEdGraphPin* ConditionPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetBoolDef()), ConditionPinName);
	UNiagaraNode::SetPinDefaultToTypeDefaultIfUnset(ConditionPin);
	ConditionPin->PersistentGuid = ConditionPinGuid;

	//Create the inputs for each path.
	for (int32 Index = 0; Index < OutputVars.Num(); Index++)
	{
		const FNiagaraVariable& Var = OutputVars[Index];
		UEdGraphPin* NewPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Var.GetType()), *(Var.GetName().ToString() + InputTruePinSuffix));
		UNiagaraNode::SetPinDefaultToTypeDefaultIfUnset(NewPin);
		NewPin->PersistentGuid = PathAssociatedPinGuids[Index].InputTruePinGuid;
	}

	for (int32 Index = 0; Index < OutputVars.Num(); Index++)
	{
		const FNiagaraVariable& Var = OutputVars[Index];
		UEdGraphPin* NewPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Var.GetType()), *(Var.GetName().ToString() + InputFalsePinSuffix));
		UNiagaraNode::SetPinDefaultToTypeDefaultIfUnset(NewPin);
		NewPin->PersistentGuid = PathAssociatedPinGuids[Index].InputFalsePinGuid;
	}

	for (int32 Index = 0; Index < OutputVars.Num(); Index++)
	{
		const FNiagaraVariable& Var = OutputVars[Index];
		UEdGraphPin* NewPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(Var.GetType()), Var.GetName());
		NewPin->PersistentGuid = PathAssociatedPinGuids[Index].OutputPinGuid;
	}

	CreateAddPin(EGPD_Output);
}

void UNiagaraNodeIf::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	checkSlow(PathAssociatedPinGuids.Num() == OutputVars.Num());
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());

	int32 Condition = Translator->CompileInputPin(GetPinByGuid(ConditionPinGuid));

	TArray<int32> PathTrue;
	PathTrue.Reserve(PathAssociatedPinGuids.Num());
	for (const FPinGuidsForPath& PerPathAssociatedPinGuids : PathAssociatedPinGuids)
	{
		const UEdGraphPin* InputTruePin = GetPinByGuid(PerPathAssociatedPinGuids.InputTruePinGuid);
		if (Schema->PinToTypeDefinition(InputTruePin) == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			Translator->Error(LOCTEXT("UnsupportedParamMapInIf", "Parameter maps are not supported in if nodes."), this, InputTruePin);
		}
		PathTrue.Add(Translator->CompileInputPin(InputTruePin));
	}
	TArray<int32> PathFalse;
	PathFalse.Reserve(PathAssociatedPinGuids.Num());
	for (const FPinGuidsForPath& PerPathAssociatedPinGuids : PathAssociatedPinGuids)
	{
		const UEdGraphPin* InputFalsePin = GetPinByGuid(PerPathAssociatedPinGuids.InputFalsePinGuid);
		if (Schema->PinToTypeDefinition(InputFalsePin) == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			Translator->Error(LOCTEXT("UnsupportedParamMapInIf", "Parameter maps are not supported in if nodes."), this, InputFalsePin);
		}
		PathFalse.Add(Translator->CompileInputPin(InputFalsePin));
	}

	Translator->If(this, OutputVars, Condition, PathTrue, PathFalse, Outputs);
}

ENiagaraNumericOutputTypeSelectionMode UNiagaraNodeIf::GetNumericOutputTypeSelectionMode() const
{
	return ENiagaraNumericOutputTypeSelectionMode::Largest;
}


void UNiagaraNodeIf::ResolveNumerics(const UEdGraphSchema_Niagara* Schema, bool bSetInline, TMap<TPair<FGuid, UEdGraphNode*>, FNiagaraTypeDefinition>* PinCache)
{
	int32 VarStartIdx = 1;
	for (int32 i = 0; i < OutputVars.Num(); ++i)
	{
		// Fix up numeric input pins and keep track of numeric types to decide the output type.
		TArray<UEdGraphPin*> InputPins;
		TArray<UEdGraphPin*> OutputPins;
		
		InputPins.Add(Pins[i + VarStartIdx]);
		InputPins.Add(Pins[i + VarStartIdx + OutputVars.Num()]);
		OutputPins.Add(Pins[i + VarStartIdx + 2 * OutputVars.Num()]);
		NumericResolutionByPins(Schema, InputPins, OutputPins,  bSetInline, PinCache);
	}
}

bool UNiagaraNodeIf::AllowExternalPinTypeChanges(const UEdGraphPin* InGraphPin) const
{
	// only allow pin changes for output pins
	if (InGraphPin->Direction == EGPD_Output)
	{
		return true;
	}

	return false;
}

bool UNiagaraNodeIf::AllowNiagaraTypeForPinTypeChange(const FNiagaraTypeDefinition& InType, UEdGraphPin* Pin) const
{
	return InType.GetScriptStruct() != nullptr
		&& InType != FNiagaraTypeDefinition::GetGenericNumericDef()
		&& !InType.IsInternalType();
}

bool UNiagaraNodeIf::OnNewPinTypeRequested(UEdGraphPin* PinToChange, FNiagaraTypeDefinition NewType)
{
	if (PinToChange->Direction == EGPD_Output)
	{
		ChangeOutputType(PinToChange, NewType);
		return true;
	}

	return false;
}

bool UNiagaraNodeIf::RefreshFromExternalChanges()
{
	// TODO - Leverage code in reallocate pins to determine if any pins have changed...
	ReallocatePins();
	return true;
}

FGuid UNiagaraNodeIf::AddOutput(FNiagaraTypeDefinition Type, const FName& Name)
{
	FPinGuidsForPath& NewPinGuidsForPath = PathAssociatedPinGuids.Add_GetRef(FPinGuidsForPath());

	FNiagaraVariable NewOutput(Type, Name);
	OutputVars.Add(NewOutput);
	FGuid Guid = FGuid::NewGuid();
	NewPinGuidsForPath.OutputPinGuid = Guid;

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	FGuid PinTrueGuid = FGuid::NewGuid();
	UEdGraphPin* PinTrue = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Type), *(Name.ToString() + InputTruePinSuffix), PathAssociatedPinGuids.Num());
	PinTrue->PersistentGuid = PinTrueGuid;
	UNiagaraNode::SetPinDefaultToTypeDefaultIfUnset(PinTrue);
	NewPinGuidsForPath.InputTruePinGuid = PinTrueGuid;

	FGuid PinFalseGuid = FGuid::NewGuid();
	UEdGraphPin* PinFalse = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Type), *(Name.ToString() + InputFalsePinSuffix), PathAssociatedPinGuids.Num() * 2);
	PinFalse->PersistentGuid = PinFalseGuid;
	UNiagaraNode::SetPinDefaultToTypeDefaultIfUnset(PinFalse);
	NewPinGuidsForPath.InputFalsePinGuid = PinFalseGuid;

	return Guid;
}

void UNiagaraNodeIf::ChangeOutputType(UEdGraphPin* OutputPin, FNiagaraTypeDefinition TypeDefinition)
{
	for (int32 i = 0; i < PathAssociatedPinGuids.Num(); i++)
	{
		const FPinGuidsForPath& PinGuids = PathAssociatedPinGuids[i];
		if (PinGuids.OutputPinGuid == OutputPin->PersistentGuid)
		{
			TSet<FName> OutputNames;
			for (const FNiagaraVariable& Output : OutputVars)
			{
				OutputNames.Add(Output.GetName());
			}
			FName OutputName = FNiagaraUtilities::GetUniqueName(*TypeDefinition.GetNameText().ToString(), OutputNames);

			// replace the old var with a new one at the same index, so the matchup between IDs and vars is still the same
			OutputVars[i] = FNiagaraVariable(TypeDefinition, OutputName);
			GetPinByPersistentGuid(PinGuids.InputTruePinGuid)->bOrphanedPin = true;
			GetPinByPersistentGuid(PinGuids.InputFalsePinGuid)->bOrphanedPin = true;
			OutputPin->bOrphanedPin = true;

			ReallocatePins(true);
			return;
		}
	}
}

const UEdGraphPin* UNiagaraNodeIf::GetPinByGuid(const FGuid& InGuid) const
{
	UEdGraphPin* FoundPin = *Pins.FindByPredicate([&InGuid](const UEdGraphPin* Pin) { return Pin->PersistentGuid == InGuid; });
	checkf(FoundPin != nullptr, TEXT("Failed to get pin by cached Guid!"));
	return FoundPin;
}

void UNiagaraNodeIf::OnPinRemoved(UEdGraphPin* PinToRemove)
{
	auto FindByOutputPinGuidPredicate = [=](const FPinGuidsForPath& PerPinGuidsForPath) { return PerPinGuidsForPath.OutputPinGuid == PinToRemove->PersistentGuid && PinToRemove->bOrphanedPin == false; };
	int32 FoundIndex = PathAssociatedPinGuids.IndexOfByPredicate(FindByOutputPinGuidPredicate);
	if (FoundIndex != INDEX_NONE)
	{
		OutputVars.RemoveAt(FoundIndex);
		PathAssociatedPinGuids.RemoveAt(FoundIndex);
	}
	ReallocatePins();
}

void UNiagaraNodeIf::OnNewTypedPinAdded(UEdGraphPin*& NewPin)
{
	Super::OnNewTypedPinAdded(NewPin);

	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraTypeDefinition OutputType = Schema->PinToTypeDefinition(NewPin);

	TSet<FName> OutputNames;
	for (const FNiagaraVariable& Output : OutputVars)
	{
		OutputNames.Add(Output.GetName());
	}
	FName OutputName = FNiagaraUtilities::GetUniqueName(*OutputType.GetNameText().ToString(), OutputNames);

	FGuid Guid = AddOutput(OutputType, OutputName);

	// Update the pin's data too so that it's connection is maintained after reallocating.
	NewPin->PinName = OutputName;
	NewPin->PersistentGuid = Guid;
}

void UNiagaraNodeIf::OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName)
{
	auto FindByOutputPinGuidPredicate = [=](const FPinGuidsForPath& PerPinGuidsForPath) { return PerPinGuidsForPath.OutputPinGuid == RenamedPin->PersistentGuid; };
	int32 FoundIndex = PathAssociatedPinGuids.IndexOfByPredicate(FindByOutputPinGuidPredicate);
	if(FoundIndex != INDEX_NONE)
	{
		TSet<FName> OutputNames;
		for (int32 Index = 0; Index < OutputVars.Num(); Index++)
		{
			if (FoundIndex != Index)
			{
				OutputNames.Add(OutputVars[Index].GetName());
			}
		}
		const FName OutputName = FNiagaraUtilities::GetUniqueName(RenamedPin->PinName, OutputNames);
		OutputVars[FoundIndex].SetName(OutputName);
	}
	ReallocatePins();
}

bool UNiagaraNodeIf::CanRenamePin(const UEdGraphPin* Pin) const
{
	return Super::CanRenamePin(Pin) && Pin->Direction == EGPD_Output;
}

bool UNiagaraNodeIf::CanRemovePin(const UEdGraphPin* Pin) const
{
	return Super::CanRemovePin(Pin) && Pin->Direction == EGPD_Output;
}


FText UNiagaraNodeIf::GetTooltipText() const
{
	return LOCTEXT("IfDesc", "If Condition is true, the output value will be the True pin, otherwise output will be the False pin.");
}

FText UNiagaraNodeIf::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("IfTitle", "If");
}

#undef LOCTEXT_NAMESPACE

