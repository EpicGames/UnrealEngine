//// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NiagaraNodeWithDynamicPins.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraNodeIf.generated.h"

USTRUCT()
struct FPinGuidsForPath
{
	GENERATED_USTRUCT_BODY()

	FPinGuidsForPath()
		: OutputPinGuid(FGuid())
		, InputTruePinGuid(FGuid())
		, InputFalsePinGuid(FGuid())
	{
	}

	bool IsValid() const
	{
		return OutputPinGuid.IsValid() && InputTruePinGuid.IsValid() && InputFalsePinGuid.IsValid();
	}

	UPROPERTY()
	FGuid OutputPinGuid;

	UPROPERTY()
	FGuid InputTruePinGuid;

	UPROPERTY()
	FGuid InputFalsePinGuid;
};

UCLASS(MinimalAPI)
class UNiagaraNodeIf : public UNiagaraNodeWithDynamicPins
{
	GENERATED_UCLASS_BODY()

public:

	/** Outputs of this branch. */
	UPROPERTY()
	TArray<FNiagaraVariable> OutputVars;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	TArray<FPinGuidsForPath> PathAssociatedPinGuids;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	FGuid ConditionPinGuid;

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End EdGraphNode Interface

	//~ Begin UNiagaraNode Interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual bool RefreshFromExternalChanges() override;
	virtual ENiagaraNumericOutputTypeSelectionMode GetNumericOutputTypeSelectionMode() const;
	virtual void ResolveNumerics(const UEdGraphSchema_Niagara* Schema, bool bSetInline, TMap<TPair<FGuid, UEdGraphNode*>, FNiagaraTypeDefinition>* PinCache);
	virtual bool AllowExternalPinTypeChanges(const UEdGraphPin* InGraphPin) const override;
	virtual bool AllowNiagaraTypeForPinTypeChange(const FNiagaraTypeDefinition& InType, UEdGraphPin* Pin) const override;
	virtual bool OnNewPinTypeRequested(UEdGraphPin* PinToChange, FNiagaraTypeDefinition NewType) override;
	//~ End UNiagaraNode Interface

protected:

	/** Helper function to create a variable to add to the OutputVars and FGuid to add to PathAssociatedPinGuids. */
	FGuid AddOutput(FNiagaraTypeDefinition Type, const FName& Name);
	void ChangeOutputType(UEdGraphPin* OutputPin, FNiagaraTypeDefinition TypeDefinition);
	
	/** Helper to get a pin in the pins list by GUID */
	const UEdGraphPin* GetPinByGuid(const FGuid& InGuid) const;

	//~ Begin EdGraphNode Interface
	virtual void OnPinRemoved(UEdGraphPin* PinToRemove) override;
	//~ End EdGraphNode Interface
	
	//~ Begin UNiagaraNodeWithDynamicPins Interface
	virtual void OnNewTypedPinAdded(UEdGraphPin*& NewPin) override;
	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName) override;
	virtual bool CanRenamePin(const UEdGraphPin* Pin) const override;
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const override;
	virtual bool CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const override { return false; }
	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const override;
	//~ End UNiagaraNodeWithDynamicPins Interface

private:
	static const FString InputTruePinSuffix;
	static const FString InputFalsePinSuffix;
	static const FName ConditionPinName;
};
