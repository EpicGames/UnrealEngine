// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeInput.h"
#include "UObject/UnrealType.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraGraph.h"

#include "NiagaraNodeOutput.h"
#include "NiagaraNodeOp.h"
#include "NiagaraNodeFunctionCall.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraConstants.h"
#include "NiagaraDataInterface.h"
#include "Widgets/SNiagaraGraphNodeInput.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeInput)

#define LOCTEXT_NAMESPACE "NiagaraNodeInput"

DECLARE_CYCLE_STAT(TEXT("NiagaraEditor - UNiagaraNodeInput - SortNodes"), STAT_NiagaraEditor_UNiagaraNodeInput_SortNodes, STATGROUP_NiagaraEditor);

UNiagaraNodeInput::UNiagaraNodeInput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Usage(ENiagaraInputNodeUsage::Parameter)
	, CallSortPriority(0)
	, DataInterface(nullptr)
	, ObjectAsset(nullptr)
{
	bCanRenameNode = true;
}

void UNiagaraNodeInput::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		UClass* Class = Input.GetType().GetClass();

		if (DataInterface)
		{
			if (!IsDataInterface() || DataInterface->GetClass() != Class)
			{
				//Class has changed so clear this out and allocate pins will create a new instance of the correct type.
				//Should we preserve old objects somewhere so settings aren't lost when switching around types?
				DataInterface = nullptr;
			}
			else
			{
				//Keep it with the same name as the input 
				if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(UNiagaraNodeInput, Input))
				{
					DataInterface->Rename(*Input.GetName().ToString());
				}
			}
		}

		if (ObjectAsset)
		{
			if (!IsObjectAsset() || ObjectAsset->GetClass() != Class)
			{
				ObjectAsset = nullptr;
			}
		}

		ReallocatePins();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UNiagaraNodeInput::PostLoad()
{
	Super::PostLoad();

	if (DataInterface != nullptr)
	{
		DataInterface->ConditionalPostLoad();
		DataInterface->OnChanged().AddUObject(this, &UNiagaraNodeInput::DataInterfaceChanged);
	}
	if (ObjectAsset != nullptr)
	{
		ObjectAsset->ConditionalPostLoad();
	}

	ValidateDataInterface();
}

void UNiagaraNodeInput::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	if (!IsNodeEnabled() && OutHistory.GetIgnoreDisabled())
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	if (Input.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
	{
		int32 ParamMapIdx = OutHistory.FindMatchingParameterMapFromContextInputs(Input);

		if (ParamMapIdx == INDEX_NONE && Usage != ENiagaraInputNodeUsage::TranslatorConstant)
		{
			ParamMapIdx = OutHistory.CreateParameterMap();
		}
		else if (ParamMapIdx == INDEX_NONE && OutHistory.Histories.Num() != 0)
		{
			ParamMapIdx = 0;
		}

		if (ParamMapIdx != INDEX_NONE)
		{
			uint32 NodeIdx = OutHistory.BeginNodeVisitation(ParamMapIdx, this);
			OutHistory.EndNodeVisitation(ParamMapIdx, NodeIdx);

			OutHistory.RegisterParameterMapPin(ParamMapIdx, GetOutputPin(0));
		}
	}
	else if (Input.GetType().IsStatic())
	{
		int32 ConstantIdx = OutHistory.FindMatchingStaticFromContextInputs(Input);

		if (ConstantIdx != INDEX_NONE)
		{
			OutHistory.RegisterConstantPin(ConstantIdx, GetOutputPin(0));
		}
	}
}

void UNiagaraNodeInput::UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const
{
	if (Usage == ENiagaraInputNodeUsage::TranslatorConstant && Input == TRANSLATOR_PARAM_CALL_ID)
	{
		static const FName NAME_ScriptUsage = TEXT("ScriptUsage");

		// The call ID should be unique for each translated node as it is used by the seeded random functions.
		// We don't want it to be shared across the spawn and update script, so functions including it will have the usage added to their name.
		InOutFunctionAliasContext.AddPermutationIdentifier(NAME_ScriptUsage, FString::FormatAsNumber(static_cast<uint8>(InOutFunctionAliasContext.ScriptUsage)));
	}
}


void UNiagaraNodeInput::AllocateDefaultPins()
{
	const FNiagaraTypeDefinition& InputType = Input.GetType();

	if (InputType.IsDataInterface())
	{
		if (DataInterface == nullptr)
		{
			const EObjectFlags DataInterfaceObjectFlags = UNiagaraDataInterface::BuildObjectFlagsForOwner(this, RF_Transactional);
			DataInterface = NewObject<UNiagaraDataInterface>(this, InputType.GetClass(), NAME_None, DataInterfaceObjectFlags);
		}
	}

	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	//If we're a parameter node for a function or a module the we allow a "default" input pin.
	UNiagaraScript* OwnerScript = GetTypedOuter<UNiagaraScript>();
	if (OwnerScript)
	{
		if ((!IsRequired() && IsExposed()) && !IsDataInterface() && !IsObjectAsset() && Usage == ENiagaraInputNodeUsage::Parameter && (OwnerScript->IsStandaloneScript()))
		{
			UEdGraphPin* NewPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Input.GetType()), TEXT("Default"));
			NewPin->bDefaultValueIsIgnored = true;
		}
	}

	CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(Input.GetType()), TEXT("Input"));
}

FText UNiagaraNodeInput::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromName(Input.GetName());
}

int32 UNiagaraNodeInput::GenerateNewSortPriority(const UNiagaraGraph* Graph, FName& ProposedName, ENiagaraInputNodeUsage Usage)
{
	int32 HighestSortOrder = -1; // Set to -1 initially, so that if there are no nodes, the return value will be zero.
	
	if (Usage == ENiagaraInputNodeUsage::Parameter && Graph != nullptr)
	{
		TArray<UNiagaraNodeInput*> InputNodes;
		Graph->GetNodesOfClass(InputNodes);
		for (UNiagaraNodeInput* InputNode : InputNodes)
		{
			if (InputNode->Usage == Usage && InputNode->CallSortPriority > HighestSortOrder)
			{
				HighestSortOrder = InputNode->CallSortPriority;
			}
		}
	}
	return HighestSortOrder + 1;
}

FName UNiagaraNodeInput::GenerateUniqueName(const UNiagaraGraph* Graph, FName& ProposedName, ENiagaraInputNodeUsage Usage)
{
	check(Usage != ENiagaraInputNodeUsage::SystemConstant && Usage != ENiagaraInputNodeUsage::Undefined);
	TSet<FName> InputNames;
	if (Usage == ENiagaraInputNodeUsage::Parameter && Graph != nullptr)
	{
		TArray<UNiagaraNodeInput*> InputNodes;
		Graph->GetNodesOfClass(InputNodes);
		for (UNiagaraNodeInput* InputNode : InputNodes)
		{
			if (InputNode->Usage == Usage)
			{
				InputNames.Add(InputNode->Input.GetName());
			}
		}
	}
	else if (Usage == ENiagaraInputNodeUsage::Attribute && Graph != nullptr)
	{
		TArray<UNiagaraNodeOutput*> OutputNodes;
		Graph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);
		for (UNiagaraNodeOutput* Node : OutputNodes)
		{
			for (const FNiagaraVariable& Output : Node->Outputs)
			{
				InputNames.Add(Output.GetName());
			}
		}
	}

	return FNiagaraUtilities::GetUniqueName(ProposedName, FNiagaraEditorUtilities::GetSystemConstantNames().Union(InputNames));
}

void UNiagaraNodeInput::OnRenameNode(const FString& NewName)
{
	UNiagaraGraph* Graph = CastChecked<UNiagaraGraph>(GetGraph());
	TArray<UNiagaraNodeInput*> InputNodes;
	Graph->GetNodesOfClass<UNiagaraNodeInput>(InputNodes);

	TArray<UNiagaraNodeInput*> AffectedNodes;
	AffectedNodes.Add(this);
	for (UNiagaraNodeInput* Node : InputNodes)
	{
		if (Node == this || Node == nullptr)
		{
			continue;
		}

		bool bNeedsSync = Node->ReferencesSameInput(this);
		if (bNeedsSync)
		{
			AffectedNodes.Add(Node);
		}
	}

	for (UNiagaraNodeInput* Node : AffectedNodes)
	{
		Node->Modify();
		Node->Input.SetName(FName(*NewName));
		if ( DataInterface )
		{
			DataInterface->Rename(*NewName);
		}
		Node->ReallocatePins(false);
		Node->MarkNodeRequiresSynchronization("Input renamed", true);
	}
}

TSharedPtr<SGraphNode> UNiagaraNodeInput::CreateVisualWidget()
{
	return SNew(SNiagaraGraphNodeInput, this);
}

FLinearColor UNiagaraNodeInput::GetNodeTitleColor() const
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	switch (Usage)
	{
	case ENiagaraInputNodeUsage::Parameter:
		return Schema->NodeTitleColor_Constant;
	case ENiagaraInputNodeUsage::SystemConstant:
		return Schema->NodeTitleColor_SystemConstant;
	case ENiagaraInputNodeUsage::Attribute:
		return Schema->NodeTitleColor_Attribute;
	case ENiagaraInputNodeUsage::TranslatorConstant:
		return Schema->NodeTitleColor_TranslatorConstant;
	case ENiagaraInputNodeUsage::RapidIterationParameter:
		return Schema->NodeTitleColor_RapidIteration;
	default:
		// TODO: Do something better here.
		return FLinearColor::Black;
	}
}

bool UNiagaraNodeInput::ReferencesSameInput(UNiagaraNodeInput* Other) const
{
	if (this == Other)
	{
		return true;
	}

	if (Other == nullptr)
	{
		return false;
	}

	if (Usage == Other->Usage)
	{	
		if (Input.GetName() == Other->Input.GetName())
		{
			return true;
		}	
	}
	return false;
}


void UNiagaraNodeInput::AutowireNewNode(UEdGraphPin* FromPin)
{
	UEdGraphPin* OutputPin = GetOutputPin(0);
	if (FromPin != nullptr && OutputPin && OutputPin->PinType == FromPin->PinType)
	{
		const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
		check(Schema);

		if (Usage == ENiagaraInputNodeUsage::Parameter)
		{
			TArray<UNiagaraNodeInput*> InputNodes;
			GetGraph()->GetNodesOfClass(InputNodes);
			int32 NumMatches = 0;
			int32 HighestSortPriority = -1; // Set to -1 initially, so that in the event of no nodes, we still get zero.
			for (UNiagaraNodeInput* InputNode : InputNodes)
			{
				if (InputNode == nullptr)
				{
					continue;
				}

				if (InputNode != this && InputNode->Usage == ENiagaraInputNodeUsage::Parameter)
				{
					if (ReferencesSameInput(InputNode))
					{
						NumMatches++;

						check(InputNode->Input.GetName() == this->Input.GetName());
						check(InputNode->ExposureOptions.bCanAutoBind == this->ExposureOptions.bCanAutoBind);
						check(InputNode->ExposureOptions.bExposed == this->ExposureOptions.bExposed);
						check(InputNode->ExposureOptions.bHidden == this->ExposureOptions.bHidden);
						check(InputNode->ExposureOptions.bRequired == this->ExposureOptions.bRequired);
						check(InputNode->DataInterface == this->DataInterface);
						check(InputNode->ObjectAsset == this->ObjectAsset);
					}

					if (InputNode->CallSortPriority > HighestSortPriority)
					{
						HighestSortPriority = InputNode->CallSortPriority;
					}
				}
			}

			FName CandidateName = Input.GetName();
			FNiagaraTypeDefinition Type = Input.GetType();
			if (Type == FNiagaraTypeDefinition::GetGenericNumericDef())
			{
				//Try to get a real type if we've been set to numeric
				Type = Schema->PinToTypeDefinition(FromPin);
			}

			CallSortPriority = HighestSortPriority + 1;
			ReallocatePins();
		}

		FPinCollectorArray OutPins;
		GetOutputPins(OutPins);
		check(OutPins.Num() == 1 && OutPins[0] != NULL);

		if (GetSchema()->TryCreateConnection(FromPin, OutPins[0]))
		{
			FromPin->GetOwningNode()->NodeConnectionListChanged();
		}
	}
}

void UNiagaraNodeInput::NotifyInputTypeChanged()
{
	ReallocatePins();
}


void UNiagaraNodeInput::NotifyExposureOptionsChanged()
{
	ReallocatePins();
}

void UNiagaraNodeInput::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	if (!IsNodeEnabled())
	{
		Outputs.Add(INDEX_NONE);
		return;
	}

	if (Input.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef())
	{
		Outputs.Add(INDEX_NONE);
		Translator->Error(LOCTEXT("InvalidPinType", "Numeric types should be able to be inferred from use by this phase of compilation."), this, nullptr);
		return;
	}

	int32 FunctionParam = INDEX_NONE;
	if (IsExposed() && Translator->GetFunctionParameter(Input, FunctionParam))
	{
		//If we're in a function and this parameter hasn't been provided, compile the local default.
		if (FunctionParam == INDEX_NONE)
		{
			FPinCollectorArray InputPins;
			GetInputPins(InputPins);
			int32 Default = InputPins.Num() > 0 ? Translator->CompileInputPin(InputPins[0]) : INDEX_NONE;
			if (Default == INDEX_NONE)
			{
				//We failed to compile the default pin so just use the value of the input.
				if (Usage == ENiagaraInputNodeUsage::Parameter)
				{
					if ( IsDataInterface() )
					{
						if (ensure(DataInterface != nullptr))
						{
							Outputs.Add(Translator->RegisterDataInterface(Input, DataInterface, false, false));
							return;
						}
					}
					else if ( IsObjectAsset() )
					{
						Outputs.Add(Translator->RegisterUObject(Input, ObjectAsset, false));
						return;
					}
				}
				Default = Translator->GetConstant(Input);
			}
			Outputs.Add(Default);
			return;
		}
	}

	switch (Usage)
	{
	case ENiagaraInputNodeUsage::Parameter:
		if (IsDataInterface())
		{
			Outputs.Add(Translator->RegisterDataInterface(Input, DataInterface, false, false));
			break;
		}
		else if ( IsObjectAsset() )
		{
			Outputs.Add(Translator->RegisterUObject(Input, ObjectAsset, false));
			break;
		}
		else
		{
			Outputs.Add(Translator->GetParameter(Input));
			break;
		}
	case ENiagaraInputNodeUsage::SystemConstant:
		Outputs.Add(Translator->GetParameter(Input)); break;
	case ENiagaraInputNodeUsage::Attribute:
		Outputs.Add(Translator->GetAttribute(Input)); break;
	case ENiagaraInputNodeUsage::TranslatorConstant:
		Outputs.Add(Translator->GetParameter(Input)); break;
	case ENiagaraInputNodeUsage::RapidIterationParameter:
		Outputs.Add(Translator->GetRapidIterationParameter(Input)); break;
	default:
		check(false);
	}
}

void UNiagaraNodeInput::SortNodes(TArray<UNiagaraNodeInput*>& InOutNodes)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_UNiagaraNodeInput_SortNodes);

	auto SortVars = [](UNiagaraNodeInput& A, UNiagaraNodeInput& B)
	{
		if (A.CallSortPriority < B.CallSortPriority)
		{
			return true;
		}
		else if (A.CallSortPriority > B.CallSortPriority)
		{
			return false;
		}

		//If equal priority, sort lexicographically.
		return A.Input.GetName().ToString() < B.Input.GetName().ToString();
	};
	InOutNodes.Sort(SortVars);
}

bool UNiagaraNodeInput::IsDataInterface() const
{
	return Input.IsDataInterface();
}

bool UNiagaraNodeInput::IsObjectAsset() const
{
	return Input.IsUObject();
}

UNiagaraDataInterface* UNiagaraNodeInput::GetDataInterface() const
{
	return Cast<UNiagaraDataInterface>(DataInterface);
}

void UNiagaraNodeInput::SetDataInterface(UNiagaraDataInterface* InDataInterface)
{
	UNiagaraDataInterface* OldDataInterface = GetDataInterface();
	if (OldDataInterface != nullptr)
	{
		OldDataInterface->OnChanged().RemoveAll(this);
	}
	DataInterface = InDataInterface;
	if (InDataInterface != nullptr)
	{
		InDataInterface->OnChanged().AddUObject(this, &UNiagaraNodeInput::DataInterfaceChanged);
	}

	ValidateDataInterface();
	DataInterfaceChanged();
}

TObjectPtr<UObject> UNiagaraNodeInput::GetObjectAsset() const
{
	return ObjectAsset;
}

void UNiagaraNodeInput::SetObjectAsset(TObjectPtr<UObject> InObject)
{
	ObjectAsset = InObject;
}

void UNiagaraNodeInput::DataInterfaceChanged()
{
	// Don't use GetNiagaraGraph() here since this may be called on a temporary node which isn't
	// in a proper graph yet.
	UNiagaraGraph* NiagaraGraph = Cast<UNiagaraGraph>(GetGraph());
	if (NiagaraGraph != nullptr)
	{
		NiagaraGraph->NotifyGraphDataInterfaceChanged();
	}
}

void UNiagaraNodeInput::ValidateDataInterface()
{
	if (DataInterface != nullptr &&
		DataInterface->GetClass()->GetMetaData("DevelopmentStatus") == TEXT("Experimental"))
	{
		UEdGraphNode::bHasCompilerMessage = true;
		UEdGraphNode::ErrorType = EMessageSeverity::Info;
		UEdGraphNode::NodeUpgradeMessage = LOCTEXT("DataInterfaceExperimental", "This data interface is marked as experimental, use with care!");
	}
}

#undef LOCTEXT_NAMESPACE

