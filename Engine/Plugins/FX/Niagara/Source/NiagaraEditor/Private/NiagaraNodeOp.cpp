// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeOp.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraHlslTranslator.h"
#include "GraphEditorSettings.h"

#include "EdGraphSchema_Niagara.h"
#include "NiagaraCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeOp)

#define LOCTEXT_NAMESPACE "NiagaraNodeOp"

UNiagaraNodeOp::UNiagaraNodeOp(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), bAllStatic(false)
{
}

void UNiagaraNodeOp::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (!OpInfo)
	{
		return;
	}

	// Create input pins from the op
	for (int32 SrcIndex = 0; SrcIndex < OpInfo->Inputs.Num(); ++SrcIndex)
	{
		const FNiagaraOpInOutInfo& InOutInfo = OpInfo->Inputs[SrcIndex];
		UEdGraphPin* Pin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(InOutInfo.DataType), *InOutInfo.Name.ToString());
		check(Pin);

		if(InOutInfo.FriendlyName.IsEmpty() == false)
		{
			Pin->PinFriendlyName = InOutInfo.FriendlyName;
		}
		
		Pin->bDefaultValueIsIgnored = false;
		Pin->bDefaultValueIsReadOnly = false;
		Pin->bNotConnectable = false;
		Pin->DefaultValue = InOutInfo.Default;
		Pin->AutogeneratedDefaultValue = InOutInfo.Default;
		Pin->PinToolTip = InOutInfo.Description.ToString();
	}

	// Restore pins added by the user
	for (int32 SrcIndex = 0; SrcIndex < AddedPins.Num(); ++SrcIndex)
	{
		FAddedPinData& OldPinData = AddedPins[SrcIndex];
		UEdGraphPin* Pin = CreatePin(EGPD_Input, OldPinData.PinType, OldPinData.PinName);
		check(Pin);
		Pin->bDefaultValueIsIgnored = false;
		Pin->bDefaultValueIsReadOnly = false;
		Pin->bNotConnectable = false;
	}
	
	// Create output pins from the op
	for (int32 OutIdx = 0; OutIdx < OpInfo->Outputs.Num(); ++OutIdx)
	{
		const FNiagaraOpInOutInfo& InOutInfo = OpInfo->Outputs[OutIdx];
		UEdGraphPin* Pin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(InOutInfo.DataType), *InOutInfo.Name.ToString());
		check(Pin);
		Pin->PinToolTip = InOutInfo.Description.ToString();
	}

	if (OpInfo->bSupportsAddedInputs)
	{
		CreateAddPin(EGPD_Input);
	}

}

FText UNiagaraNodeOp::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (!OpInfo)
	{
		FString Strn = "Unknown";
		return FText::FromString(Strn);
	}

	if (OpInfo && OpInfo->StaticVariableResolveFunction.IsBound() && OpInfo->bSupportsStaticResolution && bAllStatic)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("OpName"), OpInfo->FriendlyName);
		FText Format = LOCTEXT("OpNodeNameStatic", "{OpName} (Static)");
		return FText::Format(Format, Args);
	}
	return OpInfo->FriendlyName;
}

FText UNiagaraNodeOp::GetTooltipText()const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (!OpInfo)
	{
		FString Strn = "Unknown";
		return FText::FromString(Strn);
	}

	if (OpInfo && OpInfo->StaticVariableResolveFunction.IsBound() && OpInfo->bSupportsStaticResolution && bAllStatic)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("OpDesc"), OpInfo->Description);
		FText Format = LOCTEXT("OpNodeDescStatic", "{OpDesc}\r\n(To convert into a static version, connect all input pins to other static pins.)");
		return FText::Format(Format, Args);
	}
	return OpInfo->Description;
}

FLinearColor UNiagaraNodeOp::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->FunctionCallNodeTitleColor;
}

void UNiagaraNodeOp::PinTypeChanged(UEdGraphPin* Pin)
{
	FName PinName = Pin->PinName;

	for(FAddedPinData& AddedPinData : AddedPins)
	{
		if(AddedPinData.PinName == PinName)
		{
			AddedPinData.PinType = Pin->PinType;
			break;
		}
	}
	
	Super::PinTypeChanged(Pin);
}

void UNiagaraNodeOp::OnPostSynchronizationInReallocatePins() 
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	bool bOutputPinsNeedUpdate = false;
	for (int32 i = 0; i < InputPins.Num(); ++i)
	{
		UEdGraphPin* Pin = InputPins[i];

		FEdGraphPinType StartType = Pin->PinType;
		HandleStaticInputPinUpgrade(Pin);

		// Type was changed... now make sure that we can fix up the output pin.
		if (StartType != Pin->PinType)
		{
			bOutputPinsNeedUpdate = true;
		}
	}

	HandleStaticOutputPinUpgrade();		
}

FNiagaraTypeDefinition UNiagaraNodeOp::ResolveCustomNumericType(const TArray<FNiagaraTypeDefinition>& NonNumericInputs) const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (OpInfo && OpInfo->CustomNumericResolveFunction.IsBound())
	{
		return OpInfo->CustomNumericResolveFunction.Execute(NonNumericInputs);
	}
	
	return FNiagaraTypeDefinition::GetGenericNumericDef();
}


void UNiagaraNodeOp::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (!OpInfo)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("OpName"), FText::FromName(OpName));
		FText Format = LOCTEXT("Unknown opcode", "Unknown opcode on {OpName} node.");
		Translator->Error(FText::Format(Format, Args), this, nullptr);
		return;
	}

	int32 NumInputs = OpInfo->bSupportsAddedInputs ? Pins.Num() : OpInfo->Inputs.Num();
	int32 NumOutputs = OpInfo->Outputs.Num();

	TArray<int32> Inputs;
	bool bError = false;
	for (int32 i = 0; i < NumInputs; ++i)
	{
		UEdGraphPin *Pin = Pins[i];
		if (Pin->Direction != EGPD_Input || IsAddPin(Pin))
		{
			continue;
		}
		int32 CompiledInput = Translator->CompileInputPin(Pin);
		if (CompiledInput == INDEX_NONE)
		{
			bError = true;
			FFormatNamedArguments Args;
			Args.Add(TEXT("OpName"), GetNodeTitle(ENodeTitleType::FullTitle));
			FText Format = LOCTEXT("InputErrorFormat", "Error compiling input on {OpName} node.");
			Translator->Error(FText::Format(Format, Args), this, Pin);
		}
		else if (i < OpInfo->Inputs.Num() && OpInfo->Inputs[i].DataType == FNiagaraTypeDefinition::GetGenericNumericDef()) 
		{
			// Some nodes disallow integer or floating numeric input pins, so we guard against them here. 
			// This will catch both implicitly and explicitly set pin types.
			// Currently this is for the Random Float/Integer and Seeded Random Float/Integer ops, but might be useful for others in the future. 

			const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
			FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(Pin);
			if (TypeDef.IsFloatPrimitive() && !OpInfo->bNumericsCanBeFloats)
			{
				bError = true;
				FFormatNamedArguments Args;
				Args.Add(TEXT("OpName"), GetNodeTitle(ENodeTitleType::FullTitle));
				FText Format = LOCTEXT("InputTypeErrorFormatFloat", "The {OpName} node cannot have float based numeric input pins.");
				Translator->Error(FText::Format(Format, Args), this, Pin);
			}
			else if (!TypeDef.IsFloatPrimitive() && !OpInfo->bNumericsCanBeIntegers)
			{
				bError = true;
				FFormatNamedArguments Args;
				Args.Add(TEXT("OpName"), GetNodeTitle(ENodeTitleType::FullTitle));
				FText Format = LOCTEXT("InputTypeErrorFormatInt", "The {OpName} node cannot have integer based numeric input pins.");
				Translator->Error(FText::Format(Format, Args), this, Pin);
			}
		}
		Inputs.Add(CompiledInput);
	}
	
	Translator->Operation(this, Inputs, Outputs);
}

void UNiagaraNodeOp::PostLoad()
{
	Super::PostLoad();

	FName OriginalOpName = OpName;

	if (OpName == TEXT("Numeric::Cos"))
	{
		OpName = TEXT("Numeric::Cosine(Radians)");
	}
	else if (OpName == TEXT("Numeric::Sin"))
	{
		OpName = TEXT("Numeric::Sine(Radians)");
	}
	else if (OpName == TEXT("Numeric::Tan"))
	{
		OpName = TEXT("Numeric::Tangent(Radians)");
	}
	else if (OpName == TEXT("Numeric::ASin"))
	{
		OpName = TEXT("Numeric::ArcSine(Radians)");
	}
	else if (OpName == TEXT("Numeric::ACos"))
	{
		OpName = TEXT("Numeric::ArcCosine(Radians)");
	}
	else if (OpName == TEXT("Numeric::ATan"))
	{
		OpName = TEXT("Numeric::ArcTangent(Radians)");
	}
	else if (OpName == TEXT("Numeric::ATan2"))
	{
		OpName = TEXT("Numeric::ArcTangent2(Radians)");
	}

	if (OpName != OriginalOpName)
	{
		UE_LOGF(LogNiagaraEditor, Log, "OpNode: Converted %ls to %ls, Package: %ls", *OriginalOpName.ToString(), *OpName.ToString(), *GetOutermost()->GetName());
	}

	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraVer < FNiagaraCustomVersion::ImproveLoadTimeFixupOfOpAddPins &&
		AllowDynamicPins() &&
		Pins.FindByPredicate([this](UEdGraphPin* Pin) { return IsAddPin(Pin); }) == nullptr)
	{
		// Add the pin directly here rather than calling allocate default pins to prevent the graph id from being invalidated
		// since adding the add pin doesn't change the compile behavior.  We do modify here so that when running the resave 
		// commandlet the package will be marked dirty, it will be ignored during regular post load.
		Modify();
		CreateAddPin(EGPD_Input);
	}
}


bool UNiagaraNodeOp::RefreshFromExternalChanges()
{
	// TODO - Leverage code in reallocate pins to determine if any pins have changed...
	ReallocatePins();
	return true;
}

ENiagaraNumericOutputTypeSelectionMode UNiagaraNodeOp::GetNumericOutputTypeSelectionMode() const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (OpInfo)
	{
		return OpInfo->NumericOuputTypeSelectionMode;
	}
	else
	{
		return ENiagaraNumericOutputTypeSelectionMode::Largest;
	}
}

bool UNiagaraNodeOp::GenerateCompileHashForClassMembers(const UClass* InClass, FNiagaraCompileHashVisitor* InVisitor) const
{
	if (InClass == UNiagaraNodeOp::StaticClass())
	{
		const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);

		if(OpInfo)
		{
			InVisitor->UpdateReference(TEXT("OpInfo"), OpInfo);
		}
		
		return true;
	}
	else
	{
		return Super::GenerateCompileHashForClassMembers(InClass, InVisitor);
	}
}

bool UNiagaraNodeOp::AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (!OpInfo)
	{
		return false;
	}
	if (OpInfo->AddedInputTypeRestrictions.Num() == 0)
	{
		return true;
	}

	auto FindPredicate = [this, InType](const FNiagaraTypeDefinition& PinType)
	{
		if (bAllStatic)
			return PinType.ToStaticDef() == InType;
		else 
			return PinType == InType;
	};
	return OpInfo->AddedInputTypeRestrictions.IndexOfByPredicate(FindPredicate) != INDEX_NONE;
}

void UNiagaraNodeOp::OnPinRemoved(UEdGraphPin* PinToRemove)
{
	auto FindPredicate = [=](const FAddedPinData& PinData)
	{
		return PinData.PinName == PinToRemove->PinName && PinData.PinType == PinToRemove->PinType; 
	};
	AddedPins.RemoveAll(FindPredicate);
	ReallocatePins();
}

bool UNiagaraNodeOp::AllowDynamicPins() const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	return OpInfo && OpInfo->bSupportsAddedInputs;
}

void UNiagaraNodeOp::OnNewTypedPinAdded(UEdGraphPin*& NewPin)
{
	FName UniqueName = GetUniqueAdditionalPinName();

	FAddedPinData PinData;
	PinData.PinType = NewPin->PinType;
	PinData.PinName = UniqueName;
	NewPin->PinName = UniqueName;
	AddedPins.Add(PinData);

	if (bAllStatic)
	{
		const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
		FNiagaraTypeDefinition PinType = Schema->PinToTypeDefinition(NewPin);
		if (PinType.IsStatic() == false)
			NewPin->PinType = Schema->TypeDefinitionToPinType(PinType.ToStaticDef());
	}
}

void UNiagaraNodeOp::OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName)
{
	TSet<FName> ExistingNames;
	for (const FAddedPinData& PinData : AddedPins)
	{
		ExistingNames.Add(PinData.PinName);
	}
	FName UniqueName = FNiagaraUtilities::GetUniqueName(*RenamedPin->PinName.ToString(), ExistingNames);
	FName OldPinName(*OldName);

	for (FAddedPinData& PinData : AddedPins)
	{
		if (PinData.PinName == OldPinName && PinData.PinType == RenamedPin->PinType)
		{
			PinData.PinName = UniqueName;
			RenamedPin->PinName = UniqueName;
			break;
		}
	}
}

bool UNiagaraNodeOp::CanRemovePin(const UEdGraphPin* Pin) const
{
	if (!Pin || Pin->Direction != EGPD_Input || !Super::CanRemovePin(Pin))
	{
		return false;
	}

	// check if the pin was added by the user, only those can be removed
	for (int32 SrcIndex = 0; SrcIndex < AddedPins.Num(); ++SrcIndex)
	{
		const FAddedPinData& PinData = AddedPins[SrcIndex];
		if (Pin->PinType == PinData.PinType && Pin->PinName == PinData.PinName)
		{
			return true;
		}
	}
	return false;
}

FName UNiagaraNodeOp::GetUniqueAdditionalPinName() const
{
	// count existing input pins
	FString Name;
	TSet<FName> ExistingNames;
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EEdGraphPinDirection::EGPD_Input && !IsAddPin(Pin))
		{
			ExistingNames.Add(Pin->PinName);
		}
	}
	
	// create a new name based on the existing pins (A, B, ... AA, AB, ...)
	static FString Alphabet = TEXT("ABCDEFGHIJKLMNOPQRSTUVWXYZ");
	for (int32 Remaining = ExistingNames.Num(); Remaining > 0; Remaining = Remaining / 26)
	{
		int32 CharIndex = Remaining < 26 ? Remaining - 1 : Remaining % 26;
		Name = FString() + (*Alphabet)[CharIndex] + Name;
	}

	return FNiagaraUtilities::GetUniqueName(*Name, ExistingNames);
}

void UNiagaraNodeOp::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	if (bRecursive)
	{
		OutHistory.VisitInputPins(this, bFilterForCompilation);
	}

	if (!IsNodeEnabled() && OutHistory.GetIgnoreDisabled())
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}
	else 
	{		
		const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
		if (OpInfo && OpInfo->StaticVariableResolveFunction.IsBound())
		{
			const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
			TArray<UEdGraphPin*> InputPins;
			GetInputPins(InputPins);
			
			bool bAllPinsStatic = true;
			UEdGraphPin* OutputPin = nullptr;
			{
				for (int32 PinIdx = 0; PinIdx < Pins.Num(); PinIdx++)
				{
					if (IsAddPin(Pins[PinIdx]))
						continue;
					FNiagaraTypeDefinition InputType = Schema->PinToTypeDefinition(Pins[PinIdx]);
					if (!InputType.IsStatic())
						bAllPinsStatic = false;

					if (OutputPin == nullptr && Pins[PinIdx]->Direction == EEdGraphPinDirection::EGPD_Output)
					{
						OutputPin = Pins[PinIdx];
					}
				}
			}

			if (bAllPinsStatic)
			{
				TArray<int32> Vars;

				for (int32 InputIdx = 0; InputIdx < InputPins.Num(); InputIdx++)
				{
					if (IsAddPin(InputPins[InputIdx]))
						continue;

					FNiagaraTypeDefinition InputType = Schema->PinToTypeDefinition(InputPins[InputIdx]);
					if (!InputType.IsStatic())
					{
						return;
					}
					int32 Value = 0;
					OutHistory.SetConstantByStaticVariable(Value, InputPins[InputIdx]);
					Vars.Add(Value);
				}

				if (Vars.Num() > 0)
				{
					int32 Result = OpInfo->StaticVariableResolveFunction.Execute(Vars);
					int32 ConstantIdx = OutHistory.AddOrGetConstantFromValue(FString::FromInt(Result));

					if (UNiagaraScript::LogCompileStaticVars > 0)
					{
						UE_LOGF(LogNiagaraEditor, Log, "Inputs Static Node Op: %ls", *GetNodeTitle(ENodeTitleType::FullTitle).ToString());
						for (int32 i = 0; i < Vars.Num(); i++)
						{
							UE_LOGF(LogNiagaraEditor, Log, "[%d] %d", i, Vars[i]);
						}
						UE_LOGF(LogNiagaraEditor, Log, "Result: %d", Result);
					}

					OutHistory.RegisterConstantPin(ConstantIdx, OutputPin);
				}
			}
		}
	}
}

void UNiagaraNodeOp::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);

	FEdGraphPinType StartType = Pin->PinType;
	HandleStaticInputPinUpgrade(Pin);

	// Type was changed... now make sure that we can fix up the output pin.
	if (StartType != Pin->PinType)
	{
		HandleStaticOutputPinUpgrade();
	}
}

FText UNiagaraNodeOp::GetCompactTitle() const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if(OpInfo == nullptr)
	{
		// this will effectively turn off compact mode
		return FText::GetEmpty();
	}
	
	return OpInfo->CompactName; 
}

bool UNiagaraNodeOp::ShouldShowPinNamesInCompactMode()
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if(OpInfo == nullptr)
	{
		return false;
	}
	
	return OpInfo->bShowPinNamesInCompactMode; 
}

TOptional<float> UNiagaraNodeOp::GetCompactModeFontSizeOverride() const
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if(OpInfo == nullptr)
	{
		return {};
	}
	
	return OpInfo->CompactNameFontSizeOverride;
}

void UNiagaraNodeOp::HandleStaticInputPinUpgrade(UEdGraphPin* Pin)
{
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (Pin && OpInfo && OpInfo->StaticVariableResolveFunction.IsBound() && OpInfo->bSupportsStaticResolution)
	{
		const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
		FNiagaraVariable PinVar = Schema->PinToNiagaraVariable(Pin);

		if (Pin->Direction == EEdGraphPinDirection::EGPD_Input && !IsAddPin(Pin))
		{
			// Handle linkage...
			bool bHandled = false;
			// this shouldn't specifically check for add pins as there can be other special pins like the wildcard pin
			// this will generally attempt to set the current input pin to the same type as the other's output pin. Ok for now, but might break in the future
			if (Pin->LinkedTo.Num() > 0 && !IsAddPin(Pin->LinkedTo[0]))
			{
				FNiagaraTypeDefinition LinkedPinType = Schema->PinToTypeDefinition(Pin->LinkedTo[0]);
				if(LinkedPinType.IsStatic())
				{
					Pin->PinType = Schema->TypeDefinitionToPinType(LinkedPinType);
				}
			}
			else // Handle unlinkage...
			{
				// Downcast if the types don't match...
				if (OpInfo->bSupportsAddedInputs && OpInfo->Inputs.Num() > 0)
				{
					if (PinVar.GetType() != OpInfo->Inputs[0].DataType && OpInfo->Inputs[0].DataType != FNiagaraTypeDefinition::GetGenericNumericDef())
					{
						Pin->PinType = Schema->TypeDefinitionToPinType(OpInfo->Inputs[0].DataType);
					}
				}
				else
				{
					for (int32 i = 0; i < OpInfo->Inputs.Num(); i++)
					{
						if (OpInfo->Inputs[i].Name == PinVar.GetName())
						{
							if (PinVar.GetType() != OpInfo->Inputs[i].DataType && OpInfo->Inputs[0].DataType != FNiagaraTypeDefinition::GetGenericNumericDef())
							{
								Pin->PinType = Schema->TypeDefinitionToPinType(OpInfo->Inputs[i].DataType);
								break;
							}
							break;
						}
					}
				}
			}
		}
	}
}

void UNiagaraNodeOp::HandleStaticOutputPinUpgrade()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpName);
	if (OpInfo)
	{
		bAllStatic = false;

		bool bAllInputStatic = true;
		bool bAnyInputStatic = false;
		for (int32 i = 0; i < Pins.Num(); i++)
		{
			if (IsAddPin(Pins[i]) || Pins[i]->Direction != EEdGraphPinDirection::EGPD_Input)
				continue;

			FNiagaraTypeDefinition PinType = Schema->PinToTypeDefinition(Pins[i]);
			if (PinType.IsStatic())
			{
				bAnyInputStatic = true;
			}
			else
			{
				bAllInputStatic = false;
				continue;
			}
		}

		if (OpInfo->StaticVariableResolveFunction.IsBound() && OpInfo->bSupportsStaticResolution)
		{
			// Synchronize the output pin either way...
			bool bAllOutputStatic = true;
			bool bAnyOutputStatic = false;
			for (int32 PinIdx = 0; PinIdx < Pins.Num(); PinIdx++)
			{
				if (Pins[PinIdx]->Direction == EEdGraphPinDirection::EGPD_Output)
				{
					FNiagaraVariable PinVar = Schema->PinToNiagaraVariable(Pins[PinIdx]);

					if (IsAddPin(Pins[PinIdx]))
						continue;

					if (!PinVar.GetType().IsStatic())
					{
						bAllOutputStatic = false;
					}
					else
					{
						bAnyOutputStatic = true;
					}
				}
			}

			bool bCanAutoConvertOutputs = bAllInputStatic;
			for (int32 OutputIdx = 0; OutputIdx < OpInfo->Outputs.Num(); OutputIdx++)
			{
				if (!FNiagaraTypeRegistry::IsStaticPossible(OpInfo->Outputs[OutputIdx].DataType))
				{
					bCanAutoConvertOutputs = false;
				}
			}

			if (bCanAutoConvertOutputs)
			{
				for (int32 PinIdx = 0; PinIdx < Pins.Num(); PinIdx++)
				{
					if (IsAddPin(Pins[PinIdx]))
						continue;

					if (Pins[PinIdx]->Direction == EEdGraphPinDirection::EGPD_Output)
					{
						FNiagaraTypeDefinition PinStaticType = Schema->PinToTypeDefinition(Pins[PinIdx]).ToStaticDef();
						Pins[PinIdx]->PinType = Schema->TypeDefinitionToPinType(PinStaticType);
					}
				}
				bAllOutputStatic = true;
			}

			if (bAllOutputStatic && bAllInputStatic)
			{
				bAllStatic = true;
				NodeUpgradeMessage = FText();
			}
			else if (bAnyOutputStatic && !bAllInputStatic)
			{
				NodeUpgradeMessage = LOCTEXT("AllStaticInvalidInput", "All inputs must be static to have static outputs. Static will not be preserved.");
				MarkNodeRequiresSynchronization(__FUNCTION__, true);
			}
			else if (bAnyInputStatic && !bAllOutputStatic)
			{
				NodeUpgradeMessage = LOCTEXT("AllStaticInvalidOutput", "All outputs must be static to have static inputs. Static will not be preserved.");
				MarkNodeRequiresSynchronization(__FUNCTION__, true);
			}
			
		}
		else if (bAllStatic)
		{
			NodeUpgradeMessage = LOCTEXT("AllStaticNotSupported", "This op doesn't fully support static outputs. Static will not be preserved.");
			MarkNodeRequiresSynchronization(__FUNCTION__, true);
		}
	}
}
#undef LOCTEXT_NAMESPACE

