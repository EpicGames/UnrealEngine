// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeOutput.h"
#include "UObject/UnrealType.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraEmitter.h"
#include "NiagaraScript.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptSource.h"
#include "NiagaraCustomVersion.h"
#include "ScopedTransaction.h"
#include "EdGraphSchema_Niagara.h"
#include "ToolMenus.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraSimulationStageBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeOutput)

#define LOCTEXT_NAMESPACE "NiagaraNodeOutput"

UNiagaraNodeOutput::UNiagaraNodeOutput(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer), ScriptTypeIndex_DEPRECATED(0)
{
}

void UNiagaraNodeOutput::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
// 	for (FNiagaraVariable& Output : Outputs)
// 	{
// 		//Ensure all pins are valid.
// 		if (Output.GetStruct() == nullptr)
// 		{
// 			Output = FNiagaraVariable();
// 		}
// 	}

	if (PropertyChangedEvent.Property != nullptr)
	{
		ReallocatePins();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UNiagaraNodeOutput::RemoveOutputPin(UEdGraphPin* Pin)
{
	FScopedTransaction RemovePinTransaction(LOCTEXT("RemovePinTransaction", "Remove pin"));
	int32 Index = Outputs.IndexOfByPredicate([&](const FNiagaraVariable& InVar) { return Pin->PinName == InVar.GetName(); });
	if (Index >= 0)
	{
		Modify();
		Outputs.RemoveAt(Index);
		// Remove the pin directly here instead of using the built in reallocate to prevent the old pin from being kept as orphaned.
		RemovePin(Pin);
		MarkNodeRequiresSynchronization(__FUNCTION__, true);
	}
}

FText UNiagaraNodeOutput::GetPinNameText(UEdGraphPin* Pin) const
{
	return FText::FromName(Pin->PinName);
}

TArray<FName> UNiagaraNodeOutput::GetAllStackContextOverrides() const
{
	FVersionedNiagaraEmitter Outer = GetNiagaraGraph()->GetOwningEmitter();
	TArray<FName> Overrides;
	if (Outer.Emitter)
	{
		TArray<UNiagaraScript*> Scripts;
		FVersionedNiagaraEmitterData* EmitterData = Outer.GetEmitterData();
		EmitterData->GetScripts(Scripts, false);
		for (UNiagaraScript* Script : Scripts)
		{
			if (Script && Script->GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript)
			{
				UNiagaraSimulationStageBase* Base = EmitterData->GetSimulationStageById(Script->GetUsageId());
				if (Base)
				{
					FName StackContextAlias = Base->GetStackContextReplacementName();
					if (StackContextAlias != NAME_None)
						Overrides.AddUnique(StackContextAlias);
				}
			}
		}
	}
	return Overrides;
}

TOptional<FName> UNiagaraNodeOutput::GetStackContextOverride() const
{
	
	{
		FVersionedNiagaraEmitter Outer = GetNiagaraGraph()->GetOwningEmitter();
		if (Outer.Emitter && GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			UNiagaraSimulationStageBase* Base = Outer.GetEmitterData()->GetSimulationStageById(GetUsageId());
			if (Base)
			{
				FName StackContextAlias = Base->GetStackContextReplacementName();
				if (StackContextAlias != NAME_None)
					return StackContextAlias;
			}
		}
	}

	return TOptional<FName>();
}


bool UNiagaraNodeOutput::VerifyPinNameTextChanged(const FText& InText, FText& OutErrorMessage, UEdGraphPin* Pin) const
{
	return FNiagaraEditorUtilities::VerifyNameChangeForInputOrOutputNode(*this, Pin->PinName, InText.ToString(), OutErrorMessage);
}

void UNiagaraNodeOutput::PinNameTextCommitted(const FText& Text, ETextCommit::Type CommitType, UEdGraphPin* Pin)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		FScopedTransaction RenamePinTransaction(LOCTEXT("RenamePinTransaction", "Rename pin"));

		Modify();
		FNiagaraVariable* Var = Outputs.FindByPredicate([&](const FNiagaraVariable& InVar) {return Pin->PinName == InVar.GetName(); });
		check(Var != nullptr);
		Pin->PinName = *Text.ToString();
		Var->SetName(Pin->PinName);		
		MarkNodeRequiresSynchronization(__FUNCTION__, true);
	}
}

void UNiagaraNodeOutput::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (Context->Pin != nullptr)
	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchema_NiagaraPinActions", LOCTEXT("EditPinMenuHeader", "Edit Pin"));
		
		{
			UEdGraphPin* Pin = const_cast<UEdGraphPin*>(Context->Pin);
			TSharedRef<SWidget> RenameWidget =
				SNew(SBox)
				.WidthOverride(100)
				.Padding(FMargin(5, 0, 0, 0))
				[
					SNew(SEditableTextBox)
					.Text_UObject(this, &UNiagaraNodeOutput::GetPinNameText, Pin)
					.OnVerifyTextChanged_UObject(this, &UNiagaraNodeOutput::VerifyPinNameTextChanged, Pin)
					.OnTextCommitted_UObject(const_cast<UNiagaraNodeOutput*>(this), &UNiagaraNodeOutput::PinNameTextCommitted, Pin)
				];
			Section.AddEntry(FToolMenuEntry::InitWidget("RenameWidget", RenameWidget, LOCTEXT("NameMenuItem", "Name")));
		}

		{
			Section.AddMenuEntry(
				"RemoveDynamicPin",
				LOCTEXT("RemoveDynamicPin", "Remove pin"),
				LOCTEXT("RemoveDynamicPinToolTip", "Remove this pin and any connections."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
				FUIAction(FExecuteAction::CreateUObject(const_cast<UNiagaraNodeOutput*>(this), &UNiagaraNodeOutput::RemoveOutputPin, const_cast<UEdGraphPin*>(Context->Pin))));
		}
	}
}

void UNiagaraNodeOutput::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

// 	if (Script->Usage == ENiagaraScriptUsage::ParticleSpawnScript || Script->Usage == ENiagaraScriptUsage::ParticleUpdateScript)
// 	{
// 		//For the outermost spawn and update scripts output nodes we need an extra pin telling whether the instance is still alive or not.
// 		CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetIntDef()), TEXT("Alive"));
// 	}

	for (const FNiagaraVariable& Output : Outputs)
	{
		UEdGraphPin* Pin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Output.GetType()), Output.GetName());
		if (ScriptType == ENiagaraScriptUsage::ParticleUpdateScript)
		{
			Pin->bDefaultValueIsIgnored = true;
		}
	}
}

bool UNiagaraNodeOutput::CanUserDeleteNode() const 
{
	if (ScriptType == ENiagaraScriptUsage::ParticleEventScript)
	{
		return true;
	}
	return false; 
}

bool UNiagaraNodeOutput::CanDuplicateNode() const
{
	return false;
}

FText UNiagaraNodeOutput::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (ScriptType == ENiagaraScriptUsage::ParticleSpawnScript)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputSpawn", "Output Particle Spawn");
	}
	else if (ScriptType == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputSpawn", "Output Particle Spawn");
	}
	else if (ScriptType == ENiagaraScriptUsage::ParticleUpdateScript)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputUpdate", "Output Particle Update");
	}
	else if (ScriptType == ENiagaraScriptUsage::ParticleEventScript)
	{
		FText EventName;
		if (FNiagaraEditorUtilities::TryGetEventDisplayName(GetTypedOuter<UNiagaraEmitter>(), ScriptTypeId, EventName) == false)
		{
			EventName = NSLOCTEXT("NiagaraNodeOutput", "UnknownEventName", "Unknown");
		}
		return FText::Format(NSLOCTEXT("NiagaraNodeOutput", "OutputEvent", "Output Event {0}"), EventName);
	}
	else if (ScriptType == ENiagaraScriptUsage::ParticleSimulationStageScript)
	{
		FText EventName = FText::FromString(ScriptTypeId.ToString());
		return FText::Format(NSLOCTEXT("NiagaraNodeOutput", "OutputStage", "Output Stage {0}"), EventName);
	}
	else if (ScriptType == ENiagaraScriptUsage::Function)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputFunction", "Output Function");
	}
	else if (ScriptType == ENiagaraScriptUsage::DynamicInput)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputDynamicInput", "Output Dynamic Input");
	}
	else if (ScriptType == ENiagaraScriptUsage::Module)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputModule", "Output Module");
	}
	else  if (ScriptType == ENiagaraScriptUsage::EmitterSpawnScript)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputEmitterSpawn", "Output Emitter Spawn");
	}
	else if (ScriptType == ENiagaraScriptUsage::EmitterUpdateScript)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputEmitterUpdate", "Output Emitter Update");
	}
	else  if (ScriptType == ENiagaraScriptUsage::SystemSpawnScript)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputSystemSpawn", "Output System Spawn");
	}
	else if (ScriptType == ENiagaraScriptUsage::SystemUpdateScript)
	{
		return NSLOCTEXT("NiagaraNodeOutput", "OutputSystemUpdate", "Output System Update");
	}

	return NSLOCTEXT("NiagaraNode", "Output", "Output");
}

FLinearColor UNiagaraNodeOutput::GetNodeTitleColor() const
{
	return CastChecked<UEdGraphSchema_Niagara>(GetSchema())->NodeTitleColor_Attribute;
}

void UNiagaraNodeOutput::NotifyOutputVariablesChanged()
{
	ReallocatePins();
}

int32 UNiagaraNodeOutput::CompileInputPin(FTranslator* Translator, UEdGraphPin* Pin) const
{
	// If we are an update script, automatically fill in any unwired values with the previous frame's value...
	if (GetUsage() == ENiagaraScriptUsage::ParticleUpdateScript && Pin->LinkedTo.Num() == 0)
	{
		FNiagaraVariable OutputVariable;
		for (const FNiagaraVariable& Output : Outputs)
		{
			if (Output.GetName().ToString() == Pin->GetName() && Output.GetType() != FNiagaraTypeDefinition::GetParameterMapDef())
			{
				OutputVariable = Output;
				return Translator->GetAttribute(OutputVariable);
			}
		}
	}

	return Translator->CompileInputPin(Pin);
}

void UNiagaraNodeOutput::Compile(FTranslator* Translator, TArray<int32>& OutputExpressions) const
{
	TArray<int32> Results;
	bool bError = CompileInputPins(Translator, Results);
	if (!bError)
	{
		Translator->Output(this, Results);
	}
}

void UNiagaraNodeOutput::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	Super::BuildParameterMapHistory(OutHistory, bRecursive, bFilterForCompilation);
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	for (const UEdGraphPin* Pin : Pins)
	{
		if (Pin && Pin->Direction == EGPD_Input)
		{
			FNiagaraTypeDefinition InputType = Schema->PinToTypeDefinition(Pin);
			if (InputType.IsStatic())
			{
				OutHistory.RegisterConstantFromInputPin(Pin, Pin->DefaultValue);
			}
			else if (InputType == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				if (Pin->LinkedTo.Num() != 0)
				{
					int32 ParamMapIdx = OutHistory.TraceParameterMapOutputPin(Pin->LinkedTo[0]);			
					if (UNiagaraScript::LogCompileStaticVars > 0)
					{
						UE_LOGF(LogNiagaraEditor, Log, "Build Parameter Map History: %ls %ls PMapIdx: %d", *GetClass()->GetName(), *GetNodeTitle(ENodeTitleType::FullTitle).ToString(), ParamMapIdx);
					}
					OutHistory.RegisterParameterMapPin(ParamMapIdx, Pin);
				}
			}
		}
	}
}


void UNiagaraNodeOutput::PostLoad()
{
	Super::PostLoad();

	// In the version change for FNiagaraCustomVersion::UpdateSpawnEventGraphCombination we now require that output nodes specify the
	// compilation type that they support.
	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraVer < FNiagaraCustomVersion::UpdateSpawnEventGraphCombination && NiagaraVer > 0)
	{
		UEdGraph* Graph = GetGraph();
		if (Graph != nullptr)
		{
			UNiagaraScriptSource* Src = Cast<UNiagaraScriptSource>(Graph->GetOuter());
			if (Src != nullptr)
			{
				UNiagaraScript* Script = Cast<UNiagaraScript>(Src->GetOuter());
				if (Script != nullptr)
				{
					SetUsage(Script->GetUsage());
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

