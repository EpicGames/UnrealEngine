// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeOutputTag.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraHlslTranslator.h"
#include "Widgets/SNiagaraGraphNodeCustomHlsl.h"
#include "EdGraphSchema_Niagara.h"
#include "ScopedTransaction.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeOutputTag)

#define LOCTEXT_NAMESPACE "NiagaraNodeOutputTag"

UNiagaraNodeOutputTag::UNiagaraNodeOutputTag(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UNiagaraNodeOutputTag::AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const
{
	// Explicitly allow Numeric types, and explicitly disallow ParameterMap types
	return (Super::AllowNiagaraTypeForAddPin(InType) );
}

void UNiagaraNodeOutputTag::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	UEdGraphPin* InExecPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT(""));	
	UEdGraphPin* OutExecPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT(""));
	CreateAddPin(EGPD_Input);
}

void UNiagaraNodeOutputTag::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());

	for (UEdGraphPin* InputPin : InputPins)
	{
		if (IsAddPin(InputPin))
		{
			continue;
		}
		if (IsNodeEnabled() == false && Schema->PinToTypeDefinition(InputPin) != FNiagaraTypeDefinition::GetParameterMapDef())
		{
			continue;
		}
		int32 CompiledInput = Translator->CompileInputPin(InputPin);

		if (Schema->PinToTypeDefinition(InputPin) == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			Outputs.Add(CompiledInput);
		}
		else
		{
			Translator->WriteCompilerTag(CompiledInput, InputPin, bEditorOnly, bEmitMessageOnFailure, FailureSeverity);
		}
	}

	
	
	ensure(Outputs.Num() == 1);

}

void UNiagaraNodeOutputTag::OnPinRemoved(UEdGraphPin* PinToRemove)
{
	//ReallocatePins();
}

void UNiagaraNodeOutputTag::OnNewTypedPinAdded(UEdGraphPin*& NewPin)
{
	Super::OnNewTypedPinAdded(NewPin);
	PinPendingRename = NewPin;
}

void UNiagaraNodeOutputTag::OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName)
{
	//ReallocatePins();
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
}

bool UNiagaraNodeOutputTag::CanRenamePin(const UEdGraphPin* Pin) const
{
	return Super::CanRenamePin(Pin) && Pin->Direction == EGPD_Input;
}

bool UNiagaraNodeOutputTag::CanRemovePin(const UEdGraphPin* Pin) const
{
	return Super::CanRemovePin(Pin) && Pin->Direction == EGPD_Input;
}


bool UNiagaraNodeOutputTag::CancelEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj)
{
	if (InGraphPinObj == PinPendingRename)
	{
		PinPendingRename = nullptr;
	}
	return true;
}

bool UNiagaraNodeOutputTag::CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents)
{
	if (InGraphPinObj == PinPendingRename)
	{
		PinPendingRename = nullptr;
	}

	if (Pins.Contains(InGraphPinObj))
	{

		FString OldPinName = InGraphPinObj->PinName.ToString();
		FString NewPinName = InName.ToString();

		// Early out if the same!
		if (OldPinName == NewPinName)
		{
			return true;
		}

		FScopedTransaction AddNewPinTransaction(LOCTEXT("Rename Pin", "Renamed pin"));
		Modify();
		InGraphPinObj->Modify();

		InGraphPinObj->PinName = *NewPinName;
		InGraphPinObj->PinFriendlyName = InName;
		if (bSuppressEvents == false)
			OnPinRenamed(InGraphPinObj, OldPinName);

		return true;
	}
	return false;
}

bool UNiagaraNodeOutputTag::IsPinNameEditable(const UEdGraphPin* GraphPinObj) const
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(GraphPinObj);
	if (TypeDef.IsValid() && GraphPinObj && GraphPinObj->Direction == EGPD_Input && CanRenamePin(GraphPinObj))
	{
		return true;
	}
	else
	{
		return false;
	}
}

bool UNiagaraNodeOutputTag::IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const
{
	if (GraphPinObj == PinPendingRename)
	{

		return true;
	}
	else
	{
		return false;
	}
}


bool UNiagaraNodeOutputTag::VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const
{
	if (InName.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("InvalidName", "Invalid pin name");
		return false;
	}
	return true;
}

void UNiagaraNodeOutputTag::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	int32 ParamMapIdx = INDEX_NONE;
	uint32 NodeIdx = INDEX_NONE;

	for (int32 i = 0; i < InputPins.Num(); i++)
	{
		UEdGraphPin* InputPin = InputPins[i];
		if (IsAddPin(InputPin))
		{
			continue;
		}

		OutHistory.VisitInputPin(InputPin, bFilterForCompilation);


		if (!IsNodeEnabled() && OutHistory.GetIgnoreDisabled())
		{
			continue;
		}

		FNiagaraTypeDefinition VarTypeDef = Schema->PinToTypeDefinition(InputPin);
		if (i == 0 && InputPin != nullptr && VarTypeDef == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			UEdGraphPin* PriorParamPin = nullptr;
			if (InputPin->LinkedTo.Num() > 0)
			{
				PriorParamPin = InputPin->LinkedTo[0];
			}

			// Now plow into our ancestor node
			if (PriorParamPin)
			{
				ParamMapIdx = OutHistory.TraceParameterMapOutputPin(PriorParamPin);
				NodeIdx = OutHistory.BeginNodeVisitation(ParamMapIdx, this);
			}
		}
		else if (i > 0 && InputPin != nullptr && ParamMapIdx != INDEX_NONE)
		{
			OutHistory.HandleVariableWrite(ParamMapIdx, InputPin);
		}
	}

	if (!IsNodeEnabled() && OutHistory.GetIgnoreDisabled())
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		OutHistory.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}

	OutHistory.RegisterParameterMapPin(ParamMapIdx, GetOutputPin(0));
}

FText UNiagaraNodeOutputTag::GetTooltipText() const
{
	return LOCTEXT("OutputTagDesc", "Each Pin name will be written to the compiler output with an entry about the input value. The linked pin should ultimately route to a constant value or rapid iteration value. Anything else will result in a compile error.");
}

FText UNiagaraNodeOutputTag::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("OutputTagTitle", "Output Compiler Tag");
}


#undef LOCTEXT_NAMESPACE


