// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraNodeWithDynamicPins.h"
#include "NiagaraScript.h"
#include "NiagaraNodeOutputTag.generated.h"

UCLASS(MinimalAPI)
class UNiagaraNodeOutputTag : public UNiagaraNodeWithDynamicPins
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End EdGraphNode Interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;

	virtual bool IsPinNameEditable(const UEdGraphPin* GraphPinObj) const override;
	virtual bool IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const override;
	virtual bool VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const override;
	virtual bool CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents = false)  override;
	virtual bool CancelEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj) override;
	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;

	UPROPERTY(EditAnywhere, Category = Tag)
	bool bEditorOnly = false;

	UPROPERTY(EditAnywhere, Category=Tag)
	bool bEmitMessageOnFailure = true;

	UPROPERTY(EditAnywhere, Category = Tag)
	FNiagaraCompileEventSeverity FailureSeverity = FNiagaraCompileEventSeverity::Display;

protected:

	//~ Begin EdGraphNode Interface
	virtual void OnPinRemoved(UEdGraphPin* PinToRemove) override;
	//~ End EdGraphNode Interface

	//~ Begin UNiagaraNodeWithDynamicPins Interface
	virtual void OnNewTypedPinAdded(UEdGraphPin*& NewPin) override;
	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName) override;
	virtual bool CanRenamePin(const UEdGraphPin* Pin) const override;
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const override;
	virtual bool CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const override { return false; }
	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const override;
	//~ End UNiagaraNodeWithDynamicPins Interface

	UEdGraphPin* PinPendingRename = nullptr;
};

