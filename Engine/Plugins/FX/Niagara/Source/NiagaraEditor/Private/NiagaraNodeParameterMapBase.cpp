// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeParameterMapBase.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraNodeOutput.h"
#include "ScopedTransaction.h"
#include "Widgets/SNiagaraGraphPinAdd.h"
#include "NiagaraGraph.h"
#include "NiagaraComponent.h"
#include "NiagaraScript.h"
#include "NiagaraConstants.h"
#include "ToolMenus.h"
#include "NiagaraScriptVariable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeParameterMapBase)

#define LOCTEXT_NAMESPACE "NiagaraNodeParameterMapBase"

const FName UNiagaraNodeParameterMapBase::ParameterPinSubCategory("ParameterPin");
const FName UNiagaraNodeParameterMapBase::SourcePinName("Source");
const FName UNiagaraNodeParameterMapBase::DestPinName("Dest");
const FName UNiagaraNodeParameterMapBase::AddPinName("Add");

UNiagaraNodeParameterMapBase::UNiagaraNodeParameterMapBase() 
	: UNiagaraNodeWithDynamicPins()
	, PinPendingRename(nullptr)
{

}

TArray<FNiagaraParameterMapHistory> UNiagaraNodeParameterMapBase::GetParameterMaps(const UNiagaraGraph* InGraph)
{
	TArray<UNiagaraNodeOutput*> OutputNodes;
	InGraph->FindOutputNodes(OutputNodes);
	TArray<FNiagaraParameterMapHistory> OutputParameterMapHistories;

	for (UNiagaraNodeOutput* FoundOutputNode : OutputNodes)
	{
		FNiagaraParameterMapHistoryBuilder Builder;
		Builder.BuildParameterMaps(FoundOutputNode);

		OutputParameterMapHistories.Append(Builder.Histories);
	}

	return OutputParameterMapHistories;
}

bool UNiagaraNodeParameterMapBase::AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const
{
	return InType != FNiagaraTypeDefinition::GetGenericNumericDef()
		&& InType != FNiagaraTypeDefinition::GetParameterMapDef()
		&& !InType.IsInternalType();
}

FText UNiagaraNodeParameterMapBase::GetPinDescriptionText(UEdGraphPin* Pin) const
{
	FNiagaraVariable Var = CastChecked<UEdGraphSchema_Niagara>(GetSchema())->PinToNiagaraVariable(Pin);

	const UNiagaraGraph* Graph = GetNiagaraGraph();
	TOptional<FNiagaraVariableMetaData> MetaData = Graph->GetMetaData(Var);
	if (MetaData.IsSet())
	{
		return MetaData->Description;
	}
	return FText::GetEmpty();
}

/** Called when a pin's description text is committed. */
void UNiagaraNodeParameterMapBase::PinDescriptionTextCommitted(const FText& Text, ETextCommit::Type CommitType, UEdGraphPin* Pin)
{
	FNiagaraVariable Var = CastChecked<UEdGraphSchema_Niagara>(GetSchema())->PinToNiagaraVariable(Pin);
	UNiagaraGraph* Graph = GetNiagaraGraph();
	if (FNiagaraConstants::IsNiagaraConstant(Var))
	{
		UE_LOGF(LogNiagaraEditor, Error, "You cannot set the description for a Niagara internal constant \"%ls\"",*Var.GetName().ToString());
		return;
	}

	TOptional<FNiagaraVariableMetaData> OldMetaData = Graph->GetMetaData(Var);
	FNiagaraVariableMetaData NewMetaData;
	if (OldMetaData.IsSet())
	{
		NewMetaData = OldMetaData.GetValue();
	}

	FScopedTransaction AddNewPinTransaction(LOCTEXT("Rename Pin Desc", "Changed variable description"));
	NewMetaData.Description = Text;
	Graph->Modify();
	Graph->SetMetaData(Var, NewMetaData);
}

void UNiagaraNodeParameterMapBase::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	FText Text;
	if (GetTooltipTextForKnownPin(Pin, Text))
	{
		HoverTextOut = Text.ToString();
		return;
	}
	
	// Get hover text from metadata description.
	const UNiagaraGraph* NiagaraGraph = GetNiagaraGraph();
	if (NiagaraGraph)
	{
		const UEdGraphSchema_Niagara* Schema = Cast<UEdGraphSchema_Niagara>(NiagaraGraph->GetSchema());
		if (Schema)
		{
			if (IsAddPin(&Pin))
			{
				HoverTextOut = LOCTEXT("ParameterMapAddString", "Request a new variable from the parameter map.").ToString();
				return;
			}

			FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(&Pin);

			if (Pin.Direction == EEdGraphPinDirection::EGPD_Input)
			{
				if (&Pin == GetInputPin(0) && TypeDef == FNiagaraTypeDefinition::GetParameterMapDef())
				{
					HoverTextOut = LOCTEXT("ParameterMapInString", "The source parameter map where we pull the values from.").ToString();
					return;
				}
			}

			if (Pin.Direction == EEdGraphPinDirection::EGPD_Output)
			{
				if (&Pin == GetOutputPin(0) && TypeDef == FNiagaraTypeDefinition::GetParameterMapDef())
				{
					HoverTextOut = LOCTEXT("ParameterMapOutString", "The destination parameter map where we write the values to.").ToString();
					return;
				}
			}

			FNiagaraVariable Var = FNiagaraVariable(TypeDef, Pin.PinName);
			const FText Name = FText::FromName(Var.GetName());

			FText Description;
			TOptional<FNiagaraVariableMetaData> Metadata = NiagaraGraph->GetMetaData(Var);
			if (Metadata.IsSet())
			{
				Description = Metadata->Description;
			}

			const FText TooltipFormat = LOCTEXT("Parameters", "Name: {0} \nType: {1}\nDescription: {2}");
			const FText ToolTipText = FText::Format(TooltipFormat, FText::FromName(Var.GetName()), Var.GetType().GetNameText(), Description);
			HoverTextOut = ToolTipText.ToString();
		}
	}
}

void UNiagaraNodeParameterMapBase::SetPinName(UEdGraphPin* InPin, const FName& InName)
{
	FName OldName = InPin->PinName;
	InPin->PinName = InName;
	OnPinRenamed(InPin, OldName.ToString());
}

bool UNiagaraNodeParameterMapBase::CanRenamePin(const UEdGraphPin* Pin) const
{
	if(Super::CanRenamePin(Pin) == false)
	{
		return false;
	}

	FText Unused;
	return FNiagaraParameterUtilities::TestCanRenameWithMessage(Pin->PinName, Unused);
}

bool UNiagaraNodeParameterMapBase::GetIsPinEditNamespaceModifierPending(const UEdGraphPin* Pin)
{
	return PinsGuidsWithEditNamespaceModifierPending.Contains(Pin->PersistentGuid);
}

void UNiagaraNodeParameterMapBase::SetIsPinEditNamespaceModifierPending(const UEdGraphPin* Pin, bool bInIsEditNamespaceModifierPending)
{
	if (bInIsEditNamespaceModifierPending)
	{
		PinsGuidsWithEditNamespaceModifierPending.AddUnique(Pin->PersistentGuid);
	}
	else
	{
		PinsGuidsWithEditNamespaceModifierPending.Remove(Pin->PersistentGuid);
	}
}

bool UNiagaraNodeParameterMapBase::CanHandleDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	if (DragDropOperation->IsOfType<FNiagaraParameterGraphDragOperation>())
	{
		if (StaticCastSharedPtr<FNiagaraParameterGraphDragOperation>(DragDropOperation)->IsCurrentlyHoveringNode(this))
		{
			TSharedPtr<FNiagaraParameterGraphDragOperation> ParameterDragDropOperation = StaticCastSharedPtr<FNiagaraParameterGraphDragOperation>(DragDropOperation);
			
			if(ParameterDragDropOperation->GetSourceAction().IsValid())
			{
				if(TSharedPtr<FNiagaraParameterAction> ParameterAction = StaticCastSharedPtr<FNiagaraParameterAction>(ParameterDragDropOperation->GetSourceAction()))
				{
					FNiagaraVariable Parameter = ParameterAction->GetParameter();
					// if the parameter already exists on a node, we won't allow a drop
					if(DoesParameterExistOnNode(Parameter))
					{
						return false;
					}
				}				
			}
			
			return true;
		}
	}
	else if (DragDropOperation->IsOfType<FNiagaraParameterDragOperation>())
	{
		TSharedPtr<FNiagaraParameterDragOperation> ParameterDragDropOperation = StaticCastSharedPtr<FNiagaraParameterDragOperation>(DragDropOperation);
		TSharedPtr<const FNiagaraParameterAction> ParameterAction = StaticCastSharedPtr<const FNiagaraParameterAction>(ParameterDragDropOperation->GetSourceAction());
		EEdGraphPinDirection NewPinDirection = GetPinDirectionForNewParameters();
		if (ParameterAction.IsValid() && NewPinDirection != EGPD_MAX)
		{
			FNiagaraVariable Parameter = ParameterAction->GetParameter();
			FNiagaraParameterHandle ParameterHandle(Parameter.GetName());
			FNiagaraNamespaceMetadata ParameterNamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(ParameterHandle.GetHandleParts());
			return ParameterNamespaceMetadata.IsValid() && ParameterNamespaceMetadata.Options.Contains(ENiagaraNamespaceMetadataOptions::HideInScript) == false;
		}
	}
	return false;
}

bool UNiagaraNodeParameterMapBase::HandleDropOperation(TSharedPtr<FDragDropOperation> DropOperation)
{
	if (CanHandleDropOperation(DropOperation) && DropOperation->IsOfType<FNiagaraParameterDragOperation>())
	{
		TSharedPtr<FNiagaraParameterDragOperation> ParameterDropOperation = StaticCastSharedPtr<FNiagaraParameterDragOperation>(DropOperation);
		TSharedPtr<const FNiagaraParameterAction> ParameterAction = StaticCastSharedPtr<const FNiagaraParameterAction>(ParameterDropOperation->GetSourceAction());
		FNiagaraVariable Parameter = ParameterAction->GetParameter();
		EEdGraphPinDirection NewPinDirection = GetPinDirectionForNewParameters();

		FScopedTransaction AddNewPinTransaction(LOCTEXT("DropParameterOntoParameterMapNode", "Drop parameter onto parameter map node."));
		UEdGraphPin* AddedPin = RequestNewTypedPin(NewPinDirection, Parameter.GetType(), Parameter.GetName());
		if (AddedPin != nullptr)
		{
			CancelEditablePinName(FText::GetEmpty(), AddedPin);
		}
		return true;
	}
	return false;
}

void UNiagaraNodeParameterMapBase::OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName)
{
	RenamedPin->PinFriendlyName = FText::AsCultureInvariant(RenamedPin->PinName.ToString());

	FPinCollectorArray InOrOutPins;
	if (RenamedPin->Direction == EEdGraphPinDirection::EGPD_Input)
	{
		GetInputPins(InOrOutPins);
	}
	else
	{
		GetOutputPins(InOrOutPins);
	}

	TSet<FName> Names;
	for (const UEdGraphPin* Pin : InOrOutPins)
	{
		if (Pin != RenamedPin)
		{
			Names.Add(Pin->GetFName());
		}
	}
	const FName NewUniqueName = FNiagaraUtilities::GetUniqueName(*RenamedPin->GetName(), Names);

	FNiagaraTypeDefinition VarType = CastChecked<UEdGraphSchema_Niagara>(GetSchema())->PinToTypeDefinition(RenamedPin);
	FNiagaraVariable Var(VarType, *OldName);

	UNiagaraGraph* Graph = GetNiagaraGraph();
	Graph->RenameParameterFromPin(Var, NewUniqueName, RenamedPin);

	if (RenamedPin == PinPendingRename)
	{
		PinPendingRename = nullptr;
	}

}

void UNiagaraNodeParameterMapBase::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	UEdGraphPin* Pin = const_cast<UEdGraphPin*>(Context->Pin);
	if (Pin && !IsAddPin(Pin))
	{

		FNiagaraVariable Var = CastChecked<UEdGraphSchema_Niagara>(GetSchema())->PinToNiagaraVariable(Pin);
		const UNiagaraGraph* Graph = GetNiagaraGraph();

		UNiagaraNodeParameterMapBase* NonConstThis = const_cast<UNiagaraNodeParameterMapBase*>(this);
		UEdGraphPin* NonConstPin = const_cast<UEdGraphPin*>(Context->Pin);
		//if (!FNiagaraConstants::IsNiagaraConstant(Var))
		FToolMenuSection& EditSection = Menu->FindOrAddSection("EditPin");
		{
			EditSection.AddSubMenu(
				"ChangeNamespace",
				LOCTEXT("ChangeNamespace", "Change Namespace"),
				LOCTEXT("ChangeNamespaceToolTip", "Change the namespace for this parameter pin."),
				FNewToolMenuDelegate::CreateUObject(NonConstThis, &UNiagaraNodeParameterMapBase::GetChangeNamespaceSubMenuForPin, NonConstPin));

			EditSection.AddSubMenu(
				"ChangeNamespaceModifier",
				LOCTEXT("ChangeNamespaceModifier", "Change Namespace Modifier"),
				LOCTEXT("ChangeNamespaceModifierToolTip", "Change the namespace modifier for this parameter pin."),
				FNewToolMenuDelegate::CreateUObject(NonConstThis, &UNiagaraNodeParameterMapBase::GetChangeNamespaceModifierSubMenuForPin, NonConstPin));
		}

		{
			FToolMenuSection& Section = Menu->AddSection("EdGraphSchema_NiagaraParamAction", LOCTEXT("EditPinMenuHeader", "Parameters"));
			Section.AddMenuEntry(
				"SelectParameter",
				LOCTEXT("SelectParameterPin", "Select parameter"),
				LOCTEXT("SelectParameterPinToolTip", "Select this parameter in the paramter panel"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateUObject(NonConstThis, &UNiagaraNodeParameterMapBase::SelectParameterFromPin, Context->Pin)));
		}
	}
}

void UNiagaraNodeParameterMapBase::GetChangeNamespaceSubMenuForPin(UToolMenu* Menu, UEdGraphPin* InPin)
{
	FToolMenuSection& Section = Menu->AddSection("Section");

	TArray<FNiagaraParameterUtilities::FChangeNamespaceMenuData> MenuData;
	FNiagaraParameterUtilities::GetChangeNamespaceMenuData(InPin->PinName, FNiagaraParameterUtilities::EParameterContext::Script, MenuData);
	for(const FNiagaraParameterUtilities::FChangeNamespaceMenuData& MenuDataItem : MenuData)
	{
		bool bCanChange = MenuDataItem.bCanChange;
		FUIAction Action = FUIAction(
			FExecuteAction::CreateUObject(this, &UNiagaraNodeParameterMapBase::ChangeNamespaceForPin, InPin, MenuDataItem.Metadata),
			FCanExecuteAction::CreateLambda([bCanChange]() { return bCanChange; }));

		TSharedRef<SWidget> MenuItemWidget = FNiagaraParameterUtilities::CreateNamespaceMenuItemWidget(MenuDataItem.NamespaceParameterName, MenuDataItem.CanChangeToolTip);
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(NAME_None, Action, MenuItemWidget));
	}
}

void UNiagaraNodeParameterMapBase::ChangeNamespaceForPin(UEdGraphPin* InPin, FNiagaraNamespaceMetadata NewNamespaceMetadata)
{
	FName NewName = FNiagaraParameterUtilities::ChangeNamespace(InPin->PinName, NewNamespaceMetadata);
	if (NewName != NAME_None)
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeNamespaceTransaction", "Change parameter namespace"));
		CommitEditablePinName(FText::FromName(NewName), InPin);
	}
}

void UNiagaraNodeParameterMapBase::SelectParameterFromPin(const UEdGraphPin* InPin) 
{
	UNiagaraGraph* NiagaraGraph = GetNiagaraGraph();
	if (NiagaraGraph && InPin)
	{
		const UEdGraphSchema_Niagara* Schema = Cast<UEdGraphSchema_Niagara>(NiagaraGraph->GetSchema());
		if (Schema)
		{
			if (IsAddPin(InPin))
			{
				return;
			}

			FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(InPin);
			FNiagaraVariable PinVariable = FNiagaraVariable(TypeDef, InPin->PinName);
			TObjectPtr<UNiagaraScriptVariable>* PinAssociatedScriptVariable = NiagaraGraph->GetAllMetaData().Find(PinVariable);
			if (PinAssociatedScriptVariable != nullptr)
			{
				NiagaraGraph->OnSubObjectSelectionChanged().Broadcast(*PinAssociatedScriptVariable);
			}
		}
	}
}

bool UNiagaraNodeParameterMapBase::DoesParameterExistOnNode(FNiagaraVariable Parameter)
{
	for(UEdGraphPin* Pin : Pins)
	{
		if(Pin->PinType.PinSubCategory == ParameterPinSubCategory)
		{
			FNiagaraVariable CandidateVariable = UEdGraphSchema_Niagara::PinToNiagaraVariable(Pin);

			if(CandidateVariable == Parameter)
			{
				return true;
			}
		}
	}

	return false;
}

void UNiagaraNodeParameterMapBase::GetChangeNamespaceModifierSubMenuForPin(UToolMenu* Menu, UEdGraphPin* InPin)
{
	FToolMenuSection& Section = Menu->AddSection("Section");

	TArray<FName> OptionalNamespaceModifiers;
	FNiagaraParameterUtilities::GetOptionalNamespaceModifiers(InPin->PinName, FNiagaraParameterUtilities::EParameterContext::Script, OptionalNamespaceModifiers);
	for (FName OptionalNamespaceModifier : OptionalNamespaceModifiers)
	{
		TAttribute<FText> SetToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateUObject(
			this, &UNiagaraNodeParameterMapBase::GetSetNamespaceModifierForPinToolTip, (const UEdGraphPin*)InPin, OptionalNamespaceModifier));
		Section.AddMenuEntry(
			OptionalNamespaceModifier,
			FText::FromName(OptionalNamespaceModifier),
			SetToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(this, &UNiagaraNodeParameterMapBase::SetNamespaceModifierForPin, InPin, OptionalNamespaceModifier),
				FCanExecuteAction::CreateUObject(this, &UNiagaraNodeParameterMapBase::CanSetNamespaceModifierForPin, (const UEdGraphPin*)InPin, OptionalNamespaceModifier)));
	}

	TAttribute<FText> SetCustomToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateUObject(
		this, &UNiagaraNodeParameterMapBase::GetSetCustomNamespaceModifierForPinToolTip, (const UEdGraphPin*)InPin));
	Section.AddMenuEntry(
		"AddCustomNamespaceModifier",
		LOCTEXT("SetCustomNamespaceModifierForPin", "Custom..."),
		SetCustomToolTip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateUObject(this, &UNiagaraNodeParameterMapBase::SetCustomNamespaceModifierForPin, InPin),
			FCanExecuteAction::CreateUObject(this, &UNiagaraNodeParameterMapBase::CanSetCustomNamespaceModifierForPin, (const UEdGraphPin*)InPin)));

	TAttribute<FText> SetNoneToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateUObject(
		this, &UNiagaraNodeParameterMapBase::GetSetNamespaceModifierForPinToolTip, (const UEdGraphPin*)InPin, FName(NAME_None)));
	Section.AddMenuEntry(
		"AddNoneNamespaceModifier",
		LOCTEXT("SetNoneNamespaceModifierForPin", "Clear"),
		SetNoneToolTip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateUObject(this, &UNiagaraNodeParameterMapBase::SetNamespaceModifierForPin, InPin, FName(NAME_None)),
			FCanExecuteAction::CreateUObject(this, &UNiagaraNodeParameterMapBase::CanSetNamespaceModifierForPin, (const UEdGraphPin*)InPin, FName(NAME_None))));
}

FText UNiagaraNodeParameterMapBase::GetSetNamespaceModifierForPinToolTip(const UEdGraphPin* InPin, FName InNamespaceModifier) const
{
	FText SetMessage;
	FNiagaraParameterUtilities::TestCanSetSpecificNamespaceModifierWithMessage(InPin->PinName, InNamespaceModifier, SetMessage);
	return SetMessage;
}

bool UNiagaraNodeParameterMapBase::CanSetNamespaceModifierForPin(const UEdGraphPin* InPin, FName InNamespaceModifier) const
{
	FText Unused;
	return FNiagaraParameterUtilities::TestCanSetSpecificNamespaceModifierWithMessage(InPin->PinName, InNamespaceModifier, Unused);
}

void UNiagaraNodeParameterMapBase::SetNamespaceModifierForPin(UEdGraphPin* InPin, FName InNamespaceModifier)
{
	FName NewName = FNiagaraParameterUtilities::SetSpecificNamespaceModifier(InPin->PinName, InNamespaceModifier);
	if (NewName != NAME_None)
	{
		FScopedTransaction Transaction(LOCTEXT("AddNamespaceModifierTransaction", "Add namespace modifier"));
		CommitEditablePinName(FText::FromName(NewName), InPin);
	}
}

FText UNiagaraNodeParameterMapBase::GetSetCustomNamespaceModifierForPinToolTip(const UEdGraphPin* InPin) const
{
	FText SetMessage;
	FNiagaraParameterUtilities::TestCanSetCustomNamespaceModifierWithMessage(InPin->PinName, SetMessage);
	return SetMessage;
}

bool UNiagaraNodeParameterMapBase::CanSetCustomNamespaceModifierForPin(const UEdGraphPin* InPin) const
{
	FText Unused;
	return FNiagaraParameterUtilities::TestCanSetCustomNamespaceModifierWithMessage(InPin->PinName, Unused);
}

void UNiagaraNodeParameterMapBase::SetCustomNamespaceModifierForPin(UEdGraphPin* InPin)
{
	FName NewName = FNiagaraParameterUtilities::SetCustomNamespaceModifier(InPin->PinName);
	if (NewName != NAME_None)
	{
		if (NewName != InPin->PinName)
		{
			FScopedTransaction Transaction(LOCTEXT("AddCustomNamespaceModifierTransaction", "Add custom namespace modifier"));
			CommitEditablePinName(FText::FromName(NewName), InPin);
		}
		SetIsPinEditNamespaceModifierPending(InPin, true);
	}
}

#undef LOCTEXT_NAMESPACE
