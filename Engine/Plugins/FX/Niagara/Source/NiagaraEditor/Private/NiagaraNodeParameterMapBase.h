// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraNodeWithDynamicPins.h"
#include "SGraphPin.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraNodeParameterMapBase.generated.h"

class UEdGraphPin;

/** Base node for parameter maps. */
UCLASS()
class UNiagaraNodeParameterMapBase : public UNiagaraNodeWithDynamicPins
{
public:
	GENERATED_BODY()
	UNiagaraNodeParameterMapBase();

	/** Traverse the graph looking for the history of the parameter map specified by the input pin. This will return the list of variables discovered, any per-variable warnings (type mismatches, etc)
		encountered per variable, and an array of pins encountered in order of traversal outward from the input pin.
	*/
	static TArray<FNiagaraParameterMapHistory> GetParameterMaps(const UNiagaraGraph* InGraph);

	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const override;
	
	/** Gets the description text for a pin. */
	FText GetPinDescriptionText(UEdGraphPin* Pin) const;

	/** Called when a pin's description text is committed. */
	void PinDescriptionTextCommitted(const FText& Text, ETextCommit::Type CommitType, UEdGraphPin* Pin);

	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }

	void SetPinName(UEdGraphPin* InPin, const FName& InName);

	virtual bool CanRenamePinFromContextMenu(const UEdGraphPin* Pin) const override { return false; }
	
	virtual bool CanRenamePin(const UEdGraphPin* Pin) const override;
	
	virtual FName GetNewPinDefaultNamespace() const { return PARAM_MAP_LOCAL_MODULE_STR; }

	bool GetIsPinEditNamespaceModifierPending(const UEdGraphPin* Pin);

	void SetIsPinEditNamespaceModifierPending(const UEdGraphPin* Pin, bool bInIsEditNamespaceModifierPending);

	bool CanHandleDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation);

	bool HandleDropOperation(TSharedPtr<FDragDropOperation> DropOperation);

	bool DoesParameterExistOnNode(FNiagaraVariable Parameter);
protected:
	void GetChangeNamespaceSubMenuForPin(UToolMenu* Menu, UEdGraphPin* InPin);
	void ChangeNamespaceForPin(UEdGraphPin* InPin, FNiagaraNamespaceMetadata NewNamespaceMetadata);

	virtual void SelectParameterFromPin(const UEdGraphPin* InPin);
	
	void GetChangeNamespaceModifierSubMenuForPin(UToolMenu* Menu, UEdGraphPin* InPin);

	FText GetSetNamespaceModifierForPinToolTip(const UEdGraphPin* InPin, FName InNamespaceModifier) const;
	bool CanSetNamespaceModifierForPin(const UEdGraphPin* InPin, FName InNamespaceModifier) const;
	void SetNamespaceModifierForPin(UEdGraphPin* InPin, FName InNamespaceModifier);

	FText GetSetCustomNamespaceModifierForPinToolTip(const UEdGraphPin* InPin) const;
	bool CanSetCustomNamespaceModifierForPin(const UEdGraphPin* InPin) const;
	void SetCustomNamespaceModifierForPin(UEdGraphPin* InPin);

	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName) override;

	UEdGraphPin* PinPendingRename;

	TArray<FGuid> PinsGuidsWithEditNamespaceModifierPending;

public:
	/** The sub category for parameter pins. */
	static const FName ParameterPinSubCategory;

	static const FName SourcePinName;
	static const FName DestPinName;
	static const FName AddPinName;
};
