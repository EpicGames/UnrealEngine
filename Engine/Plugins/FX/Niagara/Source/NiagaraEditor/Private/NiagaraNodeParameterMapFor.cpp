// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeParameterMapFor.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraHlslTranslator.h"
#include "EdGraph/EdGraphNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeParameterMapFor)

#define LOCTEXT_NAMESPACE "NiagaraNodeParameterMapFor" 

UNiagaraNodeParameterMapFor::UNiagaraNodeParameterMapFor() : UNiagaraNodeParameterMapSet()
{
	UEdGraphNode::NodeUpgradeMessage = LOCTEXT("NodeExperimental", "This node is marked as experimental, use with care!");
}

void UNiagaraNodeParameterMapFor::AllocateDefaultPins()
{
	PinPendingRename = nullptr;
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), *UNiagaraNodeParameterMapBase::SourcePinName.ToString());
	UEdGraphPin* IterationPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetIntDef()), TEXT("Iteration Count"));
	IterationPin->PinToolTip = TEXT("How often the for loop should be executed - will only be evaluated once before executing the loop!");
	CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), *UNiagaraNodeParameterMapBase::DestPinName.ToString());
	CreateAddPin(EGPD_Input);
}

void UNiagaraNodeParameterMapFor::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	if (&Pin == InputPins[1])
	{
		HoverTextOut = LOCTEXT("IterationCountPinTooltip", "How often the loop should be executed - will only be evaluated once before executing the loop!").ToString();
		return;
	}
	Super::GetPinHoverText(Pin, HoverTextOut);
}

bool UNiagaraNodeParameterMapFor::CanModifyPin(const UEdGraphPin* Pin) const
{
	if(Super::CanModifyPin(Pin) == false)
	{
		return false;
	}

	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	// we don't allow the modification of the iteration count pin
	if(Pin == InputPins[1])
	{
		return false;
	}

	return true;
}

void UNiagaraNodeParameterMapFor::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	if (Translator)
	{
		if (Translator->GetSimulationTarget() == ENiagaraSimTarget::GPUComputeSim)
		{
			const int32 IterationCount = Translator->CompileInputPin(InputPins[1]);
			Translator->ParameterMapForBegin(this, IterationCount);

			UNiagaraNodeParameterMapSet::Compile(Translator, Outputs);

			Translator->ParameterMapForEnd(this);
		}
		else
		{
			UNiagaraNodeParameterMapSet::Compile(Translator, Outputs);
		}
	}
}

FText UNiagaraNodeParameterMapFor::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UNiagaraNodeParameterMapForName", "Map For");
}

bool UNiagaraNodeParameterMapFor::SkipPinCompilation(UEdGraphPin* Pin) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	return Pin == InputPins[1];
}

UNiagaraNodeParameterMapForWithContinue::UNiagaraNodeParameterMapForWithContinue() : UNiagaraNodeParameterMapFor()
{
	UEdGraphNode::NodeUpgradeMessage = LOCTEXT("NodeExperimental", "This node is marked as experimental, use with care!");
}

void UNiagaraNodeParameterMapForWithContinue::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	if (Translator)
	{
		if (Translator->GetSimulationTarget() == ENiagaraSimTarget::GPUComputeSim)
		{
			const int32 IterationCount = Translator->CompileInputPin(InputPins[1]);

			Translator->ParameterMapForBegin(this, IterationCount);
			
			const int32 IterationEnabledChunk = Translator->CompileInputPin(InputPins[2]);
			Translator->ParameterMapForContinue(this, IterationEnabledChunk);

			UNiagaraNodeParameterMapSet::Compile(Translator, Outputs);

			Translator->ParameterMapForEnd(this);
		}
		else
		{
			UNiagaraNodeParameterMapSet::Compile(Translator, Outputs);
		}
	}
}

void UNiagaraNodeParameterMapForWithContinue::AllocateDefaultPins()
{
	PinPendingRename = nullptr;
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), *UNiagaraNodeParameterMapBase::SourcePinName.ToString());
	CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetIntDef()), TEXT("Iteration Count"));
	CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetBoolDef()), TEXT("Iteration Enabled"));
	CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), *UNiagaraNodeParameterMapBase::DestPinName.ToString());
	CreateAddPin(EGPD_Input);
}

FText UNiagaraNodeParameterMapForWithContinue::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UNiagaraNodeParameterMapForWithContinueName", "Map For With Continue");
}

void UNiagaraNodeParameterMapForWithContinue::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	if (&Pin == InputPins[2])
	{
		HoverTextOut = LOCTEXT("IterationEnabledPinTooltip", "If set to false the loop will continue with the next iteration and execute no code - will be evaluated at the *start* of every iteration.").ToString();
		return;
	}
	Super::GetPinHoverText(Pin, HoverTextOut);
}

bool UNiagaraNodeParameterMapForWithContinue::CanModifyPin(const UEdGraphPin* Pin) const
{
	if(Super::CanModifyPin(Pin) == false)
	{
		return false;
	}

	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	// we don't allow the modification of the iteration count pin or the iteration enabled pin
	if(Pin == InputPins[1] || Pin == InputPins[2])
	{
		return false;
	}

	return true;
}

bool UNiagaraNodeParameterMapForWithContinue::SkipPinCompilation(UEdGraphPin* Pin) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	return Pin == InputPins[1] || Pin == InputPins[2];
}

UNiagaraNodeParameterMapForIndex::UNiagaraNodeParameterMapForIndex() : UNiagaraNode()
{
	UEdGraphNode::NodeUpgradeMessage = LOCTEXT("NodeExperimental", "This node is marked as experimental, use with care!");
}

void UNiagaraNodeParameterMapForIndex::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetIntDef()), TEXT("Module.Current Iteration"));
}

FText UNiagaraNodeParameterMapForIndex::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UNiagaraNodeParameterMapForIndex", "Map For Index");
}

void UNiagaraNodeParameterMapForIndex::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	if (Translator)
	{
		if (Translator->GetSimulationTarget() == ENiagaraSimTarget::GPUComputeSim)
		{
			Outputs.Add(Translator->ParameterMapForInnerIndex());
		}
		else
		{
			FNiagaraVariable Constant(FNiagaraTypeDefinition::GetIntDef(), TEXT("Constant"));
			Constant.SetValue(0);
			Outputs.Add(Translator->GetConstant(Constant));
		}
	}
}

#undef LOCTEXT_NAMESPACE

