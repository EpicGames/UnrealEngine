// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeParameterMapFor.generated.h"

/** A node that allows a user to set multiple values into a parameter map in a for loop. */
UCLASS()
class UNiagaraNodeParameterMapFor : public UNiagaraNodeParameterMapSet
{
public:
	GENERATED_BODY()

	UNiagaraNodeParameterMapFor();

	virtual void AllocateDefaultPins() override;

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return false; }
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	
protected:
	virtual bool CanModifyPin(const UEdGraphPin* Pin) const override;
	
	virtual bool SkipPinCompilation(UEdGraphPin* Pin) const override;
};

UCLASS()
class UNiagaraNodeParameterMapForWithContinue : public UNiagaraNodeParameterMapFor
{
public:
	GENERATED_BODY()

	UNiagaraNodeParameterMapForWithContinue();

	virtual void AllocateDefaultPins() override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return false; }
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	
protected:
	virtual bool CanModifyPin(const UEdGraphPin* Pin) const override;
	virtual bool SkipPinCompilation(UEdGraphPin* Pin) const override;
};

UCLASS()
class UNiagaraNodeParameterMapForIndex : public UNiagaraNode
{
public:
	GENERATED_BODY()

	UNiagaraNodeParameterMapForIndex();

	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
};
