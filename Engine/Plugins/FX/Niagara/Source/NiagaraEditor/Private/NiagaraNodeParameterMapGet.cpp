// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeParameterMapGet.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraHlslTranslator.h"
#include "ScopedTransaction.h"
#include "Widgets/SNiagaraGraphParameterMapGetNode.h"
#include "NiagaraEditorModule.h"
#include "INiagaraEditorTypeUtilities.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "NiagaraConstants.h"
#include "EdGraph/EdGraphNode.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraScriptVariable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeParameterMapGet)

#define LOCTEXT_NAMESPACE "NiagaraNodeParameterMapGet"

UNiagaraNodeParameterMapGet::UNiagaraNodeParameterMapGet() : UNiagaraNodeParameterMapBase()
{

}


void UNiagaraNodeParameterMapGet::AllocateDefaultPins()
{
	PinPendingRename = nullptr;
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), *UNiagaraNodeParameterMapBase::SourcePinName.ToString());
	CreateAddPin(EGPD_Output);
}


TSharedPtr<SGraphNode> UNiagaraNodeParameterMapGet::CreateVisualWidget()
{
	return SNew(SNiagaraGraphParameterMapGetNode, this);
}

bool UNiagaraNodeParameterMapGet::IsPinNameEditable(const UEdGraphPin* GraphPinObj) const
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(GraphPinObj);
	return TypeDef.IsValid() && GraphPinObj && GraphPinObj->Direction == EGPD_Output && CanRenamePin(GraphPinObj);
}

bool UNiagaraNodeParameterMapGet::IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const
{
	if (GraphPinObj == PinPendingRename && GraphPinObj->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
		FNiagaraVariable Var = Schema->PinToNiagaraVariable(GraphPinObj);
		if (!FNiagaraConstants::IsNiagaraConstant(Var))
			return true;
		else
			return false;
	}
	else
	{
		return false;
	}
}


bool UNiagaraNodeParameterMapGet::VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const
{
	if (InName.IsEmptyOrWhitespace() && InGraphPinObj->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		OutErrorMessage = LOCTEXT("InvalidName", "Invalid pin name");
		return false;
	}
	
	UNiagaraScriptVariable* ExistingScriptVariable = GetNiagaraGraph()->GetScriptVariable(FName(InName.ToString()));
	if(ExistingScriptVariable != nullptr)
	{
		// if the variable already exists and the type matches, we are trying to reference an existing parameter which is allowed.
		if(ExistingScriptVariable->Variable.GetType() == UEdGraphSchema_Niagara::PinToTypeDefinition(InGraphPinObj))
		{
			return true;
		}
		else
		{
			OutErrorMessage = LOCTEXT("InvalidName_VariableExistsDifferentType", "This variable already exists with a different type.\nChoose another name.");
			return false;
		}
	}
	
	return true;
}

UEdGraphPin* UNiagaraNodeParameterMapGet::CreateDefaultPin(UEdGraphPin* OutputPin)
{
	if (OutputPin == nullptr)
	{
		return nullptr;
	}
	
	UEdGraphPin* DefaultPin = CreatePin(EEdGraphPinDirection::EGPD_Input, OutputPin->PinType, TEXT(""));

	// make sure the new pin name is legal
	FName OutName;
	if (FNiagaraEditorUtilities::DecomposeVariableNamespace(OutputPin->GetFName(), OutName).Num() == 0)
	{
		OutputPin->PinName = FName(FNiagaraConstants::LocalNamespace.ToString() + "." + OutputPin->GetName());
	}

	// we make the pin read only because the default value is set in the parameter panel unless the default mode is set to "custom" by the user
	DefaultPin->bNotConnectable = true;
	DefaultPin->bDefaultValueIsReadOnly = true;

	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	const FNiagaraTypeDefinition NiagaraType = Schema->PinToTypeDefinition(OutputPin);
	const bool bNeedsValue = NiagaraType.IsDataInterface() == false && NiagaraType.IsUObject() == false;
	const FNiagaraVariable Var = Schema->PinToNiagaraVariable(OutputPin, bNeedsValue);

	FString PinDefaultValue;
	if (Schema->TryGetPinDefaultValueFromNiagaraVariable(Var, PinDefaultValue))
	{
		DefaultPin->DefaultValue = PinDefaultValue;
		// we set the auto generated default value to the same as our default value so we can later test if we had any changes
		DefaultPin->AutogeneratedDefaultValue = PinDefaultValue;
	}

	// If the variable of the new default pin is already in use in the graph we use the configured default value
	if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(GetGraph())) 
	{
		TOptional<ENiagaraDefaultMode> DefaultMode = Graph->GetDefaultMode(Var);
		TOptional<FNiagaraVariable> ScriptVariable = Graph->GetVariable(Var);

		if (DefaultMode && ScriptVariable)
		{
			if (*DefaultMode == ENiagaraDefaultMode::Value && ScriptVariable->IsValid() && ScriptVariable->IsDataAllocated())
			{
				FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
				TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(ScriptVariable->GetType());
				if (bNeedsValue && TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanHandlePinDefaults())
				{
					DefaultPin->DefaultValue = TypeEditorUtilities->GetPinDefaultStringFromValue(*ScriptVariable);
				}
			}
		}
	}
	
	if (!OutputPin->PersistentGuid.IsValid())
	{
		OutputPin->PersistentGuid = FGuid::NewGuid();
	}
	if (!DefaultPin->PersistentGuid.IsValid())
	{
		DefaultPin->PersistentGuid = FGuid::NewGuid();
	}
	PinOutputToPinDefaultPersistentId.Add(OutputPin->PersistentGuid, DefaultPin->PersistentGuid);

	SynchronizeDefaultInputPin(DefaultPin);
	return DefaultPin;
}

void UNiagaraNodeParameterMapGet::OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName)
{
	UNiagaraNodeParameterMapBase::OnPinRenamed(RenamedPin, OldName);

	UEdGraphPin* DefaultPin = GetDefaultPin(RenamedPin);
	if (DefaultPin)
	{
		DefaultPin->Modify();
		SynchronizeDefaultInputPin(DefaultPin);
	}

	MarkNodeRequiresSynchronization(__FUNCTION__, true);
}

void UNiagaraNodeParameterMapGet::OnPinRemoved(UEdGraphPin* InRemovedPin)
{
	FGuid PersistentGuidKey;
	if(UEdGraphPin* DefaultPin = GetDefaultPin(InRemovedPin))
	{
		PersistentGuidKey = InRemovedPin->PersistentGuid;

		if(!InRemovedPin->bOrphanedPin)
		{
			RemovePin(DefaultPin);
		}
	}

	if(UEdGraphPin* OutputPin = GetOutputPinForDefault(InRemovedPin))
	{
		PersistentGuidKey = OutputPin->PersistentGuid;
	}

	if(PersistentGuidKey.IsValid() && PinOutputToPinDefaultPersistentId.Contains(PersistentGuidKey))
	{
		PinOutputToPinDefaultPersistentId.Remove(PersistentGuidKey);
	}
	
	Super::OnPinRemoved(InRemovedPin);
}

bool UNiagaraNodeParameterMapGet::CanModifyPin(const UEdGraphPin* Pin) const
{
	if(Super::CanModifyPin(Pin) == false || Pin->Direction == EGPD_Input)
	{
		return false;
	}

	return true;
}

void UNiagaraNodeParameterMapGet::OnNewTypedPinAdded(UEdGraphPin*& NewPin)
{
	if (NewPin->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		FPinCollectorArray OutputPins;
		GetOutputPins(OutputPins);

		// Use the friendly name to build the new parameter name since it's what is displayed in the UI.
		FName NewPinName = NewPin->PinFriendlyName.IsEmpty() == false
			? *NewPin->PinFriendlyName.ToString()
			: NewPin->GetFName();

		FName PinNameWithoutNamespace;
		if (FNiagaraEditorUtilities::DecomposeVariableNamespace(NewPinName, PinNameWithoutNamespace).Num() == 0)
		{
			NewPinName = *(FNiagaraConstants::ModuleNamespace.ToString() + TEXT(".") + NewPinName.ToString());
		}

		TSet<FName> Names;
		for (const UEdGraphPin* Pin : OutputPins)
		{
			if (Pin != NewPin)
			{
				Names.Add(Pin->GetFName());
			}
		}
		const FName NewUniqueName = FNiagaraUtilities::GetUniqueName(NewPinName, Names);

		NewPin->PinName = NewUniqueName;
		NewPin->PinFriendlyName = FText::AsCultureInvariant(NewPin->PinName.ToString());
		NewPin->PinType.PinSubCategory = UNiagaraNodeParameterMapBase::ParameterPinSubCategory;

		UEdGraphPin* MatchingDefault = GetDefaultPin(NewPin);
		if (MatchingDefault == nullptr)
		{
			UEdGraphPin* DefaultPin = CreateDefaultPin(NewPin);
		}
	}

	if (HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedInitialization))
	{
		return;
	}

	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	constexpr bool bNeedsValue = false;
	FNiagaraVariable PinVariable = Schema->PinToNiagaraVariable(NewPin, bNeedsValue);
	constexpr bool bIsStaticSwitch = false;
	GetNiagaraGraph()->AddParameter(PinVariable, bIsStaticSwitch);
	NewPin->PinName = PinVariable.GetName();

	if (NewPin->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		PinPendingRename = NewPin;
	}

}

void UNiagaraNodeParameterMapGet::RemoveDynamicPin(UEdGraphPin* Pin)
{
	if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		UEdGraphPin* DefaultPin = GetDefaultPin(Pin);
		if (DefaultPin != nullptr)
		{
			RemovePin(DefaultPin);
		}
	}

	Super::RemoveDynamicPin(Pin);
}

UEdGraphPin* UNiagaraNodeParameterMapGet::GetDefaultPin(const UEdGraphPin* OutputPin) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	const FGuid* InputGuid = PinOutputToPinDefaultPersistentId.Find(OutputPin->PersistentGuid);

	if (InputGuid != nullptr)
	{
		for (UEdGraphPin* InputPin : InputPins)
		{
			if ((*InputGuid) == InputPin->PersistentGuid)
			{
				return InputPin;
			}
		}
	}

	return nullptr;
}

void UNiagaraNodeParameterMapGet::SynchronizeDefaultPins()
{
	TArray<UEdGraphPin*> OutputPins;
	GetOutputPins(OutputPins);

	for(UEdGraphPin* OutputPin : OutputPins)
	{
		if(IsAddPin(OutputPin) || OutputPin->bOrphanedPin)
		{
			continue;
		}

		UEdGraphPin* DefaultPin = GetDefaultPin(OutputPin);
		if(ensureAlwaysMsgf(DefaultPin != nullptr, TEXT("There should always be an input pin for every output pin, even if hidden.")))
		{
			SynchronizeDefaultInputPin(DefaultPin);
		}
	}
}


UEdGraphPin* UNiagaraNodeParameterMapGet::GetOutputPinForDefault(const UEdGraphPin* DefaultPin) const
{
	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);

	FGuid OutputGuid;
	for (auto It : PinOutputToPinDefaultPersistentId)
	{
		if (It.Value == DefaultPin->PersistentGuid)
		{
			OutputGuid = It.Key;
			break;
		}
	}

	if (OutputGuid.IsValid())
	{
		for (UEdGraphPin* OutputPin : OutputPins)
		{
			if (OutputGuid == OutputPin->PersistentGuid)
			{
				return OutputPin;
			}
		}
	}

	return nullptr;
}

void UNiagaraNodeParameterMapGet::PostLoad()
{
	Super::PostLoad();
	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);

	FPinCollectorArray InputPins;
	GetInputPins(InputPins);

	TSet<UEdGraphPin*> CorrectParameterDefaultPins;
	for (int32 i = 0; i < OutputPins.Num(); i++)
	{
		UEdGraphPin* OutputPin = OutputPins[i];
		if (IsAddPin(OutputPin) || OutputPin->bOrphanedPin)
		{
			continue;
		}

		UEdGraphPin* DefaultPin = GetDefaultPin(OutputPin);
		if (DefaultPin == nullptr)
		{
			DefaultPin = CreateDefaultPin(OutputPin);
		}
		else
		{
			SynchronizeDefaultInputPin(DefaultPin);
		}

		OutputPin->PinType.PinSubCategory = UNiagaraNodeParameterMapBase::ParameterPinSubCategory;
		
		CorrectParameterDefaultPins.Add(DefaultPin);
	}

	// we remove default input pins that aren't supposed to be there.
	for(UEdGraphPin* InputPin : InputPins)
	{
		if(IsParameterMapPin(InputPin) || InputPin->bOrphanedPin)
		{
			continue;
		}
		
		if(!CorrectParameterDefaultPins.Contains(InputPin))
		{
			RemovePin(InputPin);
		}
	}
}


void UNiagaraNodeParameterMapGet::SynchronizeDefaultInputPin(UEdGraphPin* DefaultPin)
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	if (!DefaultPin || DefaultPin->bOrphanedPin)
	{
		return;
	}

	UEdGraphPin* OutputPin = GetOutputPinForDefault(DefaultPin);
	if(!ensureAlwaysMsgf(OutputPin != nullptr, TEXT("There should always be an output pin found for every default pin we attempt to synchronize")))
	{
		return;
	}
	
	FNiagaraVariable Var = Schema->PinToNiagaraVariable(OutputPin);
	if (FNiagaraParameterUtilities::IsEngineParameter(Var))
	{
		DefaultPin->bDefaultValueIsIgnored = true;
		DefaultPin->bNotConnectable = true;
		DefaultPin->bHidden = true;
		DefaultPin->PinToolTip = FText::Format(LOCTEXT("DefaultValueTooltip_DisabledForEngineParameters", "Default value for {0}. Disabled for Engine Parameters."), FText::FromName(OutputPin->PinName)).ToString();
	}
	else
	{
		DefaultPin->bHidden = false;
		DefaultPin->PinToolTip = FText::Format(LOCTEXT("DefaultValueTooltip_UnlessOverridden", "Default value for {0} if no other module has set it previously in the stack."), FText::FromName(OutputPin->PinName)).ToString();
	}

	// Sync pin visibility with the configured default mode
	if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(GetGraph()))
	{
		if (TOptional<ENiagaraDefaultMode> DefaultMode = Graph->GetDefaultMode(Var))
		{
			switch (*DefaultMode)
			{
				case ENiagaraDefaultMode::Value:
				{
					DefaultPin->bNotConnectable = true;
					DefaultPin->bDefaultValueIsReadOnly = true;
					break;
				}
				case ENiagaraDefaultMode::Custom:
				{
					DefaultPin->bNotConnectable = false;
					DefaultPin->bDefaultValueIsReadOnly = false;
					break;
				}
				case ENiagaraDefaultMode::Binding:
				case ENiagaraDefaultMode::FailIfPreviouslyNotSet:
				{
					DefaultPin->bHidden = true;
					break;
				}
			}
		}
	}
}

FText UNiagaraNodeParameterMapGet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UNiagaraNodeParameterMapGetName", "Map Get");
}

void UNiagaraNodeParameterMapGet::AddOrphanedPinPairGuids(UEdGraphPin* OutputPin, UEdGraphPin* DefaultPin)
{
	if(OutputPin && DefaultPin)
	{
		ensure(OutputPin->bOrphanedPin && DefaultPin->bOrphanedPin && OutputPin->PersistentGuid.IsValid() && DefaultPin->PersistentGuid.IsValid());
		PinOutputToPinDefaultPersistentId.Add(OutputPin->PersistentGuid, DefaultPin->PersistentGuid);
	}
}

void UNiagaraNodeParameterMapGet::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	if (bRecursive)
	{
		OutHistory.VisitInputPin(GetInputPin(0), bFilterForCompilation);
	}

	if (!IsNodeEnabled() && OutHistory.GetIgnoreDisabled())
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	if (GetInputPin(0)->LinkedTo.Num() != 0)
	{
		ParamMapIdx = OutHistory.TraceParameterMapOutputPin((GetInputPin(0)->LinkedTo[0]));
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		uint32 NodeIdx = OutHistory.BeginNodeVisitation(ParamMapIdx, this);
		FPinCollectorArray OutputPins;
		GetOutputPins(OutputPins);
		for (int32 i = 0; i < OutputPins.Num(); i++)
		{
			if (IsAddPin(OutputPins[i]))
			{
				continue;
			}

			bool bUsedDefaults = false;
			if (bRecursive)
			{
				OutHistory.HandleVariableRead(ParamMapIdx, OutputPins[i], true, GetDefaultPin(OutputPins[i]), bFilterForCompilation, bUsedDefaults);
			}
			else
			{
				OutHistory.HandleVariableRead(ParamMapIdx, OutputPins[i], true, nullptr, bFilterForCompilation, bUsedDefaults);
			}
		}
		OutHistory.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}
}

void UNiagaraNodeParameterMapGet::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());

	FPinCollectorArray InputPins;
	GetCompilationInputPins(InputPins);
	FPinCollectorArray OutputPins;
	GetCompilationOutputPins(OutputPins);

	// Initialize the outputs to invalid values.
	check(Outputs.Num() == 0);
	Outputs.Init(INDEX_NONE, OutputPins.Num());

	// First compile fully down the hierarchy for our predecessors..
	TArray<int32, TInlineAllocator<16>> CompileInputs;
	CompileInputs.Reserve(InputPins.Num());

	for (int32 i = 0; i < InputPins.Num(); i++)
	{
		UEdGraphPin* InputPin = InputPins[i];

		if (InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType || 
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType ||
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum || 
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum )
		{
			int32 CompiledInput = INDEX_NONE;
			if (i == 0) // Only the zeroth item is not an default value pin.
			{
				CompiledInput = Translator->CompileInputPin(InputPin);
				if (CompiledInput == INDEX_NONE)
				{
					Translator->Error(LOCTEXT("InputError", "Error compiling input for param map get node."), this, InputPin);
				}
			}
			CompileInputs.Add(CompiledInput);
		}
	}

	UNiagaraGraph* Graph = Cast<UNiagaraGraph>(GetGraph());
	// By this point, we've visited all of our child nodes in the call graph. We can mine them to find out everyone contributing to the parameter map (and when).
	if (GetInputPin(0) != nullptr && GetInputPin(0)->LinkedTo.Num() > 0)
	{
		Translator->ParameterMapGet(this, CompileInputs, Outputs);
	}
}

bool UNiagaraNodeParameterMapGet::CancelEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj)
{
	if (InGraphPinObj == PinPendingRename)
	{
		PinPendingRename = nullptr;
	}
	return true;
}

bool UNiagaraNodeParameterMapGet::CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents)
{
	if (InGraphPinObj == PinPendingRename)
	{
		PinPendingRename = nullptr;
	}

	if (Pins.Contains(InGraphPinObj) && InGraphPinObj->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		FString OldPinName = InGraphPinObj->PinName.ToString();
		FString NewPinName = InName.ToString();

		// Early out if the same!
		if (OldPinName == NewPinName)
		{
			return true;
		}

		FScopedTransaction AddNewPinTransaction(LOCTEXT("Rename Pin", "Renamed pin"));
		Modify();
		InGraphPinObj->Modify();

		if (bSuppressEvents == false)
		{
			// we refresh the parameter references before changing the pin name, otherwise we might run into weird edge cases where a dirty graph tries to rename the wrong pin or doesn't copy the metadata correctly
			GetNiagaraGraph()->ConditionalRefreshParameterReferences();
		}

		InGraphPinObj->PinName = *InName.ToString();
		InGraphPinObj->PinFriendlyName = InName;

		if (!bSuppressEvents)
		{
			OnPinRenamed(InGraphPinObj, OldPinName);
		}

		return true;
	}
	return false;
}

void UNiagaraNodeParameterMapGet::GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const
{
	// If we are referencing any parameter collections, we need to register them here... might want to speeed this up in the future 
	// by caching any parameter collections locally.
	for (const UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Input)
		{
			continue;
		}

		if (IsAddPin(Pin))
		{
			continue;
		}

		FNameBuilder PinName(Pin->PinName);
		if (!PinName.ToView().StartsWith(FNiagaraConstants::ParameterCollectionNamespaceString))
		{
			continue;
		}

		const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());

		FNiagaraVariable Var = Schema->PinToNiagaraVariable(Pin);
		UNiagaraParameterCollection* Collection = Schema->VariableIsFromParameterCollection(Var);
		if (Collection)
		{
			FNiagaraCompileHash Hash = Collection->GetCompileHash();
			HashCollector.AddHash(Hash, Collection->GetPathName());
		}
	}
}

void UNiagaraNodeParameterMapGet::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	FText Text;
	if (GetTooltipTextForKnownPin(Pin, Text))
	{
		HoverTextOut = Text.ToString();
		return;
	}
	
	// Get hover text from metadata description.
	const UNiagaraGraph* NiagaraGraph = GetNiagaraGraph();
	if (NiagaraGraph)
	{
		const UEdGraphSchema_Niagara* Schema = Cast<UEdGraphSchema_Niagara>(NiagaraGraph->GetSchema());

		if (Schema)
		{
			FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(&Pin);
			if (IsAddPin(&Pin))
			{
				HoverTextOut = LOCTEXT("ParameterMapAddString", "Request a new variable from the parameter map.").ToString();
				return;
			}

			if (Pin.Direction == EEdGraphPinDirection::EGPD_Input)
			{
				if (&Pin == GetInputPin(0) && TypeDef == FNiagaraTypeDefinition::GetParameterMapDef())
				{
					HoverTextOut = LOCTEXT("ParameterMapInString", "The source parameter map where we pull the values from.").ToString();
					return;
				}
				const UEdGraphPin* OutputPin = GetOutputPinForDefault(&Pin);
				if (OutputPin)
				{
					FText Desc = FText::Format(LOCTEXT("DefaultValueTooltip", "Default value for \"{0}\" if no other module has set it previously in the stack.\nPlease edit this value by selecting in the parameters panel and editing in the details panel."), FText::FromName(OutputPin->PinName));
					HoverTextOut = Desc.ToString();
					return;
				}
			}
			else
			{
				FNiagaraVariable Var = FNiagaraVariable(TypeDef, Pin.PinName);
				TOptional<FNiagaraVariableMetaData> Metadata = NiagaraGraph->GetMetaData(Var);

				if (Metadata.IsSet())
				{
					FText Description = Metadata->Description;
					const FText TooltipFormat = LOCTEXT("Parameters", "Name: {0} \nType: {1}\nDescription: {2}");
					const FText Name = FText::FromName(Var.GetName());
					FName CachedParamName;

					const FText ToolTipText = FText::Format(TooltipFormat, FText::FromName(Var.GetName()), Var.GetType().GetNameText(), Description);
					HoverTextOut = ToolTipText.ToString();
				}
				else
				{
					HoverTextOut = TEXT("");
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

