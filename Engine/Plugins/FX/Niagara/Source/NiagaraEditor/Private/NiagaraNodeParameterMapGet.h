// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraNodeParameterMapGet.generated.h"

class UEdGraphPin;


/** A node that allows a user to get multiple values from a parameter map. */
UCLASS()
class UNiagaraNodeParameterMapGet : public UNiagaraNodeParameterMapBase
{
public:
	GENERATED_BODY()

	UNiagaraNodeParameterMapGet();

	virtual void AllocateDefaultPins() override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;

	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	virtual bool IsPinNameEditable(const UEdGraphPin* GraphPinObj) const override;
	virtual bool IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const override;
	virtual bool VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const override;
	virtual bool CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents)  override;
	virtual bool CancelEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj) override;

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;

	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	void AddOrphanedPinPairGuids(UEdGraphPin* OutputPin, UEdGraphPin* DefaultPin);
	
	/** Get the default value input pin for one of the output pins specified.*/
	UEdGraphPin* GetDefaultPin(const UEdGraphPin* OutputPin) const;

	void SynchronizeDefaultPins();
	
	virtual void PostLoad() override;

	virtual void GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;

	virtual FName GetNewPinDefaultNamespace() const override { return PARAM_MAP_MODULE_STR; }

	/** Convenience method to determine whether this Node is a Map Get or Map Set when adding a parameter through the parameter panel. */
	virtual EEdGraphPinDirection GetPinDirectionForNewParameters() override { return EEdGraphPinDirection::EGPD_Output; };

protected:
	virtual bool CanModifyPin(const UEdGraphPin* Pin) const override;
	
	virtual void OnNewTypedPinAdded(UEdGraphPin*& NewPin) override;
	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName) override;
	virtual void OnPinRemoved(UEdGraphPin* InRemovedPin) override;
	
	/** Synchronize the removal of the output pin with its default.*/
	virtual void RemoveDynamicPin(UEdGraphPin* Pin) override;

	/** Make sure that descriptions match up as well as any other value that can be changed.*/
	void SynchronizeDefaultInputPin(UEdGraphPin* DefaultPin);
	
	/** Reverse the lookup from GetDefaultPin.*/
	UEdGraphPin* GetOutputPinForDefault(const UEdGraphPin* DefaultPin) const;
	
	/** Properly set up the default input pin for an output pin.*/
	UEdGraphPin* CreateDefaultPin(UEdGraphPin* OutputPin);

	UPROPERTY(meta = (SkipForCompileHash="true"))
	TMap<FGuid, FGuid> PinOutputToPinDefaultPersistentId;
};
