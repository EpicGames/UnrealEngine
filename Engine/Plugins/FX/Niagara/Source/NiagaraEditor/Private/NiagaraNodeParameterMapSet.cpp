// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeParameterMapSet.h"

#include "Algo/RemoveIf.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptVariable.h"
#include "ScopedTransaction.h"
#include "Widgets/SNiagaraGraphParameterMapSetNode.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeParameterMapSet)

#define LOCTEXT_NAMESPACE "NiagaraNodeParameterMapSet"

UNiagaraNodeParameterMapSet::UNiagaraNodeParameterMapSet() : UNiagaraNodeParameterMapBase()
{

}

void UNiagaraNodeParameterMapSet::AllocateDefaultPins()
{
	PinPendingRename = nullptr;
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), *UNiagaraNodeParameterMapBase::SourcePinName.ToString());
	CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), *UNiagaraNodeParameterMapBase::DestPinName.ToString());
	CreateAddPin(EGPD_Input);
}

TSharedPtr<SGraphNode> UNiagaraNodeParameterMapSet::CreateVisualWidget()
{
	return SNew(SNiagaraGraphParameterMapSetNode, this);
}

bool UNiagaraNodeParameterMapSet::IsPinNameEditable(const UEdGraphPin* GraphPinObj) const
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraTypeDefinition TypeDef = Schema->PinToTypeDefinition(GraphPinObj);
	return TypeDef.IsValid() && GraphPinObj && GraphPinObj->Direction == EGPD_Input && CanRenamePin(GraphPinObj);
}

bool UNiagaraNodeParameterMapSet::IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const
{
	return GraphPinObj == PinPendingRename;
}

bool UNiagaraNodeParameterMapSet::VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const
{
	if (InName.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("InvalidName", "Invalid pin name");
		return false;
	}

	UNiagaraScriptVariable* ExistingScriptVariable = GetNiagaraGraph()->GetScriptVariable(FName(InName.ToString()));
	if(ExistingScriptVariable != nullptr)
	{
		// if the variable already exists and the type matches, we are trying to reference an existing parameter which is allowed.
		if(ExistingScriptVariable->Variable.GetType() == UEdGraphSchema_Niagara::PinToTypeDefinition(InGraphPinObj))
		{
			return true;
		}
		else
		{
			OutErrorMessage = LOCTEXT("InvalidName_VariableExistsDifferentType", "This variable already exists with a different type.\nChoose another name.");
			return false;
		}
	}
	
	return true;
}

void UNiagaraNodeParameterMapSet::OnNewTypedPinAdded(UEdGraphPin*& NewPin)
{
	if (HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedInitialization))
	{
		return;
	}

	if (NewPin->Direction == EEdGraphPinDirection::EGPD_Input)
	{
		FPinCollectorArray InputPins;
		GetInputPins(InputPins);
		
		// Determine if this is already namespaced or not. We need to do things differently below if not.  Also use the friendly
		// name to build the new parameter name since it's what is displayed in the UI.
		FName NewPinName = NewPin->PinFriendlyName.IsEmpty() == false 
			? *NewPin->PinFriendlyName.ToString()
			: NewPin->GetFName();

		bool bCreatedNamespace = false;
		FName PinNameWithoutNamespace;
		if (FNiagaraEditorUtilities::DecomposeVariableNamespace(NewPinName, PinNameWithoutNamespace).Num() == 0)
		{
			NewPinName = *(PARAM_MAP_LOCAL_MODULE_STR +  NewPinName.ToString());
			bCreatedNamespace = true;
		}

		TSet<FName> Names;
		Names.Reserve(InputPins.Num());
		for (const UEdGraphPin* Pin : InputPins)
		{
			if (Pin != NewPin)
			{
				Names.Add(Pin->GetFName());
			}
		}
		const FName NewUniqueName = FNiagaraUtilities::GetUniqueName(NewPinName, Names);

		//GetDefault<UEdGraphSchema_Niagara>()->PinToNiagaraVariable()
		NewPin->PinName = NewUniqueName;
		NewPin->PinFriendlyName = FText::AsCultureInvariant(NewPin->PinName.ToString());
		NewPin->PinType.PinSubCategory = UNiagaraNodeParameterMapBase::ParameterPinSubCategory;
		
		// If dragging from a function or other non-namespaced parent node, we should 
		// make a local variable to contain the value by default.
		if (bCreatedNamespace && GetNiagaraGraph())
		{
			const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
			constexpr bool bNeedsValue = false;
			FNiagaraVariable PinVariable = Schema->PinToNiagaraVariable(NewPin, bNeedsValue);
			constexpr bool bIsStaticSwitch = false;
			GetNiagaraGraph()->AddParameter(PinVariable, bIsStaticSwitch);
			NewPin->PinName = PinVariable.GetName();
		}
	}

	if (!NewPin->PersistentGuid.IsValid())
	{
		NewPin->PersistentGuid = FGuid::NewGuid();
	}

	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraVariable Var = Schema->PinToNiagaraVariable(NewPin);
	if (!FNiagaraConstants::IsNiagaraConstant(Var))
		PinPendingRename = NewPin;
	else
		PinPendingRename = nullptr;
	
}

void UNiagaraNodeParameterMapSet::OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName)
{
	UNiagaraNodeParameterMapBase::OnPinRenamed(RenamedPin, OldName);
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
}

bool UNiagaraNodeParameterMapSet::CancelEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj)
{
	if (InGraphPinObj == PinPendingRename)
	{
		PinPendingRename = nullptr;
	}
	return true;
}

bool UNiagaraNodeParameterMapSet::CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents)
{
	if (InGraphPinObj == PinPendingRename)
	{
		PinPendingRename = nullptr;
	}

	if (Pins.Contains(InGraphPinObj))
	{
		
		FString OldPinName = InGraphPinObj->PinName.ToString();
		FString NewPinName = InName.ToString();

		// Early out if the same!
		if (OldPinName == NewPinName)
		{
			return true;
		}

		FScopedTransaction AddNewPinTransaction(LOCTEXT("Rename Pin", "Renamed pin"));
		Modify();
		InGraphPinObj->Modify();

		if (bSuppressEvents == false)
		{
			// we refresh the parameter references before changing the pin name, otherwise we might run into weird edge cases where a dirty graph tries to rename the wrong pin or doesn't copy the metadata correctly
			GetNiagaraGraph()->ConditionalRefreshParameterReferences();
		}

		InGraphPinObj->PinName = *NewPinName;
		InGraphPinObj->PinFriendlyName = InName;
		if (bSuppressEvents == false)
		{
			OnPinRenamed(InGraphPinObj, OldPinName);
		}

		return true;
	}
	return false;
}

void UNiagaraNodeParameterMapSet::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	FPinCollectorArray InputPins;
	GetCompilationInputPins(InputPins);

	FPinCollectorArray OutputPins;
	GetCompilationOutputPins(OutputPins);

	// Initialize the outputs to invalid values.
	check(Outputs.Num() == 0);
	Outputs.Init(INDEX_NONE, OutputPins.Num());

	// update the translator with the culled function calls before compiling any further
	for (UEdGraphPin* InputPin : InputPins)
	{
		if (Translator->IsFunctionVariableCulledFromCompilation(InputPin->PinName))
		{
			Translator->CullMapSetInputPin(InputPin);
		}
	}

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());

	// First compile fully down the hierarchy for our predecessors..
	TArray<FTranslator::FCompiledPin, TInlineAllocator<16>> CompileInputs;
	CompileInputs.Reserve(InputPins.Num());
	for (UEdGraphPin* InputPin : InputPins)
	{
		if (SkipPinCompilation(InputPin) || Translator->IsFunctionVariableCulledFromCompilation(InputPin->PinName))
		{
			continue;
		}

		if (IsNodeEnabled() == false && Schema->PinToTypeDefinition(InputPin) != FNiagaraTypeDefinition::GetParameterMapDef())
		{
			continue;
		}

		int32 CompiledInput = Translator->CompileInputPin(InputPin);
		if (CompiledInput == INDEX_NONE)
		{
			Translator->Error(LOCTEXT("InputError", "Error compiling input for set node."), this, InputPin);
		}
		CompileInputs.Emplace(CompiledInput, InputPin);
	}

	if (GetInputPin(0) != nullptr && GetInputPin(0)->LinkedTo.Num() > 0)
	{
		Translator->ParameterMapSet(this, CompileInputs, Outputs);
	}
}

FText UNiagaraNodeParameterMapSet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UNiagaraNodeParameterMapSetName", "Map Set");
}


void UNiagaraNodeParameterMapSet::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	
	int32 ParamMapIdx = INDEX_NONE;
	uint32 NodeIdx = INDEX_NONE;

	// filter out any AddPins
	InputPins.SetNum(Algo::StableRemoveIf(InputPins, [&](UEdGraphPin* InPin)
	{
		return IsAddPin(InPin);
	}));

	for (UEdGraphPin* InputPin : InputPins)
	{
		OutHistory.VisitInputPin(InputPin, bFilterForCompilation);
	}

	if (IsNodeEnabled() || !OutHistory.GetIgnoreDisabled())
	{
		for (int32 i = 0; i < InputPins.Num(); i++)
		{
			UEdGraphPin* InputPin = InputPins[i];

			FNiagaraTypeDefinition VarTypeDef = Schema->PinToTypeDefinition(InputPin);
			if (i == 0 && InputPin != nullptr && VarTypeDef == FNiagaraTypeDefinition::GetParameterMapDef())
			{
				UEdGraphPin* PriorParamPin = nullptr;
				if (InputPin->LinkedTo.Num() > 0)
				{
					PriorParamPin = InputPin->LinkedTo[0];
				}

				// Now plow into our ancestor node
				if (PriorParamPin)
				{
					ParamMapIdx = OutHistory.TraceParameterMapOutputPin(PriorParamPin);
					NodeIdx = OutHistory.BeginNodeVisitation(ParamMapIdx, this);
				}
			}
			else if (i > 0 && InputPin != nullptr && ParamMapIdx != INDEX_NONE)
			{
				OutHistory.HandleVariableWrite(ParamMapIdx, InputPin);
			}
		}
	}
	else
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		OutHistory.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}

	OutHistory.RegisterParameterMapPin(ParamMapIdx, GetOutputPin(0));
}

void UNiagaraNodeParameterMapSet::PostLoad()
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (!Pin->PersistentGuid.IsValid())
		{
			Pin->PersistentGuid = FGuid::NewGuid();
		}

		if (Pin->Direction == EEdGraphPinDirection::EGPD_Input && Pin->GetFName() != UNiagaraNodeParameterMapBase::SourcePinName && !IsAddPin(Pin) && !Pin->bOrphanedPin)
		{
			Pin->PinType.PinSubCategory = UNiagaraNodeParameterMapBase::ParameterPinSubCategory;
		}
	}
	Super::PostLoad();
}

#undef LOCTEXT_NAMESPACE

