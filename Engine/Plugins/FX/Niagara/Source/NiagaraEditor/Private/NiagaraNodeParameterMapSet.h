// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraNodeParameterMapSet.generated.h"

class UEdGraphPin;


/** A node that allows a user to set multiple values into a parameter map. */
UCLASS()
class UNiagaraNodeParameterMapSet : public UNiagaraNodeParameterMapBase
{
public:
	GENERATED_BODY()

	UNiagaraNodeParameterMapSet();

	virtual void AllocateDefaultPins() override;

	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual bool IsPinNameEditable(const UEdGraphPin* GraphPinObj) const override;
	virtual bool IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const override;
	virtual bool VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const override;
	virtual bool CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents = false)  override;
	virtual bool CancelEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj) override;

	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	virtual bool IncludeParentNodeContextMenu() const override { return false; }
	virtual void PostLoad() override;
	virtual FName GetNewPinDefaultNamespace() const override { return PARAM_MAP_LOCAL_MODULE_STR; }

	/** Convenience method to determine whether this Node is a Map Get or Map Set when adding a parameter through the parameter panel. */
	virtual EEdGraphPinDirection GetPinDirectionForNewParameters() override { return EEdGraphPinDirection::EGPD_Input; };

protected:
	virtual void OnNewTypedPinAdded(UEdGraphPin*& NewPin) override; 
	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName) override;
	virtual bool SkipPinCompilation(UEdGraphPin* Pin) const { return false; };
};
