// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeReadDataSet.h"
#include "UObject/UnrealType.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraEvents.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeReadDataSet)

#define LOCTEXT_NAMESPACE "NiagaraNodeDataSetRead"

UNiagaraNodeReadDataSet::UNiagaraNodeReadDataSet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void UNiagaraNodeReadDataSet::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	if (DataSet.Type == ENiagaraDataSetType::Event)
	{
		//Probably need this for all data set types tbh!
		//UEdGraphPin* Pin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetBoolDef()), TEXT("Valid"));
		//Pin->bDefaultValueIsIgnored = true;
	}

	AddParameterMapPins();

	bool useFriendlyNames = (VariableFriendlyNames.Num() == Variables.Num());
	for (int32 i = 0; i < Variables.Num(); i++)
	{
		FNiagaraVariable& Var = Variables[i];
		UEdGraphPin* NewPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(Var.GetType()), Var.GetName());
		if (useFriendlyNames && VariableFriendlyNames[i].IsEmpty() == false)
		{
			NewPin->PinFriendlyName = FText::AsCultureInvariant(VariableFriendlyNames[i]);
		}
	}
}

FText UNiagaraNodeReadDataSet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(LOCTEXT("NiagaraDataSetReadFormat", "{0} Read"), FText::FromName(DataSet.Name));
}

bool UNiagaraNodeReadDataSet::CanAddToGraph(UNiagaraGraph* TargetGraph, FString& OutErrorMsg) const
{
	if (Super::CanAddToGraph(TargetGraph, OutErrorMsg) == false)
	{
		return false;
	}
	// Gather up all the referenced graphs of the one we are about to be added to.
	TArray<const UNiagaraGraph*> Graphs;
	TargetGraph->GetAllReferencedGraphs(Graphs);

	// Iterate over each graph and check to see if that graph has a UNiagaraNodeReadDataSet node. We *only* allow 
	// read nodes for graphs that have identical payloads and can therefore be coalesced into the same event instance.
	for (const UNiagaraGraph* Graph : Graphs)
	{
		TArray<UNiagaraNodeReadDataSet*> ReadNodes;
		Graph->GetNodesOfClass(ReadNodes);
		for (UNiagaraNodeReadDataSet* ReadNode : ReadNodes)
		{
			check(ReadNode);
			if (ReadNode == this)
			{
				continue;
			}

			bool bMatches = true;
			if (ReadNode->Variables.Num() != Variables.Num())
			{
				bMatches = false;
			}
			else
			{
				for (int32 i = 0; i < Variables.Num(); i++)
				{
					if (false == ReadNode->Variables[i].IsEquivalent(Variables[i]))
					{
						bMatches = false;
						break;
					}
				}
			}

			if (!bMatches)
			{
				OutErrorMsg = FText::Format(LOCTEXT("NiagaraDataSetReadCannotAddToGraph", "Cannot add to graph because Graph '{0}' already has an Event Read node of different type '{1}'."),
					Graph->GetOutermost() ? FText::FromString(Graph->GetOutermost()->GetPathName()) : FText::FromString(FString("nullptr")), FText::FromName(ReadNode->DataSet.Name)).ToString();
				return false;
			}
		}
	}
	return true;
}

void UNiagaraNodeReadDataSet::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	TArray<int32> Inputs;
	CompileInputPins(Translator, Inputs);
	
	bool bGPUSim = Translator->IsCompileOptionDefined(TEXT("GPUComputeSim"));

	if (bGPUSim)
	{
		Translator->Error(LOCTEXT("CannotRunReadDataSetGPU", "Cannot use an event read node on GPU sims!"), this, nullptr);
	}

	FString IssuesWithStruct;
	if (!IsSynchronizedWithStruct(false, &IssuesWithStruct, false))
	{
		Translator->Error(FText::FromString(IssuesWithStruct), this, nullptr);
	}
	Translator->ReadDataSet(DataSet, Variables, ENiagaraDataSetAccessMode::AppendConsume, Inputs[0], Outputs);
}

void UNiagaraNodeReadDataSet::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	if (bRecursive)
	{
		OutHistory.VisitInputPins(this, bFilterForCompilation);
	}

	if (!IsNodeEnabled() && OutHistory.GetIgnoreDisabled())
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	if (GetInputPin(0)->LinkedTo.Num() != 0)
	{
		ParamMapIdx = OutHistory.TraceParameterMapOutputPin(UNiagaraNode::TraceOutputPin(GetInputPin(0)->LinkedTo[0]));
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		uint32 NodeIdx = OutHistory.BeginNodeVisitation(ParamMapIdx, this);
		OutHistory.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}

	OutHistory.RegisterParameterMapPin(ParamMapIdx, GetOutputPin(0));
}

#undef LOCTEXT_NAMESPACE




