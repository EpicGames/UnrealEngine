// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeReroute.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeReroute)

#define LOCTEXT_NAMESPACE "NiagaraNodeReroute"

UNiagaraNodeReroute::UNiagaraNodeReroute(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNiagaraNodeReroute::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// @TODO why do we need to have this post-change property here at all? 
	// Doing a null check b/c otherwise if doing a Duplicate via Ctrl-W, we die inside AllocateDefaultPins due to 
	// the point where we get this call not being completely formed.
	if (PropertyChangedEvent.Property != nullptr)
	{
		ReallocatePins();
	}
}


ENiagaraNumericOutputTypeSelectionMode UNiagaraNodeReroute::GetNumericOutputTypeSelectionMode() const
{
	return ENiagaraNumericOutputTypeSelectionMode::Largest;
}

bool UNiagaraNodeReroute::AllowExternalPinTypeChanges(const UEdGraphPin* InGraphPin) const
{
	return true;
}

void UNiagaraNodeReroute::PostLoad()
{
	Super::PostLoad();
}

void UNiagaraNodeReroute::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	
	FEdGraphPinType Type = Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetWildcardDef());
	UEdGraphPin* MyInputPin = CreatePin(EGPD_Input, Type, FNiagaraConstants::InputPinName);
	MyInputPin->bDefaultValueIsIgnored = true;

	CreatePin(EGPD_Output, Type, FNiagaraConstants::OutputPinName);
}

FText UNiagaraNodeReroute::GetTooltipText() const
{
	return FText::GetEmpty();
}

FText UNiagaraNodeReroute::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::EditableTitle)
	{
		return FText::FromString(NodeComment);
	}

	return LOCTEXT("RerouteNodeTitle", "Reroute Node");
}

bool UNiagaraNodeReroute::ShouldOverridePinNames() const
{
	return true;
}

FText UNiagaraNodeReroute::GetPinNameOverride(const UEdGraphPin& Pin) const
{
	// Keep the pin size tiny
	return FText::GetEmpty();
}

void UNiagaraNodeReroute::OnRenameNode(const FString& NewName)
{
	NodeComment = NewName;
}


bool UNiagaraNodeReroute::CanSplitPin(const UEdGraphPin* Pin) const
{
	return false;
}

UEdGraphPin* UNiagaraNodeReroute::GetPassThroughPin(const UEdGraphPin* FromPin) const
{
	if (FromPin && Pins.Contains(FromPin))
	{
		return FromPin == Pins[0] ? Pins[1] : Pins[0];
	}

	return nullptr;
}

bool UNiagaraNodeReroute::ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const
{
	OutInputPinIndex = 0;
	OutOutputPinIndex = 1;
	return true;
}

void UNiagaraNodeReroute::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	FPinCollectorArray InputPins;
	GetCompilationInputPins(InputPins);

	FPinCollectorArray OutputPins;
	GetCompilationOutputPins(OutputPins);

	// Initialize the outputs to invalid values.
	check(Outputs.Num() == 0);
	Outputs.Reserve(OutputPins.Num());
	for (int32 i = 0; i < OutputPins.Num(); i++)
	{
		if (InputPins.IsValidIndex(i))
		{
			int32 CompiledInput = Translator->CompileInputPin(InputPins[i]);
			Outputs.Add(CompiledInput);
		}
		else
		{
			Outputs.Add(INDEX_NONE);
		}
	}

	if (InputPins.Num() != OutputPins.Num())
	{
		Translator->Error(LOCTEXT("IncorrectNumOutputsError", "Input and Output pin counts must match."), this, nullptr);
	}
}

bool UNiagaraNodeReroute::RefreshFromExternalChanges()
{
	ReallocatePins();
	PropagatePinType();
	return true;
}

void UNiagaraNodeReroute::PinConnectionListChanged(UEdGraphPin* Pin)
{
	PropagatePinType();
	Super::PinConnectionListChanged(Pin);
}

void UNiagaraNodeReroute::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	RegisterPassthroughPin(OutHistory, GetInputPin(0), GetOutputPin(0), bFilterForCompilation, true);
}

/** Traces one of this node's output pins to its source output pin if it is a reroute node output pin.*/
UEdGraphPin* UNiagaraNodeReroute::GetTracedOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bFilterForCompilation, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace) const
{
	check(Pins.Contains(LocallyOwnedOutputPin) && LocallyOwnedOutputPin->Direction == EGPD_Output);
	UEdGraphPin* InputPin = GetInputPin(0);
	if (InputPin && InputPin->LinkedTo.Num() == 1 && InputPin->LinkedTo[0] != nullptr)
	{
		UEdGraphPin* LinkedPin = InputPin->LinkedTo[0];
		UNiagaraNode* LinkedNode = CastChecked<UNiagaraNode>(LinkedPin->GetOwningNode());
		return LinkedNode->GetTracedOutputPin(LinkedPin, bFilterForCompilation, OutNodesVisitedDuringTrace);
	}
	return nullptr;
}

void UNiagaraNodeReroute::PropagatePinType()
{
	UEdGraphPin* MyInputPin = GetInputPin(0);
	UEdGraphPin* MyOutputPin = GetOutputPin(0);

	for (UEdGraphPin* Inputs : MyInputPin->LinkedTo)
	{	
		if (!UEdGraphSchema_Niagara::IsPinWildcard(Inputs))
		{
			PropagatePinTypeFromDirection(true);
			return;
		}
	}

	for (UEdGraphPin* Outputs : MyOutputPin->LinkedTo)
	{
		if(!UEdGraphSchema_Niagara::IsPinWildcard(Outputs))
		{
			PropagatePinTypeFromDirection(false);
			return;
		}
	}

	// if all inputs/outputs are wildcards, still favor the inputs first (propagate array/reference/etc. state)
	if (MyInputPin->LinkedTo.Num() > 0)
	{
		// If we can't mirror from output type, we should at least get the type information from the input connection chain
		PropagatePinTypeFromDirection(true);
	}
	else if (MyOutputPin->LinkedTo.Num() > 0)
	{
		// Try to mirror from output first to make sure we get appropriate member references
		PropagatePinTypeFromDirection(false);
	}
	else
	{
		const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());

		FEdGraphPinType WildcardPinType = Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetWildcardDef());
		// Revert to wildcard
		MyInputPin->BreakAllPinLinks();
		MyInputPin->PinType.ResetToDefaults();
		MyInputPin->PinType = WildcardPinType;

		MyOutputPin->BreakAllPinLinks();
		MyOutputPin->PinType.ResetToDefaults();
		MyOutputPin->PinType = WildcardPinType;
	}
}

void UNiagaraNodeReroute::PropagatePinTypeFromDirection(bool bFromInput)
{
	if (bRecursionGuard)
	{
		return;
	}
	// Set the type of the pin based on the source connection, and then percolate
	// that type information up until we no longer reach another Reroute node
	UEdGraphPin* MySourcePin = bFromInput ? GetInputPin(0) : GetOutputPin(0);
	UEdGraphPin* MyDestinationPin = bFromInput ? GetOutputPin(0) : GetInputPin(0);

	TGuardValue<bool> RecursionGuard(bRecursionGuard, true);

	// Make sure any source knot pins compute their type, this will try to call back
	// into this function but the recursion guard will stop it
	for (UEdGraphPin* InPin : MySourcePin->LinkedTo)
	{
		if (UNiagaraNodeReroute* KnotNode = Cast<UNiagaraNodeReroute>(InPin->GetOwningNode()))
		{
			KnotNode->PropagatePinTypeFromDirection(bFromInput);
		}
	}

	UEdGraphPin* TypeSource = MySourcePin->LinkedTo.Num() ? MySourcePin->LinkedTo[0] : nullptr;
	if (TypeSource)
	{
		MySourcePin->PinType = TypeSource->PinType;
		MyDestinationPin->PinType = TypeSource->PinType;

		for (UEdGraphPin* LinkPin : MyDestinationPin->LinkedTo)
		{
			if (UNiagaraNodeReroute* OwningNode = Cast<UNiagaraNodeReroute>(LinkPin->GetOwningNode()))
			{
				// Notify any pins in the destination direction
				if (UNiagaraNodeReroute* RerouteNode = Cast<UNiagaraNodeReroute>(OwningNode))
				{
					RerouteNode->PropagatePinTypeFromDirection(bFromInput);
				}
				else
				{
					OwningNode->PinConnectionListChanged(LinkPin);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

