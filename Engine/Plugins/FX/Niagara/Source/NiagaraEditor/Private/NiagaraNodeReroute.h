// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NiagaraNode.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraNodeReroute.generated.h"

UCLASS(MinimalAPI)
class UNiagaraNodeReroute: public UNiagaraNode
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldOverridePinNames() const override;
	virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual bool CanSplitPin(const UEdGraphPin* Pin) const override;
	virtual bool IsCompilerRelevant() const override { return false; }
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* FromPin) const override;
	virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override;
	//~ End EdGraphNode Interface

	//~ Begin UNiagaraNode Interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual bool RefreshFromExternalChanges() override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;
	virtual ENiagaraNumericOutputTypeSelectionMode GetNumericOutputTypeSelectionMode() const;
	virtual bool AllowExternalPinTypeChanges(const UEdGraphPin* InGraphPin) const override;
	
	/** Trace to an output pin that is not a reroute node output pin. If the reroute nodes resulted in a dead end and no output pin was found return nullptr.*/
	virtual UEdGraphPin* GetTracedOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bFilterForCompilation, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace = nullptr) const override;
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* LocallyOwnedOutputPin, ENiagaraScriptUsage InUsage) const override { return GetPassThroughPin(LocallyOwnedOutputPin); }
	//~ End UNiagaraNode Interface

	void PropagatePinType();

private:
	void PropagatePinTypeFromDirection(bool bFromInput);

	/** Recursion guard boolean to prevent PropagatePinType from infinitely recursing if you manage to create a loop of knots */
	bool bRecursionGuard;
};
