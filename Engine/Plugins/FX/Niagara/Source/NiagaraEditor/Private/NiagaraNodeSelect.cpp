// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeSelect.h"

#include "EditorFontGlyphs.h"
#include "FindInBlueprintManager.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraHlslTranslator.h"
#include "EdGraph/EdGraphPin.h"
#include "Widgets/SNiagaraPinTypeSelector.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSeparator.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Engine/UserDefinedEnum.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor/EditorEngine.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeSelect)

#define LOCTEXT_NAMESPACE "NiagaraNodeSelect"

UNiagaraNodeSelect::UNiagaraNodeSelect()
{
}

void UNiagaraNodeSelect::ChangeValuePinType(int32 Index, FNiagaraTypeDefinition Type)
{
	if(Index < 0 || Index >= OutputVars.Num())
	{
		UE_LOGF(LogNiagaraEditor, Error, "Tried assigning type %ls to index %i. Out of bounds.", *Type.GetName(), Index);
		return;
	}

	// swap output variables at the same location
	TSet<FName> OutputNames;
	for (const FNiagaraVariable& Output : OutputVars)
	{
		OutputNames.Add(Output.GetName());
	}
	FName OutputName = FNiagaraUtilities::GetUniqueName(Type.GetFName(), OutputNames);

	OutputVars.RemoveAt(Index);
	OutputVars.EmplaceAt(Index, FNiagaraVariable(Type, OutputName));
	
	for(UEdGraphPin* ValuePin : GetOptionPins(Index))
	{
		if(ValuePin->LinkedTo.Num() > 0 && !UEdGraphSchema_Niagara::IsPinWildcard(ValuePin))
		{
			ValuePin->bOrphanedPin = true;
		}
	}

	if(UEdGraphPin* OutputPin = GetPinByPersistentGuid(OutputVarGuids[Index]))
	{
		if(OutputPin->LinkedTo.Num() > 0 && !UEdGraphSchema_Niagara::IsPinWildcard(OutputPin))
		{
			OutputPin->bOrphanedPin = true;
		}
	}
	
	ReallocatePins(true);	
}

void UNiagaraNodeSelect::ChangeSelectorPinType(FNiagaraTypeDefinition Type)
{	
	SelectorPinType = Type;

	UEdGraphPin* SelectorPin = GetSelectorPin();

	if(OutputVars.Num() == 0)
	{
		AddOutput(FNiagaraTypeDefinition::GetWildcardDef(), FNiagaraTypeDefinition::GetWildcardDef().GetFName());
		ReallocatePins();
		return;
	}
	// ensure we have at least a minimum amount of entries if we change to integer mode
	const int32 MinimumEntries = 2;
	if (SelectorPinType.IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) && NumOptionsPerVariable < MinimumEntries)
	{
		NumOptionsPerVariable = MinimumEntries;
	}
	
	// if this is a wildcard, we don't want to orphan it
	if(SelectorPin->LinkedTo.Num() > 0 && !UEdGraphSchema_Niagara::IsPinWildcard(SelectorPin))
	{
		SelectorPin->bOrphanedPin = true;
	}
	
	ReallocatePins(true);
}

UEdGraphPin* UNiagaraNodeSelect::GetSelectorPin() const
{
	return GetPinByPersistentGuid(SelectorPinGuid);
}

TArray<UEdGraphPin*> UNiagaraNodeSelect::GetOptionPins(int32 Index) const
{
	TArray<UEdGraphPin*> OptionPins;
	TArray<UEdGraphPin*> InputPins;

	GetInputPins(InputPins);

	if (Index < 0 || Index >= OutputVars.Num())
	{
		UE_LOGF(LogNiagaraEditor, Error, "Tried retrieving option pins for index index %i. Out of bounds.", Index);
		return OptionPins;
	}
	
	// add all input pins that aren't the selector pin to the option pins array
	for(int32 PinIndex = Index; PinIndex < NumOptionsPerVariable * OutputVars.Num(); PinIndex += OutputVars.Num())
	{
		UEdGraphPin* Pin = InputPins[PinIndex];
		if (Pin->PersistentGuid != SelectorPinGuid && Pin->bOrphanedPin == false)
		{
			OptionPins.Add(Pin);
		}
	}

	if (NumOptionsPerVariable != OptionPins.Num())
	{
		UE_LOGF(LogNiagaraEditor, Error, "Could not retrieve %i option pins for output pin %i. Retrieved %i pins instead.", NumOptionsPerVariable, Index, OptionPins.Num());
		return {};
	}

	return OptionPins;
}

TArray<UEdGraphPin*> UNiagaraNodeSelect::GetValuePins(int32 Index) const
{
	TArray<UEdGraphPin*> ValuePins = GetOptionPins(Index);
	ValuePins.Add(GetPinByPersistentGuid(OutputVarGuids[Index]));

	return ValuePins;
}

UEdGraphPin* UNiagaraNodeSelect::GetOutputPin(const FNiagaraVariable& Variable) const
{
	int32 Index = OutputVars.Find(Variable);

	if(Index != INDEX_NONE)
	{
		return GetPinByPersistentGuid(OutputVarGuids[Index]);
	}

	return nullptr;
}

void UNiagaraNodeSelect::AddIntegerInputPin()
{
	FScopedTransaction Transaction(LOCTEXT("AddIntegerPinTransaction", "Added integer input pin to select node"));

	this->Modify();

	NumOptionsPerVariable++;
	ReallocatePins();
}

void UNiagaraNodeSelect::RemoveIntegerInputPin()
{
	FScopedTransaction Transaction(LOCTEXT("RemoveIntegerPinTransaction", "Removed integer input pin from select node"));

	this->Modify();

	NumOptionsPerVariable = FMath::Max(2, --NumOptionsPerVariable);
	ReallocatePins();
}

FText UNiagaraNodeSelect::GetTooltipText() const
{
	return LOCTEXT("SelectTooltipText", "Select takes multiple inputs and passes through the one chosen by the selector");
}

FText UNiagaraNodeSelect::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("SelectNodeTitle", "Select");
}

void UNiagaraNodeSelect::AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddPinSearchMetaDataInfo(Pin, OutTaggedMetaData);

	UEnum* Enum = SelectorPinType.GetEnum();
	
	if (Enum != nullptr && Enum->IsNative())
	{
		// Allow native enum switch pins to be searchable by C++ enum name
		OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_NativeName, FText::FromString(Enum->GetName())));
	}
}

void UNiagaraNodeSelect::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	// in case der selector type became invalid, we change it back to wildcard here.
	// this can happen if the underlying ustruct, for example an enum, has been deleted
	if(!SelectorPinType.IsValid())
	{
		SelectorPinType = FNiagaraTypeDefinition::GetWildcardDef();
	}
	
	TArray<int32> OptionValues = GetOptionValues();
	NumOptionsPerVariable = OptionValues.Num();

	for(int32 OptionIndex = 0; OptionIndex < OptionValues.Num(); OptionIndex++)
	{
		for(const FNiagaraVariable& Variable : OutputVars)
		{
			AddOptionPin(Variable, OptionValues[OptionIndex]);
		}
	}
	
	// create the selector pin
	UEdGraphPin* SelectorPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(SelectorPinType), GetSelectorPinName());
	SelectorPin->PersistentGuid = SelectorPinGuid;
	UNiagaraNode::SetPinDefaultToTypeDefaultIfUnset(SelectorPin);

	// create all output variable pins
	for (int32 Index = 0; Index < OutputVars.Num(); Index++)
	{
		UEdGraphPin* OutputPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(OutputVars[Index].GetType()), OutputVars[Index].GetName());
		OutputPin->PersistentGuid = OutputVarGuids[Index];
	}

	// create the add pin
	CreateAddPin(EGPD_Output);
}
	
FLinearColor UNiagaraNodeSelect::GetNodeTitleColor() const
{
	return FLinearColor(0.3f, 0.3f, 0.7f);
}

FSlateIcon UNiagaraNodeSelect::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Switch_16x");
	return Icon;
}

void UNiagaraNodeSelect::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	Super::GetPinHoverText(Pin, HoverTextOut);
	
	if(OutputVarGuids.Contains(Pin.PersistentGuid))
	{
		HoverTextOut = TEXT("The value of this pin is chosen by the selector pin.\n") + HoverTextOut;
	}
	else if(Pin.PersistentGuid == SelectorPinGuid)
	{
		HoverTextOut = TEXT("Based on the value of this pin one of the input pins gets selected for the output.\n") + HoverTextOut;
	}
}

void UNiagaraNodeSelect::PostLoad()
{
	Super::PostLoad();
	
	// we assume something changed externally if the input pins are outdated; i.e. the assigned enum changed or value order changed
	AttemptUpdatePins();
}

void UNiagaraNodeSelect::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (SelectorPinType.IsEnum() && SelectorPinType.GetEnum())
	{
		FToolMenuSection& Section = Menu->FindOrAddSection("Node");

		Section.AddMenuEntry(
			"BrowseToEnum",
			FText::Format(LOCTEXT("BrowseToEnumLabel", "Browse to {0}"), FText::FromString(SelectorPinType.GetEnum()->GetName())),
			LOCTEXT("BrowseToEnumTooltip", "Browses to the enum in the content browser."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser.Small"),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				FContentBrowserModule& Module = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				Module.Get().SyncBrowserToAssets({ FAssetData(SelectorPinType.GetEnum()) });
			}), FCanExecuteAction::CreateLambda([this]()
			{
				return Cast<UUserDefinedEnum>(SelectorPinType.GetEnum()) != nullptr;
			})));

		Section.AddMenuEntry(
			"OpenEnum",
			FText::Format(LOCTEXT("OpenEnumLabel", "Open {0}"), FText::FromString(SelectorPinType.GetEnum()->GetName())),
			LOCTEXT("OpenEnumTooltip", "Opens up the enum asset."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SelectorPinType.GetEnum());
			}), FCanExecuteAction::CreateLambda([this]()
			{
				return Cast<UUserDefinedEnum>(SelectorPinType.GetEnum()) != nullptr;
			})));
	}
}

void UNiagaraNodeSelect::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{	
	const UEdGraphPin* SelectorPin = GetSelectorPin();

	if(!SelectorPinType.IsValid())
	{
		Translator->Warning(LOCTEXT("SelectNodePinSelectorTypeInvalid", "Select node selector pin should have a valid type."), this, nullptr);
	}

	const int32 OutputVarCount = OutputVars.Num();
	TArray<int32> ConnectedOutputVarIndices;
	TArray<FNiagaraVariable> ConnectedOutputVars;
	ConnectedOutputVarIndices.Reserve(OutputVarCount);
	ConnectedOutputVars.Reserve(OutputVarCount);
	for (int32 OutputVarIndex = 0; OutputVarIndex < OutputVarCount; ++OutputVarIndex)
	{
		const FNiagaraVariable& OutputVar = OutputVars[OutputVarIndex];
		const UEdGraphPin* OutputPin = GetOutputPin(OutputVar);
		if (OutputPin && !OutputPin->LinkedTo.IsEmpty())
		{
			ConnectedOutputVarIndices.Add(OutputVarIndex);
			ConnectedOutputVars.Add(OutputVar);
		}
	}

	for (const FNiagaraVariable& Variable : ConnectedOutputVars)
	{
		if(!Variable.GetType().IsValid())
		{
			Translator->Warning(FText::Format(LOCTEXT("SelectNodePinOutputTypeInvalid", "Select node output pin should have a valid type. {0} is invalid."), FText::FromName(Variable.GetName())), this, nullptr);
		}
	}
	
	int32 Selection = Translator->CompileInputPin(SelectorPin);

	// a map from selector value to compiled option pins (i.e.: for selector value 0 all pins that should be case "if 0" get their compiled index added under key 0)
	TMap<int32, TArray<int32>> OptionValues;
	OptionValues.Reserve(NumOptionsPerVariable);

	TArray<int32> SelectorValues = GetOptionValues();
	
	if(SelectorValues.Num() != NumOptionsPerVariable)
	{
		Translator->Error(FText::Format(LOCTEXT("SelectNodePinCountMismatch", "Select node should have {0} cases. {1} found."),
			FText::FromString(FString::FromInt(NumOptionsPerVariable)), FText::FromString(FString::FromInt(SelectorValues.Num()))), this, nullptr);
		return;
	}

	// initialize every selector value so we can access the array values
	for (int32 Index = 0; Index < SelectorValues.Num(); Index++)
	{
		OptionValues.Add(SelectorValues[Index]);
	}

	for (const FNiagaraVariable& OutputVar : ConnectedOutputVars)
	{
		const UEdGraphPin* OutputPin = GetOutputPin(OutputVar);
		const FNiagaraTypeDefinition OutputPinType = UEdGraphSchema_Niagara::PinToTypeDefinition(OutputPin);

		if(OutputVar.GetType() != OutputPinType)
		{
			Translator->Error(FText::Format(LOCTEXT("PinTypeOutputVarTypeMismatch", "Internal output variable type {0} does not match pin type {1}. Please refresh node."),
				FText::FromString(OutputVar.GetType().GetName()), FText::FromString(OutputPinType.GetName())),
				this, OutputPin);
		}
		else if (UEdGraphSchema_Niagara::IsPinWildcard(OutputPin))
		{
			Translator->Error(FText::Format(LOCTEXT("PinTypeOutputWildcard", "Wildcard types are intended to be placeholders and replaced by real types.  Please resolve type for {0}."),
				FText::FromString(OutputVar.GetName().ToString())),
				this, OutputPin);
		}
	}
	
	for (int32 SelectorValueIndex = 0; SelectorValueIndex < NumOptionsPerVariable; SelectorValueIndex++)
	{
		for (int32 OutputVarIndex : ConnectedOutputVarIndices)
		{
			TArray<UEdGraphPin*> OptionPins = GetOptionPins(OutputVarIndex);
			int32 CodeChunkIndex = Translator->CompileInputPin(OptionPins[SelectorValueIndex]);
			OptionValues[SelectorValues[SelectorValueIndex]].Add(CodeChunkIndex);
		}
	}	

	TArray<int32> SelectOutputs;
	Translator->Select(this, Selection, ConnectedOutputVars, OptionValues, SelectOutputs);

	// remap the compiled outputs based on list of connected outputs
	Outputs.Init(INDEX_NONE, OutputVarCount);
	for (int32 ConnectedVarIndex = 0; ConnectedVarIndex < ConnectedOutputVars.Num(); ++ConnectedVarIndex)
	{
		Outputs[ConnectedOutputVarIndices[ConnectedVarIndex]] = SelectOutputs[ConnectedVarIndex];
	}
}

bool UNiagaraNodeSelect::AllowExternalPinTypeChanges(const UEdGraphPin* InGraphPin) const
{
	// only allow pin type changes from UI for selector and output pins
	if(InGraphPin == GetSelectorPin() || Super::AllowExternalPinTypeChanges(InGraphPin))
	{
		return true;
	}

	return false;
}

void UNiagaraNodeSelect::PinTypeChanged(UEdGraphPin* InGraphPin)
{
	int32 PinIndex = GetPinIndex(InGraphPin);
	if (PinIndex == -1)
	{
		return;
	}

	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	check(Schema);

	Super::PinTypeChanged(InGraphPin);
}

void UNiagaraNodeSelect::AddWidgetsToOutputBox(TSharedPtr<SVerticalBox> OutputBox)
{
	OutputBox->AddSlot()
	[
		SNew(SSpacer)
	];

	TAttribute<EVisibility> AddVisibilityAttribute;
	TAttribute<EVisibility> RemoveVisibilityAttribute;
	AddVisibilityAttribute.BindUObject(this, &UNiagaraNodeSelect::ShowAddIntegerButton);
	RemoveVisibilityAttribute.BindUObject(this, &UNiagaraNodeSelect::ShowRemoveIntegerButton);
	
	OutputBox->AddSlot()
	.Padding(4.f, 5.f)
	.AutoHeight()
	.HAlign(HAlign_Right)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.Visibility(RemoveVisibilityAttribute)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(GetIntegerRemoveButtonTooltipText())
			.OnPressed(FSimpleDelegate::CreateUObject(this, &UNiagaraNodeSelect::RemoveIntegerInputPin))
			[
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f, 0.9f))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.RemovePin"))
			]
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.Visibility(AddVisibilityAttribute)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(GetIntegerAddButtonTooltipText())
			.OnPressed(FSimpleDelegate::CreateUObject(this, &UNiagaraNodeSelect::AddIntegerInputPin))
			[
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f, 0.9f))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.AddPin"))
			]
		]
	];
}

void UNiagaraNodeSelect::AddWidgetsToInputBox(TSharedPtr<SVerticalBox> InputBox)
{
	// make sure we maintain our case labels and separators
	Super::AddWidgetsToInputBox(InputBox);

	// we only want to add a separator before the selector pin if we actually have at least 1 variable
	if (OutputVars.Num() < 1 || NumOptionsPerVariable < 1)
	{
		return;
	}
	
	InputBox->InsertSlot(OutputVars.Num() * NumOptionsPerVariable + 2 * NumOptionsPerVariable)
	.VAlign(VAlign_Center)
	[
		SNew(SSeparator)
	];
}

void UNiagaraNodeSelect::GetWildcardPinHoverConnectionTextAddition(const UEdGraphPin* WildcardPin, const UEdGraphPin* OtherPin, ECanCreateConnectionResponse ConnectionResponse, FString& OutString) const
{
	if(ConnectionResponse == ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW)
	{
		if (WildcardPin && WildcardPin->PersistentGuid.IsValid() && WildcardPin->PersistentGuid == SelectorPinGuid)
		{
			OutString += TEXT("\nThis wildcard pin only supports bool, integer or enum types.");
		}
		else
		{
			OutString += TEXT("\nThis wildcard pin only supports simple types (no numerics or data interfaces).");
		}
	}
}

bool UNiagaraNodeSelect::CanRemovePin(const UEdGraphPin* Pin) const
{
	return Super::CanRemovePin(Pin);
}

bool UNiagaraNodeSelect::CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const
{
	auto FindPredicate = [=](const FGuid& Guid) { return Guid == Pin->PersistentGuid; };
	int32 FoundIndex = OutputVarGuids.IndexOfByPredicate(FindPredicate);
	if (FoundIndex != INDEX_NONE && OutputVars.IsValidIndex(FoundIndex + DirectionToMove))
	{
		return Pin->Direction == EGPD_Output && Pin->bOrphanedPin == false;
	}

	return false;
}

void UNiagaraNodeSelect::MoveDynamicPin(UEdGraphPin* Pin, int32 DirectionToMove)
{
	auto FindPredicate = [=](const FGuid& Guid) { return Guid == Pin->PersistentGuid; };
	int32 FoundIndex = OutputVarGuids.IndexOfByPredicate(FindPredicate);
	if (FoundIndex != INDEX_NONE && OutputVars.IsValidIndex(FoundIndex + DirectionToMove))
	{
		this->Modify();
		
		FNiagaraVariable TmpVar = OutputVars[FoundIndex];
		OutputVars[FoundIndex] = OutputVars[FoundIndex + DirectionToMove];
		OutputVars[FoundIndex + DirectionToMove] = TmpVar;

		FGuid TmpGuid = OutputVarGuids[FoundIndex];
		OutputVarGuids[FoundIndex] = OutputVarGuids[FoundIndex + DirectionToMove];
		OutputVarGuids[FoundIndex + DirectionToMove] = TmpGuid;

		ReallocatePins();
	}	
}

bool UNiagaraNodeSelect::CanRenamePin(const UEdGraphPin* Pin) const
{
	return Super::CanRenamePin(Pin) && Pin->Direction == EGPD_Output && Pin->bOrphanedPin == false && !UEdGraphSchema_Niagara::IsPinWildcard(Pin);
}

FString UNiagaraNodeSelect::GetInputCaseName(int32 Case) const
{
	if( FNiagaraTypeDefinition::GetBoolDef().IsSameBaseDefinition(SelectorPinType))
	{
		if(Case == 0)
		{
			return TEXT("False");
		}
		if (Case == 1)
		{
			return TEXT("True");
		}
	}
	else if ( FNiagaraTypeDefinition::GetIntDef().IsSameBaseDefinition(SelectorPinType))
	{
		return FString::FromInt(Case);
	}
	else if(SelectorPinType.IsEnum() && SelectorPinType.GetEnum())
	{
		UEnum* Enum = SelectorPinType.GetEnum();
		// the display name is subject to localization and some automatic prettification. To avoid the localization aspect, we retrieve the source string of the text
		// which is essentially the still prettified non-localized base text. We have to keep it this way for backwards compatibility until we can do a full upgrade pass.
		// Same in StaticSwitch @todo
		FText EnumDisplayText = Enum->GetDisplayNameTextByValue(Case);
		return *FTextInspector::GetSourceString(EnumDisplayText);
	}

	return TEXT("");
}

void UNiagaraNodeSelect::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive, bool bFilterForCompilation) const
{
	UNiagaraNode::BuildParameterMapHistory(OutHistory, bRecursive, bFilterForCompilation);
}

void UNiagaraNodeSelect::PreChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType)
{
	// do nothing here
}

void UNiagaraNodeSelect::PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType)
{
	if (SelectorPinType.GetEnum() == Changed)
	{
		RefreshFromExternalChanges();
	}
}

bool UNiagaraNodeSelect::AllowNiagaraTypeForPinTypeChange(const FNiagaraTypeDefinition& InType, UEdGraphPin* Pin) const
{
	// if it's the selector pin, check for integer types and connections
	if(Pin == GetSelectorPin())
	{
		return InType.IsIndexType();
	}
	// else, it's a value pin. We have to treat all value pins as a single entity
	else
	{
		return Super::AllowNiagaraTypeForPinTypeChange(InType, Pin);
	}
}

bool UNiagaraNodeSelect::OnNewPinTypeRequested(UEdGraphPin* PinToChange, FNiagaraTypeDefinition NewType)
{
	PinToChange->Modify();
	PinToChange->ResetDefaultValue();

	if (PinToChange == GetSelectorPin())
	{
		SelectorPinType = NewType;
		this->ChangeSelectorPinType(NewType);
		return true;
	}
	else
	{
		if(OutputVarGuids.Contains(PinToChange->PersistentGuid))
		{
			return Super::OnNewPinTypeRequested(PinToChange, NewType);
		}
		else if(UEdGraphSchema_Niagara::IsPinWildcard(PinToChange))
		{
			// we don't reallocate to maintain pin connections, and instead just change pin types
			int32 OutputIndex = INDEX_NONE;

			TArray<UEdGraphPin*> InputPins;
			GetInputPins(InputPins);
			
			InputPins.RemoveAll([](UEdGraphPin* Pin)
			{
				return Pin->bOrphanedPin;
			});

			for(int32 Idx = 0; Idx < OutputVars.Num(); Idx++)
			{
				TArray<UEdGraphPin*> OptionPins = GetOptionPins(Idx);
				if(OptionPins.Contains(PinToChange))
				{
					OutputIndex = Idx;
				}
			}

			TSet<FName> OutputNames;
			for (const FNiagaraVariable& Output : OutputVars)
			{
				OutputNames.Add(Output.GetName());
			}
			
			FName VariableAfterTypeChangeName = FNiagaraUtilities::GetUniqueName(NewType.GetFName(), OutputNames);
			FNiagaraVariable OldVariable(OutputVars[OutputIndex].GetType(), OutputVars[OutputIndex].GetType().GetFName());
			FNiagaraVariable TmpVar(NewType, VariableAfterTypeChangeName);
			
			int32 Value = INDEX_NONE;
			TArray<int32> OptionValues = GetOptionValues();
			for(int32 OptionIndex = 0; OptionIndex < OptionValues.Num(); OptionIndex++)
			{
				FString CandidateNameForValue = GetOptionPinName(OldVariable, OptionValues[OptionIndex]).ToString();
				if(CandidateNameForValue.Equals(PinToChange->GetName()))
				{
					Value = OptionValues[OptionIndex];
				}
			}
			
			if (OutputIndex != INDEX_NONE && Value != INDEX_NONE)
			{
				PinToChange->PinName = GetOptionPinName(TmpVar, Value);
				UEdGraphPin* OutputPin = GetPinByPersistentGuid(OutputVarGuids[OutputIndex]);

				return Super::OnNewPinTypeRequested(OutputPin, NewType);
			}
		}
	}

	return false;
}

void UNiagaraNodeSelect::PostInitProperties()
{
	Super::PostInitProperties();

	if (!SelectorPinGuid.IsValid())
	{
		SelectorPinGuid = FGuid::NewGuid();
	}

	OutputVarGuids.Reserve(OutputVars.Num());
	
	for (int32 OutputIndex = 0; OutputIndex < OutputVars.Num(); OutputIndex++)
	{
		if(!OutputVarGuids[OutputIndex].IsValid())
		{
			OutputVarGuids[OutputIndex] = FGuid::NewGuid();
		}
	}

	for (int32 OutputIndex = 0; OutputIndex < OutputVars.Num(); OutputIndex++)
	{
		if (!OutputVars[OutputIndex].GetType().IsValid())
		{
			OutputVars[OutputIndex] = FNiagaraVariable(FNiagaraTypeDefinition::GetWildcardDef(), FName(TEXT("Wildcard Output") + FString::FromInt(OutputIndex)));
		}
	}
	
	if (!SelectorPinType.IsValid())
	{
		SelectorPinType = FNiagaraTypeDefinition::GetWildcardDef();
	}
}

bool UNiagaraNodeSelect::IsPinStatic(const UEdGraphPin* Pin) const
{
	return Pin->PersistentGuid.IsValid() && (Pin->PersistentGuid == SelectorPinGuid || OutputVarGuids.Contains(Pin->PersistentGuid));
}

TArray<int32> UNiagaraNodeSelect::GetOptionValues() const
{
	TArray<int32> SelectorValues;

	if(FNiagaraTypeDefinition::GetBoolDef().IsSameBaseDefinition(SelectorPinType))
	{
		SelectorValues = { 1, 0 };
	}
	else if(FNiagaraTypeDefinition::GetIntDef().IsSameBaseDefinition(SelectorPinType))
	{
		int32 NewOptionsCount = FMath::Max(2, NumOptionsPerVariable);
		SelectorValues.Reserve(NewOptionsCount);
		for(int32 Index = 0; Index < NewOptionsCount; Index++)
		{
			SelectorValues.Add(Index);
		}
	}
	else if(SelectorPinType.IsEnum() && SelectorPinType.IsValid() && SelectorPinType.GetEnum())
	{
		UEnum* Enum = SelectorPinType.GetEnum();
		const int32 TotalEnumEntryCount = Enum->NumEnums() - 1;
		if (TotalEnumEntryCount > 0)
		{
			SelectorValues.Reserve(TotalEnumEntryCount);
			int32 ValidEnumEntryCount = 0;
			for (int32 EnumIndex = 0; EnumIndex < TotalEnumEntryCount; EnumIndex++)
			{
				if(FNiagaraEditorUtilities::IsEnumIndexVisible(Enum, EnumIndex))
				{
					ValidEnumEntryCount++;
					SelectorValues.Add(static_cast<int32>(Enum->GetValueByIndex(EnumIndex)));
				}			
			}
		}
	}

	return SelectorValues;
}

FName UNiagaraNodeSelect::GetSelectorPinName() const
{
	return FName(TEXT("Selector"));
}

FText UNiagaraNodeSelect::GetIntegerAddButtonTooltipText() const
{
	return LOCTEXT("IntegerAddButtonTooltip", "Add a new input pin");
}

FText UNiagaraNodeSelect::GetIntegerRemoveButtonTooltipText() const
{
	return LOCTEXT("IntegerRemoveButtonTooltip", "Remove the input pin with the highest index");
}

EVisibility UNiagaraNodeSelect::ShowAddIntegerButton() const
{
	return SelectorPinType.IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility UNiagaraNodeSelect::ShowRemoveIntegerButton() const
{
	return SelectorPinType.IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) && NumOptionsPerVariable > 2 ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE

