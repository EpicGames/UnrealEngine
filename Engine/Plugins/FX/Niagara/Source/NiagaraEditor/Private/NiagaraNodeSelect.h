// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraNodeWithDynamicPins.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraNodeUsageSelector.h"
#include "Kismet2/EnumEditorUtils.h"
#include "Layout/Visibility.h"
#include "ToolMenu.h"

#include "NiagaraNodeSelect.generated.h"

UCLASS(MinimalAPI)
class UNiagaraNodeSelect : public UNiagaraNodeUsageSelector, public FEnumEditorUtils::INotifyOnEnumChanged
{
	GENERATED_BODY()

public:
	UNiagaraNodeSelect();
	
	UPROPERTY()
	FNiagaraTypeDefinition SelectorPinType;

	UPROPERTY()
	FGuid SelectorPinGuid;

	void ChangeValuePinType(int32 Index, FNiagaraTypeDefinition Type);
	void ChangeSelectorPinType(FNiagaraTypeDefinition Type);
	
	class UEdGraphPin* GetSelectorPin() const;
	TArray<class UEdGraphPin*> GetOptionPins(int32 Index) const;
	TArray<class UEdGraphPin*> GetValuePins(int32 Index) const;
	class UEdGraphPin* GetOutputPin(const FNiagaraVariable& Variable) const;
	
	void AddIntegerInputPin();
	void RemoveIntegerInputPin();

	virtual TArray<int32> GetOptionValues() const override;

private:
	/** UObject interface */
	virtual void PostInitProperties() override;
	
	/** UEdGraphNode interface */
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	virtual void PostLoad() override;
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;

	/** UNiagaraNode interface */
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual bool AllowExternalPinTypeChanges(const UEdGraphPin* InGraphPin) const override;
	virtual bool AllowNiagaraTypeForPinTypeChange(const FNiagaraTypeDefinition& InType, UEdGraphPin* Pin) const override;
	virtual bool OnNewPinTypeRequested(UEdGraphPin* PinToChange, FNiagaraTypeDefinition NewType) override;
	virtual void PinTypeChanged(UEdGraphPin* InGraphPin) override;
	virtual void AddWidgetsToOutputBox(TSharedPtr<SVerticalBox> OutputBox) override;
	virtual void AddWidgetsToInputBox(TSharedPtr<SVerticalBox> InputBox) override;
	virtual void GetWildcardPinHoverConnectionTextAddition(const UEdGraphPin* WildcardPin, const UEdGraphPin* OtherPin, ECanCreateConnectionResponse ConnectionResponse, FString& OutString) const override;
	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive, bool bFilterForCompilation) const override;

	/** UNiagaraNodeWithDynamicPins interface */
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const override;
	virtual bool CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const override;
	virtual void MoveDynamicPin(UEdGraphPin* Pin, int32 DirectionToMove) override;
	virtual bool CanRenamePin(const UEdGraphPin* Pin) const override;

	/** UNiagaraNodeUsageSelector interface */
	virtual FString GetInputCaseName(int32 Case) const override;
	
	/** INotifyOnEnumChanged interface */
	virtual void PreChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override;
	virtual void PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override;

	/** Checks if the pin is one of the 'static' pins (selector or output) */
	bool IsPinStatic(const UEdGraphPin* Pin) const;
	FName GetSelectorPinName() const;
	
	FText GetIntegerAddButtonTooltipText() const;
	FText GetIntegerRemoveButtonTooltipText() const;
	EVisibility ShowAddIntegerButton() const;
	EVisibility ShowRemoveIntegerButton() const;
};
