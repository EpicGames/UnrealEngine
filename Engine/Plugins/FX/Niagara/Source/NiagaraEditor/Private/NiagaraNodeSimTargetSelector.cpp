// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeSimTargetSelector.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraHlslTranslator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeSimTargetSelector)

#define LOCTEXT_NAMESPACE "NiagaraNodeSimTargetSelector"

UNiagaraNodeSimTargetSelector::UNiagaraNodeSimTargetSelector(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNiagaraNodeSimTargetSelector::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	NumOptionsPerVariable = 2;

	// create all the cpu input pins
	for (FNiagaraVariable& Var : OutputVars)
	{
		AddOptionPin(Var, 0);
	}

	// create all the gpu input pins
	for (FNiagaraVariable& Var : OutputVars)
	{
		AddOptionPin(Var, 1);
	}

	// create the output pins
	for (int32 Index = 0; Index < OutputVars.Num(); Index++)
	{
		const FNiagaraVariable& Var = OutputVars[Index];
		UEdGraphPin* NewPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(Var.GetType()), Var.GetName());
		NewPin->PersistentGuid = OutputVarGuids[Index];
	}

	CreateAddPin(EGPD_Output);
}

FString UNiagaraNodeSimTargetSelector::GetInputCaseName(int32 Case) const
{
	return Case == 0 ? TEXT("CPU VM") : TEXT("GPU Shader");
}

TArray<int32> UNiagaraNodeSimTargetSelector::GetOptionValues() const
{
	TArray<int32> OptionValues;
	for(int32 Value = 0; Value < StaticEnum<ENiagaraSimTarget>()->NumEnums(); Value++)
	{
		OptionValues.Add(Value);
	}
	
	return OptionValues; 
}

void UNiagaraNodeSimTargetSelector::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	FPinCollectorArray InputPins;
	GetCompilationInputPins(InputPins);
	FPinCollectorArray OutputPins;
	GetCompilationOutputPins(OutputPins);

	//ENiagaraSimTarget SimulationTarget = Translator->GetSimulationTarget();
	bool bCPUSim = Translator->IsCompileOptionDefined(*FNiagaraCompileOptions::CpuScriptDefine);
	bool bGPUSim = Translator->IsCompileOptionDefined(*FNiagaraCompileOptions::GpuScriptDefine);

	if (Translator->GetTargetUsage() >= ENiagaraScriptUsage::Function && Translator->GetTargetUsage() <= ENiagaraScriptUsage::DynamicInput)
	{
		// Functions through Dynamic inputs are missing the context, so just use CPU by default.
		bCPUSim = true;
	}

	int32 VarIdx;
	if (bCPUSim/*SimulationTarget == ENiagaraSimTarget::CPUSim*/)
	{
		VarIdx = 0;
	}
	else if (bGPUSim/*SimulationTarget == ENiagaraSimTarget::GPUComputeSim*/)
	{
		VarIdx = InputPins.Num() / 2;
	}
	else
	{
		Translator->Error(LOCTEXT("InvalidSimTarget", "Unknown simulation target"), this, nullptr);
		return;
	}

	Outputs.SetNumUninitialized(OutputPins.Num());
	for (int32 i = 0; i < OutputVars.Num(); i++)
	{
		int32 InputIdx = Translator->CompileInputPin(InputPins[VarIdx + i]);
		Outputs[i] = InputIdx;
	}
}

UEdGraphPin* UNiagaraNodeSimTargetSelector::GetPassThroughPin(const UEdGraphPin* LocallyOwnedOutputPin, ENiagaraScriptUsage InUsage) const
{
	return nullptr;
}

void UNiagaraNodeSimTargetSelector::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	UNiagaraNode::BuildParameterMapHistory(OutHistory, bRecursive, bFilterForCompilation);
}

FText UNiagaraNodeSimTargetSelector::GetTooltipText() const
{
	return LOCTEXT("SimTargetSelectorDesc", "If the simulation target matches, then the traversal will follow that path.");
}

FText UNiagaraNodeSimTargetSelector::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("SimTargetSelectorTitle", "Select by Simulation Target");
}

#undef LOCTEXT_NAMESPACE

