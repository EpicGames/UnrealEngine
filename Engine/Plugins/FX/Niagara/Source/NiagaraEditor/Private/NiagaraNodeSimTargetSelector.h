// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "EdGraphSchema_Niagara.h"
#include "NiagaraNodeUsageSelector.h"
#include "NiagaraNodeSimTargetSelector.generated.h"

UCLASS(MinimalAPI)
class UNiagaraNodeSimTargetSelector: public UNiagaraNodeUsageSelector
{
	GENERATED_UCLASS_BODY()

public:	
	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End EdGraphNode Interface

	//~ Begin UNiagaraNode Interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* LocallyOwnedOutputPin, ENiagaraScriptUsage InUsage) const override;
	//~ End UNiagaraNode Interface

protected:
	//~ Begin UNiagaraNodeUsageSelector Interface
	virtual FString GetInputCaseName(int32 Case) const override;
	virtual TArray<int32> GetOptionValues() const override;
	//~ End UNiagaraNodeUsageSelector Interface
};
