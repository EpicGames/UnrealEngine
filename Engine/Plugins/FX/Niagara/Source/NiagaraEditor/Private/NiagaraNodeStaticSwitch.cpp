// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraScriptVariable.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ScopedTransaction.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraSettings.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Engine/UserDefinedEnum.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor/EditorEngine.h"
#include "Editor.h"
#include "ISinglePropertyView.h"
#include "IStructureDetailsView.h"
#include "NiagaraNodeFunctionCall.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeStaticSwitch)

#define LOCTEXT_NAMESPACE "NiagaraNodeStaticSwitch"

namespace NiagaraStaticSwitchCVars
{
	static bool bEnabledAutoRefreshOldStaticSwitches = false;
	static FAutoConsoleVariableRef CVarEnableAutoRefreshOldStaticSwitches(TEXT("Niagara.StaticSwitch.EnableAutoRefreshOldStaticSwitches"), bEnabledAutoRefreshOldStaticSwitches, 
		TEXT("Enables auto refresh for old static switch nodes on post load and updates to enum assets. Enable this and cook assets to check how many old nodes operate on outdated enums"));
}

UNiagaraNodeStaticSwitch::UNiagaraNodeStaticSwitch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), InputParameterName(FName(TEXT("Undefined parameter name"))), IsValueSet(false), SwitchValue(0)
{
}

FNiagaraTypeDefinition UNiagaraNodeStaticSwitch::GetInputType() const
{
	if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Bool)
	{
		return FNiagaraTypeDefinition::GetBoolDef();
	}
	else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer)
	{
		return FNiagaraTypeDefinition::GetIntDef();
	}
	else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum && SwitchTypeData.Enum)
	{
		return FNiagaraTypeDefinition(SwitchTypeData.Enum);
	}
	return FNiagaraTypeDefinition();
}

void UNiagaraNodeStaticSwitch::OnInputCaseLabelSubmitted(const FText& Text, ETextCommit::Type Arg, int32 InputCase) const
{
	if(UNiagaraScriptVariable* StaticSwitchVariable = GetStaticSwitchScriptVariable())
	{
		StaticSwitchVariable->PreEditChange(nullptr);
		// if we have at least one case label that has been changed, we want to switch to enum style display.
		StaticSwitchVariable->Metadata.WidgetCustomization.WidgetType = ENiagaraInputWidgetType::EnumStyle;
		
		if(ensure(StaticSwitchVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.IsValidIndex(InputCase)))
		{
			StaticSwitchVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues[InputCase].DisplayName = Text;
		}

		FPropertyChangedEvent PropertyChangedEvent(nullptr);
		StaticSwitchVariable->PostEditChangeProperty(PropertyChangedEvent);
	}
}

bool UNiagaraNodeStaticSwitch::VerifyCaseLabelCandidate(const FText& InCandidate, FText& OutErrorMessage) const
{
	bool bResult = Super::VerifyCaseLabelCandidate(InCandidate, OutErrorMessage);
	if(bResult == false)
	{
		return bResult;
	}

	if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
	{		
		for(const FNiagaraWidgetNamedIntegerInputValue& Value : ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues)
		{
			if(Value.DisplayName.CompareToCaseIgnored(InCandidate) == 0)
			{
				OutErrorMessage = LOCTEXT("CaseAlreadyExists", "This case already exists. Choose a different label!");
				return false;
			}
		}
	}

	return true;
}

FText UNiagaraNodeStaticSwitch::GetInputCaseTooltip(int32 InputCase) const
{
	if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
	{
		if(ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.IsValidIndex(InputCase))
		{
			return ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues[InputCase].Tooltip;
		}
	}

	return FText::GetEmpty();
}

FText UNiagaraNodeStaticSwitch::GetInputCaseButtonTooltip(int32 Case) const
{
	return FText::FormatOrdered(LOCTEXT("SummonCaseLabelButtonTooltip", "If input is {0}.\nEdit the case label & tooltip. Editing these will set the widget type to 'Enum Style'."), FText::FromString(FString::FromInt(Case))); 
}

void UNiagaraNodeStaticSwitch::OnInputCaseTooltipSubmitted(const FText& Text, ETextCommit::Type Arg, int32 InputCase) const
{
	if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
	{
		ScriptVariable->PreEditChange(nullptr);
		if(ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.IsValidIndex(InputCase))
		{
			ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues[InputCase].Tooltip = Text;
			FPropertyChangedEvent PropertyChangedEvent(nullptr);
			ScriptVariable->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
}

TSharedRef<SWidget> UNiagaraNodeStaticSwitch::GetInputCaseContextOptions(int32 Case)
{
	if(SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer)
	{
		if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
		{
			// since we want to edit some case of a static switch 
			if(Case >= ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.Num())
			{
				int32 PreviousElementCount = ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.Num();
				ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.Reserve(Case + 1);

				for(int32 MissingElementIndex = PreviousElementCount; MissingElementIndex <= Case; MissingElementIndex++)
				{
					FNiagaraWidgetNamedIntegerInputValue& NamedIntegerInputValue = ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.AddDefaulted_GetRef();
					NamedIntegerInputValue.DisplayName = FText::FromString(FString::FromInt(Case));
				}
			}

			TSharedRef<FStructOnScope> MenuStruct = MakeShared<FStructOnScope>(StaticStruct<FNiagaraWidgetNamedIntegerInputValue>(), (uint8*) &GetStaticSwitchScriptVariable()->Metadata.WidgetCustomization.EnumStyleDropdownValues[Case]);;
			MenuStruct->SetPackage(GetPackage());
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.NotifyHook = this;
			TSharedRef<IStructureDetailsView> StructureView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, FStructureDetailsViewArgs(), MenuStruct);

			return StructureView->GetWidget().ToSharedRef();
		}
	}
	
	return SNullWidget::NullWidget;	
}

FString UNiagaraNodeStaticSwitch::GetInputCaseName(int32 Case) const
{
	if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Bool)
	{
		if (Case == 0)
		{
			return TEXT("False");
		}
		if (Case == 1)
		{
			return TEXT("True");
		}
	}
	else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer)
	{
		UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable();
		if(ScriptVariable && ScriptVariable->Metadata.WidgetCustomization.WidgetType == ENiagaraInputWidgetType::EnumStyle)
		{
			if(ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.IsValidIndex(Case))
			{
				return ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues[Case].DisplayName.ToString();
			}
		}
		
		return FString::FromInt(Case);
	}
	else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum && SwitchTypeData.Enum)
	{
		FText EnumDisplayText = SwitchTypeData.Enum->GetDisplayNameTextByValue(Case);
		return *FTextInspector::GetSourceString(EnumDisplayText);
	}

	return TEXT("");
}

#if WITH_EDITOR
void UNiagaraNodeStaticSwitch::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	
	if (MemberPropertyName != NAME_None)
	{
		ReallocatePins();
	}
}

#endif

TArray<int32> UNiagaraNodeStaticSwitch::GetOptionValues() const
{
	TArray<int32> OptionValues;
	if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Bool)
	{
		OptionValues = { 1, 0 };
	}
	else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer)
	{
		int32 EnumEntries = INDEX_NONE;
		bool bUseEnumDisplay = false;
		if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
		{
			if(ScriptVariable->Metadata.WidgetCustomization.WidgetType == ENiagaraInputWidgetType::EnumStyle)
			{
				EnumEntries = ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.Num();
				bUseEnumDisplay = true;
			}
		}
		
		int32 NewOptionsCount = bUseEnumDisplay ? EnumEntries : NumOptionsPerVariable;
		for(int32 Counter = 0; Counter < NewOptionsCount; Counter++)
		{
			OptionValues.Add(Counter);
		}
	}
	else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum && SwitchTypeData.Enum)
	{
		UEnum* Enum = SwitchTypeData.Enum;

		for (int32 EnumIndex = 0; EnumIndex < Enum->NumEnums() - 1; ++EnumIndex)
		{
			if (FNiagaraEditorUtilities::IsEnumIndexVisible(Enum, EnumIndex))
			{
				OptionValues.Add(static_cast<int32>(Enum->GetValueByIndex(EnumIndex)));
			}
		}
	}

	return OptionValues;
}

bool UNiagaraNodeStaticSwitch::CanModifyPin(const UEdGraphPin* Pin) const
{
	if(IsAddPin(Pin))
	{
		return false;
	}

	return true;
}

bool UNiagaraNodeStaticSwitch::CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const
{
	auto FindPredicate = [=](const FGuid& Guid) { return Guid == Pin->PersistentGuid; };
	int32 FoundIndex = OutputVarGuids.IndexOfByPredicate(FindPredicate);
	if (FoundIndex != INDEX_NONE && OutputVars.IsValidIndex(FoundIndex + DirectionToMove))
	{
		return Pin->Direction == EGPD_Output && Pin->bOrphanedPin == false;
	}

	return false;
}

void UNiagaraNodeStaticSwitch::MoveDynamicPin(UEdGraphPin* Pin, int32 MoveAmount)
{
	auto FindPredicate = [=](const FGuid& Guid) { return Guid == Pin->PersistentGuid; };
	int32 FoundIndex = OutputVarGuids.IndexOfByPredicate(FindPredicate);
	if (FoundIndex != INDEX_NONE && OutputVars.IsValidIndex(FoundIndex + MoveAmount))
	{
		this->Modify();
		
		FNiagaraVariable TmpVar = OutputVars[FoundIndex];
		OutputVars[FoundIndex] = OutputVars[FoundIndex + MoveAmount];
		OutputVars[FoundIndex + MoveAmount] = TmpVar;

		FGuid TmpGuid = OutputVarGuids[FoundIndex];
		OutputVarGuids[FoundIndex] = OutputVarGuids[FoundIndex + MoveAmount];
		OutputVarGuids[FoundIndex + MoveAmount] = TmpGuid;

		ReallocatePins();
	}
}

void UNiagaraNodeStaticSwitch::PreChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType)
{
	// do nothing
}

void UNiagaraNodeStaticSwitch::PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType)
{
	const bool bHasUnderlyingEnumChanged = SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum && Changed == SwitchTypeData.Enum;
	const bool bShouldAutoRefresh = SwitchTypeData.bAutoRefreshEnabled == true || NiagaraStaticSwitchCVars::bEnabledAutoRefreshOldStaticSwitches;

	// Can't call GetNiagaraGraph on the CDO, this should early out before then on the CDO
	if(bHasUnderlyingEnumChanged && bShouldAutoRefresh && !GetNiagaraGraph()->IsCompilationCopy())
	{
		RefreshFromExternalChanges();
	}
}

void UNiagaraNodeStaticSwitch::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	if(PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraWidgetNamedIntegerInputValue, DisplayName) || PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraWidgetNamedIntegerInputValue, Tooltip))
	{
		if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
		{
			ScriptVariable->PreEditChange(PropertyAboutToChange);
		}
	}
}

void UNiagaraNodeStaticSwitch::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if(PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraWidgetNamedIntegerInputValue, DisplayName) || PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraWidgetNamedIntegerInputValue, Tooltip))
	{
		if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
		{
			// since we only use this function to edit enum style entries in a menu, it's safe to assume the user wants to display this entry as enum
			if(ScriptVariable->Metadata.WidgetCustomization.WidgetType != ENiagaraInputWidgetType::EnumStyle)
			{
				ScriptVariable->Metadata.WidgetCustomization.WidgetType = ENiagaraInputWidgetType::EnumStyle;
				
				FText NotificationText = FText::FormatOrdered(LOCTEXT("WidgetTypeOfStaticSwitchSetToEnumStyle", "Variable {0}'s widget style has been set to 'Enum Style' to support display of custom names and tooltips."), FText::FromName(ScriptVariable->Variable.GetName()));
				FNotificationInfo NotificationInfo(NotificationText);
				NotificationInfo.ExpireDuration = 5.f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);				
			}
			FPropertyChangedEvent PropertyChangedEventNonConst = PropertyChangedEvent;
			ScriptVariable->PostEditChangeProperty(PropertyChangedEventNonConst);
		}
	}
}

FName UNiagaraNodeStaticSwitch::GetOptionPinName(const FNiagaraVariable& Variable, int32 Value) const
{
	FString Suffix = TEXT("");
	if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Bool)
	{
		if (Value == 0)
		{
			Suffix = TEXT("false");
		}
		if (Value == 1)
		{
			Suffix =  TEXT("true");
		}
	}
	else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer)
	{
		Suffix = FString::FromInt(Value);
	}
	else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum && SwitchTypeData.Enum)
	{
		// the display name is subject to localization and some automatic prettification. To avoid the localization aspect, we retrieve the source string of the text
		// which is essentially the still prettified non-localized base text. We have to keep it this way for backwards compatibility, until we do a full upgrade pass
		FText EnumDisplayText = SwitchTypeData.Enum->GetDisplayNameTextByValue(Value);
		Suffix = *FTextInspector::GetSourceString(EnumDisplayText);
	}

	return FName(Variable.GetName().ToString() + FString::Printf(TEXT(" if %s"), *Suffix));
}

void UNiagaraNodeStaticSwitch::UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const
{
	if (IsSetByCompiler())
	{
		if (const FNiagaraVariable* Found = FNiagaraConstants::FindStaticSwitchConstant(SwitchTypeData.SwitchConstant))
		{
			InOutFunctionAliasContext.AddPendingStaticSwitchVariable(*Found, static_cast<uint8>(SwitchTypeData.SwitchType));
		}
	}
}

void UNiagaraNodeStaticSwitch::ChangeSwitchParameterName(const FName& NewName)
{
	GetNiagaraGraph()->RenameStaticSwitch(this, NewName);
	// due to renaming of the static switch, it is possible we are now pointing to another pre-existing static switch variable, so we need to reallocate pins.
	ReallocatePins();
	VisualsChangedDelegate.Broadcast(this);
}

void UNiagaraNodeStaticSwitch::OnSwitchParameterTypeChanged(const FNiagaraTypeDefinition& OldType)
{
	TOptional<FNiagaraVariableMetaData> AlreadyExistingMetaData = GetNiagaraGraph()->GetMetaData(FNiagaraVariable(GetInputType(), InputParameterName));
	TOptional<FNiagaraVariableMetaData> OldMetaData = GetNiagaraGraph()->GetMetaData(FNiagaraVariable(OldType, InputParameterName));

	// we make sure the new parameter exists at this point
	GetNiagaraGraph()->AddParameter(FNiagaraVariable(GetInputType(), InputParameterName), true);
	
	// we overwrite the metadata only if it didn't exist already
	if (AlreadyExistingMetaData.IsSet() == false && OldMetaData.IsSet())
	{
		GetNiagaraGraph()->SetMetaData(FNiagaraVariable(GetInputType(), InputParameterName), OldMetaData.GetValue());
	}

	// if we changed to an integer, we want enum style behavior by default
	if(AlreadyExistingMetaData.IsSet() == false && GetInputType() == FNiagaraTypeDefinition::GetIntDef())
	{
		if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
		{
			ScriptVariable->Metadata.WidgetCustomization.WidgetType = ENiagaraInputWidgetType::EnumStyle;
		}
	}

	RefreshFromExternalChanges(); // Magick happens here: The old pins are destroyed and new ones are created.

	VisualsChangedDelegate.Broadcast(this);
	RemoveUnusedGraphParameter(FNiagaraVariable(OldType, InputParameterName));
}

void UNiagaraNodeStaticSwitch::SetSwitchValue(int Value)
{
	IsValueSet = true;
	SwitchValue = Value;
}

void UNiagaraNodeStaticSwitch::SetSwitchValue(const FCompileConstantResolver& ConstantResolver)
{
	if (!IsSetByCompiler() && !IsSetByPin())
	{
		return;
	}
	ClearSwitchValue();

	const FNiagaraVariable* Found = FNiagaraConstants::FindStaticSwitchConstant(SwitchTypeData.SwitchConstant);
	FNiagaraVariable Constant = Found ? *Found : FNiagaraVariable();
	if (Found && ConstantResolver.ResolveConstant(Constant))
	{
		if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Bool)
		{
			SwitchValue = Constant.GetValue<bool>();
			IsValueSet = true;
		}
		else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer ||
			SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum)
		{
			SwitchValue = Constant.GetValue<int32>();
			IsValueSet = true;
		}
	}
}

void UNiagaraNodeStaticSwitch::ClearSwitchValue()
{
	IsValueSet = false;
	SwitchValue = 0;
}

bool UNiagaraNodeStaticSwitch::IsSetByCompiler() const
{
	return !SwitchTypeData.SwitchConstant.IsNone();
}

bool UNiagaraNodeStaticSwitch::IsSetByPin() const
{
	return SwitchTypeData.bExposeAsPin && !IsSetByCompiler() && !IsDebugSwitch();
}


bool UNiagaraNodeStaticSwitch::IsDebugSwitch() const
{
	return SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum && SwitchTypeData.SwitchConstant == TEXT("Function.DebugState") && SwitchTypeData.Enum == FNiagaraTypeDefinition::GetFunctionDebugStateEnum();
}

void UNiagaraNodeStaticSwitch::RemoveUnusedGraphParameter(const FNiagaraVariable& OldParameter)
{
	TArray<FNiagaraVariable> GraphVariables = GetNiagaraGraph()->FindStaticSwitchInputs();
	int Index = GraphVariables.Find(OldParameter);
	if (Index == INDEX_NONE)
	{
		// Force delete the old static switch parameter.
		GetNiagaraGraph()->RemoveParameter(OldParameter, true);
	}
	else
	{
		GetNiagaraGraph()->NotifyGraphChanged();
	}

	// force the graph to refresh the metadata
	GetNiagaraGraph()->GetParameterReferenceMap();
}

void UNiagaraNodeStaticSwitch::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	// we ensure we have at least 2 entries in the enum style drop downs
	if(SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer)
	{
		if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
		{
			int32 PreviousEntryCount = ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.Num();
			for(int32 MissingValueIndex = PreviousEntryCount; MissingValueIndex < 2; MissingValueIndex++)
			{
				FNiagaraWidgetNamedIntegerInputValue& NamedIntegerInputValue = ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.AddDefaulted_GetRef();
				NamedIntegerInputValue.DisplayName = FText::FromString(FString::FromInt(MissingValueIndex));
			}
		}
	}
	
	TArray<int32> OptionValues = GetOptionValues();
	NumOptionsPerVariable = OptionValues.Num();
	
	for (int32 i = 0; i < OptionValues.Num(); i++)
	{
		for (const FNiagaraVariable& Variable : OutputVars)
		{
			AddOptionPin(Variable, OptionValues[i]);
		}
	}
		
	GetNiagaraGraph()->AddParameter(FNiagaraVariable(GetInputType(), InputParameterName), true);

	if (IsSetByPin())
	{		
		FNiagaraTypeDefinition Type = GetInputType().ToStaticDef();
		UEdGraphPin* NewPin  = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Type), TEXT("Selector"));
		if (SelectorGuid.IsValid() == false)
			SelectorGuid = FGuid::NewGuid();
		NewPin->PersistentGuid = SelectorGuid;
	}
	// create the output pins
	for (int32 Index = 0; Index < OutputVars.Num(); Index++)
	{
		const FNiagaraVariable& Var = OutputVars[Index];
		UEdGraphPin* NewPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(Var.GetType()), Var.GetName());
		NewPin->PersistentGuid = OutputVarGuids[Index];
	}

	CreateAddPin(EGPD_Output);

	// force the graph to refresh the metadata
	GetNiagaraGraph()->GetParameterReferenceMap();
}

bool UNiagaraNodeStaticSwitch::AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const
{
	// explicitly allow parameter maps and numeric types
	return InType.GetScriptStruct() != nullptr && !InType.IsInternalType();
}

bool UNiagaraNodeStaticSwitch::GetVarIndex(FTranslator* Translator, int32 InputPinCount, int32& VarIndexOut) const
{	
	return GetVarIndex(Translator, InputPinCount, SwitchValue, VarIndexOut);
}

UEdGraphPin* UNiagaraNodeStaticSwitch::GetSelectorPin() const
{
	if (IsSetByPin() == false)
		return nullptr;

	for (int32 i = 0; i < Pins.Num(); i++)
	{
		if (Pins[i]->PersistentGuid == SelectorGuid)
		{
			return  Pins[i];
		}
	}

	return nullptr;
}


void UNiagaraNodeStaticSwitch::UpdateCompilerConstantValue(FTranslator* Translator)
{
	if (IsSetByPin() && Translator)
	{
		ClearSwitchValue();

		const UEdGraphPin* SelectorPin = nullptr;
		for (int32 i = 0; i < Pins.Num(); i++)
		{
			if (Pins[i]->PersistentGuid == SelectorGuid)
			{
				SelectorPin = Pins[i];
				break;
			}
		}

		if (SelectorPin)
		{
			// Use the default value if disconnected...
			if (SelectorPin->LinkedTo.Num() == 0)
			{
				Translator->FillVariableWithDefaultValue(SwitchValue, SelectorPin);
				IsValueSet = true;
			}
			else 
			{
				Translator->SetConstantByStaticVariable(SwitchValue, SelectorPin);
				IsValueSet = true;
			}

			if (UNiagaraScript::LogCompileStaticVars != 0)
			{
				UE_LOGF(LogNiagaraEditor, Log, "Static Switch Value: %d  Name: \"%ls\"", SwitchValue, *GetPathName());
			}
			return;
		}
	}



	if (!IsSetByCompiler() || IsDebugSwitch() || !Translator)
	{
		return;
	}
	ClearSwitchValue();

	const FNiagaraVariable* Found = FNiagaraConstants::FindStaticSwitchConstant(SwitchTypeData.SwitchConstant);
	FNiagaraVariable Constant = Found ? *Found : FNiagaraVariable();
	if (Found && Translator->GetLiteralConstantVariable(Constant))
	{
		if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Bool)
		{
			SwitchValue = Constant.GetValue<bool>();
			IsValueSet = true;
		}
		else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer || SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum)
		{
			SwitchValue = Constant.GetValue<int32>();
			IsValueSet = true;
		}
		else
		{
			Translator->Error(LOCTEXT("InvalidSwitchType", "Invalid static switch type."), this, nullptr);
		}
	}
	else
	{
		Translator->Error(FText::Format(LOCTEXT("InvalidConstantValue", "Unable to determine constant value '{0}' for static switch."), FText::FromName(SwitchTypeData.SwitchConstant)), this, nullptr);
	}
}

bool UNiagaraNodeStaticSwitch::GetVarIndex(FTranslator* Translator, int32 InputPinCount, int32 Value, int32& VarIndexOut) const
{
	bool Success = false;
	if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Bool)
	{
		VarIndexOut = Value ? 0 : InputPinCount / 2;
		Success = true;
	}
	else if (SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer)
	{
		int32 MaxValue = NumOptionsPerVariable - 1;
		if (MaxValue >= 0)
		{
			if (Translator && (Value > MaxValue || Value < 0))
			{
				Translator->Warning(FText::Format(LOCTEXT("InvalidStaticSwitchIntValue", "The supplied int value {0} is outside the bounds for the static switch."), FText::FromString(FString::FromInt(Value))), this, nullptr);
			}
			VarIndexOut = FMath::Clamp(Value, 0, MaxValue) * (InputPinCount / (MaxValue + 1));
			Success = true;
		}
		else if (Translator)
		{
			Translator->Error(FText::Format(LOCTEXT("InvalidSwitchMaxIntValue", "Invalid max int value {0} for static switch."), FText::FromString(FString::FromInt(Value))), this, nullptr);
		}
	}
	else if ((SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum && SwitchTypeData.Enum))
	{
		TArray<int32> OptionValues = GetOptionValues();

		int32 MaxValue = OptionValues.Num();
		if (MaxValue > 0)
		{
			// do a sanity check here if the number of pins actually matches the enum count (which might have changed in the meantime without us noticing)
			FPinCollectorArray LocalOutputPins;
			GetOutputPins(LocalOutputPins);

			LocalOutputPins.RemoveAll([](UEdGraphPin* Pin)
			{
				return Pin->bOrphanedPin == true;
			});

			// remove the add pin from the count
			int32 OutputPinCount = LocalOutputPins.Num() - 1;

			if(OptionValues.Num() != NumOptionsPerVariable)
			{
				if (Translator)
				{
					Translator->Error(FText::Format(LOCTEXT("NumEnumEntriesDifferent", "Number of valid enum entries: {0}. Expected. {1}. Please refresh the node."), FText::FromString(FString::FromInt(OptionValues.Num())), FText::FromString(FString::FromInt(NumOptionsPerVariable))), this, nullptr);
				}
			}
			// if we have at least one output pin, check if input pin count and output pin count match up
			if (OutputPinCount > 0 && OutputPinCount * OptionValues.Num() != InputPinCount)
			{
				if (Translator)
				{
					Translator->Error(FText::Format(LOCTEXT("NumPinsNumEnumEntriesDifferent", "The number of pins on the static switch does not match the number of values defined in the enum."), FText::FromString(FString::FromInt(SwitchValue))), this, nullptr);
				}
			}
			if (Value <= MaxValue && Value >= 0)
			{
				VarIndexOut = Value * (InputPinCount / MaxValue);
				Success = true;
			}
			else if (Translator)
			{
				Translator->Error(FText::Format(LOCTEXT("InvalidSwitchEnumIndex", "Invalid static switch value \"{0}\" for enum value index."), FText::FromString(FString::FromInt(SwitchValue))), this, nullptr);
			}
		}
	}
	else if (Translator)
	{
		Translator->Error(LOCTEXT("InvalidSwitchType", "Invalid static switch type."), this, nullptr);
	}
	return Success;
}

UNiagaraScriptVariable* UNiagaraNodeStaticSwitch::GetStaticSwitchScriptVariable() const
{
	if(GetNiagaraGraph()->IsCompilationCopy() || IsSetByPin())
	{
		return nullptr;
	}

	FNiagaraVariable Variable(GetInputType(), InputParameterName);
	return GetNiagaraGraph()->GetScriptVariable(Variable);
}

void UNiagaraNodeStaticSwitch::CheckForOutdatedEnum(FTranslator* Translator) const
{
	// we check during compilation if the node is using outdated enum information and log it
	if(SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum)
	{
		TArray<UEdGraphPin*> InputPins;
		GetInputPins(InputPins);

		InputPins.RemoveAll([](UEdGraphPin* Pin)
		{
			return Pin->bOrphanedPin == true;
		});

		// this gets the new option values so we can retrieve the new pin names
		TArray<int32> OptionValues = GetOptionValues();

		// collect the new pin names
		TArray<FName> NewPinNames;
		for(int32 OptionIndex = 0; OptionIndex < OptionValues.Num(); OptionIndex++)
		{
			for(const FNiagaraVariable& Variable : OutputVars)
			{
				FName UpdatedPinName = GetOptionPinName(Variable, OptionValues[OptionIndex]);
				NewPinNames.Add(UpdatedPinName);		
			}
		}

		// this checks the current, possibly outdated, state for consistency. Should never be unequal but we are making sure.
		int NumChoices = IsSetByPin() ? InputPins.Num() - 1 : InputPins.Num();
		if (NumChoices != NumOptionsPerVariable * OutputVars.Num())
		{
			Translator->Message(FNiagaraCompileEventSeverity::Error, LOCTEXT("InternalCalculationPinCountError", "An internal calculation error has occured. Please refresh the node."), this, nullptr);
		}

		// this checks the new, up to date pin count against the old pin count
		if(NewPinNames.Num() != NumChoices)
		{
			if(NewPinNames.Num() == 0)
			{
				Translator->Message(FNiagaraCompileEventSeverity::Display, LOCTEXT("EmptyEnumEntries", "A static switch either has no enum assigned or the enum has no entries. A node refresh is advised."), this, nullptr, FString());
			}
			else
			{
				Translator->Message(FNiagaraCompileEventSeverity::Log, LOCTEXT("PinCountMismatch", "Static switch pins reflect outdated enum entries. While the static switch is fully functional, a node refresh is advised."), this, nullptr, FString());
			}
		}
		else
		{
			bool bAnyPinNameChangeDetected = false;
			for(int32 Index = 0; Index < NewPinNames.Num(); Index++)
			{
				if(!NewPinNames[Index].ToString().Equals(InputPins[Index]->PinName.ToString()))
				{
					bAnyPinNameChangeDetected = true;
					break;
				}
			}

			if (bAnyPinNameChangeDetected)
			{
				Translator->Message(FNiagaraCompileEventSeverity::Log, LOCTEXT("EnumPinNameChangeDetected", "Static switch pins reflect outdated enum entries. While the static switch is fully functional, a node refresh is advised."), this, nullptr, FString());
			}
		}
	}
}

void UNiagaraNodeStaticSwitch::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{	
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	int32 SelectorValue = SwitchValue;

	if (IsSetByPin())
	{
		UEdGraphPin* SelectorPin = GetSelectorPin();	
		if (SelectorPin)
		{
			Translator->CompileInputPin(SelectorPin);
		}
	}
	FPinCollectorArray OutputPins;
	GetCompilationOutputPins(OutputPins);

	// Initialize the outputs to invalid values.
	check(Outputs.Num() == 0);
	Outputs.Init(INDEX_NONE, OutputPins.Num());

	for (int i = 0; i < OutputPins.Num(); i++)
	{
		UEdGraphPin* OutPin = OutputPins[i];
		int32 VarIdx;
		int NumChoices = IsSetByPin() ? InputPins.Num() - 1 : InputPins.Num();
		if (GetVarIndex(nullptr, NumChoices, SelectorValue, VarIdx))
		{
			UEdGraphPin* InputPin = InputPins[VarIdx + i];
			if (InputPin)
			{
				int32 CompiledInput = Translator->CompileInputPin(InputPin);
				Outputs[i] = CompiledInput;
			}
		}		
	}
}

ENiagaraNumericOutputTypeSelectionMode UNiagaraNodeStaticSwitch::GetNumericOutputTypeSelectionMode() const
{
	return ENiagaraNumericOutputTypeSelectionMode::Largest;
}

void UNiagaraNodeStaticSwitch::ResolveNumerics(const UEdGraphSchema_Niagara* Schema, bool bSetInline, TMap<TPair<FGuid, UEdGraphNode*>, FNiagaraTypeDefinition>* PinCache)
{	
	FPinCollectorArray MainInputPins;
	GetInputPins(MainInputPins);
	FPinCollectorArray MainOutputPins;
	GetOutputPins(MainOutputPins);

	TArray<int32> OptionValues = GetOptionValues();
	int32 VarIdx = 0;
	for (int i = 0; i < MainOutputPins.Num(); i++)
	{
		UEdGraphPin* OutPin = MainOutputPins[i];
		if (IsAddPin(OutPin))
		{
			continue;
		}

		// Fix up numeric input pins and keep track of numeric types to decide the output type.
		TArray<UEdGraphPin*> InputPins;
		TArray<UEdGraphPin*> OutputPins; 
		
		for (int32 j = 0; j < OptionValues.Num(); j++)
		{
			const int TargetIdx = OutputVars.Num() * j + VarIdx;
			if (MainInputPins.IsValidIndex(TargetIdx))
			{
				UEdGraphPin* InputPin = MainInputPins[TargetIdx];
				InputPins.Add(InputPin);
			}
			else
			{
				UE_LOGF(LogNiagaraEditor, Warning, "Invalid index on UNiagaraNodeStaticSwitch::ResolveNumerics %ls, OptionIdx: %d VarIdx: %d MaxIndex: %d", *GetPathName(), j, VarIdx, MainInputPins.Num())
			}
		}

		OutputPins.Add(OutPin);

		NumericResolutionByPins(Schema, InputPins, OutputPins, bSetInline, PinCache);
		VarIdx++;
	}
}
bool UNiagaraNodeStaticSwitch::SubstituteCompiledPin(FTranslator* Translator, UEdGraphPin** LocallyOwnedPin)
{
	// if we compile the standalone module or function we don't have any valid input yet, so we just take the first option to satisfy the compiler
	ENiagaraScriptUsage TargetUsage = Translator->GetTargetUsage();
	bool IsDryRun = TargetUsage == ENiagaraScriptUsage::Module || TargetUsage == ENiagaraScriptUsage::Function;
	if (IsDryRun)
	{
		SwitchValue = 0;
	}
	else
	{
		UpdateCompilerConstantValue(Translator);
	}
	if (!IsValueSet && !IsDryRun)
	{
		FText ErrorMessage = FText::Format(LOCTEXT("MissingSwitchValue", "The input parameter \"{0}\" is not set to a constant value for the static switch node."), FText::FromString(InputParameterName.ToString()));
		Translator->Error(ErrorMessage, this, nullptr);
		return false;
	}

	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);

	for (int i = 0; i < OutputPins.Num(); i++)
	{
		UEdGraphPin* OutPin = OutputPins[i];
		int32 VarIdx;
		int NumChoices = IsSetByPin() ? InputPins.Num() - 1 : InputPins.Num();
		if (OutPin == *LocallyOwnedPin && GetVarIndex(Translator, NumChoices, IsDryRun ? 0 : SwitchValue, VarIdx))
		{
			UEdGraphPin* InputPin = InputPins[VarIdx + i];
			if (InputPin->LinkedTo.Num() == 1)
			{
				*LocallyOwnedPin = GetTracedOutputPin(InputPin->LinkedTo[0], true);
				return true;
			}
			else
			{
				*LocallyOwnedPin = InputPin;
			}
			return true;
		}
	}
	return false;
}

void UNiagaraNodeStaticSwitch::AddIntegerInputPin()
{
	FScopedTransaction Transaction(LOCTEXT("AddIntegerPinTransaction", "Added integer input pin to static switch"));

	this->Modify();

	NumOptionsPerVariable++;
	
	if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
	{
		// we only want to add a new inline enum entry if we are in enum style mode
		if(ScriptVariable->Metadata.WidgetCustomization.WidgetType == ENiagaraInputWidgetType::EnumStyle)
		{
			ScriptVariable->PreEditChange(nullptr);
			FNiagaraWidgetNamedIntegerInputValue& NewEntry = ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.AddDefaulted_GetRef();
			NewEntry.DisplayName = FText::FromString(FString::FromInt(ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.Num() - 1));
			FPropertyChangedEvent PropertyChangedEvent(nullptr);
			ScriptVariable->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
	
	ReallocatePins();
}

void UNiagaraNodeStaticSwitch::RemoveIntegerInputPin()
{
	FScopedTransaction Transaction(LOCTEXT("RemoveIntegerPinTransaction", "Removed integer input pin from static switch"));

	this->Modify();

	NumOptionsPerVariable = FMath::Max(2, --NumOptionsPerVariable);
	
	if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
	{
		// we only want to remove an inline enum entry if we are in enum style mode
		if(ScriptVariable->Metadata.WidgetCustomization.WidgetType == ENiagaraInputWidgetType::EnumStyle)
		{
			ScriptVariable->PreEditChange(nullptr);
			ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.RemoveAt(ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.Num() - 1);
			FPropertyChangedEvent PropertyChangedEvent(nullptr);
			ScriptVariable->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
	
	ReallocatePins();
}

FText UNiagaraNodeStaticSwitch::GetIntegerAddButtonTooltipText() const
{
	return LOCTEXT("IntegerAddButtonTooltip", "Add a new input pin");
}

FText UNiagaraNodeStaticSwitch::GetIntegerRemoveButtonTooltipText() const
{
	return LOCTEXT("IntegerRemoveButtonTooltip", "Remove the input pin with the highest index");
}

EVisibility UNiagaraNodeStaticSwitch::ShowAddIntegerButton() const
{
	return SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Integer ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility UNiagaraNodeStaticSwitch::ShowRemoveIntegerButton() const
{
	if(SwitchTypeData.SwitchType != ENiagaraStaticSwitchType::Integer)
	{
		return EVisibility::Collapsed;
	}
	
	if(UNiagaraScriptVariable* ScriptVariable = GetStaticSwitchScriptVariable())
	{
		if(ScriptVariable->Metadata.WidgetCustomization.WidgetType == ENiagaraInputWidgetType::EnumStyle)
		{
			if(ScriptVariable->Metadata.WidgetCustomization.EnumStyleDropdownValues.Num() > 2)
			{
				return EVisibility::Visible;
			}
		}
	}
	
	return NumOptionsPerVariable > 2 ? EVisibility::Visible : EVisibility::Collapsed;
}

void UNiagaraNodeStaticSwitch::PostLoad()
{
	Super::PostLoad();

	// Make sure that we are added to the static switch list.
	if (GetInputType().IsValid() && InputParameterName.IsValid())
	{
		if (UNiagaraGraph* NiagaraGraph = GetNiagaraGraph())
		{
			const FNiagaraVariable Variable(GetInputType(), InputParameterName);
			TOptional<bool> IsStaticSwitch = NiagaraGraph->IsStaticSwitch(Variable);

			if (IsStaticSwitch.IsSet() && *IsStaticSwitch == false)
			{
				UE_LOGF(LogNiagaraEditor, Log, "Static switch constant \"%ls\" in \"%ls\" didn't have static switch meta-data conversion set properly. Fixing now.", *InputParameterName.ToString(), *GetPathName())
				NiagaraGraph->SetIsStaticSwitch(Variable, true);
				MarkNodeRequiresSynchronization(TEXT("Static switch metadata updated"), true);
			}
		}
	}

	if(SwitchTypeData.bAutoRefreshEnabled == true || NiagaraStaticSwitchCVars::bEnabledAutoRefreshOldStaticSwitches)
	{
		// we assume something changed externally if the input pins are outdated; i.e. the assigned enum changed or value order changed
		AttemptUpdatePins();
	}	
}

UEdGraphPin* UNiagaraNodeStaticSwitch::GetTracedOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bFilterForCompilation, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace) const
{
	return GetTracedOutputPin(LocallyOwnedOutputPin, true, bFilterForCompilation, OutNodesVisitedDuringTrace);
}

UEdGraphPin* UNiagaraNodeStaticSwitch::GetTracedOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bRecursive, bool bFilterForCompilation, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace) const
{
	if (!bFilterForCompilation)
	{
		return LocallyOwnedOutputPin;
	}
	
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	
	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);

	if (OutNodesVisitedDuringTrace != nullptr && IsSetByPin())
	{
		OutNodesVisitedDuringTrace->Add(this);
	}

	for (int i = 0; i < OutputPins.Num(); i++)
	{
		UEdGraphPin* OutPin = OutputPins[i];
		if (IsAddPin(OutPin))
		{
			continue;
		}
		int32 VarIdx;
		int NumChoices = IsSetByPin() ? InputPins.Num() - 1 : InputPins.Num();
		if (OutPin == LocallyOwnedOutputPin && GetVarIndex(nullptr, NumChoices, SwitchValue, VarIdx))
		{
			UEdGraphPin* InputPin = InputPins[VarIdx + i];
			if (InputPin->LinkedTo.Num() == 1)
			{
				return bRecursive ? UNiagaraNode::TraceOutputPin(InputPin->LinkedTo[0], bFilterForCompilation, OutNodesVisitedDuringTrace) : InputPin->LinkedTo[0];
			}
		}
	}
	
	return LocallyOwnedOutputPin;
}

UEdGraphPin* UNiagaraNodeStaticSwitch::GetPassThroughPin(const UEdGraphPin* LocallyOwnedOutputPin,	ENiagaraScriptUsage InUsage) const
{
	if (IsValueSet)
	{
		return GetTracedOutputPin(const_cast<UEdGraphPin*>(LocallyOwnedOutputPin), true);
	}
	return Super::GetPassThroughPin(LocallyOwnedOutputPin, InUsage);
}

void UNiagaraNodeStaticSwitch::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	int32 SelectorValue = SwitchValue;

	if (!bFilterForCompilation)
	{
		UNiagaraNode::BuildParameterMapHistory(OutHistory, bRecursive, bFilterForCompilation);
	}
	else
	{
		if (IsSetByPin())
		{
			UEdGraphPin* SelectorPin = GetSelectorPin();
			OutHistory.VisitInputPin(SelectorPin, bFilterForCompilation);

			OutHistory.SetConstantByStaticVariable(SelectorValue, SelectorPin);

			if (UNiagaraScript::LogCompileStaticVars != 0)
			{
				UE_LOGF(LogNiagaraEditor, Log, "Static Switch Value: %d  Name: \"%ls\"  SwitchInputParameterName:\"%ls\"", SelectorValue, *GetPathName(), *InputParameterName.ToString());
			}
		}
	}

	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);

	for (int i = 0; i < OutputPins.Num(); i++)
	{
		UEdGraphPin* OutPin = OutputPins[i];
		if (IsAddPin(OutPin))
		{
			continue;
		}
		int32 VarIdx;
		int NumChoices = IsSetByPin() ? InputPins.Num() - 1 : InputPins.Num();
		if (bFilterForCompilation && GetVarIndex(nullptr, NumChoices, SelectorValue, VarIdx))
		{
			UEdGraphPin* InputPin = InputPins[VarIdx + i];
			RegisterPassthroughPin(OutHistory, InputPin, OutPin, bFilterForCompilation, true);
			/*if (InputPin->LinkedTo.Num() == 1)
			{
				return bRecursive ? UNiagaraNode::TraceOutputPin(InputPin->LinkedTo[0], bFilterForCompilation, OutNodesVisitedDuringTrace) : InputPin->LinkedTo[0];
			}*/
		}
		else if (!bFilterForCompilation && InputPins.Num() > 0)
		{
			const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());

			FNiagaraTypeDefinition OutDef = Schema->PinToTypeDefinition(OutPin);
			if (OutDef != FNiagaraTypeDefinition::GetParameterMapDef())
				continue;

			for (UEdGraphPin* InputPin : InputPins)
			{
				FNiagaraTypeDefinition InDef = Schema->PinToTypeDefinition(InputPin);
				if (InDef == FNiagaraTypeDefinition::GetParameterMapDef())
				{
					RegisterPassthroughPin(OutHistory, InputPin, OutPin, bFilterForCompilation, false);
					return;
				}
			}			
		}
	}
}

void UNiagaraNodeStaticSwitch::AddWidgetsToOutputBox(TSharedPtr<SVerticalBox> OutputBox)
{
	OutputBox->AddSlot()
	[
		SNew(SSpacer)
	];

	TAttribute<EVisibility> AddVisibilityAttribute;
	TAttribute<EVisibility> RemoveVisibilityAttribute;
	AddVisibilityAttribute.BindUObject(this, &UNiagaraNodeStaticSwitch::ShowAddIntegerButton);
	RemoveVisibilityAttribute.BindUObject(this, &UNiagaraNodeStaticSwitch::ShowRemoveIntegerButton);

	OutputBox->AddSlot()
	.Padding(4.f, 5.f)
	.AutoHeight()
	.HAlign(HAlign_Right)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.Visibility(RemoveVisibilityAttribute)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(GetIntegerRemoveButtonTooltipText())
			.OnPressed(FSimpleDelegate::CreateUObject(this, &UNiagaraNodeStaticSwitch::RemoveIntegerInputPin))
			[
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f, 0.9f))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.RemovePin"))
			]
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SButton)
			.Visibility(AddVisibilityAttribute)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(GetIntegerAddButtonTooltipText())
			.OnPressed(FSimpleDelegate::CreateUObject(this, &UNiagaraNodeStaticSwitch::AddIntegerInputPin))
			[
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f, 0.9f))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.AddPin"))
			]
		]
	];
}

void UNiagaraNodeStaticSwitch::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if(SwitchTypeData.SwitchType == ENiagaraStaticSwitchType::Enum && SwitchTypeData.Enum)
	{
		FToolMenuSection& Section = Menu->FindOrAddSection("Node");

		Section.AddMenuEntry(
			"BrowseToEnum",
			FText::Format(LOCTEXT("BrowseToEnumLabel", "Browse to {0}"), FText::FromString(SwitchTypeData.Enum->GetName())),
			LOCTEXT("BrowseToEnumTooltip", "Browses to the enum in the content browser."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser.Small"),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				FContentBrowserModule& Module = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				Module.Get().SyncBrowserToAssets({ FAssetData(SwitchTypeData.Enum) });
			}) , FCanExecuteAction::CreateLambda([this]()
			{
				return Cast<UUserDefinedEnum>(SwitchTypeData.Enum) != nullptr;
			})));


		Section.AddMenuEntry(
			"OpenEnum",
			FText::Format(LOCTEXT("OpenEnumLabel", "Open {0}"), FText::FromString(SwitchTypeData.Enum->GetName())),
			LOCTEXT("OpenEnumTooltip", "Opens up the enum asset."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SwitchTypeData.Enum);
			}),	FCanExecuteAction::CreateLambda([this]()
			{
				return Cast<UUserDefinedEnum>(SwitchTypeData.Enum) != nullptr;
			})));

		Section.AddMenuEntry(
			"AddEnumAsNiagaraType",
			FText::Format(LOCTEXT("AddEnumToTypesLabel", "Add enum {0} to user added enums"), FText::FromString(SwitchTypeData.Enum->GetName())),
			LOCTEXT("AddEnumToTypesTooltip", "Adds the enum to the list of user added enums so it can be used as a parameter."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				GetMutableDefault<UNiagaraSettings>()->AddEnumParameterType(SwitchTypeData.Enum);
			}), FCanExecuteAction::CreateLambda([this]()
			{
				return !GetDefault<UNiagaraSettings>()->AdditionalParameterEnums.Contains(SwitchTypeData.Enum);
			})));
	}
}

FText UNiagaraNodeStaticSwitch::GetTooltipText() const
{
	return LOCTEXT("NiagaraStaticSwitchNodeTooltip", "This is a compile-time switch that selects one branch to compile based on an input parameter.");
}

FText UNiagaraNodeStaticSwitch::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (IsSetByPin())
		return LOCTEXT("NiagaraStaticSwitchNodePinName", "Static Switch");
	return FText::FormatOrdered(LOCTEXT("StaticSwitchTitle", "Static Switch ({0})"), FText::FromName(IsSetByCompiler() ? SwitchTypeData.SwitchConstant : InputParameterName));
}

FLinearColor UNiagaraNodeStaticSwitch::GetNodeTitleColor() const
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(GetSchema());
	return Schema->NodeTitleColor_Constant;
}

#undef LOCTEXT_NAMESPACE

