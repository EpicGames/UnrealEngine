// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraNodeUsageSelector.h"
#include "Kismet2/EnumEditorUtils.h"
#include "ToolMenu.h"
#include "NiagaraNodeStaticSwitch.generated.h"

UENUM()
enum class ENiagaraStaticSwitchType : uint8
{
	Bool,
	Integer,
	Enum
};

USTRUCT()
struct FStaticSwitchTypeData
{
	GENERATED_USTRUCT_BODY()

	/** This determines how the switch value is interpreted */
	UPROPERTY()
	ENiagaraStaticSwitchType SwitchType;

	/** If the type is enum, this is the enum being switched on, otherwise it holds no sensible value */
	UPROPERTY()
	TObjectPtr<UEnum> Enum;

	/** If set, then this switch is not exposed but will rather be evaluated by the given compile-time constant */
	UPROPERTY()
	FName SwitchConstant;

	/** If true, a node will auto refresh under certain circumstances, like in post load or if the assigned enum changes */
	UPROPERTY()
	bool bAutoRefreshEnabled = false;

	UPROPERTY()
	bool bExposeAsPin = false;

	FStaticSwitchTypeData() : SwitchType(ENiagaraStaticSwitchType::Bool), Enum(nullptr)
	{ }
};

UCLASS(MinimalAPI)
class UNiagaraNodeStaticSwitch : public UNiagaraNodeUsageSelector, public FEnumEditorUtils::INotifyOnEnumChanged, public FNotifyHook
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY()
	FName InputParameterName;
	
	UPROPERTY(EditAnywhere, Category = "HiddenMetaData")
	FStaticSwitchTypeData SwitchTypeData;

	FNiagaraTypeDefinition GetInputType() const;

	void ChangeSwitchParameterName(const FName& NewName);
	void OnSwitchParameterTypeChanged(const FNiagaraTypeDefinition& OldType);

	void SetSwitchValue(int Value);
	void SetSwitchValue(const FCompileConstantResolver& ConstantResolver);
	TOptional<int> GetSwitchValue() const { return IsValueSet ? TOptional<int>(SwitchValue) : TOptional<int>();  }
	void ClearSwitchValue();
	/** If true then the value of this static switch is not set by the user but directly by the compiler via one of the engine constants (e.g. Emitter.Determinism). */
	bool IsSetByCompiler() const;

	bool IsDebugSwitch() const;

	bool IsSetByPin() const;
	UEdGraphPin* GetSelectorPin() const;
	
	/** This is a hack used in the translator to check for inconsistencies with old static switches before auto refresh was a thing */
	void CheckForOutdatedEnum(FTranslator* Translator) const;

	void UpdateCompilerConstantValue(FTranslator* Translator);

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual bool IsCompilerRelevant() const override { return false; }
	virtual void PostLoad() override;
	//~ End EdGraphNode Interface
	
	//~ Begin UNiagaraNode Interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual bool SubstituteCompiledPin(FTranslator* Translator, UEdGraphPin** LocallyOwnedPin) override;
	virtual UEdGraphPin* GetTracedOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bFilterForCompilation, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace = nullptr) const override;
	virtual UEdGraphPin* GetTracedOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bRecursive, bool bFilterForCompilation, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace = nullptr) const;
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* LocallyOwnedOutputPin, ENiagaraScriptUsage InUsage) const override;
	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const override;
	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;
	virtual void AddWidgetsToOutputBox(TSharedPtr<SVerticalBox> OutputBox) override;
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	virtual void ResolveNumerics(const UEdGraphSchema_Niagara* Schema, bool bSetInline, TMap<TPair<FGuid, UEdGraphNode*>, FNiagaraTypeDefinition>* PinCache) override;
	virtual ENiagaraNumericOutputTypeSelectionMode GetNumericOutputTypeSelectionMode() const override;
	//~ End UNiagaraNode Interface

	virtual TArray<int32> GetOptionValues() const override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:
	//~ Begin UNiagaraNodeUsageSelector Interface
	virtual bool AreInputCaseLabelsReadOnly() const override { return SwitchTypeData.SwitchType != ENiagaraStaticSwitchType::Integer || IsSetByPin(); }
	virtual void OnInputCaseLabelSubmitted(const FText& Text, ETextCommit::Type Arg, int32 InputCase) const override;
	virtual bool VerifyCaseLabelCandidate(const FText& InCandidate, FText& OutErrorMessage) const override;
	
	virtual FText GetInputCaseTooltip(int32 Case) const override;
	virtual FText GetInputCaseButtonTooltip(int32 Case) const override;
	virtual void OnInputCaseTooltipSubmitted(const FText& Text, ETextCommit::Type Arg, int32 InputCase) const override;

	virtual TSharedRef<SWidget> GetInputCaseContextOptions(int32 Case) override;
	
	virtual FString GetInputCaseName(int32 Case) const override;
	virtual FName GetOptionPinName(const FNiagaraVariable& Variable, int32 Value) const override;

	virtual void UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const override;
	//~ End UNiagaraNodeUsageSelector Interface

	virtual bool CanModifyPin(const UEdGraphPin* Pin) const override;
	virtual bool CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const override;
	virtual void MoveDynamicPin(UEdGraphPin* Pin, int32 MoveAmount) override;
private:
	/** INotifyOnEnumChanged interface */
	virtual void PreChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override;
	virtual void PostChange(const UUserDefinedEnum* Changed, FEnumEditorUtils::EEnumEditorChangeInfo ChangedType) override;

	/** FNotifyHook interface. Used to propagate changes in structure details views. */
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	
	/** This finds the first valid input pin index for the current switch value, returns false if no value can be found */
	bool GetVarIndex(FTranslator* Translator, int32 InputPinCount, int32& VarIndexOut) const;

	bool GetVarIndex(FTranslator* Translator, int32 InputPinCount, int32 Value, int32& VarIndexOut) const;

	/** Retrieve the script variable this switch currently represents. Can be nullptr in certain cases, such as if 'Expose as Pin' is active. */
	UNiagaraScriptVariable* GetStaticSwitchScriptVariable() const;
	
	void RemoveUnusedGraphParameter(const FNiagaraVariable& OldParameter);
	
	void AddIntegerInputPin();
	void RemoveIntegerInputPin();
	FText GetIntegerAddButtonTooltipText() const;
	FText GetIntegerRemoveButtonTooltipText() const;
	EVisibility ShowAddIntegerButton() const;
	EVisibility ShowRemoveIntegerButton() const;

	/** If true then the current SwitchValue should be used for compilation, otherwise the node is not yet connected to a constant value */
	bool IsValueSet = false;

	/** The current value that the node is evaluated with. For bool 0 is false, for enums the value is the index of the enum value. */
	int32 SwitchValue = 0;
};
