// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeUsageSelector.h"

#include "NiagaraCustomVersion.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraScriptSource.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeUsageSelector)

#define LOCTEXT_NAMESPACE "NiagaraNodeUsageSelector"

UNiagaraNodeUsageSelector::UNiagaraNodeUsageSelector(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNiagaraNodeUsageSelector::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// @TODO why do we need to have this post-change property here at all? 
	// Doing a null check b/c otherwise if doing a Duplicate via Ctrl-W, we die inside AllocateDefaultPins due to 
	// the point where we get this call not being completely formed.
	if (PropertyChangedEvent.Property != nullptr)
	{
		ReallocatePins();
	}
}

void UNiagaraNodeUsageSelector::PostLoad()
{
	Super::PostLoad();

	// restore num options per variable for old nodes. Only if we have at least 1 output variable.
	if (NumOptionsPerVariable == 0 && OutputVars.Num() > 0)
	{
		TArray<UEdGraphPin*> InputPins;
		GetInputPins(InputPins);

		// we remove all pins that are orphaned or have persistent guids as all option pins shouldn't have guids anyways. Used to filter out selector pin of the select node
		InputPins.RemoveAll([](UEdGraphPin* Pin)
		{
			return Pin->bOrphanedPin == true || Pin->PersistentGuid.IsValid();
		});

		TArray<int32> OptionValues = GetOptionValues();
		int32 TargetNumOptionsPerVariable = InputPins.Num() / OutputVars.Num();

		if(OptionValues.Num() != TargetNumOptionsPerVariable)
		{
			UE_LOGF(LogNiagaraEditor, Log, "Mismatch in option values and pin count detected. Please refresh script %ls. Setting to current values in the meantime.", *this->GetNiagaraGraph()->GetSource()->GetName());
		}
		
		NumOptionsPerVariable = TargetNumOptionsPerVariable;	
	}
}

bool UNiagaraNodeUsageSelector::AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const
{
	return Super::AllowNiagaraTypeForAddPin(InType) && InType != FNiagaraTypeDefinition::GetParameterMapDef();
}

void UNiagaraNodeUsageSelector::AddOptionPin(const FNiagaraVariable& OutputVariable, int32 Value, int32 InsertionSlot)
{
	UEdGraphPin* NewPin = CreatePin(EGPD_Input, UEdGraphSchema_Niagara::TypeDefinitionToPinType(OutputVariable.GetType()), GetOptionPinName(OutputVariable, Value), InsertionSlot);
	UNiagaraNode::SetPinDefaultToTypeDefaultIfUnset(NewPin);
	NewPin->PinFriendlyName = GetOptionPinFriendlyName(OutputVariable);
}

void UNiagaraNodeUsageSelector::AttemptUpdatePins()
{
	if(AreInputPinsOutdated() && GetNiagaraGraph()->IsCompilationCopy() == false)
	{
		ReallocatePins();
	}
}

TArray<int32> UNiagaraNodeUsageSelector::GetOptionValues() const
{
	TArray<int32> OptionValues;
	for(int32 Value = 0; Value < (int32) ENiagaraScriptGroup::Max; Value++)
	{
		OptionValues.Add(Value);
	}

	return OptionValues;
}

//void UNiagaraNodeUsageSelector::InsertInputPinsFor(const FNiagaraVariable& Var)
//{
//	TArray<int32> OptionValues = GetOptionValues();
//
//	// this should generally be the same anyways, but in case selector or output pins weren't specified yet these could differ
//	NumOptionsPerVariable = OptionValues.Num();
//
//	Pins.Reserve(Pins.Num() + NumOptionsPerVariable);
//
//	// we remove the error pins so we can re-add them after insertion of the new pins
//	TArray<UEdGraphPin*> ErrorPins = Pins.FilterByPredicate([=](UEdGraphPin* A)
//	{
//		return A->bOrphanedPin == true;
//	});
//
//	Pins.RemoveAll([=](UEdGraphPin* A)
//	{
//		return ErrorPins.Contains(A);
//	});
//
//	int32 NaturalIndex = 0;
//	for (int32 SlotIndex = OutputVars.Num() - 1; SlotIndex < OutputVars.Num() * NumOptionsPerVariable; SlotIndex += OutputVars.Num())
//	{
//		AddOptionPin(Var, OptionValues[NaturalIndex], SlotIndex);
//		NaturalIndex++;
//	}
//
//	// re-add them to the end of the list to keep error pins at the bottom
//	Pins.Append(ErrorPins);
//}

FText UNiagaraNodeUsageSelector::GetOptionPinFriendlyName(const FNiagaraVariable& Variable) const
{
	return FText::AsCultureInvariant(Variable.GetName().ToString());
}

bool UNiagaraNodeUsageSelector::VerifyCaseLabelCandidate(const FText& InCandidate, FText& OutErrorMessage) const
{
	if(InCandidate.IsEmptyOrWhitespace())
	{
		OutErrorMessage = LOCTEXT("CaseLabelCantBeEmpty", "Case can not be empty");
		return false;
	}

	return true;
}

FText UNiagaraNodeUsageSelector::GetInputCaseButtonTooltip(int32 Case) const
{
	return FText::FormatOrdered(LOCTEXT("SummonCaseLabelButtonTooltip", "If input is {0}.\nEdit the case label & tooltip."), FText::FromString(FString::FromInt(Case))); 
}

FString UNiagaraNodeUsageSelector::GetInputCaseName(int32 Case) const
{
	UEnum* ENiagaraScriptGroupEnum = StaticEnum<ENiagaraScriptGroup>();
	if(!ENiagaraScriptGroupEnum->GetNameStringByIndex(Case).IsEmpty())
	{
		return ENiagaraScriptGroupEnum->GetNameStringByValue((int64)Case);
	}

	return TEXT("");
}

FText UNiagaraNodeUsageSelector::GetInputCaseNameAsText(int32 Case) const
{
	return FText::FromString(GetInputCaseName(Case));
}

FName UNiagaraNodeUsageSelector::GetOptionPinName(const FNiagaraVariable& Variable, int32 Value) const
{
	return FName(Variable.GetName().ToString() + FString::Printf(TEXT(" if %s"), *GetInputCaseName(Value)));
}

bool UNiagaraNodeUsageSelector::AreInputPinsOutdated() const
{
	TArray<UEdGraphPin*> InputPins;
	GetInputPins(InputPins);

	InputPins.RemoveAll([](UEdGraphPin* Pin)
	{
		return Pin->bOrphanedPin == true || Pin->PersistentGuid.IsValid();
	});

	// this gets the new option values so we can retrieve the new pin names
	TArray<int32> OptionValues = GetOptionValues();

	// collect the new pin names
	TArray<FName> NewPinNames;
	for(int32 OptionIndex = 0; OptionIndex < OptionValues.Num(); OptionIndex++)
	{
		for(const FNiagaraVariable& Variable : OutputVars)
		{
			FName UpdatedPinName = GetOptionPinName(Variable, OptionValues[OptionIndex]);
			NewPinNames.Add(UpdatedPinName);		
		}
	}

	// this checks the current, possibly outdated, state for consistency. Should never be unequal but we are making sure.
	if (InputPins.Num() != NumOptionsPerVariable * OutputVars.Num())
	{
		return true;
	}

	// this checks the new, up to date pin count against the old pin count
	if(NewPinNames.Num() != InputPins.Num())
	{
		return true;
	}
	else
	{
		bool bAnyPinNameChangeDetected = false;
		for(int32 Index = 0; Index < NewPinNames.Num(); Index++)
		{
			if(!NewPinNames[Index].ToString().Equals(InputPins[Index]->PinName.ToString()))
			{
				bAnyPinNameChangeDetected = true;
				break;
			}
		}

		if (bAnyPinNameChangeDetected)
		{
			return true;
		}
	}

	return false;
}

void UNiagaraNodeUsageSelector::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	UEnum* ENiagaraScriptGroupEnum = StaticEnum<ENiagaraScriptGroup>();

	//Create the inputs for each path.
	NumOptionsPerVariable = (int32) ENiagaraScriptGroup::Max;
	for (int64 i = 0; i < NumOptionsPerVariable; i++)
	{
		const FString PathSuffix = ENiagaraScriptGroupEnum ? ( FString::Printf(TEXT(" if %s"), *ENiagaraScriptGroupEnum->GetNameStringByValue(i))) : TEXT("Error Unknown!");
		for (FNiagaraVariable& Var : OutputVars)
		{
			AddOptionPin(Var, static_cast<int32>(i));
		}
	}

	for (int32 Index = 0; Index < OutputVars.Num(); Index++)
	{
		const FNiagaraVariable& Var = OutputVars[Index];
		UEdGraphPin* NewPin = CreatePin(EGPD_Output, Schema->TypeDefinitionToPinType(Var.GetType()), Var.GetName());
		NewPin->PersistentGuid = OutputVarGuids[Index];
	}

	CreateAddPin(EGPD_Output);
}

bool UNiagaraNodeUsageSelector::RefreshFromExternalChanges()
{
	ReallocatePins();
	return true;
}

void UNiagaraNodeUsageSelector::AddWidgetsToInputBox(TSharedPtr<SVerticalBox> InputBox)
{
	// we only want to add separators if we actually have at least 1 variable
	if (OutputVars.Num() < 1)
	{
		return;
	}

	TArray<UEdGraphPin*> InputPins;
	GetInputPins(InputPins);

	InputPins.RemoveAll([](UEdGraphPin* Pin)
	{
		return Pin->bOrphanedPin == true;
	});
	
	int32 WidgetIndexOffset = 0;
	// insert separators to make the grouping apparent.
	TArray<int32> OptionValues = GetOptionValues();
	for (int32 Idx = 0; Idx < OutputVars.Num() * OptionValues.Num(); Idx += OutputVars.Num())
	{
		const int32 OptionValueIndex = Idx / OutputVars.Num();
		const int32 OptionValue = OptionValues[OptionValueIndex];
		// adding an offset to account for the already added separators
		InputBox->InsertSlot(Idx + WidgetIndexOffset)
		.Padding(5.f, 5.f, 0.f, 2.f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("If ")))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(AreInputCaseLabelsReadOnly() ? 0 : 1)
				+ SWidgetSwitcher::Slot()
				[
					SNew(STextBlock)
					.Text_UObject(this, &UNiagaraNodeUsageSelector::GetInputCaseNameAsText, OptionValue)
				]
				+ SWidgetSwitcher::Slot()
				[
					SNew(SComboButton)
					.OnGetMenuContent_UObject(this, &UNiagaraNodeUsageSelector::GetInputCaseContextOptions, OptionValue)
					.ToolTipText_UObject(this, &UNiagaraNodeUsageSelector::GetInputCaseButtonTooltip, OptionValue)
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text_UObject(this, &UNiagaraNodeUsageSelector::GetInputCaseNameAsText, OptionValue)
						]
					]
				]
			]
		];
		WidgetIndexOffset++;
		
		InputBox->InsertSlot(Idx + WidgetIndexOffset)
		.VAlign(VAlign_Center)
		[
			SNew(SSeparator)
		];
		WidgetIndexOffset++;
	}
}


void UNiagaraNodeUsageSelector::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	FPinCollectorArray InputPins;
	GetCompilationInputPins(InputPins);
	FPinCollectorArray OutputPins;
	GetCompilationOutputPins(OutputPins);

	ENiagaraScriptUsage CurrentUsage = Translator->GetCurrentUsage();
	ENiagaraScriptGroup UsageGroup = ENiagaraScriptGroup::Max;
	if (UNiagaraScript::ConvertUsageToGroup(CurrentUsage, UsageGroup))
	{
		int32 VarIdx = 0;
		for (int64 i = 0; i < (int64)ENiagaraScriptGroup::Max; i++)
		{
			if ((int64)UsageGroup == i)
			{
				break;
			}

			VarIdx += OutputVars.Num();
		}

		Outputs.SetNumUninitialized(OutputPins.Num());
		for (int32 i = 0; i < OutputVars.Num(); i++)
		{
			int32 InputIdx = Translator->CompileInputPin(InputPins[VarIdx + i]);
			Outputs[i] = InputIdx;
		}
	}
	else
	{
		Translator->Error(LOCTEXT("InvalidUsage", "Invalid script usage"), this, nullptr);
	}
}

UEdGraphPin* UNiagaraNodeUsageSelector::GetPassThroughPin(const UEdGraphPin* LocallyOwnedOutputPin, ENiagaraScriptUsage InUsage) const
{
	check(Pins.Contains(LocallyOwnedOutputPin) && LocallyOwnedOutputPin->Direction == EGPD_Output);

	ENiagaraScriptGroup UsageGroup = ENiagaraScriptGroup::Max;
	if (UNiagaraScript::ConvertUsageToGroup(InUsage, UsageGroup))
	{
		int32 VarIdx = 0;
		for (int64 i = 0; i < (int64)ENiagaraScriptGroup::Max; i++)
		{
			if ((int64)UsageGroup == i)
			{
				for (int32 j = 0; j < OutputVars.Num(); j++)
				{
					UEdGraphPin* OutputPin = GetOutputPin(j);
					if (OutputPin == LocallyOwnedOutputPin)
					{
						VarIdx += j;
						break;
					}
				}
				break;
			}

			VarIdx += OutputVars.Num();
		}
		UEdGraphPin* InputPin = GetInputPin(VarIdx);
		if (InputPin)
		{
			return InputPin;
		}
	}
	return nullptr;
}

void UNiagaraNodeUsageSelector::UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const
{
	static const FName NAME_Usage = TEXT("Usage");

	switch (InOutFunctionAliasContext.CompileUsage)
	{
	case ENiagaraScriptUsage::SystemSpawnScript:
	case ENiagaraScriptUsage::SystemUpdateScript:
		InOutFunctionAliasContext.AddPermutationIdentifier(NAME_Usage, TEXT("System"));
		break;
	case ENiagaraScriptUsage::EmitterSpawnScript:
	case ENiagaraScriptUsage::EmitterUpdateScript:
		InOutFunctionAliasContext.AddPermutationIdentifier(NAME_Usage, TEXT("Emitter"));
		break;
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
	case ENiagaraScriptUsage::ParticleGPUComputeScript:
		InOutFunctionAliasContext.AddPermutationIdentifier(NAME_Usage, TEXT("Particle"));
		break;
	}
}

void UNiagaraNodeUsageSelector::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	FPinCollectorArray InputPins;
	GetInputPins(InputPins);
	FPinCollectorArray OutputPins;
	GetOutputPins(OutputPins);

	ENiagaraScriptUsage BaseUsage = OutHistory.GetBaseUsageContext();
	ENiagaraScriptUsage CurrentUsage = OutHistory.GetCurrentUsageContext();	

	check(OutputPins.Num() - 1 == OutputVars.Num());

	ENiagaraScriptGroup UsageGroup = ENiagaraScriptGroup::Max;
	if (UNiagaraScript::ConvertUsageToGroup(CurrentUsage, UsageGroup))
	{
		if (bRecursive)
		{
			int32 VarIdx = 0;
			for (int64 i = 0; i < (int64)ENiagaraScriptGroup::Max; i++)
			{
				if ((int64)UsageGroup == i)
				{
					break;
				}

				VarIdx += OutputVars.Num();
			}

			for (int32 i = 0; i < OutputVars.Num(); i++)
			{
				//OutHistory.VisitInputPin(InputPins[VarIdx + i], this, bFilterForCompilation);
				RegisterPassthroughPin(OutHistory, InputPins[VarIdx + i], GetOutputPin(i), bFilterForCompilation, true);
			}
		}
	}
}

bool UNiagaraNodeUsageSelector::AllowExternalPinTypeChanges(const UEdGraphPin* InGraphPin) const
{
	// only allow pin type changes for output pins
	if (InGraphPin->Direction == EGPD_Output)
	{
		return true;
	}

	return false;
}

bool UNiagaraNodeUsageSelector::AllowNiagaraTypeForPinTypeChange(const FNiagaraTypeDefinition& InType, UEdGraphPin* Pin) const
{
	return AllowNiagaraTypeForAddPin(InType);
}

bool UNiagaraNodeUsageSelector::OnNewPinTypeRequested(UEdGraphPin* PinToChange, FNiagaraTypeDefinition NewType)
{
	auto FindPredicate = [=](const FGuid& Guid) { return Guid == PinToChange->PersistentGuid; };
	int32 FoundIndex = OutputVarGuids.IndexOfByPredicate(FindPredicate);
	if (FoundIndex != INDEX_NONE)
	{
		TSet<FName> OutputNames;
		for (const FNiagaraVariable& Output : OutputVars)
		{
			OutputNames.Add(Output.GetName());
		}
		FName OutputName = FNiagaraUtilities::GetUniqueName(NewType.GetFName(), OutputNames);

		OutputVars.RemoveAt(FoundIndex);
		OutputVars.EmplaceAt(FoundIndex, FNiagaraVariable(NewType, OutputName));
		ReallocatePins();
		return true;
	}

	return false;
}

FGuid UNiagaraNodeUsageSelector::AddOutput(FNiagaraTypeDefinition Type, const FName& Name)
{
	FNiagaraVariable NewOutput(Type, Name);
	FGuid Guid = FGuid::NewGuid();
	OutputVars.Add(NewOutput);
	OutputVarGuids.Add(Guid);
	return Guid;
}

void UNiagaraNodeUsageSelector::OnPinRemoved(UEdGraphPin* PinToRemove)
{
	auto FindPredicate = [=](const FGuid& Guid) { return Guid == PinToRemove->PersistentGuid; };
	int32 FoundIndex = OutputVarGuids.IndexOfByPredicate(FindPredicate);
	if (FoundIndex != INDEX_NONE && PinToRemove->bOrphanedPin == false)
	{
		OutputVarGuids.RemoveAt(FoundIndex);
		OutputVars.RemoveAt(FoundIndex);
	}
	ReallocatePins();
}

void UNiagaraNodeUsageSelector::OnNewTypedPinAdded(UEdGraphPin*& NewPin)
{
	Super::OnNewTypedPinAdded(NewPin);

	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraTypeDefinition OutputType = Schema->PinToTypeDefinition(NewPin);

	TSet<FName> OutputNames;
	for (const FNiagaraVariable& Output : OutputVars)
	{
		OutputNames.Add(Output.GetName());
	}
	FName OutputName = FNiagaraUtilities::GetUniqueName(OutputType.GetFName(), OutputNames);
	NewPin->PinName = OutputName;
	FGuid Guid = AddOutput(OutputType, OutputName);

	// Update the pin's data too so that it's connection is maintained after reallocating.
	NewPin->PersistentGuid = Guid;

	// reallocate pins to create the input pins for the new output pin
	ReallocatePins();
	
	// we refresh the new pin ref here as reallocation will have recreated it and the calling function requires the pin to be valid
	NewPin = GetPinByPersistentGuid(Guid);
}

void UNiagaraNodeUsageSelector::OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName)
{
	auto FindPredicate = [=](const FGuid& Guid) { return Guid == RenamedPin->PersistentGuid; };
	int32 FoundIndex = OutputVarGuids.IndexOfByPredicate(FindPredicate);
	if(FoundIndex != INDEX_NONE)
	{
		TSet<FName> OutputNames;
		for (int32 Index = 0; Index < OutputVars.Num(); Index++)
		{
			if (FoundIndex != Index)
			{
				OutputNames.Add(OutputVars[Index].GetName());
			}
		}
		const FName OutputName = FNiagaraUtilities::GetUniqueName(RenamedPin->PinName, OutputNames);
		OutputVars[FoundIndex].SetName(OutputName);

		 TArray<int32> OptionValues = GetOptionValues();
		 int32 OptionsCount = OptionValues.Num();
		
		 TArray<UEdGraphPin*> InputPins;
		 GetInputPins(InputPins);
		
		 InputPins.RemoveAll([](UEdGraphPin* Pin)
		 {
		 	return Pin->bOrphanedPin == true;
		 });

		if(OptionValues.Num() > 0)
		{
			int32 OptionIndex = 0;
			 for(int32 InputPinIndex = FoundIndex; InputPinIndex < OutputVars.Num() * OptionsCount; InputPinIndex += OutputVars.Num())
			 {
		 		InputPins[InputPinIndex]->PinName = GetOptionPinName(OutputVars[FoundIndex], OptionValues[OptionIndex]);
			 	InputPins[InputPinIndex]->PinFriendlyName = GetOptionPinFriendlyName(OutputVars[FoundIndex]);
			 	OptionIndex++;
			 }
		}
	}
}

bool UNiagaraNodeUsageSelector::CanRenamePin(const UEdGraphPin* Pin) const
{
	return Super::CanRenamePin(Pin) && Pin->Direction == EGPD_Output;
}

bool UNiagaraNodeUsageSelector::CanRemovePin(const UEdGraphPin* Pin) const
{
	return Super::CanRemovePin(Pin) && Pin->Direction == EGPD_Output;
}


FText UNiagaraNodeUsageSelector::GetTooltipText() const
{
	return LOCTEXT("UsageSelectorDesc", "If the usage matches, then the traversal will follow that path.");
}

FText UNiagaraNodeUsageSelector::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UsageSelectorTitle", "Select by Use");
}

#undef LOCTEXT_NAMESPACE

