// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NiagaraNodeWithDynamicPins.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraNodeUsageSelector.generated.h"

UCLASS(MinimalAPI)
class UNiagaraNodeUsageSelector: public UNiagaraNodeWithDynamicPins
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY()
	TArray<FNiagaraVariable> OutputVars;
	
	UPROPERTY()
	TArray<FGuid> OutputVarGuids;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	FGuid SelectorGuid;
	
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual void OnPinRemoved(UEdGraphPin* PinToRemove) override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End EdGraphNode Interface

	//~ Begin UNiagaraNode Interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual bool RefreshFromExternalChanges() override;
	virtual void AddWidgetsToInputBox(TSharedPtr<SVerticalBox> InputBox) override;
	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;
	virtual bool AllowExternalPinTypeChanges(const UEdGraphPin* InGraphPin) const override;
	virtual bool AllowNiagaraTypeForPinTypeChange(const FNiagaraTypeDefinition& InType, UEdGraphPin* Pin) const override;
	virtual bool OnNewPinTypeRequested(UEdGraphPin* PinToChange, FNiagaraTypeDefinition NewType) override;
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* LocallyOwnedOutputPin, ENiagaraScriptUsage InUsage) const override;
	virtual void UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const override;
	//~ End UNiagaraNode Interface

	/** Helper function to create a variable to add to the OutputVars and FGuid to add to OutputVarGuids. */
	FGuid AddOutput(FNiagaraTypeDefinition Type, const FName& Name);

	void AttemptUpdatePins();
	bool AreInputPinsOutdated() const;

	/** Retrieves option values. Required to determine separator entries since we no longer have case information after pin creation. */
	virtual TArray<int32> GetOptionValues() const;

protected:
	/** Use this function to add an option pin in allocation or insertion. Optionally at a specific slot. */
	void AddOptionPin(const FNiagaraVariable& OutputVariable, int32 Value, int32 InsertionSlot = INDEX_NONE);

	virtual bool AreInputCaseLabelsReadOnly() const { return true; }
	virtual void OnInputCaseLabelSubmitted(const FText& Text, ETextCommit::Type Arg, int32 OptionValue) const {}
	virtual bool VerifyCaseLabelCandidate(const FText& InCandidate, FText& OutErrorMessage) const;

	virtual FText GetInputCaseTooltip(int32 Case) const { return FText::GetEmpty(); }
	virtual FText GetInputCaseButtonTooltip(int32 Case) const;
	virtual void OnInputCaseTooltipSubmitted(const FText& Text, ETextCommit::Type Arg, int32 OptionValue) const {}
	
	virtual TSharedRef<SWidget> GetInputCaseContextOptions(int32 Case) { return SNullWidget::NullWidget; }

	/** Used to determine the text for the separators */
	virtual FString GetInputCaseName(int32 Case) const;
	FText GetInputCaseNameAsText(int32 Case) const;
	
	/** Ideally we wouldn't need this due to the separators but to maintain backwards compatibility pin names needs to stay consistent with what they were */
	virtual FName GetOptionPinName(const FNiagaraVariable& Variable, int32 Value) const;
	
	//~ Begin UNiagaraNodeWithDynamicPins Interface
	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName) override;
	virtual bool CanRenamePin(const UEdGraphPin* Pin) const override;
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const override;
	virtual bool CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const override { return false; }
	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const override;
	//~ End UNiagaraNodeWithDynamicPins Interface

protected:
	/** Primarily used for integer options where the number of entries is not derivable from something else (i.e. enum or bool which clearly define entries) */
	UPROPERTY()
	int32 NumOptionsPerVariable = 0;
private:
	/** private UNiagaraNodeWithDynamicPins interface */
	virtual void OnNewTypedPinAdded(UEdGraphPin*& NewPin) override;

	/** The output variable's name is used as the pin friendly name, as we have case labels to make them readable. */
	FText GetOptionPinFriendlyName(const FNiagaraVariable& Variable) const;
};
