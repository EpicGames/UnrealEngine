// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraNodeWithDynamicPins.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraGraph.h"
#include "Framework/Commands/UIAction.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraEditorUtilities.h"
#include "Framework/Application/SlateApplication.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeWithDynamicPins)

#define LOCTEXT_NAMESPACE "NiagaraNodeWithDynamicPins"

const FName UNiagaraNodeWithDynamicPins::AddPinSubCategory("DynamicAddPin");

void UNiagaraNodeWithDynamicPins::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);

	// Check if an add pin was connected and convert it to a typed connection.
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	if (Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryMisc && Pin->PinType.PinSubCategory == AddPinSubCategory && Pin->LinkedTo.Num() > 0)
	{
		FNiagaraTypeDefinition LinkedPinType = Schema->PinToTypeDefinition(Pin->LinkedTo[0]);

		// Handle converting numerics to their inferred type in the case where preserving the numeric isn't a valid option, like Parameter Map Get Nodes
		if (LinkedPinType == FNiagaraTypeDefinition::GetGenericNumericDef() && Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryMisc && 
			Pin->PinType.PinSubCategory == UNiagaraNodeWithDynamicPins::AddPinSubCategory && !AllowNiagaraTypeForAddPin(LinkedPinType))
		{
			LinkedPinType = GetNiagaraGraph()->GetCachedNumericConversion(Pin->LinkedTo[0]);
		}
		Pin->PinType = Schema->TypeDefinitionToPinType(LinkedPinType);

		FName NewPinName;
		FText NewPinFriendlyName;
		FNiagaraParameterHandle LinkedPinHandle(Pin->LinkedTo[0]->PinName);
		FNiagaraNamespaceMetadata LinkedPinNamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(LinkedPinHandle.GetHandleParts());
		if (LinkedPinNamespaceMetadata.IsValid())
		{
			// If the linked pin had valid namespace metadata then it's a parameter pin and we only want the name portion of the parameter.
			NewPinName = LinkedPinHandle.GetHandleParts().Last();
		}
		else 
		{
			NewPinName = Pin->LinkedTo[0]->PinName;
			NewPinFriendlyName = Pin->LinkedTo[0]->PinFriendlyName;
		}
		
		Pin->PinName = NewPinName;
		Pin->PinFriendlyName = NewPinFriendlyName;

		CreateAddPin(Pin->Direction);
		OnNewTypedPinAdded(Pin);
		MarkNodeRequiresSynchronization(__FUNCTION__, true);
		//GetGraph()->NotifyGraphChanged();
	}
}

UEdGraphPin* GetAddPin(TArray<UEdGraphPin*> Pins, EEdGraphPinDirection Direction)
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == Direction &&
			Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryMisc && 
			Pin->PinType.PinSubCategory == UNiagaraNodeWithDynamicPins::AddPinSubCategory)
		{
			return Pin;
		}
	}
	return nullptr;
}

bool UNiagaraNodeWithDynamicPins::AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const
{
	return InType.GetScriptStruct() != nullptr
		&& InType != FNiagaraTypeDefinition::GetGenericNumericDef()
		&& !InType.IsInternalType();
}

UEdGraphPin* UNiagaraNodeWithDynamicPins::RequestNewTypedPin(EEdGraphPinDirection Direction, const FNiagaraTypeDefinition& Type)
{
	TStringBuilder<128> DefaultName;
	if (Direction == EGPD_Input)
	{
		FPinCollectorArray InPins;
		GetInputPins(InPins);
		DefaultName << TEXT("Input ") << InPins.Num();
	}
	else
	{
		FPinCollectorArray OutPins;
		GetOutputPins(OutPins);
		DefaultName << TEXT("Output ") << OutPins.Num();
	}
	return RequestNewTypedPin(Direction, Type, DefaultName.ToString());
}

UEdGraphPin* UNiagaraNodeWithDynamicPins::RequestNewTypedPin(EEdGraphPinDirection Direction, const FNiagaraTypeDefinition& Type, const FName InName)
{
	FScopedTransaction Transaction(LOCTEXT("NewPinAdded", "Added new pin"));
	
	Modify();
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	UEdGraphPin* AddPin = GetAddPin(GetAllPins(), Direction);
	checkf(AddPin != nullptr, TEXT("Add pin is missing"));
	AddPin->Modify();
	AddPin->PinType = Schema->TypeDefinitionToPinType(Type);
	AddPin->PinName = InName;

	CreateAddPin(Direction);
	// we pass the pointer in as reference in case we want to reallocate so the overriding node has a chance to restore the pointer
	OnNewTypedPinAdded(AddPin);

	checkf(AddPin != nullptr && AddPin->IsPendingKill() == false, 
		TEXT("The pin was invalidated. Most likely due to reallocation in OnNewTypedPinAdded and failure to restore the pin pointer"));
	
	MarkNodeRequiresSynchronization(__FUNCTION__, true);

	return AddPin;
}

void UNiagaraNodeWithDynamicPins::CreateAddPin(EEdGraphPinDirection Direction)
{
	if (!AllowDynamicPins())
	{
		return;
	}
	CreatePin(Direction, FEdGraphPinType(UEdGraphSchema_Niagara::PinCategoryMisc, AddPinSubCategory, nullptr, EPinContainerType::None, false, FEdGraphTerminalType()), TEXT("Add"));
}

bool UNiagaraNodeWithDynamicPins::IsAddPin(const UEdGraphPin* Pin)
{
	return Pin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryMisc && 
		Pin->PinType.PinSubCategory == AddPinSubCategory;
}

bool UNiagaraNodeWithDynamicPins::IsExecPin(const UEdGraphPin* Pin) const
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	return Pin->PinType == Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()) && Pin->PinName == TEXT("");
}

bool UNiagaraNodeWithDynamicPins::CanModifyPin(const UEdGraphPin* Pin) const
{
	if(IsAddPin(Pin) || IsParameterMapPin(Pin))
	{
		return false;
	}

	return true;
}

bool UNiagaraNodeWithDynamicPins::CanRenamePin(const UEdGraphPin* Pin) const
{	
	return CanModifyPin(Pin) && !IsExecPin(Pin) && !Pin->bOrphanedPin;
}

bool UNiagaraNodeWithDynamicPins::CanRemovePin(const UEdGraphPin* Pin) const
{
	return CanModifyPin(Pin) && !IsExecPin(Pin);
}

bool UNiagaraNodeWithDynamicPins::CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const
{
	if(!CanModifyPin(Pin) || IsExecPin(Pin) || Pin->bOrphanedPin)
	{
		return false;
	}
	
	TArray<const UEdGraphPin*> PinArray;
	if(Pin->Direction == EGPD_Output)
	{
		GetOutputPins(PinArray);
	}
	else
	{
		GetInputPins(PinArray);
	}

	int32 Index = PinArray.Find(Pin);
	if(PinArray.IsValidIndex(Index + DirectionToMove))
	{
		const UEdGraphPin* PinToMoveTo = PinArray[Index + DirectionToMove];
		if(!CanModifyPin(PinToMoveTo) || PinToMoveTo->bOrphanedPin)
		{
			return false;
		}
		return true;
	}

	return false;
}

void UNiagaraNodeWithDynamicPins::MoveDynamicPin(UEdGraphPin* Pin, int32 MoveAmount)
{
	Modify();
	Pin->Modify();

	bool bModifiedPins = false;
	int32 DirectionToMove = MoveAmount > 0 ? 1 : -1;
	for (int Step = 0; Step < abs(MoveAmount); Step++)
	{
		FPinCollectorArray SameDirectionPins;
		if (Pin->Direction == EGPD_Input)
		{
			GetInputPins(SameDirectionPins);
		}
		else
		{
			GetOutputPins(SameDirectionPins);
		}
		
		for (int32 i = 0; i < SameDirectionPins.Num(); i++)
		{
			if (SameDirectionPins[i] == Pin)
			{
				if (i + DirectionToMove >= 0 && i + DirectionToMove < SameDirectionPins.Num())
				{
					UEdGraphPin* PinOld = SameDirectionPins[i + DirectionToMove];
					if (PinOld)
					{
						PinOld->Modify();
					}

					int32 RealPinIdx = INDEX_NONE;
					int32 SwapRealPinIdx = INDEX_NONE;
					Pins.Find(Pin, RealPinIdx);
					Pins.Find(PinOld, SwapRealPinIdx);
				
					Pins[SwapRealPinIdx] = Pin;
					Pins[RealPinIdx] = PinOld;
					//GetGraph()->NotifyGraphChanged();

					bModifiedPins = true;
					break;
				}
			}
		}
	}

	if (bModifiedPins)
	{
		MarkNodeRequiresSynchronization(__FUNCTION__, true);
	}
}

bool UNiagaraNodeWithDynamicPins::IsValidPinToCompile(UEdGraphPin* Pin) const
{
	return !IsAddPin(Pin) && Super::IsValidPinToCompile(Pin);
}

void UNiagaraNodeWithDynamicPins::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);
	if (Context->Pin != nullptr)
	{
		FToolMenuSection& Section = Menu->AddSection("EditPin", LOCTEXT("EditPinMenuHeader", "Edit Pin"));
		if (CanRenamePinFromContextMenu(Context->Pin))
		{
			UEdGraphPin* Pin = const_cast<UEdGraphPin*>(Context->Pin);
			TSharedRef<SWidget> RenameWidget =
				SNew(SBox)
				.WidthOverride(100)
				.Padding(FMargin(5, 0, 0, 0))
				[
					SNew(SEditableTextBox)
					.Text_UObject(this, &UNiagaraNodeWithDynamicPins::GetPinNameText, Pin)
					.OnTextCommitted_UObject(const_cast<UNiagaraNodeWithDynamicPins*>(this), &UNiagaraNodeWithDynamicPins::PinNameTextCommitted, Pin)
					.OnVerifyTextChanged_UObject(this, &UNiagaraNodeWithDynamicPins::VerifyEditablePinName, Context->Pin)
				];
			Section.AddEntry(FToolMenuEntry::InitWidget("RenameWidget", RenameWidget, LOCTEXT("NameMenuItem", "Name")));
		}
		else if (CanRenamePin(Context->Pin))
		{
			Section.AddMenuEntry(
				NAME_None,
				LOCTEXT("RenameDynamicPin", "Rename pin"),
				LOCTEXT("RenameDynamicPinToolTip", "Rename this pin."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"),
				FUIAction(FExecuteAction::CreateUObject(const_cast<UNiagaraNodeWithDynamicPins*>(this), &UNiagaraNodeWithDynamicPins::RenameDynamicPinFromMenu, const_cast<UEdGraphPin*>(Context->Pin))));
		}

		if (CanRemovePin(Context->Pin))
		{
			Section.AddMenuEntry(
				"RemoveDynamicPin",
				LOCTEXT("RemoveDynamicPin", "Remove pin"),
				LOCTEXT("RemoveDynamicPinToolTip", "Remove this pin and any connections."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
				FUIAction(FExecuteAction::CreateUObject(const_cast<UNiagaraNodeWithDynamicPins*>(this), &UNiagaraNodeWithDynamicPins::RemoveDynamicPinFromMenu, const_cast<UEdGraphPin*>(Context->Pin))));
		}
		
		FPinCollectorArray SameDirectionPins;
		if (Context->Pin->Direction == EGPD_Input)
		{
			GetInputPins(SameDirectionPins);
		}
		else
		{
			GetOutputPins(SameDirectionPins);
		}
		int32 PinIdx = INDEX_NONE;
		SameDirectionPins.Find(const_cast<UEdGraphPin*>(Context->Pin), PinIdx);

		FText MoveUpLabel = LOCTEXT("MoveDynamicPinUp", "Move pin up");
		if (PinIdx != 0 && CanMovePin(Context->Pin, -1))
		{
			FText MoveUpTooltip = LOCTEXT("MoveDynamicPinToolTipUp", "Move this pin and any connections up.");
			if (CanMovePin(Context->Pin, -2))
			{
				Section.AddSubMenu(
					"MoveDynamicPinUp",
					MoveUpLabel, 
					MoveUpTooltip,
					FNewToolMenuDelegate::CreateLambda([=,this](UToolMenu* InSubMenuBuilder)
					{
						FToolMenuSection& SubSection = InSubMenuBuilder->FindOrAddSection("MovePin");
						for (int i = PinIdx - 1; i >= 0; i--)
						{
							int32 MoveAmount = i - PinIdx;
							if (!CanMovePin(Context->Pin, MoveAmount))
							{
								break;
							}
							FText PinName = FText::FromName(SameDirectionPins[i]->PinName);
							SubSection.AddMenuEntry(FName("MoveDynamicPinUp" + FString::FromInt(i)),
								FText::Format(LOCTEXT("MoveDynamicPinUpLabel", "Move up {0} - above '{1}'"), abs(MoveAmount), PinName),
								MoveUpTooltip,
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ChevronUp"),
								FUIAction(FExecuteAction::CreateUObject(const_cast<UNiagaraNodeWithDynamicPins*>(this), &UNiagaraNodeWithDynamicPins::MoveDynamicPinFromMenu, const_cast<UEdGraphPin*>(Context->Pin), MoveAmount))
							);
						}
					}));
			}
			else
			{
				Section.AddMenuEntry(
				"MoveDynamicPinUp",
				MoveUpLabel,
				LOCTEXT("MoveDynamicPinToolTipUpSlot", "Move this pin and any connections one slot up."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ChevronUp"),
				FUIAction(FExecuteAction::CreateUObject(const_cast<UNiagaraNodeWithDynamicPins*>(this), &UNiagaraNodeWithDynamicPins::MoveDynamicPinFromMenu, const_cast<UEdGraphPin*>(Context->Pin), -1)));
			}
		}

		FText MoveDownLabel = LOCTEXT("MoveDynamicPinDown", "Move pin down");
		if (PinIdx >= 0 && CanMovePin(Context->Pin, 1) && PinIdx < SameDirectionPins.Num() - 1)
		{
			FText MoveDownTooltip = LOCTEXT("MoveDynamicPinToolTipDown", "Move this pin and any connections down.");
			if (CanMovePin(Context->Pin, 2))
			{
				Section.AddSubMenu(
					"MoveDynamicPinDown",
					MoveDownLabel, 
					MoveDownTooltip,
					FNewToolMenuDelegate::CreateLambda([=, this](UToolMenu* InSubMenuBuilder)
				{
					FToolMenuSection& SubSection = InSubMenuBuilder->FindOrAddSection("MovePin");
					for (int i = PinIdx + 1; i < SameDirectionPins.Num(); i++)
					{
						int32 MoveAmount = i - PinIdx;
						if (!CanMovePin(Context->Pin, MoveAmount))
						{
							break;
						}
						FText PinName = FText::FromName(SameDirectionPins[i]->PinName);
						SubSection.AddMenuEntry(FName("MoveDynamicPinDown" + FString::FromInt(i)),
							FText::Format(LOCTEXT("MoveDynamicPinDownLabel", "Move down {0} - below '{1}'"), abs(MoveAmount), PinName),
							MoveDownTooltip,
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ChevronDown"),
							FUIAction(FExecuteAction::CreateUObject(const_cast<UNiagaraNodeWithDynamicPins*>(this), &UNiagaraNodeWithDynamicPins::MoveDynamicPinFromMenu, const_cast<UEdGraphPin*>(Context->Pin), MoveAmount))
						);
					}
				}));
			}
			else
			{
				Section.AddMenuEntry(
					"MoveDynamicPinDown",
					MoveDownLabel,
					MoveDownTooltip,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ChevronDown"),
					FUIAction(FExecuteAction::CreateUObject(const_cast<UNiagaraNodeWithDynamicPins*>(this), &UNiagaraNodeWithDynamicPins::MoveDynamicPinFromMenu, const_cast<UEdGraphPin*>(Context->Pin), 1)));
			}
		}
		
	}
}

void UNiagaraNodeWithDynamicPins::AddParameter(FNiagaraVariable Parameter, TEnumAsByte<EEdGraphPinDirection> Direction)
{
	// discard return pin, so the function can be used as delegate
	AddParameterPin(Parameter, Direction);
}

UEdGraphPin* UNiagaraNodeWithDynamicPins::AddParameterPin(FNiagaraVariable Parameter, TEnumAsByte<EEdGraphPinDirection> Direction)
{
	if (this->IsA<UNiagaraNodeParameterMapBase>())
	{
		// Parameter map type nodes create new parameters when adding pins.
		FScopedTransaction AddNewPinTransaction(LOCTEXT("AddNewPinTransaction", "Add pin to node"));
		
		UNiagaraGraph* Graph = GetNiagaraGraph();
		checkf(Graph != nullptr, TEXT("Failed to get niagara graph when adding pin!"));
		
		// Resolve the unique parameter name before adding to the graph if the current parameter name is not reserved.
		if (FNiagaraConstants::FindEngineConstant(Parameter) == nullptr)
		{
			Parameter.SetName(Graph->MakeUniqueParameterName(Parameter.GetName()));
		}

		Graph->Modify();
		Graph->AddParameter(Parameter);

		Modify();
		return this->RequestNewTypedPin(Direction, Parameter.GetType(), Parameter.GetName());
	}
	return RequestNewTypedPin(Direction, Parameter.GetType(), Parameter.GetName());
}

void UNiagaraNodeWithDynamicPins::AddParameter(const UNiagaraScriptVariable* ScriptVar, TEnumAsByte<EEdGraphPinDirection> Direction)
{
	// discard return pin, so the function can be used as delegate
	AddParameterPin(ScriptVar, Direction);
}

UEdGraphPin* UNiagaraNodeWithDynamicPins::AddParameterPin(const UNiagaraScriptVariable* ScriptVar, TEnumAsByte<EEdGraphPinDirection> Direction)
{
	const FNiagaraVariable& Parameter = ScriptVar->Variable;
	if (this->IsA<UNiagaraNodeParameterMapBase>())
	{
		// Parameter map type nodes create new parameters when adding pins.
		FScopedTransaction AddNewPinTransaction(LOCTEXT("AddNewPinTransaction", "Add pin to node"));

		UNiagaraGraph* Graph = GetNiagaraGraph();
		checkf(Graph != nullptr, TEXT("Failed to get niagara graph when adding pin!"));

		Graph->Modify();
		Graph->AddParameter(ScriptVar);

		Modify();
		return this->RequestNewTypedPin(Direction, Parameter.GetType(), Parameter.GetName());
	}
	return RequestNewTypedPin(Direction, Parameter.GetType(), Parameter.GetName());
}

void UNiagaraNodeWithDynamicPins::AddExistingParameter(FNiagaraVariable Parameter, const UEdGraphPin* AddPin)
{
	if (this->IsA<UNiagaraNodeParameterMapBase>())
	{
		// Parameter map type nodes create new parameters when adding pins.
		FScopedTransaction AddNewPinTransaction(LOCTEXT("AddNewPinTransaction", "Add pin to node"));
		
		UNiagaraGraph* Graph = GetNiagaraGraph();
		checkf(Graph != nullptr, TEXT("Failed to get niagara graph when adding pin!"));
		
		Modify();
		UEdGraphPin* Pin = this->RequestNewTypedPin(AddPin->Direction, Parameter.GetType(), Parameter.GetName());
	}
	else
	{
		RequestNewTypedPin(AddPin->Direction, Parameter.GetType(), Parameter.GetName());
	}
}

void UNiagaraNodeWithDynamicPins::RemoveDynamicPin(UEdGraphPin* Pin)
{
	RemovePin(Pin);
	MarkNodeRequiresSynchronization(__FUNCTION__, true);
}

FText UNiagaraNodeWithDynamicPins::GetPinNameText(UEdGraphPin* Pin) const
{
	return FText::FromName(Pin->PinName);
}

void UNiagaraNodeWithDynamicPins::PinNameTextCommitted(const FText& Text, ETextCommit::Type CommitType, UEdGraphPin* Pin)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		FScopedTransaction RemovePinTransaction(LOCTEXT("RenamePinTransaction", "Rename pin"));
		Modify();
		FString PinOldName = Pin->PinName.ToString();
		Pin->PinName = *Text.ToString();
		OnPinRenamed(Pin, PinOldName);
		MarkNodeRequiresSynchronization(__FUNCTION__, true);
	}
}

void UNiagaraNodeWithDynamicPins::RenameDynamicPinFromMenu(UEdGraphPin* Pin)
{
	SetIsPinRenamePending(Pin, true);
}

void UNiagaraNodeWithDynamicPins::RemoveDynamicPinFromMenu(UEdGraphPin* Pin)
{
	FScopedTransaction RemovePinTransaction(LOCTEXT("RemovePinTransaction", "Remove pin"));
	RemoveDynamicPin(Pin);
}

void UNiagaraNodeWithDynamicPins::MoveDynamicPinFromMenu(UEdGraphPin* Pin, int32 DirectionToMove)
{
	FScopedTransaction MovePinTransaction(LOCTEXT("MovePinTransaction", "Moved pin"));
	MoveDynamicPin(Pin, DirectionToMove);
}

void UNiagaraNodeWithDynamicPins::GetCompilationInputPins(FPinCollectorArray& InputPins) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Input && !Pin->bOrphanedPin && !IsAddPin(Pin))
		{
			InputPins.Add(Pin);
		}
	}
}

void UNiagaraNodeWithDynamicPins::GetCompilationOutputPins(FPinCollectorArray& OutputPins) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Output && !Pin->bOrphanedPin && !IsAddPin(Pin))
		{
			OutputPins.Add(Pin);
		}
	}
}

#undef LOCTEXT_NAMESPACE

