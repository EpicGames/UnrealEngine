// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraNodeWriteDataSet.h"
#include "UObject/UnrealType.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraEvents.h"
#include "Widgets/SNiagaraGraphNodeWriteDataSet.h"
#include "EdGraphSchema_Niagara.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraNodeWriteDataSet)

#define LOCTEXT_NAMESPACE "NiagaraNodeWriteDataSet"

UNiagaraNodeWriteDataSet::UNiagaraNodeWriteDataSet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void UNiagaraNodeWriteDataSet::AddConditionPin(int32 PinIndex)
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	const bool ConditionPinDefaulValue = true;
	UEdGraphPin* ConditionPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetBoolDef()), ConditionVarName, PinIndex);
	ConditionPin->bDefaultValueIsIgnored = false;
	ConditionPin->DefaultValue = ConditionPinDefaulValue ? TEXT("true") : TEXT("false");
	ConditionPin->PinFriendlyName = LOCTEXT("UNiagaraNodeWriteDataSetConditionPin", "Condition");

}


void UNiagaraNodeWriteDataSet::AllocateDefaultPins()
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	if (DataSet.Type == ENiagaraDataSetType::Event)
	{
		//Probably need this for all data set types tbh!
		//UEdGraphPin* Pin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetBoolDef()), TEXT("Valid"));
		//Pin->bDefaultValueIsIgnored = true;
	}

	AddParameterMapPins();
	AddConditionPin();
	
	bool useFriendlyNames = (VariableFriendlyNames.Num() == Variables.Num());
	for (int32 i = 0; i < Variables.Num(); i++)
	{
		FNiagaraVariable& Var = Variables[i];
		UEdGraphPin* NewPin = CreatePin(EGPD_Input, Schema->TypeDefinitionToPinType(Var.GetType()), Var.GetName());
		if (useFriendlyNames && VariableFriendlyNames[i].IsEmpty() == false)
		{
			NewPin->PinFriendlyName = FText::AsCultureInvariant(VariableFriendlyNames[i]);
		}
	}
}

TSharedPtr<SGraphNode> UNiagaraNodeWriteDataSet::CreateVisualWidget()
{
	return SNew(SNiagaraGraphNodeWriteDataSet, this);
}

bool UNiagaraNodeWriteDataSet::SynchronizeWithStruct()
{
	bool bSynchronized = Super::SynchronizeWithStruct();
	if (EventName.IsNone())
	{
		EventName = DataSet.Name;
		VisualsChangedDelegate.Broadcast(this);
	}
	return bSynchronized;
}

FText UNiagaraNodeWriteDataSet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(LOCTEXT("NiagaraDataSetWriteFormat", "{0} Write"), FText::FromName(DataSet.Name));
}

void UNiagaraNodeWriteDataSet::Compile(FTranslator* Translator, TArray<int32>& Outputs) const
{
	// this is dirty, but we'll continue doing this for now because it's all going to go away with Digested graphs
	UNiagaraNodeWriteDataSet* MutableThis = const_cast<UNiagaraNodeWriteDataSet*>(this);

	bool bError = false;

	//TODO implement writing to data sets in hlsl compiler and vm.

	TArray<int32> Inputs;
	CompileInputPins(Translator, Inputs);

	bool bGPUSim = Translator->IsCompileOptionDefined(TEXT("GPUComputeSim"));

	if (bGPUSim)
	{
		Translator->Error(LOCTEXT("CannotRunWriteDataSetGPU", "Cannot use an event write node on GPU sims!"), this, nullptr);
	}

	bool bIsEventScript = UNiagaraScript::IsParticleEventScript(Translator->GetTargetUsage());
	if (bIsEventScript)
	{
		Translator->Error(LOCTEXT("CannotRunWriteDataSetFromEventScript", "Cannot use an event write node in an Event Handler!"), this, nullptr);
	}

	FString IssuesWithStruct;
	if (!IsSynchronizedWithStruct(true, &IssuesWithStruct,false))
	{
		Translator->Error(FText::FromString(IssuesWithStruct), this, nullptr);
	}
	
	if (EventName.IsNone())
	{
		MutableThis->EventName = DataSet.Name;
		VisualsChangedDelegate.Broadcast(MutableThis);
	}

	FNiagaraDataSetID AlteredDataSet = DataSet;
	AlteredDataSet.Name = EventName;
	Translator->WriteDataSet(AlteredDataSet, Variables, ENiagaraDataSetAccessMode::AppendConsume, Inputs, Outputs);

}

void UNiagaraNodeWriteDataSet::PostLoad()
{
	Super::PostLoad();

	bool bFoundMatchingPin = false;
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); PinIndex++)
	{
		if (Pins[PinIndex]->Direction == EGPD_Input && Pins[PinIndex]->PinName == ConditionVarName)
		{
			bFoundMatchingPin = true;
			break;
		}
	}

	if (!bFoundMatchingPin)
	{
		AddConditionPin(1);
	}

	if (EventName.IsNone())
	{
		EventName = DataSet.Name;
	}
}

void UNiagaraNodeWriteDataSet::BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/) const
{
	if (bRecursive)
	{
		OutHistory.VisitInputPins(this, bFilterForCompilation);
	}

	if (!IsNodeEnabled() && OutHistory.GetIgnoreDisabled())
	{
		RouteParameterMapAroundMe(OutHistory, bRecursive);
		return;
	}

	int32 ParamMapIdx = INDEX_NONE;
	if (GetInputPin(0)->LinkedTo.Num() != 0)
	{
		ParamMapIdx = OutHistory.TraceParameterMapOutputPin(UNiagaraNode::TraceOutputPin(GetInputPin(0)->LinkedTo[0]));
	}

	if (ParamMapIdx != INDEX_NONE)
	{
		uint32 NodeIdx = OutHistory.BeginNodeVisitation(ParamMapIdx, this);
		OutHistory.EndNodeVisitation(ParamMapIdx, NodeIdx);
	}

	OutHistory.RegisterDataSetWrite(ParamMapIdx, DataSet);

	OutHistory.RegisterParameterMapPin(ParamMapIdx, GetOutputPin(0));
}


#undef LOCTEXT_NAMESPACE






