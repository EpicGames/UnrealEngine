// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Notifications/INotificationWidget.h"

class FNiagaraParameterNotificationWidgetProvider : public INotificationWidget
{
public:
	FNiagaraParameterNotificationWidgetProvider(TSharedRef<SWidget> WidgetToDisplay) : NotificationWidget(WidgetToDisplay) {}

	virtual ~FNiagaraParameterNotificationWidgetProvider() = default;

	virtual void OnSetCompletionState(SNotificationItem::ECompletionState State) override {}
	virtual TSharedRef<SWidget> AsWidget() override { return NotificationWidget; }

private:
	TSharedRef<SWidget> NotificationWidget;
};
