// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraOutliner.h"
#include "NiagaraDebugger.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraSimCache.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraOutliner)


UNiagaraOutliner::UNiagaraOutliner(const FObjectInitializer& Initializer)
{

}

#if WITH_EDITOR
void UNiagaraOutliner::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnChanged();

	//Ensure the capture trigger is not left on.
	CaptureSettings.bTriggerCapture = false;

	SaveConfig();
}
#endif

void UNiagaraOutliner::OnChanged()
{
#if WITH_EDITOR
	Modify();
#endif

	OnChangedDelegate.Broadcast();
}

void UNiagaraOutliner::Reset()
{
	Data.WorldData.Reset();
	SystemSimCaches.Reset();
	OnChanged();
}

void UNiagaraOutliner::UpdateData(const FNiagaraOutlinerData& NewData)
{
	FNiagaraOutlinerData::StaticStruct()->CopyScriptStruct(&Data, &NewData);

	//Clear out any sim caches when we update the outliner.
	//TODO: We should keep them all and just track info about when the were taken etc.
	SystemSimCaches.Reset();

	OnChanged();
	//TODO: Do some kind of diff on the data and collect any recently removed components etc into their own area.
	//Possibly keep them in the UI optionally but colour/mark them as dead until the user opts to remove them or on some timed interval.
}

void UNiagaraOutliner::UpdateSystemSimCache(FName ComponentName, TObjectPtr<UNiagaraSimCache> NewSimCache)
{
	if (NewSimCache)
	{
		TObjectPtr<UNiagaraSimCache>& SimCache = SystemSimCaches.FindOrAdd(ComponentName);
		SimCache = NewSimCache;
	}
	else
	{
		UE_LOGF(LogNiagaraEditor, Warning, "Niagara Outliner received empty sim cache data.");
	}
	OnChanged();
}

UNiagaraSimCache* UNiagaraOutliner::FindSimCache(FName ComponentName)
{
	if (TObjectPtr<UNiagaraSimCache>* SimCachePtr = SystemSimCaches.Find(ComponentName))
	{
		return SimCachePtr->Get();
	}
	return nullptr;
}

const FNiagaraOutlinerWorldData* UNiagaraOutliner::FindWorldData(const FString& WorldName)
{
	return Data.WorldData.Find(WorldName);
}

const FNiagaraOutlinerSystemData* UNiagaraOutliner::FindSystemData(const FString& WorldName, const FString& SystemName)
{
	if (const FNiagaraOutlinerWorldData* WorldData = FindWorldData(WorldName))
	{
		return WorldData->Systems.Find(SystemName);
	}
	return nullptr;
}

const FNiagaraOutlinerSystemInstanceData* UNiagaraOutliner::FindComponentData(const FString& WorldName, const FString& SystemName, const FString& ComponentName)
{
	if (const FNiagaraOutlinerSystemData* SystemData = FindSystemData(WorldName, SystemName))
	{
		return SystemData->SystemInstances.FindByPredicate([&](const FNiagaraOutlinerSystemInstanceData& InData){ return ComponentName == InData.ComponentName;});
	}
	return nullptr;
}

const FNiagaraOutlinerEmitterInstanceData* UNiagaraOutliner::FindEmitterData(const FString& WorldName, const FString& SystemName, const FString& ComponentName, const FString& EmitterName)
{
	if (const FNiagaraOutlinerSystemInstanceData* InstData = FindComponentData(WorldName, SystemName, ComponentName))
	{
		return InstData->Emitters.FindByPredicate([&](const FNiagaraOutlinerEmitterInstanceData& Emitter){ return EmitterName == Emitter.EmitterName;});
	}
	return nullptr;
}

