// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDebuggerCommon.h"
#include "Misc/NotifyHook.h"
#include "NiagaraSimCache.h"
#include "NiagaraOutliner.generated.h"

UENUM()
enum class ENiagaraOutlinerViewModes: uint8
{
	/** Outliner displays the main state data for each item. */
	State,
	/** Outliner displays performance data for each item. */
	Performance,
	/** Outliner displays debugging controls for each item. */
	Debug,
};

UENUM()
enum class ENiagaraOutlinerSortMode : uint8
{
	/** Context dependent default sorting. In State view mode this will sort by filter matches. In Performance mode this will sort by average time. */
	Auto,
	/** Sort by the number of items matching the current filters. */
	FilterMatches,
	/** Sort by the average game thread time. */
	AverageTime,
	/** Sort by the maximum game thread time. */
	MaxTime,
};

UENUM()
enum class ENiagaraOutlinerTimeUnits : uint8
{
	Microseconds UMETA(DisplayName="us", ToolTip="Microseconds"),
	Milliseconds UMETA(DisplayName = "ms", ToolTip = "Milliseconds"),
	Seconds UMETA(DisplayName = "s", ToolTip = "Seconds"),
};

/** View settings used in the Niagara Outliner. */
USTRUCT()
struct FNiagaraOutlinerFilterSettings
{
	GENERATED_BODY()

	FNiagaraOutlinerFilterSettings()
		: bFilterBySystemExecutionState(0)
		, bFilterByEmitterExecutionState(0)
		, bFilterByEmitterSimTarget(0)
		, bFilterBySystemCullState(0)
	{}

	FORCEINLINE bool AnyActive()const 
	{
		return bFilterBySystemExecutionState || bFilterByEmitterExecutionState || bFilterByEmitterSimTarget || bFilterBySystemCullState;
	}
	
	UPROPERTY(EditAnywhere, Category = "Filters", meta = (InlineEditConditionToggle))
	uint32 bFilterBySystemExecutionState : 1;

	UPROPERTY(EditAnywhere, Category = "Filters", meta = (InlineEditConditionToggle))
	uint32 bFilterByEmitterExecutionState : 1;
	
	UPROPERTY(EditAnywhere, Category = "Filters", meta = (InlineEditConditionToggle))
	uint32 bFilterByEmitterSimTarget : 1;

	UPROPERTY(EditAnywhere, Category = "Filters", meta = (InlineEditConditionToggle))
	uint32 bFilterBySystemCullState : 1;

	/** Only show systems with the following execution state. */
	UPROPERTY(EditAnywhere, Config, Category="Filters", meta = (EditCondition = "bFilterBySystemExecutionState"))
	ENiagaraExecutionState SystemExecutionState = ENiagaraExecutionState::Active;

	/** Only show emitters with the following execution state. */
	UPROPERTY(EditAnywhere, Config, Category="Filters", meta = (EditCondition = "bFilterByEmitterExecutionState"))
	ENiagaraExecutionState EmitterExecutionState = ENiagaraExecutionState::Active;

	/** Only show emitters with this SimTarget. */
	UPROPERTY(EditAnywhere, Config, Category = "Filters", meta = (EditCondition = "bFilterByEmitterSimTarget"))
	ENiagaraSimTarget EmitterSimTarget = ENiagaraSimTarget::CPUSim;

	/** Only show system instances with this cull state. */
 	UPROPERTY(EditAnywhere, Config, Category = "Filters", meta = (EditCondition = "bFilterBySystemCullState"))
	bool bSystemCullState = true;
};

/** View settings used in the Niagara Outliner. */
USTRUCT()
struct FNiagaraOutlinerViewSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "View")
	ENiagaraOutlinerViewModes ViewMode = ENiagaraOutlinerViewModes::State;
	
	UPROPERTY(EditAnywhere, Category = "View")
	FNiagaraOutlinerFilterSettings FilterSettings;

	/** Whether to sort ascending or descending. */
	UPROPERTY(EditAnywhere, Category="View")
	bool bSortDescending = true;
	
	UPROPERTY(EditAnywhere, Category = "View")
	ENiagaraOutlinerSortMode SortMode = ENiagaraOutlinerSortMode::Auto;

	/** Units used to display time data in performance view mode. */
	UPROPERTY(EditAnywhere, Category = "View")
	ENiagaraOutlinerTimeUnits TimeUnits = ENiagaraOutlinerTimeUnits::Microseconds;

	FORCEINLINE ENiagaraOutlinerSortMode GetSortMode()
	{
		if (SortMode == ENiagaraOutlinerSortMode::Auto)
		{
			return ViewMode == ENiagaraOutlinerViewModes::State ? ENiagaraOutlinerSortMode::FilterMatches : ENiagaraOutlinerSortMode::AverageTime;
		}
		return SortMode;
	}
};

UCLASS(config = EditorPerProjectUserSettings, defaultconfig)
class UNiagaraOutliner : public UObject, public FNotifyHook
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FOnChanged);
	
	FOnChanged OnChangedDelegate;

	void OnChanged();

#if WITH_EDITOR
	//UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)override;
	//UObject Interface END

	//FNotifyHook Interface
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange)override { PreEditChange(PropertyAboutToChange);	}
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)override { PostEditChange(); }
	virtual void NotifyPreChange(class FEditPropertyChain* PropertyAboutToChange)override{ PreEditChange(nullptr);	}
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged)override	{ PostEditChange();	}
	//FNotifyHook Interface END
#endif

	void Reset();
	void UpdateData(const FNiagaraOutlinerData& NewData);
	void UpdateSystemSimCache(FName CompName, TObjectPtr<UNiagaraSimCache> NewSimCache);

	UNiagaraSimCache* FindSimCache(FName ComponentName);

	const FNiagaraOutlinerWorldData* FindWorldData(const FString& WorldName);
	const FNiagaraOutlinerSystemData* FindSystemData(const FString& WorldName, const FString& SystemName);
	const FNiagaraOutlinerSystemInstanceData* FindComponentData(const FString& WorldName, const FString& SystemName, const FString& ComponentName);
	const FNiagaraOutlinerEmitterInstanceData* FindEmitterData(const FString& WorldName, const FString& SystemName, const FString& ComponentName, const FString& EmitterName);

	const void GetAllSystemNames(TArray<FString>& OutSystems);
	const void GetAllComponentNames(TArray<FString>& OutSystems);
	const void GetAllEmitterNames(TArray<FString>& OutSystems);

	UPROPERTY(EditAnywhere, Category="Settings")
	FNiagaraOutlinerCaptureSettings CaptureSettings;

	UPROPERTY(EditAnywhere, Category = "Filters")
	FNiagaraOutlinerViewSettings ViewSettings;

	UPROPERTY(VisibleAnywhere, Category="Outliner", Transient)
	FNiagaraOutlinerData Data;

	UPROPERTY()
	TMap<FName, TObjectPtr<UNiagaraSimCache>> SystemSimCaches;
};