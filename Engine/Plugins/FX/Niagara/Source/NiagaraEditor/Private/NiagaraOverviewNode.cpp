// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraOverviewNode.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "EdGraphSchema_NiagaraSystemOverview.h"
#include "Modules/ModuleManager.h"
#include "ToolMenuSection.h"
#include "ToolMenu.h"
#include "GraphEditorActions.h"
#include "Framework/Commands/GenericCommands.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraOverviewNode)

#define LOCTEXT_NAMESPACE "NiagaraOverviewNodeStackItem"

bool UNiagaraOverviewNode::bColorsAreInitialized = false;
FLinearColor UNiagaraOverviewNode::EmitterColor;
FLinearColor UNiagaraOverviewNode::StatelessEmitterColor;
FLinearColor UNiagaraOverviewNode::SystemColor;
FLinearColor UNiagaraOverviewNode::IsolatedColor;
FLinearColor UNiagaraOverviewNode::NotIsolatedColor;

UNiagaraOverviewNode::UNiagaraOverviewNode()
	: OwningSystem(nullptr)
	, EmitterHandleGuid(FGuid())
	, bRenamePending(false)
{
	UEdGraphNode::bCanRenameNode = true;
};

static FNiagaraEmitterHandle* FindEmitterHandleByID(UNiagaraSystem* System, const FGuid& Guid)
{
	check(System != nullptr);

	for (int Idx = 0; Idx < System->GetNumEmitters(); ++Idx)
	{
		FNiagaraEmitterHandle& EmitterHandle = System->GetEmitterHandle(Idx);
		if (EmitterHandle.GetId() == Guid)
		{
			return &EmitterHandle;
		}
	}

	return nullptr;
}

void UNiagaraOverviewNode::Initialize(UNiagaraSystem* InOwningSystem)
{
	OwningSystem = InOwningSystem;
}

void UNiagaraOverviewNode::Initialize(UNiagaraSystem* InOwningSystem, FGuid InEmitterHandleGuid)
{
	OwningSystem = InOwningSystem;
	EmitterHandleGuid = InEmitterHandleGuid;
}

void UNiagaraOverviewNode::UpdateStatus()
{
}

const FGuid UNiagaraOverviewNode::GetEmitterHandleGuid() const
{
	return EmitterHandleGuid;
}

FNiagaraEmitterHandle* UNiagaraOverviewNode::TryGetEmitterHandle() const
{
	return FindEmitterHandleByID(GetOwningSystem(), GetEmitterHandleGuid());
}

FText UNiagaraOverviewNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (OwningSystem == nullptr)
	{
		return FText::GetEmpty();
	}

	if (EmitterHandleGuid.IsValid())
	{
		const FNiagaraEmitterHandle* Handle = FindEmitterHandleByID(OwningSystem, EmitterHandleGuid);
		if (ensureMsgf(Handle != nullptr, TEXT("Failed to find matching emitter handle for existing overview node!")))
		{
			return FText::FromName(Handle->GetName());
		}

		return LOCTEXT("UnknownEmitterName", "Unknown Emitter");
	}
	else
	{
		return FText::FromString(OwningSystem->GetName());
	}
}

FLinearColor UNiagaraOverviewNode::GetNodeTitleColor() const
{
	if (bColorsAreInitialized == false)
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		EmitterColor = NiagaraEditorModule.GetWidgetProvider()->GetColorForExecutionCategory(UNiagaraStackEntry::FExecutionCategoryNames::Emitter);
		StatelessEmitterColor = NiagaraEditorModule.GetWidgetProvider()->GetColorForExecutionCategory(UNiagaraStackEntry::FExecutionCategoryNames::StatelessEmitter);
		SystemColor = NiagaraEditorModule.GetWidgetProvider()->GetColorForExecutionCategory(UNiagaraStackEntry::FExecutionCategoryNames::System);
		IsolatedColor = FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.OverviewNode.IsolatedColor");
		NotIsolatedColor = FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.OverviewNode.NotIsolatedColor");
		bColorsAreInitialized = true;
	}

	if (EmitterHandleGuid.IsValid())
	{
		if (OwningSystem != nullptr)
		{
			const FNiagaraEmitterHandle* Handle = FindEmitterHandleByID(OwningSystem, EmitterHandleGuid);
			if (ensureMsgf(Handle != nullptr, TEXT("Failed to find matching emitter handle for existing overview node!")))
			{
				if (OwningSystem->GetIsolateEnabled())
				{
					return Handle->IsIsolated() ? IsolatedColor : NotIsolatedColor;
				}
				else if ( Handle->GetEmitterMode() == ENiagaraEmitterMode::Stateless )
				{
					return StatelessEmitterColor;
				}
			}
		}
		
		return EmitterColor;
	}
	
	return SystemColor;
}

static bool IsSystemAsset(UNiagaraSystem* System)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::Get().LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	TSharedPtr<FNiagaraSystemViewModel> OwningSystemViewModel = NiagaraEditorModule.GetExistingViewModelForSystem(System);
	if (OwningSystemViewModel.IsValid())
	{
		if (OwningSystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
		{
			return true;
		}
	}

	return false;
}

bool UNiagaraOverviewNode::GetCanRenameNode() const
{
	return IsSystemAsset(OwningSystem) && EmitterHandleGuid.IsValid();
}

bool UNiagaraOverviewNode::CanUserDeleteNode() const
{
	return IsSystemAsset(OwningSystem) && EmitterHandleGuid.IsValid();
}

bool UNiagaraOverviewNode::CanDuplicateNode() const
{
	// The class object must return true for can duplicate otherwise the CanImportNodesFromText utility function fails.
	return HasAnyFlags(RF_ClassDefaultObject) || EmitterHandleGuid.IsValid();
}

void UNiagaraOverviewNode::OnRenameNode(const FString& NewName)
{
	bRenamePending = false;

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::Get().LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	TSharedPtr<FNiagaraSystemViewModel> OwningSystemViewModel = NiagaraEditorModule.GetExistingViewModelForSystem(OwningSystem);
	if (OwningSystemViewModel.IsValid())
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelPtr = OwningSystemViewModel->GetEmitterHandleViewModelById(EmitterHandleGuid);
		if (EmitterHandleViewModelPtr.IsValid())
		{
			EmitterHandleViewModelPtr->SetName(*NewName);
		}
	}
}

void UNiagaraOverviewNode::GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::Get().LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	TSharedPtr<FNiagaraSystemViewModel> OwningSystemViewModel = NiagaraEditorModule.GetExistingViewModelForSystem(OwningSystem);
	if (OwningSystemViewModel.IsValid())
	{
		FToolMenuSection& Section = Menu->AddSection("Emitter Actions", LOCTEXT("EmitterActions", "Emitter Actions"));

		bool bSingleSelection = OwningSystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds().Num() == 1;

		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelPtr = OwningSystemViewModel->GetEmitterHandleViewModelById(EmitterHandleGuid);
		if (EmitterHandleViewModelPtr.IsValid())
		{
			TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterHandleViewModelPtr.ToSharedRef();
			TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel = EmitterHandleViewModel->GetEmitterViewModel();
			{
				if (OwningSystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
				{
					Section.AddMenuEntry(
						"ToggleEmittersEnabled",
						LOCTEXT("ToggleEmittersEnabled", "Enabled"),
						LOCTEXT("ToggleEmittersEnabledToolTip", "Toggle whether or not the selected emitters are enabled."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateStatic(&FNiagaraEditorUtilities::ToggleSelectedEmittersEnabled, OwningSystemViewModel.ToSharedRef()),
							FCanExecuteAction(),
							FGetActionCheckState::CreateStatic(&FNiagaraEditorUtilities::GetSelectedEmittersEnabledCheckState, OwningSystemViewModel.ToSharedRef())
						),
						EUserInterfaceActionType::ToggleButton
					);

					Section.AddMenuEntry(
						"ToggleEmittersIsolated",
						LOCTEXT("ToggleEmittersIsolated", "Isolated"),
						LOCTEXT("ToggleEmittersIsolatedToolTip", "Toggle whether or not the selected emitters are isolated."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateStatic(&FNiagaraEditorUtilities::ToggleSelectedEmittersIsolated, OwningSystemViewModel.ToSharedRef()),
							FCanExecuteAction(),
							FGetActionCheckState::CreateStatic(&FNiagaraEditorUtilities::GetSelectedEmittersIsolatedCheckState, OwningSystemViewModel.ToSharedRef())
						),
						EUserInterfaceActionType::ToggleButton
					);

					Section.AddMenuEntry(
						"RenameEmitter",
						LOCTEXT("RenameEmitter", "Rename Emitter"),
						LOCTEXT("RenameEmitterToolTip", "Rename this local emitter copy."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(EmitterHandleViewModel, &FNiagaraEmitterHandleViewModel::SetIsRenamePending, true)
						)
					);

					Section.AddSeparator("DebugEmitterSplit");
					Section.AddMenuEntry(
						"DebugEmitter",
						LOCTEXT("DebugEmitter", "Watch Emitter In Niagara Debugger"),
						LOCTEXT("DebugEmitterToolTip", "Open Niagara Debugger and track this emitter in the world"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(EmitterHandleViewModel, &FNiagaraEmitterHandleViewModel::BeginDebugEmitter)
						)
					);
					Section.AddSeparator("DebugEmitterSplit2");
				}

				Section.AddMenuEntry(
					"RemoveParentEmitter",
					LOCTEXT("RemoveParentEmitter", "Remove Parent Emitter"),
					LOCTEXT("RemoveParentEmitterToolTip", "Removes this emitter's parent, preventing inheritance of any further changes."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(EmitterViewModel, &FNiagaraEmitterViewModel::RemoveParentEmitter),
						FCanExecuteAction::CreateLambda(
							[bSingleSelection, bHasParent = EmitterViewModel->HasParentEmitter()]()
							{
								return bSingleSelection && bHasParent;
							}
						)
					)
				);

				Section.AddMenuEntry(
					"ShowEmitterInContentBrowser",
					LOCTEXT("ShowEmitterInContentBrowser", "Show in Content Browser"),
					LOCTEXT("ShowEmitterInContentBrowserToolTip", "Show the selected emitter in the Content Browser."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FNiagaraEditorUtilities::ShowParentEmitterInContentBrowser, EmitterViewModel),
						FCanExecuteAction::CreateLambda(
							[bSingleSelection, bHasParent = EmitterViewModel->HasParentEmitter()]()
							{
								return bSingleSelection && bHasParent;
							}
						)
					)
				);

				Section.AddMenuEntry(
					"CreateAssetFromThis",
					LOCTEXT("CreateAssetFromThisEmitter", "Create Asset From This"),
					LOCTEXT("CreateAssetFromThisEmitterToolTip", "Create an emitter asset from this emitter."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&FNiagaraEditorUtilities::CreateAssetFromEmitter, EmitterHandleViewModel),
						FCanExecuteAction::CreateLambda(
							[bSingleSelection, EmitterHandleViewModel]()
							{
								return bSingleSelection && EmitterHandleViewModel->GetOwningSystemEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset;
							}
						)
					)
				);
			}
		}
	}
	
	{
		FToolMenuSection& EditSection = Menu->AddSection("EmitterEdit", LOCTEXT("Edit", "Edit"));

		EditSection.AddMenuEntry(FGenericCommands::Get().Cut);
		EditSection.AddMenuEntry(FGenericCommands::Get().Copy);
		EditSection.AddMenuEntry(FGenericCommands::Get().Delete);
		EditSection.AddMenuEntry(FGenericCommands::Get().Rename);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("Alignment", LOCTEXT("Alignment", "Alignment"));
		Section.AddSubMenu(
			"Alignment",
			LOCTEXT("AlignmentHeader", "Alignment"),
			FText(),
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			{
				FToolMenuSection& SubMenuSection = InMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
			}

			{
				FToolMenuSection& SubMenuSection = InMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
				SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
			}
		}));
	}
}

bool UNiagaraOverviewNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA<UEdGraphSchema_NiagaraSystemOverview>();
}

UNiagaraSystem* UNiagaraOverviewNode::GetOwningSystem() const
{
	return OwningSystem;
}

#undef LOCTEXT_NAMESPACE

