// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterCollectionFactoryNew.h"
#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "NiagaraSettings.h"
#include "NiagaraParameterCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraParameterCollectionFactoryNew)

#define LOCTEXT_NAMESPACE "NiagaraParameterCollectionFactory"

UNiagaraParameterCollectionFactoryNew::UNiagaraParameterCollectionFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UNiagaraParameterCollection::StaticClass();
	bCreateNew = false;
	bEditAfterNew = true;
	bCreateNew = true;
}

UObject* UNiagaraParameterCollectionFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UNiagaraParameterCollection::StaticClass()));

	UNiagaraParameterCollection* NewCollection = NewObject<UNiagaraParameterCollection>(InParent, Class, Name, Flags | RF_Transactional);
	NewCollection->RefreshCompileId();

	return NewCollection;
}

//////////////////////////////////////////////////////////////////////////

UNiagaraParameterCollectionInstanceFactoryNew::UNiagaraParameterCollectionInstanceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UNiagaraParameterCollectionInstance::StaticClass();
	bCreateNew = false;
	bEditAfterNew = true;
	bCreateNew = true;

	InitialParent = nullptr;
}

UObject* UNiagaraParameterCollectionInstanceFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UNiagaraParameterCollectionInstance::StaticClass()));

	UNiagaraParameterCollectionInstance* NewCollection = NewObject<UNiagaraParameterCollectionInstance>(InParent, Class, Name, Flags | RF_Transactional);

	if (InitialParent)
	{
		NewCollection->SetParent(InitialParent);
	}

	return NewCollection;
}

#undef LOCTEXT_NAMESPACE
