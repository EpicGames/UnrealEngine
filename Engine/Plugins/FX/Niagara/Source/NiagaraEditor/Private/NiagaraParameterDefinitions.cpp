// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterDefinitions.h"

#include "ScopedTransaction.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraScriptVariable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraParameterDefinitions)

#define LOCTEXT_NAMESPACE "NiagaraParameterDefinitions"

UNiagaraParameterDefinitions::UNiagaraParameterDefinitions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bPromoteToTopInAddMenus = false;
	MenuSortOrder = 100;
	if (UniqueId.IsValid() == false)
	{
		UniqueId = FGuid::NewGuid();
	}
}

UNiagaraParameterDefinitions::~UNiagaraParameterDefinitions()
{
	for (FParameterDefinitionsBindingNameSubscription& Subscription : ExternalParameterDefinitionsSubscriptions)
	{
		Subscription.SubscribedParameterDefinitions->GetOnParameterDefinitionsChanged().RemoveAll(this);
	}
}

void UNiagaraParameterDefinitions::PostLoad()
{
	Super::PostLoad();
	if (HasAnyFlags(RF_ClassDefaultObject) == false && GetPackage() != GetTransientPackage())
	{
		FNiagaraEditorModule::Get().EnsureReservedDefinitionUnique(UniqueId);
		SynchronizeWithSubscribedParameterDefinitions();
	}
}

void UNiagaraParameterDefinitions::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	OnParameterDefinitionsChangedDelegate.Broadcast();
}

void UNiagaraParameterDefinitions::AddParameter(const FNiagaraVariable& NewVariable)
{
	FScopedTransaction AddTransaction(LOCTEXT("AddParameter", "Add Parameter"));
	Modify();
	TObjectPtr<UNiagaraScriptVariable>& NewScriptVariable = ScriptVariables.Add_GetRef(NewObject<UNiagaraScriptVariable>(this, FName(), RF_Transactional));
	NewScriptVariable->Init(NewVariable, FNiagaraVariableMetaData());
	NewScriptVariable->SetIsStaticSwitch(false);
	NewScriptVariable->SetIsSubscribedToParameterDefinitions(true);
	NotifyParameterDefinitionsChanged();
}

bool UNiagaraParameterDefinitions::HasParameter(const FNiagaraVariable& Variable)
{
	TObjectPtr<UNiagaraScriptVariable>* NewScriptVariable = ScriptVariables.FindByPredicate([Variable](TObjectPtr<UNiagaraScriptVariable> ScriptVariable)
	{
		return ScriptVariable->Variable == Variable;
	});

	return NewScriptVariable != nullptr;
}

void UNiagaraParameterDefinitions::FindOrAddParameter(const FNiagaraVariable& Variable)
{
	TObjectPtr<UNiagaraScriptVariable>* NewScriptVariable = ScriptVariables.FindByPredicate([Variable](TObjectPtr<UNiagaraScriptVariable> ScriptVariable)
	{
		return ScriptVariable->Variable == Variable;
	});

	if(NewScriptVariable == nullptr)
	{
		AddParameter(Variable);
	}
}

void UNiagaraParameterDefinitions::RemoveParameter(const FNiagaraVariable& VariableToRemove)
{
	const int32 Idx = ScriptVariables.IndexOfByPredicate([&VariableToRemove](const UNiagaraScriptVariable* ScriptVariable){ return ScriptVariable->Variable == VariableToRemove; });
	if (Idx != INDEX_NONE)
	{
		Modify();
		const FGuid RemovedScriptVarGuid = ScriptVariables[Idx]->Metadata.GetVariableGuid();
		// Make sure to remove any links to binding name subscriptions to external parameter libraries.
		UnsubscribeBindingNameFromExternalParameterDefinitions(RemovedScriptVarGuid);
		ScriptVariables.RemoveAtSwap(Idx, EAllowShrinking::No);
		NotifyParameterDefinitionsChanged();
	}
}

void UNiagaraParameterDefinitions::RenameParameter(const FNiagaraVariable& VariableToRename, const FName NewName)
{
	if (TObjectPtr<UNiagaraScriptVariable>* ScriptVariablePtr = ScriptVariables.FindByPredicate([&VariableToRename](const UNiagaraScriptVariable* ScriptVariable) { return ScriptVariable->Variable == VariableToRename; }))
	{
		const FNiagaraVariable OldVariable = VariableToRename;
		Modify();
		UNiagaraScriptVariable* ScriptVariable = *ScriptVariablePtr;
		ScriptVariable->Modify();
		ScriptVariable->Variable.SetName(NewName);
		ScriptVariable->UpdateChangeId();
		NotifyParameterDefinitionsChanged();
	}
}

const TArray<UNiagaraScriptVariable*>& UNiagaraParameterDefinitions::GetParametersConst() const
{
	return ScriptVariables;
}

int32 UNiagaraParameterDefinitions::GetChangeIdHash() const
{
	int32 ChangeIdHash = 0;
	for (const UNiagaraScriptVariable* ScriptVar : ScriptVariables)
	{
		ChangeIdHash = HashCombine(ChangeIdHash, GetTypeHash(ScriptVar->GetChangeId()));
	}
	return ChangeIdHash;
}

TSet<FGuid> UNiagaraParameterDefinitions::GetParameterIds() const
{
	TSet<FGuid> OutParameterIds;
	OutParameterIds.Reserve(ScriptVariables.Num());
	for (const UNiagaraScriptVariable* ScriptVariable : ScriptVariables)
	{
		OutParameterIds.Add(ScriptVariable->Metadata.GetVariableGuid());
	}
	return OutParameterIds;
}

UNiagaraScriptVariable* UNiagaraParameterDefinitions::GetScriptVariable(const FNiagaraVariable& Var)
{
	if (TObjectPtr<UNiagaraScriptVariable> const* ScriptVarPtr = ScriptVariables.FindByPredicate([Var](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Variable == Var; }))
	{
		return *ScriptVarPtr;
	}
	return nullptr;
}

UNiagaraScriptVariable* UNiagaraParameterDefinitions::GetScriptVariable(const FGuid& ScriptVarId)
{
	if (TObjectPtr<UNiagaraScriptVariable> const* ScriptVarPtr = ScriptVariables.FindByPredicate([ScriptVarId](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Metadata.GetVariableGuid() == ScriptVarId; }))
	{
		return *ScriptVarPtr;
	}
	return nullptr;
}

void UNiagaraParameterDefinitions::SubscribeBindingNameToExternalParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions, const FGuid& ExternalScriptVarId, const FGuid& InternalScriptVarId)
{
	FParameterDefinitionsBindingNameSubscription* LibrarySubscription = ExternalParameterDefinitionsSubscriptions.FindByPredicate([NewParameterDefinitions](const FParameterDefinitionsBindingNameSubscription& Subscription) { return Subscription.SubscribedParameterDefinitions->GetUniqueID() == NewParameterDefinitions->GetUniqueID(); });
	if (!LibrarySubscription)
	{
		FParameterDefinitionsBindingNameSubscription& NewLibrarySubscription = ExternalParameterDefinitionsSubscriptions.AddDefaulted_GetRef();
		NewLibrarySubscription.SubscribedParameterDefinitions = NewParameterDefinitions;
		LibrarySubscription = &NewLibrarySubscription;
	}

	FScriptVarBindingNameSubscription* BindingNameSubscription = LibrarySubscription->BindingNameSubscriptions.FindByPredicate([&ExternalScriptVarId](const FScriptVarBindingNameSubscription& Subscription){ return Subscription.ExternalScriptVarId == ExternalScriptVarId; });
	if (!BindingNameSubscription)
	{
		FScriptVarBindingNameSubscription& NewBindingNameSubscription = LibrarySubscription->BindingNameSubscriptions.AddDefaulted_GetRef();
		NewBindingNameSubscription.ExternalScriptVarId = ExternalScriptVarId;
		BindingNameSubscription = &NewBindingNameSubscription;
	}

	if (ensureMsgf(BindingNameSubscription->InternalScriptVarIds.Contains(InternalScriptVarId) == false, TEXT("Tried to add internal script var key that was already linked!")))
	{
		BindingNameSubscription->InternalScriptVarIds.Add(InternalScriptVarId);
	}
}

void UNiagaraParameterDefinitions::UnsubscribeBindingNameFromExternalParameterDefinitions(const FGuid& InternalScriptVarToUnsubscribeId)
{
	for (FParameterDefinitionsBindingNameSubscription& LibrarySubscription : ExternalParameterDefinitionsSubscriptions)
	{
		for (FScriptVarBindingNameSubscription& BindingNameSubscription : LibrarySubscription.BindingNameSubscriptions)
		{
			BindingNameSubscription.InternalScriptVarIds.Remove(InternalScriptVarToUnsubscribeId);
		}
	}
}

void UNiagaraParameterDefinitions::SynchronizeWithSubscribedParameterDefinitions()
{
	for (int32 LibraryIdx = ExternalParameterDefinitionsSubscriptions.Num() - 1; LibraryIdx > -1; --LibraryIdx)
	{
		FParameterDefinitionsBindingNameSubscription& LibrarySubscription = ExternalParameterDefinitionsSubscriptions[LibraryIdx];
		if (LibrarySubscription.SubscribedParameterDefinitions == nullptr)
		{
			ExternalParameterDefinitionsSubscriptions.RemoveAt(LibraryIdx);
			continue;
		}

		for (int32 BindingIdx = LibrarySubscription.BindingNameSubscriptions.Num() - 1; BindingIdx > -1; --BindingIdx)
		{
			const FScriptVarBindingNameSubscription& BindingNameSubscription = LibrarySubscription.BindingNameSubscriptions[BindingIdx];
			const FGuid& ExternalScriptVarId = BindingNameSubscription.ExternalScriptVarId;
			if (const UNiagaraScriptVariable* ExternalScriptVar = LibrarySubscription.SubscribedParameterDefinitions->GetScriptVariable(ExternalScriptVarId))
			{
				const FGuid& ExternalScriptVarChangeId = ExternalScriptVar->GetChangeId();
				const FName& ExternalScriptVarName = ExternalScriptVar->Variable.GetName();
				for (const FGuid& InternalScriptVarId : BindingNameSubscription.InternalScriptVarIds)
				{
					if (TObjectPtr<UNiagaraScriptVariable> const* ScriptVarPtr = ScriptVariables.FindByPredicate([&InternalScriptVarId](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Metadata.GetVariableGuid() == InternalScriptVarId; }))
					{
						UNiagaraScriptVariable* ScriptVar = *ScriptVarPtr;
						if (ScriptVar->GetChangeId() != ExternalScriptVarChangeId)
						{
							ScriptVar->DefaultBinding.SetName(ExternalScriptVarName);
							ScriptVar->SetChangeId(ExternalScriptVarChangeId);
							NotifyParameterDefinitionsChanged();
						}
					}
					else
					{
						ensureMsgf(false, TEXT("Failed to find script variable with matching key to subscriptions list! Deleted a parameter without deleting the subscription record!"));
					}
				}
			}
			else
			{
				// Did not find external script var in the external parameter libraries script variables; it has been deleted since last synchronization. Remove this subscription record.
				LibrarySubscription.BindingNameSubscriptions.RemoveAt(BindingIdx);
				if (LibrarySubscription.BindingNameSubscriptions.Num() == 0)
				{
					ExternalParameterDefinitionsSubscriptions.RemoveAt(LibraryIdx);
				}
			}
		}
	}
}

TArray<UNiagaraParameterDefinitions*> UNiagaraParameterDefinitions::GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions) const
{
	const TArray<UNiagaraParameterDefinitions*> SubscribedParameterDefinitions = bSkipSubscribedParameterDefinitions ? GetSubscribedParameterDefinitions() : TArray<UNiagaraParameterDefinitions*>();
	auto GetParameterDefinitionsIsSubscribed = [&SubscribedParameterDefinitions](const UNiagaraParameterDefinitions* ParameterDefinitions)->bool {
		return SubscribedParameterDefinitions.ContainsByPredicate([&ParameterDefinitions](const UNiagaraParameterDefinitions* SubscribedParameterDefinitions) { return ParameterDefinitions->GetDefinitionsUniqueId() == SubscribedParameterDefinitions->GetDefinitionsUniqueId(); });
	};

	TArray<FAssetData> ParameterDefinitionsAssetData;
	TArray<UNiagaraParameterDefinitions*> AvailableParameterDefinitions;
	TArray<FString> ExternalPackagePaths = { GetPackage()->GetOutermost()->GetPathName() };
	ensureMsgf(FNiagaraEditorUtilities::GetAvailableParameterDefinitions(ExternalPackagePaths, ParameterDefinitionsAssetData), TEXT("Failed to get parameter libraries!"));

	for (const FAssetData& ParameterDefinitionsAssetDatum : ParameterDefinitionsAssetData)
	{
		UNiagaraParameterDefinitions* ParameterDefinitions = Cast<UNiagaraParameterDefinitions>(ParameterDefinitionsAssetDatum.GetAsset());
		if (ParameterDefinitions == nullptr)
		{
			continue;
		}
		else if (bSkipSubscribedParameterDefinitions && GetParameterDefinitionsIsSubscribed(ParameterDefinitions))
		{
			continue;
		}
		AvailableParameterDefinitions.Add(ParameterDefinitions);
	}

	return AvailableParameterDefinitions;
}

const TArray<UNiagaraParameterDefinitions*> UNiagaraParameterDefinitions::GetSubscribedParameterDefinitions() const
{
	TArray<UNiagaraParameterDefinitions*> OutSubscribedParameterDefinitions;
	for (const FParameterDefinitionsBindingNameSubscription& Subscription : ExternalParameterDefinitionsSubscriptions)
	{
		OutSubscribedParameterDefinitions.Add(Subscription.SubscribedParameterDefinitions);
	}
	return OutSubscribedParameterDefinitions;
}

void UNiagaraParameterDefinitions::NotifyParameterDefinitionsChanged()
{
	OnParameterDefinitionsChangedDelegate.Broadcast();
}

#undef LOCTEXT_NAMESPACE

