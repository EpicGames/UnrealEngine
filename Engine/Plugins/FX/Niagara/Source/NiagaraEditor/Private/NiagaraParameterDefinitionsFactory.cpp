// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterDefinitionsFactory.h"
#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "NiagaraSettings.h"
#include "NiagaraParameterDefinitions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraParameterDefinitionsFactory)

UNiagaraParameterDefinitionsFactory::UNiagaraParameterDefinitionsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UNiagaraParameterDefinitions::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
}

UObject* UNiagaraParameterDefinitionsFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UNiagaraParameterDefinitions::StaticClass()));

	UNiagaraParameterDefinitions* NewLibrary = NewObject<UNiagaraParameterDefinitions>(InParent, Class, Name, Flags | RF_Transactional);

	return NewLibrary;
}

