// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterMapHistory.h"

#include "EdGraphSchema_Niagara.h"
#include "INiagaraEditorTypeUtilities.h"
#include "Modules/ModuleManager.h"
#include "NiagaraCompilationBridge.h"
#include "NiagaraCompilationPrivate.h"
#include "NiagaraCompiler.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraGraphDigest.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraNode.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemImpl.h"
#include "UObject/UObjectThreadContext.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

#define LOCTEXT_NAMESPACE "NiagaraEditor"

static int32 GNiagaraLogNamespaceFixup = 0;
static FAutoConsoleVariableRef CVarNiagaraLogNamespaceFixup(
	TEXT("fx.NiagaraLogNamespaceFixup"),
	GNiagaraLogNamespaceFixup,
	TEXT("Log matched variables and pin name changes in precompile. \n"),
	ECVF_Default
);

static int32 GNiagaraForcePrecompilerCullDataset = 0;
static FAutoConsoleVariableRef CVarNiagaraForcePrecompilerCullDataset(
	TEXT("fx.NiagaraEnablePrecompilerNamespaceDatasetCulling"),
	GNiagaraForcePrecompilerCullDataset,
	TEXT("Force the namespace fixup precompiler process to cull unused Dataset parameters. Only enabled if fx.NiagaraEnablePrecompilerNamespaceFixup is also enabled. \n"),
	ECVF_Default
);

//#define NIAGARA_SHOULD_DO_PER_VARIABLE_SUBSTRING_MATCH_LOGGING 1
#if defined(NIAGARA_SHOULD_DO_PER_VARIABLE_SUBSTRING_MATCH_LOGGING)
static TArray<FString> CheckTheseNameSubstrings;// = { TEXT("XXXX") };

bool NiagaraDebugShouldLogEntriesByNameSubstring(const FString& InNameStr)
{
	for (const FString& Str : CheckTheseNameSubstrings)
	{
		if (InNameStr.EndsWith(Str))
		{
			return true;
		}
	}

	return false;
}
bool NiagaraDebugShouldLogEntriesByNameSubstring(const FName& InName)
{
	if (CheckTheseNameSubstrings.Num() == 0)
	{
		return false;
	}
	
	FString NameStr = InName.ToString();
	return NiagaraDebugShouldLogEntriesByNameSubstring(NameStr);
}
#else

#define NiagaraDebugShouldLogEntriesByNameSubstring(expr) (false) // Because the code may do specific allocations, we want to bypass the expression

#endif

void FGraphTraversalHandle::PushNode(const UEdGraphNode* Node)
{
	if (ensure(Node))
	{
		ensure(Node->NodeGuid.IsValid());
		Path.Push(Node->NodeGuid);
		FriendlyPath.Push(Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString());
	}
	else
	{
		Path.Push(FGuid());
		FriendlyPath.Push(FString());
	}
}

void FGraphTraversalHandle::PushNode(const FNiagaraCompilationNode* Node)
{
	if (ensure(Node))
	{
		ensure(Node->NodeGuid.IsValid());
		Path.Push(Node->NodeGuid);
		FriendlyPath.Push(Node->FullTitle);
	}
	else
	{
		Path.Push(FGuid());
		FriendlyPath.Push(FString());
	}
}

void FGraphTraversalHandle::PushPin(const UEdGraphPin* Pin)
{
	if (ensure(Pin && Pin->GetOwningNode()))
	{
		PushNode(Pin->GetOwningNode());

		if (Pin->PersistentGuid.IsValid())
		{
			Path.Push(Pin->PersistentGuid);
		}
		else
		{
			int32 Index = Pin->GetOwningNode()->GetPinIndex((UEdGraphPin*)Pin);
			FGuid PinIndexGuid(Index, 0, 0, 0);
			Path.Push(PinIndexGuid);
		}

		FriendlyPath.Push(Pin->GetOwningNode()->GetNodeTitle(ENodeTitleType::FullTitle).ToString() + TEXT("->") + Pin->PinName.ToString());
	}
	else
	{
		Path.Push(FGuid());
		FriendlyPath.Push(FString());

		Path.Push(FGuid());
		FriendlyPath.Push(FString());
	}
}

void FGraphTraversalHandle::PushPin(const FNiagaraCompilationPin* Pin)
{
	if (ensure(Pin && Pin->OwningNode))
	{
		PushNode(Pin->OwningNode);

		if (Pin->PersistentGuid.IsValid())
		{
			Path.Push(Pin->PersistentGuid);
		}
		else
		{
			FGuid PinIndexGuid(Pin->SourcePinIndex, 0, 0, 0);
			Path.Push(PinIndexGuid);
		}

		FriendlyPath.Push(Pin->OwningNode->FullTitle + TEXT("->") + Pin->PinName.ToString());
	}
	else
	{
		Path.Push(FGuid());
		FriendlyPath.Push(FString());

		Path.Push(FGuid());
		FriendlyPath.Push(FString());
	}
}

void FGraphTraversalHandle::PopNode()
{
	Path.Pop();
	FriendlyPath.Pop();
}


void FGraphTraversalHandle::PopPin()
{
	PopNode();

	Path.Pop();
	FriendlyPath.Pop();
}

template<typename GraphBridge>
TNiagaraParameterMapHistory<GraphBridge>::TNiagaraParameterMapHistory()
{
	OriginatingScriptUsage = ENiagaraScriptUsage::Function;
}

void FNiagaraParameterUtilities::GetValidNamespacesForReading(const UNiagaraScript* InScript, TArray<FString>& OutputNamespaces)
{
	GetValidNamespacesForReading(InScript->GetUsage(), 0, OutputNamespaces);
}

void FNiagaraParameterUtilities::GetValidNamespacesForReading(ENiagaraScriptUsage InScriptUsage, int32 InUsageBitmask, TArray<FString>& OutputNamespaces)
{
	TArray<ENiagaraScriptUsage> SupportedContexts;
	SupportedContexts.Add(InScriptUsage);
	if (UNiagaraScript::IsStandaloneScript(InScriptUsage))
	{
		SupportedContexts.Append(UNiagaraScript::GetSupportedUsageContextsForBitmask(InUsageBitmask));
	}

	OutputNamespaces.Add(PARAM_MAP_MODULE_STR);
	OutputNamespaces.Add(PARAM_MAP_ENGINE_STR);
	OutputNamespaces.Add(PARAM_MAP_NPC_STR);
	OutputNamespaces.Add(PARAM_MAP_USER_STR);
	OutputNamespaces.Add(PARAM_MAP_SYSTEM_STR);
	OutputNamespaces.Add(PARAM_MAP_EMITTER_STR);
	OutputNamespaces.Add(PARAM_MAP_INDICES_STR);

	for (ENiagaraScriptUsage Usage : SupportedContexts)
	{
		if (UNiagaraScript::IsParticleScript(Usage))
		{
			OutputNamespaces.Add(PARAM_MAP_ATTRIBUTE_STR);
			break;
		}
	}
}

FString FNiagaraParameterUtilities::GetNamespace(const FNiagaraVariable& InVar, bool bIncludeDelimiter)
{
	TArray<FString> SplitName;
	InVar.GetName().ToString().ParseIntoArray(SplitName, TEXT("."));

	check(SplitName.Num() > 0);

	if (bIncludeDelimiter)
	{
		return SplitName[0] + TEXT(".");
	}
	else
	{
		return SplitName[0];
	}
}

bool FNiagaraParameterUtilities::IsValidNamespaceForReading(ENiagaraScriptUsage InScriptUsage, int32 InUsageBitmask, FString Namespace)
{
	TArray<FString> OutputNamespaces;
	GetValidNamespacesForReading(InScriptUsage, InUsageBitmask, OutputNamespaces);

	TArray<FString> ConcernedNamespaces;
	ConcernedNamespaces.Add(PARAM_MAP_MODULE_STR);
	ConcernedNamespaces.Add(PARAM_MAP_ENGINE_STR);
	ConcernedNamespaces.Add(PARAM_MAP_NPC_STR);
	ConcernedNamespaces.Add(PARAM_MAP_USER_STR);
	ConcernedNamespaces.Add(PARAM_MAP_SYSTEM_STR);
	ConcernedNamespaces.Add(PARAM_MAP_EMITTER_STR);
	ConcernedNamespaces.Add(PARAM_MAP_ATTRIBUTE_STR);
	ConcernedNamespaces.Add(PARAM_MAP_INDICES_STR);

	
	if (!Namespace.EndsWith(TEXT(".")))
	{
		Namespace.Append(TEXT("."));
	}

	// Pass if we are in the allowed list
	for (const FString& ValidNamespace : OutputNamespaces)
	{
		if (Namespace.StartsWith(ValidNamespace))
		{
			return true;
		}
	}

	// Only fail if we're using a namespace that we know is one of the reserved ones.
	for (const FString& ConcernedNamespace : ConcernedNamespaces)
	{
		if (Namespace.StartsWith(ConcernedNamespace))
		{
			return false;
		}
	}

	// This means that we are using a namespace that isn't one of the primary engine namespaces, so we don't care and let it go.
	return true;
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistory<GraphBridge>::RegisterParameterMapPin(const FPin* Pin)
{
	int32 RetIdx =  MapPinHistory.Add(Pin);
	return RetIdx;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistory<GraphBridge>::RegisterConstantPin(const FGraphTraversalHandle& InTraversalPath, const FPin* InPin, const FString& InValue)
{
	if (GraphBridge::IsStaticPin(InPin))
	{
		FGraphTraversalHandle Handle = InTraversalPath;
		Handle.PushPin(InPin);

		FString* FoundValue = PinToConstantValues.Find(Handle);
		if (FoundValue != nullptr)
		{
			ensure(*FoundValue == InValue);
		}
		PinToConstantValues.Add(Handle, InValue);

		if (UNiagaraScript::LogCompileStaticVars > 0 || NiagaraDebugShouldLogEntriesByNameSubstring(Handle.ToString(true)))
		{
			UE_LOGF(LogNiagaraEditor, Log, "RegisterConstantPin \"%ls\" Pin: \"%ls\"", *InValue, *Handle.ToString(true));
		}
	}
}


template<typename GraphBridge>
void TNiagaraParameterMapHistory<GraphBridge>::RegisterConstantVariableWrite(const FString& InValue, int32 VarIdx, bool bIsSettingDefault, bool bLinkNotValue)
{
	
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

	if (bIsSettingDefault == false && PerVariableConstantValue.Num() > VarIdx && VarIdx >= 0)
	{
		PerVariableConstantValue[VarIdx].AddUnique(InValue);

		if (UNiagaraScript::LogCompileStaticVars > 0 || NiagaraDebugShouldLogEntriesByNameSubstring(Variables[VarIdx].GetName().ToString()))
		{
			UE_LOGF(LogNiagaraEditor, Log, "RegisterConstantVariableWrite Value \"%ls\" Var: \"%ls\"", *InValue, *Variables[VarIdx].GetName().ToString());
		}
	}
	else if (bIsSettingDefault == true && PerVariableConstantDefaultValue.Num() > VarIdx && VarIdx >= 0)
	{
		PerVariableConstantDefaultValue[VarIdx].AddUnique(InValue);

		if (UNiagaraScript::LogCompileStaticVars > 0 || NiagaraDebugShouldLogEntriesByNameSubstring(Variables[VarIdx].GetName().ToString()))
		{
			UE_LOGF(LogNiagaraEditor, Log, "RegisterConstantVariableWrite Default \"%ls\" Var: \"%ls\"", *InValue, *Variables[VarIdx].GetName().ToString());
		}
	}

	
}

template<typename GraphBridge>
uint32 TNiagaraParameterMapHistory<GraphBridge>::BeginNodeVisitation(const FNode* Node)
{
	uint32 AddedIndex = MapNodeVisitations.Add(Node);
	MapNodeVariableMetaData.Add(TTuple<uint32, uint32>(Variables.Num(), 0));
	check(MapNodeVisitations.Num() == MapNodeVariableMetaData.Num());
	return AddedIndex;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistory<GraphBridge>::EndNodeVisitation(uint32 IndexFromBeginNode)
{
	check(IndexFromBeginNode < (uint32)MapNodeVisitations.Num());
	check(MapNodeVisitations.Num() == MapNodeVariableMetaData.Num());
	MapNodeVariableMetaData[IndexFromBeginNode].Value = Variables.Num();
}


template<typename GraphBridge>
int32 TNiagaraParameterMapHistory<GraphBridge>::FindVariableByName(const FName& VariableName, bool bAllowPartialMatch) const
{
	if (!bAllowPartialMatch)
	{
		int32 FoundIdx = Variables.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
		{
			return (InObj.GetName() == VariableName);
		});

		return FoundIdx;
	}
	else
	{
		return FNiagaraVariable::SearchArrayForPartialNameMatch(Variables, VariableName);
	}
}


template<typename GraphBridge>
int32 TNiagaraParameterMapHistory<GraphBridge>::FindVariable(const FName& VariableName, const FNiagaraTypeDefinition& Type) const
{
	int32 FoundIdx = Variables.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
	{
		return (InObj.GetName() == VariableName && InObj.GetType() == Type);
	});

	return FoundIdx;
}

// because of how we do variable traversal we encounter situations where we are registering variables without having the context
// of what modules might be relevant when it comes to aliasing.  As an example, when processing a dynamic input for Multiply Float
// we could be doing a ParamMapSet on Mulitply_Float.A outside of the module where we're not able to provide AddVariable with
// InVar=Multiply_Float.A and InAliasedVar=Module.A.  As a way around this problem we'll conditionally update the aliased version
// of the variable as we encounter variables through the rest of the parameter map traversal
template<typename GraphBridge>
void TNiagaraParameterMapHistory<GraphBridge>::ConditionalUpdateAliasedVariable(int32 VariableIndex, const FNiagaraVariableBase& InAliasedVariable)
{
	if (VariablesWithOriginalAliasesIntact.IsValidIndex(VariableIndex))
	{
		FNiagaraVariable& AliasedVariable = VariablesWithOriginalAliasesIntact[VariableIndex];

		if (AliasedVariable.GetName() != InAliasedVariable.GetName()
			&& !FNiagaraParameterUtilities::IsAliasedModuleParameter(AliasedVariable)
			&& FNiagaraParameterUtilities::IsAliasedModuleParameter(InAliasedVariable))
		{
			AliasedVariable.SetName(InAliasedVariable.GetName());
		}
	}
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistory<GraphBridge>::AddVariable(
	  const FNiagaraVariable& InVar
	, const FNiagaraVariable& InAliasedVar
	, FName ModuleName
	, const FPin* InPin
	, TOptional<FNiagaraVariableMetaData> InMetaData /*= TOptional<FNiagaraVariableMetaData>()*/)
{
	FNiagaraVariable Var = InVar;

	int32 FoundIdx = FindVariable(Var.GetName(), Var.GetType());
	if (FoundIdx == INDEX_NONE)
	{
		FoundIdx = Variables.Add(Var);
		VariablesWithOriginalAliasesIntact.Add(InAliasedVar);
		PerVariableWarnings.AddDefaulted(1);
		PerVariableWriteHistory.AddDefaulted(1);
		PerVariableReadHistory.AddDefaulted(1);
		PerVariableConstantValue.AddDefaulted(1);
		PerVariableConstantDefaultValue.AddDefaulted(1);

		if (InMetaData.IsSet())
		{
			VariableMetaData.Add(InMetaData.GetValue());
			check(Variables.Num() == VariableMetaData.Num());
		}
	}
	else
	{
		if (Variables[FoundIdx].GetType() != Var.GetType())
		{
			PerVariableWarnings[FoundIdx].Append(FString::Printf(TEXT("Type mismatch %s instead of %s in map!"), *Var.GetType().GetName(), *Variables[FoundIdx].GetType().GetName()));
		}

		ConditionalUpdateAliasedVariable(FoundIdx, InAliasedVar);
	}

	if (InPin != nullptr)
	{
		PerVariableWriteHistory[FoundIdx].Emplace(InPin, ModuleName);
	}

	check(Variables.Num() == PerVariableWarnings.Num());
	check(Variables.Num() == PerVariableReadHistory.Num());
	check(Variables.Num() == PerVariableWriteHistory.Num());
	check(Variables.Num() == PerVariableConstantValue.Num());
	check(Variables.Num() == PerVariableConstantDefaultValue.Num());

	return FoundIdx;
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistory<GraphBridge>::AddExternalVariable(const FNiagaraVariable& Var)
{
	return AddVariable(Var, Var, NAME_None, nullptr);
}

template<typename GraphBridge>
const typename GraphBridge::FPin* TNiagaraParameterMapHistory<GraphBridge>::GetFinalPin() const
{
	if (MapPinHistory.Num() > 0)
	{
		return MapPinHistory[MapPinHistory.Num() - 1];
	}
	return nullptr;
}

template<typename GraphBridge>
const typename GraphBridge::FPin* TNiagaraParameterMapHistory<GraphBridge>::GetOriginalPin() const
{
	if (MapPinHistory.Num() > 0)
	{
		return MapPinHistory[0];
	}
	return nullptr;
}


FName FNiagaraParameterUtilities::ResolveEmitterAlias(const FName& InName, const FString& InAlias)
{
	// If the alias is empty than the name can't be resolved.
	if (InAlias.IsEmpty())
	{
		return InName;
	}

	FNiagaraVariable Var(FNiagaraTypeDefinition::GetFloatDef(), InName);
	FNiagaraAliasContext ResolveAliasesContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript);
	ResolveAliasesContext.ChangeEmitterToEmitterName(InAlias);
	Var = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);
	return Var.GetName();
}



FString FNiagaraParameterUtilities::MakeSafeNamespaceString(const FString& InStr)
{
	FString  Sanitized = FNiagaraHlslTranslator::GetSanitizedSymbolName(InStr);
	return Sanitized;
}


FNiagaraVariable FNiagaraParameterUtilities::ResolveAsBasicAttribute(const FNiagaraVariable& InVar, bool bSanitizeString)
{
	if (IsAttribute(InVar))
	{
		FString ParamName = InVar.GetName().ToString();
		ParamName.RemoveAt(0, FString(PARAM_MAP_ATTRIBUTE_STR).Len());

		if (bSanitizeString)
		{
			ParamName = MakeSafeNamespaceString(ParamName);
		}
		FNiagaraVariable RetVar = InVar;
		RetVar.SetName(*ParamName);
		return RetVar;
	}
	else
	{
		return InVar;
	}
}

FNiagaraVariable FNiagaraParameterUtilities::BasicAttributeToNamespacedAttribute(const FNiagaraVariable& InVar, bool bSanitizeString)
{
	FString ParamName = InVar.GetName().ToString();
	ParamName.InsertAt(0, FString(PARAM_MAP_ATTRIBUTE_STR));

	if (bSanitizeString)
	{
		ParamName = MakeSafeNamespaceString(ParamName);
	}

	FNiagaraVariable RetVar = InVar;
	RetVar.SetName(*ParamName);
	return RetVar;
}

FNiagaraVariable FNiagaraParameterUtilities::VariableToNamespacedVariable(const FNiagaraVariable& InVar, FString Namespace)
{
	FString ParamName = Namespace;
	if (Namespace.EndsWith(TEXT(".")))
	{
		ParamName += InVar.GetName().ToString();
	}
	else
	{
		ParamName += TEXT(".") + InVar.GetName().ToString();
	}
	

	FNiagaraVariable RetVar = InVar;
	RetVar.SetName(*ParamName);
	return RetVar;
}

bool FNiagaraParameterUtilities::IsInNamespace(const FNiagaraVariableBase& InVar, const FString& Namespace)
{
	const bool FastTest = InVar.IsInNameSpace(Namespace);

	// Leaving old code commented out for now just as a reference to what was before.
	//if (Namespace.EndsWith(TEXT(".")))
	//{
	//	bool SlowTest =  InVar.GetName().ToString().StartsWith(Namespace);
	//	check(FastTest == SlowTest);
	//	return FastTest;
	//}
	//else
	//{
	//	bool SlowTest = InVar.GetName().ToString().StartsWith(Namespace + TEXT("."));
	//	check(FastTest == SlowTest);
	//	return FastTest;
	//}

	return FastTest;
}

bool FNiagaraParameterUtilities::IsAliasedModuleParameter(const FNiagaraVariable& InVar)
{
	return IsInNamespace(InVar, PARAM_MAP_MODULE_STR);
}

bool FNiagaraParameterUtilities::IsAliasedEmitterParameter(const FNiagaraVariable& InVar)
{
	return IsInNamespace(InVar, PARAM_MAP_EMITTER_STR);
}

bool FNiagaraParameterUtilities::IsAliasedEmitterParameter(const FString& InVarName)
{
	return IsAliasedEmitterParameter(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), *InVarName));
}


bool FNiagaraParameterUtilities::IsSystemParameter(const FNiagaraVariable& InVar)
{
	return IsInNamespace(InVar, PARAM_MAP_SYSTEM_STR);
}

bool FNiagaraParameterUtilities::IsEngineParameter(const FNiagaraVariable& InVar)
{
	return IsInNamespace(InVar, PARAM_MAP_ENGINE_STR);
}

bool FNiagaraParameterUtilities::IsPerInstanceEngineParameter(const FNiagaraVariable& InVar, const FString& EmitterAlias)
{
	FString EmitterEngineNamespaceAlias = TEXT("Engine.") + EmitterAlias + TEXT(".");
	return IsInNamespace(InVar, PARAM_MAP_ENGINE_OWNER_STR) || IsInNamespace(InVar, PARAM_MAP_ENGINE_SYSTEM_STR) || IsInNamespace(InVar, PARAM_MAP_ENGINE_EMITTER_STR) || 
		IsInNamespace(InVar, EmitterEngineNamespaceAlias);
}

bool FNiagaraParameterUtilities::IsUserParameter(const FNiagaraVariable& InVar)
{
	return IsInNamespace(InVar, PARAM_MAP_USER_STR);
}

bool FNiagaraParameterUtilities::IsRapidIterationParameter(const FNiagaraVariable& InVar)
{
	return IsInNamespace(InVar, PARAM_MAP_RAPID_ITERATION_STR);
}

bool FNiagaraParameterUtilities::SplitRapidIterationParameterName(const FNiagaraVariable& InVar, ENiagaraScriptUsage InUsage, FString& EmitterName, FString& FunctionCallName, FString& InputName)
{
	TArray<FString> SplitName;
	InVar.GetName().ToString().ParseIntoArray(SplitName, TEXT("."));
	int32 MinimumSplitCount = InUsage == ENiagaraScriptUsage::SystemSpawnScript || InUsage == ENiagaraScriptUsage::SystemUpdateScript ? 3 : 4;
	if (SplitName.Num() >= MinimumSplitCount && (SplitName[0] + ".") == PARAM_MAP_RAPID_ITERATION_STR)
	{
		int32 CurrentIndex = 1;
		if (InUsage == ENiagaraScriptUsage::SystemSpawnScript || InUsage == ENiagaraScriptUsage::SystemUpdateScript)
		{
			EmitterName = FString();
		}
		else
		{
			EmitterName = SplitName[CurrentIndex];
			CurrentIndex++;
		}

		FunctionCallName = SplitName[CurrentIndex];
		CurrentIndex++;

		// Join any remaining name parts with a .
		InputName = FString::Join(TArrayView<FString>(SplitName).Slice(CurrentIndex, SplitName.Num() - CurrentIndex), TEXT("."));
		return true;
	}
	return false;
}

bool FNiagaraParameterUtilities::IsAttribute(const FNiagaraVariableBase& InVar)
{
	return IsInNamespace(InVar, PARAM_MAP_ATTRIBUTE_STR);
}

template<typename GraphBridge>
const typename GraphBridge::FInputPin* TNiagaraParameterMapHistory<GraphBridge>::GetDefaultValuePin(int32 VarIdx) const
{
	if (PerVariableWriteHistory[VarIdx].Num() > 0)
	{
		const TModuleScopedPin<FPin>& ScopedPin = PerVariableWriteHistory[VarIdx][0];
		if (ScopedPin.Pin != nullptr && ScopedPin.Pin->Direction == EEdGraphPinDirection::EGPD_Input && GraphBridge::IsParameterMapGet(GraphBridge::GetOwningNode(ScopedPin.Pin)))
		{
			return GraphBridge::GetPinAsInput(ScopedPin.Pin);
		}
	}
	return nullptr;
}

static bool DoesVariableIncludeNamespace(const FName& InVariableName, const TCHAR* Namespace)
{
	TArray<FString> SplitName;
	InVariableName.ToString().ParseIntoArray(SplitName, TEXT("."));

	for (int32 i = 1; i < SplitName.Num() - 1; i++)
	{
		if (SplitName[i].Equals(Namespace, ESearchCase::IgnoreCase))
		{
			return true;
		}
	}
	return false;
}

static FName ReplaceVariableNamespace(const FName& InVariableName, const TCHAR* Namespace)
{
	TArray<FString> SplitName;
	InVariableName.ToString().ParseIntoArray(SplitName, TEXT("."));

	TArray<FString> JoinString;
	bool bFound = false;
	for (int32 i = 0; i < SplitName.Num(); i++)
	{
		if (!bFound && SplitName[i].Equals(Namespace, ESearchCase::IgnoreCase))
		{
			bFound = true;
			continue;
		}
		else
		{
			JoinString.Add(SplitName[i]);
		}
	}

	return *FString::Join(JoinString, TEXT("."));
}

bool FNiagaraParameterUtilities::IsInitialName(const FName& InVariableName)
{
	return DoesVariableIncludeNamespace(InVariableName, PARAM_MAP_INITIAL_BASE_STR);
}

bool FNiagaraParameterUtilities::IsInitialValue(const FNiagaraVariableBase& InVar)
{
	return IsInitialName(InVar.GetName());
}

bool FNiagaraParameterUtilities::IsPreviousName(const FName& InVariableName)
{
	return DoesVariableIncludeNamespace(InVariableName, PARAM_MAP_PREVIOUS_BASE_STR);
}

bool FNiagaraParameterUtilities::IsPreviousValue(const FNiagaraVariableBase& InVar)
{
	return IsPreviousName(InVar.GetName());
}

FNiagaraVariable FNiagaraParameterUtilities::GetSourceForInitialValue(const FNiagaraVariable& InVar)
{
	FNiagaraVariable Var = InVar;
	Var.SetName(GetSourceForInitialValue(InVar.GetName()));
	return Var;
}

FName FNiagaraParameterUtilities::GetSourceForInitialValue(const FName& InVariableName)
{
	return ReplaceVariableNamespace(InVariableName, PARAM_MAP_INITIAL_BASE_STR);
}

FNiagaraVariable FNiagaraParameterUtilities::GetSourceForPreviousValue(const FNiagaraVariable& InVar)
{
	FNiagaraVariable Var = InVar;
	Var.SetName(GetSourceForPreviousValue(InVar.GetName()));
	return Var;
}

FName FNiagaraParameterUtilities::GetSourceForPreviousValue(const FName& InVariableName)
{
	return ReplaceVariableNamespace(InVariableName, PARAM_MAP_PREVIOUS_BASE_STR);
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistory<GraphBridge>::IsPrimaryDataSetOutput(const FNiagaraVariableBase& InVar, const UNiagaraScript* InScript,  bool bAllowDataInterfaces, bool bAllowStatics) const
{
	return IsPrimaryDataSetOutput(InVar, InScript->GetUsage(),  bAllowDataInterfaces);
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistory<GraphBridge>::IsPrimaryDataSetOutput(const FNiagaraVariableBase& InVar, ENiagaraScriptUsage Usage, bool bAllowDataInterfaces, bool bAllowStatics) const
{
	if (bAllowDataInterfaces == false && InVar.GetType().GetClass() != nullptr)
	{
		return false;
	}

	if (bAllowStatics == false && InVar.GetType().IsStatic())
	{
		return false;
	}

	if (Usage == ENiagaraScriptUsage::EmitterSpawnScript || Usage == ENiagaraScriptUsage::EmitterUpdateScript || 
		Usage == ENiagaraScriptUsage::SystemSpawnScript || Usage == ENiagaraScriptUsage::SystemUpdateScript)
	{
		// In the case of system/emitter scripts we must include the variables in the overall system namespace as well as any of 
		// the child emitters that were encountered.
		for (const FString& EmitterEncounteredNamespace : EmitterNamespacesEncountered)
		{
			if (InVar.IsInNameSpace(EmitterEncounteredNamespace))
			{
				return true;
			}
		}
		return InVar.IsInNameSpace(FStringView(PARAM_MAP_SYSTEM_STR)) || InVar.IsInNameSpace(FStringView(PARAM_MAP_EMITTER_STR));
	}
	else if (Usage == ENiagaraScriptUsage::Module || Usage == ENiagaraScriptUsage::Function)
	{
		return InVar.IsInNameSpace(FStringView(PARAM_MAP_MODULE_STR));
	}
	return InVar.IsInNameSpace(FStringView(PARAM_MAP_ATTRIBUTE_STR));
}

bool FNiagaraParameterUtilities::IsWrittenToScriptUsage(const FNiagaraVariable& InVar, ENiagaraScriptUsage Usage, bool bAllowDataInterfaces)
{
	if (bAllowDataInterfaces == false && InVar.GetType().GetClass() != nullptr)
	{
		return false;
	}

	if (Usage == ENiagaraScriptUsage::EmitterSpawnScript || Usage == ENiagaraScriptUsage::EmitterUpdateScript ||
		Usage == ENiagaraScriptUsage::SystemSpawnScript || Usage == ENiagaraScriptUsage::SystemUpdateScript)
	{
		return IsInNamespace(InVar, PARAM_MAP_SYSTEM_STR) || IsInNamespace(InVar, PARAM_MAP_EMITTER_STR);
	}
	else if (Usage == ENiagaraScriptUsage::Module || Usage == ENiagaraScriptUsage::Function)
	{
		return IsInNamespace(InVar, PARAM_MAP_MODULE_STR);
	}
	return IsInNamespace(InVar, PARAM_MAP_ATTRIBUTE_STR);
}

FNiagaraVariable FNiagaraParameterUtilities::MoveToExternalConstantNamespaceVariable(const FNiagaraVariable& InVar, ENiagaraScriptUsage InUsage)
{
	if (UNiagaraScript::IsParticleScript(InUsage))
	{
		return VariableToNamespacedVariable(InVar, PARAM_MAP_EMITTER_STR);
	}
	else if (UNiagaraScript::IsStandaloneScript(InUsage))
	{
		return VariableToNamespacedVariable(InVar, PARAM_MAP_MODULE_STR);
	}
	else if (UNiagaraScript::IsEmitterSpawnScript(InUsage) || UNiagaraScript::IsEmitterUpdateScript(InUsage) || UNiagaraScript::IsSystemSpawnScript(InUsage) || UNiagaraScript::IsSystemUpdateScript(InUsage))
	{
		return VariableToNamespacedVariable(InVar, PARAM_MAP_USER_STR);
	}
	return InVar;
}

FNiagaraVariable FNiagaraParameterUtilities::MoveToExternalConstantNamespaceVariable(const FNiagaraVariable& InVar, const UNiagaraScript* InScript)
{
	return MoveToExternalConstantNamespaceVariable(InVar, InScript->GetUsage());
}

bool FNiagaraParameterUtilities::IsExportableExternalConstant(const FNiagaraVariable& InVar, const UNiagaraScript* InScript)
{
	if (InScript->IsEquivalentUsage(ENiagaraScriptUsage::SystemSpawnScript))
	{
		return IsExternalConstantNamespace(InVar, InScript, FGuid());
	}
	else
	{
		return false;
	}
}

bool FNiagaraParameterUtilities::IsExternalConstantNamespace(const FNiagaraVariable& InVar, ENiagaraScriptUsage InUsage, int32 InUsageBitmask)
{
	// Parameter collections are always constants
	if (IsInNamespace(InVar, PARAM_MAP_NPC_STR))
	{
		return true;
	}

	// Engine parameters are never writable.
	if (IsInNamespace(InVar, PARAM_MAP_ENGINE_STR))
	{
		return true;
	}

	if (IsInNamespace(InVar, PARAM_MAP_USER_STR))
	{
		return true;
	}

	if (IsInNamespace(InVar, PARAM_MAP_INDICES_STR))
	{
		return true;
	}
	

	// Modules and functions need to act as if they are within the script types that they 
	// say that they support rather than using their exact script type.
	if (UNiagaraScript::IsStandaloneScript(InUsage))
	{
		TArray<ENiagaraScriptUsage> SupportedContexts = UNiagaraScript::GetSupportedUsageContextsForBitmask(InUsageBitmask);
		if (((!SupportedContexts.Contains(ENiagaraScriptUsage::EmitterSpawnScript) && !SupportedContexts.Contains(ENiagaraScriptUsage::EmitterUpdateScript)) && IsInNamespace(InVar, PARAM_MAP_EMITTER_STR))
			|| ((!SupportedContexts.Contains(ENiagaraScriptUsage::SystemSpawnScript) && !SupportedContexts.Contains(ENiagaraScriptUsage::SystemUpdateScript)) && IsInNamespace(InVar, PARAM_MAP_SYSTEM_STR)))
		{
			return true;
		}
	}

	// Particle scripts cannot write to the emitter or system namespace.
	if (UNiagaraScript::IsParticleScript(InUsage))
	{
		if (IsInNamespace(InVar, PARAM_MAP_EMITTER_STR) || IsInNamespace(InVar, PARAM_MAP_SYSTEM_STR))
		{
			return true;
		}
	}

	return false;
}

bool FNiagaraParameterUtilities::IsExternalConstantNamespace(const FNiagaraVariable& InVar, const UNiagaraScript* InScript, const FGuid& VersionGuid)
{
	return IsExternalConstantNamespace(InVar, InScript->GetUsage(), InScript->GetScriptData(VersionGuid)->ModuleUsageBitmask);
}

template<typename GraphBridge>
const typename GraphBridge::FOutputNode* TNiagaraParameterMapHistory<GraphBridge>::GetFinalOutputNode() const
{
	if (const FPin* Pin = GetFinalPin())
	{
		return GraphBridge::AsOutputNode(GraphBridge::GetOwningNode(Pin));
	}

	return nullptr;
}

FNiagaraVariable FNiagaraParameterUtilities::ConvertVariableToRapidIterationConstantName(FNiagaraVariable InVar, const TCHAR* InEmitterName, ENiagaraScriptUsage InUsage)
{
	return FNiagaraUtilities::ConvertVariableToRapidIterationConstantName(InVar, InEmitterName, InUsage);
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistory<GraphBridge>::ShouldIgnoreVariableDefault(const FNiagaraVariable& Var)const
{
	// NOTE(mv): Used for variables that are explicitly assigned to (on spawn) and should not be default initialized
	//           These are explicitly written to in NiagaraHlslTranslator::DefineMain
	bool bShouldBeIgnored = false;
	bShouldBeIgnored |= (Var == FNiagaraVariable(FNiagaraTypeDefinition::GetIDDef(), TEXT("Particles.ID")));
	bShouldBeIgnored |= (Var == FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("Particles.UniqueID")));
	return bShouldBeIgnored;
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistory<GraphBridge>::IsVariableFromCustomIterationNamespaceOverride(const FNiagaraVariable& InVar) const
{
	for (const FName& Name : IterationNamespaceOverridesEncountered)
	{
		if (Name.IsValid())
		{
			if (InVar.IsInNameSpace(Name))
				return true;
		}
	}
	return false;
}

template<typename GraphBridge>
TNiagaraParameterMapHistoryBuilder<GraphBridge>::TNiagaraParameterMapHistoryBuilder()
{
	ContextuallyVisitedNodes.AddDefaulted(1);
	PinToParameterMapIndices.AddDefaulted(1);
	PinToConstantIndices.AddDefaulted(1);
	VariableToConstantIndices.AddDefaulted(1);
	bFilterByScriptAllowList = false;
	bIgnoreDisabled = true;
	FilterScriptType = ENiagaraScriptUsage::Function;

	ConstantResolver = MakePimpl<FConstantResolver, EPimplPtrMode::DeepCopy>();

	// depending on when this builder is created, we could be doing a PostLoad.  In this scenario
	// we need to avoid triggering any loads for dependent NiagaraParameterCollection.  Mark the builder
	// to skip those steps if necessary and we'll ensure that the generated histories don't require the
	// collection information
	bIncludeParameterCollectionInfo = !FUObjectThreadContext::Get().IsRoutingPostLoad;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::BuildParameterMaps(const FOutputNode* OutputNode, bool bRecursive)
{
	const ENiagaraScriptUsage NodeUsage = GraphBridge::GetOutputNodeUsage(OutputNode);

	if (UNiagaraScript::LogCompileStaticVars > 0)
	{
		UE_LOGF(LogNiagaraEditor, Log, "************** BuildParameterMaps %d  **************", int(NodeUsage));
	}
	TOptional<FName> StackContextAlias = GraphBridge::GetOutputNodeStackContextOverride(OutputNode);
	BeginUsage(NodeUsage, StackContextAlias.Get(ScriptUsageContextNameStack.Num() > 0 ? ScriptUsageContextNameStack.Top() : NAME_None));

	constexpr bool bFilterForCompilation = true;
	OutputNode->BuildParameterMapHistory(*this, bRecursive, bFilterForCompilation);

	EndUsage();
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::EnableScriptAllowList(bool bInEnable, ENiagaraScriptUsage InScriptType)
{
	bFilterByScriptAllowList = bInEnable;
	FilterScriptType = InScriptType;
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::HasCurrentUsageContext() const
{
	return RelevantScriptUsageContext.Num() > 0;
}


template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::ContextContains(ENiagaraScriptUsage InUsage) const
{
	if (RelevantScriptUsageContext.Num() == 0)
	{
		return false;
	}
	else
	{
		return RelevantScriptUsageContext.Contains(InUsage);
	}
}

template<typename GraphBridge>
ENiagaraScriptUsage TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetCurrentUsageContext()const
{
	if (RelevantScriptUsageContext.Num() == 0)
		return ENiagaraScriptUsage::Function;
	return RelevantScriptUsageContext.Last();
}

template<typename GraphBridge>
ENiagaraScriptUsage TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetBaseUsageContext()const
{
	if (RelevantScriptUsageContext.Num() == 0)
		return ENiagaraScriptUsage::Function;
	return RelevantScriptUsageContext[0];
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::CreateParameterMap()
{
	int32 RetValue = Histories.AddDefaulted(1);
	FHistory& History = Histories[RetValue];

	FName StackName = ScriptUsageContextNameStack.Num() > 0 ? ScriptUsageContextNameStack.Top() : NAME_None;

	if (StackName.IsValid())
	{
		History.IterationNamespaceOverridesEncountered.AddUnique(StackName);
	}

	History.bParameterCollectionsSkipped = !bIncludeParameterCollectionInfo;

	return RetValue;
}

template<typename GraphBridge>
uint32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::BeginNodeVisitation(int32 WhichParameterMap, const FNode* Node)
{
	if (WhichParameterMap != INDEX_NONE)
	{
		return Histories[WhichParameterMap].BeginNodeVisitation(Node);
	}
	else
	{
		return INDEX_NONE;
	}
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::EndNodeVisitation(int32 WhichParameterMap, uint32 IndexFromBeginNode)
{
	if (WhichParameterMap != INDEX_NONE)
	{
		return Histories[WhichParameterMap].EndNodeVisitation(IndexFromBeginNode);
	}
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::RegisterDataSetWrite(int32 WhichParameterMap, const FNiagaraDataSetID& DataSet)
{
	if (Histories.IsValidIndex(WhichParameterMap))
	{
		Histories[WhichParameterMap].AdditionalDataSetWrites.AddUnique(DataSet);
	}
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::RegisterParameterMapPin(int32 WhichParameterMap, const FPin* Pin)
{
	if (Pin && UNiagaraScript::LogCompileStaticVars > 0)
	{
		UE_LOGF(LogNiagaraEditor, Log, "FNiagaraParameterMapHistoryBuilder::RegisterParameterMapPin %d > %ls .... %p", WhichParameterMap, *GraphBridge::GetNodeTitle(GraphBridge::GetOwningNode(Pin)), Pin);
	}
	if (Pin && WhichParameterMap != INDEX_NONE)
	{
		if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
		{
			PinToParameterMapIndices.Last().Add(Pin, WhichParameterMap);
			if (UNiagaraScript::LogCompileStaticVars > 0)
			{
				UE_LOGF(LogNiagaraEditor, Log, "FNiagaraParameterMapHistoryBuilder::RegisterParameterMapPin Added %d", WhichParameterMap);
			}
		}
		else if (UNiagaraScript::LogCompileStaticVars > 0)
		{
			UE_LOGF(LogNiagaraEditor, Log, "FNiagaraParameterMapHistoryBuilder::RegisterParameterMapPin Not Output %p", Pin);
		}

		return Histories[WhichParameterMap].RegisterParameterMapPin(Pin);
	}
	else
	{
		return INDEX_NONE;
	}
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::RegisterConstantPin(int32 WhichConstant, const FPin* Pin)
{
	if (WhichConstant != INDEX_NONE)
	{
		PinToConstantIndices.Last().Add(Pin, WhichConstant);

		if (Histories.Num() > 0)
			Histories.Last().RegisterConstantPin(ActivePath, Pin, Constants[WhichConstant]);

		return WhichConstant;
	}
	else
	{
		return INDEX_NONE;
	}
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::IsStaticVariableExportableToOuterScopeBasedOnCurrentContext(const FNiagaraVariable& Var) const
{
	if ((IsInEncounteredEmitterNamespace(Var) || FNiagaraParameterUtilities::IsAliasedEmitterParameter(Var)) && (ContextContains(ENiagaraScriptUsage::EmitterSpawnScript) || ContextContains(ENiagaraScriptUsage::EmitterUpdateScript)))
	{
		return true;
	}
	else if (FNiagaraParameterUtilities::IsAttribute(Var) && (UNiagaraScript::IsParticleScript(GetBaseUsageContext())))
	{
		return true;
	}
	else if (FNiagaraParameterUtilities::IsSystemParameter(Var) && (UNiagaraScript::IsSystemScript(GetBaseUsageContext()) || UNiagaraScript::IsEmitterScript(GetBaseUsageContext())))
	{
		return true;
	}
	return false;
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::RegisterConstantVariableWrite(int32 WhichParamMapIdx, int32 WhichConstant,  int32 WhichVarIdx, bool bIsSettingDefault, bool bIsLinkNotValue)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	if (WhichConstant != INDEX_NONE)
	{
		const FNiagaraVariableBase& Var = Histories[WhichParamMapIdx].Variables[WhichVarIdx];
		if (Var.IsValid())
		{
			VariableToConstantIndices.Last().Add(Var, WhichConstant);

			if (WhichParamMapIdx != INDEX_NONE)
			{
				Histories[WhichParamMapIdx].RegisterConstantVariableWrite(Constants[WhichConstant], WhichVarIdx, bIsSettingDefault, bIsLinkNotValue);
			}

			if (bIsLinkNotValue == false && Constants[WhichConstant].Len() != 0 && Var.GetType().IsStatic())
			{
				TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(Var.GetType());
				if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanHandlePinDefaults())
				{
					FNiagaraVariable VarWithValue = Var;
					VarWithValue.AllocateData();
					if (TypeEditorUtilities->SetValueFromPinDefaultString(Constants[WhichConstant], VarWithValue))
					{
						// Index of uses == operator, which only checks name and type. This will allow us to detect instances of the duplicate
						// data down the line.
						int32 LastStaticVarIdx = StaticVariables.Find(Var);
						int32 AddedIdx = INDEX_NONE;
						if (LastStaticVarIdx == INDEX_NONE)
						{
							AddedIdx = StaticVariables.Add(VarWithValue); // Didn't find it, so add it.
						}
						else if (false == VarWithValue.HoldsSameData(StaticVariables[LastStaticVarIdx]))
						{
							AddedIdx = StaticVariables.Add(VarWithValue); // Add as a duplicate here. We will filter out later
						}

						if (AddedIdx != INDEX_NONE)
						{
							bool bExportable = IsStaticVariableExportableToOuterScopeBasedOnCurrentContext(Var);
							int32 AddedBoolIdx = StaticVariableExportable.Emplace(bExportable);
							ensure(AddedBoolIdx == AddedIdx);
						}
					}
				}
			}
		}

		return WhichConstant;
	}
	else
	{
		return INDEX_NONE;
	}
}


template<typename GraphBridge>
FString TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetConstant(int32 InIndex)
{
	if (Constants.IsValidIndex(InIndex))
		return Constants[InIndex];
	return FString();
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::AddOrGetConstantFromValue(const FString& Value)
{
	return Constants.AddUnique(Value);
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::RegisterConstantFromInputPin(const FInputPin* InputPin, const FString& PinDefaultValue)
{
	int32 ConstantIdx = INDEX_NONE;
	if (InputPin && InputPin->Direction == EEdGraphPinDirection::EGPD_Input)
	{
		if (const FOutputPin* ConnectedOutputPin = GraphBridge::GetLinkedOutputPin(InputPin))
		{
			ConstantIdx = GetConstantFromOutputPin(ConnectedOutputPin);
			RegisterConstantPin(ConstantIdx, InputPin);
		}
		else
		{
			ensure(InputPin->Direction == EEdGraphPinDirection::EGPD_Input);
			if (!PinDefaultValue.IsEmpty() && !InputPin->PinName.IsNone())
			{
				ConstantIdx = AddOrGetConstantFromValue(PinDefaultValue);
				RegisterConstantPin(ConstantIdx, InputPin);
			}
		}
	}

	if (UNiagaraScript::LogCompileStaticVars > 0 || NiagaraDebugShouldLogEntriesByNameSubstring(InputPin->PinName.ToString()))
	{
		FGraphTraversalHandle TestPath = ActivePath;
		TestPath.PushPin(InputPin);
		UE_LOGF(LogNiagaraEditor, Log, "RegisterConstantFromInputPin Value \"%ls\" Var: \"%ls\"", ConstantIdx != INDEX_NONE ? *Constants[ConstantIdx] : TEXT("INDEX_NONE"), *TestPath.ToString());
	}

	return ConstantIdx;
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetConstantFromInputPin(const FInputPin* InputPin) const
{
	if (InputPin && InputPin->Direction == EEdGraphPinDirection::EGPD_Input)
	{
		if (InputPin)
		{
			const int32* IdxPtr = PinToConstantIndices.Last().Find(InputPin);
			if (IdxPtr != nullptr)
			{
				if (UNiagaraScript::LogCompileStaticVars > 0 || NiagaraDebugShouldLogEntriesByNameSubstring(InputPin->PinName.ToString()))
				{
					FGraphTraversalHandle TestPath = ActivePath;
					TestPath.PushPin(InputPin);
					UE_LOGF(LogNiagaraEditor, Log, "GetConstantFromInputPin Value \"%ls\" Var: \"%ls\"", *Constants[*IdxPtr], *TestPath.ToString());
				}
				return *IdxPtr;
			}
		}
	}
	return INDEX_NONE;
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetConstantFromOutputPin(const FOutputPin* OutputPin) const
{
	if (OutputPin && OutputPin->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		if (OutputPin)
		{
			const int32* IdxPtr = PinToConstantIndices.Last().Find(OutputPin);
			if (IdxPtr != nullptr)
			{
				if (UNiagaraScript::LogCompileStaticVars > 0 || NiagaraDebugShouldLogEntriesByNameSubstring(OutputPin->GetName()))
				{
					FGraphTraversalHandle TestPath = ActivePath;
					TestPath.PushPin(OutputPin);
					UE_LOGF(LogNiagaraEditor, Log, "GetConstantFromOutputPin Value \"%ls\" Var: \"%ls\"", *Constants[*IdxPtr], *TestPath.ToString());
				}
				return *IdxPtr;
			}
		}
	}
	return INDEX_NONE;
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetConstantFromVariableRead(const FNiagaraVariableBase& InVar)
{
	if (InVar.IsValid())
	{
		for (int32 i = VariableToConstantIndices.Num() - 1; i >= 0; i--)
		{
			const int32* IdxPtr = VariableToConstantIndices[i].Find(InVar);
			if (IdxPtr != nullptr)
			{
				if (UNiagaraScript::LogCompileStaticVars > 0 || NiagaraDebugShouldLogEntriesByNameSubstring(ActivePath.ToString()))
				{
					UE_LOGF(LogNiagaraEditor, Log, "GetConstantFromVariableRead Value \"%ls\" Var: \"%ls\" Path: \"%ls\"", *Constants[*IdxPtr], *InVar.GetName().ToString(),  *ActivePath.ToString());
				}
				return *IdxPtr;
			}
		}
		
	}
	return INDEX_NONE;
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::TraceParameterMapOutputPin(const FOutputPin* OutputPin)
{
	if (OutputPin && OutputPin->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		if (OutputPin)
		{
			const int32* IdxPtr = PinToParameterMapIndices.Last().Find(OutputPin);
			if (IdxPtr != nullptr)
			{
				return *IdxPtr;
			}
		}
	}
	return INDEX_NONE;
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetPinPreviouslyVisited(const FPin* InPin) const
{
	if (InPin != nullptr)
	{
		return GetNodePreviouslyVisited(GraphBridge::GetOwningNode(InPin));
	}

	return true;
}


template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetNodePreviouslyVisited(const FNode* Node) const
{
	return ContextuallyVisitedNodes.Last().Contains(Node);
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::FindMatchingParameterMapFromContextInputs(const FNiagaraVariable& InVar) const
{
	if (CallingContext.Num() == 0)
	{
		return INDEX_NONE;
	}
	const FNode* Node = CallingContext.Last();
	TArray<const FInputPin*> Inputs = GraphBridge::GetInputPins(Node);

	for (int32 i = 0; i < Inputs.Num(); i++)
	{
		FNiagaraVariable CallInputVar = GraphBridge::GetPinVariable(Inputs[i], false, ENiagaraStructConversion::UserFacing);
		if (CallInputVar.IsEquivalent(InVar) && CallInputVar.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			const FOutputPin* OutputPin = GraphBridge::GetLinkedOutputPin(Inputs[i]);
			if (OutputPin && PinToParameterMapIndices.Num() >= 2)
			{
				const int32* ParamMapIdxPtr = PinToParameterMapIndices[PinToParameterMapIndices.Num() - 2].Find(OutputPin);
				if (ParamMapIdxPtr != nullptr)
				{
					return *ParamMapIdxPtr;
				}
			}
		}
	}
	return INDEX_NONE;
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::FindMatchingStaticFromContextInputs(const FNiagaraVariable& InVar) const
{
	if (CallingContext.Num() == 0 || PinToConstantIndices.Num() < 2)
	{
		return INDEX_NONE;
	}

	const TMap<const FPin*, int32>& ConstantIndexMapping = PinToConstantIndices[PinToConstantIndices.Num() - 2];
	const FNode* Node = CallingContext.Last();
	TArray<const FInputPin*> Inputs = GraphBridge::GetInputPins(Node);

	for (int32 i = 0; i < Inputs.Num(); i++)
	{
		FNiagaraVariable CallInputVar = GraphBridge::GetPinVariable(Inputs[i], false, ENiagaraStructConversion::UserFacing);
		if (CallInputVar.IsEquivalent(InVar))
		{
			const int32* IdxPtr = nullptr;

			if (const FOutputPin* OutputPin = GraphBridge::GetLinkedOutputPin(Inputs[i]))
			{
				// If linked to something, then the output we are linked to will carry the constant.
				IdxPtr = ConstantIndexMapping.Find(OutputPin);
			}
			else
			{
				// If just using the default, the input pin will carry the constant.
				IdxPtr = ConstantIndexMapping.Find(Inputs[i]);
			}

			if (IdxPtr)
			{
				return *IdxPtr;
			}
		}
	}
	return INDEX_NONE;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::BeginTranslation(const UNiagaraScript* Script)
{
	//For now this will just tell particle scripts what emitter they're being compiled as part of but maybe we want to do more here.
	//This is mainly so that parameter names match up between System/Emitter scripts and the parameters they drive within particle scripts.
	//I dislike this coupling of the translator to emitters but for now it'll have to do.
	//Will refactor in the future.
	UNiagaraEmitter* Emitter = Script->GetTypedOuter<UNiagaraEmitter>();
	BeginTranslation(Emitter);
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::EndTranslation(const UNiagaraScript* Script)
{
	EmitterNameContextStack.Reset();
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::BeginTranslation(const UNiagaraEmitter* Emitter)
{
	//For now this will just tell particle scripts what emitter they're being compiled as part of but maybe we want to do more here.
	//This is mainly so that parameter names match up between System/Emitter scripts and the parameters they drive within particle scripts.
	//I dislike this coupling of the translator to emitters but for now it'll have to do.
	//Will refactor in the future.
	if (Emitter)
	{
		FString EmitterUniqueName = Emitter->GetUniqueEmitterName();
		EmitterNameContextStack.Add(*EmitterUniqueName);
	}
	BuildCurrentAliases();
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::EndTranslation(const UNiagaraEmitter* Emitter)
{
	EmitterNameContextStack.Reset();
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::BeginTranslation(const FString& EmitterUniqueName)
{
	//For now this will just tell particle scripts what emitter they're being compiled as part of but maybe we want to do more here.
	//This is mainly so that parameter names match up between System/Emitter scripts and the parameters they drive within particle scripts.
	//I dislike this coupling of the translator to emitters but for now it'll have to do.
	//Will refactor in the future.
	if (EmitterUniqueName.IsEmpty() == false)
	{
		EmitterNameContextStack.Add(*EmitterUniqueName);
	}
	BuildCurrentAliases();
}
template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::EndTranslation(const FString& EmitterUniqueName)
{
	EmitterNameContextStack.Reset();
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::BeginUsage(ENiagaraScriptUsage InUsage, FName InStageName)
{
	RelevantScriptUsageContext.Push(InUsage);
	ScriptUsageContextNameStack.Push(InStageName);
		
	BuildCurrentAliases();
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::EndUsage()
{
	RelevantScriptUsageContext.Pop();
	ScriptUsageContextNameStack.Pop();
}

template<typename GraphBridge>
const typename GraphBridge::FNode* TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetCallingContext() const
{
	return CallingContext.IsEmpty() ? nullptr : CallingContext.Last();
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::InTopLevelFunctionCall(ENiagaraScriptUsage InFilterScriptType) const
{
	if (InFilterScriptType == ENiagaraScriptUsage::EmitterSpawnScript || InFilterScriptType == ENiagaraScriptUsage::EmitterUpdateScript || InFilterScriptType == ENiagaraScriptUsage::SystemSpawnScript || InFilterScriptType == ENiagaraScriptUsage::SystemUpdateScript)
	{
		if (CallingContext.Num() <= 1) // Handles top-level system graph and any function calls off of it.
		{
			return true;
		}
		else if (CallingContext.Num() <= 2 && GraphBridge::GetNodeAsEmitter(CallingContext[0]) != nullptr) // Handle a function call off of an emitter
		{
			return true;
		}
	}
	else if (UNiagaraScript::IsParticleScript(InFilterScriptType))
	{
		if (CallingContext.Num() <= 1) // Handle a function call
		{
			return true;
		}
	}

	return false;
}


template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::EnterFunction(const FString& InNodeName, const FGraph* InGraph, const FNode* Node)
{
	RegisterNodeVisitation(Node);
	CallingContext.Push(Node);
	ActivePath.PushNode(Node);
	CallingGraphContext.Push(InGraph);
	PinToParameterMapIndices.Emplace();
	PinToConstantIndices.Emplace();
	FunctionNameContextStack.Emplace(*InNodeName);
	BuildCurrentAliases();
	if (EncounteredFunctionNames.Num() != 0)
	{
		EncounteredFunctionNames.Last().AddUnique(InNodeName);
	}
	ContextuallyVisitedNodes.Emplace();
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::ExitFunction(const FNode* Node)
{
	CallingContext.Pop();
	CallingGraphContext.Pop();
	ActivePath.PopNode();
	PinToParameterMapIndices.Pop();
	FunctionNameContextStack.Pop();
	PinToConstantIndices.Pop();
	BuildCurrentAliases();
	ContextuallyVisitedNodes.Pop();
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::EnterEmitter(const FString& InEmitterName, const FGraph* InGraph, const FNode* Node)
{
	RegisterNodeVisitation(Node);
	CallingContext.Push(Node);
	ActivePath.PushNode(Node);
	CallingGraphContext.Push(InGraph);
	EmitterNameContextStack.Emplace(*InEmitterName);
	BuildCurrentAliases();

	// Emitters must record their namespaces to their histories as well as
	// make sure to record their current usage type is so that we can filter variables
	// for relevance downstream.
	const FEmitterNode* EmitterNode = GraphBridge::GetNodeAsEmitter(Node);
	if (EmitterNode != nullptr)
	{
		RelevantScriptUsageContext.Emplace(GraphBridge::GetEmitterUsage(EmitterNode));
	}
	else
	{
		RelevantScriptUsageContext.Emplace(ENiagaraScriptUsage::EmitterSpawnScript);
	}

	for (FHistory& History : Histories)
	{
		History.EmitterNamespacesEncountered.AddUnique(InEmitterName);
	}
	EncounteredEmitterNames.AddUnique(InEmitterName);
	TArray<FString> EmptyFuncs;
	EncounteredFunctionNames.Push(EmptyFuncs);
	ContextuallyVisitedNodes.Emplace();
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::ExitEmitter(const FString& InEmitterName, const FNode* Node)
{
	CallingContext.Pop();
	CallingGraphContext.Pop();
	EmitterNameContextStack.Pop();
	ActivePath.PopNode();
	BuildCurrentAliases();
	ContextuallyVisitedNodes.Pop();
	EncounteredFunctionNames.Pop();
}


template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::IsInEncounteredFunctionNamespace(const FNiagaraVariable& InVar) const
{
	if (EncounteredFunctionNames.Num() != 0)
	{
		for (FString EncounteredNamespace : EncounteredFunctionNames.Last())
		{
			if (FNiagaraParameterUtilities::IsInNamespace(InVar, EncounteredNamespace))
			{
				return true;
			}
		}
	}
	return false;
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::IsInEncounteredEmitterNamespace(const FNiagaraVariable& InVar) const
{
	for (FString EmitterEncounteredNamespace : EncounteredEmitterNames)
	{
		if (FNiagaraParameterUtilities::IsInNamespace(InVar, EmitterEncounteredNamespace))
		{
			return true;
		}
	}
	return false;
}

/**
* Use the current alias map to resolve any aliases in this input variable name.
*/
template<typename GraphBridge>
FNiagaraVariable TNiagaraParameterMapHistoryBuilder<GraphBridge>::ResolveAliases(const FNiagaraVariable& InVar) const
{
	FNiagaraVariable Var = FNiagaraUtilities::ResolveAliases(InVar, ResolveAliasContext);
	//ensure(!Var.IsInNameSpace(FNiagaraConstants::StackContextNamespace));
	return Var;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::RegisterNodeVisitation(const FNode* Node)
{
	if (OnNodeVisitedDelegate.IsBound())
	{
		OnNodeVisitedDelegate.Broadcast(Node);
	}

	ContextuallyVisitedNodes.Last().AddUnique(Node);
}


template<typename GraphBridge>
const FString* TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetModuleAlias() const
{
	return ResolveAliasContext.GetModuleName().IsSet()
		? &ResolveAliasContext.GetModuleName().GetValue()
		: nullptr;
}

template<typename GraphBridge>
const FString* TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetEmitterAlias() const
{
	return ResolveAliasContext.GetEmitterName().IsSet()
		? &ResolveAliasContext.GetEmitterName().GetValue()
		: nullptr;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::VisitInputPin(const FInputPin* Pin, bool bFilterForCompilation)
{
	if (Pin && ensure(Pin->Direction == EEdGraphPinDirection::EGPD_Input))
	{
		if (const FOutputPin* LinkedPin = GraphBridge::GetLinkedOutputPin(Pin))
		{
			const FNode* Node = GraphBridge::GetOwningNode(LinkedPin);

			if (!GetNodePreviouslyVisited(Node))
			{
				if (UNiagaraScript::LogCompileStaticVars > 0)
				{
					UE_LOGF(LogNiagaraEditor, Log, "Build Parameter Map History: %ls %ls", *GraphBridge::GetNodeClassName(Node), *GraphBridge::GetNodeTitle(Node));
				}
				Node->BuildParameterMapHistory(*this, true, bFilterForCompilation);
				RegisterNodeVisitation(Node);
			}

			if (GraphBridge::IsParameterMapPin(Pin))
			{
				int32 ParamMapIdx = TraceParameterMapOutputPin(LinkedPin);
					
				if (UNiagaraScript::LogCompileStaticVars > 0)
				{
					if (ParamMapIdx == INDEX_NONE)
					{
						UE_LOGF(LogNiagaraEditor, Log, "Found bad pin!");
					}

					UE_LOGF(LogNiagaraEditor, Log, "Build Parameter Map History: %ls %ls PMapIdx: %d", *GraphBridge::GetNodeClassName(Node), *GraphBridge::GetNodeTitle(Node), ParamMapIdx);
				}

				RegisterParameterMapPin(ParamMapIdx, Pin);
			}
		} 
	}
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::VisitInputPins(const FNode* InNode, bool bFilterForCompilation)
{
	TArray<const FInputPin*> InputPins = GraphBridge::GetInputPins(InNode);

	for (int32 i = 0; i < InputPins.Num(); i++)
	{
		VisitInputPin(InputPins[i], bFilterForCompilation);
	}
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::IsNamespacedVariableRelevantToScriptType(const FNiagaraVariable& InVar, ENiagaraScriptUsage InFilterScriptType)
{
	return true;
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::ShouldTrackVariable(const FNiagaraVariable& InVar)
{
	if (!bFilterByScriptAllowList)
	{
		return true;
	}
	if (IsNamespacedVariableRelevantToScriptType(InVar, FilterScriptType))
	{
		return true;
	}
	return false;
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::HandleVariableWrite(int32 ParamMapIdx, const FInputPin* InPin)
{
	FNiagaraVariable Var = GraphBridge::GetPinVariable(InPin, false, ENiagaraStructConversion::UserFacing);

	if (!ShouldTrackVariable(Var))
	{
		return INDEX_NONE;
	}
	FNiagaraVariable AliasedVar = Var;
	Var = ResolveAliases(Var);

	int32 FoundIdx = AddVariableToHistory(Histories[ParamMapIdx], Var, AliasedVar, InPin);
	if (InPin && Var.GetType().IsStatic())
	{
		const FOutputPin* LinkedPin = GraphBridge::GetLinkedOutputPin(InPin);
		const int32 ConstantIdx = LinkedPin
			? GetConstantFromOutputPin(LinkedPin)
			: AddOrGetConstantFromValue(InPin->DefaultValue);

		RegisterConstantPin(ConstantIdx, InPin);
		if (FoundIdx != INDEX_NONE && ParamMapIdx != INDEX_NONE && ConstantIdx != INDEX_NONE)
		{
			RegisterConstantVariableWrite(ParamMapIdx, ConstantIdx, FoundIdx, false, false);
		}
	}

	return FoundIdx;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::HandleParameterCollectionRead(const FNiagaraVariableBase& InVar, int32 ParamMapIndex)
{
	if (bIncludeParameterCollectionInfo)
	{
		AddParameterCollection(this->AvailableCollections->FindCollection(InVar), ParamMapIndex);
	}
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::HandleVariableWrite(int32 ParameterMapIndex, const FNiagaraVariable& Var)
{
	if (!ShouldTrackVariable(Var))
	{
		return INDEX_NONE;
	}
	FNiagaraVariable ResolvedVar = ResolveAliases(Var);

	int32 FoundIdx = AddVariableToHistory(Histories[ParameterMapIndex], ResolvedVar, Var, nullptr);
	return FoundIdx;
}


template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::HandleVariableRead(int32 ParamMapIdx, const FPin* InPin, bool RegisterReadsAsVariables, const FInputPin* InDefaultPin, bool bFilterForCompilation, bool& OutUsedDefault)
{
	FString DefaultValue;
	OutUsedDefault = false;
	FNiagaraVariable Var = GraphBridge::GetPinVariable(InPin, false, ENiagaraStructConversion::UserFacing);

	FHistory& History = Histories[ParamMapIdx];

	if (!ShouldTrackVariable(Var))
	{
		return INDEX_NONE;
	}
	FNiagaraVariable AliasedVar = Var;
	Var = ResolveAliases(Var);

	//Track any parameter collections we're referencing.
	HandleParameterCollectionRead(Var, ParamMapIdx);

	const FString* ModuleAlias = GetModuleAlias();
	FName ModuleName = ModuleAlias ? FName(**ModuleAlias) : NAME_None;

	bool AddWriteHistory = true;
	bool bUseRapidIterationParam = true;

	int32 FoundIdx = History.FindVariable(Var.GetName(), Var.GetType());
	if (FoundIdx == INDEX_NONE)
	{
		if (RegisterReadsAsVariables)
		{
			OutUsedDefault = true;

			if (InDefaultPin)
			{
				VisitInputPin(InDefaultPin, bFilterForCompilation);
				DefaultValue = InDefaultPin->DefaultValue;
			}

			FoundIdx = AddVariableToHistory(History, Var, AliasedVar, InDefaultPin);
			AddWriteHistory = false;

			// Add the default binding as well to the parameter history, if used.
			if (const FGraph* Graph = GraphBridge::GetOwningGraph(GraphBridge::GetOwningNode(InPin)))
			{
				FNiagaraScriptVariableBinding VariableBinding;
				TOptional<ENiagaraDefaultMode> DefaultMode = GraphBridge::GetGraphDefaultMode(Graph, AliasedVar, VariableBinding);
				if (DefaultMode.IsSet() && *DefaultMode == ENiagaraDefaultMode::Binding && VariableBinding.IsValid())
				{
					FNiagaraVariable BoundVar = FNiagaraVariable(Var.GetType(), VariableBinding.GetName());
					if (FNiagaraConstants::GetOldPositionTypeVariables().Contains(BoundVar))
					{
						// Old assets often have vector inputs that default bind to what is now a position type. If we detect that, we change the type to prevent a compiler error.
						BoundVar.SetType(FNiagaraTypeDefinition::GetPositionDef());
					}

					FNiagaraVariable AliasedBoundVar = BoundVar;
					BoundVar = ResolveAliases(AliasedBoundVar);

					int32 FoundIdxBinding = AddVariableToHistory(History, BoundVar, AliasedBoundVar, nullptr);

					DefaultValue = History.Variables[FoundIdxBinding].GetName().ToString();
					bUseRapidIterationParam = false;

					History.PerVariableReadHistory[FoundIdxBinding].AddDefaulted_GetRef().ReadPin = typename FHistory::FModuleScopedPin(InDefaultPin, ModuleName);
				}
			}
		}
	}
	else
	{
		if (History.Variables[FoundIdx].GetType() != Var.GetType())
		{
			History.PerVariableWarnings[FoundIdx].Append(FString::Printf(TEXT("Type mismatch %s instead of %s in map!"), *Var.GetType().GetName(), *History.Variables[FoundIdx].GetType().GetName()));
		}

		History.ConditionalUpdateAliasedVariable(FoundIdx, AliasedVar);
	}

	typename FHistory::FReadHistory& ReadEntry = History.PerVariableReadHistory[FoundIdx].AddDefaulted_GetRef();
	ReadEntry.ReadPin = typename FHistory::FModuleScopedPin(InPin, ModuleName);
	if (AddWriteHistory && History.PerVariableWriteHistory[FoundIdx].Num())
	{
		ReadEntry.PreviousWritePin = History.PerVariableWriteHistory[FoundIdx].Last();
	}

	check(History.Variables.Num() == History.PerVariableWarnings.Num());
	check(History.Variables.Num() == History.PerVariableWriteHistory.Num());
	check(History.Variables.Num() == History.PerVariableReadHistory.Num());
	check(History.Variables.Num() == History.PerVariableConstantValue.Num());
	check(History.Variables.Num() == History.PerVariableConstantDefaultValue.Num());

	if (OutUsedDefault && InDefaultPin && GraphBridge::IsStaticPin(InDefaultPin))
	{
		int32 ConstantIdx = INDEX_NONE;
		ENiagaraScriptUsage Usage = GetCurrentUsageContext();

		FString EmitterNamespace;
		if (!UNiagaraScript::IsSystemScript(Usage) && Histories[ParamMapIdx].EmitterNamespacesEncountered.Num() > 0)
		{
			EmitterNamespace = Histories[ParamMapIdx].EmitterNamespacesEncountered.Last();
		}
		else if (!UNiagaraScript::IsSystemScript(Usage) )
		{
			if (EmitterNamespace.IsEmpty() && EmitterNameContextStack.Num() > 0)
				EmitterNamespace = EmitterNameContextStack.Last().ToString();
			else
				EmitterNamespace = TEXT("Emitter");
		}

		bool bWroteVarNameToDefaultValue = false;

		if (bUseRapidIterationParam && /*InTopLevelFunctionCall(Usage) && */FNiagaraParameterUtilities::IsAliasedModuleParameter(Histories[ParamMapIdx].VariablesWithOriginalAliasesIntact[FoundIdx]))
		{					
			FNiagaraVariable VarRapid = FNiagaraParameterUtilities::ConvertVariableToRapidIterationConstantName(Histories[ParamMapIdx].Variables[FoundIdx], EmitterNamespace.Len() == 0 ? nullptr : *EmitterNamespace, Usage);
			
			int32 StaticIdx = INDEX_NONE;
			
			// See the comments in FNiagaraStackGraphUtilities::GetStackFunctionInputPinsWithoutCache to see 
			// why this check is necessary.
			if (!bIgnoreStaticRapidIterationParameters)
			{
				StaticIdx = StaticVariables.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
					{
						return (InObj.GetName() == VarRapid.GetName());
					});;
			}

			if (StaticIdx == INDEX_NONE)
			{
				StaticIdx = StaticVariables.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
					{
						return (InObj.GetName() == Var.GetName());
					});;
			}

			if (ConstantResolver->ResolveConstant(VarRapid))
			{
				DefaultValue = VarRapid.GetType().ToString(VarRapid.GetData());
			}
			else if (StaticIdx != INDEX_NONE)
			{
				DefaultValue = StaticVariables[StaticIdx].GetType().ToString(StaticVariables[StaticIdx].GetData());	
			}
			else if (DefaultValue.Len() == 0)
			{
				//DefaultValue = VarRapid.GetName().ToString();
				FNiagaraVariable VarDefault = Var;
				FNiagaraEditorUtilities::ResetVariableToDefaultValue(VarDefault);// Do this to handle non-zero defaults
				DefaultValue = VarDefault.GetType().ToString(VarDefault.GetData());
			}
			
			if (UNiagaraScript::LogCompileStaticVars > 0)
			{
				FGraphTraversalHandle TestPath = ActivePath;
				TestPath.PushPin(InPin);
				UE_LOGF(LogNiagaraEditor, Log, "HandleVariableRead RapidIt Value \"%ls\" Var: \"%ls\" Path: \"%ls\"", *DefaultValue, *VarRapid.GetName().ToString(),  *TestPath.ToString());
			}
			

		}
		else if (FNiagaraParameterUtilities::IsExternalConstantNamespace(Var, Usage, 0) || Var.IsInNameSpace(EmitterNamespace) || Var.IsInNameSpace(FNiagaraConstants::SystemNamespace) || Var.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespace))
		{
			const FNiagaraVariable& VarRapid = Histories[ParamMapIdx].Variables[FoundIdx];
			int32 StaticIdx = StaticVariables.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
			{
				return (InObj.GetName() == VarRapid.GetName());
			});;

			if (ConstantResolver->ResolveConstant(Var))
			{
				DefaultValue = Var.GetType().ToString(Var.GetData());
			}
			else if (StaticIdx != INDEX_NONE)
			{
				DefaultValue = StaticVariables[StaticIdx].GetType().ToString(StaticVariables[StaticIdx].GetData());
			}
			else
			{
				DefaultValue = Var.GetName().ToString();
				bWroteVarNameToDefaultValue = true;
			}
			
			if (UNiagaraScript::LogCompileStaticVars > 0)
			{
				FGraphTraversalHandle TestPath = ActivePath;
				TestPath.PushPin(InPin);
				UE_LOGF(LogNiagaraEditor, Log, "HandleVariableRead Ext Value \"%ls\" Var: \"%ls\" Path: \"%ls\"", *DefaultValue, *VarRapid.GetName().ToString(),  *TestPath.ToString());
			}
		}

		ConstantIdx = AddOrGetConstantFromValue(DefaultValue);

		RegisterConstantPin(ConstantIdx, InPin);
		if (FoundIdx != INDEX_NONE && ParamMapIdx != INDEX_NONE && ConstantIdx != INDEX_NONE)
		{
			RegisterConstantVariableWrite(ParamMapIdx, ConstantIdx, FoundIdx, true, bWroteVarNameToDefaultValue);
		}
	}
	else if (FoundIdx != INDEX_NONE && ParamMapIdx != INDEX_NONE && GraphBridge::IsStaticPin(InPin))
	{
		int32 ConstantIdx = GetConstantFromVariableRead(Histories[ParamMapIdx].Variables[FoundIdx]);
		RegisterConstantPin(ConstantIdx, InPin);
	}

	return FoundIdx;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::RegisterEncounterableVariables(TConstArrayView<FNiagaraVariable> Variables)
{
	EncounterableExternalVariables.Append(Variables.GetData(), Variables.Num());
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::RegisterExternalStaticVariables(TConstArrayView<FNiagaraVariable> Variables)
{
	int32 LastIdx = StaticVariables.Num();
	StaticVariables.Append(Variables.GetData(), Variables.Num());

	// Register all new entries as exportable true
	for (int32 i = LastIdx; i < StaticVariables.Num(); i++)
	{
		StaticVariableExportable.Emplace(true);
	}

	if (UNiagaraScript::LogCompileStaticVars > 0)
	{
		UE_LOGF(LogNiagaraEditor, Log, "FNiagaraParameterMapHistoryBuilder::RegisterExternalStaticVariables ===================================");
		for (int32 i = 0; i < StaticVariables.Num(); i++)
		{
			UE_LOGF(LogNiagaraEditor, Log, "[%d] %ls", i, *StaticVariables[i].ToString());
		}
		UE_LOGF(LogNiagaraEditor, Log, "  ===================================");
	}
}

void UpdateAliasedVariable(FNiagaraVariable& AliasedVar, const FNiagaraVariable& OriginalUnaliasedVar, const FNiagaraVariable& UpdatedUnaliasedVar)
{
	AliasedVar.SetType(UpdatedUnaliasedVar.GetType());

	TArray<FString> AliasedSplitName;
	AliasedVar.GetName().ToString().ParseIntoArray(AliasedSplitName, TEXT("."));

	TArray<FString> OriginalUnaliasedSplitName;
	OriginalUnaliasedVar.GetName().ToString().ParseIntoArray(OriginalUnaliasedSplitName, TEXT("."));

	TArray<FString> UpdatedUnaliasedSplitName;
	UpdatedUnaliasedVar.GetName().ToString().ParseIntoArray(UpdatedUnaliasedSplitName, TEXT("."));

	TArray<FString> JoinName;
	for (int32 i = 0; i < AliasedSplitName.Num(); i++)
	{
		if (i >= OriginalUnaliasedSplitName.Num() || i >= UpdatedUnaliasedSplitName.Num())
		{
			continue;
		}

		//if (UpdatedUnaliasedSplitName[i] == OriginalUnaliasedSplitName[i])
		{
			JoinName.Add(AliasedSplitName[i]);
		}
		//else
		//{
		//	JoinName.Add(AliasedSplitName[i]);
		//}
	}

	FString OutVarStrName = FString::Join(JoinName, TEXT("."));
	AliasedVar.SetName(*OutVarStrName);
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::HandleExternalVariableRead(int32 ParamMapIdx, const FName& Name)
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	FHistory& History = Histories[ParamMapIdx];

	FNiagaraVariable Var(FNiagaraTypeDefinition(), Name);

	if (!ShouldTrackVariable(Var))
	{
		return INDEX_NONE;
	}
	FNiagaraVariable AliasedVar = Var;
	Var = ResolveAliases(Var);
	FNiagaraVariable OriginalUnaliasedVar = Var;

	//Track any parameter collections we're referencing.
	if (bIncludeParameterCollectionInfo)
	{
		FNiagaraVariable FoundTempVar;
		FParameterCollection Collection = this->AvailableCollections->FindMatchingCollection(Name, true, FoundTempVar);

		if (AddParameterCollection(Collection, ParamMapIdx))
		{
			Var = FoundTempVar;
			UpdateAliasedVariable(AliasedVar, OriginalUnaliasedVar, Var);
		}
	}

	int32 FoundIdx = Histories[ParamMapIdx].FindVariableByName(Name, true);
	
	if (FoundIdx == -1)
	{
		const FNiagaraVariable* TempKnownConstant = FNiagaraConstants::GetKnownConstant(Name, true);

		if (!Var.IsValid() && TempKnownConstant != nullptr)
		{
			Var = *TempKnownConstant;
			UpdateAliasedVariable(AliasedVar, OriginalUnaliasedVar, Var);
		}

		if (!Var.IsValid())
		{
			int32 EncounterableFoundIdx = FNiagaraVariable::SearchArrayForPartialNameMatch(EncounterableExternalVariables, Name);

			if (EncounterableFoundIdx != INDEX_NONE)
			{
				Var = EncounterableExternalVariables[EncounterableFoundIdx];
				UpdateAliasedVariable(AliasedVar, OriginalUnaliasedVar, Var);
			}
		}

		if (Var.IsValid())
		{
			FoundIdx = AddVariableToHistory(Histories[ParamMapIdx], Var, AliasedVar, nullptr);
		}
		else
		{
			// This is overly spammy and doesn't provide useful info. Disabling for now.
			//UE_LOGF(LogNiagaraEditor, Log, "Could not resolve variable: %ls", *Name.ToString());
		}
		
	}
	else
	{
		// Do nothing here
	}

	return FoundIdx;
}

template<>
bool TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationGraphBridge>::AddParameterCollection(FParameterCollection Collection, int32 ParamMapIndex)
{
	if (!Collection)
	{
		return false;
	}

	FNiagaraCompilationGraphBridge::FParameterCollectionStore& CollectionStore = Histories[ParamMapIndex].EncounteredParameterCollections;

	int32 Index = CollectionStore.Collections.AddUnique(Collection);
	CollectionStore.CollectionNamespaces.SetNum(CollectionStore.Collections.Num());
	CollectionStore.CollectionVariables.SetNum(CollectionStore.Collections.Num());
	CollectionStore.CollectionNamespaces[Index] = Collection->GetFullNamespaceName();
	CollectionStore.CollectionVariables[Index] = Collection->GetParameters();

	return true;
}


template<>
bool TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationDigestBridge>::AddParameterCollection(FParameterCollection Collection, int32 ParamMapIndex)
{
	if (!Collection.IsValid())
	{
		return false;
	}

	Histories[ParamMapIndex].EncounteredParameterCollections.Handles.AddUnique(Collection);
	return true;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::SetConstantByStaticVariable(int32& OutValue, const FInputPin* InDefaultPin)
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	OutValue = 0;
	FNiagaraVariable Var = GraphBridge::GetPinVariable(InDefaultPin, true, ENiagaraStructConversion::UserFacing);
	FNiagaraVariable VarDefault = Var;
	FNiagaraEditorUtilities::ResetVariableToDefaultValue(VarDefault);// Do this to handle non-zero defaults
	if (VarDefault.GetType().IsStatic())
	{
		FNiagaraVariable VarWithValue = FNiagaraVariable(Var.GetType(), Var.GetName());
		FString Value;
		if (const FOutputPin* ConnectedOutputPin = GraphBridge::GetLinkedOutputPin(InDefaultPin))
		{
			int32 InputConstant = GetConstantFromOutputPin(ConnectedOutputPin);
			if (InputConstant != INDEX_NONE)
			{
				Value = Constants[InputConstant];
			}
		}
		else if (ensure(InDefaultPin->Direction == EEdGraphPinDirection::EGPD_Input))
		{
			Value = InDefaultPin->DefaultValue;
		}

		// If we found a string, we should try and map to the actual value of that variable..
		if (Value.Len() != 0)
		{
			TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(Var.GetType());
			if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanHandlePinDefaults() )
			{
				TypeEditorUtilities->SetValueFromPinDefaultString(Value, VarWithValue);
			}
		}

		if (VarWithValue.IsDataAllocated())
		{
			if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
			{
				OutValue = VarWithValue.GetValue<bool>();
			}
			else if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || VarWithValue.GetType().IsEnum())
			{
				OutValue = VarWithValue.GetValue<int32>();
			}
		}
	}
}

template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::FindStaticVariable(const FNiagaraVariable& Var) const
{
	FString Value = Var.GetName().ToString();
	int32 FoundOverrideIdx = StaticVariables.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
		{
			return (InObj.GetName() == *Value);
		});;

	return FoundOverrideIdx;
}

template<typename GraphBridge>
bool TNiagaraParameterMapHistoryBuilder<GraphBridge>::ShouldProcessDepthTraversal(const FGraph* Graph)
{
	// first check if there are limits setup for how deep we should look
	if (MaxGraphDepthTraversal == INDEX_NONE || CurrentGraphDepth < MaxGraphDepthTraversal)
	{
		// next, if we only care about static variables, check if the graph we're going to go into actually includes any
		if (StaticVariableSearchContext.bCollectingStaticVariablesOnly)
		{
			return GraphBridge::GetGraphReferencesStaticVariables(Graph, StaticVariableSearchContext);
		}

		return true;
	}

	return false;
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::SetConstantByStaticVariable(int32& OutValue, const FNiagaraVariable& Var)
{
	
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

	OutValue = 0;
	FNiagaraVariable VarDefault = Var;
	FNiagaraEditorUtilities::ResetVariableToDefaultValue(VarDefault);// Do this to handle non-zero defaults
	if (VarDefault.GetType().IsStatic())
	{
		FNiagaraVariable VarWithValue = FNiagaraVariable(Var.GetType(), Var.GetName());
		FString Value = Var.GetName().ToString();
		int32 Input = INDEX_NONE;

		// If we found a string, we should try and map to the actual value of that variable..
		if (Value.Len() != 0)
		{
			if (!VarWithValue.IsDataAllocated())
			{
				int32 FoundOverrideIdx = StaticVariables.IndexOfByPredicate([&](const FNiagaraVariable& InObj) -> bool
					{
						return (InObj.GetName() == *Value);
					});;

				if (FoundOverrideIdx != INDEX_NONE)
				{
					VarWithValue.SetData(StaticVariables[FoundOverrideIdx].GetData());
				}
			}
		}

		if (VarWithValue.IsDataAllocated())
		{
			if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
			{
				OutValue = VarWithValue.GetValue<bool>();
			}
			else if (VarWithValue.GetType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetIntDef()) || VarWithValue.GetType().IsEnum())
			{
				OutValue = VarWithValue.GetValue<int32>();
			}
		}
	}
	
}


template<typename GraphBridge>
int32 TNiagaraParameterMapHistoryBuilder<GraphBridge>::AddVariableToHistory(FHistory& History, const FNiagaraVariable& InVar, const FNiagaraVariable& InAliasedVar, const FPin* InPin)
{
	const FString* ModuleAlias = GetModuleAlias();
	const FName ModuleName = ModuleAlias ? FName(**ModuleAlias) : NAME_None;
	
	return History.AddVariable(InVar, InAliasedVar, ModuleName, InPin);
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryBuilder<GraphBridge>::BuildCurrentAliases()
{
	if(RelevantScriptUsageContext.Num() > 0)
	{
		ResolveAliasContext = FNiagaraAliasContext(RelevantScriptUsageContext.Top());
	}
	else
	{
		ResolveAliasContext = FNiagaraAliasContext();
	}

	{
		TStringBuilder<1024> Callstack;
		for (int32 i = 0; i < FunctionNameContextStack.Num(); i++)
		{
			if (i != 0)
			{
				Callstack << TEXT(".");
			}
			FunctionNameContextStack[i].AppendString(Callstack);
		}

		if (Callstack.Len() > 0)
		{
			ResolveAliasContext.ChangeModuleToModuleName(Callstack.ToString());
		}

		Callstack.Reset();
		for (int32 i = 0; i < EmitterNameContextStack.Num(); i++)
		{
			if (i != 0)
			{
				Callstack << TEXT(".");
			}
			EmitterNameContextStack[i].AppendString(Callstack);
		}

		if (Callstack.Len() > 0)
		{
			ResolveAliasContext.ChangeEmitterToEmitterName(Callstack.ToString());
		}
	}

	{
		FString Callstack;
		for (int32 i = 0; i < RelevantScriptUsageContext.Num(); i++)
		{
			switch (RelevantScriptUsageContext[i])
			{
				/** The script defines a function for use in modules. */
			case ENiagaraScriptUsage::Function:
			case ENiagaraScriptUsage::Module:
			case ENiagaraScriptUsage::DynamicInput:
				break;
			case ENiagaraScriptUsage::ParticleSpawnScript:
			case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
			case ENiagaraScriptUsage::ParticleUpdateScript:
			case ENiagaraScriptUsage::ParticleEventScript:
				Callstack = TEXT("Particles");
				break;
			case ENiagaraScriptUsage::ParticleSimulationStageScript:
			{
				if (ScriptUsageContextNameStack.Num() == 0 || ScriptUsageContextNameStack.Last() == NAME_None)
					Callstack = TEXT("Particles");
				else
					Callstack = ScriptUsageContextNameStack.Last().ToString();
			}
			break;
			case ENiagaraScriptUsage::ParticleGPUComputeScript:
				Callstack = TEXT("Particles");
				break;
			case ENiagaraScriptUsage::EmitterSpawnScript:
			case ENiagaraScriptUsage::EmitterUpdateScript:
				Callstack = TEXT("Emitter");
				{
					if (ResolveAliasContext.GetEmitterName().IsSet())
					{
						Callstack = *ResolveAliasContext.GetEmitterName().GetValue();
					}
				}
				break;
			case ENiagaraScriptUsage::SystemSpawnScript:
			case ENiagaraScriptUsage::SystemUpdateScript:
				Callstack = TEXT("System");
				break;
			}
		}

		if (!Callstack.IsEmpty())
		{
			ResolveAliasContext.ChangeStackContext(Callstack);
		}
	}
}

bool FCompileConstantResolver::ResolveConstant(FNiagaraVariable& OutConstant) const
{
	if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetFunctionDebugStateEnum(), TEXT("Function.DebugState")))
	{
		FNiagaraInt32 EnumValue;
		EnumValue.Value = (uint8)CalculateDebugState();
		OutConstant.SetValue(EnumValue);
		return true;
	}

	// handle translator case
	if (Translator)
	{
		return Translator->GetLiteralConstantVariable(OutConstant);
	}

	// handle emitter case
	if (FVersionedNiagaraEmitterData* EmitterData = Emitter.GetEmitterData())
	{
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.Localspace")))
		{
			OutConstant.SetValue(EmitterData->bLocalSpace ? FNiagaraBool(true) : FNiagaraBool(false));
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.Determinism")))
		{
			OutConstant.SetValue(EmitterData->bDeterminism ? FNiagaraBool(true) : FNiagaraBool(false));
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.InterpolatedSpawn")))
		{
			OutConstant.SetValue(EmitterData->UsesInterpolatedSpawning() ? FNiagaraBool(true) : FNiagaraBool(false));
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetSimulationTargetEnum(), TEXT("Emitter.SimulationTarget")))
		{
			FNiagaraInt32 EnumValue;
			EnumValue.Value = (uint8)EmitterData->SimTarget;
			OutConstant.SetValue(EnumValue);
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetScriptUsageEnum(), TEXT("Script.Usage")))
		{
			FNiagaraInt32 EnumValue;
			EnumValue.Value = (uint8)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchUsage(Usage);
			OutConstant.SetValue(EnumValue);
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetScriptContextEnum(), TEXT("Script.Context")))
		{
			FNiagaraInt32 EnumValue;
			EnumValue.Value = (uint8)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchContext(Usage);
			OutConstant.SetValue(EnumValue);
			return true;
		}
	}

	// handle system case
	if (System)
	{
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.Localspace")))
		{
			OutConstant.SetValue(FNiagaraBool(false));
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.Determinism")))
		{
			OutConstant.SetValue(FNiagaraBool(true));
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("Emitter.InterpolatedSpawn")))
		{
			OutConstant.SetValue(FNiagaraBool(false));
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetSimulationTargetEnum(), TEXT("Emitter.SimulationTarget")))
		{
			FNiagaraInt32 EnumValue;
			EnumValue.Value = (uint8)ENiagaraSimTarget::CPUSim;
			OutConstant.SetValue(EnumValue);
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetScriptUsageEnum(), TEXT("Script.Usage")))
		{
			FNiagaraInt32 EnumValue;
			EnumValue.Value = (uint8)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchUsage(Usage);
			OutConstant.SetValue(EnumValue);
			return true;
		}
		if (OutConstant == FNiagaraVariable(FNiagaraTypeDefinition::GetScriptContextEnum(), TEXT("Script.Context")))
		{
			FNiagaraInt32 EnumValue;
			EnumValue.Value = (uint8)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchContext(Usage);
			OutConstant.SetValue(EnumValue);
			return true;
		}
	}

	return false;
}

ENiagaraFunctionDebugState FCompileConstantResolver::CalculateDebugState() const
{
	// System or Emitter
	const UNiagaraSystem* OwnerSystem = System ? System : (Emitter.Emitter ? Cast<const UNiagaraSystem>(Emitter.Emitter->GetOuter()) : nullptr);
	if ( OwnerSystem != nullptr )
	{
		return OwnerSystem->ShouldDisableDebugSwitches() ? ENiagaraFunctionDebugState::NoDebug : DebugState;
	}

	// Translator
	if ( Translator != nullptr )
	{
		return Translator->DisableDebugSwitches() ? ENiagaraFunctionDebugState::NoDebug : DebugState;
	}

	return DebugState;
}


FCompileConstantResolver FCompileConstantResolver::WithDebugState(ENiagaraFunctionDebugState InDebugState) const
{
	FCompileConstantResolver Copy = *this;
	Copy.DebugState = InDebugState;
	return Copy;
}

FCompileConstantResolver FCompileConstantResolver::WithUsage(ENiagaraScriptUsage ScriptUsage) const
{
	FCompileConstantResolver Copy = *this;
	Copy.Usage = ScriptUsage;
	return Copy;
}

FCompileConstantResolver FCompileConstantResolver::AsEmitter(const FVersionedNiagaraEmitter& InEmitter) const
{
	FCompileConstantResolver Copy = *this;
	Copy.Emitter = InEmitter;
	Copy.System = nullptr;
	return Copy;
}


uint32 FCompileConstantResolver::BuildTypeHash() const
{
	union
	{
		struct  
		{
			uint8 Usage;
			uint8 DebugState;
			uint8 EmitterSimTarget;
			uint8 bHasSystem : 1;
			uint8 bHasEmitter : 1;
			uint8 bEmitterLocalSpace : 1;
			uint8 bEmitterDeterminism : 1;
			uint8 bEmitterInterpolatedSpawn : 1;
		};
		uint32 Hash;
	} HashUnion;

	HashUnion.Hash = 0;

	if (Emitter.Emitter)
	{
		HashUnion.bHasEmitter = true;
		if (FVersionedNiagaraEmitterData* EmitterData = Emitter.GetEmitterData())
		{
			HashUnion.EmitterSimTarget = static_cast<uint8>(EmitterData->SimTarget);
			HashUnion.bEmitterLocalSpace = EmitterData->bLocalSpace;
			HashUnion.bEmitterDeterminism = EmitterData->bDeterminism;
			HashUnion.bEmitterInterpolatedSpawn = EmitterData->UsesInterpolatedSpawning();
		}
	}

	if (System)
	{
		HashUnion.bHasSystem = true;
	}

	HashUnion.Usage = static_cast<uint8>(Usage);
	HashUnion.DebugState = static_cast<uint8>(DebugState);

	return HashUnion.Hash;
}

template<typename GraphBuilder>
int32 TNiagaraParameterMapHistoryWithMetaDataBuilder<GraphBuilder>::AddVariableToHistory(FHistory& History, const FNiagaraVariable& InVar, const FNiagaraVariable& InAliasedVar, const FPin* InPin)
{
	const FString* ModuleAlias = GetModuleAlias();
	const FName ModuleName = ModuleAlias ? FName(**ModuleAlias) : NAME_None;

	TOptional<FNiagaraVariableMetaData> MetaData = CallingGraphContext.Last()->GetMetaData(InAliasedVar);
	if (MetaData.IsSet())
	{
		return History.AddVariable(InVar, InAliasedVar, ModuleName, InPin, MetaData);
	}
	TOptional<FNiagaraVariableMetaData> BlankMetaData = FNiagaraVariableMetaData();
	return History.AddVariable(InVar, InAliasedVar, ModuleName, InPin, BlankMetaData);
}

namespace NiagaraGraphCachedBuiltHistoryImpl
{

template<typename T>
static void CollectScriptChangeIds(const T* InObject, TArray<FGuid>& ChangeIds)
{
	if (InObject)
	{
		InObject->ForEachScript([&ChangeIds](const UNiagaraScript* Script)
		{
			ChangeIds.Add(Script->GetBaseChangeID());
		});
	}
}

};

bool FNiagaraGraphCachedBuiltHistory::IsValidForSystem(const UNiagaraSystem* InSystem) const
{
	if (!InSystem)
	{
		return false;
	}

	TArray<FGuid> TestChangeIds;
	NiagaraGraphCachedBuiltHistoryImpl::CollectScriptChangeIds(InSystem, TestChangeIds);
	return TestChangeIds == SourceAssetChangeIds;
}

void FNiagaraGraphCachedBuiltHistory::SetSourceSystem(const UNiagaraSystem* InSystem)
{
	SourceAssetChangeIds.Reset();

	if (InSystem)
	{
		NiagaraGraphCachedBuiltHistoryImpl::CollectScriptChangeIds(InSystem, SourceAssetChangeIds);
	}
}

bool FNiagaraGraphCachedBuiltHistory::IsValidForEmitter(const FVersionedNiagaraEmitterData* InEmitterData) const
{
	if (!InEmitterData)
	{
		return false;
	}

	TArray<FGuid> TestChangeIds;
	NiagaraGraphCachedBuiltHistoryImpl::CollectScriptChangeIds(InEmitterData, TestChangeIds);
	return TestChangeIds == SourceAssetChangeIds;
}

void FNiagaraGraphCachedBuiltHistory::SetSourceEmitter(const FVersionedNiagaraEmitterData* InEmitterData)
{
	SourceAssetChangeIds.Reset();

	if (InEmitterData)
	{
		NiagaraGraphCachedBuiltHistoryImpl::CollectScriptChangeIds(InEmitterData, SourceAssetChangeIds);
	}
}

template<typename GraphBridge>
void TNiagaraParameterMapHistoryWithMetaDataBuilder<GraphBridge>::AddGraphToCallingGraphContextStack(const FGraph* InGraph)
{
	CallingGraphContext.Add(InGraph);
};

//////////////////////////////////////////////////////////////////////////

template struct TNiagaraParameterMapHistory<FNiagaraCompilationGraphBridge>;
template struct TNiagaraParameterMapHistory<FNiagaraCompilationDigestBridge>;

template class TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationGraphBridge>;
template class TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationDigestBridge>;

template class TNiagaraParameterMapHistoryWithMetaDataBuilder<FNiagaraCompilationGraphBridge>;
template class TNiagaraParameterMapHistoryWithMetaDataBuilder<FNiagaraCompilationDigestBridge>;

#undef LOCTEXT_NAMESPACE
