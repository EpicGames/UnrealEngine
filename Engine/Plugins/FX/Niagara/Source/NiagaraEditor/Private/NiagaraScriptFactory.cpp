// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "AssetToolsModule.h"
#include "EdGraphSchema_Niagara.h"
#include "IAssetTypeActions.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraScript.h"
#include "NiagaraScriptFactoryNew.h"
#include "NiagaraScriptSource.h"
#include "AssetDefinitions/AssetDefinition_NiagaraScript.h"
#include "EdGraph/EdGraph.h"
#include "Modules/ModuleManager.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptFactory"

UNiagaraScriptFactoryNew::UNiagaraScriptFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UNiagaraScript::StaticClass();
	bCreateNew = false;
	bEditAfterNew = true;
	bCreateNew = true;
}

UObject* UNiagaraScriptFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UNiagaraScript::StaticClass()));
	
	const UNiagaraEditorSettings* Settings = GetDefault<UNiagaraEditorSettings>();
	check(Settings);

	UNiagaraScript* NewScript;
	const FSoftObjectPath& SettingDefaultScript = GetDefaultScriptFromSettings(Settings);
	UNiagaraScript* Default = Cast<UNiagaraScript>(SettingDefaultScript.TryLoad());
	if (Default != nullptr &&
		Cast<UNiagaraScriptSource>(Default->GetLatestSource()) != nullptr &&
		Cast<UNiagaraScriptSource>(Default->GetLatestSource())->NodeGraph != nullptr)
	{
		NewScript = Cast<UNiagaraScript>(StaticDuplicateObject(Default, InParent, Name, Flags, Class));
	}
	else
	{
		UE_LOGF(LogNiagaraEditor, Log, "Default Script \"%ls\" could not be loaded. Creating graph procedurally.", *SettingDefaultScript.ToString());

		NewScript = NewObject<UNiagaraScript>(InParent, Class, Name, Flags | RF_Transactional);
		NewScript->Usage = GetScriptUsage();
		InitializeScript(NewScript);
	}

	return NewScript;
}

const FSoftObjectPath& UNiagaraScriptFactoryNew::GetDefaultScriptFromSettings(const UNiagaraEditorSettings* Settings)
{
	switch (GetScriptUsage())
	{
	case ENiagaraScriptUsage::Module:
		if (Settings->DefaultModuleScript.IsValid())
		{
			return Settings->DefaultModuleScript;
		}
		break;
	case ENiagaraScriptUsage::DynamicInput:
		if (Settings->DefaultDynamicInputScript.IsValid())
		{
			return Settings->DefaultDynamicInputScript;
		}
		break;
	case ENiagaraScriptUsage::Function:
		if (Settings->DefaultFunctionScript.IsValid())
		{
			return Settings->DefaultFunctionScript;
		}
		break;
	}
	return Settings->DefaultScript;
}

FText UNiagaraScriptFactoryNew::GetDisplayName() const
{
	return FText::FromName(GetAssetTypeActionName());
}

void UNiagaraScriptFactoryNew::InitializeScript(UNiagaraScript* NewScript)
{
	if(NewScript != nullptr)
	{
		UNiagaraScriptSource* Source = NewObject<UNiagaraScriptSource>(NewScript, NAME_None, RF_Transactional);
		if(Source)
		{
			UNiagaraGraph* CreatedGraph = NewObject<UNiagaraGraph>(Source, NAME_None, RF_Transactional);
			Source->NodeGraph = CreatedGraph;				
			
			const UEdGraphSchema_Niagara* NiagaraSchema = Cast<UEdGraphSchema_Niagara>(CreatedGraph->GetSchema());
			
			FGraphNodeCreator<UNiagaraNodeOutput> OutputNodeCreator(*CreatedGraph);
			UNiagaraNodeOutput* OutputNode = OutputNodeCreator.CreateNode();
			OutputNode->SetUsage(NewScript->Usage);

			FGraphNodeCreator<UNiagaraNodeInput> InputNodeCreator(*CreatedGraph);
			UNiagaraNodeInput* InputNode = InputNodeCreator.CreateNode();
			InputNode->Usage = ENiagaraInputNodeUsage::Parameter;

			UNiagaraNodeParameterMapGet* GetNode = nullptr;
			UNiagaraNodeParameterMapSet* SetNode = nullptr;

			switch (NewScript->Usage)
			{
			case ENiagaraScriptUsage::DynamicInput:
				{
					FNiagaraVariable InVar(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("MapIn"));
					InputNode->Input = InVar;

					FNiagaraVariable OutVar(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Output"));
					OutputNode->Outputs.Add(OutVar);
					
					OutputNodeCreator.Finalize();
					InputNodeCreator.Finalize();

					FGraphNodeCreator<UNiagaraNodeParameterMapGet> GetNodeCreator(*CreatedGraph);
					GetNode = GetNodeCreator.CreateNode();
					GetNodeCreator.Finalize();
					UEdGraphPin* FloatOutPin = GetNode->RequestNewTypedPin(EGPD_Output, FNiagaraTypeDefinition::GetFloatDef(), TEXT("Module.InputArg"));

					if (FloatOutPin)
					{
						NiagaraSchema->TryCreateConnection(InputNode->GetOutputPin(0), GetNode->GetInputPin(0));
						NiagaraSchema->TryCreateConnection(FloatOutPin, OutputNode->GetInputPin(0));
					}

				}
				break;
			case ENiagaraScriptUsage::Module:
				{
					FNiagaraVariable InVar(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("MapIn"));
					InputNode->Input = InVar;

					FNiagaraVariable OutVar(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("Output"));
					OutputNode->Outputs.Add(OutVar);

					OutputNodeCreator.Finalize();
					InputNodeCreator.Finalize();

					FGraphNodeCreator<UNiagaraNodeParameterMapGet> GetNodeCreator(*CreatedGraph);
					GetNode = GetNodeCreator.CreateNode();
					GetNodeCreator.Finalize();
					UEdGraphPin* FloatOutPin = GetNode->RequestNewTypedPin(EGPD_Output, FNiagaraTypeDefinition::GetFloatDef(), TEXT("Module.InputArg"));

					FGraphNodeCreator<UNiagaraNodeParameterMapSet> SetNodeCreator(*CreatedGraph);
					SetNode = SetNodeCreator.CreateNode();
					SetNodeCreator.Finalize();
					UEdGraphPin* FloatInPin = SetNode->RequestNewTypedPin(EGPD_Input, FNiagaraTypeDefinition::GetFloatDef(), TEXT("Particles.DummyFloat"));

					if (FloatOutPin)
					{
						NiagaraSchema->TryCreateConnection(InputNode->GetOutputPin(0), SetNode->GetInputPin(0));
						NiagaraSchema->TryCreateConnection(InputNode->GetOutputPin(0), GetNode->GetInputPin(0));
						NiagaraSchema->TryCreateConnection(SetNode->GetOutputPin(0), OutputNode->GetInputPin(0));
						NiagaraSchema->TryCreateConnection(FloatOutPin, FloatInPin);
					}
				}
				break;
			default:
			case ENiagaraScriptUsage::Function:
				{
					FNiagaraVariable InVar(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Input"));
					InputNode->Input = InVar;

					FNiagaraVariable OutVar(FNiagaraTypeDefinition::GetFloatDef(), TEXT("Output"));
					OutputNode->Outputs.Add(OutVar);

					OutputNodeCreator.Finalize();
					InputNodeCreator.Finalize();

					NiagaraSchema->TryCreateConnection(InputNode->GetOutputPin(0), OutputNode->GetInputPin(0));
				}
				break;
			}

			FNiagaraStackGraphUtilities::RelayoutGraph(*CreatedGraph);
			// Set pointer in script to source
			NewScript->SetLatestSource(Source);


			NewScript->RequestCompile(FGuid());
		}
	}
}

ENiagaraScriptUsage UNiagaraScriptFactoryNew::GetScriptUsage() const
{
	return ENiagaraScriptUsage::Module;
}

/************************************************************************/
/* UNiagaraScriptModulesFactory											*/
/************************************************************************/
UNiagaraModuleScriptFactory::UNiagaraModuleScriptFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FName UNiagaraModuleScriptFactory::GetNewAssetThumbnailOverride() const
{
	return TEXT("NiagaraEditor.Thumbnails.Modules");
}

FName UNiagaraModuleScriptFactory::GetAssetTypeActionName() const
{
	return UAssetDefinition_NiagaraScript::ModuleScriptName;
}

ENiagaraScriptUsage UNiagaraModuleScriptFactory::GetScriptUsage() const
{
	return ENiagaraScriptUsage::Module;
}

/************************************************************************/
/* UNiagaraScriptFunctionsFactory										*/
/************************************************************************/
UNiagaraFunctionScriptFactory::UNiagaraFunctionScriptFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FName UNiagaraFunctionScriptFactory::GetNewAssetThumbnailOverride() const
{
	return TEXT("NiagaraEditor.Thumbnails.Functions");
}

FName UNiagaraFunctionScriptFactory::GetAssetTypeActionName() const
{
	return UAssetDefinition_NiagaraScript::FunctionScriptName;
}

ENiagaraScriptUsage UNiagaraFunctionScriptFactory::GetScriptUsage() const
{
	return ENiagaraScriptUsage::Function;
}

/************************************************************************/
/* UNiagaraScriptDynamicInputsFactory									*/
/************************************************************************/
UNiagaraDynamicInputScriptFactory::UNiagaraDynamicInputScriptFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FName UNiagaraDynamicInputScriptFactory::GetNewAssetThumbnailOverride() const
{
	return TEXT("NiagaraEditor.Thumbnails.DynamicInputs");
}

FName UNiagaraDynamicInputScriptFactory::GetAssetTypeActionName() const
{
	return UAssetDefinition_NiagaraScript::DynamicInputScriptName;
}

ENiagaraScriptUsage UNiagaraDynamicInputScriptFactory::GetScriptUsage() const
{
	return ENiagaraScriptUsage::DynamicInput;
}

#undef LOCTEXT_NAMESPACE
