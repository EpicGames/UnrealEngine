// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptMergeManager.h"
#include "NiagaraEmitter.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraRendererProperties.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraGraph.h"
#include "NiagaraDataInterface.h"
#include "NiagaraMessages.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraEditorUtilities.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "NiagaraClipboard.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraStackEditorData.h"

#include "UObject/PropertyPortFlags.h"

#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Modules/ModuleManager.h"
#include "NiagaraConstants.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraSystemFactoryNew.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptMergeManager"

DECLARE_CYCLE_STAT(TEXT("Niagara - ScriptMergeManager - DiffEmitters"), STAT_NiagaraEditor_ScriptMergeManager_DiffEmitters, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - ScriptMergeManager - MergeEmitter"), STAT_NiagaraEditor_ScriptMergeManager_MergeEmitter, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - ScriptMergeManager - IsModuleInputDifferentFromBase"), STAT_NiagaraEditor_ScriptMergeManager_IsModuleInputDifferentFromBase, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - ScriptMergeManager - DoesSummaryItemExistInBase"), STAT_NiagaraEditor_ScriptMergeManager_DoesSummaryItemExistInBase, STATGROUP_NiagaraEditor)

int32 GNiagaraForceFailIfPreviouslyNotSetOnMerge = 0;
static FAutoConsoleVariableRef CVarForceErrorIfMissingDefaultOnMergeh(
	TEXT("fx.ForceFailIfPreviouslyNotSetOnMerge"),
	GNiagaraForceFailIfPreviouslyNotSetOnMerge,
	TEXT("If > 0, when merging in from parent emitters swap linked variables in the stack to be \"Fail If Previously Not Set\" for their default type. \n"),
	ECVF_Default
);

FNiagaraStackFunctionInputOverrideMergeAdapter::FNiagaraStackFunctionInputOverrideMergeAdapter(
	const FVersionedNiagaraEmitter& InOwningEmitter,
	UNiagaraScript& InOwningScript,
	UNiagaraNodeFunctionCall& InOwningFunctionCallNode,
	UEdGraphPin& InOverridePin
)
	: OwningScript(&InOwningScript)
	, OwningFunctionCallNode(&InOwningFunctionCallNode)
	, OverridePin(&InOverridePin)
{
	
	InputName = FNiagaraParameterHandle(OverridePin->PinName).GetName().ToString();
	OverrideNode = CastChecked<UNiagaraNodeParameterMapSet>(OverridePin->GetOwningNode());
	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	Type = NiagaraSchema->PinToTypeDefinition(OverridePin);

	if (OverridePin->LinkedTo.Num() == 0)
	{
		LocalValueString = OverridePin->DefaultValue;
	}
	else if (OverridePin->LinkedTo.Num() == 1)
	{
		OverrideValueNodePersistentId = OverridePin->LinkedTo[0]->GetOwningNode()->NodeGuid;

		if (OverridePin->LinkedTo[0]->GetOwningNode()->IsA<UNiagaraNodeParameterMapGet>())
		{
			LinkedValueData = FNiagaraStackLinkedValueData();
			LinkedValueData->LinkedParameterValue = NiagaraSchema->PinToNiagaraVariable(OverridePin->LinkedTo[0]);
			FNiagaraParameterHandle LinkedValueHandle = FNiagaraParameterHandle(LinkedValueData->LinkedParameterValue.GetName());
			if (LinkedValueHandle.IsOutputHandle() ||
				LinkedValueHandle.IsStackContextHandle() ||
				LinkedValueHandle.IsEmitterHandle() ||
				LinkedValueHandle.IsParticleAttributeHandle())
			{
				// If the linked handle is a module output or module data set attribute, record the node id so that we can check for renamed nodes
				// when applying the diff.
				TArray<FName> HandleParts = LinkedValueHandle.GetHandleParts();
				if (HandleParts.Num() > 2)
				{
					FString FunctionName = HandleParts[1].ToString();
					TObjectPtr<UEdGraphNode>* ReferencedFunctionCallNodePtr = OwningFunctionCallNode->GetGraph()->Nodes.FindByPredicate(
						[FunctionName](UEdGraphNode* Node) { return Node->IsA<UNiagaraNodeFunctionCall>() && CastChecked<UNiagaraNodeFunctionCall>(Node)->GetFunctionName() == FunctionName; });
					if (ReferencedFunctionCallNodePtr != nullptr)
					{
						LinkedValueData->LinkedFunctionNodeId = (*ReferencedFunctionCallNodePtr)->NodeGuid;
					}
				}
			}
		}
		else if (OverridePin->LinkedTo[0]->GetOwningNode()->IsA<UNiagaraNodeInput>())
		{
			UNiagaraNodeInput* DataInputNode = CastChecked<UNiagaraNodeInput>(OverridePin->LinkedTo[0]->GetOwningNode());
			if ( DataInputNode->IsDataInterface() )
			{
				DataInterfaceValueInputName = DataInputNode->Input.GetName();
				DataInterfaceValue = DataInputNode->GetDataInterface();
			}
			else
			{
				ObjectAssetInputVariable = DataInputNode->Input;
				ObjectAssetValue = DataInputNode->GetObjectAsset();
			}
		}
		else if (OverridePin->LinkedTo[0]->GetOwningNode()->IsA<UNiagaraNodeFunctionCall>())
		{
			DynamicValueFunction = MakeShared<FNiagaraStackFunctionMergeAdapter>(InOwningEmitter, *OwningScript.Get(), *CastChecked<UNiagaraNodeFunctionCall>(OverridePin->LinkedTo[0]->GetOwningNode()), INDEX_NONE);
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Error, "Invalid Stack Graph - Unsupported input node connection. Owning Node: %ls", *OverrideNode->GetPathName());
		}
	}
	else
	{
		UE_LOGF(LogNiagaraEditor, Error, "Invalid Stack Graph - Input had multiple connections. Owning Node: %ls", *OverrideNode->GetPathName());
	}
}

FNiagaraStackFunctionInputOverrideMergeAdapter::FNiagaraStackFunctionInputOverrideMergeAdapter(
	UNiagaraScript& InOwningScript,
	UNiagaraNodeFunctionCall& InOwningFunctionCallNode,
	FStringView InInputName,
	FNiagaraVariable InRapidIterationParameter
)
	: OwningScript(&InOwningScript)
	, OwningFunctionCallNode(&InOwningFunctionCallNode)
	, InputName(InInputName)
	, Type(InRapidIterationParameter.GetType())
	, LocalValueRapidIterationParameter(InRapidIterationParameter)
{
}

FNiagaraStackFunctionInputOverrideMergeAdapter::FNiagaraStackFunctionInputOverrideMergeAdapter(UEdGraphPin* InStaticSwitchPin)
	: OwningFunctionCallNode(CastChecked<UNiagaraNodeFunctionCall>(InStaticSwitchPin->GetOwningNode()))
	, InputName(InStaticSwitchPin->PinName.ToString())
	, StaticSwitchValue(InStaticSwitchPin->DefaultValue)
{
	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	Type = NiagaraSchema->PinToTypeDefinition(InStaticSwitchPin);
}

UNiagaraScript* FNiagaraStackFunctionInputOverrideMergeAdapter::GetOwningScript() const
{
	return OwningScript.Get();
}

UNiagaraNodeFunctionCall* FNiagaraStackFunctionInputOverrideMergeAdapter::GetOwningFunctionCall() const
{
	return OwningFunctionCallNode.Get();
}

FString FNiagaraStackFunctionInputOverrideMergeAdapter::GetInputName() const
{
	return InputName;
}

UNiagaraNodeParameterMapSet* FNiagaraStackFunctionInputOverrideMergeAdapter::GetOverrideNode() const
{
	return OverrideNode.Get();
}

const FNiagaraTypeDefinition& FNiagaraStackFunctionInputOverrideMergeAdapter::GetType() const
{
	return Type;
}

UEdGraphPin* FNiagaraStackFunctionInputOverrideMergeAdapter::GetOverridePin() const
{
	return OverridePin;
}

const FGuid& FNiagaraStackFunctionInputOverrideMergeAdapter::GetOverrideNodeId() const
{
	return OverrideValueNodePersistentId;
}

TOptional<FString> FNiagaraStackFunctionInputOverrideMergeAdapter::GetLocalValueString() const
{
	return LocalValueString;
}

TOptional<FNiagaraVariable> FNiagaraStackFunctionInputOverrideMergeAdapter::GetLocalValueRapidIterationParameter() const
{
	return LocalValueRapidIterationParameter;
}

TOptional<FNiagaraStackLinkedValueData> FNiagaraStackFunctionInputOverrideMergeAdapter::GetLinkedValueData() const
{
	return LinkedValueData;
}

TOptional<FName> FNiagaraStackFunctionInputOverrideMergeAdapter::GetDataInterfaceValueInputName() const
{
	return DataInterfaceValueInputName;
}

TOptional<FNiagaraVariableBase> FNiagaraStackFunctionInputOverrideMergeAdapter::GetObjectAssetInputVariable() const
{
	return ObjectAssetInputVariable;
}

UObject* FNiagaraStackFunctionInputOverrideMergeAdapter::GetObjectAssetValue() const
{
	return ObjectAssetValue;
}

UNiagaraDataInterface* FNiagaraStackFunctionInputOverrideMergeAdapter::GetDataInterfaceValue() const
{
	return DataInterfaceValue;
}

TSharedPtr<FNiagaraStackFunctionMergeAdapter> FNiagaraStackFunctionInputOverrideMergeAdapter::GetDynamicValueFunction() const
{
	return DynamicValueFunction;
}

TOptional<FString> FNiagaraStackFunctionInputOverrideMergeAdapter::GetStaticSwitchValue() const
{
	return StaticSwitchValue;
}

FNiagaraStackFunctionMergeAdapter::FNiagaraStackFunctionMergeAdapter(const FVersionedNiagaraEmitter& InOwningEmitter, UNiagaraScript& InOwningScript, UNiagaraNodeFunctionCall& InFunctionCallNode, int32 InStackIndex)
{
	OwningScript = &InOwningScript;
	FunctionCallNode = &InFunctionCallNode;
	StackIndex = InStackIndex;

	FVersionedNiagaraEmitterData* EmitterData = InOwningEmitter.GetEmitterData();
	bUsesScratchPadScript = EmitterData->ScratchPads->Scripts.Contains(FunctionCallNode->FunctionScript) ||
		EmitterData->ParentScratchPads->Scripts.Contains(FunctionCallNode->FunctionScript);

	FString UniqueEmitterName = InOwningEmitter.Emitter->GetUniqueEmitterName();

	UNiagaraNodeOutput* FunctionCallOutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*FunctionCallNode);
	if (FunctionCallOutputNode == nullptr)
	{
		UE_LOGF(LogNiagaraEditor, Warning, "FunctionCallOutputNode is nullptr in FNiagaraStackFunctionMergeAdapter, results may be incorrect");
	}
	FCompileConstantResolver ConstantResolver = FunctionCallOutputNode ? FCompileConstantResolver(InOwningEmitter, FunctionCallOutputNode->GetUsage()) : FCompileConstantResolver();

	// Collect explicit overrides set via parameter map set nodes.
	TSet<FName> AliasedInputsAdded;
	UNiagaraNodeParameterMapSet* OverrideNode = FNiagaraStackGraphUtilities::GetStackFunctionOverrideNode(*FunctionCallNode);
	if (OverrideNode != nullptr)
	{
		for(UEdGraphPin* OverridePin : FNiagaraStackGraphUtilities::GetOverridePinsForFunction(*OverrideNode, *FunctionCallNode))
		{
			InputOverrides.Add(MakeShared<FNiagaraStackFunctionInputOverrideMergeAdapter>(InOwningEmitter, *OwningScript.Get(), *FunctionCallNode.Get(), *OverridePin));
			AliasedInputsAdded.Add(*OverridePin->PinName.ToString());
		}
	}

	// If we have a valid function script collect up the default values of the rapid iteration parameters so that default values in the parameter store
	// can be ignored since they're not actually overrides.  This is usually not an issue due to the PreparateRapidIterationParameters call in the emitter
	// editor, but modifications to modules can cause inconsistency here in the emitter.
	TArray<FNiagaraVariable> RapidIterationInputDefaultValues;
	if (InFunctionCallNode.FunctionScript != nullptr)
	{
		TArray<FNiagaraVariable> FunctionInputVariables;
		TArray<FName> FunctionInputVariableNames;

		{
			FCompileConstantResolver Resolver(InOwningEmitter, InOwningScript.GetUsage());
			TArray<FNiagaraVariable> AllFunctionInputVariables;
			GetStackFunctionInputs(*FunctionCallNode, AllFunctionInputVariables, Resolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly, false);

			const int32 AllFunctionInputVariableCount = AllFunctionInputVariables.Num();
			FunctionInputVariables.Reserve(AllFunctionInputVariableCount);
			FunctionInputVariableNames.Reserve(AllFunctionInputVariableCount);

			for (const FNiagaraVariable& FunctionInputVariable : AllFunctionInputVariables)
			{
				if (FunctionInputVariable.IsValid() && FNiagaraStackGraphUtilities::IsRapidIterationType(FunctionInputVariable.GetType()))
				{
					FunctionInputVariables.Add(FunctionInputVariable);
					FunctionInputVariableNames.Add(FunctionInputVariable.GetName());
				}
			}
		}

		TArray<UEdGraphPin*> FunctionInputDefaultValuePins;
		FunctionInputDefaultValuePins.AddZeroed(FunctionInputVariables.Num());
		FunctionCallNode->FindParameterMapDefaultValuePins(FunctionInputVariableNames, InOwningScript.GetUsage(), ConstantResolver, FunctionInputDefaultValuePins);

		for (int32 i = 0; i < FunctionInputVariables.Num(); i++)
		{
			FNiagaraVariable FunctionInputVariable = FunctionInputVariables[i];
			UEdGraphPin* FunctionInputDefaultPin = FunctionInputDefaultValuePins[i];
			if (FunctionInputDefaultPin != nullptr)
			{
				// Try to get the default value from the default pin.
				FNiagaraVariable FunctionInputDefaultVariable = UEdGraphSchema_Niagara::PinToNiagaraVariable(FunctionInputDefaultPin);
				if (FunctionInputDefaultVariable.GetData() != nullptr)
				{
					FunctionInputVariable.SetData(FunctionInputDefaultVariable.GetData());
				}
			}

			if (FunctionInputVariable.GetData() == nullptr)
			{
				// If the pin didn't have a default value then use the type default.
				FNiagaraEditorUtilities::ResetVariableToDefaultValue(FunctionInputVariable);
			}

			if (FunctionInputVariable.GetData() != nullptr)
			{
				FNiagaraParameterHandle AliasedFunctionInputHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(FNiagaraParameterHandle(FunctionInputVariable.GetName()), &InFunctionCallNode);
				FNiagaraVariable FunctionInputRapidIterationParameter =
					FNiagaraStackGraphUtilities::CreateRapidIterationParameter(UniqueEmitterName, OwningScript->GetUsage(), AliasedFunctionInputHandle.GetParameterHandleString(), FunctionInputVariable.GetType());
				FunctionInputRapidIterationParameter.SetData(FunctionInputVariable.GetData());
				RapidIterationInputDefaultValues.Add(FunctionInputRapidIterationParameter);
			}
		}
	}

	// Collect rapid iteration parameters which aren't at the function default values.
	FString RapidIterationParameterNamePrefix = TEXT("Constants." + UniqueEmitterName + ".");
	TArray<FNiagaraVariable> RapidIterationParameters;
	OwningScript->RapidIterationParameters.GetParameters(RapidIterationParameters);
	for (const FNiagaraVariable& RapidIterationParameter : RapidIterationParameters)
	{
		FNiagaraParameterHandle AliasedInputHandle(*RapidIterationParameter.GetName().ToString().RightChop(RapidIterationParameterNamePrefix.Len()));
		if (AliasedInputHandle.GetNamespace().ToString() == FunctionCallNode->GetFunctionName())
		{
			// Currently rapid iteration parameters for assignment nodes in emitter scripts get double aliased which prevents their inputs from
			// being diffed correctly, so we need to un-mangle the names here so that the diffs are correct.
			if (FunctionCallNode->IsA<UNiagaraNodeAssignment>() &&
				(OwningScript->GetUsage() == ENiagaraScriptUsage::EmitterSpawnScript || OwningScript->GetUsage() == ENiagaraScriptUsage::EmitterUpdateScript))
			{
				FString InputName = AliasedInputHandle.GetName().ToString();
				if (InputName.StartsWith(UniqueEmitterName + TEXT(".")))
				{
					FString UnaliasedInputName = TEXT("Emitter") + InputName.RightChop(UniqueEmitterName.Len());
					AliasedInputHandle = FNiagaraParameterHandle(AliasedInputHandle.GetNamespace(), *UnaliasedInputName);
				}
			}

			if (AliasedInputsAdded.Contains(AliasedInputHandle.GetParameterHandleString()) == false)
			{
				// Check if the input is at the current default and if so it can be skipped.
				bool bMatchesDefault = false;
				FNiagaraVariable* RapidIterationInputDefaultValue = RapidIterationInputDefaultValues.FindByPredicate([RapidIterationParameter](const FNiagaraVariable& DefaultValue) 
					{ return DefaultValue.GetName() == RapidIterationParameter.GetName() && DefaultValue.GetType() == RapidIterationParameter.GetType(); });
				if (RapidIterationInputDefaultValue != nullptr)
				{
					const uint8* CurrentValueData = OwningScript->RapidIterationParameters.GetParameterData(RapidIterationParameter);
					if (CurrentValueData != nullptr)
					{
						bMatchesDefault = FMemory::Memcmp(CurrentValueData, RapidIterationInputDefaultValue->GetData(), RapidIterationParameter.GetSizeInBytes()) == 0;
					}
				}

				if (bMatchesDefault == false)
				{
					InputOverrides.Add(MakeShared<FNiagaraStackFunctionInputOverrideMergeAdapter>(*OwningScript.Get(), *FunctionCallNode.Get(), AliasedInputHandle.GetName().ToString(), RapidIterationParameter));
				}
			}
		}
	}

	TArray<UEdGraphPin*> StaticSwitchPins;
	TSet<UEdGraphPin*> StaticSwitchPinsHidden;
	FNiagaraStackGraphUtilities::GetStackFunctionStaticSwitchPins(*FunctionCallNode.Get(), StaticSwitchPins, StaticSwitchPinsHidden, ConstantResolver);

	for (UEdGraphPin* StaticSwitchPin : StaticSwitchPins)
	{
		if (StaticSwitchPin->AutogeneratedDefaultValue.IsEmpty() || StaticSwitchPin->DoesDefaultValueMatchAutogenerated() == false)
		{
			InputOverrides.Add(MakeShared<FNiagaraStackFunctionInputOverrideMergeAdapter>(StaticSwitchPin));
		}
	}
}

UNiagaraNodeFunctionCall* FNiagaraStackFunctionMergeAdapter::GetFunctionCallNode() const
{
	return FunctionCallNode.Get();
}

int32 FNiagaraStackFunctionMergeAdapter::GetStackIndex() const
{
	return StackIndex;
}

bool FNiagaraStackFunctionMergeAdapter::GetUsesScratchPadScript() const
{
	return bUsesScratchPadScript;
}

const TArray<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>>& FNiagaraStackFunctionMergeAdapter::GetInputOverrides() const
{
	return InputOverrides;
}

TSharedPtr<FNiagaraStackFunctionInputOverrideMergeAdapter> FNiagaraStackFunctionMergeAdapter::GetInputOverrideByInputNameAndType(const FString& InputName, const FNiagaraTypeDefinition& InputType) const
{
	for (TSharedPtr<FNiagaraStackFunctionInputOverrideMergeAdapter> InputOverride : InputOverrides)
	{
		if (InputOverride->GetInputName() == InputName && InputOverride->GetType() == InputType)
		{
			return InputOverride;
		}
	}
	return TSharedPtr<FNiagaraStackFunctionInputOverrideMergeAdapter>();
}

void FNiagaraStackFunctionMergeAdapter::GatherFunctionCallNodes(TArray<UNiagaraNodeFunctionCall*>& OutFunctionCallNodes) const
{
	if (FunctionCallNode.IsValid())
	{
		OutFunctionCallNodes.Add(FunctionCallNode.Get());
	}

	for (TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> InputOverride : InputOverrides)
	{
		if (InputOverride->GetDynamicValueFunction().IsValid())
		{
			InputOverride->GetDynamicValueFunction()->GatherFunctionCallNodes(OutFunctionCallNodes);
		}
	}
}

FNiagaraScriptStackMergeAdapter::FNiagaraScriptStackMergeAdapter(const FVersionedNiagaraEmitter& InOwningEmitter, UNiagaraNodeOutput& InOutputNode, UNiagaraScript& InScript)
{
	OutputNode = &InOutputNode;
	InputNode.Reset();
	Script = &InScript;
	UniqueEmitterName = InOwningEmitter.Emitter->GetUniqueEmitterName();
	
	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackGroups;
	GetStackNodeGroups(*OutputNode, StackGroups);

	if (StackGroups.Num() >= 2 && StackGroups[0].EndNode->IsA<UNiagaraNodeInput>())
	{
		InputNode = Cast<UNiagaraNodeInput>(StackGroups[0].EndNode);
	}

	if (StackGroups.Num() > 2 && StackGroups[0].EndNode->IsA<UNiagaraNodeInput>() && StackGroups.Last().EndNode->IsA<UNiagaraNodeOutput>())
	{
		for (int i = 1; i < StackGroups.Num() - 1; i++)
		{
			UNiagaraNodeFunctionCall* ModuleFunctionCallNode = Cast<UNiagaraNodeFunctionCall>(StackGroups[i].EndNode);
			if (ModuleFunctionCallNode != nullptr)
			{
				// The first stack node group is the input node, so we subtract one to get the index of the module.
				int32 StackIndex = i - 1;
				ModuleFunctions.Add(MakeShared<FNiagaraStackFunctionMergeAdapter>(InOwningEmitter, *Script.Get(), *ModuleFunctionCallNode, StackIndex));
			}
		}
	}
}

UNiagaraNodeInput* FNiagaraScriptStackMergeAdapter::GetInputNode() const
{
	return InputNode.Get();
}

UNiagaraNodeOutput* FNiagaraScriptStackMergeAdapter::GetOutputNode() const
{
	return OutputNode.Get();
}

UNiagaraScript* FNiagaraScriptStackMergeAdapter::GetScript() const
{
	return Script.Get();
}

FString FNiagaraScriptStackMergeAdapter::GetUniqueEmitterName() const
{
	return UniqueEmitterName;
}

const TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>>& FNiagaraScriptStackMergeAdapter::GetModuleFunctions() const
{
	return ModuleFunctions;
}

TSharedPtr<FNiagaraStackFunctionMergeAdapter> FNiagaraScriptStackMergeAdapter::GetModuleFunctionById(FGuid FunctionCallNodeId) const
{
	for (TSharedRef<FNiagaraStackFunctionMergeAdapter> Modulefunction : ModuleFunctions)
	{
		if (Modulefunction->GetFunctionCallNode()->NodeGuid == FunctionCallNodeId)
		{
			return Modulefunction;
		}
	}
	return TSharedPtr<FNiagaraStackFunctionMergeAdapter>();
}

void FNiagaraScriptStackMergeAdapter::GatherFunctionCallNodes(TArray<UNiagaraNodeFunctionCall*>& OutFunctionCallNodes) const
{
	for (TSharedRef<FNiagaraStackFunctionMergeAdapter> ModuleFunction : ModuleFunctions)
	{
		ModuleFunction->GatherFunctionCallNodes(OutFunctionCallNodes);
	}
}

FNiagaraEventHandlerMergeAdapter::FNiagaraEventHandlerMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, const FNiagaraEventScriptProperties* InEventScriptProperties, UNiagaraNodeOutput* InOutputNode)
{
	Initialize(InEmitter, InEventScriptProperties, nullptr, InOutputNode);
}

FNiagaraEventHandlerMergeAdapter::FNiagaraEventHandlerMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, FNiagaraEventScriptProperties* InEventScriptProperties, UNiagaraNodeOutput* InOutputNode)
{
	Initialize(InEmitter, InEventScriptProperties, InEventScriptProperties, InOutputNode);
}

FNiagaraEventHandlerMergeAdapter::FNiagaraEventHandlerMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, UNiagaraNodeOutput* InOutputNode)
{
	Initialize(InEmitter, nullptr, nullptr, InOutputNode);
}

void FNiagaraEventHandlerMergeAdapter::Initialize(const FVersionedNiagaraEmitter& InEmitter, const FNiagaraEventScriptProperties* InEventScriptProperties, FNiagaraEventScriptProperties* InEditableEventScriptProperties, UNiagaraNodeOutput* InOutputNode)
{
	Emitter =InEmitter.ToWeakPtr();

	EventScriptProperties = InEventScriptProperties;
	EditableEventScriptProperties = InEditableEventScriptProperties;

	OutputNode = InOutputNode;

	if (EventScriptProperties != nullptr && OutputNode != nullptr)
	{
		EventStack = MakeShared<FNiagaraScriptStackMergeAdapter>(InEmitter, *OutputNode.Get(), *EventScriptProperties->Script);
		InputNode = EventStack->GetInputNode();
	}
}

FVersionedNiagaraEmitter FNiagaraEventHandlerMergeAdapter::GetEmitter() const
{
	return Emitter.ResolveWeakPtr();
}

FGuid FNiagaraEventHandlerMergeAdapter::GetUsageId() const
{
	if (EventScriptProperties != nullptr)
	{
		return EventScriptProperties->Script->GetUsageId();
	}
	else
	{
		return OutputNode->GetUsageId();
	}
}

const FNiagaraEventScriptProperties* FNiagaraEventHandlerMergeAdapter::GetEventScriptProperties() const
{
	return EventScriptProperties;
}

FNiagaraEventScriptProperties* FNiagaraEventHandlerMergeAdapter::GetEditableEventScriptProperties() const
{
	return EditableEventScriptProperties;
}

UNiagaraNodeOutput* FNiagaraEventHandlerMergeAdapter::GetOutputNode() const
{
	return OutputNode.Get();
}

UNiagaraNodeInput* FNiagaraEventHandlerMergeAdapter::GetInputNode() const
{
	return InputNode.Get();
}

TSharedPtr<FNiagaraScriptStackMergeAdapter> FNiagaraEventHandlerMergeAdapter::GetEventStack() const
{
	return EventStack;
}

FNiagaraSimulationStageMergeAdapter::FNiagaraSimulationStageMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, const UNiagaraSimulationStageBase* InSimulationStage, int32 InSimulationStageIndex, UNiagaraNodeOutput* InOutputNode)
{
	Initialize(InEmitter, InSimulationStage, nullptr, InSimulationStageIndex, InOutputNode);
}

FNiagaraSimulationStageMergeAdapter::FNiagaraSimulationStageMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, UNiagaraSimulationStageBase* InSimulationStage, int32 InSimulationStageIndex, UNiagaraNodeOutput* InOutputNode)
{
	Initialize(InEmitter, InSimulationStage, InSimulationStage, InSimulationStageIndex, InOutputNode);
}

FNiagaraSimulationStageMergeAdapter::FNiagaraSimulationStageMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, UNiagaraNodeOutput* InOutputNode)
{
	Initialize(InEmitter, nullptr, nullptr, INDEX_NONE, InOutputNode);
}

void FNiagaraSimulationStageMergeAdapter::Initialize(const FVersionedNiagaraEmitter& InEmitter, const UNiagaraSimulationStageBase* InSimulationStage, UNiagaraSimulationStageBase* InEditableSimulationStage, int32 InSimulationStageIndex, UNiagaraNodeOutput* InOutputNode)
{
	Emitter = InEmitter.ToWeakPtr();

	SimulationStage = InSimulationStage;
	EditableSimulationStage = InEditableSimulationStage;

	OutputNode = InOutputNode;
	SimulationStageIndex = InSimulationStageIndex;

	if (SimulationStage != nullptr && OutputNode != nullptr)
	{
		SimulationStageStack = MakeShared<FNiagaraScriptStackMergeAdapter>(InEmitter, *OutputNode.Get(), *SimulationStage->Script);
		InputNode = SimulationStageStack->GetInputNode();
	}
}

FVersionedNiagaraEmitter FNiagaraSimulationStageMergeAdapter::GetEmitter() const
{
	return Emitter.ResolveWeakPtr();
}

FGuid FNiagaraSimulationStageMergeAdapter::GetUsageId() const
{
	if (SimulationStage != nullptr)
	{
		return SimulationStage->Script->GetUsageId();
	}
	else
	{
		return OutputNode->GetUsageId();
	}
}

const UNiagaraSimulationStageBase* FNiagaraSimulationStageMergeAdapter::GetSimulationStage() const
{
	return SimulationStage;
}

UNiagaraSimulationStageBase* FNiagaraSimulationStageMergeAdapter::GetEditableSimulationStage() const
{
	return EditableSimulationStage;
}

UNiagaraNodeOutput* FNiagaraSimulationStageMergeAdapter::GetOutputNode() const
{
	return OutputNode.Get();
}

UNiagaraNodeInput* FNiagaraSimulationStageMergeAdapter::GetInputNode() const
{
	return InputNode.Get();
}

int32 FNiagaraSimulationStageMergeAdapter::GetSimulationStageIndex() const
{
	return SimulationStageIndex;
}

TSharedPtr<FNiagaraScriptStackMergeAdapter> FNiagaraSimulationStageMergeAdapter::GetSimulationStageStack() const
{
	return SimulationStageStack;
}

FNiagaraRendererMergeAdapter::FNiagaraRendererMergeAdapter(UNiagaraRendererProperties& InRenderer)
{
	Renderer = &InRenderer;
}

UNiagaraRendererProperties* FNiagaraRendererMergeAdapter::GetRenderer()
{
	return Renderer.Get();
}

FNiagaraInputSummaryMergeAdapter::FNiagaraInputSummaryMergeAdapter(const FFunctionInputSummaryViewKey& Key, const FFunctionInputSummaryViewMetadata& Value)
		: Key(Key)
		, Value(Value)
{
}

const FFunctionInputSummaryViewKey& FNiagaraInputSummaryMergeAdapter::GetKey() const
{
	return Key;
}
const FFunctionInputSummaryViewMetadata& FNiagaraInputSummaryMergeAdapter::GetValue() const
{
	return Value;
}

FNiagaraScratchPadMergeAdapter::FNiagaraScratchPadMergeAdapter()
	: bIsInitialized(false)
{
}

FNiagaraScratchPadMergeAdapter::FNiagaraScratchPadMergeAdapter(const FVersionedNiagaraEmitter InTargetEmitter, const FVersionedNiagaraEmitter& InInstanceEmitter, const FVersionedNiagaraEmitter& InParentEmitter)
	: TargetEmitter(InTargetEmitter)
	, InstanceEmitter(InInstanceEmitter)
	, ParentEmitter(InParentEmitter)
	, bIsInitialized(false)
{
}

UNiagaraScript* FNiagaraScratchPadMergeAdapter::GetScratchPadScriptForFunctionId(FGuid FunctionId)
{
	if (bIsInitialized == false)
	{
		Initialize();
	}

	UNiagaraScript** ParentScratchPadScriptPtr = FunctionIdToScratchPadScript.Find(FunctionId);
	return ParentScratchPadScriptPtr != nullptr ? *ParentScratchPadScriptPtr : nullptr;
}


TArray< FNiagaraScratchPadMergeAdapter::FMergeRecord > FNiagaraScratchPadMergeAdapter::GetMergedEmitterRecords() const
{
	TArray< FNiagaraScratchPadMergeAdapter::FMergeRecord > FoundArray;

	// Logic needs to be kept in sync with FNiagaraScratchPadMergeAdapter::Initialize()
	TArray<FVersionedNiagaraEmitter> ParentEmitters;
	FVersionedNiagaraEmitter CurrentParent = ParentEmitter;
	while (CurrentParent.Emitter != nullptr)
	{
		ParentEmitters.Insert(CurrentParent, 0);
		CurrentParent = CurrentParent.GetEmitterData()->GetParent();
	}

	for (FVersionedNiagaraEmitter CurrentParentEmitter : ParentEmitters)
	{
		FVersionedNiagaraEmitterData* CurrentParentEmitterData = CurrentParentEmitter.GetEmitterData();
		if (CurrentParentEmitterData->GetParent().Emitter == nullptr)
		{
			for (UNiagaraScript* Script : CurrentParentEmitterData->ParentScratchPads->Scripts)
			{
				// Right now we don't version scratch scripts. If we end up doing that, we'll need to account for that here.
				FoundArray.Emplace(CurrentParentEmitter, Script, FGuid());
			}
		}
		for (UNiagaraScript* Script : CurrentParentEmitterData->ScratchPads->Scripts)
		{
			// Right now we don't version scratch scripts. If we end up doing that, we'll need to account for that here.
			FoundArray.Emplace(CurrentParentEmitter, Script, FGuid());
		}
	}

	return FoundArray;
}

void FNiagaraScratchPadMergeAdapter::Initialize()
{
	FVersionedNiagaraEmitterData* TargetEmitterData = TargetEmitter.GetEmitterData();
	if (TargetEmitterData->ScratchPads->Scripts.Num() > 0 || TargetEmitterData->ParentScratchPads->Scripts.Num() > 0)
	{
		// Collect the parent emitters in order
		TArray<FVersionedNiagaraEmitter> ParentEmitters;
		FVersionedNiagaraEmitter CurrentParent = ParentEmitter;
		while (CurrentParent.Emitter != nullptr)
		{
			ParentEmitters.Insert(CurrentParent, 0);
			CurrentParent = CurrentParent.GetEmitterData()->GetParent();
		}

		// Create a mapping from the actual parent scratch pad scripts to their copies in the target emitter.
		TMap<UNiagaraScript*, UNiagaraScript*> SourceScratchPadScriptToTargetCopyScratchPadScript;
		int32 TargetCopyIndex = 0;
		for (FVersionedNiagaraEmitter CurrentParentEmitter : ParentEmitters)
		{
			TArray<UNiagaraScript*> ParentScratchPadScripts;
			FVersionedNiagaraEmitterData* CurrentParentEmitterData = CurrentParentEmitter.GetEmitterData();
			if (CurrentParentEmitterData->GetParent().Emitter == nullptr)
			{
				ParentScratchPadScripts.Append(CurrentParentEmitterData->ParentScratchPads->Scripts);
			}
			ParentScratchPadScripts.Append(CurrentParentEmitterData->ScratchPads->Scripts);

			for (UNiagaraScript* ParentScratchPadScript : ParentScratchPadScripts)
			{
				UNiagaraScript* TargetCopy;
				if (ensureMsgf(TargetCopyIndex < TargetEmitterData->ParentScratchPads->Scripts.Num(), TEXT("Parent scratch pad script was missing from the Target's copies. Emitter %s"), *GetPathNameSafe(InstanceEmitter.Emitter)))
				{
					TargetCopy = TargetEmitterData->ParentScratchPads->Scripts[TargetCopyIndex];
				}
				else
				{
					TargetCopy = nullptr;
				}
				SourceScratchPadScriptToTargetCopyScratchPadScript.Add(ParentScratchPadScript, TargetCopy);
				TargetCopyIndex++;
			}
		}

		// Create a mapping from the instance scratch pad scripts to their copies in the target emitter.
		if (FVersionedNiagaraEmitterData* InstanceEmitterData = InstanceEmitter.GetEmitterData())
		{
			for (int32 InstanceScratchPadScriptIndex = 0; InstanceScratchPadScriptIndex < InstanceEmitterData->ScratchPads->Scripts.Num(); InstanceScratchPadScriptIndex++)
			{
				UNiagaraScript* TargetCopy;
				if (ensureMsgf(InstanceScratchPadScriptIndex < TargetEmitterData->ScratchPads->Scripts.Num(), TEXT("Instance scratch pad script was missing from the Target's copies. Emitter %s"), *GetPathNameSafe(InstanceEmitter.Emitter)))
				{
					TargetCopy = TargetEmitterData->ScratchPads->Scripts[InstanceScratchPadScriptIndex];
				}
				else
				{
					TargetCopy = nullptr;
				}
				SourceScratchPadScriptToTargetCopyScratchPadScript.Add(InstanceEmitterData->ScratchPads->Scripts[InstanceScratchPadScriptIndex], TargetCopy);
			}
		}

		// For each source emitter collect up the traversed function ids that use the source scratch pad scripts and cache the corresponding target copies so that
		// any usages encountered when applying the diff can be hooked up correctly.
		TArray<FVersionedNiagaraEmitter> SourceEmitters;
		SourceEmitters.Append(ParentEmitters);
		if (InstanceEmitter.Emitter != nullptr)
		{
			SourceEmitters.Add(InstanceEmitter);
		}
		for (FVersionedNiagaraEmitter SourceEmitter : SourceEmitters)
		{
			FVersionedNiagaraEmitterData* SourceEmitterData = SourceEmitter.GetEmitterData();
			if (SourceEmitterData->ScratchPads->Scripts.Num() > 0 || (SourceEmitterData->GetParent().Emitter == nullptr && SourceEmitterData->ParentScratchPads->Scripts.Num() > 0))
			{
				UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(SourceEmitterData->GraphSource);
				if (ScriptSource != nullptr)
				{
					TArray<UNiagaraNodeOutput*> OutputNodes;
					ScriptSource->NodeGraph->GetNodesOfClass(OutputNodes);
					for (UNiagaraNodeOutput* OutputNode : OutputNodes)
					{
						TArray<UNiagaraNode*> TraversedNodes;
						UNiagaraGraph::BuildTraversal(TraversedNodes, OutputNode, false);
						for (UNiagaraNode* TraversedNode : TraversedNodes)
						{
							UNiagaraNodeFunctionCall* TraversedFunctionNode = Cast<UNiagaraNodeFunctionCall>(TraversedNode);
							if (TraversedFunctionNode != nullptr &&
								TraversedFunctionNode->FunctionScript != nullptr &&
								TraversedFunctionNode->FunctionScript->IsAsset() == false &&
								TraversedFunctionNode->GetClass() == UNiagaraNodeFunctionCall::StaticClass())
							{
								UNiagaraScript* ScratchPadScript = TraversedFunctionNode->FunctionScript;
								if (SourceEmitterData->ScratchPads->Scripts.Contains(ScratchPadScript) ||
									(SourceEmitterData->GetParent().Emitter == nullptr && SourceEmitterData->ParentScratchPads->Scripts.Contains(ScratchPadScript)))
								{
									FunctionIdToScratchPadScript.Add(TraversedFunctionNode->NodeGuid, SourceScratchPadScriptToTargetCopyScratchPadScript[TraversedFunctionNode->FunctionScript]);
								}
							}
						}
					}
				}
			}
		}
	}

	bIsInitialized = true;
}

FNiagaraEmitterMergeAdapter::FNiagaraEmitterMergeAdapter(const FVersionedNiagaraEmitter& InEmitter)
{
	Initialize(InEmitter, InEmitter);
}

void FNiagaraEmitterMergeAdapter::Initialize(const FVersionedNiagaraEmitter& InEmitter, const FVersionedNiagaraEmitter& InEditableEmitter)
{
	Emitter = InEmitter.ToWeakPtr();
	EditableEmitter = InEditableEmitter.ToWeakPtr();
	FVersionedNiagaraEmitterData* EmitterData = Emitter.GetEmitterData();
	if (EmitterData == nullptr)
	{
		return;
	}
	UNiagaraScriptSource* EmitterScriptSource = Cast<UNiagaraScriptSource>(EmitterData->GraphSource);
	UNiagaraGraph* Graph = EmitterScriptSource->NodeGraph;
	TArray<UNiagaraNodeOutput*> OutputNodes;
	Graph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);

	TArray<UNiagaraNodeOutput*> EventOutputNodes;
	TArray<UNiagaraNodeOutput*> SimulationStageOutputNodes;
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		if (UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::EmitterSpawnScript))
		{
			EmitterSpawnStack = MakeShared<FNiagaraScriptStackMergeAdapter>(InEmitter, *OutputNode, *EmitterData->EmitterSpawnScriptProps.Script);
		}
		else if (UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::EmitterUpdateScript))
		{
			EmitterUpdateStack = MakeShared<FNiagaraScriptStackMergeAdapter>(InEmitter, *OutputNode, *EmitterData->EmitterUpdateScriptProps.Script);
		}
		else if (UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::ParticleSpawnScript))
		{
			ParticleSpawnStack = MakeShared<FNiagaraScriptStackMergeAdapter>(InEmitter, *OutputNode, *EmitterData->SpawnScriptProps.Script);
		}
		else if (UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::ParticleUpdateScript))
		{
			ParticleUpdateStack = MakeShared<FNiagaraScriptStackMergeAdapter>(InEmitter, *OutputNode, *EmitterData->UpdateScriptProps.Script);
		}
		else if(UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::ParticleEventScript))
		{
			EventOutputNodes.Add(OutputNode);
		}
		else if (UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), ENiagaraScriptUsage::ParticleSimulationStageScript))
		{
			SimulationStageOutputNodes.Add(OutputNode);
		}
	}

	// Create an event handler adapter for each usage id even if it's missing an event script properties struct or an output node.  These
	// incomplete adapters will be caught if they are diffed.
	for (const FNiagaraEventScriptProperties& EventScriptProperties : EmitterData->GetEventHandlers())
	{
		UNiagaraNodeOutput** MatchingOutputNodePtr = EventOutputNodes.FindByPredicate(
			[=](UNiagaraNodeOutput* EventOutputNode) { return EventOutputNode->GetUsageId() == EventScriptProperties.Script->GetUsageId(); });

		UNiagaraNodeOutput* MatchingOutputNode = MatchingOutputNodePtr != nullptr ? *MatchingOutputNodePtr : nullptr;

		if (EditableEmitter.Emitter == nullptr)
		{
			EventHandlers.Add(MakeShared<FNiagaraEventHandlerMergeAdapter>(InEmitter, &EventScriptProperties, MatchingOutputNode));
		}
		else
		{
			FNiagaraEventScriptProperties* EditableEventScriptProperties = EditableEmitter.GetEmitterData()->GetEventHandlerByIdUnsafe(EventScriptProperties.Script->GetUsageId());
			EventHandlers.Add(MakeShared<FNiagaraEventHandlerMergeAdapter>(InEmitter, EditableEventScriptProperties, MatchingOutputNode));
		}

		if (MatchingOutputNode != nullptr)
		{
			EventOutputNodes.Remove(MatchingOutputNode);
		}
	}

	for (UNiagaraNodeOutput* EventOutputNode : EventOutputNodes)
	{
		EventHandlers.Add(MakeShared<FNiagaraEventHandlerMergeAdapter>(InEmitter, EventOutputNode));
	}

	// Create an shader stage adapter for each usage id even if it's missing a shader stage object or an output node.  These
	// incomplete adapters will be caught if they are diffed.
	for (int32 SimulationStageIndex = 0; SimulationStageIndex < EmitterData->GetSimulationStages().Num(); SimulationStageIndex++)
	{
		const UNiagaraSimulationStageBase* SimulationStage = EmitterData->GetSimulationStages()[SimulationStageIndex];
	
		UNiagaraNodeOutput** MatchingOutputNodePtr = SimulationStageOutputNodes.FindByPredicate(
			[=](UNiagaraNodeOutput* SimulationStageOutputNode) { return SimulationStageOutputNode->GetUsageId() == SimulationStage->Script->GetUsageId(); });

		UNiagaraNodeOutput* MatchingOutputNode = MatchingOutputNodePtr != nullptr ? *MatchingOutputNodePtr : nullptr;

		if (EditableEmitter.Emitter == nullptr)
		{
			SimulationStages.Add(MakeShared<FNiagaraSimulationStageMergeAdapter>(InEmitter, SimulationStage, SimulationStageIndex, MatchingOutputNode));
		}
		else
		{
			UNiagaraSimulationStageBase* EditableSimulationStage = EditableEmitter.GetEmitterData()->GetSimulationStageById(SimulationStage->Script->GetUsageId());
			SimulationStages.Add(MakeShared<FNiagaraSimulationStageMergeAdapter>(InEmitter, EditableSimulationStage, SimulationStageIndex, MatchingOutputNode));
		}

		if (MatchingOutputNode != nullptr)
		{
			SimulationStageOutputNodes.Remove(MatchingOutputNode);
		}
	}

	for (UNiagaraNodeOutput* SimulationStageOutputNode : SimulationStageOutputNodes)
	{
		SimulationStages.Add(MakeShared<FNiagaraSimulationStageMergeAdapter>(InEmitter, SimulationStageOutputNode));
	}

	// Renderers
	for (UNiagaraRendererProperties* RendererProperties : EmitterData->GetRenderers())
	{
		Renderers.Add(MakeShared<FNiagaraRendererMergeAdapter>(*RendererProperties));
	}

	EditorData = Cast<const UNiagaraEmitterEditorData>(EmitterData->GetEditorData());
}

FVersionedNiagaraEmitter FNiagaraEmitterMergeAdapter::GetEditableEmitter() const
{
	return EditableEmitter.ResolveWeakPtr();
}

TSharedPtr<FNiagaraScriptStackMergeAdapter> FNiagaraEmitterMergeAdapter::GetEmitterSpawnStack() const
{
	return EmitterSpawnStack;
}

TSharedPtr<FNiagaraScriptStackMergeAdapter> FNiagaraEmitterMergeAdapter::GetEmitterUpdateStack() const
{
	return EmitterUpdateStack;
}

TSharedPtr<FNiagaraScriptStackMergeAdapter> FNiagaraEmitterMergeAdapter::GetParticleSpawnStack() const
{
	return ParticleSpawnStack;
}

TSharedPtr<FNiagaraScriptStackMergeAdapter> FNiagaraEmitterMergeAdapter::GetParticleUpdateStack() const
{
	return ParticleUpdateStack;
}

const TArray<TSharedRef<FNiagaraEventHandlerMergeAdapter>> FNiagaraEmitterMergeAdapter::GetEventHandlers() const
{
	return EventHandlers;
}

const TArray<TSharedRef<FNiagaraSimulationStageMergeAdapter>> FNiagaraEmitterMergeAdapter::GetSimulationStages() const
{
	return SimulationStages;
}

const TArray<TSharedRef<FNiagaraRendererMergeAdapter>> FNiagaraEmitterMergeAdapter::GetRenderers() const
{
	return Renderers;
}

const UNiagaraEmitterEditorData* FNiagaraEmitterMergeAdapter::GetEditorData() const
{
	return EditorData.Get();
}

TSharedPtr<FNiagaraScriptStackMergeAdapter> FNiagaraEmitterMergeAdapter::GetScriptStack(ENiagaraScriptUsage Usage, FGuid ScriptUsageId)
{
	switch (Usage)
	{
	case ENiagaraScriptUsage::EmitterSpawnScript:
		return EmitterSpawnStack;
	case ENiagaraScriptUsage::EmitterUpdateScript:
		return EmitterUpdateStack;
	case ENiagaraScriptUsage::ParticleSpawnScript:
		return ParticleSpawnStack;
	case ENiagaraScriptUsage::ParticleUpdateScript:
		return ParticleUpdateStack;
	case ENiagaraScriptUsage::ParticleEventScript:
		for (TSharedPtr<FNiagaraEventHandlerMergeAdapter> EventHandler : EventHandlers)
		{
			if (EventHandler->GetUsageId() == ScriptUsageId)
			{
				return EventHandler->GetEventStack();
			}
		}
		break;
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
		for (TSharedPtr<FNiagaraSimulationStageMergeAdapter> SimulationStage : SimulationStages)
		{
			if (SimulationStage->GetUsageId() == ScriptUsageId)
			{
				return SimulationStage->GetSimulationStageStack();
			}
		}
		break;
	default:
		checkf(false, TEXT("Unsupported usage"));
	}

	return TSharedPtr<FNiagaraScriptStackMergeAdapter>();
}

TSharedPtr<FNiagaraEventHandlerMergeAdapter> FNiagaraEmitterMergeAdapter::GetEventHandler(FGuid EventScriptUsageId)
{
	for (TSharedRef<FNiagaraEventHandlerMergeAdapter> EventHandler : EventHandlers)
	{
		if (EventHandler->GetUsageId() == EventScriptUsageId)
		{
			return EventHandler;
		}
	}
	return TSharedPtr<FNiagaraEventHandlerMergeAdapter>();
}

TSharedPtr<FNiagaraSimulationStageMergeAdapter> FNiagaraEmitterMergeAdapter::GetSimulationStage(FGuid SimulationStageUsageId)
{
	for (TSharedRef<FNiagaraSimulationStageMergeAdapter> SimulationStage : SimulationStages)
	{
		if (SimulationStage->GetUsageId() == SimulationStageUsageId)
		{
			return SimulationStage;
		}
	}
	return TSharedPtr<FNiagaraSimulationStageMergeAdapter>();
}

TSharedPtr<FNiagaraRendererMergeAdapter> FNiagaraEmitterMergeAdapter::GetRenderer(FGuid RendererMergeId)
{
	for (TSharedRef<FNiagaraRendererMergeAdapter> Renderer : Renderers)
	{
		if (Renderer->GetRenderer()->GetMergeId() == RendererMergeId)
		{
			return Renderer;
		}
	}
	return TSharedPtr<FNiagaraRendererMergeAdapter>();
}

void FNiagaraEmitterMergeAdapter::GatherFunctionCallNodes(TArray<UNiagaraNodeFunctionCall*>& OutFunctionCallNodes) const
{
	EmitterSpawnStack->GatherFunctionCallNodes(OutFunctionCallNodes);
	EmitterUpdateStack->GatherFunctionCallNodes(OutFunctionCallNodes);
	ParticleSpawnStack->GatherFunctionCallNodes(OutFunctionCallNodes);
	ParticleUpdateStack->GatherFunctionCallNodes(OutFunctionCallNodes);
	for (TSharedRef<FNiagaraEventHandlerMergeAdapter> EventHandler : EventHandlers)
	{
		if (TSharedPtr<FNiagaraScriptStackMergeAdapter> EventStack = EventHandler->GetEventStack())
		{
			EventStack->GatherFunctionCallNodes(OutFunctionCallNodes);
		}
	}
	for (TSharedRef<FNiagaraSimulationStageMergeAdapter> SimulationStage : SimulationStages)
	{
		SimulationStage->GetSimulationStageStack()->GatherFunctionCallNodes(OutFunctionCallNodes);
	}
}

FNiagaraScriptStackDiffResults::FNiagaraScriptStackDiffResults()
	: bIsValid(true)
{
}

bool FNiagaraScriptStackDiffResults::IsEmpty() const
{
	return
		RemovedBaseModules.Num() == 0 &&
		AddedOtherModules.Num() == 0 &&
		MovedBaseModules.Num() == 0 &&
		MovedOtherModules.Num() == 0 &&
		ChangedVersionBaseModules.Num() == 0 &&
		ChangedVersionOtherModules.Num() == 0 &&
		EnabledChangedBaseModules.Num() == 0 &&
		EnabledChangedOtherModules.Num() == 0 &&
		RemovedBaseInputOverrides.Num() == 0 &&
		AddedOtherInputOverrides.Num() == 0 &&
		ModifiedOtherInputOverrides.Num() == 0 &&
		ChangedBaseUsage.IsSet() == false &&
		ChangedOtherUsage.IsSet() == false;
}

bool FNiagaraScriptStackDiffResults::IsValid() const
{
	return bIsValid;
}

void FNiagaraScriptStackDiffResults::AddError(FText ErrorMessage)
{
	ErrorMessages.Add(ErrorMessage);
	bIsValid = false;
}

const TArray<FText>& FNiagaraScriptStackDiffResults::GetErrorMessages() const
{
	return ErrorMessages;
}

FNiagaraEmitterDiffResults::FNiagaraEmitterDiffResults()
	: bScratchPadModified(false)
	, bIsValid(true)
{
}

bool FNiagaraEmitterDiffResults::IsValid() const
{
	bool bEventHandlerDiffsAreValid = true;
	for (const FNiagaraModifiedEventHandlerDiffResults& EventHandlerDiffResults : ModifiedEventHandlers)
	{
		if (EventHandlerDiffResults.ScriptDiffResults.IsValid() == false)
		{
			bEventHandlerDiffsAreValid = false;
			break;
		}
	}
	bool bSimulationStageDiffsAreValid = true;
	for (const FNiagaraModifiedSimulationStageDiffResults& SimulationStageDiffResults : ModifiedSimulationStages)
	{
		if (SimulationStageDiffResults.ScriptDiffResults.IsValid() == false)
		{
			bSimulationStageDiffsAreValid = false;
			break;
		}
	}
	return bIsValid &&
		bEventHandlerDiffsAreValid &&
		bSimulationStageDiffsAreValid &&
		EmitterSpawnDiffResults.IsValid() &&
		EmitterUpdateDiffResults.IsValid() &&
		ParticleSpawnDiffResults.IsValid() &&
		ParticleUpdateDiffResults.IsValid();
}

bool FNiagaraEmitterDiffResults::IsEmpty() const
{
	return DifferentEmitterProperties.Num() == 0 &&
		EmitterSpawnDiffResults.IsEmpty() &&
		EmitterUpdateDiffResults.IsEmpty() &&
		ParticleSpawnDiffResults.IsEmpty() &&
		ParticleUpdateDiffResults.IsEmpty() &&
		RemovedBaseEventHandlers.Num() == 0 &&
		AddedOtherEventHandlers.Num() == 0 &&
		ModifiedEventHandlers.Num() == 0 &&
		RemovedBaseSimulationStages.Num() == 0 &&
		AddedOtherSimulationStages.Num() == 0 &&
		ModifiedSimulationStages.Num() == 0 &&
		RemovedBaseRenderers.Num() == 0 &&
		AddedOtherRenderers.Num() == 0 &&
		ModifiedBaseRenderers.Num() == 0 &&
		ModifiedOtherRenderers.Num() == 0 &&
		AddedSummaryEntriesInOther.Num() == 0 &&
		AddedSummarySectionsInOther.Num() == 0 &&
		ModifiedStackEntryDisplayNames.Num() == 0 &&
		AddedOrModifiedStackNotesInOther.Num() == 0 &&
		bScratchPadModified == false &&
		NewShouldShowSummaryViewValue.IsSet() == false;
}

void FNiagaraEmitterDiffResults::AddError(FText ErrorMessage)
{
	ErrorMessages.Add(ErrorMessage);
	bIsValid = false;
}

const TArray<FText>& FNiagaraEmitterDiffResults::GetErrorMessages() const
{
	return ErrorMessages;
}

FString FNiagaraEmitterDiffResults::GetErrorMessagesString() const
{
	TArray<FString> ErrorMessageStrings;
	for (FText ErrorMessage : ErrorMessages)
	{
		ErrorMessageStrings.Add(ErrorMessage.ToString());
	}
	return FString::Join(ErrorMessageStrings, TEXT("\n"));
}

bool FNiagaraEmitterDiffResults::HasVersionChanges() const
{
	for (auto& EventHandlerDiff : ModifiedEventHandlers)
	{
		if (EventHandlerDiff.ScriptDiffResults.ChangedVersionOtherModules.Num() > 0)
		{
			return true;
		}
	}
	for (auto& SimStageDiff : ModifiedSimulationStages)
	{
		if (SimStageDiff.ScriptDiffResults.ChangedVersionOtherModules.Num() > 0)
		{
			return true;
		}
	}
	return	EmitterSpawnDiffResults.ChangedVersionOtherModules.Num() > 0 ||
			EmitterUpdateDiffResults.ChangedVersionOtherModules.Num() > 0 ||
			ParticleSpawnDiffResults.ChangedVersionOtherModules.Num() > 0 ||
			ParticleUpdateDiffResults.ChangedVersionOtherModules.Num() > 0;
}

TSharedRef<FNiagaraScriptMergeManager> FNiagaraScriptMergeManager::Get()
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	return NiagaraEditorModule.GetScriptMergeManager();
}

void FNiagaraScriptMergeManager::GetForcedChangeIds(
	const TMap<FGuid, UNiagaraNodeFunctionCall*>& InParentFunctionIdToNodeMap,
	const TMap<FGuid, UNiagaraNodeFunctionCall*>& InParentAtLastMergeFunctionIdToNodeMap,
	const TMap<FGuid, UNiagaraNodeFunctionCall*>& InInstanceFunctionIdToNodeMap,
	TMap<FGuid, FGuid>& OutFunctionIdToForcedChangeId) const
{
	for(const TPair<FGuid, UNiagaraNodeFunctionCall*>& InstanceFunctionIdNodePair : InInstanceFunctionIdToNodeMap)
	{
		const FGuid& InstanceFunctionId = InstanceFunctionIdNodePair.Key;
		UNiagaraNodeFunctionCall* InstanceFunctionNode = InstanceFunctionIdNodePair.Value;

		UNiagaraNodeFunctionCall*const* MatchingParentFunctionNodePtr = InParentFunctionIdToNodeMap.Find(InstanceFunctionId);
		UNiagaraNodeFunctionCall*const* MatchingParentAtLastMergeFunctionNodePtr = InParentAtLastMergeFunctionIdToNodeMap.Find(InstanceFunctionId);

		if (MatchingParentFunctionNodePtr == nullptr && MatchingParentAtLastMergeFunctionNodePtr == nullptr)
		{
			// If neither the current parent or parent at last merge had a function node with this id, force the current change id of the node
			// since it only exists in the instance.
			OutFunctionIdToForcedChangeId.Add(InstanceFunctionId, InstanceFunctionNode->GetChangeId());
		}
		else if (MatchingParentFunctionNodePtr != nullptr && MatchingParentAtLastMergeFunctionNodePtr != nullptr)
		{
			if ((*MatchingParentFunctionNodePtr)->GetChangeId() == (*MatchingParentAtLastMergeFunctionNodePtr)->GetChangeId())
			{
				// If both the parent and parent at last merge agree on the change id, then the most recent changes will have happened in
				// the instance so we can force the instance change id.
				OutFunctionIdToForcedChangeId.Add(InstanceFunctionId, InstanceFunctionNode->GetChangeId());
			}
			else
			{
				if (InstanceFunctionNode->GetChangeId() == (*MatchingParentAtLastMergeFunctionNodePtr)->GetChangeId())
				{
					// If the parent and the parent at last merge have different change ids, and the instances change id matches
					// the parent at last merge change id, then the parent has changed, and the instance has not, so we can
					// force the id from the parent.
					OutFunctionIdToForcedChangeId.Add(InstanceFunctionId, (*MatchingParentFunctionNodePtr)->GetChangeId());
				}
			}
		}
		else if (MatchingParentAtLastMergeFunctionNodePtr != nullptr)
		{
			// If the parent did not have this function id but the parent at the last merge did have the id, it was deleted in the parent
			// so only the instance id is still relevant.
			OutFunctionIdToForcedChangeId.Add(InstanceFunctionId, InstanceFunctionNode->GetChangeId());
		}
		else
		{
			ensureMsgf(MatchingParentFunctionNodePtr == nullptr,
				TEXT("Error while merging changes for emitter instance function node: %s.  A node with this change id was not found in the last merge parent, but it was found on parent node: %s"),
				*InstanceFunctionNode->GetPathName(), *(*MatchingParentFunctionNodePtr)->GetPathName());
		}
	}
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::ForceInstanceChangeIds(TSharedRef<FNiagaraEmitterMergeAdapter> MergedInstanceAdapter, const FVersionedNiagaraEmitter& OriginalEmitterInstance, const TMap<FGuid, FGuid>& FunctionIdToForcedChangedId) const
{
	FApplyDiffResults DiffResults;

	if (FunctionIdToForcedChangedId.Num() != 0)
	{
		auto It = FunctionIdToForcedChangedId.CreateConstIterator();
		FVersionedNiagaraEmitter Emitter = MergedInstanceAdapter->GetEditableEmitter();

		TArray<UNiagaraGraph*> Graphs;
		TArray<UNiagaraScript*> Scripts;
		Emitter.GetEmitterData()->GetScripts(Scripts);

		TArray<UNiagaraScript*> OriginalScripts;
		OriginalEmitterInstance.GetEmitterData()->GetScripts(OriginalScripts);

		// First gather all the graphs used by this emitter..
		for (UNiagaraScript* Script : Scripts)
		{
			if (Script != nullptr && Script->GetLatestSource() != nullptr)
			{
				UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(Script->GetLatestSource());
				if (ScriptSource != nullptr)
				{
					Graphs.AddUnique(ScriptSource->NodeGraph);
				}
			}
		}

		// Now gather up all the nodes
		TArray<UNiagaraNode*> Nodes;
		for (UNiagaraGraph* Graph : Graphs)
		{
			Graph->GetNodesOfClass(Nodes);
		}

		// Now go through all the nodes and set the persistent change ids if we encounter a node that needs it's change id kept.
		bool bAnySet = false;
		while (It)
		{
			for (UNiagaraNode* Node : Nodes)
			{
				if (Node->NodeGuid == It.Key())
				{
					Node->ForceChangeId(It.Value(), false);
					bAnySet = true;
					break;
				}
			}
			++It;
		}

		if (bAnySet)
		{
			for (UNiagaraGraph* Graph : Graphs)
			{
				Graph->MarkGraphRequiresSynchronization(TEXT("Overwrote change id's within graph."));
			}
		}

		DiffResults.bModifiedGraph = bAnySet;

		if (bAnySet)
		{
			bool bAnyUpdated = false;
			TMap<FString, FString> RenameMap;
			RenameMap.Add(TEXT("Emitter"), TEXT("Emitter"));
			for (UNiagaraScript* Script : Scripts)
			{
				for (UNiagaraScript* OriginalScript : OriginalScripts)
				{
					if (Script->Usage == OriginalScript->Usage && Script->GetUsageId() == OriginalScript->GetUsageId())
					{
						bAnyUpdated |= Script->SynchronizeExecutablesWithCompilation(OriginalScript, RenameMap);
					}
				}
			}

			if (bAnyUpdated)
			{
				//Emitter->OnPostCompile();
			}
		}
	}

	DiffResults.bSucceeded = true;
	return DiffResults;
}

void FNiagaraScriptMergeManager::UpdateModuleVersions(const FVersionedNiagaraEmitter& Instance, const FNiagaraEmitterDiffResults& DiffResults) const
{
	// gather all the changes
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> ChangedVersionBaseModules;
	ChangedVersionBaseModules.Append(DiffResults.EmitterSpawnDiffResults.ChangedVersionBaseModules);
	ChangedVersionBaseModules.Append(DiffResults.EmitterUpdateDiffResults.ChangedVersionBaseModules);
	ChangedVersionBaseModules.Append(DiffResults.ParticleSpawnDiffResults.ChangedVersionBaseModules);
	ChangedVersionBaseModules.Append(DiffResults.ParticleUpdateDiffResults.ChangedVersionBaseModules);
	for (auto& EventHandlerDiff : DiffResults.ModifiedEventHandlers)
	{
		ChangedVersionBaseModules.Append(EventHandlerDiff.ScriptDiffResults.ChangedVersionBaseModules);
	}
	for (auto& SimStageDiff : DiffResults.ModifiedSimulationStages)
	{
		ChangedVersionBaseModules.Append(SimStageDiff.ScriptDiffResults.ChangedVersionBaseModules);
	}

	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> ChangedVersionOtherModules;
	ChangedVersionOtherModules.Append(DiffResults.EmitterSpawnDiffResults.ChangedVersionOtherModules);
	ChangedVersionOtherModules.Append(DiffResults.EmitterUpdateDiffResults.ChangedVersionOtherModules);
	ChangedVersionOtherModules.Append(DiffResults.ParticleSpawnDiffResults.ChangedVersionOtherModules);
	ChangedVersionOtherModules.Append(DiffResults.ParticleUpdateDiffResults.ChangedVersionOtherModules);
	for (auto& EventHandlerDiff : DiffResults.ModifiedEventHandlers)
	{
		ChangedVersionOtherModules.Append(EventHandlerDiff.ScriptDiffResults.ChangedVersionOtherModules);
	}
	for (auto& SimStageDiff : DiffResults.ModifiedSimulationStages)
	{
		ChangedVersionOtherModules.Append(SimStageDiff.ScriptDiffResults.ChangedVersionOtherModules);
	}
	
	// the python update script needs a view model, so we need to create a temporary system here to instantiate the view models
	UNiagaraSystem* System = nullptr;
	if (ChangedVersionOtherModules.Num() > 0)
	{
		TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
		
		System = NewObject<UNiagaraSystem>(GetTransientPackage(), NAME_None, RF_Transient);
		UNiagaraSystemFactoryNew::InitializeSystem(System, true);
		SystemViewModel = MakeShared<FNiagaraSystemViewModel>();
		FNiagaraSystemViewModelOptions SystemOptions;
		SystemOptions.bCanModifyEmittersFromTimeline = false;
		SystemOptions.bCanSimulate = false;
		SystemOptions.bCanAutoCompile = false;
		SystemOptions.bIsForDataProcessingOnly = true;
		SystemOptions.EditMode = ENiagaraSystemViewModelEditMode::EmitterDuringMerge;
		SystemOptions.MessageLogGuid = System->GetAssetGuid();
		SystemViewModel->Initialize(*System, SystemOptions);
		SystemViewModel->GetEditorData().SetOwningSystemIsPlaceholder(true, *System);
		SystemViewModel->AddEmitter(Instance);

		// apply version changes
		for (int i = 0; i < ChangedVersionOtherModules.Num(); i++)
		{
			TSharedRef<FNiagaraStackFunctionMergeAdapter> BaseModule = ChangedVersionBaseModules[i];
			TSharedRef<FNiagaraStackFunctionMergeAdapter> ChangedModule = ChangedVersionOtherModules[i];
			FGuid NewScriptVersion = BaseModule->GetFunctionCallNode()->SelectedScriptVersion;

			// search for the stack item of the module node we are merging
			UNiagaraStackModuleItem* ModuleItem = nullptr;
			UNiagaraStackViewModel* EmitterStackViewModel = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterStackViewModel();
			TArray<UNiagaraStackEntry*> EntriesToCheck;
			EmitterStackViewModel->GetRootEntry()->GetUnfilteredChildren(EntriesToCheck);
			while (EntriesToCheck.Num() > 0)
			{
				UNiagaraStackEntry* Entry = EntriesToCheck.Pop();
				UNiagaraStackModuleItem* ModuleItemToCheck = Cast<UNiagaraStackModuleItem>(Entry);
				if (ModuleItemToCheck && ModuleItemToCheck->GetModuleNode().NodeGuid == ChangedModule->GetFunctionCallNode()->NodeGuid)
				{
					ModuleItem = ModuleItemToCheck;
					break;
				}
				Entry->GetUnfilteredChildren(EntriesToCheck);
			}
		
			FNiagaraScriptVersionUpgradeContext UpgradeContext;
			UpgradeContext.ConstantResolver = FCompileConstantResolver(Instance, FNiagaraStackGraphUtilities::GetOutputNodeUsage(*ChangedModule->GetFunctionCallNode()));
			if (ModuleItem)
			{
				UpgradeContext.CreateClipboardCallback = [ModuleItem](UNiagaraClipboardContent* ClipboardContent)
				{
					ModuleItem->RefreshChildren();
					ModuleItem->Copy(ClipboardContent);
					if (ClipboardContent->Functions.Num() > 0)
					{
						ClipboardContent->FunctionInputs = ClipboardContent->Functions[0]->Inputs;
						ClipboardContent->Functions.Empty();
					}
				};
				UpgradeContext.ApplyClipboardCallback = [ModuleItem](UNiagaraClipboardContent* ClipboardContent, FText& OutWarning) { ModuleItem->Paste(ClipboardContent, OutWarning); };
			}
			ChangedModule->GetFunctionCallNode()->ChangeScriptVersion(NewScriptVersion, UpgradeContext, true /*bShowNotesInStack*/, true /*bDeferOverridePinUpdate*/);
			ChangedModule->GetFunctionCallNode()->RefreshFromExternalChanges();
		}

		// now that we've applied the changes to all of the changed modules we go back through them and ensure their override nodes are up to date
		// this is done after the fact to avoid constantly regenerating the static variables that may be at play
		for (TSharedRef<FNiagaraStackFunctionMergeAdapter>& ChangedModule : ChangedVersionOtherModules)
		{
			FNiagaraScriptVersionUpgradeContext UpgradeContext;
			UpgradeContext.ConstantResolver = FCompileConstantResolver(Instance, FNiagaraStackGraphUtilities::GetOutputNodeUsage(*ChangedModule->GetFunctionCallNode()));

			ChangedModule->GetFunctionCallNode()->UpdateOverridePins(UpgradeContext);
		}
	}

	// now that we're done with our transient System we need to reset it so that if it does get pulled in (via TObjectIterator as an example)
	// it won't conflict with legitimate systems holding onto the supplied emitter
	if (System)
	{
		System->ResetToEmptySystem();
		System->MarkAsGarbage();
	}
}

enum class EmitterMergeState
{
	Failed, Unchanged, DuplicateParent, CopyProperties
};

FNiagaraScriptMergeManager::EmitterMergeState FNiagaraScriptMergeManager::GetEmitterMergeState(const FNiagaraEmitterDiffResults& DiffResults, const FVersionedNiagaraEmitter& Parent, const FVersionedNiagaraEmitter& Instance) const
{
	EmitterMergeState State = EmitterMergeState::CopyProperties;
	if (DiffResults.IsValid() == false)
	{
		State = EmitterMergeState::Failed;
	}
	else if (DiffResults.IsEmpty())
	{
		// If there were no changes made on the instance, check if the instance matches the parent.
		FNiagaraEmitterDiffResults DiffResultsFromParent = DiffEmitters(Parent, Instance);
		if (DiffResultsFromParent.IsValid() && DiffResultsFromParent.IsEmpty())
		{
			State = EmitterMergeState::Unchanged;
		}
		else if (Instance.Emitter->IsVersioningEnabled())
		{
			// if this emitter is versioned we cannot just duplicate the parent, as we would lose version data
			State = EmitterMergeState::CopyProperties;
		}
		else
		{
			State = EmitterMergeState::DuplicateParent;
		}
	}
	return State;
}

INiagaraMergeManager::FMergeEmitterResults FNiagaraScriptMergeManager::MergeEmitter(const FVersionedNiagaraEmitter& Parent, const FVersionedNiagaraEmitter& ParentAtLastMerge, const FVersionedNiagaraEmitter& Instance) const
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_ScriptMergeManager_MergeEmitter);
	FMergeEmitterResults MergeResults;
	const bool bNoParentAtLastMerge = (ParentAtLastMerge.Emitter == nullptr);
	FVersionedNiagaraEmitter FirstEmitterToDiffAgainst = bNoParentAtLastMerge ? Parent : ParentAtLastMerge;
	FNiagaraEmitterDiffResults DiffResults = DiffEmitters(FirstEmitterToDiffAgainst, Instance);

	// depending on the diff result and versioning state of the emitters, we decide what to do next
	EmitterMergeState State = GetEmitterMergeState(DiffResults, Parent, Instance);

	if (State == EmitterMergeState::Unchanged)
	{
		MergeResults.MergeResult = EMergeEmitterResult::SucceededNoDifferences;
	}
	else if (State == EmitterMergeState::DuplicateParent)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EmitterMergeState::DuplicateParent);

		// If there were differences from the parent or the parent diff failed we can just return a copy of the parent as the merged instance since there
		// were no changes in the instance which need to be applied.
		MergeResults.MergeResult = EMergeEmitterResult::SucceededDifferencesApplied;
		MergeResults.MergedInstance = Parent.Emitter->DuplicateWithoutMerging(GetTransientPackage());
		MergeResults.MergedInstance->DisableVersioning(Parent.Version);
		FVersionedNiagaraEmitterData* MergedEmitterData = MergeResults.MergedInstance->GetLatestEmitterData();
		MergedEmitterData->ParentScratchPads->AppendScripts(MergedEmitterData->ScratchPads);
	}
	else if (State == EmitterMergeState::Failed)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EmitterMergeState::Failed);

		MergeResults.MergeResult = EMergeEmitterResult::FailedToDiff;
		MergeResults.ErrorMessages = DiffResults.GetErrorMessages();

		auto ReportScriptStackDiffErrors = [](FMergeEmitterResults& EmitterMergeResults, const FNiagaraScriptStackDiffResults& ScriptStackDiffResults, FText ScriptName)
		{
			FText ScriptStackDiffInvalidFormat = LOCTEXT("ScriptStackDiffInvalidFormat", "Failed to diff {0} script stack.  {1} Errors:");
			if (ScriptStackDiffResults.IsValid() == false)
			{
				EmitterMergeResults.ErrorMessages.Add(FText::Format(ScriptStackDiffInvalidFormat, ScriptName, ScriptStackDiffResults.GetErrorMessages().Num()));
				for (const FText& ErrorMessage : ScriptStackDiffResults.GetErrorMessages())
				{
					EmitterMergeResults.ErrorMessages.Add(ErrorMessage);
				}
			}
		};

		ReportScriptStackDiffErrors(MergeResults, DiffResults.EmitterSpawnDiffResults, LOCTEXT("EmitterSpawnScriptName", "Emitter Spawn"));
		ReportScriptStackDiffErrors(MergeResults, DiffResults.EmitterUpdateDiffResults, LOCTEXT("EmitterUpdateScriptName", "Emitter Update"));
		ReportScriptStackDiffErrors(MergeResults, DiffResults.ParticleSpawnDiffResults, LOCTEXT("ParticleSpawnScriptName", "Particle Spawn"));
		ReportScriptStackDiffErrors(MergeResults, DiffResults.ParticleUpdateDiffResults, LOCTEXT("ParticleUpdateScriptName", "Particle Update"));

		for (const FNiagaraModifiedEventHandlerDiffResults& EventHandlerDiffResults : DiffResults.ModifiedEventHandlers)
		{
			FText EventHandlerName = FText::Format(LOCTEXT("EventHandlerScriptNameFormat", "Event Handler - {0}"), FText::FromName(EventHandlerDiffResults.BaseAdapter->GetEventScriptProperties()->SourceEventName));
			ReportScriptStackDiffErrors(MergeResults, EventHandlerDiffResults.ScriptDiffResults, EventHandlerName);
		}
	}
	else if (State == EmitterMergeState::CopyProperties) //-V547
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EmitterMergeState::CopyProperties);

		UNiagaraEmitter* MergedInstance = Parent.Emitter->DuplicateWithoutMerging(GetTransientPackage());
		MergedInstance->DisableVersioning(Parent.Version);
		FVersionedNiagaraEmitter VersionedMergedInstance = FVersionedNiagaraEmitter(MergedInstance, Parent.Version);
		TSharedRef<FNiagaraEmitterMergeAdapter> MergedInstanceAdapter = MakeShared<FNiagaraEmitterMergeAdapter>(VersionedMergedInstance);

		// diff for version changes and apply them first. We need to upgrade both the current emitter and the parent at last merge
		// to the newest version and then diff them again. Otherwise we won't know which changes were made by the version upgrade
		// and which changes were made to the parent emitter.
		FNiagaraEmitterDiffResults VersionChangeDiffResults = DiffEmitters(Parent, Instance);
		if (VersionChangeDiffResults.HasVersionChanges())
		{
			// apply version upgrade to the merged emitter instance
			UpdateModuleVersions(Instance, VersionChangeDiffResults);

			// apply version upgrade to the parent at last merge
			UNiagaraEmitter* ParentAtLastMergeCopy = Cast<UNiagaraEmitter>(StaticDuplicateObject(FirstEmitterToDiffAgainst.Emitter, GetTransientPackage()));
			ParentAtLastMergeCopy->ClearFlags(RF_Standalone | RF_Public);
			FVersionedNiagaraEmitter VersionedParentCopy = FVersionedNiagaraEmitter(ParentAtLastMergeCopy, FirstEmitterToDiffAgainst.Version);
			FNiagaraEmitterDiffResults ParentsDiffResults = DiffEmitters(Parent, VersionedParentCopy);
			UpdateModuleVersions(VersionedParentCopy, ParentsDiffResults);

			// now diff again
			DiffResults = DiffEmitters(VersionedParentCopy, Instance);
			ensureMsgf(DiffResults.HasVersionChanges() == false, TEXT("Emitter still has pending version changes after merge! Asset %s"), *Instance.Emitter->GetPathName());
		}

		TMap<FGuid, UNiagaraNodeFunctionCall*> ParentFunctionIdToNodeMap;
		TMap<FGuid, UNiagaraNodeFunctionCall*> LastMergedParentFunctionIdToNodeMap;
		TMap<FGuid, UNiagaraNodeFunctionCall*> InstanceFunctionIdToNodeMap;

		auto PopulateIdToNodeMap = [](TSharedRef<FNiagaraEmitterMergeAdapter> EmitterAdapter, TMap<FGuid, UNiagaraNodeFunctionCall*>& Map)
		{
			TArray<UNiagaraNodeFunctionCall*> EmitterFunctionCalls;
			EmitterAdapter->GatherFunctionCallNodes(EmitterFunctionCalls);
			for (UNiagaraNodeFunctionCall* EmitterFunctionCall : EmitterFunctionCalls)
			{
				Map.Add(EmitterFunctionCall->NodeGuid, EmitterFunctionCall);
			}
		};


		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Chunk1);
			PopulateIdToNodeMap(MakeShared<FNiagaraEmitterMergeAdapter>(Parent), ParentFunctionIdToNodeMap);
			PopulateIdToNodeMap(DiffResults.BaseEmitterAdapter.ToSharedRef(), LastMergedParentFunctionIdToNodeMap);
			PopulateIdToNodeMap(DiffResults.OtherEmitterAdapter.ToSharedRef(), InstanceFunctionIdToNodeMap);
		}

		TMap<FGuid, FGuid> FunctionIdToForcedChangeId;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Chunk2);
			GetForcedChangeIds(ParentFunctionIdToNodeMap, LastMergedParentFunctionIdToNodeMap, InstanceFunctionIdToNodeMap, FunctionIdToForcedChangeId);
		}

		FVersionedNiagaraEmitterData* MergedEmitterData = VersionedMergedInstance.GetEmitterData();
		MergedEmitterData->ParentScratchPads->AppendScripts(MergedEmitterData->ScratchPads);
		CopyInstanceScratchPadScripts(VersionedMergedInstance, Instance.GetEmitterData());
		TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter = MakeShared<FNiagaraScratchPadMergeAdapter>(VersionedMergedInstance, Instance, Parent);

		MergeResults.MergeResult = EMergeEmitterResult::SucceededDifferencesApplied;
		FApplyDiffResults EmitterSpawnResults = ApplyScriptStackDiff(MergedInstanceAdapter, MergedInstanceAdapter->GetEmitterSpawnStack().ToSharedRef(), ScratchPadAdapter, DiffResults.EmitterSpawnDiffResults, bNoParentAtLastMerge);
		if (EmitterSpawnResults.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= EmitterSpawnResults.bModifiedGraph;
		MergeResults.ErrorMessages.Append(EmitterSpawnResults.ErrorMessages);

		FApplyDiffResults EmitterUpdateResults = ApplyScriptStackDiff(MergedInstanceAdapter, MergedInstanceAdapter->GetEmitterUpdateStack().ToSharedRef(), ScratchPadAdapter, DiffResults.EmitterUpdateDiffResults, bNoParentAtLastMerge);
		if (EmitterUpdateResults.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= EmitterUpdateResults.bModifiedGraph;
		MergeResults.ErrorMessages.Append(EmitterUpdateResults.ErrorMessages);

		FApplyDiffResults ParticleSpawnResults = ApplyScriptStackDiff(MergedInstanceAdapter, MergedInstanceAdapter->GetParticleSpawnStack().ToSharedRef(), ScratchPadAdapter, DiffResults.ParticleSpawnDiffResults, bNoParentAtLastMerge);
		if (ParticleSpawnResults.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= ParticleSpawnResults.bModifiedGraph;
		MergeResults.ErrorMessages.Append(ParticleSpawnResults.ErrorMessages);

		FApplyDiffResults ParticleUpdateResults = ApplyScriptStackDiff(MergedInstanceAdapter, MergedInstanceAdapter->GetParticleUpdateStack().ToSharedRef(), ScratchPadAdapter, DiffResults.ParticleUpdateDiffResults, bNoParentAtLastMerge);
		if (ParticleUpdateResults.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= ParticleUpdateResults.bModifiedGraph;
		MergeResults.ErrorMessages.Append(ParticleUpdateResults.ErrorMessages);

		FApplyDiffResults EventHandlerResults = ApplyEventHandlerDiff(MergedInstanceAdapter, ScratchPadAdapter, DiffResults, bNoParentAtLastMerge);
		if (EventHandlerResults.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= EventHandlerResults.bModifiedGraph;
		MergeResults.ErrorMessages.Append(EventHandlerResults.ErrorMessages);

		FApplyDiffResults SimulationStageResults = ApplySimulationStageDiff(MergedInstanceAdapter, ScratchPadAdapter, DiffResults, bNoParentAtLastMerge);
		if (SimulationStageResults.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= SimulationStageResults.bModifiedGraph;
		MergeResults.ErrorMessages.Append(SimulationStageResults.ErrorMessages);

		FApplyDiffResults RendererResults = ApplyRendererDiff(VersionedMergedInstance, DiffResults, bNoParentAtLastMerge);
		if (RendererResults.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= RendererResults.bModifiedGraph;
		MergeResults.ErrorMessages.Append(RendererResults.ErrorMessages);

		FApplyDiffResults InputSummaryResults = ApplyEmitterSummaryDiff(VersionedMergedInstance, DiffResults);
		if (InputSummaryResults.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= InputSummaryResults.bModifiedGraph;
		MergeResults.ErrorMessages.Append(InputSummaryResults.ErrorMessages);

		CopyPropertiesToBase(VersionedMergedInstance.GetEmitterData(), Instance.GetEmitterData(), DiffResults.DifferentEmitterProperties);

		// Make sure we resync to interpolated spawning usage as it could have been adjusted previously and it needs to reflect the emitter not the parent
		if (FVersionedNiagaraEmitterData* MergedDataForUsageSync = VersionedMergedInstance.GetEmitterData())
		{
			if (UNiagaraScript* MergedSpawnScript = MergedDataForUsageSync->SpawnScriptProps.Script)
			{
				const ENiagaraScriptUsage RequiredUsage = MergedDataForUsageSync->UsesInterpolatedSpawning() ? ENiagaraScriptUsage::ParticleSpawnScriptInterpolated : ENiagaraScriptUsage::ParticleSpawnScript;
				if (MergedSpawnScript->GetUsage() != RequiredUsage)
				{
					MergedSpawnScript->Modify();
					MergedSpawnScript->SetUsage(RequiredUsage);
					if (MergedDataForUsageSync->GraphSource)
					{
						MergedDataForUsageSync->GraphSource->MarkNotSynchronized(TEXT("Resync SpawnScript Usage with InterpolatedSpawnMode after merge"));
					}
				}
			}
		}

		FApplyDiffResults StackEntryDisplayNameDiffs = ApplyStackEntryDisplayNameDiffs(VersionedMergedInstance, DiffResults);
		if (StackEntryDisplayNameDiffs.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= StackEntryDisplayNameDiffs.bModifiedGraph;
		MergeResults.ErrorMessages.Append(StackEntryDisplayNameDiffs.ErrorMessages);

		FApplyDiffResults StackNotesDiffs = ApplyStackNoteDiffs(VersionedMergedInstance, DiffResults);
		if (StackNotesDiffs.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= StackNotesDiffs.bModifiedGraph;
		MergeResults.ErrorMessages.Append(StackNotesDiffs.ErrorMessages);

#if 0
		UE_LOGF(LogNiagaraEditor, Log, "A");
		//for (FNiagaraEmitterHandle& EmitterHandle : EmitterHandles)
		{
			//UE_LOGF(LogNiagara, Log, "Emitter Handle: %ls", *EmitterHandle.GetUniqueInstanceName());
			UNiagaraScript* UpdateScript = MergedInstance->GetScript(ENiagaraScriptUsage::ParticleUpdateScript, FGuid());
			UNiagaraScript* SpawnScript = MergedInstance->GetScript(ENiagaraScriptUsage::ParticleSpawnScript, FGuid());
			UE_LOGF(LogNiagaraEditor, Log, "Spawn Parameters");
			SpawnScript->GetVMExecutableData().Parameters.DumpParameters();
			UE_LOGF(LogNiagaraEditor, Log, "Spawn RI Parameters");
			SpawnScript->RapidIterationParameters.DumpParameters();
			UE_LOGF(LogNiagaraEditor, Log, "Update Parameters");
			UpdateScript->GetVMExecutableData().Parameters.DumpParameters();
			UE_LOGF(LogNiagaraEditor, Log, "Update RI Parameters");
			UpdateScript->RapidIterationParameters.DumpParameters();
		}
#endif
		
		FApplyDiffResults ChangeIdResults = ForceInstanceChangeIds(MergedInstanceAdapter, Instance, FunctionIdToForcedChangeId);
		if (ChangeIdResults.bSucceeded == false)
		{
			MergeResults.MergeResult = EMergeEmitterResult::FailedToMerge;
		}
		MergeResults.bModifiedGraph |= ChangeIdResults.bModifiedGraph;
		MergeResults.ErrorMessages.Append(ChangeIdResults.ErrorMessages);
		
#if 0
		UE_LOGF(LogNiagaraEditor, Log, "B");
		//for (FNiagaraEmitterHandle& EmitterHandle : EmitterHandles)
		{
			//UE_LOGF(LogNiagara, Log, "Emitter Handle: %ls", *EmitterHandle.GetUniqueInstanceName());
			UNiagaraScript* UpdateScript = MergedInstance->GetScript(ENiagaraScriptUsage::ParticleUpdateScript, FGuid());
			UNiagaraScript* SpawnScript = MergedInstance->GetScript(ENiagaraScriptUsage::ParticleSpawnScript, FGuid());
			UE_LOGF(LogNiagaraEditor, Log, "Spawn Parameters");
			SpawnScript->GetVMExecutableData().Parameters.DumpParameters();
			UE_LOGF(LogNiagaraEditor, Log, "Spawn RI Parameters");
			SpawnScript->RapidIterationParameters.DumpParameters();
			UE_LOGF(LogNiagaraEditor, Log, "Update Parameters");
			UpdateScript->GetVMExecutableData().Parameters.DumpParameters();
			UE_LOGF(LogNiagaraEditor, Log, "Update RI Parameters");
			UpdateScript->RapidIterationParameters.DumpParameters();
		}
#endif

		FNiagaraStackGraphUtilities::CleanUpStaleRapidIterationParameters(VersionedMergedInstance);

#if 0
		UE_LOGF(LogNiagaraEditor, Log, "C");
		//for (FNiagaraEmitterHandle& EmitterHandle : EmitterHandles)
		{
			//UE_LOGF(LogNiagara, Log, "Emitter Handle: %ls", *EmitterHandle.GetUniqueInstanceName());
			UNiagaraScript* UpdateScript = MergedInstance->GetScript(ENiagaraScriptUsage::ParticleUpdateScript, FGuid());
			UNiagaraScript* SpawnScript = MergedInstance->GetScript(ENiagaraScriptUsage::ParticleSpawnScript, FGuid());
			UE_LOGF(LogNiagaraEditor, Log, "Spawn Parameters");
			SpawnScript->GetVMExecutableData().Parameters.DumpParameters();
			UE_LOGF(LogNiagaraEditor, Log, "Spawn RI Parameters");
			SpawnScript->RapidIterationParameters.DumpParameters();
			UE_LOGF(LogNiagaraEditor, Log, "Update Parameters");
			UpdateScript->GetVMExecutableData().Parameters.DumpParameters();
			UE_LOGF(LogNiagaraEditor, Log, "Update RI Parameters");
			UpdateScript->RapidIterationParameters.DumpParameters();
		}
#endif
		if (MergeResults.MergeResult == EMergeEmitterResult::SucceededDifferencesApplied)
		{
			UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(MergedEmitterData->GraphSource);
			FNiagaraStackGraphUtilities::RelayoutGraph(*ScriptSource->NodeGraph);
			MergeResults.MergedInstance = MergedInstance;
		}

		TMap<FGuid, FGuid> FinalChangeIds;
		FNiagaraEditorUtilities::GatherChangeIds(*MergedInstance, FinalChangeIds, TEXT("Final"));
	}
	else
	{
		checkf(false, TEXT("Unknown merge state!"));
	}

	if (MergeResults.MergeResult != INiagaraMergeManager::EMergeEmitterResult::SucceededNoDifferences && MergeResults.MergeResult != INiagaraMergeManager::EMergeEmitterResult::SucceededDifferencesApplied)
	{
		TArray<FText> Errors;
		Errors.Add(FText::Format(LOCTEXT("MergeFailedFormat", "Failed to merge changes from parent emitter {0}"), FText::FromString(Parent.Emitter->GetName())));
		Errors.Append(MergeResults.ErrorMessages);

		UNiagaraMessageDataText* Message = NewObject<UNiagaraMessageDataText>(GetTransientPackage());
		Message->Init(
			FText::Join(FText::FromString("\n"), Errors),
			LOCTEXT("MergeFailedFormatShort", "Failed to merge changes from parent emitter"),
			ENiagaraMessageSeverity::Error, "Emitter Merge");
		Message->SetAllowDismissal(false);
		MergeResults.MergeNiagaraMessage = Message;
	}

	return MergeResults;
}

bool FNiagaraScriptMergeManager::IsMergeableScriptUsage(ENiagaraScriptUsage ScriptUsage) const
{
	return ScriptUsage == ENiagaraScriptUsage::EmitterSpawnScript ||
		ScriptUsage == ENiagaraScriptUsage::EmitterUpdateScript ||
		ScriptUsage == ENiagaraScriptUsage::ParticleSpawnScript ||
		ScriptUsage == ENiagaraScriptUsage::ParticleUpdateScript ||
		ScriptUsage == ENiagaraScriptUsage::ParticleEventScript ||
		ScriptUsage == ENiagaraScriptUsage::ParticleSimulationStageScript;
}

bool FNiagaraScriptMergeManager::HasBaseModule(const FVersionedNiagaraEmitter& BaseEmitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FGuid ModuleId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);
	TSharedPtr<FNiagaraScriptStackMergeAdapter> BaseScriptStackAdapter = BaseEmitterAdapter->GetScriptStack(ScriptUsage, ScriptUsageId);
	return BaseScriptStackAdapter.IsValid() && BaseScriptStackAdapter->GetModuleFunctionById(ModuleId).IsValid();
}

bool FNiagaraScriptMergeManager::FindBaseModule(const FVersionedNiagaraEmitter& BaseEmitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FGuid ModuleId, class UNiagaraScript*& OutActualScript, FGuid& OutScriptVersionGuid, FVersionedNiagaraEmitter& OutBaseEmitter)
{
	// Search through the existing scratch pads local to this versioned emitter and the ones inherited from parents.
	OutScriptVersionGuid = FGuid();
	OutActualScript = nullptr;
	OutBaseEmitter = FVersionedNiagaraEmitter();

	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);
	TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadMergeAdapter = GetScratchPadMergeAdapterUsingCache(BaseEmitter);
	TSharedPtr<FNiagaraScriptStackMergeAdapter> BaseScriptStackAdapter = BaseEmitterAdapter->GetScriptStack(ScriptUsage, ScriptUsageId);
	if (BaseScriptStackAdapter.IsValid())
	{
		TSharedPtr<FNiagaraStackFunctionMergeAdapter> AdapterFound = BaseScriptStackAdapter->GetModuleFunctionById(ModuleId);
		if (AdapterFound.IsValid())
		{
			UNiagaraNodeFunctionCall* CallNode = AdapterFound->GetFunctionCallNode();
			if (CallNode)
			{
				UNiagaraScript* Script = CallNode->FunctionScript;
				if (Script)
				{
					int32 IndexToMatch = INDEX_NONE;
					bool bParents = false;
					FVersionedNiagaraEmitterData* EmitterData = BaseEmitter.Emitter->GetEmitterData(BaseEmitter.Version);
					if (EmitterData)
					{
						if (EmitterData->ScratchPads)
						{
							IndexToMatch = EmitterData->ScratchPads->FindIndexForScript(Script);
						}
						if (INDEX_NONE == IndexToMatch && EmitterData->ParentScratchPads)
						{
							IndexToMatch = EmitterData->ParentScratchPads->FindIndexForScript(Script);
							bParents = true;
						}
					}

					if (bParents && IndexToMatch != INDEX_NONE)
					{
						TArray< FNiagaraScratchPadMergeAdapter::FMergeRecord > MergeRecordsInOrder = ScratchPadMergeAdapter->GetMergedEmitterRecords();
						if (MergeRecordsInOrder.IsValidIndex(IndexToMatch))
						{
							OutScriptVersionGuid = MergeRecordsInOrder[IndexToMatch].OriginalScriptVersion;
							OutActualScript = MergeRecordsInOrder[IndexToMatch].OriginalScript;
							OutBaseEmitter = MergeRecordsInOrder[IndexToMatch].OriginalEmitter;
							return true;
						}

					}
					else if (!bParents && IndexToMatch != INDEX_NONE)
					{
						OutScriptVersionGuid = CallNode->SelectedScriptVersion;
						OutActualScript = Script;
						OutBaseEmitter = BaseEmitter;
						return true;
					}

					
				}
				return false;
			}
		}
	}

	return false;
}

bool FNiagaraScriptMergeManager::IsModuleInputDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FGuid ModuleId, FNiagaraVariableBase Variable)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_ScriptMergeManager_IsModuleInputDifferentFromBase);

	TSharedRef<FNiagaraEmitterMergeAdapter> EmitterAdapter = GetEmitterMergeAdapterUsingCache(Emitter);
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);

	TSharedRef<FNiagaraScriptStackMergeAdapter> ScriptStackAdapter = EmitterAdapter->GetScriptStack(ScriptUsage, ScriptUsageId).ToSharedRef();
	TSharedPtr<FNiagaraScriptStackMergeAdapter> BaseScriptStackAdapter = BaseEmitterAdapter->GetScriptStack(ScriptUsage, ScriptUsageId);

	if (BaseScriptStackAdapter.IsValid() == false)
	{
		return false;
	}

	FNiagaraScriptStackDiffResults ScriptStackDiffResults;
	DiffScriptStacks(BaseScriptStackAdapter.ToSharedRef(), ScriptStackAdapter, ScriptStackDiffResults);

	if (ScriptStackDiffResults.IsValid() == false)
	{
		return true;
	}

	if (ScriptStackDiffResults.IsEmpty())
	{
		return false;
	}

	auto FindInputOverrideByInputName = [=](TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> InputOverride)
	{
		FNiagaraVariableBase CandidateVariable(InputOverride->GetType(), FName(InputOverride->GetInputName()));
		return InputOverride->GetOwningFunctionCall()->NodeGuid == ModuleId && CandidateVariable == Variable;
	};

	return
		ScriptStackDiffResults.RemovedBaseInputOverrides.FindByPredicate(FindInputOverrideByInputName) != nullptr ||
		ScriptStackDiffResults.AddedOtherInputOverrides.FindByPredicate(FindInputOverrideByInputName) != nullptr ||
		ScriptStackDiffResults.ModifiedOtherInputOverrides.FindByPredicate(FindInputOverrideByInputName) != nullptr;
}

bool FNiagaraScriptMergeManager::DoesSummaryItemExistInBase(const FVersionedNiagaraEmitter& Emitter, FHierarchyElementIdentity Identity)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_ScriptMergeManager_DoesSummaryItemExistInBase);

	FVersionedNiagaraEmitter BaseEmitter = Emitter.GetEmitterData()->GetParent();
	if(BaseEmitter.GetEmitterData() != nullptr)
	{
		TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);
		return BaseEmitterAdapter->GetEditorData()->GetSummaryRoot()->FindChildWithIdentity(Identity, true) != nullptr;
	}

	return false;
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::ResetModuleInputToBase(const FVersionedNiagaraEmitter& VersionedEmitter, const FVersionedNiagaraEmitter& VersionedBaseEmitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FGuid ModuleId, FString InputName)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> EmitterAdapter = GetEmitterMergeAdapterUsingCache(VersionedEmitter);
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(VersionedBaseEmitter);
	TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadMergeAdapter = GetScratchPadMergeAdapterUsingCache(VersionedEmitter);

	// Diff from the emitter to the base to create a diff which will reset the emitter back to the base.
	FNiagaraScriptStackDiffResults ResetDiffResults;
	DiffScriptStacks(EmitterAdapter->GetScriptStack(ScriptUsage, ScriptUsageId).ToSharedRef(), BaseEmitterAdapter->GetScriptStack(ScriptUsage, ScriptUsageId).ToSharedRef(), ResetDiffResults);

	FText EmitterPath = FText::FromString(VersionedEmitter.Emitter->GetPathName());
	if (ResetDiffResults.IsValid() == false)
	{
		FApplyDiffResults Results;
		Results.bSucceeded = false;
		Results.bModifiedGraph = false;
		Results.ErrorMessages.Add(FText::Format(LOCTEXT("ResetFailedBecauseOfDiffMessage", "Failed to reset input back to it's base value.  It couldn't be diffed successfully.  Emitter: {0}  Input:{1}"),
		    EmitterPath, FText::FromString(InputName)));
		return Results;
	}

	if (ResetDiffResults.IsEmpty())
	{
		FApplyDiffResults Results;
		Results.bSucceeded = false;
		Results.bModifiedGraph = false;
		Results.ErrorMessages.Add(FText::Format(LOCTEXT("ResetFailedBecauseOfEmptyDiffMessage", "Failed to reset input back to it's base value.  It wasn't different from the base.  Emitter: {0}  Input:{1}"),
			EmitterPath, FText::FromString(InputName)));
		return Results;
	}

	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	FVersionedNiagaraEmitterData* BaseEmitterData = VersionedBaseEmitter.GetEmitterData();
	if (EmitterData->ParentScratchPads->Scripts.Num() != (BaseEmitterData->ParentScratchPads->Scripts.Num() + BaseEmitterData->ScratchPads->Scripts.Num()))
	{
		FApplyDiffResults Results;
		Results.bSucceeded = false;
		Results.bModifiedGraph = false;
		Results.ErrorMessages.Add(FText::Format(LOCTEXT("ResetFailedBecauseOfScratchPadScripts", "Failed to reset input back to it's base value.  Its scratch pad scripts were out of sync.  Emitter: {0}  Input:{1}"),
			EmitterPath, FText::FromString(InputName)));
		return Results;
	}
	
	// Remove items from the diff which are not relevant to this input.
	ResetDiffResults.RemovedBaseModules.Empty();
	ResetDiffResults.AddedOtherModules.Empty();

	auto FindUnrelatedInputOverrides = [=](TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> InputOverride)
	{
		return InputOverride->GetOwningFunctionCall()->NodeGuid != ModuleId || InputOverride->GetInputName() != InputName;
	};

	ResetDiffResults.RemovedBaseInputOverrides.RemoveAll(FindUnrelatedInputOverrides);
	ResetDiffResults.AddedOtherInputOverrides.RemoveAll(FindUnrelatedInputOverrides);
	ResetDiffResults.ModifiedBaseInputOverrides.RemoveAll(FindUnrelatedInputOverrides);
	ResetDiffResults.ModifiedOtherInputOverrides.RemoveAll(FindUnrelatedInputOverrides);

	return ApplyScriptStackDiff(EmitterAdapter, EmitterAdapter->GetScriptStack(ScriptUsage, ScriptUsageId).ToSharedRef(), ScratchPadMergeAdapter, ResetDiffResults, false);
}

bool FNiagaraScriptMergeManager::HasBaseEventHandler(const FVersionedNiagaraEmitter& BaseEmitter, FGuid EventScriptUsageId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);
	return BaseEmitterAdapter->GetEventHandler(EventScriptUsageId).IsValid();
}

bool FNiagaraScriptMergeManager::IsEventHandlerPropertySetDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid EventScriptUsageId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> EmitterAdapter = GetEmitterMergeAdapterUsingCache(Emitter);
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);

	TSharedPtr<FNiagaraEventHandlerMergeAdapter> EventHandlerAdapter = EmitterAdapter->GetEventHandler(EventScriptUsageId);
	TSharedPtr<FNiagaraEventHandlerMergeAdapter> BaseEventHandlerAdapter = BaseEmitterAdapter->GetEventHandler(EventScriptUsageId);

	if (EventHandlerAdapter->GetEditableEventScriptProperties() == nullptr || BaseEventHandlerAdapter->GetEventScriptProperties() == nullptr)
	{
		return true;
	}

	TArray<FProperty*> DifferentProperties;
	DiffEditableProperties(BaseEventHandlerAdapter->GetEventScriptProperties(), EventHandlerAdapter->GetEventScriptProperties(), *FNiagaraEventScriptProperties::StaticStruct(), DifferentProperties);
	return DifferentProperties.Num() > 0;
}

void FNiagaraScriptMergeManager::ResetEventHandlerPropertySetToBase(const FVersionedNiagaraEmitter& VersionedEmitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid EventScriptUsageId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> EmitterAdapter = GetEmitterMergeAdapterUsingCache(VersionedEmitter);
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);

	TSharedPtr<FNiagaraEventHandlerMergeAdapter> EventHandlerAdapter = EmitterAdapter->GetEventHandler(EventScriptUsageId);
	TSharedPtr<FNiagaraEventHandlerMergeAdapter> BaseEventHandlerAdapter = BaseEmitterAdapter->GetEventHandler(EventScriptUsageId);

	if (EventHandlerAdapter->GetEditableEventScriptProperties() == nullptr || BaseEventHandlerAdapter->GetEventScriptProperties() == nullptr)
	{
		// TODO: Display an error to the user.
		return;
	}

	TArray<FProperty*> DifferentProperties;
	DiffEditableProperties(BaseEventHandlerAdapter->GetEventScriptProperties(), EventHandlerAdapter->GetEventScriptProperties(), *FNiagaraEventScriptProperties::StaticStruct(), DifferentProperties);
	CopyPropertiesToBase(EventHandlerAdapter->GetEditableEventScriptProperties(), BaseEventHandlerAdapter->GetEventScriptProperties(), DifferentProperties);
	VersionedEmitter.Emitter->PostEditChange();
}

bool FNiagaraScriptMergeManager::HasBaseSimulationStage(const FVersionedNiagaraEmitter& BaseEmitter, FGuid SimulationStageScriptUsageId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);
	return BaseEmitterAdapter->GetSimulationStage(SimulationStageScriptUsageId).IsValid();
}

bool FNiagaraScriptMergeManager::IsSimulationStagePropertySetDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid SimulationStageScriptUsageId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> EmitterAdapter = GetEmitterMergeAdapterUsingCache(Emitter);
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);

	TSharedPtr<FNiagaraSimulationStageMergeAdapter> SimulationStageAdapter = EmitterAdapter->GetSimulationStage(SimulationStageScriptUsageId);
	TSharedPtr<FNiagaraSimulationStageMergeAdapter> BaseSimulationStageAdapter = BaseEmitterAdapter->GetSimulationStage(SimulationStageScriptUsageId);

	if (SimulationStageAdapter->GetEditableSimulationStage() == nullptr || BaseSimulationStageAdapter->GetSimulationStage() == nullptr)
	{
		return true;
	}

	TArray<FProperty*> DifferentProperties;
	DiffEditableProperties(BaseSimulationStageAdapter->GetSimulationStage(), SimulationStageAdapter->GetSimulationStage(), *BaseSimulationStageAdapter->GetSimulationStage()->GetClass(), DifferentProperties);
	return DifferentProperties.Num() > 0;
}

void FNiagaraScriptMergeManager::ResetSimulationStagePropertySetToBase(const FVersionedNiagaraEmitter& VersionedEmitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid SimulationStageScriptUsageId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> EmitterAdapter = GetEmitterMergeAdapterUsingCache(VersionedEmitter);
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);

	TSharedPtr<FNiagaraSimulationStageMergeAdapter> SimulationStageAdapter = EmitterAdapter->GetSimulationStage(SimulationStageScriptUsageId);
	TSharedPtr<FNiagaraSimulationStageMergeAdapter> BaseSimulationStageAdapter = BaseEmitterAdapter->GetSimulationStage(SimulationStageScriptUsageId);

	if (SimulationStageAdapter->GetEditableSimulationStage() == nullptr || BaseSimulationStageAdapter->GetSimulationStage() == nullptr)
	{
		// TODO: Display an error to the user.
		return;
	}

	TArray<FProperty*> DifferentProperties;
	DiffEditableProperties(BaseSimulationStageAdapter->GetSimulationStage(), SimulationStageAdapter->GetSimulationStage(), *BaseSimulationStageAdapter->GetSimulationStage()->GetClass(), DifferentProperties);
	CopyPropertiesToBase(SimulationStageAdapter->GetEditableSimulationStage(), BaseSimulationStageAdapter->GetSimulationStage(), DifferentProperties);
	VersionedEmitter.Emitter->PostEditChange();
}

bool FNiagaraScriptMergeManager::HasBaseRenderer(const FVersionedNiagaraEmitter& BaseEmitter, FGuid RendererMergeId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);
	return BaseEmitterAdapter->GetRenderer(RendererMergeId).IsValid();
}

bool FNiagaraScriptMergeManager::IsRendererDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid RendererMergeId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> EmitterAdapter = GetEmitterMergeAdapterUsingCache(Emitter);
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);

	FNiagaraEmitterDiffResults DiffResults;
	DiffRenderers(BaseEmitterAdapter->GetRenderers(), EmitterAdapter->GetRenderers(), DiffResults);

	if (DiffResults.IsValid() == false)
	{
		return true;
	}

	if (DiffResults.ModifiedOtherRenderers.Num() == 0)
	{
		return false;
	}

	auto FindRendererByMergeId = [=](TSharedRef<FNiagaraRendererMergeAdapter> Renderer) { return Renderer->GetRenderer()->GetMergeId() == RendererMergeId; };
	return DiffResults.ModifiedOtherRenderers.FindByPredicate(FindRendererByMergeId) != nullptr;
}

void FNiagaraScriptMergeManager::ResetRendererToBase(FVersionedNiagaraEmitter Emitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid RendererMergeId)
{
	TSharedRef<FNiagaraEmitterMergeAdapter> EmitterAdapter = GetEmitterMergeAdapterUsingCache(Emitter);
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = GetEmitterMergeAdapterUsingCache(BaseEmitter);

	// Diff from the current emitter to the base emitter to create a diff which will reset the emitter back to the base.
	FNiagaraEmitterDiffResults ResetDiffResults;
	DiffRenderers(EmitterAdapter->GetRenderers(), BaseEmitterAdapter->GetRenderers(), ResetDiffResults);

	auto FindUnrelatedRenderers = [=](TSharedRef<FNiagaraRendererMergeAdapter> Renderer)
	{
		return Renderer->GetRenderer()->GetMergeId() != RendererMergeId;
	};

	// Removed added and removed renderers, as well as changes to renderers with different ids from the one being reset.
	ResetDiffResults.RemovedBaseRenderers.Empty();
	ResetDiffResults.AddedOtherRenderers.Empty();
	ResetDiffResults.ModifiedBaseRenderers.RemoveAll(FindUnrelatedRenderers);
	ResetDiffResults.ModifiedOtherRenderers.RemoveAll(FindUnrelatedRenderers);
	
	ApplyRendererDiff(Emitter, ResetDiffResults, false);
}

bool FNiagaraScriptMergeManager::IsEmitterEditablePropertySetDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter)
{
	TArray<FProperty*> DifferentProperties;
	DiffEditableProperties(BaseEmitter.GetEmitterData(), Emitter.GetEmitterData(), *FVersionedNiagaraEmitterData::StaticStruct(), DifferentProperties);
	return DifferentProperties.Num() > 0;
}

void FNiagaraScriptMergeManager::ResetEmitterEditablePropertySetToBase(const FVersionedNiagaraEmitter& VersionedEmitter, const FVersionedNiagaraEmitter& BaseEmitter)
{
	TArray<FProperty*> DifferentProperties;
	DiffEditableProperties(BaseEmitter.GetEmitterData(), VersionedEmitter.GetEmitterData(), *FVersionedNiagaraEmitterData::StaticStruct(), DifferentProperties);
	CopyPropertiesToBase(VersionedEmitter.GetEmitterData(), BaseEmitter.GetEmitterData(), DifferentProperties);
	VersionedEmitter.Emitter->PostEditChange();
}

FNiagaraEmitterDiffResults FNiagaraScriptMergeManager::DiffEmitters(const FVersionedNiagaraEmitter& BaseEmitter, const FVersionedNiagaraEmitter& OtherEmitter) const
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_ScriptMergeManager_DiffEmitters);

	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter = MakeShared<FNiagaraEmitterMergeAdapter>(BaseEmitter);
	TSharedRef<FNiagaraEmitterMergeAdapter> OtherEmitterAdapter = MakeShared<FNiagaraEmitterMergeAdapter>(OtherEmitter);

	FNiagaraEmitterDiffResults EmitterDiffResults;
	if (BaseEmitterAdapter->GetEmitterSpawnStack().IsValid() && OtherEmitterAdapter->GetEmitterSpawnStack().IsValid())
	{
		DiffScriptStacks(BaseEmitterAdapter->GetEmitterSpawnStack().ToSharedRef(), OtherEmitterAdapter->GetEmitterSpawnStack().ToSharedRef(), EmitterDiffResults.EmitterSpawnDiffResults);
	}
	else
	{
		EmitterDiffResults.AddError(LOCTEXT("EmitterSpawnStacksInvalidMessage", "One of the emitter spawn script stacks was invalid."));
	}

	if (BaseEmitterAdapter->GetEmitterUpdateStack().IsValid() && OtherEmitterAdapter->GetEmitterUpdateStack().IsValid())
	{
		DiffScriptStacks(BaseEmitterAdapter->GetEmitterUpdateStack().ToSharedRef(), OtherEmitterAdapter->GetEmitterUpdateStack().ToSharedRef(), EmitterDiffResults.EmitterUpdateDiffResults);
	}
	else
	{
		EmitterDiffResults.AddError(LOCTEXT("EmitterUpdateStacksInvalidMessage", "One of the emitter update script stacks was invalid."));
	}

	if (BaseEmitterAdapter->GetParticleSpawnStack().IsValid() && OtherEmitterAdapter->GetParticleSpawnStack().IsValid())
	{
		DiffScriptStacks(BaseEmitterAdapter->GetParticleSpawnStack().ToSharedRef(), OtherEmitterAdapter->GetParticleSpawnStack().ToSharedRef(), EmitterDiffResults.ParticleSpawnDiffResults);
	}
	else
	{
		EmitterDiffResults.AddError(LOCTEXT("ParticleSpawnStacksInvalidMessage", "One of the particle spawn script stacks was invalid."));
	}

	if (BaseEmitterAdapter->GetParticleUpdateStack().IsValid() && OtherEmitterAdapter->GetParticleUpdateStack().IsValid())
	{
		DiffScriptStacks(BaseEmitterAdapter->GetParticleUpdateStack().ToSharedRef(), OtherEmitterAdapter->GetParticleUpdateStack().ToSharedRef(), EmitterDiffResults.ParticleUpdateDiffResults);
	}
	else
	{
		EmitterDiffResults.AddError(LOCTEXT("ParticleUpdateStacksInvalidMessage", "One of the particle update script stacks was invalid."));
	}
	FVersionedNiagaraEmitterData* BaseEmitterData = BaseEmitter.GetEmitterData();
	FVersionedNiagaraEmitterData* OtherEmitterData = OtherEmitter.GetEmitterData();

	DiffEventHandlers(BaseEmitterAdapter->GetEventHandlers(), OtherEmitterAdapter->GetEventHandlers(), EmitterDiffResults);
	DiffSimulationStages(BaseEmitterAdapter->GetSimulationStages(), OtherEmitterAdapter->GetSimulationStages(), EmitterDiffResults);
	DiffRenderers(BaseEmitterAdapter->GetRenderers(), OtherEmitterAdapter->GetRenderers(), EmitterDiffResults);
	DiffEmitterSummary(BaseEmitterAdapter->GetEditorData(), OtherEmitterAdapter->GetEditorData(), EmitterDiffResults);
	DiffEditableProperties(BaseEmitterData, OtherEmitterData, *FVersionedNiagaraEmitterData::StaticStruct(), EmitterDiffResults.DifferentEmitterProperties);
	DiffStackEntryDisplayNames(BaseEmitterAdapter->GetEditorData(), OtherEmitterAdapter->GetEditorData(), EmitterDiffResults.ModifiedStackEntryDisplayNames);
	DiffStackNotes(BaseEmitterAdapter->GetEditorData(), OtherEmitterAdapter->GetEditorData(), EmitterDiffResults);
	
	TArray<UNiagaraScript*> BaseScratchPadScripts;
	BaseScratchPadScripts.Append(BaseEmitterData->ParentScratchPads->Scripts);
	BaseScratchPadScripts.Append(BaseEmitterData->ScratchPads->Scripts);
	TArray<UNiagaraScript*> OtherScratchPadScripts;
	OtherScratchPadScripts.Append(OtherEmitterData->ParentScratchPads->Scripts);
	OtherScratchPadScripts.Append(OtherEmitterData->ScratchPads->Scripts);
	DiffScratchPadScripts(BaseScratchPadScripts, OtherScratchPadScripts, EmitterDiffResults);

	EmitterDiffResults.BaseEmitterAdapter = BaseEmitterAdapter;
	EmitterDiffResults.OtherEmitterAdapter = OtherEmitterAdapter;

	return EmitterDiffResults;
}

template<typename ValueType>
struct FCommonValuePair
{
	FCommonValuePair(ValueType InBaseValue, ValueType InOtherValue)
		: BaseValue(InBaseValue)
		, OtherValue(InOtherValue)
	{
	}

	ValueType BaseValue;
	ValueType OtherValue;
};

template<typename ValueType>
struct FListDiffResults
{
	TArray<ValueType> RemovedBaseValues;
	TArray<ValueType> AddedOtherValues;
	TArray<FCommonValuePair<ValueType>> CommonValuePairs;
};

template<typename ValueType, typename KeyType, typename KeyFromValueDelegate>
FListDiffResults<ValueType> DiffLists(const TArray<ValueType>& BaseList, const TArray<ValueType> OtherList, KeyFromValueDelegate KeyFromValue)
{
	FListDiffResults<ValueType> DiffResults;

	TMap<KeyType, ValueType> BaseKeyToValueMap;
	TSet<KeyType> BaseKeys;
	for (ValueType BaseValue : BaseList)
	{
		KeyType BaseKey = KeyFromValue(BaseValue);
		BaseKeyToValueMap.Add(BaseKey, BaseValue);
		BaseKeys.Add(BaseKey);
	}

	TMap<KeyType, ValueType> OtherKeyToValueMap;
	TSet<KeyType> OtherKeys;
	for (ValueType OtherValue : OtherList)
	{
		KeyType OtherKey = KeyFromValue(OtherValue);
		OtherKeyToValueMap.Add(OtherKey, OtherValue);
		OtherKeys.Add(OtherKey);
	}

	for (KeyType RemovedKey : BaseKeys.Difference(OtherKeys))
	{
		DiffResults.RemovedBaseValues.Add(BaseKeyToValueMap[RemovedKey]);
	}

	for (KeyType AddedKey : OtherKeys.Difference(BaseKeys))
	{
		DiffResults.AddedOtherValues.Add(OtherKeyToValueMap[AddedKey]);
	}

	for (KeyType CommonKey : BaseKeys.Intersect(OtherKeys))
	{
		DiffResults.CommonValuePairs.Add(FCommonValuePair<ValueType>(BaseKeyToValueMap[CommonKey], OtherKeyToValueMap[CommonKey]));
	}

	return DiffResults;
}

void FNiagaraScriptMergeManager::DiffEventHandlers(const TArray<TSharedRef<FNiagaraEventHandlerMergeAdapter>>& BaseEventHandlers, const TArray<TSharedRef<FNiagaraEventHandlerMergeAdapter>>& OtherEventHandlers, FNiagaraEmitterDiffResults& DiffResults) const
{
	FListDiffResults<TSharedRef<FNiagaraEventHandlerMergeAdapter>> EventHandlerListDiffResults = DiffLists<TSharedRef<FNiagaraEventHandlerMergeAdapter>, FGuid>(
		BaseEventHandlers,
		OtherEventHandlers,
		[](TSharedRef<FNiagaraEventHandlerMergeAdapter> EventHandler) { return EventHandler->GetUsageId(); });

	DiffResults.RemovedBaseEventHandlers.Append(EventHandlerListDiffResults.RemovedBaseValues);
	DiffResults.AddedOtherEventHandlers.Append(EventHandlerListDiffResults.AddedOtherValues);

	for (const FCommonValuePair<TSharedRef<FNiagaraEventHandlerMergeAdapter>>& CommonValuePair : EventHandlerListDiffResults.CommonValuePairs)
	{
		if (CommonValuePair.BaseValue->GetEventScriptProperties() == nullptr || CommonValuePair.BaseValue->GetOutputNode() == nullptr)
		{
			DiffResults.AddError(FText::Format(LOCTEXT("InvalidBaseEventHandlerDiffFailedFormat", "Failed to diff event handlers, the base event handler was invalid.  Script Usage Id: {0}"),
				FText::FromString(CommonValuePair.BaseValue->GetUsageId().ToString())));
		}
		else if(CommonValuePair.OtherValue->GetEventScriptProperties() == nullptr || CommonValuePair.OtherValue->GetOutputNode() == nullptr)
		{
			DiffResults.AddError(FText::Format(LOCTEXT("InvalidOtherEventHandlerDiffFailedFormat", "Failed to diff event handlers, the other event handler was invalid.  Script Usage Id: {0}"),
				FText::FromString(CommonValuePair.OtherValue->GetUsageId().ToString())));
		}
		else
		{
			TArray<FProperty*> DifferentProperties;
			DiffEditableProperties(CommonValuePair.BaseValue->GetEventScriptProperties(), CommonValuePair.OtherValue->GetEventScriptProperties(), *FNiagaraEventScriptProperties::StaticStruct(), DifferentProperties);

			FNiagaraScriptStackDiffResults EventHandlerScriptStackDiffResults;
			DiffScriptStacks(CommonValuePair.BaseValue->GetEventStack().ToSharedRef(), CommonValuePair.OtherValue->GetEventStack().ToSharedRef(), EventHandlerScriptStackDiffResults);

			if (DifferentProperties.Num() > 0 || EventHandlerScriptStackDiffResults.IsValid() == false || EventHandlerScriptStackDiffResults.IsEmpty() == false)
			{
				FNiagaraModifiedEventHandlerDiffResults ModifiedEventHandlerResults;
				ModifiedEventHandlerResults.BaseAdapter = CommonValuePair.BaseValue;
				ModifiedEventHandlerResults.OtherAdapter = CommonValuePair.OtherValue;
				ModifiedEventHandlerResults.ChangedProperties.Append(DifferentProperties);
				ModifiedEventHandlerResults.ScriptDiffResults = EventHandlerScriptStackDiffResults;
				DiffResults.ModifiedEventHandlers.Add(ModifiedEventHandlerResults);
			}

			if (EventHandlerScriptStackDiffResults.IsValid() == false)
			{
				for (const FText& EventHandlerScriptStackDiffErrorMessage : EventHandlerScriptStackDiffResults.GetErrorMessages())
				{
					DiffResults.AddError(EventHandlerScriptStackDiffErrorMessage);
				}
			}
		}
	}
}

void FNiagaraScriptMergeManager::DiffSimulationStages(const TArray<TSharedRef<FNiagaraSimulationStageMergeAdapter>>& BaseSimulationStages, const TArray<TSharedRef<FNiagaraSimulationStageMergeAdapter>>& OtherSimulationStages, FNiagaraEmitterDiffResults& DiffResults) const
{
	FListDiffResults<TSharedRef<FNiagaraSimulationStageMergeAdapter>> SimulationStageListDiffResults = DiffLists<TSharedRef<FNiagaraSimulationStageMergeAdapter>, FGuid>(
		BaseSimulationStages,
		OtherSimulationStages,
		[](TSharedRef<FNiagaraSimulationStageMergeAdapter> SimulationStage) { return SimulationStage->GetUsageId(); });

	// Sort the diff results for easier diff applying and testing.
	auto OrderSimulationStageByIndex = [](TSharedRef<FNiagaraSimulationStageMergeAdapter> SimulationStageA, TSharedRef<FNiagaraSimulationStageMergeAdapter> SimulationStageB)
	{
		return SimulationStageA->GetSimulationStageIndex() < SimulationStageB->GetSimulationStageIndex();
	};

	SimulationStageListDiffResults.RemovedBaseValues.Sort(OrderSimulationStageByIndex);
	SimulationStageListDiffResults.AddedOtherValues.Sort(OrderSimulationStageByIndex);

	auto OrderCommonSimulationStagePairByBaseIndex = [](
		const FCommonValuePair<TSharedRef<FNiagaraSimulationStageMergeAdapter>>& CommonValuesA,
		const FCommonValuePair<TSharedRef<FNiagaraSimulationStageMergeAdapter>>& CommonValuesB)
	{
		return CommonValuesA.BaseValue->GetSimulationStageIndex() < CommonValuesB.BaseValue->GetSimulationStageIndex();
	};

	SimulationStageListDiffResults.CommonValuePairs.Sort(OrderCommonSimulationStagePairByBaseIndex);

	// Populate results from the sorted diff.
	DiffResults.RemovedBaseSimulationStages.Append(SimulationStageListDiffResults.RemovedBaseValues);
	DiffResults.AddedOtherSimulationStages.Append(SimulationStageListDiffResults.AddedOtherValues);

	for (const FCommonValuePair<TSharedRef<FNiagaraSimulationStageMergeAdapter>>& CommonValuePair : SimulationStageListDiffResults.CommonValuePairs)
	{
		if (CommonValuePair.BaseValue->GetSimulationStage() == nullptr || CommonValuePair.BaseValue->GetOutputNode() == nullptr)
		{
			DiffResults.AddError(FText::Format(LOCTEXT("InvalidBaseSimulationStageDiffFailedFormat", "Failed to diff shader stages, the base shader stage was invalid.  Script Usage Id: {0}"),
				FText::FromString(CommonValuePair.BaseValue->GetUsageId().ToString())));
		}
		else if (CommonValuePair.OtherValue->GetSimulationStage() == nullptr || CommonValuePair.OtherValue->GetOutputNode() == nullptr)
		{
			DiffResults.AddError(FText::Format(LOCTEXT("InvalidOtherSimulationStageDiffFailedFormat", "Failed to diff shader stage, the other shader stage was invalid.  Script Usage Id: {0}"),
				FText::FromString(CommonValuePair.OtherValue->GetUsageId().ToString())));
		}
		else
		{
			TArray<FProperty*> DifferentProperties;
			DiffEditableProperties(CommonValuePair.BaseValue->GetSimulationStage(), CommonValuePair.OtherValue->GetSimulationStage(), *CommonValuePair.BaseValue->GetSimulationStage()->GetClass(), DifferentProperties);

			FNiagaraScriptStackDiffResults SimulationStageScriptStackDiffResults;
			DiffScriptStacks(CommonValuePair.BaseValue->GetSimulationStageStack().ToSharedRef(), CommonValuePair.OtherValue->GetSimulationStageStack().ToSharedRef(), SimulationStageScriptStackDiffResults);

			if (DifferentProperties.Num() > 0 || SimulationStageScriptStackDiffResults.IsValid() == false || SimulationStageScriptStackDiffResults.IsEmpty() == false)
			{
				FNiagaraModifiedSimulationStageDiffResults ModifiedSimulationStageResults;
				ModifiedSimulationStageResults.BaseAdapter = CommonValuePair.BaseValue;
				ModifiedSimulationStageResults.OtherAdapter = CommonValuePair.OtherValue;
				ModifiedSimulationStageResults.ChangedProperties.Append(DifferentProperties);
				ModifiedSimulationStageResults.ScriptDiffResults = SimulationStageScriptStackDiffResults;
				DiffResults.ModifiedSimulationStages.Add(ModifiedSimulationStageResults);
			}

			if (SimulationStageScriptStackDiffResults.IsValid() == false)
			{
				for (const FText& SimulationStageScriptStackDiffErrorMessage : SimulationStageScriptStackDiffResults.GetErrorMessages())
				{
					DiffResults.AddError(SimulationStageScriptStackDiffErrorMessage);
				}
			}
		}
	}
}

void FNiagaraScriptMergeManager::DiffRenderers(const TArray<TSharedRef<FNiagaraRendererMergeAdapter>>& BaseRenderers, const TArray<TSharedRef<FNiagaraRendererMergeAdapter>>& OtherRenderers, FNiagaraEmitterDiffResults& DiffResults) const
{
	FListDiffResults<TSharedRef<FNiagaraRendererMergeAdapter>> RendererListDiffResults = DiffLists<TSharedRef<FNiagaraRendererMergeAdapter>, FGuid>(
		BaseRenderers,
		OtherRenderers,
		[](TSharedRef<FNiagaraRendererMergeAdapter> Renderer) { return Renderer->GetRenderer()->GetMergeId(); });

	DiffResults.RemovedBaseRenderers.Append(RendererListDiffResults.RemovedBaseValues);
	DiffResults.AddedOtherRenderers.Append(RendererListDiffResults.AddedOtherValues);

	for (const FCommonValuePair<TSharedRef<FNiagaraRendererMergeAdapter>>& CommonValuePair : RendererListDiffResults.CommonValuePairs)
	{
		if (CommonValuePair.BaseValue->GetRenderer()->Equals(CommonValuePair.OtherValue->GetRenderer()) == false)
		{
			DiffResults.ModifiedBaseRenderers.Add(CommonValuePair.BaseValue);
			DiffResults.ModifiedOtherRenderers.Add(CommonValuePair.OtherValue);
		}
	}
}

void FNiagaraScriptMergeManager::DiffEmitterSummary(const UNiagaraEmitterEditorData* BaseEditorData, const UNiagaraEmitterEditorData* OtherEditorData, FNiagaraEmitterDiffResults& DiffResults) const
{
	if(BaseEditorData == nullptr || OtherEditorData == nullptr)
	{
		return;
	}

	DiffResults.NewShouldShowSummaryViewValue = OtherEditorData->ShouldShowSummaryView();
	
	TArray<const UHierarchyElement*> AddedItemsInOther;
	TArray<const UHierarchyElement*> RemovedItemsInBase;
	TArray<const UHierarchySection*> AddedSections;
	
	UHierarchyRoot* BaseRoot = BaseEditorData->GetSummaryRoot();
	UHierarchyRoot* OtherRoot = OtherEditorData->GetSummaryRoot();	

	TArray<UHierarchyElement*> BaseItems;
	BaseRoot->GetChildrenOfType(BaseItems, true);

	TArray<UHierarchyElement*> OtherItems;
	OtherRoot->GetChildrenOfType(OtherItems, true);
	
	FListDiffResults<UHierarchyElement*> SummaryItemDiff = DiffLists<UHierarchyElement*, FHierarchyElementIdentity>(
		BaseItems,
		OtherItems,
		[](UHierarchyElement* HierarchyItem) { return HierarchyItem->GetPersistentIdentity(); });
	
	DiffResults.AddedSummaryEntriesInOther.Append(SummaryItemDiff.AddedOtherValues);
	
	TConstArrayView<const TObjectPtr<UHierarchySection>> BaseSections = BaseRoot->GetSectionData();
	TConstArrayView<const TObjectPtr<UHierarchySection>> OtherSections = OtherRoot->GetSectionData();

	for (const UHierarchySection* OtherSection : OtherSections)
	{
		FHierarchyElementIdentity OtherSectionIdentity = OtherSection->GetPersistentIdentity();
		if(!BaseSections.ContainsByPredicate([OtherSectionIdentity](const UHierarchySection* BaseSection)
		{
			return BaseSection->GetPersistentIdentity() == OtherSectionIdentity;
		}))
		{
			AddedSections.Add(OtherSection);
		}
	}

	DiffResults.AddedSummarySectionsInOther = AddedSections;
}

void FNiagaraScriptMergeManager::DiffScriptStacks(TSharedRef<FNiagaraScriptStackMergeAdapter> BaseScriptStackAdapter, TSharedRef<FNiagaraScriptStackMergeAdapter> OtherScriptStackAdapter, FNiagaraScriptStackDiffResults& DiffResults) const
{
	// Diff the module lists.
	FListDiffResults<TSharedRef<FNiagaraStackFunctionMergeAdapter>> ModuleListDiffResults = DiffLists<TSharedRef<FNiagaraStackFunctionMergeAdapter>, FGuid>(
		BaseScriptStackAdapter->GetModuleFunctions(),
		OtherScriptStackAdapter->GetModuleFunctions(),
		[](TSharedRef<FNiagaraStackFunctionMergeAdapter> FunctionAdapter) { return FunctionAdapter->GetFunctionCallNode()->NodeGuid; });

	// Sort the diff results for easier diff applying and testing.
	auto OrderModuleByStackIndex = [](TSharedRef<FNiagaraStackFunctionMergeAdapter> ModuleA, TSharedRef<FNiagaraStackFunctionMergeAdapter> ModuleB)
	{
		return ModuleA->GetStackIndex() < ModuleB->GetStackIndex();
	};

	ModuleListDiffResults.RemovedBaseValues.Sort(OrderModuleByStackIndex);
	ModuleListDiffResults.AddedOtherValues.Sort(OrderModuleByStackIndex);

	auto OrderCommonModulePairByBaseStackIndex = [](
		const FCommonValuePair<TSharedRef<FNiagaraStackFunctionMergeAdapter>>& CommonValuesA,
		const FCommonValuePair<TSharedRef<FNiagaraStackFunctionMergeAdapter>>& CommonValuesB)
	{
		return CommonValuesA.BaseValue->GetStackIndex() < CommonValuesB.BaseValue->GetStackIndex();
	};

	ModuleListDiffResults.CommonValuePairs.Sort(OrderCommonModulePairByBaseStackIndex);

	// Populate results from the sorted diff.
	DiffResults.RemovedBaseModules.Append(ModuleListDiffResults.RemovedBaseValues);
	DiffResults.AddedOtherModules.Append(ModuleListDiffResults.AddedOtherValues);

	for (const FCommonValuePair<TSharedRef<FNiagaraStackFunctionMergeAdapter>>& CommonValuePair : ModuleListDiffResults.CommonValuePairs)
	{
		if (CommonValuePair.BaseValue->GetStackIndex() != CommonValuePair.OtherValue->GetStackIndex())
		{
			DiffResults.MovedBaseModules.Add(CommonValuePair.BaseValue);
			DiffResults.MovedOtherModules.Add(CommonValuePair.OtherValue);
		}

		if (CommonValuePair.BaseValue->GetFunctionCallNode()->IsNodeEnabled() != CommonValuePair.OtherValue->GetFunctionCallNode()->IsNodeEnabled())
		{
			DiffResults.EnabledChangedBaseModules.Add(CommonValuePair.BaseValue);
			DiffResults.EnabledChangedOtherModules.Add(CommonValuePair.OtherValue);
		}

		if (CommonValuePair.BaseValue->GetFunctionCallNode()->SelectedScriptVersion != CommonValuePair.OtherValue->GetFunctionCallNode()->SelectedScriptVersion)
		{
			DiffResults.ChangedVersionBaseModules.Add(CommonValuePair.BaseValue);
			DiffResults.ChangedVersionOtherModules.Add(CommonValuePair.OtherValue);
		}

		UNiagaraScript* BaseFunctionScript = CommonValuePair.BaseValue->GetFunctionCallNode()->FunctionScript;
		UNiagaraScript* OtherFunctionScript = CommonValuePair.OtherValue->GetFunctionCallNode()->FunctionScript;
		bool bFunctionScriptsMatch = BaseFunctionScript == OtherFunctionScript;
		bool bFunctionScriptsAreNotAssets =
			BaseFunctionScript != nullptr && BaseFunctionScript->IsAsset() == false &&
			OtherFunctionScript != nullptr && OtherFunctionScript->IsAsset() == false;
		if (bFunctionScriptsMatch || bFunctionScriptsAreNotAssets)
		{
			DiffFunctionInputs(CommonValuePair.BaseValue, CommonValuePair.OtherValue, DiffResults);
		}
		else
		{
			FText ErrorMessage = FText::Format(LOCTEXT("FunctionScriptMismatchFormat", "Function scripts for function {0} did not match.  Parent: {1} Child: {2}.  This can be fixed by removing the module from the parent, merging the removal to the child, then removing it from the child, and then re-adding it to the parent and merging again."),
				FText::FromString(CommonValuePair.BaseValue->GetFunctionCallNode()->GetFunctionName()), 
				FText::FromString(CommonValuePair.BaseValue->GetFunctionCallNode()->FunctionScript != nullptr ? CommonValuePair.BaseValue->GetFunctionCallNode()->FunctionScript->GetPathName() : TEXT("(null)")),
				FText::FromString(CommonValuePair.OtherValue->GetFunctionCallNode()->FunctionScript != nullptr ? CommonValuePair.OtherValue->GetFunctionCallNode()->FunctionScript->GetPathName() : TEXT("(null)")));
			DiffResults.AddError(ErrorMessage);
		}
	}

	// We need to ignore the interpolation setting when checking script usage as the emitter is the controller for this
	// and we don't merge the emitter properties right now
	auto GetUsageIgnoreInterpolation =
		[](ENiagaraScriptUsage InUsage) -> ENiagaraScriptUsage
		{
			return InUsage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated ? ENiagaraScriptUsage::ParticleSpawnScript : InUsage;
		};

	if (GetUsageIgnoreInterpolation(BaseScriptStackAdapter->GetScript()->GetUsage()) != GetUsageIgnoreInterpolation(OtherScriptStackAdapter->GetScript()->GetUsage()))
	{
		DiffResults.ChangedBaseUsage = BaseScriptStackAdapter->GetScript()->GetUsage();
		DiffResults.ChangedOtherUsage = OtherScriptStackAdapter->GetScript()->GetUsage();
	}
}

void FNiagaraScriptMergeManager::DiffFunctionInputs(TSharedRef<FNiagaraStackFunctionMergeAdapter> BaseFunctionAdapter, TSharedRef<FNiagaraStackFunctionMergeAdapter> OtherFunctionAdapter, FNiagaraScriptStackDiffResults& DiffResults) const
{
	FListDiffResults<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>> ListDiffResults = DiffLists<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>, FNiagaraVariableBase>(
		BaseFunctionAdapter->GetInputOverrides(),
		OtherFunctionAdapter->GetInputOverrides(),
		[](TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> InputOverrideAdapter)
		{
			return FNiagaraVariableBase(InputOverrideAdapter->GetType(), FName(InputOverrideAdapter->GetInputName()));
		});

	DiffResults.RemovedBaseInputOverrides.Append(ListDiffResults.RemovedBaseValues);
	DiffResults.AddedOtherInputOverrides.Append(ListDiffResults.AddedOtherValues);

	for (const FCommonValuePair<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>>& CommonValuePair : ListDiffResults.CommonValuePairs)
	{
		TOptional<bool> FunctionMatch = DoFunctionInputOverridesMatch(CommonValuePair.BaseValue, CommonValuePair.OtherValue);
		if (FunctionMatch.IsSet())
		{
			if (FunctionMatch.GetValue() == false)
			{
				DiffResults.ModifiedBaseInputOverrides.Add(CommonValuePair.BaseValue);
				DiffResults.ModifiedOtherInputOverrides.Add(CommonValuePair.OtherValue);
			}
		}
		else
		{
			DiffResults.AddError(FText::Format(LOCTEXT("FunctionInputDiffFailedFormat", "Failed to diff function inputs.  Function name: {0}  Input Name: {1}"),
				FText::FromString(BaseFunctionAdapter->GetFunctionCallNode()->GetFunctionName()),
				FText::FromString(CommonValuePair.BaseValue->GetInputName())));
		}
	}
}

void FNiagaraScriptMergeManager::DiffEditableProperties(const void* BaseDataAddress, const void* OtherDataAddress, UStruct& Struct, TArray<FProperty*>& OutDifferentProperties) const
{
	for (TFieldIterator<FProperty> PropertyIterator(&Struct); PropertyIterator; ++PropertyIterator)
	{
		if ((PropertyIterator->HasAllPropertyFlags(CPF_Edit) && PropertyIterator->HasMetaData("NiagaraNoMerge") == false))
		{
			if (PropertyIterator->Identical(
				PropertyIterator->ContainerPtrToValuePtr<void>(BaseDataAddress),
				PropertyIterator->ContainerPtrToValuePtr<void>(OtherDataAddress), PPF_DeepComparison) == false)
			{
				OutDifferentProperties.Add(*PropertyIterator);
			}
		}
	}
}

void FNiagaraScriptMergeManager::DiffStackEntryDisplayNames(const UNiagaraEmitterEditorData* BaseEditorData, const UNiagaraEmitterEditorData* OtherEditorData, TMap<FString, FText>& OutModifiedStackEntryDisplayNames) const
{
	if (BaseEditorData != nullptr && OtherEditorData != nullptr)
	{
		// find display names that have been added or changed in the instance
		const TMap<FString, FText>& OtherRenames = OtherEditorData->GetStackEditorData().GetAllStackEntryDisplayNames();
		for (auto& Pair : OtherRenames)
		{
			const FText* BaseDisplayName = BaseEditorData->GetStackEditorData().GetStackEntryDisplayName(Pair.Key);
			if (BaseDisplayName == nullptr || !BaseDisplayName->EqualTo(Pair.Value))
			{
				OutModifiedStackEntryDisplayNames.Add(Pair.Key, Pair.Value);
			}
		}
	}
}

void FNiagaraScriptMergeManager::DiffStackNotes(const UNiagaraEmitterEditorData* BaseEditorData, const UNiagaraEmitterEditorData* OtherEditorData, FNiagaraEmitterDiffResults& DiffResults) const
{
	if (BaseEditorData != nullptr && OtherEditorData != nullptr)
	{
		const TMap<FString, FNiagaraStackNoteData>& OtherStackNotes = OtherEditorData->GetStackEditorData().GetAllStackNotes();
		for(const auto& Pair : OtherStackNotes)
		{
			// if the other stack notes differ in any way, we will overwrite base with other
			TOptional<FNiagaraStackNoteData> BaseStackNote = BaseEditorData->GetStackEditorData().GetStackNote(Pair.Key);
			
			if(BaseStackNote.Get(FNiagaraStackNoteData()) != Pair.Value)
			{
				DiffResults.AddedOrModifiedStackNotesInOther.Add(Pair.Key, Pair.Value);
			}
		}		
	}
}

void FNiagaraScriptMergeManager::DiffScratchPadScripts(const TArray<UNiagaraScript*>& BaseScratchPadScripts, const TArray<UNiagaraScript*>& OtherEmitterScratchPadScripts, FNiagaraEmitterDiffResults& DiffResults) const
{
	if (BaseScratchPadScripts.Num() != OtherEmitterScratchPadScripts.Num())
	{
		DiffResults.bScratchPadModified = true;
		return;
	}

	for (int32 ScratchPadScriptIndex = 0; ScratchPadScriptIndex < BaseScratchPadScripts.Num(); ScratchPadScriptIndex++)
	{
		UNiagaraScript* BaseScratchPadScript = BaseScratchPadScripts[ScratchPadScriptIndex];
		UNiagaraScript* OtherScratchPadScript = OtherEmitterScratchPadScripts[ScratchPadScriptIndex];
		if (!BaseScratchPadScript->GetFName().IsEqual(OtherScratchPadScript->GetFName(), ENameCase::IgnoreCase, false) ||
			BaseScratchPadScript->GetBaseChangeID() != OtherScratchPadScript->GetBaseChangeID())
		{
			DiffResults.bScratchPadModified = true;
			return;
		}
	}
}

TOptional<bool> FNiagaraScriptMergeManager::DoFunctionInputOverridesMatch(TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> BaseFunctionInputAdapter, TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> OtherFunctionInputAdapter) const
{
	// Local String Value.
	if ((BaseFunctionInputAdapter->GetLocalValueString().IsSet() && OtherFunctionInputAdapter->GetLocalValueString().IsSet() == false) ||
		(BaseFunctionInputAdapter->GetLocalValueString().IsSet() == false && OtherFunctionInputAdapter->GetLocalValueString().IsSet()))
	{
		return false;
	}

	if (BaseFunctionInputAdapter->GetLocalValueString().IsSet() && OtherFunctionInputAdapter->GetLocalValueString().IsSet())
	{
		return BaseFunctionInputAdapter->GetLocalValueString().GetValue() == OtherFunctionInputAdapter->GetLocalValueString().GetValue();
	}

	// Local rapid iteration parameter value.
	if ((BaseFunctionInputAdapter->GetLocalValueRapidIterationParameter().IsSet() && OtherFunctionInputAdapter->GetLocalValueRapidIterationParameter().IsSet() == false) ||
		(BaseFunctionInputAdapter->GetLocalValueRapidIterationParameter().IsSet() == false && OtherFunctionInputAdapter->GetLocalValueRapidIterationParameter().IsSet()))
	{
		return false;
	}

	if (BaseFunctionInputAdapter->GetLocalValueRapidIterationParameter().IsSet() && OtherFunctionInputAdapter->GetLocalValueRapidIterationParameter().IsSet())
	{
		const uint8* BaseRapidIterationParameterValue = BaseFunctionInputAdapter->GetOwningScript()->RapidIterationParameters
			.GetParameterData(BaseFunctionInputAdapter->GetLocalValueRapidIterationParameter().GetValue());
		const uint8* OtherRapidIterationParameterValue = OtherFunctionInputAdapter->GetOwningScript()->RapidIterationParameters
			.GetParameterData(OtherFunctionInputAdapter->GetLocalValueRapidIterationParameter().GetValue());
		if (BaseRapidIterationParameterValue == nullptr || OtherRapidIterationParameterValue == nullptr)
		{
			return TOptional<bool>();
		}
		return FMemory::Memcmp(
			BaseRapidIterationParameterValue,
			OtherRapidIterationParameterValue,
			BaseFunctionInputAdapter->GetLocalValueRapidIterationParameter().GetValue().GetSizeInBytes()) == 0;
	}

	// Linked value
	if ((BaseFunctionInputAdapter->GetLinkedValueData().IsSet() && OtherFunctionInputAdapter->GetLinkedValueData().IsSet() == false) ||
		(BaseFunctionInputAdapter->GetLinkedValueData().IsSet() == false && OtherFunctionInputAdapter->GetLinkedValueData().IsSet()))
	{
		return false;
	}

	if (BaseFunctionInputAdapter->GetLinkedValueData().IsSet() && OtherFunctionInputAdapter->GetLinkedValueData().IsSet())
	{
		return BaseFunctionInputAdapter->GetLinkedValueData().GetValue() == OtherFunctionInputAdapter->GetLinkedValueData().GetValue();
	}

	// Data value
	if ((BaseFunctionInputAdapter->GetDataInterfaceValueInputName().IsSet() && OtherFunctionInputAdapter->GetDataInterfaceValueInputName().IsSet() == false) ||
		(BaseFunctionInputAdapter->GetDataInterfaceValueInputName().IsSet() == false && OtherFunctionInputAdapter->GetDataInterfaceValueInputName().IsSet()) ||
		(BaseFunctionInputAdapter->GetDataInterfaceValue() != nullptr && OtherFunctionInputAdapter->GetDataInterfaceValue() == nullptr) ||
		(BaseFunctionInputAdapter->GetDataInterfaceValue() == nullptr && OtherFunctionInputAdapter->GetDataInterfaceValue() != nullptr))
	{
		return false;
	}

	if (BaseFunctionInputAdapter->GetDataInterfaceValueInputName().IsSet() && OtherFunctionInputAdapter->GetDataInterfaceValueInputName().IsSet() &&
		BaseFunctionInputAdapter->GetDataInterfaceValue() != nullptr && OtherFunctionInputAdapter->GetDataInterfaceValue() != nullptr)
	{
		return 
			BaseFunctionInputAdapter->GetDataInterfaceValueInputName().GetValue() == OtherFunctionInputAdapter->GetDataInterfaceValueInputName().GetValue() &&
			BaseFunctionInputAdapter->GetDataInterfaceValue()->Equals(OtherFunctionInputAdapter->GetDataInterfaceValue());
	}

	// Object Reference
	if (BaseFunctionInputAdapter->GetObjectAssetInputVariable().IsSet() != OtherFunctionInputAdapter->GetObjectAssetInputVariable().IsSet())
	{
		return false;
	}

	if (BaseFunctionInputAdapter->GetObjectAssetInputVariable().IsSet() && OtherFunctionInputAdapter->GetObjectAssetInputVariable().IsSet())
	{
		return
			BaseFunctionInputAdapter->GetObjectAssetInputVariable().GetValue() == OtherFunctionInputAdapter->GetObjectAssetInputVariable().GetValue() &&
			BaseFunctionInputAdapter->GetObjectAssetValue() == OtherFunctionInputAdapter->GetObjectAssetValue();
	}

	// Dynamic value
	if ((BaseFunctionInputAdapter->GetDynamicValueFunction().IsValid() && OtherFunctionInputAdapter->GetDynamicValueFunction().IsValid() == false) ||
		(BaseFunctionInputAdapter->GetDynamicValueFunction().IsValid() == false && OtherFunctionInputAdapter->GetDynamicValueFunction().IsValid()))
	{
		return false;
	}

	if (BaseFunctionInputAdapter->GetDynamicValueFunction().IsValid() && OtherFunctionInputAdapter->GetDynamicValueFunction().IsValid())
	{
		TSharedRef<FNiagaraStackFunctionMergeAdapter> BaseDynamicValueFunction = BaseFunctionInputAdapter->GetDynamicValueFunction().ToSharedRef();
		TSharedRef<FNiagaraStackFunctionMergeAdapter> OtherDynamicValueFunction = OtherFunctionInputAdapter->GetDynamicValueFunction().ToSharedRef();

		UNiagaraNodeCustomHlsl* BaseCustomHlsl = Cast<UNiagaraNodeCustomHlsl>(BaseDynamicValueFunction->GetFunctionCallNode());
		UNiagaraNodeCustomHlsl* OtherCustomHlsl = Cast<UNiagaraNodeCustomHlsl>(OtherDynamicValueFunction->GetFunctionCallNode());
		if (BaseCustomHlsl != nullptr || OtherCustomHlsl != nullptr)
		{
			if ((BaseCustomHlsl != nullptr && OtherCustomHlsl == nullptr) ||
				(BaseCustomHlsl == nullptr && OtherCustomHlsl != nullptr))
			{
				return false;
			}

			if (BaseCustomHlsl->GetCustomHlsl() != OtherCustomHlsl->GetCustomHlsl() || BaseCustomHlsl->ScriptUsage != OtherCustomHlsl->ScriptUsage)
			{
				return false;
			}
		}
		else if (BaseDynamicValueFunction->GetUsesScratchPadScript() || OtherDynamicValueFunction->GetUsesScratchPadScript())
		{
			if ((BaseDynamicValueFunction->GetUsesScratchPadScript() && OtherDynamicValueFunction->GetUsesScratchPadScript() == false) ||
				(BaseDynamicValueFunction->GetUsesScratchPadScript() == false && OtherDynamicValueFunction->GetUsesScratchPadScript()))
			{
				return false;
			}

			if (BaseDynamicValueFunction->GetFunctionCallNode()->NodeGuid != OtherDynamicValueFunction->GetFunctionCallNode()->NodeGuid)
			{
				return false;
			}
		}
		else if (BaseDynamicValueFunction->GetFunctionCallNode()->FunctionScript != OtherDynamicValueFunction->GetFunctionCallNode()->FunctionScript)
		{
			return false;
		}

		FNiagaraScriptStackDiffResults FunctionDiffResults;
		DiffFunctionInputs(BaseDynamicValueFunction, OtherDynamicValueFunction, FunctionDiffResults);

		return
			FunctionDiffResults.RemovedBaseInputOverrides.Num() == 0 &&
			FunctionDiffResults.AddedOtherInputOverrides.Num() == 0 &&
			FunctionDiffResults.ModifiedOtherInputOverrides.Num() == 0;
	}

	// Static switch
	if (BaseFunctionInputAdapter->GetStaticSwitchValue().IsSet() && OtherFunctionInputAdapter->GetStaticSwitchValue().IsSet())
	{
		return BaseFunctionInputAdapter->GetStaticSwitchValue().GetValue() == OtherFunctionInputAdapter->GetStaticSwitchValue().GetValue();
	}

	return TOptional<bool>();
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::AddModule(
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
	TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
	UNiagaraScript& OwningScript,
	UNiagaraNodeOutput& TargetOutputNode,
	TSharedRef<FNiagaraStackFunctionMergeAdapter> AddModule) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::AddModule);

	FApplyDiffResults Results;

	UNiagaraNodeFunctionCall* AddedModuleNode = nullptr;
	if (AddModule->GetFunctionCallNode()->IsA<UNiagaraNodeAssignment>())
	{
		UNiagaraNodeAssignment* AssignmentNode = CastChecked<UNiagaraNodeAssignment>(AddModule->GetFunctionCallNode());
		const TArray<FNiagaraVariable>& Targets = AssignmentNode->GetAssignmentTargets();
		const TArray<FString>& Defaults = AssignmentNode->GetAssignmentDefaults();
		AddedModuleNode = FNiagaraStackGraphUtilities::AddParameterModuleToStack(Targets, TargetOutputNode, AddModule->GetStackIndex(),Defaults);
		AddedModuleNode->NodeGuid = AddModule->GetFunctionCallNode()->NodeGuid;
		AddedModuleNode->RefreshFromExternalChanges();
		Results.bModifiedGraph = true;
	}
	else
	{
		UNiagaraScript* FunctionScript = nullptr;
		FGuid VersionGuid;
		if (AddModule->GetUsesScratchPadScript())
		{
			FunctionScript = ScratchPadAdapter->GetScratchPadScriptForFunctionId(AddModule->GetFunctionCallNode()->NodeGuid);
		}
		else
		{
			FunctionScript = AddModule->GetFunctionCallNode()->FunctionScript;
			VersionGuid = AddModule->GetFunctionCallNode()->SelectedScriptVersion;
		}

		AddedModuleNode = FNiagaraStackGraphUtilities::AddScriptModuleToStack(FunctionScript, TargetOutputNode, AddModule->GetStackIndex(), AddModule->GetFunctionCallNode()->GetFunctionName(), VersionGuid);
		AddedModuleNode->NodeGuid = AddModule->GetFunctionCallNode()->NodeGuid; // Synchronize the node Guid across runs so that the compile id's sync up.
		Results.bModifiedGraph = true;
	}

	if (AddedModuleNode != nullptr)
	{
		AddedModuleNode->NodeGuid = AddModule->GetFunctionCallNode()->NodeGuid; // Synchronize the node Guid across runs so that the compile id's sync up.

		AddedModuleNode->SetEnabledState(AddModule->GetFunctionCallNode()->GetDesiredEnabledState(), AddModule->GetFunctionCallNode()->HasUserSetTheEnabledState());
		for (TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> InputOverride : AddModule->GetInputOverrides())
		{
			FApplyDiffResults AddInputResults = AddInputOverride(BaseEmitterAdapter, ScratchPadAdapter, OwningScript, *AddedModuleNode, InputOverride);
			Results.bSucceeded &= AddInputResults.bSucceeded;
			Results.bModifiedGraph |= AddInputResults.bModifiedGraph;
			Results.ErrorMessages.Append(AddInputResults.ErrorMessages);
		}
	}
	else
	{
		Results.bSucceeded = false;
		Results.ErrorMessages.Add(LOCTEXT("AddModuleFailed", "Failed to add module from diff."));
	}

	return Results;
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::RemoveInputOverride(UNiagaraScript& OwningScript, TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> OverrideToRemove) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::RemoveInputOverride);

	FApplyDiffResults Results;
	Results.bSucceeded = true;
	Results.bModifiedGraph = false;

	// If there is a dynamic input we need to call remove recursively to make sure that all rapid iteration parameters are removed.
	if (OverrideToRemove->GetDynamicValueFunction().IsValid())
	{
		for (TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> DynamicValueInputOverrideToRemove : OverrideToRemove->GetDynamicValueFunction()->GetInputOverrides())
		{
			FApplyDiffResults DynamicValueInputResults = RemoveInputOverride(OwningScript, DynamicValueInputOverrideToRemove);
			Results.bSucceeded &= DynamicValueInputResults.bSucceeded;
			Results.bModifiedGraph |= DynamicValueInputResults.bModifiedGraph;
			Results.ErrorMessages.Append(DynamicValueInputResults.ErrorMessages);
		}
	}

	if (OverrideToRemove->GetOverridePin() != nullptr && OverrideToRemove->GetOverrideNode() != nullptr)
	{
		FNiagaraStackGraphUtilities::RemoveNodesForStackFunctionInputOverridePin(*OverrideToRemove->GetOverridePin());
		OverrideToRemove->GetOverrideNode()->RemovePin(OverrideToRemove->GetOverridePin());
		Results.bModifiedGraph = true;
	}
	else if (OverrideToRemove->GetLocalValueRapidIterationParameter().IsSet())
	{
		OwningScript.Modify();
		OwningScript.RapidIterationParameters.RemoveParameter(OverrideToRemove->GetLocalValueRapidIterationParameter().GetValue());
	}
	else if (OverrideToRemove->GetStaticSwitchValue().IsSet())
	{
		const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
		UEdGraphPin** StaticSwitchPinPtr = OverrideToRemove->GetOwningFunctionCall()->Pins.FindByPredicate([&OverrideToRemove, &Schema](UEdGraphPin* Pin)
		{
			if (Pin->Direction != EGPD_Input)
			{
				return false;
			}
			FNiagaraVariable PinVariable = Schema->PinToNiagaraVariable(Pin);
			return PinVariable.GetName() == *OverrideToRemove->GetInputName() &&
				PinVariable.GetType() == OverrideToRemove->GetType();
		});

		if (StaticSwitchPinPtr != nullptr && (*StaticSwitchPinPtr)->AutogeneratedDefaultValue.IsEmpty() == false)
		{
			(*StaticSwitchPinPtr)->DefaultValue = (*StaticSwitchPinPtr)->AutogeneratedDefaultValue;
			Results.bModifiedGraph = true;
		}
	}
	else
	{
		Results.bSucceeded = false;
		Results.ErrorMessages.Add(LOCTEXT("RemoveInputOverrideFailed", "Failed to remove input override because it was invalid."));
	}

	return Results;
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::AddInputOverride(
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
	TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
	UNiagaraScript& OwningScript,
	UNiagaraNodeFunctionCall& TargetFunctionCall,
	TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> OverrideToAdd) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::AddInputOverride);

	FApplyDiffResults Results;

	// If an assignment node, make sure that we have an assignment target for the input override.
	UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(&TargetFunctionCall);
	if (AssignmentNode)
	{
		FNiagaraParameterHandle FunctionInputHandle(FNiagaraConstants::ModuleNamespace, *OverrideToAdd->GetInputName());
		UNiagaraNodeAssignment* PreviousVersionAssignmentNode = Cast<UNiagaraNodeAssignment>(OverrideToAdd->GetOwningFunctionCall());
		bool bAnyAdded = false;
		for (int32 i = 0; i < PreviousVersionAssignmentNode->NumTargets(); i++)
		{
			const FNiagaraVariable& Var = PreviousVersionAssignmentNode->GetAssignmentTarget(i);

			int32 FoundVarIdx = AssignmentNode->FindAssignmentTarget(Var.GetName());
			if (FoundVarIdx == INDEX_NONE)
			{
				AssignmentNode->AddAssignmentTarget(Var, &PreviousVersionAssignmentNode->GetAssignmentDefaults()[i]);
				bAnyAdded = true;
			}
		}

		if (bAnyAdded)
		{
			AssignmentNode->RefreshFromExternalChanges();
		}
	}

	FNiagaraParameterHandle FunctionInputHandle(FNiagaraConstants::ModuleNamespace, *OverrideToAdd->GetInputName());
	FNiagaraParameterHandle AliasedFunctionInputHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(FunctionInputHandle, &TargetFunctionCall);

	if (OverrideToAdd->GetOverridePin() != nullptr)
	{
		const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
		FNiagaraTypeDefinition InputType = NiagaraSchema->PinToTypeDefinition(OverrideToAdd->GetOverridePin());

		FNiagaraVariable InputVariable(InputType, FunctionInputHandle.GetParameterHandleString());
		TOptional<FNiagaraVariableMetaData> InputMetaData = TargetFunctionCall.GetNiagaraGraph()->GetMetaData(InputVariable);
		FGuid InputVariableGuid = InputMetaData.IsSet() ? InputMetaData->GetVariableGuid() : FGuid();

		UEdGraphPin& InputOverridePin = FNiagaraStackGraphUtilities::GetOrCreateStackFunctionInputOverridePin(TargetFunctionCall, AliasedFunctionInputHandle, InputType, InputVariableGuid, OverrideToAdd->GetOverrideNode()->NodeGuid);
		if (InputOverridePin.LinkedTo.Num() != 0)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(
				LOCTEXT("AddPinBasedInputOverrideFailedOverridePinStillLinkedFormat", "Failed to add input override because the target override pin was still linked to other nodes.  Target Script Usage: {0} Target Script Usage Id: {1} Target Node: {2} Target Input Handle: {3} Linked Node: {4} Linked Pin: {5}"),
				FNiagaraTypeDefinition::GetScriptUsageEnum()->GetDisplayNameTextByValue((int64)OwningScript.GetUsage()),
				FText::FromString(OwningScript.GetUsageId().ToString(EGuidFormats::DigitsWithHyphens)),
				FText::FromString(TargetFunctionCall.GetFunctionName()),
				FText::FromName(AliasedFunctionInputHandle.GetParameterHandleString()),
				InputOverridePin.LinkedTo[0] != nullptr && InputOverridePin.LinkedTo[0]->GetOwningNode() != nullptr
					? InputOverridePin.LinkedTo[0]->GetOwningNode()->GetNodeTitle(ENodeTitleType::ListView)
					: FText::FromString(TEXT("(null)")),
				InputOverridePin.LinkedTo[0] != nullptr
					? FText::FromName(InputOverridePin.LinkedTo[0]->PinName)
					: FText::FromString(TEXT("(null)"))));
		}
		else
		{
			if (OverrideToAdd->GetLocalValueString().IsSet())
			{
				InputOverridePin.DefaultValue = OverrideToAdd->GetLocalValueString().GetValue();
				Results.bSucceeded = true;
			}
			else if (OverrideToAdd->GetLinkedValueData().IsSet())
			{
				check(OverrideToAdd->GetOverrideNode() && OverrideToAdd->GetOverrideNode()->GetNiagaraGraph());
				FNiagaraVariableBase OldLinkedParameter = OverrideToAdd->GetLinkedValueData()->LinkedParameterValue;
				FNiagaraParameterHandle OldLinkedParameterHandle = FNiagaraParameterHandle(OldLinkedParameter.GetName());
				FGuid LinkedFunctionCallNodeId = OverrideToAdd->GetLinkedValueData()->LinkedFunctionNodeId;
				UNiagaraScriptVariable* ScriptVar = OverrideToAdd->GetOverrideNode()->GetNiagaraGraph()->GetScriptVariable(OldLinkedParameter);
				ENiagaraDefaultMode DesiredMode = ENiagaraDefaultMode::Value;
				if (ScriptVar)
					DesiredMode = ScriptVar->DefaultMode;
				if (GNiagaraForceFailIfPreviouslyNotSetOnMerge > 0)
					DesiredMode = ENiagaraDefaultMode::FailIfPreviouslyNotSet;

				// If the linked value handle has a valid function call node id, then we need to check to see if that function call node has been
				// renamed due to the merge, and if so we need to update the handle.
				FNiagaraVariableBase NewLinkedParameter;
				if (LinkedFunctionCallNodeId.IsValid() &&
					OldLinkedParameterHandle.GetHandleParts().Num() >= 3 && (
						OldLinkedParameterHandle.IsOutputHandle() ||
						OldLinkedParameterHandle.IsStackContextHandle() ||
						OldLinkedParameterHandle.IsEmitterHandle() ||
						OldLinkedParameterHandle.IsParticleAttributeHandle()))
				{
					TObjectPtr<UEdGraphNode>* ReferencedFunctionCallNodePtr = TargetFunctionCall.GetGraph()->Nodes.FindByPredicate(
						[&LinkedFunctionCallNodeId](UEdGraphNode* Node) { return Node->IsA<UNiagaraNodeFunctionCall>() && Node->NodeGuid == LinkedFunctionCallNodeId; });
					if (ReferencedFunctionCallNodePtr != nullptr)
					{
						UNiagaraNodeFunctionCall* ReferencedFunctionCallNode = CastChecked<UNiagaraNodeFunctionCall>(*ReferencedFunctionCallNodePtr);
						FString FunctionName = OldLinkedParameterHandle.GetHandleParts()[1].ToString();
						if (ReferencedFunctionCallNode->GetFunctionName() != FunctionName)
						{
							// The function node has been renamed so we need to update the handle.  The handle is in the format [Namespace].[Function Call Name].[NameParts]
							TArray<FName> OldHandleParts = OldLinkedParameterHandle.GetHandleParts();
							TArray<FString> NewHandleNameParts;
							NewHandleNameParts.Add(OldHandleParts[0].ToString());
							NewHandleNameParts.Add(ReferencedFunctionCallNode->GetFunctionName());
							for (int32 i = 2; i < OldHandleParts.Num(); i++)
							{
								NewHandleNameParts.Add(OldHandleParts[i].ToString());
							}
							NewLinkedParameter = FNiagaraVariableBase(OldLinkedParameter.GetType(), *FString::Join(NewHandleNameParts, TEXT(".")));
						}
					}
				}
				if (NewLinkedParameter.IsValid() == false)
				{
					NewLinkedParameter = OldLinkedParameter;
				}

				TSet KnownParameters = { NewLinkedParameter };
				FNiagaraStackGraphUtilities::SetLinkedParameterValueForFunctionInput(InputOverridePin, NewLinkedParameter, KnownParameters, DesiredMode, OverrideToAdd->GetOverrideNodeId());
				FGuid LinkedOutputId = FNiagaraStackGraphUtilities::GetScriptVariableIdForLinkedModuleParameter(NewLinkedParameter, *TargetFunctionCall.GetNiagaraGraph());
				if (LinkedOutputId.IsValid())
				{
					TargetFunctionCall.UpdateInputNameBinding(LinkedOutputId, NewLinkedParameter.GetName());
				}
				Results.bSucceeded = true;
			}
			else if (OverrideToAdd->GetDataInterfaceValueInputName().IsSet() && OverrideToAdd->GetDataInterfaceValue() != nullptr)
			{
				FName OverrideValueInputName = OverrideToAdd->GetDataInterfaceValueInputName().GetValue();
				UNiagaraDataInterface* OverrideValueObject = OverrideToAdd->GetDataInterfaceValue();
				UNiagaraDataInterface* NewOverrideValueObject;
				FNiagaraStackGraphUtilities::SetDataInterfaceValueForFunctionInput(InputOverridePin, OverrideToAdd->GetDataInterfaceValue()->GetClass(), OverrideValueInputName.ToString(), NewOverrideValueObject, OverrideToAdd->GetOverrideNodeId());
				OverrideValueObject->CopyTo(NewOverrideValueObject);
				Results.bSucceeded = true;
			}
			else if (OverrideToAdd->GetObjectAssetInputVariable().IsSet())
			{
				FNiagaraVariableBase OverrideVariable = OverrideToAdd->GetObjectAssetInputVariable().GetValue();
				UObject* OverrideObjectValue = OverrideToAdd->GetObjectAssetValue();
				FNiagaraStackGraphUtilities::SetObjectAssetValueForFunctionInput(InputOverridePin, OverrideVariable.GetType().GetClass(), OverrideVariable.GetName().ToString(), OverrideObjectValue);
				Results.bSucceeded = true;
			}
			else if (OverrideToAdd->GetDynamicValueFunction().IsValid())
			{
				UNiagaraNodeCustomHlsl* CustomHlslFunction = Cast<UNiagaraNodeCustomHlsl>(OverrideToAdd->GetDynamicValueFunction()->GetFunctionCallNode());
				if (CustomHlslFunction != nullptr)
				{
					UNiagaraNodeCustomHlsl* DynamicInputFunctionCall;
					FNiagaraStackGraphUtilities::SetCustomExpressionForFunctionInput(InputOverridePin, CustomHlslFunction->GetCustomHlsl(), DynamicInputFunctionCall, OverrideToAdd->GetOverrideNodeId());
					for (TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> DynamicInputInputOverride : OverrideToAdd->GetDynamicValueFunction()->GetInputOverrides())
					{
						FApplyDiffResults AddResults = AddInputOverride(BaseEmitterAdapter, ScratchPadAdapter, OwningScript, *((UNiagaraNodeFunctionCall*)DynamicInputFunctionCall), DynamicInputInputOverride);
						Results.bSucceeded &= AddResults.bSucceeded;
						Results.bModifiedGraph |= AddResults.bModifiedGraph;
						Results.ErrorMessages.Append(AddResults.ErrorMessages);
					}
				}
				else if (OverrideToAdd->GetDynamicValueFunction()->GetFunctionCallNode())
				{
					UNiagaraScript* FunctionScript = nullptr;
					if (OverrideToAdd->GetDynamicValueFunction()->GetUsesScratchPadScript())
					{
						FunctionScript = ScratchPadAdapter->GetScratchPadScriptForFunctionId(OverrideToAdd->GetOverrideNodeId());
					}
					else
					{
						FunctionScript = OverrideToAdd->GetDynamicValueFunction()->GetFunctionCallNode()->FunctionScript;
					}

					UNiagaraNodeFunctionCall* DynamicInputFunctionCall;
					FNiagaraStackGraphUtilities::SetDynamicInputForFunctionInput(InputOverridePin, FunctionScript, 
						DynamicInputFunctionCall, OverrideToAdd->GetOverrideNodeId(), OverrideToAdd->GetDynamicValueFunction()->GetFunctionCallNode()->GetFunctionName(), OverrideToAdd->GetDynamicValueFunction()->GetFunctionCallNode()->SelectedScriptVersion);
					for (TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> DynamicInputInputOverride : OverrideToAdd->GetDynamicValueFunction()->GetInputOverrides())
					{
						FApplyDiffResults AddResults = AddInputOverride(BaseEmitterAdapter, ScratchPadAdapter, OwningScript, *DynamicInputFunctionCall, DynamicInputInputOverride);
						Results.bSucceeded &= AddResults.bSucceeded;
						Results.bModifiedGraph |= AddResults.bModifiedGraph;
						Results.ErrorMessages.Append(AddResults.ErrorMessages);
					}
				}
			}
			else
			{
				Results.bSucceeded = false;
				Results.ErrorMessages.Add(FText::Format(LOCTEXT("AddPinBasedInputOverrideFailed", "Failed to add input override {0} to function {1} because it was invalid."),
					FText::FromString(OverrideToAdd->GetInputName()), FText::FromString(TargetFunctionCall.GetFunctionName())));
			}
		}
		Results.bModifiedGraph = true;
	}
	else
	{
		if (OverrideToAdd->GetLocalValueRapidIterationParameter().IsSet())
		{
			FNiagaraVariable RapidIterationParameter = FNiagaraStackGraphUtilities::CreateRapidIterationParameter(
				BaseEmitterAdapter->GetEditableEmitter().Emitter->GetUniqueEmitterName(), OwningScript.GetUsage(), AliasedFunctionInputHandle.GetParameterHandleString(), OverrideToAdd->GetLocalValueRapidIterationParameter().GetValue().GetType());
			const uint8* SourceData = OverrideToAdd->GetOwningScript()->RapidIterationParameters.GetParameterData(OverrideToAdd->GetLocalValueRapidIterationParameter().GetValue());
			OwningScript.Modify();
			bool bAddParameterIfMissing = true;
			OwningScript.RapidIterationParameters.SetParameterData(SourceData, RapidIterationParameter, bAddParameterIfMissing);
			Results.bSucceeded = true;
		}
		else if (OverrideToAdd->GetStaticSwitchValue().IsSet())
		{
			TArray<UEdGraphPin*> StaticSwitchPins;
			TSet<UEdGraphPin*> StaticSwitchPinsHidden;
			FCompileConstantResolver ConstantResolver(BaseEmitterAdapter->GetEditableEmitter(), OwningScript.GetUsage());
			FNiagaraStackGraphUtilities::GetStackFunctionStaticSwitchPins(TargetFunctionCall, StaticSwitchPins, StaticSwitchPinsHidden, ConstantResolver);
			UEdGraphPin** MatchingStaticSwitchPinPtr = StaticSwitchPins.FindByPredicate([&OverrideToAdd](UEdGraphPin* StaticSwitchPin) { return StaticSwitchPin->PinName == *OverrideToAdd->GetInputName(); });
			if (MatchingStaticSwitchPinPtr != nullptr)
			{
				UEdGraphPin* MatchingStaticSwitchPin = *MatchingStaticSwitchPinPtr;
				const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
				FNiagaraTypeDefinition SwitchType = NiagaraSchema->PinToTypeDefinition(MatchingStaticSwitchPin);
				if (SwitchType == OverrideToAdd->GetType())
				{
					MatchingStaticSwitchPin->DefaultValue = OverrideToAdd->GetStaticSwitchValue().GetValue();
					TargetFunctionCall.MarkNodeRequiresSynchronization(TEXT("Static Switch Value Changed"), true);
					Results.bModifiedGraph = true;
					Results.bSucceeded = true;
				}
				else
				{
					Results.bSucceeded = false;
					Results.ErrorMessages.Add(FText::Format(LOCTEXT("AddStaticInputOverrideFailedWrongTypeFormat", "Failed to add static switch input override {0} to function {1} because a the type of the pin matched by name did not match."),
						FText::FromString(OverrideToAdd->GetInputName()), FText::FromString(TargetFunctionCall.GetFunctionName())));
				}
			}
			else
			{
				FNiagaraVariableBase StaticSwitchVariable(OverrideToAdd->GetType(), *OverrideToAdd->GetInputName());
				TargetFunctionCall.AddOrphanedStaticSwitchPinForDataRetention(StaticSwitchVariable, OverrideToAdd->GetStaticSwitchValue().GetValue());
				Results.bSucceeded = true;
			}
		}
		else
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(LOCTEXT("AddParameterBasedInputOverrideFailedFormat", "Failed to add input override {0} to function {1} because it was invalid."),
				FText::FromString(OverrideToAdd->GetInputName()), FText::FromString(TargetFunctionCall.GetFunctionName())));
		}
		Results.bModifiedGraph = false;
	}
	return Results;
}

void FNiagaraScriptMergeManager::CopyPropertiesToBase(void* BaseDataAddress, const void* OtherDataAddress, TArray<FProperty*> PropertiesToCopy) const
{
	for (FProperty* PropertyToCopy : PropertiesToCopy)
	{
		PropertyToCopy->CopyCompleteValue(PropertyToCopy->ContainerPtrToValuePtr<void>(BaseDataAddress), PropertyToCopy->ContainerPtrToValuePtr<void>(OtherDataAddress));
	}
}

void FNiagaraScriptMergeManager::CopyInstanceScratchPadScripts(const FVersionedNiagaraEmitter& MergedInstance, FVersionedNiagaraEmitterData* SourceInstance) const
{
	TObjectPtr<UNiagaraScratchPadContainer> ScratchPadContainer = MergedInstance.GetEmitterData()->ScratchPads;
	for (UNiagaraScript* SourceScratchPadScript : SourceInstance->ScratchPads->Scripts)
	{
		FName UniqueObjectName = FNiagaraEditorUtilities::GetUniqueObjectName<UNiagaraScript>(ScratchPadContainer, SourceScratchPadScript->GetName());
		UNiagaraScript* MergedInstanceScratchPadScript = CastChecked<UNiagaraScript>(StaticDuplicateObject(SourceScratchPadScript, ScratchPadContainer, UniqueObjectName));
		ScratchPadContainer->Scripts.Add(MergedInstanceScratchPadScript);
	}
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::ApplyScriptStackDiff(
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
	TSharedRef<FNiagaraScriptStackMergeAdapter> BaseScriptStackAdapter,
	TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
	const FNiagaraScriptStackDiffResults& DiffResults,
	const bool bNoParentAtLastMerge) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::ApplyScriptStackDiff);

	FApplyDiffResults Results;

	if (DiffResults.IsEmpty())
	{
		Results.bSucceeded = true;
		Results.bModifiedGraph = false;
		return Results;
	}

	struct FAddInputOverrideActionData
	{
		UNiagaraNodeFunctionCall* TargetFunctionCall;
		TSharedPtr<FNiagaraStackFunctionInputOverrideMergeAdapter> OverrideToAdd;
	};

	// Collect the graph actions from the adapter and diff first.
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> RemoveModules;
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> AddModules;
	TArray<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>> RemoveInputOverrides;
	TArray<FAddInputOverrideActionData> AddInputOverrideActionDatas;
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> EnableModules;
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> DisableModules;

	for (TSharedRef<FNiagaraStackFunctionMergeAdapter> RemovedModule : DiffResults.RemovedBaseModules)
	{
		TSharedPtr<FNiagaraStackFunctionMergeAdapter> MatchingModuleAdapter = BaseScriptStackAdapter->GetModuleFunctionById(RemovedModule->GetFunctionCallNode()->NodeGuid);
		if (MatchingModuleAdapter.IsValid())
		{
			if (bNoParentAtLastMerge)
			{
				// If there is no last known parent we don't know if the module was removed in the child, or added in the parent, so
				// instead of removing the parent module we disable it in this case, since removing modules in child emitters isn't
				// supported through the UI.
				DisableModules.Add(MatchingModuleAdapter.ToSharedRef());
			}
			else
			{
				RemoveModules.Add(MatchingModuleAdapter.ToSharedRef());
			}
		}
	}

	AddModules.Append(DiffResults.AddedOtherModules);

	for (TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> RemovedInputOverrideAdapter : DiffResults.RemovedBaseInputOverrides)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraStackGraphUtilities::RemovedInputOverrideAdapter);

		TSharedPtr<FNiagaraStackFunctionMergeAdapter> MatchingModuleAdapter = BaseScriptStackAdapter->GetModuleFunctionById(RemovedInputOverrideAdapter->GetOwningFunctionCall()->NodeGuid);
		if (MatchingModuleAdapter.IsValid())
		{
			TSharedPtr<FNiagaraStackFunctionInputOverrideMergeAdapter> MatchingInputOverrideAdapter =
				MatchingModuleAdapter->GetInputOverrideByInputNameAndType(RemovedInputOverrideAdapter->GetInputName(), RemovedInputOverrideAdapter->GetType());
			if (MatchingInputOverrideAdapter.IsValid())
			{
				RemoveInputOverrides.Add(MatchingInputOverrideAdapter.ToSharedRef());
			}
		}
	}

	for (TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> AddedInputOverrideAdapter : DiffResults.AddedOtherInputOverrides)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraStackGraphUtilities::AddedInputOverrideAdapter);

		TSharedPtr<FNiagaraStackFunctionMergeAdapter> MatchingModuleAdapter = BaseScriptStackAdapter->GetModuleFunctionById(AddedInputOverrideAdapter->GetOwningFunctionCall()->NodeGuid);
		if (MatchingModuleAdapter.IsValid())
		{
			TSharedPtr<FNiagaraStackFunctionInputOverrideMergeAdapter> MatchingInputOverrideAdapter = 
				MatchingModuleAdapter->GetInputOverrideByInputNameAndType(AddedInputOverrideAdapter->GetInputName(), AddedInputOverrideAdapter->GetType());
			if (MatchingInputOverrideAdapter.IsValid())
			{
				RemoveInputOverrides.AddUnique(MatchingInputOverrideAdapter.ToSharedRef());
			}

			FAddInputOverrideActionData AddInputOverrideActionData;
			AddInputOverrideActionData.TargetFunctionCall = MatchingModuleAdapter->GetFunctionCallNode();
			AddInputOverrideActionData.OverrideToAdd = AddedInputOverrideAdapter;
			AddInputOverrideActionDatas.Add(AddInputOverrideActionData);
		}
	}

	for (TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> ModifiedInputOverrideAdapter : DiffResults.ModifiedOtherInputOverrides)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraStackGraphUtilities::ModifiedInputOverrideAdapter);

		TSharedPtr<FNiagaraStackFunctionMergeAdapter> MatchingModuleAdapter = BaseScriptStackAdapter->GetModuleFunctionById(ModifiedInputOverrideAdapter->GetOwningFunctionCall()->NodeGuid);
		if (MatchingModuleAdapter.IsValid())
		{
			TSharedPtr<FNiagaraStackFunctionInputOverrideMergeAdapter> MatchingInputOverrideAdapter = 
				MatchingModuleAdapter->GetInputOverrideByInputNameAndType(ModifiedInputOverrideAdapter->GetInputName(), ModifiedInputOverrideAdapter->GetType());
			if (MatchingInputOverrideAdapter.IsValid())
			{
				RemoveInputOverrides.AddUnique(MatchingInputOverrideAdapter.ToSharedRef());
			}

			FAddInputOverrideActionData AddInputOverrideActionData;
			AddInputOverrideActionData.TargetFunctionCall = MatchingModuleAdapter->GetFunctionCallNode();
			AddInputOverrideActionData.OverrideToAdd = ModifiedInputOverrideAdapter;
			AddInputOverrideActionDatas.Add(AddInputOverrideActionData);
		}
	}

	for (TSharedRef<FNiagaraStackFunctionMergeAdapter> EnabledChangedModule : DiffResults.EnabledChangedOtherModules)
	{
		TSharedPtr<FNiagaraStackFunctionMergeAdapter> MatchingModuleAdapter = BaseScriptStackAdapter->GetModuleFunctionById(EnabledChangedModule->GetFunctionCallNode()->NodeGuid);
		if (MatchingModuleAdapter.IsValid())
		{
			if (EnabledChangedModule->GetFunctionCallNode()->IsNodeEnabled())
			{
				EnableModules.Add(MatchingModuleAdapter.ToSharedRef());
			}
			else
			{
				DisableModules.Add(MatchingModuleAdapter.ToSharedRef());
			}
		}
	}
	
	// Update the usage if different
	if (DiffResults.ChangedOtherUsage.IsSet())
	{
		BaseScriptStackAdapter->GetScript()->SetUsage(DiffResults.ChangedOtherUsage.GetValue());
		BaseScriptStackAdapter->GetOutputNode()->SetUsage(DiffResults.ChangedOtherUsage.GetValue());
	}

	// Apply the graph actions.
	for (TSharedRef<FNiagaraStackFunctionMergeAdapter> RemoveModule : RemoveModules)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraStackGraphUtilities::RemoveModuleFromStack);

		bool bRemoveResults = FNiagaraStackGraphUtilities::RemoveModuleFromStack(*BaseScriptStackAdapter->GetScript(), *RemoveModule->GetFunctionCallNode());
		if (bRemoveResults == false)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(LOCTEXT("RemoveModuleFailedMessage", "Failed to remove module while applying diff"));
		}
		else
		{
			Results.bModifiedGraph = true;
		}
	}

	for (TSharedRef<FNiagaraStackFunctionMergeAdapter> AddModuleAdapter : AddModules)
	{
		FApplyDiffResults AddModuleResults = AddModule(BaseEmitterAdapter, ScratchPadAdapter, *BaseScriptStackAdapter->GetScript(), *BaseScriptStackAdapter->GetOutputNode(), AddModuleAdapter);
		Results.bSucceeded &= AddModuleResults.bSucceeded;
		Results.bModifiedGraph |= AddModuleResults.bModifiedGraph;
		Results.ErrorMessages.Append(AddModuleResults.ErrorMessages);
	}

	for (TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> RemoveInputOverrideItem : RemoveInputOverrides)
	{
		FApplyDiffResults RemoveInputOverrideResults = RemoveInputOverride(*BaseScriptStackAdapter->GetScript(), RemoveInputOverrideItem);
		Results.bSucceeded &= RemoveInputOverrideResults.bSucceeded;
		Results.bModifiedGraph |= RemoveInputOverrideResults.bModifiedGraph;
		Results.ErrorMessages.Append(RemoveInputOverrideResults.ErrorMessages);
	}

	for (const FAddInputOverrideActionData& AddInputOverrideActionData : AddInputOverrideActionDatas)
	{
		FApplyDiffResults AddInputOverrideResults = AddInputOverride(BaseEmitterAdapter, ScratchPadAdapter, *BaseScriptStackAdapter->GetScript(),
			*AddInputOverrideActionData.TargetFunctionCall, AddInputOverrideActionData.OverrideToAdd.ToSharedRef());
		Results.bSucceeded &= AddInputOverrideResults.bSucceeded;
		Results.bModifiedGraph |= AddInputOverrideResults.bModifiedGraph;
		Results.ErrorMessages.Append(AddInputOverrideResults.ErrorMessages);
	}

	// Apply enabled state last so that it applies to function calls added  from input overrides;
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::SetModuleIsEnabled);
	for (TSharedRef<FNiagaraStackFunctionMergeAdapter> EnableModule : EnableModules)
	{
		FNiagaraStackGraphUtilities::SetModuleIsEnabled(*EnableModule->GetFunctionCallNode(), true);
	}
	for (TSharedRef<FNiagaraStackFunctionMergeAdapter> DisableModule : DisableModules)
	{
		FNiagaraStackGraphUtilities::SetModuleIsEnabled(*DisableModule->GetFunctionCallNode(), false);
	}

	return Results;
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::ApplyEventHandlerDiff(
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
	TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
	const FNiagaraEmitterDiffResults& DiffResults,
	const bool bNoParentAtLastMerge) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::ApplyEventHandlerDiff);

	FApplyDiffResults Results;
	if (DiffResults.RemovedBaseEventHandlers.Num() > 0)
	{
		// If this becomes supported, it needs to handle the bNoParentAtLastMerge case
		Results.bSucceeded = false;
		Results.bModifiedGraph = false;
		Results.ErrorMessages.Add(LOCTEXT("RemovedEventHandlersUnsupported", "Apply diff failed, removed event handlers are currently unsupported."));
		return Results;
	}

	// Apply the modifications first since adding new event handlers may invalidate the adapter.
	for (const FNiagaraModifiedEventHandlerDiffResults& ModifiedEventHandler : DiffResults.ModifiedEventHandlers)
	{
		if (ModifiedEventHandler.OtherAdapter->GetEventScriptProperties() == nullptr)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(
				LOCTEXT("MissingModifiedEventPropertiesFormat", "Apply diff failed.  The modified event handler with id: {0} was missing it's event properties."),
				FText::FromString(ModifiedEventHandler.OtherAdapter->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens))));
		}
		else if (ModifiedEventHandler.OtherAdapter->GetOutputNode() == nullptr)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(
				LOCTEXT("MissingModifiedEventOutputNodeFormat", "Apply diff failed.  The modified event handler with id: {0} was missing it's output node."),
				FText::FromString(ModifiedEventHandler.OtherAdapter->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens))));
		}
		else
		{
			TSharedPtr<FNiagaraEventHandlerMergeAdapter> MatchingBaseEventHandlerAdapter = BaseEmitterAdapter->GetEventHandler(ModifiedEventHandler.OtherAdapter->GetUsageId());
			if (MatchingBaseEventHandlerAdapter.IsValid())
			{
				if (ModifiedEventHandler.ChangedProperties.Num() > 0)
				{
					CopyPropertiesToBase(MatchingBaseEventHandlerAdapter->GetEditableEventScriptProperties(), ModifiedEventHandler.OtherAdapter->GetEditableEventScriptProperties(), ModifiedEventHandler.ChangedProperties);
				}
				if (ModifiedEventHandler.ScriptDiffResults.IsEmpty() == false)
				{
					FApplyDiffResults ApplyEventHandlerStackDiffResults = ApplyScriptStackDiff(
						BaseEmitterAdapter, MatchingBaseEventHandlerAdapter->GetEventStack().ToSharedRef(),	ScratchPadAdapter,
						ModifiedEventHandler.ScriptDiffResults, bNoParentAtLastMerge);
					Results.bSucceeded &= ApplyEventHandlerStackDiffResults.bSucceeded;
					Results.bModifiedGraph |= ApplyEventHandlerStackDiffResults.bModifiedGraph;
					Results.ErrorMessages.Append(ApplyEventHandlerStackDiffResults.ErrorMessages);
				}
			}
		}
	}

	UNiagaraScriptSource* EmitterSource = CastChecked<UNiagaraScriptSource>(BaseEmitterAdapter->GetEditableEmitter().GetEmitterData()->GraphSource);
	UNiagaraGraph* EmitterGraph = EmitterSource->NodeGraph;
	for (TSharedRef<FNiagaraEventHandlerMergeAdapter> AddedEventHandler : DiffResults.AddedOtherEventHandlers)
	{
		if (AddedEventHandler->GetEventScriptProperties() == nullptr)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(
				LOCTEXT("MissingAddedEventPropertiesFormat", "Apply diff failed.  The added event handler with id: {0} was missing it's event properties."),
				FText::FromString(AddedEventHandler->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens))));
		}
		else if (AddedEventHandler->GetOutputNode() == nullptr)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(
				LOCTEXT("MissingAddedEventOutputNodeFormat", "Apply diff failed.  The added event handler with id: {0} was missing it's output node."),
				FText::FromString(AddedEventHandler->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens))));
		}
		else
		{
			FVersionedNiagaraEmitter BaseEmitter = BaseEmitterAdapter->GetEditableEmitter();
			FNiagaraEventScriptProperties AddedEventScriptProperties = *AddedEventHandler->GetEventScriptProperties();
			AddedEventScriptProperties.Script = NewObject<UNiagaraScript>(BaseEmitter.Emitter, MakeUniqueObjectName(BaseEmitter.Emitter, UNiagaraScript::StaticClass(), "EventScript"),
			                                                              RF_Transactional);
			AddedEventScriptProperties.Script->SetUsage(ENiagaraScriptUsage::ParticleEventScript);
			AddedEventScriptProperties.Script->SetUsageId(AddedEventHandler->GetUsageId());
			AddedEventScriptProperties.Script->SetLatestSource(EmitterSource);
			BaseEmitter.Emitter->AddEventHandler(AddedEventScriptProperties, BaseEmitter.Version);

			FGuid PreferredOutputNodeGuid = AddedEventHandler->GetOutputNode()->NodeGuid;
			FGuid PreferredInputNodeGuid = AddedEventHandler->GetInputNode()->NodeGuid;
			UNiagaraNodeOutput* EventOutputNode = FNiagaraStackGraphUtilities::ResetGraphForOutput(*EmitterGraph, ENiagaraScriptUsage::ParticleEventScript, AddedEventScriptProperties.Script->GetUsageId(), PreferredOutputNodeGuid, PreferredInputNodeGuid);
			for (TSharedRef<FNiagaraStackFunctionMergeAdapter> ModuleAdapter : AddedEventHandler->GetEventStack()->GetModuleFunctions())
			{
				FApplyDiffResults AddModuleResults = AddModule(BaseEmitterAdapter, ScratchPadAdapter, *AddedEventScriptProperties.Script, *EventOutputNode, ModuleAdapter);
				Results.bSucceeded &= AddModuleResults.bSucceeded;
				Results.ErrorMessages.Append(AddModuleResults.ErrorMessages);
			}

			// Force the base compile id of the new event handler to match the added instance event handler.
			UNiagaraScriptSource* AddedEventScriptSourceFromDiff = Cast<UNiagaraScriptSource>(AddedEventHandler->GetEventScriptProperties()->Script->GetLatestSource());
			UNiagaraGraph* AddedEventScriptGraphFromDiff = AddedEventScriptSourceFromDiff->NodeGraph;
			FGuid ScriptBaseIdFromDiff = AddedEventScriptGraphFromDiff->GetBaseId(ENiagaraScriptUsage::ParticleEventScript, AddedEventHandler->GetUsageId());
			UNiagaraScriptSource* AddedEventScriptSource = Cast<UNiagaraScriptSource>(AddedEventScriptProperties.Script->GetLatestSource());
			UNiagaraGraph* AddedEventScriptGraph = AddedEventScriptSource->NodeGraph;
			AddedEventScriptGraph->ForceBaseId(ENiagaraScriptUsage::ParticleEventScript, AddedEventHandler->GetUsageId(), ScriptBaseIdFromDiff);

			Results.bModifiedGraph = true;
		}
	}
	return Results;
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::ApplySimulationStageDiff(
	TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
	TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
	const FNiagaraEmitterDiffResults& DiffResults,
	const bool bNoParentAtLastMerge) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::ApplySimulationStageDiff);

	FApplyDiffResults Results;
	if (DiffResults.RemovedBaseSimulationStages.Num() > 0)
	{
		Results.bSucceeded = false;
		Results.bModifiedGraph = false;
		// If this becomes supported, it needs to handle the bNoParentAtLastMerge case
		Results.ErrorMessages.Add(LOCTEXT("RemovedSimulationStagesUnsupported", "Apply diff failed, removed shader stages are currently unsupported."));
		return Results;
	}

	for (const FNiagaraModifiedSimulationStageDiffResults& ModifiedSimulationStage : DiffResults.ModifiedSimulationStages)
	{
		if (ModifiedSimulationStage.OtherAdapter->GetSimulationStage() == nullptr)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(
				LOCTEXT("MissingModifiedSimulationStageObjectFormat", "Apply diff failed.  The modified shader stage with id: {0} was missing it's shader stage object."),
				FText::FromString(ModifiedSimulationStage.OtherAdapter->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens))));
		}
		else if (ModifiedSimulationStage.OtherAdapter->GetOutputNode() == nullptr)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(
				LOCTEXT("MissingModifiedSimulationStageOutputNodeFormat", "Apply diff failed.  The modified shader stage with id: {0} was missing it's output node."),
				FText::FromString(ModifiedSimulationStage.OtherAdapter->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens))));
		}
		else
		{
			TSharedPtr<FNiagaraSimulationStageMergeAdapter> MatchingBaseSimulationStageAdapter = BaseEmitterAdapter->GetSimulationStage(ModifiedSimulationStage.OtherAdapter->GetUsageId());
			if (MatchingBaseSimulationStageAdapter.IsValid())
			{
				if (ModifiedSimulationStage.ChangedProperties.Num() > 0)
				{
					CopyPropertiesToBase(MatchingBaseSimulationStageAdapter->GetEditableSimulationStage(), ModifiedSimulationStage.OtherAdapter->GetEditableSimulationStage(), ModifiedSimulationStage.ChangedProperties);
				}
				if (ModifiedSimulationStage.ScriptDiffResults.IsEmpty() == false)
				{
					FApplyDiffResults ApplySimulationStageStackDiffResults = ApplyScriptStackDiff(
						BaseEmitterAdapter,	MatchingBaseSimulationStageAdapter->GetSimulationStageStack().ToSharedRef(), ScratchPadAdapter,
						ModifiedSimulationStage.ScriptDiffResults, bNoParentAtLastMerge);
					Results.bSucceeded &= ApplySimulationStageStackDiffResults.bSucceeded;
					Results.bModifiedGraph |= ApplySimulationStageStackDiffResults.bModifiedGraph;
					Results.ErrorMessages.Append(ApplySimulationStageStackDiffResults.ErrorMessages);
				}
			}
		}
	}

	FVersionedNiagaraEmitterData* BaseEmitterData = BaseEmitterAdapter->GetEditableEmitter().GetEmitterData();
	UNiagaraScriptSource* EmitterSource = CastChecked<UNiagaraScriptSource>(BaseEmitterData->GraphSource);
	UNiagaraGraph* EmitterGraph = EmitterSource->NodeGraph;
	for (TSharedRef<FNiagaraSimulationStageMergeAdapter> AddedOtherSimulationStage : DiffResults.AddedOtherSimulationStages)
	{
		if (AddedOtherSimulationStage->GetSimulationStage() == nullptr)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(
				LOCTEXT("MissingAddedSimulationStageObjectFormat", "Apply diff failed.  The added shader stage with id: {0} was missing it's shader stage object."),
				FText::FromString(AddedOtherSimulationStage->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens))));
		}
		else if (AddedOtherSimulationStage->GetOutputNode() == nullptr)
		{
			Results.bSucceeded = false;
			Results.ErrorMessages.Add(FText::Format(
				LOCTEXT("MissingAddedSimulationStageOutputNodeFormat", "Apply diff failed.  The added shader stage with id: {0} was missing it's output node."),
				FText::FromString(AddedOtherSimulationStage->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens))));
		}
		else
		{
			UNiagaraEmitter* BaseEmitter = BaseEmitterAdapter->GetEditableEmitter().Emitter;
			UNiagaraSimulationStageBase* AddedSimulationStage = CastChecked<UNiagaraSimulationStageBase>(StaticDuplicateObject(AddedOtherSimulationStage->GetSimulationStage(), BaseEmitter));
			AddedSimulationStage->Script = NewObject<UNiagaraScript>(AddedSimulationStage, MakeUniqueObjectName(AddedSimulationStage, UNiagaraScript::StaticClass(), "SimulationStage"),
			                                                         RF_Transactional);
			AddedSimulationStage->Script->SetUsage(ENiagaraScriptUsage::ParticleSimulationStageScript);
			AddedSimulationStage->Script->SetUsageId(AddedOtherSimulationStage->GetUsageId());
			AddedSimulationStage->Script->SetLatestSource(EmitterSource);
			BaseEmitter->AddSimulationStage(AddedSimulationStage, BaseEmitterAdapter->GetEditableEmitter().Version);

			int32 TargetIndex = FMath::Min(AddedOtherSimulationStage->GetSimulationStageIndex(), BaseEmitterData->GetSimulationStages().Num() - 1);
			BaseEmitter->MoveSimulationStageToIndex(AddedSimulationStage, TargetIndex, BaseEmitterAdapter->GetEditableEmitter().Version);

			FGuid PreferredOutputNodeGuid = AddedOtherSimulationStage->GetOutputNode()->NodeGuid;
			FGuid PreferredInputNodeGuid = AddedOtherSimulationStage->GetInputNode()->NodeGuid;
			UNiagaraNodeOutput* SimulationStageOutputNode = FNiagaraStackGraphUtilities::ResetGraphForOutput(*EmitterGraph, ENiagaraScriptUsage::ParticleSimulationStageScript, AddedSimulationStage->Script->GetUsageId(), PreferredOutputNodeGuid, PreferredInputNodeGuid);
			for (TSharedRef<FNiagaraStackFunctionMergeAdapter> ModuleAdapter : AddedOtherSimulationStage->GetSimulationStageStack()->GetModuleFunctions())
			{
				FApplyDiffResults AddModuleResults = AddModule(BaseEmitterAdapter, ScratchPadAdapter, *AddedSimulationStage->Script, *SimulationStageOutputNode, ModuleAdapter);
				Results.bSucceeded &= AddModuleResults.bSucceeded;
				Results.ErrorMessages.Append(AddModuleResults.ErrorMessages);
			}

			// Force the base compile id of the new shader stage to match the added instance shader stage.
			UNiagaraScriptSource* AddedSimulationStageSourceFromDiff = Cast<UNiagaraScriptSource>(AddedOtherSimulationStage->GetSimulationStage()->Script->GetLatestSource());
			UNiagaraGraph* AddedSimulationStageGraphFromDiff = AddedSimulationStageSourceFromDiff->NodeGraph;
			FGuid ScriptBaseIdFromDiff = AddedSimulationStageGraphFromDiff->GetBaseId(ENiagaraScriptUsage::ParticleSimulationStageScript, AddedOtherSimulationStage->GetUsageId());
			UNiagaraScriptSource* AddedSimulationStageSource = Cast<UNiagaraScriptSource>(AddedSimulationStage->Script->GetLatestSource());
			UNiagaraGraph* AddedSimulationStageGraph = AddedSimulationStageSource->NodeGraph;
			AddedSimulationStageGraph->ForceBaseId(ENiagaraScriptUsage::ParticleSimulationStageScript, AddedOtherSimulationStage->GetUsageId(), ScriptBaseIdFromDiff);

			Results.bModifiedGraph = true;
		}
	}
	return Results;
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::ApplyRendererDiff(const FVersionedNiagaraEmitter& BaseEmitter, const FNiagaraEmitterDiffResults& DiffResults, const bool bNoParentAtLastMerge) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::ApplyRendererDiff);

	TArray<UNiagaraRendererProperties*> RenderersToRemove;
	TArray<UNiagaraRendererProperties*> RenderersToAdd;
	TArray<UNiagaraRendererProperties*> RenderersToDisable;
	FVersionedNiagaraEmitterData* BaseEmitterData = BaseEmitter.GetEmitterData();
	
	for (TSharedRef<FNiagaraRendererMergeAdapter> RemovedRenderer : DiffResults.RemovedBaseRenderers)
	{
		auto FindRendererByMergeId = [=](UNiagaraRendererProperties* Renderer) { return Renderer->GetMergeId() == RemovedRenderer->GetRenderer()->GetMergeId(); };
		UNiagaraRendererProperties* const* MatchingRendererPtr = BaseEmitterData->GetRenderers().FindByPredicate(FindRendererByMergeId);
		if (MatchingRendererPtr != nullptr)
		{
			if (bNoParentAtLastMerge)
			{
				// If there is no last known parent we don't know if the renderer was removed in the child, or added in the parent, so
				// instead of removing the parent renderer we disable it in this case, since removing renderers in child emitters isn't
				// supported through the UI, and instead the user is expected to disable it.
				RenderersToDisable.Add(*MatchingRendererPtr);
			}
			else
			{
				RenderersToRemove.Add(*MatchingRendererPtr);
			}
		}
	}

	for (TSharedRef<FNiagaraRendererMergeAdapter> AddedRenderer : DiffResults.AddedOtherRenderers)
	{
		RenderersToAdd.Add(Cast<UNiagaraRendererProperties>(StaticDuplicateObject(AddedRenderer->GetRenderer(), BaseEmitter.Emitter)));
	}

	for (TSharedRef<FNiagaraRendererMergeAdapter> ModifiedRenderer : DiffResults.ModifiedOtherRenderers)
	{
		auto FindRendererByMergeId = [=](UNiagaraRendererProperties* Renderer) { return Renderer->GetMergeId() == ModifiedRenderer->GetRenderer()->GetMergeId(); };
		UNiagaraRendererProperties*const* MatchingRendererPtr = BaseEmitterData->GetRenderers().FindByPredicate(FindRendererByMergeId);
		if (MatchingRendererPtr != nullptr)
		{
			RenderersToRemove.Add(*MatchingRendererPtr);
			RenderersToAdd.Add(Cast<UNiagaraRendererProperties>(StaticDuplicateObject(ModifiedRenderer->GetRenderer(), BaseEmitter.Emitter)));
		}
	}

	for (UNiagaraRendererProperties* RendererToRemove : RenderersToRemove)
	{
		BaseEmitter.Emitter->RemoveRenderer(RendererToRemove, BaseEmitter.Version);
	}

	for (UNiagaraRendererProperties* RendererToAdd : RenderersToAdd)
	{
		BaseEmitter.Emitter->AddRenderer(RendererToAdd, BaseEmitter.Version);
	}

	for (UNiagaraRendererProperties* RendererToDisable : RenderersToDisable)
	{
		RendererToDisable->bIsEnabled = false;
	}

	FApplyDiffResults Results;
	Results.bSucceeded = true;
	Results.bModifiedGraph = false;
	return Results;
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::ApplyEmitterSummaryDiff(const FVersionedNiagaraEmitter& BaseEmitter, const FNiagaraEmitterDiffResults& DiffResults) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::ApplyEmitterSummaryDiff);

	FVersionedNiagaraEmitterData* BaseEmitterData = BaseEmitter.GetEmitterData();
	UNiagaraEmitterEditorData* EditorData = Cast<UNiagaraEmitterEditorData>(BaseEmitterData->GetEditorData());

	//-TEMP: Temporary code until we figure out why we don't have valid editor data
	if (EditorData == nullptr)
	{
		FApplyDiffResults Results;
		Results.bSucceeded = false;
		Results.bModifiedGraph = false;
		return Results;
	}

	check(EditorData != GetDefault<UNiagaraEmitterEditorData>());
	check(EditorData->GetOuter() == BaseEmitter.Emitter);
	
	EditorData->Modify();

	UHierarchyRoot* BaseRoot = EditorData->GetSummaryRoot();

	// this will copy over the section without any child elements
	for (const UHierarchySection* AddedOtherSection : DiffResults.AddedSummarySectionsInOther)
	{
		BaseRoot->DuplicateSectionFromOtherRoot(*AddedOtherSection);
	}

	for (const UHierarchyElement* AddedInput : DiffResults.AddedSummaryEntriesInOther)
	{
		if (UHierarchyElement* SummaryItemParent = AddedInput->GetTypedOuter<UHierarchyElement>())
		{
			FHierarchyElementIdentity ParentIdentity = SummaryItemParent->GetPersistentIdentity();
	
			if(ParentIdentity.IsValid())
			{
				const bool bParentIsRoot = SummaryItemParent->IsA<UHierarchyRoot>();
				// if the parent is the root, we can't use the parent identity to copy over the child as the two roots are supposed to be different, so we add it directly
				if(bParentIsRoot)
				{
					/** Core principal of adding new items:
					* Any added item should come without children, as the parent could have added the same child somewhere else
					* Otherwise we'd need to identify not just new items vs. old items, but also new items with old children.
					* The children we removed but actually belong will be added back during another iteration of the loop
					*/
					if(BaseRoot->FindChildWithIdentity(AddedInput->GetPersistentIdentity(), false) == nullptr)
					{
						UHierarchyElement* NewChild = BaseRoot->CopyAndAddItemAsChild(*AddedInput);
						NewChild->GetChildrenMutable().Empty();

						if(FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = NewChild->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>())
						{
							if(SectionAssociation->Section.IsValid())
							{
								const TObjectPtr<UHierarchySection>* NewSection = BaseRoot->GetSectionData().FindByPredicate([OriginalSection = SectionAssociation->Section](UHierarchySection* SectionCandidate)
									{
										return SectionCandidate->GetSectionName().IsEqual(OriginalSection->GetSectionName());
									});

								if(NewSection)
								{
									SectionAssociation->Section = (*NewSection);
								}
							}
						}
					}				
				}
				else
				{
					// it's possible the child was already added via ownership link
					// (i.e. if categoryA owns inputA and categoryA is copied first, inputA will be included too)
					// we only want to add that child if it hasn't been added already
					if(BaseRoot->FindChildWithIdentity(AddedInput->GetPersistentIdentity(), true) == nullptr)
					{
						UHierarchyElement* NewChild = BaseRoot->CopyAndAddItemUnderParentIdentity(*AddedInput, ParentIdentity);
						
						//if(!ensure(NewChild != nullptr))
						if(NewChild == nullptr)
						{
							UE_LOGF(LogNiagaraEditor, Log, "Item %ls could not be added during merge process", *AddedInput->ToString());
						}
						else
						{
							// see above
							NewChild->GetChildrenMutable().Empty();
						}
					}
				}
			}
		}
	}

	if (DiffResults.NewShouldShowSummaryViewValue.IsSet())
	{
		EditorData->SetShowSummaryView(DiffResults.NewShouldShowSummaryViewValue.GetValue());
	}

	FApplyDiffResults Results;
	Results.bSucceeded = true;
	Results.bModifiedGraph = false;
	return Results;
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::ApplyStackEntryDisplayNameDiffs(FVersionedNiagaraEmitter BaseEmitter, const FNiagaraEmitterDiffResults& DiffResults) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::ApplyStackEntryDisplayNameDiffs);
	if (DiffResults.ModifiedStackEntryDisplayNames.Num() > 0)
	{
		UNiagaraEmitterEditorData* EditorData = Cast<UNiagaraEmitterEditorData>(BaseEmitter.GetEmitterData()->GetEditorData());

		for (auto& Pair : DiffResults.ModifiedStackEntryDisplayNames)
		{
			EditorData->GetStackEditorData().SetStackEntryDisplayName(Pair.Key, Pair.Value);
		}
	}

	FApplyDiffResults Results;
	Results.bSucceeded = true;
	Results.bModifiedGraph = false;
	return Results;
}

FNiagaraScriptMergeManager::FApplyDiffResults FNiagaraScriptMergeManager::ApplyStackNoteDiffs(FVersionedNiagaraEmitter BaseEmitter, const FNiagaraEmitterDiffResults& DiffResults) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FNiagaraScriptMergeManager::ApplyStackNoteDiffs);
	if(DiffResults.AddedOrModifiedStackNotesInOther.Num() > 0)
	{
		if(UNiagaraEmitterEditorData* EmitterEditorData = Cast<UNiagaraEmitterEditorData>(BaseEmitter.GetEmitterData()->GetEditorData()))
		{
			for(const auto& Pair : DiffResults.AddedOrModifiedStackNotesInOther)
			{
				EmitterEditorData->GetStackEditorData().AddOrReplaceStackNote(Pair.Key, Pair.Value, false);
			}
		}
	}

	FApplyDiffResults Results;
	Results.bSucceeded = true;
	Results.bModifiedGraph = false;
	return Results;
}

FNiagaraScriptMergeManager::FCachedMergeAdapter* FNiagaraScriptMergeManager::FindOrAddMergeAdapterCacheForEmitter(const FVersionedNiagaraEmitter& VersionedEmitter)
{
	FCachedMergeAdapter* CachedMergeAdapter = CachedMergeAdapters.Find(VersionedEmitter);
	if (CachedMergeAdapter == nullptr)
	{
		CachedMergeAdapter = &CachedMergeAdapters.Add(VersionedEmitter);
	}
	else
	{
		if (CachedMergeAdapter->ChangeId != VersionedEmitter.Emitter->GetChangeId())
		{
			CachedMergeAdapter->ChangeId = VersionedEmitter.Emitter->GetChangeId();
			CachedMergeAdapter->EmitterMergeAdapter.Reset();
			CachedMergeAdapter->ScratchPadMergeAdapter.Reset();
		}
	}
	return CachedMergeAdapter;
}

void FNiagaraScriptMergeManager::ClearMergeAdapterCache()
{
	CachedMergeAdapters.Empty();
}

TSharedRef<FNiagaraEmitterMergeAdapter> FNiagaraScriptMergeManager::GetEmitterMergeAdapterUsingCache(const FVersionedNiagaraEmitter& Emitter)
{
	FCachedMergeAdapter* CachedMergeAdapter = FindOrAddMergeAdapterCacheForEmitter(Emitter);

	if (CachedMergeAdapter->EmitterMergeAdapter.IsValid() == false ||
		CachedMergeAdapter->EmitterMergeAdapter->GetEditableEmitter().Emitter == nullptr)
	{
		CachedMergeAdapter->EmitterMergeAdapter = MakeShared<FNiagaraEmitterMergeAdapter>(Emitter);
	}

	return CachedMergeAdapter->EmitterMergeAdapter.ToSharedRef();
}

TSharedRef<FNiagaraScratchPadMergeAdapter> FNiagaraScriptMergeManager::GetScratchPadMergeAdapterUsingCache(const FVersionedNiagaraEmitter& VersionedEmitter)
{
	FCachedMergeAdapter* CachedMergeAdapter = FindOrAddMergeAdapterCacheForEmitter(VersionedEmitter);

	if (CachedMergeAdapter->ScratchPadMergeAdapter.IsValid() == false)
	{
		CachedMergeAdapter->ScratchPadMergeAdapter = MakeShared<FNiagaraScratchPadMergeAdapter>(VersionedEmitter, FVersionedNiagaraEmitter(), VersionedEmitter.GetEmitterData()->GetParent());
	}

	return CachedMergeAdapter->ScratchPadMergeAdapter.ToSharedRef();
}


#undef LOCTEXT_NAMESPACE
