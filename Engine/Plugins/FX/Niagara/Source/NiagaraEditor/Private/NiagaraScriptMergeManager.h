// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraModule.h"
#include "INiagaraMergeManager.h"
#include "NiagaraTypes.h"
#include "NiagaraCommon.h"
#include "NiagaraEmitter.h"
#include "NiagaraMessages.h"
#include "NiagaraEmitterEditorData.h"

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/ObjectKey.h"

class UNiagaraEmitter;
class UNiagaraEmitterEditorData;
class UNiagaraScript;
class UNiagaraNodeOutput;
class UNiagaraNodeInput;
class UNiagaraNodeFunctionCall;
class UNiagaraNodeParameterMapSet;
class UEdGraphNode;
class UEdGraphPin;
class UNiagaraDataInterface;
struct FNiagaraEventScriptProperties;
class UNiagaraSimulationStageBase;
class UNiagaraRendererProperties;

class FNiagaraStackFunctionMergeAdapter;

struct FNiagaraStackLinkedValueData
{
	FNiagaraVariableBase LinkedParameterValue;
	FGuid LinkedFunctionNodeId;

	bool operator==(const FNiagaraStackLinkedValueData& Other) const
	{
		return LinkedParameterValue == Other.LinkedParameterValue && LinkedFunctionNodeId == Other.LinkedFunctionNodeId;
	}
};

class FNiagaraStackFunctionInputOverrideMergeAdapter
{
public:
	FNiagaraStackFunctionInputOverrideMergeAdapter(
		const FVersionedNiagaraEmitter& InOwningEmitter, 
		UNiagaraScript& InOwningScript, 
		UNiagaraNodeFunctionCall& InOwningFunctionCallNode, 
		UEdGraphPin& InOverridePin);

	FNiagaraStackFunctionInputOverrideMergeAdapter(
		UNiagaraScript& InOwningScript,
		UNiagaraNodeFunctionCall& InOwningFunctionCallNode,
		FStringView InInputName,
		FNiagaraVariable InRapidIterationParameter);

	FNiagaraStackFunctionInputOverrideMergeAdapter(UEdGraphPin* InStaticSwitchPin);

	UNiagaraScript* GetOwningScript() const;
	UNiagaraNodeFunctionCall* GetOwningFunctionCall() const;
	FString GetInputName() const;
	UNiagaraNodeParameterMapSet* GetOverrideNode() const;
	UEdGraphPin* GetOverridePin() const;
	const FGuid& GetOverrideNodeId() const;
	const FNiagaraTypeDefinition& GetType() const;

	TOptional<FString> GetLocalValueString() const;
	TOptional<FNiagaraVariable> GetLocalValueRapidIterationParameter() const;
	TOptional<FNiagaraStackLinkedValueData> GetLinkedValueData() const;
	TOptional<FName> GetDataInterfaceValueInputName() const;
	UNiagaraDataInterface* GetDataInterfaceValue() const;
	TOptional<FNiagaraVariableBase> GetObjectAssetInputVariable() const;
	UObject* GetObjectAssetValue() const;
	TSharedPtr<FNiagaraStackFunctionMergeAdapter> GetDynamicValueFunction() const;
	TOptional<FString> GetStaticSwitchValue() const;

private:
	TWeakObjectPtr<UNiagaraScript> OwningScript;
	TWeakObjectPtr<UNiagaraNodeFunctionCall> OwningFunctionCallNode;
	FString InputName;
	FNiagaraTypeDefinition Type;

	TWeakObjectPtr<UNiagaraNodeParameterMapSet> OverrideNode;
	UEdGraphPin* OverridePin = nullptr;

	TOptional<FString> LocalValueString;
	TOptional<FNiagaraVariable> LocalValueRapidIterationParameter;
	TOptional<FNiagaraStackLinkedValueData> LinkedValueData;
	TOptional<FName> DataInterfaceValueInputName;
	UNiagaraDataInterface* DataInterfaceValue = nullptr;
	TOptional<FNiagaraVariableBase> ObjectAssetInputVariable;
	UObject* ObjectAssetValue = nullptr;
	TSharedPtr<FNiagaraStackFunctionMergeAdapter> DynamicValueFunction;
	TOptional<FString> StaticSwitchValue;

	FGuid OverrideValueNodePersistentId;
};

class FNiagaraStackFunctionMergeAdapter
{
public:
	FNiagaraStackFunctionMergeAdapter(const FVersionedNiagaraEmitter& InOwningEmitter, UNiagaraScript& InOwningScript, UNiagaraNodeFunctionCall& InFunctionCallNode,	int32 InStackIndex);

	UNiagaraNodeFunctionCall* GetFunctionCallNode() const;

	int32 GetStackIndex() const;

	bool GetUsesScratchPadScript() const;

	const TArray<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>>& GetInputOverrides() const;

	TSharedPtr<FNiagaraStackFunctionInputOverrideMergeAdapter> GetInputOverrideByInputNameAndType(const FString& InputName, const FNiagaraTypeDefinition& InputType) const;

	void GatherFunctionCallNodes(TArray<UNiagaraNodeFunctionCall*>& OutFunctionCallNodes) const;

private:
	TWeakObjectPtr<UNiagaraScript> OwningScript;
	TWeakObjectPtr<UNiagaraNodeFunctionCall> FunctionCallNode;
	int32 StackIndex;
	bool bUsesScratchPadScript;
	TArray<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>> InputOverrides;
};

class FNiagaraScriptStackMergeAdapter
{
public:
	FNiagaraScriptStackMergeAdapter(const FVersionedNiagaraEmitter& InOwningEmitter, UNiagaraNodeOutput& InOutputNode, UNiagaraScript& InScript);

	UNiagaraNodeOutput* GetOutputNode() const;
	UNiagaraNodeInput* GetInputNode() const;

	UNiagaraScript* GetScript() const;

	FString GetUniqueEmitterName() const;

	const TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>>& GetModuleFunctions() const;

	TSharedPtr<FNiagaraStackFunctionMergeAdapter> GetModuleFunctionById(FGuid FunctionCallNodeId) const;

	void GatherFunctionCallNodes(TArray<UNiagaraNodeFunctionCall*>& OutFunctionCallNodes) const;

private:
	TWeakObjectPtr<UNiagaraNodeInput> InputNode;
	TWeakObjectPtr<UNiagaraNodeOutput> OutputNode;
	TWeakObjectPtr<UNiagaraScript> Script;
	FString UniqueEmitterName;
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> ModuleFunctions;
};

class FNiagaraEventHandlerMergeAdapter
{
public:
	FNiagaraEventHandlerMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, const FNiagaraEventScriptProperties* InEventScriptProperties, UNiagaraNodeOutput* InOutputNode);
	FNiagaraEventHandlerMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, FNiagaraEventScriptProperties* InEventScriptProperties, UNiagaraNodeOutput* InOutputNode);
	FNiagaraEventHandlerMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, UNiagaraNodeOutput* InOutputNode);

	FGuid GetUsageId() const;
	FVersionedNiagaraEmitter GetEmitter() const;
	const FNiagaraEventScriptProperties* GetEventScriptProperties() const;
	FNiagaraEventScriptProperties* GetEditableEventScriptProperties() const;
	UNiagaraNodeOutput* GetOutputNode() const;
	UNiagaraNodeInput* GetInputNode() const;
	TSharedPtr<FNiagaraScriptStackMergeAdapter> GetEventStack() const;

private:
	void Initialize(const FVersionedNiagaraEmitter& InEmitter, const FNiagaraEventScriptProperties* InEventScriptProperties, FNiagaraEventScriptProperties* InEditableEventScriptProperties, UNiagaraNodeOutput* InOutputNode);

private:
	FVersionedNiagaraEmitterWeakPtr Emitter;
	const FNiagaraEventScriptProperties* EventScriptProperties;
	FNiagaraEventScriptProperties* EditableEventScriptProperties;
	TWeakObjectPtr<UNiagaraNodeOutput> OutputNode;
	TWeakObjectPtr<UNiagaraNodeInput> InputNode;

	TSharedPtr<FNiagaraScriptStackMergeAdapter> EventStack;
};

class FNiagaraSimulationStageMergeAdapter
{
public:
	FNiagaraSimulationStageMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, const UNiagaraSimulationStageBase* InSimulationStage, int32 InSimulationStageIndex, UNiagaraNodeOutput* InOutputNode);
	FNiagaraSimulationStageMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, UNiagaraSimulationStageBase* InSimulationStage, int32 InSimulationStageIndex, UNiagaraNodeOutput* InOutputNode);
	FNiagaraSimulationStageMergeAdapter(const FVersionedNiagaraEmitter& InEmitter, UNiagaraNodeOutput* InOutputNode);

	FGuid GetUsageId() const;
	FVersionedNiagaraEmitter GetEmitter() const;
	const UNiagaraSimulationStageBase* GetSimulationStage() const;
	UNiagaraSimulationStageBase* GetEditableSimulationStage() const;
	UNiagaraNodeOutput* GetOutputNode() const;
	UNiagaraNodeInput* GetInputNode() const;
	int32 GetSimulationStageIndex() const;
	TSharedPtr<FNiagaraScriptStackMergeAdapter> GetSimulationStageStack() const;

private:
	void Initialize(const FVersionedNiagaraEmitter& InEmitter, const UNiagaraSimulationStageBase* InSimulationStage, UNiagaraSimulationStageBase* InEditableSimulationStage, int32 InSimulationStageIndex, UNiagaraNodeOutput* InOutputNode);

private:
	FVersionedNiagaraEmitterWeakPtr Emitter;
	const UNiagaraSimulationStageBase* SimulationStage;
	UNiagaraSimulationStageBase* EditableSimulationStage;
	TWeakObjectPtr<UNiagaraNodeOutput> OutputNode;
	TWeakObjectPtr<UNiagaraNodeInput> InputNode;
	int32 SimulationStageIndex;

	TSharedPtr<FNiagaraScriptStackMergeAdapter> SimulationStageStack;
};

class FNiagaraRendererMergeAdapter
{
public:
	FNiagaraRendererMergeAdapter(UNiagaraRendererProperties& InRenderer);

	UNiagaraRendererProperties* GetRenderer();

private:
	TWeakObjectPtr<UNiagaraRendererProperties> Renderer;
};

class FNiagaraInputSummaryMergeAdapter
{
public:
	FNiagaraInputSummaryMergeAdapter(const FFunctionInputSummaryViewKey& Key, const FFunctionInputSummaryViewMetadata& Value);

	const FFunctionInputSummaryViewKey& GetKey() const;
	const FFunctionInputSummaryViewMetadata& GetValue() const;

private:
	const FFunctionInputSummaryViewKey Key;
	const FFunctionInputSummaryViewMetadata Value;
};

class FNiagaraScratchPadMergeAdapter
{
public:
	FNiagaraScratchPadMergeAdapter();

	FNiagaraScratchPadMergeAdapter(const FVersionedNiagaraEmitter InTargetEmitter, const FVersionedNiagaraEmitter& InInstanceEmitter, const FVersionedNiagaraEmitter& InParentEmitter);

	UNiagaraScript* GetScratchPadScriptForFunctionId(FGuid FunctionId);

	// Struct to aid in book-keeping around scratch pads and merge inheritance.
	struct FMergeRecord
	{
		FMergeRecord(FVersionedNiagaraEmitter& InOriginalEmitter, UNiagaraScript* InOriginalScript, const FGuid& InOriginalScriptVersion) : OriginalEmitter(InOriginalEmitter), OriginalScript(InOriginalScript), OriginalScriptVersion(InOriginalScriptVersion){}

		FVersionedNiagaraEmitter OriginalEmitter;
		UNiagaraScript* OriginalScript;
		FGuid OriginalScriptVersion;
	};

	// Get the collapsed merge records for this merge session
	TArray< FMergeRecord > GetMergedEmitterRecords() const;

private:
	void Initialize();

private:
	FVersionedNiagaraEmitter TargetEmitter;
	FVersionedNiagaraEmitter InstanceEmitter;
	FVersionedNiagaraEmitter ParentEmitter;
	bool bIsInitialized;
	TMap<FGuid, UNiagaraScript*> FunctionIdToScratchPadScript;
};

class FNiagaraEmitterMergeAdapter
{
public:
	FNiagaraEmitterMergeAdapter(const FVersionedNiagaraEmitter& InEmitter);

	FVersionedNiagaraEmitter GetEditableEmitter() const;

	TSharedPtr<FNiagaraScriptStackMergeAdapter> GetEmitterSpawnStack() const;
	TSharedPtr<FNiagaraScriptStackMergeAdapter> GetEmitterUpdateStack() const;
	TSharedPtr<FNiagaraScriptStackMergeAdapter> GetParticleSpawnStack() const;
	TSharedPtr<FNiagaraScriptStackMergeAdapter> GetParticleUpdateStack() const;
	const TArray<TSharedRef<FNiagaraEventHandlerMergeAdapter>> GetEventHandlers() const;
	const TArray<TSharedRef<FNiagaraSimulationStageMergeAdapter>> GetSimulationStages() const;
	const TArray<TSharedRef<FNiagaraRendererMergeAdapter>> GetRenderers() const;
	const UNiagaraEmitterEditorData* GetEditorData() const;

	TSharedPtr<FNiagaraScriptStackMergeAdapter> GetScriptStack(ENiagaraScriptUsage Usage, FGuid UsageId);

	TSharedPtr<FNiagaraEventHandlerMergeAdapter> GetEventHandler(FGuid EventScriptUsageId);

	TSharedPtr<FNiagaraSimulationStageMergeAdapter> GetSimulationStage(FGuid SimulationStageUsageId);

	TSharedPtr<FNiagaraRendererMergeAdapter> GetRenderer(FGuid RendererMergeId);

	void GatherFunctionCallNodes(TArray<UNiagaraNodeFunctionCall*>& OutFunctionCallNodes) const;

private:
	void Initialize(const FVersionedNiagaraEmitter& InEmitter, const FVersionedNiagaraEmitter& InEditableEmitter);

private:
	FVersionedNiagaraEmitterWeakPtr Emitter;
	FVersionedNiagaraEmitterWeakPtr EditableEmitter;
	TSharedPtr<FNiagaraScriptStackMergeAdapter> EmitterSpawnStack;
	TSharedPtr<FNiagaraScriptStackMergeAdapter> EmitterUpdateStack;
	TSharedPtr<FNiagaraScriptStackMergeAdapter> ParticleSpawnStack;
	TSharedPtr<FNiagaraScriptStackMergeAdapter> ParticleUpdateStack;
	TArray<TSharedRef<FNiagaraEventHandlerMergeAdapter>> EventHandlers;
	TArray<TSharedRef<FNiagaraSimulationStageMergeAdapter>> SimulationStages;
	TArray<TSharedRef<FNiagaraRendererMergeAdapter>> Renderers;
	TWeakObjectPtr<const UNiagaraEmitterEditorData> EditorData;
};

struct FNiagaraStackFunctionMessageData
{
	TSharedPtr<FNiagaraStackFunctionMergeAdapter> Function;
	FNiagaraStackMessage StackMessage;
};
struct FNiagaraScriptStackDiffResults
{
	FNiagaraScriptStackDiffResults();

	bool IsEmpty() const;

	bool IsValid() const;

	void AddError(FText ErrorMessage);

	const TArray<FText>& GetErrorMessages() const;

	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> RemovedBaseModules;
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> AddedOtherModules;

	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> MovedBaseModules;
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> MovedOtherModules;

	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> ChangedVersionBaseModules;
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> ChangedVersionOtherModules;

	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> EnabledChangedBaseModules;
	TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> EnabledChangedOtherModules;
	
	TArray<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>> RemovedBaseInputOverrides;
	TArray<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>> AddedOtherInputOverrides;
	TArray<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>> ModifiedBaseInputOverrides;
	TArray<TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter>> ModifiedOtherInputOverrides;

	TOptional<ENiagaraScriptUsage> ChangedBaseUsage;
	TOptional<ENiagaraScriptUsage> ChangedOtherUsage;

private:
	bool bIsValid;
	TArray<FText> ErrorMessages;
};

struct FNiagaraModifiedEventHandlerDiffResults
{
	TSharedPtr<FNiagaraEventHandlerMergeAdapter> BaseAdapter;
	TSharedPtr<FNiagaraEventHandlerMergeAdapter> OtherAdapter;
	TArray<FProperty*> ChangedProperties;
	FNiagaraScriptStackDiffResults ScriptDiffResults;
};

struct FNiagaraModifiedSimulationStageDiffResults
{
	TSharedPtr<FNiagaraSimulationStageMergeAdapter> BaseAdapter;
	TSharedPtr<FNiagaraSimulationStageMergeAdapter> OtherAdapter;
	TArray<FProperty*> ChangedProperties;
	FNiagaraScriptStackDiffResults ScriptDiffResults;
};

struct FNiagaraEmitterDiffResults
{
	FNiagaraEmitterDiffResults();

	bool IsValid() const;

	bool IsEmpty() const;

	void AddError(FText ErrorMessage);

	const TArray<FText>& GetErrorMessages() const;

	FString GetErrorMessagesString() const;
	bool HasVersionChanges() const;

	TArray<FProperty*> DifferentEmitterProperties;

	TSharedPtr<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter;
	TSharedPtr<FNiagaraEmitterMergeAdapter> OtherEmitterAdapter;

	FNiagaraScriptStackDiffResults EmitterSpawnDiffResults;
	FNiagaraScriptStackDiffResults EmitterUpdateDiffResults;
	FNiagaraScriptStackDiffResults ParticleSpawnDiffResults;
	FNiagaraScriptStackDiffResults ParticleUpdateDiffResults;

	TArray<TSharedRef<FNiagaraEventHandlerMergeAdapter>> RemovedBaseEventHandlers;
	TArray<TSharedRef<FNiagaraEventHandlerMergeAdapter>> AddedOtherEventHandlers;
	TArray<FNiagaraModifiedEventHandlerDiffResults> ModifiedEventHandlers;

	TArray<TSharedRef<FNiagaraSimulationStageMergeAdapter>> RemovedBaseSimulationStages;
	TArray<TSharedRef<FNiagaraSimulationStageMergeAdapter>> AddedOtherSimulationStages;
	TArray<FNiagaraModifiedSimulationStageDiffResults> ModifiedSimulationStages;

	TArray<TSharedRef<FNiagaraRendererMergeAdapter>> RemovedBaseRenderers;
	TArray<TSharedRef<FNiagaraRendererMergeAdapter>> AddedOtherRenderers;
	TArray<TSharedRef<FNiagaraRendererMergeAdapter>> ModifiedBaseRenderers;
	TArray<TSharedRef<FNiagaraRendererMergeAdapter>> ModifiedOtherRenderers;

	TArray<const UHierarchyElement*> AddedSummaryEntriesInOther;	
	TArray<const UHierarchySection*> AddedSummarySectionsInOther;

	TOptional<bool> NewShouldShowSummaryViewValue;
	
	bool bScratchPadModified;

	TMap<FString, FText> ModifiedStackEntryDisplayNames;

	TMap<FString, FNiagaraStackNoteData> AddedOrModifiedStackNotesInOther;
	
private:
	bool bIsValid;
	TArray<FText> ErrorMessages;
};

class FNiagaraScriptMergeManager : public INiagaraMergeManager
{
public:
	struct FApplyDiffResults
	{
		FApplyDiffResults()
			: bSucceeded(true)
			, bModifiedGraph(false)
		{
		}

		bool bSucceeded;
		bool bModifiedGraph;
		TArray<FText> ErrorMessages;
	};

public:
	void UpdateModuleVersions(const FVersionedNiagaraEmitter& Instance, const FNiagaraEmitterDiffResults& EmitterDiffResults) const;
	
	virtual INiagaraMergeManager::FMergeEmitterResults MergeEmitter(const FVersionedNiagaraEmitter& Parent, const FVersionedNiagaraEmitter& ParentAtLastMerge, const FVersionedNiagaraEmitter& Instance) const override;

	static TSharedRef<FNiagaraScriptMergeManager> Get();

	FNiagaraEmitterDiffResults DiffEmitters(const FVersionedNiagaraEmitter& BaseEmitter, const FVersionedNiagaraEmitter& OtherEmitter) const;

	bool IsMergeableScriptUsage(ENiagaraScriptUsage ScriptUsage) const;

	bool HasBaseModule(const FVersionedNiagaraEmitter& BaseEmitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FGuid ModuleId);

	// Find the module originating this module id in the scratch lists
	bool FindBaseModule(const FVersionedNiagaraEmitter& BaseEmitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FGuid ModuleId, class UNiagaraScript*& OutActualScript, FGuid& OutScriptVersionGuid, FVersionedNiagaraEmitter& OutBaseEmitter);

	bool IsModuleInputDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FGuid ModuleId, FNiagaraVariableBase Variable);

	/** Check whether a given summary item identity exists in the parent of a given emitter. */
	bool DoesSummaryItemExistInBase(const FVersionedNiagaraEmitter& Emitter, FHierarchyElementIdentity Identity);
	
	FApplyDiffResults ResetModuleInputToBase(const FVersionedNiagaraEmitter& VersionedEmitter, const FVersionedNiagaraEmitter& VersionedBaseEmitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FGuid ModuleId, FString InputName);

	bool HasBaseEventHandler(const FVersionedNiagaraEmitter& BaseEmitter, FGuid EventScriptUsageId);

	bool IsEventHandlerPropertySetDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid EventScriptUsageId);

	void ResetEventHandlerPropertySetToBase(const FVersionedNiagaraEmitter& VersionedEmitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid EventScriptUsageId);

	bool HasBaseSimulationStage(const FVersionedNiagaraEmitter& BaseEmitter, FGuid SimulationStageScriptUsageId);

	bool IsSimulationStagePropertySetDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid SimulationStageScriptUsageId);

	void ResetSimulationStagePropertySetToBase(const FVersionedNiagaraEmitter& VersionedEmitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid SimulationStageScriptUsageId);

	bool HasBaseRenderer(const FVersionedNiagaraEmitter& BaseEmitter, FGuid RendererMergeId);

	bool IsRendererDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid RendererMergeId);

	void ResetRendererToBase(FVersionedNiagaraEmitter Emitter, const FVersionedNiagaraEmitter& BaseEmitter, FGuid RendererMergeId);

	bool IsEmitterEditablePropertySetDifferentFromBase(const FVersionedNiagaraEmitter& Emitter, const FVersionedNiagaraEmitter& BaseEmitter);

	void ResetEmitterEditablePropertySetToBase(const FVersionedNiagaraEmitter& VersionedEmitter, const FVersionedNiagaraEmitter& BaseEmitter);

	void DiffEventHandlers(const TArray<TSharedRef<FNiagaraEventHandlerMergeAdapter>>& BaseEventHandlers, const TArray<TSharedRef<FNiagaraEventHandlerMergeAdapter>>& OtherEventHandlers, FNiagaraEmitterDiffResults& DiffResults) const;

	void DiffSimulationStages(const TArray<TSharedRef<FNiagaraSimulationStageMergeAdapter>>& BaseSimulationStages, const TArray<TSharedRef<FNiagaraSimulationStageMergeAdapter>>& OtherSimulationStages, FNiagaraEmitterDiffResults& DiffResults) const;

	void DiffRenderers(const TArray<TSharedRef<FNiagaraRendererMergeAdapter>>& BaseRenderers, const TArray<TSharedRef<FNiagaraRendererMergeAdapter>>& OtherRenderers, FNiagaraEmitterDiffResults& DiffResults) const;
	
	void DiffEmitterSummary(const UNiagaraEmitterEditorData* BaseEditorData, const UNiagaraEmitterEditorData* OtherEditorData, FNiagaraEmitterDiffResults& DiffResults) const;

	void DiffScriptStacks(TSharedRef<FNiagaraScriptStackMergeAdapter> BaseScriptStackAdapter, TSharedRef<FNiagaraScriptStackMergeAdapter> OtherScriptStackAdapter, FNiagaraScriptStackDiffResults& DiffResults) const;

	void DiffFunctionInputs(TSharedRef<FNiagaraStackFunctionMergeAdapter> BaseFunctionAdapter, TSharedRef<FNiagaraStackFunctionMergeAdapter> OtherFunctionAdapter, FNiagaraScriptStackDiffResults& DiffResults) const;

	virtual void DiffEditableProperties(const void* BaseDataAddress, const void* OtherDataAddress, UStruct& Struct, TArray<FProperty*>& OutDifferentProperties) const override;

	void DiffStackEntryDisplayNames(const UNiagaraEmitterEditorData* BaseEditorData, const UNiagaraEmitterEditorData* OtherEditorData, TMap<FString, FText>& OutModifiedStackEntryDisplayNames) const;

	void DiffStackNotes(const UNiagaraEmitterEditorData* BaseEditorData, const UNiagaraEmitterEditorData* OtherEditorData, FNiagaraEmitterDiffResults& DiffResults) const;
	
	void DiffScratchPadScripts(const TArray<UNiagaraScript*>& BaseScratchPadScripts, const TArray<UNiagaraScript*>& OtherEmitterScratchPadScripts, FNiagaraEmitterDiffResults& DiffResults) const;

	virtual void CopyPropertiesToBase(void* BaseDataAddress, const void* OtherDataAddress, TArray<FProperty*> PropertiesToCopy) const override;

	void ClearMergeAdapterCache();

private:
	struct FCachedMergeAdapter
	{
		FGuid ChangeId;
		TSharedPtr<FNiagaraEmitterMergeAdapter> EmitterMergeAdapter;
		TSharedPtr<FNiagaraScratchPadMergeAdapter> ScratchPadMergeAdapter;
	};

	enum class EmitterMergeState
	{
		Failed, Unchanged, DuplicateParent, CopyProperties
	};

	EmitterMergeState GetEmitterMergeState(const FNiagaraEmitterDiffResults& DiffResults, const FVersionedNiagaraEmitter& Parent, const FVersionedNiagaraEmitter& Instance) const;

	TOptional<bool> DoFunctionInputOverridesMatch(TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> BaseFunctionInputAdapter, TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> OtherFunctionInputAdapter) const;

	void CopyInstanceScratchPadScripts(const FVersionedNiagaraEmitter& MergedInstance, FVersionedNiagaraEmitterData* SourceInstance) const;

	FApplyDiffResults ApplyScriptStackDiff(
		TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
		TSharedRef<FNiagaraScriptStackMergeAdapter> BaseScriptStackAdapter,
		TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
		const FNiagaraScriptStackDiffResults& DiffResults,
		const bool bNoParentAtLastMerge) const;

	FApplyDiffResults ApplyEventHandlerDiff(
		TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
		TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
		const FNiagaraEmitterDiffResults& DiffResults,
		const bool bNoParentAtLastMerge) const;

	FApplyDiffResults ApplySimulationStageDiff(
		TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
		TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
		const FNiagaraEmitterDiffResults& DiffResults,
		const bool bNoParentAtLastMerge) const;

	FApplyDiffResults ApplyRendererDiff(const FVersionedNiagaraEmitter& BaseEmitter, const FNiagaraEmitterDiffResults& DiffResults, const bool bNoParentAtLastMerge) const;

	FApplyDiffResults ApplyEmitterSummaryDiff(const FVersionedNiagaraEmitter& BaseEmitter, const FNiagaraEmitterDiffResults& DiffResults) const;
	
	FApplyDiffResults ApplyStackEntryDisplayNameDiffs(FVersionedNiagaraEmitter BaseEmitter, const FNiagaraEmitterDiffResults& DiffResults) const;

	FApplyDiffResults ApplyStackNoteDiffs(FVersionedNiagaraEmitter BaseEmitter, const FNiagaraEmitterDiffResults& DiffResults) const;
	
	FApplyDiffResults AddModule(
		TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
		TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
		UNiagaraScript& OwningScript,
		UNiagaraNodeOutput& TargetOutputNode,
		TSharedRef<FNiagaraStackFunctionMergeAdapter> AddModule) const;

	FApplyDiffResults RemoveInputOverride(UNiagaraScript& OwningScript, TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> OverrideToRemove) const;

	FApplyDiffResults AddInputOverride(
		TSharedRef<FNiagaraEmitterMergeAdapter> BaseEmitterAdapter,
		TSharedRef<FNiagaraScratchPadMergeAdapter> ScratchPadAdapter,
		UNiagaraScript& OwningScript,
		UNiagaraNodeFunctionCall& TargetFunctionCall,
		TSharedRef<FNiagaraStackFunctionInputOverrideMergeAdapter> OverrideToAdd) const;

	FCachedMergeAdapter* FindOrAddMergeAdapterCacheForEmitter(const FVersionedNiagaraEmitter& VersionedEmitter);

	TSharedRef<FNiagaraEmitterMergeAdapter> GetEmitterMergeAdapterUsingCache(const FVersionedNiagaraEmitter& Emitter);

	TSharedRef<FNiagaraScratchPadMergeAdapter> GetScratchPadMergeAdapterUsingCache(const FVersionedNiagaraEmitter& VersionedEmitter);

	void GetForcedChangeIds(
		const TMap<FGuid, UNiagaraNodeFunctionCall*>& InParentFunctionIdToNodeMap,
		const TMap<FGuid, UNiagaraNodeFunctionCall*>& InParentAtLastMergeFunctionIdToNodeMap,
		const TMap<FGuid, UNiagaraNodeFunctionCall*>& InInstanceFunctionIdToNodeMap,
		TMap<FGuid, FGuid>& OutFunctionIdToForcedChangeId) const;

	FApplyDiffResults ForceInstanceChangeIds(TSharedRef<FNiagaraEmitterMergeAdapter> MergedInstanceAdapter, const FVersionedNiagaraEmitter& OriginalEmitterInstance, const TMap<FGuid, FGuid>& ChangeIdsThatNeedToBeReset) const;

	TMap<FVersionedNiagaraEmitter, FCachedMergeAdapter> CachedMergeAdapters;
};