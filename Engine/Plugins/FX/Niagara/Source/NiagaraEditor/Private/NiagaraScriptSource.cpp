// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptSource.h"
#include "EdGraphSchema_Niagara.h"
#include "EdGraphUtilities.h"
#include "GraphEditAction.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraAnalytics.h"
#include "NiagaraCommon.h"
#include "NiagaraComponent.h"
#include "NiagaraConstants.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraDataInterface.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraHlslTranslator.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraScript.h"
#include "Logging/LogMacros.h"
#include "ViewModels/HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScriptSource)

DECLARE_CYCLE_STAT(TEXT("Niagara - ScriptSource - Compile"), STAT_NiagaraEditor_ScriptSource_Compile, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - ScriptSource - InitializeNewRapidIterationParameters"), STAT_NiagaraEditor_ScriptSource_InitializeNewRapidIterationParameters, STATGROUP_NiagaraEditor);

void UNiagaraScriptSource::RegisterVMCompilationIdDependencies(FNiagaraScriptHashCollector& Collector, ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const
{
	if (!AllowShaderCompiling())
	{
		return;
	}

	if (UNiagaraScript::IsGPUScript(InUsage)) // GPU particle scripts are baked in via other usages
	{
		return;
	}

	if (UNiagaraScript::IsParticleScript(InUsage) || UNiagaraScript::IsEmitterScript(InUsage) || UNiagaraScript::IsSystemScript(InUsage))
	{
		if (NodeGraph)
		{
			NodeGraph->RebuildCachedCompileIds();
			FNiagaraCompileHash Hash = FNiagaraCompileHash(NodeGraph->GetCompileReferencedDataHash(InUsage, InUsageId));
			
			if (Hash.IsValid())
			{
				Collector.AddHash(Hash, GetPathName());
			}
			else
			{
				UE_LOGF(LogNiagaraEditor, Warning, "Failed to get a valid referenced data hash back! %ls", *GetPathNameSafe(this));
			}
		}
	}
}


void UNiagaraScriptSource::ComputeVMCompilationId(FNiagaraVMExecutableDataId& Id, FNiagaraScriptHashCollector& HashCollector, ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const
{
	if (!AllowShaderCompiling())
	{
		return;
	}

	Id.ScriptUsageType = InUsage;
	Id.ScriptUsageTypeID = InUsageId;
	Id.CompilerVersionID = FNiagaraCustomVersion::GetLatestScriptCompileVersion();
	if (NodeGraph)
	{
		NodeGraph->RebuildCachedCompileIds();
		Id.BaseScriptCompileHash = FNiagaraCompileHash(NodeGraph->GetCompileDataHash(InUsage, InUsageId));

		NodeGraph->GatherExternalDependencyData(InUsage, InUsageId, HashCollector);
	}

	// Add in any referenced HLSL files.
	FSHAHash Hash = GetShaderFileHash((TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf")), EShaderPlatform::SP_PCD3D_SM5);
	HashCollector.AddHash(FNiagaraCompileHash(Hash.Hash, sizeof(Hash.Hash) / sizeof(uint8)), TEXT("/Plugin/FX/Niagara/Private/NiagaraEmitterInstanceShader.usf"));
	Hash = GetShaderFileHash((TEXT("/Plugin/FX/Niagara/Private/NiagaraShaderVersion.ush")), EShaderPlatform::SP_PCD3D_SM5);
	HashCollector.AddHash(FNiagaraCompileHash(Hash.Hash, sizeof(Hash.Hash) / sizeof(uint8)), TEXT("/Plugin/FX/Niagara/Private/NiagaraShaderVersion.ush"));
	Hash = GetShaderFileHash((TEXT("/Engine/Public/ShaderVersion.ush")), EShaderPlatform::SP_PCD3D_SM5);
	HashCollector.AddHash(FNiagaraCompileHash(Hash.Hash, sizeof(Hash.Hash) / sizeof(uint8)), TEXT("/Engine/Public/ShaderVersion.ush"));

	// Incorporate the cbuffer definitions used by the script
	FSHAHash StructHash = FNiagaraGlobalParameters::GetStructHash();
	HashCollector.AddHash(FNiagaraCompileHash(StructHash.Hash, sizeof(StructHash.Hash) / sizeof(uint8)), TEXT("FNiagaraGlobalParameters"));

	if (!UNiagaraScript::IsSystemScript(InUsage))
	{
		StructHash = FNiagaraSystemParameters::GetStructHash();
		HashCollector.AddHash(FNiagaraCompileHash(StructHash.Hash, sizeof(StructHash.Hash) / sizeof(uint8)), TEXT("FNiagaraSystemParameters"));
		StructHash = FNiagaraOwnerParameters::GetStructHash();
		HashCollector.AddHash(FNiagaraCompileHash(StructHash.Hash, sizeof(StructHash.Hash) / sizeof(uint8)), TEXT("FNiagaraOwnerParameters"));
		StructHash = FNiagaraEmitterParameters::GetStructHash();
		HashCollector.AddHash(FNiagaraCompileHash(StructHash.Hash, sizeof(StructHash.Hash) / sizeof(uint8)), TEXT("FNiagaraEmitterParameters"));
	}
}

void UNiagaraScriptSource::RefreshGraphCompileId()
{
	if (NodeGraph)
	{
		NodeGraph->ConditionalRebuildCompileIdCache();
	}
}

void UNiagaraScriptSource::ReportAnalyticsData(FNiagaraScriptSourceAnalytics& InData) const
{
	static const UEnum* EnumType = StaticEnum<ENiagaraScriptUsage>();

	TArray<UNiagaraNodeOutput*> OutputNodes;
	NodeGraph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);	
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		FString UsageString = TEXT("Script.") + EnumType->GetNameStringByValue(static_cast<uint64>(OutputNode->GetUsage()));

		// gather module data
		TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
		FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*OutputNode, ModuleNodes);

		for (UNiagaraNodeFunctionCall* Node : ModuleNodes)
		{
			if (!Node->FunctionScript || !Node->GetScriptData())
			{
				continue;
			}
			
			if (!Node->IsNodeEnabled())
			{
				InData.DisabledModules++;
				continue;
			}
			InData.ActiveModules++;
			if (NiagaraAnalytics::IsPluginAsset(Node->FunctionScript))
			{
				FNiagaraAssetVersion AssetVersion = Node->GetScriptData()->Version;
				InData.UsedNiagaraModules.Add(FString::Format(TEXT("{0}:{1}.{2}"), {GetPathNameSafe(Node->FunctionScript->GetPackage()), AssetVersion.MajorVersion, AssetVersion.MinorVersion}));
			}
		}
	}
}

FGuid UNiagaraScriptSource::GetCompileBaseId(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const
{
	return NodeGraph->GetBaseId(InUsage, InUsageId);
}

FNiagaraCompileHash UNiagaraScriptSource::GetCompileHash(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const
{
	return NodeGraph->GetCompileDataHash(InUsage, InUsageId);
}

void UNiagaraScriptSource::ForceGraphToRecompileOnNextCheck()
{
	NodeGraph->ForceGraphToRecompileOnNextCheck();
}

void UNiagaraScriptSource::RefreshFromExternalChanges()
{
	if (NodeGraph)
	{
		for (UEdGraphNode* Node : NodeGraph->Nodes)
		{
			if (UNiagaraNode* NiagaraNode = Cast<UNiagaraNode>(Node))
			{
				NiagaraNode->RefreshFromExternalChanges();
			}
		}
	}
}

void UNiagaraScriptSource::PostLoad()
{
	Super::PostLoad();

	if (NodeGraph)
	{
		// We need to make sure that the node-graph is already resolved b/c we may be asked IsSyncrhonized later...
		NodeGraph->ConditionalPostLoad();

		// Hook up event handlers so the on changed handler can be called correctly.
		NodeGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateUObject(this, &UNiagaraScriptSource::OnGraphChanged));
		NodeGraph->AddOnGraphNeedsRecompileHandler(FOnGraphChanged::FDelegate::CreateUObject(this, &UNiagaraScriptSource::OnGraphChanged));
		NodeGraph->OnDataInterfaceChanged().AddUObject(this, &UNiagaraScriptSource::OnGraphDataInterfaceChanged);
	}
}

UNiagaraScriptSource* UNiagaraScriptSource::CreateCompilationCopy(const TArray<ENiagaraScriptUsage>& CompileUsages)
{
	UNiagaraScriptSource* Result = NewObject<UNiagaraScriptSource>();
	Result->NodeGraph = NodeGraph->CreateCompilationCopy(CompileUsages);
	Result->bIsCompilationCopy = true;
	Result->AddToRoot();
	return Result;
}

void UNiagaraScriptSource::ReleaseCompilationCopy()
{
	if (bIsCompilationCopy && !bIsReleased)
	{
		bIsReleased = true;
		NodeGraph = nullptr;
		RemoveFromRoot();
		MarkAsGarbage();
	}
}

bool UNiagaraScriptSource::IsSynchronized(const FGuid& InChangeId)
{
	if (NodeGraph)
	{
		return NodeGraph->IsOtherSynchronized(InChangeId);
	}
	else
	{
		return false;
	}
}

void UNiagaraScriptSource::MarkNotSynchronized(FString Reason)
{
	if (NodeGraph)
	{
		NodeGraph->MarkGraphRequiresSynchronization(Reason);
	}
}

void UNiagaraScriptSource::PostLoadFromEmitter(FVersionedNiagaraEmitter OwningEmitter)
{
	const int32 NiagaraCustomVersion = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraCustomVersion < FNiagaraCustomVersion::ScriptsNowUseAGuidForIdentificationInsteadOfAnIndex)
	{
		TArray<UNiagaraNodeOutput*> OutputNodes;
		NodeGraph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);
		FVersionedNiagaraEmitterData* EmitterData = OwningEmitter.GetEmitterData();
		for (int32 i = 0; i < EmitterData->GetEventHandlers().Num(); i++)
		{
			const FNiagaraEventScriptProperties& EventScriptProperties = EmitterData->GetEventHandlers()[i];
			EventScriptProperties.Script->SetUsageId(FGuid::NewGuid());

			auto FindOutputNodeByUsageIndex = [=](UNiagaraNodeOutput* OutputNode) 
			{ 
				return OutputNode->GetUsage() == ENiagaraScriptUsage::ParticleEventScript && OutputNode->ScriptTypeIndex_DEPRECATED == EventScriptProperties.Script->UsageIndex_DEPRECATED; 
			};
			UNiagaraNodeOutput** MatchingOutputNodePtr = OutputNodes.FindByPredicate(FindOutputNodeByUsageIndex);
			if (MatchingOutputNodePtr != nullptr)
			{
				UNiagaraNodeOutput* MatchingOutputNode = *MatchingOutputNodePtr;
				MatchingOutputNode->SetUsageId(EventScriptProperties.Script->GetUsageId());
			}
		}
		NodeGraph->MarkGraphRequiresSynchronization("Modified while handling a change to the niagara custom version.");
	}
}

void FindObjectNamesRecursive(UNiagaraGraph* InGraph, ENiagaraScriptUsage Usage, FGuid UsageId, FString EmitterUniqueName, TSet<UNiagaraGraph*>& VisitedGraphs, TMap<FName, UNiagaraDataInterface*>& Result)
{
	if (!InGraph)
	{
		return;
	}

	TArray<UNiagaraNode*> Nodes;
	UNiagaraNodeOutput* OutputNode = InGraph->FindEquivalentOutputNode(Usage, UsageId);
	InGraph->BuildTraversal(Nodes, OutputNode);
	for (UEdGraphNode* Node : Nodes)
	{
		if (UNiagaraNode* InNode = Cast<UNiagaraNode>(Node))
		{
			if (UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(InNode))
			{
				if (InputNode->Input.IsDataInterface())
				{
					UNiagaraDataInterface* DataInterface = InputNode->GetDataInterface();
					bool bIsParameterMapDataInterface = false;
					FName DIName = FNiagaraHlslTranslator::GetDataInterfaceName(InputNode->Input.GetName(), EmitterUniqueName, bIsParameterMapDataInterface);
					if (Result.Contains(DIName) && Result[DIName] != DataInterface)
					{
						UE_LOGF(LogNiagaraEditor, Verbose, "Duplicate data interface name %ls in graph %ls. One of the data interfaces will override the other when resolving the name.", *DIName.ToString(), *InGraph->GetPathName());
					}
					Result.Add(DIName, DataInterface);
				}
				continue;
			}

			if (UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(InNode))
			{
				UNiagaraGraph* FunctionGraph = FunctionCallNode->GetCalledGraph();
				if(VisitedGraphs.Contains(FunctionGraph) == false)
				{
					VisitedGraphs.Add(FunctionGraph);
					FindObjectNamesRecursive(FunctionGraph, FunctionCallNode->GetCalledUsage(), FGuid(), EmitterUniqueName, VisitedGraphs, Result);
				}
			}
		}
	}
}

TMap<FName, UNiagaraDataInterface*> UNiagaraScriptSource::ComputeObjectNameMap(UNiagaraSystem& System, ENiagaraScriptUsage Usage, FGuid UsageId, FString EmitterUniqueName) const
{
	TMap<FName, UNiagaraDataInterface*> Result;
	if (!NodeGraph)
	{
		return Result;
	}

	TSet<UNiagaraGraph*> VisitedGraphs;
	if (Usage == ENiagaraScriptUsage::ParticleGPUComputeScript)
	{
		// GPU scripts need to include spawn, update, and sim stage script's data interfaces.
		TArray<UNiagaraNodeOutput*> OutputNodes;
		NodeGraph->GetNodesOfClass(OutputNodes);
		for (UNiagaraNodeOutput* OutputNode : OutputNodes)
		{
			if (OutputNode->GetUsage() == ENiagaraScriptUsage::ParticleSpawnScript ||
				OutputNode->GetUsage() == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated ||
				OutputNode->GetUsage() == ENiagaraScriptUsage::ParticleUpdateScript ||
				OutputNode->GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript)
			{
				FindObjectNamesRecursive(NodeGraph, OutputNode->GetUsage(), OutputNode->GetUsageId(), EmitterUniqueName, VisitedGraphs, Result);
			}
		}
	}
	else
	{
		// Collect the data interfaces for the current script.
		FindObjectNamesRecursive(NodeGraph, Usage, UsageId, EmitterUniqueName, VisitedGraphs, Result);

		if (Usage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated)
		{
			// The interpolated spawn script needs to include the particle update script's data interfaces as well.
			FindObjectNamesRecursive(NodeGraph, ENiagaraScriptUsage::ParticleUpdateScript, FGuid(), EmitterUniqueName, VisitedGraphs, Result);
		}
		else if (Usage == ENiagaraScriptUsage::SystemSpawnScript || Usage == ENiagaraScriptUsage::SystemUpdateScript)
		{
			// System scripts need to include data interfaces from the corresponding emitter scripts.
			ENiagaraScriptUsage EmitterUsage = Usage == ENiagaraScriptUsage::SystemSpawnScript
				? ENiagaraScriptUsage::EmitterSpawnScript 
				: ENiagaraScriptUsage::EmitterUpdateScript;

			for (const FNiagaraEmitterHandle& EmitterHandle : System.GetEmitterHandles())
			{
				FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetIsEnabled() ? EmitterHandle.GetEmitterData() : nullptr;
				if (EmitterData)
				{
					if (UNiagaraScriptSource* EmitterSource = Cast<UNiagaraScriptSource>(EmitterData->GraphSource); EmitterSource != nullptr)
					{
						FindObjectNamesRecursive(EmitterSource->NodeGraph, EmitterUsage, FGuid(), EmitterHandle.GetUniqueInstanceName(), VisitedGraphs, Result);
					}
				}
			}
		}
	}
	return Result;
}

bool UNiagaraScriptSource::AddModuleIfMissing(FString ModulePath, ENiagaraScriptUsage Usage, bool& bOutFoundModule)
{
	FSoftObjectPath SystemUpdateScriptRef(ModulePath);
	FAssetData ModuleScriptAsset;
	ModuleScriptAsset.PackageName = SystemUpdateScriptRef.GetAssetPath().GetPackageName();
	ModuleScriptAsset.AssetName = SystemUpdateScriptRef.GetAssetPath().GetAssetName();
	bOutFoundModule = false;

	if (ModuleScriptAsset.IsValid())
	{
		bOutFoundModule = true;
		if (UNiagaraNodeOutput* OutputNode = NodeGraph->FindOutputNode(Usage))
		{
			TArray<UNiagaraNodeFunctionCall*> FoundCalls;
			if (!FNiagaraStackGraphUtilities::FindScriptModulesInStack(ModuleScriptAsset, *OutputNode, FoundCalls))
			{
				FNiagaraStackGraphUtilities::AddScriptModuleToStack(ModuleScriptAsset, *OutputNode);
				return true;
			}
		}
	}

	return false;
}

void UNiagaraScriptSource::MigrateParameterDataToHierarchyRoot(FVersionedNiagaraScriptData& OwnerData)
{
	if (NodeGraph)
	{
		NodeGraph->MigrateParameterScriptDataToHierarchyRoot(OwnerData);
	}
}

void UNiagaraScriptSource::FixupRenamedParameters(UNiagaraNodeFunctionCall* FunctionCallNode, TConstArrayView<FNiagaraVariable> ModuleInputVariables, FNiagaraParameterStore& RapidIterationParameters, const TArray<FNiagaraVariable>& OldRapidIterationVariables, const FVersionedNiagaraEmitter& VersionedEmitter, ENiagaraScriptUsage ScriptUsage) const
{
	// the rapid iteration parameters and the function input pins use different variable naming schemes, so most of this is just used to convert one name to the other 
	UNiagaraGraph* Graph = FunctionCallNode->GetCalledGraph();
	const FString UniqueEmitterName = VersionedEmitter.Emitter ? VersionedEmitter.Emitter->GetUniqueEmitterName() : FString();

	TArray<TOptional<FNiagaraVariableMetaData>, TInlineAllocator<16>> ModulePinMetaData;
	TArray<FName, TInlineAllocator<16>> ModulePinAliasedName;
	TArray<FString, TInlineAllocator<16>> ModulePinConstantName;
	TArray<int32, TInlineAllocator<16>> ModulePinNameSpaceEnd;

	const int32 ModuleInputPinCount = ModuleInputVariables.Num();
	ModulePinMetaData.Reserve(ModuleInputPinCount);
	ModulePinAliasedName.Reserve(ModuleInputPinCount);
	ModulePinConstantName.Reserve(ModuleInputPinCount);
	ModulePinNameSpaceEnd.Reserve(ModuleInputPinCount);

	const FName FunctionName = *FunctionCallNode->GetFunctionName();

	for (int32 ModuleInputPinIt = 0; ModuleInputPinIt < ModuleInputPinCount; ++ModuleInputPinIt)
	{
		const FNiagaraVariable& ModuleInputVariable = ModuleInputVariables[ModuleInputPinIt];

		ModulePinMetaData.Add(Graph->GetMetaData(ModuleInputVariable));
		FNiagaraParameterHandle AliasedFunctionInputHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(ModuleInputVariable.GetName(), FunctionName);
		const FName& AliasedName = ModulePinAliasedName.Add_GetRef(AliasedFunctionInputHandle.GetParameterHandleString());
		const FString& ConstantName = ModulePinConstantName.Add_GetRef(FNiagaraUtilities::CreateRapidIterationConstantName(AliasedName, *UniqueEmitterName, ScriptUsage));

		ConstantName.FindLastChar(TEXT('.'), ModulePinNameSpaceEnd.AddDefaulted_GetRef());
	}

	// go through the existing rapid iteration params to see if they are either still valid or were renamed
	for (const FNiagaraVariable& OldRapidIterationVar : OldRapidIterationVariables)
	{
		if (FGuid* BoundGuid = RapidIterationParameters.ParameterGuidMapping.Find(OldRapidIterationVar))
		{
			for (int32 ModuleInputPinIt = 0; ModuleInputPinIt < ModuleInputPinCount; ++ModuleInputPinIt)
			{
				const FNiagaraVariable& InputVar = ModuleInputVariables[ModuleInputPinIt];
				const FStringView ConstantName(ModulePinConstantName[ModuleInputPinIt]);
				const int32 NameSpaceEnd = ModulePinNameSpaceEnd[ModuleInputPinIt];

				bool bNameMatches = FStringView(FNameBuilder(OldRapidIterationVar.GetName())) == ConstantName;
				
				// move on if the namespaces don't match
				if (NameSpaceEnd != INDEX_NONE)
				{
					if (!OldRapidIterationVar.IsInNameSpace(ConstantName.Left(NameSpaceEnd)))
					{
						continue;
					}
				}
				
				const TOptional<FNiagaraVariableMetaData>& VariableMetaData = ModulePinMetaData[ModuleInputPinIt];

				if (VariableMetaData.IsSet() && VariableMetaData->GetVariableGuid() == *BoundGuid)
				{
					// if the names match we have nothing to do
					if (bNameMatches)
					{
						continue;
					}

					// if the guid matches but the names differ then the parameter was renamed, so lets update the rapid iteration parameter
					const FName& AliasedName = ModulePinAliasedName[ModuleInputPinIt];
					FNiagaraVariable NewRapidIterationVar = FNiagaraStackGraphUtilities::CreateRapidIterationParameter(UniqueEmitterName, ScriptUsage, AliasedName, InputVar.GetType());
					RapidIterationParameters.RenameParameter(OldRapidIterationVar, NewRapidIterationVar.GetName());
					break;
				}

				// check if maybe the variable type changed from vector to position and update the rapid iteration param accordingly
				if (OldRapidIterationVar.GetType() == FNiagaraTypeDefinition::GetVec3Def() && InputVar.GetType() == FNiagaraTypeDefinition::GetPositionDef())
				{
					// the names have to match here, as we don't rename anything but change the type
					if (!bNameMatches)
					{
						continue;
					}

					// if the guid matches but the types differ then the parameter was changed from vector to position, so lets update the existing rapid iteration parameter
					if (RapidIterationParameters.IndexOf(OldRapidIterationVar) != INDEX_NONE)
					{
						FNiagaraVariable RapidIterationAlias(FNiagaraTypeDefinition::GetPositionDef(), OldRapidIterationVar.GetName());
						RapidIterationParameters.RemoveParameter(RapidIterationAlias);
						RapidIterationParameters.ChangeParameterType(OldRapidIterationVar, FNiagaraTypeDefinition::GetPositionDef());
						break;
					}
				}
			}
		}
	}
}

void UNiagaraScriptSource::InitializeNewParameters(UNiagaraNodeFunctionCall* FunctionCallNode, TConstArrayView<FNiagaraVariable> ModuleInputVariables, FNiagaraParameterStore& RapidIterationParameters, const FVersionedNiagaraEmitter& VersionedEmitter, ENiagaraScriptUsage ScriptUsage, TSet<FNiagaraVariableBase>& ValidRapidIterationParameters) const
{
	const FString UniqueEmitterName = VersionedEmitter.Emitter ? VersionedEmitter.Emitter->GetUniqueEmitterName() : FString();
	UNiagaraGraph* Graph = FunctionCallNode->GetCalledGraph();
	const UEdGraphSchema_Niagara* NiagaraSchema = Graph->GetNiagaraSchema();

	TArray<FName, TInlineAllocator<16>> PinNames;
	TArray<UEdGraphPin*, TInlineAllocator<16>> DefaultPins;
	TArray<FNiagaraParameterHandle, TInlineAllocator<16>> AliasHandles;
	TArray<FNiagaraVariable, TInlineAllocator<16>> Parameters;

	const int32 FunctionInputPinCount = ModuleInputVariables.Num();
	PinNames.Reserve(FunctionInputPinCount);
	DefaultPins.Reserve(FunctionInputPinCount);
	AliasHandles.Reserve(FunctionInputPinCount);
	Parameters.Reserve(FunctionInputPinCount);

	const FName FunctionNodeName = *FunctionCallNode->GetFunctionName();

	for (int32 FunctionInputPinIt = 0; FunctionInputPinIt < FunctionInputPinCount; ++FunctionInputPinIt)
	{
		const FNiagaraVariable& ModuleInputVariable = ModuleInputVariables[FunctionInputPinIt];
		if (ModuleInputVariable.GetType().IsValid() == false)
		{
			UE_LOGF(LogNiagaraEditor, Error, "Invalid input type found while attempting initialize new rapid iteration parameters. Function Node: %ls %ls Input Name: %ls",
				*FunctionCallNode->GetPathName(), *FunctionCallNode->GetFunctionName(), *ModuleInputVariable.GetName().ToString());
			continue;
		}

		if (FNiagaraStackGraphUtilities::IsRapidIterationType(ModuleInputVariable.GetType()))
		{
			FNiagaraParameterHandle AliasedFunctionInputHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(ModuleInputVariable.GetName(), FunctionNodeName);
			FNiagaraVariable RapidIterationParameter = FNiagaraStackGraphUtilities::CreateRapidIterationParameter(UniqueEmitterName, ScriptUsage, AliasedFunctionInputHandle.GetParameterHandleString(), ModuleInputVariable.GetType());
			ValidRapidIterationParameters.Add(RapidIterationParameter);
			int32 ParameterIndex = RapidIterationParameters.IndexOf(RapidIterationParameter);

			// Only set a value for the parameter if it's not already set.
			if (ParameterIndex == INDEX_NONE)
			{
				PinNames.Add(ModuleInputVariable.GetName());
				AliasHandles.Add(AliasedFunctionInputHandle);
				Parameters.Add(RapidIterationParameter);
			}
		}
	}

	if (!PinNames.IsEmpty())
	{
		const int32 DefaultPinCount = PinNames.Num();
		DefaultPins.Init(nullptr, DefaultPinCount);

		FCompileConstantResolver ConstantResolver(VersionedEmitter, ScriptUsage);
		FunctionCallNode->FindParameterMapDefaultValuePins(PinNames, ScriptUsage, ConstantResolver, DefaultPins);

		for (int32 DefaultPinIt = 0; DefaultPinIt < DefaultPinCount; ++DefaultPinIt)
		{
			UEdGraphPin* DefaultPin = DefaultPins[DefaultPinIt];
			if (DefaultPin != nullptr && DefaultPin->LinkedTo.Num() == 0)
			{
				// Only set values for inputs without override pins, since and override pin means it's being read from a different value.
				UEdGraphPin* OverridePin = FNiagaraStackGraphUtilities::GetStackFunctionInputOverridePin(*FunctionCallNode, AliasHandles[DefaultPinIt]);
				if (OverridePin == nullptr)
				{
					FNiagaraVariable DefaultVariable = NiagaraSchema->PinToNiagaraVariable(DefaultPin, true);
					check(DefaultVariable.GetData() != nullptr);
					bool bAddParameterIfMissing = true;
					RapidIterationParameters.SetParameterData(DefaultVariable.GetData(), Parameters[DefaultPinIt], bAddParameterIfMissing);
				}
			}
		}
	}
}

void UNiagaraScriptSource::CleanUpOldAndInitializeNewRapidIterationParameters(const FVersionedNiagaraEmitter& Emitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FNiagaraParameterStore& RapidIterationParameters) const
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_ScriptSource_InitializeNewRapidIterationParameters);
	TArray<UNiagaraNodeOutput*> OutputNodes;
	if (ScriptUsage == ENiagaraScriptUsage::ParticleGPUComputeScript)
	{
		TArray<UNiagaraNodeOutput*> TempOutputNodes;
		NodeGraph->FindOutputNodes(TempOutputNodes);
		for (UNiagaraNodeOutput* OutputNode : TempOutputNodes)
		{
			if (UNiagaraScript::IsParticleScript(OutputNode->GetUsage()))
			{
				OutputNodes.AddUnique(OutputNode);
			}
		}
	}
	else
	{
		UNiagaraNodeOutput* OutputNode = NodeGraph->FindEquivalentOutputNode(ScriptUsage, ScriptUsageId);
		OutputNodes.Add(OutputNode);
	}
	
	TArray<FNiagaraVariable> OldRapidIterationVariables;
	RapidIterationParameters.GetParameters(OldRapidIterationVariables);

	TSet<FNiagaraVariableBase> ValidRapidIterationParameters;
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		if (OutputNode != nullptr)
		{
			TArray<UNiagaraNode*> Nodes;
			NodeGraph->BuildTraversal(Nodes, OutputNode);

			for(UNiagaraNode* Node : Nodes)
			{
				UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(Node);
				if (FunctionCallNode == nullptr || FunctionCallNode->HasValidScriptAndGraph() == false)
				{
					continue;
				}

				// find out which inputs the module offers
				UNiagaraGraph* Graph = FunctionCallNode->GetCalledGraph();
				const UEdGraphSchema_Niagara* NiagaraSchema = Graph->GetNiagaraSchema();

				TArray<FNiagaraVariable> ModuleInputVariables;
				FCompileConstantResolver ConstantResolver(Emitter, ScriptUsage);

				GetStackFunctionInputs(*FunctionCallNode, ModuleInputVariables, ConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly);

				// if we found any module input pins, then we can continue with fixing up and initializing them
				if (!ModuleInputVariables.IsEmpty())
				{
					FixupRenamedParameters(FunctionCallNode, ModuleInputVariables, RapidIterationParameters, OldRapidIterationVariables, Emitter, ScriptUsage);
					InitializeNewParameters(FunctionCallNode, ModuleInputVariables, RapidIterationParameters, Emitter, OutputNode->GetUsage(), ValidRapidIterationParameters);
				}
			}
		}
	}

	// Clean up old rapid iteration parameters.
	TArray<FNiagaraVariable> CurrentRapidIterationVariables;
	RapidIterationParameters.GetParameters(CurrentRapidIterationVariables);
	for (const FNiagaraVariable& CurrentRapidIterationVariable : CurrentRapidIterationVariables)
	{
		if (ValidRapidIterationParameters.Contains(CurrentRapidIterationVariable) == false)
		{
			RapidIterationParameters.RemoveParameter(CurrentRapidIterationVariable);
		}
	}
}

/*
ENiagaraScriptCompileStatus UNiagaraScriptSource::Compile(UNiagaraScript* ScriptOwner, FString& OutGraphLevelErrorMessages)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_ScriptSource_Compile);
	bool bDoPostCompile = false;
	if (!bIsPrecompiled)
	{
		PreCompile(nullptr, TArray<FNiagaraVariable>());
		bDoPostCompile = true;
	}

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::Get().LoadModuleChecked<FNiagaraEditorModule>(TEXT("NiagaraEditor"));
	ENiagaraScriptCompileStatus Status = NiagaraEditorModule.CompileScript(ScriptOwner, OutGraphLevelErrorMessages);
	check(ScriptOwner != nullptr && IsSynchronized(ScriptOwner->GetChangeID()));
	
	if (bDoPostCompile)
	{
		PostCompile();
	}
	return Status;

// 	FNiagaraConstants& ExternalConsts = ScriptOwner->ConstantData.GetExternalConstants();
// 
// 	//Build the constant list. 
// 	//This is mainly just jumping through some hoops for the custom UI. Should be removed and have the UI just read directly from the constants stored in the UScript.
// 	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(NodeGraph->GetSchema());
// 	ExposedVectorConstants.Empty();
// 	for (int32 ConstIdx = 0; ConstIdx < ExternalConsts.GetNumVectorConstants(); ConstIdx++)
// 	{
// 		FNiagaraVariableInfo Info;
// 		FVector4 Value;
// 		ExternalConsts.GetVectorConstant(ConstIdx, Value, Info);
// 		if (Schema->IsSystemConstant(Info))
// 		{
// 			continue;//System constants are "external" but should not be exposed to the editor.
// 		}
// 			
// 		EditorExposedVectorConstant *Const = new EditorExposedVectorConstant();
// 		Const->ConstName = Info.Name;
// 		Const->Value = Value;
// 		ExposedVectorConstants.Add(MakeShareable(Const));
// 	}

}
*/

void UNiagaraScriptSource::OnGraphChanged(const FEdGraphEditAction &Action)
{
	OnChangedDelegate.Broadcast();
}

void UNiagaraScriptSource::OnGraphDataInterfaceChanged()
{
	OnChangedDelegate.Broadcast();
}

FGuid UNiagaraScriptSource::GetChangeID() 
{ 
	return NodeGraph->GetChangeID(); 
}

FVersionedNiagaraEmitter UNiagaraScriptSource::GetOuterEmitter() const
{
	UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(GetOuter());
	if (Emitter == nullptr)
	{
		return FVersionedNiagaraEmitter();
	}
	for (FNiagaraAssetVersion& Version : Emitter->GetAllAvailableVersions())
	{
		if (FVersionedNiagaraEmitterData* EmitterData = Emitter->GetEmitterData(Version.VersionGuid); EmitterData->GraphSource == this)
		{
			return FVersionedNiagaraEmitter(Emitter, Version.VersionGuid);
		}
	}
	return FVersionedNiagaraEmitter();
}

void UNiagaraScriptSource::CollectDataInterfaces(TArray<const UNiagaraDataInterfaceBase*>& DataInterfaces) const
{
	if (!NodeGraph)
	{
		return;
	}
	TArray<UNiagaraNodeOutput*> OutputNodes;
	NodeGraph->FindOutputNodes(OutputNodes);

	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		TArray<UNiagaraNode*> TraversalNodes;
		NodeGraph->BuildTraversal(TraversalNodes, OutputNode, true);

		for (const UNiagaraNode* TraversalNode : TraversalNodes)
		{
			if (const UNiagaraNodeInput* NodeInput = Cast<const UNiagaraNodeInput>(TraversalNode))
			{
				if (NodeInput->Input.IsDataInterface())
				{
					DataInterfaces.Add(NodeInput->GetDataInterface());
				}
			}
		}
	}
}

void UNiagaraScriptSource::SynchronizeGraphParametersWithParameterDefinitions(
	const TArray<UNiagaraParameterDefinitionsBase*> TargetDefinitions,
	const TArray<UNiagaraParameterDefinitionsBase*> AllDefinitions,
	const TSet<FGuid>& AllDefinitionsParameterIds,
	INiagaraParameterDefinitionsSubscriber* Subscriber,
	FSynchronizeWithParameterDefinitionsArgs Args)
{
	if (!NodeGraph)
	{
		return;
	}
	TArray<UNiagaraParameterDefinitions*> QualifiedTargetDefinitions = FNiagaraEditorUtilities::DowncastParameterDefinitionsBaseArray(TargetDefinitions);
	TArray<UNiagaraParameterDefinitions*> QualifiedAllDefinitions = FNiagaraEditorUtilities::DowncastParameterDefinitionsBaseArray(AllDefinitions);
	NodeGraph->SynchronizeParametersWithParameterDefinitions(QualifiedTargetDefinitions, QualifiedAllDefinitions, AllDefinitionsParameterIds, Subscriber, Args);
}

void UNiagaraScriptSource::RenameGraphAssignmentAndSetNodePins(const FName OldName, const FName NewName)
{
	if (!NodeGraph)
	{
		return;
	}
	NodeGraph->RenameAssignmentAndSetNodePins(OldName, NewName);
}

void UNiagaraScriptSource::GetLinkedPositionTypeInputs(const TArray<FNiagaraVariable>& ParametersToCheck, TSet<FNiagaraVariable>& OutLinkedParameters)
{
	if (!NodeGraph)
	{
		return;
	}
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	TArray<UNiagaraNodeOutput*> OutputNodes;
	NodeGraph->FindOutputNodes(OutputNodes);
	
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		TArray<UNiagaraNode*> TraversalNodes;
		NodeGraph->BuildTraversal(TraversalNodes, OutputNode, true);

		for (UNiagaraNode* TraversalNode : TraversalNodes)
		{
			UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(TraversalNode);
			if (!FunctionCallNode)
			{
				continue;
			}
			UNiagaraNodeParameterMapSet* OverrideNode = FNiagaraStackGraphUtilities::GetStackFunctionOverrideNode(*FunctionCallNode);
			if (!OverrideNode)
			{
				continue;
			}
			
			// find inputs that are overriden
			TArray<UEdGraphPin*> OverrideInputPins;
			TArray<UEdGraphPin*> LinkedModuleOutputs;
			OverrideNode->GetInputPins(OverrideInputPins);
			for (UEdGraphPin* OverridePin : OverrideInputPins)
			{
				if (OverridePin->Direction != EGPD_Input || !OverridePin->PinName.ToString().StartsWith(FunctionCallNode->GetFunctionName()))
				{
					continue;
				}

				// Gather linked value inputs
				if (UEdGraphPin* LinkedValuePin = FNiagaraStackGraphUtilities::GetLinkedValueHandleForFunctionInput(*OverridePin))
				{
					for (const FNiagaraVariable& UserVar : ParametersToCheck)
					{
						if (LinkedValuePin->PinName == UserVar.GetName())
						{
							FNiagaraVariable LinkedInputVar = Schema->PinToNiagaraVariable(OverridePin);
							if (LinkedInputVar.GetType() == FNiagaraTypeDefinition::GetPositionDef())
							{
								OutLinkedParameters.Add(UserVar);
							}
						}
					}
				}
			}
		}
	}
}

void UNiagaraScriptSource::ChangedLinkedInputTypes(const FNiagaraVariable& ParametersToChange, const FNiagaraTypeDefinition& NewType)
{
	if (!NodeGraph)
	{
		return;
	}
	NodeGraph->ChangeParameterType({ParametersToChange}, NewType);
}

void UNiagaraScriptSource::ReplaceScriptReferences(UNiagaraScript* OldScript, UNiagaraScript* NewScript)
{
	if (!NodeGraph)
	{
		return;
	}
	NodeGraph->ReplaceScriptReferences(OldScript, NewScript);
}

