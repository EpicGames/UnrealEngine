// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptVariable.h"

#include "EdGraphSchema_Niagara.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraCommon.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraParameterDefinitions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScriptVariable)

UNiagaraScriptVariable::UNiagaraScriptVariable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaultMode(ENiagaraDefaultMode::Value)
	, DefaultValueVariant()
	, StaticSwitchDefaultValue(0)
	, bIsStaticSwitch(false)
	, bSubscribedToParameterDefinitions(false)
	, ChangeId(FGuid())
	, bOverrideParameterDefinitionsDefaultValue(false)
{
}

void UNiagaraScriptVariable::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

#if WITH_EDITOR
	UpdateChangeId();

	if (UNiagaraGraph* Graph = Cast<UNiagaraGraph>(GetOuter()))
	{
		Graph->ScriptVariableChanged(Variable);
		Graph->NotifyGraphNeedsRecompile();
	}
	else if (UNiagaraParameterDefinitions* Definitions = Cast<UNiagaraParameterDefinitions>(GetOuter()))
	{
		Definitions->NotifyParameterDefinitionsChanged();
	}
#endif	//#if WITH_EDITOR
}

void UNiagaraScriptVariable::Init(const FNiagaraVariable& InVar, const FNiagaraVariableMetaData& InVarMetaData)
{
	Variable = InVar;
	Metadata = InVarMetaData;
	AllocateData();

	if (!Metadata.GetVariableGuid().IsValid())
	{
		Metadata.CreateNewGuid();
	}
	if (ChangeId.IsValid() == false)
	{
		UpdateChangeId();
	}
}

void UNiagaraScriptVariable::InitFrom(const UNiagaraScriptVariable* Value, bool bCreateNewGuid)
{
	for (TFieldIterator<FProperty> PropertyIt(GetClass(), EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		const uint8* SourceAddr = Property->ContainerPtrToValuePtr<uint8>(Value);
		uint8* DestinationAddr = Property->ContainerPtrToValuePtr<uint8>(this);

		Property->CopyCompleteValue(DestinationAddr, SourceAddr);
	}

	// we generally want to create a new metadata guid as that is used to identify variables for renames, unless we specifically want to copy the entire variable
	if(bCreateNewGuid)
	{
		Metadata.CreateNewGuid();
	}
}

void UNiagaraScriptVariable::PostLoad()
{
	Super::PostLoad();

	if (GIsEditor)
	{
		SetFlags(RF_Transactional);
	}

	if (!Metadata.GetVariableGuid().IsValid())
	{
		Metadata.SetVariableGuid(GenerateStableGuid(this));
	}
	if (ChangeId.IsValid() == false)
	{
		UpdateChangeId();
	}

	// Fix up autogenerated default values if necessary.
	const int32 NiagaraCustomVersion = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);
	if (NiagaraCustomVersion < FNiagaraCustomVersion::MoveDefaultValueFromFNiagaraVariableMetaDataToUNiagaraScriptVariable)
	{
		AllocateData();
		if (Metadata.GetIsStaticSwitch_DEPRECATED())
		{
			SetIsStaticSwitch(true);
			SetStaticSwitchDefaultValue(Metadata.GetStaticSwitchDefaultValue_DEPRECATED());
		}
	}
}

bool UNiagaraScriptVariable::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	bool bSuccess = InVisitor->UpdatePOD<uint8>(TEXT("NiagaraScriptVariable.DefaultMode"), static_cast<uint8>(DefaultMode))
		&& InVisitor->UpdateName(TEXT("NiagaraScriptVariable.DefaultBinding.Name"), DefaultBinding.Name)
		&& InVisitor->UpdatePOD(TEXT("NiagaraScriptVariable.StaticSwitchDefaultValue"), StaticSwitchDefaultValue)
		&& InVisitor->UpdatePOD(TEXT("NiagaraScriptVariable.bIsStaticSwitch"), bIsStaticSwitch)
		&& Variable.AppendCompileHash(InVisitor)
		&& DefaultValueVariant.AppendCompileHash(InVisitor);

	for (const FName& AlternateAlias : Metadata.AlternateAliases)
	{
		bSuccess = bSuccess && InVisitor->UpdateName(TEXT("NiagaraScriptVariable.Metadata.AlternateAliases"), AlternateAlias);
	}

	return bSuccess;
}

void UNiagaraScriptVariable::SetIsSubscribedToParameterDefinitions(bool bInSubscribedToParameterDefinitions)
{
	// Lazy init the ChangeId so that we are not generating a new GUID for every UNiagaraScriptVariable.
	if (bInSubscribedToParameterDefinitions && (ChangeId.IsValid() == false))
	{
		UpdateChangeId();
	}
	bSubscribedToParameterDefinitions = bInSubscribedToParameterDefinitions;
}

void UNiagaraScriptVariable::SetIsOverridingParameterDefinitionsDefaultValue(bool bInOverridingParameterDefinitionsDefaultValue)
{
	ensureMsgf(GetIsSubscribedToParameterDefinitions(), TEXT("Set overriding parameter definitions default value but script var is not synchronizing with a parameter definitions!"));
	bOverrideParameterDefinitionsDefaultValue = bInOverridingParameterDefinitionsDefaultValue;
}


bool UNiagaraScriptVariable::DefaultsAreEquivalent(const UNiagaraScriptVariable* ScriptVarA, const UNiagaraScriptVariable* ScriptVarB)
{
	if (ScriptVarA->DefaultMode != ScriptVarB->DefaultMode)
	{
		return false;
	}

	switch (ScriptVarA->DefaultMode) {
	case ENiagaraDefaultMode::Binding:
		return ScriptVarA->DefaultBinding.GetName() == ScriptVarB->DefaultBinding.GetName();
	case ENiagaraDefaultMode::Custom:
		return false;
	case ENiagaraDefaultMode::FailIfPreviouslyNotSet:
		return true;
	case ENiagaraDefaultMode::Value:
		ensureMsgf(
			ScriptVarA->GetDefaultValueVariant().GetMode() != ENiagaraVariantMode::None && ScriptVarB->GetDefaultValueVariant().GetMode() != ENiagaraVariantMode::None,
			TEXT("Encountered UNiagaraScriptVariable with invalid DefaultValueVariant mode! Was it not postloaded?"));
		return ScriptVarA->GetDefaultValueVariant() == ScriptVarB->GetDefaultValueVariant();
	};

	ensureMsgf(false, TEXT("Encountered unknown ENiagaraDefaultMode when comparing UNiagaraScriptVariable defaults!"));
	return false;
}

FGuid UNiagaraScriptVariable::GenerateStableGuid(const UNiagaraScriptVariable* ScriptVariable)
{
	// To create a guid that will be stable when generated across sessions we're going to generate a hash from 
	// an identifier string and then use that to generate the values for the GUID.  This won't result in a value
	// that's a true GUID but since every object path in a project must be unique, the md5 of these values should
	// be more than unique enough for our purposes.
	FString IdentifierString = ScriptVariable->GetPathName() + ScriptVariable->Variable.GetName().ToString() + ScriptVariable->Variable.GetType().GetName();

	// Create an md5 hash for the string data.
	uint32 HashBuffer[]{ 0, 0, 0, 0 };
	FMD5 IdentifierStringHash;
	IdentifierStringHash.Update((uint8*)GetData(IdentifierString), IdentifierString.Len() * sizeof(TCHAR));
	IdentifierStringHash.Final((uint8*)&HashBuffer);

	// Assign the guid components from the hash.
	return FGuid(HashBuffer[0], HashBuffer[1], HashBuffer[2], HashBuffer[3]);
}

void FNiagaraScriptVariableData::Init(const FNiagaraVariable& InVariable, const FNiagaraVariableMetaData& InMetadata)
{
	Variable = InVariable;
	Metadata = InMetadata;

	AllocateData();

	if (!Metadata.GetVariableGuid().IsValid())
	{
		Metadata.CreateNewGuid();
	}
	if (ChangeId.IsValid() == false)
	{
		UpdateChangeId();
	}
}

void FNiagaraScriptVariableData::InitFrom(const UNiagaraScriptVariable& ScriptVariable, bool bCreateNewGuid)
{
	Init(ScriptVariable.Variable, ScriptVariable.Metadata);

	DefaultMode = ScriptVariable.DefaultMode;
	DefaultBinding = ScriptVariable.DefaultBinding;
	bSubscribedToParameterDefinitions = ScriptVariable.GetIsSubscribedToParameterDefinitions();
	bOverrideParameterDefinitionsDefaultValue = ScriptVariable.GetIsOverridingParameterDefinitionsDefaultValue();

	DefaultValueVariant = ScriptVariable.GetDefaultValueVariant();
	StaticSwitchDefaultValue = ScriptVariable.GetStaticSwitchDefaultValue();
	bIsStaticSwitch = ScriptVariable.GetIsStaticSwitch();
	ChangeId = ScriptVariable.GetChangeId();

	// we generally want to create a new metadata guid as that is used to identify variables for renames, unless we specifically want to copy the entire variable
	if (bCreateNewGuid)
	{
		Metadata.CreateNewGuid();
	}
}

void FNiagaraScriptVariableData::InitFrom(const FNiagaraScriptVariableData& Source, bool bCreateNewGuid)
{
	Init(Source.Variable, Source.Metadata);

	DefaultMode = Source.DefaultMode;
	DefaultBinding = Source.DefaultBinding;
	bSubscribedToParameterDefinitions = Source.GetIsSubscribedToParameterDefinitions();
	bOverrideParameterDefinitionsDefaultValue = Source.GetIsOverridingParameterDefinitionsDefaultValue();

	DefaultValueVariant = Source.DefaultValueVariant;
	StaticSwitchDefaultValue = Source.StaticSwitchDefaultValue;
	bIsStaticSwitch = Source.bIsStaticSwitch;
	ChangeId = Source.ChangeId;

	// we generally want to create a new metadata guid as that is used to identify variables for renames, unless we specifically want to copy the entire variable
	if (bCreateNewGuid)
	{
		Metadata.CreateNewGuid();
	}
}

bool FNiagaraScriptVariableData::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	if (!FNiagaraEditorUtilities::NestedPropertiesAppendCompileHash(static_cast<const void*>(this), StaticStruct(), EFieldIteratorFlags::ExcludeSuper, StaticStruct()->GetName(), InVisitor))
	{
		return false;
	}
	return true;
}
