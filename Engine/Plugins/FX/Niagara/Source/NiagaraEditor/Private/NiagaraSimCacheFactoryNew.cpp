// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCacheFactoryNew.h"
#include "NiagaraSimCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSimCacheFactoryNew)

UNiagaraSimCacheFactoryNew::UNiagaraSimCacheFactoryNew(const FObjectInitializer& ObjectInitializer)
{
	SupportedClass = UNiagaraSimCache::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UNiagaraSimCacheFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

UObject* UNiagaraSimCacheFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UNiagaraSimCache>(InParent, InClass, InName, Flags);
}

