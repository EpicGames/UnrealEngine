// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStackEditorData.h"

#include "NiagaraNodeFunctionCall.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackEditorData)

bool UNiagaraStackEditorData::GetStackEntryIsRenamePending(const FString& StackEntryKey) const
{
	const bool* bIsRenamePendingPtr = StackEntryKeyToRenamePendingMap.Find(StackEntryKey);
	return bIsRenamePendingPtr != nullptr && *bIsRenamePendingPtr;
}

void UNiagaraStackEditorData::SetStackEntryIsRenamePending(const FString& StackEntryKey, bool bIsRenamePending)
{
	StackEntryKeyToRenamePendingMap.FindOrAdd(StackEntryKey) = bIsRenamePending;
}

bool UNiagaraStackEditorData::GetStackEntryIsExpanded(const FString& StackEntryKey, bool bIsExpandedDefault) const
{
	const bool* bIsExpandedPtr = StackEntryKeyToExpandedMap.Find(StackEntryKey);
	return bIsExpandedPtr != nullptr ? *bIsExpandedPtr : bIsExpandedDefault;
}

void UNiagaraStackEditorData::SetStackEntryIsExpanded(const FString& StackEntryKey, bool bIsExpanded)
{
	bool bBroadcast = false;
	if (ensureMsgf(StackEntryKey.IsEmpty() == false, TEXT("Can not set the expanded state with an empty key")))
	{
		// we assume elements are expanded by default, so collapsing (bIsExpanded = false) will cause a broadcast
		bBroadcast = GetStackEntryIsExpanded(StackEntryKey, true) != bIsExpanded;
		StackEntryKeyToExpandedMap.FindOrAdd(StackEntryKey) = bIsExpanded;
	}

	if (bBroadcast)
	{
		OnPersistentDataChanged().Broadcast();
	}
}

bool UNiagaraStackEditorData::GetStackEntryIsExpandedInOverview(const FString& StackEntryKey, bool bIsExpandedDefault) const
{
	const bool* bIsExpandedPtr = StackEntryKeyToExpandedOverviewMap.Find(StackEntryKey);
	return bIsExpandedPtr != nullptr ? *bIsExpandedPtr : bIsExpandedDefault;
}

void UNiagaraStackEditorData::SetStackEntryIsExpandedInOverview(const FString& StackEntryKey, bool bIsExpanded)
{
	bool bBroadcast = false;
	if (ensureMsgf(StackEntryKey.IsEmpty() == false, TEXT("Can not set the expanded state with an empty key")))
	{
		// we assume elements are expanded by default, so collapsing (bIsExpanded = false) will cause a broadcast
		bBroadcast = GetStackEntryIsExpandedInOverview(StackEntryKey, true) != bIsExpanded;
		StackEntryKeyToExpandedOverviewMap.FindOrAdd(StackEntryKey) = bIsExpanded;
	}

	if (bBroadcast)
	{
		OnPersistentDataChanged().Broadcast();
	}
}

bool UNiagaraStackEditorData::GetStackEntryWasExpandedPreSearch(const FString& StackEntryKey, bool bWasExpandedPreSearchDefault) const
{
	const bool* bWasExpandedPreSearchPtr = StackEntryKeyToPreSearchExpandedMap.Find(StackEntryKey);
	return bWasExpandedPreSearchPtr != nullptr ? *bWasExpandedPreSearchPtr : bWasExpandedPreSearchDefault;
}

void UNiagaraStackEditorData::SetStackEntryWasExpandedPreSearch(const FString& StackEntryKey, bool bWasExpandedPreSearch)
{
	if (ensureMsgf(StackEntryKey.IsEmpty() == false, TEXT("Can not set the pre-search expanded state with an empty key")))
	{
		StackEntryKeyToPreSearchExpandedMap.FindOrAdd(StackEntryKey) = bWasExpandedPreSearch;
	}
}

const TMap<FString, FNiagaraStackNoteData>& UNiagaraStackEditorData::GetAllStackNotes()
{
	return StackNotes;
}

TOptional<FNiagaraStackNoteData> UNiagaraStackEditorData::GetStackNote(const FString& StackEntryKey)
{
	if(StackNotes.Contains(StackEntryKey))
	{
		return StackNotes[StackEntryKey];
	}

	return {};
}

bool UNiagaraStackEditorData::HasStackNote(const FString& StackEntryKey)
{
	if(StackNotes.Contains(StackEntryKey))
	{
		return true;
	}

	return false;
}

void UNiagaraStackEditorData::AddOrReplaceStackNote(const FString& StackEntryKey, FNiagaraStackNoteData NewStackMessage, bool bBroadcast)
{
	StackNotes.Add(StackEntryKey, NewStackMessage);

	if(bBroadcast)
	{
		OnPersistentDataChanged().Broadcast();
	}
}

void UNiagaraStackEditorData::DeleteStackNote(const FString& StackEntryKey)
{
	if(StackNotes.Contains(StackEntryKey))
	{
		StackNotes.Remove(StackEntryKey);
	}
}

void UNiagaraStackEditorData::TransferDeprecatedStackNotes(const UNiagaraNodeFunctionCall& FunctionCallNode)
{
	for(const FNiagaraStackMessage& StackMessage : FunctionCallNode.GetDeprecatedCustomNotes())
	{
		FNiagaraStackNoteData StackNoteData;
		StackNoteData.MessageHeader = StackMessage.ShortDescription;
		StackNoteData.Message = StackMessage.MessageText;
		AddOrReplaceStackNote(FNiagaraStackGraphUtilities::StackKeys::GenerateStackModuleEditorDataKey(FunctionCallNode), StackNoteData, false);
	}
}

ENiagaraStackEntryInlineDisplayMode UNiagaraStackEditorData::GetStackEntryInlineDisplayMode(const FString& StackEntryKey) const
{
	const ENiagaraStackEntryInlineDisplayMode* InlineDisplayModePtr = StackEntryKeyToInlineDisplayModeMap.Find(StackEntryKey);
	return InlineDisplayModePtr != nullptr ? *InlineDisplayModePtr : ENiagaraStackEntryInlineDisplayMode::None;
}

void UNiagaraStackEditorData::SetStackEntryInlineDisplayMode(const FString& StackEntryKey, ENiagaraStackEntryInlineDisplayMode InlineDisplayMode)
{
	bool bBroadcast = false;
	if (ensureMsgf(StackEntryKey.IsEmpty() == false, TEXT("Can not set the inline display mode with an empty key")))
	{
		bBroadcast = GetStackEntryInlineDisplayMode(StackEntryKey) != InlineDisplayMode;
		StackEntryKeyToInlineDisplayModeMap.FindOrAdd(StackEntryKey) = InlineDisplayMode;
	}

	if (bBroadcast)
	{
		OnPersistentDataChanged().Broadcast();
	}
}

bool UNiagaraStackEditorData::GetStackItemShowAdvanced(const FString& StackEntryKey, bool bShowAdvancedDefault) const
{
	const bool* bShowAdvancedPtr = StackItemKeyToShowAdvancedMap.Find(StackEntryKey);
	return bShowAdvancedPtr != nullptr ? *bShowAdvancedPtr : bShowAdvancedDefault;
}

void UNiagaraStackEditorData::SetStackItemShowAdvanced(const FString& StackEntryKey, bool bShowAdvanced)
{
	if (ensureMsgf(StackEntryKey.IsEmpty() == false, TEXT("Can not set the show advanced state with an empty key")))
	{
		StackItemKeyToShowAdvancedMap.FindOrAdd(StackEntryKey) = bShowAdvanced;
	}
}

FText UNiagaraStackEditorData::GetStackEntryActiveSection(const FString& StackEntryKey, FText ActiveSectionDefault) const
{
	const FText* ActiveSectionPtr = StackEntryKeyToActiveSectionMap.Find(StackEntryKey);
	return ActiveSectionPtr != nullptr ? *ActiveSectionPtr : ActiveSectionDefault;
}

void UNiagaraStackEditorData::SetStackEntryActiveSection(const FString& StackEntryKey, FText ActiveSection)
{
	if (ensureMsgf(StackEntryKey.IsEmpty() == false, TEXT("Can not set the active section with an empty key")))
	{
		StackEntryKeyToActiveSectionMap.FindOrAdd(StackEntryKey) = ActiveSection;
	}
}

void UNiagaraStackEditorData::ClearStackEntryActiveSection(const FString& StackEntryKey)
{
	StackEntryKeyToActiveSectionMap.Remove(StackEntryKey);
}

const FText* UNiagaraStackEditorData::GetStackEntryDisplayName(const FString& StackEntryKey) const
{
	return StackEntryKeyToDisplayName.Find(StackEntryKey);
}

void UNiagaraStackEditorData::SetStackEntryDisplayName(const FString& StackEntryKey, const FText& InDisplayName)
{
	bool bBroadcast = false;

	if (InDisplayName.IsEmptyOrWhitespace())
	{
		// we assume here that the display name has changed
		StackEntryKeyToDisplayName.Remove(StackEntryKey);
		bBroadcast = true;
	}
	else if (ensureMsgf(StackEntryKey.IsEmpty() == false, TEXT("Can not set the display name with an empty key")))
	{
		StackEntryKeyToDisplayName.FindOrAdd(StackEntryKey) = InDisplayName;
		bBroadcast = true;
	}

	if (bBroadcast)
	{
		OnPersistentDataChanged().Broadcast();
	}
}

bool UNiagaraStackEditorData::GetShowOnlyModified() const
{
	return bShowOnlyModified;
}

void UNiagaraStackEditorData::SetShowOnlyModified(bool bInShowOnlyModified)
{
	bShowOnlyModified = bInShowOnlyModified;
}

bool UNiagaraStackEditorData::GetShowAllAdvanced() const
{
	return bShowAllAdvanced;
}

void UNiagaraStackEditorData::SetShowAllAdvanced(bool bInShowAllAdvanced)
{
	bShowAllAdvanced = bInShowAllAdvanced;
}

bool UNiagaraStackEditorData::GetShowOutputs() const
{
	return bShowOutputs;
}

void UNiagaraStackEditorData::SetShowOutputs(bool bInShowOutputs)
{
	bShowOutputs = bInShowOutputs;
}

bool UNiagaraStackEditorData::GetShowLinkedInputs() const
{
	return bShowLinkedInputs;
}

void UNiagaraStackEditorData::SetShowLinkedInputs(bool bInShowLinkedInputs)
{
	bShowLinkedInputs = bInShowLinkedInputs;
}

bool UNiagaraStackEditorData::GetShowOnlyIssues() const
{
	return bShowOnlyIssues;
}

void UNiagaraStackEditorData::SetShowOnlyIssues(bool bInShowOnlyIssues)
{
	bShowOnlyIssues = bInShowOnlyIssues;
}

double UNiagaraStackEditorData::GetLastScrollPosition() const
{
	return LastScrollPosition;
}

void UNiagaraStackEditorData::SetLastScrollPosition(double InLastScrollPosition)
{
	LastScrollPosition = InLastScrollPosition;
}

void UNiagaraStackEditorData::DismissStackIssue(FString IssueId)
{
	DismissedStackIssueIds.AddUnique(IssueId);
}

void UNiagaraStackEditorData::UndismissAllIssues()
{
	DismissedStackIssueIds.Empty();
}

const TArray<FString>& UNiagaraStackEditorData::GetDismissedStackIssueIds()
{
	return DismissedStackIssueIds;
}

bool UNiagaraStackEditorData::GetStatelessModuleShowWhenDisabled(const FString& StackEntryKey) const
{
	const FNiagaraStatelessModuleEditorData* StatelessModuleEditorData = StackEntryKeyToStatelessModuleEditorData.Find(StackEntryKey);
	return StatelessModuleEditorData != nullptr && StatelessModuleEditorData->bShowWhenDisabled;
}

void UNiagaraStackEditorData::SetStatelessModuleShowWhenDisabled(const FString& StackEntryKey, bool bInShowWhenDisabled)
{
	FNiagaraStatelessModuleEditorData& StatelessModuleEditorData = StackEntryKeyToStatelessModuleEditorData.FindOrAdd(StackEntryKey);
	StatelessModuleEditorData.bShowWhenDisabled = bInShowWhenDisabled;
}

