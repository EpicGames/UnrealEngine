// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStackFunctionInputBinder.h"
#include "NiagaraScript.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraScriptSource.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"

#include "EdGraph/EdGraphPin.h"

#define LOCTEXT_NAMESPACE "StackFunctionInputBinder"

FNiagaraStackFunctionInputBinder::FNiagaraStackFunctionInputBinder()
	: DefaultPin(nullptr)
	, OverridePin(nullptr)
{
}

bool FNiagaraStackFunctionInputBinder::TryBind(
	UNiagaraScript* InScript,
	TArray<UNiagaraScript*> InDependentScripts,
	FCompileConstantResolver InConstantResolver,
	FString InOwningEmitterUniqueName,
	UNiagaraNodeFunctionCall* InFunctionCallNode,
	FName InMetaDataKey,
	FString InMetaDataValue,
	TOptional<FNiagaraTypeDefinition> InInputType,
	bool bInIsRequired,
	FText& OutErrorMessage)
{
	UNiagaraGraph* FunctionGraph = InFunctionCallNode->GetCalledGraph();

	FInputMatchesPredicate InputMatches;
	InputMatches.BindLambda([=](FNiagaraVariable InputVariable)
	{
		TOptional<FNiagaraVariableMetaData> InputMetadata = FunctionGraph->GetMetaData(InputVariable);
		if (InputMetadata.IsSet())
		{
			FString* ValueString = InputMetadata->PropertyMetaData.Find(InMetaDataKey);
			if (ValueString != nullptr && *ValueString == InMetaDataValue)
			{
				return true;
			}
		}
		return false;
	});
	
	FText InputMatchDescription = FText::Format(LOCTEXT("MetadataMatchDescriptionFormat", "metadata key = '{0}' and value = '{1}'"),
		FText::FromName(InMetaDataKey), FText::FromString(InMetaDataValue));
	return TryBindInternal(InScript, InDependentScripts, InConstantResolver, InOwningEmitterUniqueName, InFunctionCallNode, InputMatches,
		InputMatchDescription, InInputType, bInIsRequired, OutErrorMessage);
}

bool FNiagaraStackFunctionInputBinder::TryBind(
	UNiagaraScript* InScript,
	TArray<UNiagaraScript*> InDependentScripts,
	FCompileConstantResolver InConstantResolver,
	FString InOwningEmitterUniqueName,
	UNiagaraNodeFunctionCall* InFunctionCallNode,
	FName InInputName,
	TOptional<FNiagaraTypeDefinition> InInputType,
	bool bInIsRequired,
	FText& OutErrorMessage)
{
	FName QualifiedInputName = *(TEXT("Module.") + InInputName.ToString());
	FInputMatchesPredicate InputMatches = FInputMatchesPredicate::CreateLambda([=](FNiagaraVariable InputVariable)
	{
		return InputVariable.GetName() == InInputName || InputVariable.GetName() == QualifiedInputName;
	});

	FText InputMatchDescription = FText::Format(LOCTEXT("NameMatchDescriptionFormat", "name = '{0}'"), FText::FromName(InInputName));
	return TryBindInternal(InScript, InDependentScripts, InConstantResolver, InOwningEmitterUniqueName, InFunctionCallNode, InputMatches,
		InputMatchDescription, InInputType, bInIsRequired, OutErrorMessage);
}

bool FNiagaraStackFunctionInputBinder::TryBindInternal(
	UNiagaraScript* InScript,
	TArray<UNiagaraScript*> InDependentScripts,
	FCompileConstantResolver InConstantResolver,
	FString InOwningEmitterUniqueName,
	UNiagaraNodeFunctionCall* InFunctionCallNode,
	FInputMatchesPredicate InputMatchesCallback,
	FText InputMatchDescription,
	TOptional<FNiagaraTypeDefinition> InInputType,
	bool bInIsRequired,
	FText& OutErrorMessage)
{
	Script = InScript;
	for (UNiagaraScript* DependentScript : InDependentScripts)
	{
		DependentScripts.Add(TWeakObjectPtr<UNiagaraScript>(DependentScript));
	}
	FunctionCallNode = InFunctionCallNode;

	TArray<FNiagaraVariable> InputVariables;
	TSet<FNiagaraVariable> HiddenInputVariables;
	FNiagaraStackGraphUtilities::GetStackFunctionInputs(*FunctionCallNode, InputVariables, HiddenInputVariables, InConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly);
	bool bInputFound = false;
	for (const FNiagaraVariable& InputVariable : InputVariables)
	{
		if (HiddenInputVariables.Contains(InputVariable))
		{
			continue;
		}

		if(InputMatchesCallback.Execute(InputVariable))
		{
			InputName = InputVariable.GetName();

			if(InInputType.IsSet() && InputVariable.GetType() != InInputType.GetValue())
			{
				OutErrorMessage = FText::Format(LOCTEXT("TypeErrorFormat", "Input type {0} didn't match the required type {1}."),
					InputVariable.GetType().GetNameText(), InputType.GetNameText());
				Reset();
				return false;
			}

			InputType = InputVariable.GetType();

			TOptional<FNiagaraVariableMetaData> InputMetadata = InFunctionCallNode->GetNiagaraGraph()->GetMetaData(InputVariable);
			InputVariableGuid = InputMetadata.IsSet() ? InputMetadata->GetVariableGuid() : FGuid();

			AliasedParameterHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(FNiagaraParameterHandle(InputVariable.GetName()), FunctionCallNode.Get());
				
			RefreshGraphPins();
			if (OverridePin != nullptr && OverridePin->LinkedTo.Num() > 0)
			{
				OutErrorMessage = LOCTEXT("InputOverriddenError", "Input is overridden in the stack and doesn't support setting a local value.");
				Reset();
				return false;
			}

			if (DefaultPin == nullptr)
			{
				OutErrorMessage = LOCTEXT("MiddingDefaultError", "Could not find the default pin for this input in the source graph, it may be hidden by a static switch.");
				Reset();
				return false;
			}

			if (FNiagaraStackGraphUtilities::IsRapidIterationType(InputType))
			{
				RapidIterationParameter = FNiagaraStackGraphUtilities::CreateRapidIterationParameter(
					InOwningEmitterUniqueName,
					Script->GetUsage(),
					AliasedParameterHandle.GetParameterHandleString(),
					InputVariable.GetType());
			}

			bInputFound = true;
			break;
		}
	}

	// For the EditCondition and VisibleCondition, we want to be able to bind to static switch input variables
	UNiagaraGraph* CalledGraph = FunctionCallNode->GetCalledGraph();
	if (!bInputFound && CalledGraph)
	{
		for (FNiagaraVariable InputVar : CalledGraph->FindStaticSwitchInputs())
		{
			if (InputMatchesCallback.Execute(InputVar) && (!InInputType.IsSet() || InputVar.GetType() == InInputType.GetValue()))
			{
				InputName = InputVar.GetName();
				InputType = InputVar.GetType();
				AliasedParameterHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(FNiagaraParameterHandle(InputVar.GetName()), FunctionCallNode.Get());
				RefreshGraphPins();
				bInputFound = true;
				break;
			}
		}
	}

	if (bInputFound == false)
	{
		Reset();
		if (bInIsRequired)
		{
			OutErrorMessage = FText::Format(LOCTEXT("RequiredButNotFoundErrorFormat", "No input found with {0}"), InputMatchDescription);
			return false;
		}
	}

	return true;
}

bool FNiagaraStackFunctionInputBinder::IsValid() const
{
	if (Script.IsValid())
	{
		if (ValidScriptGraphChangeIdForOverridePin != Script->GetSource(FunctionCallNode->SelectedScriptVersion)->GetChangeID() ||
			ValidScriptGraphChangeIdForDefaultPin != GetChangeIdFromFunctionScript())
		{
			RefreshGraphPins();
		}
		return (OverridePin == nullptr || OverridePin->LinkedTo.Num() == 0) && DefaultPin != nullptr;
	}
	return false;
}

void FNiagaraStackFunctionInputBinder::Reset()
{
	Script.Reset();
	DependentScripts.Empty();
	FunctionCallNode.Reset();
	InputType = FNiagaraTypeDefinition();
	InputName = NAME_None;
	AliasedParameterHandle = FNiagaraParameterHandle();
	ValidScriptGraphChangeIdForOverridePin = FGuid();
	ValidScriptGraphChangeIdForDefaultPin = FGuid();
	RapidIterationParameter = FNiagaraVariable();
	DefaultPin = nullptr;
	OverridePin = nullptr;
}

FName FNiagaraStackFunctionInputBinder::GetInputName() const
{
	return InputName;
}

FNiagaraTypeDefinition FNiagaraStackFunctionInputBinder::GetInputType() const
{
	return InputType;
}

UNiagaraNodeFunctionCall* FNiagaraStackFunctionInputBinder::GetFunctionCallNode() const
{
	return FunctionCallNode.Get();
}

TArray<uint8> FNiagaraStackFunctionInputBinder::GetData() const
{
	checkf(Script.IsValid(), TEXT("Bound script is not valid"));
	if (ValidScriptGraphChangeIdForOverridePin != Script->GetSource(FunctionCallNode->SelectedScriptVersion)->GetChangeID() ||
		ValidScriptGraphChangeIdForDefaultPin != GetChangeIdFromFunctionScript())
	{
		RefreshGraphPins();
		if (!OverridePin || OverridePin->LinkedTo.Num() == 0)
		{
			UE_LOGF(LogNiagaraEditor, Warning, "FNiagaraStackFunctionInputBinder - OverridePin is no longer valid because the function was overriden");
		}
	}

	// We copy the data to an array here rather than return a pointer to the data because values stored in pins are 
	// backed by string so their value data allocated on the stack via conversion and so can't be returned by pointer.
	TArray<uint8> Data;
	Data.AddUninitialized(InputType.GetSize());
	if (RapidIterationParameter.IsValid() && Script->RapidIterationParameters.IndexOf(RapidIterationParameter) != -1)
	{
		Script->RapidIterationParameters.CopyParameterData(RapidIterationParameter, Data.GetData());
	}
	else
	{
		const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
		UEdGraphPin* ValuePin = OverridePin != nullptr ? OverridePin : DefaultPin;
		FMemory::Memcpy(Data.GetData(), Schema->PinToNiagaraVariable(ValuePin, true).GetData(), InputType.GetSize());
	}
	return Data;
}

void FNiagaraStackFunctionInputBinder::SetData(const uint8* InValue, int32 InSize)
{
	checkf(InSize == InputType.GetSize(), TEXT("Set value size doesn't match bound value size"));
	if (FMemory::Memcmp(GetData().GetData(), InValue, InSize) != 0)
	{
		if ( RapidIterationParameter.IsValid() )
		{
			Script->Modify();
			Script->RapidIterationParameters.SetParameterData(InValue, RapidIterationParameter, true);
			for (TWeakObjectPtr<UNiagaraScript> DependentScript : DependentScripts)
			{
				checkf(DependentScript.IsValid(), TEXT("Bound dependent script is no longer valid"));
				DependentScript->Modify();
				DependentScript->RapidIterationParameters.SetParameterData(InValue, RapidIterationParameter, true);
			}
		}

		// Static variables are RI parameters and will therefore not force a recompile unless we take this path
		// Not taking this path will result in the Script StaticVariablesWritten being out of date
		if ( !RapidIterationParameter.IsValid() || InputType.IsStatic() )
		{
			checkf(FunctionCallNode.IsValid(), TEXT("Bound function call is no longer valid"));

			FNiagaraVariable TempVariable(InputType, NAME_None);
			TempVariable.SetData(InValue);

			FString PinDefaultValue;
			const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
			bool bSuccess = Schema->TryGetPinDefaultValueFromNiagaraVariable(TempVariable, PinDefaultValue);
			checkf(bSuccess, TEXT("Default value not supported for type %s"), *InputType.GetName());

			if (OverridePin == nullptr)
			{
				OverridePin = &FNiagaraStackGraphUtilities::GetOrCreateStackFunctionInputOverridePin(*FunctionCallNode, AliasedParameterHandle, InputType, InputVariableGuid, FGuid());
			}

			OverridePin->Modify();
			OverridePin->DefaultValue = PinDefaultValue;

			Cast<UNiagaraNode>(OverridePin->GetOwningNode())->MarkNodeRequiresSynchronization(TEXT("OverridePin Default Value Changed"), true);
			ValidScriptGraphChangeIdForOverridePin = Script->GetSource(FunctionCallNode->SelectedScriptVersion)->GetChangeID();
			ValidScriptGraphChangeIdForDefaultPin = GetChangeIdFromFunctionScript();
		}
	}
}

FGuid FNiagaraStackFunctionInputBinder::GetChangeIdFromFunctionScript() const
{
	if (FunctionCallNode->FunctionScript != nullptr && FunctionCallNode->GetFunctionScriptSource() != nullptr)
	{
		return FunctionCallNode->GetFunctionScriptSource()->GetChangeID();
	}
	return FGuid();
}

void FNiagaraStackFunctionInputBinder::RefreshGraphPins() const
{
	if (FunctionCallNode.IsValid() == false || FunctionCallNode->FunctionScript == nullptr || FunctionCallNode->GetFunctionScriptSource() == nullptr)
	{
		OverridePin = nullptr;
		DefaultPin = nullptr;
		return;
	}

	OverridePin = FNiagaraStackGraphUtilities::GetStackFunctionInputOverridePin(*FunctionCallNode, AliasedParameterHandle);
	DefaultPin = FunctionCallNode->FindParameterMapDefaultValuePin(InputName, Script->GetUsage(), FCompileConstantResolver());
	
	// if we bind to a static switch value the default value pin cannot be found via the parameter map
	if (DefaultPin == nullptr)
	{
		DefaultPin = FunctionCallNode->FindStaticSwitchInputPin(InputName);
	}
	ValidScriptGraphChangeIdForOverridePin = Script->GetSource(FunctionCallNode->SelectedScriptVersion)->GetChangeID();
	ValidScriptGraphChangeIdForDefaultPin = GetChangeIdFromFunctionScript();
}

#undef LOCTEXT_NAMESPACE