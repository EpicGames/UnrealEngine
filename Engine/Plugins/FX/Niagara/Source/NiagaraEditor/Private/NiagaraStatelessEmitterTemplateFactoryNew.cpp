// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStatelessEmitterTemplateFactoryNew.h"
#include "Stateless/NiagaraStatelessEmitterTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStatelessEmitterTemplateFactoryNew)

namespace NiagaraStatelessEmitterTemplateFactoryNewPrivate
{
	static bool GbAllowTemplateCreation = false;
	static FAutoConsoleVariableRef CVarAllowTemplateCreation(
		TEXT("fx.NiagaraStateless.AllowTemplateCreation"),
		GbAllowTemplateCreation,
		TEXT("Allow experimental template creation.  Data may be lost."),
		ECVF_Default
	);
}

UNiagaraStatelessEmitterTemplateFactoryNew::UNiagaraStatelessEmitterTemplateFactoryNew(const FObjectInitializer& ObjectInitializer)
{
	SupportedClass = UNiagaraStatelessEmitterTemplate::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UNiagaraStatelessEmitterTemplateFactoryNew::ShouldShowInNewMenu() const
{
	using namespace NiagaraStatelessEmitterTemplateFactoryNewPrivate;
	return GbAllowTemplateCreation;
}

UObject* UNiagaraStatelessEmitterTemplateFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UNiagaraStatelessEmitterTemplate>(InParent, InClass, InName, Flags);
}

