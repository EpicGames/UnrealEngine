// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "NiagaraStatelessEmitterTemplateFactoryNew.generated.h"

UCLASS(hidecategories=Object, MinimalAPI)
class UNiagaraStatelessEmitterTemplateFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	virtual bool ShouldShowInNewMenu() const override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};
