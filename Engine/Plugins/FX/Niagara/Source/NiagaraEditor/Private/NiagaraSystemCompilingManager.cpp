// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemCompilingManager.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "NiagaraCompilationTasks.h"
#include "NiagaraEditorModule.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraScriptSource.h"
#include "NiagaraShaderType.h"
#include "Misc/ScopeRWLock.h"
#include "ProfilingDebugging/CookStats.h"
#include "ShaderCompiler.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectThreadContext.h"

#define LOCTEXT_NAMESPACE "NiagaraCompilationManager"

static int GNiagaraCompilationMaxActiveTaskCount = 48;
static FAutoConsoleVariableRef CVarNiagaraCompilationMaxActiveTaskCount(
	TEXT("fx.Niagara.Compilation.MaxActiveTaskCount"),
	GNiagaraCompilationMaxActiveTaskCount,
	TEXT("The maximum number of active Niagara system compilations that can be going concurrantly."),
	ECVF_Default
);

static float GNiagaraCompilationStalledTaskWarningTime = 10.0f * 60.0f;
static FAutoConsoleVariableRef CVarNiagaraCompilationStalledTaskWarningTime(
	TEXT("fx.Niagara.Compilation.StalledTaskWarningTime"),
	GNiagaraCompilationStalledTaskWarningTime,
	TEXT("The length of time a task is being processed before warnings are generated."),
	ECVF_Default
);

#if ENABLE_COOK_STATS
namespace NiagaraSystemCookStats
{
	FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("NiagaraSystem.Usage"), TEXT(""));
	});
}
#endif

namespace NiagaraSystemCompilingManagerImpl
{

FNiagaraShaderType* GetNiagaraShaderType()
{
	FNiagaraShaderType* FoundShaderType = nullptr;
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		if (FNiagaraShaderType* ShaderType = ShaderTypeIt->GetNiagaraShaderType())
		{
			if (ensure(FoundShaderType == nullptr))
			{
				FoundShaderType = ShaderType;
			}
		}
	}

	return FoundShaderType;
}

FNiagaraShaderMapId BuildShaderMapId(FNiagaraShaderType* ShaderType, const ITargetPlatform* TargetPlatform, EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, const FNiagaraVMExecutableDataId& BaseScriptId)
{
	FNiagaraShaderMapId ShaderMapId;
	ShaderMapId.FeatureLevel = FeatureLevel;
	ShaderMapId.bUsesRapidIterationParams = BaseScriptId.bUsesRapidIterationParams;
	BaseScriptId.BaseScriptCompileHash.ToSHAHash(ShaderMapId.BaseCompileHash);
	ShaderMapId.CompilerVersionID = BaseScriptId.CompilerVersionID;

	ShaderMapId.ReferencedCompileHashes.Reserve(BaseScriptId.ReferencedCompileHashes.Num());
	for (const FNiagaraCompileHash& Hash : BaseScriptId.ReferencedCompileHashes)
	{
		Hash.ToSHAHash(ShaderMapId.ReferencedCompileHashes.AddDefaulted_GetRef());
	}

	ShaderMapId.AdditionalDefines.Empty(BaseScriptId.AdditionalDefines.Num());
	for (const FString& Define : BaseScriptId.AdditionalDefines)
	{
		ShaderMapId.AdditionalDefines.Emplace(Define);
	}

	const TArray<FString> AdditionalVariableStrings = BaseScriptId.GetAdditionalVariableStrings();
	ShaderMapId.AdditionalVariables.Empty(AdditionalVariableStrings.Num());
	for (const FString& Variable : AdditionalVariableStrings)
	{
		ShaderMapId.AdditionalVariables.Emplace(Variable);
	}

	ShaderMapId.ShaderTypeDependencies.Emplace(ShaderType, ShaderPlatform);

	if (TargetPlatform)
	{
		ShaderMapId.LayoutParams.InitializeForPlatform(TargetPlatform->IniPlatformName(), TargetPlatform->HasEditorOnlyData());
	}
	else
	{
		ShaderMapId.LayoutParams.InitializeForCurrent();
	}

	return ShaderMapId;
}

void EnsureGlobalsInitialized(const FNiagaraSystemCompilationTask& CompilationTask)
{
	// in order to work around the fact that the target platform API is not thread safe we need to make sure that the target platform and the
	// various shader formats it handles has been initialized
	GetTargetPlatformManagerRef().ShaderFormatVersion(TEXT("VVM_1_0"));

	// additionally, make sure that some shader compiler internals are also initialized appropriately
	GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory();
}

void SyncEmitterEnabledState(UNiagaraSystem* System)
{
	if (!System)
	{
		return;
	}

	// use the system scripts to find the appropriate emitter nodes to synchronize
	TArray<UNiagaraNodeEmitter*> EmitterNodes;

	constexpr int32 ExpectedNodesPerEmitter = 2; // spawn + update script
	EmitterNodes.Reserve(ExpectedNodesPerEmitter * System->GetEmitterHandles().Num());

	for (UNiagaraScript* SystemScript : { System->GetSystemSpawnScript(), System->GetSystemUpdateScript() })
	{
		FVersionedNiagaraScriptData* ScriptData = SystemScript ? SystemScript->GetLatestScriptData() : nullptr;
		if (UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(ScriptData ? ScriptData->GetSource() : nullptr))
		{
			if (ScriptSource->NodeGraph)
			{
				TArray<UNiagaraNodeEmitter*> GraphEmitterNodes;
				ScriptSource->NodeGraph->GetNodesOfClass(GraphEmitterNodes);

				for (UNiagaraNodeEmitter* GraphEmitterNode : GraphEmitterNodes)
				{
					if (GraphEmitterNode)
					{
						EmitterNodes.AddUnique(GraphEmitterNode);
					}
				}
			}
		}
	}

	for (UNiagaraNodeEmitter* EmitterNode : EmitterNodes)
	{
		EmitterNode->SyncEnabledState();
	}
}

};

FNiagaraSystemCompilingManager& FNiagaraSystemCompilingManager::Get()
{
	static FNiagaraSystemCompilingManager Singleton;
	return Singleton;
}

FName FNiagaraSystemCompilingManager::GetAssetTypeName() const
{
	return TEXT("UE-NiagaraSystem");
}

FTextFormat FNiagaraSystemCompilingManager::GetAssetNameFormat() const
{
	return LOCTEXT("NiagaraSystemAssetFormat", "{0}|plural(one=Niagara System,other=Niagara Systems)");
}

TArrayView<FName> FNiagaraSystemCompilingManager::GetDependentTypeNames() const
{
	static FName DependentTypeNames[] =
	{
		FShaderCompilingManager::GetStaticAssetTypeName(),
	};

	return TArrayView<FName>(DependentTypeNames);
}

int32 FNiagaraSystemCompilingManager::GetNumRemainingAssets() const
{
	int32 RemainingAssetCount = 0;

	{
		// note that we don't worry about including RequestsAwaitingRetrieval because
		// those tasks do not reflect significant remaining work for the compilation manager.
		// Additionally, it can cause deadlocks in some scenarios as calling code could wait
		// for the remaining assets to get to 0 before advancing to polling for results.
		FReadScopeLock Read(QueueLock);
		RemainingAssetCount = QueuedRequests.Num() + ActiveTasks.Num();
	}

	return RemainingAssetCount;
}

void FNiagaraSystemCompilingManager::FinishCompilationForObjects(TArrayView<UObject* const> InObjects)
{
	if (InObjects.Num() == 0)
	{
		return;
	}
	for (UObject* Iter : InObjects)
	{
		if (UNiagaraSystem* Asset = Cast<UNiagaraSystem>(Iter))
		{
			Asset->WaitForCompilationComplete_SkipPendingOnDemand(true);
		}
	}
}

void FNiagaraSystemCompilingManager::FinishAllCompilation()
{
	TArray<UObject*> SystemsToFinish;
	for (TObjectIterator<UNiagaraSystem> SystemIterator; SystemIterator; ++SystemIterator)
	{
		UNiagaraSystem* Asset = *SystemIterator;
		if (Asset->HasOutstandingCompilationRequests(true))
		{
			SystemsToFinish.Add(Asset);
		}
	}
	FinishCompilationForObjects(SystemsToFinish);
}

void FNiagaraSystemCompilingManager::Shutdown()
{
}

void FNiagaraSystemCompilingManager::CheckStalledTask(double CurrentTime, FNiagaraSystemCompilationTask* Task) const
{
	const double ElapsedTime = CurrentTime - Task->LaunchStartTime;

	if (ElapsedTime > GNiagaraCompilationStalledTaskWarningTime)
	{
		bool bWarn = true;

		if (!Task->bStalled)
		{
			Task->LastStallWarningTime = CurrentTime;
			Task->bStalled = true;

		}
		else
		{
			const double TimeSinceLastWarning = CurrentTime - Task->LastStallWarningTime;
			if (TimeSinceLastWarning < GNiagaraCompilationStalledTaskWarningTime)
			{
				bWarn = false;
			}
		}

		if (bWarn)
		{
			UE_LOGF(LogNiagaraEditor, Log, "NiagaraSystemCompilingManager - compilation task [%ls] stalled for %f seconds.  Status - %ls",
				*Task->GetDescription(), (float)ElapsedTime, *Task->GetStatusString());

			Task->LastStallWarningTime = CurrentTime;
		}
	}
}

void FNiagaraSystemCompilingManager::ProcessAsyncTasks(bool bLimitExecutionTime)
{
	{
		COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeSyncWork(); Timer.TrackCyclesOnly(););

		// process any pending GameThreadTasks
		{
			TArray<FGameThreadFunction> PendingFunctions;

			{
				FWriteScopeLock Write(GameThreadFunctionLock);
				PendingFunctions = MoveTemp(GameThreadFunctions);
			}

			for (FGameThreadFunction& PendingFunction : PendingFunctions)
			{
				PendingFunction();
			}
		}

		{
			double CurrentTime = FPlatformTime::Seconds();

			FReadScopeLock ReadScope(QueueLock);
			for (FNiagaraCompilationTaskHandle TaskHandle : ActiveTasks)
			{
				FTaskPtr TaskPtr = SystemRequestMap.FindRef(TaskHandle);
				if (TaskPtr.IsValid())
				{
					CheckStalledTask(CurrentTime, TaskPtr.Get());
					TaskPtr->Tick();
				}
			}
		}

		{
			FWriteScopeLock WriteScope(QueueLock);

			// find the list of tasks that we can remove
			TArray<FNiagaraCompilationTaskHandle> TasksToRemove;
			TArray<FNiagaraCompilationTaskHandle> TasksToRetrieve;
			for (TArray<FNiagaraCompilationTaskHandle>::TIterator TaskIt(ActiveTasks); TaskIt; ++TaskIt)
			{
				FTaskPtr TaskPtr = SystemRequestMap.FindRef(*TaskIt);

				bool bRemoveCurrent = true;
				if (TaskPtr.IsValid())
				{
					if (TaskPtr->AreResultsPending())
					{
						TasksToRetrieve.Add(*TaskIt);
					}
					else if (TaskPtr->CanRemove())
					{
						TasksToRemove.Add(*TaskIt);
					}
					else
					{
						bRemoveCurrent = false;
					}
				}

				if (bRemoveCurrent)
				{
					TaskIt.RemoveCurrent();
				}
			}

			// go through the entries that are awaiting retrieval and clean up any that have been retrieved
			for (TArray<FNiagaraCompilationTaskHandle>::TIterator TaskIt(RequestsAwaitingRetrieval); TaskIt; ++TaskIt)
			{
				FTaskPtr TaskPtr = SystemRequestMap.FindRef(*TaskIt);
				if (!TaskPtr.IsValid() || TaskPtr->CanRemove())
				{
					TasksToRemove.Add(*TaskIt);
					TaskIt.RemoveCurrent();
				}
			}

			// remove tasks that can be erased
			for (FNiagaraCompilationTaskHandle TaskToRemove : TasksToRemove)
			{
				ensure(!QueuedRequests.Contains(TaskToRemove));
				SystemRequestMap.Remove(TaskToRemove);
			}

			// finally populate RequestsAwaitingRetrieval with any new entries
			for (FNiagaraCompilationTaskHandle TaskToRetrieve : TasksToRetrieve)
			{
				ensure(!QueuedRequests.Contains(TaskToRetrieve));
				RequestsAwaitingRetrieval.Add(TaskToRetrieve);
			}
		}
	}

	// queue up as many tasks as we can
	while (ConditionalLaunchTask())
	{
		// just keep launching tasks until we run out of room
	}
}

FNiagaraCompilationTaskHandle FNiagaraSystemCompilingManager::AddSystem(UNiagaraSystem* System, FCompileOptions CompileOptions)
{
	check(IsInGameThread());

	struct FCompilableScriptInfo
	{
		FCompilableScriptInfo(UNiagaraScript* InScript, bool bForced, int32 InEmitterIndex, bool InGpuEmitter, bool& bHasCompilation)
			: Script(InScript)
			, EmitterIndex(InEmitterIndex)
		{
			const bool bIsValidScriptTarget = Script
				&& Script->IsCompilable();
			// when we successfully get rid of the CPU side scripts (particle spawn/update) for GPU emitters we can reinstate this check
			//	&& InGpuEmitter == Script->IsGPUScript();

			if (!bIsValidScriptTarget)
			{
				return;
			}

			Script->ComputeVMCompilationId(CompileId, FGuid());
			bRequiresCompilation = !CompileId.IsValid() || CompileId != Script->GetVMExecutableDataCompilationId();

			bHasCompilation = bHasCompilation || bRequiresCompilation;
		}

		void UpdateComputeShaders(bool InGpuEmitter, FNiagaraShaderType* ShaderType, const ITargetPlatform* TargetPlatform, TConstArrayView<FPlatformFeatureLevelPair> FeatureLevels, bool& bHasCompilation)
		{
			// check if the GPU shaders need compilation
			if (UNiagaraScript::AreGpuScriptsCompiledBySystem() && InGpuEmitter && Script->IsGPUScript())
			{
				ShaderRequests.Reserve(FeatureLevels.Num());

				for (const FPlatformFeatureLevelPair& PlatformFeatureLevel : FeatureLevels)
				{
					const bool bScriptIsMissingOrDirty = true;

					const FNiagaraShaderMapId ShaderMapId = NiagaraSystemCompilingManagerImpl::BuildShaderMapId(
						ShaderType,
						TargetPlatform,
						PlatformFeatureLevel.Key,
						PlatformFeatureLevel.Value,
						CompileId);

					if (bRequiresCompilation || !Script->IsShaderMapCached(TargetPlatform, ShaderMapId))
					{
						if (Script->ShouldCompile(PlatformFeatureLevel.Key))
						{
							FNiagaraSystemCompilationTask::FShaderCompileRequest& Request = ShaderRequests.AddDefaulted_GetRef();
							Request.ShaderMapId = ShaderMapId;
							Request.ShaderPlatform = PlatformFeatureLevel.Key;
						}
					}
				}

				// for GPU scripts we only need to worry about compilation if we actually have some shaders that are required.  So we
				// override bRequiresCompilation based on that so platforms that exclude all shaders will not generate a compile request
				bRequiresCompilation = !ShaderRequests.IsEmpty();
			}

			bHasCompilation = bHasCompilation || bRequiresCompilation;
		}

		FNiagaraVMExecutableDataId CompileId;
		TArray<FNiagaraSystemCompilationTask::FShaderCompileRequest> ShaderRequests;
		UNiagaraScript* Script;
		int32 EmitterIndex = INDEX_NONE;
		bool bRequiresCompilation = false;
	};

	// before we evaluate the system and it's scripts we need to do a quick validation check on the NiagaraEmitterNodes.  This is done in the original
	// compilation mode during PrecompileDuplicate, and while not strictly part of the compilation mode we need to be sure to correct any emitters that
	// are incorrectly disabled.  There's likely a bug somewhere else that is resulting in some nodes sometimes getting stuck in the disabled mode, but
	// this is the best way we have to clean that up for now.
	NiagaraSystemCompilingManagerImpl::SyncEmitterEnabledState(System);

	TArray<FPlatformFeatureLevelPair> FeatureLevels;
	FindOrAddFeatureLevels(CompileOptions, FeatureLevels);

	TArray<UNiagaraScript*> AllScripts;
	TArray<FCompilableScriptInfo> ScriptsToCompile;
	bool bHasScriptToCompile = false;

	// ensure that all necessary graphs have been digested
	for (UNiagaraScript* SystemScript : { System->GetSystemSpawnScript(), System->GetSystemUpdateScript() })
	{
		AllScripts.Add(SystemScript);
		ScriptsToCompile.Emplace(SystemScript, CompileOptions.bForced, INDEX_NONE /*EmitterIndex*/, false /*InGpuEmitter*/, bHasScriptToCompile);
	}

	const TArray<FNiagaraEmitterHandle>& EmitterHandles = System->GetEmitterHandles();
	const int32 EmitterCount = EmitterHandles.Num();
	for (int32 EmitterIt = 0; EmitterIt < EmitterCount; ++EmitterIt)
	{
		const FNiagaraEmitterHandle& Handle = EmitterHandles[EmitterIt];
		if (!Handle.GetIsEnabled())
		{
			continue;
		}

		if (CompileOptions.TargetPlatform)
		{
			if (const UNiagaraEmitter* Emitter = Handle.GetInstance().Emitter)
			{
				if (!Emitter->NeedsLoadForTargetPlatform(CompileOptions.TargetPlatform))
				{
					continue;
				}
			}
		}

		if (const FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData())
		{
			constexpr bool bCompilableOnly = false; // we want to include emitter scripts for parameter store processing
			constexpr bool bEnabledOnly = true; // skip disabled stages

			TArray<UNiagaraScript*> EmitterScripts;
			EmitterData->GetScripts(EmitterScripts, bCompilableOnly, bEnabledOnly);

			const bool bGpuEmitter = EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim;
			for (UNiagaraScript* EmitterScript : EmitterScripts)
			{
				AllScripts.Add(EmitterScript);
				FCompilableScriptInfo& ScriptToCompile = ScriptsToCompile.Emplace_GetRef(EmitterScript, CompileOptions.bForced, EmitterIt, bGpuEmitter, bHasScriptToCompile);
				ScriptToCompile.UpdateComputeShaders(bGpuEmitter, NiagaraShaderType, CompileOptions.TargetPlatform, FeatureLevels, bHasScriptToCompile);
			}
		}
	}

	if (!bHasScriptToCompile)
	{
		return INDEX_NONE;
	}

	FNiagaraCompilationTaskHandle RequestHandle = NextTaskHandle++;

	{
		COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeSyncWork(); Timer.TrackCyclesOnly(););


		// do we really need to care about wrapping?
		if (RequestHandle == INDEX_NONE)
		{
			RequestHandle = NextTaskHandle++;
		}

		FTaskPtr CompilationTask;

		{
			FWriteScopeLock WriteLock(QueueLock);

			QueuedRequests.Add(RequestHandle);
			CompilationTask = SystemRequestMap.Add(
				RequestHandle,
				MakeShared<FNiagaraSystemCompilationTask, ESPMode::ThreadSafe>(RequestHandle, System, CompileOptions.RIParamMode));
		}

		CompilationTask->PrepareStartTime = FPlatformTime::Seconds();

		// we're going to have to compile something so let's digest all the collections and build our compilation task
		CompilationTask->DigestParameterCollections(CompileOptions.ParameterCollections);
		CompilationTask->DigestSystemInfo();
		CompilationTask->DigestShaderInfo(CompileOptions.TargetPlatform, NiagaraShaderType);

		CompilationTask->bForced = CompileOptions.bForced;

		for (const FCompilableScriptInfo& ScriptToCompile : ScriptsToCompile)
		{
			CompilationTask->AddScript(ScriptToCompile.EmitterIndex, ScriptToCompile.Script, ScriptToCompile.CompileId, ScriptToCompile.bRequiresCompilation, ScriptToCompile.ShaderRequests);
		}

		NiagaraSystemCompilingManagerImpl::EnsureGlobalsInitialized(*CompilationTask);

		CompilationTask->QueueStartTime = FPlatformTime::Seconds();
	}

	ConditionalLaunchTask();

	return RequestHandle;
}

bool FNiagaraSystemCompilingManager::PollSystemCompile(FNiagaraCompilationTaskHandle TaskHandle, bool bPeek, bool bWait, FNiagaraSystemAsyncCompileResults& Results)
{
	FTaskPtr TaskPtr;

	{
		FReadScopeLock ReadLock(QueueLock);

		TaskPtr = SystemRequestMap.FindRef(TaskHandle);
	}

	if (TaskPtr.IsValid())
	{
		if (bWait)
		{
			COOK_STAT(auto Timer = NiagaraSystemCookStats::UsageStats.TimeAsyncWait(); Timer.TrackCyclesOnly(););

			TaskPtr->WaitTillCompileCompletion();
		}

		if (TaskPtr->Poll(Results))
		{
			if (!bPeek)
			{
				TaskPtr->ResultsRetrieved = true;
			}

			return true;
		}
	}

	return false;
}

void FNiagaraSystemCompilingManager::AbortSystemCompile(FNiagaraCompilationTaskHandle TaskHandle)
{
	FTaskPtr TaskPtr;

	{
		FReadScopeLock ReadLock(QueueLock);

		TaskPtr = SystemRequestMap.FindRef(TaskHandle);
	}

	if (TaskPtr.IsValid())
	{
		TaskPtr->Abort();
	}
}

void FNiagaraSystemCompilingManager::AdvanceAsyncTasks()
{
	GShaderCompilingManager->ProcessAsyncResults(true /*TimeSlice*/, false /*BlockOnGlobalShaderCompletion*/);
	ProcessAsyncTasks(true /*bLimitExecutionTime*/);
}

void FNiagaraSystemCompilingManager::QueueGameThreadFunction(FGameThreadFunction GameThreadTask)
{
	FWriteScopeLock Write(GameThreadFunctionLock);

	GameThreadFunctions.Add(GameThreadTask);
}

bool FNiagaraSystemCompilingManager::ConditionalLaunchTask()
{
	{
		FReadScopeLock Read(QueueLock);

		const int32 ActiveRequestCount = ActiveTasks.Num();
		const int32 QueuedRequestCount = QueuedRequests.Num();

		if (QueuedRequestCount == 0)
		{
			return false;
		}
		else if (ActiveRequestCount >= GNiagaraCompilationMaxActiveTaskCount)
		{
			PeakPendingTaskCount = FMath::Max(PeakPendingTaskCount, QueuedRequestCount);

			if (PeakPendingTaskCount > PeakReportedPendingTaskCount)
			{
				constexpr double MinTimeBetweenReports = 5.0;
				const double CurrentTime = FPlatformTime::Seconds();
				if (CurrentTime - LastTaskCountReportTime >= MinTimeBetweenReports)
				{
					PeakReportedPendingTaskCount = PeakPendingTaskCount;
					LastTaskCountReportTime = CurrentTime;

					UE_LOGF(LogNiagaraEditor, Display, "Wanted to launch a task, but there's already %d active tasks - keeping %d requests queued. Consider increasing fx.Niagara.Compilation.MaxActiveTaskCount which is currently %d",
						ActiveRequestCount, PeakPendingTaskCount, GNiagaraCompilationMaxActiveTaskCount);
				}
			}
			return false;
		}
	}

	{
		FWriteScopeLock Write(QueueLock);

		if (!QueuedRequests.IsEmpty() && ActiveTasks.Num() < GNiagaraCompilationMaxActiveTaskCount)
		{
			FNiagaraCompilationTaskHandle TaskHandle = QueuedRequests[0];
			QueuedRequests.RemoveAt(0);

			FTaskPtr CompileTask = SystemRequestMap.FindRef(TaskHandle);
			if (CompileTask.IsValid())
			{
				CompileTask->LaunchStartTime = FPlatformTime::Seconds();
				CompileTask->BeginTasks();
				ActiveTasks.Add(TaskHandle);

				return true;
			}
		}
	}

	return false;
}

void FNiagaraSystemCompilingManager::FindOrAddFeatureLevels(const FCompileOptions& CompileOptions, TArray<FPlatformFeatureLevelPair>& FeatureLevels)
{
	if (NiagaraShaderType == nullptr)
	{
		NiagaraShaderType = NiagaraSystemCompilingManagerImpl::GetNiagaraShaderType();
	}

	if (ensure(NiagaraShaderType))
	{
		if (CompileOptions.TargetPlatform)
		{
			TArray<FPlatformFeatureLevelPair>* CachedFeatureLevels = PlatformFeatureLevels.Find(CompileOptions.TargetPlatform);
			if (!CachedFeatureLevels)
			{
				CachedFeatureLevels = &PlatformFeatureLevels.Add(CompileOptions.TargetPlatform);

				TArray<FName> DesiredShaderFormats;
				CompileOptions.TargetPlatform->GetAllTargetedShaderFormats(DesiredShaderFormats);
				for (const FName& ShaderFormat : DesiredShaderFormats)
				{
					const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(ShaderFormat);
					ERHIFeatureLevel::Type TargetFeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);

					if (NiagaraShaderType->ShouldCompilePermutation(FShaderPermutationParameters(ShaderPlatform)))
					{
						CachedFeatureLevels->AddUnique(MakeTuple(ShaderPlatform, TargetFeatureLevel));
					}
				}
			}

			FeatureLevels = *CachedFeatureLevels;
		}
		else
		{
			// if no target platform has been supplied then we just use the current preview feature level
			FeatureLevels.Add(MakeTuple(CompileOptions.PreviewShaderPlatform, CompileOptions.PreviewFeatureLevel));
		}
	}
}

FNiagaraCompilationTaskHandle FNiagaraEditorModule::RequestCompileSystem(UNiagaraSystem* System, bool bForced, const ITargetPlatform* TargetPlatform)
{
	FNiagaraSystemCompilingManager::FCompileOptions CompileOptions;
	CompileOptions.bForced = bForced;
	CompileOptions.TargetPlatform = TargetPlatform;

	CompileOptions.PreviewFeatureLevel = UNiagaraScript::GetPreviewFeatureLevel();
	CompileOptions.PreviewShaderPlatform = GShaderPlatformForFeatureLevel[CompileOptions.PreviewFeatureLevel];

	ParameterCollectionAssetCache.RefreshCache(!FUObjectThreadContext::Get().IsRoutingPostLoad /*bAllowLoading*/);
	const TArray<TWeakObjectPtr<UNiagaraParameterCollection>>& Collections = ParameterCollectionAssetCache.Get();
	CompileOptions.ParameterCollections = ParameterCollectionAssetCache.Get();

	return FNiagaraSystemCompilingManager::Get().AddSystem(System, CompileOptions);
}

bool FNiagaraEditorModule::PollSystemCompile(FNiagaraCompilationTaskHandle TaskHandle, FNiagaraSystemAsyncCompileResults& Results, bool bWait, bool bPeek)
{
	return FNiagaraSystemCompilingManager::Get().PollSystemCompile(TaskHandle, bPeek, bWait, Results);
}

void FNiagaraEditorModule::AbortSystemCompile(FNiagaraCompilationTaskHandle TaskHandle)
{
	return FNiagaraSystemCompilingManager::Get().AbortSystemCompile(TaskHandle);
}

#undef LOCTEXT_NAMESPACE