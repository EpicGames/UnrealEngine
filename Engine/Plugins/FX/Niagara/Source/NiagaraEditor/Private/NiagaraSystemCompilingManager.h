// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAssetCompilingManager.h"

#include "NiagaraCompilationPrivate.h"
#include "NiagaraCompilationTypes.h"
#include "NiagaraSystem.h"

struct FNiagaraSystemCompilationTask;
class ITargetPlatform;

class FNiagaraSystemCompilingManager : public IAssetCompilingManager
{
public:
	struct FCompileOptions
	{
		TArray<TWeakObjectPtr<UNiagaraParameterCollection>> ParameterCollections;
		const ITargetPlatform* TargetPlatform = nullptr;
		EShaderPlatform PreviewShaderPlatform = EShaderPlatform::SP_NumPlatforms;
		ERHIFeatureLevel::Type PreviewFeatureLevel = ERHIFeatureLevel::Num;
		bool bForced = false;
		ENiagaraCompileRIParamMode RIParamMode = ENiagaraCompileRIParamMode::GameThread;
	};

	static FNiagaraSystemCompilingManager& Get();

	FNiagaraCompilationTaskHandle AddSystem(UNiagaraSystem* System, FCompileOptions CompileOptions);
	bool PollSystemCompile(FNiagaraCompilationTaskHandle TaskHandle, bool bPeek, bool bWait, FNiagaraSystemAsyncCompileResults& Results);
	void AbortSystemCompile(FNiagaraCompilationTaskHandle TaskHandle);

	void AdvanceAsyncTasks();

	using FGameThreadFunction = TFunction<void()>;
	void QueueGameThreadFunction(FGameThreadFunction GameThreadTask);

protected:
	// Begin - IAssetCompilingManager
	virtual FName GetAssetTypeName() const override;
	virtual FTextFormat GetAssetNameFormat() const override;
	virtual TArrayView<FName> GetDependentTypeNames() const override;
	virtual int32 GetNumRemainingAssets() const override;
	virtual void FinishCompilationForObjects(TArrayView<UObject* const> InObjects) override;
	virtual void FinishAllCompilation() override;
	virtual void Shutdown() override;
	virtual void ProcessAsyncTasks(bool bLimitExecutionTime = false) override;
	// End - IAssetCompilingManager

	bool ConditionalLaunchTask();
	void CheckStalledTask(double CurrentTime, FNiagaraSystemCompilationTask* Task) const;

	FNiagaraShaderType* NiagaraShaderType = nullptr;

	using FPlatformFeatureLevelPair = TPair<EShaderPlatform, ERHIFeatureLevel::Type>;
	TMap<const ITargetPlatform*, TArray<FPlatformFeatureLevelPair>> PlatformFeatureLevels;
	void FindOrAddFeatureLevels(const FCompileOptions& CompileOptions, TArray<FPlatformFeatureLevelPair>& FeatureLevels);

	using FTaskPtr = TSharedPtr<FNiagaraSystemCompilationTask, ESPMode::ThreadSafe>;

	mutable FRWLock QueueLock;
	TMap<FNiagaraCompilationTaskHandle, FTaskPtr> SystemRequestMap;
	TArray<FNiagaraCompilationTaskHandle> ActiveTasks;
	TArray<FNiagaraCompilationTaskHandle> QueuedRequests;
	TArray<FNiagaraCompilationTaskHandle> RequestsAwaitingRetrieval;

	std::atomic<FNiagaraCompilationTaskHandle> NextTaskHandle = { 0 };

	mutable FRWLock GameThreadFunctionLock;
	TArray<FGameThreadFunction> GameThreadFunctions;

	int32 PeakReportedPendingTaskCount = 0;
	int32 PeakPendingTaskCount = 0;
	double LastTaskCountReportTime = 0;
};
