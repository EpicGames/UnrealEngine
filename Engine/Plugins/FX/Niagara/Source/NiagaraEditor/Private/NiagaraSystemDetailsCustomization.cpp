// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemDetailsCustomization.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "NiagaraSystem.h"
#include "Toolkits/NiagaraSystemToolkit.h"

TSharedRef<IDetailCustomization> FNiagaraSystemDetails::MakeInstance()
{
	return MakeShared<FNiagaraSystemDetails>();
}

void FNiagaraSystemDetails::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{
	// we display the scalability category within scalability mode, which is why we hide it here
	InDetailLayout.HideCategory("Scalability");

	InDetailLayout.SortCategories(
		[](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
		{
			for (const TPair<FName, IDetailCategoryBuilder*>& Pair : CategoryMap)
			{
				int32 SortOrder = Pair.Value->GetSortOrder();
				const FName& CategoryName = Pair.Key;

				if (CategoryName == "System")
				{
					SortOrder = 1;
				}
				else if (CategoryName == "Rendering")
				{
					SortOrder = 2;
				}
				else if (CategoryName == "Debug")
				{
					SortOrder = 3;
				}
				else if (CategoryName == "Performance")
				{
					SortOrder = 4;
				}
				else
				{
					SortOrder += 5;
				}

				Pair.Value->SetSortOrder(SortOrder);
			}
		}
	);
}
