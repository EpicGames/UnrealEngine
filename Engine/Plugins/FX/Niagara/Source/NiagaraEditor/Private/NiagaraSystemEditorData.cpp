// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemEditorData.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraCustomVersion.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraOverviewNode.h"
#include "NiagaraNode.h"
#include "EdGraphSchema_NiagaraSystemOverview.h"
#include "NiagaraConstants.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSettings.h"
#include "EdGraph/EdGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSystemEditorData)

static constexpr float SystemOverviewNodePadding = 250.0f;

const FName UNiagaraSystemEditorFolder::GetFolderName() const
{
	return FolderName;
}

void UNiagaraSystemEditorFolder::SetFolderName(FName InFolderName)
{
	FolderName = InFolderName;
}

const TArray<UNiagaraSystemEditorFolder*>& UNiagaraSystemEditorFolder::GetChildFolders() const
{
	return ChildFolders;
}

void UNiagaraSystemEditorFolder::AddChildFolder(UNiagaraSystemEditorFolder* ChildFolder)
{
	Modify();
	ChildFolders.Add(ChildFolder);
}

void UNiagaraSystemEditorFolder::RemoveChildFolder(UNiagaraSystemEditorFolder* ChildFolder)
{
	Modify();
	ChildFolders.Remove(ChildFolder);
}

const TArray<FGuid>& UNiagaraSystemEditorFolder::GetChildEmitterHandleIds() const
{
	return ChildEmitterHandleIds;
}

void UNiagaraSystemEditorFolder::AddChildEmitterHandleId(FGuid ChildEmitterHandleId)
{
	Modify();
	ChildEmitterHandleIds.Add(ChildEmitterHandleId);
}

void UNiagaraSystemEditorFolder::RemoveChildEmitterHandleId(FGuid ChildEmitterHandleId)
{
	Modify();
	ChildEmitterHandleIds.Remove(ChildEmitterHandleId);
}

UNiagaraSystemEditorData::UNiagaraSystemEditorData(const FObjectInitializer& ObjectInitializer)
{
	RootFolder = ObjectInitializer.CreateDefaultSubobject<UNiagaraSystemEditorFolder>(this, TEXT("RootFolder"));
	StackEditorData = ObjectInitializer.CreateDefaultSubobject<UNiagaraStackEditorData>(this, TEXT("StackEditorData"));
	UserParameterHierarchy = CreateDefaultSubobject<UHierarchyRoot>(TEXT("UserParameterHierarchyRoot"));
	OwnerTransform.SetLocation(FVector(0.0f, 0.0f, 0.0f));
	PlaybackRangeMin = 0;
	PlaybackRangeMax = 10;
	bSystemIsPlaceholder = false;
	
	if(const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>())
	{
		SetUseOrbitMode(Settings->bSystemViewportInOrbitMode);
	}
}

void UNiagaraSystemEditorData::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad) == false)
	{
		SystemOverviewGraph = NewObject<UEdGraph>(this, "NiagaraOverview", RF_Transactional);
		SystemOverviewGraph->Schema = UEdGraphSchema_NiagaraSystemOverview::StaticClass();
	}
}

void UNiagaraSystemEditorData::PostLoadFromOwner(UObject* InOwner)
{
	UNiagaraSystem* OwningSystem = CastChecked<UNiagaraSystem>(InOwner, ECastCheckedType::NullChecked);

	if (RootFolder == nullptr)
	{
		RootFolder = NewObject<UNiagaraSystemEditorFolder>(this, TEXT("RootFolder"), RF_Transactional);
	}
	if (StackEditorData == nullptr)
	{
		StackEditorData = NewObject<UNiagaraStackEditorData>(this, TEXT("StackEditorData"), RF_Transactional);
	}
	if (UserParameterHierarchy == nullptr)
	{
		UserParameterHierarchy = NewObject<UHierarchyRoot>(this, TEXT("UserParameterHierarchyRoot"), RF_Transactional);
	}

	TArray<FNiagaraVariable> UserParameters;
	OwningSystem->GetExposedParameters().GetUserParameters(UserParameters);
	for(FNiagaraVariable& UserParameter : UserParameters)
	{
		FNiagaraVariable RedirectedUserParameter = UserParameter;
		OwningSystem->GetExposedParameters().RedirectUserVariable(RedirectedUserParameter);
		FindOrAddUserScriptVariable(RedirectedUserParameter, *OwningSystem);
	}
	
	const int32 NiagaraVer = GetLinkerCustomVersion(FNiagaraCustomVersion::GUID);

	if (NiagaraVer < FNiagaraCustomVersion::PlaybackRangeStoredOnSystem)
	{
		UpdatePlaybackRangeFromEmitters(*OwningSystem);
	}

	if (SystemOverviewGraph == nullptr)
	{
		SystemOverviewGraph = NewObject<UEdGraph>(this, "NiagaraOverview", RF_Transactional);
		SystemOverviewGraph->Schema = UEdGraphSchema_NiagaraSystemOverview::StaticClass();
	}

	// we make sure to refresh the script variables mapping to the user parameter store
	SyncUserScriptVariables(OwningSystem);
	InitOnSyncScriptVariables(*OwningSystem);

	if(SystemOverviewGraph->Nodes.Num() == 0)
	{
		SynchronizeOverviewGraphWithSystem(*OwningSystem);
	}

	PostLoad_TransferModuleStackNotesToNewFormat(OwningSystem);
	
	// Remove any niagara nodes which may have been pasted into the overview graph in error.  This is no longer possible.
	TArray<UNiagaraNode*> NiagaraNodes;
	SystemOverviewGraph->GetNodesOfClass<UNiagaraNode>(NiagaraNodes);
	for (UNiagaraNode* NiagaraNode : NiagaraNodes)
	{
		NiagaraNode->DestroyNode();
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraSystemEditorData::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UEdGraph::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraOverviewNode::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UNiagaraScriptVariable::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UHierarchyRoot::StaticClass()));
}
#endif

UNiagaraSystemEditorFolder& UNiagaraSystemEditorData::GetRootFolder() const
{
	return *RootFolder;
}

UNiagaraStackEditorData& UNiagaraSystemEditorData::GetStackEditorData() const
{
	return *StackEditorData;
}

TRange<float> UNiagaraSystemEditorData::GetPlaybackRange() const
{
	return TRange<float>(PlaybackRangeMin, PlaybackRangeMax);
}

void UNiagaraSystemEditorData::SetPlaybackRange(TRange<float> InPlaybackRange)
{
	PlaybackRangeMin = InPlaybackRange.GetLowerBoundValue();
	PlaybackRangeMax = InPlaybackRange.GetUpperBoundValue();
}

void UNiagaraSystemEditorData::SetPlaybackFrameRate(const FFrameRate InPlaybackFrameRate)
{
	PlaybackFrameRate = InPlaybackFrameRate;
}

void UNiagaraSystemEditorData::SetLockPlaybackFrameRate(bool bInLockPlaybackFrameRate)
{
	bLockPlaybackFrameRate = bInLockPlaybackFrameRate;
}

UEdGraph* UNiagaraSystemEditorData::GetSystemOverviewGraph() const
{
	return SystemOverviewGraph;	
}

const FNiagaraGraphViewSettings& UNiagaraSystemEditorData::GetSystemOverviewGraphViewSettings() const
{
	return OverviewGraphViewSettings;
}

void UNiagaraSystemEditorData::SetSystemOverviewGraphViewSettings(const FNiagaraGraphViewSettings& InOverviewGraphViewSettings)
{
	OverviewGraphViewSettings = InOverviewGraphViewSettings;
}

bool UNiagaraSystemEditorData::GetOwningSystemIsPlaceholder() const
{
	return bSystemIsPlaceholder;
}

void UNiagaraSystemEditorData::SetOwningSystemIsPlaceholder(bool bInSystemIsPlaceholder, UNiagaraSystem& OwnerSystem)
{
	bSystemIsPlaceholder = bInSystemIsPlaceholder;
	SynchronizeOverviewGraphWithSystem(OwnerSystem);
}

void UNiagaraSystemEditorData::UpdatePlaybackRangeFromEmitters(UNiagaraSystem& OwnerSystem)
{
	if (OwnerSystem.GetEmitterHandles().Num() > 0)
	{
		float EmitterPlaybackRangeMin = TNumericLimits<float>::Max();
		float EmitterPlaybackRangeMax = TNumericLimits<float>::Lowest();

		for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem.GetEmitterHandles())
		{
			UNiagaraEmitterEditorData* EmitterEditorData = Cast<UNiagaraEmitterEditorData>(EmitterHandle.GetEmitterData()->GetEditorData());
			if (EmitterEditorData != nullptr)
			{
				EmitterPlaybackRangeMin = FMath::Min(PlaybackRangeMin, EmitterEditorData->GetPlaybackRange().GetLowerBoundValue());
				EmitterPlaybackRangeMax = FMath::Max(PlaybackRangeMax, EmitterEditorData->GetPlaybackRange().GetUpperBoundValue());
			}
		}

		PlaybackRangeMin = EmitterPlaybackRangeMin;
		PlaybackRangeMax = EmitterPlaybackRangeMax;
	}
}

void UNiagaraSystemEditorData::PostLoad_TransferModuleStackNotesToNewFormat(UObject* InOwner)
{
	// since we lack graph context during post load, we do this workaround to find the emitter graph
	UNiagaraSystem* System = Cast<UNiagaraSystem>(InOwner);

	TArray<UNiagaraScriptSourceBase*> SourceBases;
	
	if(UNiagaraScript* SystemSpawnScript = System->GetSystemSpawnScript())
	{
		SystemSpawnScript->ConditionalPostLoad();
		
		if (UNiagaraScriptSource* GraphSource = Cast<UNiagaraScriptSource>(SystemSpawnScript->GetLatestSource()))
        {
			GraphSource->ConditionalPostLoad();
			SourceBases.Add(GraphSource);
        }
	}
	
	if(UNiagaraScript* SystemUpdateScript = System->GetSystemUpdateScript())
	{
		SystemUpdateScript->ConditionalPostLoad();

		if (UNiagaraScriptSource* GraphSource = Cast<UNiagaraScriptSource>(SystemUpdateScript->GetLatestSource()))
		{
			GraphSource->ConditionalPostLoad();
			SourceBases.Add(GraphSource);
		}
	}
	
	for(UNiagaraScriptSourceBase* SourceBase : SourceBases)
	{		
		UNiagaraScriptSource* Source = CastChecked<UNiagaraScriptSource>(SourceBase);

		TArray<UNiagaraNodeFunctionCall*> FunctionCallNodes;
		Source->NodeGraph->GetNodesOfClass(FunctionCallNodes);

		for(UNiagaraNodeFunctionCall* FunctionCallNode : FunctionCallNodes)
		{
			if(FunctionCallNode->GetDeprecatedCustomNotes().Num() > 0)
			{
				StackEditorData->TransferDeprecatedStackNotes(*FunctionCallNode);
			}
		}
	}	
}

void FindOuterNodes(const TArray<UNiagaraOverviewNode*> OverviewNodes, UNiagaraOverviewNode*& OutLeft, UNiagaraOverviewNode*& OutRight)
{
	if (OverviewNodes.Num() > 0)
	{
		OutLeft = OverviewNodes[0];
		OutRight = OutLeft;
		for (int32 i = 1; i < OverviewNodes.Num(); i++)
		{
			UNiagaraOverviewNode* Current = OverviewNodes[i];
			if (Current->NodePosX < OutLeft->NodePosX)
			{
				OutLeft = Current;
			}
			if (Current->NodePosX > OutRight->NodePosX)
			{
				OutRight = Current;
			}
		}
	}
	else
	{
		OutLeft = nullptr;
		OutRight = nullptr;
	}
}

void UNiagaraSystemEditorData::SynchronizeOverviewGraphWithSystem(UNiagaraSystem& OwnerSystem)
{
	TArray<UNiagaraOverviewNode*> OverviewNodes;
	SystemOverviewGraph->GetNodesOfClass<UNiagaraOverviewNode>(OverviewNodes);

	UNiagaraOverviewNode* LeftNode;
	UNiagaraOverviewNode* RightNode;
	FindOuterNodes(OverviewNodes, LeftNode, RightNode);

	// Validate system node.
	int32 SystemNodeIndex = OverviewNodes.IndexOfByPredicate([] (UNiagaraOverviewNode* OverviewNode) { return OverviewNode->GetEmitterHandleGuid().IsValid() == false; });
	if (SystemNodeIndex != INDEX_NONE)
	{
		if (bSystemIsPlaceholder)
		{
			// Systems which are placeholder are for editing emitter assets and shouldn't have system nodes in their overview graph.
			UNiagaraOverviewNode* SystemNode = OverviewNodes[SystemNodeIndex];
			SystemNode->DestroyNode();
		}

		// Remove the node from the list to track which nodes are no longer used.
		OverviewNodes.RemoveAt(SystemNodeIndex);
	}
	else
	{
		if(bSystemIsPlaceholder == false)
		{
			SystemOverviewGraph->Modify();
			FGraphNodeCreator<UNiagaraOverviewNode> SystemOverviewNodeCreator(*SystemOverviewGraph);
			UNiagaraOverviewNode* SystemOverviewNode = SystemOverviewNodeCreator.CreateNode(false);
			SystemOverviewNode->Initialize(&OwnerSystem);

			FVector2D SystemOverviewNodeLocation = LeftNode != nullptr
				? FVector2D(LeftNode->NodePosX - SystemOverviewNodePadding, LeftNode->NodePosY)
				: SystemOverviewGraph->GetGoodPlaceForNewNode();
			SystemOverviewNode->NodePosX = SystemOverviewNodeLocation.X;
			SystemOverviewNode->NodePosY = SystemOverviewNodeLocation.Y;

			SystemOverviewNodeCreator.Finalize();

			if (RightNode == nullptr)
			{
				RightNode = SystemOverviewNode;
			}
		}
	}

	// Validate emitter nodes.
	for (const FNiagaraEmitterHandle& EmitterHandle : OwnerSystem.GetEmitterHandles())
	{
		int32 EmitterNodeIndex = OverviewNodes.IndexOfByPredicate([&EmitterHandle] (UNiagaraOverviewNode* OverviewNode) { return OverviewNode->GetEmitterHandleGuid() == EmitterHandle.GetId(); });
		UNiagaraOverviewNode* EmitterNode = nullptr;
		if (EmitterNodeIndex != INDEX_NONE)
		{
			// If a node for this emitter exists already, remove it from the collection so that we can track nodes for emitters which no longer exist.
			EmitterNode = OverviewNodes[EmitterNodeIndex];
			OverviewNodes.RemoveAt(EmitterNodeIndex);
		}
		else
		{
			SystemOverviewGraph->Modify();
			FGraphNodeCreator<UNiagaraOverviewNode> EmitterOverviewNodeCreator(*SystemOverviewGraph);
			UNiagaraOverviewNode* EmitterOverviewNode = EmitterOverviewNodeCreator.CreateNode(false);
			EmitterOverviewNode->Initialize(&OwnerSystem, EmitterHandle.GetId());

			FVector2D CurrentPos(EmitterOverviewNode->NodePosX, EmitterOverviewNode->NodePosY);
			FVector2D EmitterOverviewNodeLocation = CurrentPos;

			// callbacks in CreateNode might have set an explicit position already. If so, we don't update the position
			if(EmitterOverviewNodeLocation.IsZero())
			{
				EmitterOverviewNodeLocation = RightNode != nullptr
					? FVector2D(RightNode->NodePosX + SystemOverviewNodePadding, RightNode->NodePosY)
					: SystemOverviewGraph->GetGoodPlaceForNewNode();
			}

			checkf(EmitterOverviewNode != nullptr, TEXT("Emitter overview node creation failed!"));
			EmitterOverviewNode->NodePosX = EmitterOverviewNodeLocation.X;
			EmitterOverviewNode->NodePosY = EmitterOverviewNodeLocation.Y;

			EmitterOverviewNodeCreator.Finalize();

			RightNode = EmitterOverviewNode;
			EmitterNode = EmitterOverviewNode;
		}
		EmitterNode->Modify();
		EmitterNode->SetEnabledState(EmitterHandle.GetIsEnabled() ? ENodeEnabledState::Enabled : ENodeEnabledState::Disabled);
	}

	// If there are any nodes remaining in the list they're no longer being used so destroy them.
	for (UNiagaraOverviewNode* OverviewNode : OverviewNodes)
	{
		OverviewNode->Modify();
		OverviewNode->DestroyNode();
	}

	// Dispatch an empty graph changed message here so that any graph UIs which are visible refresh their node's widgets.
	SystemOverviewGraph->NotifyGraphChanged();
}

void UNiagaraSystemEditorData::InitOnSyncScriptVariables(UNiagaraSystem& System)
{
	System.GetExposedParameters().AddOnChangedHandler(FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraSystemEditorData::SyncUserScriptVariables, &System));
}

void UNiagaraSystemEditorData::SyncUserScriptVariables(UNiagaraSystem* System)
{
	TArray<FNiagaraVariable> UserParameters;
	System->GetExposedParameters().GetUserParameters(UserParameters);

	TArray<UNiagaraScriptVariable*> CurrentVariables = UserParameterMetaData;

	TSet<UNiagaraScriptVariable*> UpdatedScriptVariables;
	for(FNiagaraVariable UserParameter : UserParameters)
	{
		UpdatedScriptVariables.Add(FindOrAddUserScriptVariable(UserParameter, *System));
	}
	
	for(UNiagaraScriptVariable* CurrentVariable : CurrentVariables)
	{
		if(!UpdatedScriptVariables.Contains(CurrentVariable))
		{
			UserParameterMetaData.Remove(CurrentVariable);
		}
	}

	OnUserParameterScriptVariablesSyncedDelegate.Broadcast();
}

UNiagaraScriptVariable* UNiagaraSystemEditorData::FindOrAddUserScriptVariable(FNiagaraVariable UserParameter, UNiagaraSystem& System)
{
	bool bSuccess = System.GetExposedParameters().RedirectUserVariable(UserParameter);
	if(bSuccess)
	{
		TObjectPtr<UNiagaraScriptVariable>* ExistingScriptVariable = UserParameterMetaData.FindByPredicate([UserParameter](const UNiagaraScriptVariable* ScriptVariable)
			{
				return ScriptVariable->Variable == UserParameter;
			});

		if(ExistingScriptVariable == nullptr)
		{
			TObjectPtr<UNiagaraScriptVariable> NewScriptVariable = NewObject<UNiagaraScriptVariable>(this);
			FNiagaraVariableMetaData MetaData;
			NewScriptVariable->Init(UserParameter, MetaData);
			UserParameterMetaData.Add(NewScriptVariable);
			return NewScriptVariable;
		}
		
		return *ExistingScriptVariable;
	}

	return nullptr;
}

const UNiagaraScriptVariable* UNiagaraSystemEditorData::FindUserScriptVariable(FGuid UserParameterGuid) const
{
	const TObjectPtr<UNiagaraScriptVariable>* ExistingScriptVariable = UserParameterMetaData.FindByPredicate([UserParameterGuid](const UNiagaraScriptVariable* ScriptVariable)
		{
			return ScriptVariable->Metadata.GetVariableGuid() == UserParameterGuid;
		});

	if(ExistingScriptVariable)
	{
		return *ExistingScriptVariable;
	}
	
	return nullptr;
}

const UNiagaraScriptVariable* UNiagaraSystemEditorData::FindUserScriptVariable(FNiagaraVariable UserParameter, const UNiagaraSystem& System) const
{
	bool bSuccess = System.GetExposedParameters().RedirectUserVariable(UserParameter);
	if (bSuccess)
	{
		const TObjectPtr<UNiagaraScriptVariable>* ExistingScriptVariable = UserParameterMetaData.FindByPredicate([UserParameter]
			(const UNiagaraScriptVariable* ScriptVariable) { return ScriptVariable->Variable == UserParameter; });
		if (ExistingScriptVariable != nullptr)
		{
			return *ExistingScriptVariable;
		}
	}
	return nullptr;
}

bool UNiagaraSystemEditorData::RenameUserScriptVariable(FNiagaraVariable OldVariable, FName NewName)
{
	TObjectPtr<UNiagaraScriptVariable>* ExistingScriptVariable = UserParameterMetaData.FindByPredicate([OldVariable](const UNiagaraScriptVariable* ScriptVariable)
		{
			return ScriptVariable->Variable == OldVariable;
		});

	if(ExistingScriptVariable == nullptr)
	{
		return false;
	}

	(*ExistingScriptVariable)->Modify();
	(*ExistingScriptVariable)->Variable.SetName(NewName);
	(*ExistingScriptVariable)->UpdateChangeId();
	return true;
}

bool UNiagaraSystemEditorData::RemoveUserScriptVariable(FNiagaraVariable Variable)
{
	return UserParameterMetaData.RemoveAll([Variable](const UNiagaraScriptVariable* ScriptVariable)
		{
			return ScriptVariable->Variable == Variable;
		}) > 0;
}

const FNiagaraPerAssetViewportSettings& UNiagaraSystemEditorData::GetAssetViewportSettings() const
{
	return AssetViewportSettings;
}

void UNiagaraSystemEditorData::SetAssetViewportSettings(FNiagaraPerAssetViewportSettings InSettings)
{
	AssetViewportSettings = InSettings;
}

void UNiagaraSystemEditorData::SetUseOrbitMode(bool bInUseOrbitMode)
{
	AssetViewportSettings.bUseOrbitMode = bInUseOrbitMode;
}
