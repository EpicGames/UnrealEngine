// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemFactoryNew.h"
#include "CoreMinimal.h"
#include "NiagaraSystem.h"
#include "NiagaraScriptSource.h"
#include "NiagaraGraph.h"
#include "NiagaraEditorSettings.h"
#include "AssetRegistry/AssetData.h"
#include "ContentBrowserModule.h"
#include "EditorDirectories.h"
#include "ImageUtils.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Framework/Application/SlateApplication.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraRecentAndFavoritesManager.h"
#include "NiagaraSettings.h"
#include "ObjectTools.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Widgets/STaggedAssetBrowser.h"
#include "Widgets/STaggedAssetBrowserAssetFactoryWindow.h"
#include "Widgets/AssetBrowser/SNiagaraAddEmitterToSystemWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSystemFactoryNew)

#define LOCTEXT_NAMESPACE "NiagaraSystemFactory"

UNiagaraSystemFactoryNew::UNiagaraSystemFactoryNew()
{
	SupportedClass = UNiagaraSystem::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
	SupportedWorkflows = (uint8) (EFactoryCreateWorkflow::Default | EFactoryCreateWorkflow::Asynchronous);
}

bool UNiagaraSystemFactoryNew::ConfigureProperties()
{
	// Needs to return true to allow async path
	return true;
}

bool UNiagaraSystemFactoryNew::ConfigurePropertiesAsync(FOnFactoryConfigurePropertiesAsyncComplete OnComplete, FOnFactoryConfigurePropertiesAsyncCancelled OnCancelled)
{	
	FSoftObjectPath Path = GetDefault<UNiagaraEditorSettings>()->SystemAssetWizardConfiguration;
	UObject* ConfigUObject = Path.TryLoad();
	if(UTaggedAssetBrowserConfiguration* ConfigurationAsset = Cast<UTaggedAssetBrowserConfiguration>(ConfigUObject))
	{		
		STaggedAssetBrowser::FInterfaceOverrideProfiles OverrideProfiles;
		OverrideProfiles.AssetViewOptionsProfileName = FName("TaggedAssetBrowser");
		OverrideProfiles.DefaultFilterMenuExpansion = EAssetTypeCategories::FX;
		OverrideProfiles.FilterBarSaveName = FName("TaggedAssetBrowser.NiagaraSystemFactory");
		
		FDefaultDetailsTabConfiguration DefaultDetailsTabConfiguration;
		DefaultDetailsTabConfiguration.bUseDefaultDetailsTab = true;
		DefaultDetailsTabConfiguration.EmptySelectionMessage = LOCTEXT("EmptySystemFactorySelectionMessage", "Select an emitter or system as a starting point for your new system.\nA system consists of one or more emitters.");

		// We create a pseudo asset based on the path we currently want to create an asset at.
		// This is then used for asset reference filtering to only show assets that this pseudo asset/path would be compatible with
		
		// @TODO (ME): This is not ideal. Ideally the factory would have access to creation path and/or property handle (if created from property asset picker -> create menu)
		// then we wouldn't need to use GetLastDirectory which is relatively unsafe
		FString PackagePath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
		FString AssetName = "TmpAsset";
		FName PackageName = FName(PackagePath + "/" + AssetName);
		FAssetData PseudoReferencingAsset(PackageName, FName(PackagePath), FName(AssetName), UObject::StaticClass()->GetClassPathName());
		
		STaggedAssetBrowser::FArguments AssetBrowserArgs;
		AssetBrowserArgs
			.AvailableClasses({UNiagaraEmitter::StaticClass(), UNiagaraSystem::StaticClass()})
			.DefaultDetailsTabConfiguration(DefaultDetailsTabConfiguration)
			.InterfaceOverrideProfiles(OverrideProfiles)
			.RecentAndFavoritesList(FNiagaraEditorModule::Get().GetRecentsManager()->GetRecentEmitterAndSystemsList())
			.OnGetExtraFrontendFilters_UObject(this, &UNiagaraSystemFactoryNew::OnGetExtraFrontendFilters)
			.OnExtendAddFilterMenu_UObject(this, &UNiagaraSystemFactoryNew::OnExtendAddFilterMenu)
			.AdditionalReferencingAssets({ PseudoReferencingAsset });

		SWindow::FArguments WindowArgs;
		WindowArgs.Title(LOCTEXT("SystemAssetBrowserWindowTitle", "Create Niagara System - Select an emitter or system as a base"));
		WindowArgs.SupportsMaximize(false);
		WindowArgs.SupportsMinimize(false);
		WindowArgs.ClientSize(FVector2D(1400, 750));
		WindowArgs.SizingRule(ESizingRule::UserSized);
		
		STaggedAssetBrowserWindow::FArguments AssetBrowserWindowArgs;
		AssetBrowserWindowArgs.AssetBrowserArgs(AssetBrowserArgs);
		AssetBrowserWindowArgs.WindowArgs(WindowArgs);

		STaggedAssetBrowserAssetFactoryWindow::FAsyncFactoryArguments AsyncArguments;
		AsyncArguments.Factory.Reset(this);
		AsyncArguments.OnAssetsActivated = FOnAssetsActivated::CreateUObject(this, &UNiagaraSystemFactoryNew::OnAssetsActivated);
		AsyncArguments.OnFactoryConfigurePropertiesComplete = OnComplete;
		AsyncArguments.OnFactoryConfigurePropertiesCancelled = OnCancelled;
		
		TSharedRef<STaggedAssetBrowserAssetFactoryWindow> CreateAssetBrowserWindow = SNew(STaggedAssetBrowserAssetFactoryWindow, *ConfigurationAsset, *UNiagaraSystem::StaticClass())
			.AssetBrowserWindowArgs(AssetBrowserWindowArgs)
			.AsyncFactoryArguments(AsyncArguments);

		FSlateApplication::Get().AddWindow(CreateAssetBrowserWindow, true);

		return true;
	}

	// If no browser configuration was found, we call OnComplete immediately, creating our asset
	OnComplete.ExecuteIfBound(this);
	return true;
}

UObject* UNiagaraSystemFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UNiagaraSystem::StaticClass()));

	const UNiagaraEditorSettings* Settings = GetDefault<UNiagaraEditorSettings>();
	check(Settings);

	UNiagaraSystem* NewSystem;
	
	if (SystemToCopy.IsValid())
	{
		if (SystemToCopy->IsReadyToRun() == false)
		{
			SystemToCopy->WaitForCompilationComplete();
		}
		
		FNiagaraEditorModule::Get().GetRecentsManager()->SystemUsed(*SystemToCopy);

		NewSystem = Cast<UNiagaraSystem>(StaticDuplicateObject(SystemToCopy.Get(), InParent, Name, Flags, Class));
		NewSystem->TemplateAssetDescription = FText();
		NewSystem->Category = FText();
		//NewSystem->AssetTags.Empty();

		// if the new system doesn't have a thumbnail image, check the thumbnail map of the original asset's upackage
		if(NewSystem->ThumbnailImage == nullptr)
		{
			FString ObjectFullName = SystemToCopy->GetFullName();
			FName ObjectName = FName(ObjectFullName);
			FString PackageFullName;
			ThumbnailTools::QueryPackageFileNameForObject(ObjectFullName, PackageFullName);
			FThumbnailMap ThumbnailMap;
			ThumbnailTools::ConditionallyLoadThumbnailsFromPackage(PackageFullName, {FName(ObjectFullName)}, ThumbnailMap);

			// there should always be a dummy thumbnail in here
			if(ThumbnailMap.Contains(ObjectName))
			{
				FObjectThumbnail Thumbnail = ThumbnailMap[ObjectName];
				// we only want to copy the thumbnail over if it's not a dummy
				if(Thumbnail.GetImageWidth() != 0 && Thumbnail.GetImageHeight() != 0)
				{
					ThumbnailTools::CacheThumbnail(NewSystem->GetFullName(), &Thumbnail, NewSystem->GetOutermost());
				}
			}
		}
	}
	else if (EmittersToAddToNewSystem.Num() > 0)
	{
		NewSystem = NewObject<UNiagaraSystem>(InParent, Class, Name, Flags | RF_Transactional);
		InitializeSystem(NewSystem, true);

		for (FVersionedNiagaraEmitter& EmitterToAddToNewSystem : EmittersToAddToNewSystem)
		{
			FNiagaraEditorUtilities::AddEmitterToSystem(*NewSystem, *EmitterToAddToNewSystem.Emitter, EmitterToAddToNewSystem.Version);
		}
	}
	else
	{
		NewSystem = NewObject<UNiagaraSystem>(InParent, Class, Name, Flags | RF_Transactional);
		InitializeSystem(NewSystem, true);
	}

	TryAssignDefaultEffectType(NewSystem);
	
	NewSystem->RequestCompile(false);

	FNiagaraEditorModule::Get().GetRecentsManager()->SystemUsed(*NewSystem);
	return NewSystem;
}

void UNiagaraSystemFactoryNew::InitializeSystem(UNiagaraSystem* System, bool bCreateDefaultNodes)
{
	UNiagaraScript* SystemSpawnScript = System->GetSystemSpawnScript();
	UNiagaraScript* SystemUpdateScript = System->GetSystemUpdateScript();

	UNiagaraScriptSource* SystemScriptSource = NewObject<UNiagaraScriptSource>(SystemSpawnScript, "SystemScriptSource", RF_Transactional);

	if (SystemScriptSource)
	{
		SystemScriptSource->NodeGraph = NewObject<UNiagaraGraph>(SystemScriptSource, "SystemScriptGraph", RF_Transactional);
	}
	
	SystemSpawnScript->SetLatestSource(SystemScriptSource);
	SystemUpdateScript->SetLatestSource(SystemScriptSource);

	if (bCreateDefaultNodes)
	{
		FSoftObjectPath SystemUpdateScriptRef = GetDefault<UNiagaraEditorSettings>()->RequiredSystemUpdateScript;
		UNiagaraScript* Script = Cast<UNiagaraScript>(SystemUpdateScriptRef.TryLoad());

		FAssetData ModuleScriptAsset(Script);
		if (SystemScriptSource && ModuleScriptAsset.IsValid())
		{
			UNiagaraNodeOutput* SpawnOutputNode = FNiagaraStackGraphUtilities::ResetGraphForOutput(*SystemScriptSource->NodeGraph, ENiagaraScriptUsage::SystemSpawnScript, SystemSpawnScript->GetUsageId());
			UNiagaraNodeOutput* UpdateOutputNode = FNiagaraStackGraphUtilities::ResetGraphForOutput(*SystemScriptSource->NodeGraph, ENiagaraScriptUsage::SystemUpdateScript, SystemUpdateScript->GetUsageId());

			if (UpdateOutputNode)
			{
				FNiagaraStackGraphUtilities::AddScriptModuleToStack(ModuleScriptAsset, *UpdateOutputNode);
			}
			FNiagaraStackGraphUtilities::RelayoutGraph(*SystemScriptSource->NodeGraph);
		}
	}
}

void UNiagaraSystemFactoryNew::TryAssignDefaultEffectType(UNiagaraSystem* System)
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);
	
	if(System->GetEffectType() == nullptr && Settings->GetDefaultEffectType())
	{
		UNiagaraEffectType* DefaultEffectType = Settings->GetDefaultEffectType();
		System->SetEffectType(DefaultEffectType);
	}
}

void UNiagaraSystemFactoryNew::OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type ActivationMethod)
{
	if(AssetData.Num() == 1)
	{
		const FAssetData& FirstAssetData = AssetData[0];
		if(FirstAssetData.GetClass() == UNiagaraSystem::StaticClass())
		{
			SystemToCopy = Cast<UNiagaraSystem>(FirstAssetData.GetAsset());
		}
		else if(FirstAssetData.GetClass() == UNiagaraEmitter::StaticClass())
		{
			UNiagaraEmitter* EmitterAsset = Cast<UNiagaraEmitter>(FirstAssetData.GetAsset());
			FVersionedNiagaraEmitter VersionedNiagaraEmitter(EmitterAsset, EmitterAsset->GetExposedVersion().VersionGuid);
				
			EmittersToAddToNewSystem.Add(VersionedNiagaraEmitter);
		}
	}
}

bool UNiagaraSystemFactoryNew::OnAdditionalShouldFilterAsset(const FAssetData& AssetData) const
{
	const UNiagaraEditorSettings* Settings = GetDefault<UNiagaraEditorSettings>();
	if(Settings->IsAllowedAssetInNiagaraAssetBrowser(AssetData) == false)
	{
		return true;
	}
	
	if(Settings->IsAllowedAssetByClassUsage(AssetData) == false)
	{
		return true;
	}

	return false;
}

void UNiagaraSystemFactoryNew::OnExtendAddFilterMenu(UToolMenu* ToolMenu) const
{
	for (FToolMenuSection& Section : ToolMenu->Sections)
	{
		Section.Blocks.RemoveAll([](FToolMenuEntry& ToolMenuEntry)
		{
			return ToolMenuEntry.Name == FName("Common");
		});
	}
	
	TArray<FName> SectionsToKeep { FName("FilterBarAdvanced"), FName("FilterBarResetFilters"), FName("FX Filters"), FName("AssetFilterBarFilterAdvancedAsset"), FName("Niagara Filters"), FName("Niagara Tags") };
	ToolMenu->Sections.RemoveAll([&SectionsToKeep](FToolMenuSection& ToolMenuSection)
	{
		return SectionsToKeep.Contains(ToolMenuSection.Name) == false;
	});
}

TArray<TSharedRef<FFrontendFilter>> UNiagaraSystemFactoryNew::OnGetExtraFrontendFilters() const
{
	TArray<TSharedRef<FFrontendFilter>> Result;
	
	TSharedRef<FFrontendFilterCategory> NiagaraAdditionalFiltersCategory = MakeShared<FFrontendFilterCategory>(LOCTEXT("NiagaraPropertyFilterCategoryLabel", "Niagara Filters"), LOCTEXT("NiagaraAdditionalFiltersTooltip", "Additional filters for filtering Niagara assets"));
	
	Result.Add(MakeShared<FFrontendFilter_NiagaraEmitterInheritance>(true, NiagaraAdditionalFiltersCategory));
	Result.Add(MakeShared<FFrontendFilter_NiagaraEmitterInheritance>(false, NiagaraAdditionalFiltersCategory));
	
	Result.Add(MakeShared<FFrontendFilter_NiagaraSystemEffectType>(true, NiagaraAdditionalFiltersCategory));
	Result.Add(MakeShared<FFrontendFilter_NiagaraSystemEffectType>(false, NiagaraAdditionalFiltersCategory));

	return Result;
}

#undef LOCTEXT_NAMESPACE

