// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemStateDataResolver.h"

#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraParameterMapHistory.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraStackFunctionInputBinder.h"

namespace FNiagaraSystemStateDataResolver
{
	class FSystemStateScriptResolver : public FNiagaraSystemStateData::IScriptResolver
	{
	public:
		explicit FSystemStateScriptResolver(const UNiagaraSystem* NiagaraSystem, UNiagaraNodeFunctionCall* SystemStateFunctionCall)
		{
			FCompileConstantResolver ConstantResolver = FCompileConstantResolver(NiagaraSystem, ENiagaraScriptUsage::SystemUpdateScript);

			// We need to const_cast as the binder API can both read and write, but we only need read
			UNiagaraScript* SystemScript = const_cast<UNiagaraScript*>(NiagaraSystem->GetSystemUpdateScript());

			// Bind static switch
			{
				const FName NAME_LoopBehavior("Loop Behavior");
				const FName NAME_UseLoopDelay("UseLoopDelay");
				const FName NAME_InactiveReponse("Inactive Response");

				TArray<UNiagaraNodeStaticSwitch*> StaticSwitches;
				UNiagaraGraph* NiagaraGraph = SystemStateFunctionCall->GetCalledGraph();
				NiagaraGraph->GetNodesOfClass<UNiagaraNodeStaticSwitch>(StaticSwitches);

				auto FindStaticSwitch =
					[&StaticSwitches](FName SwitchName) -> UNiagaraNodeStaticSwitch*
					{
						for (UNiagaraNodeStaticSwitch* Switch : StaticSwitches)
						{
							if (Switch->InputParameterName == SwitchName)
							{
								return Switch;
							}
						}
						return nullptr;
					};

				WeakInactiveResponseSwitch = FindStaticSwitch(NAME_InactiveReponse);
				WeakLoopBehaviorSwitch = FindStaticSwitch(NAME_LoopBehavior);
				WeakUseLoopDelaySwitch = FindStaticSwitch(NAME_UseLoopDelay);
			}

			// Bind constants
			{
				const FNiagaraVariableBase DelayFirstLoopOnly(FNiagaraTypeDefinition::GetBoolDef(), FName("Module.Delay First Loop Only"));
				const FNiagaraVariableBase LoopCount(FNiagaraTypeDefinition::GetIntDef(), FName("Module.Loop Count"));
				const FNiagaraVariableBase LoopDelay(FNiagaraTypeDefinition::GetFloatDef(), FName("Module.Loop Delay"));
				const FNiagaraVariableBase LoopDuration(FNiagaraTypeDefinition::GetFloatDef(), FName("Module.Loop Duration"));
				const FNiagaraVariableBase RecalculateDurationEachLoop(FNiagaraTypeDefinition::GetBoolDef(), FName("Module.Recalculate Duration Each Loop"));

				TArray<UNiagaraScript*> DependentScripts;
				FString OwningEmitterName;
				FText ErrorMessage;

				DelayFirstLoopOnlyInputBinder.TryBind(SystemScript, DependentScripts, ConstantResolver, OwningEmitterName, SystemStateFunctionCall, DelayFirstLoopOnly.GetName(), DelayFirstLoopOnly.GetType(), false, ErrorMessage);
				LoopCountInputBinder.TryBind(SystemScript, DependentScripts, ConstantResolver, OwningEmitterName, SystemStateFunctionCall, LoopCount.GetName(), LoopCount.GetType(), false, ErrorMessage);
				LoopDelayInputBinder.TryBind(SystemScript, DependentScripts, ConstantResolver, OwningEmitterName, SystemStateFunctionCall, LoopDelay.GetName(), LoopDelay.GetType(), false, ErrorMessage);
				LoopDurationInputBinder.TryBind(SystemScript, DependentScripts, ConstantResolver, OwningEmitterName, SystemStateFunctionCall, LoopDuration.GetName(), LoopDuration.GetType(), false, ErrorMessage);
				RecalculateDurationEachLoopInputBinder.TryBind(SystemScript, DependentScripts, ConstantResolver, OwningEmitterName, SystemStateFunctionCall, RecalculateDurationEachLoop.GetName(), RecalculateDurationEachLoop.GetType(), false, ErrorMessage);
			}
		}

		bool IsValid() const
		{
			return WeakInactiveResponseSwitch.IsValid() && WeakLoopBehaviorSwitch.IsValid() && WeakUseLoopDelaySwitch.IsValid();
		}

		virtual void ResolveData(FNiagaraSystemStateData& SystemStateData) const override
		{
			// Resolving can fail so set to run by default
			SystemStateData.bRunUpdateScript = true;

			if (!ResolveInactiveReponse(SystemStateData))
			{
				return;
			}

			if (!ResolveLoopBehavior(SystemStateData))
			{
				return;
			}

			if (!ResolveLoopDelay(SystemStateData))
			{
				return;
			}

			// Everything resolved as expected we don't need to run any scripts
			SystemStateData.bRunUpdateScript = false;
		}

	private:
		bool ResolveInactiveReponse(FNiagaraSystemStateData& SystemStateData) const
		{
			if (const UNiagaraNodeStaticSwitch* InactiveResponseSwitch = WeakInactiveResponseSwitch.Get())
			{
				TOptional<int> Value = InactiveResponseSwitch->GetSwitchValue();
				if (!Value.IsSet())
				{
					return false;
				}
				SystemStateData.InactiveResponse = ENiagaraSystemInactiveResponse(Value.GetValue());
				return SystemStateData.InactiveResponse == ENiagaraSystemInactiveResponse::Complete || SystemStateData.InactiveResponse == ENiagaraSystemInactiveResponse::Kill;
			}
			return false;
		}

		bool ResolveLoopBehavior(FNiagaraSystemStateData& SystemStateData) const
		{
			// LoopBehavior
			{
				const UNiagaraNodeStaticSwitch* LoopBehaviorSwitch = WeakLoopBehaviorSwitch.Get();
				TOptional<int> LoopBehaviorValue = LoopBehaviorSwitch ? TOptional<int>(LoopBehaviorSwitch->GetSwitchValue()) : TOptional<int>();
				switch (LoopBehaviorValue.Get(-1))
				{
				case 0:	SystemStateData.LoopBehavior = ENiagaraLoopBehavior::Infinite;	break;
				case 1:	SystemStateData.LoopBehavior = ENiagaraLoopBehavior::Once;		break;
				case 2:	SystemStateData.LoopBehavior = ENiagaraLoopBehavior::Multiple;	break;

				default:
					return false;
				}
			}

			// LoopDuration
			{
				TOptional<float> LoopDurationValue = LoopDurationInputBinder.IsValid() ? TOptional<float>(LoopDurationInputBinder.GetValue<float>()) : TOptional<float>();
				if (!LoopDurationValue.IsSet())
				{
					return false;
				}
				SystemStateData.LoopDuration = FNiagaraDistributionRangeFloat(LoopDurationValue.GetValue());
			}

			// LoopCount
			if (SystemStateData.LoopBehavior == ENiagaraLoopBehavior::Multiple)
			{
				TOptional<int> LoopCountValue = LoopCountInputBinder.IsValid() ? TOptional<int>(LoopCountInputBinder.GetValue<int>()) : TOptional<int>();
				if (!LoopCountValue.IsSet())
				{
					return false;
				}
				SystemStateData.LoopCount = LoopCountValue.GetValue();
			}

			// bRecalculateDurationEachLoop
			if (SystemStateData.LoopBehavior != ENiagaraLoopBehavior::Once)
			{
				TOptional<int> RecalculateDurationEachLoopValue = RecalculateDurationEachLoopInputBinder.IsValid() ? TOptional<int>(RecalculateDurationEachLoopInputBinder.GetValue<int>()) : TOptional<int>();
				if (!RecalculateDurationEachLoopValue.IsSet())
				{
					return false;
				}
				SystemStateData.bRecalculateDurationEachLoop = RecalculateDurationEachLoopValue.GetValue() != 0;
			}
			return true;
		}

		bool ResolveLoopDelay(FNiagaraSystemStateData& SystemStateData) const
		{
			// UseLoopDelay
			{
				const UNiagaraNodeStaticSwitch* UseLoopDelaySwitch = WeakUseLoopDelaySwitch.Get();
				TOptional<int> UseLoopDelayValue = UseLoopDelaySwitch ? TOptional<int>(UseLoopDelaySwitch->GetSwitchValue()) : TOptional<int>();
				if (!UseLoopDelayValue.IsSet())
				{
					return false;
				}
				SystemStateData.bLoopDelayEnabled = UseLoopDelayValue.GetValue() != 0;
				if (!SystemStateData.bLoopDelayEnabled)
				{
					return true;
				}
			}

			// bDelayFirstLoopOnly
			{
				TOptional<int> DelayFirstLoopOnlyValue = DelayFirstLoopOnlyInputBinder.IsValid() ? TOptional<int>(DelayFirstLoopOnlyInputBinder.GetValue<int>()) : TOptional<int>();
				if (!DelayFirstLoopOnlyValue.IsSet())
				{
					return false;
				}
				SystemStateData.bDelayFirstLoopOnly = DelayFirstLoopOnlyValue.GetValue() != 0;
			}

			// LoopDelay
			{
				TOptional<float> LoopDelayValue = LoopDelayInputBinder.IsValid() ? TOptional<float>(LoopDelayInputBinder.GetValue<float>()) : TOptional<float>();
				if (!LoopDelayValue.IsSet())
				{
					return false;
				}
				SystemStateData.LoopDelay = FNiagaraDistributionRangeFloat(LoopDelayValue.GetValue());
			}
			return true;
		}

	private:
		TWeakObjectPtr<UNiagaraNodeStaticSwitch>	WeakInactiveResponseSwitch;
		TWeakObjectPtr<UNiagaraNodeStaticSwitch>	WeakLoopBehaviorSwitch;
		TWeakObjectPtr<UNiagaraNodeStaticSwitch>	WeakUseLoopDelaySwitch;

		FNiagaraStackFunctionInputBinder	DelayFirstLoopOnlyInputBinder;
		FNiagaraStackFunctionInputBinder	LoopCountInputBinder;
		FNiagaraStackFunctionInputBinder	LoopDelayInputBinder;
		FNiagaraStackFunctionInputBinder	LoopDurationInputBinder;
		FNiagaraStackFunctionInputBinder	RecalculateDurationEachLoopInputBinder;
	};

	static bool GbAllowSystemStateResolve = true;
	static FAutoConsoleVariableRef CVarNiagaraDumpSystemData(
		TEXT("fx.Niagara.AllowSystemStateResolve"),
		GbAllowSystemStateResolve,
		TEXT("When enabled we allow system state to be resolved from the module, otherwise the module must be disabled."),
		ECVF_Default
	);

	TOptional<FNiagaraSystemStateData> TryResolve(const UNiagaraSystem& NiagaraSystem)
	{
		// All emitters must be stateless currently
		// We can perhaps look at this again, but we always write Emitter.RandomSeed currently even with an empty script
		for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem.GetEmitterHandles())
		{
			if ( EmitterHandle.GetIsEnabled() && EmitterHandle.GetEmitterMode() != ENiagaraEmitterMode::Stateless )
			{
				return TOptional<FNiagaraSystemStateData>();
			}
		}

		// Try to resolve system state from the system scripts
		const UNiagaraScript* Script = NiagaraSystem.GetSystemSpawnScript();
		const UNiagaraScriptSource* ScriptSource = Script ? Cast<UNiagaraScriptSource>(Script->GetLatestSource()) : nullptr;
		if (!ScriptSource || !ScriptSource->NodeGraph)
		{
			return TOptional<FNiagaraSystemStateData>();
		}

		// Look for update script nodes to see if it's possible to avoid running the update script
		FNiagaraSystemStateData SystemStateData;
		if (UNiagaraNodeOutput* UpdateScriptOutput = ScriptSource->NodeGraph->FindEquivalentOutputNode(ENiagaraScriptUsage::SystemUpdateScript, FGuid()))
		{
			TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
			FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*UpdateScriptOutput, ModuleNodes);
			ModuleNodes.RemoveAll([](UNiagaraNodeFunctionCall* Node) { return !Node || !Node->IsNodeEnabled(); });

			const TCHAR* SystemStateName = TEXT("/Niagara/Modules/System/SystemState.SystemState");
			if (ModuleNodes.Num() == 0)
			{
				SystemStateData.bRunUpdateScript = false;
			}
			else if (ModuleNodes.Num() == 1 && ModuleNodes[0]->FunctionScript->GetPathName() == SystemStateName)
			{
				if (GbAllowSystemStateResolve)
				{
					UNiagaraNodeFunctionCall* SystemStateFunctionCall = ModuleNodes[0];

					TSharedRef<FSystemStateScriptResolver> ScriptResolver = MakeShared<FSystemStateScriptResolver>(&NiagaraSystem, SystemStateFunctionCall);
					if (ScriptResolver->IsValid())
					{
						// This ensures the graph holds the right values
						//-TODO: Is there a better way than this?
						FNiagaraParameterMapHistoryBuilder History;
						SystemStateFunctionCall->BuildParameterMapHistory(History);

						SystemStateData.ScriptResolver = ScriptResolver;
						ScriptResolver->ResolveData(SystemStateData);

						SystemStateData.bRunUpdateScript = false;
					}
				}
			}
		}
		
		// If we don't need to execute the update script, do we need to execute the spawn script?
		if (SystemStateData.bRunUpdateScript == false)
		{
			if (UNiagaraNodeOutput* SpawnScriptOutput = ScriptSource->NodeGraph->FindEquivalentOutputNode(ENiagaraScriptUsage::SystemSpawnScript, FGuid()))
			{
				TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
				FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*SpawnScriptOutput, ModuleNodes);
				ModuleNodes.RemoveAll([](UNiagaraNodeFunctionCall* Node) { return !Node || !Node->IsNodeEnabled(); });

				SystemStateData.bRunSpawnScript = ModuleNodes.Num() != 0;
			}
		}

		return SystemStateData;
	}
} //namespace FNiagaraSystemStateDataResolver
