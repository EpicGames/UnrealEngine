// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystem.h"
#include "NiagaraSystemEmitterState.h"

namespace FNiagaraSystemStateDataResolver
{
	TOptional<FNiagaraSystemStateData> TryResolve(const UNiagaraSystem& System);
}
