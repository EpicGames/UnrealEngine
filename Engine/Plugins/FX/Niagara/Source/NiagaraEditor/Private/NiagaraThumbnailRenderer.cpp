// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraThumbnailRenderer.h"
#include "CanvasTypes.h"
#include "Engine/Texture2D.h"
#include "NiagaraEmitter.h"
#include "NiagaraSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraThumbnailRenderer)

bool UNiagaraThumbnailRendererBase::CanVisualizeAsset(UObject* Object)
{
	return GetThumbnailTextureFromObject(Object) != nullptr;
}

void UNiagaraThumbnailRendererBase::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	Super::GetThumbnailSize(GetThumbnailTextureFromObject(Object), Zoom, OutWidth, OutHeight);
}

void UNiagaraThumbnailRendererBase::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	Super::Draw(GetThumbnailTextureFromObject(Object), X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);
}

UTexture2D* UNiagaraEmitterThumbnailRenderer::GetThumbnailTextureFromObject(UObject* Object) const
{
	UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Object);
	if (Emitter && Emitter->ThumbnailImage)
	{
		// we have to have completed build in order to get draw/get size
		Emitter->ThumbnailImage->BlockOnAnyAsyncBuild();
		return Emitter->ThumbnailImage;
	}
	return nullptr;
}

UTexture2D* UNiagaraSystemThumbnailRenderer::GetThumbnailTextureFromObject(UObject* Object) const
{
	UNiagaraSystem* System = Cast<UNiagaraSystem>(Object);
	if (System && System->ThumbnailImage)
	{
		// we have to have completed build in order to get draw/get size
		System->ThumbnailImage->BlockOnAnyAsyncBuild();
		return System->ThumbnailImage;
	}
	return nullptr;
}
