// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraTraversalStateContext.h"

#include "Logging/StructuredLog.h"
#include "NiagaraCompilationPrivate.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraphDigest.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraParameterMapHistory.h"

void FNiagaraTraversalStateContext::PushGraphInternal(const FNiagaraCompilationNode* CallingNode, const FNiagaraCompilationGraph* Graph, const FNiagaraFixedConstantResolver& ConstantResolver)
{
	if (!Graph)
	{
		return;
	}

	// now we actually go through the graph and populate the values
	for (const TUniquePtr<FNiagaraCompilationNode>& Node : Graph->Nodes)
	{
		if (const FNiagaraCompilationNodeStaticSwitch* StaticSwitchNode = Node->AsType<FNiagaraCompilationNodeStaticSwitch>())
		{
			int32 SwitchValue = 0;
			bool IsValueSet = false;

			if (StaticSwitchNode->bSetByCompiler || StaticSwitchNode->bSetByPin)
			{
				const FNiagaraVariable* Found = FNiagaraConstants::FindStaticSwitchConstant(StaticSwitchNode->SwitchConstant);
				FNiagaraVariable Constant = Found ? *Found : FNiagaraVariable();
				if (Found && ConstantResolver.ResolveConstant(Constant))
				{
					if (StaticSwitchNode->SwitchType == ENiagaraStaticSwitchType::Bool)
					{
						SwitchValue = Constant.GetValue<bool>();
						IsValueSet = true;
					}
					else if (StaticSwitchNode->SwitchType == ENiagaraStaticSwitchType::Integer ||
						StaticSwitchNode->SwitchType == ENiagaraStaticSwitchType::Enum)
					{
						SwitchValue = Constant.GetValue<int32>();
						IsValueSet = true;
					}
				}
			}
			else if (!StaticSwitchNode->bSetByCompiler && CallingNode)
			{
				for (const FNiagaraCompilationInputPin& InputPin : CallingNode->InputPins)
				{
					if (InputPin.PinName.IsEqual(StaticSwitchNode->InputParameterName) && InputPin.Variable.GetType() == StaticSwitchNode->InputType)
					{
						if (FNiagaraCompilationNodeStaticSwitch::ResolveConstantValue(InputPin, SwitchValue))
						{
							IsValueSet = true;
						}
					}
				}
			}

			if (IsValueSet)
			{
				const FGuid SwitchNodeHash = FGuid::Combine(TraversalStack.Top().FullStackHash, StaticSwitchNode->NodeGuid);
				if (ensure(!StaticSwitchValueMap.Contains(SwitchNodeHash)))
				{
					StaticSwitchValueMap.Add(SwitchNodeHash, SwitchValue);
				}
			}
			else
			{
				// value doesn't have to be set; it could be set as we process the pins for this graph for real and end up
				// gathering the value by it's static pin
				ensure(StaticSwitchNode->bSetByPin);
			}
		}
		else if (const FNiagaraCompilationNodeFunctionCall* InnerFunctionNode = Node->AsType<FNiagaraCompilationNodeFunctionCall>())
		{
			const FGuid InnerFunctionNodeHash = FGuid::Combine(TraversalStack.Top().FullStackHash, InnerFunctionNode->NodeGuid);

			// based on the original code bInheritDebugState drives whether we use the system value vs NoDebug.  Note
			// that the serialized Debugstate is never used.
			ENiagaraFunctionDebugState CachedDebugState = (CallingNode && InnerFunctionNode->bInheritDebugState)
				? ConstantResolver.GetDebugState()
				: InnerFunctionNode->DebugState;

			if (!FunctionDebugStateMap.Contains(InnerFunctionNodeHash))
			{
				FunctionDebugStateMap.Add(InnerFunctionNodeHash, CachedDebugState);
			}
			else
			{
				UE_LOGFMT(LogNiagaraEditor, Warning, "FNiagaraTraversalStateContext::PushGraphInternal() generated a non-unique InnerFunctionNodeHash.\n" \
					"\t[SourceScript] {SourceScript}\n" \
					"\t[CurrentHash] {CurrentHash}\n" \
					"\t[ExistingDebugState] {ExistingDebugState}\n" \
					"\t[NewDebugState] {NewDebugState}\n" \
					"\t[InnerFunctionNode] {InnerFunctionNodeName} - {InnerFunctionNodeScriptName}",
					("SourceScript", Graph->SourceScriptFullName),
					("CurrentHash", InnerFunctionNodeHash.ToString(EGuidFormats::DigitsWithHyphens)),
					("ExistingDebugState", static_cast<int32>(FunctionDebugStateMap[InnerFunctionNodeHash])),
					("NewDebugState", static_cast<int32>(CachedDebugState)),
					("InnerFunctionNodeName", InnerFunctionNode->FunctionName),
					("InnerFunctionNodeScriptName", InnerFunctionNode->FunctionScriptName)
				);

				UE_LOGF(LogNiagaraEditor, Warning, "FNiagaraTraversalStateContext - Stack");
				for (int32 StackIt = TraversalStack.Num() - 1; StackIt >= 0; --StackIt)
				{
					FString StackMessage = FString::Printf(TEXT("[%d] - %s, %s"),
						StackIt,
						*TraversalStack[StackIt].NodeGuid.ToString(EGuidFormats::DigitsWithHyphens),
						*TraversalStack[StackIt].FullStackHash.ToString(EGuidFormats::DigitsWithHyphens));

#if WITH_NIAGARA_TRAVERSAL_FRIENDLY_NAME
					StackMessage.Append(TEXT(", "));
					StackMessage.Append(TraversalStack[StackIt].FriendlyName);
#endif
					UE_LOGF(LogNiagaraEditor, Warning, "%ls", *StackMessage);
				}
			}

			if (CallingNode)
			{
				for (const FNiagaraCompilationNodeFunctionCall::FTaggedVariable& TaggedVariable : InnerFunctionNode->PropagatedStaticSwitchParameters)
				{
					const FNiagaraCompilationInputPin* ValuePin = InnerFunctionNode->InputPins.FindByPredicate([&TaggedVariable](const FNiagaraCompilationInputPin& InputPin) -> bool
					{
						return InputPin.PinName == TaggedVariable.Key.GetName();
					});

					if (ValuePin)
					{
						const FNiagaraCompilationInputPin* CallerInputPin = CallingNode->InputPins.FindByPredicate([&TaggedVariable](const FNiagaraCompilationInputPin& InputPin) -> bool
						{
							return InputPin.PinName == TaggedVariable.Value;
						});

						if (CallerInputPin)
						{
							const FFunctionDefaultValueMapKey DefaultValueKey = MakeTuple(InnerFunctionNodeHash, ValuePin->PinName);

							const FString* ExistingDefaultValue = FunctionDefaultValueMap.Find(DefaultValueKey);
							if (ensure(!ExistingDefaultValue))
							{
								FunctionDefaultValueMap.Add(DefaultValueKey, CallerInputPin->DefaultValue);
							}
							else
							{
								// generate an error message to help track down the scenario where this has happened
								UE_LOGFMT(LogNiagaraEditor, Warning, "FNiagaraTraversalStateContext::PushGraphInternal() generated a non-unique function call.\n" \
									"\t[SourceScript] {SourceScript}\n" \
									"\t[ExistingDefaultValue] {ExistingDefaultValue}\n" \
									"\t[NewDefaultValue] {NewDefaultValue}\n" \
									"\t[ValuePin->PinName] {PinName}\n" \
									"\t[TaggedVariable] {TaggedVariableName} - {TaggedVariableValue}\n" \
									"\t[InnerFunctionNode] {FunctionName} - {FunctionScriptName}",
									("SourceScript", Graph->SourceScriptFullName),
									("ExistingDefaultValue", ExistingDefaultValue ? *ExistingDefaultValue : TEXT("<null>")),
									("NewDefaultValue", CallerInputPin->DefaultValue),
									("PinName", ValuePin->PinName.ToString()),
									("TaggedVariableName", TaggedVariable.Key.GetName().ToString()),
									("TaggedVariableValue", TaggedVariable.Value.ToString()),
									("FunctionName", InnerFunctionNode->FunctionName),
									("FunctionScriptName", InnerFunctionNode->FunctionScriptName)
									);
								
								UE_LOGF(LogNiagaraEditor, Warning, "FNiagaraTraversalStateContext - Stack");
								for (int32 StackIt = TraversalStack.Num() - 1; StackIt >= 0; --StackIt)
								{
									FString StackMessage = FString::Printf(TEXT("[%d] - %s, %s"),
										StackIt,
										*TraversalStack[StackIt].NodeGuid.ToString(EGuidFormats::DigitsWithHyphens),
										*TraversalStack[StackIt].FullStackHash.ToString(EGuidFormats::DigitsWithHyphens));

#if WITH_NIAGARA_TRAVERSAL_FRIENDLY_NAME
									StackMessage.Append(TEXT(", "));
									StackMessage.Append(TraversalStack[StackIt].FriendlyName);
#endif
									UE_LOGF(LogNiagaraEditor, Warning, "%ls",  *StackMessage);
								}
							}
						}
					}
				}
			}
		}
	}
}

void FNiagaraTraversalStateContext::BeginContext(const FNiagaraCompilationGraph* ParentGraph, const FNiagaraFixedConstantResolver& ConstantResolver)
{
	if (ParentGraph)
	{
		FNiagaraTraversalStackEntry& StackTop = TraversalStack.AddDefaulted_GetRef();
#if WITH_NIAGARA_TRAVERSAL_FRIENDLY_NAME
		StackTop.FriendlyName = TEXT("Root");
#endif

		PushGraphInternal(nullptr, ParentGraph, ConstantResolver);
	}
}

void FNiagaraTraversalStateContext::PushFunction(const FNiagaraCompilationNodeFunctionCall* FunctionCall, const FNiagaraFixedConstantResolver& ConstantResolver)
{
	TOptional<FGuid> CurrentStackHash;
	if (!TraversalStack.IsEmpty())
	{
		CurrentStackHash = TraversalStack.Top().FullStackHash;
	}

	FNiagaraTraversalStackEntry& StackTop = TraversalStack.AddDefaulted_GetRef();
#if WITH_NIAGARA_TRAVERSAL_FRIENDLY_NAME
	StackTop.FriendlyName = FString::Printf(TEXT("FunctionName - %s | FullName - %s | FullTitle - %s | NodeType - %d"),
		*FunctionCall->FunctionName,
		*FunctionCall->FullName,
		*FunctionCall->FullTitle,
		(int32)FunctionCall->NodeType);
#endif

	StackTop.NodeGuid = FunctionCall->NodeGuid;
	StackTop.FullStackHash = CurrentStackHash.IsSet()
		? FGuid::Combine(*CurrentStackHash, StackTop.NodeGuid)
		: StackTop.NodeGuid;

	if (FunctionCall->CalledGraph)
	{
		PushGraphInternal(FunctionCall, FunctionCall->CalledGraph.Get(), ConstantResolver);
	}
}

void FNiagaraTraversalStateContext::PushEmitter(const FNiagaraCompilationNodeEmitter* Emitter)
{
	TOptional<FGuid> CurrentStackHash;
	if (!TraversalStack.IsEmpty())
	{
		CurrentStackHash = TraversalStack.Top().FullStackHash;
	}

	FNiagaraTraversalStackEntry& StackTop = TraversalStack.AddDefaulted_GetRef();
#if WITH_NIAGARA_TRAVERSAL_FRIENDLY_NAME
	StackTop.FriendlyName = FString::Printf(TEXT("EmitterName - %s | FullName - %s | FullTitle - %s | NodeType - %d"),
		*Emitter->EmitterUniqueName,
		*Emitter->FullName,
		*Emitter->FullTitle,
		(int32)Emitter->NodeType);
#endif

	StackTop.NodeGuid = Emitter->NodeGuid;
	StackTop.FullStackHash = CurrentStackHash.IsSet()
		? FGuid::Combine(*CurrentStackHash, StackTop.NodeGuid)
		: StackTop.NodeGuid;
}

void FNiagaraTraversalStateContext::PopFunction(const FNiagaraCompilationNodeFunctionCall* FunctionCall)
{
	check(!TraversalStack.IsEmpty() && TraversalStack.Top().NodeGuid == FunctionCall->NodeGuid);
	TraversalStack.Pop();
}

void FNiagaraTraversalStateContext::PopEmitter(const FNiagaraCompilationNodeEmitter* Emitter)
{
	check(!TraversalStack.IsEmpty() && TraversalStack.Top().NodeGuid == Emitter->NodeGuid);
	TraversalStack.Pop();
}

bool FNiagaraTraversalStateContext::GetStaticSwitchValue(const FGuid& NodeGuid, int32& StaticSwitchValue) const
{
	if (!TraversalStack.IsEmpty())
	{
		if (const int32* ValuePtr = StaticSwitchValueMap.Find(FGuid::Combine(TraversalStack.Top().FullStackHash, NodeGuid)))
		{
			StaticSwitchValue = *ValuePtr;
			return true;
		}
	}
	return false;
}

bool FNiagaraTraversalStateContext::GetFunctionDefaultValue(const FGuid& NodeGuid, FName PinName, FString& FunctionDefaultValue) const
{
	if (!TraversalStack.IsEmpty())
	{
		const FGuid StackGuid = FGuid::Combine(TraversalStack.Top().FullStackHash, NodeGuid);
		if (const FString* ValuePtr = FunctionDefaultValueMap.Find(MakeTuple(StackGuid, PinName)))
		{
			FunctionDefaultValue = *ValuePtr;
			return true;
		}
	}
	return false;
}

bool FNiagaraTraversalStateContext::GetFunctionDebugState(const FGuid& NodeGuid, ENiagaraFunctionDebugState& DebugState) const
{
	if (!TraversalStack.IsEmpty())
	{
		const FGuid NodeHash = FGuid::Combine(TraversalStack.Top().FullStackHash, NodeGuid);
		if (const ENiagaraFunctionDebugState* ValuePtr = FunctionDebugStateMap.Find(NodeHash))
		{
			DebugState = *ValuePtr;
			return true;
		}
	}
	return false;
}

bool FNiagaraTraversalStateContext::GetCurrentDebugState(ENiagaraFunctionDebugState& DebugState) const
{
	if (!TraversalStack.IsEmpty())
	{
		if (const ENiagaraFunctionDebugState* ValuePtr = FunctionDebugStateMap.Find(TraversalStack.Top().FullStackHash))
		{
			DebugState = *ValuePtr;
			return true;
		}
	}
	return false;
}

FNiagaraFixedConstantResolver::FNiagaraFixedConstantResolver()
{
	InitConstants();
	SetScriptUsage(ENiagaraScriptUsage::Function);
	SetDebugState(ENiagaraFunctionDebugState::NoDebug);
}

FNiagaraFixedConstantResolver::FNiagaraFixedConstantResolver(const FTranslator* InTranslator, ENiagaraScriptUsage ScriptUsage, ENiagaraFunctionDebugState DebugState)
	: Translator(InTranslator)
{
	InitConstants();
	SetScriptUsage(ScriptUsage);
	SetDebugState(DebugState);
}

FNiagaraFixedConstantResolver::FNiagaraFixedConstantResolver(const FCompileConstantResolver& SrcConstantResolver)
{
	InitConstants();
	SetScriptUsage(SrcConstantResolver.GetUsage());
	SetDebugState(SrcConstantResolver.CalculateDebugState());

	for (FNiagaraVariable& ResolvedConstant : ResolvedConstants)
	{
		SrcConstantResolver.ResolveConstant(ResolvedConstant);
	}
}

void FNiagaraFixedConstantResolver::InitConstants()
{
	static const FName ConstantNames[(uint8)EResolvedConstant::Count] =
	{
		TEXT("Function.DebugState"),
		TEXT("Script.Usage"),
		TEXT("Script.Context"),
		TEXT("Emitter.Localspace"),
		TEXT("Emitter.Determinism"),
		TEXT("Emitter.InterpolatedSpawn"),
		TEXT("Emitter.SimulationTarget")
	};

	ResolvedConstants =
	{
		{ FNiagaraTypeDefinition::GetFunctionDebugStateEnum(), ConstantNames[(uint8)EResolvedConstant::FunctionDebugState] },
		{ FNiagaraTypeDefinition::GetScriptUsageEnum(), ConstantNames[(uint8)EResolvedConstant::ScriptUsage] },
		{ FNiagaraTypeDefinition::GetScriptContextEnum(), ConstantNames[(uint8)EResolvedConstant::ScriptContext] },
		{ FNiagaraTypeDefinition::GetBoolDef(), ConstantNames[(uint8)EResolvedConstant::EmitterLocalspace] },
		{ FNiagaraTypeDefinition::GetBoolDef(), ConstantNames[(uint8)EResolvedConstant::EmitterDeterminism] },
		{ FNiagaraTypeDefinition::GetBoolDef(), ConstantNames[(uint8)EResolvedConstant::EmitterInterpolatedSpawn] },
		{ FNiagaraTypeDefinition::GetSimulationTargetEnum(), ConstantNames[(uint8)EResolvedConstant::EmitterSimulationTarget] }
	};
}

void FNiagaraFixedConstantResolver::SetScriptUsage(ENiagaraScriptUsage ScriptUsage)
{
	{
		FNiagaraInt32 EnumValue;
		EnumValue.Value = (uint8)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchUsage(ScriptUsage);
		ResolvedConstants[(uint8)EResolvedConstant::ScriptUsage].SetValue(EnumValue);
	}

	{
		FNiagaraInt32 EnumValue;
		EnumValue.Value = (uint8)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchContext(ScriptUsage);
		ResolvedConstants[(uint8)EResolvedConstant::ScriptContext].SetValue(EnumValue);
	}
}

void FNiagaraFixedConstantResolver::SetDebugState(ENiagaraFunctionDebugState DebugState)
{
	FNiagaraInt32 EnumValue;
	EnumValue.Value = (uint8)DebugState;
	ResolvedConstants[(uint8)EResolvedConstant::FunctionDebugState].SetValue(EnumValue);
}

bool FNiagaraFixedConstantResolver::ResolveConstant(FNiagaraVariable& OutConstant) const
{
	// handle translator case
	if (Translator && Translator->GetLiteralConstantVariable(OutConstant))
	{
		return true;
	}

	if (const FNiagaraVariable* ResolvedConstant = ResolvedConstants.FindByKey(OutConstant))
	{
		if (ResolvedConstant->IsDataAllocated())
		{
			OutConstant.SetData(ResolvedConstant->GetData());
			return true;
		}
	}

	return false;
}

FNiagaraFixedConstantResolver FNiagaraFixedConstantResolver::WithDebugState(ENiagaraFunctionDebugState InDebugState) const
{
	FNiagaraFixedConstantResolver Copy = *this;
	Copy.SetDebugState(InDebugState);
	return Copy;
}

FNiagaraFixedConstantResolver FNiagaraFixedConstantResolver::WithUsage(ENiagaraScriptUsage ScriptUsage) const
{
	FNiagaraFixedConstantResolver Copy = *this;
	Copy.SetScriptUsage(ScriptUsage);
	return Copy;
}

ENiagaraFunctionDebugState FNiagaraFixedConstantResolver::GetDebugState() const
{
	FNiagaraInt32 EnumValue = ResolvedConstants[(uint8)EResolvedConstant::FunctionDebugState].GetValue<FNiagaraInt32>();
	return (ENiagaraFunctionDebugState)EnumValue.Value;
}

void FNiagaraFixedConstantResolver::AddChildResolver(const FGuid& ChildId, const FNiagaraFixedConstantResolver& ChildResolver)
{
	if (ensure(FindChildResolver(ChildId) == nullptr))
	{
		ChildResolvers.Emplace(ChildId, ChildResolver);
	}
}

const FNiagaraFixedConstantResolver* FNiagaraFixedConstantResolver::FindChildResolver(const FGuid& ChildId) const
{
	const FNamedResolverPair* ChildResolver = ChildResolvers.FindByPredicate([ChildId](const FNamedResolverPair& NamedPair) -> bool
	{
			return NamedPair.Key == ChildId;
	});

	return ChildResolver ? &ChildResolver->Value : nullptr;
}
