// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"

class FNiagaraCompilationGraph;
class FNiagaraCompilationNode;
class FNiagaraCompilationNodeEmitter;
class FNiagaraCompilationNodeFunctionCall;
class FNiagaraFixedConstantResolver;

#define WITH_NIAGARA_TRAVERSAL_FRIENDLY_NAME (UE_BUILD_DEBUG)

struct FNiagaraTraversalStackEntry
{
	FGuid NodeGuid;
	FGuid FullStackHash;
#if WITH_NIAGARA_TRAVERSAL_FRIENDLY_NAME
	FString FriendlyName;
#endif
};

struct FNiagaraTraversalStateContext
{
	void BeginContext(const FNiagaraCompilationGraph* ParentGraph, const FNiagaraFixedConstantResolver& ConstantResolver);

	void PushFunction(const FNiagaraCompilationNodeFunctionCall* FunctionCall, const FNiagaraFixedConstantResolver& ConstantResolver);
	void PopFunction(const FNiagaraCompilationNodeFunctionCall* FunctionCall);

	void PushEmitter(const FNiagaraCompilationNodeEmitter* Emitter);
	void PopEmitter(const FNiagaraCompilationNodeEmitter* Emitter);

	bool GetStaticSwitchValue(const FGuid& NodeGuid, int32& StaticSwitchValue) const;
	bool GetFunctionDefaultValue(const FGuid& NodeGuid, FName PinName, FString& FunctionDefaultValue) const;
	bool GetFunctionDebugState(const FGuid& NodeGuid, ENiagaraFunctionDebugState& DebugState) const;

	bool GetCurrentDebugState(ENiagaraFunctionDebugState& DebugState) const;

	TArray<FNiagaraTraversalStackEntry> TraversalStack;

	using FFunctionDefaultValueMapKey = TTuple<FGuid /*Traversal stack guid*/, FName /*PinName*/>;

	TMap<FGuid, int32> StaticSwitchValueMap;
	TMap<FFunctionDefaultValueMapKey, FString> FunctionDefaultValueMap;
	TMap<FGuid, ENiagaraFunctionDebugState> FunctionDebugStateMap;

protected:
	void PushGraphInternal(const FNiagaraCompilationNode* CallingNode, const FNiagaraCompilationGraph* Graph, const FNiagaraFixedConstantResolver& ConstantResolver);
};
