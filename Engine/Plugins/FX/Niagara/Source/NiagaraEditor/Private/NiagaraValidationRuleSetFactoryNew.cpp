// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraValidationRuleSetFactoryNew.h"
#include "NiagaraValidationRuleSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraValidationRuleSetFactoryNew)

UNiagaraValidationRuleSetFactoryNew::UNiagaraValidationRuleSetFactoryNew(const FObjectInitializer& ObjectInitializer)
{
	SupportedClass = UNiagaraValidationRuleSet::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UNiagaraValidationRuleSetFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

UObject* UNiagaraValidationRuleSetFactoryNew::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UNiagaraValidationRuleSet>(InParent, InClass, InName, Flags);
}

