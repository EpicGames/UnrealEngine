// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "NiagaraTypes.h"
#include "NiagaraVersionUpgradeBase.h"
#include "Engine/EngineTypes.h"
#include "UObject/Object.h"
#include "NiagaraVersionMetaData.generated.h"

UCLASS(MinimalAPI)
class UNiagaraVersionMetaData : public UObject
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API UNiagaraVersionMetaData();

	/** If true then this version is exposed to the user and is used as the default version for new assets. */
	UPROPERTY(EditAnywhere, Category="Version Details", meta=(EditCondition="!bIsExposedVersion"))
	bool bIsExposedVersion;

	/** Changelist displayed to the user when upgrading to a new script version. */
	UPROPERTY(EditAnywhere, Category="Version Details", meta = (MultiLine = true))
	FText ChangeDescription;

	/** If false then this version is not visible in the version selector dropdown menu of the stack. This is useful to hide work in progress versions without removing the module from the search menu.
	 *  The exposed version is always visible to users. */
	UPROPERTY(EditAnywhere, Category="Version Details", meta=(EditCondition="!bIsExposedVersion"))
	bool bIsVisibleInVersionSelector = true;

	/* True if this asset version is deprecated and should no longer be used.*/
	UPROPERTY(EditAnywhere, Category="Version Details")
	bool bDeprecated = false;

	/* Message to display when the asset is used in an emitter. */
	UPROPERTY(EditAnywhere, Category="Version Details", meta = (EditCondition = "bDeprecated", MultiLine = true))
	FText DeprecationMessage;

	/** Internal version guid, mainly useful for debugging version conflicts. */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category="Version Details")
	FGuid VersionGuid;

	/** Reference to a python script that is executed when the user updates from a previous version to this version. */
	UPROPERTY(EditAnywhere, Category="Scripting")
	ENiagaraPythonUpdateScriptReference UpdateScriptExecution;

	/** Python script to run when updating to this script version. */
	UPROPERTY(EditAnywhere, Category="Scripting", meta=(MultiLine = true, EditCondition="UpdateScriptExecution == ENiagaraPythonUpdateScriptReference::DirectTextEntry"))
	FString PythonUpdateScript;

	/** Asset reference to a python script to run when updating to this script version. */
	UPROPERTY(EditAnywhere, Category="Scripting", meta=(EditCondition="UpdateScriptExecution == ENiagaraPythonUpdateScriptReference::ScriptAsset"))
	FFilePath ScriptAsset;

	/* Custom logic class to convert between version. Will also be used as fallback when no python script is assigned or the python runtime is unavailable. Needs to be a subclass of UNiagaraVersionUpgradeBase.*/
	UPROPERTY(EditAnywhere, Category="Scripting")
	TSubclassOf<UNiagaraVersionUpgradeBase> UpdateUtilityClass;
};
