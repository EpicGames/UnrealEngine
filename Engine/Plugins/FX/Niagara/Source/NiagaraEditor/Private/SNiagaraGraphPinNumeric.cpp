// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphPinNumeric.h"
#include "EdGraphSchema_Niagara.h"

void SNiagaraGraphPinNumeric::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

FSlateColor SNiagaraGraphPinNumeric::GetPinColor() const
{
	FSlateColor OriginalColor = SGraphPin::GetPinColor();
	
	if(UEdGraphPin* SourcePin = GetPinObj())
	{
		if(!SourcePin->IsPendingKill() && SourcePin->LinkedTo.Num() == 1)
		{
			if(UEdGraphPin* ConnectedPin = SourcePin->LinkedTo[0])
			{
				FNiagaraTypeDefinition ConnectedPiNType = UEdGraphSchema_Niagara::PinTypeToTypeDefinition(ConnectedPin->PinType);
				
				if (bIsDiffHighlighted)
				{
					return FSlateColor(FLinearColor(0.9f, 0.2f, 0.15f));
				}
				if (SourcePin->bOrphanedPin)
				{
					return FSlateColor(FLinearColor::Red);
				}
				if (const UEdGraphSchema* Schema = SourcePin->GetSchema())
				{
					if (!GetPinObj()->GetOwningNode()->IsNodeEnabled() || GetPinObj()->GetOwningNode()->IsDisplayAsDisabledForced() || !IsEditingEnabled() || GetPinObj()->GetOwningNode()->IsNodeUnrelated())
					{
						return Schema->GetPinTypeColor(ConnectedPin->PinType) * FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);
					}

					return Schema->GetPinTypeColor(ConnectedPin->PinType) * PinColorModifier;
				}

				return FLinearColor::White;
			}
		}
	}

	return OriginalColor;
}
