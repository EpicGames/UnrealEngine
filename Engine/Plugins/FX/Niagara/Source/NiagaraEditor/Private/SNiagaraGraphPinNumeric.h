// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphPin.h"

/** A graph pin widget for displaying niagara numeric inputs and outputs. */
class SNiagaraGraphPinNumeric : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphPinNumeric) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

private:
	virtual FSlateColor GetPinColor() const override;
};