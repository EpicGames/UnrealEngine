// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraParameterEditor.h"

const float SNiagaraParameterEditor::DefaultInputSize = 125.0f;

void SNiagaraParameterEditor::Construct(const FArguments& InArgs)
{
	HorizontalAlignment = InArgs._HAlign;
	VerticalAlignment = InArgs._VAlign;
	MinimumDesiredWidth = InArgs._MinimumDesiredWidth;
	MaximumDesiredWidth = InArgs._MaximumDesiredWidth;
	bContinousChangeActive = false;
}

void SNiagaraParameterEditor::SetOnBeginValueChange(FOnValueChange InOnBeginValueChange)
{
	OnBeginValueChange = InOnBeginValueChange;
}

void SNiagaraParameterEditor::SetOnEndValueChange(FOnValueChange InOnEndValueChange)
{
	OnEndValueChange = InOnEndValueChange;
}

void SNiagaraParameterEditor::SetOnValueChanged(FOnValueChange InOnValueChanged)
{
	OnValueChanged = InOnValueChanged;
}

const TOptional<float>& SNiagaraParameterEditor::GetMinimumDesiredWidth() const
{
	return MinimumDesiredWidth;
}

void SNiagaraParameterEditor::SetMinimumDesiredWidth(TOptional<float> InMinimumDesiredWidth)
{
	MinimumDesiredWidth = InMinimumDesiredWidth;
}

const TOptional<float>& SNiagaraParameterEditor::GetMaximumDesiredWidth() const
{
	return MaximumDesiredWidth;
}

void SNiagaraParameterEditor::SetMaximumDesiredWidth(TOptional<float> InMaximumDesiredWidth)
{
	MaximumDesiredWidth = InMaximumDesiredWidth;
}

EHorizontalAlignment SNiagaraParameterEditor::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

EVerticalAlignment SNiagaraParameterEditor::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

FVector2D SNiagaraParameterEditor::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D ComputedDesiredSize = SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
	
	if (MinimumDesiredWidth.IsSet())
	{
		ComputedDesiredSize.X = FMath::Max(MinimumDesiredWidth.GetValue(), ComputedDesiredSize.X);
	}

	if (MaximumDesiredWidth.IsSet())
	{
		ComputedDesiredSize.X = FMath::Min(MaximumDesiredWidth.GetValue(), ComputedDesiredSize.X);
	}

	return ComputedDesiredSize;
}

bool SNiagaraParameterEditor::GetIsEditingExclusively()
{
	return bIsEditingExclusively;
}

void SNiagaraParameterEditor::SetIsEditingExclusively(bool bInIsEditingExclusively)
{
	bIsEditingExclusively = bInIsEditingExclusively;
}

void SNiagaraParameterEditor::ExecuteOnBeginValueChange()
{
	bContinousChangeActive = true;
	OnBeginValueChange.ExecuteIfBound();
}

void SNiagaraParameterEditor::ExecuteOnEndValueChange()
{
	bContinousChangeActive = false;
	OnEndValueChange.ExecuteIfBound();
}

void SNiagaraParameterEditor::ExecuteOnValueChanged()
{
	if (bContinousChangeActive == false)
	{
		OnBeginValueChange.ExecuteIfBound();
	}
	OnValueChanged.ExecuteIfBound();
	if (bContinousChangeActive == false)
	{
		OnEndValueChange.ExecuteIfBound();
	}
}