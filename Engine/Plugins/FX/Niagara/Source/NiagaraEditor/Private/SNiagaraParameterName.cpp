// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraParameterName.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraEditorUtilities.h"

#include "UObject/UObjectIterator.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

namespace SNiagaraParameterNamePrivate
{
	void TryResolveParameterCollectionPath(FName CollectionName, FNiagaraNamespaceMetadata& NamespaceMetadata)
	{
		for (TObjectIterator<UNiagaraParameterCollection> NPCIt; NPCIt; ++NPCIt)
		{
			UNiagaraParameterCollection* NPC = *NPCIt;
			if (::IsValid(NPC) && NPC->IsAsset() && (NPC->GetNamespace() == CollectionName))
			{
				CollectionName = FName(*NPC->GetPathName());
				break;
			}
		}

		NamespaceMetadata.SetDescription(FText::Format(NSLOCTEXT("NiagaraParameterName", "NPCAssetToolTipFormat", "References parameter collection '{0}'."), FText::FromName(CollectionName)));
	}
}

void SNiagaraParameterName::Construct(const FArguments& InArgs)
{
	EditableTextStyle = InArgs._EditableTextStyle;
	ReadOnlyTextStyle = InArgs._ReadOnlyTextStyle;
	ParameterName = InArgs._ParameterName;
	bIsReadOnly = InArgs._IsReadOnly;
	SingleNameDisplayMode = InArgs._SingleNameDisplayMode;
	HighlightText = InArgs._HighlightText;
	OnVerifyNameChangeDelegate = InArgs._OnVerifyNameChange;
	OnNameChangedDelegate = InArgs._OnNameChanged;
	OnDoubleClickedDelegate = InArgs._OnDoubleClicked;
	IsSelected = InArgs._IsSelected;
	DecoratorLocation = InArgs._DecoratorLocation;
	DecoratorHAlign = InArgs._DecoratorHAlign;
	DecoratorPadding = InArgs._DecoratorPadding;
	Decorator = InArgs._Decorator.Widget;
	bModifierIsPendingEdit = false;

	UpdateContent(ParameterName.Get());
}

void SNiagaraParameterName::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	FName CurrentParameterName = ParameterName.Get();
	if (DisplayedParameterName != CurrentParameterName)
	{
		UpdateContent(CurrentParameterName);
	}

	if (EditableModifierTextBlock.IsValid() && bModifierIsPendingEdit)
	{
		bModifierIsPendingEdit = false;
		EditableModifierTextBlock->EnterEditingMode();
	}
}

FReply SNiagaraParameterName::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (OnDoubleClickedDelegate.IsBound())
	{
		return OnDoubleClickedDelegate.Execute(InMyGeometry, InMouseEvent);
	}
	else
	{
		return SCompoundWidget::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
	}
}

TSharedRef<SBorder> SNiagaraParameterName::CreateNamespaceWidget(FText NamespaceDisplayName, FText NamespaceDescription, FLinearColor NamespaceBorderColor, FName NamespaceForegroundStyle)
{
	return SNew(SBorder)
	.BorderImage(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.ParameterName.NamespaceBorder"))
	.BorderBackgroundColor(NamespaceBorderColor)
	.ToolTipText(NamespaceDescription)
	.VAlign(VAlign_Center)
	.Padding(FMargin(5.0f, 1.0f, 5.0f, 1.0f))
	[
		SNew(STextBlock)
		.TextStyle(FNiagaraEditorStyle::Get(), NamespaceForegroundStyle)
		.Text(NamespaceDisplayName)
		.HighlightText(HighlightText)
	];
}

void SNiagaraParameterName::UpdateContent(FName InDisplayedParameterName, int32 InEditableNamespaceModifierIndex)
{
	DisplayedParameterName = InDisplayedParameterName;
	EditableNameTextBlock.Reset();
	EditableModifierTextBlock.Reset();

	FString DisplayedParameterNameString = DisplayedParameterName.ToString();
	TArray<FString> NamePartStrings;
	DisplayedParameterNameString.ParseIntoArray(NamePartStrings, TEXT("."));

	if (NamePartStrings.Num() == 0)
	{
		return;
	}

	TArray<FName> NameParts;
	for (int32 i = 0; i < NamePartStrings.Num(); i++)
	{
		NameParts.Add(*NamePartStrings[i]);
	}

	TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);

	auto AddDecorator =
		[this, &ContentBox](bool bShouldAdd)
		{
			if (bShouldAdd && Decorator.IsValid() && Decorator != SNullWidget::NullWidget)
			{
				ContentBox->AddSlot()
					.VAlign(VAlign_Center)
					.HAlign(DecoratorHAlign)
					.Padding(DecoratorPadding)
					[
						Decorator.ToSharedRef()
					];
			}
		};


	AddDecorator(DecoratorLocation == ENiagaraParameterDecoratorLocation::BeforeName);

	bool bAddNameWidget = true;
	if (NameParts.Num() > 1 || SingleNameDisplayMode == ESingleNameDisplayMode::Namespace)
	{
		FNiagaraNamespaceMetadata DefaultNamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetDefaultNamespaceMetadata();
		FNiagaraNamespaceMetadata DefaultNamespaceModifierMetadata = GetDefault<UNiagaraEditorSettings>()->GetDefaultNamespaceModifierMetadata();

		// Add the namespace widget.
		FNiagaraNamespaceMetadata NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(NameParts);
		int32 NamespaceNamePartCount;
		TSharedPtr<SWidget> NamespaceWidget;
		if (NamespaceMetadata.IsValid())
		{
			NamespaceNamePartCount = NamespaceMetadata.Namespaces.Num();
			NamespaceWidget = CreateNamespaceWidget(
				NamespaceMetadata.DisplayName.ToUpper(), NamespaceMetadata.Description,
				NamespaceMetadata.BackgroundColor, NamespaceMetadata.ForegroundStyle);
		}
		else
		{
			NamespaceNamePartCount = 1;
			FText NamespaceDisplayName = FText::FromString(FName::NameToDisplayString(NameParts[0].ToString(), false).ToUpper());
			NamespaceWidget = CreateNamespaceWidget(
				NamespaceDisplayName, DefaultNamespaceMetadata.Description,
				DefaultNamespaceMetadata.BackgroundColor, DefaultNamespaceMetadata.ForegroundStyle);
		}

		ContentBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f, 0.0f, 5.0f, 0.0f)
			[
				NamespaceWidget.ToSharedRef()
			];

		if (NamespaceNamePartCount == NameParts.Num())
		{
			bAddNameWidget = false;
		}

		// Next add namespace modifier widgets if needed.
		for(int32 NamePartIndex = NamespaceNamePartCount; NamePartIndex < NameParts.Num() - 1; NamePartIndex++)
		{
			if (NamePartIndex == InEditableNamespaceModifierIndex)
			{
				ContentBox->AddSlot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 5.0f, 0.0f)
					[
						SNew(SBox)
						.MinDesiredWidth(100)
						[
							SAssignNew(EditableModifierTextBlock, SInlineEditableTextBlock)
							.Style(EditableTextStyle)
							.Text(FText::FromName(NameParts[NamePartIndex]))
							.OnVerifyTextChanged(this, &SNiagaraParameterName::VerifyNamespaceModifierTextChange, NameParts[NamePartIndex])
							.OnTextCommitted(this, &SNiagaraParameterName::NamespaceModifierTextCommitted)
						]
					];
				bModifierIsPendingEdit = true;
			}
			else
			{
				FName NamespaceModifier = NameParts[NamePartIndex];
				FNiagaraNamespaceMetadata NamespaceModifierMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaceModifier(NamespaceModifier);
				FText NamespaceModifierDisplayName;
				if (NamespaceModifierMetadata.IsValid())
				{
					NamespaceModifierDisplayName = NamespaceModifierMetadata.DisplayName.ToUpper();
				}
				else
				{
					NamespaceModifierMetadata = DefaultNamespaceModifierMetadata;
					NamespaceModifierDisplayName = FText::FromString(FName::NameToDisplayString(NamespaceModifier.ToString(), false).ToUpper());

					if (NamePartIndex == 1 && NameParts[0] == FNiagaraConstants::ParameterCollectionNamespace)
					{
						SNiagaraParameterNamePrivate::TryResolveParameterCollectionPath(NameParts[1], NamespaceModifierMetadata);
					}
				}

				TSharedRef<SBorder> NamespaceModifierBorder = CreateNamespaceWidget(
					NamespaceModifierDisplayName, NamespaceModifierMetadata.Description,
					NamespaceModifierMetadata.BackgroundColor, NamespaceModifierMetadata.ForegroundStyle);

				ContentBox->AddSlot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0.0f, 0.0f, 5.0f, 0.0f)
					[
						NamespaceModifierBorder
					];
			}
		}
	}

	if (bAddNameWidget)
	{
		TSharedPtr<SWidget> NameWidget;
		if (bIsReadOnly)
		{
			NameWidget = SNew(STextBlock)
				.TextStyle(ReadOnlyTextStyle)
				.Text(FText::FromName(NameParts.Last()))
				.HighlightText(HighlightText);
		}
		else
		{
			NameWidget = SAssignNew(EditableNameTextBlock, SInlineEditableTextBlock)
				.Style(EditableTextStyle)
				.Text(FText::FromName(NameParts.Last()))
				.IsSelected(IsSelected)
				.OnVerifyTextChanged(this, &SNiagaraParameterName::VerifyNameTextChange, NameParts.Last())
				.OnTextCommitted(this, &SNiagaraParameterName::NameTextCommitted)
				.HighlightText(HighlightText);
		}

		ContentBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			NameWidget.ToSharedRef()
		];
	}

	AddDecorator(DecoratorLocation == ENiagaraParameterDecoratorLocation::AfterName);

	ChildSlot
	[
		ContentBox
	];
}

FName SNiagaraParameterName::ReconstructNameFromEditText(const FText& InEditText)
{
	FString CurrentParameterNameString = ParameterName.Get().ToString();
	TArray<FString> NameParts;
	CurrentParameterNameString.ParseIntoArray(NameParts, TEXT("."));

	NameParts[NameParts.Num() - 1] = InEditText.ToString().Replace(TEXT("."), TEXT("_"));
	FString NewParameterNameString = FString::Join(NameParts, TEXT("."));
	return *NewParameterNameString;
}

bool SNiagaraParameterName::VerifyNameTextChange(const FText& InNewName, FText& OutErrorMessage, FName InOriginalName)
{
	if (InNewName.IsEmpty())
	{
		OutErrorMessage = NSLOCTEXT("NiagaraParameterName", "EmptyNameErrorMessage", "Parameter name can not be empty.");
		return false;
	}

	int32 NewLength = ParameterName.Get().GetStringLength() - InOriginalName.GetStringLength() + InNewName.ToString().Len();
	if (NewLength > FNiagaraConstants::MaxParameterLength)
	{
		OutErrorMessage = FText::Format(NSLOCTEXT("NiagaraParameterName", "NameTooLongErrorFormat", "The name entered is too long.\nThe maximum parameter length is {0}."), FText::AsNumber(FNiagaraConstants::MaxParameterLength));
		return false;
	}

	FName NewParameterName = ReconstructNameFromEditText(InNewName);
	if (OnVerifyNameChangeDelegate.IsBound())
	{
		return OnVerifyNameChangeDelegate.Execute(NewParameterName, OutErrorMessage);
	}
	return true;
}

void SNiagaraParameterName::NameTextCommitted(const FText& InNewNameText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::OnUserMovedFocus)
	{
		FName NewParameterName = ReconstructNameFromEditText(InNewNameText);
		OnNameChangedDelegate.ExecuteIfBound(NewParameterName);
	}
}

bool SNiagaraParameterName::VerifyNamespaceModifierTextChange(const FText& InNewNamespaceModifier, FText& OutErrorMessage, FName InOriginalNamespaceModifier)
{
	int32 NewLength = ParameterName.Get().GetStringLength() - InOriginalNamespaceModifier.GetStringLength() + InNewNamespaceModifier.ToString().Len();
	if (NewLength > FNiagaraConstants::MaxParameterLength)
	{
		OutErrorMessage = FText::Format(NSLOCTEXT("NiagaraParameterName", "NamespaceModifierTooLongErrorFormat", "The namespace modifier entered is too long.\nThe maximum parameter length is {0}."), FText::AsNumber(FNiagaraConstants::MaxParameterLength));
		return false;
	}

	FName NewNamespaceModifierName = *InNewNamespaceModifier.ToString().Replace(TEXT("."), TEXT(""));
	FName NewParameterName = FNiagaraParameterUtilities::SetSpecificNamespaceModifier(ParameterName.Get(), NewNamespaceModifierName);
	if (NewParameterName != NAME_None && OnVerifyNameChangeDelegate.IsBound())
	{
		return OnVerifyNameChangeDelegate.Execute(NewParameterName, OutErrorMessage);
	}
	return true;
}

void SNiagaraParameterName::NamespaceModifierTextCommitted(const FText& InNewNameText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::OnUserMovedFocus)
	{
		FName NewNamespaceModifier = *InNewNameText.ToString().Replace(TEXT("."), TEXT(""));
		FName NewParameterName = FNiagaraParameterUtilities::SetSpecificNamespaceModifier(ParameterName.Get(), NewNamespaceModifier);
		if (NewParameterName != NAME_None)
		{
			OnNameChangedDelegate.ExecuteIfBound(NewParameterName);
		}
	}
	UpdateContent(ParameterName.Get());
}

void SNiagaraParameterName::EnterEditingMode()
{
	if (EditableNameTextBlock.IsValid())
	{
		EditableNameTextBlock->EnterEditingMode();
	}
}

void SNiagaraParameterName::EnterNamespaceModifierEditingMode()
{
	FNiagaraParameterHandle ParameterHandle(ParameterName.Get());
	FNiagaraNamespaceMetadata NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetMetaDataForNamespaces(ParameterHandle.GetHandleParts());
	if (NamespaceMetadata.IsValid())
	{
		int32 NamePartIndexForEditableModifier = FNiagaraParameterUtilities::GetNumberOfNamePartsBeforeEditableModifier(NamespaceMetadata);
		if (NamePartIndexForEditableModifier != INDEX_NONE)
		{
			UpdateContent(ParameterName.Get(), NamePartIndexForEditableModifier);
		}
	}
}

void SNiagaraParameterName::UpdateDecorator(TSharedRef<SWidget> InDecorator)
{
	Decorator = InDecorator;
	UpdateContent(ParameterName.Get());
}

void SNiagaraParameterNameTextBlock::Construct(const FArguments& InArgs)
{
	ParameterText = InArgs._ParameterText;
	OnVerifyNameTextChangedDelegate = InArgs._OnVerifyTextChanged;
	OnNameTextCommittedDelegate = InArgs._OnTextCommitted;
	OnDragDetectedHandlerDelegate = InArgs._OnDragDetected;

	ChildSlot
	[
		SAssignNew(ParameterName, SNiagaraParameterName)
		.EditableTextStyle(InArgs._EditableTextStyle)
		.ReadOnlyTextStyle(InArgs._ReadOnlyTextStyle)
		.ParameterName(this, &SNiagaraParameterNameTextBlock::GetParameterName)
		.IsReadOnly(InArgs._IsReadOnly)
		.HighlightText(InArgs._HighlightText)
		.IsSelected(InArgs._IsSelected)
		.OnVerifyNameChange(this, &SNiagaraParameterNameTextBlock::VerifyNameChange)
		.OnNameChanged(this, &SNiagaraParameterNameTextBlock::NameChanged)
		.DecoratorLocation(InArgs._DecoratorLocation)
		.DecoratorHAlign(InArgs._DecoratorHAlign)
		.DecoratorPadding(InArgs._DecoratorPadding)
		.Decorator()
		[
			InArgs._Decorator.Widget
		]
	];
}

FName SNiagaraParameterNameTextBlock::GetParameterName() const
{
	FText CurrentPinText = ParameterText.Get();
	if (CurrentPinText.IdenticalTo(DisplayedParameterTextCache) == false)
	{
		DisplayedParameterTextCache = CurrentPinText;
		ParameterNameCache = *DisplayedParameterTextCache.ToString();
	}
	return ParameterNameCache;
}

bool SNiagaraParameterNameTextBlock::VerifyNameChange(FName InNewName, FText& OutErrorMessage)
{
	if (OnVerifyNameTextChangedDelegate.IsBound())
	{
		return OnVerifyNameTextChangedDelegate.Execute(FText::FromName(InNewName), OutErrorMessage);
	}
	else
	{
		return true;
	}
}

void SNiagaraParameterNameTextBlock::NameChanged(FName InNewName)
{
	OnNameTextCommittedDelegate.ExecuteIfBound(FText::FromName(InNewName), ETextCommit::OnEnter);
}

FReply SNiagaraParameterNameTextBlock::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// for OnDragDetected to fire, the widget has to handle mouse button down
	if(OnDragDetectedHandlerDelegate.IsBound())
	{
		return FReply::Handled().DetectDrag(SharedThis(this), MouseEvent.GetEffectingButton());
	}

	return FReply::Unhandled();
}

FReply SNiagaraParameterNameTextBlock::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(OnDragDetectedHandlerDelegate.IsBound())
	{
		return OnDragDetectedHandlerDelegate.Execute(MyGeometry, MouseEvent);
	}

	return FReply::Unhandled();
}

void SNiagaraParameterNameTextBlock::EnterEditingMode()
{
	ParameterName->EnterEditingMode();
}

void SNiagaraParameterNameTextBlock::EnterNamespaceModifierEditingMode()
{
	ParameterName->EnterNamespaceModifierEditingMode();
}

void SNiagaraParameterNamePinLabel::Construct(const FArguments& InArgs, UEdGraphPin* InTargetPin)
{
	TargetPin = InTargetPin;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
		.ForegroundColor(this, &SNiagaraParameterNamePinLabel::GetForegroundColor)
		[
			SAssignNew(ParameterNameTextBlock, SNiagaraParameterNameTextBlock)
			.EditableTextStyle(InArgs._EditableTextStyle)
			.ParameterText(InArgs._ParameterText)
			.IsReadOnly(InArgs._IsReadOnly)
			.HighlightText(InArgs._HighlightText)
			.OnVerifyTextChanged(InArgs._OnVerifyTextChanged)
			.OnTextCommitted(InArgs._OnTextCommitted)
			.IsSelected(InArgs._IsSelected)
			.DecoratorLocation(InArgs._DecoratorLocation)
			.DecoratorHAlign(InArgs._DecoratorHAlign)
			.DecoratorPadding(InArgs._DecoratorPadding)
			.Decorator()
			[
				InArgs._Decorator.Widget
			]
		]
	];		
}

void SNiagaraParameterNamePinLabel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UNiagaraNodeParameterMapBase* ParameterMapNode = Cast<UNiagaraNodeParameterMapBase>(TargetPin->GetOwningNode());
	if (ParameterMapNode != nullptr)
	{
		if (ParameterMapNode->GetIsPinEditNamespaceModifierPending(TargetPin))
		{
			ParameterMapNode->SetIsPinEditNamespaceModifierPending(TargetPin, false);
			ParameterNameTextBlock->EnterNamespaceModifierEditingMode();
		}
	}
}

FSlateColor SNiagaraParameterNamePinLabel::GetForegroundColor() const
{
	return TargetPin->bOrphanedPin ? FLinearColor::Red : FLinearColor::White;
}

void SNiagaraParameterNamePinLabel::EnterEditingMode()
{
	ParameterNameTextBlock->EnterEditingMode();
}
