// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraConvertInPlace_InitializeParticle.h"
#include "NiagaraClipboard.h"
#include "NiagaraScript.h"
#include "NiagaraNodeFunctionCall.h"
#include "ViewModels/Stack/NiagaraStackScriptHierarchyRoot.h"

#include "Logging/StructuredLog.h"

bool UNiagaraConvertInPlace_InitializeParticle::ConvertInputs(UNiagaraScript* InOldScript, UNiagaraClipboardContent* InOldClipboardContent,	UNiagaraScript* InNewScript, UNiagaraClipboardContent* InNewClipboardContent, FText& OutMessage)
{
	UE_LOGFMT(LogNiagaraEditor, Log, "Converting {OldScript} to {NewScript}", InOldScript->GetPathName(), InNewScript->GetPathName());

	if (InOldScript->GetPathName() == TEXT("/Niagara/Modules/Spawn/Initialization/InitializeParticle.InitializeParticle") && InNewScript->GetPathName() == TEXT("/Niagara/Modules/Spawn/Initialization/V2/InitializeParticle.InitializeParticle"))
	{
		bool bCorrectInputCount = InOldClipboardContent->Functions.Num() == 1 && InNewClipboardContent->Functions.Num() == 1 && InOldClipboardContent->Functions[0] && InNewClipboardContent->Functions[0];
		if (!bCorrectInputCount)
		{
			return false;
		}

		UUpgradeNiagaraScriptResults* UpgradeContext = NewObject<UUpgradeNiagaraScriptResults>();
		UpgradeContext->OldInputs = GetFunctionCallInputs(InOldClipboardContent->Functions[0]->Inputs);
		UpgradeContext->NewInputs = GetFunctionCallInputs(InNewClipboardContent->Functions[0]->Inputs);
		UpgradeContext->Init();

		if (UNiagaraPythonScriptModuleInput* WriteLifetime = UpgradeContext->GetOldInput("Write Lifetime"))
		{
			if (WriteLifetime->IsLocalValue())
			{
				if (WriteLifetime->AsBool())
				{
					UpgradeContext->SetEnumInput("Lifetime Mode", "Direct Set");
					UpgradeContext->SetNewInput("Lifetime", UpgradeContext->GetOldInput("Lifetime"));
				}
				else
				{
					UpgradeContext->SetFloatInput("Lifetime", 1);
				}
			}
		}

		if (UNiagaraPythonScriptModuleInput* WritePosition = UpgradeContext->GetOldInput("Write Position"))
		{
			if (WritePosition->IsLocalValue())
			{
				if (WritePosition->AsBool())
				{
					UNiagaraPythonScriptModuleInput* OldPos = UpgradeContext->GetOldInput("Position");
					if (OldPos->IsSet()) {
						UpgradeContext->SetEnumInput("Position Mode", "Direct Set");
						UpgradeContext->SetNewInput("Position", OldPos);
					}
				}
				else
				{
					UpgradeContext->SetEnumInput("Position Mode", "Simulation Position");
				}
			}
		}

		if (UNiagaraPythonScriptModuleInput* WriteMass = UpgradeContext->GetOldInput("Write Mass"))
		{
			if (WriteMass->IsLocalValue())
			{
				if (WriteMass->AsBool())
				{
					UpgradeContext->SetEnumInput("Mass Mode", "Direct Set");
					UpgradeContext->SetNewInput("Mass", UpgradeContext->GetOldInput("Mass"));
				}
				else
				{
					UpgradeContext->SetEnumInput("Mass Mode", "Unset / (Mass of 1)");
				}
			}
		}

		if (UNiagaraPythonScriptModuleInput* WriteColor = UpgradeContext->GetOldInput("Write Color"))
		{
			if (WriteColor->IsLocalValue())
			{
				if (WriteColor->AsBool())
				{
					UpgradeContext->SetEnumInput("Color Mode", "Direct Set");
					UpgradeContext->SetNewInput("Color", UpgradeContext->GetOldInput("Color"));
				}
				else
				{
					UpgradeContext->SetColorInput("Color", FLinearColor::White);
				}
			}
		}

		if (UNiagaraPythonScriptModuleInput* WriteSpriteSize = UpgradeContext->GetOldInput("Write SpriteSize"))
		{
			if (WriteSpriteSize->IsLocalValue())
			{
				if (WriteSpriteSize->AsBool())
				{
					UpgradeContext->SetEnumInput("Sprite Size Mode", "Non-Uniform");
					UpgradeContext->SetNewInput("Sprite Size", UpgradeContext->GetOldInput("Sprite Size"));
				}
				else
				{
					UpgradeContext->SetEnumInput("Sprite Size Mode", "Non-Uniform");
					UpgradeContext->SetVec2Input("Sprite Size", FVector2D(10.0f, 10.0f));
				}
			}
		}

		if (UNiagaraPythonScriptModuleInput* WriteSpriteRotation = UpgradeContext->GetOldInput("Write SpriteRotation"))
		{
			if (WriteSpriteRotation->IsLocalValue())
			{
				if (WriteSpriteRotation->AsBool())
				{
					UpgradeContext->SetEnumInput("Sprite Rotation Mode", "Direct Angle (Degrees)");
					UpgradeContext->SetNewInput("Sprite Rotation Angle", UpgradeContext->GetOldInput("Sprite Rotation"));
				}
				else
				{
					UpgradeContext->SetEnumInput("Sprite Rotation Mode", "Unset");
				}
			}
		}

		if (UNiagaraPythonScriptModuleInput* WriteMeshScale = UpgradeContext->GetOldInput("Write Scale"))
		{
			if (WriteMeshScale->IsLocalValue())
			{
				if (WriteMeshScale->AsBool())
				{
					UpgradeContext->SetEnumInput("Mesh Scale Mode", "Non-Uniform");
					UpgradeContext->SetNewInput("Mesh Scale", UpgradeContext->GetOldInput("Mesh Scale"));
				}
				else
				{
					UpgradeContext->SetEnumInput("Mesh Scale Mode", "Unset");
				}
			}
		}

		UNiagaraClipboardFunction* NewClipboardFunction = const_cast<UNiagaraClipboardFunction*>(InNewClipboardContent->Functions[0].Get());
		NewClipboardFunction->Inputs.Empty();
		for (UNiagaraPythonScriptModuleInput* NewInput : UpgradeContext->NewInputs)
		{
			NewClipboardFunction->Inputs.Add(NewInput->Input);
		}
		
		return true;
	}
	return false;
}

TArray<UNiagaraPythonScriptModuleInput*> UNiagaraConvertInPlace_InitializeParticle::GetFunctionCallInputs(const TArray<TObjectPtr<const UNiagaraClipboardFunctionInput>>& ClipboardContent)
{
	TArray<UNiagaraPythonScriptModuleInput*> ScriptInputs;

	TArray<const UNiagaraClipboardFunctionInput*> FunctionInputsWorkingSet = ClipboardContent;
	for(auto It = FunctionInputsWorkingSet.CreateIterator(); It; ++It)
	{
		UNiagaraPythonScriptModuleInput* ScriptInput = NewObject<UNiagaraPythonScriptModuleInput>();
		ScriptInput->Input = *It;
		ScriptInputs.Add(ScriptInput);

		FunctionInputsWorkingSet.Append((*It)->ChildrenInputs);
		It.RemoveCurrent();
	}

	return ScriptInputs;
}
