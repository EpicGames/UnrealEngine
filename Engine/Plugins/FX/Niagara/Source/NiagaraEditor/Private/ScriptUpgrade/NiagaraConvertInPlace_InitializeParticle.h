// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraConvertInPlaceUtilityBase.h"
#include "NiagaraConvertInPlace_InitializeParticle.generated.h"

class UNiagaraPythonScriptModuleInput;
class UNiagaraClipboardFunctionInput;

UCLASS(MinimalAPI)
class UNiagaraConvertInPlace_InitializeParticle : public UNiagaraConvertInPlaceUtilityBase
{
	GENERATED_BODY()
public:
	virtual bool UsesLegacyConversion() override { return false; }
	virtual bool ConvertInputs(UNiagaraScript* InOldScript, UNiagaraClipboardContent* InOldClipboardContent, UNiagaraScript* InNewScript, UNiagaraClipboardContent* InNewClipboardContent, FText& OutMessage) override;

private:
	TArray<UNiagaraPythonScriptModuleInput*> GetFunctionCallInputs(const TArray<TObjectPtr<const UNiagaraClipboardFunctionInput>>& ClipboardContent);
};
