// Copyright Epic Games, Inc. All Rights Reserved.


#include "NiagaraModuleVersionUpgrades.h"

#include "Logging/StructuredLog.h"
#include "NiagaraClipboard.h"
#include "UpgradeNiagaraScriptResults.h"

namespace NiagaraVersionUpgradeHelper
{
	UNiagaraPythonScriptModuleInput* GetOldInput(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, const FName& InputName)
	{
		UNiagaraPythonScriptModuleInput* ScriptInput = NewObject<UNiagaraPythonScriptModuleInput>();
		
		for (UNiagaraClipboardFunctionInput* Input : OldInputs.FunctionInputs)
		{
			if (Input->InputName == InputName)
			{
				ScriptInput->Input = Input;
				break;
			}
		}
		return ScriptInput;
	}

	UNiagaraClipboardFunctionInput* GetNewInput(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName)
	{
		for (UNiagaraClipboardFunctionInput* Input : NewInputs.FunctionInputs)
		{
			if (Input->InputName == InputName)
			{
				return Input;
			}
		}
		return nullptr;
	}

	template<typename T>
	const UNiagaraClipboardFunctionInput* CreateLocalValue(UNiagaraClipboardFunctionInput* Input, T Data)
	{
		TArray<uint8> LocalData;
		LocalData.SetNumZeroed(sizeof(T));
		FMemory::Memcpy(LocalData.GetData(), &Data, sizeof(T));
		return UNiagaraClipboardFunctionInput::CreateLocalValue(GetTransientPackage(), Input->InputName, Input->InputType, Input->bEditConditionValue, LocalData);
	}

	void SetFunctionInputValues(UNiagaraClipboardFunctionInput* InputToSet, const UNiagaraClipboardFunctionInput* NewValues)
	{
		InputToSet->ValueMode = NewValues->ValueMode;
		InputToSet->Local = NewValues->Local;
		InputToSet->Linked = NewValues->Linked;
		InputToSet->Data = NewValues->Data;
		InputToSet->Expression = NewValues->Expression;
		InputToSet->Dynamic = NewValues->Dynamic;
	}

	void SetFloatInput(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, float Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType == FNiagaraTypeDefinition::GetFloatDef())
			{
				SetFunctionInputValues(Input, CreateLocalValue(Input, Value));
			}
		}
	}

	void SetIntInput(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, int32 Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType == FNiagaraTypeDefinition::GetIntDef())
			{
				SetFunctionInputValues(Input, CreateLocalValue(Input, Value));
			}
		}
	}

	void SetBoolInput(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, bool Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType == FNiagaraTypeDefinition::GetBoolDef())
			{
				const int32 BoolAsIntValue = Value ? 1 : 0;
				TArray<uint8> IntValue;
				IntValue.AddUninitialized(Input->InputType.GetSize());
				FMemory::Memcpy(IntValue.GetData(), &BoolAsIntValue, Input->InputType.GetSize());
				SetFunctionInputValues(Input, UNiagaraClipboardFunctionInput::CreateLocalValue(GetTransientPackage(), Input->InputName, Input->InputType, Input->bEditConditionValue, IntValue));
			}
		}
	}

	void SetVec2Input(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, FVector2D Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType == FNiagaraTypeDefinition::GetVec2Def())
			{
				SetFunctionInputValues(Input, CreateLocalValue(Input, Value));
			}
		}
	}

	void SetVec3Input(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, FVector Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType == FNiagaraTypeDefinition::GetVec3Def())
			{
				SetFunctionInputValues(Input, CreateLocalValue(Input, Value));
			}
		}
	}

	void SetVec4Input(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, FVector4 Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType == FNiagaraTypeDefinition::GetVec4Def())
			{
				SetFunctionInputValues(Input, CreateLocalValue(Input, Value));
			}
		}
	}

	void SetColorInput(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, FLinearColor Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType == FNiagaraTypeDefinition::GetColorDef())
			{
				SetFunctionInputValues(Input, CreateLocalValue(Input, Value));
			}
		}
	}

	void SetQuatInput(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, FQuat Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType == FNiagaraTypeDefinition::GetQuatDef())
			{
				SetFunctionInputValues(Input, CreateLocalValue(Input, Value));
			}
		}
	}

	void SetEnumInput(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, FString Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType.IsEnum())
			{
				int32 EnumValue = Input->InputType.GetEnum()->GetValueByNameString(Value, EGetByNameFlags::ErrorIfNotFound | EGetByNameFlags::CheckAuthoredName);
				SetFunctionInputValues(Input, CreateLocalValue(Input, EnumValue));
			}
		}
	}

	void SetEnumInputFromInt(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, int32 Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (Input->InputType.IsEnum())
			{
				if (!Input->InputType.GetEnum()->IsValidEnumValue(Value))
				{
					UE_LOGFMT(LogNiagaraEditor, Error, "Value {Value} is not a valid enum value for input {InputName}", Value, InputName);
				}
				SetFunctionInputValues(Input, CreateLocalValue(Input, Value));
			}
		}
	}
	
	void SetLinkedInput(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, FString Value)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			FNiagaraVariableBase LinkedParameter = FNiagaraVariableBase(Input->InputType, *Value);
			SetFunctionInputValues(Input, UNiagaraClipboardFunctionInput::CreateLinkedValue(GetTransientPackage(), Input->InputName, Input->InputType, Input->bEditConditionValue, LinkedParameter));
		}
	}
	
	void SetNewInput(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName, UNiagaraClipboardFunctionInput* OldInputValue)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			if (OldInputValue && OldInputValue->InputName == InputName)
			{
				if (OldInputValue->InputType.IsValid() && OldInputValue->InputType == Input->InputType)
				{
					SetFunctionInputValues(Input, OldInputValue);
				} else
				{
					SetFunctionInputValues(Input, UNiagaraClipboardFunctionInput::CreateDefaultInputValue(GetTransientPackage(), Input->InputName, Input->InputType));
				}
			}
		}
	}

	void ResetToDefault(NiagaraVersionUpgrade::ScriptInputs& NewInputs, const FName& InputName)
	{
		if (UNiagaraClipboardFunctionInput* Input = GetNewInput(NewInputs, InputName))
		{
			SetFunctionInputValues(Input, UNiagaraClipboardFunctionInput::CreateDefaultInputValue(GetTransientPackage(), Input->InputName, Input->InputType));
		}
	}
}

bool UNiagaraVersionUpgrade_SampleParticleFromEmitter::ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage)
{
	// upgrade from 1.0 -> 2.0
	if (OldInputs.Version.MajorVersion == 1 && NewInputs.Version.MajorVersion == 2)
	{
		NiagaraVersionUpgradeHelper::SetBoolInput(NewInputs, FName("Transform To Emitter Space"), false);
	}

	// upgrade from 2.0 -> 3.0
	if (OldInputs.Version.MajorVersion == 2 && NewInputs.Version.MajorVersion == 3)
	{
		if (UNiagaraPythonScriptModuleInput* SpaceOption = NiagaraVersionUpgradeHelper::GetOldInput(OldInputs, FName("Transform To Emitter Space")))
		{
			if (SpaceOption->IsLocalValue())
			{
				NiagaraVersionUpgradeHelper::SetIntInput(NewInputs, FName("Transform Data Option"), 1);
			}
			else
			{
				NiagaraVersionUpgradeHelper::SetIntInput(NewInputs, FName("Transform Data Option"), SpaceOption->AsBool() ? 0 : 1);
			}
		}
	}
	return true;
}

bool UNiagaraVersionUpgrade_DynamicMaterialParameters::ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage)
{
	// upgrade from 1.2 -> 2.0
	if (OldInputs.Version.MajorVersion == 1 && OldInputs.Version.MinorVersion == 2 && NewInputs.Version.MajorVersion == 2 && NewInputs.Version.MinorVersion == 0)
	{
		if (UNiagaraPythonScriptModuleInput* WriteParam = NiagaraVersionUpgradeHelper::GetOldInput(OldInputs, FName("Write Parameter Index 0")))
		{
			if (WriteParam->IsLocalValue())
			{
				NiagaraVersionUpgradeHelper::SetEnumInput(NewInputs, WriteParam->Input->InputName, FString(WriteParam->AsBool() ? "Float" : "Off"));
			}
		}
		if (UNiagaraPythonScriptModuleInput* WriteParam = NiagaraVersionUpgradeHelper::GetOldInput(OldInputs, FName("Write Parameter Index 1")))
		{
			if (WriteParam->IsLocalValue())
			{
				NiagaraVersionUpgradeHelper::SetEnumInput(NewInputs, WriteParam->Input->InputName, FString(WriteParam->AsBool() ? "Float" : "Off"));
			}
		}
		if (UNiagaraPythonScriptModuleInput* WriteParam = NiagaraVersionUpgradeHelper::GetOldInput(OldInputs, FName("Write Parameter Index 2")))
		{
			if (WriteParam->IsLocalValue())
			{
				NiagaraVersionUpgradeHelper::SetEnumInput(NewInputs, WriteParam->Input->InputName, FString(WriteParam->AsBool() ? "Float" : "Off"));
			}
		}
		if (UNiagaraPythonScriptModuleInput* WriteParam = NiagaraVersionUpgradeHelper::GetOldInput(OldInputs, FName("Write Parameter Index 3")))
		{
			if (WriteParam->IsLocalValue())
			{
				NiagaraVersionUpgradeHelper::SetEnumInput(NewInputs, WriteParam->Input->InputName, FString(WriteParam->AsBool() ? "Float" : "Off"));
			}
		}
	}
	return true;
}

bool UNiagaraVersionUpgrade_CurlNoise::ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage)
{
	// upgrade from 2.0 -> 3.0
	if (OldInputs.Version.MajorVersion == 2 && NewInputs.Version.MajorVersion == 3)
	{
		if (UNiagaraPythonScriptModuleInput* Remap = NiagaraVersionUpgradeHelper::GetOldInput(OldInputs, FName("Remap Range")))
		{
			if (Remap->AsBool())
			{
				NiagaraVersionUpgradeHelper::SetEnumInput(NewInputs, FName("Remap Output Value"), FString("Uniform"));
			}
		}
	}
	return true;
}

bool UNiagaraVersionUpgrade_PartitionParticles::ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage)
{
	// upgrade from 1.1 -> 2.0
	if (OldInputs.Version.MajorVersion == 1 && OldInputs.Version.MinorVersion == 1 && NewInputs.Version.MajorVersion == 2 && NewInputs.Version.MinorVersion == 0)
	{
		if (UNiagaraPythonScriptModuleInput* Mode = NiagaraVersionUpgradeHelper::GetOldInput(OldInputs, FName("Partition Mode")))
		{
			if (Mode->IsLocalValue())
			{
				NiagaraVersionUpgradeHelper::SetEnumInputFromInt(NewInputs, Mode->Input->InputName, Mode->AsInt());
			}
		}
	}
	return true;
}

bool UNiagaraVersionUpgrade_Grid3DTurbulence::ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage)
{
	// upgrade from 6.0 -> 7.0
	if (OldInputs.Version.MajorVersion == 6 && NewInputs.Version.MajorVersion == 7)
	{
		if (UNiagaraPythonScriptModuleInput* Freq = NiagaraVersionUpgradeHelper::GetOldInput(OldInputs, FName("Turbulence Frequency")))
		{
			float FreqValue = Freq->AsFloat();
			if (FreqValue != 0)
			{
				NiagaraVersionUpgradeHelper::SetVec3Input(NewInputs, Freq->Input->InputName, FVector(FreqValue));
			}
		}
		if (UNiagaraPythonScriptModuleInput* Gain = NiagaraVersionUpgradeHelper::GetOldInput(OldInputs, FName("Turbulence Gain")))
		{
			float GainValue = Gain->AsFloat();
			if (GainValue != 0)
			{
				NiagaraVersionUpgradeHelper::SetVec3Input(NewInputs, Gain->Input->InputName, FVector(GainValue));
			}
		}
	}
	return true;
}
