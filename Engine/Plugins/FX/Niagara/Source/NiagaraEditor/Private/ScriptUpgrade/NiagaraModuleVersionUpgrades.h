// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraVersionUpgradeBase.h"

#include "NiagaraModuleVersionUpgrades.generated.h"

UCLASS()
class UNiagaraVersionUpgrade_SampleParticleFromEmitter : public UNiagaraVersionUpgradeBase
{
	GENERATED_BODY()

public:
	virtual bool ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage) override;
};

UCLASS()
class UNiagaraVersionUpgrade_UpdateParticleFromEmitter : public UNiagaraVersionUpgrade_SampleParticleFromEmitter
{
	GENERATED_BODY()

	// the conversion script is currently the same as UNiagaraVersionUpgrade_SampleParticleFromEmitter, so it just reuses that implementation.
};

UCLASS()
class UNiagaraVersionUpgrade_DynamicMaterialParameters : public UNiagaraVersionUpgradeBase
{
	GENERATED_BODY()

public:
	virtual bool ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage) override;
};

UCLASS()
class UNiagaraVersionUpgrade_CurlNoise : public UNiagaraVersionUpgradeBase
{
	GENERATED_BODY()

public:
	virtual bool ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage) override;
};

UCLASS()
class UNiagaraVersionUpgrade_PartitionParticles : public UNiagaraVersionUpgradeBase
{
	GENERATED_BODY()

public:
	virtual bool ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage) override;
};

UCLASS()
class UNiagaraVersionUpgrade_Grid3DTurbulence : public UNiagaraVersionUpgradeBase
{
	GENERATED_BODY()

public:
	virtual bool ConvertScript(const NiagaraVersionUpgrade::ScriptInputs& OldInputs, NiagaraVersionUpgrade::ScriptInputs& NewInputs, FText& OutMessage) override;
};