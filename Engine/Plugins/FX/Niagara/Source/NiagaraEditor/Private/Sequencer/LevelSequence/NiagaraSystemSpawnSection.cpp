// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/LevelSequence/NiagaraSystemSpawnSection.h"
#include "MovieScene/MovieSceneNiagaraSystemSpawnSection.h"
#include "SequencerSectionPainter.h"

FNiagaraSystemSpawnSection::FNiagaraSystemSpawnSection(UMovieSceneSection& InSection)
	: Section(Cast<UMovieSceneNiagaraSystemSpawnSection>(&InSection))
{
}

UMovieSceneSection* FNiagaraSystemSpawnSection::FNiagaraSystemSpawnSection::GetSectionObject()
{
	return Section.Get();
}

int32 FNiagaraSystemSpawnSection::OnPaintSection(FSequencerSectionPainter& InPainter) const
{
	return InPainter.PaintSectionBackground(FLinearColor::Red);
}

FText FNiagaraSystemSpawnSection::GetSectionTitle() const
{
	return NSLOCTEXT("NiagaraSystemSpawnSection", "LifeCycleLabel", "Life Cycle");
}