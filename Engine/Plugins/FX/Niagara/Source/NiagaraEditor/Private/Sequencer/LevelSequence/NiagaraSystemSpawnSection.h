// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"

class UMovieSceneSection;
class UMovieSceneNiagaraSystemSpawnSection;

class FNiagaraSystemSpawnSection : public ISequencerSection
{
public:
	FNiagaraSystemSpawnSection(UMovieSceneSection& InSection);

	//~ ISequencerSection interface
	virtual UMovieSceneSection* GetSectionObject() override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override;
	virtual FText GetSectionTitle() const override;

private:
	TWeakObjectPtr<UMovieSceneNiagaraSystemSpawnSection> Section;
};