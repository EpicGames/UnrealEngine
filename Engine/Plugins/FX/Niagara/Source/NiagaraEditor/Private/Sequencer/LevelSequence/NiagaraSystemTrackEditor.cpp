// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemTrackEditor.h"

#include "NiagaraActor.h"
#include "MovieScene/MovieSceneNiagaraSystemTrack.h"
#include "MovieScene/MovieSceneNiagaraSystemSpawnSection.h"
#include "MovieScene/Parameters/MovieSceneNiagaraParameterTrack.h"
#include "NiagaraSystemSpawnSection.h"
#include "NiagaraComponent.h"
#include "NiagaraSystem.h"
#include "NiagaraEditorUtilities.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "NiagaraEditorModule.h"
#include "NiagaraSystemEditorData.h"
#include "SEnumCombo.h"

#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

#define LOCTEXT_NAMESPACE "NiagaraSystemTrackEditor"

TSharedRef<ISequencerTrackEditor> FNiagaraSystemTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FNiagaraSystemTrackEditor(InSequencer));
}

FNiagaraSystemTrackEditor::FNiagaraSystemTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{
}

FText FNiagaraSystemTrackEditor::GetDisplayName() const
{
	return LOCTEXT("NiagaraSystemTrackEditor_DisplayName", "Niagara System");
}

TSharedRef<ISequencerSection> FNiagaraSystemTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	checkf(SectionObject.GetClass()->IsChildOf<UMovieSceneNiagaraSystemSpawnSection>(), TEXT("Unsupported section."));
	return MakeShareable(new FNiagaraSystemSpawnSection(SectionObject));
}

bool FNiagaraSystemTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneNiagaraSystemTrack::StaticClass();
}

bool HasLifeCycleTrack(UMovieScene& MovieScene, FGuid ObjectBinding)
{
	if (const FMovieSceneBinding* Binding = MovieScene.FindBinding(ObjectBinding))
	{
		for (const UMovieSceneTrack* Track : Binding->GetTracks())
		{
			if (Track->IsA<UMovieSceneNiagaraSystemTrack>())
			{
				return true;
			}
		}
	}
	return false;
}

void GetAnimatedParameterNames(UMovieScene& MovieScene, FGuid ObjectBinding, TSet<FName>& AnimatedParameterNames)
{
	if (const FMovieSceneBinding* Binding = MovieScene.FindBinding(ObjectBinding))
	{
		for (const UMovieSceneTrack* Track : Binding->GetTracks())
		{
			const UMovieSceneNiagaraParameterTrack* ParameterTrack = Cast<UMovieSceneNiagaraParameterTrack>(Track);
			if (ParameterTrack != nullptr)
			{
				AnimatedParameterNames.Add(ParameterTrack->GetParameter().GetName());
			}
		}
	}
}

void FNiagaraSystemTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(UNiagaraComponent::StaticClass()) && HasLifeCycleTrack(*GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), ObjectBindings[0]) == false)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AddNiagaraSystemTrack", "Niagara System Life Cycle Track"),
			LOCTEXT("AddNiagaraSystemTrackToolTip", "Add a track for controlling niagara system life cycle behavior."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FNiagaraSystemTrackEditor::AddNiagaraSystemTrack, ObjectBindings))
		);
	}

	TArrayView<TWeakObjectPtr<UObject>> BoundObjects = GetSequencer()->FindBoundObjects(ObjectBindings[0], GetSequencer()->GetFocusedTemplateID());

	UNiagaraSystem* System = nullptr;
	for (TWeakObjectPtr<UObject> BoundObject : BoundObjects)
	{
		UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(BoundObject.Get());
		if (NiagaraComponent != nullptr)
		{
			System = NiagaraComponent->GetAsset();
		}
	}

	if (System != nullptr)
	{
		TArray<FNiagaraVariable> ParameterVariables;
		System->GetExposedParameters().GetUserParameters(ParameterVariables);

		TSet<FName> AnimatedParameterNames;
		GetAnimatedParameterNames(*GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), ObjectBindings[0], AnimatedParameterNames);

		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");

		UNiagaraComponent* NiagaraComponent = BoundObjects.Num() == 1 ? Cast<UNiagaraComponent>(BoundObjects[0].Get()) : nullptr;
		for (const FNiagaraVariable& ParameterVariable : ParameterVariables)
		{
			if (ParameterVariable.GetType().IsDataInterface() == false &&
				NiagaraEditorModule.CanCreateParameterTrackForType(*ParameterVariable.GetType().GetScriptStruct()) && 
				AnimatedParameterNames.Contains(ParameterVariable.GetName()) == false)
			{
				TArray<uint8> DefaultValueData;
				DefaultValueData.AddZeroed(ParameterVariable.GetSizeInBytes());
				if (NiagaraComponent != nullptr)
				{
					const uint8* OverrideParameterData = NiagaraComponent->GetOverrideParameters().GetParameterData(ParameterVariable);
					if (OverrideParameterData != nullptr)
					{
						FMemory::Memcpy(DefaultValueData.GetData(), OverrideParameterData, ParameterVariable.GetSizeInBytes());
					}
				}

				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("AddNiagaraParameterTrackFormat", "{0} Parameter Track"), FText::FromName(ParameterVariable.GetName())),
					FText::Format(LOCTEXT("AddNiagaraSystemTrackToolTipFormat", "Add a track for animating the {0} parameter."), FText::FromName(ParameterVariable.GetName())),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &FNiagaraSystemTrackEditor::AddNiagaraParameterTrack, ObjectBindings, ParameterVariable, DefaultValueData))
				);
			}
		}
	}
}

TSharedPtr<SWidget> FNiagaraSystemTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	using namespace UE::Sequencer;

	UMovieSceneTrack* Track = nullptr;
	if (TViewModelPtr<ITrackExtension> TrackModel = Params.ViewModel->FindAncestorOfType<ITrackExtension>(true))
	{
		Track = TrackModel->GetTrack();
	}

	UMovieSceneNiagaraSystemTrack* NiagaraSystemTrack = Cast<UMovieSceneNiagaraSystemTrack>(Track);
	if (!NiagaraSystemTrack)
	{
		return nullptr;
	}

	if (ColumnName == FCommonOutlinerNames::Edit)
	{
		return SNew(SBox)
			.HAlign(HAlign_Left)
			[
				SNew(SEnumComboBox, StaticEnum<ENiagaraAgeUpdateMode>())
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.CurrentValue_Lambda([NiagaraSystemTrack]() -> int32
				{
					const TArray<UMovieSceneSection*>& Sections = NiagaraSystemTrack->GetAllSections();
					UMovieSceneNiagaraSystemSpawnSection* Section = Sections.Num() == 1 ? Cast<UMovieSceneNiagaraSystemSpawnSection>(Sections[0]) : nullptr;
					return static_cast<int32>(Section ? Section->GetAgeUpdateMode() : ENiagaraAgeUpdateMode::TickDeltaTime);
				})
				.OnEnumSelectionChanged_Lambda([NiagaraSystemTrack](int32 NewSelection, ESelectInfo::Type)
				{
					if (NewSelection == INDEX_NONE)
					{
						return;
					}

					TArray<UMovieSceneSection*> Sections = NiagaraSystemTrack->GetAllSections();
					UMovieSceneNiagaraSystemSpawnSection* Section = Sections.Num() == 1 ? Cast<UMovieSceneNiagaraSystemSpawnSection>(Sections[0]) : nullptr;
					if (Section)
					{
						FScopedTransaction Transaction(LOCTEXT("ChangeValue", "Change Value"));
						Section->Modify();
						Section->SetAgeUpdateMode(static_cast<ENiagaraAgeUpdateMode>(NewSelection));
					}
				})
				.IsEnabled_Lambda([NiagaraSystemTrack]() -> bool
				{
					TArray<UMovieSceneSection*> Sections = NiagaraSystemTrack->GetAllSections();
					return Sections.Num() == 1 && Cast<UMovieSceneNiagaraSystemSpawnSection>(Sections[0]);
				})
			];
	}
	return FMovieSceneTrackEditor::BuildOutlinerColumnWidget(Params, ColumnName);
}

void FNiagaraSystemTrackEditor::AddDefaultSystemTracks(const AActor& SourceActor, const FGuid& Binding,	TSharedPtr<ISequencer> Sequencer)
{
	ENiagaraAddDefaultsTrackMode TrackMode = GetDefault<UNiagaraEditorSettings>()->DefaultsSequencerSubtracks;
	if (!SourceActor.IsA<ANiagaraActor>() || TrackMode == ENiagaraAddDefaultsTrackMode::NoSubtracks)
	{
		return;
	}

	const ANiagaraActor* NiagaraActor = Cast<ANiagaraActor>(&SourceActor);
	UNiagaraComponent* NiagaraComponent = NiagaraActor->GetNiagaraComponent();
	if (NiagaraComponent && NiagaraComponent->GetAsset())
	{
		UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		FGuid ComponentBinding = Sequencer->GetHandleToObject(NiagaraComponent);
		
		if (MovieScene->FindSpawnable(Binding))
		{
			// we only want to add tracks for possessables
			return;
		}

		if (TrackMode == ENiagaraAddDefaultsTrackMode::ComponentTrackOnly)
		{
			FMovieScenePossessable* BoundComponent = MovieScene->FindPossessable(ComponentBinding);
			if (!BoundComponent)
			{
				Sequence->CreatePossessable(NiagaraComponent);
			}
		}
		else if (TrackMode == ENiagaraAddDefaultsTrackMode::LifecycleTrack)
		{
			UClass* TrackClass = UMovieSceneNiagaraSystemTrack::StaticClass();
			UMovieSceneTrack* NewTrack = MovieScene->FindTrack(TrackClass, ComponentBinding);
			if (!NewTrack)
			{
				UNiagaraSystem* System = NiagaraComponent->GetAsset();

				UMovieSceneNiagaraSystemTrack* NiagaraSystemTrack = MovieScene->AddTrack<UMovieSceneNiagaraSystemTrack>(ComponentBinding);
				NiagaraSystemTrack->SetDisplayName(LOCTEXT("SystemLifeCycleTrackName", "System Life Cycle"));
				UMovieSceneNiagaraSystemSpawnSection* SpawnSection = NewObject<UMovieSceneNiagaraSystemSpawnSection>(NiagaraSystemTrack, NAME_None, RF_Transactional);

				FFrameRate FrameResolution = MovieScene->GetTickResolution();
				FFrameTime SpawnSectionStartTime = Sequencer->GetLocalTime().ConvertTo(FrameResolution);
				FFrameTime SpawnSectionDuration;

				UNiagaraSystemEditorData* SystemEditorData = Cast<UNiagaraSystemEditorData>(System->GetEditorData());
				if (SystemEditorData != nullptr && SystemEditorData->GetPlaybackRange().HasLowerBound() && SystemEditorData->GetPlaybackRange().HasUpperBound())
				{
					SpawnSectionDuration = FrameResolution.AsFrameTime(SystemEditorData->GetPlaybackRange().Size<float>());
				}
				else
				{
					SpawnSectionDuration = FrameResolution.AsFrameTime(5.0);
				}

				SpawnSection->SetRange(TRange<FFrameNumber>(
					SpawnSectionStartTime.RoundToFrame(),
					(SpawnSectionStartTime + SpawnSectionDuration).RoundToFrame()));
				NiagaraSystemTrack->AddSection(*SpawnSection);
			}
		}
	}
}

void FNiagaraSystemTrackEditor::AddNiagaraSystemTrack(TArray<FGuid> ObjectBindings)
{
	UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (MovieScene->IsReadOnly())
	{
		return;
	}

	FScopedTransaction AddTrackTransaction(LOCTEXT("AddNiagaraSystemLifeCycleTrackTransaction", "Add Niagara System Life Cycle Track"));
	MovieScene->Modify();

	for (FGuid ObjectBinding : ObjectBindings)
	{
		TArrayView<TWeakObjectPtr<UObject>> BoundObjects = GetSequencer()->FindBoundObjects(ObjectBinding, GetSequencer()->GetFocusedTemplateID());

		UNiagaraSystem* System = nullptr;
		for (TWeakObjectPtr<UObject> BoundObject : BoundObjects)
		{
			UNiagaraComponent* NiagaraComponent = Cast<UNiagaraComponent>(BoundObject.Get());
			if (NiagaraComponent != nullptr)
			{
				System = NiagaraComponent->GetAsset();
			}
		}

		if (System != nullptr)
		{
			UMovieSceneNiagaraSystemTrack* NiagaraSystemTrack = MovieScene->AddTrack<UMovieSceneNiagaraSystemTrack>(ObjectBinding);
			NiagaraSystemTrack->SetDisplayName(LOCTEXT("SystemLifeCycleTrackName", "System Life Cycle"));
			UMovieSceneNiagaraSystemSpawnSection* SpawnSection = NewObject<UMovieSceneNiagaraSystemSpawnSection>(NiagaraSystemTrack, NAME_None, RF_Transactional);

			FFrameRate FrameResolution = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene()->GetTickResolution();
			FFrameTime SpawnSectionStartTime = GetSequencer()->GetLocalTime().ConvertTo(FrameResolution);
			FFrameTime SpawnSectionDuration;

			UNiagaraSystemEditorData* SystemEditorData = Cast<UNiagaraSystemEditorData>(System->GetEditorData());
			if (SystemEditorData != nullptr && SystemEditorData->GetPlaybackRange().HasLowerBound() && SystemEditorData->GetPlaybackRange().HasUpperBound())
			{
				SpawnSectionDuration = FrameResolution.AsFrameTime(SystemEditorData->GetPlaybackRange().Size<float>());
			}
			else
			{
				SpawnSectionDuration = FrameResolution.AsFrameTime(5.0);
			}

			SpawnSection->SetRange(TRange<FFrameNumber>(
				SpawnSectionStartTime.RoundToFrame(),
				(SpawnSectionStartTime + SpawnSectionDuration).RoundToFrame()));
			NiagaraSystemTrack->AddSection(*SpawnSection);
		}
	}
		
	GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

void FNiagaraSystemTrackEditor::AddNiagaraParameterTrack(TArray<FGuid> ObjectBindings, FNiagaraVariable Parameter, TArray<uint8> DefaultValueData)
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	if (NiagaraEditorModule.CanCreateParameterTrackForType(*Parameter.GetType().GetScriptStruct()))
	{
		UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();
		if (MovieScene->IsReadOnly())
		{
			return;
		}

		FScopedTransaction AddTrackTransaction(LOCTEXT("AddNiagaraParameterTrackTransaction", "Add Niagara Parameter Track"));
		MovieScene->Modify();

		for (FGuid ObjectBinding : ObjectBindings)
		{
			UMovieSceneNiagaraParameterTrack* ParameterTrack = NiagaraEditorModule.CreateParameterTrackForType(*Parameter.GetType().GetScriptStruct(), Parameter);
			MovieScene->AddGivenTrack(ParameterTrack, ObjectBinding);

			ParameterTrack->SetParameter(Parameter);
			ParameterTrack->SetDisplayName(FText::FromName(Parameter.GetName()));

			UMovieSceneSection* ParameterSection = ParameterTrack->CreateNewSection();
			ParameterTrack->SetSectionChannelDefaults(ParameterSection, DefaultValueData);
			ParameterSection->SetRange(TRange<FFrameNumber>::All());
			ParameterTrack->AddSection(*ParameterSection);
		}

		GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}
}

#undef LOCTEXT_NAMESPACE