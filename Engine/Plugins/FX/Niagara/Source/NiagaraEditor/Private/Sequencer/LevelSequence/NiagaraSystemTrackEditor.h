// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrackEditor.h"
#include "NiagaraTypes.h"

class UMovieSceneNiagaraParameterTrack;

template<typename MovieSceneTrackType, typename MovieSceneSectionType>
class FNiagaraSystemParameterTrackEditor : public FMovieSceneTrackEditor
{
public:
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
	{
		return MakeShareable(new FNiagaraSystemParameterTrackEditor<MovieSceneTrackType, MovieSceneSectionType>(InSequencer));
	}

public:
	FNiagaraSystemParameterTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FMovieSceneTrackEditor(InSequencer)
	{
	}

	virtual FText GetDisplayName() const override
	{
		return NSLOCTEXT("Sequencer", "NiagaraSystemParameterTrackEditor_DisplayName", "Niagara System Parameter");
	}

	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override
	{
		return Type == MovieSceneTrackType::StaticClass();
	}
};

class FNiagaraSystemTrackEditor : public FMovieSceneTrackEditor
{
public:
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);
	static void AddDefaultSystemTracks(const AActor& SourceActor, const FGuid& Binding, TSharedPtr<ISequencer> Sequencer);

public:
	FNiagaraSystemTrackEditor(TSharedRef<ISequencer> InSequencer);

	//~ ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;

private:
	void AddNiagaraSystemTrack(TArray<FGuid> ObjectBindings);
	void AddNiagaraParameterTrack(TArray<FGuid> ObjectBindings, FNiagaraVariable Parameter, TArray<uint8> DefaultValue);
};