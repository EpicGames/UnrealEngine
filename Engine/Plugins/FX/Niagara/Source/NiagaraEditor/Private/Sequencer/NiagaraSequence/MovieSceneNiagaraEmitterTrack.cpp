// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraEmitterTrack.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraScriptSource.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "NiagaraEditorStyle.h"
#include "Sections/MovieSceneNiagaraEmitterSection.h"

#include "ISequencerSection.h"
#include "NiagaraEditorUtilities.h"
#include "SequencerSectionPainter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraEmitterTrack)

#define LOCTEXT_NAMESPACE "MovieSceneNiagaraEmitterTrack"

class FNiagaraInvalidSequencerSection : public ISequencerSection
{
public:
	virtual UMovieSceneSection* GetSectionObject()  override
	{
		unimplemented();
		return nullptr;
	}

	virtual int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override
	{
		unimplemented();
		return InPainter.LayerId;
	}
};

void UMovieSceneNiagaraEmitterSectionBase::Initialize(FNiagaraSystemViewModel& InSystemViewModel, TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel)
{
	ClearFlags(RF_Transactional);
	SystemViewModel = &InSystemViewModel;
	EmitterHandleViewModel = InEmitterHandleViewModel;
}

FNiagaraSystemViewModel& UMovieSceneNiagaraEmitterSectionBase::GetSystemViewModel() const
{
	return *SystemViewModel;
}

TSharedPtr<FNiagaraEmitterHandleViewModel> UMovieSceneNiagaraEmitterSectionBase::GetEmitterHandleViewModel() const
{
	return EmitterHandleViewModel.Pin();
}

FName UMovieSceneNiagaraEmitterSectionBase::GetInstanceName() const
{
	return InstanceName;
}

void UMovieSceneNiagaraEmitterSectionBase::SetInstanceName(FName InInstanceName)
{
	InstanceName = InInstanceName;
}

TSharedRef<ISequencerSection> UMovieSceneNiagaraEmitterSectionBase::MakeInvalidSectionInterface()
{
	checkf(false, TEXT("Invalid section interface can't be used."));
	return MakeShared<FNiagaraInvalidSequencerSection>();
}

UMovieSceneNiagaraEmitterTrack::UMovieSceneNiagaraEmitterTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), SystemViewModel(nullptr)
{
	bSectionsWereModified = false;
}

void UMovieSceneNiagaraEmitterTrack::Initialize(FNiagaraSystemViewModel& InSystemViewModel, TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel, const FFrameRate& InFrameResolution)
{
	ClearFlags(RF_Transactional);
	SystemViewModel = &InSystemViewModel;
	EmitterHandleViewModel = InEmitterHandleViewModel;
	SetDisplayName(InEmitterHandleViewModel->GetNameText());
	EmitterHandleId = InEmitterHandleViewModel->GetId();
	SystemPath = SystemViewModel->GetSystem().GetPathName();
	SetColorTint(FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.NiagaraSequence.DefaultTrackColor").ToFColor(true));
	CreateSections(InFrameResolution);

	// since we are updating the color on tick when scalability mode is active, we reset the color when it is deactivated
	SystemViewModel->GetScalabilityViewModel()->OnScalabilityModeChanged().AddUObject(this, &UMovieSceneNiagaraEmitterTrack::RestoreDefaultTrackColor);
}

UMovieSceneNiagaraEmitterTrack::~UMovieSceneNiagaraEmitterTrack()
{
	//SystemViewModel->GetScalabilityViewModel()->OnScalabilityModeChanged().RemoveAll(this);
}

bool UMovieSceneNiagaraEmitterTrack::CanRename() const
{
	if (SystemViewModel && SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		return true;
	}
	return false;
}

bool UMovieSceneNiagaraEmitterTrack::ValidateDisplayName(const FText& NewDisplayName, FText& OutErrorMessage) const
{
	return EmitterHandleViewModel.Pin()->VerifyNameTextChanged(NewDisplayName, OutErrorMessage);
}

FNiagaraSystemViewModel& UMovieSceneNiagaraEmitterTrack::GetSystemViewModel() const
{
	return *SystemViewModel;
}

TSharedPtr<FNiagaraEmitterHandleViewModel> UMovieSceneNiagaraEmitterTrack::GetEmitterHandleViewModel() const
{
	return EmitterHandleViewModel.Pin();
}

void UMovieSceneNiagaraEmitterTrack::UpdateTrackFromEmitterGraphChange(const FFrameRate& InFrameResolution)
{
	SetDisplayName(EmitterHandleViewModel.Pin()->GetNameText());
	Sections.Empty();
	CreateSections(InFrameResolution);
}

void UMovieSceneNiagaraEmitterTrack::UpdateTrackFromEmitterParameterChange(const FFrameRate& InFrameResolution)
{
	for (UMovieSceneSection* Section : Sections)
	{
		CastChecked<UMovieSceneNiagaraEmitterSectionBase>(Section)->UpdateSectionFromModules(InFrameResolution);
		Section->SetIsActive(EmitterHandleViewModel.Pin()->GetIsEnabled());
	}
}

void UMovieSceneNiagaraEmitterTrack::UpdateEmitterHandleFromTrackChange(const FFrameRate& InFrameResolution)
{
	if (Sections.Num() > 0)
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelPinned = EmitterHandleViewModel.Pin();
		checkf(EmitterHandleViewModelPinned.IsValid(), TEXT("Emitter handle view model is invalid while it's track is still active."));

		TOptional<bool> SectionsActive = Sections[0]->IsActive();
		for (UMovieSceneSection* Section : Sections)
		{
			CastChecked<UMovieSceneNiagaraEmitterSectionBase>(Section)->UpdateModulesFromSection(InFrameResolution);
			if (SectionsActive.IsSet() && Section->IsActive() != SectionsActive.GetValue())
			{
				SectionsActive.Reset();
			}
		}

		if (SectionsActive.IsSet())
		{
			EmitterHandleViewModelPinned->SetIsEnabled(SectionsActive.GetValue());
		}
		else
		{
			// If there were mixed active states on the sections then one was toggled from the emitter handle state so we need
			// to update the emitter is active and the other sections.
			EmitterHandleViewModelPinned->SetIsEnabled(!EmitterHandleViewModelPinned->GetIsEnabled());
			for (UMovieSceneSection* Section : Sections)
			{
				Section->SetIsActive(EmitterHandleViewModelPinned->GetIsEnabled());
			}
		}
	}
}

bool UMovieSceneNiagaraEmitterTrack::GetSectionsWereModified() const
{
	return bSectionsWereModified;
}

void UMovieSceneNiagaraEmitterTrack::Tick(float DeltaTime)
{
	if(!HasAnyFlags(RF_ClassDefaultObject) && bScalabilityModeActive)
	{
		bool bIsAllowedByScalability = false;

		if (TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterViewModel = EmitterHandleViewModel.Pin())
		{
			FNiagaraEmitterHandle* EmitterHandle = EmitterViewModel->GetEmitterHandle();
			bIsAllowedByScalability = EmitterHandle ? EmitterHandle->IsAllowedByScalability() : false;
		}

		if (bIsAllowedByScalability)
		{
			SetColorTint(FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.SystemOverview.ExcludedFromScalability").ToFColor(true));
		}
		else
		{
			SetColorTint(FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.NiagaraSequence.DefaultTrackColor").ToFColor(true));
		}
	}
}

TStatId UMovieSceneNiagaraEmitterTrack::GetStatId() const
{
	return TStatId();
}

bool UMovieSceneNiagaraEmitterTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

void UMovieSceneNiagaraEmitterTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
	bSectionsWereModified = true;
}

UMovieSceneSection* UMovieSceneNiagaraEmitterTrack::CreateNewSection()
{
	return nullptr;
}

bool UMovieSceneNiagaraEmitterTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneNiagaraEmitterSection::StaticClass();
}

void UMovieSceneNiagaraEmitterTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
	bSectionsWereModified = true;
}

void UMovieSceneNiagaraEmitterTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
	bSectionsWereModified = true;
}

bool UMovieSceneNiagaraEmitterTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

const TArray<UMovieSceneSection*>& UMovieSceneNiagaraEmitterTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneNiagaraEmitterTrack::SupportsMultipleRows() const
{
	return true;
}

FGuid UMovieSceneNiagaraEmitterTrack::GetEmitterHandleId() const
{
	return EmitterHandleId;
}

const FString& UMovieSceneNiagaraEmitterTrack::GetSystemPath() const
{
	return SystemPath;
}

const TArray<FText>& UMovieSceneNiagaraEmitterTrack::GetSectionInitializationErrors() const
{
	return SectionInitializationErrors;
}

void UMovieSceneNiagaraEmitterTrack::CreateSections(const FFrameRate& InFrameResolution)
{
	SectionInitializationErrors.Empty();

	const FVersionedNiagaraEmitterData* EmitterData = GetEmitterHandleViewModel()->GetEmitterHandle()->GetEmitterData();
	if (EmitterData == nullptr)
	{
		return;
	}

	UNiagaraScript* EmitterUpdateScript = EmitterData->GetScript(ENiagaraScriptUsage::EmitterUpdateScript, FGuid());
	UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(EmitterUpdateScript->GetLatestSource());
	UNiagaraNodeOutput* OutputNode = ScriptSource->NodeGraph->FindOutputNode(ENiagaraScriptUsage::EmitterUpdateScript);

	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackGroups;
	FNiagaraStackGraphUtilities::GetStackNodeGroups(*OutputNode, StackGroups);
	for (FNiagaraStackGraphUtilities::FStackNodeGroup& StackGroup : StackGroups)
	{
		UNiagaraNodeFunctionCall* FunctionNode = Cast<UNiagaraNodeFunctionCall>(StackGroup.EndNode);
		if (FunctionNode != nullptr && FunctionNode->FunctionScript != nullptr)
		{
			if (FString* SectionClassName = FunctionNode->GetScriptData()->ScriptMetaData.Find("NiagaraTimelineSectionClass"))
			{
				if (UClass* SectionClass = FindFirstObject<UClass>(**SectionClassName, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous))
				{
					UMovieSceneNiagaraEmitterSectionBase* EmitterSection = nullptr;
					bool bSectionCreated = false;

					TObjectPtr<UMovieSceneSection>* SharedEmitterSection = Sections.FindByPredicate([=](UMovieSceneSection* Section) {	return Section->IsA(SectionClass); });
					if (SharedEmitterSection != nullptr)
					{
						EmitterSection = CastChecked<UMovieSceneNiagaraEmitterSectionBase>(*SharedEmitterSection);
					}

					if(EmitterSection == nullptr)
					{
						EmitterSection = NewObject<UMovieSceneNiagaraEmitterSectionBase>(this, SectionClass, NAME_None, RF_Transient);
						EmitterSection->Initialize(*SystemViewModel, EmitterHandleViewModel.Pin().ToSharedRef());
						bSectionCreated = true;
					}

					FText AddError;
					if (EmitterSection->TryAddModule(*FunctionNode, AddError))
					{
						if (bSectionCreated)
						{
							EmitterSection->SetRowIndex(Sections.Num());
							Sections.Add(EmitterSection);
						}
					}
					else
					{
						FText SectionInitializationError = FText::Format(LOCTEXT("AddModuleErrorFormat", "Failed to add module {0} to section of type {1}.\nMessage: {2}"),
							FText::FromString(FunctionNode->GetFunctionName()), FText::FromString(*SectionClassName), AddError);
						SectionInitializationErrors.Add(SectionInitializationError);
					}
				}
				else
				{
					SectionInitializationErrors.Add(
						FText::Format(LOCTEXT("SectionClassNotFoundErrorFormat", "Module script {0} tried to use {1} as it's timeline section class, but it could not be found."),
							FText::FromString(FunctionNode->FunctionScript->GetPathName()),
							FText::FromString(*SectionClassName)));
				}
			}
		}
	}

	UpdateTrackFromEmitterParameterChange(InFrameResolution);
	bSectionsWereModified = false;
}

void UMovieSceneNiagaraEmitterTrack::RestoreDefaultTrackColor(bool bScalabilityModeActivated)
{
	bScalabilityModeActive = bScalabilityModeActivated;
	
	if(bScalabilityModeActive == false)
	{
		SetColorTint(FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.NiagaraSequence.DefaultTrackColor").ToFColor(true));
	}
}

#undef LOCTEXT_NAMESPACE


