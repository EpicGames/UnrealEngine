// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "TickableEditorObject.h"
#include "MovieSceneNiagaraEmitterTrack.generated.h"

class UNiagaraSystem;
class FNiagaraSystemViewModel;
class FNiagaraEmitterHandleViewModel;
class UNiagaraNodeFunctionCall;
class ISequencerSection;

UCLASS(abstract, MinimalAPI)
class UMovieSceneNiagaraEmitterSectionBase : public UMovieSceneSection
{
	GENERATED_BODY()

public:
	void Initialize(FNiagaraSystemViewModel& InSystemViewModel, TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel);

	FNiagaraSystemViewModel& GetSystemViewModel() const;

	TSharedPtr<FNiagaraEmitterHandleViewModel> GetEmitterHandleViewModel() const;

	FName GetInstanceName() const;

	void SetInstanceName(FName InInstanceName);

	virtual bool TryAddModule(UNiagaraNodeFunctionCall& InModule, FText& OutErrorMessage) PURE_VIRTUAL(UMovieSceneNiagaraEmitterSectionBase::TryAddModule, return false;);

	virtual void UpdateSectionFromModules(const FFrameRate& InFrameResolution) PURE_VIRTUAL(UMovieSceneNiagaraEmitterSectionBase::UpdateSectionFromModules, );

	virtual void UpdateModulesFromSection(const FFrameRate& InFrameResolution) PURE_VIRTUAL(UMovieSceneNiagaraEmitterSectionBase::UpdateModulesFromSection, );

	virtual TSharedRef<ISequencerSection> MakeSectionInterface() PURE_VIRTUAL(UMovieSceneNiagaraEmitterSectionBase::UpdateModulesFromSection, return MakeInvalidSectionInterface(););

private:
	TSharedRef<ISequencerSection> MakeInvalidSectionInterface();

	FNiagaraSystemViewModel* SystemViewModel;

	TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel;

	FName InstanceName;
};

/**
*	A track representing an emitter in the niagara effect editor timeline.
*/
UCLASS(MinimalAPI)
class UMovieSceneNiagaraEmitterTrack
	: public UMovieSceneNameableTrack, public FTickableEditorObject
{
	GENERATED_UCLASS_BODY()

public:
	void Initialize(FNiagaraSystemViewModel& SystemViewModel, TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel, const FFrameRate& InFrameResolution);
	virtual ~UMovieSceneNiagaraEmitterTrack();
	
	virtual bool CanRename() const override;

	FNiagaraSystemViewModel& GetSystemViewModel() const;

	TSharedPtr<FNiagaraEmitterHandleViewModel> GetEmitterHandleViewModel() const;

	void UpdateTrackFromEmitterGraphChange(const FFrameRate& InFrameResolution);

	void UpdateTrackFromEmitterParameterChange(const FFrameRate& InFrameResolution);

	void UpdateEmitterHandleFromTrackChange(const FFrameRate& InFrameResolution);

	bool GetSectionsWereModified() const;

	// FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;

	//~ UMovieSceneTrack interface
	virtual void RemoveAllAnimationData() override { }
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool SupportsMultipleRows() const override;
	virtual bool ValidateDisplayName(const FText& NewDisplayName, FText& OutErrorMessage) const override;
	

	/** Gets the unique id for the emitter handle that was associated with this track; used for copy/paste detection */
	FGuid GetEmitterHandleId() const;

	/** Gets the string path of the system which owns the emitter associated with this track; used for copy/paste detection */
	const FString& GetSystemPath() const;

	const TArray<FText>& GetSectionInitializationErrors() const;

private:
	void CreateSections(const FFrameRate& InFrameResolution);
	void RestoreDefaultTrackColor(bool bScalabilityModeActivated);

private:
	FNiagaraSystemViewModel* SystemViewModel;

	TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	UPROPERTY()
	bool bSectionsWereModified;

	// Used for detecting copy/paste 
	UPROPERTY()
	FGuid EmitterHandleId;

	// Used for detecting copy/paste
	UPROPERTY()
	FString SystemPath;

	TArray<FText> SectionInitializationErrors;

	bool bScalabilityModeActive = false;
};