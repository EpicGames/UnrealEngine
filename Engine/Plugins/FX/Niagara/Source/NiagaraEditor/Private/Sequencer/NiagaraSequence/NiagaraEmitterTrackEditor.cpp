// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterTrackEditor.h"
#include "NiagaraEmitter.h"
#include "NiagaraRendererProperties.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Sequencer/NiagaraSequence/NiagaraSequence.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Modules/ModuleManager.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"

#include "MVVM/ViewModelPtr.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"
#include "MVVM/Views/SOutlinerItemViewBase.h"
#include "MVVM/Views/SOutlinerTrackColorPicker.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

#define LOCTEXT_NAMESPACE "NiagaraEmitterTrackEditor"

class SEmitterTrackWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEmitterTrackWidget) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UMovieSceneNiagaraEmitterTrack& InEmitterTrack)
	{
		EmitterTrack = &InEmitterTrack;

		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TSharedRef<SHorizontalBox> TrackBox = SNew(SHorizontalBox)
			// Track initialization error icon.
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(3, 0, 0, 0)
			[
				SNew(SImage)
				.Visibility(this, &SEmitterTrackWidget::GetTrackErrorIconVisibility)
				.Image(FAppStyle::GetBrush("Icons.Info"))
				.ToolTipText(this, &SEmitterTrackWidget::GetTrackErrorIconToolTip)
			]
			// Stack issues icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(3, 0, 0, 0)
			[
				NiagaraEditorModule.GetWidgetProvider()->CreateStackIssueIcon( 
					*EmitterTrack->GetEmitterHandleViewModel()->GetEmitterStackViewModel(),
					*EmitterTrack->GetEmitterHandleViewModel()->GetEmitterStackViewModel()->GetRootEntry())
			]
			// Isolate toggle
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(3, 0, 0, 0)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.HAlign(HAlign_Center)
				.ContentPadding(1)
				.ToolTipText(this, &SEmitterTrackWidget::GetToggleIsolateToolTip)
				.OnClicked(this, &SEmitterTrackWidget::OnToggleIsolateButtonClicked)
				.Visibility(this, &SEmitterTrackWidget::GetIsolateToggleVisibility)
				.IsFocusable(false)
				.Content()
				[
					SNew(SImage)
					.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Isolate"))
					.ColorAndOpacity(this, &SEmitterTrackWidget::GetToggleIsolateImageColor)
				]
			];

		// Renderer buttons
		TrackBox->AddSlot()
			.AutoWidth()
			[
				SAssignNew(RenderersBox, SHorizontalBox)
			];
		ConstructRendererWidgets();

		// We refresh the renderer widgets when renderers changed.
		// The OnRenderersChanged delegate will cause the renderer items to be created that are used for widget creation,
		// Due to delegate bind order, the item creation happens after the refresh. To solve this, we just wait a frame. 
		UNiagaraEmitterBase* NiagaraEmitter = EmitterTrack->GetEmitterHandleViewModel()->GetEmitterHandle()->GetEmitterBase();
		WeakNiagaraEmitter = NiagaraEmitter;
		if (NiagaraEmitter)
		{
			NiagaraEmitter->OnRenderersChanged().AddSP(this, &SEmitterTrackWidget::RefreshRenderers);
		}

		// Enabled checkbox.
		TrackBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(3, 0, 0, 0)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("EnabledTooltip", "Toggle whether or not this emitter is enabled."))
				.IsChecked(this, &SEmitterTrackWidget::GetEnabledCheckState)
				.OnCheckStateChanged(this, &SEmitterTrackWidget::OnEnabledCheckStateChanged)
				.Visibility(this, &SEmitterTrackWidget::GetEnableCheckboxVisibility)
			];

		ChildSlot
		[
			TrackBox
		];
	}

	~SEmitterTrackWidget()
	{
		if ( UNiagaraEmitterBase* NiagaraEmitter = WeakNiagaraEmitter.Get() )
		{
			NiagaraEmitter->OnRenderersChanged().RemoveAll(this);
		}
	}

private:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if(bShouldRefreshRenderers)
		{
			ConstructRendererWidgets();
			bShouldRefreshRenderers = false;
		}
	}
	
	void ConstructRendererWidgets()
	{
		RenderersBox->ClearChildren();

		if(EmitterTrack.IsValid())
		{
			TArray<UNiagaraStackEntry*> RendererEntryData;
			EmitterTrack->GetEmitterHandleViewModel()->GetRendererEntries(RendererEntryData);
			for (UNiagaraStackEntry* RendererEntry : RendererEntryData)
			{
				if (UNiagaraStackRendererItem* RendererItem = Cast<UNiagaraStackRendererItem>(RendererEntry))
				{
					UNiagaraRendererProperties* Renderer = RendererItem->GetRendererProperties();
					RenderersBox->AddSlot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(3, 0, 0, 0)
						[
							SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.IsFocusable(false)
							.ToolTipText(FText::Format(LOCTEXT("RenderButtonToolTip", "{0} - Press to select."), FText::FromString(FName::NameToDisplayString(Renderer->GetName(), false))))
							.OnClicked(this, &SEmitterTrackWidget::OnRenderButtonClicked, RendererEntry)
							[
								SNew(SImage)
								.Image(FSlateIconFinder::FindIconBrushForClass(Renderer->GetClass()))
							]
						];
				}
			}
		}
	}

	EVisibility GetTrackErrorIconVisibility() const 
	{
		return EmitterTrack.IsValid() && EmitterTrack.Get()->GetSectionInitializationErrors().Num() > 0
			? EVisibility::Visible
			: EVisibility::Collapsed;
	}

	FText GetTrackErrorIconToolTip() const
	{
		if(TrackErrorIconToolTip.IsSet() == false && EmitterTrack.IsValid())
		{
			FString TrackErrorIconToolTipBuilder;
			for (const FText& SectionInitializationError : EmitterTrack.Get()->GetSectionInitializationErrors())
			{
				if (TrackErrorIconToolTipBuilder.IsEmpty() == false)
				{
					TrackErrorIconToolTipBuilder.Append(TEXT("\n"));
				}
				TrackErrorIconToolTipBuilder.Append(SectionInitializationError.ToString());
			}
			TrackErrorIconToolTip = FText::FromString(TrackErrorIconToolTipBuilder);
		}
		return TrackErrorIconToolTip.GetValue();
	}

	ECheckBoxState GetEnabledCheckState() const
	{
		return EmitterTrack.IsValid() && EmitterTrack->GetEmitterHandleViewModel()->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnEnabledCheckStateChanged(ECheckBoxState InCheckState)
	{
		if (EmitterTrack.IsValid())
		{
			EmitterTrack->GetEmitterHandleViewModel()->SetIsEnabled(InCheckState == ECheckBoxState::Checked);
		}
	}

	FReply OnToggleIsolateButtonClicked()
	{
		if (EmitterTrack.IsValid())
		{
			TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterTrack->GetEmitterHandleViewModel();
			if (EmitterHandleViewModel.IsValid())
			{
				if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
				{
					// Shift+click: toggle isolation only on this emitter, leaving other emitters' isolation state untouched.
					EmitterHandleViewModel->ToggleIsIsolated();
				}
				else
				{
					EmitterTrack->GetSystemViewModel().ToggleIsolateEmitterAndSelectedEmitters(EmitterHandleViewModel->GetId());
				}
			}
		}
		return FReply::Handled();
	}

	FText GetToggleIsolateToolTip() const
	{
		return EmitterTrack.IsValid() && EmitterTrack->GetEmitterHandleViewModel()->GetIsIsolated()
			? LOCTEXT("TurnOffEmitterIsolation", "Disable emitter isolation.\nShift+click: toggle only this emitter without affecting other emitters' isolation state.")
			: LOCTEXT("IsolateThisEmitter", "Enable isolation for this emitter.\nShift+click: toggle only this emitter without affecting other emitters' isolation state.");
	}

	FSlateColor GetToggleIsolateImageColor() const
	{
		return EmitterTrack.IsValid() && EmitterTrack->GetEmitterHandleViewModel()->GetIsIsolated()
			? FAppStyle::GetSlateColor("SelectionColor")
			: FLinearColor::Gray;
	}

	FReply OnRenderButtonClicked(UNiagaraStackEntry* InRendererEntry)
	{
		if (EmitterTrack.IsValid())
		{
			TArray<UNiagaraStackEntry*> SelectedEntries;
			SelectedEntries.Add(InRendererEntry);
			TArray<UNiagaraStackEntry*> DeselectedEntries;
			EmitterTrack->GetSystemViewModel().GetSelectionViewModel()->UpdateSelectedEntries(SelectedEntries, DeselectedEntries, true);
		}
		return FReply::Handled();
	}

	EVisibility GetEnableCheckboxVisibility() const
	{
		return EmitterTrack.IsValid() && EmitterTrack->GetSystemViewModel().GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetIsolateToggleVisibility() const
	{
		return EmitterTrack.IsValid() && EmitterTrack->GetSystemViewModel().GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset ? EVisibility::Visible : EVisibility::Collapsed;
	}

	void RefreshRenderers()
	{
		bShouldRefreshRenderers = true;
	}

private:
	TWeakObjectPtr<UMovieSceneNiagaraEmitterTrack> EmitterTrack;
	TWeakObjectPtr<UNiagaraEmitterBase> WeakNiagaraEmitter;
	mutable TOptional<FText> TrackErrorIconToolTip;
	TSharedPtr<SHorizontalBox> RenderersBox;
	bool bShouldRefreshRenderers = false;
};

FNiagaraEmitterTrackEditor::FNiagaraEmitterTrackEditor(TSharedPtr<ISequencer> Sequencer) 
	: FMovieSceneTrackEditor(Sequencer.ToSharedRef())
{
}

TSharedRef<ISequencerTrackEditor> FNiagaraEmitterTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FNiagaraEmitterTrackEditor(InSequencer));
}

FText FNiagaraEmitterTrackEditor::GetDisplayName() const
{
	return LOCTEXT("NiagaraEmitterTrackEditor_DisplayName", "Niagara Emitter");
}

bool FNiagaraEmitterTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const 
{
	if (TrackClass == UMovieSceneNiagaraEmitterTrack::StaticClass())
	{
		return true;
	}
	return false;
}

TSharedRef<ISequencerSection> FNiagaraEmitterTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	UMovieSceneNiagaraEmitterSectionBase* EmitterSection = CastChecked<UMovieSceneNiagaraEmitterSectionBase>(&SectionObject);
	return EmitterSection->MakeSectionInterface();
}

bool FNiagaraEmitterTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	UNiagaraEmitter* EmitterAsset = Cast<UNiagaraEmitter>(Asset);
	UNiagaraSequence* NiagaraSequence = Cast<UNiagaraSequence>(GetSequencer()->GetRootMovieSceneSequence());
	if (EmitterAsset != nullptr && NiagaraSequence != nullptr && NiagaraSequence->GetSystemViewModel().GetCanModifyEmittersFromTimeline())
	{
		NiagaraSequence->GetSystemViewModel().AddEmitter(*EmitterAsset, EmitterAsset->GetExposedVersion().VersionGuid);
	}
	return false;
}

void FNiagaraEmitterTrackEditor::BuildTrackContextMenu( FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track )
{
	UMovieSceneNiagaraEmitterTrack* EmitterTrack = CastChecked<UMovieSceneNiagaraEmitterTrack>(Track);
	FNiagaraSystemViewModel& SystemViewModel = EmitterTrack->GetSystemViewModel();

	if (SystemViewModel.GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		FNiagaraEditorUtilities::AddEmitterContextMenuActions(MenuBuilder, EmitterTrack->GetEmitterHandleViewModel());
	}
}

TSharedPtr<SWidget> FNiagaraEmitterTrackEditor::BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName)
{
	using namespace UE::Sequencer;

	TViewModelPtr<IOutlinerExtension> OutlinerItem = Params.ViewModel.ImplicitCast();
	if (!OutlinerItem)
	{
		return nullptr;
	}

	if (ColumnName == FCommonOutlinerNames::Edit)
	{
		UMovieSceneNiagaraEmitterTrack* Track = nullptr;
		if (TViewModelPtr<ITrackExtension> TrackModel = Params.ViewModel->FindAncestorOfType<ITrackExtension>(true))
		{
			Track = CastChecked<UMovieSceneNiagaraEmitterTrack>(TrackModel->GetTrack());
		}

		return Track ? SNew(SEmitterTrackWidget, *Track) : SNullWidget::NullWidget;;
	}

	if (ColumnName == FCommonOutlinerNames::Label)
	{
		return SNew(SOutlinerItemViewBase, OutlinerItem, Params.Editor, Params.TreeViewRow);
	}

	if (ColumnName == FCommonOutlinerNames::ColorPicker)
	{
		return SNew(SOutlinerTrackColorPicker, OutlinerItem, Params.Editor);
	}


	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
