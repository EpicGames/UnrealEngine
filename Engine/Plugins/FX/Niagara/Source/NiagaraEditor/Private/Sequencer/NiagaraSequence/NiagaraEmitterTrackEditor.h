// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrackEditor.h"

#include "MovieSceneNiagaraEmitterTrack.h"


/**
*	Track editor for Niagara emitter tracks
*/
class FNiagaraEmitterTrackEditor
	: public FMovieSceneTrackEditor
{
public:
	FNiagaraEmitterTrackEditor(TSharedPtr<ISequencer> Sequencer);

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);

	//~ FMovieSceneTrackEditor interface.
	virtual FText GetDisplayName() const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	virtual void BuildTrackContextMenu( FMenuBuilder& MenuBuilder, UMovieSceneTrack* Track ) override; 
	virtual TSharedPtr<SWidget> BuildOutlinerColumnWidget(const FBuildColumnWidgetParams& Params, const FName& ColumnName) override;

private:
	void ToggleEmitterIsolation(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterToIsolate);

	bool CanIsolateSelectedEmitters() const;

	void IsolateSelectedEmitters();
};
