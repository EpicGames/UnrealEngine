// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSequence.h"
#include "ViewModels/NiagaraSystemViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSequence)

UNiagaraSequence::UNiagaraSequence(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MovieScene(nullptr)
{ }

void UNiagaraSequence::Initialize(FNiagaraSystemViewModel* InSystemViewModel, UMovieScene* InMovieScene)
{
	SystemViewModel = InSystemViewModel;
	MovieScene = InMovieScene;
}

FNiagaraSystemViewModel& UNiagaraSequence::GetSystemViewModel() const
{
	checkf(SystemViewModel != nullptr, TEXT("Niagara sequence not initialized"));
	return *SystemViewModel;
}

void UNiagaraSequence::BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context)
{
}


bool UNiagaraSequence::CanPossessObject(UObject& Object, UObject* InPlaybackContext) const
{
	return false;
}

UMovieScene* UNiagaraSequence::GetMovieScene() const
{
	checkf(MovieScene != nullptr, TEXT("Niagara sequence not initialized"));
	return MovieScene;
}


UObject* UNiagaraSequence::GetParentObject(UObject* Object) const
{
	return nullptr;
}


void UNiagaraSequence::UnbindPossessableObjects(const FGuid& ObjectId)
{
}

