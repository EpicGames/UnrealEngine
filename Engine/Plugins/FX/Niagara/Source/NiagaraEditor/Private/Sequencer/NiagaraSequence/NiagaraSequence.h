// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSequence.h"
#include "NiagaraSequence.generated.h"

class UMovieScene;
class FNiagaraSystemViewModel;

/**
 * Movie scene sequence used by Niagara.
 */
UCLASS(BlueprintType, MinimalAPI)
class UNiagaraSequence
	: public UMovieSceneSequence
{
	GENERATED_UCLASS_BODY()

public:
	void Initialize(FNiagaraSystemViewModel* InSystemViewModel, UMovieScene* InMovieScene);

	FNiagaraSystemViewModel& GetSystemViewModel() const;

	//~ UMovieSceneSequence interface
	virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override;
	virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override;
	virtual void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override { /* Unused Pure Virtual */ }
	virtual UMovieScene* GetMovieScene() const override;
	virtual UObject* GetParentObject(UObject* Object) const override;
	virtual void UnbindPossessableObjects(const FGuid& ObjectId) override;
	virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context) override { /* Unused Pure Virtual */ }
	virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context) override { /* Unused Pure Virtual */ }

private:
	/** Pointer to the movie scene that controls this sequence. */
	UPROPERTY()
	TObjectPtr<UMovieScene> MovieScene;

	/** The system view model which owns this niagara sequence. */
	FNiagaraSystemViewModel* SystemViewModel;
};
