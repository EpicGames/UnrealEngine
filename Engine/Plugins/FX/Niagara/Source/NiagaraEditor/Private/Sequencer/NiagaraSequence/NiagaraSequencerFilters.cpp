// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSequencerFilters.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "MovieScene/MovieSceneNiagaraTrack.h"
#include "NiagaraActor.h"
#include "NiagaraComponent.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSequencerFilters)

#define LOCTEXT_NAMESPACE "NiagaraSequencerTrackFilters"

class FSequencerTrackFilter_NiagaraFilterCommands
	: public TCommands<FSequencerTrackFilter_NiagaraFilterCommands>
{
public:
	FSequencerTrackFilter_NiagaraFilterCommands()
		: TCommands<FSequencerTrackFilter_NiagaraFilterCommands>(
			TEXT("FSequencerTrackFilter_Niagara"),
			LOCTEXT("FSequencerTrackFilter_Niagara", "Niagara Filters"),
			NAME_None,
			FAppStyle::GetAppStyleSetName())
	{}

	TSharedPtr<FUICommandInfo> ToggleFilter_Niagara;

	virtual void RegisterCommands() override
	{
		UI_COMMAND(ToggleFilter_Niagara, "Toggle Niagara Filter", "Toggle the filter for Niagara tracks", EUserInterfaceActionType::ToggleButton, FInputChord());
	}
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_Niagara : public FSequencerTrackFilter
{
public:
	FSequencerTrackFilter_Niagara(ISequencerTrackFilters& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr)
		: FSequencerTrackFilter(InFilterInterface, MoveTemp(InCategory))
		, BindingCount(0)
	{
		FSequencerTrackFilter_NiagaraFilterCommands::Register();
	}

	virtual ~FSequencerTrackFilter_Niagara() override
	{
		BindingCount--;
		if (BindingCount < 1)
		{
			FSequencerTrackFilter_NiagaraFilterCommands::Unregister();
		}
	}

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override { return LOCTEXT("SequencerTrackFilter_Niagara", "Niagara"); }
	virtual FSlateIcon GetIcon() const override { return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PlacementBrowser.Icons.VisualEffects")); } // TEXT("ClassIcon.ParticleSystem")
	//~ End FFilterBase

	//~ Begin IFilter

	virtual FString GetName() const override { return TEXT("Niagara"); }

	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override
	{
		FSequencerFilterData& FilterData = GetFilterInterface().GetFilterData();

		const UObject* const BoundObject = FilterData.ResolveTrackBoundObject(GetSequencer(), InItem);
		if (IsValid(BoundObject))
		{
			if (BoundObject->IsA(UMovieSceneNiagaraTrack::StaticClass())
				|| BoundObject->IsA(ANiagaraActor::StaticClass())
				|| BoundObject->IsA(UNiagaraComponent::StaticClass()))
			{
				return true;
			}
		}

		const AActor* const Actor = Cast<const AActor>(BoundObject);
		if (IsValid(Actor) && Actor->FindComponentByClass(UNiagaraComponent::StaticClass()))
		{
			return true;
		}

		return false;
	}

	//~ End IFilter

	//~ Begin FSequencerTrackFilter

	virtual FText GetDefaultToolTipText() const override
	{
		return LOCTEXT("SequencerTrackFilter_NiagaraToolTip", "Show only Niagara tracks");
	}

	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override
	{
		return FSequencerTrackFilter_NiagaraFilterCommands::Get().ToggleFilter_Niagara;
	}

	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override
	{
		return IsSequenceTrackSupported<UMovieSceneNiagaraTrack>(InSequence);
	}

	//~ End FSequencerTrackFilter

private:
	mutable uint32 BindingCount;
};

//////////////////////////////////////////////////////////////////////////
//

void UNiagaraSequencerTrackFilter::AddTrackFilterExtensions(ISequencerTrackFilters& InFilterInterface, const TSharedRef<FFilterCategory>& InPreferredCategory, TArray<TSharedRef<FSequencerTrackFilter>>& InOutFilterList) const
{
	InOutFilterList.Add(MakeShared<FSequencerTrackFilter_Niagara>(InFilterInterface, InPreferredCategory));
}

#undef LOCTEXT_NAMESPACE
