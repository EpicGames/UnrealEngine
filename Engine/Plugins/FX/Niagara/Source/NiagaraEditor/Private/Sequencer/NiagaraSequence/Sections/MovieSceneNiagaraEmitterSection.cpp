// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneNiagaraEmitterSection.h"
#include "NiagaraEmitterSection.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraEmitter.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraScriptSource.h"
#include "NiagaraNodeFunctionCall.h"

#include "Channels/MovieSceneChannelProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneNiagaraEmitterSection)

#define LOCTEXT_NAMESPACE "MovieSceneNiagaraEmitterTimedSection"

FName TimelineModeKey("TimelineMode");

FString TimeRangeTimelineModeValue("TimeRange");
FString KeyTimelineModeValue("Key");

FName InputUsageKey("TimelineInputUsage");

FString UseStartTimeInputUsageValue("UseStartTime");
FString StartTimeInputUsageValue("StartTime");
FString LengthInputUsageValue("Length");
FString NumLoopsInputUsageValue("NumLoops");
FString StartTimeIncludedInFirstLoopOnlyInputUsageValue("StartTimeIncludedInFirstLoopOnly");

FString KeyTimeInputUsageValue("KeyTime");
FString KeyValueInputUsageValue("KeyValue");

bool FMovieSceneNiagaraEmitterChannel::Evaluate(FFrameTime InTime, FNiagaraEmitterSectionKey& OutValue) const
{
	if (Times.Num())
	{
		const int32 Index = FMath::Max(0, Algo::UpperBound(Times, InTime.FrameNumber) - 1);
		OutValue = Values[Index];
		return true;
	}

	return false;
}

void FMovieSceneNiagaraEmitterChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	return GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneNiagaraEmitterChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneNiagaraEmitterChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneNiagaraEmitterChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneNiagaraEmitterChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneNiagaraEmitterChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	// Insert a key at the current time to maintain evaluation
	if (GetData().GetTimes().Num() > 0)
	{
		FNiagaraEmitterSectionKey Value;
		if (Evaluate(InTime, Value))
		{
			GetData().UpdateOrAddKey(InTime, Value);
		}
	}

	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneNiagaraEmitterChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneNiagaraEmitterChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneNiagaraEmitterChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneNiagaraEmitterChannel::Reset()
{
	Times.Reset();
	Values.Reset();
	KeyHandles.Reset();
}

void FMovieSceneNiagaraEmitterChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

TSharedPtr<FStructOnScope> GetKeyStruct(TMovieSceneChannelHandle<FMovieSceneNiagaraEmitterChannel> Channel, FKeyHandle InHandle)
{
	int32 KeyValueIndex = Channel.Get()->GetData().GetIndex(InHandle);
	if (KeyValueIndex != INDEX_NONE)
	{
		FNiagaraTypeDefinition KeyType = Channel.Get()->GetData().GetValues()[KeyValueIndex].Value.GetType();
		uint8* KeyData = Channel.Get()->GetData().GetValues()[KeyValueIndex].Value.GetData();
		return MakeShared<FStructOnScope>(KeyType.GetStruct(), KeyData);
	}
	return TSharedPtr<FStructOnScope>();
}

bool UMovieSceneNiagaraEmitterSection::TryAddModule(UNiagaraNodeFunctionCall& InModule, FText& OutErrorMessage)
{
	FString* TimelineModeValue = InModule.GetScriptData()->ScriptMetaData.Find(TimelineModeKey);

	if (TimelineModeValue == nullptr)
	{
		OutErrorMessage = LOCTEXT("TimelineModeMissing", "Module script missing 'TimelineMode' meta data.");
		return false;
	}

	if (*TimelineModeValue == TimeRangeTimelineModeValue)
	{
		return TryAddTimeRangeModule(InModule, OutErrorMessage);
	}
	else if (*TimelineModeValue == KeyTimelineModeValue)
	{
		return TryAddKeyModule(InModule, OutErrorMessage);
	}
	else
	{
		OutErrorMessage = FText::Format(LOCTEXT("InvalidTimelineModeFormat", "{0} is not a valid value for TimelineMode.  Must be {1} or {2}."),
			FText::FromString(*TimelineModeValue), FText::FromString(TimeRangeTimelineModeValue), FText::FromString(KeyTimelineModeValue));
		return false;
	}
}

bool UMovieSceneNiagaraEmitterSection::TryAddTimeRangeModule(UNiagaraNodeFunctionCall& InModule, FText& OutErrorMessage)
{
	if (SectionTimingModule.IsValid())
	{
		OutErrorMessage = LOCTEXT("CanNotBeShared", "Only one module can use the time range mode.");
		return false;
	}
	
	const FVersionedNiagaraEmitterData* EmitterData = GetEmitterHandleViewModel()->GetEmitterHandle()->GetEmitterData();
	if (EmitterData == nullptr)
	{
		OutErrorMessage = LOCTEXT("EmiterNotSupport", "This emitter type is not supported for this operation.");
		return false;
	}

	SectionTimingModule = &InModule;
	UNiagaraScript* EmitterUpdateScript = EmitterData->GetScript(ENiagaraScriptUsage::EmitterUpdateScript, FGuid());
	UNiagaraScript* SystemUpdateScript = GetSystemViewModel().GetSystem().GetSystemUpdateScript();
	TArray<UNiagaraScript*> BinderDependentScripts;
	BinderDependentScripts.Add(SystemUpdateScript);
	FCompileConstantResolver EmitterUpdateConstantResolver = FCompileConstantResolver(GetEmitterHandleViewModel()->GetEmitterHandle()->GetInstance(), EmitterUpdateScript->GetUsage());

	FText UseStartTimeErrorText;
	bool bUseStartTimeIsRequired = false;
	bool bUseStartTime = true;
	
	if (UseStartTimeBinder.TryBind(EmitterUpdateScript, BinderDependentScripts, EmitterUpdateConstantResolver, GetEmitterHandleViewModel()->GetEmitterHandle()->GetUniqueInstanceName(), SectionTimingModule.Get(),
		InputUsageKey, UseStartTimeInputUsageValue, FNiagaraTypeDefinition::GetBoolDef(), bUseStartTimeIsRequired, UseStartTimeErrorText) == false)
	{
		OutErrorMessage = FText::Format(LOCTEXT("UseStartTimeErrorFormat", "Failed to bind 'use start time' for module.  Message: {0}"), UseStartTimeErrorText);
		return false;
	}
	else
	{
		bUseStartTime = UseStartTimeBinder.IsValid() && UseStartTimeBinder.GetValue<bool>();
	}

	FText StartTimeErrorText;
	bool bStartTimeIsRequired = bUseStartTime;
	if (StartTimeBinder.TryBind(EmitterUpdateScript, BinderDependentScripts, EmitterUpdateConstantResolver, GetEmitterHandleViewModel()->GetEmitterHandle()->GetUniqueInstanceName(), SectionTimingModule.Get(),
		InputUsageKey, StartTimeInputUsageValue, FNiagaraTypeDefinition::GetFloatDef(), bStartTimeIsRequired, StartTimeErrorText) == false)
	{
		OutErrorMessage = FText::Format(LOCTEXT("StartTimeErrorFormat", "Failed to bind 'start time' for module.  Message: {0}"), StartTimeErrorText);
		return false;
	}

	FText LengthErrorText;
	bool bLengthIsRequired = true;
	if (LengthBinder.TryBind(EmitterUpdateScript, BinderDependentScripts, EmitterUpdateConstantResolver, GetEmitterHandleViewModel()->GetEmitterHandle()->GetUniqueInstanceName(), SectionTimingModule.Get(),
		InputUsageKey, LengthInputUsageValue, FNiagaraTypeDefinition::GetFloatDef(), bLengthIsRequired, LengthErrorText) == false)
	{
		OutErrorMessage = FText::Format(LOCTEXT("LengthErrorFormat", "Failed to bind 'length' for module.  Message: {0}"), LengthErrorText);
		return false;
	}

	FText NumLoopsErrorText;
	bool bNumLoopsIsRequired = false;
	if (NumLoopsBinder.TryBind(EmitterUpdateScript, BinderDependentScripts, EmitterUpdateConstantResolver, GetEmitterHandleViewModel()->GetEmitterHandle()->GetUniqueInstanceName(), SectionTimingModule.Get(),
		InputUsageKey, NumLoopsInputUsageValue, FNiagaraTypeDefinition::GetIntDef(), bNumLoopsIsRequired, NumLoopsErrorText) == false)
	{
		OutErrorMessage = FText::Format(LOCTEXT("NumLoopsErrorFormat", "Failed to bind 'num loops' for module.  Message: {0}"), NumLoopsErrorText);
		return false;
	}

	FText StartTimeIncludedInFirstLoopOnlyErrorText;
	bool bStartTimeIncludedInFirstLoopOnlyIsRequired = false;
	if (StartTimeIncludedInFirstLoopOnlyBinder.TryBind(EmitterUpdateScript, BinderDependentScripts, EmitterUpdateConstantResolver, GetEmitterHandleViewModel()->GetEmitterHandle()->GetUniqueInstanceName(), SectionTimingModule.Get(),
		InputUsageKey, StartTimeIncludedInFirstLoopOnlyInputUsageValue, FNiagaraTypeDefinition::GetBoolDef(), bStartTimeIncludedInFirstLoopOnlyIsRequired, StartTimeIncludedInFirstLoopOnlyErrorText) == false)
	{
		OutErrorMessage = FText::Format(LOCTEXT("StartTimeIncludedInFirstLoopOnlyErrorFormat", "Failed to bind 'start time included in first loop only' for module.  Message: {0}"), StartTimeIncludedInFirstLoopOnlyErrorText);
		return false;
	}

	return true;
}

bool UMovieSceneNiagaraEmitterSection::TryAddKeyModule(UNiagaraNodeFunctionCall& InModule, FText& OutErrorMessage)
{
	FModuleAndBinders ModuleAndBinders;
	if (TrySetupModuleAndBinders(InModule, ModuleAndBinders, OutErrorMessage) == false)
	{
		return false;
	}

	FChannelAndModules* ChannelAndModule = ChannelsAndModules.FindByPredicate([&](FChannelAndModules& ChannelAndModules) { return ChannelAndModules.KeyedScript == InModule.FunctionScript; });
	if (ChannelAndModule == nullptr)
	{
		ChannelAndModule = &ChannelsAndModules.AddDefaulted_GetRef();
		ChannelAndModule->KeyedScript = InModule.FunctionScript;
		ChannelAndModule->ValueInputName = ModuleAndBinders.ValueBinder.GetInputName();
		ChannelAndModule->ValueInputType = ModuleAndBinders.ValueBinder.GetInputType();
		ReconstructChannelProxy();
	}

	ChannelAndModule->ModulesAndBinders.Add(ModuleAndBinders);
	return true;
}

bool UMovieSceneNiagaraEmitterSection::TrySetupModuleAndBinders(UNiagaraNodeFunctionCall& InModule, FModuleAndBinders& InOutModuleAndBinders, FText& OutErrorMessage)
{
	const FVersionedNiagaraEmitterData* EmitterData = GetEmitterHandleViewModel()->GetEmitterHandle()->GetEmitterData();
	if (EmitterData == nullptr)
	{
		OutErrorMessage = LOCTEXT("EmiterNotSupport", "This emitter type is not supported for this operation.");
		return false;
	}

	UNiagaraScript* EmitterUpdateScript = EmitterData->GetScript(ENiagaraScriptUsage::EmitterUpdateScript, FGuid());
	UNiagaraScript* SystemUpdateScript = GetSystemViewModel().GetSystem().GetSystemUpdateScript();
	TArray<UNiagaraScript*> BinderDependentScripts;
	BinderDependentScripts.Add(SystemUpdateScript);
	FCompileConstantResolver EmitterUpdateConstantResolver = FCompileConstantResolver(GetEmitterHandleViewModel()->GetEmitterHandle()->GetInstance(), EmitterUpdateScript->GetUsage());

	InOutModuleAndBinders.Module = &InModule;

	bool bRequired = true;

	FText TimeErrorText;
	if (InOutModuleAndBinders.TimeBinder.TryBind(EmitterUpdateScript, BinderDependentScripts, EmitterUpdateConstantResolver, GetEmitterHandleViewModel()->GetEmitterHandle()->GetUniqueInstanceName(), &InModule,
		InputUsageKey, KeyTimeInputUsageValue, FNiagaraTypeDefinition::GetFloatDef(), bRequired, TimeErrorText) == false)
	{
		OutErrorMessage = FText::Format(LOCTEXT("TimeErrorFormat", "Failed to bind 'time' for module.\nMessage: {0}"), TimeErrorText);
		return false;
	}

	FText ValueErrorText;
	if (InOutModuleAndBinders.ValueBinder.TryBind(EmitterUpdateScript, BinderDependentScripts, EmitterUpdateConstantResolver, GetEmitterHandleViewModel()->GetEmitterHandle()->GetUniqueInstanceName(), &InModule,
		InputUsageKey, KeyValueInputUsageValue, TOptional<FNiagaraTypeDefinition>(), bRequired, ValueErrorText) == false)
	{
		OutErrorMessage = FText::Format(LOCTEXT("ValueErrorFormat", "Failed to bind 'value' for module.\nMessage: {0}"), ValueErrorText);
		return false;
	}

	return true;
}

TSharedRef<ISequencerSection> UMovieSceneNiagaraEmitterSection::MakeSectionInterface()
{
	return MakeShared<FNiagaraEmitterSection>(*this);
}

void UMovieSceneNiagaraEmitterSection::UpdateSectionFromModules(const FFrameRate& InTickResolution)
{
	UpdateSectionFromTimeRangeModule(InTickResolution);
	for (FChannelAndModules& ChannelAndModules : ChannelsAndModules)
	{
		UpdateSectionFromKeyModules(ChannelAndModules, InTickResolution);
	}
}

void UMovieSceneNiagaraEmitterSection::UpdateSectionFromTimeRangeModule(const FFrameRate& InTickResolution)
{
	bool bUseStartTime = UseStartTimeBinder.IsValid() == false || UseStartTimeBinder.GetValue<bool>();
	if ((bUseStartTime == false || StartTimeBinder.IsValid()) && LengthBinder.IsValid())
	{
		float ModuleStartTime = bUseStartTime ? StartTimeBinder.GetValue<float>() : 0.0f;
		float ModuleLength = LengthBinder.GetValue<float>();

		if (ModuleLength < 0)
		{
			// TODO: Add ui support for this issue rather than a log error.
			UE_LOGF(LogNiagaraEditor, Error, "Invalid length in niagara editor timeline.  Bound Module: %ls Bound Input: %ls",
				LengthBinder.GetFunctionCallNode() != nullptr ? *LengthBinder.GetFunctionCallNode()->GetFunctionName() : TEXT("Unknown"),
				*LengthBinder.GetInputName().ToString());
			ModuleLength = 0;
		}

		FFrameNumber StartFrame = (ModuleStartTime * InTickResolution).RoundToFrame();
		FFrameNumber EndFrame = ((ModuleStartTime + ModuleLength) * InTickResolution).RoundToFrame();
		if (EndFrame < StartFrame)
		{
			// The frame value has overflowed and is negative so clamp to the max frame.
			// TODO: Add ui support for this issue rather than a log error.
			UE_LOGF(LogNiagaraEditor, Error, "Invalid length in niagara editor timeline.  Bound Module: %ls Bound Input: %ls",
				LengthBinder.GetFunctionCallNode() != nullptr ? *LengthBinder.GetFunctionCallNode()->GetFunctionName() : TEXT("Unknown"),
				*LengthBinder.GetInputName().ToString());
			EndFrame.Value = TNumericLimits<int32>::Max();
		}
		
		SetRange(TRange<FFrameNumber>(StartFrame, EndFrame));
	}
	else
	{
		SetRange(TRange<FFrameNumber>(
			TRangeBound<FFrameNumber>(FFrameNumber(0)),
			TRangeBound<FFrameNumber>()));
	}

	NumLoops = NumLoopsBinder.IsValid() ?
		NumLoopsBinder.GetValue<int32>() : 1;
	bStartTimeIncludedInFirstLoopOnly = StartTimeIncludedInFirstLoopOnlyBinder.IsValid() ?
		StartTimeIncludedInFirstLoopOnlyBinder.GetValue<bool>() : false;
}

void UMovieSceneNiagaraEmitterSection::UpdateSectionFromKeyModules(FChannelAndModules& ChannelAndModules, const FFrameRate& InTickResolution)
{
	TMovieSceneChannelData<FNiagaraEmitterSectionKey> ChannelData = ChannelAndModules.Channel.GetData();
	ChannelData.Reset();
	for (FModuleAndBinders& ModuleAndBinders : ChannelAndModules.ModulesAndBinders)
	{
		checkf(ModuleAndBinders.TimeBinder.IsValid() && ModuleAndBinders.ValueBinder.IsValid(), TEXT("Input binders are no longer valid"));
		float Time = ModuleAndBinders.TimeBinder.GetValue<float>();

		TArray<uint8> Value = ModuleAndBinders.ValueBinder.GetData();
		FNiagaraVariable KeyVariable(ChannelAndModules.ValueInputType, ChannelAndModules.ValueInputName);
		KeyVariable.SetData(Value.GetData());

		FNiagaraEmitterSectionKey ModuleKey;
		ModuleKey.ModuleId = ModuleAndBinders.Module->NodeGuid;
		ModuleKey.Value = KeyVariable;

		// TODO: Support relative and absolute time modes, this currently assumes the module time is relative to the start time of the section.
		FFrameNumber KeyFrameNumber = GetInclusiveStartFrame() + (Time * InTickResolution).RoundToFrame();
		ChannelData.AddKey(KeyFrameNumber, ModuleKey);
	}
}

void UMovieSceneNiagaraEmitterSection::UpdateModulesFromSection(const FFrameRate& InTickResolution)
{
	UpdateTimeRangeModuleFromSection(InTickResolution);
	for (FChannelAndModules& ChannelAndModules : ChannelsAndModules)
	{
		UpdateKeyModulesFromSection(ChannelAndModules, InTickResolution);
	}
}

void UMovieSceneNiagaraEmitterSection::UpdateTimeRangeModuleFromSection(const FFrameRate& InTickResolution)
{
	bool bUseStartTime = UseStartTimeBinder.IsValid() == false || UseStartTimeBinder.GetValue<bool>();
	if(bUseStartTime)
	{
		if(StartTimeBinder.IsValid() && LengthBinder.IsValid())
		{
			float StartTime = (float)InTickResolution.AsSeconds(GetInclusiveStartFrame());
			float EndTime = (float)InTickResolution.AsSeconds(GetExclusiveEndFrame());

			StartTimeBinder.SetValue<float>(StartTime);
			LengthBinder.SetValue<float>(EndTime - StartTime);
		}
	}
	else
	{
		if(LengthBinder.IsValid())
		{
			float StartTime = (float)InTickResolution.AsSeconds(GetInclusiveStartFrame());
			float EndTime = (float)InTickResolution.AsSeconds(GetExclusiveEndFrame());
			float Length = EndTime - StartTime;
			if (StartTime != 0.0f)
			{
				// If we're not using the start time and the start time was changed by sequencer, reset the section bounds and ignore the change.
				FFrameNumber StartFrame = 0;
				FFrameNumber EndFrame = (LengthBinder.GetValue<float>() * InTickResolution).RoundToFrame();
				SetRange(TRange<FFrameNumber>(StartFrame, EndFrame));
			}
			else
			{
				LengthBinder.SetValue<float>(Length);
			}
		}
	}

	if (NumLoopsBinder.IsValid())
	{
		NumLoopsBinder.SetValue<int32>(NumLoops);
	}

	if (StartTimeIncludedInFirstLoopOnlyBinder.IsValid())
	{
		StartTimeIncludedInFirstLoopOnlyBinder.SetValue<bool>(bStartTimeIncludedInFirstLoopOnly);
	}
}

void UMovieSceneNiagaraEmitterSection::UpdateKeyModulesFromSection(FChannelAndModules& ChannelAndModules, const FFrameRate& InTickResolution)
{
	// Collect modules with missing keys, these modules have been deleted, and
	// collect keys which have been synchronized to help with finding newly added keys.
	TArray<UNiagaraNodeFunctionCall*> ModulesWithMissingKeys;
	TSet<int32> SynchronizedKeyIndices;
	for (FModuleAndBinders& ModuleAndBinders : ChannelAndModules.ModulesAndBinders)
	{
		bool bKeyFound = false;
		for (int32 i = 0; i < ChannelAndModules.Channel.GetValues().Num(); i++)
		{
			if (ChannelAndModules.Channel.GetValues()[i].ModuleId == ModuleAndBinders.Module->NodeGuid)
			{
				bKeyFound = true;

				// TODO: Support relative and absolute time modes, this currently assumes the module time is relative to the start time of the section.
				float ModuleTime = (float)InTickResolution.AsSeconds(ChannelAndModules.Channel.GetTimes()[i] - GetInclusiveStartFrame());

				ModuleAndBinders.TimeBinder.SetValue<float>(ModuleTime);
				ModuleAndBinders.ValueBinder.SetData(ChannelAndModules.Channel.GetValues()[i].Value.GetData(), ChannelAndModules.ValueInputType.GetSize());
				SynchronizedKeyIndices.Add(i);
				break;
			}
		}

		if (bKeyFound == false)
		{
			ModulesWithMissingKeys.Add(ModuleAndBinders.Module.Get());
		}
	}

	// Find keys which weren't synchronized with a module these keys were added.
	TArray<uint32> NewKeysIndices;
	for (int32 i = 0; i < ChannelAndModules.Channel.GetValues().Num(); i++)
	{
		if (SynchronizedKeyIndices.Contains(i) == false)
		{
			NewKeysIndices.Add(i);
		}
	}

	// Remove deleted modules.
	for (UNiagaraNodeFunctionCall* ModuleWithMissingKey : ModulesWithMissingKeys)
	{
		ChannelAndModules.ModulesAndBinders.RemoveAll(
			[=](FModuleAndBinders& ModuleAndBinders) { return ModuleAndBinders.Module == ModuleWithMissingKey; });
		FNiagaraStackGraphUtilities::RemoveModuleFromStack(GetSystemViewModel().GetSystem(), GetEmitterHandleViewModel()->GetId(), *ModuleWithMissingKey);
	}

	// Create new modules for new keys.
	const FVersionedNiagaraEmitterData* EmitterData = GetEmitterHandleViewModel()->GetEmitterHandle()->GetEmitterData();
	if (NewKeysIndices.Num() > 0 && EmitterData)
	{
		UNiagaraScript* EmitterUpdateScript = EmitterData->GetScript(ENiagaraScriptUsage::EmitterUpdateScript, FGuid());
		UNiagaraScript* SystemUpdateScript = GetSystemViewModel().GetSystem().GetSystemUpdateScript();
		TArray<UNiagaraScript*> BinderDependentScripts;
		BinderDependentScripts.Add(SystemUpdateScript);

		UNiagaraScriptSource* EmitterUpdateSource = CastChecked<UNiagaraScriptSource>(EmitterUpdateScript->GetLatestSource());
		UNiagaraNodeOutput* EmitterUpdateOutputNode = EmitterUpdateSource->NodeGraph->FindOutputNode(ENiagaraScriptUsage::EmitterUpdateScript);
		
		bool bRequired = true;
		FText TimeErrorText;

		for (int32 NewKeyIndex : NewKeysIndices)
		{
			FFrameNumber NewFrame = ChannelAndModules.Channel.GetTimes()[NewKeyIndex];
			FNiagaraEmitterSectionKey NewKey = ChannelAndModules.Channel.GetValues()[NewKeyIndex];

			UNiagaraNodeFunctionCall* AddedModule = FNiagaraStackGraphUtilities::AddScriptModuleToStack(ChannelAndModules.KeyedScript.Get(), *EmitterUpdateOutputNode);
			FModuleAndBinders AddedModuleAndBinders;
			FText UnusedError;
			TrySetupModuleAndBinders(*AddedModule, AddedModuleAndBinders, UnusedError);

			// TODO: Support relative and absolute time modes, this currently assumes the module time is relative to the start time of the section.
			float NewModuleTime = (float)InTickResolution.AsSeconds(NewFrame - GetInclusiveStartFrame());
			AddedModuleAndBinders.TimeBinder.SetValue<float>(NewModuleTime);

			if (NewKey.ModuleId.IsValid())
			{
				// Keys with valid modules IDs which have not been initialized were created by copying existing module keys so copy their value to the new module.
				AddedModuleAndBinders.ValueBinder.SetData(NewKey.Value.GetData(), NewKey.Value.GetType().GetSize());
			}
			else
			{
				// Keys without valid modules IDs were created new directly by sequencer.  There is no data to copy for the module value input, but the key's
				// value needs to be initialized properly with the correct name, type, and default value from the module.
				NewKey.Value.SetName(ChannelAndModules.ValueInputName);
				NewKey.Value.SetType(ChannelAndModules.ValueInputType);
				NewKey.Value.SetData(AddedModuleAndBinders.ValueBinder.GetData().GetData());
			}

			NewKey.ModuleId = AddedModule->NodeGuid;
			ChannelAndModules.Channel.GetData().UpdateOrAddKey(NewFrame, NewKey);
			ChannelAndModules.ModulesAndBinders.Add(AddedModuleAndBinders);
		}
	}
}

void UMovieSceneNiagaraEmitterSection::ReconstructChannelProxy()
{
	FMovieSceneChannelProxyData Channels;
	int32 ChannelIndex = 0;
	for (FChannelAndModules& ChannelAndModules : ChannelsAndModules)
	{
		FNiagaraParameterHandle InputHandle(ChannelAndModules.ValueInputName);
		FText DisplayName = FText::Format(LOCTEXT("CurveDisplayNameFormat", "{0} - {1}"),
			FText::FromString(FName::NameToDisplayString(ChannelAndModules.KeyedScript->GetName(), false)),
			FText::FromName(InputHandle.GetName()));
		FMovieSceneChannelMetaData ChannelEditorData(ChannelAndModules.ValueInputName, DisplayName);
		ChannelEditorData.SortOrder = ChannelIndex++;
		ChannelEditorData.bCanCollapseToTrack = false;
		Channels.Add(ChannelAndModules.Channel, ChannelEditorData);
	}

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

int32 UMovieSceneNiagaraEmitterSection::GetNumLoops() const
{
	return NumLoops;
}

#undef LOCTEXT_NAMESPACE // MovieSceneNiagaraEmitterTimedSection
