// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sequencer/NiagaraSequence/MovieSceneNiagaraEmitterTrack.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "NiagaraStackFunctionInputBinder.h"
#include "MovieSceneClipboard.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "MovieSceneNiagaraEmitterSection.generated.h"

class FNiagaraEmitterHandleViewModel;
class UNiagaraNodeFunctionCall;
class UNiagaraScript;

/** Defines data for keys in this emitter section. */
USTRUCT()
struct FNiagaraEmitterSectionKey
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ModuleId;

	UPROPERTY(EditAnywhere, Category = "Value")
	FNiagaraVariable Value;
};

namespace MovieSceneClipboard
{
	template<> inline FName GetKeyTypeName<FNiagaraEmitterSectionKey>()
	{
		return "FNiagaraEmitterSectionKey";
	}
}

USTRUCT()
struct FMovieSceneNiagaraEmitterChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	/**
	* Access a mutable interface for this channel's data
	*
	* @return An object that is able to manipulate this channel's data
	*/
	FORCEINLINE TMovieSceneChannelData<FNiagaraEmitterSectionKey> GetData()
	{
		return TMovieSceneChannelData<FNiagaraEmitterSectionKey>(&Times, &Values, this, &KeyHandles);
	}

	/**
	* Access a constant interface for this channel's data
	*
	* @return An object that is able to interrogate this channel's data
	*/
	FORCEINLINE TMovieSceneChannelData<const FNiagaraEmitterSectionKey> GetData() const
	{
		return TMovieSceneChannelData<const FNiagaraEmitterSectionKey>(&Times, &Values);
	}

	/**
	* Const access to this channel's times
	*/
	FORCEINLINE TArrayView<const FFrameNumber> GetTimes() const
	{
		return Times;
	}

	/**
	* Const access to this channel's values
	*/
	FORCEINLINE TArrayView<const FNiagaraEmitterSectionKey> GetValues() const
	{
		return Values;
	}

	/**
	* Evaluate this channel
	*
	* @param InTime     The time to evaluate at
	* @param OutValue   A value to receive the result
	* @return true if the channel was evaluated successfully, false otherwise
	*/
	bool Evaluate(FFrameTime InTime, FNiagaraEmitterSectionKey& OutValue) const;

public:
	// ~ FMovieSceneChannel Interface
	virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	virtual int32 GetNumKeys() const override;
	virtual void Reset() override;
	virtual void Offset(FFrameNumber DeltaPosition) override;

private:
	/** Storage for the key times in the section. */
	TArray<FFrameNumber> Times;

	/** Storage for the key values in the section. */
	TArray<FNiagaraEmitterSectionKey> Values;

	FMovieSceneKeyHandleMap KeyHandles;
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneNiagaraEmitterChannel> : TMovieSceneChannelTraitsBase<FMovieSceneNiagaraEmitterChannel>
{
	enum { SupportsDefaults = false };
};

TSharedPtr<FStructOnScope> GetKeyStruct(TMovieSceneChannelHandle<FMovieSceneNiagaraEmitterChannel> Channel, FKeyHandle InHandle);

/** 
 *	Niagara editor movie scene section; represents one emitter in the timeline
 */
UCLASS()
class UMovieSceneNiagaraEmitterSection : public UMovieSceneNiagaraEmitterSectionBase
{
	GENERATED_BODY()

private:
	struct FModuleAndBinders
	{
		TWeakObjectPtr<UNiagaraNodeFunctionCall> Module;
		FNiagaraStackFunctionInputBinder TimeBinder;
		FNiagaraStackFunctionInputBinder ValueBinder;
	};

	struct FChannelAndModules
	{
		TWeakObjectPtr<UNiagaraScript> KeyedScript;
		FName ValueInputName;
		FNiagaraTypeDefinition ValueInputType;
		TArray<FModuleAndBinders> ModulesAndBinders;
		FMovieSceneNiagaraEmitterChannel Channel;
	};

public:
	virtual bool TryAddModule(UNiagaraNodeFunctionCall& InModule, FText& OutErrorMessage) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface() override;
	virtual void UpdateSectionFromModules(const FFrameRate& InTickResolution) override;
	virtual void UpdateModulesFromSection(const FFrameRate& InTickResolution) override;

	int32 GetNumLoops() const;
	bool LoopLengthIncludesStartTimeOffset() const;

private:
	bool TryAddTimeRangeModule(UNiagaraNodeFunctionCall& InModule, FText& OutErrorMessage);

	bool TryAddKeyModule(UNiagaraNodeFunctionCall& InModule, FText& OutErrorMessage);

	bool TrySetupModuleAndBinders(UNiagaraNodeFunctionCall& Module, FModuleAndBinders& InOutModuleAndBinders, FText& OutErrorMessage);

	void UpdateSectionFromTimeRangeModule(const FFrameRate& InTickResolution);
	void UpdateSectionFromKeyModules(FChannelAndModules& ChannelAndModules, const FFrameRate& InTickResolution);

	void UpdateTimeRangeModuleFromSection(const FFrameRate& InTickResolution);
	void UpdateKeyModulesFromSection(FChannelAndModules& ChannelAndModules, const FFrameRate& InTickResolution);

	void ReconstructChannelProxy();

private:
	UPROPERTY(EditAnywhere, Category = Looping)
	int32 NumLoops;

	UPROPERTY(EditAnywhere, Category = Looping)
	bool bStartTimeIncludedInFirstLoopOnly;

	TWeakObjectPtr<UNiagaraNodeFunctionCall> SectionTimingModule;

	FNiagaraStackFunctionInputBinder UseStartTimeBinder;
	FNiagaraStackFunctionInputBinder StartTimeBinder;
	FNiagaraStackFunctionInputBinder LengthBinder;
	FNiagaraStackFunctionInputBinder NumLoopsBinder;
	FNiagaraStackFunctionInputBinder StartTimeIncludedInFirstLoopOnlyBinder;

	TArray<FChannelAndModules> ChannelsAndModules;
};