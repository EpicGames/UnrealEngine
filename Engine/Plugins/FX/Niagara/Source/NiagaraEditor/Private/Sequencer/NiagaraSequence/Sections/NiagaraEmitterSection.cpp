// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterSection.h"
#include "MovieSceneNiagaraEmitterSection.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"

#include "MVVM/ViewModels/ViewDensity.h"
#include "SequencerSectionPainter.h"
#include "ISectionLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "NiagaraEmitterTimedSection"

FNiagaraEmitterSection::FNiagaraEmitterSection(UMovieSceneNiagaraEmitterSection &InEmitterSection)
{
	EmitterSection = &InEmitterSection;
}

UMovieSceneSection* FNiagaraEmitterSection::GetSectionObject(void)
{
	return EmitterSection.Get();
}

float FNiagaraEmitterSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	return ViewDensity.UniformHeight.Get(20.0f);
}

int32 FNiagaraEmitterSection::OnPaintSection(FSequencerSectionPainter& InPainter) const
{
	return InPainter.PaintSectionBackground();

	// TODO: Fix the looping drawing and interaction
	/*
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = EmitterSection->GetEmitterHandleViewModel();
	if (EmitterHandleViewModel.IsValid() == false)
	{
		return InPainter.LayerId;
	}

	// draw the first run of the emitter
	FSlateDrawElement::MakeBox
	(
		InPainter.DrawElements,
		InPainter.LayerId + 1,
		InPainter.SectionGeometry.ToPaintGeometry(),
		FAppStyle::GetBrush("CurveEd.TimelineArea"),
		ESlateDrawEffect::None,
		FLinearColor(0.3f, 0.3f, 0.6f)
	);

	// draw all loops of the emitter as 'ghosts' of the original section
	float X = InPainter.SectionGeometry.AbsolutePosition.X;
	float GeomW = InPainter.SectionGeometry.GetLocalSize().X;
	int32 NumLoops = EmitterSection->GetNumLoops();
	for (int32 Loop = 0; Loop < NumLoops; Loop++)
	{
		FSlateDrawElement::MakeBox
		(
			InPainter.DrawElements,
			InPainter.LayerId + 1,
			InPainter.SectionGeometry.ToOffsetPaintGeometry(FVector2D(GeomW*(Loop + 1), 0.0f)),
			FAppStyle::GetBrush("CurveEd.TimelineArea"),
			ESlateDrawEffect::None,
			FLinearColor(0.3f, 0.3f, 0.6f, 0.25f)
		);
	}
	return InPainter.LayerId;
	*/
}

FText FNiagaraEmitterSection::GetSectionTitle(void) const
{
	if (UMovieSceneNiagaraEmitterSection* SectionObj = EmitterSection.Get())
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleviewModel = SectionObj->GetEmitterHandleViewModel();
		if (EmitterHandleviewModel.IsValid())
		{
			return EmitterHandleviewModel->GetNameText();
		}
	}
	return FText();
}

#undef LOCTEXT_NAMESPACE
