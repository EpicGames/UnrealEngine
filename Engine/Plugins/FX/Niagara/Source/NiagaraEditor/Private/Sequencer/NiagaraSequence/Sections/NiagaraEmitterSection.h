// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"

class UMovieSceneNiagaraEmitterSection;

/**
*	Visual representation of UMovieSceneNiagaraEmitterTimedSection
*/
class FNiagaraEmitterSection : public ISequencerSection
{
public:
	FNiagaraEmitterSection(UMovieSceneNiagaraEmitterSection &InEmitterSection);

	//~ ISequencerSection interface
	virtual UMovieSceneSection* GetSectionObject(void) override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override;
	virtual FText GetSectionTitle(void) const override;
	virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;

private:
	TWeakObjectPtr<UMovieSceneNiagaraEmitterSection> EmitterSection;
};