// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraDistributionIntPropertyCustomization.h"

#include "DetailWidgetRow.h"
#include "Editor.h"
#include "EditorUndoClient.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraParameterMapHistory.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSystem.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Stateless/NiagaraStatelessDistribution.h"
#include "Styling/StyleColors.h"
#include "TypeEditorUtilities/NiagaraIntegerTypeEditorUtilities.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/NiagaraDistributionEditorUtilities.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/INiagaraDistributionAdapter.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"

//#include "Stateless/SNiagaraStatelessExpressionWidget.h"
//#include "NiagaraStatelessExpressionTypeData.h"

#define LOCTEXT_NAMESPACE "NiagaraDistributionIntPropertyCustomization"

class SNiagaraDistributionIntPropertyWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraDistributionIntPropertyWidget) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InPropertyHandle, UObject* InOwnerObject, FNiagaraDistributionRangeInt* InDistribution)
	{
		WeakPropertyHandle	= InPropertyHandle;
		WeakOwnerObject = InOwnerObject;
		Distribution = InDistribution;
		WidgetCustomizationOptions = FNiagaraInputParameterCustomization::MakeFromProperty(InPropertyHandle);

		static const FName UnitsName("Units");
		if (InPropertyHandle->HasMetaData(UnitsName))
		{
			FString UnitString = InPropertyHandle->GetMetaData(UnitsName);
			TOptional<EUnit> PropertyUnit = FUnitConversion::UnitFromString(*UnitString);
			DisplayUnit = PropertyUnit.Get(EUnit::Unspecified);
		}

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 10, 0)
			[
				SNew(SComboButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ContentPadding(FMargin(0))
				.OnGetMenuContent(this, &SNiagaraDistributionIntPropertyWidget::OnGetDistributionModeMenuContent)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(this, &SNiagaraDistributionIntPropertyWidget::GetDistributionModeBrush)
					.ToolTipText(this, &SNiagaraDistributionIntPropertyWidget::GetDistributionModeToolTip)
				]
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(ContentBox, SBox)
				[
					ConstructContentForMode()
				]
			]
		];
	}

	virtual ~SNiagaraDistributionIntPropertyWidget()
	{
	}

	ENiagaraDistributionEditorMode GetDistributionMode() const
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
		UObject* OwnerObject = WeakOwnerObject.Get();
		if (PropertyHandle.IsValid() && OwnerObject && Distribution)
		{
			switch (Distribution->Mode)
			{
				case ENiagaraDistributionMode::Array:			return ENiagaraDistributionEditorMode::Array;
				case ENiagaraDistributionMode::Binding:			return ENiagaraDistributionEditorMode::Binding;
				//case ENiagaraDistributionMode::Expression:		return ENiagaraDistributionEditorMode::Expression;
				case ENiagaraDistributionMode::UniformConstant: return ENiagaraDistributionEditorMode::UniformConstant;
				case ENiagaraDistributionMode::UniformRange:	return ENiagaraDistributionEditorMode::UniformRange;
				default:										break;
			}
		}
		return ENiagaraDistributionEditorMode::UniformConstant;
	}

	void SetDistributionMode(ENiagaraDistributionEditorMode NewMode)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
		UObject* OwnerObject = WeakOwnerObject.Get();
		if (PropertyHandle.IsValid() && OwnerObject && Distribution)
		{
			const FScopedTransaction Transaction(LOCTEXT("SetDistributionMode", "Set distribution mode"));
			OwnerObject->Modify();
			PropertyHandle->NotifyPreChange();

			switch (NewMode)
			{
				case ENiagaraDistributionEditorMode::Array:				Distribution->Mode = ENiagaraDistributionMode::Array; break;
				case ENiagaraDistributionEditorMode::Binding:			Distribution->Mode = ENiagaraDistributionMode::Binding; break;
				//case ENiagaraDistributionEditorMode::Expression:		Distribution->Mode = ENiagaraDistributionMode::Expression; break;
				case ENiagaraDistributionEditorMode::UniformConstant:	Distribution->Mode = ENiagaraDistributionMode::UniformConstant; break;
				case ENiagaraDistributionEditorMode::UniformRange:		Distribution->Mode = ENiagaraDistributionMode::UniformRange; break;
				default:												break;
			}
			ContentBox->SetContent(ConstructContentForMode());

			PropertyHandle->NotifyPostChange(bContinuousChangeActive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
		}
	}

	bool IsDistributionModeSelected(ENiagaraDistributionEditorMode Mode) const
	{
		return Mode == GetDistributionMode();
	}

	TSharedRef<SWidget> OnGetDistributionModeMenuContent()
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		TArray<ENiagaraDistributionEditorMode, TInlineAllocator<3>> SupportedModes;

		TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
		if (PropertyHandle.IsValid())
		{
			const FName DisableBindingDistributionName("DisableBindingDistribution");
			if (!PropertyHandle->HasMetaData(DisableBindingDistributionName))
			{
				SupportedModes.Add(ENiagaraDistributionEditorMode::Binding);
				//const FNiagaraStatelessExpressionTypeData& TypeData = FNiagaraStatelessExpressionTypeData::GetTypeData(Distribution->GetBindingTypeDef());
				//if (TypeData.IsValid())
				//{
				//	SupportedModes.Add(ENiagaraDistributionEditorMode::Expression);
				//}
			}
			const FName ArrayBindingDistributionName("DisableArrayDistribution");
			if (!PropertyHandle->HasMetaData(ArrayBindingDistributionName))
			{
				//-TODO: Array support for ints
				//SupportedModes.Add(ENiagaraDistributionEditorMode::Array);
			}
			SupportedModes.Add(ENiagaraDistributionEditorMode::UniformConstant);
			SupportedModes.Add(ENiagaraDistributionEditorMode::UniformRange);
		}

		for (ENiagaraDistributionEditorMode SupportedMode : SupportedModes)
		{
			MenuBuilder.AddMenuEntry(
				FNiagaraDistributionEditorUtilities::DistributionModeToDisplayName(SupportedMode),
				FNiagaraDistributionEditorUtilities::DistributionModeToToolTipText(SupportedMode),
				FNiagaraDistributionEditorUtilities::DistributionModeToIcon(SupportedMode),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraDistributionIntPropertyWidget::SetDistributionMode, SupportedMode),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SNiagaraDistributionIntPropertyWidget::IsDistributionModeSelected, SupportedMode)
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		return MenuBuilder.MakeWidget();
	}

	const FSlateBrush* GetDistributionModeBrush() const
	{
		return FNiagaraDistributionEditorUtilities::DistributionModeToIconBrush(GetDistributionMode());
	}

	FText GetDistributionModeToolTip() const
	{
		return FNiagaraDistributionEditorUtilities::DistributionModeToToolTipText(GetDistributionMode());
	}

	TSharedRef<SWidget> ConstructContentForMode()
	{
		const ENiagaraDistributionEditorMode Mode = GetDistributionMode();
		if (Mode == ENiagaraDistributionEditorMode::Binding)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					ConstructBindingWidget()
				];
		}
		//-TODO: Support ints
		//else if (Mode == ENiagaraDistributionEditorMode::Array)
		//{
			//return SNew(SHorizontalBox)
			//	+ SHorizontalBox::Slot()
			//	[
			//		ConstructBindingWidget()
			//	];
		//}
		//else if (Mode == ENiagaraDistributionEditorMode::Expression)
		//{
		//	return SNew(SHorizontalBox)
		//		+ SHorizontalBox::Slot()
		//		[
		//			ConstructExpressionWidget()
		//		];
		//}
		else if (Mode == ENiagaraDistributionEditorMode::UniformConstant)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					ConstructCustomizableWidget(0)
				];
		}
		else if (Mode == ENiagaraDistributionEditorMode::UniformRange)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					ConstructIntWidget(0, LOCTEXT("MinLabel", "Min"))
				]
				+ SHorizontalBox::Slot()
				[
					ConstructIntWidget(1, LOCTEXT("MaxLabel", "Max"))
				];
		}
		return SNullWidget::NullWidget;
	}

	TSharedRef<SWidget> ConstructIntWidget(int32 ValueIndex, const FText& LabelText)
	{
		TSharedRef<SWidget> LabelWidget = SNullWidget::NullWidget;
		if ( !LabelText.IsEmpty() )
		{
			LabelWidget =
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(LabelText);
		}

		return SNew(SNumericEntryBox<int32>)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			.Value(this, &SNiagaraDistributionIntPropertyWidget::GetValue, ValueIndex)
			.OnValueChanged(this, &SNiagaraDistributionIntPropertyWidget::ValueChanged, ValueIndex)
			.OnValueCommitted(this, &SNiagaraDistributionIntPropertyWidget::ValueCommitted, ValueIndex)
			.OnBeginSliderMovement(this, &SNiagaraDistributionIntPropertyWidget::BeginValueSliderMovement)
			.OnEndSliderMovement(this, &SNiagaraDistributionIntPropertyWidget::EndValueSliderMovement)
			.AllowSpin(true)
			.MinValue(TOptional<int32>())
			.MaxValue(TOptional<int32>())
			.MinSliderValue(TOptional<int32>())
			.MaxSliderValue(TOptional<int32>())
			.BroadcastValueChangesPerKey(false)
			.TypeInterface(MakeShareable(new TNumericUnitTypeInterface<int32>(DisplayUnit)))
			.LabelVAlign(EVerticalAlignment::VAlign_Center)
			.MinDesiredValueWidth(30)
			.Label()
			[
				LabelWidget
			];
	}

	TSharedRef<SWidget> ConstructCustomizableWidget(int32 ValueIndex)
	{
		return SNew(SNiagaraIntegerParameterEditor, DisplayUnit, WidgetCustomizationOptions)
			.Value(this, &SNiagaraDistributionIntPropertyWidget::GetValueInt, ValueIndex)
			.OnValueChanged(this, &SNiagaraDistributionIntPropertyWidget::ValueChanged, ValueIndex)
			.OnBeginValueChange(this, &SNiagaraDistributionIntPropertyWidget::BeginValueSliderMovement)
			.OnEndValueChange(this, &SNiagaraDistributionIntPropertyWidget::EndValueSliderMovement);
	}

	TOptional<int32> GetValue(int32 ValueIndex) const
	{
		TOptional<int32> Value;
		TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
		UObject* OwnerObject = WeakOwnerObject.Get();
		if (PropertyHandle.IsValid() && OwnerObject && Distribution)
		{
			Value = ValueIndex == 0  ? Distribution->Min : Distribution->Max;
		}
		return Value;
	}

	int32 GetValueInt(int32 ValueIndex) const
	{
		TOptional<int32> Value = GetValue(ValueIndex);
		return Value.Get(0);
	}

	void ValueChanged(int32 Value, int32 ValueIndex)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
		UObject* OwnerObject = WeakOwnerObject.Get();
		if (PropertyHandle.IsValid() && OwnerObject && Distribution)
		{
			const FScopedTransaction Transaction(LOCTEXT("SetDistributionValue", "Set distribution value"));
			OwnerObject->Modify();
			PropertyHandle->NotifyPreChange();

			Distribution->Min = (ValueIndex == 0) || Distribution->Mode == ENiagaraDistributionMode::UniformConstant ? Value : Distribution->Min;
			Distribution->Max = (ValueIndex != 0) || Distribution->Mode == ENiagaraDistributionMode::UniformConstant ? Value : Distribution->Max;

			PropertyHandle->NotifyPostChange(bContinuousChangeActive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
		}
	}

	void ValueCommitted(int32 Value, ETextCommit::Type CommitInfo, int32 ValueIndex)
	{
		if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
		{
			ValueChanged(Value, ValueIndex);
		}
	}

	void BeginValueSliderMovement()
	{
		bContinuousChangeActive = true;
	}

	void EndValueSliderMovement(int32 Value)
	{
		bContinuousChangeActive = false;
	}
	
	//TSharedRef<SWidget> ConstructExpressionWidget()
	//{
	//	//-TODO: Not supported currently
	//}

	TSharedRef<SWidget> ConstructBindingWidget()
	{
		return SNew(SComboButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ContentPadding(FMargin(0))
			.OnGetMenuContent(this, &SNiagaraDistributionIntPropertyWidget::OnGetBindingMenuContent)
			.ButtonContent()
			[
				SNew(SNiagaraParameterName)
				.ParameterName(this, &SNiagaraDistributionIntPropertyWidget::GetBindingValueName)
				.IsReadOnly(true)
			];
	}

	virtual TArray<FNiagaraVariableBase> GetAvailableBindings() const
	{
		TArray<FNiagaraVariableBase> AvailableBindings;
		
		UObject* OwnerObject = WeakOwnerObject.Get();
		UNiagaraSystem* OwnerSystem = OwnerObject ? OwnerObject->GetTypedOuter<UNiagaraSystem>() : nullptr;
		const FNiagaraTypeDefinition AllowedTypeDef = Distribution->GetBindingTypeDef();
		if (OwnerSystem && AllowedTypeDef.IsValid())
		{
			if (UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(OwnerSystem->GetSystemUpdateScript()->GetLatestSource()))
			{
				TArray<FNiagaraParameterMapHistory> Histories = UNiagaraNodeParameterMapBase::GetParameterMaps(Source->NodeGraph);
				for (const FNiagaraParameterMapHistory& History : Histories)
				{
					for (const FNiagaraVariable& Variable : History.Variables)
					{
						if (Variable.GetType() == AllowedTypeDef && Variable.IsInNameSpace(FNiagaraConstants::SystemNamespaceString))
						{
							AvailableBindings.Add(Variable);
						}
					}
				}
			}

			for (const FNiagaraVariableBase& Variable : OwnerSystem->GetExposedParameters().ReadParameterVariables())
			{
				if (Variable.GetType() == AllowedTypeDef)
				{
					AvailableBindings.Add(Variable);
				}
			}
		}
		return AvailableBindings;
	}

	TSharedRef<SWidget> OnGetBindingMenuContent()
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		const TArray<FNiagaraVariableBase> AvailableBindings = GetAvailableBindings();
		if (AvailableBindings.Num() > 0)
		{
			for (const FNiagaraVariableBase& Variable : AvailableBindings)
			{
				TSharedRef<SWidget> Widget = SNew(SNiagaraParameterName)
					.ParameterName(Variable.GetName())
					.IsReadOnly(true);

				MenuBuilder.AddMenuEntry(
					FUIAction(
						FExecuteAction::CreateSP(this, &SNiagaraDistributionIntPropertyWidget::SetBindingValue, Variable)
					),
					Widget
				);
			}
		}
		else
		{
			const FNiagaraTypeDefinition AllowedTypeDef = Distribution->GetBindingTypeDef();
			if (AllowedTypeDef.IsValid())
			{
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("NoBindings", "No bindings of type '{0}' found"), AllowedTypeDef.GetNameText()),
					LOCTEXT("NoBindingsDesc", "No bindings for the property type were found."),
					FSlateIcon(),
					FUIAction()
				);
			}
			else
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("InvalidTypeDef", "Bindings Unavailable"),
					LOCTEXT("InvalidTypeDefDesc", "Bindings can not be set on this property."),
					FSlateIcon(),
					FUIAction()
				);
			}
		}

		return MenuBuilder.MakeWidget();
	}

	FName GetBindingValueName() const
	{
		UObject* OwnerObject = WeakOwnerObject.Get();
		return OwnerObject ? Distribution->ParameterBinding.GetName() : NAME_None;
	}

	void SetBindingValue(FNiagaraVariableBase Binding)
	{
		UObject* OwnerObject = WeakOwnerObject.Get();
		TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
		if (OwnerObject == nullptr || !PropertyHandle.IsValid())
		{
			return;
		}

		const FScopedTransaction Transaction(LOCTEXT("SetBinding", "Set binding value"));
		OwnerObject->Modify();
		PropertyHandle->NotifyPreChange();

		Distribution->ParameterBinding = Binding;

		PropertyHandle->NotifyPostChange(bContinuousChangeActive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
	}

private:
	TWeakPtr<IPropertyHandle>		WeakPropertyHandle;
	TWeakObjectPtr<>			WeakOwnerObject;
	FNiagaraDistributionRangeInt*	Distribution = nullptr;
	bool							bContinuousChangeActive = false;
	TSharedPtr<SBox>				ContentBox;
	EUnit							DisplayUnit = EUnit::Unspecified;
	FNiagaraInputParameterCustomization WidgetCustomizationOptions;
};

TSharedRef<IPropertyTypeCustomization> FNiagaraDistributionIntPropertyCustomization::MakeIntInstance(UObject* OptionalOuter)
{
	return MakeShared<FNiagaraDistributionIntPropertyCustomization>(OptionalOuter);
}

void FNiagaraDistributionIntPropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	UObject* OwnerObject = nullptr;
	FNiagaraDistributionRangeInt* Distribution = nullptr;
	{
		TArray<UObject*> OuterObjects;
		if (UObject* OuterObject = WeakOuterObject.Get())
		{
			TArray<TSharedPtr<FStructOnScope>> OutStructOnScopes;
			PropertyHandle->GetOuterStructs(OutStructOnScopes);
			if (OutStructOnScopes.Num() == 1)
			{
				OuterObjects.Add(OuterObject);
			}
		}
		else
		{
			PropertyHandle->GetOuterObjects(OuterObjects);
		}

		if (OuterObjects.Num() == 1)
		{
			if (FProperty* Property = PropertyHandle->GetProperty())
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(Property);
				if (StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(FNiagaraDistributionRangeInt::StaticStruct()))
				{
					void* ValuePtr = nullptr;
					if (PropertyHandle->GetValueData(ValuePtr) == FPropertyAccess::Success)
					{
						OwnerObject = OuterObjects[0];
						Distribution = static_cast<FNiagaraDistributionRangeInt*>(ValuePtr);
					}
				}
			}
		}
	}

	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
	HeaderRow.ValueContent()
	[
		SNew(SNiagaraDistributionIntPropertyWidget, PropertyHandle, OwnerObject, Distribution)
	];
}

void FNiagaraDistributionIntPropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

#undef LOCTEXT_NAMESPACE
