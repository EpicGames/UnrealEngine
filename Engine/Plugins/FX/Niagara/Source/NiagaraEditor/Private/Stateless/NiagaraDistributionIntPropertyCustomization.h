// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class INiagaraDistributionAdapter;

class FNiagaraDistributionIntPropertyCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeIntInstance(UObject* OptionalOuter);
	static TSharedRef<IPropertyTypeCustomization> MakeIntInstance() { return MakeIntInstance(nullptr); }

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	FNiagaraDistributionIntPropertyCustomization(UObject* InOuterObject) : WeakOuterObject(InOuterObject) { }

private:
	TWeakObjectPtr<UObject> WeakOuterObject;
};
