// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraDistributionPropertyCustomization.h"

#include "DetailWidgetRow.h"
#include "Editor.h"
#include "EditorUndoClient.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/MessageDialog.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Stateless/NiagaraStatelessDistribution.h"
#include "Styling/StyleColors.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/NiagaraDistributionEditorUtilities.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNiagaraDistributionEditor.h"

#include "NiagaraEditorUtilities.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraParameterMapHistory.h"
#include "NiagaraScriptSource.h"
#include "NiagaraStatelessExpressionTypeData.h"
#include "NiagaraSystem.h"

#define LOCTEXT_NAMESPACE "NiagaraDistributionPropertyCustomization"

class FNiagaraDistributionAdapter : public INiagaraDistributionAdapter, public TSharedFromThis<FNiagaraDistributionAdapter>
{
public:
	FNiagaraDistributionAdapter()
		: PropertyHandle(nullptr)
		, OwnerObjectWeak(nullptr)
		, SourceNumChannels(INDEX_NONE)
	{
	}

	void Initialize(TSharedPtr<IPropertyHandle> InPropertyHandle, UObject* InOwningObject, FNiagaraDistributionBase* InDistribution, int32 InNumChannels)
	{
		PropertyHandle = InPropertyHandle;
		OwnerObjectWeak = TWeakObjectPtr<>(InOwningObject);
		SourceDistribution = InDistribution;
		SourceNumChannels = InNumChannels;
		if (IsValid())
		{
			// Use UNiagaraNotifyOnChanged if available as it is more robust against copy and paste changes vs SetOnPropertyValueChanged
			if (UNiagaraNotifyOnChanged* OwnerAsNotifyOnChanged = Cast<UNiagaraNotifyOnChanged>(InOwningObject))
			{
				OwnerAsNotifyOnChanged->OnChanged().AddSP(AsShared(), &FNiagaraDistributionAdapter::DistributionPropertyChanged);
			}
			else
			{
				PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(AsShared(), &FNiagaraDistributionAdapter::DistributionPropertyChanged));
			}
			bDisplayAsColor = InDistribution->DisplayAsColor();
			EditorMode = GetDistributionEditorModeFromSourceMode(SourceNumChannels, bDisplayAsColor, SourceDistribution->Mode);
			EditorNumChannels = FNiagaraDistributionEditorUtilities::IsUniform(EditorMode) ? 1 : SourceNumChannels;
			WidgetCustomizationOptions = FNiagaraInputParameterCustomization::MakeFromProperty(InPropertyHandle);

			static const FName UnitsName("Units");
			if (PropertyHandle->HasMetaData(UnitsName))
			{
				FString UnitString = PropertyHandle->GetMetaData(UnitsName);
				TOptional<EUnit> PropertyUnit = FUnitConversion::UnitFromString(*UnitString);
				DisplayUnit = PropertyUnit.Get(EUnit::Unspecified);
			}
		}
	}

	virtual bool IsValid() const override { return PropertyHandle.IsValid() && SourceDistribution != nullptr; }

	virtual int32 GetNumChannels() const override { return EditorNumChannels; }

	virtual FText GetChannelDisplayName(int32 ChannelIndex) const override 
	{
		if (bDisplayAsColor)
		{
			switch (ChannelIndex)
			{
			case 0:
				return LOCTEXT("RChannelName", "R");
			case 1:
				return LOCTEXT("GChannelName", "G");
			case 2:
				return LOCTEXT("BChannelName", "B");
			case 3:
				return LOCTEXT("AChannelName", "A");
			}
		}
		else
		{
			const EAxisList::Type ComponentAxis = FNiagaraEditorUtilities::VectorComponentToAxis(EditorNumChannels, ChannelIndex);
			if (ComponentAxis != EAxisList::None)
			{
				return AxisDisplayInfo::GetAxisDisplayNameShort(ComponentAxis);
			}

			switch(ChannelIndex)
			{
			case 0:
				return LOCTEXT("XChannelName", "X");
			case 1:
				return LOCTEXT("YChannelName", "Y");
			case 2:
				return LOCTEXT("ZChannelName", "Z");
			case 3:
				return LOCTEXT("WChannelName", "W");
			}
		}
		return FText();
	}

	virtual FText GetChannelToolTip(int32 ChannelIndex) const override
	{
		if (bDisplayAsColor == false)
		{
			const EAxisList::Type ComponentAxis = FNiagaraEditorUtilities::VectorComponentToAxis(EditorNumChannels, ChannelIndex);
			if (ComponentAxis != EAxisList::None)
			{
				return AxisDisplayInfo::GetAxisDisplayName(ComponentAxis);
			}
		}

		return FText();
	}

	virtual FSlateColor GetChannelColor(int32 ChannelIndex) const override
	{
		if (EditorNumChannels != 1)
		{
			EAxisList::Type ComponentAxis = bDisplayAsColor ? EAxisList::None : FNiagaraEditorUtilities::VectorComponentToAxis(EditorNumChannels, ChannelIndex);
			if (ComponentAxis != EAxisList::None)
			{
				return AxisDisplayInfo::GetAxisColor(ComponentAxis);
			}

			switch (ChannelIndex)
			{
			case 0:
				return FSlateColor(EStyleColor::AccentRed);
			case 1:
				return FSlateColor(EStyleColor::AccentGreen);
			case 2:
				return FSlateColor(EStyleColor::AccentBlue);
			case 3:
				return FSlateColor(EStyleColor::AccentWhite);
			}
		}
		return FSlateColor(EStyleColor::AccentWhite);
	}

	virtual FNiagaraInputParameterCustomization GetWidgetCustomization() override
	{
		return WidgetCustomizationOptions;
	}

	virtual EUnit GetDisplayUnit() override
	{
		return DisplayUnit;	
	}

	virtual void GetSupportedDistributionModes(TArray<ENiagaraDistributionEditorMode>& OutSupportedModes) const override
	{
		const FProperty* Property = PropertyHandle ? PropertyHandle->GetProperty() : nullptr;
		if (SourceDistribution == nullptr)
		{
			return;
		}

		if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::Array, Property) )
		{
			OutSupportedModes.Add(ENiagaraDistributionEditorMode::Array);
		}

		if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::Binding, Property))
		{
			OutSupportedModes.Add(ENiagaraDistributionEditorMode::Binding);

			const FNiagaraStatelessExpressionTypeData& TypeData = FNiagaraStatelessExpressionTypeData::GetTypeData(SourceDistribution->GetBindingTypeDef());
			if (TypeData.IsValid())
			{
				OutSupportedModes.Add(ENiagaraDistributionEditorMode::Expression);
			}
		}

		if (bDisplayAsColor)
		{
			if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::NonUniformConstant, Property))
			{
				OutSupportedModes.Add(ENiagaraDistributionEditorMode::ColorConstant);
			}
			if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::NonUniformRange, Property))
			{
				OutSupportedModes.Add(ENiagaraDistributionEditorMode::ColorRange);
			}
			if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::ColorGradient, Property))
			{
				OutSupportedModes.Add(ENiagaraDistributionEditorMode::ColorGradient);
			}
			if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::NonUniformCurve, Property))
			{
				OutSupportedModes.Add(ENiagaraDistributionEditorMode::NonUniformCurve);
			}
		}
		else
		{
			const bool bSingleChannel = SourceNumChannels == 1;

			if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::UniformConstant, Property))
			{
				OutSupportedModes.Add(bSingleChannel ? ENiagaraDistributionEditorMode::Constant : ENiagaraDistributionEditorMode::UniformConstant);
			}
			if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::UniformRange, Property))
			{
				OutSupportedModes.Add(bSingleChannel ? ENiagaraDistributionEditorMode::Range : ENiagaraDistributionEditorMode::UniformRange);
			}
			if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::UniformCurve, Property))
			{
				OutSupportedModes.Add(bSingleChannel ? ENiagaraDistributionEditorMode::Curve : ENiagaraDistributionEditorMode::UniformCurve);
			}

			if (!bSingleChannel)
			{
				if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::NonUniformConstant, Property))
				{
					OutSupportedModes.Add(ENiagaraDistributionEditorMode::NonUniformConstant);
				}
				if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::NonUniformRange, Property))
				{
					OutSupportedModes.Add(ENiagaraDistributionEditorMode::NonUniformRange);
				}
				if (SourceDistribution->AllowDistributionMode(ENiagaraDistributionMode::NonUniformCurve, Property))
				{
					OutSupportedModes.Add(ENiagaraDistributionEditorMode::NonUniformCurve);
				}
			}
		}
	}

	ENiagaraDistributionEditorMode GetDistributionEditorModeFromSourceMode(int32 InSourceNumChannels, bool bInDisplayAsColor, ENiagaraDistributionMode InSourceMode, TOptional<ENiagaraDistributionEditorMode> CurrentMode = {})
	{
		if (bInDisplayAsColor)
		{
			switch (InSourceMode)
			{
			case ENiagaraDistributionMode::Array:
				return ENiagaraDistributionEditorMode::Array;
			case ENiagaraDistributionMode::Binding:
				return ENiagaraDistributionEditorMode::Binding;
			case ENiagaraDistributionMode::Expression:
				return ENiagaraDistributionEditorMode::Expression;
			case ENiagaraDistributionMode::NonUniformConstant:
				return ENiagaraDistributionEditorMode::ColorConstant;
			case ENiagaraDistributionMode::NonUniformRange:
				return ENiagaraDistributionEditorMode::ColorRange;
			case ENiagaraDistributionMode::NonUniformCurve:
				return ENiagaraDistributionEditorMode::NonUniformCurve;
			case ENiagaraDistributionMode::ColorGradient:
				return ENiagaraDistributionEditorMode::ColorGradient;
			default:
				return ENiagaraDistributionEditorMode::ColorConstant;
			}
		}
		else if (InSourceNumChannels == 1)
		{
			switch (InSourceMode)
			{
			case ENiagaraDistributionMode::Array:
				return ENiagaraDistributionEditorMode::Array;
			case ENiagaraDistributionMode::Binding:
				return ENiagaraDistributionEditorMode::Binding;
			case ENiagaraDistributionMode::Expression:
				return ENiagaraDistributionEditorMode::Expression;
			case ENiagaraDistributionMode::UniformConstant:
				return ENiagaraDistributionEditorMode::Constant;
			case ENiagaraDistributionMode::UniformRange:
				return ENiagaraDistributionEditorMode::Range;
			case ENiagaraDistributionMode::UniformCurve:
				return ENiagaraDistributionEditorMode::Curve;
			default:
				return ENiagaraDistributionEditorMode::Constant;
			}
		}
		else
		{
			switch (InSourceMode)
			{
			case ENiagaraDistributionMode::Array:
				return ENiagaraDistributionEditorMode::Array;
			case ENiagaraDistributionMode::Binding:
				return ENiagaraDistributionEditorMode::Binding;
			case ENiagaraDistributionMode::Expression:
				return ENiagaraDistributionEditorMode::Expression;
			case ENiagaraDistributionMode::UniformConstant:
				return ENiagaraDistributionEditorMode::UniformConstant;
			case ENiagaraDistributionMode::NonUniformConstant:
				return ENiagaraDistributionEditorMode::NonUniformConstant;
			case ENiagaraDistributionMode::UniformRange:
				return ENiagaraDistributionEditorMode::UniformRange;
			case ENiagaraDistributionMode::NonUniformRange:
				return ENiagaraDistributionEditorMode::NonUniformRange;
			case ENiagaraDistributionMode::UniformCurve:
				return ENiagaraDistributionEditorMode::UniformCurve;
			case ENiagaraDistributionMode::NonUniformCurve:
				return ENiagaraDistributionEditorMode::NonUniformCurve;
			case ENiagaraDistributionMode::ColorGradient:
				return ENiagaraDistributionEditorMode::ColorGradient;
			default:
				return ENiagaraDistributionEditorMode::UniformConstant;
			}
		}
	}

	static ENiagaraDistributionMode GetDistributionSourceModeFromEditorMode(ENiagaraDistributionEditorMode InEditorMode)
	{
		switch (InEditorMode)
		{
		case ENiagaraDistributionEditorMode::Array:
			return ENiagaraDistributionMode::Array;
		case ENiagaraDistributionEditorMode::Binding:
			return ENiagaraDistributionMode::Binding;
		case ENiagaraDistributionEditorMode::Expression:
			return ENiagaraDistributionMode::Expression;
		case ENiagaraDistributionEditorMode::Constant:
		case ENiagaraDistributionEditorMode::UniformConstant:
			return ENiagaraDistributionMode::UniformConstant;
		case ENiagaraDistributionEditorMode::NonUniformConstant:
		case ENiagaraDistributionEditorMode::ColorConstant:
			return ENiagaraDistributionMode::NonUniformConstant;
		case ENiagaraDistributionEditorMode::Range:
		case ENiagaraDistributionEditorMode::UniformRange:
			return ENiagaraDistributionMode::UniformRange;
		case ENiagaraDistributionEditorMode::NonUniformRange:
		case ENiagaraDistributionEditorMode::ColorRange:
			return ENiagaraDistributionMode::NonUniformRange;
		case ENiagaraDistributionEditorMode::Curve:
		case ENiagaraDistributionEditorMode::UniformCurve:
			return ENiagaraDistributionMode::UniformCurve;
		case ENiagaraDistributionEditorMode::NonUniformCurve:
			return ENiagaraDistributionMode::NonUniformCurve;
		case ENiagaraDistributionEditorMode::ColorGradient:
			return ENiagaraDistributionMode::ColorGradient;
		default:
			return ENiagaraDistributionMode::UniformConstant;
		}
	}

	virtual ENiagaraDistributionEditorMode GetDistributionMode() const override { return EditorMode; }

	virtual void SetDistributionMode(ENiagaraDistributionEditorMode InEditorMode) override
	{
		if (InEditorMode == EditorMode)
		{
			return;
		}

		UObject* OwnerObject = OwnerObjectWeak.Get();
		if (OwnerObject == nullptr)
		{
			return;
		}

		const ENiagaraDistributionMode NewSourceMode = GetDistributionSourceModeFromEditorMode(InEditorMode);

		// If the distribution mode is not changing and we are switching to be a color gradient we need to make sure the keys are all normalized (i.e. 0-1 time)
		// if they are not we need to perform a destructive action and trim the keys to 0-1 time range or the widget will not display the keys correctly.
		if (InEditorMode == ENiagaraDistributionEditorMode::ColorGradient && !AreCurveKeyTimesNormalized())
		{
			const EAppReturnType::Type DialogResponse = FMessageDialog::Open(
				EAppMsgType::OkCancel,
				EAppReturnType::Ok,
				LOCTEXT("TrimKeysForColorGradientDesc", "The color gradient editor only allows keys with a time range between 0.0 and 1.0, your existing curve has values outside of this time range.  Would you like to trim the keys to 0.0 to 1.0 time range?"),
				LOCTEXT("TrimKeysForColorGradientTitle", "Trim key times 0.0 to 1.0?")
			);
			if (DialogResponse != EAppReturnType::Ok)
			{
				return;
			}
		}

		TGuardValue<bool> UpdateGuard(bUpdatingDistributionModeInternally, true);
		const int32 NewEditorNumChannels = FNiagaraDistributionEditorUtilities::IsUniform(InEditorMode) ? 1 : SourceNumChannels;

		if (SourceDistribution->Mode != NewSourceMode)
		{
			const FScopedTransaction Transaction(LOCTEXT("SetDistributionModeTransaction", "Set distribution mode"));
			OwnerObject->Modify();
			PropertyHandle->NotifyPreChange();

			MigrateDataFromModeChange(NewSourceMode, NewEditorNumChannels);
			SourceDistribution->Mode = NewSourceMode;

			PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}

		EditorNumChannels = NewEditorNumChannels;
		EditorMode = InEditorMode;
		OnDistributionEditorModeChangedDelegate.Broadcast();
	}

	virtual FSimpleMulticastDelegate& OnDistributionEditorModeChanged() override
	{
		return OnDistributionEditorModeChangedDelegate;
	}

	float GetConstantOrRangeValue(int32 ChannelIndex, int32 ValueIndex) const override
	{
		UObject* OwnerObject = OwnerObjectWeak.Get();
		int32 DataIndex = ValueIndex * EditorNumChannels + ChannelIndex;
		if (OwnerObject != nullptr &&
			SourceDistribution != nullptr &&
			ChannelIndex < EditorNumChannels &&
			DataIndex < SourceDistribution->ChannelConstantsAndRanges.Num())
		{
			return SourceDistribution->ChannelConstantsAndRanges[DataIndex];
		}
		return 0.0f;
	}

	void SetConstantOrRangeValue(int32 ChannelIndex, int32 ValueIndex, float InValue) override
	{
		UObject* OwnerObject = OwnerObjectWeak.Get();
		if (OwnerObject == nullptr ||
			ChannelIndex >= EditorNumChannels ||
			GetConstantOrRangeValue(ChannelIndex, ValueIndex) == InValue)
		{
			return;
		}

		FText TransactionText;
		int32 RequiredValueCount = 0;
		if (FNiagaraDistributionEditorUtilities::IsConstant(GetDistributionMode()))
		{
			TransactionText = LOCTEXT("SetConstantValueTransaction", "Set constant value");
			RequiredValueCount = EditorNumChannels;
		}
		else if (FNiagaraDistributionEditorUtilities::IsRange(GetDistributionMode()))
		{
			TransactionText = ValueIndex == 0
				? LOCTEXT("SetRangeMinValueTransaction", "Set range min value")
				: LOCTEXT("SetRangeMaxValueTransaction", "Set range max value");
			RequiredValueCount = EditorNumChannels * 2;
		}

		if (bContinuousTransactionPending)
		{
			bContinuousTransactionPending = false;
			ContinuousTransactionIndex = GEditor->BeginTransaction(TransactionText);
		}

		TGuardValue<bool> UpdateValueGuard(bUpdatingDistributionValueInternally, true);
		const FScopedTransaction Transaction(TransactionText, ContinuousTransactionIndex.IsSet() == false);

		OwnerObject->Modify();
		PropertyHandle->NotifyPreChange();

		if (SourceDistribution->ChannelConstantsAndRanges.Num() < RequiredValueCount)
		{
			SourceDistribution->ChannelConstantsAndRanges.AddZeroed(RequiredValueCount - SourceDistribution->ChannelConstantsAndRanges.Num());
		}
		else if (SourceDistribution->ChannelConstantsAndRanges.Num() > RequiredValueCount)
		{
			SourceDistribution->ChannelConstantsAndRanges.SetNum(RequiredValueCount);
		}

		SourceDistribution->ChannelConstantsAndRanges[ValueIndex * EditorNumChannels + ChannelIndex] = InValue;
		PropertyHandle->NotifyPostChange(bContinuousChangeActive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
	}

	virtual void SetConstantOrRangeValues(int32 ValueIndex, const TArray<float>& InValues) override
	{
		UObject* OwnerObject = OwnerObjectWeak.Get();
		if (OwnerObject == nullptr || InValues.Num() != EditorNumChannels)
		{
			return;
		}

		bool bAllChannelsTheSame = true;
		for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ++ChannelIndex)
		{
			if (GetConstantOrRangeValue(ChannelIndex, ValueIndex) != InValues[ChannelIndex])
			{
				bAllChannelsTheSame = false;
				break;
			}
		}

		if (bAllChannelsTheSame)
		{
			return;
		}

		FText TransactionText;
		int32 RequiredValueCount = 0;
		if (FNiagaraDistributionEditorUtilities::IsConstant(GetDistributionMode()))
		{
			TransactionText = LOCTEXT("SetConstantValueTransaction", "Set constant value");
			RequiredValueCount = EditorNumChannels;
		}
		else if (FNiagaraDistributionEditorUtilities::IsRange(GetDistributionMode()))
		{
			TransactionText = ValueIndex == 0
				? LOCTEXT("SetRangeMinValueTransaction", "Set range min value")
				: LOCTEXT("SetRangeMaxValueTransaction", "Set range max value");
			RequiredValueCount = EditorNumChannels * 2;
		}

		if (bContinuousTransactionPending)
		{
			bContinuousTransactionPending = false;
			ContinuousTransactionIndex = GEditor->BeginTransaction(TransactionText);
		}

		TGuardValue<bool> UpdateValueGuard(bUpdatingDistributionValueInternally, true);
		const FScopedTransaction Transaction(TransactionText, ContinuousTransactionIndex.IsSet() == false);

		OwnerObject->Modify();
		PropertyHandle->NotifyPreChange();

		if (SourceDistribution->ChannelConstantsAndRanges.Num() < RequiredValueCount)
		{
			SourceDistribution->ChannelConstantsAndRanges.AddZeroed(RequiredValueCount - SourceDistribution->ChannelConstantsAndRanges.Num());
		}
		else if (SourceDistribution->ChannelConstantsAndRanges.Num() > RequiredValueCount)
		{
			SourceDistribution->ChannelConstantsAndRanges.SetNum(RequiredValueCount);
		}

		for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ++ChannelIndex)
		{
			SourceDistribution->ChannelConstantsAndRanges[ValueIndex * EditorNumChannels + ChannelIndex] = InValues[ChannelIndex];
		}
		PropertyHandle->NotifyPostChange(bContinuousChangeActive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
	}

	virtual TConstArrayView<float> GetArrayValues() const override
	{
		return MakeArrayView(SourceDistribution->ChannelConstantsAndRanges);
	}

	virtual void SetArrayValues(TConstArrayView<float> ArrayData) override
	{
		check((ArrayData.Num() % GetNumChannels()) == 0);

		FText TransactionText = LOCTEXT("SetArrayValues", "Set Array Value");
		ExecuteTransaction(
			LOCTEXT("SetArrayValues", "Set Array Value"),
			[&]()
			{
				SourceDistribution->ChannelConstantsAndRanges = ArrayData;
			}
		);
	}

	virtual FNiagaraTypeDefinition GetExpressionTypeDef() const override
	{
		return SourceDistribution->GetBindingTypeDef();
	}

	virtual FInstancedStruct& GetExpressionRoot() const
	{
		return SourceDistribution->ParameterExpression;
	}

	virtual void ExecuteTransaction(FText TransactionText, TFunction<void()> TransactionFunc) override
	{
		if (UObject* OwnerObject = OwnerObjectWeak.Get())
		{
			if (bContinuousTransactionPending)
			{
				bContinuousTransactionPending = false;
				ContinuousTransactionIndex = GEditor->BeginTransaction(TransactionText);
			}

			TGuardValue<bool> UpdateValueGuard(bUpdatingDistributionValueInternally, true);
			const FScopedTransaction Transaction(TransactionText, ContinuousTransactionIndex.IsSet() == false);

			OwnerObject->Modify();
			PropertyHandle->NotifyPreChange();

			TransactionFunc();

			PropertyHandle->NotifyPostChange(bContinuousChangeActive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
		}
	}

	virtual FNiagaraVariableBase GetBindingValue() const
	{
		UObject* OwnerObject = OwnerObjectWeak.Get();
		if (OwnerObject == nullptr)
		{
			return FNiagaraVariableBase();
		}
		return SourceDistribution->ParameterBinding;
	}

	virtual void SetBindingValue(FNiagaraVariableBase Binding)
	{
		ExecuteTransaction(
			LOCTEXT("SetBinding", "Set binding value"),
			[&]()
			{
				SourceDistribution->ParameterBinding = Binding;
			}
		);
	}

	virtual TArray<FNiagaraVariableBase> GetAvailableBindings() const
	{
		TArray<FNiagaraVariableBase> AvailableBindings;
		UObject* OwnerObject = OwnerObjectWeak.Get();
		UNiagaraSystem* OwnerSystem = OwnerObject ? OwnerObject->GetTypedOuter<UNiagaraSystem>() : nullptr;
		const FNiagaraTypeDefinition AllowedTypeDef = SourceDistribution->GetBindingTypeDef();
		if ( OwnerSystem && AllowedTypeDef.IsValid() )
		{
			if (UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(OwnerSystem->GetSystemUpdateScript()->GetLatestSource()))
			{
				TArray<FNiagaraParameterMapHistory> Histories = UNiagaraNodeParameterMapBase::GetParameterMaps(Source->NodeGraph);
				for (const FNiagaraParameterMapHistory& History : Histories)
				{
					for (const FNiagaraVariable& Variable : History.Variables)
					{
						if (Variable.GetType() == AllowedTypeDef && Variable.IsInNameSpace(FNiagaraConstants::SystemNamespaceString))
						{
							AvailableBindings.AddUnique(Variable);
						}
					}
				}
			}

			for (const FNiagaraVariableBase& Variable : OwnerSystem->GetExposedParameters().ReadParameterVariables())
			{
				if (Variable.GetType() == AllowedTypeDef)
				{
					AvailableBindings.AddUnique(Variable);
				}
			}

			TArray<UNiagaraParameterCollection*> AvailableParameterCollections;
			FNiagaraEditorUtilities::GetAvailableParameterCollections(AvailableParameterCollections);
			for (UNiagaraParameterCollection* NPCollection : AvailableParameterCollections)
			{
				for (const FNiagaraVariable& NPCVariable : NPCollection->GetParameters())
				{
					if (NPCVariable.GetType() == AllowedTypeDef)
					{
						AvailableBindings.AddUnique(NPCVariable);
					}
				}
			}
		}
		return AvailableBindings;
	}

	virtual const FRichCurve* GetCurveValue(int32 ChannelIndex) const override
	{
		if (ChannelIndex < EditorNumChannels && ChannelIndex < SourceDistribution->ChannelCurves.Num())
		{
			return &SourceDistribution->ChannelCurves[ChannelIndex];
		}
		return nullptr;
	}

	virtual FRichCurve* GetCurveValue(int32 ChannelIndex) override
	{
		if (ChannelIndex < EditorNumChannels && ChannelIndex < SourceDistribution->ChannelCurves.Num())
		{
			return &SourceDistribution->ChannelCurves[ChannelIndex];
		}
		return nullptr;
	}

	virtual void NotifyCurveValueChangedExternally(int32 ChannelIndex) override
	{
		if (PropertyHandle.IsValid())
		{
			PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}

	virtual void SetCurveValue(int32 ChannelIndex, const FRichCurve& InValue, bool bShouldTransact) override
	{
		UObject* OwnerObject = OwnerObjectWeak.Get();
		const FRichCurve* CurrentValue = GetCurveValue(ChannelIndex);
		if (OwnerObject != nullptr &&
			(CurrentValue == nullptr || *CurrentValue != InValue) &&
			ChannelIndex < EditorNumChannels)
		{
			FText TransactionText = LOCTEXT("SetCurveValueTransaction", "Set curve value");

			if (bContinuousTransactionPending)
			{
				bContinuousTransactionPending = false;
				ContinuousTransactionIndex = GEditor->BeginTransaction(TransactionText);
			}

			TGuardValue<bool> UpdateValueGuard(bUpdatingDistributionValueInternally, true);
			const FScopedTransaction Transaction(TransactionText, ContinuousTransactionIndex.IsSet() == false && bShouldTransact);

			if (bShouldTransact)
			{
				OwnerObject->Modify();
				PropertyHandle->NotifyPreChange();
			}

			const int32 RequiredCurveCount = EditorNumChannels;
			if (SourceDistribution->ChannelCurves.Num() < RequiredCurveCount)
			{
				SourceDistribution->ChannelCurves.AddDefaulted(RequiredCurveCount - SourceDistribution->ChannelCurves.Num());
			}
			else if (SourceDistribution->ChannelCurves.Num() > RequiredCurveCount)
			{
				SourceDistribution->ChannelCurves.SetNum(RequiredCurveCount);
			}

			SourceDistribution->ChannelCurves[ChannelIndex] = InValue;
			if (bShouldTransact)
			{
				PropertyHandle->NotifyPostChange(bContinuousChangeActive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
			}
		}
	}

	virtual bool AllowInterpolationMode() const override
	{
		return SourceDistribution->AllowInterpolationMode(PropertyHandle ? PropertyHandle->GetProperty() : nullptr);
	}

	virtual ENiagaraDistributionInterpolationMode GetInterpolationMode() const override
	{
		return SourceDistribution->GetInterpolationMode();
	}

	virtual void SetInterpolationMode(ENiagaraDistributionInterpolationMode Mode) override
	{
		ExecuteTransaction(
			LOCTEXT("SetInterpolationMode", "Set Interpolation Mode"),
			[&]()
			{
				SourceDistribution->SetInterpolationMode(Mode);
			}
		);
	}

	virtual bool AllowAddressMode() const override
	{
		return SourceDistribution->AllowAddressMode(PropertyHandle ? PropertyHandle->GetProperty() : nullptr);
	}

	virtual ENiagaraDistributionAddressMode GetAddressMode() const override
	{
		return SourceDistribution->GetAddressMode();
	}

	virtual void SetAddressMode(ENiagaraDistributionAddressMode Mode) override
	{
		ExecuteTransaction(
			LOCTEXT("SetAddressMode", "Set Address Mode"),
			[&]()
			{
				SourceDistribution->SetAddressMode(Mode);
			}
		);
	}

	virtual bool AllowLookupValueMode() const override
	{
		return SourceDistribution->AllowLookupValueMode(PropertyHandle ? PropertyHandle->GetProperty() : nullptr);
	}

	virtual const UEnum* GetLookupValueModeEnum() const override
	{
		FProperty* DistributionProperty = PropertyHandle->GetProperty();
		return DistributionProperty != nullptr
			? FNiagaraDistributionBase::GetLookupValueModeEnum(DistributionProperty)
			: nullptr;
	}

	virtual uint8 GetLookupValueMode() const override
	{
		return SourceDistribution->GetLookupValueMode();
	}

	virtual void SetLookupValueMode(uint8 Mode) override
	{
		ExecuteTransaction(
			LOCTEXT("SetLookupValueMode", "Set Lookup Value Mode"),
			[&]()
			{
				SourceDistribution->SetLookupValueMode(Mode);
			}
		);
	}

	virtual void BeginContinuousChange() override
	{
		bContinuousTransactionPending = true;
		bContinuousChangeActive = true;
	}

	virtual void EndContinuousChange() override
	{
		if (ContinuousTransactionIndex.IsSet())
		{
			GEditor->EndTransaction();
		}
		bContinuousTransactionPending = false;
		ContinuousTransactionIndex.Reset();
		bContinuousChangeActive = false;
		PropertyHandle->NotifyFinishedChangingProperties();
	}

	virtual void CancelContinuousChange() override
	{
		if (ContinuousTransactionIndex.IsSet())
		{
			GEditor->CancelTransaction(ContinuousTransactionIndex.GetValue());
		}
		bContinuousTransactionPending = false;
		ContinuousTransactionIndex.Reset();
		bContinuousChangeActive = false;
		PropertyHandle->NotifyFinishedChangingProperties();
	}

	virtual FSimpleMulticastDelegate& OnDistributionValueChanged()
	{
		return OnDistributionValueChangedDelegate;
	}

	virtual void GetOwners(TArray<UObject*>& OutOwners) const
	{
		UObject* OwnerObject = OwnerObjectWeak.Get();
		if (OwnerObject != nullptr)
		{
			OutOwners.AddUnique(OwnerObject);
		}
	}

	virtual void ModifyOwners() override
	{
		UObject* OwnerObject = OwnerObjectWeak.Get();
		if (OwnerObject != nullptr)
		{
			OwnerObject->Modify();
		}
	}

private:
	void DistributionPropertyChanged()
	{
		// Watch for editor mode changes from external sources like copy/paste.
		const ENiagaraDistributionEditorMode NewEditorMode = GetDistributionEditorModeFromSourceMode(SourceNumChannels, bDisplayAsColor, SourceDistribution->Mode, EditorMode);
		if (NewEditorMode != EditorMode && bUpdatingDistributionModeInternally == false)
		{
			EditorMode = NewEditorMode;
			EditorNumChannels = FNiagaraDistributionEditorUtilities::IsUniform(EditorMode) ? 1 : SourceNumChannels;
			OnDistributionEditorModeChangedDelegate.Broadcast();
		}

		if (bUpdatingDistributionValueInternally == false)
		{
			OnDistributionValueChangedDelegate.Broadcast();
		}
	}

	void MigrateDataFromModeChange(ENiagaraDistributionMode NewMode, int32 NewEditorNumChannels)
	{
		if (NewMode == ENiagaraDistributionMode::UniformConstant || NewMode == ENiagaraDistributionMode::NonUniformConstant)
		{
			MigrateDataToConstants(NewEditorNumChannels);
		}
		else if(NewMode == ENiagaraDistributionMode::UniformRange || NewMode == ENiagaraDistributionMode::NonUniformRange)
		{
			MigrateDataToRanges(NewEditorNumChannels);
		}
		else if (NewMode == ENiagaraDistributionMode::UniformCurve || NewMode == ENiagaraDistributionMode::NonUniformCurve)
		{
			MigrateDataToCurves(NewEditorNumChannels);
		}
		else if (NewMode == ENiagaraDistributionMode::ColorGradient)
		{
			MigrateDataToColorGradient();
		}
	}

	void MigrateDataToConstants(int32 NewEditorNumChannels)
	{
		TArray<float> ConstantValues;
		switch (SourceDistribution->Mode)
		{
		case ENiagaraDistributionMode::UniformConstant:
		case ENiagaraDistributionMode::UniformRange:
		case ENiagaraDistributionMode::NonUniformConstant:
		case ENiagaraDistributionMode::NonUniformRange:
			for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ChannelIndex++)
			{
				ConstantValues.Add(GetConstantOrRangeValue(ChannelIndex, 0));
			}
			break;
		case ENiagaraDistributionMode::UniformCurve:
		case ENiagaraDistributionMode::NonUniformCurve:
			for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ChannelIndex++)
			{
				const FRichCurve* Curve = GetCurveValue(ChannelIndex);
				float ConstantValue = Curve != nullptr && Curve->GetNumKeys() > 0 ? Curve->GetFirstKey().Value : 0;
				ConstantValues.Add(ConstantValue);
			}
			break;
		default:
			ConstantValues.Add(0);
		}

		SourceDistribution->ChannelConstantsAndRanges.SetNum(NewEditorNumChannels);
		SourceDistribution->ChannelCurves.Empty();
		for (int32 ChannelIndex = 0; ChannelIndex < NewEditorNumChannels; ChannelIndex++)
		{
			int32 ValueIndex = ChannelIndex < ConstantValues.Num() ? ChannelIndex : 0;
			SourceDistribution->ChannelConstantsAndRanges[ChannelIndex] = ConstantValues[ValueIndex];
		}
	}

	void MigrateDataToRanges(int32 NewEditorNumChannels)
	{
		TArray<float> MinValues;
		TArray<float> MaxValues;
		switch (SourceDistribution->Mode)
		{
		case ENiagaraDistributionMode::UniformConstant:
		case ENiagaraDistributionMode::NonUniformConstant:
			for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ChannelIndex++)
			{
				float ConstantValue = GetConstantOrRangeValue(ChannelIndex, 0);
				MinValues.Add(ConstantValue);
				MaxValues.Add(ConstantValue);
			}
			break;
		case ENiagaraDistributionMode::UniformRange:
		case ENiagaraDistributionMode::NonUniformRange:
			for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ChannelIndex++)
			{
				MinValues.Add(GetConstantOrRangeValue(ChannelIndex, 0));
				MaxValues.Add(GetConstantOrRangeValue(ChannelIndex, 1));
			}
			break;
		case ENiagaraDistributionMode::UniformCurve:
		case ENiagaraDistributionMode::NonUniformCurve:
		case ENiagaraDistributionMode::ColorGradient:
			for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ChannelIndex++)
			{
				const FRichCurve* Curve = GetCurveValue(ChannelIndex);
				if (Curve != nullptr && Curve->GetNumKeys() > 0)
				{
					MinValues.Add(Curve->GetFirstKey().Value);
					MaxValues.Add(Curve->GetLastKey().Value);
				}
				else
				{
					MinValues.Add(0);
					MaxValues.Add(0);
				}
			}
			break;
		default:
			MinValues.Add(0);
			MaxValues.Add(0);
			break;
		}

		SourceDistribution->ChannelConstantsAndRanges.SetNum(NewEditorNumChannels * 2);
		SourceDistribution->ChannelCurves.Empty();
		for (int32 ChannelIndex = 0; ChannelIndex < NewEditorNumChannels; ChannelIndex++)
		{
			int32 MinValueIndex = ChannelIndex < MinValues.Num() ? ChannelIndex : 0;
			SourceDistribution->ChannelConstantsAndRanges[ChannelIndex] = MinValues[MinValueIndex];

			int32 MaxValueIndex = ChannelIndex < MaxValues.Num() ? ChannelIndex : 0;
			SourceDistribution->ChannelConstantsAndRanges[NewEditorNumChannels + ChannelIndex] = MaxValues[MaxValueIndex];
		}
	}

	void MigrateDataToCurves(int32 NewEditorNumChannels)
	{
		TArray<FRichCurve> CurveValues;
		switch (SourceDistribution->Mode)
		{
		case ENiagaraDistributionMode::UniformConstant:
		case ENiagaraDistributionMode::NonUniformConstant:
			for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ChannelIndex++)
			{
				FRichCurve& CurveValue = CurveValues.AddDefaulted_GetRef();
				CurveValue.AddKey(0, GetConstantOrRangeValue(ChannelIndex, 0));
			}
			break;
		case ENiagaraDistributionMode::UniformRange:
		case ENiagaraDistributionMode::NonUniformRange:
			for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ChannelIndex++)
			{
				FRichCurve& CurveValue = CurveValues.AddDefaulted_GetRef();
				CurveValue.AddKey(0, GetConstantOrRangeValue(ChannelIndex, 0));
				CurveValue.AddKey(1, GetConstantOrRangeValue(ChannelIndex, 1));
			}
			break;
		case ENiagaraDistributionMode::UniformCurve:
		case ENiagaraDistributionMode::NonUniformCurve:
		case ENiagaraDistributionMode::ColorGradient:
			for (int32 ChannelIndex = 0; ChannelIndex < EditorNumChannels; ChannelIndex++)
			{
				const FRichCurve* Curve = GetCurveValue(ChannelIndex);
				if (Curve != nullptr)
				{
					CurveValues.Add(*Curve);
				}
				else
				{
					FRichCurve& CurveValue = CurveValues.AddDefaulted_GetRef();
					CurveValue.AddKey(0, 0);
				}
			}
			break;
		default:
			FRichCurve& CurveValue = CurveValues.AddDefaulted_GetRef();
			CurveValue.AddKey(0, 0);
			break;
		}

		SourceDistribution->ChannelCurves.SetNum(NewEditorNumChannels);
		SourceDistribution->ChannelConstantsAndRanges.Empty();
		for (int32 ChannelIndex = 0; ChannelIndex < NewEditorNumChannels; ChannelIndex++)
		{
			int32 ValueIndex = ChannelIndex < CurveValues.Num() ? ChannelIndex : 0;
			SourceDistribution->ChannelCurves[ChannelIndex] = CurveValues[ValueIndex];
		}
	}

	void MigrateDataToColorGradient()
	{
		MigrateDataToCurves(4);

		for (FRichCurve& ChannelCurve : SourceDistribution->ChannelCurves)
		{
			if (ChannelCurve.Keys.Num() == 0)
			{
				continue;
			}

			// Trim Start
			if (ChannelCurve.Keys[0].Time < 0.0f)
			{
				ChannelCurve.AddKey(0.0f, ChannelCurve.Eval(0.0f, 0.0));
				while (ChannelCurve.Keys[0].Time < 0.0f)
				{
					ChannelCurve.Keys.RemoveAt(0);
				}
			}

			// Trim End
			if (ChannelCurve.Keys.Last().Time > 1.0f)
			{
				ChannelCurve.AddKey(1.0f, ChannelCurve.Eval(1.0f, 0.0));
				while (ChannelCurve.Keys.Last().Time > 1.0f)
				{
					ChannelCurve.Keys.RemoveAt(ChannelCurve.Keys.Num() - 1);
				}
			}
		}
	}

	bool AreCurveKeyTimesNormalized() const
	{
		for (FRichCurve& ChannelCurve : SourceDistribution->ChannelCurves)
		{
			if (ChannelCurve.Keys.Num() == 0)
			{
				continue;
			}

			if (ChannelCurve.Keys[0].Time < 0.0f)
			{
				return false;
			}

			if (ChannelCurve.Keys.Last().Time > 1.0f)
			{
				return false;
			}
		}
		return true;
	}


private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TWeakObjectPtr<UObject> OwnerObjectWeak;
	FNiagaraDistributionBase* SourceDistribution = nullptr;
	int32 SourceNumChannels = 0;

	ENiagaraDistributionEditorMode EditorMode = ENiagaraDistributionEditorMode::Constant;
	int32 EditorNumChannels = 0;
	FNiagaraInputParameterCustomization WidgetCustomizationOptions;
	EUnit DisplayUnit = EUnit::Unspecified;

	bool bContinuousTransactionPending = false;
	TOptional<int32> ContinuousTransactionIndex;
	bool bContinuousChangeActive = false;

	bool bDisplayAsColor = false;

	FSimpleMulticastDelegate OnDistributionEditorModeChangedDelegate;
	bool bUpdatingDistributionModeInternally = false;

	FSimpleMulticastDelegate OnDistributionValueChangedDelegate;
	bool bUpdatingDistributionValueInternally = false;
};

class FNiagaraDistributionAdapterRotator : public FNiagaraDistributionAdapter
{
	virtual FText GetChannelDisplayName(int32 ChannelIndex) const override
	{
		switch (ChannelIndex)
		{
			case 0:	return LOCTEXT("RollChannelName", "Roll");
			case 1:	return LOCTEXT("PitchChannelName", "Pitch");
			case 2:	return LOCTEXT("YawChannelName", "Yaw");
		}
		return {};
	}
};

class SNiagaraDistributionPropertyWidget : public SCompoundWidget, public FEditorUndoClient
{
	SLATE_BEGIN_ARGS(SNiagaraDistributionPropertyWidget) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InDistributionPropertyHandle, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter)
	{
		GEditor->RegisterForUndo(this);
		DistributionPropertyHandleWeak = InDistributionPropertyHandle;
		DistributionAdapter = InDistributionAdapter;
		InDistributionPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SNiagaraDistributionPropertyWidget::HandleValueChanged));

		ChildSlot
		[
			SNew(SNiagaraDistributionEditor, DistributionAdapter.ToSharedRef())
		];
	}

	virtual ~SNiagaraDistributionPropertyWidget()
	{
		if (GEditor != nullptr)
		{
			GEditor->UnregisterForUndo(this);
		}
	}

	// Begin FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override
	{
	}

	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

private:
	void HandleValueChanged()
	{
		if (bUpdatingHandle == false)
		{
		}
	}

private:
	TWeakPtr<IPropertyHandle> DistributionPropertyHandleWeak;
	TSharedPtr<INiagaraDistributionAdapter> DistributionAdapter;
	bool bUpdatingHandle = false;
};

TSharedRef<IPropertyTypeCustomization> FNiagaraDistributionPropertyCustomization::MakeFloatInstance(UObject* OptionalOuter)
{
	FPropertyHandleToDistributionAdapter FloatDistributionPropertyHandleToDistributionAdapter = FPropertyHandleToDistributionAdapter::CreateLambda([OptionalOuter](TSharedRef<IPropertyHandle> FloatDistributionPropertyHandle)
	{
		void* ValueData = nullptr;
		TArray<UObject*> OuterObjects;
		if (OptionalOuter)
		{
			TArray<TSharedPtr<FStructOnScope>> OutStructOnScopes;
			FloatDistributionPropertyHandle->GetOuterStructs(OutStructOnScopes);
			if (OutStructOnScopes.Num() == 1)
			{
				OuterObjects.Add(OptionalOuter);
			}
		}
		else
		{
			FloatDistributionPropertyHandle->GetOuterObjects(OuterObjects);
		}
		if (OuterObjects.Num() == 1 && FloatDistributionPropertyHandle->GetValueData(ValueData) == FPropertyAccess::Success)
		{
			FNiagaraDistributionBase* FloatDistribution = static_cast<FNiagaraDistributionBase*>(ValueData);
			TSharedRef<FNiagaraDistributionAdapter> DistributionAdapter = MakeShared<FNiagaraDistributionAdapter>();
			DistributionAdapter->Initialize(FloatDistributionPropertyHandle, OuterObjects[0], FloatDistribution, 1);
			return DistributionAdapter;
		}
		return MakeShared<FNiagaraDistributionAdapter>();
	});

	return MakeShareable<FNiagaraDistributionPropertyCustomization>(new FNiagaraDistributionPropertyCustomization(FloatDistributionPropertyHandleToDistributionAdapter));
}

TSharedRef<IPropertyTypeCustomization> FNiagaraDistributionPropertyCustomization::MakeFloatInstance()
{
	return MakeFloatInstance(nullptr);
}

TSharedRef<IPropertyTypeCustomization> FNiagaraDistributionPropertyCustomization::MakeVector2Instance()
{
	FPropertyHandleToDistributionAdapter Vector2DistributionPropertyHandleToDistributionAdapter = FPropertyHandleToDistributionAdapter::CreateLambda([](TSharedRef<IPropertyHandle> Vector2DistributionPropertyHandle)
	{
		void* ValueData = nullptr;
		TArray<UObject*> OuterObjects;
		Vector2DistributionPropertyHandle->GetOuterObjects(OuterObjects);
		if (OuterObjects.Num() == 1 && Vector2DistributionPropertyHandle->GetValueData(ValueData) == FPropertyAccess::Success)
		{
			FNiagaraDistributionBase* Vector2Distribution = static_cast<FNiagaraDistributionBase*>(ValueData);
			TSharedRef<FNiagaraDistributionAdapter> DistributionAdapter = MakeShared<FNiagaraDistributionAdapter>();
			DistributionAdapter->Initialize(Vector2DistributionPropertyHandle, OuterObjects[0], Vector2Distribution, 2);
			return DistributionAdapter;
		}
		return MakeShared<FNiagaraDistributionAdapter>();
	});

	return MakeShareable<FNiagaraDistributionPropertyCustomization>(new FNiagaraDistributionPropertyCustomization(Vector2DistributionPropertyHandleToDistributionAdapter));
}

TSharedRef<IPropertyTypeCustomization> FNiagaraDistributionPropertyCustomization::MakeVector3Instance()
{
	FPropertyHandleToDistributionAdapter Vector3DistributionPropertyHandleToDistributionAdapter = FPropertyHandleToDistributionAdapter::CreateLambda([](TSharedRef<IPropertyHandle> Vector3DistributionPropertyHandle)
	{
		void* ValueData = nullptr;
		TArray<UObject*> OuterObjects;
		Vector3DistributionPropertyHandle->GetOuterObjects(OuterObjects);
		if (OuterObjects.Num() == 1 && Vector3DistributionPropertyHandle->GetValueData(ValueData) == FPropertyAccess::Success)
		{
			FNiagaraDistributionBase* Vector3Distribution = static_cast<FNiagaraDistributionBase*>(ValueData);
			TSharedRef<FNiagaraDistributionAdapter> DistributionAdapter = MakeShared<FNiagaraDistributionAdapter>();
			DistributionAdapter->Initialize(Vector3DistributionPropertyHandle, OuterObjects[0], Vector3Distribution, 3);
			return DistributionAdapter;
		}
		return MakeShared<FNiagaraDistributionAdapter>();
	});

	return MakeShareable<FNiagaraDistributionPropertyCustomization>(new FNiagaraDistributionPropertyCustomization(Vector3DistributionPropertyHandleToDistributionAdapter));
}

TSharedRef<IPropertyTypeCustomization> FNiagaraDistributionPropertyCustomization::MakePositionInstance()
{
	return MakeVector3Instance();
}

TSharedRef<IPropertyTypeCustomization> FNiagaraDistributionPropertyCustomization::MakeColorInstance()
{
	FPropertyHandleToDistributionAdapter ColorDistributionPropertyHandleToDistributionAdapter = FPropertyHandleToDistributionAdapter::CreateLambda([](TSharedRef<IPropertyHandle> ColorDistributionPropertyHandle)
	{
		void* ValueData = nullptr;
		TArray<UObject*> OuterObjects;
		ColorDistributionPropertyHandle->GetOuterObjects(OuterObjects);
		if (OuterObjects.Num() == 1 && ColorDistributionPropertyHandle->GetValueData(ValueData) == FPropertyAccess::Success)
		{
			FNiagaraDistributionBase* ColorDistribution = static_cast<FNiagaraDistributionBase*>(ValueData);
			TSharedRef<FNiagaraDistributionAdapter> DistributionAdapter = MakeShared<FNiagaraDistributionAdapter>();
			DistributionAdapter->Initialize(ColorDistributionPropertyHandle, OuterObjects[0], ColorDistribution, 4);
			return DistributionAdapter;
		}
		return MakeShared<FNiagaraDistributionAdapter>();
	});

	return MakeShareable<FNiagaraDistributionPropertyCustomization>(new FNiagaraDistributionPropertyCustomization(ColorDistributionPropertyHandleToDistributionAdapter));
}

TSharedRef<IPropertyTypeCustomization> FNiagaraDistributionPropertyCustomization::MakeRotatorInstance()
{
	FPropertyHandleToDistributionAdapter DistributionPropertyHandleToDistributionAdapter = FPropertyHandleToDistributionAdapter::CreateLambda(
		[](TSharedRef<IPropertyHandle> DistributionPropertyHandle)
		{
			void* ValueData = nullptr;
			TArray<UObject*> OuterObjects;
			DistributionPropertyHandle->GetOuterObjects(OuterObjects);
			if (OuterObjects.Num() == 1 && DistributionPropertyHandle->GetValueData(ValueData) == FPropertyAccess::Success)
			{
				FNiagaraDistributionBase* Distribution = static_cast<FNiagaraDistributionBase*>(ValueData);
				TSharedRef<FNiagaraDistributionAdapter> DistributionAdapter = MakeShared<FNiagaraDistributionAdapterRotator>();
				DistributionAdapter->Initialize(DistributionPropertyHandle, OuterObjects[0], Distribution, Distribution->GetBaseNumberOfChannels());
				return DistributionAdapter;
			}
			return MakeShared<FNiagaraDistributionAdapter>();
		});

	return MakeShareable<FNiagaraDistributionPropertyCustomization>(new FNiagaraDistributionPropertyCustomization(DistributionPropertyHandleToDistributionAdapter));
}

void FNiagaraDistributionPropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
	HeaderRow.ValueContent()
	[
		SNew(SNiagaraDistributionPropertyWidget, PropertyHandle, PropertyHandleToDistributionAdapter.Execute(PropertyHandle))
	];
}

void FNiagaraDistributionPropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

#undef LOCTEXT_NAMESPACE