// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class INiagaraDistributionAdapter;

class FNiagaraDistributionPropertyCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeFloatInstance(UObject* OptionalOuter);
	static TSharedRef<IPropertyTypeCustomization> MakeFloatInstance();
	static TSharedRef<IPropertyTypeCustomization> MakeVector2Instance();
	static TSharedRef<IPropertyTypeCustomization> MakeVector3Instance();
	static TSharedRef<IPropertyTypeCustomization> MakePositionInstance();
	static TSharedRef<IPropertyTypeCustomization> MakeColorInstance();
	static TSharedRef<IPropertyTypeCustomization> MakeRotatorInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<INiagaraDistributionAdapter>, FPropertyHandleToDistributionAdapter, TSharedRef<IPropertyHandle> /* InDistributionPropertyHandle */)

private:
	FNiagaraDistributionPropertyCustomization(FPropertyHandleToDistributionAdapter InPropertyHandleToDistributionAdapter)
		: PropertyHandleToDistributionAdapter(InPropertyHandleToDistributionAdapter)
	{
	}

private:
	
	FPropertyHandleToDistributionAdapter PropertyHandleToDistributionAdapter;
};