// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraEmitterStatePropertyCustomization.h"
#include "NiagaraSystemEmitterState.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "DetailWidgetRow.h"

#define LOCTEXT_NAMESPACE "NiagaraEmitterStatePropertyCustomization"


TSharedRef<IPropertyTypeCustomization> FNiagaraEmitterStatePropertyCustomization::MakeInstance()
{
	return MakeShared<FNiagaraEmitterStatePropertyCustomization>();
}

void FNiagaraEmitterStatePropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FNiagaraEmitterStatePropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	EmitterStatePropertyHandle = StructPropertyHandle;
	EmitterStateOwnerObject = nullptr;
	{
		TArray<UObject*> OuterObjects;
		StructPropertyHandle->GetOuterObjects(OuterObjects);
		EmitterStateOwnerObject = OuterObjects.Num() == 1 ? OuterObjects[0] : nullptr;
	}

	if (EmitterStateOwnerObject == nullptr)
	{
		return;
	}

	IDetailLayoutBuilder& LayoutBuilder = ChildBuilder.GetParentCategory().GetParentLayout();
	LayoutBuilder.EditCategory(FName("Emitter Properties"));

	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);
	for (uint32 iChild=0; iChild < NumChildren; ++iChild)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(iChild);
		IDetailPropertyRow& PropertyRow = LayoutBuilder.AddPropertyToCategory(ChildHandle.ToSharedRef());

		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraEmitterStateData, MinDistance) || PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraEmitterStateData, MaxDistance))
		{
			PropertyRow.Visibility(TAttribute<EVisibility>::CreateSP(this, &FNiagaraEmitterStatePropertyCustomization::GetEnableDistanceCullingVisibility));
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraEmitterStateData, MinDistanceReaction))
		{
			PropertyRow.Visibility(TAttribute<EVisibility>::CreateSP(this, &FNiagaraEmitterStatePropertyCustomization::GetMinDistanceVisibility));
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraEmitterStateData, MaxDistanceReaction))
		{
			PropertyRow.Visibility(TAttribute<EVisibility>::CreateSP(this, &FNiagaraEmitterStatePropertyCustomization::GetMaxDistanceVisibility));
		}
	}
}

const FNiagaraEmitterStateData* FNiagaraEmitterStatePropertyCustomization::GetEmitterState() const
{
	uint8* ObjectAddress = reinterpret_cast<uint8*>(EmitterStateOwnerObject.Get());
	if (EmitterStatePropertyHandle && ObjectAddress)
	{
		return reinterpret_cast<const FNiagaraEmitterStateData*>(EmitterStatePropertyHandle->GetValueBaseAddress(ObjectAddress));
	}
	return nullptr;
}

EVisibility FNiagaraEmitterStatePropertyCustomization::GetEnableDistanceCullingVisibility() const
{
	const FNiagaraEmitterStateData* EmitterState = GetEmitterState();
	return EmitterState && EmitterState->bEnableDistanceCulling ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FNiagaraEmitterStatePropertyCustomization::GetMinDistanceVisibility() const
{
	const FNiagaraEmitterStateData* EmitterState = GetEmitterState();
	return EmitterState && EmitterState->bEnableDistanceCulling && EmitterState->bMinDistanceEnabled ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FNiagaraEmitterStatePropertyCustomization::GetMaxDistanceVisibility() const
{
	const FNiagaraEmitterStateData* EmitterState = GetEmitterState();
	return EmitterState && EmitterState->bEnableDistanceCulling && EmitterState->bMaxDistanceEnabled ? EVisibility::Visible : EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
