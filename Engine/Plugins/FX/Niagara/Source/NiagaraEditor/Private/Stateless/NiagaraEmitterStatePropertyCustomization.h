// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyHandle.h"
#include "IPropertyTypeCustomization.h"

struct FNiagaraEmitterStateData;

class FNiagaraEmitterStatePropertyCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	const FNiagaraEmitterStateData* GetEmitterState() const;

	EVisibility GetEnableDistanceCullingVisibility() const;
	EVisibility GetMinDistanceVisibility() const;
	EVisibility GetMaxDistanceVisibility() const;

private:
	TSharedPtr<IPropertyHandle> EmitterStatePropertyHandle;
	TWeakObjectPtr<UObject>		EmitterStateOwnerObject;
};
