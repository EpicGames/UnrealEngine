// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraSpawnInfoPropertyCustomization.h"
#include "Stateless/NiagaraStatelessSpawnInfo.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "DetailWidgetRow.h"

TSharedRef<IDetailCustomization> FNiagaraSpawnInfoDetailCustomization::MakeInstance()
{
	return MakeShared<FNiagaraSpawnInfoDetailCustomization>();
}

void FNiagaraSpawnInfoDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& DetailCategory = DetailBuilder.EditCategory("Spawn");
	{
		TArray<TSharedRef<IPropertyHandle>> Properties;
		DetailCategory.GetDefaultProperties(Properties, true, true);

		for (TSharedPtr<IPropertyHandle> Property : Properties)
		{
			IDetailPropertyRow& DetailPropertyRow = DetailCategory.AddProperty(Property);
			if (Property->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraStatelessSpawnInfo, LoopCountLimit))
			{
				DetailPropertyRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FNiagaraSpawnInfoDetailCustomization::GetLoopCountLimitVisibility)));
			}
			else if (Property->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FNiagaraStatelessSpawnInfo, Type))
			{
				SpawnTypeProperty = Property;
			}
		}
	}
}

EVisibility FNiagaraSpawnInfoDetailCustomization::GetLoopCountLimitVisibility() const
{
	if ( SpawnTypeProperty )
	{
		uint8 UntypedValue = -1;
		const FPropertyAccess::Result GetResult = SpawnTypeProperty->GetValue(UntypedValue);
		if (GetResult == FPropertyAccess::Result::Success)
		{
			return ENiagaraStatelessSpawnInfoType(UntypedValue) == ENiagaraStatelessSpawnInfoType::Burst ? EVisibility::Visible : EVisibility::Hidden;
		}
	}

	return EVisibility::Hidden;
}
