// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyHandle.h"
#include "IDetailCustomization.h"

struct FNiagaraStatelessSpawnInfo;

class FNiagaraSpawnInfoDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	EVisibility GetLoopCountLimitVisibility() const;

private:
	TSharedPtr<IPropertyHandle> SpawnTypeProperty;
};
