// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessEmitterDetails.h"
#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "NiagaraSystem.h"

#include "Algo/Sort.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SComboButton.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NiagaraStatelessEmitter"

TSharedRef<IDetailCustomization> FNiagaraStatelessEmitterDetails::MakeInstance()
{
	return MakeShared<FNiagaraStatelessEmitterDetails>();
}

void FNiagaraStatelessEmitterDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	UNiagaraStatelessEmitter* Emitter = nullptr;
	{
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
		Emitter = ObjectsBeingCustomized.Num() == 1 ? Cast<UNiagaraStatelessEmitter>(ObjectsBeingCustomized[0]) : nullptr;
	}
	if (Emitter == nullptr)
	{
		return;
	}
	WeakEmitter = Emitter;

	static const FName NAME_EmitterProperties("Emitter Properties");
	static const FName NAME_FixedBounds("FixedBounds");
	static const FName NAME_EmitterTemplate("EmitterTemplate");

	IDetailCategoryBuilder& DetailCategory = DetailBuilder.EditCategory(NAME_EmitterProperties);

	TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
	DetailCategory.GetDefaultProperties(CategoryProperties);

	for (TSharedRef<IPropertyHandle>& PropertyHandle : CategoryProperties)
	{
		const FName PropertyName = PropertyHandle->GetProperty() ? PropertyHandle->GetProperty()->GetFName() : NAME_None;
		if (PropertyName == NAME_FixedBounds)
		{
			IDetailPropertyRow& PropertyRow = DetailCategory.AddProperty(PropertyHandle);
			PropertyRow.IsEnabled(TAttribute<bool>::CreateSP(this, &FNiagaraStatelessEmitterDetails::GetFixedBoundsEnabled));
		}
		else if (PropertyName == NAME_EmitterTemplate)
		{
			IDetailPropertyRow& PropertyRow = DetailCategory.AddProperty(PropertyHandle);
			PropertyRow.CustomWidget()
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &FNiagaraStatelessEmitterDetails::OnGetTemplateActions)
					.ContentPadding(1)
					.ToolTipText(LOCTEXT("SelectEmitterTemplate", "Select the template to use for this emitter."))
					.ButtonContent()
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(this, &FNiagaraStatelessEmitterDetails::GetSelectedTemplateName)
					]
				]
			];
		}
		else
		{
			DetailCategory.AddProperty(PropertyHandle);
		}
	}
}

bool FNiagaraStatelessEmitterDetails::GetFixedBoundsEnabled() const
{
	if (const UNiagaraStatelessEmitter* Emitter = WeakEmitter.Get())
	{
		if (const UNiagaraSystem* System = Emitter->GetTypedOuter<UNiagaraSystem>())
		{
			return System->bFixedBounds == false;
		}
	}
	return false;
}

FText FNiagaraStatelessEmitterDetails::GetSelectedTemplateName() const
{
	if (const UNiagaraStatelessEmitter* Emitter = WeakEmitter.Get())
	{
		if (const UNiagaraStatelessEmitterTemplate* EmitterTemplate = Emitter->GetEmitterTemplate())
		{
			return EmitterTemplate->IsAsset() ? FText::FromName(EmitterTemplate->GetFName()) : EmitterTemplate->GetClass()->GetDisplayNameText();
		}
	}
	return FText();
}

void FNiagaraStatelessEmitterDetails::SetTemplate(UNiagaraStatelessEmitterTemplate* Template) const
{
	UNiagaraStatelessEmitter* Emitter = WeakEmitter.Get();
	if (Template == nullptr || Emitter == nullptr)
	{
		return;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("SetEmitterTemplate", "Set Emitter Template"));
	Emitter->Modify();
	Emitter->SetEmitterTemplate(Template);
}

void FNiagaraStatelessEmitterDetails::SetTemplate(FSoftObjectPath TemplatePath) const
{
	if (UNiagaraStatelessEmitterTemplate* Template = Cast<UNiagaraStatelessEmitterTemplate>(TemplatePath.TryLoad()))
	{
		SetTemplate(Template);
	}
}

TSharedRef<SWidget> FNiagaraStatelessEmitterDetails::OnGetTemplateActions() const
{
	struct FMenuAction
	{
		FText		Label;
		FText		ToolTip;
		FUIAction	Action;
	};
	TArray<FMenuAction> Actions;

	// Find all Build In Types
	{
		TSet<UClass*> VisitedClasses;
		for (TObjectIterator<UClass> It; It; ++It)
		{
			UClass* TemplateClass = *It;
			if (TemplateClass && TemplateClass != UNiagaraStatelessEmitterTemplate::StaticClass() && TemplateClass->IsChildOf(UNiagaraStatelessEmitterTemplate::StaticClass()))
			{
				const FText DisplayName = TemplateClass->GetDisplayNameText();
				const FText AssetName = FText::FromName(TemplateClass->GetFName());
				Actions.Emplace(
					DisplayName,
					FText::Format(LOCTEXT("SetBuiltInTemplate", "Set to the built in template '{0}' class '{1}'"), DisplayName, AssetName),
					FUIAction(FExecuteAction::CreateSP(this, &FNiagaraStatelessEmitterDetails::SetTemplate, Cast<UNiagaraStatelessEmitterTemplate>(TemplateClass->GetDefaultObject())))
				);
			}
		}
	}

	// Find all Template Assets
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		FARFilter Filter;
		Filter.bRecursiveClasses = true;
		Filter.ClassPaths.Add(FTopLevelAssetPath(UNiagaraStatelessEmitterTemplate::StaticClass()));

		TArray<FAssetData> TemplateAssetDatas;
		AssetRegistryModule.Get().GetAssets(Filter, TemplateAssetDatas);

		for (const FAssetData& AssetData : TemplateAssetDatas)
		{
			if (UNiagaraStatelessEmitterTemplate::IsExposedToLibrary(AssetData) == false)
			{
				continue;
			}

			const FSoftObjectPath ObjectPath = AssetData.GetSoftObjectPath();
			const FText DisplayName = FText::FromString(ObjectPath.GetAssetName());
			const FText AssetName = FText::FromString(ObjectPath.ToString());
			Actions.Emplace(
				DisplayName,
				FText::Format(LOCTEXT("SetAssetTemplate", "Set to template '{0}' asset path '{1}'"), DisplayName, AssetName),
				FUIAction(FExecuteAction::CreateSP(this, &FNiagaraStatelessEmitterDetails::SetTemplate, ObjectPath))
			);
		}
	}

	// Sort Options
	Algo::SortBy(Actions, [](const FMenuAction& Item) { return Item.Label; }, FText::FSortPredicate());

	// Create Menu
	FMenuBuilder MenuBuilder(true, NULL);
	for (const FMenuAction& Action : Actions)
	{
		MenuBuilder.AddMenuEntry(
			Action.Label,
			Action.ToolTip,
			FSlateIcon(),
			Action.Action
		);
	}
	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
