// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class IDetailLayoutBuilder;
class SWidget;

class UNiagaraStatelessEmitter;
class UNiagaraStatelessEmitterTemplate;

class FNiagaraStatelessEmitterDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	// Begin: IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End: IDetailCustomization

private:
	bool GetFixedBoundsEnabled() const;

	FText GetSelectedTemplateName() const;
	void SetTemplate(UNiagaraStatelessEmitterTemplate* Template) const;
	void SetTemplate(FSoftObjectPath TemplatePath) const;
	TSharedRef<SWidget> OnGetTemplateActions() const;

protected:
	TWeakObjectPtr<UNiagaraStatelessEmitter> WeakEmitter;
};
