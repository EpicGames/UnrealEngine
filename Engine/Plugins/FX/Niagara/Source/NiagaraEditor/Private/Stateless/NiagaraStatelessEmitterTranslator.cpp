// Copyright Epic Games, Inc. All Rights Reserved.

#include "Stateless/NiagaraStatelessEmitterTranslator.h"
#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessModule.h"

#include "ShaderCore.h"

#define NIAGARA_STATELESS_TRANSLATION_OLD_MODE 1

namespace FNiagaraStatelessEmitterTranslator
{
	static const TCHAR* DefaultIncludes[] =
	{
		TEXT("/Plugin/FX/Niagara/Private/NiagaraQuaternionUtils.ush"),
		TEXT("/Plugin/FX/Niagara/Private/Stateless/NiagaraStatelessCommon.ush"),
	};

	static TConstArrayView<FNiagaraVariableBase> GetImplicitVariables()
	{
		static TArray<FNiagaraVariableBase> ImplicitVariables =
		{
			FNiagaraStatelessGlobals::Get().MaterialRandomVariable,
			FNiagaraStatelessGlobals::Get().UniqueIDVariable,
		};
		return ImplicitVariables;
	}

	static FString GetModuleHlslName(const UClass* ModuleClass, int ModuleIndex)
	{
		//return FString::Printf(TEXT("%s_%d"), *ModuleClass->GetName(), ModuleIndex);
		return ModuleClass->GetName();
	}

	static FString SanitizeVariableName(const FName VariableName)
	{
		FString SanitizedName(VariableName.ToString());
		SanitizedName.ReplaceInline(TEXT("."), TEXT(""));
		return SanitizedName;
	}

	static const TCHAR* GetVariableHlslType(const FNiagaraTypeDefinition& TypeDef)
	{
		if (TypeDef == FNiagaraTypeDefinition::GetFloatDef())
		{
			return TEXT("float");
		}
		else if (TypeDef == FNiagaraTypeDefinition::GetVec2Def())
		{
			return TEXT("float2");
		}
		else if (TypeDef == FNiagaraTypeDefinition::GetVec3Def() || TypeDef == FNiagaraTypeDefinition::GetPositionDef())
		{
			return TEXT("float3");
		}
		else if (TypeDef == FNiagaraTypeDefinition::GetVec4Def() || TypeDef == FNiagaraTypeDefinition::GetColorDef() || TypeDef == FNiagaraTypeDefinition::GetQuatDef())
		{
			return TEXT("float4");
		}
		else if (TypeDef == FNiagaraTypeDefinition::GetIntDef())
		{
			return TEXT("int");
		}
		checkNoEntry();
		return TEXT("");
	}

	void WriteHeader(FString& HlslOutput)
	{
		HlslOutput.Append(TEXT("// Copyright Epic Games, Inc. All Rights Reserved.\n"));
		HlslOutput.Append(TEXT("\n"));
		for (const TCHAR* Include : DefaultIncludes)
		{
			HlslOutput.Appendf(TEXT("#include \"%s\"\n"), Include);
		}
	}

	void WriteParticleStruct(FString& HlslOutput, UNiagaraStatelessEmitterTemplate* EmitterTemplate)
	{
	#if NIAGARA_STATELESS_TRANSLATION_OLD_MODE
		HlslOutput.Append(TEXT("\n"));
		HlslOutput.Append(TEXT("#define PARTICLE_ATTRIBUTES \\\n"));

		TConstArrayView<FNiagaraVariableBase> ModuleVariables = EmitterTemplate->GetModuleVariables();
		if (ModuleVariables.Num() > 0)
		{
			TConstArrayView<FNiagaraVariableBase> ImplicitVariables = EmitterTemplate->GetImplicitVariables();
			for (const FNiagaraVariableBase& ParticleVariable : ImplicitVariables)
			{
				const TCHAR* HlslType = GetVariableHlslType(ParticleVariable.GetType());
				const FString HlslName = SanitizeVariableName(ParticleVariable.GetName());
				HlslOutput.Appendf(TEXT("\tPARTICLE_ATTRIBUTE_OUTPUT_IMPLICIT(%s, %s) \\\n"), HlslType, *HlslName);
			}

			for (const FNiagaraVariableBase& ParticleVariable : ModuleVariables)
			{
				if (ImplicitVariables.Contains(ParticleVariable))
				{
					continue;
				}

				const TCHAR* HlslType = GetVariableHlslType(ParticleVariable.GetType());
				const FString HlslName = SanitizeVariableName(ParticleVariable.GetName());
				HlslOutput.Appendf(TEXT("\tPARTICLE_ATTRIBUTE_OUTPUT(%s, %s) \\\n"), HlslType, *HlslName);
			}
		}
	#else
		HlslOutput.Append(TEXT("\n"));
		HlslOutput.Append(TEXT("struct FStatelessParticle\n"));
		HlslOutput.Append(TEXT("{\n"));
		HlslOutput.Append(TEXT("\tbool bAlive;					// If the particle should remain alive or not (only evaluated at specific points)\n"));
		HlslOutput.Append(TEXT("\tuint UniqueID;				// Unique particle index, sequential based on previous spawn info accumulation\n"));
		HlslOutput.Append(TEXT("\tfloat MaterialRandom;			// Random float passed to materials\n"));
		HlslOutput.Append(TEXT("\tfloat Lifetime;					// Overall lifetime for the particle\n"));
		HlslOutput.Append(TEXT("\tfloat Age;						// Current age for the particle\n"));
		HlslOutput.Append(TEXT("\tfloat NormalizedAge;			// Current normalized age for the particle\n"));
		HlslOutput.Append(TEXT("\tfloat PreviousAge;				// Previous age for the particle\n"));
		HlslOutput.Append(TEXT("\tfloat PreviousNormalizedAge;	// Previous normalized age for the particle\n"));
		HlslOutput.Append(TEXT("\tfloat DeltaTime;				// Simulation DT\n"));
		HlslOutput.Append(TEXT("\tfloat InvDeltaTime;				// Simulation Inverse DT\n"));

		TConstArrayView<FNiagaraVariableBase> ModuleVariables = EmitterTemplate->GetModuleVariables();
		if (ModuleVariables.Num() > 0)
		{
			TConstArrayView<FNiagaraVariableBase> ImplicitVariables = EmitterTemplate->GetImplicitVariables();

			HlslOutput.Append(TEXT("\n"));
			HlslOutput.Append(TEXT("\t// Module Variables\n"));

			for (const FNiagaraVariableBase& ParticleVariable : ModuleVariables)
			{
				if (ImplicitVariables.Contains(ParticleVariable))
				{
					continue;
				}

				const TCHAR* HlslType = GetVariableHlslType(ParticleVariable.GetType());
				const FString HlslName = SanitizeVariableName(ParticleVariable.GetName());
				HlslOutput.Appendf(TEXT("\t%s %s;\n"), HlslType, *HlslName);
			}
		}
		HlslOutput.Append(TEXT("};\n"));
	#endif
	}

	void WriteModules(FString& HlslOutput, UNiagaraStatelessEmitterTemplate* EmitterTemplate)
	{
	#if NIAGARA_STATELESS_TRANSLATION_OLD_MODE
		HlslOutput.Append(TEXT("\n"));
		HlslOutput.Append(TEXT("#define PARTICLE_MODULES \\\n"));

		TConstArrayView<UClass*> Modules = EmitterTemplate->GetModules();
		for (int iModule = 0; iModule < Modules.Num(); ++iModule)
		{
			const UClass* ModuleClass = Modules[iModule];
			const UNiagaraStatelessModule* Module = ModuleClass->GetDefaultObject<UNiagaraStatelessModule>();
			const TCHAR* ModuleShaderTemplatePath = Module ? Module->GetShaderTemplatePath() : nullptr;

			FString ShortClassName = ModuleClass->GetName();
			ShortClassName.RemoveFromStart(TEXT("NiagaraStatelessModule_"));

			if (ModuleShaderTemplatePath == nullptr)
			{
				HlslOutput.Appendf(TEXT("\tPARTICLE_MODULE(Null) /* %s */ \\\n"), *ShortClassName);
			}
			else
			{
				HlslOutput.Appendf(TEXT("\tPARTICLE_MODULE(%s) \\\n"), *ShortClassName);
			}
		}

		HlslOutput.Append(TEXT("\n"));
		HlslOutput.Append(TEXT("#include \"/Plugin/FX/Niagara/Private/Stateless/Template/GenerateCS_PreModules.ush\"\n"));
		HlslOutput.Append(TEXT("\n"));

		for (int iModule = 0; iModule < Modules.Num(); ++iModule)
		{
			const UClass* ModuleClass = Modules[iModule];
			const UNiagaraStatelessModule* Module = ModuleClass->GetDefaultObject<UNiagaraStatelessModule>();
			const TCHAR* ModuleShaderTemplatePath = Module ? Module->GetShaderTemplatePath() : nullptr;
			if (ModuleShaderTemplatePath)
			{
				HlslOutput.Appendf(TEXT("#include \"%s\"\n"), ModuleShaderTemplatePath);
			}
		}

		HlslOutput.Append(TEXT("\n"));
		HlslOutput.Append(TEXT("#include \"/Plugin/FX/Niagara/Private/Stateless/Template/GenerateCS_PostModules.ush\"\n"));
	#else
		TConstArrayView<UClass*> Modules = EmitterTemplate->GetModules();
		if (Modules.Num() == 0)
		{
			return;
		}

		HlslOutput.Append(TEXT("\n"));
		for (int iModule = 0; iModule < Modules.Num(); ++iModule)
		{
			const UClass* ModuleClass = Modules[iModule];
			const UNiagaraStatelessModule* Module = ModuleClass->GetDefaultObject<UNiagaraStatelessModule>();
			const TCHAR* ModuleShaderTemplatePath = Module ? Module->GetShaderTemplatePath() : nullptr;
			if (ModuleShaderTemplatePath == nullptr)
			{
				continue;
			}

			FString TemplateFile;
			LoadShaderSourceFileChecked(ModuleShaderTemplatePath, EShaderPlatform::SP_PCD3D_SM5, TemplateFile);

			//const FString ModuleHLSLName = GetModuleHlslName(ModuleClass, iModule);
			//TemplateFile.ReplaceInline(Thing, ModuleHLSLName, ESearchCase::CaseSensitive);

			HlslOutput.Append(TemplateFile);

			//HlslOutput.Appendf(TEXT("#define MODULE_NAME %s\n"), *ModuleHLSLName);
			//HlslOutput.Appendf(TEXT("#include \"%s\"\n"), ModuleShaderTemplatePath);
			//HlslOutput.Append(TEXT("#undef MODULE_NAME\n"));
		}
	#endif
	}

	void WriteSimulateFunction(FString& HlslOutput, UNiagaraStatelessEmitterTemplate* EmitterTemplate)
	{
	#if NIAGARA_STATELESS_TRANSLATION_OLD_MODE == 0
		TConstArrayView<UClass*> Modules = EmitterTemplate->GetModules();
		if (Modules.Num() == 0)
		{
			return;
		}

		HlslOutput.Append(TEXT("\n"));
		HlslOutput.Append(TEXT("void SimulateParticle(inout FStatelessParticle Particle, int RandomSeedOffset)\n"));
		HlslOutput.Append(TEXT("{\n"));
		for (int iModule = 0; iModule < Modules.Num(); ++iModule)
		{
			const UClass* ModuleClass = Modules[iModule];
			const UNiagaraStatelessModule* Module = ModuleClass->GetDefaultObject<UNiagaraStatelessModule>();
			const TCHAR* ModuleShaderTemplatePath = Module ? Module->GetShaderTemplatePath() : nullptr;
			if (ModuleShaderTemplatePath == nullptr)
			{
				continue;
			}

			const FString ModuleHLSLName = GetModuleHlslName(ModuleClass, iModule);
			HlslOutput.Append(TEXT("\tGRandomSeedInternal = FNiagaraStatelessDefinitions::MakeRandomSeed(Common_RandomSeed, Particle.UniqueID, RandomSeedOffset + iModule, 0);\n"));
			HlslOutput.Appendf(TEXT("\t%s_Simulate(Particle);\n"), *ModuleHLSLName);
		}
		HlslOutput.Append(TEXT("}\n"));
	#endif
	}

	void WriteOutputFunction(FString& HlslOutput, UNiagaraStatelessEmitterTemplate* EmitterTemplate)
	{
	#if NIAGARA_STATELESS_TRANSLATION_OLD_MODE == 0
		TConstArrayView<FNiagaraVariableBase> ShaderOutputVariables = EmitterTemplate->GetShaderOutputVariables();
		if (ShaderOutputVariables.Num() == 0)
		{
			return;
		}

		HlslOutput.Append(TEXT("\n"));
		HlslOutput.Appendf(TEXT("DECLARE_SCALAR_ARRAY(uint, ShaderOutputOffsets, %d);\n"), ShaderOutputVariables.Num());

		HlslOutput.Append(TEXT("\n"));
		HlslOutput.Append(TEXT("void OutputParticle(inout FStatelessParticle Particle, uint ParticleIndex)\n"));
		HlslOutput.Append(TEXT("{\n"));
		for (int i = 0; i < ShaderOutputVariables.Num(); ++i)
		{
			const FNiagaraVariableBase& ParticleVariable = ShaderOutputVariables[i];
			const FString HlslName = ParticleVariable.GetName().ToString();

			HlslOutput.Appendf(TEXT("\tif ( IsValidComponent(GET_SCALAR_ARRAY_ELEMENT(ShaderOutputOffsets, %d)) )\n"), i);
			HlslOutput.Append(TEXT("\t{\n"));
			HlslOutput.Appendf(TEXT("\t\tOutputComponentData(OutputIndex, GET_SCALAR_ARRAY_ELEMENT(ShaderOutputOffsets, %d), Particle.%s);\n"), i, *HlslName);
			HlslOutput.Append(TEXT("\t}\n"));
		}
		HlslOutput.Append(TEXT("}\n"));
	#endif
	}

	void TranslateToCompute(FString& HlslOutput, UNiagaraStatelessEmitterTemplate* EmitterTemplate)
	{
		TConstArrayView<TObjectPtr<UClass>> Modules = EmitterTemplate->GetModules();

		WriteHeader(HlslOutput);
		WriteParticleStruct(HlslOutput, EmitterTemplate);
		WriteModules(HlslOutput, EmitterTemplate);
		WriteSimulateFunction(HlslOutput, EmitterTemplate);
		WriteOutputFunction(HlslOutput, EmitterTemplate);

	#if NIAGARA_STATELESS_TRANSLATION_OLD_MODE == 0
		HlslOutput.Append(TEXT("\n"));
		HlslOutput.Append(TEXT("#include \"NiagaraStatelessSimulationTemplate.ush\"\n"));
	#endif
	}
}

#undef NIAGARA_STATELESS_TRANSLATION_OLD_MODE
