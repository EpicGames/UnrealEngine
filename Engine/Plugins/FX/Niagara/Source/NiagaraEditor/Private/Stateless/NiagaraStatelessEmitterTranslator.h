// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessCommon.h"

class UNiagaraStatelessEmitterTemplate;

namespace FNiagaraStatelessEmitterTranslator
{
	void TranslateToCompute(FString& HlslOutput, UNiagaraStatelessEmitterTemplate* EmitterTemplate);
}
