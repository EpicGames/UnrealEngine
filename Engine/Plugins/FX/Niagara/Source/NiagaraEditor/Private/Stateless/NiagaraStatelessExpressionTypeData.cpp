// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStatelessExpressionTypeData.h"
#include "Stateless/NiagaraStatelessExpression.h"

#include "HAL/IConsoleManager.h"
#include "UObject/UObjectIterator.h"

static bool GNiagaraStatelessExpressionsEnabled = false;
static FAutoConsoleVariableRef CVarNiagaraStatelessExpressionsEnabled(
	TEXT("fx.NiagaraStateless.ExpressionsEnabled"),
	GNiagaraStatelessExpressionsEnabled,
	TEXT("CVar to enable stateless expressions"),
	ECVF_Default
);

bool FNiagaraStatelessExpressionTypeData::IsValid() const
{
	return GNiagaraStatelessExpressionsEnabled && ValueExpression.IsValid();
}

bool FNiagaraStatelessExpressionTypeData::ContainsExpression(const FInstancedStruct* Expresssion) const
{
	const UScriptStruct* ExpressionType = Expresssion ? Expresssion->GetScriptStruct() : nullptr;
	return
		ExpressionType != nullptr &&
		(
			ExpressionType == ValueExpression.Get() ||
			ExpressionType == BindingExpression.Get() ||
			OperationExpressions.Contains(ExpressionType)
		);
}

bool FNiagaraStatelessExpressionTypeData::IsBindingExpression(const FInstancedStruct* Expression) const
{
	return Expression && Expression->GetScriptStruct() == BindingExpression;
}

bool FNiagaraStatelessExpressionTypeData::IsValueExpression(const FInstancedStruct* Expression) const
{
	return Expression && Expression->GetScriptStruct() == ValueExpression;
}

bool FNiagaraStatelessExpressionTypeData::IsOperationExpression(const FInstancedStruct* Expression) const
{
	return Expression && OperationExpressions.Contains(Expression->GetScriptStruct());
}

FName FNiagaraStatelessExpressionTypeData::GetBindingName(const FInstancedStruct* Expression) const
{
	check(Expression && Expression->GetScriptStruct() == BindingExpression);
	return BindingNameField->GetPropertyValue_InContainer(Expression->GetMemory());
}

FInstancedStruct FNiagaraStatelessExpressionTypeData::MakeBindingStruct(FName BindingName) const
{
	FInstancedStruct NewExpression(BindingExpression.Get());
	BindingNameField->SetPropertyValue_InContainer(NewExpression.GetMutableMemory(), BindingName);
	return NewExpression;
}

const FNiagaraStatelessExpressionTypeData& FNiagaraStatelessExpressionTypeData::GetTypeData(const FNiagaraTypeDefinition& TypeDef)
{
	static TMap<FNiagaraTypeDefinition, FNiagaraStatelessExpressionTypeData> TypeDefToExpressionData;
	if (TypeDefToExpressionData.IsEmpty())
	{
		static const FName NAME_BindingExpression("BindingExpression");
		static const FName NAME_ValueExpression("ValueExpression");
		static const FName NAME_OperationExpression("OperationExpression");

		for (TObjectIterator<UScriptStruct> StructIt; StructIt; ++StructIt)
		{
			UScriptStruct* ScriptStruct = *StructIt;
			if (!ScriptStruct || !ScriptStruct->IsChildOf<FNiagaraStatelessExpression>())
			{
				continue;
			}

			const bool bIsBinding = ScriptStruct->HasMetaData(NAME_BindingExpression);
			const bool bIsValue = ScriptStruct->HasMetaData(NAME_ValueExpression);
			const bool bIsOperation = ScriptStruct->HasMetaData(NAME_OperationExpression);
			const bool bIsAny = bIsBinding || bIsValue || bIsOperation;
			if (!bIsAny)
			{
				continue;
			}

			FInstancedStruct TempStruct;
			TempStruct.InitializeAs(ScriptStruct, nullptr);
			const FNiagaraStatelessExpression& Expression = TempStruct.Get<FNiagaraStatelessExpression>();
			FNiagaraStatelessExpressionTypeData& ExpressionTypeData = TypeDefToExpressionData.FindOrAdd(Expression.GetOutputTypeDef());
			ExpressionTypeData.TypeDef = Expression.GetOutputTypeDef();

			if (bIsBinding)
			{
				FNameProperty* NameProperty = CastField<FNameProperty>(ScriptStruct->ChildProperties);
				if (ensure(NameProperty && NameProperty->Next == nullptr))
				{
					ExpressionTypeData.BindingExpression = ScriptStruct;
					ExpressionTypeData.BindingNameField = NameProperty;
				}
			}
			else if (bIsValue)
			{
				FProperty* ValueProperty = CastField<FProperty>(ScriptStruct->ChildProperties);
				if (ensure(ValueProperty && ValueProperty->Next == nullptr))
				{
					if (ensure(FNiagaraTypeDefinition(ValueProperty, FNiagaraTypeDefinition::EAllowUnfriendlyStruct::Deny) == Expression.GetOutputTypeDef()))
					{
						ExpressionTypeData.ValueExpression = ScriptStruct;
						ExpressionTypeData.ValueProperty = ValueProperty;
					}
				}
			}
			else if (bIsOperation)
			{
				ExpressionTypeData.OperationExpressions.Add(ScriptStruct);
			}
		}
	}

	return TypeDefToExpressionData.FindOrAdd(TypeDef);
}

const FNiagaraStatelessExpressionTypeData& FNiagaraStatelessExpressionTypeData::GetTypeData(const FInstancedStruct* Expression)
{
	check(Expression);
	return GetTypeData(Expression->Get<const FNiagaraStatelessExpression>().GetOutputTypeDef());
}
