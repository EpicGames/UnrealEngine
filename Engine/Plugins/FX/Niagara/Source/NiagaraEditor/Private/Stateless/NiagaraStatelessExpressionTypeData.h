// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"

struct FInstancedStruct;

struct FNiagaraStatelessExpressionTypeData
{
	FNiagaraTypeDefinition					TypeDef;

	TWeakObjectPtr<UScriptStruct>			ValueExpression;
	FProperty*								ValueProperty = nullptr;

	TWeakObjectPtr<UScriptStruct>			BindingExpression;
	FNameProperty*							BindingNameField = nullptr;

	TArray<TWeakObjectPtr<UScriptStruct>>	OperationExpressions;

	bool IsValid() const;
	bool ContainsExpression(const FInstancedStruct* Expresssion) const;

	bool IsBindingExpression(const FInstancedStruct* Expression) const;
	bool IsValueExpression(const FInstancedStruct* Expression) const;
	bool IsOperationExpression(const FInstancedStruct* Expression) const;

	FName GetBindingName(const FInstancedStruct* Expression) const;
	FInstancedStruct MakeBindingStruct(FName BindingName) const;

	static const FNiagaraStatelessExpressionTypeData& GetTypeData(const FNiagaraTypeDefinition& TypeDef);
	static const FNiagaraStatelessExpressionTypeData& GetTypeData(const FInstancedStruct* Expression);
};
