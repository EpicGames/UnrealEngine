// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraStatelessExpressionWidget.h"

#include "Stateless/NiagaraStatelessExpression.h"
#include "NiagaraStatelessExpressionTypeData.h"

#include "NiagaraEditorModule.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraEditorStyle.h"

#include "Curves/CurveOwnerInterface.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "NiagaraEditorStyle.h"
#include "SColorGradientEditor.h"
#include "SNiagaraParameterEditor.h"
#include "TypeEditorUtilities/NiagaraFloatTypeEditorUtilities.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/NiagaraDistributionEditorUtilities.h"
#include "Widgets/SNiagaraColorEditor.h"
#include "Widgets/SNiagaraDistributionCurveEditor.h"
#include "Widgets/SNiagaraParameterName.h"

#include "Modules/ModuleManager.h"
#include "IPropertyRowGenerator.h"
#include "PropertyEditorModule.h"
#include "IDetailTreeNode.h"
#include "IStructureDetailsView.h"

#include "SGraphActionMenu.h"

#define LOCTEXT_NAMESPACE "NiagaraStatelessExpression"

/////////////////////////////////////////////////////////////////////////////////////// 
namespace SNiagaraStatelessExpressionPrivate
{
	DECLARE_DELEGATE_TwoParams(FOnExecuteTransaction, FText, TFunction<void()>)
	DECLARE_DELEGATE_RetVal(TArray<FNiagaraVariableBase>, FOnGetAvailableBindings)

	/////////////////////////////////////////////////////////////////////////////////////////////////// 

	struct FSelectExpressionTypeAction : public FEdGraphSchemaAction
	{
		FSelectExpressionTypeAction() : FEdGraphSchemaAction() {}

		FSelectExpressionTypeAction(FInstancedStruct InExpressionStruct, FText InCategory, FText InDisplayName, FText InToolTipText)
			: FEdGraphSchemaAction(
				MoveTemp(InCategory),
				MoveTemp(InDisplayName),
				MoveTemp(InToolTipText),
				0,
				FText()
			)
			, ExpressionStruct(MoveTemp(InExpressionStruct))
		{
		}

		//~ Begin FEdGraphSchemaAction Interface
		virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override
		{
			return nullptr;
		}
		//~ End FEdGraphSchemaAction Interface

		FInstancedStruct ExpressionStruct;
	};

	class SExpressionTypeSelector : public SCompoundWidget
	{
	public:
		DECLARE_DELEGATE_OneParam(FOnActionSelected, const FInstancedStruct&)

		SLATE_BEGIN_ARGS(SExpressionTypeSelector) { }
			SLATE_ARGUMENT(FNiagaraTypeDefinition,	ExpressionTypeDef)
			SLATE_EVENT(FOnActionSelected,			OnActionSelected)
			SLATE_EVENT(FOnGetAvailableBindings,	OnGetAvailableBindings)
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs)
		{
			ExpressionTypeDef		= InArgs._ExpressionTypeDef;
			ActionSelected			= InArgs._OnActionSelected;
			GetAvailableBindings	= InArgs._OnGetAvailableBindings;

			ChildSlot
			[
				SNew(SComboButton)					
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FSlateColor::UseForeground())
				.OnGetMenuContent(this, &SExpressionTypeSelector::OnGetMenuContent)
				.ContentPadding(2)
				.MenuPlacement(MenuPlacement_BelowRightAnchor)
				//.ToolTipText(this, &SExpressionTypeSelector::GetTooltipText)
			];
		}

		TSharedRef<SWidget> OnGetMenuContent() const
		{
			FGraphActionMenuBuilder MenuBuilder;

			return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(5)
				[
					SNew(SBox)
					[
						SNew(SGraphActionMenu)
						.OnActionSelected(this, &SExpressionTypeSelector::OnActionSelected)
						.OnCreateWidgetForAction(this, &SExpressionTypeSelector::OnCreateWidgetForAction)
						.OnCollectAllActions(this, &SExpressionTypeSelector::CollectAllActions)
						.AutoExpandActionMenu(true)
						.ShowFilterTextBox(true)
					]
				];
		}

		void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions, ESelectInfo::Type InSelectionType) const
		{
			if ((InSelectionType != ESelectInfo::OnMouseClick && InSelectionType != ESelectInfo::OnKeyPress))
			{
				return;
			}

			FSlateApplication::Get().DismissAllMenus();

			for (const TSharedPtr<FEdGraphSchemaAction>& SelectedAction : SelectedActions)
			{
				if (SelectedAction.IsValid() == false)
				{
					continue;
				}

				ActionSelected.ExecuteIfBound(static_cast<FSelectExpressionTypeAction*>(SelectedAction.Get())->ExpressionStruct);
			}
		}

		TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData) const
		{
			const FNiagaraStatelessExpressionTypeData& TypeData = FNiagaraStatelessExpressionTypeData::GetTypeData(ExpressionTypeDef);
			if (TypeData.IsValid() == false)
			{
				return SNullWidget::NullWidget;
			}

			FSelectExpressionTypeAction* ExpressionAction = static_cast<FSelectExpressionTypeAction*>(InCreateData->Action.Get());
			if (ExpressionAction->ExpressionStruct.GetScriptStruct() == TypeData.BindingExpression)
			{
				const FName BindingName = TypeData.GetBindingName(&ExpressionAction->ExpressionStruct);

				return
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SNiagaraParameterName)
						.ParameterName(BindingName)
						.IsReadOnly(true)
					];
			}
			else
			{
				return
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(InCreateData->Action->GetMenuDescription())
						.ToolTipText(InCreateData->Action->GetTooltipDescription())
					];
			}
		}

		void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) const
		{
			const FNiagaraStatelessExpressionTypeData& TypeData = FNiagaraStatelessExpressionTypeData::GetTypeData(ExpressionTypeDef);
			if (TypeData.IsValid() == false)
			{
				return;
			}

			// Add Value expression
			if (UScriptStruct* ValueExpression = TypeData.ValueExpression.Get())
			{
				OutAllActions.AddAction(
					MakeShared<FSelectExpressionTypeAction>(
						FInstancedStruct(ValueExpression),
						FText(),
						LOCTEXT("NewLocalValue", "New Local Value"),
						LOCTEXT("NewLocalValueTooltip", "Set to a constant value")
					)
				);
			}

			// Add Binding expressions
			if (UScriptStruct* BindingExpression = TypeData.BindingExpression.Get())
			{
				if (GetAvailableBindings.IsBound())
				{
					const TArray<FNiagaraVariableBase> AvailableBindings = GetAvailableBindings.Execute();
					for (const FNiagaraVariableBase& Binding : AvailableBindings)
					{
						if (Binding.GetType() != ExpressionTypeDef)
						{
							continue;
						}

						OutAllActions.AddAction(
							MakeShared<FSelectExpressionTypeAction>(
								TypeData.MakeBindingStruct(Binding.GetName()),
								LOCTEXT("LinkInput", "Link Input"),
								FText::FromName(Binding.GetName()),
								LOCTEXT("LinkInputTooltip", "Set the parameter")	//-TOOD:
							)
						);
					}
				}
			}

			// Add Operation expressions
			for (TWeakObjectPtr<UScriptStruct> WeakExpressionStruct : TypeData.OperationExpressions)
			{
				if (UScriptStruct* ExpressionStruct = WeakExpressionStruct.Get())
				{
					OutAllActions.AddAction(
						MakeShared<FSelectExpressionTypeAction>(
							FInstancedStruct(ExpressionStruct),
							LOCTEXT("Operation", "Operation"),
							ExpressionStruct->GetDisplayNameText(),
							ExpressionStruct->GetToolTipText()
						)
					);
				}
			}
		}

		FNiagaraTypeDefinition	ExpressionTypeDef;
		FOnActionSelected		ActionSelected;
		FOnGetAvailableBindings	GetAvailableBindings;
	};

	/////////////////////////////////////////////////////////////////////////////////////////////////// 

	class SExpressionWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SExpressionWidget) { }
			SLATE_EVENT(FOnGetAvailableBindings,	OnGetAvailableBindings)
			SLATE_EVENT(FOnExecuteTransaction,		OnExecuteTransaction)
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs, FInstancedStruct* InExpressionStruct, int32 InDepth, FText InDisplayName)
		{
			Depth					= InDepth;
			DisplayName				= InDisplayName;
			ExpressionStruct		= InExpressionStruct;
			WidgetContainer			= SNew(SVerticalBox);
			GetAvailableBindings	= InArgs._OnGetAvailableBindings;
			ExecuteTransaction		= InArgs._OnExecuteTransaction;

			ChildSlot
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(10 * Depth, 0, 0, 0)
				[
					WidgetContainer.ToSharedRef()
				]
			];

			RebuildChildren();
		}

		void Construct(const FArguments& InArgs, FInstancedStruct* InExpressionStruct)
		{
			Construct(InArgs, InExpressionStruct, 0, FText());
		}

	private:
		TSharedRef<SWidget> GetExpressionWidget() const
		{
			const FNiagaraStatelessExpressionTypeData& TypeData = FNiagaraStatelessExpressionTypeData::GetTypeData(ExpressionStruct);

			// Show binding information
			if (TypeData.IsBindingExpression(ExpressionStruct))
			{
				return
					SNew(SNiagaraParameterName)
					.ParameterName(TypeData.GetBindingName(ExpressionStruct))
					.IsReadOnly(true);
			}

			// Show editable value widget		
			if (TypeData.IsValueExpression(ExpressionStruct))
			{
				FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
				TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(TypeData.TypeDef);
				if (TypeEditorUtilities.IsValid())
				{
					TSharedRef<FStructOnScope> ValueStructOnScope = MakeShared<FStructOnScope>(
						TypeData.TypeDef.GetStruct(),
						const_cast<uint8*>(TypeData.ValueProperty->ContainerPtrToValuePtr<uint8>(ExpressionStruct->GetMemory()))	//-TODO: Remove?
					);

					FNiagaraInputParameterCustomization CustomizationOptions;
					CustomizationOptions.bBroadcastValueChangesOnCommitOnly = true;
					TSharedPtr<SNiagaraParameterEditor> ValueParameterEditor = TypeEditorUtilities->CreateParameterEditor(TypeData.TypeDef, EUnit::Unspecified, CustomizationOptions);
					ValueParameterEditor->UpdateInternalValueFromStruct(ValueStructOnScope);
					ValueParameterEditor->SetOnValueChanged(
						SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &SExpressionWidget::OnValueChanged, ValueParameterEditor, ValueStructOnScope)
					);

					return
						ValueParameterEditor.ToSharedRef();
				}
				return SNullWidget::NullWidget;
			}

			// Show selected expression
			check(TypeData.IsOperationExpression(ExpressionStruct));

			return
				SNew(STextBlock)
				.Text(ExpressionStruct->GetScriptStruct()->GetDisplayNameText())
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText");
		}

		TSharedRef<SWidget> CreateTypeSelector()
		{
			const FNiagaraStatelessExpressionTypeData& TypeData = FNiagaraStatelessExpressionTypeData::GetTypeData(ExpressionStruct);
			return
				SNew(SExpressionTypeSelector)
				.ExpressionTypeDef(TypeData.TypeDef)
				.OnActionSelected(this, &SExpressionWidget::OnChangeExpression)
				.OnGetAvailableBindings(GetAvailableBindings);
		}

		void RebuildChildren()
		{
			// Clear existing
			WidgetContainer->ClearChildren();

			// Build row
			{
				TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

				if (DisplayName.IsEmpty() == false)
				{
					HorizontalBox->AddSlot()
					.Padding(0, 0, 5, 0)
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(DisplayName)
					];
				}

				HorizontalBox->AddSlot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					GetExpressionWidget()
				];

				HorizontalBox->AddSlot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					CreateTypeSelector()
				];

				WidgetContainer->AddSlot()
				.VAlign(VAlign_Top)
				.AutoHeight()
				[
					HorizontalBox
				];
			}

			// For operations recurse into child structures
			const FNiagaraStatelessExpressionTypeData& TypeData = FNiagaraStatelessExpressionTypeData::GetTypeData(ExpressionStruct);
			if (TypeData.IsOperationExpression(ExpressionStruct))
			{
				for (TFieldIterator<FProperty> PropertyIt(ExpressionStruct->GetScriptStruct()); PropertyIt; ++PropertyIt)
				{
					const FStructProperty* StructProperty = CastField<const FStructProperty>(*PropertyIt);
					if (StructProperty == nullptr || StructProperty->Struct != FInstancedStruct::StaticStruct())
					{
						continue;
					}

					FInstancedStruct* InnerExpressionStruct = StructProperty->ContainerPtrToValuePtr<FInstancedStruct>(ExpressionStruct->GetMutableMemory());

					WidgetContainer->AddSlot()
					.VAlign(VAlign_Top)
					.AutoHeight()
					[
						SNew(SExpressionWidget, InnerExpressionStruct, Depth + 1, StructProperty->GetDisplayNameText())
						.OnGetAvailableBindings(GetAvailableBindings)
						.OnExecuteTransaction(ExecuteTransaction)
					];
				}
			}
		}

		void OnChangeExpression(const FInstancedStruct& NewExpression)
		{
			WidgetContainer->ClearChildren();

			ExecuteTransaction.ExecuteIfBound(
				LOCTEXT("ChangeExpression", " Change Expression"),		//-TODO: Make more verbose
				[&]()
				{
					ExpressionStruct->InitializeAs(NewExpression.GetScriptStruct(), NewExpression.GetMemory());
				}
			);

			RebuildChildren();
		}

		void OnValueChanged(TSharedPtr<SNiagaraParameterEditor> ParameterEditor, TSharedRef<FStructOnScope> ValueStructOnScope) const
		{
			ExecuteTransaction.ExecuteIfBound(
				LOCTEXT("ChangeValue", " Change Expression Value"),		//-TODO: Make more verbose
				[&]()
				{
					ParameterEditor->UpdateStructFromInternalValue(ValueStructOnScope);
				}
			);
		}

	private:
		int32						Depth = 0;
		FText						DisplayName;
		FInstancedStruct*			ExpressionStruct = nullptr;
		TSharedPtr<SVerticalBox>	WidgetContainer;
		FOnGetAvailableBindings		GetAvailableBindings;
		FOnExecuteTransaction		ExecuteTransaction;
	};
}

/////////////////////////////////////////////////////////////////////////////////////////////////// 

void SNiagaraStatelessExpressionWidget::Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter)
{
	DistributionAdapter = InDistributionAdapter;

	// Get the type data
	FInstancedStruct& ExpressionStruct = DistributionAdapter->GetExpressionRoot();
	const FNiagaraStatelessExpressionTypeData& TypeData = FNiagaraStatelessExpressionTypeData::GetTypeData(DistributionAdapter->GetExpressionTypeDef());
	if (TypeData.IsValid() == false)
	{
		checkNoEntry();		//-TODO:
		return;
	}

	// Make sure root is initialized, all other expression should self initialize and provide the type requirements
	if (TypeData.ContainsExpression(&ExpressionStruct) == false)
	{
		ExpressionStruct.InitializeAs(TypeData.ValueExpression.Get());
	}

	ChildSlot
	[
		SNew(SNiagaraStatelessExpressionPrivate::SExpressionWidget, &ExpressionStruct)
		.OnGetAvailableBindings(this, &SNiagaraStatelessExpressionWidget::GetAvailableBindings)
		.OnExecuteTransaction(this, &SNiagaraStatelessExpressionWidget::ExecuteTransaction)
	];

#if 0


	//const FNiagaraTypeDefinition TypeDef = DistributionAdapter->GetExpressionTypeDef();


	//FStructIterat

	//-TODO: Check to see if the expression is valid or not, if not create a default entry
	//FInstancedStruct* RootExpression = DistributionAdapter->GetRootExpression()

	//FInstancedStruct* RootExpression = DistributionAdapter->GetRootExpression();
	//DistributionAdapter->GetExpressionTypeDef()
	// Constant
	// Operation	:dropdown:
	// - Constant	:dropdown: (inline value)
	// - Operation	:dropdown:
	// -- Constant	:dropdown: (inline value)
	// -- Binding	:dropdown: (inline value)

	// For Each Property
	// - FInstanceStruct -> Drop Down Selection for the correct type def
	// - FName -> Show binding information

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
#if 0
	FPropertyRowGeneratorArgs Args;
	//Args.NotifyHook = this;
	PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(Args);
	PropertyRowGenerator->SetStructure(InDistributionAdapter->MakeStructOnScope());

	TArray<TSharedRef<IDetailTreeNode>> RootNodes = PropertyRowGenerator->GetRootTreeNodes();
	if (RootNodes.Num() == 1)
	{
		static TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
		RootNodes[0]->GetChildren(ChildNodes);

		FNodeWidgets NodeWidgets = ChildNodes[1]->CreateNodeWidgets();
		ChildSlot
		[
			NodeWidgets.ValueWidget.ToSharedRef()
		];
	}
#else
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowSearch = false;
	//DetailsViewArgs.NotifyHook = DebugHudSettings;

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	TSharedRef<IStructureDetailsView> StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(
		DetailsViewArgs, StructureViewArgs, DistributionAdapter->MakeStructOnScope()
	);

	ChildSlot
	[
		StructureDetailsView->GetWidget().ToSharedRef()
	];
#endif
	//FNodeWidgets NodeWidgets = InItem->CreateNodeWidgets();
	//if (NodeWidgets.WholeRowWidget.IsValid())
	//{
	//	TableRow->SetContent(NodeWidgets.WholeRowWidget.ToSharedRef());
	//}
	//else if (NodeWidgets.NameWidget.IsValid() && NodeWidgets.ValueWidget.IsValid())
	//{
	//	TableRow->SetContent(
	//		SNew(SHorizontalBox)
	//		+ SHorizontalBox::Slot()
	//		[
	//			NodeWidgets.NameWidget.ToSharedRef()
	//		]
	//		+ SHorizontalBox::Slot()
	//		[
	//			NodeWidgets.ValueWidget.ToSharedRef()
	//		]
	//	);
	//}
#endif
}

TArray<FNiagaraVariableBase> SNiagaraStatelessExpressionWidget::GetAvailableBindings() const
{
	return DistributionAdapter->GetAvailableBindings();
}

void SNiagaraStatelessExpressionWidget::ExecuteTransaction(FText TransactionText, TFunction<void()> TransactionFunc)
{
	DistributionAdapter->ExecuteTransaction(MoveTemp(TransactionText), MoveTemp(TransactionFunc));
}

#undef LOCTEXT_NAMESPACE
