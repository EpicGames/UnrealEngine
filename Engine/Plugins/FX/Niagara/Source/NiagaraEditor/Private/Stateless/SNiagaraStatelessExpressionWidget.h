// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Widgets/INiagaraDistributionAdapter.h"

class INiagaraDistributionAdapter;
class IPropertyRowGenerator;

class SNiagaraStatelessExpressionWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStatelessExpressionWidget) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter);

private:
	TArray<FNiagaraVariableBase> GetAvailableBindings() const;
	void ExecuteTransaction(FText TransactionText, TFunction<void()> TransactionFunc);

private:
	TSharedPtr<INiagaraDistributionAdapter>	DistributionAdapter;
	TSharedPtr<IPropertyRowGenerator>		PropertyRowGenerator;
};
