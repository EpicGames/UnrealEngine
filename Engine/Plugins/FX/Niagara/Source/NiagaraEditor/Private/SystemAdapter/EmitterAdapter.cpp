// Copyright Epic Games, Inc. All Rights Reserved.

#include "SystemAdapter/EmitterAdapter.h"

#include "NiagaraConstants.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraSystem.h"
#include "SystemAdapter/EmitterAdapter.h"
#include "SystemAdapter/StackScriptAdapter.h"

namespace UE::Niagara
{
	class FEmitterScriptAdapterFactory : public TAdapterPtr<UNiagaraSystem, FStackScriptAdapter>::IFactory
	{
	public:
		FEmitterScriptAdapterFactory(const FGuid& InEmitterHandleId, ENiagaraScriptUsage InUsage)
			: EmitterHandleId(InEmitterHandleId)
			, Usage(InUsage)
		{
		}

		virtual TSharedRef<FStackScriptAdapter> CreateAdapter(UNiagaraSystem* InSourceObject) const override
		{
			return CreateAdapterInternal(InSourceObject);
		}

		virtual TSharedRef<FStackScriptAdapter> CreateAdapter(const UNiagaraSystem* InSourceObject) const override
		{
			return CreateAdapterInternal(InSourceObject);
		}

	private:
		template<typename TSystem>
		TSharedRef<FStackScriptAdapter> CreateAdapterInternal(TSystem OwningSystem) const
		{
			if (OwningSystem != nullptr)
			{
				const FNiagaraEmitterHandle* EmitterHandle = OwningSystem->GetEmitterHandles().FindByPredicate([this](const FNiagaraEmitterHandle& EmitterHandle)
					{ return EmitterHandle.GetId() == EmitterHandleId; });
				if (EmitterHandle != nullptr)
				{
					FVersionedNiagaraEmitterData* EmitterData = EmitterHandle->GetEmitterData();
					switch (Usage)
					{
					case ENiagaraScriptUsage::EmitterSpawnScript:
						return FStackScriptAdapter::Create(OwningSystem, EmitterHandleId, EmitterData->EmitterSpawnScriptProps.Script);
					case ENiagaraScriptUsage::EmitterUpdateScript:
						return FStackScriptAdapter::Create(OwningSystem, EmitterHandleId, EmitterData->EmitterUpdateScriptProps.Script);
					case ENiagaraScriptUsage::ParticleSpawnScript:
					case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
						return FStackScriptAdapter::Create(OwningSystem, EmitterHandleId, EmitterData->SpawnScriptProps.Script);
					case ENiagaraScriptUsage::ParticleUpdateScript:
						return FStackScriptAdapter::Create(OwningSystem, EmitterHandleId, EmitterData->UpdateScriptProps.Script);
					}
				}
			}
			return MakeShared<FStackScriptAdapter>();
		}

	private:
		FGuid EmitterHandleId;
		ENiagaraScriptUsage Usage;
	};

	TSharedRef<FEmitterAdapter> FEmitterAdapter::Create(UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle)
	{
		TSharedRef<FEmitterAdapter> EmitterAdapter = MakeShared<FEmitterAdapter>();
		EmitterAdapter->Initialize(InOwningSystem, InEmitterHandle);
		return EmitterAdapter;
	}

	TSharedRef<FEmitterAdapter> FEmitterAdapter::Create(const UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle)
	{
		TSharedRef<FEmitterAdapter> EmitterAdapter = MakeShared<FEmitterAdapter>();
		EmitterAdapter->Initialize(InOwningSystem, InEmitterHandle);
		return EmitterAdapter;
	}

	TSharedRef<const FEmitterAdapter> FEmitterAdapter::CreateConst(const UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle)
	{
		TSharedRef<FEmitterAdapter> EmitterAdapter = MakeShared<FEmitterAdapter>();
		EmitterAdapter->Initialize(InOwningSystem, InEmitterHandle);
		return EmitterAdapter;
	}

	bool FEmitterAdapter::IsValidAdapter() const
	{
		return OwningSystemConstWeak.Get() != nullptr && EmitterHandleId.IsValid();
	}

	bool FEmitterAdapter::IsValidWriteAdapter() const
	{
		return OwningSystemWeak.IsSet() && OwningSystemWeak.GetValue().Get() != nullptr && EmitterHandleId.IsValid();
	}

	TSharedRef<FStackScriptAdapter> FEmitterAdapter::GetEmitterSpawnScript()
	{
		return EmitterSpawnScript.Get();
	}

	TSharedRef<const FStackScriptAdapter> FEmitterAdapter::GetEmitterSpawnScript() const
	{
		return EmitterSpawnScript.Get();
	}

	TSharedRef<FStackScriptAdapter> FEmitterAdapter::GetEmitterUpdateScript()
	{
		return EmitterUpdateScript.Get();
	}

	TSharedRef<const FStackScriptAdapter> FEmitterAdapter::GetEmitterUpdateScript() const
	{
		return EmitterUpdateScript.Get();
	}

	TSharedRef<FStackScriptAdapter> FEmitterAdapter::GetParticleSpawnScript()
	{
		return ParticleSpawnScript.Get();
	}

	TSharedRef<const FStackScriptAdapter> FEmitterAdapter::GetParticleSpawnScript() const
	{
		return ParticleSpawnScript.Get();
	}

	TSharedRef<FStackScriptAdapter> FEmitterAdapter::GetParticleUpdateScript()
	{
		return ParticleUpdateScript.Get();
	}

	TSharedRef<const FStackScriptAdapter> FEmitterAdapter::GetParticleUpdateScript() const
	{
		return ParticleUpdateScript.Get();
	}


	int32 FEmitterAdapter::GetNumScripts() const
	{
		return 4;
	}

	TSharedRef<FStackScriptAdapter> FEmitterAdapter::GetScriptAt(int32 Index)
	{
		checkf(Index > INDEX_NONE && Index < GetNumScripts(), TEXT("Index out of range"));
		switch (Index)
		{
		case 0:
			return GetEmitterSpawnScript();
		case 1:
			return GetEmitterUpdateScript();
		case 2:
			return GetParticleSpawnScript();
		default: // case 3:
			return GetParticleUpdateScript();
		}
	}

	FScriptSharedRefCollection FEmitterAdapter::GetScripts()
	{
		return FScriptSharedRefCollection(&StackScriptAdapterCollection);
	}

	FScriptConstSharedRefCollection FEmitterAdapter::GetScripts() const
	{
		return FScriptConstSharedRefCollection(&StackScriptAdapterCollection);
	}

	void FEmitterAdapter::Initialize(UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle)
	{
		OwningSystemWeak = InOwningSystem;
		OwningSystemConstWeak = InOwningSystem;
		EmitterName = *InEmitterHandle.GetUniqueInstanceName();
		EmitterHandleId = InEmitterHandle.GetId();
		EmitterSpawnScript.Initialize(InOwningSystem, MakeShared<FEmitterScriptAdapterFactory>(EmitterHandleId, ENiagaraScriptUsage::EmitterSpawnScript));
		EmitterUpdateScript.Initialize(InOwningSystem, MakeShared<FEmitterScriptAdapterFactory>(EmitterHandleId, ENiagaraScriptUsage::EmitterUpdateScript));
		ParticleSpawnScript.Initialize(InOwningSystem, MakeShared<FEmitterScriptAdapterFactory>(EmitterHandleId, ENiagaraScriptUsage::ParticleSpawnScript));
		ParticleUpdateScript.Initialize(InOwningSystem, MakeShared<FEmitterScriptAdapterFactory>(EmitterHandleId, ENiagaraScriptUsage::ParticleUpdateScript));
		StackScriptAdapterCollection.Initialize(this->AsShared());
	}

	void FEmitterAdapter::Initialize(const UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle)
	{
		OwningSystemConstWeak = InOwningSystem;
		EmitterName = *InEmitterHandle.GetUniqueInstanceName();
		EmitterHandleId = InEmitterHandle.GetId();
		EmitterSpawnScript.Initialize(InOwningSystem, MakeShared<FEmitterScriptAdapterFactory>(EmitterHandleId, ENiagaraScriptUsage::EmitterSpawnScript));
		EmitterUpdateScript.Initialize(InOwningSystem, MakeShared<FEmitterScriptAdapterFactory>(EmitterHandleId, ENiagaraScriptUsage::EmitterUpdateScript));
		ParticleSpawnScript.Initialize(InOwningSystem, MakeShared<FEmitterScriptAdapterFactory>(EmitterHandleId, ENiagaraScriptUsage::ParticleSpawnScript));
		ParticleUpdateScript.Initialize(InOwningSystem, MakeShared<FEmitterScriptAdapterFactory>(EmitterHandleId, ENiagaraScriptUsage::ParticleUpdateScript));
		StackScriptAdapterCollection.Initialize(this->AsShared());
	}

} // namespace UE::Niagara