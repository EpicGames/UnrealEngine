// Copyright Epic Games, Inc. All Rights Reserved.

#include "SystemAdapter/StackFunctionAdapter.h"

#include "NiagaraEmitterHandle.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraSystem.h"
#include "SystemAdapter/StackFunctionInputAdapter.h"
#include "TraversalCache/StaticContextBuilder.h"
#include "TraversalCache/TraversalCache.h"
#include "TraversalCache/TraversalShared.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

namespace UE::Niagara
{
	class FStackFunctionInputAdapterFactory : public TAdapterRefListThreeSources<UNiagaraSystem, UNiagaraScript, UNiagaraNodeFunctionCall, FStackFunctionInputAdapter>::IFactory
	{
	public:
		FStackFunctionInputAdapterFactory(FGuid InOwningEmitterHandleId)
			: OwningEmitterHandleId(InOwningEmitterHandleId)
		{
		}

		virtual void CreateAdapters(UNiagaraSystem* OwningSystem, UNiagaraScript* OwningScript, UNiagaraNodeFunctionCall* OwningFunctionCallNode, TArray<TSharedRef<FStackFunctionInputAdapter>>& OutAdapters) const override
		{
			CreateAdaptersInternal(OwningSystem, OwningScript, OwningFunctionCallNode, OutAdapters);
		}

		virtual void CreateAdapters(const UNiagaraSystem* OwningSystem, const UNiagaraScript* OwningScript, const UNiagaraNodeFunctionCall* OwningFunctionCallNode, TArray<TSharedRef<FStackFunctionInputAdapter>>& OutAdapters) const override
		{
			CreateAdaptersInternal(OwningSystem, OwningScript, OwningFunctionCallNode, OutAdapters);
		}

	private:
		template<typename TSystem, typename TScript, typename TFunction>
		void CreateAdaptersInternal(TSystem* OwningSystem, TScript* OwningScript, TFunction* OwningFunctionCallNode, TArray<TSharedRef<FStackFunctionInputAdapter>>& OutAdapters) const
		{
			using namespace UE::Niagara::TraversalCache;
			FVersionedNiagaraEmitterData* OwningEmitterData = nullptr;
			if (OwningEmitterHandleId.IsValid())
			{
				const FNiagaraEmitterHandle* OwningEmitterHandle = OwningSystem->GetEmitterHandles().FindByPredicate([this](const FNiagaraEmitterHandle& EmitterHandle)
					{ return EmitterHandle.GetId() == OwningEmitterHandleId; });
				if (OwningEmitterHandle == nullptr)
				{
					return;
				}
				OwningEmitterData = OwningEmitterHandle->GetEmitterData();
			}

			FTopLevelScriptStaticContext StaticContext;
			FStaticContextBuilder::CreateTopLevelScriptContext(*OwningSystem, OwningEmitterData, *OwningScript, StaticContext);
			TArray<FNiagaraVariable> InputReads;
			TSet<FNiagaraVariable> HiddenInputReads;
			FTraversalCache::GetStackFunctionReads(StaticContext, *OwningFunctionCallNode, InputReads, HiddenInputReads, TraversalCache::EStackFunctionReadFilterFlags::InputsOnly);

			for (const FNiagaraVariable& InputRead : InputReads)
			{
				OutAdapters.Add(FStackFunctionInputAdapter::Create(OwningFunctionCallNode, InputRead));
			}
		}

	private:
		FGuid OwningEmitterHandleId;
	};

	TSharedRef<FStackFunctionAdapter> FStackFunctionAdapter::Create(UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, UNiagaraScript* InOwningScript, UNiagaraNodeFunctionCall* InFunctionCallNode)
	{
		TSharedRef<FStackFunctionAdapter> FunctionAdapter = MakeShared<FStackFunctionAdapter>();
		FunctionAdapter->Initialize(InOwningSystem, InOwningEmitterHandleId, InOwningScript, InFunctionCallNode);
		return FunctionAdapter;
	}

	TSharedRef<FStackFunctionAdapter> FStackFunctionAdapter::Create(const UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, const UNiagaraScript* InOwningScript, const UNiagaraNodeFunctionCall* InFunctionCallNode)
	{
		TSharedRef<FStackFunctionAdapter> FunctionAdapter = MakeShared<FStackFunctionAdapter>();
		FunctionAdapter->Initialize(InOwningSystem, InOwningEmitterHandleId, InOwningScript, InFunctionCallNode);
		return FunctionAdapter;
	}

	TSharedRef<const FStackFunctionAdapter> FStackFunctionAdapter::CreateConst(const UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, const UNiagaraScript* InOwningScript, const UNiagaraNodeFunctionCall* InFunctionCallNode)
	{
		TSharedRef<FStackFunctionAdapter> FunctionAdapter = MakeShared<FStackFunctionAdapter>();
		FunctionAdapter->Initialize(InOwningSystem, InOwningEmitterHandleId, InOwningScript, InFunctionCallNode);
		return FunctionAdapter;
	}

	bool FStackFunctionAdapter::IsValidAdapter() const
	{
		return FunctionCallNodeConstWeak.Get() != nullptr;
	}

	bool FStackFunctionAdapter::IsValidWriteAdapter() const
	{
		return FunctionCallNodeWeak.IsSet() && FunctionCallNodeWeak.GetValue().Get() != nullptr;
	}

	FString FStackFunctionAdapter::GetFunctionName() const
	{
		const UNiagaraNodeFunctionCall* FunctionCallNode = FunctionCallNodeConstWeak.Get();
		if (FunctionCallNode != nullptr)
		{
			return FunctionCallNode->GetFunctionName();
		}
		return FString();
	}

	void FStackFunctionAdapter::GetSetParameters(TArray<FNiagaraVariableBase>& OutParameters) const
	{
		const UNiagaraNodeAssignment* AssignmentNode = Cast<const UNiagaraNodeAssignment>(FunctionCallNodeConstWeak.Get());
		if (AssignmentNode != nullptr)
		{
			for (const FNiagaraVariable& AssignmentTarget : AssignmentNode->GetAssignmentTargets())
			{
				OutParameters.Add(AssignmentTarget);
			}
		}
	}

	bool FStackFunctionAdapter::SetsParameter(const FNiagaraVariableBase& InParameter) const
	{
		const UNiagaraNodeAssignment* AssignmentNode = Cast<const UNiagaraNodeAssignment>(FunctionCallNodeConstWeak.Get());
		return AssignmentNode != nullptr && AssignmentNode->GetAssignmentTargets().Contains(InParameter);
	}

	TSharedRefCollection<FStackFunctionInputAdapter> FStackFunctionAdapter::GetInputs()
	{
		return Inputs.Get();
	}

	TConstSharedRefCollection<FStackFunctionInputAdapter> FStackFunctionAdapter::GetInputs() const
	{
		return Inputs.Get();
	}

	TSharedPtr<FStackFunctionInputAdapter> FStackFunctionAdapter::FindInputByName(FName InputName)
	{
		return Inputs.Get().FindByPredicate([&InputName](const TSharedRef<FStackFunctionInputAdapter>& Input)
			{ return Input->GetName() == InputName; });
	}

	TSharedPtr<const FStackFunctionInputAdapter> FStackFunctionAdapter::FindInputByName(FName InputName) const
	{
		return Inputs.Get().FindByPredicate([&InputName](const TSharedRef<const FStackFunctionInputAdapter>& Input)
			{ return Input->GetName() == InputName; });
	}

	void FStackFunctionAdapter::Initialize(UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, UNiagaraScript* InOwningScript, UNiagaraNodeFunctionCall* InFunctionCallNode)
	{
		FunctionCallNodeWeak = InFunctionCallNode;
		FunctionCallNodeConstWeak = InFunctionCallNode;
		Type = InFunctionCallNode->IsA<UNiagaraNodeAssignment>() ? EFunctionType::SetParametersModule : EFunctionType::ScriptModule;
		Inputs.Initialize(InOwningSystem, InOwningScript, InFunctionCallNode, MakeShared<FStackFunctionInputAdapterFactory>(InOwningEmitterHandleId));
	}

	void FStackFunctionAdapter::Initialize(const UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, const UNiagaraScript* InOwningScript, const UNiagaraNodeFunctionCall* InFunctionCallNode)
	{
		FunctionCallNodeConstWeak = InFunctionCallNode;
		Type = InFunctionCallNode->IsA<UNiagaraNodeAssignment>() ? EFunctionType::SetParametersModule : EFunctionType::ScriptModule;
		Inputs.Initialize(InOwningSystem, InOwningScript, InFunctionCallNode, MakeShared<FStackFunctionInputAdapterFactory>(InOwningEmitterHandleId));
	}

} // namespace UE::Niagara