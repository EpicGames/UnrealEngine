// Copyright Epic Games, Inc. All Rights Reserved.

#include "SystemAdapter/StackFunctionInputAdapter.h"

#include "NiagaraNodeFunctionCall.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"

namespace UE::Niagara
{
	TSharedRef<FStackFunctionInputAdapter> FStackFunctionInputAdapter::Create(UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter)
	{
		TSharedRef<FStackFunctionInputAdapter> FunctionInputAdapter = MakeShared<FStackFunctionInputAdapter>();
		FunctionInputAdapter->Initialize(InFunctionCallNode, InInputParameter);
		return FunctionInputAdapter;
	}

	TSharedRef<FStackFunctionInputAdapter> FStackFunctionInputAdapter::Create(const UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter)
	{
		TSharedRef<FStackFunctionInputAdapter> FunctionInputAdapter = MakeShared<FStackFunctionInputAdapter>();
		FunctionInputAdapter->Initialize(InFunctionCallNode, InInputParameter);
		return FunctionInputAdapter;
	}

	TSharedRef<const FStackFunctionInputAdapter> FStackFunctionInputAdapter::CreateConst(const UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter)
	{
		TSharedRef<FStackFunctionInputAdapter> FunctionInputAdapter = MakeShared<FStackFunctionInputAdapter>();
		FunctionInputAdapter->Initialize(InFunctionCallNode, InInputParameter);
		return FunctionInputAdapter;
	}

	bool FStackFunctionInputAdapter::IsValidAdapter() const
	{
		return FunctionCallNodeConstWeak.Get() != nullptr && InputParameter.IsValid();
	}

	bool FStackFunctionInputAdapter::IsValidWriteAdapter() const
	{
		return FunctionCallNodeWeak.IsSet() && FunctionCallNodeWeak.GetValue().Get() != nullptr && InputParameter.IsValid();
	}

	FName FStackFunctionInputAdapter::GetName() const
	{
		return InputName;
	}

	const FNiagaraTypeDefinition& FStackFunctionInputAdapter::GetType() const
	{
		return InputParameter.GetType();
	}

	bool FStackFunctionInputAdapter::TryGetLocalValue(bool& OutValue) const
	{
		return false;
	}

	bool FStackFunctionInputAdapter::TryGetLocalValue(int32& OutValue) const
	{
		return false;
	}

	bool FStackFunctionInputAdapter::TryGetLocalValue(float& OutValue) const
	{
		return false;
	}

	bool FStackFunctionInputAdapter::TryGetLocalValue(FVector2f& OutValue) const
	{
		return false;
	}

	bool FStackFunctionInputAdapter::TryGetLocalValue(FVector3f& OutValue) const
	{
		return false;
	}

	bool FStackFunctionInputAdapter::TryGetLocalValue(FVector4f& OutValue) const
	{
		return false;
	}

	bool FStackFunctionInputAdapter::TryGetLocalValueData(TNotNull<UStruct*> ValueType, const uint8*& OutData) const
	{
		return false;
	}

	FNiagaraVariableBase FStackFunctionInputAdapter::GetLinkedValue() const
	{
		return FNiagaraVariableBase();
	}

	UNiagaraDataInterface* FStackFunctionInputAdapter::GetDataValue() const
	{
		return nullptr;
	}

	UObject* FStackFunctionInputAdapter::GetObjectValue() const
	{
		return nullptr;
	}

	TSharedPtr<FStackFunctionAdapter> FStackFunctionInputAdapter::GetDynamicValue() const
	{
		return TSharedPtr<FStackFunctionAdapter>();
	}

	void FStackFunctionInputAdapter::Initialize(UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter)
	{
		InputParameter = InInputParameter;
		InputName = FNiagaraParameterHandle(InputParameter.GetName()).GetName();
		FunctionCallNodeWeak = InFunctionCallNode;
		FunctionCallNodeConstWeak = InFunctionCallNode;
	}

	void FStackFunctionInputAdapter::Initialize(const UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter)
	{
		InputParameter = InInputParameter;
		InputName = FNiagaraParameterHandle(InputParameter.GetName()).GetName();
		FunctionCallNodeConstWeak = InFunctionCallNode;
	}

} // namespace UE::Niagara