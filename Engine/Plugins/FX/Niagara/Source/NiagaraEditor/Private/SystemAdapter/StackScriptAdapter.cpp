// Copyright Epic Games, Inc. All Rights Reserved.

#include "SystemAdapter/StackScriptAdapter.h"

#include "NiagaraGraph.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSystem.h"
#include "SystemAdapter/StackFunctionAdapter.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

namespace UE::Niagara
{
	class FStackModuleAdapterFactory : public TAdapterRefListTwoSources<UNiagaraSystem, UNiagaraScript, FStackFunctionAdapter>::IFactory
	{
	public:
		FStackModuleAdapterFactory(FGuid InOwningEmitterHandleId)
			: OwningEmitterHandleId(InOwningEmitterHandleId)
		{
		}

		virtual void CreateAdapters(UNiagaraSystem* OwningSystem, UNiagaraScript* OwningScript, TArray<TSharedRef<FStackFunctionAdapter>>& OutAdapters) const override
		{
			CreateAdaptersInternal(OwningSystem, OwningScript, OutAdapters);
		}

		virtual void CreateAdapters(const UNiagaraSystem* OwningSystem, const UNiagaraScript* OwningScript, TArray<TSharedRef<FStackFunctionAdapter>>& OutAdapters) const override
		{
			CreateAdaptersInternal(OwningSystem, OwningScript, OutAdapters);
		}

	private:
		template<typename TSystem, typename TScript>
		void CreateAdaptersInternal(TSystem* OwningSystem, TScript OwningScript, TArray<TSharedRef<FStackFunctionAdapter>>& OutAdapters) const
		{
			const UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(OwningScript->GetLatestSource());
			UNiagaraNodeOutput* OutputNode = ScriptSource->NodeGraph->FindEquivalentOutputNode(OwningScript->GetUsage(), OwningScript->GetUsageId());
			if (OutputNode != nullptr)
			{
				TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackGroups;
				FNiagaraStackGraphUtilities::GetStackNodeGroups(*OutputNode, StackGroups);
				for (const FNiagaraStackGraphUtilities::FStackNodeGroup& StackGroup : StackGroups)
				{
					UNiagaraNodeFunctionCall* ModuleNode = Cast<UNiagaraNodeFunctionCall>(StackGroup.EndNode);
					if (ModuleNode != nullptr)
					{
						OutAdapters.Add(FStackFunctionAdapter::Create(OwningSystem, OwningEmitterHandleId, OwningScript, ModuleNode));
					}
				}
			}
		}

	private:
		FGuid OwningEmitterHandleId;
	};

	TSharedRef<FStackScriptAdapter> FStackScriptAdapter::Create(UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, UNiagaraScript* InScript)
	{
		TSharedRef<FStackScriptAdapter> ScriptAdapter = MakeShared<FStackScriptAdapter>();
		ScriptAdapter->Initialize(InOwningSystem, InOwningEmitterHandleId, InScript);
		return ScriptAdapter;
	}

	TSharedRef<FStackScriptAdapter> FStackScriptAdapter::Create(const UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, const UNiagaraScript* InScript)
	{
		TSharedRef<FStackScriptAdapter> ScriptAdapter = MakeShared<FStackScriptAdapter>();
		ScriptAdapter->Initialize(InOwningSystem, InOwningEmitterHandleId, InScript);
		return ScriptAdapter;
	}

	TSharedRef<const FStackScriptAdapter> FStackScriptAdapter::CreateConst(const UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, const UNiagaraScript* InScript)
	{
		TSharedRef<FStackScriptAdapter> ScriptAdapter = MakeShared<FStackScriptAdapter>();
		ScriptAdapter->Initialize(InOwningSystem, InOwningEmitterHandleId, InScript);
		return ScriptAdapter;
	}

	bool FStackScriptAdapter::IsValidAdapter() const
	{
		return ScriptConstWeak.Get() != nullptr;
	}

	bool FStackScriptAdapter::IsValidWriteAdapter() const
	{
		return ScriptWeak.IsSet() && ScriptWeak.GetValue().Get() != nullptr;
	}

	ENiagaraScriptUsage FStackScriptAdapter::GetUsage() const
	{
		const UNiagaraScript* ConstScript = ScriptConstWeak.Get();
		return ConstScript != nullptr ? ConstScript->GetUsage() : ENiagaraScriptUsage::Function;
	}

	FGuid FStackScriptAdapter::GetUsageId() const
	{
		const UNiagaraScript* ConstScript = ScriptConstWeak.Get();
		return ConstScript != nullptr ? ConstScript->GetUsageId() : FGuid();
	}

	TSharedRefCollection<FStackFunctionAdapter> FStackScriptAdapter::GetModules()
	{
		return Modules.Get();
	}

	TConstSharedRefCollection<FStackFunctionAdapter> FStackScriptAdapter::GetModules() const
	{
		return Modules.Get();
	}

	TSharedPtr<FStackFunctionAdapter> FStackScriptAdapter::FindModuleByName(const FName& InName)
	{
		FString NameString = InName.ToString();
		return Modules.Get().FindByPredicate([&NameString](const TSharedRef<FStackFunctionAdapter>& ModuleAdapter)
			{ return ModuleAdapter->GetFunctionName() == NameString; });
	}

	TSharedPtr<const FStackFunctionAdapter> FStackScriptAdapter::FindModuleByName(const FName& InName) const
	{
		FString NameString = InName.ToString();
		return Modules.Get().FindByPredicate([&NameString](TSharedRef<const FStackFunctionAdapter> ModuleAdapter)
			{ return ModuleAdapter->GetFunctionName() == NameString; });
	}

	void FStackScriptAdapter::FindSetParametersModulesBySetParameter(const FNiagaraVariableBase& InParameter, TArray<TSharedRef<FStackFunctionAdapter>>& OutSetParametersModules)
	{
		Modules.Get().FindAllByPredicate(OutSetParametersModules, [&InParameter](const TSharedRef<FStackFunctionAdapter>& ModuleAdapter)
			{ return ModuleAdapter->GetType() == FStackFunctionAdapter::EFunctionType::SetParametersModule && ModuleAdapter->SetsParameter(InParameter); });
	}

	void FStackScriptAdapter::FindSetParametersModulesBySetParameter(const FNiagaraVariableBase& InParameter, TArray<TSharedRef<const FStackFunctionAdapter>>& OutSetParametersModules) const
	{
		Modules.Get().FindAllByPredicate(OutSetParametersModules, [&InParameter](const TSharedRef<const FStackFunctionAdapter>& ModuleAdapter)
			{ return ModuleAdapter->GetType() == FStackFunctionAdapter::EFunctionType::SetParametersModule && ModuleAdapter->SetsParameter(InParameter); });
	}

	TSharedPtr<FStackFunctionAdapter> FStackScriptAdapter::FindFirstSetParametersModuleBySetParameter(const FNiagaraVariableBase& InParameter)
	{
		return Modules.Get().FindByPredicate([&InParameter](const TSharedRef<FStackFunctionAdapter>& ModuleAdapter)
			{ return ModuleAdapter->GetType() == FStackFunctionAdapter::EFunctionType::SetParametersModule && ModuleAdapter->SetsParameter(InParameter); });
	}

	TSharedPtr<const FStackFunctionAdapter> FStackScriptAdapter::FindFirstSetParametersModuleBySetParameter(const FNiagaraVariableBase& InParameter) const
	{
		return Modules.Get().FindByPredicate([&InParameter](const TSharedRef<const FStackFunctionAdapter>& ModuleAdapter)
			{ return ModuleAdapter->GetType() == FStackFunctionAdapter::EFunctionType::SetParametersModule && ModuleAdapter->SetsParameter(InParameter); });
	}

	void FStackScriptAdapter::Initialize(UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, UNiagaraScript* InScript)
	{
		ScriptWeak = InScript;
		ScriptConstWeak = InScript;
		Modules.Initialize(InOwningSystem, InScript, MakeShared<FStackModuleAdapterFactory>(InOwningEmitterHandleId));
	}

	void FStackScriptAdapter::Initialize(const UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, const UNiagaraScript* InScript)
	{
		ScriptConstWeak = InScript;
		Modules.Initialize(InOwningSystem, InScript, MakeShared<FStackModuleAdapterFactory>(InOwningEmitterHandleId));
	}
}