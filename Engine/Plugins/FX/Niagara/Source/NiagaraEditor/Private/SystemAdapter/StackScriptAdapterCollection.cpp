// Copyright Epic Games, Inc. All Rights Reserved.

#include "SystemAdapter/StackScriptAdapterCollection.h"

namespace UE::Niagara
{
	void FStackScriptAdapterCollection::Initialize(TSharedRef<IStackScriptAdapterOwner> InScriptOwner)
	{
		ScriptOwnerWeak = InScriptOwner;
	}

	int32 FStackScriptAdapterCollection::Num() const
	{
		TSharedPtr<IStackScriptAdapterOwner> ScriptOwner = ScriptOwnerWeak.Pin();
		return ScriptOwnerWeak.IsValid() ? ScriptOwner->GetNumScripts() : 0;
	}

	TSharedRef<FStackScriptAdapter> FStackScriptAdapterCollection::operator[](int32 Index) const
	{
		TSharedPtr<IStackScriptAdapterOwner> ScriptOwner = ScriptOwnerWeak.Pin();
		checkf(ScriptOwner.IsValid(), TEXT("Script owner is invalid"));
		return ScriptOwner->GetScriptAt(Index);
	}

} // namespace UE::Niagara