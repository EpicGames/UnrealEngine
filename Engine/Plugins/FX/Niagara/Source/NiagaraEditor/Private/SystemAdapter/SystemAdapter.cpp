// Copyright Epic Games, Inc. All Rights Reserved.

#include "SystemAdapter/SystemAdapter.h"

#include "NiagaraConstants.h"
#include "NiagaraSystem.h"
#include "SystemAdapter/EmitterAdapter.h"
#include "SystemAdapter/StackScriptAdapter.h"

namespace UE::Niagara
{
	TSharedRef<FUserParameterAdapter> FUserParameterAdapter::Create(UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset)
	{
		TSharedRef<FUserParameterAdapter> UserParameterAdapter = MakeShared<FUserParameterAdapter>();
		UserParameterAdapter->Initialize(InSystem, InUserParameterWithOffset);
		return UserParameterAdapter;
	}

	TSharedRef<FUserParameterAdapter> FUserParameterAdapter::Create(const UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset)
	{
		TSharedRef<FUserParameterAdapter> UserParameterAdapter = MakeShared<FUserParameterAdapter>();
		UserParameterAdapter->Initialize(InSystem, InUserParameterWithOffset);
		return UserParameterAdapter;
	}

	TSharedRef<const FUserParameterAdapter> FUserParameterAdapter::CreateConst(const UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset)
	{
		TSharedRef<FUserParameterAdapter> UserParameterAdapter = MakeShared<FUserParameterAdapter>();
		UserParameterAdapter->Initialize(InSystem, InUserParameterWithOffset);
		return UserParameterAdapter;
	}

	bool FUserParameterAdapter::IsValidAdapter() const
	{
		return SystemConstWeak.Get() != nullptr;
	}

	bool FUserParameterAdapter::IsValidWriteAdapter() const
	{
		return SystemWeak.IsSet() && SystemWeak.GetValue().Get() != nullptr;
	}

	FName FUserParameterAdapter::GetName() const
	{
		if (NameCache == NAME_None)
		{
			const int32 UserPrefixLength = FNiagaraConstants::UserNamespace.GetStringLength() + 1;
			FString FullNameString = UserParameterWithOffset.GetName().ToString();
			NameCache = FName(FullNameString.RightChop(UserPrefixLength));
		}
		return NameCache;
	}

	bool FUserParameterAdapter::TryGetValue(bool& OutValue) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType() == FNiagaraTypeDefinition::GetBoolDef())
			{
				OutValue = SystemConst->GetExposedParameters().GetParameterValueFromOffset<FNiagaraBool>(UserParameterWithOffset.Offset).GetValue();
				return true;
			}
		}
		return false;
	}

	bool FUserParameterAdapter::TryGetValue(int32& OutValue) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType() == FNiagaraTypeDefinition::GetIntDef())
			{
				OutValue = SystemConst->GetExposedParameters().GetParameterValueFromOffset<FNiagaraInt32>(UserParameterWithOffset.Offset).Value;
				return true;
			}
		}
		return false;
	}

	bool FUserParameterAdapter::TryGetValue(float& OutValue) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType() == FNiagaraTypeDefinition::GetFloatDef())
			{
				OutValue = SystemConst->GetExposedParameters().GetParameterValueFromOffset<FNiagaraFloat>(UserParameterWithOffset.Offset).Value;
				return true;
			}
		}
		return false;
	}

	bool FUserParameterAdapter::TryGetValue(FVector2f& OutValue) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType() == FNiagaraTypeDefinition::GetVec2Def())
			{
				OutValue = SystemConst->GetExposedParameters().GetParameterValueFromOffset<FVector2f>(UserParameterWithOffset.Offset);
				return true;
			}
		}
		return false;
	}

	bool FUserParameterAdapter::TryGetValue(FVector3f& OutValue) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType() == FNiagaraTypeDefinition::GetVec3Def())
			{
				OutValue = SystemConst->GetExposedParameters().GetParameterValueFromOffset<FVector3f>(UserParameterWithOffset.Offset);
				return true;
			}
		}
		return false;
	}

	bool FUserParameterAdapter::TryGetValue(FVector4f& OutValue) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType() == FNiagaraTypeDefinition::GetVec4Def())
			{
				OutValue = SystemConst->GetExposedParameters().GetParameterValueFromOffset<FVector4f>(UserParameterWithOffset.Offset);
				return true;
			}
		}
		return false;
	}

	bool FUserParameterAdapter::TryGetValue(FVector& OutValue) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType() == FNiagaraTypeDefinition::GetPositionDef())
			{
				const FVector* Position = SystemConst->GetExposedParameters().GetPositionParameterValue(UserParameterWithOffset.GetName());
				if (Position != nullptr)
				{
					OutValue = *Position;
					return true;
				}
			}
		}
		return false;
	}

	bool FUserParameterAdapter::TryGetValue(FLinearColor& OutValue) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType() == FNiagaraTypeDefinition::GetColorDef())
			{
				OutValue = SystemConst->GetExposedParameters().GetParameterValueFromOffset<FLinearColor>(UserParameterWithOffset.Offset);
				return true;
			}
		}
		return false;
	}

	bool FUserParameterAdapter::TryGetValue(FQuat4f& OutValue) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType() == FNiagaraTypeDefinition::GetQuatDef())
			{
				OutValue = SystemConst->GetExposedParameters().GetParameterValueFromOffset<FQuat4f>(UserParameterWithOffset.Offset);
				return true;
			}
		}
		return false;
	}

	bool FUserParameterAdapter::TryGetValueData(TNotNull<UStruct*> ValueType, const uint8*& OutData) const
	{
		const UNiagaraSystem* SystemConst = SystemConstWeak.Get();
		if (SystemConst != nullptr)
		{
			if (UserParameterWithOffset.GetType().GetStruct() == ValueType)
			{
				const uint8* Data = SystemConst->GetExposedParameters().GetParameterData(UserParameterWithOffset.Offset, ValueType->GetStructureSize());
				if (Data != nullptr)
				{
					OutData = Data;
					return true;
				}
			}
		}
		return false;
	}

	void FUserParameterAdapter::Initialize(UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset)
	{
		SystemWeak = InSystem;
		SystemConstWeak = InSystem;
		UserParameterWithOffset = InUserParameterWithOffset;
	}

	void FUserParameterAdapter::Initialize(const UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset)
	{
		SystemConstWeak = InSystem;
		UserParameterWithOffset = InUserParameterWithOffset;
	}

	class FUserParameterAdapterFactory : public TAdapterRefListOneSource<UNiagaraSystem, FUserParameterAdapter>::IFactory
	{
	public:
		virtual void CreateAdapters(UNiagaraSystem* SourceObject, TArray<TSharedRef<FUserParameterAdapter>>& OutAdapters) const override
		{
			CreateAdaptersInternal(SourceObject, OutAdapters);
		}

		virtual void CreateAdapters(const UNiagaraSystem* SourceObject, TArray<TSharedRef<FUserParameterAdapter>>& OutAdapters) const override
		{
			CreateAdaptersInternal(SourceObject, OutAdapters);
		}

	private:
		template<typename TSystem>
		static void CreateAdaptersInternal(TSystem* SourceObject, TArray<TSharedRef<FUserParameterAdapter>>& OutAdapters)
		{
			if (SourceObject != nullptr && SourceObject->GetExposedParameters().Num() > 0)
			{
				for (const FNiagaraVariableWithOffset& UserParameter : SourceObject->GetExposedParameters().ReadParameterVariables())
				{
					OutAdapters.Add(FUserParameterAdapter::Create(SourceObject, UserParameter));
				}
			}
		}
	};

	class FEmitterAdapterFactory : public TAdapterRefListOneSource<UNiagaraSystem, FEmitterAdapter>::IFactory
	{
	public:
		virtual void CreateAdapters(UNiagaraSystem* InOwningSystem, TArray<TSharedRef<FEmitterAdapter>>& OutAdapters) const override
		{
			CreateAdaptersInternal(InOwningSystem, OutAdapters);
		}

		virtual void CreateAdapters(const UNiagaraSystem* InOwningSystem, TArray<TSharedRef<FEmitterAdapter>>& OutAdapters) const override
		{
			CreateAdaptersInternal(InOwningSystem, OutAdapters);
		}

	private:
		template<typename TSystem>
		static void CreateAdaptersInternal(TSystem* InOwningSystem, TArray<TSharedRef<FEmitterAdapter>>& OutAdapters)
		{
			if (InOwningSystem == nullptr)
			{
				return;
			}
			for (const FNiagaraEmitterHandle& EmitterHandle : InOwningSystem->GetEmitterHandles())
			{
				OutAdapters.Add(FEmitterAdapter::Create(InOwningSystem, EmitterHandle));
			}
		}
	};

	class FSystemScriptAdapterFactory : public TAdapterPtr<UNiagaraSystem, FStackScriptAdapter>::IFactory
	{
	public:
		FSystemScriptAdapterFactory(ENiagaraScriptUsage InUsage)
			: Usage(InUsage)
		{
		}

		virtual TSharedRef<FStackScriptAdapter> CreateAdapter(UNiagaraSystem* InSourceObject) const override
		{
			return CreateAdapterInternal(InSourceObject);
		}

		virtual TSharedRef<FStackScriptAdapter> CreateAdapter(const UNiagaraSystem* InSourceObject) const override
		{
			return CreateAdapterInternal(InSourceObject);
		}

	private:
		template<typename TSystem>
		TSharedRef<FStackScriptAdapter> CreateAdapterInternal(TSystem InSystem) const
		{
			if (InSystem != nullptr)
			{
				if (Usage == ENiagaraScriptUsage::SystemSpawnScript)
				{
					return FStackScriptAdapter::Create(InSystem, FGuid(), InSystem->GetSystemSpawnScript());
				}
				else if (Usage == ENiagaraScriptUsage::SystemUpdateScript)
				{
					return FStackScriptAdapter::Create(InSystem, FGuid(), InSystem->GetSystemUpdateScript());
				}
			}
			return MakeShared<FStackScriptAdapter>();
		}

	private:
		ENiagaraScriptUsage Usage;
	};

	TSharedRef<FSystemAdapter> FSystemAdapter::Create(UNiagaraSystem* InSystem)
	{
		TSharedRef<FSystemAdapter> SystemAdapter = MakeShared<FSystemAdapter>();
		SystemAdapter->Initialize(InSystem);
		return SystemAdapter;
	}

	TSharedRef<const FSystemAdapter> FSystemAdapter::CreateConst(const UNiagaraSystem* InSystem)
	{
		TSharedRef<FSystemAdapter> SystemAdapter = MakeShared<FSystemAdapter>();
		SystemAdapter->Initialize(InSystem);
		return SystemAdapter;
	}

	bool FSystemAdapter::IsValidAdapter() const
	{
		return SystemConstWeak.Get() != nullptr;
	}

	bool FSystemAdapter::IsValidWriteAdapter() const
	{
		return SystemWeak.IsSet() && SystemWeak->Get() != nullptr;
	}

	TSharedRefCollection<FUserParameterAdapter> FSystemAdapter::GetUserParameters()
	{
		return UserParameters.Get();
	}

	TConstSharedRefCollection<FUserParameterAdapter> FSystemAdapter::GetUserParameters() const
	{
		return UserParameters.Get();
	}

	TSharedPtr<FUserParameterAdapter> FSystemAdapter::GetUserParameterByName(const FName& InName)
	{
		return UserParameters.Get().FindByPredicate([&InName](const TSharedRef<FUserParameterAdapter>& UserParameterAdapter)
			{ return UserParameterAdapter->GetName() == InName; });
	}

	TSharedPtr<const FUserParameterAdapter> FSystemAdapter::GetUserParameterByName(const FName& InName) const
	{
		return UserParameters.Get().FindByPredicate([&InName](const TSharedRef<const FUserParameterAdapter>& UserParameterAdapter)
			{ return UserParameterAdapter->GetName() == InName; });
	}

	TSharedPtr<FUserParameterAdapter> FSystemAdapter::GetUserParameterByNamespacedName(const FName& InNamespacedName)
	{
		return UserParameters.Get().FindByPredicate([&InNamespacedName](const TSharedRef<FUserParameterAdapter>& UserParameterAdapter)
			{ return UserParameterAdapter->GetNamespacedName() == InNamespacedName; });
	}

	TSharedPtr<const FUserParameterAdapter> FSystemAdapter::GetUserParameterByNamespacedName(const FName& InNamespacedName) const
	{
		return UserParameters.Get().FindByPredicate([&InNamespacedName](const TSharedRef<const FUserParameterAdapter>& UserParameterAdapter)
			{ return UserParameterAdapter->GetNamespacedName() == InNamespacedName; });
	}

	TSharedRefCollection<FEmitterAdapter> FSystemAdapter::GetEmitters()
	{
		return Emitters.Get();
	}

	TConstSharedRefCollection<FEmitterAdapter> FSystemAdapter::GetEmitters() const
	{
		return Emitters.Get();
	}

	TSharedPtr<FEmitterAdapter> FSystemAdapter::GetEmitterByName(FName InEmitterName)
	{
		return Emitters.Get().FindByPredicate([&InEmitterName](const TSharedRef<FEmitterAdapter>& EmitterAdapter)
			{ return EmitterAdapter->GetName() == InEmitterName; });
	}

	TSharedPtr<const FEmitterAdapter> FSystemAdapter::GetEmitterByName(FName InEmitterName) const
	{
		return Emitters.Get().FindByPredicate([&InEmitterName](const TSharedRef<const FEmitterAdapter>& EmitterAdapter)
			{ return EmitterAdapter->GetName() == InEmitterName; });
	}

	TSharedRef<FStackScriptAdapter> FSystemAdapter::GetSpawnScript()
	{
		return SpawnScript.Get();
	}

	TSharedRef<const FStackScriptAdapter> FSystemAdapter::GetSpawnScript() const
	{
		return SpawnScript.Get();
	}

	TSharedRef<FStackScriptAdapter> FSystemAdapter::GetUpdateScript()
	{
		return UpdateScript.Get();
	}

	TSharedRef<const FStackScriptAdapter> FSystemAdapter::GetUpdateScript() const
	{
		return UpdateScript.Get();
	}

	int32 FSystemAdapter::GetNumScripts() const
	{
		return 2;
	}

	TSharedRef<FStackScriptAdapter> FSystemAdapter::GetScriptAt(int32 Index)
	{
		checkf(Index > INDEX_NONE && Index < GetNumScripts(), TEXT("Index out of range"));
		if (Index == 0)
		{
			return GetSpawnScript();
		}
		else // Index == 1
		{
			return GetUpdateScript();
		}
	}

	FScriptSharedRefCollection FSystemAdapter::GetScripts()
	{
		return FScriptSharedRefCollection(&StackScriptAdapterCollection);
	}

	FScriptConstSharedRefCollection FSystemAdapter::GetScripts() const
	{
		return FScriptConstSharedRefCollection(&StackScriptAdapterCollection);
	}

	void FSystemAdapter::Initialize(UNiagaraSystem* InSystem)
	{
		SystemWeak = InSystem;
		SystemConstWeak = InSystem;
		UserParameters.Initialize(InSystem, MakeShared<FUserParameterAdapterFactory>());
		Emitters.Initialize(InSystem, MakeShared<FEmitterAdapterFactory>());
		SpawnScript.Initialize(InSystem, MakeShared<FSystemScriptAdapterFactory>(ENiagaraScriptUsage::SystemSpawnScript));
		UpdateScript.Initialize(InSystem, MakeShared<FSystemScriptAdapterFactory>(ENiagaraScriptUsage::SystemUpdateScript));
		StackScriptAdapterCollection.Initialize(StaticCastSharedRef<IStackScriptAdapterOwner>(this->AsShared()));
	}

	void FSystemAdapter::Initialize(const UNiagaraSystem* InSystem)
	{
		SystemConstWeak = InSystem;
		UserParameters.Initialize(InSystem, MakeShared<FUserParameterAdapterFactory>());
		Emitters.Initialize(InSystem, MakeShared<FEmitterAdapterFactory>());
		SpawnScript.Initialize(InSystem, MakeShared<FSystemScriptAdapterFactory>(ENiagaraScriptUsage::SystemSpawnScript));
		UpdateScript.Initialize(InSystem, MakeShared<FSystemScriptAdapterFactory>(ENiagaraScriptUsage::SystemUpdateScript));
		StackScriptAdapterCollection.Initialize(StaticCastSharedRef<IStackScriptAdapterOwner>(this->AsShared()));
	}

} // namespace UE::Niagara