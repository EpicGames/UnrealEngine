// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "NiagaraNode.h"
#include "NiagaraNodeParameterMapBase.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/SNiagaraPinTypeSelector.h"
#include "NiagaraNodeCustomHlsl.h"

/** A graph pin widget for allowing a pin to have an editable name for a pin. */
template< class BaseClass >
class TNiagaraGraphPinEditableName : public BaseClass
{
public:
	SLATE_BEGIN_ARGS(TNiagaraGraphPinEditableName<BaseClass>) {}
	SLATE_END_ARGS()

	FORCENOINLINE void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
	{
		bPendingRename = false;
		BaseClass::Construct(typename BaseClass::FArguments(), InGraphPinObj);
	}

protected:
	FText GetParentPinLabel() const
	{
		return BaseClass::GetPinLabel();
	}

	EVisibility GetParentPinVisibility() const
	{
		return BaseClass::GetPinLabelVisibility();
	}

	FSlateColor GetParentPinTextColor() const
	{
		return BaseClass::GetPinTextColor();
	}

	bool OnVerifyTextChanged(const FText& InName, FText& OutErrorMessage) const
	{
		UNiagaraNode* ParentNode = Cast<UNiagaraNode>(this->GraphPinObj->GetOwningNode());
		if (ParentNode)
		{
			return ParentNode->VerifyEditablePinName(InName, OutErrorMessage, this->GraphPinObj);
		}
		return false;
	}

	void OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
	{
		if (!this->GraphPinObj->PinName.ToString().Equals(InText.ToString(), ESearchCase::CaseSensitive))
		{
			UNiagaraNode* ParentNode = Cast<UNiagaraNode>(this->GraphPinObj->GetOwningNode());
			if (ParentNode != nullptr)
			{
				ParentNode->CommitEditablePinName(InText, this->GraphPinObj);
			}
		}
		else
		{
			UNiagaraNode* ParentNode = Cast<UNiagaraNode>(this->GraphPinObj->GetOwningNode());
			if (ParentNode != nullptr)
			{
				ParentNode->CancelEditablePinName(InText, this->GraphPinObj);
			}
		}
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if (bPendingRename == false)
		{
			UNiagaraNode* ParentNode = Cast<UNiagaraNode>(this->GraphPinObj->GetOwningNode());
			if (ParentNode != nullptr && ParentNode->GetIsPinRenamePending(this->GraphPinObj))
			{
				bPendingRename = true;
				ParentNode->SetIsPinRenamePending(this->GraphPinObj, false);
			}
		}
		if (bPendingRename)
		{
			if (CreatedTextBlock.IsValid())
			{
				CreatedTextBlock->EnterEditingMode();
			}
			else if (CreatedParameterNamePinLabel.IsValid())
			{
				CreatedParameterNamePinLabel->EnterEditingMode();
			}
			bPendingRename = false;
		}
		BaseClass::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}

	virtual TSharedRef<SWidget> GetLabelWidget(const FName& InLabelStyle) override
	{
		UNiagaraNode* ParentNode = Cast<UNiagaraNode>(this->GraphPinObj->GetOwningNode());

		if(ParentNode == nullptr)
		{
			return BaseClass::GetLabelWidget(InLabelStyle);
		}
		
		auto CreateRenamableLabelTextBlock = [&]()->TSharedRef<SWidget> {
			CreatedTextBlock = SNew(SInlineEditableTextBlock)
				.Style(&FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("Graph.Node.InlineEditablePinName"))
				.Text(this, &TNiagaraGraphPinEditableName<BaseClass>::GetParentPinLabel)
				.Visibility(this, &TNiagaraGraphPinEditableName<BaseClass>::GetParentPinVisibility)
				.ColorAndOpacity(this, &TNiagaraGraphPinEditableName<BaseClass>::GetParentPinTextColor)
				.OnVerifyTextChanged(this, &TNiagaraGraphPinEditableName<BaseClass>::OnVerifyTextChanged)
				.OnTextCommitted(this, &TNiagaraGraphPinEditableName<BaseClass>::OnTextCommitted);
			return CreatedTextBlock.ToSharedRef();
		};
		
		const bool bAllowPinTypeChanges = ParentNode->AllowExternalPinTypeChanges(this->GraphPinObj) && this->GraphPinObj->bOrphanedPin == false;
		const bool bIsPinEditable = ParentNode->IsPinNameEditable(this->GraphPinObj);
		
		if (bIsPinEditable && ParentNode->IsPinNameEditableUponCreation(this->GraphPinObj))
		{
			bPendingRename = true;
		}
		
		if (ParentNode->IsA<UNiagaraNodeParameterMapBase>())
		{
			return SNew(SBox)
			.Padding(FMargin(0.0f, 1.0f, 0.0f, 1.0f))
			[
				SAssignNew(CreatedParameterNamePinLabel, SNiagaraParameterNamePinLabel, this->GraphPinObj)
				.EditableTextStyle(&FNiagaraEditorStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("NiagaraEditor.Graph.Node.InlineEditablePinName"))
				.ParameterText(this, &TNiagaraGraphPinEditableName<BaseClass>::GetParentPinLabel)
				.IsReadOnly(bIsPinEditable == false)
				.Visibility(this, &TNiagaraGraphPinEditableName<BaseClass>::GetParentPinVisibility)
				.OnVerifyTextChanged(this, &TNiagaraGraphPinEditableName<BaseClass>::OnVerifyTextChanged)
				.OnTextCommitted(this, &TNiagaraGraphPinEditableName<BaseClass>::OnTextCommitted)
			];
		}
		else if(bIsPinEditable)
		{
			return CreateRenamableLabelTextBlock();
		}
		// we want the pin type selector in the label widget only for output pins, otherwise we put it in the value widget
		else if (bAllowPinTypeChanges && this->GraphPinObj->Direction == EGPD_Output)
		{				
			TSharedRef<SHorizontalBox> LabelWidgetContainer = SNew(SHorizontalBox);
			LabelWidgetContainer->AddSlot().AutoWidth().Padding(3.f, 0.f)
			[
				SNew(SNiagaraPinTypeSelector, this->GraphPinObj)
			];
			LabelWidgetContainer->AddSlot()
			[
				BaseClass::GetLabelWidget(InLabelStyle)
			];
			return LabelWidgetContainer;
		}

		return BaseClass::GetLabelWidget(InLabelStyle);
	}

	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		UNiagaraNode* ParentNode = Cast<UNiagaraNode>(this->GraphPinObj->GetOwningNode());
		const bool bAllowPinTypeChanges = ParentNode->AllowExternalPinTypeChanges(this->GraphPinObj) && this->GraphPinObj->bOrphanedPin == false;

		TSharedRef<SHorizontalBox> ValueWidgetContainer = SNew(SHorizontalBox);

		ValueWidgetContainer->AddSlot()[BaseClass::GetDefaultValueWidget()];

		// value widget should only exist for input pins anyways, but we'll make sure here
		if (bAllowPinTypeChanges && this->GraphPinObj->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			ValueWidgetContainer->AddSlot().AutoWidth().Padding(3.f, 0.f)
			[SNew(SNiagaraPinTypeSelector, this->GraphPinObj)];
		}

		return ValueWidgetContainer;

	}

	bool bPendingRename;
	bool bPendingEditNamespaceModifier;
	TSharedPtr<SInlineEditableTextBlock> CreatedTextBlock;
	TSharedPtr<SNiagaraParameterNamePinLabel> CreatedParameterNamePinLabel;
};
