// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorTestUtilities.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "NiagaraScriptSource.h"
#include "NiagaraGraph.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "NiagaraSystemFactoryNew.h"

#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"

UNiagaraEmitter* FNiagaraEditorTestUtilities::CreateEmptyTestEmitter()
{
	UNiagaraEmitter* Emitter = NewObject<UNiagaraEmitter>();
	Emitter->CheckVersionDataAvailable();
	UNiagaraScriptSource* ScriptSource = NewObject<UNiagaraScriptSource>(Emitter);
	UNiagaraGraph* Graph = NewObject<UNiagaraGraph>(ScriptSource);

	ScriptSource->NodeGraph = Graph;
	FVersionedNiagaraEmitterData* Data = Emitter->GetLatestEmitterData();
	Data->GraphSource = ScriptSource;
	Data->SpawnScriptProps.Script->SetLatestSource(ScriptSource);
	Data->UpdateScriptProps.Script->SetLatestSource(ScriptSource);

	FNiagaraStackGraphUtilities::ResetGraphForOutput(*Graph, ENiagaraScriptUsage::EmitterSpawnScript, FGuid());
	FNiagaraStackGraphUtilities::ResetGraphForOutput(*Graph, ENiagaraScriptUsage::EmitterUpdateScript, FGuid());
	FNiagaraStackGraphUtilities::ResetGraphForOutput(*Graph, ENiagaraScriptUsage::ParticleSpawnScript, FGuid());
	FNiagaraStackGraphUtilities::ResetGraphForOutput(*Graph, ENiagaraScriptUsage::ParticleUpdateScript, FGuid());

	return Emitter;
}

UNiagaraSystem* FNiagaraEditorTestUtilities::CreateTestSystemForEmitter(UNiagaraEmitter* Emitter)
{
	UNiagaraSystem* System = NewObject<UNiagaraSystem>((UObject*)GetTransientPackage(), NAME_None, RF_Transient);
	UNiagaraSystemFactoryNew::InitializeSystem(System, false);
	System->AddEmitterHandle(*Emitter, Emitter->GetFName(), FGuid());
	return System;
}

TSharedRef<FNiagaraSystemViewModel> FNiagaraEditorTestUtilities::CreateTestSystemViewModelForEmitter(UNiagaraEmitter* Emitter)
{
	UNiagaraSystem* System = CreateTestSystemForEmitter(Emitter);

	FNiagaraSystemViewModelOptions SystemViewModelOptions;
	SystemViewModelOptions.bCanAutoCompile = false;
	SystemViewModelOptions.bCanSimulate = false;

	TSharedRef<FNiagaraSystemViewModel> SystemViewModel = MakeShared<FNiagaraSystemViewModel>();
	SystemViewModel->Initialize(*System, SystemViewModelOptions);

	return SystemViewModel;
}

UNiagaraEmitter* FNiagaraEditorTestUtilities::LoadEmitter(FString AssetPath)
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	const FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));
	return Cast<UNiagaraEmitter>(AssetData.GetAsset());
}