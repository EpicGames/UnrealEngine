// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Containers/UnrealString.h"

class UNiagaraEmitter;
class UNiagaraSystem;
class FNiagaraSystemViewModel;

namespace FNiagaraEditorTestUtilities
{
	UNiagaraEmitter* CreateEmptyTestEmitter();

	UNiagaraSystem* CreateTestSystemForEmitter(UNiagaraEmitter* Emitter);

	TSharedRef<FNiagaraSystemViewModel> CreateTestSystemViewModelForEmitter(UNiagaraEmitter* Emitter);

	UNiagaraEmitter* LoadEmitter(FString AssetPath);
}

#define ASSERT_TRUE(Expression, FailMessage) \
	if (!(Expression)) \
	{ \
		AddError(FString::Printf(TEXT("ASSERT_TRUE Failed - Message: %s  Expression: %s"), FailMessage, TEXT(#Expression))); \
		return false; \
	}

#define ASSERT_FALSE(Expression, FailMessage) \
	if ((Expression)) \
	{ \
		AddError(FString::Printf(TEXT("ASSERT_FALSE Failed - Message: %s  Expression: %s"), FailMessage, TEXT(#Expression))); \
		return false; \
	}