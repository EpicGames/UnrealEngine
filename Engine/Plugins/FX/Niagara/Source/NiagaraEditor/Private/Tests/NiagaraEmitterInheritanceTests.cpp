// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEmitterInheritanceTests.h"
#include "NiagaraScriptMergeManager.h"
#include "NiagaraEmitter.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraEditorTestUtilities.h"

#include "Misc/AutomationTest.h"
#include "NiagaraEditorModule.h"

FString TestDataRoot = TEXT("/Game/Developers/FrankFella/AutomatedTestEmitters/");

bool CheckDiffResults(const FNiagaraScriptStackDiffResults& DiffResults, 
	int32 RemovedModuleCount, int32 AddedModuleCount, int32 MovedModuleCount,
	int32 RemovedInputOverrideCount, int32 AddedInputOverrideCount, int32 ModifiedInputOverrideCount,
	FString& ErrorMessage)
{
	bool bMatches = true;

	if (DiffResults.RemovedBaseModules.Num() != RemovedModuleCount)
	{
		ErrorMessage += FString::Printf(TEXT("\nRemoved module count - Expected: %i Actual: %i"), RemovedModuleCount, DiffResults.RemovedBaseModules.Num());
		bMatches = false;
	}
	if (DiffResults.AddedOtherModules.Num() != AddedModuleCount)
	{
		ErrorMessage += FString::Printf(TEXT("\nAdded module count - Expected: %i Actual: %i"), AddedModuleCount, DiffResults.AddedOtherModules.Num());
		bMatches = false;
	}
	if (DiffResults.MovedBaseModules.Num() != MovedModuleCount || DiffResults.MovedOtherModules.Num() != MovedModuleCount)
	{
		ErrorMessage += FString::Printf(TEXT("\nMoved module count - Expected: %i Actual Base: %i Other Base: %i"), MovedModuleCount, DiffResults.MovedBaseModules.Num(), DiffResults.MovedOtherModules.Num());
		bMatches = false;
	}

	if (DiffResults.RemovedBaseInputOverrides.Num() != RemovedInputOverrideCount)
	{
		ErrorMessage += FString::Printf(TEXT("\nRemoved input override count - Expected: %i Actual: %i"), RemovedInputOverrideCount, DiffResults.RemovedBaseInputOverrides.Num());
		bMatches = false;
	}
	if (DiffResults.AddedOtherInputOverrides.Num() != AddedInputOverrideCount)
	{
		ErrorMessage += FString::Printf(TEXT("\nAdded input override count - Expected: %i Actual: %i"), AddedInputOverrideCount, DiffResults.AddedOtherInputOverrides.Num());
		bMatches = false;
	}
	if (DiffResults.ModifiedBaseInputOverrides.Num() != ModifiedInputOverrideCount || DiffResults.ModifiedOtherInputOverrides.Num() != ModifiedInputOverrideCount)
	{
		ErrorMessage += FString::Printf(TEXT("\nModified input override count - Expected: %i Actual Base: %i Actual Other: %i"),
			ModifiedInputOverrideCount, DiffResults.ModifiedBaseInputOverrides.Num(), DiffResults.ModifiedOtherInputOverrides.Num());
		bMatches = false;
	}

	return bMatches;
}

bool CheckModuleListsMatch(TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> ModuleListA, TArray<TSharedRef<FNiagaraStackFunctionMergeAdapter>> ModuleListB, FString& ErrorMessage)
{
	if (ModuleListA.Num() != ModuleListB.Num())
	{
		ErrorMessage = TEXT("Module list counts don't match");
		return false;
	}

	TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
	for (int32 i = 0; i < ModuleListA.Num(); i++)
	{
		TSharedRef<FNiagaraStackFunctionMergeAdapter> ModuleA = ModuleListA[i];
		TSharedRef<FNiagaraStackFunctionMergeAdapter> ModuleB = ModuleListB[i];

		if (ModuleA->GetStackIndex() != ModuleB->GetStackIndex())
		{
			ErrorMessage = FString::Printf(TEXT("Modules at list index %i had different stack indices."), i);
			return false;
		}

		UNiagaraNodeAssignment* AssignmentNodeA = Cast<UNiagaraNodeAssignment>(ModuleA->GetFunctionCallNode());
		UNiagaraNodeAssignment* AssignmentNodeB = Cast<UNiagaraNodeAssignment>(ModuleB->GetFunctionCallNode());

		if ((AssignmentNodeA == nullptr && AssignmentNodeB != nullptr) || (AssignmentNodeA != nullptr && AssignmentNodeB == nullptr))
		{
			ErrorMessage = FString::Printf(TEXT("Modules at list index %i had different function call node types."), i);
			return false;
		}

		if (AssignmentNodeA != nullptr)
		{
			// Assignment nodes have generated scripts based on the assignment target so check that instead of the script directly.
			if (AssignmentNodeA->GetAssignmentTargets().Num() != AssignmentNodeB->GetAssignmentTargets().Num())
			{
				ErrorMessage = FString::Printf(TEXT("Modules at list index %i were assignment nodes but had different assignment target counts."), i);
				return false;
			}

			for (int32 k = 0; k < AssignmentNodeA->GetAssignmentTargets().Num(); k++)
			{
				if (AssignmentNodeA->GetAssignmentTarget(k) != AssignmentNodeB->GetAssignmentTarget(k))
				{
					ErrorMessage = FString::Printf(TEXT("Modules at list index %i were assignment nodes but had different assignment targets at %d."), i, k);
					return false;
				}
			}
		}
		else
		{
			// Module isn't an assignment so a function script pointer check validates they are the same.
			if (ModuleA->GetFunctionCallNode()->FunctionScript != ModuleB->GetFunctionCallNode()->FunctionScript)
			{
				ErrorMessage = FString::Printf(TEXT("Modules at list index %i had different function scripts."), i);
				return false;
			}
		}

		FNiagaraScriptStackDiffResults FunctionInputDiffResults;
		MergeManager->DiffFunctionInputs(ModuleA, ModuleB, FunctionInputDiffResults);
		FString InputDiffErrorMessage;
		if (CheckDiffResults(FunctionInputDiffResults, 0, 0, 0, 0, 0, 0, InputDiffErrorMessage) == false)
		{
			ErrorMessage = InputDiffErrorMessage;
			return false;
		}
	}
	return true;
}

/*
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorCreateEmitterMergeAdapterTest, "System.Plugins.Niagara.Editor.Inheritance.Create Emitter Merge Adapter", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorCreateEmitterMergeAdapterTest::RunTest(const FString& Parameters)
{
	UNiagaraEmitter* TestEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestEmitter.TestEmitter"));
	ASSERT_TRUE(TestEmitter != nullptr, TEXT("Could not load test emitter"));

	FNiagaraEmitterMergeAdapter MergeAdapter(*TestEmitter);

	// Emitter spawn
	ASSERT_TRUE(MergeAdapter.GetEmitterSpawnStack().IsValid(), TEXT("Stack was invalid"));

	TSharedPtr<FNiagaraScriptStackMergeAdapter> EmitterSpawnStackAdapter = MergeAdapter.GetEmitterSpawnStack();
	ASSERT_TRUE(EmitterSpawnStackAdapter->GetModuleFunctions().Num() == 0, TEXT("Stack module count was wrong"));

	// Emitter update
	ASSERT_TRUE(MergeAdapter.GetEmitterUpdateStack().IsValid(), TEXT("Stack was invalid"));

	TSharedPtr<FNiagaraScriptStackMergeAdapter> EmitterUpdateStackAdapter = MergeAdapter.GetEmitterUpdateStack();
	ASSERT_TRUE(EmitterUpdateStackAdapter->GetModuleFunctions().Num() == 2, TEXT("Stack module count was wrong"));

	TSharedPtr<FNiagaraStackFunctionMergeAdapter> EmitterLifetimeAdapter = EmitterUpdateStackAdapter->GetModuleFunctions()[0];
	ASSERT_TRUE(EmitterLifetimeAdapter->GetFunctionCallNode() != nullptr, TEXT("Module function call node was null"));
	ASSERT_TRUE(EmitterLifetimeAdapter->GetFunctionCallNode()->GetFunctionName() == TEXT("EmitterLifetime"), TEXT("Module function call was incorrect"));
	ASSERT_TRUE(EmitterLifetimeAdapter->GetInputOverrides().Num() == 4, TEXT("Wrong number of input overrides"));

	TSharedPtr<FNiagaraStackFunctionMergeAdapter> EmitterSpawnRateAdapter = EmitterUpdateStackAdapter->GetModuleFunctions()[1];
	ASSERT_TRUE(EmitterSpawnRateAdapter->GetFunctionCallNode() != nullptr, TEXT("Module function call node was null"));
	ASSERT_TRUE(EmitterSpawnRateAdapter->GetFunctionCallNode()->GetFunctionName() == TEXT("SpawnRate"), TEXT("Module function call was incorrect"));
	ASSERT_TRUE(EmitterSpawnRateAdapter->GetInputOverrides().Num() == 1, TEXT("Module function had the wrong number of input overrides"));

	TSharedPtr<FNiagaraStackFunctionInputOverrideMergeAdapter> EmitterSpawnRateSpawnRateInputAdapter = EmitterSpawnRateAdapter->GetInputOverrides()[0];
	ASSERT_TRUE(EmitterSpawnRateSpawnRateInputAdapter->GetOverridePin() != nullptr, TEXT("Input override pin was null"));
	ASSERT_TRUE(EmitterSpawnRateSpawnRateInputAdapter->GetOverrideNode() != nullptr, TEXT("Input override node was null"));
	ASSERT_FALSE(EmitterSpawnRateSpawnRateInputAdapter->GetLocalValueString().IsSet(), TEXT("Input local value string was incorrect"));
	ASSERT_FALSE(EmitterSpawnRateSpawnRateInputAdapter->GetLocalValueRapidIterationParameter().IsSet(), TEXT("Input local value rapid iteration parameter was incorrect"));
	ASSERT_FALSE(EmitterSpawnRateSpawnRateInputAdapter->GetLinkedValueHandle().IsSet(), TEXT("Input linked value handle was incorrect"));
	ASSERT_TRUE(EmitterSpawnRateSpawnRateInputAdapter->GetDataValueObject() == nullptr, TEXT("Input data value node was incorrect"));
	ASSERT_TRUE(EmitterSpawnRateSpawnRateInputAdapter->GetDynamicValueFunction().IsValid(), TEXT("Input dynamic value function was incorrect"));

	TSharedPtr<FNiagaraStackFunctionMergeAdapter> EmitterSpawnRateSpawnRateInputDynamicInputFunction = EmitterSpawnRateSpawnRateInputAdapter->GetDynamicValueFunction();
	ASSERT_TRUE(EmitterSpawnRateSpawnRateInputDynamicInputFunction->GetFunctionCallNode() != nullptr, TEXT("Dynamic input function call node was null"));
	ASSERT_TRUE(EmitterSpawnRateSpawnRateInputDynamicInputFunction->GetFunctionCallNode()->GetFunctionName() == TEXT("UniformRangedFloat001"), TEXT("Dynamic input function call was incorrect"));
	ASSERT_TRUE(EmitterSpawnRateSpawnRateInputDynamicInputFunction->GetInputOverrides().Num() == 2, TEXT("Dynamic input function had the wrong number of input overrides"));

	// Particle spawn
	ASSERT_TRUE(MergeAdapter.GetParticleSpawnStack().IsValid(), TEXT("Stack was invalid"));

	TSharedPtr<FNiagaraScriptStackMergeAdapter> ParticleSpawnStackAdapter = MergeAdapter.GetParticleSpawnStack();
	ASSERT_TRUE(ParticleSpawnStackAdapter->GetModuleFunctions().Num() == 5, TEXT("Stack module count was wrong"));

	// Particle update
	ASSERT_TRUE(MergeAdapter.GetParticleUpdateStack().IsValid(), TEXT("Stack was invalid"));

	TSharedPtr<FNiagaraScriptStackMergeAdapter> ParticleUpdateStackAdapter = MergeAdapter.GetParticleUpdateStack();
	ASSERT_TRUE(ParticleUpdateStackAdapter->GetModuleFunctions().Num() == 6, TEXT("Stack module count was wrong"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorDiffSameEmittersTest, "System.Plugins.Niagara.Editor.Inheritance.Diff same emitters", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorDiffSameEmittersTest::RunTest(const FString& Parameters)
{
	UNiagaraEmitter* TestEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestEmitter.TestEmitter"));
	ASSERT_TRUE(TestEmitter != nullptr, TEXT("Could not load test emitter"));

	UNiagaraEmitter* TestEmitterCopy = CastChecked<UNiagaraEmitter>(StaticDuplicateObject(TestEmitter, (UObject*)GetTransientPackage()));

	FNiagaraEmitterDiffResults DiffResults = FNiagaraScriptMergeManager::Get()->DiffEmitters(*TestEmitter, *TestEmitterCopy);

	ASSERT_TRUE(DiffResults.IsValid(), *FString::Printf(TEXT("Emitter diff failed.  Message: %s"), *DiffResults.GetErrorMessagesString()));

	ASSERT_TRUE(DiffResults.EmitterSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));
	ASSERT_TRUE(DiffResults.EmitterUpdateDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));
	ASSERT_TRUE(DiffResults.ParticleSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorDiffEmittersWithAddedModulesTest, "System.Plugins.Niagara.Editor.Inheritance.Diff emitters with added modules", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorDiffEmittersWithAddedModulesTest::RunTest(const FString& Parameters)
{
	UNiagaraEmitter* DefaultEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestEmitter.TestEmitter"));
	ASSERT_TRUE(DefaultEmitter != nullptr, TEXT("Could not load default emitter"));
	UNiagaraEmitter* EmitterWithAddedModules = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestAddedModules.TestAddedModules"));
	ASSERT_TRUE(EmitterWithAddedModules != nullptr, TEXT("Could not load emitter with added modules"));

	// Changes
	//   Emitter Spawn
	//     Added SetParticles.NewNiagaraFloat0
	//   Emitter Update
	//     Added SetParticles.NewNiagaraFloat1
	//     Added SetParticles.NewNiagaraFloat2
	//   Particle Spawn
	//     Added SetParticles.NewNiagaraFloat3
	//     Added SetParticles.NewNiagaraFloat4
	//     Added SetParticles.NewNiagaraFloat5
	//   Particle Update
	//     Added SetParticles.NewNiagaraFloat6 (moved 6 -> 4)
	//     Added SetParticles.NewNiagaraFloat7
	//     Added SetParticles.NewNiagaraFloat8
	//     Added SetParticles.NewNiagaraFloat9
	FNiagaraEmitterDiffResults DiffResults = FNiagaraScriptMergeManager::Get()->DiffEmitters(*DefaultEmitter, *EmitterWithAddedModules);

	ASSERT_TRUE(DiffResults.IsValid(), *FString::Printf(TEXT("Emitter diff failed.  Message: %s"), *DiffResults.GetErrorMessagesString()));

	FString EmitterSpawnDiffErrors;
	bool EmitterSpawnDiffMatches = CheckDiffResults(DiffResults.EmitterSpawnDiffResults, 0, 1, 0, 0, 0, 0, EmitterSpawnDiffErrors);
	ASSERT_TRUE(EmitterSpawnDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *EmitterSpawnDiffErrors));

	FString EmitterUpdateDiffErrors;
	bool EmitterUpdateDiffMatches = CheckDiffResults(DiffResults.EmitterUpdateDiffResults, 0, 2, 0, 0, 0, 0, EmitterUpdateDiffErrors);
	ASSERT_TRUE(EmitterUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *EmitterUpdateDiffErrors));

	FString ParticleSpawnDiffErrors;
	bool ParticleSpawnDiffMatches = CheckDiffResults(DiffResults.ParticleSpawnDiffResults, 0, 3, 0, 0, 0, 0, ParticleSpawnDiffErrors);
	ASSERT_TRUE(ParticleSpawnDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *ParticleSpawnDiffErrors));

	FString ParticleUpdateDiffErrors;
	bool ParticleUpdateDiffMatches = CheckDiffResults(DiffResults.ParticleUpdateDiffResults, 0, 4, 2, 0, 0, 0, ParticleUpdateDiffErrors);
	ASSERT_TRUE(ParticleUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *ParticleUpdateDiffErrors));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorDiffEmittersWithRemovedModulesTest, "System.Plugins.Niagara.Editor.Inheritance.Diff emitters with removed modules", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorDiffEmittersWithRemovedModulesTest::RunTest(const FString& Parameters)
{
	UNiagaraEmitter* DefaultEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestEmitter.TestEmitter"));
	ASSERT_TRUE(DefaultEmitter != nullptr, TEXT("Could not load default emitter"));
	UNiagaraEmitter* EmitterWithRemovedModules = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestRemovedModules.TestRemovedModules"));
	ASSERT_TRUE(EmitterWithRemovedModules != nullptr, TEXT("Could not load emitter with removed modules"));

	// Changes
	// Remove EmitterUpdate->SpawnRate
	// Remove ParticleUpdate->ApplyVelocity
	// Remove ParticleUpdate->ScaleSizeByVelocity
	FNiagaraEmitterDiffResults DiffResults = FNiagaraScriptMergeManager::Get()->DiffEmitters(*DefaultEmitter, *EmitterWithRemovedModules);

	ASSERT_TRUE(DiffResults.IsValid(), *FString::Printf(TEXT("Emitter diff failed.  Message: %s"), *DiffResults.GetErrorMessagesString()));

	ASSERT_TRUE(DiffResults.EmitterSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));

	FString EmitterUpdateDiffErrors;
	bool EmitterUpdateDiffMatches = CheckDiffResults(DiffResults.EmitterUpdateDiffResults, 1, 0, 0, 0, 0, 0, EmitterUpdateDiffErrors);
	ASSERT_TRUE(EmitterUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *EmitterUpdateDiffErrors));

	ASSERT_TRUE(DiffResults.ParticleSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));

	FString ParticleUpdateDiffErrors;
	bool ParticleUpdateDiffMatches = CheckDiffResults(DiffResults.ParticleUpdateDiffResults, 2, 0, 0, 0, 0, 0, ParticleUpdateDiffErrors);
	ASSERT_TRUE(ParticleUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *ParticleUpdateDiffErrors));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorDiffEmittersWithMovedModulesTest, "System.Plugins.Niagara.Editor.Inheritance.Diff emitters with moved modules", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorDiffEmittersWithMovedModulesTest::RunTest(const FString& Parameters)
{
	UNiagaraEmitter* DefaultEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestEmitter.TestEmitter"));
	ASSERT_TRUE(DefaultEmitter != nullptr, TEXT("Could not load default emitter"));
	UNiagaraEmitter* EmitterWithMovedModules = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestMovedModules.TestMovedModules"));
	ASSERT_TRUE(EmitterWithMovedModules != nullptr, TEXT("Could not load emitter with Moved modules"));

	// Changes
	// Emitter Update
	//   Move SpawnRate 1 -> 0
	//   Move EmitterLifetime 0 -> 1 (side effect)
	// Particle Update
	//   Move AnimatedColor 2 -> 0 (swap)
	//   Move Lifetime to index 0 -> 2 (swap)
	//   Move SclaeSizeByVelocty 5 -> 3
	//   Move ApplyAcceleration to index 3 -> 4 (side effect)
	//   Move ApplyVelocity to index 4 -> 5 (side effect)
	FNiagaraEmitterDiffResults DiffResults = FNiagaraScriptMergeManager::Get()->DiffEmitters(*DefaultEmitter, *EmitterWithMovedModules);

	ASSERT_TRUE(DiffResults.IsValid(), *FString::Printf(TEXT("Emitter diff failed.  Message: %s"), *DiffResults.GetErrorMessagesString()));

	ASSERT_TRUE(DiffResults.EmitterSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));

	FString EmitterUpdateDiffErrors;
	bool EmitterUpdateDiffMatches = CheckDiffResults(DiffResults.EmitterUpdateDiffResults, 0, 0, 2, 0, 0, 0, EmitterUpdateDiffErrors);
	ASSERT_TRUE(EmitterUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *EmitterUpdateDiffErrors));
	ASSERT_TRUE(DiffResults.EmitterUpdateDiffResults.MovedBaseModules[0]->GetStackIndex() == 0, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.EmitterUpdateDiffResults.MovedOtherModules[0]->GetStackIndex() == 1, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.EmitterUpdateDiffResults.MovedBaseModules[1]->GetStackIndex() == 1, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.EmitterUpdateDiffResults.MovedOtherModules[1]->GetStackIndex() == 0, TEXT("Diff results moved module indices were wrong."));

	ASSERT_TRUE(DiffResults.ParticleSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));

	FString ParticleUpdateDiffErrors;
	bool ParticleUpdateDiffMatches = CheckDiffResults(DiffResults.ParticleUpdateDiffResults, 0, 0, 5, 0, 0, 0, ParticleUpdateDiffErrors);
	ASSERT_TRUE(ParticleUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *ParticleUpdateDiffErrors));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedBaseModules[0]->GetStackIndex() == 0, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedOtherModules[0]->GetStackIndex() == 2, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedBaseModules[1]->GetStackIndex() == 2, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedOtherModules[1]->GetStackIndex() == 0, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedBaseModules[2]->GetStackIndex() == 3, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedOtherModules[2]->GetStackIndex() == 4, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedBaseModules[3]->GetStackIndex() == 4, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedOtherModules[3]->GetStackIndex() == 5, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedBaseModules[4]->GetStackIndex() == 5, TEXT("Diff results moved module indices were wrong."));
	ASSERT_TRUE(DiffResults.ParticleUpdateDiffResults.MovedOtherModules[4]->GetStackIndex() == 3, TEXT("Diff results moved module indices were wrong."));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorDiffEmittersWithChangedInputOverridesTest, "System.Plugins.Niagara.Editor.Inheritance.Diff emitters with changed input overrides", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorDiffEmittersWithChangedInputOverridesTest::RunTest(const FString& Parameters)
{
	UNiagaraEmitter* DefaultEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestEmitter.TestEmitter"));
	ASSERT_TRUE(DefaultEmitter != nullptr, TEXT("Could not load default emitter"));
	UNiagaraEmitter* EmitterWithChangedInputs = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestChangedInputs.TestChangedInputs"));
	ASSERT_TRUE(EmitterWithChangedInputs != nullptr, TEXT("Could not load emitter with changed input overrides."));

	FNiagaraEmitterDiffResults DiffResults = FNiagaraScriptMergeManager::Get()->DiffEmitters(*DefaultEmitter, *EmitterWithChangedInputs);

	ASSERT_TRUE(DiffResults.IsValid(), *FString::Printf(TEXT("Emitter diff failed.  Message: %s"), *DiffResults.GetErrorMessagesString()));

	FString EmitterUpdateDiffErrors;
	// EmitterLifetime->DelayFirstLoopOnly - Reset to default - 1 removed input override.
	// EmitterLifetime->DurationRecalcEachLoop - Change local value - 1 added input override.
	bool EmitterUpdateDiffMatches = CheckDiffResults(DiffResults.EmitterUpdateDiffResults, 0, 0, 0, 1, 1, 0, EmitterUpdateDiffErrors);
	ASSERT_TRUE(EmitterUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *EmitterUpdateDiffErrors));

	FString ParticleSpawnDiffErrors;
	// BoxLocation->BoxSize - Change local value (20) - 1 modified input override
	// SetParticles.Lifetime->Input - Change to different dynamic input (mod) - 1 modified input override
	// SetParticles.SpriteSize - Remove dynamic input (ranged float) - 1 modified input override
	// SetParticles.SpriteRotation->Value - Change To Dynamic Input (ranged float) - 1 modified input override
	bool ParticleSpawnDiffMatches = CheckDiffResults(DiffResults.ParticleSpawnDiffResults, 0, 0, 0, 0, 0, 4, ParticleSpawnDiffErrors);
	ASSERT_TRUE(ParticleSpawnDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *ParticleSpawnDiffErrors));

	// Lifetime->Lifetime - Change linked parameter handle - 1 modified input override
	// AnimatedColor->Animation - Change curve value - 1 modified input override
	FString ParticleUpdateDiffErrors;
	bool ParticleUpdateDiffMatches = CheckDiffResults(DiffResults.ParticleUpdateDiffResults, 0, 0, 0, 0, 0, 2, ParticleUpdateDiffErrors);
	ASSERT_TRUE(ParticleUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *ParticleUpdateDiffErrors));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorMergeEmitterWithChangedInputOverridesTest, "System.Plugins.Niagara.Editor.Inheritance.Merge emitter with changed input overrides", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorMergeEmitterWithChangedInputOverridesTest::RunTest(const FString& Parameters)
{
	TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();

	// Changes to source
	//   Emitter Update
	//	   EmitterLifetime->NextLoopDuration - Changed to 3
	//   Particle Spawn
	//     SetParticles.SpriteSize->Input->Minimum - changed to 10
	//	   SetParticles.SpriteRotation - removed.
	//	   UniformDistributionColor - Added and moved up 2 spaces.
	// Changes to instance
	//   Emitter Update
	//     EmitterLifetime->DelayFirstLoopOnly - Reset to default - 1 removed input override.
	//     EmitterLifetime->DurationRecalcEachLoop - Change local value - 1 added input override.
	//    Particle Spawn
	//      BoxLocation->BoxSize - Change local value (20) - 1 modified input override
	//      SetParticles.Lifetime->Input - Change to different dynamic input (mod) - 1 modified input override
	//      SetParticles.SpriteSize - Remove dynamic input (ranged float) - 1 modified input override
	//      SetParticles.SpriteRotation->Value - Change To Dynamic Input (ranged float) - 1 modified input override
	//   Particle Update
	//     Lifetime->Lifetime - Change linked parameter handle - 1 modified input override
	//     AnimatedColor->Animation - Change curve value - 1 modified input override
	UNiagaraEmitter* DefaultEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestEmitter.TestEmitter"));
	ASSERT_TRUE(DefaultEmitter != nullptr, TEXT("Could not load default emitter"));
	UNiagaraEmitter* EmitterWithChangedInputs = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestChangedInputs.TestChangedInputs"));
	ASSERT_TRUE(EmitterWithChangedInputs != nullptr, TEXT("Could not load emitter with changed input overrides."));
	UNiagaraEmitter* MergeSourceEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestMergeSource.TestMergeSource"));
	ASSERT_TRUE(MergeSourceEmitter != nullptr, TEXT("Could not load emitter with changed input overrides merge source."));
	UNiagaraEmitter* MergeTargetEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestMergeTarget.TestMergeTarget"));
	ASSERT_TRUE(MergeTargetEmitter != nullptr, TEXT("Could not load emitter with changed input overrides merge target."));

	INiagaraModule::FMergeEmitterResults MergeResults = MergeManager->MergeEmitter(*MergeSourceEmitter, *DefaultEmitter, *EmitterWithChangedInputs);

	ASSERT_TRUE(MergeResults.bSucceeded, *FString::Printf(TEXT("Merge failed.  Message: %s"), *MergeResults.GetErrorMessagesString()));
	ASSERT_TRUE(MergeResults.bModifiedGraph, TEXT("Merge reported success, but reported that the graph wasn't modified."));

	FNiagaraEmitterDiffResults MergedEmitterDiffResults = MergeManager->DiffEmitters(*MergeResults.MergedInstance, *MergeTargetEmitter);

	ASSERT_TRUE(MergedEmitterDiffResults.EmitterSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));
	ASSERT_TRUE(MergedEmitterDiffResults.EmitterUpdateDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));
	ASSERT_TRUE(MergedEmitterDiffResults.ParticleSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));
	ASSERT_TRUE(MergedEmitterDiffResults.ParticleUpdateDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorMergeEmitterWithRemovedModulesTest, "System.Plugins.Niagara.Editor.Inheritance.Merge emitter with removed modules", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorMergeEmitterWithRemovedModulesTest::RunTest(const FString& Parameters)
{
	TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();

	// Changes to source
	//   Emitter Update
	//	   EmitterLifetime->NextLoopDuration - Changed to 3
	//   Particle Spawn
	//     SetParticles.SpriteSize->Input->Minimum - changed to 10
	//	   SetParticles.SpriteRotation - removed.
	//	   UniformDistributionColor - Added and moved up 2 spaces.
	// Changes to instance
	//   Emitter Update
	//     Remove SpawnRate
	//   Particle Update
	//     Remove ApplyVelocity
	//     Remove ScaleSizeByVelocity
	UNiagaraEmitter* DefaultEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestEmitter.TestEmitter"));
	ASSERT_TRUE(DefaultEmitter != nullptr, TEXT("Could not load default emitter"));
	UNiagaraEmitter* EmitterWithRemovedModules = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestRemovedModules.TestRemovedModules"));
	ASSERT_TRUE(EmitterWithRemovedModules != nullptr, TEXT("Could not load emitter with removed modules."));
	UNiagaraEmitter* MergeSourceEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestMergeSource.TestMergeSource"));
	ASSERT_TRUE(MergeSourceEmitter != nullptr, TEXT("Could not load emitter with removed modules merge source."));
	UNiagaraEmitter* MergeTargetEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("RemovedModulesMergeTarget.RemovedModulesMergeTarget"));
	ASSERT_TRUE(MergeTargetEmitter != nullptr, TEXT("Could not load emitter with removed modules merge target."));

	INiagaraModule::FMergeEmitterResults MergeResults = MergeManager->MergeEmitter(*MergeSourceEmitter, *DefaultEmitter, *EmitterWithRemovedModules);

	ASSERT_TRUE(MergeResults.bSucceeded, *FString::Printf(TEXT("Merge failed.  Message: %s"), *MergeResults.GetErrorMessagesString()));
	ASSERT_TRUE(MergeResults.bModifiedGraph, TEXT("Merge reported success, but reported that the graph wasn't modified."));

	FNiagaraEmitterDiffResults MergedEmitterDiffResults = MergeManager->DiffEmitters(*MergeResults.MergedInstance, *MergeTargetEmitter);

	ASSERT_TRUE(MergedEmitterDiffResults.EmitterSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));
	ASSERT_TRUE(MergedEmitterDiffResults.EmitterUpdateDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));
	ASSERT_TRUE(MergedEmitterDiffResults.ParticleSpawnDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));
	ASSERT_TRUE(MergedEmitterDiffResults.ParticleUpdateDiffResults.IsEmpty(), TEXT("Script stack diff results weren't empty"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorMergeEmitterWithAddedModulesTest, "System.Plugins.Niagara.Editor.Inheritance.Merge emitter with added modules", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorMergeEmitterWithAddedModulesTest::RunTest(const FString& Parameters)
{
	TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();

	// Changes to source
	//   Emitter Update
	//	   EmitterLifetime->NextLoopDuration - Changed to 3
	//   Particle Spawn
	//     SetParticles.SpriteSize->Input->Minimum - changed to 10
	//	   SetParticles.SpriteRotation - removed.
	//	   UniformDistributionColor - Added and moved up 2 spaces.
	// Changes to instance
	//   Emitter Spawn
	//     Added SetParticles.NewNiagaraFloat0
	//   Emitter Update
	//     Added SetParticles.NewNiagaraFloat1
	//     Added SetParticles.NewNiagaraFloat2
	//   Particle Spawn
	//     Added SetParticles.NewNiagaraFloat3
	//     Added SetParticles.NewNiagaraFloat4
	//     Added SetParticles.NewNiagaraFloat5
	//   Particle Update
	//     Added SetParticles.NewNiagaraFloat6 (moved 6 -> 4)
	//     Added SetParticles.NewNiagaraFloat7
	//     Added SetParticles.NewNiagaraFloat8
	//     Added SetParticles.NewNiagaraFloat9
	UNiagaraEmitter* DefaultEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestEmitter.TestEmitter"));
	ASSERT_TRUE(DefaultEmitter != nullptr, TEXT("Could not load default emitter"));
	UNiagaraEmitter* EmitterWithAddedModules = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestAddedModules.TestAddedModules"));
	ASSERT_TRUE(EmitterWithAddedModules != nullptr, TEXT("Could not load emitter with added modules."));
	UNiagaraEmitter* MergeSourceEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("TestMergeSource.TestMergeSource"));
	ASSERT_TRUE(MergeSourceEmitter != nullptr, TEXT("Could not load emitter with added modules merge source."));
	UNiagaraEmitter* MergeTargetEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / TEXT("AddedModulesMergeTarget.AddedModulesMergeTarget"));
	ASSERT_TRUE(MergeTargetEmitter != nullptr, TEXT("Could not load emitter with added modules merge target."));

	INiagaraModule::FMergeEmitterResults MergeResults = MergeManager->MergeEmitter(*MergeSourceEmitter, *DefaultEmitter, *EmitterWithAddedModules);

	ASSERT_TRUE(MergeResults.bSucceeded, *FString::Printf(TEXT("Merge failed.  Message: %s"), *MergeResults.GetErrorMessagesString()));
	ASSERT_TRUE(MergeResults.bModifiedGraph, TEXT("Merge reported success, but reported that the graph wasn't modified."));

	FNiagaraEmitterDiffResults MergedEmitterDiffResults = MergeManager->DiffEmitters(*MergeResults.MergedInstance, *MergeTargetEmitter);

	// We don't end up with an empty diff here because the Guids for the added modules in the instance and target won't
	// match.  Instead we verify that they are at the correct index and the correct nodes.
	
	// Emitter Spawn
	FString EmitterSpawnDiffErrors;
	bool EmitterSpawnDiffMatches = CheckDiffResults(MergedEmitterDiffResults.EmitterSpawnDiffResults, 1, 1, 0, 0, 0, 0, EmitterSpawnDiffErrors);
	ASSERT_TRUE(EmitterSpawnDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *EmitterSpawnDiffErrors));

	FString EmitterSpawnModuleListErrors;
	bool EmitterSpawnModuleListsMatch = CheckModuleListsMatch(
		MergedEmitterDiffResults.EmitterSpawnDiffResults.RemovedBaseModules, MergedEmitterDiffResults.EmitterSpawnDiffResults.AddedOtherModules,
		EmitterSpawnModuleListErrors);
	ASSERT_TRUE(EmitterSpawnModuleListsMatch, *FString::Printf(TEXT("Added and removed module lists were different: %s"), *EmitterSpawnModuleListErrors));

	// Emitter Update
	FString EmitterUpdateDiffErrors;
	bool EmitterUpdateDiffMatches = CheckDiffResults(MergedEmitterDiffResults.EmitterUpdateDiffResults, 2, 2, 0, 0, 0, 0, EmitterUpdateDiffErrors);
	ASSERT_TRUE(EmitterUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *EmitterUpdateDiffErrors));

	FString EmitterUpdateModuleListErrors;
	bool EmitterUpdateModuleListsMatch = CheckModuleListsMatch(
		MergedEmitterDiffResults.EmitterUpdateDiffResults.RemovedBaseModules, MergedEmitterDiffResults.EmitterUpdateDiffResults.AddedOtherModules,
		EmitterUpdateModuleListErrors);
	ASSERT_TRUE(EmitterUpdateModuleListsMatch, *FString::Printf(TEXT("Added and removed module lists were different: %s"), *EmitterUpdateModuleListErrors));

	// Particle Spawn
	FString ParticleSpawnDiffErrors;
	bool ParticleSpawnDiffMatches = CheckDiffResults(MergedEmitterDiffResults.ParticleSpawnDiffResults, 3, 3, 0, 0, 0, 0, ParticleSpawnDiffErrors);
	ASSERT_TRUE(ParticleSpawnDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *ParticleSpawnDiffErrors));

	FString ParticleSpawnModuleListErrors;
	bool ParticleSpawnModuleListsMatch = CheckModuleListsMatch(
		MergedEmitterDiffResults.ParticleSpawnDiffResults.RemovedBaseModules, MergedEmitterDiffResults.ParticleSpawnDiffResults.AddedOtherModules,
		ParticleSpawnModuleListErrors);
	ASSERT_TRUE(ParticleSpawnModuleListsMatch, *FString::Printf(TEXT("Added and removed module lists were different: %s"), *ParticleSpawnModuleListErrors));

	// Particle Update
	FString ParticleUpdateDiffErrors;
	bool ParticleUpdateDiffMatches = CheckDiffResults(MergedEmitterDiffResults.ParticleUpdateDiffResults, 4, 4, 0, 0, 0, 0, ParticleUpdateDiffErrors);
	ASSERT_TRUE(ParticleUpdateDiffMatches, *FString::Printf(TEXT("Script stack diff was wrong.%s"), *ParticleUpdateDiffErrors));

	FString ParticleUpdateModuleListErrors;
	bool ParticleUpdateModuleListsMatch = CheckModuleListsMatch(
		MergedEmitterDiffResults.ParticleUpdateDiffResults.RemovedBaseModules, MergedEmitterDiffResults.ParticleUpdateDiffResults.AddedOtherModules,
		ParticleUpdateModuleListErrors);
	ASSERT_TRUE(ParticleUpdateModuleListsMatch, *FString::Printf(TEXT("Added and removed module lists were different: %s"), *ParticleUpdateModuleListErrors));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorMergeEmitterWithModifiedDuplicateDynamicInptus, "System.Plugins.Niagara.Editor.Inheritance.Merge Emitter With Modified Duplicate Dynamic Inptus", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorMergeEmitterWithModifiedDuplicateDynamicInptus::RunTest(const FString& Parameters)
{
	TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();

	FString SubDirectory = "ModifiedDuplicateDynamicInputs";

	UNiagaraEmitter* OriginalSourceEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / SubDirectory / TEXT("OriginalSource.OriginalSource"));
	ASSERT_TRUE(OriginalSourceEmitter != nullptr, TEXT("Could not load original source emitter"));

	UNiagaraEmitter* InstanceEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / SubDirectory / TEXT("Instance.Instance"));
	ASSERT_TRUE(InstanceEmitter != nullptr, TEXT("Could not load instance emitter"));

	UNiagaraEmitter* NewSourceEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / SubDirectory / TEXT("NewSource.NewSource"));
	ASSERT_TRUE(NewSourceEmitter != nullptr, TEXT("Could not load new source emitter"));

	UNiagaraEmitter* MergedEmitter = FNiagaraEditorTestUtilities::LoadEmitter(TestDataRoot / SubDirectory / TEXT("Merged.Merged"));
	ASSERT_TRUE(MergedEmitter != nullptr, TEXT("Could not load new source emitter"));

	INiagaraModule::FMergeEmitterResults MergeResults = MergeManager->MergeEmitter(*NewSourceEmitter, *OriginalSourceEmitter, *InstanceEmitter);
	ASSERT_TRUE(MergeResults.bSucceeded, *FString::Printf(TEXT("Merge failed.  Message: %s"), *MergeResults.GetErrorMessagesString()));
	ASSERT_TRUE(MergeResults.bModifiedGraph, TEXT("Merge reported success, but reported that the graph wasn't modified."));

	FNiagaraEmitterDiffResults MergedEmitterDiffResults = MergeManager->DiffEmitters(*MergeResults.MergedInstance, *MergedEmitter);
	ASSERT_TRUE(MergedEmitterDiffResults.IsEmpty(), TEXT("Actual merged emitter was different from the expected merged emitter"));

	return true;
}
*/





