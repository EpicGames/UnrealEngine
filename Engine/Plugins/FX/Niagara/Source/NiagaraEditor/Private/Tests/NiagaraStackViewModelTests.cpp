// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStackViewModelTests.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"
#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraEditorTestUtilities.h"

#include "Misc/AutomationTest.h"


/*
UNiagaraStackViewModel* CreateStackViewModelForEmitter(UNiagaraEmitter* Emitter)
{
	TSharedRef<FNiagaraSystemViewModel> SystemViewModel = FNiagaraEditorTestUtilities::CreateTestSystemViewModelForEmitter(Emitter);
	TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = SystemViewModel->GetEmitterHandleViewModels()[0];
	UNiagaraStackViewModel* StackViewModel = NewObject<UNiagaraStackViewModel>();
	StackViewModel->Initialize(SystemViewModel, EmitterHandleViewModel);
	return StackViewModel;
}
*/

/*
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNiagaraEditorStackViewModelAddParameterModulesTest, "System.Plugins.Niagara.Editor.StackViewModel.Add Parameter Modules", EAutomationTestFlags::EditorContext | EAutomationTestFlags::CriticalPriority | EAutomationTestFlags::EngineFilter)

bool FNiagaraEditorStackViewModelAddParameterModulesTest::RunTest(const FString& Parameters)
{

	UNiagaraEmitter* Emitter = FNiagaraEditorTestUtilities::CreateEmptyTestEmitter();
	UNiagaraStackViewModel* StackViewModel = CreateStackViewModelForEmitter(Emitter);

	UNiagaraStackRoot* StackRoot = CastChecked<UNiagaraStackRoot>(StackViewModel->GetRootEntries()[0]);
	ASSERT_TRUE(StackRoot != nullptr, TEXT("Get root entry failed"));

	UNiagaraStackScriptItemGroup* EmitterSpawnGroup = StackRoot->FindChildOfTypeByPredicate<UNiagaraStackScriptItemGroup>(
		[](UNiagaraStackScriptItemGroup* Group) { return Group->GetScriptUsage() == ENiagaraScriptUsage::EmitterSpawnScript; });
	ASSERT_TRUE(EmitterSpawnGroup != nullptr, TEXT("Get group entry failed"));

	UNiagaraStackAddModuleItem* EmitterSpawnAddModuleItem = EmitterSpawnGroup->FindChildOfTypeByPredicate<UNiagaraStackAddModuleItem>(
		[](UNiagaraStackAddModuleItem* AddItem) { return true; });
	ASSERT_TRUE(EmitterSpawnAddModuleItem != nullptr, TEXT("Get add module item entry failed"));

	EmitterSpawnAddModuleItem->AddParameterModule(
		FNiagaraVariable(
			FNiagaraTypeDefinition::GetFloatDef(), 
			FName(FNiagaraParameterHandle(FNiagaraParameterHandle::EmitterNamespace, TEXT("MyFloat1")).GetParameterHandleString())),
		false);

	EmitterSpawnAddModuleItem->AddParameterModule(
		FNiagaraVariable(
			FNiagaraTypeDefinition::GetFloatDef(),
			FName(FNiagaraParameterHandle(FNiagaraParameterHandle::EmitterNamespace, TEXT("MyFloat2")).GetParameterHandleString())),
		false);

	UNiagaraStackModuleItem* MyFloat1Module = EmitterSpawnGroup->FindChildOfTypeByPredicate<UNiagaraStackModuleItem>(
		[](UNiagaraStackModuleItem* ModuleItem) { return ModuleItem->GetModuleNode().GetFunctionName().Contains(TEXT("MyFloat1")); });
	ASSERT_TRUE(MyFloat1Module != nullptr, TEXT("Get module entry failed"));

	UNiagaraStackModuleItem* MyFloat2Module = EmitterSpawnGroup->FindChildOfTypeByPredicate<UNiagaraStackModuleItem>(
		[](UNiagaraStackModuleItem* ModuleItem) { return ModuleItem->GetModuleNode().GetFunctionName().Contains(TEXT("MyFloat2")); });
	ASSERT_TRUE(MyFloat2Module != nullptr, TEXT("Get module entry failed"));

	UNiagaraStackFunctionInputCollection* MyFloat1ModuleInputs = MyFloat1Module->FindChildOfTypeByPredicate<UNiagaraStackFunctionInputCollection>(
		[](UNiagaraStackFunctionInputCollection* InputCollection) { return InputCollection->GetDisplayName().ToString() == TEXT("Inputs"); });
	ASSERT_TRUE(MyFloat1ModuleInputs != nullptr, TEXT("Get input collection entry failed"));

	UNiagaraStackFunctionInput* MyFloatInput = MyFloat1ModuleInputs->FindChildOfTypeByPredicate<UNiagaraStackFunctionInput>(
		[](UNiagaraStackFunctionInput* Input) { return Input->GetInputParameterHandle().GetName() == TEXT("MyFloat1"); });
	ASSERT_TRUE(MyFloatInput != nullptr, TEXT("Get input entry failed"));

	return true;
}
*/