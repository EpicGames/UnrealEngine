// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterCollectionToolkit.h"
#include "NiagaraEditorModule.h"
#include "Widgets/SNiagaraParameterCollection.h"

#include "NiagaraObjectSelection.h"
#include "Widgets/SNiagaraSelectedObjectsDetails.h"
#include "NiagaraEditorCommands.h"
#include "NiagaraEditorStyle.h"
#include "ScopedTransaction.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "Styling/AppStyle.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Framework/Docking/WorkspaceItem.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "ViewModels/NiagaraCollectionParameterViewModel.h"
#include "NiagaraParameterCollection.h"
#include "ViewModels/NiagaraParameterCollectionAssetViewModel.h"

#include "PropertyEditorModule.h"
#include "IDetailCustomization.h"
#include "IDetailsView.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "NiagaraParameterCollectionEditor"

namespace NiagaraParameterCollectionToolkitPrivate
{
	class FNPCDetailCustomization : public IDetailCustomization
	{
	public:
		static TSharedRef<IDetailCustomization> MakeInstance()
		{
			return MakeShared<FNPCDetailCustomization>();
		}

		// Begin: IDetailCustomization
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override
		{
			static const FName NAME_ParameterColection("Parameter Collection");
			static const FName NAME_Namespace("Namespace");

			IDetailCategoryBuilder& DetailCategory = DetailBuilder.EditCategory(NAME_ParameterColection);

			TArray<TSharedRef<IPropertyHandle>> CategoryProperties;
			DetailCategory.GetDefaultProperties(CategoryProperties);

			for (TSharedRef<IPropertyHandle>& PropertyHandle : CategoryProperties)
			{
				const FName PropertyName = PropertyHandle->GetProperty() ? PropertyHandle->GetProperty()->GetFName() : NAME_None;
				if (PropertyName == NAME_Namespace)
				{
					TOptional<FAssetData> OwningAsset;
					{
						TArray<UObject*> OwningObjects;
						PropertyHandle->GetOuterObjects(OwningObjects);

						if (OwningObjects.Num() == 1)
						{
							OwningAsset.Emplace(OwningObjects[0]->GetPackage());
						}
					}


					IDetailPropertyRow& PropertyRow = DetailCategory.AddProperty(PropertyHandle);
					PropertyRow.CustomWidget()
					.NameContent()
					[
						PropertyHandle->CreatePropertyNameWidget()
					]
					.ValueContent()
					[
						SNew(SEditableTextBox)
						.Text_Lambda(
							[PropertyHandle]()
							{
								FText Val;
								PropertyHandle->GetValueAsFormattedText(Val);
								return Val;
							}
						)
						.OnTextCommitted_Lambda(
							[PropertyHandle, OwningAsset](const FText& NewText, ETextCommit::Type CommitType)
							{
								FText ExistingText;
								PropertyHandle->GetValueAsFormattedText(ExistingText);
								if (NewText.EqualTo(ExistingText))
								{
									return;
								}

								// Intentionally set the value as the dialog will cause reenterance into the function and we will early out
								PropertyHandle->SetValueFromFormattedString(NewText.ToString());

								// Look to see if any assets would be impacted by this change
								// If so allow the user to opt out of the change as committed would break the asset
								if (OwningAsset.IsSet())
								{
									TArray<FAssetData> FoundReferencers;
									const int32 ReferencedAssetCount =
										FNiagaraEditorUtilities::GetReferencedAssetCount(
										OwningAsset.GetValue(),
										[&FoundReferencers](const FAssetData& AssetToCheck)
										{
											// Ignore any references from NPC / NPCIs but we need to include all other referencers
											if (AssetToCheck.GetClass() == UNiagaraParameterCollection::StaticClass() || AssetToCheck.GetClass() == UNiagaraParameterCollectionInstance::StaticClass())
											{
												return FNiagaraEditorUtilities::ETrackAssetResult::Ignore;
											}
											FoundReferencers.Add(AssetToCheck);
											return FNiagaraEditorUtilities::ETrackAssetResult::Count;
										}
									);

									if (ReferencedAssetCount > 0)
									{
										const EAppReturnType::Type DialogReturnValue =
											FMessageDialog::Open(
												EAppMsgType::OkCancel,
												FText::Format(LOCTEXT("CommitNamespaceChangeMessage", "Changing the namespace will impact {0} known asset(s), you will need to manually fix up these references.  See the log for a full list."), ReferencedAssetCount),
												LOCTEXT("CommitNamespaceChangeTitle", "Commit namespace change?")
											);

										UE_LOGF(LogNiagaraEditor, Log, "Known NPC referencers...");
										for (const FAssetData& FoundReferencer : FoundReferencers)
										{
											UE_LOGF(LogNiagaraEditor, Log, "\t%ls", *FoundReferencer.GetFullName());
										}

										if (DialogReturnValue == EAppReturnType::Cancel)
										{
											PropertyHandle->SetValueFromFormattedString(ExistingText.ToString());
										}
									}
								}
							}
						)
					];
				}
				else
				{
					DetailCategory.AddProperty(PropertyHandle);
				}
			}
		}
		// End: IDetailCustomization
	};
} // namespace NiagaraParameterCollectionToolkitPrivate

const FName FNiagaraParameterCollectionToolkit::MainTabID(TEXT("NiagaraParameterCollectionEditor_Main"));

void FNiagaraParameterCollectionToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_NiagaraParameterCollectionEditor", "Niagara Parameter Collection"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(MainTabID, FOnSpawnTab::CreateSP(this, &FNiagaraParameterCollectionToolkit::SpawnTab_Main))
		.SetDisplayName(LOCTEXT("Parameters", "Parameters"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));
}

void FNiagaraParameterCollectionToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(MainTabID);
}


FNiagaraParameterCollectionToolkit::~FNiagaraParameterCollectionToolkit()
{
}

void FNiagaraParameterCollectionToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Collection);
	Collector.AddReferencedObject(Instance);
}

void FNiagaraParameterCollectionToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraParameterCollection* InCollection)
{
	Collection = InCollection;
	checkf(Collection != nullptr, TEXT("Can not create toolkit with null parameter collection."));
	Instance = Collection->GetDefaultInstance();

	ParameterCollectionViewModel = MakeShareable(new FNiagaraParameterCollectionAssetViewModel(Collection, FText::FromString(Collection->GetName()), ENiagaraParameterEditMode::EditAll));

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_Niagara_ParameterCollection_Layout_V1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.1f)
				->AddTab(MainTabID, ETabState::OpenedTab)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FNiagaraEditorModule::NiagaraEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, Collection);

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	AddMenuExtender(NiagaraEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	SetupCommands();
	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

void FNiagaraParameterCollectionToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraParameterCollectionInstance* InInstance)
{
	Instance = InInstance;
	checkf(Instance != nullptr, TEXT("Can not create toolkit with null parameter collection instance."));
	Collection = Instance->GetParent();

	ParameterCollectionViewModel = MakeShareable(new FNiagaraParameterCollectionAssetViewModel(InInstance, FText::FromString(InInstance->GetName()), ENiagaraParameterEditMode::EditAll));

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_Niagara_ParameterCollection_Layout_V1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.1f)
				->AddTab(MainTabID, ETabState::OpenedTab)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FNiagaraEditorModule::NiagaraEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InInstance);

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	AddMenuExtender(NiagaraEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	SetupCommands();
	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

FName FNiagaraParameterCollectionToolkit::GetToolkitFName() const
{
	return FName("Niagara");
}

FText FNiagaraParameterCollectionToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Niagara");
}

FString FNiagaraParameterCollectionToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Niagara ").ToString();
}


FLinearColor FNiagaraParameterCollectionToolkit::GetWorldCentricTabColorScale() const
{
	return FNiagaraEditorModule::WorldCentricTabColorScale;
}

TSharedRef<SDockTab> FNiagaraParameterCollectionToolkit::SpawnTab_Main(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == MainTabID);

	ParameterCollection = SNew(SNiagaraParameterCollection, ParameterCollectionViewModel.ToSharedRef());

	TSharedRef<SVerticalBox> Contents = SNew(SVerticalBox);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = ParameterCollectionViewModel.Get();
	DetailsViewArgs.bAllowSearch = false;
	
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->RegisterInstancedCustomPropertyLayout(UNiagaraParameterCollection::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&NiagaraParameterCollectionToolkitPrivate::FNPCDetailCustomization::MakeInstance));

	if (Instance->IsDefaultInstance())
	{
		DetailsView->SetObject(Collection);
	}
	else
	{
		DetailsView->SetObject(Instance);
	}

	Contents->AddSlot()
	.AutoHeight()
	.Padding(FMargin(0.0f, 2.0f))
	[
		SNew(SSearchBox)
		.OnTextChanged(this, &FNiagaraParameterCollectionToolkit::OnSearchTextChanged)
	];
	
	Contents->AddSlot()
	.AutoHeight()
	.Padding(FMargin(0.0f, 2.0f))
	[
		DetailsView
	];

	Contents->AddSlot()
	.Padding(FMargin(0.0f, 2.0f))
	[
		ParameterCollection.ToSharedRef()
	];
	
	TSharedRef<SDockTab> SpawnedTab =
	SNew(SDockTab)
	[
		Contents
	];

	return SpawnedTab;
}

void FNiagaraParameterCollectionToolkit::OnSearchTextChanged(const FText& InSearchText)
{
	ParameterCollectionViewModel->UpdateParameterSelectionFromSearch(InSearchText);
}

void FNiagaraParameterCollectionToolkit::SetupCommands()
{
	// 	GetToolkitCommands()->MapAction(
	// 		FNiagaraEditorCommands::Get().ToggleUnlockToChanges,
	// 		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ToggleUnlockToChanges),
	// 		FCanExecuteAction(),
	// 		FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsToggleUnlockToChangesChecked));
}

void FNiagaraParameterCollectionToolkit::ExtendToolbar()
{
	// 	struct Local
	// 	{
	// 		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FOnGetContent GetEmitterMenuContent, FNiagaraSystemToolkit* Toolkit)
	// 		{
	// 			ToolbarBuilder.BeginSection("LockEmitters");
	// 			{
	// 				ToolbarBuilder.AddToolBarButton(FNiagaraEditorCommands::Get().ToggleUnlockToChanges, NAME_None,
	// 					TAttribute<FText>(Toolkit, &FNiagaraSystemToolkit::GetEmitterLockToChangesLabel),
	// 					TAttribute<FText>(Toolkit, &FNiagaraSystemToolkit::GetEmitterLockToChangesLabelTooltip),
	// 					TAttribute<FSlateIcon>(Toolkit, &FNiagaraSystemToolkit::GetEmitterLockToChangesIcon));
	// 			}
	// 			ToolbarBuilder.EndSection();
	// 			ToolbarBuilder.BeginSection("AddEmitter");
	// 			{
	// 				ToolbarBuilder.AddComboButton(
	// 					FUIAction(),
	// 					GetEmitterMenuContent,
	// 					LOCTEXT("AddEmitterButtonText", "Add Emitter"),
	// 					LOCTEXT("AddEmitterButtonTextToolTip", "Adds an emitter to the system from an existing emitter asset."),
	// 					FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.AddEmitter"));
	// 			}
	// 			ToolbarBuilder.EndSection();
	// 		}
	// 	};

	//TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	//	FOnGetContent GetEmitterMenuContent = FOnGetContent::CreateSP(this, &FNiagaraSystemToolkit::CreateAddEmitterMenuContent);

	// 	ToolbarExtender->AddToolBarExtension(
	// 		"Asset",
	// 		EExtensionHook::After,
	// 		GetToolkitCommands(),
	// 		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, GetEmitterMenuContent, this)
	// 	);

	//AddToolbarExtender(ToolbarExtender);

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	AddToolbarExtender(NiagaraEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

#undef LOCTEXT_NAMESPACE
