// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/NotifyHook.h"
#include "Input/Reply.h"
#include "UObject/GCObject.h"
#include "Toolkits/IToolkitHost.h"

#include "Toolkits/AssetEditorToolkit.h"

class SNiagaraParameterCollection;
class FNiagaraParameterCollectionAssetViewModel;

/** Viewer/editor for a NiagaraParameterCollections
*/
class FNiagaraParameterCollectionToolkit : public FAssetEditorToolkit, public FGCObject
{
public:
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/** Edits the specified Niagara Script */
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UNiagaraParameterCollection* InCollection);
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UNiagaraParameterCollectionInstance* InInstance);

	/** Destructor */
	virtual ~FNiagaraParameterCollectionToolkit();

	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Interface

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraParameterCollectionToolkit");
	}

private:

	TSharedRef<SDockTab> SpawnTab_Main(const FSpawnTabArgs& Args);

	void OnSearchTextChanged(const FText& InSearchText);
	
	/** Builds the toolbar widget */
	void ExtendToolbar();
	void SetupCommands();

	/** The Collection being edited. */
	TObjectPtr<UNiagaraParameterCollection> Collection;

	/** The Instance being edited. */
	TObjectPtr<UNiagaraParameterCollectionInstance> Instance;

	/** Widget for editing parameter collection. */
	TSharedPtr<SNiagaraParameterCollection>	ParameterCollection;

	/* The view model for the NPC being edited */
	TSharedPtr<FNiagaraParameterCollectionAssetViewModel> ParameterCollectionViewModel;

	/** The command list for this editor */
	TSharedPtr<FUICommandList> EditorCommands;

	/**	Main tab */
	static const FName MainTabID;
};
