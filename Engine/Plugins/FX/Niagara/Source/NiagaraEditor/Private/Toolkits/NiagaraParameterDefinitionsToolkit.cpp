// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterDefinitionsToolkit.h"

#include "BusyCursor.h"
#include "Engine/Selection.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "NiagaraEditorCommands.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraParameterDefinitions.h"
#include "PropertyEditorModule.h"
#include "UObject/Linker.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SNiagaraParameterPanel.h"
#include "Widgets/SNiagaraSelectedObjectsDetails.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ScopedTransaction.h"


#define LOCTEXT_NAMESPACE "NiagaraParameterDefinitionsToolkit"

const FName FNiagaraParameterDefinitionsToolkit::ParameterDefinitionsDetailsTabId(TEXT("NiagaraParameterDefinitionsEditor_Options"));
const FName FNiagaraParameterDefinitionsToolkit::ParameterPanelTabId(TEXT("NiagaraParameterDefinitionsEditor_Parameters"));
const FName FNiagaraParameterDefinitionsToolkit::SelectedDetailsTabId(TEXT("NiagaraParameterDefinitionsEditor_SelectedDetails"));

FNiagaraParameterDefinitionsToolkit::FNiagaraParameterDefinitionsToolkit()
{}

FNiagaraParameterDefinitionsToolkit::~FNiagaraParameterDefinitionsToolkit()
{
	if (ParameterPanelViewModel.IsValid())
	{
		ParameterPanelViewModel->Cleanup();
	}
	if (SelectedScriptVarDetailsWidget.IsValid())
	{
		SelectedScriptVarDetailsWidget->OnFinishedChangingProperties().RemoveAll(this);
	}
}

void FNiagaraParameterDefinitionsToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_NiagaraEditor", "Niagara"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	InTabManager->RegisterTabSpawner(ParameterDefinitionsDetailsTabId, FOnSpawnTab::CreateSP(this, &FNiagaraParameterDefinitionsToolkit::SpawnTab_ParameterDefinitionsDetails))
		.SetDisplayName(LOCTEXT("ParameterDefinitionsDetailsTab", "Options"))
		.SetGroup(WorkspaceMenuCategoryRef);

	InTabManager->RegisterTabSpawner(SelectedDetailsTabId, FOnSpawnTab::CreateSP(this, &FNiagaraParameterDefinitionsToolkit::SpawnTab_SelectedScriptVarDetails))
		.SetDisplayName(LOCTEXT("SelectedDetailsTab", "Selected Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(ParameterPanelTabId, FOnSpawnTab::CreateSP(this, &FNiagaraParameterDefinitionsToolkit::SpawnTab_ParameterPanel))
		.SetDisplayName(LOCTEXT("ParameterPanel", "Parameters"))
		.SetGroup(WorkspaceMenuCategoryRef);
}

void FNiagaraParameterDefinitionsToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(ParameterDefinitionsDetailsTabId);
	InTabManager->UnregisterTabSpawner(SelectedDetailsTabId);
	InTabManager->UnregisterTabSpawner(ParameterPanelTabId);
}

void FNiagaraParameterDefinitionsToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraParameterDefinitions* InParameterDefinitions)
{
	bChangesDiscarded = false;
	bEditedParameterDefinitionsHasPendingChanges = false;

	check(InParameterDefinitions != nullptr);
	ParameterDefinitionsSource = InParameterDefinitions;

	// No need to reset loader or versioning on the transient package, there should never be any set
	ParameterDefinitionsInstance = Cast<UNiagaraParameterDefinitions>(StaticDuplicateObject(ParameterDefinitionsSource, GetTransientPackage(), NAME_None, ~RF_Standalone, UNiagaraParameterDefinitions::StaticClass()));
	ParameterDefinitionsInstance->ClearFlags(RF_Standalone | RF_Public);
	LastSyncedDefinitionsChangeIdHash = ParameterDefinitionsInstance->GetChangeIdHash();
	ParameterDefinitionsInstance->GetOnParameterDefinitionsChanged().AddRaw(this, &FNiagaraParameterDefinitionsToolkit::OnEditedParameterDefinitionsChanged);

	// Determine display name for panel heading based on asset usage type
	FText DisplayName = LOCTEXT("NiagaraParameterDefinitionsDisplayName", "Niagara Parameter Definitions");

	DetailsScriptSelection = MakeShareable(new FNiagaraObjectSelection());
	ParameterPanelViewModel = MakeShareable(new FNiagaraParameterDefinitionsToolkitParameterPanelViewModel(ParameterDefinitionsInstance, DetailsScriptSelection));

	FParameterDefinitionsToolkitUIContext UIContext = FParameterDefinitionsToolkitUIContext(
		FSimpleDelegate::CreateSP(ParameterPanelViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh),
		FSimpleDelegate::CreateSP(DetailsScriptSelection.ToSharedRef(), &FNiagaraObjectSelection::Refresh)
	);

	ParameterPanelViewModel->Init(UIContext);

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_Niagara_ParameterDefinitionsLayout_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.15f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(.15f)
					->AddTab(ParameterDefinitionsDetailsTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(.85f)
					->AddTab(ParameterPanelTabId, ETabState::OpenedTab)
					->SetForegroundTab(ParameterPanelTabId)
				)

			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.5f)
				->AddTab(SelectedDetailsTabId, ETabState::OpenedTab)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FNiagaraEditorModule::NiagaraEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InParameterDefinitions);

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	AddMenuExtender(NiagaraEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	SetupCommands();
	ExtendToolbar();

	RegenerateMenusAndToolbars();
}

FName FNiagaraParameterDefinitionsToolkit::GetToolkitFName() const
{
	return FName("Niagara");
}

FText FNiagaraParameterDefinitionsToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Niagara");
}

FString FNiagaraParameterDefinitionsToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Niagara ").ToString();
}

FLinearColor FNiagaraParameterDefinitionsToolkit::GetWorldCentricTabColorScale() const
{
	return FNiagaraEditorModule::WorldCentricTabColorScale;
}

void FNiagaraParameterDefinitionsToolkit::GetSaveableObjects(TArray<UObject*>& OutObjects) const
{
	OutObjects.Add(static_cast<UObject*>(ParameterDefinitionsSource));
}

void FNiagaraParameterDefinitionsToolkit::SaveAsset_Execute()
{
	OnApply();
	FAssetEditorToolkit::SaveAsset_Execute();
}

void FNiagaraParameterDefinitionsToolkit::SaveAssetAs_Execute()
{
	OnApply();
	FAssetEditorToolkit::SaveAsset_Execute();
}

bool FNiagaraParameterDefinitionsToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	if (InCloseReason != EAssetEditorCloseReason::AssetForceDeleted && bChangesDiscarded == false && OnApplyEnabled())
	{
		// find out the user wants to do with this dirty NiagaraScript
		EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(
			EAppMsgType::YesNoCancel,
			LOCTEXT("Prompt_NiagaraParameterDefinitionsEditorClose", "Would you like to apply changes to these parameter definitions?\n(No will lose all changes!)"));

		// act on it
		switch (YesNoCancelReply)
		{
		case EAppReturnType::Yes:
			// update ParameterDefinitionsSource and exit
			OnApply();
			break;

		case EAppReturnType::No:
			// Set the changes discarded to avoid showing the dialog multiple times when request close is called multiple times on shut down.
			bChangesDiscarded = true;
			break;

		case EAppReturnType::Cancel:
			// don't exit
			return false;
		}
	}

	return true;
}

TSharedRef<SDockTab> FNiagaraParameterDefinitionsToolkit::SpawnTab_ParameterDefinitionsDetails(const FSpawnTabArgs& Args)
{
	checkf(Args.GetTabId().TabType == ParameterDefinitionsDetailsTabId, TEXT("Wrong tab ID in NiagaraParameterDefinitionsToolkit!"));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	ParameterDefinitionsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	ParameterDefinitionsDetailsView->SetObject(ParameterDefinitionsInstance);

	return SNew(SDockTab)
	.Label(LOCTEXT("ParameterDefinitionsDetailsTabLabel", "Options"))
	.TabColorScale(GetTabColorScale())
	[
		ParameterDefinitionsDetailsView.ToSharedRef()
	];
}

TSharedRef<SDockTab> FNiagaraParameterDefinitionsToolkit::SpawnTab_ParameterPanel(const FSpawnTabArgs& Args)
{
	checkf(Args.GetTabId().TabType == ParameterPanelTabId, TEXT("Wrong tab ID in NiagaraScriptToolkit"));

	return SNew(SDockTab)
	[
		SNew(SNiagaraParameterPanel, ParameterPanelViewModel, GetToolkitCommands())
		.ShowParameterSynchronizingWithLibraryIcon(false)
		.ShowParameterReferenceCounter(false)
	];
}

TSharedRef<SDockTab> FNiagaraParameterDefinitionsToolkit::SpawnTab_SelectedScriptVarDetails(const FSpawnTabArgs& Args)
{
	checkf(Args.GetTabId().TabType == SelectedDetailsTabId, TEXT("Wrong tab ID in NiagaraParameterDefinitionsToolkit!"));

	TSharedRef<SDockTab> DockTab =
		SNew(SDockTab)
		.Label(LOCTEXT("SelectedDetailsTabLabel", "Selected Details"))
		.TabColorScale(GetTabColorScale())
		[
			SAssignNew(SelectedScriptVarDetailsWidget, SNiagaraSelectedObjectsDetails, DetailsScriptSelection.ToSharedRef())
			.AllowEditingLibraryScriptVariables(true)
		];

	SelectedScriptVarDetailsWidget->OnFinishedChangingProperties().AddSP(this, &FNiagaraParameterDefinitionsToolkit::OnEditedParameterDefinitionsPropertyFinishedChanging);

	return DockTab;
}

void FNiagaraParameterDefinitionsToolkit::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FNiagaraParameterDefinitionsToolkit* Toolkit)
		{
			if (Toolkit->ParameterDefinitionsSource != nullptr)
			{
				ToolbarBuilder.BeginSection("Apply");
				{
					ToolbarBuilder.AddToolBarButton(FNiagaraEditorCommands::Get().Apply,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.Apply"),
						FName(TEXT("ApplyNiagaraEmitter")));
				}
				ToolbarBuilder.EndSection();
			}
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, this)
	);

	AddToolbarExtender(ToolbarExtender);

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	AddToolbarExtender(NiagaraEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void FNiagaraParameterDefinitionsToolkit::SetupCommands()
{
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().Apply,
		FExecuteAction::CreateSP(this, &FNiagaraParameterDefinitionsToolkit::OnApply),
		FCanExecuteAction::CreateSP(this, &FNiagaraParameterDefinitionsToolkit::OnApplyEnabled));
}

void FNiagaraParameterDefinitionsToolkit::OnApply()
{
	const FScopedBusyCursor BusyCursor;
	const FText LocalizedScriptEditorApply = NSLOCTEXT("UnrealEd", "ToolTip_NiagaraParameterDefinitionsEditorApply", "Apply changes to parameter definitions.");
	GWarn->BeginSlowTask(LocalizedScriptEditorApply, true);
	GWarn->StatusUpdate(1, 1, LocalizedScriptEditorApply);

	if (ParameterDefinitionsSource->IsSelected())
	{
		GEditor->GetSelectedObjects()->Deselect(ParameterDefinitionsSource);
	}

	ResetLoaders(ParameterDefinitionsSource->GetOutermost()); // Make sure that we're not going to get invalid version number linkers into the package we are going into. 

	ParameterDefinitionsSource->PreEditChange(nullptr);
	// overwrite the original parameter definitions in place by constructing a new one with the same name
	ParameterDefinitionsSource = (UNiagaraParameterDefinitions*)StaticDuplicateObject(ParameterDefinitionsInstance, ParameterDefinitionsSource->GetOuter(),
		ParameterDefinitionsSource->GetFName(), RF_AllFlags, ParameterDefinitionsSource->GetClass());

	// Restore RF_Standalone and RF_Public on the original parameter definitions.
	ParameterDefinitionsSource->SetFlags(RF_Standalone | RF_Public);

	// Invoke onchanged events for listeners.
	ParameterDefinitionsSource->PostEditChange();
	ParameterDefinitionsSource->NotifyParameterDefinitionsChanged();
	
	// If the ChangeIdHash is different, apply to all scripts/emitters/systems in editor. Record the last synced ChangeIdHash to detect future changes.
	if (ParameterDefinitionsInstance->GetChangeIdHash() != LastSyncedDefinitionsChangeIdHash)
	{
		FScopedTransaction Transaction(LOCTEXT("ApplyParameterDefinitionsChanges", "Apply Parameter Definitions Changes"));
		FRefreshAllScriptsFromExternalChangesArgs Args;
		Args.OriginatingParameterDefinitions = ParameterDefinitionsSource;
		FNiagaraEditorUtilities::RefreshAllScriptsFromExternalChanges(Args);
		LastSyncedDefinitionsChangeIdHash = ParameterDefinitionsInstance->GetChangeIdHash();
	}

	bEditedParameterDefinitionsHasPendingChanges = false;
	GWarn->EndSlowTask();
	FNiagaraEditorModule::Get().InvalidateCachedScriptAssetData();
}

bool FNiagaraParameterDefinitionsToolkit::OnApplyEnabled() const
{
	return bEditedParameterDefinitionsHasPendingChanges;
}

void FNiagaraParameterDefinitionsToolkit::OnEditedParameterDefinitionsPropertyFinishedChanging(const FPropertyChangedEvent& InEvent)
{
	bEditedParameterDefinitionsHasPendingChanges = true;
}

void FNiagaraParameterDefinitionsToolkit::OnEditedParameterDefinitionsChanged()
{
	bEditedParameterDefinitionsHasPendingChanges = true;
}

#undef LOCTEXT_NAMESPACE /*NiagaraParameterDefinitionsToolkit*/
