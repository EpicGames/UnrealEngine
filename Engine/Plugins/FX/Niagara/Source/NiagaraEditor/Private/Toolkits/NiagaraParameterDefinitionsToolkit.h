// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"

	class IDetailsView;
	class FNiagaraObjectSelection;
	class FNiagaraParameterDefinitionsToolkitParameterPanelViewModel;
	class SNiagaraSelectedObjectsDetails;

	class UNiagaraParameterDefinitions;

// /** Viewer/editor for Parameter Definitions */
class FNiagaraParameterDefinitionsToolkit : public FAssetEditorToolkit
{
public:
	FNiagaraParameterDefinitionsToolkit();

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;

	/** Edits the specified Niagara Parameter Definitions */
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraParameterDefinitions* InParameterDefinitions);

	/** Destructor */
	virtual ~FNiagaraParameterDefinitionsToolkit();

	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Interface

protected:
	//~ FAssetEditorToolkit interface
	virtual void GetSaveableObjects(TArray<UObject*>& OutObjects) const override;
	virtual void SaveAsset_Execute() override;
	virtual void SaveAssetAs_Execute() override;
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;

private:
	TSharedRef<SDockTab> SpawnTab_ParameterDefinitionsDetails(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_ParameterPanel(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SelectedScriptVarDetails(const FSpawnTabArgs& Args);

	void ExtendToolbar();
	void SetupCommands();

	void OnApply();
	bool OnApplyEnabled() const;

	void OnEditedParameterDefinitionsPropertyFinishedChanging(const FPropertyChangedEvent& InEvent);
	void OnEditedParameterDefinitionsChanged();

private:
	/** The duplicate instance of ParameterDefinitions to be live edited. Overwrites ParameterDefinitionsSource when changes are applied. */
	UNiagaraParameterDefinitions* ParameterDefinitionsInstance;

	/** The source instance of ParameterDefinitions to be saved to disk. Is overwritten by ParameterDefinitionsInstance when changes are applied. */
	UNiagaraParameterDefinitions* ParameterDefinitionsSource;

	/** The value of the parameter definitions instance change id hash from the last time it was in sync with the parameter definitions source. */
	int32 LastSyncedDefinitionsChangeIdHash;

	/** The details view for the edited parameter definitions asset. */
	TSharedPtr<class IDetailsView> ParameterDefinitionsDetailsView;

	/** The Parameter Panel displaying parameter definitions. */
	TSharedPtr<FNiagaraParameterDefinitionsToolkitParameterPanelViewModel> ParameterPanelViewModel;

	TSharedPtr<SNiagaraSelectedObjectsDetails> SelectedScriptVarDetailsWidget;

	/** The selection displayed by the details tab. */
	TSharedPtr<FNiagaraObjectSelection> DetailsScriptSelection;

	bool bChangesDiscarded = false;
	bool bEditedParameterDefinitionsHasPendingChanges = false;

	/**	The tab ids for the Parameter Definitions editor */
	static const FName ParameterDefinitionsDetailsTabId;
	static const FName ParameterPanelTabId;
	static const FName SelectedDetailsTabId;
};
