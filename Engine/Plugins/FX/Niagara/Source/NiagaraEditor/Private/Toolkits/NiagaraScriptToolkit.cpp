// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptToolkit.h"

#include "AssetDefinitions/AssetDefinition_NiagaraScript.h"
#include "BusyCursor.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IMessageLogListing.h"
#include "MessageLogInitializationOptions.h"
#include "MessageLogModule.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleManager.h"
#include "NiagaraEditorCommands.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraGraph.h"
#include "ViewModels/NiagaraMessageLogViewModel.h"
#include "NiagaraNode.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraScript.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraScriptInputCollectionViewModel.h"
#include "NiagaraScriptSource.h"
#include "ViewModels/NiagaraStandaloneScriptViewModel.h"
#include "NiagaraVersionMetaData.h"
#include "PropertyEditorModule.h"
#include "SGraphActionMenu.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Widgets/SNiagaraScriptVersionWidget.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "ViewModels/NiagaraParameterDefinitionsPanelViewModel.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"
#include "Widgets/SDataHierarchyEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SNiagaraParameterDefinitionsPanel.h"
#include "Widgets/SNiagaraParameterPanel.h"
#include "Widgets/SNiagaraScriptGraph.h"
#include "Widgets/SNiagaraScriptInputPreviewPanel.h"
#include "Widgets/SNiagaraSelectedObjectsDetails.h"
#include "Widgets/Input/SButton.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptToolkit"

DECLARE_CYCLE_STAT(TEXT("Niagara - ScriptToolkit - OnApply"), STAT_NiagaraEditor_ScriptToolkit_OnApply, STATGROUP_NiagaraEditor);

const FName FNiagaraScriptToolkit::NodeGraphTabId(TEXT("NiagaraEditor_NodeGraph")); 
const FName FNiagaraScriptToolkit::ScriptDetailsTabId(TEXT("NiagaraEditor_ScriptDetails"));
const FName FNiagaraScriptToolkit::SelectedDetailsTabId(TEXT("NiagaraEditor_SelectedDetails"));
const FName FNiagaraScriptToolkit::ParametersTabId(TEXT("NiagaraEditor_Parameters"));
const FName FNiagaraScriptToolkit::InputPreviewTabId(TEXT("NiagaraEditor_InputPreview"));
const FName FNiagaraScriptToolkit::HierarchyEditor_ParametersTabId(TEXT("NiagaraEditor_HierarchyEditor_Parameters"));
const FName FNiagaraScriptToolkit::ParameterDefinitionsTabId(TEXT("NiagaraEditor_ParameterDefinitions"));
const FName FNiagaraScriptToolkit::StatsTabId(TEXT("NiagaraEditor_Stats"));
const FName FNiagaraScriptToolkit::MessageLogTabID(TEXT("NiagaraEditor_MessageLog"));
const FName FNiagaraScriptToolkit::VersioningTabID(TEXT("NiagaraEditor_Versioning"));

FNiagaraScriptToolkit::FNiagaraScriptToolkit()
{
}

FNiagaraScriptToolkit::~FNiagaraScriptToolkit()
{
	// Cleanup viewmodels that use the script viewmodel before cleaning up the script viewmodel itself.
	if (ParameterPanelViewModel.IsValid())
	{
		ParameterPanelViewModel->Cleanup();
	}
	if (ParameterDefinitionsPanelViewModel.IsValid())
	{
		ParameterDefinitionsPanelViewModel->Cleanup();
	}

	ParametersHierarchyViewModel->Finalize();
	ParametersHierarchyViewModel = nullptr;

	EditedNiagaraScript.Script->OnVMScriptCompiled().RemoveAll(this);
	ScriptViewModel->GetGraphViewModel()->GetGraph()->RemoveOnGraphNeedsRecompileHandler(OnEditedScriptGraphChangedHandle);

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	NiagaraEditorModule.GetOnScriptToolkitsShouldFocusGraphElement().RemoveAll(this);
	GEditor->UnregisterForUndo(this);
}

void FNiagaraScriptToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_NiagaraEditor", "Niagara"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	
	TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	InTabManager->RegisterTabSpawner(NodeGraphTabId, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabNodeGraph))
		.SetDisplayName( LOCTEXT("NodeGraph", "Node Graph") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Custom);

	InTabManager->RegisterTabSpawner(ScriptDetailsTabId, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabScriptDetails))
		.SetDisplayName(LOCTEXT("ScriptDetailsTab", "Script Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"))
		.SetReadOnlyBehavior(ETabReadOnlyBehavior::Custom);

	InTabManager->RegisterTabSpawner(SelectedDetailsTabId, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabSelectedDetails))
		.SetDisplayName(LOCTEXT("SelectedDetailsTab", "Selected Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(ParametersTabId, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabScriptParameters))
		.SetDisplayName(LOCTEXT("ParametersTab", "Parameters"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Parameters"));

	InTabManager->RegisterTabSpawner(InputPreviewTabId, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabInputsPreview))
		.SetDisplayName(LOCTEXT("InputPreviewParametersTab", "Input Preview"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Parameters"));

	InTabManager->RegisterTabSpawner(HierarchyEditor_ParametersTabId, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabHierarchyEditor))
		.SetDisplayName(LOCTEXT("HierarchyEditorTab", "Hierarchy Editor"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Parameters"));
	//@todo(ng) disable parameter definitions panel pending bug fixes
// 	InTabManager->RegisterTabSpawner(ParameterDefinitionsTabId, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabParameterDefinitions))
// 		.SetDisplayName(LOCTEXT("ParameterDefinitions", "Parameter Definitions"))
// 		.SetGroup(WorkspaceMenuCategoryRef);

	InTabManager->RegisterTabSpawner(StatsTabId, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabStats))
		.SetDisplayName(LOCTEXT("StatsTab", "Stats"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(MessageLogTabID, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabMessageLog))
		.SetDisplayName(LOCTEXT("NiagaraMessageLogTab", "Niagara Message Log"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Log"));

	InTabManager->RegisterTabSpawner(VersioningTabID, FOnSpawnTab::CreateSP(this, &FNiagaraScriptToolkit::SpawnTabVersioning))
        .SetDisplayName(LOCTEXT("VersioningTab", "Versioning"))
        .SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Versions"));
}

void FNiagaraScriptToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(NodeGraphTabId);
	InTabManager->UnregisterTabSpawner(ScriptDetailsTabId);
	InTabManager->UnregisterTabSpawner(SelectedDetailsTabId);
	InTabManager->UnregisterTabSpawner(ParametersTabId);
	InTabManager->UnregisterTabSpawner(InputPreviewTabId);
	//@todo(ng) disable parameter definitions panel pending bug fixes
	//InTabManager->UnregisterTabSpawner(ParameterDefinitionsTabId);
	InTabManager->UnregisterTabSpawner(StatsTabId);
	InTabManager->UnregisterTabSpawner(VersioningTabID);
}

FText FNiagaraScriptToolkit::GetGraphEditorDisplayName() const
{
	// Determine display name for panel heading based on asset usage type
	FText DisplayName = LOCTEXT("NiagaraScriptDisplayName", "Niagara Script");
	if (EditedNiagaraScript.Script->GetUsage() == ENiagaraScriptUsage::Function)
	{
		DisplayName = UAssetDefinition_NiagaraScript::FunctionScriptNameText;
	}
	else if (EditedNiagaraScript.Script->GetUsage() == ENiagaraScriptUsage::Module)
	{
		DisplayName = UAssetDefinition_NiagaraScript::ModuleScriptNameText;
	}
	else if (EditedNiagaraScript.Script->GetUsage() == ENiagaraScriptUsage::DynamicInput)
	{
		DisplayName = UAssetDefinition_NiagaraScript::DynamicInputScriptNameText;
	}

	FVersionedNiagaraScriptData* ScriptData = EditedNiagaraScript.Script->GetScriptData(EditedNiagaraScript.Version);
	if (EditedNiagaraScript.Script->IsVersioningEnabled() && ScriptData)
	{
		DisplayName = FText::Format(FText::FromString("{0} - Version {1}.{2}"), DisplayName, ScriptData->Version.MajorVersion, ScriptData->Version.MinorVersion);
	}
	return DisplayName;
}

void FNiagaraScriptToolkit::Initialize( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraScript* InputScript )
{
	check(InputScript != NULL);
	FGuid ScriptVersion = InputScript->IsVersioningEnabled() && InputScript->VersionToOpenInEditor.IsValid() ? InputScript->VersionToOpenInEditor : InputScript->GetExposedVersion().VersionGuid;
	OriginalNiagaraScript.Script = InputScript;
	OriginalNiagaraScript.Version = ScriptVersion;
	// No need to reset loader or versioning on the transient package, there should never be any set
	EditedNiagaraScript.Script = (UNiagaraScript*)StaticDuplicateObject(InputScript, GetTransientPackage(), NAME_None, ~RF_Standalone, UNiagaraScript::StaticClass());
	EditedNiagaraScript.Script->OnVMScriptCompiled().AddSP(this, &FNiagaraScriptToolkit::OnVMScriptCompiled);
	EditedNiagaraScript.Version = ScriptVersion;

	const FGuid MessageLogGuidKey = FGuid::NewGuid();
	NiagaraMessageLogViewModel = MakeShared<FNiagaraMessageLogViewModel>(GetNiagaraScriptMessageLogName(EditedNiagaraScript), MessageLogGuidKey, NiagaraMessageLog);
	
	bool bIsForDataProcessingOnly = false;
	ScriptViewModel = MakeShareable(new FNiagaraStandaloneScriptViewModel(GetGraphEditorDisplayName(), ENiagaraParameterEditMode::EditAll, NiagaraMessageLogViewModel, MessageLogGuidKey, bIsForDataProcessingOnly));
	ScriptViewModel->Initialize(EditedNiagaraScript, OriginalNiagaraScript);

	ParametersHierarchyViewModel = NewObject<UNiagaraScriptParametersHierarchyViewModel>();
	ParametersHierarchyViewModel->Initialize(ScriptViewModel.ToSharedRef());
	ParametersHierarchyViewModel->GetOnHierarchyChangedDelegate().AddSP(this, &FNiagaraScriptToolkit::OnHierarchyChanged);
	
	ParameterPanelViewModel = MakeShareable(new FNiagaraScriptToolkitParameterPanelViewModel(ScriptViewModel));
	ParameterDefinitionsPanelViewModel = MakeShareable(new FNiagaraScriptToolkitParameterDefinitionsPanelViewModel(ScriptViewModel));

	FScriptToolkitUIContext UIContext = FScriptToolkitUIContext(
		FSimpleDelegate::CreateSP(ParameterPanelViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh),
		FSimpleDelegate::CreateSP(ParameterDefinitionsPanelViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh),
		FSimpleDelegate::CreateRaw(this, &FNiagaraScriptToolkit::RefreshDetailsPanel)
	);
	ParameterPanelViewModel->Init(UIContext);
	ParameterDefinitionsPanelViewModel->Init(UIContext);

	OnEditedScriptGraphChangedHandle = ScriptViewModel->GetGraphViewModel()->GetGraph()->AddOnGraphNeedsRecompileHandler(
		FOnGraphChanged::FDelegate::CreateRaw(this, &FNiagaraScriptToolkit::OnEditedScriptGraphChanged));

	DetailsScriptSelection = MakeShareable(new FNiagaraObjectSelection());
	DetailsScriptSelection->SetSelectedObject(EditedNiagaraScript.Script, &EditedNiagaraScript.Version);

 	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog"); //@todo(message manager) remove stats listing
 	FMessageLogInitializationOptions LogOptions;
 	// Show Pages so that user is never allowed to clear log messages
	LogOptions.bShowPages = false;
	LogOptions.bShowFilters = false;
	LogOptions.bAllowClear = false;
	LogOptions.MaxPageCount = 1;
	StatsListing = MessageLogModule.CreateLogListing("MaterialEditorStats", LogOptions);
	Stats = MessageLogModule.CreateLogListingWidget(StatsListing.ToSharedRef());

	VersionMetadata = NewObject<UNiagaraVersionMetaData>(InputScript, "VersionMetadata", RF_Transient);
	SAssignNew(VersionsWidget, SNiagaraScriptVersionWidget, EditedNiagaraScript.Script, VersionMetadata, InputScript->GetOutermost()->GetName())
		.OnChangeToVersion(this, &FNiagaraScriptToolkit::SwitchToVersion)
		.OnVersionDataChanged_Lambda([this](const FPropertyChangedEvent* PropertyChangedEvent, FGuid SelectedVersion)
		{
	        if (UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(EditedNiagaraScript.Script->GetSource(EditedNiagaraScript.Version)))
	        {
	            ScriptSource->NodeGraph->NotifyGraphChanged();
	        }

			static FProperty* VersionProperty = FindFProperty<FProperty>(UNiagaraScript::StaticClass(), FName("VersionData"));
			FPropertyChangedEvent ChangeEvent = PropertyChangedEvent ? *PropertyChangedEvent : FPropertyChangedEvent(VersionProperty);
			EditedNiagaraScript.Script->PostEditChangeVersionedProperty(ChangeEvent, SelectedVersion);
		});

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_Niagara_Layout_v13")
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.15f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(ScriptDetailsTabId, ETabState::OpenedTab)
					->SetForegroundTab(ScriptDetailsTabId)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.4f)
					->AddTab(ParametersTabId, ETabState::OpenedTab)
					->AddTab(InputPreviewTabId, ETabState::OpenedTab)
					->SetForegroundTab(InputPreviewTabId)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.1f)
					->AddTab(StatsTabId, ETabState::OpenedTab)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.8f)
				->Split
				(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab(NodeGraphTabId, ETabState::OpenedTab)
				)
				->Split
				(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.2f)
				->AddTab(MessageLogTabID, ETabState::OpenedTab)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.15f)
				->AddTab(SelectedDetailsTabId, ETabState::OpenedTab)
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, FNiagaraEditorModule::NiagaraEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InputScript );

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>( "NiagaraEditor" );
	AddMenuExtender(NiagaraEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	NiagaraEditorModule.GetOnScriptToolkitsShouldFocusGraphElement().AddSP(this, &FNiagaraScriptToolkit::FocusGraphElementIfSameScriptID);

	SetupCommands();
	ExtendToolbar();
	RegenerateMenusAndToolbars();
	
	UpdateModuleStats();

	// @todo toolkit world centric editing
	/*// Setup our tool's layout
	if( IsWorldCentricAssetEditor() )
	{
		const FString TabInitializationPayload(TEXT(""));		// NOTE: Payload not currently used for table properties
		SpawnToolkitTab( NodeGraphTabId, TabInitializationPayload, EToolkitTabSpot::Details );
	}*/

	bChangesDiscarded = false;

	GEditor->RegisterForUndo(this);
}

void FNiagaraScriptToolkit::InitViewWithVersionedData()
{
	// remove old listeners
	ScriptViewModel->GetGraphViewModel()->GetGraph()->RemoveOnGraphNeedsRecompileHandler(OnEditedScriptGraphChangedHandle);

	// reinitialize the ui with the new version data
	const FSimpleDelegate RefreshParameterPanelDelegate = ParameterPanelViewModel.IsValid() ? FSimpleDelegate::CreateSP(ParameterPanelViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh) : FSimpleDelegate();
	const FSimpleDelegate RefreshParameterDefinitionsPanelDelegate = ParameterDefinitionsPanelViewModel.IsValid() ? FSimpleDelegate::CreateSP(ParameterDefinitionsPanelViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh) : FSimpleDelegate();
	const FSimpleDelegate RefreshDetailsPanelDelegate = FSimpleDelegate::CreateRaw(this, &FNiagaraScriptToolkit::RefreshDetailsPanel);
	FScriptToolkitUIContext UIContext = FScriptToolkitUIContext(
		RefreshParameterPanelDelegate,
		RefreshParameterDefinitionsPanelDelegate,
		RefreshDetailsPanelDelegate
	);

	// As the input preview panel is bound to the previously edited niagara script, make sure to remove delegates before the script view model points to the new version
	if(InputPreviewPanel)
	{
		InputPreviewPanel->RemoveDelegates();
	}
	
	ScriptViewModel->Initialize(EditedNiagaraScript, OriginalNiagaraScript);
	ScriptViewModel->GetGraphViewModel()->SetDisplayName(GetGraphEditorDisplayName());
	if (ParameterPanelViewModel)
	{
		ParameterPanelViewModel->Init(UIContext);
		ParameterPanelViewModel->RefreshNextTick();
	}
	if (ParameterDefinitionsPanelViewModel)
	{
		ParameterDefinitionsPanelViewModel->Init(UIContext);
		ParameterDefinitionsPanelViewModel->RefreshNextTick();
	}
	DetailsScriptSelection->SetSelectedObject(EditedNiagaraScript.Script, &EditedNiagaraScript.Version);
	if (NiagaraScriptGraphWidget)
	{
		NiagaraScriptGraphWidget->UpdateViewModel(ScriptViewModel->GetGraphViewModel());
		NiagaraScriptGraphWidget->RecreateGraphWidget();
	}
	if (DetailsView)
	{
		DetailsView->SetObjects(DetailsScriptSelection->GetSelectedObjectsResolved().Array(), true);
	}
	
	if(ParametersHierarchyViewModel)
	{
		ParametersHierarchyViewModel->Finalize();
		ParametersHierarchyViewModel->Initialize(ScriptViewModel.ToSharedRef());
	}

	if(InputPreviewPanel)
	{
		InputPreviewPanel->SetupDelegates();
		InputPreviewPanel->Refresh();
	}	

	// add listeners
	OnEditedScriptGraphChangedHandle = ScriptViewModel->GetGraphViewModel()->GetGraph()->AddOnGraphNeedsRecompileHandler(
        FOnGraphChanged::FDelegate::CreateRaw(this, &FNiagaraScriptToolkit::OnEditedScriptGraphChanged));

	RegenerateMenusAndToolbars();	
	UpdateModuleStats();
}

FName FNiagaraScriptToolkit::GetToolkitFName() const
{
	return FName("NiagaraScriptEditor");
}

FText FNiagaraScriptToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Niagara Script Editor");
}

FString FNiagaraScriptToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Niagara Script").ToString();
}


FLinearColor FNiagaraScriptToolkit::GetWorldCentricTabColorScale() const
{
	return FNiagaraEditorModule::WorldCentricTabColorScale;
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabNodeGraph( const FSpawnTabArgs& Args )
{
	checkf(Args.GetTabId().TabType == NodeGraphTabId, TEXT("Wrong tab ID in NiagaraScriptToolkit"));
	checkf(ScriptViewModel.IsValid(), TEXT("NiagaraScriptToolkit - Script editor view model is invalid"));

	return
		SNew(SDockTab)
		[
			SAssignNew(NiagaraScriptGraphWidget, SNiagaraScriptGraph, ScriptViewModel->GetGraphViewModel(), FAssetData(OriginalNiagaraScript.Script))
			.GraphTitle(LOCTEXT("SpawnGraphTitle", "Script"))
			.IsEditable(TAttribute<bool>::CreateSPLambda(this, [this]()
			{
				return GetOpenMethod() == EAssetOpenMethod::Edit;
			}))
			.DisplayAsReadOnly(TAttribute<bool>::CreateSPLambda(this, [this]()
			{
				return GetOpenMethod() == EAssetOpenMethod::View;
			}))
		];
}

void FNiagaraScriptToolkit::OnEditedScriptPropertyFinishedChanging(const FPropertyChangedEvent& InEvent)
{
	// We need to synchronize the Usage field in the property editor with the actual node
	// in the graph.
	if (InEvent.Property != nullptr && InEvent.Property->GetName().Equals(TEXT("Usage")))
	{
		if (EditedNiagaraScript.Script && EditedNiagaraScript.Script->GetSource(EditedNiagaraScript.Version))
		{
			UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(EditedNiagaraScript.Script->GetSource(EditedNiagaraScript.Version));
			if (ScriptSource != nullptr)
			{
				TArray<UNiagaraNodeOutput*> OutputNodes;
				ScriptSource->NodeGraph->FindOutputNodes(OutputNodes);

				bool bChanged = false;
				for (UNiagaraNodeOutput* Output : OutputNodes)
				{
					if (Output->GetUsage() != EditedNiagaraScript.Script->GetUsage())
					{
						Output->Modify();
						Output->SetUsage(EditedNiagaraScript.Script->GetUsage());
						bChanged = true;
					}
				}

				if (bChanged)
				{
					ScriptSource->NodeGraph->NotifyGraphChanged();
				}
			}
		}
	}

	PromptVersioningWarning();
}

void FNiagaraScriptToolkit::OnVMScriptCompiled(UNiagaraScript*, const FGuid&)
{
	UpdateModuleStats();
	if (ParameterPanelViewModel.IsValid())
	{
		ParameterPanelViewModel->Refresh();
	}
	if (SelectedDetailsWidget.IsValid())
	{
		SelectedDetailsWidget->SelectedObjectsChanged();
	}
}

void FNiagaraScriptToolkit::OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload)
{
	ScriptViewModel->GetStandaloneScript().GetScriptData()->GetSource()->MarkNotSynchronized("Input Hierarchy Changed");
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabScriptDetails(const FSpawnTabArgs& Args)
{
	checkf(Args.GetTabId().TabType == ScriptDetailsTabId, TEXT("Wrong tab ID in NiagaraScriptToolkit"));
	checkf(ScriptViewModel.IsValid(), TEXT("NiagaraScriptToolkit - Script editor view model is invalid"));

	TWeakPtr<FNiagaraScriptViewModel> ScriptViewModelWeakPtr = ScriptViewModel;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsView->OnFinishedChangingProperties().AddRaw(this, &FNiagaraScriptToolkit::OnEditedScriptPropertyFinishedChanging);
	DetailsView->SetObjects(DetailsScriptSelection->GetSelectedObjectsResolved().Array());

	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([this]
	{
		return GetOpenMethod() == EAssetOpenMethod::Edit;
	}));
	
	return SNew(SDockTab)
		.Label(LOCTEXT("ScriptDetailsTabLabel", "Script Details"))
		.TabColorScale(GetTabColorScale())
		[
			DetailsView.ToSharedRef()
		];
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabSelectedDetails(const FSpawnTabArgs& Args)
{
	checkf(Args.GetTabId().TabType == SelectedDetailsTabId, TEXT("Wrong tab ID in NiagaraScriptToolkit"));
	checkf(ScriptViewModel.IsValid(), TEXT("NiagaraScriptToolkit - Script editor view model is invalid"));

	return SNew(SDockTab)
		.Label(LOCTEXT("SelectedDetailsTabLabel", "Selected Details"))
		.TabColorScale(GetTabColorScale())
		[
			SAssignNew(SelectedDetailsWidget, SNiagaraSelectedObjectsDetails, ScriptViewModel->GetGraphViewModel()->GetNodeSelection(), ScriptViewModel->GetVariableSelection())
		];
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabScriptParameters(const FSpawnTabArgs& Args)
{
	checkf(Args.GetTabId().TabType == ParametersTabId, TEXT("Wrong tab ID in NiagaraScriptToolkit"));

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			SNew(SNiagaraParameterPanel, ParameterPanelViewModel, GetToolkitCommands())
			.SearchAdjacentWidget()
			[
				SNew(SButton)
				.OnClicked(this, &FNiagaraScriptToolkit::SummonHierarchyEditor)
				.ButtonStyle(FAppStyle::Get(), "RoundButton")
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EditHierarchy_ScriptInputs", "Edit Input Hierarchy"))
				]
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabInputsPreview(const FSpawnTabArgs& Args)
{
	checkf(Args.GetTabId().TabType == InputPreviewTabId, TEXT("Wrong tab ID in NiagaraScriptToolkit"));

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SButton)
				.OnClicked(this, &FNiagaraScriptToolkit::SummonParametersEditor)
			]
			+ SVerticalBox::Slot()
			[
				SAssignNew(InputPreviewPanel, SNiagaraScriptInputPreviewPanel, FNiagaraScriptToolkit::SharedThis(this), ScriptViewModel->GetVariableSelection())
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabParameterDefinitions(const FSpawnTabArgs& Args)
{
	checkf(Args.GetTabId().TabType == ParameterDefinitionsTabId, TEXT("Wrong tab ID in NiagaraScriptToolkit"));

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			SNew(SNiagaraParameterDefinitionsPanel, ParameterDefinitionsPanelViewModel, GetToolkitCommands())
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabStats(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == StatsTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("ModuleStatsTitle", "Stats"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ModuleStats")))
			[
				Stats.ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabHierarchyEditor(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == HierarchyEditor_ParametersTabId);
	
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("UserParametersHierarchyTitle", "Parameters Editor"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("UserParameters")))
			[
				SNew(SDataHierarchyEditor, ParametersHierarchyViewModel)
				.OnGenerateRowContentWidget_Static(&FNiagaraEditorUtilities::HierarchyEditor::Scripts::GenerateRowContentForScriptParameterHierarchyEditor)
				//.OnGenerateCustomDetailsPanelNameWidget_Static(&GenerateCustomDetailsPanelNameWidgetForUserParameterEditor)
			]
		];
	
	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabVersioning(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == VersioningTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
        .Label(LOCTEXT("ModuleVersioningTitle", "Versioning"))
        [
            SNew(SBox)
            .AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ModuleStats")))
            [
                VersionsWidget.ToSharedRef()
            ]
        ];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraScriptToolkit::SpawnTabMessageLog(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == MessageLogTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("NiagaraMessageLogTitle", "Niagara Log"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("NiagaraLog")))
			[
				NiagaraMessageLog.ToSharedRef()
			]
		];

	return SpawnedTab;
}

FReply FNiagaraScriptToolkit::SummonParametersEditor()
{
	TSharedPtr<SDockTab> DockTab = TabManager->TryInvokeTab(HierarchyEditor_ParametersTabId);

	if(DockTab)
	{
		DockTab->FlashTab();
	}
	
	return FReply::Handled();
}

TSharedRef<SWidget> FNiagaraScriptToolkit::GenerateVersioningDropdownMenu(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	TArray<FNiagaraAssetVersion> AssetVersions = EditedNiagaraScript.Script->GetAllAvailableVersions();
	for (FNiagaraAssetVersion& Version : AssetVersions)
	{
		FText Tooltip = LOCTEXT("NiagaraSelectVersion", "Select this version to edit in the module editor");
		FUIAction UIAction(FExecuteAction::CreateSP(this, &FNiagaraScriptToolkit::SwitchToVersion, Version.VersionGuid),
        FCanExecuteAction(),
        FIsActionChecked::CreateSP(this, &FNiagaraScriptToolkit::IsVersionSelected, Version));
		TAttribute<FText> Label = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraScriptToolkit::GetVersionMenuLabel, Version));
		MenuBuilder.AddMenuEntry(Label, Tooltip, FSlateIcon(), UIAction, NAME_None, EUserInterfaceActionType::RadioButton);	
	}

	return MenuBuilder.MakeWidget();
}

void FNiagaraScriptToolkit::SetupCommands()
{
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().Apply,
		FExecuteAction::CreateSP(this, &FNiagaraScriptToolkit::OnApply),
		FCanExecuteAction::CreateSP(this, &FNiagaraScriptToolkit::OnApplyEnabled));
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().Compile,
		FExecuteAction::CreateRaw(this, &FNiagaraScriptToolkit::CompileScript, false));
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().RefreshNodes,
		FExecuteAction::CreateRaw(this, &FNiagaraScriptToolkit::RefreshNodes));
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ModuleVersioning,
		FExecuteAction::CreateSP(this, &FNiagaraScriptToolkit::ManageVersions));
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().FindInCurrentView,
		FExecuteAction::CreateRaw(this, &FNiagaraScriptToolkit::FindInCurrentView));
}

const FName FNiagaraScriptToolkit::GetNiagaraScriptMessageLogName(FVersionedNiagaraScript InScript) const
{
	checkf(InScript.Script, TEXT("Tried to get MessageLog name for NiagaraScript but InScript was null!"));
	FName LogListingName = *FString::Printf(TEXT("%s_%s_MessageLog"), *InScript.Script->GetBaseChangeID(InScript.Version).ToString(), *InScript.Script->GetName());
	return LogListingName;
}

TSharedPtr<FNiagaraScriptToolkit> GetNiagaraScriptToolkitFromMenuContext(UAssetEditorToolkitMenuContext* InContext)
{
	if (InContext)
	{
		if (TSharedPtr<FAssetEditorToolkit> Toolkit = InContext->Toolkit.Pin())
		{
			// Note: This will not detect subclasses of NiagaraScriptToolkit
			if (Toolkit->GetToolkitFName() == TEXT("NiagaraScriptEditor"))
			{
				return StaticCastSharedPtr<FNiagaraScriptToolkit>(Toolkit);
			}
		}
	}

	return nullptr;
}

void FNiagaraScriptToolkit::ExtendToolbar()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("AssetEditor.NiagaraScriptEditor.ToolBar");

	{
		FToolMenuSection& ScriptAssetSection = Menu->AddSection("ScriptAsset");

		ScriptAssetSection.InsertPosition = FToolMenuInsert("Asset", EToolMenuInsertType::After);
		ScriptAssetSection.AddDynamicEntry("ScriptAssetDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if(TSharedPtr<FNiagaraScriptToolkit> NiagaraScriptToolkit = GetNiagaraScriptToolkitFromMenuContext(InSection.FindContext<UAssetEditorToolkitMenuContext>()))
			{				
				InSection.AddEntry(FToolMenuEntry::InitToolBarButton(FNiagaraEditorCommands::Get().Apply,
					TAttribute<FText>(), TAttribute<FText>(),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.Apply")));

				InSection.AddDynamicEntry("CompileDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& DynamicSection)
				{
					if(TSharedPtr<FNiagaraScriptToolkit> NiagaraScriptToolkit = GetNiagaraScriptToolkitFromMenuContext(DynamicSection.FindContext<UAssetEditorToolkitMenuContext>()))
					{
						DynamicSection.AddEntry(FToolMenuEntry::InitToolBarButton(FNiagaraEditorCommands::Get().Compile,
						TAttribute<FText>(),
						TAttribute<FText>(NiagaraScriptToolkit.ToSharedRef(), &FNiagaraScriptToolkit::GetCompileStatusTooltip),
						TAttribute<FSlateIcon>(NiagaraScriptToolkit.ToSharedRef(), &FNiagaraScriptToolkit::GetCompileStatusImage)));

						DynamicSection.AddEntry(FToolMenuEntry::InitComboButton("CompileOptions",
							FUIAction(),
							FNewToolMenuChoice(FOnGetContent::CreateRaw(NiagaraScriptToolkit.Get(), &FNiagaraScriptToolkit::GenerateCompileMenuContent)),
							LOCTEXT("CompileCombo_Label", "Compile options"),
							LOCTEXT("CompileComboToolTip", "Compile options menu"),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Build"),
							true
							));
					}			
				}));

				InSection.AddEntry(FToolMenuEntry::InitToolBarButton(FNiagaraEditorCommands::Get().RefreshNodes,
					TAttribute<FText>(),
					TAttribute<FText>(NiagaraScriptToolkit.ToSharedRef(), &FNiagaraScriptToolkit::GetRefreshStatusTooltip),
					TAttribute<FSlateIcon>(NiagaraScriptToolkit.ToSharedRef(), &FNiagaraScriptToolkit::GetRefreshStatusImage)));
			}
		}));

		FToolMenuSection& VersioningSection = Menu->AddSection("Versioning");

		VersioningSection.AddDynamicEntry("VersioningDynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			if(TSharedPtr<FNiagaraScriptToolkit> NiagaraScriptToolkit = GetNiagaraScriptToolkitFromMenuContext(InSection.FindContext<UAssetEditorToolkitMenuContext>()))
			{
				InSection.AddEntry(FToolMenuEntry::InitToolBarButton(FNiagaraEditorCommands::Get().ModuleVersioning,
				TAttribute<FText>(NiagaraScriptToolkit.ToSharedRef(), &FNiagaraScriptToolkit::GetVersionButtonLabel),
				LOCTEXT("NiagaraShowModuleVersionsTooltip", "Manage different versions of this module script."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Versions")));

				FUIAction DropdownAction;
				DropdownAction.IsActionVisibleDelegate = FIsActionButtonVisible::CreateLambda([WeakNiagaraScriptToolkit = NiagaraScriptToolkit.ToWeakPtr()]() { return WeakNiagaraScriptToolkit.IsValid() ? WeakNiagaraScriptToolkit.Pin()->EditedNiagaraScript.Script->GetAllAvailableVersions().Num() > 1 : false; });

				InSection.AddEntry(FToolMenuEntry::InitComboButton("VersionOptions",
					DropdownAction,
					FOnGetContent::CreateSP(NiagaraScriptToolkit.ToSharedRef(), &FNiagaraScriptToolkit::GenerateVersioningDropdownMenu, NiagaraScriptToolkit->GetToolkitCommands()),
					TAttribute<FText>(), LOCTEXT("NiagaraShowVersions_ToolTip", "Select version to edit"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Versions"),
					true					
					));
			}			
		}));
	}
}

FSlateIcon FNiagaraScriptToolkit::GetCompileStatusImage() const
{
	ENiagaraScriptCompileStatus Status = ScriptViewModel->GetLatestCompileStatus(EditedNiagaraScript.Version);
	static const FName CompileStatusBackground("AssetEditor.CompileStatus.Background");
	static const FName CompileStatusUnknown("AssetEditor.CompileStatus.Overlay.Unknown");
	static const FName CompileStatusError("AssetEditor.CompileStatus.Overlay.Error");
	static const FName CompileStatusGood("AssetEditor.CompileStatus.Overlay.Good");
	static const FName CompileStatusWarning("AssetEditor.CompileStatus.Overlay.Warning");
	switch (Status)
	{
	default:
	case ENiagaraScriptCompileStatus::NCS_Unknown:
	case ENiagaraScriptCompileStatus::NCS_Dirty:
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
	case ENiagaraScriptCompileStatus::NCS_Error:
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusError);
	case ENiagaraScriptCompileStatus::NCS_UpToDate:
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusGood);
	case ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings:
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusWarning);
	}
}
void FNiagaraScriptToolkit::Tick(float DeltaTime)
{
	if (bRefreshSelected)
	{
		if (SelectedDetailsWidget.IsValid())
			SelectedDetailsWidget->SelectedObjectsChanged();
		bRefreshSelected = false;
	}
}

bool FNiagaraScriptToolkit::IsTickable() const
{
	return true;
}

TStatId FNiagaraScriptToolkit::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraScriptToolkit, STATGROUP_Tickables);
}

FText FNiagaraScriptToolkit::GetCompileStatusTooltip() const
{
	ENiagaraScriptCompileStatus Status = ScriptViewModel->GetLatestCompileStatus(EditedNiagaraScript.Version);
	return FNiagaraEditorUtilities::StatusToText(Status);
}

TSharedRef<SWidget> FNiagaraScriptToolkit::GenerateCompileMenuContent()
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	static const FName CompileStatusBackground("AssetEditor.CompileStatus.Background");
	static const FName CompileStatusUnknown("AssetEditor.CompileStatus.Overlay.Unknown");
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	FUIAction ForceCompileAction(
		FExecuteAction::CreateRaw(this, &FNiagaraScriptToolkit::CompileScript, true));

	MenuBuilder.AddMenuEntry(LOCTEXT("ForceCompile", "Force Compile"),
		LOCTEXT("ForceCompileTooltip", "Triggers a recompilation of this script, ignoring the change tracking and cached results."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown),
		ForceCompileAction, NAME_None, EUserInterfaceActionType::Button);

	return MenuBuilder.MakeWidget();
}

FSlateIcon FNiagaraScriptToolkit::GetRefreshStatusImage() const
{
	return FSlateIcon(FAppStyle::Get().GetStyleSetName(), "FontEditor.Update");
}

FText FNiagaraScriptToolkit::GetRefreshStatusTooltip() const
{
	return LOCTEXT("Refresh_Status", "Currently dependencies up-to-date. Consider refreshing if status isn't accurate.");
}

FText FNiagaraScriptToolkit::GetVersionButtonLabel() const
{
	FText BaseLabel = LOCTEXT("NiagaraShowModuleVersions", "Versioning");
	if (EditedNiagaraScript.Script && EditedNiagaraScript.Script->IsVersioningEnabled())
	{
		if (FVersionedNiagaraScriptData* ScriptData = EditedNiagaraScript.Script->GetScriptData(EditedNiagaraScript.Version))
		{
			FNiagaraAssetVersion ExposedVersion = EditedNiagaraScript.Script->GetExposedVersion();
			return FText::Format(FText::FromString("{0} ({1}.{2}{3})"), BaseLabel, ScriptData->Version.MajorVersion, ScriptData->Version.MinorVersion, ScriptData->Version <= ExposedVersion ? FText::FromString("*") : FText());
		}
	}
	return BaseLabel;
}

void FNiagaraScriptToolkit::CompileScript(bool bForce)
{
	ScriptViewModel->CompileStandaloneScript(bForce);
}

void FNiagaraScriptToolkit::RefreshNodes()
{
	ScriptViewModel->RefreshNodes();
}

void FNiagaraScriptToolkit::ManageVersions()
{
	TabManager->TryInvokeTab(VersioningTabID);
}

void FNiagaraScriptToolkit::FindInCurrentView() const
{
	if (NiagaraScriptGraphWidget.IsValid())
	{
		NiagaraScriptGraphWidget->FocusGraphSearchBox();
	}
}

void FNiagaraScriptToolkit::SwitchToVersion(FGuid VersionGuid)
{
	EditedNiagaraScript.Version = VersionGuid;

	InitViewWithVersionedData();
}

bool FNiagaraScriptToolkit::IsVersionSelected(FNiagaraAssetVersion Version) const
{
	return EditedNiagaraScript.Version == Version.VersionGuid;
}

FText FNiagaraScriptToolkit::GetVersionMenuLabel(FNiagaraAssetVersion Version) const
{
	bool bIsExposed = Version == EditedNiagaraScript.Script->GetExposedVersion();
	return FText::Format(FText::FromString("v{0}.{1} {2}"), Version.MajorVersion, Version.MinorVersion, bIsExposed ? LOCTEXT("NiagaraExposedVersionHint", "(exposed)") : FText());
}

bool FNiagaraScriptToolkit::IsEditScriptDifferentFromOriginalScript() const
{
	return OriginalNiagaraScript.Script->GetBaseChangeID(EditedNiagaraScript.Version) != EditedNiagaraScript.Script->GetBaseChangeID(EditedNiagaraScript.Version);
}

void FNiagaraScriptToolkit::OnApply()
{
	if(GetOpenMethod() == EAssetOpenMethod::Edit)
	{
		SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_ScriptToolkit_OnApply);
		UE_LOGF(LogNiagaraEditor, Log, "Applying Niagara Script %ls", *GetEditingObjects()[0]->GetName());
		UpdateOriginalNiagaraScript();
		FNiagaraEditorModule::Get().ScriptApplied(OriginalNiagaraScript.Script, OriginalNiagaraScript.Version);
	}
}

bool FNiagaraScriptToolkit::OnApplyEnabled() const
{
	return IsEditScriptDifferentFromOriginalScript() && GetOpenMethod() == EAssetOpenMethod::Edit;
}

void FNiagaraScriptToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(OriginalNiagaraScript.Script);
	Collector.AddReferencedObject(EditedNiagaraScript.Script);
	Collector.AddReferencedObject(VersionMetadata);
	Collector.AddReferencedObject(ParametersHierarchyViewModel);
}

UNiagaraScriptParametersHierarchyViewModel* FNiagaraScriptToolkit::GetHierarchyViewModel() const
{
	return ParametersHierarchyViewModel;
}

void FNiagaraScriptToolkit::UpdateModuleStats()
{
	TArray< TSharedRef<class FTokenizedMessage> > Messages;

	TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Info);
	uint32 LastOpCount = EditedNiagaraScript.Script->GetVMExecutableData().LastOpCount;
	Line->AddToken(FTextToken::Create(FText::Format(FText::FromString("LastOpCount: {0}"), LastOpCount)));
	Messages.Add(Line);
	
	StatsListing->ClearMessages();
	StatsListing->AddMessages(Messages);
}

bool FNiagaraScriptToolkit::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const
{
	const auto* Graph = ScriptViewModel->GetGraphViewModel()->GetGraph();
	UNiagaraScriptSourceBase* Source = EditedNiagaraScript.GetScriptData() ? EditedNiagaraScript.Script->GetSource(EditedNiagaraScript.Version) : nullptr;
	if (Graph || Source)
	{
		for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjects)
		{
			UObject* Object = TransactionObjectPair.Key;
			while (Object != nullptr)
			{
				if (Object == Graph || Object == Source)
				{
					return true;
				}
				Object = Object->GetOuter();
			}
		}
	}
	return false;
}

void FNiagaraScriptToolkit::PostUndo(bool bSuccess)
{
	if (EditedNiagaraScript.Version.IsValid() && EditedNiagaraScript.GetScriptData() == nullptr)
	{
		if (EditedNiagaraScript.Script && EditedNiagaraScript.Script->IsVersioningEnabled())
		{
			SwitchToVersion(EditedNiagaraScript.Script->GetExposedVersion().VersionGuid);
		}
		else
		{
			EditedNiagaraScript.Version = FGuid();
		}
	}
	PromptVersioningWarning();
}

void FNiagaraScriptToolkit::GetSaveableObjects(TArray<UObject*>& OutObjects) const
{
	OutObjects.Add(OriginalNiagaraScript.Script);
}


void FNiagaraScriptToolkit::SaveAsset_Execute()
{
	UE_LOGF(LogNiagaraEditor, Log, "Saving and Compiling NiagaraScript %ls", *GetEditingObjects()[0]->GetName());

	if(IsEditScriptDifferentFromOriginalScript())
	{
		UpdateOriginalNiagaraScript();
	}

	FAssetEditorToolkit::SaveAsset_Execute();
}

void FNiagaraScriptToolkit::SaveAssetAs_Execute()
{
	UE_LOGF(LogNiagaraEditor, Log, "Saving and Compiling NiagaraScript %ls", *GetEditingObjects()[0]->GetName());

	if(IsEditScriptDifferentFromOriginalScript())
	{
		UpdateOriginalNiagaraScript();
	}

	FAssetEditorToolkit::SaveAssetAs_Execute();
}

void FNiagaraScriptToolkit::UpdateOriginalNiagaraScript()
{
	const FScopedBusyCursor BusyCursor;

	const FText LocalizedScriptEditorApply = NSLOCTEXT("UnrealEd", "ToolTip_NiagaraScriptEditorApply", "Apply changes to original script and its use in the world.");
	GWarn->BeginSlowTask(LocalizedScriptEditorApply, true);
	GWarn->StatusUpdate(1, 1, LocalizedScriptEditorApply);

	if (OriginalNiagaraScript.Script->IsSelected())
	{
		GEditor->GetSelectedObjects()->Deselect(OriginalNiagaraScript.Script);
	}

	ResetLoaders(OriginalNiagaraScript.Script->GetOutermost()); // Make sure that we're not going to get invalid version number linkers into the package we are going into. 

	// Compile and then overwrite the original script in place by constructing a new one with the same name
	ScriptViewModel->CompileStandaloneScript();
	OriginalNiagaraScript.Script = (UNiagaraScript*)StaticDuplicateObject(EditedNiagaraScript.Script, OriginalNiagaraScript.Script->GetOuter(), OriginalNiagaraScript.Script->GetFName(),
		RF_AllFlags,
		OriginalNiagaraScript.Script->GetClass());

	// Restore RF_Standalone on the original material, as it had been removed from the preview material so that it could be GC'd.
	OriginalNiagaraScript.Script->SetFlags(RF_Standalone);

	// Now there might be other Scripts with functions that referenced this script. So let's update them. They'll need a recompile.
	// Note that we don't discriminate between the version that are open in transient packages (likely duplicates for editing) and the
	// original in-scene versions.
	FRefreshAllScriptsFromExternalChangesArgs Args;
	Args.OriginatingScript = OriginalNiagaraScript.Script;
	Args.OriginatingGraph = CastChecked<UNiagaraScriptSource>(OriginalNiagaraScript.Script->GetSource(EditedNiagaraScript.Version))->NodeGraph;
	FNiagaraEditorUtilities::RefreshAllScriptsFromExternalChanges(Args);

	GWarn->EndSlowTask();
	FNiagaraEditorModule::Get().InvalidateCachedScriptAssetData();
}


bool FNiagaraScriptToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	if(GetOpenMethod() == EAssetOpenMethod::Edit)
	{
		if (bChangesDiscarded == false && IsEditScriptDifferentFromOriginalScript())
		{
			// find out the user wants to do with this dirty NiagaraScript
			EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(EAppMsgType::YesNoCancel,
				FText::Format(
					NSLOCTEXT("UnrealEd", "Prompt_NiagaraScriptEditorClose", "Would you like to apply changes to this NiagaraScript to the original NiagaraScript?\n{0}\n(No will lose all changes!)"),
					FText::FromString(OriginalNiagaraScript.Script->GetPathName())));

			// act on it
			switch (YesNoCancelReply)
			{
			case EAppReturnType::Yes:
				// update NiagaraScript and exit
				UpdateOriginalNiagaraScript();
				break;

			case EAppReturnType::No:
				// Set the changes discarded to avoid showing the dialog multiple times when request close is called multiple times on shut down.
				bChangesDiscarded = true;
				break;

			case EAppReturnType::Cancel:
				// don't exit
				return false;
			}
		}
	}

	return true;
}

void FNiagaraScriptToolkit::OnEditedScriptGraphChanged(const FEdGraphEditAction& InAction)
{
	PromptVersioningWarning();
	bRefreshSelected = true;
}

void FNiagaraScriptToolkit::PromptVersioningWarning()
{
	if (!bShowedEditingVersionWarning && EditedNiagaraScript.Script->IsVersioningEnabled())
	{
		FNiagaraAssetVersion ExposedVersion = EditedNiagaraScript.Script->GetExposedVersion();
		FNiagaraAssetVersion const* VersionData = EditedNiagaraScript.Script->FindVersionData(EditedNiagaraScript.Version);
		if (VersionData && *VersionData <= ExposedVersion)
		{
			bShowedEditingVersionWarning = true;
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("EditingExposedAssetWarning", "Warning: you are editing an already exposed asset version. Saving these changes will force-push them out to existing usages!\nConsider creating a new version instead to make those changes."));	
		}
	}
}

void FNiagaraScriptToolkit::FocusGraphElementIfSameScriptID(const FNiagaraScriptIDAndGraphFocusInfo* FocusInfo)
{
	if (FocusInfo->GetScriptUniqueAssetID() == OriginalNiagaraScript.Script->GetUniqueID())
	{
		NiagaraScriptGraphWidget->FocusGraphElement(FocusInfo->GetScriptGraphFocusInfo().Get());
	}
}

FReply FNiagaraScriptToolkit::SummonHierarchyEditor() const
{
	TabManager->TryInvokeTab(HierarchyEditor_ParametersTabId);
	return FReply::Handled();
}

void FNiagaraScriptToolkit::RefreshDetailsPanel()
{
	if (SelectedDetailsWidget.IsValid())
	{
		SelectedDetailsWidget->RefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE
