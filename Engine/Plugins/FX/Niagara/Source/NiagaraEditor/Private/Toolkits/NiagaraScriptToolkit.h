// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/GCObject.h"

#include "TickableEditorObject.h"
#include "EditorUndoClient.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraScript.h"
#include "ViewModels/HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"

class UNiagaraScriptParametersHierarchyViewModel;
class UNiagaraVersionMetaData;
struct FCustomExpanderData;
class IDetailsView;
class SGraphEditor;
class UEdGraph;
class UNiagaraScript;
class UNiagaraScriptSource;
class FNiagaraScriptViewModel;
class FNiagaraObjectSelection;
struct FEdGraphEditAction;
class FNiagaraMessageLogViewModel;
class FNiagaraStandaloneScriptViewModel;
class FNiagaraScriptToolkitParameterPanelViewModel;
class FNiagaraScriptToolkitParameterDefinitionsPanelViewModel;
class SNiagaraSelectedObjectsDetails;
class SNiagaraScriptInputPreviewPanel;

/** Viewer/editor for a DataTable */
class FNiagaraScriptToolkit : public FAssetEditorToolkit, public FGCObject, public FTickableEditorObject, public FEditorUndoClient
{
public:
	FNiagaraScriptToolkit();

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	
	/** Edits the specified Niagara Script */
	void Initialize( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraScript* Script );

	/** Destructor */
	virtual ~FNiagaraScriptToolkit() override;
	
	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Interface

	/** The original NiagaraScript being edited by this editor. */
	FVersionedNiagaraScript OriginalNiagaraScript;

	/** The transient, duplicated script that is being edited by this editor.*/
	FVersionedNiagaraScript EditedNiagaraScript;

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraScriptToolkit");
	}
	
	UNiagaraScriptParametersHierarchyViewModel* GetHierarchyViewModel() const;
	
	/**
	* Updates list of module info used to show stats
	*/
	void UpdateModuleStats();

	//~ Begin FEditorUndoClient Interface
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient


	//~ FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;
protected:
	//~ FAssetEditorToolkit interface
	virtual void GetSaveableObjects(TArray<UObject*>& OutObjects) const override;
	virtual void SaveAsset_Execute() override;
	virtual void SaveAssetAs_Execute() override;
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;

private:
	void OnEditedScriptPropertyFinishedChanging(const FPropertyChangedEvent& InEvent);

	void OnVMScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion);

	void OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload);
	
	/** Spawns the tab with the update graph inside */
	TSharedRef<SDockTab> SpawnTabNodeGraph(const FSpawnTabArgs& Args);

	/** Spawns the tab with the script details inside. */
	TSharedRef<SDockTab> SpawnTabScriptDetails(const FSpawnTabArgs& Args);

	/** Spawns the tab with the details of the current selection. */
	TSharedRef<SDockTab> SpawnTabSelectedDetails(const FSpawnTabArgs& Args);

	/** Spawns the tab with the parameter view. */
	TSharedRef<SDockTab> SpawnTabScriptParameters(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabInputsPreview(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabParameterDefinitions(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabStats(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTabHierarchyEditor(const FSpawnTabArgs& Args);

	/** Spawns the tab with the version management. */
	TSharedRef<SDockTab> SpawnTabVersioning(const FSpawnTabArgs& Args);

	TSharedPtr< SNiagaraSelectedObjectsDetails> SelectedDetailsWidget;

	TSharedRef<SDockTab> SpawnTabMessageLog(const FSpawnTabArgs& Args);

	FReply SummonParametersEditor();
	TSharedRef<SWidget> GenerateVersioningDropdownMenu(TSharedRef<FUICommandList> InCommandList);

	/** Sets up commands for the toolkit toolbar. */
	void SetupCommands();

	const FName GetNiagaraScriptMessageLogName(FVersionedNiagaraScript InScript) const;
	FSlateIcon GetCompileStatusImage() const;
	FText GetCompileStatusTooltip() const;
	
	TSharedRef<SWidget> GenerateCompileMenuContent();

	/** Builds the toolbar widget */
	void ExtendToolbar();

	/** Compiles the script. */
	void CompileScript(bool bForce);

	/** Refreshes the nodes in the script graph, updating the pins to match external changes. */
	void RefreshNodes();

	/** Opens the module versioning tab. */
	void ManageVersions();

	/** Focuses the search bar in the graph. */
	void FindInCurrentView() const;

	void InitViewWithVersionedData();
	void SwitchToVersion(FGuid VersionGuid);
	bool IsVersionSelected(FNiagaraAssetVersion Version) const;
	FText GetVersionMenuLabel(FNiagaraAssetVersion Version) const;

	FSlateIcon GetRefreshStatusImage() const;
	FText GetRefreshStatusTooltip() const;
	FText GetVersionButtonLabel() const;

	bool IsEditScriptDifferentFromOriginalScript() const;

	/** Command for the apply button */
	void OnApply();
	bool OnApplyEnabled() const;

	void UpdateOriginalNiagaraScript();

	void OnEditedScriptGraphChanged(const FEdGraphEditAction& InAction);

	void PromptVersioningWarning();

	/** Navigates to element in graph (node, pin, etc.) 
	* @Param ElementToFocus Defines the graph element to navigate to and select.
	*/
	void FocusGraphElementIfSameScriptID(const FNiagaraScriptIDAndGraphFocusInfo* ElementToFocus);

	FReply SummonHierarchyEditor() const;
private:
	/** The Script being edited */
	TSharedPtr<FNiagaraStandaloneScriptViewModel> ScriptViewModel;

	TObjectPtr<UNiagaraScriptParametersHierarchyViewModel> ParametersHierarchyViewModel;

	/** The Parameter Panel displaying graph variables */
	TSharedPtr<FNiagaraScriptToolkitParameterPanelViewModel> ParameterPanelViewModel;

	/** The Parameter Definitions Panel displaying included libraries */
	TSharedPtr<FNiagaraScriptToolkitParameterDefinitionsPanelViewModel> ParameterDefinitionsPanelViewModel;

	/** The selection displayed by the details tab. */
	TSharedPtr<FNiagaraObjectSelection> DetailsScriptSelection;

	/** Message log, with the log listing that it reflects */
	TSharedPtr<FNiagaraMessageLogViewModel> NiagaraMessageLogViewModel;
	TSharedPtr<class SWidget> NiagaraMessageLog;

	/** Stats log, with the log listing that it reflects */
	TSharedPtr<class SWidget> Stats;
	TSharedPtr<class IMessageLogListing> StatsListing;

	/** Version management widget */
	TSharedPtr<class SWidget> VersionsWidget;

	FDelegateHandle OnEditedScriptGraphChangedHandle;
	
	bool bChangesDiscarded = false;
	bool bRefreshSelected = false;
	bool bShowedEditingVersionWarning = false;

	TSharedPtr<class SNiagaraScriptGraph> NiagaraScriptGraphWidget;
	TSharedPtr<SNiagaraScriptInputPreviewPanel> InputPreviewPanel;
	TSharedPtr<class IDetailsView> DetailsView;
	TObjectPtr<UNiagaraVersionMetaData> VersionMetadata = nullptr;
	FText GetGraphEditorDisplayName() const;
	
	void RefreshDetailsPanel();

public:
	/**	The tab ids for the Niagara editor */
	static const FName NodeGraphTabId; 
	static const FName ScriptDetailsTabId;
	static const FName SelectedDetailsTabId;
	static const FName ParametersTabId;
	static const FName InputPreviewTabId;
	static const FName HierarchyEditor_ParametersTabId;
	static const FName ParameterDefinitionsTabId;
	static const FName StatsTabId;
	static const FName MessageLogTabID;
	static const FName VersioningTabID;
};
