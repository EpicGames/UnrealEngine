// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSimCacheToolkit.h"
#include "NiagaraEditorModule.h"

// Widgets & ViewModels
#include "AdvancedPreviewSceneModule.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "NiagaraSystem.h"
#include "Widgets/Docking/SDockTab.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/SNiagaraSimCacheTreeView.h"
#include "Widgets/SNiagaraSimCacheOverview.h"
#include "Widgets/SNiagaraSimCacheViewport.h"
#include "Widgets/SNiagaraSimCacheViewTimeline.h"
#include "Widgets/SNiagaraSimCacheViewTransportControls.h"

#define LOCTEXT_NAMESPACE "NiagaraSimCacheToolkit"

const FName FNiagaraSimCacheToolkit::NiagaraSimCacheSpreadsheetTabId(TEXT("NiagaraSimCacheEditor_Spreadsheet"));
const FName FNiagaraSimCacheToolkit::NiagaraSimCacheViewportTabId(TEXT("NiagaraSimCacheEditor_Viewport"));
const FName FNiagaraSimCacheToolkit::NiagaraSimCachePreviewSettingsTabId(TEXT("NiagaraSimCacheEditor_PreviewSettings"));
const FName FNiagaraSimCacheToolkit::NiagaraSimCacheOverviewTabId(TEXT("NiagaraSimCacheEditor_Overview"));

FNiagaraSimCacheToolkit::FNiagaraSimCacheToolkit()
{
		
}

FNiagaraSimCacheToolkit::~FNiagaraSimCacheToolkit()
{

}

void FNiagaraSimCacheToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_NiagaraSimCacheEditor", "Niagara"));
	
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(NiagaraSimCacheSpreadsheetTabId, FOnSpawnTab::CreateSP(this, &FNiagaraSimCacheToolkit::SpawnTab_SimCacheSpreadsheet))
		.SetDisplayName(LOCTEXT("SpreadsheetTab", "Cache Spreadsheet"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(NiagaraSimCacheViewportTabId, FOnSpawnTab::CreateSP(this, &FNiagaraSimCacheToolkit::SpawnTab_SimCacheViewport))
		.SetDisplayName(LOCTEXT("ViewpoirtTab", "Cache Viewport"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(NiagaraSimCachePreviewSettingsTabId, FOnSpawnTab::CreateSP(this, &FNiagaraSimCacheToolkit::SpawnTab_PreviewSettings))
		.SetDisplayName(LOCTEXT("PreviewSceneSettingsTab","Preview Scene Settings"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(NiagaraSimCacheOverviewTabId, FOnSpawnTab::CreateSP(this, &FNiagaraSimCacheToolkit::SpawnTab_Overview))
		.SetDisplayName(LOCTEXT("OverviewTab", "Overview"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

void FNiagaraSimCacheToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(NiagaraSimCacheSpreadsheetTabId);
}

void FNiagaraSimCacheToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraSimCache* InSimCache)
{
	SimCache = InSimCache;
	if (SimCache.IsValid())
	{
		SimCacheViewModel = MakeShared<FNiagaraSimCacheViewModel>();
		
		SimCacheViewModel->Initialize(SimCache);
		

		const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_Niagara_SimCache_Layout_V1")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
				->Split(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.3f)
						->AddTab(NiagaraSimCacheViewportTabId, ETabState::OpenedTab)
					)
					->Split(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.7f)
						->AddTab(NiagaraSimCacheOverviewTabId, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->AddTab(NiagaraSimCacheSpreadsheetTabId, ETabState::OpenedTab)
				)
			);

		constexpr bool bCreateDefaultStandaloneMenu = true;
		constexpr bool bCreateDefaultToolbar = true;
		InitAssetEditor(Mode, InitToolkitHost, FNiagaraEditorModule::NiagaraEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, SimCache.Get());
		ExtendToolbar();
		RegenerateMenusAndToolbars();
	}
}

FName FNiagaraSimCacheToolkit::GetToolkitFName() const
{
	return FName("Niagara");
}

FText FNiagaraSimCacheToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Niagara");
}

FString FNiagaraSimCacheToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Niagara ").ToString();
}

FLinearColor FNiagaraSimCacheToolkit::GetWorldCentricTabColorScale() const
{
	return FNiagaraEditorModule::WorldCentricTabColorScale;
}

TSharedRef<SDockTab> FNiagaraSimCacheToolkit::SpawnTab_SimCacheSpreadsheet(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == NiagaraSimCacheSpreadsheetTabId);

	SimCacheSpreadsheetView =
		SNew(SNiagaraSimCacheView)
		.SimCacheViewModel(SimCacheViewModel);

	const TSharedRef<SVerticalBox> Contents = SNew(SVerticalBox);

	Contents->AddSlot()
		.FillHeight(1.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SimCacheSpreadsheetView->AsShared()
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f)
				[
					SNew(SNiagaraSimCacheViewTransportControls)
					.WeakViewModel(SimCacheViewModel->AsWeak())
				]
				+SHorizontalBox::Slot()
				.Padding(4.0f)
				[
					SNew(SNiagaraSimCacheViewTimeline)
					.WeakViewModel(SimCacheViewModel->AsWeak())
				]
				
			]
			
		];

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			Contents
		];
	
	return SpawnedTab;
}

TSharedRef<SDockTab>  FNiagaraSimCacheToolkit::SpawnTab_SimCacheViewport(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == NiagaraSimCacheViewportTabId);

	if (!Viewport.IsValid())
	{
		Viewport = SNew(SNiagaraSimCacheViewport);
	}
	

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			Viewport.ToSharedRef()
		];

	Viewport->SetPreviewComponent(SimCacheViewModel->GetPreviewComponent());
	
	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSimCacheToolkit::SpawnTab_PreviewSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == NiagaraSimCachePreviewSettingsTabId);

	TSharedRef<SWidget> InWidget = SNullWidget::NullWidget;
	if (Viewport.IsValid())
	{
		FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
		InWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(Viewport->GetPreviewScene());
	}

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		[
			InWidget
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSimCacheToolkit::SpawnTab_Overview(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("OverviewTab", "Overview"))
		[
			SNew(SNiagaraSimCacheOverview)
			.SimCacheViewModel(SimCacheViewModel)
		];

	return SpawnedTab;
}

void FNiagaraSimCacheToolkit::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FNiagaraSimCacheToolkit* Toolkit)
		{
			TSharedRef<FNiagaraSimCacheViewModel> SimCacheViewModelRef = Toolkit->SimCacheViewModel.ToSharedRef();

			if (Toolkit->SimCache.IsValid() && Toolkit->SimCache->GetSystem())
			{
				ToolbarBuilder.BeginSection("System");
				{
					TWeakObjectPtr<UNiagaraSystem> NiagaraSystem = Toolkit->SimCache->GetSystem();
					FUIAction GotoAction = FUIAction(
						FExecuteAction::CreateLambda([NiagaraSystem] ()
						{
							if (NiagaraSystem.IsValid())
							{
								const TArray<FAssetData>& Assets = { NiagaraSystem.Get() };
								const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
								ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
							}
						}),
						FCanExecuteAction::CreateLambda([] () { return true; })
					);
					ToolbarBuilder.AddToolBarButton(
						GotoAction,
						NAME_None,
						FText::FromString(NiagaraSystem->GetName()),
						LOCTEXT("GotoSystem", "Browses to the associated Niagara system asset"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser.Small")
					);
				}
			}
			ToolbarBuilder.EndSection();
			ToolbarBuilder.BeginSection("Export");
			{
				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::CopyActiveToClipboard),
						FCanExecuteAction::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::CanCopyActiveToClipboard)
					),
					NAME_None,
					LOCTEXT("CopyToCSV", "Copy to CSV"),
					LOCTEXT("CopyToCSVTooltip", "Copies the currently selected view to the clipboard in CSV format."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy")
				);

				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::ExportToDisk),
						FCanExecuteAction::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::CanExportToDisk)
					),
					NAME_None,
					LOCTEXT("ExportToJson", "Export To Disk"),
					LOCTEXT("ExportToJsonTooltip", "Exports the raw data for each frame to disk. Note that data from data interfaces is only exported if they implement support for it."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save")
				);

				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::CreateAsset),
						FCanExecuteAction(),
						FIsActionChecked(),
						FIsActionButtonVisible::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::IsCreateAssetVisible)
					),
					NAME_None,
					LOCTEXT("CreateAsset", "Create Asset"),
					LOCTEXT("CreateAssetTooltip", "Create an asset from this debug (transient) cache.")
				);
			}
			ToolbarBuilder.EndSection();
			ToolbarBuilder.BeginSection("Modify");
			{
				ToolbarBuilder.AddToolBarButton(
					FUIAction(
						FExecuteAction::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::RemoveDebugData),
						FCanExecuteAction(),
						FIsActionChecked(),
						FIsActionButtonVisible::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::CanRemoveDebugData)
					),
					NAME_None,
					LOCTEXT("RemoveDebugData", "Remove Debug Data"),
					LOCTEXT("RemoveDebugDataTooltip", "Remove the debug data that is present in the cache, this will reduce the file size.")
				);
			}
			ToolbarBuilder.EndSection();
		}
	};
	
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, this)
	);

	AddToolbarExtender(ToolbarExtender);
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	AddToolbarExtender(NiagaraEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

#undef LOCTEXT_NAMESPACE /** NiagaraSimCacheToolkit **/
