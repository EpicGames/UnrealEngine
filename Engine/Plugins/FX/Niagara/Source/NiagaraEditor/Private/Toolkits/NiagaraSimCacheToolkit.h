// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SNiagaraSimCacheView.h"
#include "Widgets/SNiagaraSimCacheViewport.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"

class FNiagaraSimCacheViewModel;
class UNiagaraSimCache;

// Editor for Niagara Simulation Caches
class FNiagaraSimCacheToolkit : public FAssetEditorToolkit
{
public:
	FNiagaraSimCacheToolkit();
	~FNiagaraSimCacheToolkit();
	
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;

	// Initialize with a sim cache to view. 
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraSimCache* InSimCache);

	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Interface

protected:
	/** The workspace menu category of this toolkit */
	TSharedPtr<FWorkspaceItem> WorkspaceMenuCategory;
	

private:
	TSharedRef<SDockTab> SpawnTab_SimCacheSpreadsheet(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SimCacheViewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_PreviewSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Overview(const FSpawnTabArgs& Args);

	void ExtendToolbar();

	// The sim cache being viewed.
	TWeakObjectPtr <UNiagaraSimCache> SimCache;

	// The view model for the sim cache
	TSharedPtr<FNiagaraSimCacheViewModel> SimCacheViewModel;

	// Spreadsheet widget
	TSharedPtr<SNiagaraSimCacheView> SimCacheSpreadsheetView;

	// Viewport Widget
	TSharedPtr<SNiagaraSimCacheViewport> Viewport;

	static const FName NiagaraSimCacheSpreadsheetTabId;
	static const FName NiagaraSimCacheViewportTabId;
	static const FName NiagaraSimCachePreviewSettingsTabId;
	static const FName NiagaraSimCacheOverviewTabId;
};