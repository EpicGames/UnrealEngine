// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStatelessEmitterTemplateToolkit.h"
#include "NiagaraEditorModule.h"
#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "ViewModels/NiagaraStatelessEmitterTemplateViewModel.h"
#include "Widgets/SNiagaraStatelessEmitterTemplateWidgets.h"

#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "NiagaraStatelessEmitterTemplateToolkit"

namespace NiagaraStatelessEmitterTemplateToolkitPrivate
{
	const FName FeaturesTabID(TEXT("NiagaraStatelessEmitterTemplateEditor_Features"));
	const FName ModulesTabID(TEXT("NiagaraStatelessEmitterTemplateEditor_Modules"));
	const FName OutputVariablesTabID(TEXT("NiagaraStatelessEmitterTemplateEditor_OutputVariables"));
	const FName CodeViewTabID(TEXT("NiagaraStatelessEmitterTemplateEditor_CodeView"));
}

FNiagaraStatelessEmitterTemplateToolkit::FNiagaraStatelessEmitterTemplateToolkit()
{
}

FNiagaraStatelessEmitterTemplateToolkit::~FNiagaraStatelessEmitterTemplateToolkit()
{
}

void FNiagaraStatelessEmitterTemplateToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UNiagaraStatelessEmitterTemplate* EmitterTemplate)
{
	using namespace NiagaraStatelessEmitterTemplateToolkitPrivate;

	WeakEmitterTemplate = EmitterTemplate;
	if (EmitterTemplate == nullptr)
	{
		return;
	}

	ViewModel = MakeShared<FNiagaraStatelessEmitterTemplateViewModel>(EmitterTemplate);
		
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout =
		FTabManager::NewLayout("Standalone_Niagara_StatelessEmitterTemplateLayout_V2_Dev")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.3f)
				->Split(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(FeaturesTabID, ETabState::OpenedTab)
				)
				->Split(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(OutputVariablesTabID, ETabState::OpenedTab)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.8f)
				->AddTab(CodeViewTabID, ETabState::OpenedTab)
				->AddTab(ModulesTabID, ETabState::OpenedTab)
			)
		);

	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	InitAssetEditor(Mode, InitToolkitHost, FNiagaraEditorModule::NiagaraEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, EmitterTemplate);

	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

void FNiagaraStatelessEmitterTemplateToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	using namespace NiagaraStatelessEmitterTemplateToolkitPrivate;

	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_NiagaraStatelessEmitterTemplate", "Lightweight Emitter Template"));
	
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(
		FeaturesTabID,
		FOnSpawnTab::CreateSP(this, &FNiagaraStatelessEmitterTemplateToolkit::SpawnTab_Features))
		.SetDisplayName(LOCTEXT("TemplateFeatures", "Features"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef()
	);
	InTabManager->RegisterTabSpawner(
		ModulesTabID,
		FOnSpawnTab::CreateSP(this, &FNiagaraStatelessEmitterTemplateToolkit::SpawnTab_Modules))
		.SetDisplayName(LOCTEXT("TemplateModules", "Modules"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef()
	);
	InTabManager->RegisterTabSpawner(
		OutputVariablesTabID,
		FOnSpawnTab::CreateSP(this, &FNiagaraStatelessEmitterTemplateToolkit::SpawnTab_OutputVariables))
		.SetDisplayName(LOCTEXT("TemplateOutputVariables", "Output Variables"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef()
	);
	InTabManager->RegisterTabSpawner(
		CodeViewTabID,
		FOnSpawnTab::CreateSP(this, &FNiagaraStatelessEmitterTemplateToolkit::SpawnTab_CodeView))
		.SetDisplayName(LOCTEXT("TemplateCodeView", "Code View"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef()
	);
}

void FNiagaraStatelessEmitterTemplateToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	using namespace NiagaraStatelessEmitterTemplateToolkitPrivate;

	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(FeaturesTabID);
	InTabManager->UnregisterTabSpawner(ModulesTabID);
	InTabManager->UnregisterTabSpawner(OutputVariablesTabID);
}

FName FNiagaraStatelessEmitterTemplateToolkit::GetToolkitFName() const
{
	return FName("Niagara");
}

FText FNiagaraStatelessEmitterTemplateToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Niagara");
}

FString FNiagaraStatelessEmitterTemplateToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Niagara ").ToString();
}

FLinearColor FNiagaraStatelessEmitterTemplateToolkit::GetWorldCentricTabColorScale() const
{
	return FNiagaraEditorModule::WorldCentricTabColorScale;
}

void FNiagaraStatelessEmitterTemplateToolkit::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, TSharedRef<FNiagaraStatelessEmitterTemplateViewModel> ViewModel)
		{
		//	ToolbarBuilder.BeginSection("Export");
		//	ToolbarBuilder.EndSection();
		}
	};
	
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, ViewModel->AsShared())
	);

	AddToolbarExtender(ToolbarExtender);
}

TSharedRef<SDockTab> FNiagaraStatelessEmitterTemplateToolkit::SpawnTab_Features(const FSpawnTabArgs& Args)
{
	return
		SNew(SDockTab)
		[
			SNew(SNiagaraStatelessEmitterTemplateFeatures)
			.ViewModel(ViewModel)
		];
}

TSharedRef<SDockTab> FNiagaraStatelessEmitterTemplateToolkit::SpawnTab_Modules(const FSpawnTabArgs& Args)
{
	return
		SNew(SDockTab)
		[
			SNew(SNiagaraStatelessEmitterTemplateModules)
			.ViewModel(ViewModel)
		];
}

TSharedRef<SDockTab> FNiagaraStatelessEmitterTemplateToolkit::SpawnTab_OutputVariables(const FSpawnTabArgs& Args)
{
	return
		SNew(SDockTab)
		[
			SNew(SNiagaraStatelessEmitterTemplateOutputVariables)
			.ViewModel(ViewModel)
		];
}

TSharedRef<SDockTab> FNiagaraStatelessEmitterTemplateToolkit::SpawnTab_CodeView(const FSpawnTabArgs& Args)
{
	return
		SNew(SDockTab)
		[
			SNew(SNiagaraStatelessEmitterTemplateCodeView)
			.ViewModel(ViewModel)
		];
}

#undef LOCTEXT_NAMESPACE
