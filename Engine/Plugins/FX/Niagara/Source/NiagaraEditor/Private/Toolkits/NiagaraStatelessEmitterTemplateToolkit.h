// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"

class UNiagaraStatelessEmitterTemplate;
class FNiagaraStatelessEmitterTemplateViewModel;

class SDockTab;
class FSpawnTabArgs;
class FWorkspaceItem;

// Editor for Niagara Stateless Emitter Templates
class FNiagaraStatelessEmitterTemplateToolkit : public FAssetEditorToolkit
{
public:
	FNiagaraStatelessEmitterTemplateToolkit();
	~FNiagaraStatelessEmitterTemplateToolkit();

	// Initialize with a sim cache to view. 
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UNiagaraStatelessEmitterTemplate* EmitterTemplate);

	//~ Begin IToolkit Interface
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Interface

protected:
	void ExtendToolbar();

	TSharedRef<SDockTab> SpawnTab_Features(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Modules(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_OutputVariables(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_CodeView(const FSpawnTabArgs& Args);

private:
	TWeakObjectPtr<UNiagaraStatelessEmitterTemplate>		WeakEmitterTemplate;
	TSharedPtr<FNiagaraStatelessEmitterTemplateViewModel>	ViewModel;

	TSharedPtr<FWorkspaceItem>								WorkspaceMenuCategory;	
};
