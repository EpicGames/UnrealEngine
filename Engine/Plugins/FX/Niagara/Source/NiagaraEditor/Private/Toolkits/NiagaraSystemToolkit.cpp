// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemToolkit.h"
#include "Engine/Texture2D.h"
#include "NiagaraEditorModule.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraObjectSelection.h"
#include "UObject/Linker.h"
#include "UObject/UObjectIterator.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraSystemScriptViewModel.h"
#include "Widgets/SNiagaraSystemScript.h"
#include "Widgets/SNiagaraSystemViewport.h"
#include "Widgets/SNiagaraParameterPanel.h"
#include "Widgets/SNiagaraDebugger.h"
#include "NiagaraEditorCommands.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraSystemFactoryNew.h"
#include "NiagaraComponent.h"
#include "NiagaraSystemEditorData.h"
#include "ViewModels/NiagaraScriptStatsViewModel.h"
#include "ViewModels/NiagaraBakerViewModel.h"
#include "NiagaraToolkitCommon.h"
#include "ViewModels/NiagaraSystemEditorDocumentsViewModel.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "Styling/AppStyle.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ScopedTransaction.h"

#include "BusyCursor.h"
#include "Dialogs/Dialogs.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "Framework/Application/SlateApplication.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleManager.h"
#include "ViewModels/NiagaraMessageLogViewModel.h"
#include "NiagaraVersionMetaData.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Interfaces/IMainFrameModule.h"
#include "Logging/StructuredLog.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"
#include "SystemToolkitModes/NiagaraSystemToolkitMode_Default.h"
#include "SystemToolkitModes/NiagaraSystemToolkitMode_Scalability.h"
#include "ViewModels/NiagaraParameterDefinitionsPanelViewModel.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNiagaraEmitterVersionWidget.h"
#include "Widgets/AssetBrowser/SNiagaraAddEmitterToSystemWindow.h"

#define LOCTEXT_NAMESPACE "NiagaraSystemEditor"

DECLARE_CYCLE_STAT(TEXT("Niagara - SystemToolkit - OnApply"), STAT_NiagaraEditor_SystemToolkit_OnApply, STATGROUP_NiagaraEditor);

static int32 GbLogNiagaraSystemChanges = 0;
static FAutoConsoleVariableRef CVarLogNiagaraSystemChanges(
	TEXT("fx.LogNiagaraSystemChanges"),
	GbLogNiagaraSystemChanges,
	TEXT("If > 0 Niagara Systems will be written to a text format when opened and closed in the editor. \n"),
	ECVF_Default
);

FNiagaraSystemToolkit::~FNiagaraSystemToolkit()
{
	// Cleanup viewmodels that use the system viewmodel before cleaning up the system viewmodel itself.
	if (ParameterPanelViewModel.IsValid())
	{
		ParameterPanelViewModel->Cleanup();
	}
	if (ParameterDefinitionsPanelViewModel.IsValid())
	{
		ParameterDefinitionsPanelViewModel->Cleanup();
	}

	if (SystemViewModel.IsValid())
	{
		if (SystemViewModel->GetSelectionViewModel() != nullptr)
		{
			SystemViewModel->GetSelectionViewModel()->OnSystemIsSelectedChanged().RemoveAll(this);
			SystemViewModel->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().RemoveAll(this);
		}
		SystemViewModel->Cleanup();
	}
	SystemViewModel.Reset();
}

void FNiagaraSystemToolkit::AddReferencedObjects(FReferenceCollector& Collector) 
{
	Collector.AddReferencedObject(System);
	Collector.AddReferencedObject(VersionMetadata);
}

TSharedPtr<SWidget> FNiagaraSystemToolkit::GetSystemOverview() const
{
	return SystemOverview;
}

void FNiagaraSystemToolkit::SetSystemOverview(const TSharedPtr<SWidget>& InSystemOverview)
{
	this->SystemOverview = InSystemOverview;
}

TSharedPtr<SWidget> FNiagaraSystemToolkit::GetScriptScratchpadManager() const
{
	return ScriptScratchpadManager;
}


void FNiagaraSystemToolkit::SetScriptScratchpadManager(const TSharedPtr<SWidget>& InScriptScratchpadManager)
{
	this->ScriptScratchpadManager = InScriptScratchpadManager;
}

void FNiagaraSystemToolkit::InitializeWithSystem(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraSystem& InSystem)
{
	System = &InSystem;
	Emitter = nullptr;
	System->EnsureFullyLoaded();

	// order of registering commands matters. SetupCommands before InitAssetEditor will make the toolkit prioritize niagara commands
	SetupCommands();
	
	FNiagaraSystemViewModelOptions SystemOptions;
	SystemOptions.bCanModifyEmittersFromTimeline = true;
	SystemOptions.EditMode = ENiagaraSystemViewModelEditMode::SystemAsset;
	SystemOptions.OnGetSequencerAddMenuContent.BindSP(this, &FNiagaraSystemToolkit::GetSequencerAddMenuContent);
	SystemOptions.MessageLogGuid = InSystem.GetAssetGuid();
	SystemOptions.bCompileForEdit = IsStatPerformanceChecked() ? IsProfileInEditMode() : true;

	SystemViewModel = MakeShared<FNiagaraSystemViewModel>();
	SystemViewModel->Initialize(*System, SystemOptions);
	ParameterPanelViewModel = MakeShared<FNiagaraSystemToolkitParameterPanelViewModel>(SystemViewModel);
	ParameterDefinitionsPanelViewModel = MakeShared<FNiagaraSystemToolkitParameterDefinitionsPanelViewModel>(SystemViewModel);
	FSystemToolkitUIContext UIContext = FSystemToolkitUIContext(
		FSimpleDelegate::CreateSP(ParameterPanelViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh),
		FSimpleDelegate::CreateSP(ParameterDefinitionsPanelViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh)
	);
	ParameterPanelViewModel->Init(UIContext);
	ParameterDefinitionsPanelViewModel->Init(UIContext);
	
	SystemViewModel->SetToolkitCommands(GetToolkitCommands());
	SystemViewModel->SetParameterPanelViewModel(ParameterPanelViewModel);
	SystemToolkitMode = ESystemToolkitMode::System;

	if (GbLogNiagaraSystemChanges > 0)
	{
		FString ExportText;
		SystemViewModel->DumpToText(ExportText);
		FString FilePath = System->GetOutermost()->GetLoadedPath().GetPackageName();
		FString PathPart, FilenamePart, ExtensionPart;
		FPaths::Split(FilePath, PathPart, FilenamePart, ExtensionPart);

		FNiagaraEditorUtilities::WriteTextFileToDisk(FPaths::ProjectLogDir(), FilenamePart + TEXT(".onLoad.txt"), ExportText, true);
	}

	InitializeInternal(Mode, InitToolkitHost, SystemOptions.MessageLogGuid.GetValue());
}

void FNiagaraSystemToolkit::InitializeRapidIterationParameters(const FVersionedNiagaraEmitter& VersionedEmitter)
{
	// Before copying the emitter prepare the rapid iteration parameters so that the post compile prepare doesn't
	// cause the change ids to become out of sync.
	TArray<UNiagaraScript*> Scripts;
	TMap<UNiagaraScript*, UNiagaraScript*> ScriptDependencyMap;
	TMap<UNiagaraScript*, FVersionedNiagaraEmitter> ScriptToEmitterMap;

	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	Scripts.Add(EmitterData->EmitterSpawnScriptProps.Script);
	ScriptToEmitterMap.Add(EmitterData->EmitterSpawnScriptProps.Script, VersionedEmitter);

	Scripts.Add(EmitterData->EmitterUpdateScriptProps.Script);
	ScriptToEmitterMap.Add(EmitterData->EmitterUpdateScriptProps.Script, VersionedEmitter);

	Scripts.Add(EmitterData->SpawnScriptProps.Script);
	ScriptToEmitterMap.Add(EmitterData->SpawnScriptProps.Script, VersionedEmitter);

	Scripts.Add(EmitterData->UpdateScriptProps.Script);
	ScriptToEmitterMap.Add(EmitterData->UpdateScriptProps.Script, VersionedEmitter);

	if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		Scripts.Add(EmitterData->GetGPUComputeScript());
		ScriptToEmitterMap.Add(EmitterData->GetGPUComputeScript(), VersionedEmitter);
		ScriptDependencyMap.Add(EmitterData->SpawnScriptProps.Script, EmitterData->GetGPUComputeScript());
		ScriptDependencyMap.Add(EmitterData->UpdateScriptProps.Script, EmitterData->GetGPUComputeScript());
	} 
	else if (EmitterData->UsesInterpolatedSpawning())
	{
		ScriptDependencyMap.Add(EmitterData->UpdateScriptProps.Script, EmitterData->SpawnScriptProps.Script);
	}

	FNiagaraUtilities::PrepareRapidIterationParameters(Scripts, ScriptDependencyMap, ScriptToEmitterMap);
}

void FNiagaraSystemToolkit::InitializeWithEmitter(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraEmitter& InEmitter)
{
	System = NewObject<UNiagaraSystem>(GetTransientPackage(), NAME_None, RF_Transient | RF_Transactional);
	UNiagaraSystemFactoryNew::InitializeSystem(System, true);
	System->EnsureFullyLoaded();

	// order of registering commands matters. SetupCommands before InitAssetEditor will make the toolkit prioritize niagara commands
	SetupCommands();

	InEmitter.UpdateEmitterAfterLoad();
	Emitter = &InEmitter;

	FGuid VersionGuid = Emitter->IsVersioningEnabled() && Emitter->VersionToOpenInEditor.IsValid() ? Emitter->VersionToOpenInEditor : Emitter->GetExposedVersion().VersionGuid;
	FVersionedNiagaraEmitter VersionedEmitter = FVersionedNiagaraEmitter(Emitter, VersionGuid);
	InitializeRapidIterationParameters(VersionedEmitter);

	// No need to reset loader or versioning on the transient package, there should never be any set 
	
	bEmitterThumbnailUpdated = false;

	FNiagaraSystemViewModelOptions SystemOptions;
	SystemOptions.bCanModifyEmittersFromTimeline = false;
	SystemOptions.EditMode = ENiagaraSystemViewModelEditMode::EmitterAsset;
	SystemOptions.MessageLogGuid = System->GetAssetGuid();

	SystemViewModel = MakeShared<FNiagaraSystemViewModel>();
	SystemViewModel->Initialize(*System, SystemOptions);
	SystemViewModel->GetEditorData().SetOwningSystemIsPlaceholder(true, *System);
	SystemViewModel->SetToolkitCommands(GetToolkitCommands());

	SystemViewModel->OnGetWorkflowMode().BindSP(this, &FNiagaraSystemToolkit::GetCurrentMode);

	SystemViewModel->AddEmitter(VersionedEmitter);

	ParameterPanelViewModel = MakeShared<FNiagaraSystemToolkitParameterPanelViewModel>(SystemViewModel);
	ParameterDefinitionsPanelViewModel = MakeShared<FNiagaraSystemToolkitParameterDefinitionsPanelViewModel>(SystemViewModel);
	FSystemToolkitUIContext UIContext = FSystemToolkitUIContext(
		FSimpleDelegate::CreateSP(ParameterPanelViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh),
		FSimpleDelegate::CreateSP(ParameterDefinitionsPanelViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh)
	);
	ParameterPanelViewModel->Init(UIContext);
	ParameterDefinitionsPanelViewModel->Init(UIContext);

	// Adding the emitter to the system has made a copy of it and we set this to the copy's change id here instead of the original emitter's change 
	// id because the copy's change id may have been updated from the original as part of post load and we use this id to detect if the editable 
	// emitter has been changed.
	LastSyncedEmitterChangeId = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterViewModel()->GetEmitter().Emitter->GetChangeId();

	// Mirror the system setup above so that the parameter panel updates appropriately
	SystemViewModel->SetParameterPanelViewModel(ParameterPanelViewModel);
	SystemToolkitMode = ESystemToolkitMode::Emitter;

	if (GbLogNiagaraSystemChanges > 0)
	{
		FString ExportText;
		SystemViewModel->DumpToText(ExportText);
		FString FilePath = Emitter->GetOutermost()->GetLoadedPath().GetPackageName();
		FString PathPart, FilenamePart, ExtensionPart;
		FPaths::Split(FilePath, PathPart, FilenamePart, ExtensionPart);

		FNiagaraEditorUtilities::WriteTextFileToDisk(FPaths::ProjectLogDir(), FilenamePart + TEXT(".onLoad.txt"), ExportText, true);
	}

	InitializeInternal(Mode, InitToolkitHost, SystemOptions.MessageLogGuid.GetValue());
}

void FNiagaraSystemToolkit::InitializeInternal(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, const FGuid& MessageLogGuid)
{
	NiagaraMessageLogViewModel = MakeShared<FNiagaraMessageLogViewModel>(GetNiagaraSystemMessageLogName(System), MessageLogGuid, NiagaraMessageLog);
	ObjectSelectionForParameterMapView = MakeShared<FNiagaraObjectSelection>();
	ScriptStats = MakeShared<FNiagaraScriptStatsViewModel>();
	ScriptStats->Initialize(SystemViewModel);
	BakerViewModel = MakeShared<FNiagaraBakerViewModel>();
	BakerViewModel->Initialize(SystemViewModel);

	SimCacheViewModel = MakeShared<FNiagaraSimCacheViewModel>();
	SimCacheViewModel->Initialize(nullptr);

	SystemViewModel->OnEmitterHandleViewModelsChanged().AddSP(this, &FNiagaraSystemToolkit::RefreshParameters);
	SystemViewModel->GetSelectionViewModel()->OnSystemIsSelectedChanged().AddSP(this, &FNiagaraSystemToolkit::OnSystemSelectionChanged);
	SystemViewModel->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().AddSP(this, &FNiagaraSystemToolkit::OnSystemSelectionChanged);
	SystemViewModel->GetOnPinnedEmittersChanged().AddSP(this, &FNiagaraSystemToolkit::RefreshParameters);
	SystemViewModel->OnRequestFocusTab().AddSP(this, &FNiagaraSystemToolkit::OnViewModelRequestFocusTab);
	SystemViewModel->OnGetWorkflowMode().BindSP(this, &FNiagaraSystemToolkit::GetCurrentMode);
	SystemViewModel->OnChangeWorkflowMode().BindSP(this, &FNiagaraSystemToolkit::SetCurrentMode);
	SystemViewModel->OnEmitterThumbnailRequested().BindSP(this, &FNiagaraSystemToolkit::CaptureEmitterThumbnail);
	SystemViewModel->GetDocumentViewModel()->InitializePreTabManager(SharedThis(this));
	
	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	UObject* ToolkitObject = SystemToolkitMode == ESystemToolkitMode::System ? (UObject*)System : (UObject*)Emitter;
	// // order of registering commands matters. SetupCommands before InitAssetEditor will make the toolkit prioritize niagara commands
	// SetupCommands();
	// Now that everything should be initialized, we can link the command lists of various places
	LinkCommandLists();

	const TSharedRef<FTabManager::FLayout> DummyLayout = FTabManager::NewLayout("NullLayout")->AddArea(FTabManager::NewPrimaryArea());
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FNiagaraEditorModule::NiagaraEditorAppIdentifier,
		DummyLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ToolkitObject);
	
	SystemViewModel->GetDocumentViewModel()->InitializePostTabManager(SharedThis(this));

	AddApplicationMode(FNiagaraSystemToolkitModeBase::DefaultModeName, MakeShared<FNiagaraSystemToolkitMode_Default>(SharedThis(this)));
	AddApplicationMode(FNiagaraSystemToolkitModeBase::ScalabilityModeName, MakeShared<FNiagaraSystemToolkitMode_Scalability>(SharedThis(this)));

	// set up the versioning widget
	VersionMetadata = NewObject<UNiagaraVersionMetaData>(ToolkitObject, "VersionMetadata", RF_Transient);
	SAssignNew(VersionsWidget, SNiagaraEmitterVersionWidget, HasEmitter() ? GetEditedEmitterViewModel()->GetEmitter().Emitter : nullptr, VersionMetadata, HasEmitter() ? Emitter->GetOutermost()->GetName() : TEXT(""))
		.OnChangeToVersion(this, &FNiagaraSystemToolkit::SwitchToVersion)
		.OnVersionDataChanged_Lambda([this](const FPropertyChangedEvent* InChangedEvent, FGuid SelectedVersion)
		{
			if (TSharedPtr<FNiagaraEmitterViewModel> EditableEmitterViewModel = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterViewModel())
			{
				FVersionedNiagaraEmitter EditableEmitter = EditableEmitterViewModel->GetEmitter();
				FProperty* VersionProperty = FindFProperty<FProperty>(UNiagaraEmitter::StaticClass(), FName("VersionData"));
				FPropertyChangedEvent ChangeEvent = InChangedEvent ? *InChangedEvent : FPropertyChangedEvent(VersionProperty);
				EditableEmitter.Emitter->PostEditChangeProperty(ChangeEvent);
			}
		});
	
	SetCurrentMode(FNiagaraSystemToolkitModeBase::DefaultModeName);
	
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	AddMenuExtender(NiagaraEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	AddToolbarExtender(NiagaraEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	RegenerateMenusAndToolbars();

	bChangesDiscarded = false;
	bScratchPadChangesDiscarded = false;

	GEditor->RegisterForUndo(this);
#if WITH_NIAGARA_GPU_PROFILER
	GpuProfilerListener.Reset(new FNiagaraGpuProfilerListener);
#endif
}

FName FNiagaraSystemToolkit::GetToolkitFName() const
{
	return FName("Niagara");
}

FText FNiagaraSystemToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Niagara");
}

FString FNiagaraSystemToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Niagara ").ToString();
}


FLinearColor FNiagaraSystemToolkit::GetWorldCentricTabColorScale() const
{
	return FNiagaraEditorModule::WorldCentricTabColorScale;
}

bool FNiagaraSystemToolkit::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const
{
	TArray<UNiagaraGraph*> Graphs;

	for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScratchPadViewModel : SystemViewModel->GetScriptScratchPadViewModel()->GetScriptViewModels())
	{
		Graphs.AddUnique(ScratchPadViewModel->GetGraphViewModel()->GetGraph());
	}

	LastUndoGraphs.Empty();

	if (Graphs.Num() > 0)
	{
		for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjects)
		{
			UObject* Object = TransactionObjectPair.Key;
			while (Object != nullptr)
			{
				if (Graphs.Contains(Object))
				{
					LastUndoGraphs.AddUnique(Cast<UNiagaraGraph>(Object));
					return true;
				}
				Object = Object->GetOuter();
			}
		}
	}
	return LastUndoGraphs.Num() > 0;
}

void FNiagaraSystemToolkit::PostUndo(bool bSuccess)
{
	for (TWeakObjectPtr<UNiagaraGraph> Graph : LastUndoGraphs)
	{
		if (Graph.IsValid())
		{
			Graph->NotifyGraphNeedsRecompile();
		}
	}

	LastUndoGraphs.Empty();
}

void FNiagaraSystemToolkit::SetupCommands()
{
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().Compile,
		FExecuteAction::CreateRaw(this, &FNiagaraSystemToolkit::CompileSystem, false));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ResetSimulation,
		FExecuteAction::CreateRaw(this, &FNiagaraSystemToolkit::ResetSimulation)); 

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleStatPerformance,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ToggleStatPerformance),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsStatPerformanceChecked),
		FIsActionButtonVisible::CreateSP(this, &FNiagaraSystemToolkit::SupportsStatPerformance));
	GetToolkitCommands()->MapAction(
        FNiagaraEditorCommands::Get().ClearStatPerformance,
        FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ClearStatPerformance));
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleProfileInEditMode,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ToggleProfileInEditMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsProfileInEditMode));
	GetToolkitCommands()->MapAction(
        FNiagaraEditorCommands::Get().ToggleStatPerformanceGPU,
        FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ToggleStatPerformanceGPU),
        FCanExecuteAction(),
        FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsStatPerformanceGPUChecked));
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleStatPerformanceTypeAvg,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ToggleStatPerformanceTypeAvg),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsStatPerformanceTypeAvg));
	GetToolkitCommands()->MapAction(
        FNiagaraEditorCommands::Get().ToggleStatPerformanceTypeMax,
        FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ToggleStatPerformanceTypeMax),
        FCanExecuteAction(),
        FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsStatPerformanceTypeMax));
	GetToolkitCommands()->MapAction(
        FNiagaraEditorCommands::Get().ToggleStatPerformanceModeAbsolute,
        FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ToggleStatPerformanceModeAbsolute),
        FCanExecuteAction(),
        FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsStatPerformanceModeAbsolute));
	GetToolkitCommands()->MapAction(
        FNiagaraEditorCommands::Get().ToggleStatPerformanceModePercent,
        FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ToggleStatPerformanceModePercent),
        FCanExecuteAction(),
        FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsStatPerformanceModePercent));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleBounds,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OnToggleBounds),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsToggleBoundsChecked));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleBounds_SetFixedBounds_SelectedEmitters,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OnToggleBoundsSetFixedBounds_Emitters),
		FCanExecuteAction::CreateLambda([this]()
		{
			return (this->SystemToolkitMode == ESystemToolkitMode::System && this->SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds().Num() > 0) ||
				this->SystemToolkitMode == ESystemToolkitMode::Emitter;				
		}));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleBounds_SetFixedBounds_System,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OnToggleBoundsSetFixedBounds_System));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().SaveThumbnailImage,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::CaptureAssetThumbnail));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().Apply,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OnApply),
		FCanExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OnApplyEnabled));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ApplyScratchPadChanges,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OnApplyScratchPadChanges),
		FCanExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OnApplyScratchPadChangesEnabled));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleAutoPlay,
		FExecuteAction::CreateLambda([]()
		{
			UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
			Settings->SetAutoPlay(!Settings->GetAutoPlay());
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]() { return GetDefault<UNiagaraEditorSettings>()->GetAutoPlay(); }));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleResetSimulationOnChange,
		FExecuteAction::CreateLambda([]()
		{
			UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
			Settings->SetResetSimulationOnChange(!Settings->GetResetSimulationOnChange());
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]() { return GetDefault<UNiagaraEditorSettings>()->GetResetSimulationOnChange(); }));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleResimulateOnChangeWhilePaused,
		FExecuteAction::CreateLambda([]()
		{
			UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
			Settings->SetResimulateOnChangeWhilePaused(!Settings->GetResimulateOnChangeWhilePaused());
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]() { return GetDefault<UNiagaraEditorSettings>()->GetResimulateOnChangeWhilePaused(); }));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().ToggleResetDependentSystems,
		FExecuteAction::CreateLambda([]()
		{
			UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
			Settings->SetResetDependentSystemsWhenEditingEmitters(!Settings->GetResetDependentSystemsWhenEditingEmitters());
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([]() { return GetDefault<UNiagaraEditorSettings>()->GetResetDependentSystemsWhenEditingEmitters(); }),
		FIsActionButtonVisible::CreateLambda([this]() { return SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::EmitterAsset; }));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().IsolateSelectedEmitters,
		FExecuteAction::CreateLambda([this]()
		{
			SystemViewModel->IsolateSelectedEmitters();
		}),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().EnableSelectedEmitters,
		FExecuteAction::CreateLambda([this]()
			{
				SystemViewModel->EnableSelectedEmitters();
			}),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().DisableSelectedEmitters,
		FExecuteAction::CreateLambda([this]()
		{
			SystemViewModel->DisableSelectedEmitters();
		}),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().OpenAddEmitterMenu,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OpenAddEmitterMenu),
		FCanExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::CanAddEmitters),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FNiagaraSystemToolkit::CanAddEmitters)
	);
	
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().OpenDebugHUD,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OpenDebugHUD));
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().OpenDebugOutliner,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OpenDebugOutliner));	
	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().OpenAttributeSpreadsheet,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::OpenAttributeSpreadsheet));

	GetToolkitCommands()->MapAction(
		FNiagaraEditorCommands::Get().EmitterVersioning,
		FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ManageVersions));
}

void FNiagaraSystemToolkit::LinkCommandLists()
{
	// appending the sequencer commands will make the toolkit also check for sequencer commands (last)
	GetToolkitCommands()->Append(SystemViewModel->GetSequencer()->GetCommandBindings(ESequencerCommandBindings::Sequencer).ToSharedRef());
	SystemViewModel->GetSequencer()->GetCommandBindings(ESequencerCommandBindings::Sequencer)->Append(GetToolkitCommands());
}

void FNiagaraSystemToolkit::ManageVersions()
{
	TabManager->TryInvokeTab(FNiagaraSystemToolkitModeBase::VersioningTabID);
}

TSharedPtr<FNiagaraEmitterViewModel> FNiagaraSystemToolkit::GetEditedEmitterViewModel() const
{
	return HasEmitter() ? SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterViewModel() : TSharedPtr<FNiagaraEmitterViewModel>(); 
}

void FNiagaraSystemToolkit::OpenAddEmitterMenu()
{
	IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
	TSharedPtr<SWindow>	ParentWindow = MainFrame.GetParentWindow();

	FSoftObjectPath Path = GetDefault<UNiagaraEditorSettings>()->EmitterAssetWizardConfiguration;
	UObject* ConfigUObject = Path.TryLoad();
	if(UTaggedAssetBrowserConfiguration* ConfigurationAsset = Cast<UTaggedAssetBrowserConfiguration>(ConfigUObject))
	{
		TSharedRef<SWindow> AddEmitterWindow = SNew(SNiagaraAddEmitterToSystemWindow, *ConfigurationAsset, GetSystemViewModel().ToSharedRef());
		FSlateApplication::Get().AddModalWindow(AddEmitterWindow, ParentWindow);
	}
	else
	{
		UE_LOGFMT(LogNiagaraEditor, Warning, "Could not find emitter wizard configuration asset. Check Niagara Editor settings under Project Settings");
	}
}

bool FNiagaraSystemToolkit::CanAddEmitters() const
{
	return GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset ? true : false;
}

void FNiagaraSystemToolkit::GetSequencerAddMenuContent(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> Sequencer)
{
	MenuBuilder.PushCommandList(GetToolkitCommands());
	
	MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().OpenAddEmitterMenu);

	MenuBuilder.PopCommandList();
}

void FNiagaraSystemToolkit::CaptureAssetThumbnail() const
{
	if (Viewport.IsValid())
	{		
		Viewport->CreateThumbnail(SystemToolkitMode == ESystemToolkitMode::System ? static_cast<UObject*>(System) : Emitter, {});
	}
}

void FNiagaraSystemToolkit::CaptureEmitterThumbnail(FGuid EmitterGuid)
{
	TSharedPtr<FNiagaraEmitterHandleViewModel> EditableEmitterHandleViewModel = SystemViewModel->GetEmitterHandleViewModelById(EmitterGuid);
	if(EditableEmitterHandleViewModel)
	{
		if(Viewport.IsValid())
		{
			Viewport->CreateThumbnail(EditableEmitterHandleViewModel->GetEmitterViewModel()->GetEmitter().Emitter, EmitterGuid);
		}
	}
}

void FNiagaraSystemToolkit::OnThumbnailCaptured(UTexture2D* Thumbnail, TOptional<FGuid> EmitterGuid)
{
	if (SystemToolkitMode == ESystemToolkitMode::System)
	{
		System->MarkPackageDirty();
		// in a system, we can either capture thumbnails for individual emitters, or for the system itself
		if(EmitterGuid.IsSet())
		{
			if(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = GetSystemViewModel()->GetEmitterHandleViewModelById(EmitterGuid.GetValue()))
			{
				EmitterHandleViewModel->GetEmitterViewModel()->GetEditorData().SetThumbnail(Thumbnail);
			}
			
			// since we cached the isolation state to capture only individual emitters, we restore the state here
			GetSystemViewModel()->RestoreIsolatedEmitterState();

			// we force a refresh to the render state which wouldn't occur if taking a thumbnail while paused
			if(Viewport.IsValid())
			{
				Viewport->GetPreviewComponent()->MarkRenderStateDirty();
			}
		}
		else
		{
			System->ThumbnailImage = Thumbnail;
		}
		// Broadcast an object property changed event to update the content browser
		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(System, EmptyPropertyChangedEvent);
	}
	else if (SystemToolkitMode == ESystemToolkitMode::Emitter) 
	{
		TSharedPtr<FNiagaraEmitterViewModel> EditableEmitterViewModel = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterViewModel();
		UNiagaraEmitter* EditableEmitter = EditableEmitterViewModel->GetEmitter().Emitter;
		EditableEmitter->ThumbnailImage = Thumbnail;
		EditableEmitterViewModel->GetEditorData().SetThumbnail(Thumbnail);
		bEmitterThumbnailUpdated = true;
		// Broadcast an object property changed event to update the content browser
		FPropertyChangedEvent EmptyPropertyChangedEvent(nullptr);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(EditableEmitter, EmptyPropertyChangedEvent);
	}
}

void FNiagaraSystemToolkit::ResetSimulation()
{
	SystemViewModel->ResetSystem(FNiagaraSystemViewModel::ETimeResetMode::AllowResetTime, FNiagaraSystemViewModel::EMultiResetMode::AllowResetAllInstances, FNiagaraSystemViewModel::EReinitMode::ReinitializeSystem);
}

TSharedRef<SWidget> FNiagaraSystemToolkit::GenerateBoundsMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	if(SystemToolkitMode == ESystemToolkitMode::System)
	{
		MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleBounds_SetFixedBounds_System);		
	}
	
	MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleBounds_SetFixedBounds_SelectedEmitters);

	MenuBuilder.AddMenuSeparator();

	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		MenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("ShowEmitterBounds", "Show {0} Emitter Bounds"), EmitterViewModel->GetNameText()),
			LOCTEXT("ShowEmitterBoundsTooltip", "Shows the emitter bounds."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::ToggleShowEmitterBounds, EmitterViewModel),
				FCanExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::CanShowEmitterBounds),
				FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsShowEmitterBounds, EmitterViewModel)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FNiagaraSystemToolkit::GenerateStatConfigMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ClearStatPerformance);
	MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleProfileInEditMode);
	MenuBuilder.AddMenuSeparator();
	MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleStatPerformanceGPU);
	MenuBuilder.AddMenuSeparator();
	MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleStatPerformanceTypeAvg);
	MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleStatPerformanceTypeMax);
	MenuBuilder.AddMenuSeparator();
	MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleStatPerformanceModePercent);
	MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleStatPerformanceModeAbsolute);

	return MenuBuilder.MakeWidget();
}

const FName FNiagaraSystemToolkit::GetNiagaraSystemMessageLogName(UNiagaraSystem* InSystem) const
{
	checkf(InSystem, TEXT("Tried to get MessageLog name for NiagaraSystem but InSystem was null!"));
	FName LogListingName = *FString::Printf(TEXT("%s_%s_MessageLog"), *FString::FromInt(InSystem->GetUniqueID()), *InSystem->GetName());
	return LogListingName;
}

FText FNiagaraSystemToolkit::GetVersionButtonLabel() const
{
	FText BaseLabel = LOCTEXT("NiagaraShowEmitterVersions", "Versioning");
	TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = GetEditedEmitterViewModel();
	if (EmitterViewModel && EmitterViewModel->GetEmitter().Emitter->IsVersioningEnabled())
	{
		if (FVersionedNiagaraEmitterData* EmitterData = EmitterViewModel->GetEmitter().GetEmitterData())
		{
			FNiagaraAssetVersion ExposedVersion = EmitterViewModel->GetEmitter().Emitter->GetExposedVersion();
			return FText::Format(FText::FromString("{0} ({1}.{2}{3})"), BaseLabel, EmitterData->Version.MajorVersion, EmitterData->Version.MinorVersion, EmitterData->Version <= ExposedVersion ? FText::FromString("*") : FText());
		}
	}
	return BaseLabel;
}

TArray<FNiagaraAssetVersion> FNiagaraSystemToolkit::GetEmitterVersions() const
{
	TArray<FNiagaraAssetVersion> Result;
	if (TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = GetEditedEmitterViewModel())
	{
		Result.Append(EmitterViewModel->GetEmitter().Emitter->GetAllAvailableVersions());
	}
	return Result;
}

TSharedRef<SWidget> FNiagaraSystemToolkit::GenerateVersioningDropdownMenu(TSharedRef<FUICommandList> InCommandList)
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	TArray<FNiagaraAssetVersion> AssetVersions = GetEmitterVersions();
	for (FNiagaraAssetVersion& Version : AssetVersions)
	{
		FText Tooltip = LOCTEXT("NiagaraSelectVersion", "Select this emitter version to edit");
		FUIAction UIAction(FExecuteAction::CreateSP(this, &FNiagaraSystemToolkit::SwitchToVersion, Version.VersionGuid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FNiagaraSystemToolkit::IsVersionSelected, Version));
		TAttribute<FText> Label = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraSystemToolkit::GetVersionMenuLabel, Version));
		MenuBuilder.AddMenuEntry(Label, Tooltip, FSlateIcon(), UIAction, NAME_None, EUserInterfaceActionType::RadioButton);	
	}

	return MenuBuilder.MakeWidget();
}

void FNiagaraSystemToolkit::SwitchToVersion(FGuid VersionGuid)
{
	if (TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = GetEditedEmitterViewModel())
	{
		if (EmitterViewModel->GetEmitter().Version == VersionGuid)
		{
			return;
		}
		
		FScopedTransaction Transaction(LOCTEXT("ChangeEmitterVersion", "Switch to emitter version"));
		InitializeRapidIterationParameters(FVersionedNiagaraEmitter(EmitterViewModel->GetEmitter().Emitter, VersionGuid));
		if (GetSystemViewModel()->ChangeEmitterVersion(EmitterViewModel->GetEmitter(), VersionGuid))
		{
			GetSystemViewModel()->GetSelectionViewModel()->EmptySelection();
			GetSystemViewModel()->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectDeferred(EmitterViewModel->GetEmitter().Emitter);
		}
	}
}

bool FNiagaraSystemToolkit::IsVersionSelected(FNiagaraAssetVersion Version) const
{
	return HasEmitter() && GetEditedEmitterViewModel()->GetEmitter().Version == Version.VersionGuid;
}

FText FNiagaraSystemToolkit::GetVersionMenuLabel(FNiagaraAssetVersion Version) const
{
	TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = GetEditedEmitterViewModel();
	bool bIsExposed = EmitterViewModel && Version == EmitterViewModel->GetEmitter().Emitter->GetExposedVersion();
	return FText::Format(FText::FromString("v{0}.{1} {2}"), Version.MajorVersion, Version.MinorVersion, bIsExposed ? LOCTEXT("NiagaraExposedVersionHint", "(exposed)") : FText());
}

TSharedRef<SWidget> FNiagaraSystemToolkit::GenerateCompileMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	FUIAction Action(
		FExecuteAction::CreateStatic(&FNiagaraSystemToolkit::ToggleCompileEnabled),
		FCanExecuteAction(),
		FIsActionChecked::CreateStatic(&FNiagaraSystemToolkit::IsAutoCompileEnabled));

	FUIAction FullRebuildAction(
		FExecuteAction::CreateRaw(this, &FNiagaraSystemToolkit::CompileSystem, true));


	static const FName CompileStatusBackground("AssetEditor.CompileStatus.Background");
	static const FName CompileStatusUnknown("AssetEditor.CompileStatus.Overlay.Unknown");


	MenuBuilder.AddMenuEntry(LOCTEXT("FullRebuild", "Full Rebuild"),
		LOCTEXT("FullRebuildTooltip", "Triggers a full rebuild of this system, ignoring the change tracking."),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown),
		FullRebuildAction, NAME_None, EUserInterfaceActionType::Button);
	MenuBuilder.AddMenuEntry(LOCTEXT("AutoCompile", "Automatically compile when graph changes"),
		FText(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::ToggleButton);

	return MenuBuilder.MakeWidget();
}

FSlateIcon FNiagaraSystemToolkit::GetCompileStatusImage() const
{
	ENiagaraScriptCompileStatus Status = SystemViewModel->GetLatestCompileStatus();

	static const FName CompileStatusBackground("AssetEditor.CompileStatus.Background");
	static const FName CompileStatusUnknown("AssetEditor.CompileStatus.Overlay.Unknown");
	static const FName CompileStatusError("AssetEditor.CompileStatus.Overlay.Error");
	static const FName CompileStatusGood("AssetEditor.CompileStatus.Overlay.Good");
	static const FName CompileStatusWarning("AssetEditor.CompileStatus.Overlay.Warning");

	switch (Status)
	{
	default:
	case ENiagaraScriptCompileStatus::NCS_Unknown:
	case ENiagaraScriptCompileStatus::NCS_Dirty:
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
	case ENiagaraScriptCompileStatus::NCS_Error:
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusError);
	case ENiagaraScriptCompileStatus::NCS_UpToDate:
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusGood);
	case ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings:
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusWarning);
	}
}

FText FNiagaraSystemToolkit::GetCompileStatusTooltip() const
{
	ENiagaraScriptCompileStatus Status = SystemViewModel->GetLatestCompileStatus();
	return FNiagaraEditorUtilities::StatusToText(Status);
}


void FNiagaraSystemToolkit::CompileSystem(bool bFullRebuild)
{
	SystemViewModel->CompileSystem(bFullRebuild);
}

TSharedPtr<FNiagaraSystemViewModel> FNiagaraSystemToolkit::GetSystemViewModel() const
{
	return SystemViewModel;
}

TSharedPtr<FNiagaraSimCacheViewModel> FNiagaraSystemToolkit::GetSimCacheViewModel()
{
	return SimCacheViewModel;
}

TSharedPtr<FNiagaraSystemGraphSelectionViewModel> FNiagaraSystemToolkit::GetSystemGraphSelectionViewModel()
{
	return SystemViewModel->GetSystemGraphSelectionViewModel();
}

void FNiagaraSystemToolkit::RegisterToolbarTab(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

FAssetData FNiagaraSystemToolkit::GetEditedAsset() const
{
	if (HasEmitter())
	{
		return FAssetData(Emitter);
	}
	if (HasSystem())
	{
		return FAssetData(System);
	}
	return FAssetData();
}

const TArray<UObject*>& FNiagaraSystemToolkit::GetObjectsBeingEdited() const
{
	return GetEditingObjects();
}

void FNiagaraSystemToolkit::OnToggleBounds()
{
	ToggleDrawOption(SNiagaraSystemViewport::Bounds);
}

bool FNiagaraSystemToolkit::IsToggleBoundsChecked() const
{
	return IsDrawOptionEnabled(SNiagaraSystemViewport::Bounds);
}

bool FNiagaraSystemToolkit::CanShowEmitterBounds() const
{
	return SystemViewModel->GetSystem().bFixedBounds == false;
}

bool FNiagaraSystemToolkit::IsShowEmitterBounds(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterViewModel) const
{
	FNiagaraEmitterHandle* EmitterHandle = EmitterViewModel->GetEmitterHandle();
	return EmitterHandle ? EmitterHandle->GetDebugShowBounds() : false;
}

void FNiagaraSystemToolkit::ToggleShowEmitterBounds(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterViewModel)
{
	if (FNiagaraEmitterHandle* EmitterHandle = EmitterViewModel->GetEmitterHandle())
	{
		EmitterHandle->SetDebugShowBounds(!EmitterHandle->GetDebugShowBounds());
		if (Viewport.IsValid())
		{
			Viewport->RefreshViewport();
		}
	}
}

void FNiagaraSystemToolkit::ToggleDrawOption(int32 Element)
{
	if (Viewport.IsValid() && Viewport->GetViewportClient().IsValid())
	{
		Viewport->ToggleDrawElement((SNiagaraSystemViewport::EDrawElements)Element);
		Viewport->RefreshViewport();
	}
}

bool FNiagaraSystemToolkit::IsDrawOptionEnabled(int32 Element) const
{
	if (Viewport.IsValid() && Viewport->GetViewportClient().IsValid())
	{
		return Viewport->GetDrawElement((SNiagaraSystemViewport::EDrawElements)Element);
	}
	else
	{
		return false;
	}
}

void FNiagaraSystemToolkit::OpenDebugHUD()
{
#if WITH_NIAGARA_DEBUGGER
	SNiagaraDebugger::InvokeDebugger(&SystemViewModel->GetSystem());
#endif
}

void FNiagaraSystemToolkit::OpenDebugOutliner()
{
#if WITH_NIAGARA_DEBUGGER
	TSharedPtr<SDockTab> DebugTab = FGlobalTabmanager::Get()->TryInvokeTab(SNiagaraDebugger::DebugWindowName);

	if (DebugTab.IsValid())
	{
		TSharedRef<SNiagaraDebugger> Content = StaticCastSharedRef<SNiagaraDebugger>(DebugTab->GetContent());
		Content->FocusOutlineTab();
	}
#endif
}

void FNiagaraSystemToolkit::OpenAttributeSpreadsheet()
{
	InvokeTab(FNiagaraSystemToolkitModeBase::DebugCacheSpreadsheetTabID);
}


void FNiagaraSystemToolkit::OnToggleBoundsSetFixedBounds_Emitters()
{
	FScopedTransaction Transaction(LOCTEXT("SetFixedBoundsEmitters", "Set Fixed Bounds (Emitters)"));

	SystemViewModel->UpdateEmitterFixedBounds();
}

void FNiagaraSystemToolkit::OnToggleBoundsSetFixedBounds_System()
{
	FScopedTransaction Transaction(LOCTEXT("SetFixedBoundsSystem", "Set Fixed Bounds (System)"));

	SystemViewModel->UpdateSystemFixedBounds();
}

bool FNiagaraSystemToolkit::SupportsStatPerformance() const
{
#if STATS
	return SystemViewModel->GetSystem().GetPerformanceStateScopeMode() != ENiagaraPerformanceStateScopeMode::None;
#else
	return false;
#endif
}


void FNiagaraSystemToolkit::ClearStatPerformance()
{
#if STATS
	SystemViewModel->ClearSystemStats();
	SystemViewModel->ClearEmitterStats();
#endif
}

void FNiagaraSystemToolkit::ToggleStatPerformance()
{
	const bool IsEnabled = IsStatPerformanceChecked();
	if (SystemViewModel.IsValid())
	{
		SystemViewModel->SetPerformanceStatsEnabled(!IsEnabled);

		const bool bCompileForEdit = !IsEnabled ? bProfileInEditMode : true;
		SystemViewModel->SetCompileForEdit(bCompileForEdit);
	}
	if (IsStatPerformanceGPUChecked() == IsEnabled)
	{
		ToggleStatPerformanceGPU();
	}
}

void FNiagaraSystemToolkit::ToggleStatPerformanceTypeAvg()
{
	SystemViewModel->StatEvaluationType = ENiagaraStatEvaluationType::Average;
}

void FNiagaraSystemToolkit::ToggleStatPerformanceTypeMax()
{
	SystemViewModel->StatEvaluationType = ENiagaraStatEvaluationType::Maximum;
}

bool FNiagaraSystemToolkit::IsStatPerformanceTypeAvg()
{
	return SystemViewModel->StatEvaluationType == ENiagaraStatEvaluationType::Average;
}

bool FNiagaraSystemToolkit::IsStatPerformanceTypeMax()
{
	return SystemViewModel->StatEvaluationType == ENiagaraStatEvaluationType::Maximum;
}

void FNiagaraSystemToolkit::ToggleStatPerformanceModePercent()
{
	SystemViewModel->StatDisplayMode = ENiagaraStatDisplayMode::Percent;
}

void FNiagaraSystemToolkit::ToggleStatPerformanceModeAbsolute()
{
	SystemViewModel->StatDisplayMode = ENiagaraStatDisplayMode::Absolute;
}

bool FNiagaraSystemToolkit::IsStatPerformanceModePercent()
{
	return SystemViewModel->StatDisplayMode == ENiagaraStatDisplayMode::Percent;
}

bool FNiagaraSystemToolkit::IsStatPerformanceModeAbsolute()
{
	return SystemViewModel->StatDisplayMode == ENiagaraStatDisplayMode::Absolute;
}

bool FNiagaraSystemToolkit::IsStatPerformanceChecked()
{
	return SystemViewModel.IsValid() ? SystemViewModel->IsPerformanceStatsEnabled() : false;
}

void FNiagaraSystemToolkit::ToggleStatPerformanceGPU()
{
#if WITH_NIAGARA_GPU_PROFILER
	if (GpuProfilerListener)
	{
		GpuProfilerListener->SetEnabled(!GpuProfilerListener->IsEnabled());
	}
#endif
}

bool FNiagaraSystemToolkit::IsStatPerformanceGPUChecked()
{
#if WITH_NIAGARA_GPU_PROFILER
	if (GpuProfilerListener)
	{
		return GpuProfilerListener->IsEnabled();
	}
#endif
	return false;
}

void FNiagaraSystemToolkit::ToggleProfileInEditMode()
{
	bProfileInEditMode = !bProfileInEditMode;

	if (SystemViewModel.IsValid() && IsStatPerformanceChecked())
	{
		SystemViewModel->SetCompileForEdit(bProfileInEditMode);
	}
}

bool FNiagaraSystemToolkit::IsProfileInEditMode() const
{
	return bProfileInEditMode;
}

void FNiagaraSystemToolkit::UpdateOriginalEmitter()
{
	checkf(SystemToolkitMode == ESystemToolkitMode::Emitter, TEXT("There is no original emitter to update in system mode."));

	TSharedPtr<FNiagaraEmitterViewModel> EditableEmitterViewModel = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterViewModel();
	FVersionedNiagaraEmitter EditableEmitter = EditableEmitterViewModel->GetEmitter();
	FVersionedNiagaraEmitterData* EditableEmitterData = EditableEmitter.GetEmitterData();
	UNiagaraEmitter* Source = Emitter;
	if (EditableEmitter.Emitter->GetChangeId() != LastSyncedEmitterChangeId)
	{
		if (EditableEmitter.Emitter->IsVersioningEnabled() && EditableEmitterData->Version <= EditableEmitter.Emitter->GetExposedVersion())
		{
			FSuppressableWarningDialog::FSetupInfo Info( 
				LOCTEXT("ApplyExposedVersionChangesPrompt", "You are about to apply changes to an already exposed asset version. Saving these changes will force-push them out to existing usages!\nConsider creating a new version instead to make those changes."), 
				LOCTEXT("ApplyExposedVersionChangesTitle", "Warning: editing exposed emitter version"), 
				TEXT("ApplyExposedEmitterChanges"));
			Info.ConfirmText = LOCTEXT("ApplyExposedVersionChanges_ConfirmText", "Apply Changes");
			Info.CancelText = LOCTEXT("ApplyExposedVersionChanges_CancelText", "Cancel");
			Info.CheckBoxText = LOCTEXT("ApplyExposedVersionChanges_CheckBoxText", "Don't Ask Again");

			if (FSuppressableWarningDialog(Info).ShowModal() == FSuppressableWarningDialog::EResult::Cancel)
			{
				return;
			}
		}
		
		const FScopedBusyCursor BusyCursor;
		const FText LocalizedScriptEditorApply = NSLOCTEXT("UnrealEd", "ToolTip_NiagaraEmitterEditorApply", "Apply changes to original emitter and its use in the world.");
		GWarn->BeginSlowTask(LocalizedScriptEditorApply, true);
		GWarn->StatusUpdate(1, 1, LocalizedScriptEditorApply);

		if (Source->IsSelected())
		{
			GEditor->GetSelectedObjects()->Deselect(Source);
		}

		ResetLoaders(Source->GetOutermost()); // Make sure that we're not going to get invalid version number linkers into the package we are going into. 

		TArray<UNiagaraScript*> AllScripts;
		EditableEmitterData->GetScripts(AllScripts, true);
		for (UNiagaraScript* Script : AllScripts)
		{
			checkfSlow(Script->AreScriptAndSourceSynchronized(), TEXT("Editable Emitter Script change ID is out of date when applying to Original Emitter!"));
		}
		Source->PreEditChange(nullptr);
		// overwrite the original script in place by constructing a new one with the same name
		Source = Cast<UNiagaraEmitter>(StaticDuplicateObject(EditableEmitter.Emitter, Source->GetOuter(), Source->GetFName(), RF_AllFlags, Source->GetClass()));

		// Restore RF_Standalone and RF_Public on the original emitter, as it had been removed from the preview emitter so that it could be GC'd.
		Source->SetFlags(RF_Standalone | RF_Public);

		Source->PostEditChange();

		TArray<UNiagaraScript*> EmitterScripts;
		Source->GetEmitterData(EditableEmitter.Version)->GetScripts(EmitterScripts, false);

		TArray<UNiagaraScript*> EditableEmitterScripts;
		EditableEmitterData->GetScripts(EditableEmitterScripts, false);

		// Validate that the change ids on the original emitters match the editable emitters ids to ensure the DDC contents are up to data without having to recompile.
		if (ensureMsgf(EmitterScripts.Num() == EditableEmitterScripts.Num(), TEXT("Script count mismatch after copying from editable emitter to original emitter.")))
		{
			for (UNiagaraScript* EmitterScript : EmitterScripts)
			{
				UNiagaraScript** MatchingEditableEmitterScriptPtr = EditableEmitterScripts.FindByPredicate([EmitterScript](UNiagaraScript* EditableEmitterScript) { 
					return EditableEmitterScript->GetUsage() == EmitterScript->GetUsage() && EditableEmitterScript->GetUsageId() == EmitterScript->GetUsageId(); });
				if (ensureMsgf(MatchingEditableEmitterScriptPtr != nullptr, TEXT("Matching script could not be found in editable emitter after copying to original emitter.")))
				{
					ensureMsgf((*MatchingEditableEmitterScriptPtr)->GetBaseChangeID() == EmitterScript->GetBaseChangeID(), TEXT("Script change ids didn't match after copying from editable emitter to original emitter."));
				}
			}
		}

		// Record the last synced change id to detect future changes.
		LastSyncedEmitterChangeId = EditableEmitter.Emitter->GetChangeId();
		bEmitterThumbnailUpdated = false;

		UpdateExistingEmitters();
		GWarn->EndSlowTask();
	}
	else if(bEmitterThumbnailUpdated)
	{
		Source->MarkPackageDirty();
		Source->ThumbnailImage = (UTexture2D*)StaticDuplicateObject(EditableEmitter.Emitter->ThumbnailImage, Source);
		Source->PostEditChange();
		bEmitterThumbnailUpdated = false;
	}
}

void MergeEmittersRecursively(const UNiagaraEmitter* ChangedEmitter, const TMap<UNiagaraEmitter*, TArray<UNiagaraEmitter*>>& EmitterToReferencingEmittersMap, TSet<UNiagaraEmitter*>& OutMergedEmitters)
{
	const TArray<UNiagaraEmitter*>* ReferencingEmitters = EmitterToReferencingEmittersMap.Find(ChangedEmitter);
	if (ReferencingEmitters != nullptr)
	{
		for (UNiagaraEmitter* ReferencingEmitter : (*ReferencingEmitters))
		{
			if (ReferencingEmitter->IsSynchronizedWithParent() == false)
			{
				ReferencingEmitter->MergeChangesFromParent();
				OutMergedEmitters.Add(ReferencingEmitter);
				MergeEmittersRecursively(ReferencingEmitter, EmitterToReferencingEmittersMap, OutMergedEmitters);
			}
		}
	}
}

void FNiagaraSystemToolkit::UpdateExistingEmitters()
{
	// Build a tree of references from the currently loaded emitters so that we can efficiently find all emitters that reference the modified emitter.
	TMap<UNiagaraEmitter*, TArray<UNiagaraEmitter*>> EmitterToReferencingEmittersMap;
	UNiagaraEmitter* EditableCopy = System->GetEmitterHandles()[0].GetInstance().Emitter;
	for (TObjectIterator<UNiagaraEmitter> EmitterIterator; EmitterIterator; ++EmitterIterator)
	{
		UNiagaraEmitter* LoadedEmitter = *EmitterIterator;
		if (LoadedEmitter != EditableCopy)
		{
			for (FNiagaraAssetVersion Version : LoadedEmitter->GetAllAvailableVersions())
			{
				FVersionedNiagaraEmitterData* EmitterData = LoadedEmitter->GetEmitterData(Version.VersionGuid);
				if (UNiagaraEmitter* ParentEmitter = EmitterData->GetParent().Emitter)
				{
					TArray<UNiagaraEmitter*>& ReferencingEmitters = EmitterToReferencingEmittersMap.FindOrAdd(ParentEmitter);
					ReferencingEmitters.AddUnique(LoadedEmitter);
				}
			}
		}
	}

	// Recursively merge emitters by traversing the reference chains.
	TSet<UNiagaraEmitter*> MergedEmitters;
	MergeEmittersRecursively(Emitter, EmitterToReferencingEmittersMap, MergedEmitters);

	// find referencing systems, aside from the system being edited by this toolkit and request that they recompile,
	// also refresh their view models, and reinitialize their components.
	for (TObjectIterator<UNiagaraSystem> SystemIterator; SystemIterator; ++SystemIterator)
	{
		UNiagaraSystem* LoadedSystem = *SystemIterator;
		if (LoadedSystem != System &&
			IsValid(LoadedSystem) && 
			LoadedSystem->HasAnyFlags(RF_ClassDefaultObject) == false)
		{
			bool bUsesMergedEmitterDirectly = false;
			for (const FNiagaraEmitterHandle& EmitterHandle : LoadedSystem->GetEmitterHandles())
			{
				if (MergedEmitters.Contains(EmitterHandle.GetInstance().Emitter))
				{
					bUsesMergedEmitterDirectly = true;
					break;
				}
			}

			if (bUsesMergedEmitterDirectly)
			{
				// Request that the system recompile.
				bool bForce = false;
				LoadedSystem->RequestCompile(bForce);

				// Invalidate any view models.
				TArray<TSharedPtr<FNiagaraSystemViewModel>> ReferencingSystemViewModels;
				FNiagaraSystemViewModel::GetAllViewModelsForObject(LoadedSystem, ReferencingSystemViewModels);
				for (TSharedPtr<FNiagaraSystemViewModel> ReferencingSystemViewModel : ReferencingSystemViewModels)
				{
					ReferencingSystemViewModel->RefreshAll();
				}

				// Reinit any running components
				for (TObjectIterator<UNiagaraComponent> ComponentIterator; ComponentIterator; ++ComponentIterator)
				{
					UNiagaraComponent* Component = *ComponentIterator;
					if (Component->GetAsset() == LoadedSystem)
					{
						Component->ReinitializeSystem();
					}
				}
			}
		}
	}
}

void FNiagaraSystemToolkit::GetSaveableObjects(TArray<UObject*>& OutObjects) const
{
	if (SystemToolkitMode == ESystemToolkitMode::Emitter)
	{
		OutObjects.Add(Emitter);
	}
	else
	{
		FAssetEditorToolkit::GetSaveableObjects(OutObjects);
	}
}

void FNiagaraSystemToolkit::SaveAsset_Execute()
{
	SystemViewModel->NotifyPreSave();
	if (SystemToolkitMode == ESystemToolkitMode::Emitter)
	{
		UE_LOGF(LogNiagaraEditor, Log, "Saving and Compiling NiagaraEmitter %ls", *GetEditingObjects()[0]->GetName());
		UpdateOriginalEmitter();
	}
	FAssetEditorToolkit::SaveAsset_Execute();
	SystemViewModel->NotifyPostSave();
}

void FNiagaraSystemToolkit::SaveAssetAs_Execute()
{
	SystemViewModel->NotifyPreSave();
	if (SystemToolkitMode == ESystemToolkitMode::Emitter)
	{
		UE_LOGF(LogNiagaraEditor, Log, "Saving and Compiling NiagaraEmitter %ls", *GetEditingObjects()[0]->GetName());
		UpdateOriginalEmitter();
	}
	FAssetEditorToolkit::SaveAssetAs_Execute();
	SystemViewModel->NotifyPostSave();
}

bool FNiagaraSystemToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	if (GbLogNiagaraSystemChanges > 0)
	{
		FString ExportText;
		SystemViewModel->DumpToText(ExportText);
		FString FilePath;

		if (SystemToolkitMode == ESystemToolkitMode::System)
		{
			FilePath = System->GetOutermost()->GetLoadedPath().GetPackageName();
		}
		else if (SystemToolkitMode == ESystemToolkitMode::Emitter)
		{
			FilePath = Emitter->GetOutermost()->GetLoadedPath().GetPackageName();
		}

		FString PathPart, FilenamePart, ExtensionPart;
		FPaths::Split(FilePath, PathPart, FilenamePart, ExtensionPart);

		FNiagaraEditorUtilities::WriteTextFileToDisk(FPaths::ProjectLogDir(), FilenamePart + TEXT(".onClose.txt"), ExportText, true);
	}

	SystemViewModel->NotifyPreClose();
	SystemViewModel->GetSystem().ReportAnalyticsData(false);

	bool bHasUnappliedScratchPadChanges = false;
	for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScratchPadViewModel : SystemViewModel->GetScriptScratchPadViewModel()->GetScriptViewModels())
	{
		if (ScratchPadViewModel->HasUnappliedChanges())
		{
			bHasUnappliedScratchPadChanges = true;
			break;
		}
	}

	if (InCloseReason != EAssetEditorCloseReason::AssetForceDeleted && bScratchPadChangesDiscarded == false && bHasUnappliedScratchPadChanges)
	{
		// find out the user wants to do with their dirty scratch pad scripts.
		EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(EAppMsgType::YesNoCancel,
			NSLOCTEXT("NiagaraEditor", "UnsavedScratchPadScriptsPrompt", "Would you like to apply changes to scratch pad scripts? (No will discard unapplied changes)"));

		switch (YesNoCancelReply)
		{
		case EAppReturnType::Yes:
			SystemViewModel->GetScriptScratchPadViewModel()->ApplyScratchPadChanges();
			break;
		case EAppReturnType::No:
			bScratchPadChangesDiscarded = true;
			break;
		case EAppReturnType::Cancel:
			return false;
		}
	}

	if (SystemToolkitMode == ESystemToolkitMode::Emitter && InCloseReason != EAssetEditorCloseReason::AssetForceDeleted)
	{
		TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterViewModel();
		if (bChangesDiscarded == false && (EmitterViewModel->GetEmitter().Emitter->GetChangeId() != LastSyncedEmitterChangeId || bEmitterThumbnailUpdated))
		{
			// find out the user wants to do with this dirty emitter.
			EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(EAppMsgType::YesNoCancel,
				FText::Format(
					NSLOCTEXT("UnrealEd", "Prompt_NiagaraEmitterEditorClose", "Would you like to apply changes to this Emitter to the original Emitter?\n{0}\n(No will lose all changes!)"),
					FText::FromString(Emitter->GetPathName())));

			// act on it
			switch (YesNoCancelReply)
			{
			case EAppReturnType::Yes:
				// update NiagaraScript and exit
				UpdateOriginalEmitter();
				break;
			case EAppReturnType::No:
				// Set the changes discarded to avoid showing the dialog multiple times when request close is called multiple times on shut down.
				bChangesDiscarded = true;
				break;
			case EAppReturnType::Cancel:
				// don't exit
				bScratchPadChangesDiscarded = false;
				return false;
			}
		}
		GEngine->ForceGarbageCollection(true);
		return true;
	}
	
	GEngine->ForceGarbageCollection(true);
	return FAssetEditorToolkit::OnRequestClose(InCloseReason);
}

void FNiagaraSystemToolkit::ToggleCompileEnabled()
{
	UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
	Settings->SetAutoCompile(!Settings->GetAutoCompile());
}

bool FNiagaraSystemToolkit::IsAutoCompileEnabled()
{
	return GetDefault<UNiagaraEditorSettings>()->GetAutoCompile();
}

void FNiagaraSystemToolkit::OnApply()
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_SystemToolkit_OnApply);
	UpdateOriginalEmitter();
}

bool FNiagaraSystemToolkit::OnApplyEnabled() const
{
	if (Emitter != nullptr)
	{
		TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterViewModel();
		return EmitterViewModel->GetEmitter().Emitter->GetChangeId() != LastSyncedEmitterChangeId || bEmitterThumbnailUpdated;
	}
	return false;
}

void FNiagaraSystemToolkit::OnApplyScratchPadChanges()
{
	if (SystemViewModel.IsValid() && SystemViewModel->GetScriptScratchPadViewModel() != nullptr)
	{
		SystemViewModel->GetScriptScratchPadViewModel()->ApplyScratchPadChanges();
	}
}

bool FNiagaraSystemToolkit::OnApplyScratchPadChangesEnabled() const
{
	return SystemViewModel.IsValid() && SystemViewModel->GetScriptScratchPadViewModel() != nullptr && SystemViewModel->GetScriptScratchPadViewModel()->HasUnappliedChanges();
}

void FNiagaraSystemToolkit::OnPinnedCurvesChanged()
{
	// does this work due to modes? @todo
	TabManager->TryInvokeTab(FNiagaraSystemToolkitModeBase::CurveEditorTabID);
}

void FNiagaraSystemToolkit::RefreshParameters()
{
	TArray<UObject*> NewParameterViewSelection;

	// Always display the system parameters
	NewParameterViewSelection.Add(&SystemViewModel->GetSystem());

	TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> EmitterHandlesToDisplay;
	EmitterHandlesToDisplay.Append(SystemViewModel->GetPinnedEmitterHandles());
		
	TArray<FGuid> SelectedEmitterHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		if (SelectedEmitterHandleIds.Contains(EmitterHandleViewModel->GetId()))
		{
			EmitterHandlesToDisplay.AddUnique(EmitterHandleViewModel);
		}
	}

	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleToDisplay : EmitterHandlesToDisplay)
	{
		if (EmitterHandleToDisplay->IsValid() && EmitterHandleToDisplay->GetEmitterViewModel()->GetEmitter().Emitter != nullptr)
		{
			NewParameterViewSelection.Add(EmitterHandleToDisplay->GetEmitterViewModel()->GetEmitter().Emitter);
		}
	}

	ObjectSelectionForParameterMapView->SetSelectedObjects(NewParameterViewSelection);
}

void FNiagaraSystemToolkit::OnSystemSelectionChanged()
{
	RefreshParameters();
}

void FNiagaraSystemToolkit::OnViewModelRequestFocusTab(FName TabName, bool bDrawAttention)
{
	TSharedPtr<SDockTab> DockTab = GetTabManager()->TryInvokeTab(TabName);

	if(DockTab.IsValid() && bDrawAttention)
	{
		DockTab->FlashTab();
	}
}


#undef LOCTEXT_NAMESPACE
