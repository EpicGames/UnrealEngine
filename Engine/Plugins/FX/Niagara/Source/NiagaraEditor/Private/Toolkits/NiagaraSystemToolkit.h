// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Toolkits/IToolkitHost.h"
#include "EditorUndoClient.h"
#include "NiagaraEditorModule.h"

#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"

#include "ViewModels/NiagaraEmitterViewModel.h"
#include "Widgets/SItemSelector.h"
#include "ViewModels/NiagaraSystemGraphSelectionViewModel.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

#if WITH_NIAGARA_GPU_PROFILER
	#include "NiagaraGPUProfilerInterface.h"
#endif

class SNiagaraEmitterVersionWidget;
class UNiagaraVersionMetaData;
class FNiagaraSystemInstance;
class FNiagaraSystemViewModel;
class FNiagaraObjectSelection;
class SNiagaraSystemEditorViewport;
class SNiagaraSystemEditorWidget;
class SNiagaraSystemViewport;
class SNiagaraSystemEditor;
class UNiagaraSystem;
class UNiagaraEmitter;
class UNiagaraSequence;
struct FAssetData;
class FMenuBuilder;
class ISequencer;
class FNiagaraEmitterHandleViewModel;
class FNiagaraMessageLogViewModel;
class FNiagaraSystemToolkitParameterPanelViewModel;
class FNiagaraSystemToolkitParameterDefinitionsPanelViewModel;
class FNiagaraScriptStatsViewModel;
class FNiagaraBakerViewModel;

/** Viewer/editor for a NiagaraSystem
*/
class FNiagaraSystemToolkit : public FWorkflowCentricApplication, public FGCObject, public FEditorUndoClient
{
private:
	enum class ESystemToolkitMode
	{
		System,
		Emitter,
	};

	enum class ESystemToolkitWorkflowMode
	{
		Default,
		Scalability
	};

public:
	/** Edits the specified Niagara System */
	void InitializeWithSystem(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraSystem& InSystem);

	/** Edits the specified Niagara Emitter */
	void InitializeWithEmitter(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UNiagaraEmitter& InEmitter);

	/** Destructor */
	virtual ~FNiagaraSystemToolkit() override;
	
	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Interface

		//~ Begin FEditorUndoClient Interface
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraSystemToolkit");
	}

	FSlateIcon GetCompileStatusImage() const;
	FText GetCompileStatusTooltip() const;

	/** Compiles the system script. */
	void CompileSystem(bool bFullRebuild);

	TSharedPtr<FNiagaraSystemViewModel> GetSystemViewModel() const;
	TSharedPtr<FNiagaraSystemGraphSelectionViewModel> GetSystemGraphSelectionViewModel();

	TSharedPtr<FNiagaraSimCacheViewModel> GetSimCacheViewModel();
	
	// @todo This is a hack for now until we reconcile the default toolbar with application modes [duplicated from counterpart in Blueprint Editor]
	void RegisterToolbarTab(const TSharedRef<class FTabManager>& TabManager);

	TSharedPtr<SWidget> GetSystemOverview() const;
	void SetSystemOverview(const TSharedPtr<SWidget>& SystemOverview);
	TSharedPtr<SWidget> GetScriptScratchpadManager() const;
	void SetScriptScratchpad(const TSharedPtr<SWidget>& ScriptScratchpad);
	TSharedPtr<SWidget> GetVersioningWidget() const { return VersionsWidget; }
	FText GetVersionButtonLabel() const;
	TArray<FNiagaraAssetVersion> GetEmitterVersions() const;
	TSharedRef<SWidget> GenerateVersioningDropdownMenu(TSharedRef<FUICommandList> InCommandList);
	void SwitchToVersion(FGuid VersionGuid);
	bool IsVersionSelected(FNiagaraAssetVersion Version) const;
	FText GetVersionMenuLabel(FNiagaraAssetVersion Version) const;

	void SetScriptScratchpadManager(const TSharedPtr<SWidget>& ScriptScratchpad);
	
	/** Mode exposed functions */
	TSharedRef<SWidget> GenerateCompileMenuContent();
	TSharedRef<SWidget> GenerateBoundsMenuContent(TSharedRef<FUICommandList> InCommandList);
	TSharedRef<SWidget> GenerateStatConfigMenuContent(TSharedRef<FUICommandList> InCommandList);

	bool HasEmitter() const { return SystemToolkitMode == ESystemToolkitMode::Emitter && Emitter != nullptr; }

	bool HasSystem() const { return SystemToolkitMode == ESystemToolkitMode::System && System != nullptr; }

	FAssetData GetEditedAsset() const; 

	const TArray<UObject*>& GetObjectsBeingEdited() const;

	
public:
	FRefreshItemSelectorDelegate RefreshItemSelector;

protected:
	void OnToggleBounds();
	bool IsToggleBoundsChecked() const;
	void OnToggleBoundsSetFixedBounds_Emitters();
	void OnToggleBoundsSetFixedBounds_System();

	bool SupportsStatPerformance() const;
	void ClearStatPerformance();
	void ToggleStatPerformance();
	bool IsStatPerformanceChecked();
	void ToggleProfileInEditMode();
	bool IsProfileInEditMode() const;
	void ToggleStatPerformanceGPU();
	bool IsStatPerformanceGPUChecked();
	void ToggleStatPerformanceTypeAvg();
	void ToggleStatPerformanceTypeMax();
	bool IsStatPerformanceTypeAvg();
	bool IsStatPerformanceTypeMax();
	void ToggleStatPerformanceModePercent();
	void ToggleStatPerformanceModeAbsolute();
	bool IsStatPerformanceModePercent();
	bool IsStatPerformanceModeAbsolute();

	void ToggleDrawOption(int32 Element);
	bool IsDrawOptionEnabled(int32 Element) const;

	bool CanShowEmitterBounds() const;
	bool IsShowEmitterBounds(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterViewModel) const;
	void ToggleShowEmitterBounds(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterViewModel);

	void OpenDebugHUD();
	void OpenDebugOutliner();
	void OpenAttributeSpreadsheet();

	//~ FAssetEditorToolkit interface
	virtual void GetSaveableObjects(TArray<UObject*>& OutObjects) const override;
	virtual void SaveAsset_Execute() override;
	virtual void SaveAssetAs_Execute() override;
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	
private:
	void InitializeInternal(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, const FGuid& MessageLogGuid);
	void InitializeRapidIterationParameters(const FVersionedNiagaraEmitter& VersionedEmitter);

	void UpdateOriginalEmitter();
	void UpdateExistingEmitters();
	
	void SetupCommands();
	void LinkCommandLists();

	void ResetSimulation();

	static void ToggleCompileEnabled();
	static bool IsAutoCompileEnabled();
	
	void OnApply();
	bool OnApplyEnabled() const;

	void OnApplyScratchPadChanges();
	bool OnApplyScratchPadChangesEnabled() const;

	void OnPinnedCurvesChanged();
	void RefreshParameters();
	void OnSystemSelectionChanged();
	void OnViewModelRequestFocusTab(FName TabName, bool bDrawAttention = false);
	
	const FName GetNiagaraSystemMessageLogName(UNiagaraSystem* InSystem) const;

	void CaptureAssetThumbnail() const;
	void CaptureEmitterThumbnail(FGuid EmitterGuid);
	void OnThumbnailCaptured(UTexture2D* Thumbnail, TOptional<FGuid> EmitterGuid);

	void ManageVersions();
	TSharedPtr<FNiagaraEmitterViewModel> GetEditedEmitterViewModel() const;

	void OpenAddEmitterMenu();
	bool CanAddEmitters() const;
	void GetSequencerAddMenuContent(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> Sequencer);

private:
	/** The System being edited in system mode, or the placeholder system being edited in emitter mode. */
	TObjectPtr<UNiagaraSystem> System = nullptr;

	/** The emitter being edited in emitter mode, or null when editing in system mode. Note that we are NOT using the FVersionedEmitter here, because this is the full asset. The version being edited is defined by the emitter view model. */
	UNiagaraEmitter* Emitter = nullptr;

	/** Temp object to hold version data being edited in the versioning widget's property editor. */
	TObjectPtr<UNiagaraVersionMetaData> VersionMetadata = nullptr;

	/** The value of the emitter change id from the last time it was in sync with the original emitter. */
	FGuid LastSyncedEmitterChangeId;

	/** The graphs being undone/redone currently so we only mark for compile the right ones */
	mutable TArray<TWeakObjectPtr<UNiagaraGraph>> LastUndoGraphs;

	/** Whether or not the emitter thumbnail has been updated.  The is needed because after the first update the
		screenshot uobject is reused, so a pointer comparison doesn't work to checking if the images has been updated. */
	bool bEmitterThumbnailUpdated;

	bool bProfileInEditMode = false;

	ESystemToolkitMode SystemToolkitMode;

	/** The currently active workflow mode */
	ESystemToolkitWorkflowMode ActiveWorkflowMode;
	
	TSharedPtr<SNiagaraSystemViewport> Viewport;
	
	/* The view model for the System being edited */
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
	
	/** Message log, with the log listing that it reflects */
	TSharedPtr<FNiagaraMessageLogViewModel> NiagaraMessageLogViewModel;
	TSharedPtr<class SWidget> NiagaraMessageLog;

	/** The command list for this editor */
	TSharedPtr<FUICommandList> EditorCommands;

	/** Display for script stats on selected platforms */
	TSharedPtr<FNiagaraScriptStatsViewModel> ScriptStats;

	/** Baker preview */
	TSharedPtr<FNiagaraBakerViewModel> BakerViewModel;

	TSharedPtr<FNiagaraSystemToolkitParameterPanelViewModel> ParameterPanelViewModel;
	TSharedPtr<FNiagaraSystemToolkitParameterDefinitionsPanelViewModel> ParameterDefinitionsPanelViewModel;
	TSharedPtr<class SNiagaraParameterPanel> ParameterPanel;

	/** Sim Cache editor view model**/
	TSharedPtr<FNiagaraSimCacheViewModel> SimCacheViewModel;

	TSharedPtr<FNiagaraObjectSelection> ObjectSelectionForParameterMapView;

	bool bChangesDiscarded;
	bool bScratchPadChangesDiscarded;

	TSharedPtr<SWidget> SystemOverview;
	TSharedPtr<SWidget> VersionsWidget;
	TSharedPtr<SWidget> ScriptScratchpadManager;
	
	static IConsoleVariable* VmStatEnabledVar;

#if WITH_NIAGARA_GPU_PROFILER
	TUniquePtr<FNiagaraGpuProfilerListener> GpuProfilerListener;
#endif
	static IConsoleVariable* GpuStatEnabledVar;
	
	friend class FNiagaraSystemToolkitModeBase;
};
