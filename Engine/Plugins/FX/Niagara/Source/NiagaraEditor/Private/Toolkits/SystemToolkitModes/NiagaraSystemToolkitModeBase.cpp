// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"

#include "AdvancedPreviewSceneModule.h"
#include "ClassIconFinder.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraNodeAssignment.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "NiagaraScriptSource.h"
#include "ViewModels/NiagaraScriptStatsViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/NiagaraParameterDefinitionsPanelViewModel.h"
#include "Widgets/SNiagaraSystemViewport.h"
#include "Widgets/SNiagaraSystemScript.h"
#include "Widgets/SNiagaraParameterPanel.h"
#include "Widgets/SNiagaraParameterDefinitionsPanel.h"
#include "Widgets/SNiagaraScriptGraph.h"
#include "Widgets/SNiagaraSelectedObjectsDetails.h"
#include "Widgets/SNiagaraGeneratedCodeView.h"
#include "Widgets/SDataHierarchyEditor.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "Widgets/SNiagaraSelectedObjectsDetails.h"
#include "Widgets/SNiagaraHierarchyModuleInput.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraSimulationStageBase.h"
#include "Widgets/SNiagaraDebugCaptureView.h"
#include "Widgets/SNiagaraSimCacheView.h"
#include "Widgets/SNiagaraSimCacheViewTimeline.h"
#include "Widgets/SNiagaraSimCacheViewTransportControls.h"
#include "Widgets/SNiagaraSystemUserParameters.h"
#include "Customizations/NiagaraComponentDetails.h"
#include "Styling/StyleColors.h"
#include "ViewModels/NiagaraBakerViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraSystemEditorDocumentsViewModel.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"
#include "ViewModels/HierarchyEditor/NiagaraUserParametersHierarchyViewModel.h"
#include "Widgets/SNiagaraHierarchyAssignment.h"

#define LOCTEXT_NAMESPACE "NiagaraSystemToolkitModeBase"

const FName FNiagaraSystemToolkitModeBase::DefaultModeName(TEXT("NiagaraSystemToolkitMode_Default"));
const FName FNiagaraSystemToolkitModeBase::ScalabilityModeName(TEXT("NiagaraSystemToolkitMode_Scalability"));

const FName FNiagaraSystemToolkitModeBase::ViewportTabID(TEXT("NiagaraSystemEditor_Viewport"));
const FName FNiagaraSystemToolkitModeBase::CurveEditorTabID(TEXT("NiagaraSystemEditor_CurveEditor"));
const FName FNiagaraSystemToolkitModeBase::SequencerTabID(TEXT("NiagaraSystemEditor_Sequencer"));
const FName FNiagaraSystemToolkitModeBase::SystemScriptTabID(TEXT("NiagaraSystemEditor_SystemScript"));
const FName FNiagaraSystemToolkitModeBase::SystemParametersTabID(TEXT("NiagaraSystemEditor_SystemParameters"));
const FName FNiagaraSystemToolkitModeBase::SystemParameterDefinitionsTabID(TEXT("NiagaraSystemEditor_SystemParameterDefinitions"));
const FName FNiagaraSystemToolkitModeBase::DetailsTabID(TEXT("NiagaraSystemEditor_Details"));
const FName FNiagaraSystemToolkitModeBase::SelectedEmitterGraphTabID(TEXT("NiagaraSystemEditor_SelectedEmitterGraph"));
const FName FNiagaraSystemToolkitModeBase::DebugCacheSpreadsheetTabID(TEXT("NiagaraSystemEditor_DebugCacheSpreadsheet"));
const FName FNiagaraSystemToolkitModeBase::PreviewSettingsTabId(TEXT("NiagaraSystemEditor_PreviewSettings"));
const FName FNiagaraSystemToolkitModeBase::GeneratedCodeTabID(TEXT("NiagaraSystemEditor_GeneratedCode"));
const FName FNiagaraSystemToolkitModeBase::MessageLogTabID(TEXT("NiagaraSystemEditor_MessageLog"));
const FName FNiagaraSystemToolkitModeBase::SystemOverviewTabID(TEXT("NiagaraSystemEditor_SystemOverview"));
const FName FNiagaraSystemToolkitModeBase::ScriptStatsTabID(TEXT("NiagaraSystemEditor_ScriptStats"));
const FName FNiagaraSystemToolkitModeBase::BakerTabID(TEXT("NiagaraSystemEditor_Baker"));
const FName FNiagaraSystemToolkitModeBase::VersioningTabID(TEXT("NiagaraSystemEditor_Versioning"));
const FName FNiagaraSystemToolkitModeBase::ScratchPadScriptsTabID(TEXT("NiagaraSystemEditor_ScratchPadScripts"));
const FName FNiagaraSystemToolkitModeBase::UserParametersTabID(TEXT("NiagaraSystemEditor_UserParameters"));
const FName FNiagaraSystemToolkitModeBase::UserParametersHierarchyTabID(TEXT("NiagaraSystemEditor_UserParametersHierarchy"));
const FName FNiagaraSystemToolkitModeBase::EmitterSummaryViewEditorTabID(TEXT("NiagaraSystemEditor_SummaryViewHierarchyEditor"));
const FName FNiagaraSystemToolkitModeBase::ScratchPadHierarchyEditorTabID(TEXT("NiagaraSystemEditor_ScratchPadHierarchyEditor"));

FNiagaraSystemToolkitModeBase::FNiagaraSystemToolkitModeBase(FName InModeName, TWeakPtr<FNiagaraSystemToolkit> InSystemToolkit) : FApplicationMode(InModeName), SystemToolkit(InSystemToolkit), SwitcherIdx(0)
{
	DocChangedHandle = InSystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->OnActiveDocumentChanged().AddRaw(this, &FNiagaraSystemToolkitModeBase::OnActiveDocumentChanged);
	ObjectSelection = MakeShared<FNiagaraObjectSelection>();
	LayoutExtender = MakeShared<FLayoutExtender>();
}

FNiagaraSystemToolkitModeBase::~FNiagaraSystemToolkitModeBase()
{
	if (SystemToolkit.Pin().IsValid())
	{
		SystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->OnActiveDocumentChanged().Remove(DocChangedHandle);
		
		if(UpdateSummaryViewHandle.IsValid())
		{
			SystemToolkit.Pin()->GetSystemViewModel()->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().Remove(UpdateSummaryViewHandle);
		}

		if(UpdateScratchPadScriptHierarchyHandle.IsValid())
		{
			SystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->OnActiveDocumentChanged().Remove(UpdateScratchPadScriptHierarchyHandle);
		}
	}
}

void FNiagaraSystemToolkitModeBase::OnActiveDocumentChanged(TSharedPtr<SDockTab> NewActiveTab)
{
	TSharedPtr<FNiagaraSystemToolkit> Toolkit = StaticCastSharedPtr<FNiagaraSystemToolkit>(SystemToolkit.Pin());

	SwitcherIdx = 0;
	TSharedPtr<SDockTab>  ActiveTab = Toolkit->GetSystemViewModel()->GetDocumentViewModel()->GetActiveDocumentTab().Pin();
	// active tab might be nullptr due to a scratch pad tab closing. In that case, the details panel should update to clear the selected objects
	UpdateSelectionForActiveDocument();
	if (ActiveTab.IsValid())
	{
		if (ActiveTab->GetLayoutIdentifier().TabType == SystemOverviewTabID)
		{
			SwitcherIdx = 0;
		}
		else
		{
			SwitcherIdx = 1;
		}

		TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchScriptVM = Toolkit->GetSystemViewModel()->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
		if (ScratchScriptVM.IsValid())
		{
			LastSelectionUpdateDelegate = ScratchScriptVM->GetGraphViewModel()->GetNodeSelection()->OnSelectedObjectsChanged().AddSP(this, &FNiagaraSystemToolkitModeBase::UpdateSelectionForActiveDocument);
			if (INiagaraParameterPanelViewModel* ParameterPanelViewModel = Toolkit->GetSystemViewModel()->GetParameterPanelViewModel())
			{
				LastParamPanelSelectionUpdateDelegate = ParameterPanelViewModel->GetVariableObjectSelection()->OnSelectedObjectsChanged().AddSP(this, &FNiagaraSystemToolkitModeBase::OnParameterPanelViewModelExternalSelectionChanged);
			}
			LastSystemSelectionUpdateDelegate = Toolkit->GetSystemViewModel()->GetSelectionViewModel()->OnEntrySelectionChanged().AddSP(this, &FNiagaraSystemToolkitModeBase::OnSystemSelectionChanged);
			LastGraphEditDelegate = ScratchScriptVM->GetGraphViewModel()->GetGraph()->AddOnGraphNeedsRecompileHandler(FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraSystemToolkitModeBase::OnEditedScriptGraphChanged));
		}
	}
}

void FNiagaraSystemToolkitModeBase::OnEditedScriptGraphChanged(const FEdGraphEditAction& InAction)
{
	TSharedPtr<FNiagaraSystemToolkit> Toolkit = StaticCastSharedPtr<FNiagaraSystemToolkit>(SystemToolkit.Pin());
	// We need to update the parameter panel view model with new parameters potentially
	if (Toolkit->GetSystemViewModel()->GetParameterPanelViewModel())
		Toolkit->GetSystemViewModel()->GetParameterPanelViewModel()->RefreshFullNextTick(true);
}

void FNiagaraSystemToolkitModeBase::OnParameterPanelViewModelExternalSelectionChanged()
{
	TSharedPtr<FNiagaraSystemToolkit> Toolkit = StaticCastSharedPtr<FNiagaraSystemToolkit>(SystemToolkit.Pin());
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchScriptVM = Toolkit->GetSystemViewModel()->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
	if (ScratchScriptVM.IsValid())
	{
		TSharedPtr<FNiagaraObjectSelection> ParamSel = Toolkit->GetSystemViewModel()->GetParameterPanelViewModel()->GetVariableObjectSelection();
		if (ParamSel.IsValid())
		{
			TArray<UObject*> SelectedNodes = ParamSel->GetSelectedObjectsResolved().Array();
			if (SelectedNodes.Num() != 0)
			{
				TSet<UObject*> CurrentlySelectedObjects = ObjectSelection->GetSelectedObjectsResolved();
				if(CurrentlySelectedObjects.Array() != SelectedNodes)
				{
					ObjectSelection->SetSelectedObject(SelectedNodes[0]);
					ObjectSelectionSubHeaderText = LOCTEXT("ParamPanelSel", "Parameter Panel Selection");
				}
			}
		}
	}
}

void FNiagaraSystemToolkitModeBase::OnSystemSelectionChanged()
{
	// Do nothing for now
}

void FNiagaraSystemToolkitModeBase::PostActivateMode()
{
	// by default we want to close the user parameters hierarchy tab if it has been open before.
	// This fixes the issue of an empty tab being summoned from a system's cached layout when applied to an emitter.
	// for consistency, we close it not only for emitters but for systems too
	if(TSharedPtr<SDockTab> UserParametersHierarchyDockTab = SystemToolkit.Pin()->GetTabManager()->FindExistingLiveTab(UserParametersHierarchyTabID))
	{
		UserParametersHierarchyDockTab->RequestCloseTab();
	}

	if(TSharedPtr<SDockTab> ScratchPadScriptHierarchyDockTab = SystemToolkit.Pin()->GetTabManager()->FindExistingLiveTab(ScratchPadHierarchyEditorTabID))
	{
		ScratchPadScriptHierarchyDockTab->RequestCloseTab();
	}
}

void FNiagaraSystemToolkitModeBase::UpdateSelectionForActiveDocument()
{
	TSharedPtr<FNiagaraSystemToolkit> Toolkit = StaticCastSharedPtr<FNiagaraSystemToolkit>(SystemToolkit.Pin());
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchScriptVM = Toolkit->GetSystemViewModel()->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
	if (ScratchScriptVM.IsValid())
	{
		TArray<UObject*> SelectedNodes = ScratchScriptVM->GetGraphViewModel()->GetNodeSelection()->GetSelectedObjectsResolved().Array();
		if (SelectedNodes.Num() == 0)
		{
			if (ScratchScriptVM)
			{
				ObjectSelection->SetSelectedObject(ScratchScriptVM->GetEditScript().Script);
				ObjectSelectionSubHeaderText = ScratchScriptVM->GetDisplayName();
			}
			else
			{
				ObjectSelection->ClearSelectedObjects();
				ObjectSelectionSubHeaderText = LOCTEXT("InvalidSelection","Missing selection");
			}
		}
		else if (SelectedNodes.Num() == 1)
		{
			UEdGraphNode* SelectedGraphNode = CastChecked<UEdGraphNode>(SelectedNodes[0]);
			ObjectSelectionSubHeaderText = SelectedGraphNode->GetNodeTitle(ENodeTitleType::ListView);
			ObjectSelection->SetSelectedObject(SelectedGraphNode);
		}
		else
		{
			ObjectSelection->SetSelectedObjects(SelectedNodes);
			ObjectSelectionSubHeaderText = LOCTEXT("MultipleNodesSelection", "Multiple items selected");
		}
	}
	else
	{
		ObjectSelection->ClearSelectedObjects();
		ObjectSelectionSubHeaderText = LOCTEXT("EmptySelection", "Nothing selected");
	}

	// We need to update the parameter panel view model with new parameters potentially
	if (Toolkit->GetSystemViewModel()->GetParameterPanelViewModel())
		Toolkit->GetSystemViewModel()->GetParameterPanelViewModel()->RefreshDueToActiveDocumentChanged();
}

void FNiagaraSystemToolkitModeBase::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_NiagaraSystemEditor", "Niagara System"));

	TSharedPtr<FNiagaraSystemToolkit> PinnedSystemToolkit = SystemToolkit.Pin();
	PinnedSystemToolkit->RegisterToolbarTab(InTabManager.ToSharedRef());
	
	InTabManager->RegisterTabSpawner(ViewportTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("Preview", "Preview"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Viewport"));

	InTabManager->RegisterTabSpawner(CurveEditorTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_CurveEd))
		.SetDisplayName(LOCTEXT("Curves", "Curves"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Curves"));

	InTabManager->RegisterTabSpawner(SequencerTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_Sequencer))
		.SetDisplayName(LOCTEXT("Timeline", "Timeline"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Timeline"));

	InTabManager->RegisterTabSpawner(SystemScriptTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_SystemScript))
		.SetDisplayName(LOCTEXT("SystemScript", "System Script"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetAutoGenerateMenuEntry(GbShowNiagaraDeveloperWindows != 0);

	InTabManager->RegisterTabSpawner(SystemParametersTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_SystemParameters))
		.SetDisplayName(LOCTEXT("SystemParameters", "Parameters"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Parameters"));

//@todo(ng) disable parameter definitions panel pending bug fixes
// 	InTabManager->RegisterTabSpawner(SystemParameterDefinitionsTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkit::SpawnTab_SystemParameterDefinitions)) 
// 		.SetDisplayName(LOCTEXT("SystemParameterDefinitions", "Parameter Definitions"))
// 		.SetGroup(WorkspaceMenuCategory.ToSharedRef());

	InTabManager->RegisterTabSpawner(DetailsTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Details"));

	InTabManager->RegisterTabSpawner(SelectedEmitterGraphTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_SelectedEmitterGraph))
		.SetDisplayName(LOCTEXT("SelectedEmitterGraph", "Selected Emitter Graph"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetAutoGenerateMenuEntry(GbShowNiagaraDeveloperWindows != 0);

	InTabManager->RegisterTabSpawner(DebugCacheSpreadsheetTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_DebugCacheSpreadsheet))
		.SetDisplayName(LOCTEXT("DebugSpreadshseet", "Attribute Spreadsheet"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Spreadsheet"));

	InTabManager->RegisterTabSpawner(PreviewSettingsTabId, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_PreviewSettings))
		.SetDisplayName(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Settings"));

	InTabManager->RegisterTabSpawner(GeneratedCodeTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_GeneratedCode))
		.SetDisplayName(LOCTEXT("GeneratedCode", "Generated Code"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.GeneratedCode"));

	InTabManager->RegisterTabSpawner(MessageLogTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_MessageLog))
		.SetDisplayName(LOCTEXT("NiagaraMessageLog", "Niagara Log"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.Log"));

	InTabManager->RegisterTabSpawner(SystemOverviewTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_SystemOverview))
		.SetDisplayName(LOCTEXT("SystemOverviewTabName", "System Overview"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.SystemOverview"));

	InTabManager->RegisterTabSpawner(ScratchPadScriptsTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_ScratchPadScripts))
		.SetDisplayName(LOCTEXT("ScratchPadModulesTabName", "Local Modules"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.ScratchPad"));

	InTabManager->RegisterTabSpawner(ScriptStatsTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_ScriptStats))
		.SetDisplayName(LOCTEXT("NiagaraScriptsStatsTab", "Script Stats"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.ScriptStats"));

	InTabManager->RegisterTabSpawner(BakerTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_Baker))
		.SetDisplayName(LOCTEXT("NiagaraBakerTab", "Baker"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.BakerIcon"));

	InTabManager->RegisterTabSpawner(VersioningTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_Versioning))
		.SetDisplayName(LOCTEXT("VersioningTab", "Versioning"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Versions"));
	
	InTabManager->RegisterTabSpawner(UserParametersTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_UserParameters))
		.SetDisplayName(LOCTEXT("UserParametersTab", "User Parameters"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.UserParameters"));
	
	InTabManager->RegisterTabSpawner(UserParametersHierarchyTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_UserParametersHierarchyEditor))
		.SetDisplayName(LOCTEXT("UserParametersHierarchyTab", "User Parameters Hierarchy"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.UserParameterHierarchy"));
		
	InTabManager->RegisterTabSpawner(EmitterSummaryViewEditorTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_SummaryViewEditor))
		.SetDisplayName(LOCTEXT("SummaryViewEditorTitle", "Edit Summary View"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.UserParameterHierarchy"));

	InTabManager->RegisterTabSpawner(ScratchPadHierarchyEditorTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitModeBase::SpawnTab_ScratchPadHierarchyEditor))
		.SetDisplayName(LOCTEXT("ScratchPadHierarchyEditor", "Edit Scratch Pad Hierarchy"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.UserParameterHierarchy"));

	TSharedPtr<FNiagaraSystemToolkit> SystemToolkitPinned = SystemToolkit.Pin();
	if (SystemToolkitPinned.IsValid())
	{
		const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
		NiagaraEditorSettings->GetExternalTabSpawnersForAsset(SystemToolkitPinned.ToSharedRef(), SystemToolkitPinned->GetSystemViewModel()->GetSystem(), ExternalTabSpawners);
		if(ExternalTabSpawners.Num() > 0)
		{
			for (TSharedRef<FWorkflowTabFactory>& ExternalTabSpawner : ExternalTabSpawners)
			{
				ExternalTabSpawner->RegisterTabSpawner(InTabManager.ToSharedRef(), this);
			}
		}
	}

	RegisterTabFactoriesWithAppAndManager(PinnedSystemToolkit.Get(), InTabManager.ToSharedRef());
}

int FNiagaraSystemToolkitModeBase::GetActiveSelectionDetailsIndex() const
{
	return SwitcherIdx;
}


class SNiagaraSelectedEmitterGraph : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSelectedEmitterGraph)
	{}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
	{
		SystemViewModel = InSystemViewModel;
		SystemViewModel->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().AddSP(this, &SNiagaraSelectedEmitterGraph::SystemSelectionChanged);
		ChildSlot
		[
			SAssignNew(GraphWidgetContainer, SBox)
		];
		UpdateGraphWidget();
	}

	virtual ~SNiagaraSelectedEmitterGraph() override
	{
		if (SystemViewModel.IsValid() && SystemViewModel->GetSelectionViewModel())
		{
			SystemViewModel->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().RemoveAll(this);
		}
	}

private:
	void SystemSelectionChanged()
	{
		UpdateGraphWidget();
	}

	void UpdateGraphWidget()
	{
		TArray<FGuid> SelectedEmitterHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();
		if (SelectedEmitterHandleIds.Num() == 1)
		{
			TSharedPtr<FNiagaraEmitterHandleViewModel> SelectedEmitterHandle = SystemViewModel->GetEmitterHandleViewModelById(SelectedEmitterHandleIds[0]);
			TSharedRef<SWidget> EmitterWidget = 
				SNew(SSplitter)
				+ SSplitter::Slot()
				.Value(.25f)
				[
					SNew(SNiagaraSelectedObjectsDetails, SelectedEmitterHandle->GetEmitterViewModel()->GetSharedScriptViewModel()->GetGraphViewModel()->GetNodeSelection())
				]
				+ SSplitter::Slot()
				.Value(.75f)
				[
					SNew(SNiagaraScriptGraph, SelectedEmitterHandle->GetEmitterViewModel()->GetSharedScriptViewModel()->GetGraphViewModel())
				];

			FVersionedNiagaraEmitterData* LastMergedEmitter = SelectedEmitterHandle->GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetParentAtLastMerge().GetEmitterData();
			if (LastMergedEmitter != nullptr)
			{
				UNiagaraScriptSource* LastMergedScriptSource = CastChecked<UNiagaraScriptSource>(LastMergedEmitter->GraphSource);
				bool bIsForDataProcessingOnly = false;
				TSharedRef<FNiagaraScriptGraphViewModel> LastMergedScriptGraphViewModel = MakeShared<FNiagaraScriptGraphViewModel>(FText(), bIsForDataProcessingOnly);
				LastMergedScriptGraphViewModel->SetScriptSource(LastMergedScriptSource);
				TSharedRef<SWidget> LastMergedEmitterWidget = 
					SNew(SSplitter)
					+ SSplitter::Slot()
					.Value(.25f)
					[
						SNew(SNiagaraSelectedObjectsDetails, LastMergedScriptGraphViewModel->GetNodeSelection())
					]
					+ SSplitter::Slot()
					.Value(.75f)
					[
						SNew(SNiagaraScriptGraph, LastMergedScriptGraphViewModel)
					];

				GraphWidgetContainer->SetContent
				(
					SNew(SSplitter)
					.Orientation(Orient_Vertical)
					+ SSplitter::Slot()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
							.Text(FText::FromString(TEXT("Emitter")))
						]
						+ SVerticalBox::Slot()
						[
							EmitterWidget
						]
					]
					+ SSplitter::Slot()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
							.Text(FText::FromString(TEXT("Last Merged Emitter")))
						]
						+ SVerticalBox::Slot()
						[
							LastMergedEmitterWidget
						]
					]
				);
			}
			else
			{
				GraphWidgetContainer->SetContent(EmitterWidget);
			}
		}
		else
		{
			GraphWidgetContainer->SetContent(SNullWidget::NullWidget);
		}
	}

private:
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
	TSharedPtr<SBox> GraphWidgetContainer;
};

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == ViewportTabID);

	TSharedPtr<FNiagaraSystemToolkit> Toolkit = StaticCastSharedPtr<FNiagaraSystemToolkit>(SystemToolkit.Pin());

	if(!Toolkit->Viewport.IsValid())
	{
		Toolkit->Viewport = SNew(SNiagaraSystemViewport, Toolkit->GetSystemViewModel().ToSharedRef())
			.OnThumbnailCaptured(Toolkit.Get(), &FNiagaraSystemToolkit::OnThumbnailCaptured)
			.Sequencer(Toolkit->GetSystemViewModel()->GetSequencer())
			.AssetEditorToolkit(SystemToolkit);
	}

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			Toolkit->Viewport.ToSharedRef()
		];

	Toolkit->Viewport->SetPreviewComponent(Toolkit->GetSystemViewModel()->GetPreviewComponent());
	Toolkit->Viewport->OnAddedToTab(SpawnedTab);

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_PreviewSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PreviewSettingsTabId);

	TSharedRef<SWidget> InWidget = SNullWidget::NullWidget;
	if (SystemToolkit.Pin()->Viewport.IsValid())
	{
		FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
		InWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(SystemToolkit.Pin()->Viewport->GetPreviewScene());
	}

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		[
			InWidget
		];

	return SpawnedTab;
}


TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_CurveEd(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == CurveEditorTabID);

	TSharedPtr<FNiagaraSystemToolkit> Toolkit = StaticCastSharedPtr<FNiagaraSystemToolkit>(SystemToolkit.Pin());

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			FNiagaraEditorModule::Get().GetWidgetProvider()->CreateCurveOverview(SystemToolkit.Pin()->GetSystemViewModel().ToSharedRef())
		];

	return SpawnedTab;
}


TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_Sequencer(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == SequencerTabID);

	TSharedPtr<FNiagaraSystemToolkit> Toolkit = StaticCastSharedPtr<FNiagaraSystemToolkit>(SystemToolkit.Pin());

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			Toolkit->GetSystemViewModel()->GetSequencer()->GetSequencerWidget()
		];

	return SpawnedTab;
}


TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_SystemScript(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == SystemScriptTabID);

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			SNew(SNiagaraSystemScript, SystemToolkit.Pin()->GetSystemViewModel().ToSharedRef())
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_SystemParameters(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == SystemParametersTabID);


	TArray<TSharedRef<FNiagaraObjectSelection>> ObjectSelections;
	ObjectSelections.Add(SystemToolkit.Pin()->ObjectSelectionForParameterMapView.ToSharedRef());

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			SAssignNew(SystemToolkit.Pin()->ParameterPanel, SNiagaraParameterPanel, SystemToolkit.Pin()->ParameterPanelViewModel, SystemToolkit.Pin()->GetToolkitCommands())
			.ShowParameterSynchronizingWithLibraryIconExternallyReferenced(false)
			.SearchAdjacentWidget()
			[
				SNew(SButton)
				.OnClicked(this, &FNiagaraSystemToolkitModeBase::SummonScratchPadScriptHierarchyEditor)
				.ButtonStyle(FAppStyle::Get(), "RoundButton")
				.Visibility(this, &FNiagaraSystemToolkitModeBase::GetSummonScratchPadHierarchyEditorButtonVisibility)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EditHierarchy_ScriptInputs", "Edit Input Hierarchy"))
				]
			]
		];
	SystemToolkit.Pin()->RefreshParameters();

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_SystemParameterDefinitions(const FSpawnTabArgs& Args)
{
	checkf(Args.GetTabId().TabType == SystemParameterDefinitionsTabID, TEXT("Wrong tab ID in NiagaraScriptToolkit"));

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			SNew(SNiagaraParameterDefinitionsPanel, SystemToolkit.Pin()->ParameterDefinitionsPanelViewModel, SystemToolkit.Pin()->GetToolkitCommands())
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == DetailsTabID);

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("SystemOverviewDetails", "Details"))
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &FNiagaraSystemToolkitModeBase::GetActiveSelectionDetailsIndex)
			+ SWidgetSwitcher::Slot()
			[
				NiagaraEditorModule.GetWidgetProvider()->CreateStackView(*SystemToolkit.Pin()->GetSystemViewModel()->GetSelectionViewModel()->GetSelectionStackViewModel())
			]
			+ SWidgetSwitcher::Slot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f, 2.0f, 2.0f, 5.0f)
				[
					SNew(STextBlock)
					.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ScratchPad.SubSectionHeaderText")
					.Visibility(this, &FNiagaraSystemToolkitModeBase::GetObjectSelectionSubHeaderTextVisibility)
					.Text(this, &FNiagaraSystemToolkitModeBase::GetObjectSelectionSubHeaderText)
				]
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.AutoHeight()
				.Padding(FMargin(0.0f, 10.0f, 0.0f, 0.0f))
				[
					SNew(STextBlock)
					.Visibility(this, &FNiagaraSystemToolkitModeBase::GetObjectSelectionNoSelectionTextVisibility)
					.Text(LOCTEXT("NoObjectSelection", "No object selected"))
				]
				+ SVerticalBox::Slot()
				[
					SNew(SNiagaraSelectedObjectsDetails, ObjectSelection.ToSharedRef())
				]
			]
		];

	SDockTab::FOnTabClosedCallback TabClosedCallback = SDockTab::FOnTabClosedCallback::CreateLambda([this](TSharedRef<SDockTab> DockTab)
	{
		SystemToolkit.Pin()->GetSystemViewModel()->GetSelectionViewModel()->GetSelectionStackViewModel()->ResetSearchText();
	});
	
	SpawnedTab->SetOnTabClosed(TabClosedCallback);
	return SpawnedTab;
}

EVisibility FNiagaraSystemToolkitModeBase::GetObjectSelectionSubHeaderTextVisibility() const
{
	return ObjectSelection->GetSelectedObjects().Num() != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FNiagaraSystemToolkitModeBase::GetObjectSelectionSubHeaderText() const
{
	return ObjectSelectionSubHeaderText;
}

EVisibility FNiagaraSystemToolkitModeBase::GetObjectSelectionNoSelectionTextVisibility() const
{
	return ObjectSelection->GetSelectedObjects().Num() == 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_SelectedEmitterGraph(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == SelectedEmitterGraphTabID);

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			SNew(SNiagaraSelectedEmitterGraph, SystemToolkit.Pin()->SystemViewModel.ToSharedRef())
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_DebugCacheSpreadsheet(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == DebugCacheSpreadsheetTabID);

	TSharedRef<SNiagaraSimCacheView> SimCacheSpreadsheetView =
		SNew(SNiagaraSimCacheView)
		.SimCacheViewModel(SystemToolkit.Pin()->GetSimCacheViewModel());

	TSharedRef<SNiagaraDebugCaptureView> DebugCaptureView =
		SNew(SNiagaraDebugCaptureView, SystemToolkit.Pin()->GetSystemViewModel().ToSharedRef(), SystemToolkit.Pin()->GetSimCacheViewModel().ToSharedRef());

	const TSharedRef<SVerticalBox> Contents = SNew(SVerticalBox);

	Contents->AddSlot()
		.FillHeight(1.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				+SSplitter::Slot()
				.SizeRule(SSplitter::SizeToContent)
				[
					DebugCaptureView
				]
				+SSplitter::Slot()
				.Value(0.8f)
				[
					SimCacheSpreadsheetView
				]
			]
		];

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			Contents
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_GeneratedCode(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == GeneratedCodeTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab);
	SpawnedTab->SetContent(SNew(SNiagaraGeneratedCodeView, SystemToolkit.Pin()->GetSystemViewModel().ToSharedRef(), SpawnedTab));
	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_MessageLog(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == MessageLogTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("NiagaraMessageLogTitle", "Niagara Log"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("NiagaraLog")))
			[
				SystemToolkit.Pin()->NiagaraMessageLog.ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_SystemOverview(const FSpawnTabArgs& Args)
{
	TSharedPtr<FNiagaraSystemToolkit, ESPMode::ThreadSafe> NiagaraSystemToolkit = SystemToolkit.Pin();
	if(!NiagaraSystemToolkit->GetSystemOverview().IsValid())
	{
		NiagaraSystemToolkit->SetSystemOverview(FNiagaraEditorModule::Get().GetWidgetProvider()->CreateSystemOverview(NiagaraSystemToolkit->GetSystemViewModel().ToSharedRef(), NiagaraSystemToolkit->GetEditedAsset()));
	}
	
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("SystemOverviewTabLabel", "System Overview"))
		[
			NiagaraSystemToolkit->GetSystemOverview().ToSharedRef()
		];


	SpawnedTab->SetOnTabActivated(SDockTab::FOnTabActivatedCallback::CreateLambda([this](TSharedRef<SDockTab> Input, ETabActivationCause)
		{
			SystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->SetActiveDocumentTab(Input);			
		}));

	SpawnedTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([this](TSharedRef<SDockTab>)
	{
		SystemToolkit.Pin()->SetSystemOverview(nullptr);
	}));
	
	return SpawnedTab;
}



TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_ScratchPadScripts(const FSpawnTabArgs& Args)
{
	if (!SystemToolkit.Pin()->GetScriptScratchpadManager().IsValid())
	{
		SystemToolkit.Pin()->SetScriptScratchpadManager(FNiagaraEditorModule::Get().GetWidgetProvider()->CreateScriptScratchPadManager(*SystemToolkit.Pin()->GetSystemViewModel()->GetScriptScratchPadViewModel()));
	}

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("ScratchPadLocalModulesTabLabel", "Local Modules"))
		[
			SystemToolkit.Pin()->GetScriptScratchpadManager().ToSharedRef()
		];

	SpawnedTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([this](TSharedRef<SDockTab>)
		{
			SystemToolkit.Pin()->SetScriptScratchpadManager(nullptr);
		}));

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_ScriptStats(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == ScriptStatsTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("NiagaraScriptStatsTitle", "Script Stats"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ScriptStats")))
			[
				SystemToolkit.Pin()->ScriptStats->GetWidget().ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_Baker(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == BakerTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("NiagaraBakerTitle", "Baker"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Baker")))
			[
				SystemToolkit.Pin()->BakerViewModel->GetWidget().ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_Versioning(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == VersioningTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("EmitterVersioningTitle", "Versioning"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("EmitterVersioning")))
			[
				SystemToolkit.Pin()->GetVersioningWidget().ToSharedRef()
			]
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_UserParameters(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == UserParametersTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
	.Label(LOCTEXT("UserParametersTabTitle", "User Parameters"));

	if(SystemToolkit.Pin()->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		SpawnedTab->SetContent(SNew(SNiagaraSystemUserParameters, SystemToolkit.Pin()->GetSystemViewModel()));
		return SpawnedTab;
	}

	TSharedRef<SWidget> EmptyTabContent = SNew(SBox)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock).Text(LOCTEXT("EmptyUserParametersTabText", "User Parameters are only supported in System assets.")).AutoWrapText(true)
	];
	
	SpawnedTab->SetContent(EmptyTabContent);
	return SpawnedTab;
}

TSharedRef<SWidget> GenerateRowContentForUserParameterHierarchyEditor(TSharedRef<FHierarchyElementViewModel> HierarchyItem, TSharedPtr<FNiagaraSystemViewModel> SystemViewModel)
{
	if(HierarchyItem->GetDataMutable()->IsA<UHierarchyCategory>())
	{
		TSharedRef<FHierarchyCategoryViewModel> TreeViewCategory = StaticCastSharedRef<FHierarchyCategoryViewModel>(HierarchyItem);
		return SNew(SHierarchyElement, TreeViewCategory)
			.Style(&FDataHierarchyEditorStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("HierarchyEditor.Category"));
	}
	else if(const UNiagaraHierarchyUserParameter* UserParameter = Cast<UNiagaraHierarchyUserParameter>(HierarchyItem->GetData()))
	{
		TSharedRef<SWidget> ParameterWidget = FNiagaraParameterUtilities::GetParameterWidget(UserParameter->GetUserParameter(), true, false);
		UNiagaraScriptVariable* ScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(UserParameter->GetUserParameter(), SystemViewModel->GetSystem());

		ParameterWidget->SetToolTipText(TAttribute<FText>::CreateLambda([ScriptVariable]()
		{
			return ScriptVariable->Metadata.Description;
		}));
		
		return ParameterWidget;
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> GenerateCustomDetailsPanelNameWidgetForUserParameterEditor(TSharedPtr<FHierarchyElementViewModel> HierarchyItem)
{
	if(!HierarchyItem.IsValid())
	{
		return SNew(STextBlock).Text(FText::FromString("None selected"));
	}
	
	if(HierarchyItem->GetData()->IsA<UHierarchyCategory>() || HierarchyItem->GetData()->IsA<UHierarchySection>())
	{
		return SNew(SBox).Padding(2.f)
		[
			SNew(STextBlock)
			.Text_Lambda([HierarchyItem]
			{
				return FText::FromString(HierarchyItem->ToString());
			})
			.TextStyle(&FDataHierarchyEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("HierarchyEditor.CategoryTextBlock"))
		];
	}
	else if(const UNiagaraHierarchyUserParameter* UserParameter = Cast<UNiagaraHierarchyUserParameter>(HierarchyItem->GetData()))
	{
		return FNiagaraParameterUtilities::GetParameterWidget(UserParameter->GetUserParameter(), false, false);
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_UserParametersHierarchyEditor(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == UserParametersHierarchyTabID);
	
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("UserParametersHierarchyTitle", "User Parameters Hierarchy"))
		[
			SNew(SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("UserParameters")))
			[
				SNew(SDataHierarchyEditor, SystemToolkit.Pin()->GetSystemViewModel()->GetUserParametersHierarchyViewModel())
				.OnGenerateRowContentWidget_Static(&GenerateRowContentForUserParameterHierarchyEditor, SystemToolkit.Pin()->GetSystemViewModel())
				.OnGenerateCustomDetailsPanelNameWidget_Static(&GenerateCustomDetailsPanelNameWidgetForUserParameterEditor)
			]
		];
	
	return SpawnedTab;
}

TSharedRef<SWidget> GenerateRowContentForSummaryViewHierarchyEditor(TSharedRef<FHierarchyElementViewModel> HierarchyItem, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel)
{
	if(const UNiagaraHierarchyModuleInput* ModuleInput = Cast<UNiagaraHierarchyModuleInput>(HierarchyItem->GetData()))
	{
		TSharedRef<FNiagaraModuleInputViewModel> ModuleInputViewModel = StaticCastSharedRef<FNiagaraModuleInputViewModel>(HierarchyItem);
		return SNew(SNiagaraHierarchyModuleInput, ModuleInputViewModel);
	}
	else if(const UNiagaraHierarchyAssignmentInput* AssignmentInput = Cast<UNiagaraHierarchyAssignmentInput>(HierarchyItem->GetData()))
	{
		TSharedRef<FNiagaraAssignmentInputViewModel> AssignmentInputViewModel = StaticCastSharedRef<FNiagaraAssignmentInputViewModel>(HierarchyItem);
		TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData> InputData = AssignmentInputViewModel->GetInputData();

		if(ensure(InputData.IsSet()))
		{
			return FNiagaraParameterUtilities::GetParameterWidget(FNiagaraVariable(InputData->Type, InputData->InputName), false, false);
		}
	}
	else if(const UNiagaraHierarchyModule* Module = Cast<UNiagaraHierarchyModule>(HierarchyItem->GetDataMutable()))
	{
		TSharedPtr<FHierarchyElementViewModel> ItemPtr = HierarchyItem;
		TSharedPtr<FNiagaraFunctionViewModel> ModuleViewModel = StaticCastSharedPtr<FNiagaraFunctionViewModel>(ItemPtr);

		if(UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(ModuleViewModel->GetFunctionCallNode()))
		{
			return SNew(SNiagaraHierarchyAssignment, *AssignmentNode);
		}
		else
		{
			return SNew(SNiagaraHierarchyModule, ModuleViewModel);
		}		
	}
	else if(const UNiagaraHierarchyRenderer* Renderer = Cast<UNiagaraHierarchyRenderer>(HierarchyItem->GetDataMutable()))
	{
		TSharedPtr<FHierarchyElementViewModel> ItemPtr = HierarchyItem;
		TSharedPtr<FNiagaraHierarchyRendererViewModel> RendererViewModel = StaticCastSharedPtr<FNiagaraHierarchyRendererViewModel>(ItemPtr);
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(SImage).Image(FSlateIconFinder::FindIconBrushForClass(RendererViewModel->GetRendererProperties()->GetClass()))
			]
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock).Text(RendererViewModel.ToSharedRef(), &FNiagaraHierarchyRendererViewModel::ToStringAsText)
			];
	}
	else if(const UNiagaraHierarchyEventHandler* EventHandler = Cast<UNiagaraHierarchyEventHandler>(HierarchyItem->GetDataMutable()))
	{
		TSharedPtr<FHierarchyElementViewModel> ItemPtr = HierarchyItem;
		TSharedPtr<FNiagaraHierarchyEventHandlerViewModel> EventHandlerViewModel = StaticCastSharedPtr<FNiagaraHierarchyEventHandlerViewModel>(ItemPtr);
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(SImage).Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.EventIcon"))
			]
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock).Text(EventHandlerViewModel.ToSharedRef(), &FNiagaraHierarchyEventHandlerViewModel::ToStringAsText)
			];
	}
	else if(const UNiagaraHierarchySimStage* SimStage = Cast<UNiagaraHierarchySimStage>(HierarchyItem->GetDataMutable()))
	{
		TSharedPtr<FHierarchyElementViewModel> ItemPtr = HierarchyItem;
		TSharedPtr<FNiagaraHierarchySimStageViewModel> SimStageViewModel = StaticCastSharedPtr<FNiagaraHierarchySimStageViewModel>(ItemPtr);
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(SImage).Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.SimulationStageIcon"))
			]
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock).Text(SimStageViewModel.ToSharedRef(), &FNiagaraHierarchySimStagePropertiesViewModel::ToStringAsText)
			];
	}
	else if(const UNiagaraHierarchySimStageProperties* SimStageProperties = Cast<UNiagaraHierarchySimStageProperties>(HierarchyItem->GetDataMutable()))
	{
		TSharedPtr<FHierarchyElementViewModel> ItemPtr = HierarchyItem;
		TSharedPtr<FNiagaraHierarchySimStagePropertiesViewModel> SimStagePropertiesViewModel = StaticCastSharedPtr<FNiagaraHierarchySimStagePropertiesViewModel>(ItemPtr);
		return SNew(STextBlock).Text(SimStagePropertiesViewModel.ToSharedRef(), &FNiagaraHierarchySimStagePropertiesViewModel::ToStringAsText);
	}
	else if(HierarchyItem->GetDataMutable()->IsA<UHierarchyCategory>())
	{
		TSharedRef<FHierarchyCategoryViewModel> TreeViewCategory = StaticCastSharedRef<FHierarchyCategoryViewModel>(HierarchyItem);
		return SNew(SHierarchyElement, TreeViewCategory)
			.Style(&FDataHierarchyEditorStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("HierarchyEditor.Category"));
	}
	else if(const UHierarchyItem* Item = Cast<UHierarchyItem>(HierarchyItem->GetData()))
	{
		return SNew(STextBlock).Text(FText::FromString(HierarchyItem->ToString()));
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> GenerateCustomDetailsPanelNameWidgetForSummaryViewEditor(TSharedPtr<FHierarchyElementViewModel> HierarchyItem)
{
	if(!HierarchyItem.IsValid())
	{
		return SNew(STextBlock).Text(FText::FromString("None selected"));
	}
	
	if(HierarchyItem->GetData()->IsA<UHierarchyCategory>() || HierarchyItem->GetData()->IsA<UHierarchySection>())
	{
		return SNew(SBox).Padding(2.f)
		[
			SNew(STextBlock)
			.Text_Lambda([HierarchyItem]
			{
				return FText::FromString(HierarchyItem->ToString());
			})
		];
	}
	else if(HierarchyItem->GetData()->IsA<UNiagaraHierarchyModuleInput>())
	{
		TSharedPtr<FNiagaraModuleInputViewModel> ModuleInputViewModel = StaticCastSharedPtr<FNiagaraModuleInputViewModel>(HierarchyItem);
		return SNew(SNiagaraHierarchyModuleInput, ModuleInputViewModel.ToSharedRef());
	}
	else if(HierarchyItem->GetData()->IsA<UHierarchyItem>())
	{
		return SNew(STextBlock).Text(FText::FromString(HierarchyItem->ToString()));
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FNiagaraSystemToolkitModeBase::CreateSummaryViewWidget() const
{
	TArray<FGuid> SelectedEmitterHandleGuids = SystemToolkit.Pin()->GetSystemViewModel()->GetSelectionViewModel()->GetSelectedEmitterHandleIds();

	TSharedPtr<SWidget> ContentWidget = SNullWidget::NullWidget;
	if(SystemToolkit.Pin()->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = SystemToolkit.Pin()->GetSystemViewModel()->GetEmitterHandleViewModels()[0]->GetEmitterViewModel();
		UNiagaraSummaryViewViewModel* SummaryViewHierarchyViewModel = EmitterViewModel->GetSummaryHierarchyViewModel();
		
		ContentWidget = SNew(SDataHierarchyEditor, SummaryViewHierarchyViewModel)
		.OnGenerateRowContentWidget_Static(&GenerateRowContentForSummaryViewHierarchyEditor, EmitterViewModel)
		.OnGenerateCustomDetailsPanelNameWidget_Static(&GenerateCustomDetailsPanelNameWidgetForSummaryViewEditor);
	}
	else
	{
		if(SelectedEmitterHandleGuids.Num() != 1)
		{
			ContentWidget = SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(LOCTEXT("SummaryViewEditorInvalidSelection", "Please select a single emitter to display its summary view options."))
			];
		}
		else
		{		
			TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = SystemToolkit.Pin()->GetSystemViewModel()->GetEmitterHandleViewModelById(SelectedEmitterHandleGuids[0])->GetEmitterViewModel();
			UNiagaraSummaryViewViewModel* SummaryViewHierarchyViewModel = EmitterViewModel->GetSummaryHierarchyViewModel();
		
			ContentWidget = SNew(SDataHierarchyEditor, SummaryViewHierarchyViewModel)
			.OnGenerateRowContentWidget_Static(&GenerateRowContentForSummaryViewHierarchyEditor, EmitterViewModel)
			.OnGenerateCustomDetailsPanelNameWidget_Static(&GenerateCustomDetailsPanelNameWidgetForSummaryViewEditor);
		}
	}

	return ContentWidget.ToSharedRef();
}

void FNiagaraSystemToolkitModeBase::UpdateSummaryViewOnSelectionChanged() const
{
	if(SystemToolkit.Pin()->GetTabManager()->FindExistingLiveTab(EmitterSummaryViewEditorTabID).IsValid())
	{
		SummaryViewContainer->SetContent(CreateSummaryViewWidget());
	}
}

void FNiagaraSystemToolkitModeBase::OnSummaryViewEditorClosed(TSharedRef<SDockTab> DockTab) const
{
	SummaryViewContainer->SetContent(SNullWidget::NullWidget);
}

FReply FNiagaraSystemToolkitModeBase::SummonScratchPadScriptHierarchyEditor()
{
	SystemToolkit.Pin()->GetTabManager()->TryInvokeTab(ScratchPadHierarchyEditorTabID);
	return FReply::Handled();
}

TSharedRef<SWidget> FNiagaraSystemToolkitModeBase::CreateScratchPadHierarchyWidget()
{
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ActiveScratchPadViewModel = SystemToolkit.Pin()->GetSystemViewModel()->GetScriptScratchPadViewModel()->GetActiveScriptViewModel();

	ActiveScratchPadViewModel = SystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
	
	TSharedPtr<SWidget> ContentWidget = SNullWidget::NullWidget;
	if(ActiveScratchPadViewModel.IsValid())
	{
		UNiagaraScriptParametersHierarchyViewModel* ScriptHierarchyViewModel = ActiveScratchPadViewModel->GetHierarchyViewModel();
		
		ContentWidget = SNew(SDataHierarchyEditor, ScriptHierarchyViewModel)
		.OnGenerateRowContentWidget_Static(&FNiagaraEditorUtilities::HierarchyEditor::Scripts::GenerateRowContentForScriptParameterHierarchyEditor);
		//.OnGenerateCustomDetailsPanelNameWidget_Static(&GenerateCustomDetailsPanelNameWidgetForSummaryViewEditor);
	}
	else
	{
		ContentWidget = SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock).Text(LOCTEXT("ScratchPadHierarchyEditorInvalidSelection", "Please select a scratch pad."))
		];		
	}

	return ContentWidget.ToSharedRef();
}

void FNiagaraSystemToolkitModeBase::UpdateScratchPadActiveScriptChanged(TSharedPtr<SDockTab> DockTab)
{
	if(SystemToolkit.Pin()->GetTabManager()->FindExistingLiveTab(ScratchPadHierarchyEditorTabID).IsValid() && ScratchPadHierarchyContainer.IsValid())
	{
		TSharedPtr<FNiagaraScratchPadScriptViewModel> ActiveScratchPadViewModel = SystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
		if(LastActiveScratchPadViewModel != ActiveScratchPadViewModel)
		{
			ScratchPadHierarchyContainer->SetContent(CreateScratchPadHierarchyWidget());
		}
			
		LastActiveScratchPadViewModel = ActiveScratchPadViewModel;
	}
}

void FNiagaraSystemToolkitModeBase::OnScratchPadHierarchyEditorClosed(TSharedRef<SDockTab> DockTab)
{
	ScratchPadHierarchyContainer->SetContent(SNullWidget::NullWidget);
}

EVisibility FNiagaraSystemToolkitModeBase::GetSummonScratchPadHierarchyEditorButtonVisibility() const
{
	return SystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet() != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_SummaryViewEditor(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == EmitterSummaryViewEditorTabID);

	if(UpdateSummaryViewHandle.IsValid())
	{
		SystemToolkit.Pin()->GetSystemViewModel()->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().Remove(UpdateSummaryViewHandle);
	}

	if(SystemToolkit.Pin()->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		UpdateSummaryViewHandle = SystemToolkit.Pin()->GetSystemViewModel()->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().AddSP(this, &FNiagaraSystemToolkitModeBase::UpdateSummaryViewOnSelectionChanged);
	}
	
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.OnTabClosed(this, &FNiagaraSystemToolkitModeBase::OnSummaryViewEditorClosed)
		.Label(LOCTEXT("SummaryViewHierarchyTitle", "Edit Summary View"))
		[
			SAssignNew(SummaryViewContainer, SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("SummaryView")))
		];

	SummaryViewContainer->SetContent(CreateSummaryViewWidget());
	
	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitModeBase::SpawnTab_ScratchPadHierarchyEditor(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == ScratchPadHierarchyEditorTabID);

	if(UpdateScratchPadScriptHierarchyHandle.IsValid())
	{
		SystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->OnActiveDocumentChanged().Remove(UpdateScratchPadScriptHierarchyHandle);
	}
	
	UpdateScratchPadScriptHierarchyHandle = SystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->OnActiveDocumentChanged().AddSP(this, &FNiagaraSystemToolkitModeBase::UpdateScratchPadActiveScriptChanged);
	
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.OnTabClosed(this, &FNiagaraSystemToolkitModeBase::OnScratchPadHierarchyEditorClosed)
		.Label(LOCTEXT("ScratchPadHierarchyTitle", "Edit Hierarchy"))
		[
			SAssignNew(ScratchPadHierarchyContainer, SBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ScratchPadHierarchy")))
		];

	TSharedPtr<FNiagaraScratchPadScriptViewModel> ActiveScratchPadViewModel = SystemToolkit.Pin()->GetSystemViewModel()->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
	LastActiveScratchPadViewModel = ActiveScratchPadViewModel;
	
	ScratchPadHierarchyContainer->SetContent(CreateScratchPadHierarchyWidget());
	
	return SpawnedTab;
}

#undef LOCTEXT_NAMESPACE
