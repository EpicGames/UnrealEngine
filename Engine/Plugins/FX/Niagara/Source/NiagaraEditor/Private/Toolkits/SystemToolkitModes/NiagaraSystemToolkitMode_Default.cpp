// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemToolkitMode_Default.h"

#include "NiagaraEditorCommands.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "NiagaraDebuggerCommon.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraSystemEditorDocumentsViewModel.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "NiagaraSystemToolkitMode_Default"

FNiagaraSystemToolkitMode_Default::FNiagaraSystemToolkitMode_Default(TWeakPtr<FNiagaraSystemToolkit> InSystemToolkit) : FNiagaraSystemToolkitModeBase(FNiagaraSystemToolkitModeBase::DefaultModeName, InSystemToolkit)
{
	TabLayout = FTabManager::NewLayout("Standalone_Niagara_System_Layout_v30")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.33f)
					->Split(
						// Left top
						FTabManager::NewStack()
						->AddTab(ViewportTabID, ETabState::OpenedTab)
					)
					
					->Split(
						// Left sidebar
						FTabManager::NewStack()
						->AddTab(SystemParametersTabID, ETabState::OpenedTab)
						->AddTab(UserParametersTabID, ETabState::OpenedTab)
						->AddTab(ScratchPadScriptsTabID, ETabState::OpenedTab)
						->SetForegroundTab(SystemParametersTabID)
					)
				)
				->SetSizeCoefficient(0.33f)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->Split(
						// Center top
						FTabManager::NewStack()
						->AddTab(SystemOverviewTabID, ETabState::OpenedTab)
						->AddTab("Document", ETabState::ClosedTab)
						->AddTab(BakerTabID, ETabState::ClosedTab)
						->SetForegroundTab(SystemOverviewTabID)
					)
					->Split(
						// center bottom
						FTabManager::NewStack()
						->SetSizeCoefficient(0.33f)
						->AddTab(SequencerTabID, ETabState::OpenedTab)
						->AddTab(CurveEditorTabID, ETabState::OpenedTab)
						->AddTab(MessageLogTabID, ETabState::OpenedTab)
						->AddTab(ScriptStatsTabID, ETabState::OpenedTab)
						->SetForegroundTab(SequencerTabID)
					)
				)
				->Split
				(
					// Right
					FTabManager::NewStack()
					->SetSizeCoefficient(0.33f)
					->AddTab(DetailsTabID, ETabState::OpenedTab)
					->AddTab(SelectedEmitterGraphTabID, ETabState::ClosedTab)
					->AddTab(SystemScriptTabID, ETabState::ClosedTab)
					->AddTab(PreviewSettingsTabId, ETabState::ClosedTab)
					->AddTab(GeneratedCodeTabID, ETabState::ClosedTab)
				)
			)
		);

	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	NiagaraEditorSettings->ExternalExtendLayoutForAsset(LayoutExtender.ToSharedRef(), SystemToolkit.Pin()->GetSystemViewModel()->GetSystem());
	TabLayout->ProcessExtensions(*LayoutExtender.Get());
	
	ExtendToolbar();
}

void FNiagaraSystemToolkitMode_Default::ExtendToolbar()
{
	struct Local
	{
		static TSharedRef<SWidget> FillSimulationOptionsMenu(FNiagaraSystemToolkit* Toolkit)
		{
			FMenuBuilder MenuBuilder(true, Toolkit->GetToolkitCommands());
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleAutoPlay);
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleResetSimulationOnChange);
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleResimulateOnChangeWhilePaused);
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleResetDependentSystems);
			return MenuBuilder.MakeWidget();
		}

		static TSharedRef<SWidget> FillDebugOptionsMenu(FNiagaraSystemToolkit* Toolkit)
		{
			FMenuBuilder MenuBuilder(true, Toolkit->GetToolkitCommands());

#if WITH_NIAGARA_DEBUGGER
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().OpenDebugHUD);
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().OpenDebugOutliner);
#endif
			MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().OpenAttributeSpreadsheet);
			return MenuBuilder.MakeWidget();
		}

		static TSharedRef<SWidget> FillSettingsMenu(FNiagaraSystemToolkit* Toolkit)
		{
			FMenuBuilder MenuBuilder(true, Toolkit->GetToolkitCommands());
			if (FProperty* AutoPlayProperty = UNiagaraEditorSettings::StaticClass()->FindPropertyByName("bAutoPlay"))
			{
				MenuBuilder.AddMenuEntry(
					AutoPlayProperty->GetDisplayNameText(),
					AutoPlayProperty->GetToolTipText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda(
							[]()
							{
								UNiagaraEditorSettings* EditorSettings = GetMutableDefault<UNiagaraEditorSettings>();
								EditorSettings->SetAutoPlay(!EditorSettings->GetAutoPlay());
							}
						),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda(
							[]() -> bool
							{
								return GetDefault<UNiagaraEditorSettings>()->GetAutoPlay();
							}
						)
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton
				);
			}

			return MenuBuilder.MakeWidget();
		}

		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FNiagaraSystemToolkit* Toolkit)
		{
			ToolbarBuilder.BeginSection("Apply");
			{
				if (Toolkit->HasEmitter())
				{
					ToolbarBuilder.AddToolBarButton(FNiagaraEditorCommands::Get().Apply,
						NAME_None, TAttribute<FText>(), TAttribute<FText>(),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.Apply"),
						FName(TEXT("ApplyNiagaraEmitter")));
				}
				ToolbarBuilder.AddToolBarButton(FNiagaraEditorCommands::Get().ApplyScratchPadChanges,
					NAME_None, TAttribute<FText>(), TAttribute<FText>(),
					FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.ApplyScratchPadChanges"),
					FName(TEXT("ApplyScratchPadChanges")));
			}
			ToolbarBuilder.EndSection();
			ToolbarBuilder.BeginSection("Compile");
			{
				ToolbarBuilder.AddToolBarButton(FNiagaraEditorCommands::Get().Compile,
					NAME_None,
					TAttribute<FText>(),
					TAttribute<FText>(Toolkit, &FNiagaraSystemToolkit::GetCompileStatusTooltip),
					TAttribute<FSlateIcon>(Toolkit, &FNiagaraSystemToolkit::GetCompileStatusImage),
					FName(TEXT("CompileNiagaraSystem")));
				ToolbarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateRaw(Toolkit, &FNiagaraSystemToolkit::GenerateCompileMenuContent),
					LOCTEXT("BuildCombo_Label", "Auto-Compile Options"),
					LOCTEXT("BuildComboToolTip", "Auto-Compile options menu"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Build"),
					true);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("NiagaraThumbnail");
			{
				ToolbarBuilder.AddToolBarButton(FNiagaraEditorCommands::Get().SaveThumbnailImage, NAME_None,
					LOCTEXT("GenerateThumbnail", "Thumbnail"),
					LOCTEXT("GenerateThumbnailTooltip","Generate a thumbnail image."),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveThumbnail"));
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("NiagaraPreviewOptions");
			{
				ToolbarBuilder.AddToolBarButton(FNiagaraEditorCommands::Get().ToggleBounds, NAME_None,
					LOCTEXT("ShowBounds", "Bounds"),
					LOCTEXT("ShowBoundsTooltip", "Show the bounds for the scene."),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.ToggleShowBounds"));
				ToolbarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateRaw(Toolkit, &FNiagaraSystemToolkit::GenerateBoundsMenuContent, Toolkit->GetToolkitCommands()),
					LOCTEXT("BoundsMenuCombo_Label", "Bounds Options"),
					LOCTEXT("BoundsMenuCombo_ToolTip", "Bounds options"),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.ToggleShowBounds"),
					true
				);
			}
			ToolbarBuilder.EndSection();
			
#if STATS
			ToolbarBuilder.BeginSection("NiagaraStatisticsOptions");
			{
				ToolbarBuilder.AddToolBarButton(FNiagaraEditorCommands::Get().ToggleStatPerformance, NAME_None,
                    LOCTEXT("NiagaraShowPerformance", "Performance"),
                    LOCTEXT("NiagaraShowPerformanceTooltip", "Show runtime performance for particle scripts."),
                    FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.ToggleStats"));
				ToolbarBuilder.AddComboButton(
                    FUIAction(),
                    FOnGetContent::CreateRaw(Toolkit, &FNiagaraSystemToolkit::GenerateStatConfigMenuContent, Toolkit->GetToolkitCommands()),
                    FText(),
                    LOCTEXT("NiagaraShowPerformanceCombo_ToolTip", "Runtime performance options"),
                    FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.ToggleStats"),
                    true);
				ToolbarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateStatic(Local::FillDebugOptionsMenu, Toolkit),
					LOCTEXT("DebugOptions", "Debug"),
					LOCTEXT("DebugOptionsTooltip", "Debug options"),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Debug")
				);
			}
			ToolbarBuilder.EndSection();
#endif
			
			ToolbarBuilder.BeginSection("PlaybackOptions");
			{
				ToolbarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateStatic(Local::FillSimulationOptionsMenu, Toolkit),
					LOCTEXT("SimulationOptions", "Simulation"),
					LOCTEXT("SimulationOptionsTooltip", "Simulation options"),
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.Simulate")
				);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("Baker");
			{
				FUIAction BakerToggleAction(
					FExecuteAction::CreateLambda(
						[TabManager = Toolkit->GetTabManager()]()
						{
							TSharedPtr<SDockTab> ExistingTab = TabManager->FindExistingLiveTab(FNiagaraSystemToolkitModeBase::BakerTabID);
							if (ExistingTab)
							{
								ExistingTab->RequestCloseTab();
							}
							else
							{
								TabManager->TryInvokeTab(FNiagaraSystemToolkitModeBase::BakerTabID);
							}
						}
					),
					nullptr,
					FIsActionChecked::CreateLambda(
						[TabManager = Toolkit->GetTabManager()]()
						{
							return TabManager->FindExistingLiveTab(FNiagaraSystemToolkitModeBase::BakerTabID).IsValid();
						}
					)
				);

				ToolbarBuilder.AddToolBarButton(
					BakerToggleAction,
					NAME_None,
					LOCTEXT("BakerLabel", "Baker"),
					LOCTEXT("BakerTooltip", "Toggles the baker tab."),
					FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.BakerIcon"),
					EUserInterfaceActionType::ToggleButton
				);
			}
			ToolbarBuilder.EndSection();
			
			ToolbarBuilder.BeginSection("Scalability");
			{
				FUIAction ScalabilityToggleAction(FExecuteAction::CreateRaw(Toolkit, &FNiagaraSystemToolkit::SetCurrentMode, FNiagaraSystemToolkitModeBase::ScalabilityModeName));
				ScalabilityToggleAction.GetActionCheckState = FGetActionCheckState::CreateLambda([Toolkit]()
				{
					return Toolkit->GetCurrentMode() == FNiagaraSystemToolkitModeBase::ScalabilityModeName ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				});
				
				ToolbarBuilder.AddToolBarButton(
					ScalabilityToggleAction,
					NAME_None, 
					LOCTEXT("ScalabilityLabel", "Scalability"),
					LOCTEXT("ScalabilityTooltip", "Turn on scalability mode to optimize your effects for various platforms and quality settings."),
					FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.Scalability"),
					EUserInterfaceActionType::ToggleButton
				);
			}
			ToolbarBuilder.EndSection();

			if (Toolkit->HasEmitter())
			{
				ToolbarBuilder.BeginSection("Versioning");
				{
					ToolbarBuilder.AddToolBarButton(FNiagaraEditorCommands::Get().EmitterVersioning, NAME_None,
					TAttribute<FText>(Toolkit, &FNiagaraSystemToolkit::GetVersionButtonLabel),
					LOCTEXT("NiagaraShowModuleVersionsTooltip", "Manage different versions of this emitter."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Versions"));

					FUIAction DropdownAction;
					DropdownAction.IsActionVisibleDelegate = FIsActionButtonVisible::CreateLambda([Toolkit]() { return Toolkit->GetEmitterVersions().Num() > 1; });
					ToolbarBuilder.AddComboButton(
					 DropdownAction,
					 FOnGetContent::CreateRaw(Toolkit, &FNiagaraSystemToolkit::GenerateVersioningDropdownMenu, Toolkit->GetToolkitCommands()),
					 FText(),
					 LOCTEXT("NiagaraShowVersions_ToolTip", "Select version to edit"),
					 FSlateIcon(FAppStyle::GetAppStyleSetName(), "Versions"),
					 true);
				}
				ToolbarBuilder.EndSection();
			}

			ToolbarBuilder.BeginSection("Settings");
			{
				ToolbarBuilder.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateStatic(Local::FillSettingsMenu, Toolkit),
					FText(),
					LOCTEXT("NiagaraEditorGlobalSettings", "Niagara Editor Settings"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "DetailsView.ViewOptions")
				);
			}
			ToolbarBuilder.EndSection();

			if (GbShowNiagaraDeveloperWindows != 0)
			{
				// a simple ticker to visually see how fast the ui is ticking and to see any hitches due to stack actions 
				ToolbarBuilder.AddSeparator();
				ToolbarBuilder.AddWidget(SNew(STextBlock)
					.Margin(FMargin(5, 10, 0, 0))
					.ColorAndOpacity_Lambda([]()
					{
						static int32 Counter = 0;
						Counter = (Counter + 1) % 255;
						return FLinearColor::MakeFromHSV8(Counter, 192, 255);
					})
					.Text_Lambda([]()
					{
						static int32 Counter = 0;
						Counter = (Counter + 1) % 16;
						switch (Counter / 2)
						{
						default:
						case 0: return FText::FromString(".oO     ");
						case 1: return FText::FromString(" .oO    ");
						case 2: return FText::FromString("  .oO   ");
						case 3: return FText::FromString("   .oO  ");
						case 4: return FText::FromString("    .oO ");
						case 5: return FText::FromString("     .oO");
						case 6: return FText::FromString("O     .o");
						case 7: return FText::FromString("oO     .");
						}
					}));
			}
		}
	};

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		SystemToolkit.Pin()->GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, SystemToolkit.Pin().Get())
		);

	//SystemToolkit.Pin()->AddToolbarExtender(ToolbarExtender);
	
	// FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	// SystemToolkit.Pin()->AddToolbarExtender(NiagaraEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(SystemToolkit.Pin()->GetToolkitCommands(), SystemToolkit.Pin()->GetObjectsBeingEdited()));
}

void FNiagaraSystemToolkitMode_Default::PostActivateMode()
{
	FNiagaraSystemToolkitModeBase::PostActivateMode();
	
	// We need to register the primary tab ID with the document system so that we can easily update and 
	// reference it elsewhere and only this class knows it's identity since it is dynamically allocated.
	TSharedPtr<FNiagaraSystemToolkit> Toolkit = StaticCastSharedPtr<FNiagaraSystemToolkit>(SystemToolkit.Pin());
	if (Toolkit.IsValid())
	{
		TSharedPtr<FNiagaraSystemViewModel> SystemVM = Toolkit->GetSystemViewModel();
		if (SystemVM.IsValid())
		{
			SystemVM->GetDocumentViewModel()->SetPrimaryDocumentID(SystemOverviewTabID);
		}
	}
}


void FNiagaraSystemToolkitMode_Default::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	// this will register the common tab factories shared across modes and will call ExtendToolbar
	FNiagaraSystemToolkitModeBase::RegisterTabFactories(InTabManager);

	// add additional tab factories here that are exclusive to this mode
}

#undef LOCTEXT_NAMESPACE
