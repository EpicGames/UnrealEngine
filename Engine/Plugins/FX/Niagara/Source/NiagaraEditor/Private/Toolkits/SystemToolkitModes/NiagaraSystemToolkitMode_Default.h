// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"

class FNiagaraSystemToolkit;

class FNiagaraSystemToolkitMode_Default : public FNiagaraSystemToolkitModeBase
{
public:
	FNiagaraSystemToolkitMode_Default(TWeakPtr<FNiagaraSystemToolkit> InSystemToolkit);

	/** FApplicationMode interface */
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	
	void ExtendToolbar();
	
	virtual void PostActivateMode() override;
};
