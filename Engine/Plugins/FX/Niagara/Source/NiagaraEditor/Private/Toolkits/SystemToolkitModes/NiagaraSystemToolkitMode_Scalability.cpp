// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemToolkitMode_Scalability.h"

#include "IDetailTreeNode.h"
#include "NiagaraEditorModule.h"
#include "NiagaraSettings.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "NiagaraSystem.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "Widgets/Scalability/SNiagaraSystemResolvedScalabilitySettings.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "Widgets/Scalability/SNiagaraScalabilityContext.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Stateless/NiagaraStatelessEmitter.h"

#define LOCTEXT_NAMESPACE "NiagaraSystemToolkitMode_Scalability"

const FName FNiagaraSystemToolkitMode_Scalability::ScalabilityContextTabID(TEXT("NiagaraSystemEditor_ScalabilityMode_ScalabilityContext"));
const FName FNiagaraSystemToolkitMode_Scalability::ScalabilityViewModeTabID(TEXT("NiagaraSystemEditor_ScalabilityMode_ScalabilityViewMode"));
const FName FNiagaraSystemToolkitMode_Scalability::ResolvedScalabilityTabID(TEXT("NiagaraSystemEditor_ScalabilityMode_ResolvedScalability"));
const FName FNiagaraSystemToolkitMode_Scalability::EffectTypeTabID(TEXT("NiagaraSystemEditor_ScalabilityMode_EffectType"));

FNiagaraSystemToolkitMode_Scalability::FNiagaraSystemToolkitMode_Scalability(TWeakPtr<FNiagaraSystemToolkit> InSystemToolkit) : FNiagaraSystemToolkitModeBase(FNiagaraSystemToolkitModeBase::ScalabilityModeName, InSystemToolkit)
{
	TabLayout = ChooseTabLayout();

	ExtendToolbar();
}

void FNiagaraSystemToolkitMode_Scalability::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	FNiagaraSystemToolkitModeBase::RegisterTabFactories(InTabManager);

	InTabManager->RegisterTabSpawner(ScalabilityContextTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitMode_Scalability::SpawnTab_ScalabilityContext)).SetDisplayName(LOCTEXT("ScalabilityContextTabName", "Scalability Context"));
	
	if(SystemToolkit.Pin()->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		InTabManager->RegisterTabSpawner(ResolvedScalabilityTabID, FOnSpawnTab::CreateSP(this, &FNiagaraSystemToolkitMode_Scalability::SpawnTab_ResolvedScalability)).SetDisplayName(LOCTEXT("ResolvedScalabilityTabName", "Active Scalability"));
	}
}

void FNiagaraSystemToolkitMode_Scalability::PreDeactivateMode()
{
	FNiagaraSystemToolkitModeBase::PreDeactivateMode();

	SystemToolkit.Pin()->RemoveToolbarExtender(ToolbarExtender);

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemToolkit.Pin()->GetSystemViewModel();
	UNiagaraSystemScalabilityViewModel* ScalabilityViewModel = SystemViewModel->GetScalabilityViewModel();

	SystemViewModel->OnEmitterHandleViewModelsChanged().RemoveAll(this);
	SystemViewModel->GetSystem().OnSystemPostEditChange().RemoveAll(this);
	
	UnbindEmitterPreviewOverrides();
	UnbindSystemPreviewOverrides();
	
	FNiagaraPlatformSet::InvalidateCachedData();
	SystemViewModel->GetSystem().UpdateScalability();
	
	ScalabilityViewModel->OnScalabilityModeChanged().Broadcast(false);
}

void FNiagaraSystemToolkitMode_Scalability::PostActivateMode()
{
	FNiagaraSystemToolkitModeBase::PostActivateMode();

	SystemToolkit.Pin()->AddToolbarExtender(ToolbarExtender);

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemToolkit.Pin()->GetSystemViewModel();
	UNiagaraSystemScalabilityViewModel* ScalabilityViewModel = SystemViewModel->GetScalabilityViewModel();

	// if our emitter handles change, we rebind the
	SystemViewModel->OnEmitterHandleViewModelsChanged().AddSP(this, &FNiagaraSystemToolkitMode_Scalability::BindEmitterUpdates);
	SystemViewModel->GetSystem().OnSystemPostEditChange().AddSP(this, &FNiagaraSystemToolkitMode_Scalability::BindSystemPreviewOverrides);

	BindEmitterUpdates();
	BindSystemPreviewOverrides(&SystemViewModel->GetSystem());
	
	FNiagaraPlatformSet::InvalidateCachedData();
	SystemViewModel->GetSystem().UpdateScalability();

	ScalabilityViewModel->OnScalabilityModeChanged().Broadcast(true);
}

void FNiagaraSystemToolkitMode_Scalability::UnbindEmitterUpdates()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemToolkit.Pin()->GetSystemViewModel();

	for(const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
		if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Standard)
		{
			EmitterHandle->GetInstance().Emitter->OnRenderersChanged().RemoveAll(this);
			EmitterHandle->GetInstance().Emitter->OnPropertiesChanged().RemoveAll(this);
		}
		else if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Stateless)
		{
			EmitterHandle->GetStatelessEmitter()->OnRenderersChanged().RemoveAll(this);
		}
	}
}

void FNiagaraSystemToolkitMode_Scalability::BindEmitterPreviewOverrides()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemToolkit.Pin()->GetSystemViewModel();
	UNiagaraSystemScalabilityViewModel* ScalabilityViewModel = SystemViewModel->GetScalabilityViewModel();
	
	for(const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
		if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Standard)
		{
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle->GetEmitterData();

			// we bind the quality level and device profile of the emitter itself to use our scalability preview
			EmitterData->Platforms.OnOverrideQualityLevelDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewQualityLevel);
			EmitterData->Platforms.OnOverrideActiveDeviceProfileDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewDeviceProfile);

			// we do the same for the overrides
			for (FNiagaraEmitterScalabilityOverride& EmitterScalabilityOverride : EmitterData->ScalabilityOverrides.Overrides)
			{
				EmitterScalabilityOverride.Platforms.OnOverrideQualityLevelDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewQualityLevel);
				EmitterScalabilityOverride.Platforms.OnOverrideActiveDeviceProfileDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewDeviceProfile);
			}

			// and also for the renderers
			EmitterData->ForEachRenderer([=](UNiagaraRendererProperties* RendererProperties)
			{
				RendererProperties->Platforms.OnOverrideQualityLevelDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewQualityLevel);
				RendererProperties->Platforms.OnOverrideActiveDeviceProfileDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewDeviceProfile);
			});
		}
		else if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Stateless)
		{
			UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandle->GetStatelessEmitter();

			// we bind the quality level and device profile of the emitter itself to use our scalability preview
			StatelessEmitter->GetPlatformSet().OnOverrideQualityLevelDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewQualityLevel);
			StatelessEmitter->GetPlatformSet().OnOverrideActiveDeviceProfileDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewDeviceProfile);

			// we do the same for the overrides
			for (FNiagaraEmitterScalabilityOverride& EmitterScalabilityOverride : StatelessEmitter->GetScalabilityOverrides().Overrides)
			{
				EmitterScalabilityOverride.Platforms.OnOverrideQualityLevelDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewQualityLevel);
				EmitterScalabilityOverride.Platforms.OnOverrideActiveDeviceProfileDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewDeviceProfile);
			}

			// and also for the renderers
			StatelessEmitter->ForEachRenderer([=](UNiagaraRendererProperties* RendererProperties)
			{
				RendererProperties->Platforms.OnOverrideQualityLevelDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewQualityLevel);
				RendererProperties->Platforms.OnOverrideActiveDeviceProfileDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewDeviceProfile);
			});
		}
	}
}

void FNiagaraSystemToolkitMode_Scalability::BindSystemPreviewOverrides(UNiagaraSystem* InSystem)
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemToolkit.Pin()->GetSystemViewModel();
	UNiagaraSystemScalabilityViewModel* ScalabilityViewModel = SystemViewModel->GetScalabilityViewModel();

	SystemViewModel->GetSystem().GetScalabilityPlatformSet().OnOverrideQualityLevelDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewQualityLevel);
	for(FNiagaraSystemScalabilityOverride& SystemScalabilityOverride : SystemViewModel->GetSystem().GetSystemScalabilityOverrides().Overrides)
	{
		SystemScalabilityOverride.Platforms.OnOverrideQualityLevelDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewQualityLevel);
		SystemScalabilityOverride.Platforms.OnOverrideActiveDeviceProfileDelegate.BindUObject(ScalabilityViewModel, &UNiagaraSystemScalabilityViewModel::GetPreviewDeviceProfile);
	}
}

void FNiagaraSystemToolkitMode_Scalability::UnbindEmitterPreviewOverrides()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemToolkit.Pin()->GetSystemViewModel();

	// we undo all the bindings we created in the Bind function here 
	for(const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
		if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Standard)
		{
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandleViewModel->GetEmitterHandle()->GetEmitterData();
			EmitterData->Platforms.OnOverrideQualityLevelDelegate.Unbind();
			EmitterData->Platforms.OnOverrideActiveDeviceProfileDelegate.Unbind();
		
			for(FNiagaraEmitterScalabilityOverride& EmitterScalabilityOverride : EmitterData->ScalabilityOverrides.Overrides)
			{
				EmitterScalabilityOverride.Platforms.OnOverrideQualityLevelDelegate.Unbind();
				EmitterScalabilityOverride.Platforms.OnOverrideActiveDeviceProfileDelegate.Unbind();
			}

			EmitterData->ForEachRenderer([](UNiagaraRendererProperties* RendererProperties)
			{
				RendererProperties->Platforms.OnOverrideQualityLevelDelegate.Unbind();
				RendererProperties->Platforms.OnOverrideActiveDeviceProfileDelegate.Unbind();
			});
		}
		else if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Stateless)
		{
			UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandle->GetStatelessEmitter();

			StatelessEmitter->GetPlatformSet().OnOverrideQualityLevelDelegate.Unbind();
			StatelessEmitter->GetPlatformSet().OnOverrideActiveDeviceProfileDelegate.Unbind();

			for (FNiagaraEmitterScalabilityOverride& EmitterScalabilityOverride : StatelessEmitter->GetScalabilityOverrides().Overrides)
			{
				EmitterScalabilityOverride.Platforms.OnOverrideQualityLevelDelegate.Unbind();
				EmitterScalabilityOverride.Platforms.OnOverrideActiveDeviceProfileDelegate.Unbind();
			}

			StatelessEmitter->ForEachRenderer([](UNiagaraRendererProperties* RendererProperties)
			{
				RendererProperties->Platforms.OnOverrideQualityLevelDelegate.Unbind();
				RendererProperties->Platforms.OnOverrideActiveDeviceProfileDelegate.Unbind();
			});
		}
	}
}

void FNiagaraSystemToolkitMode_Scalability::UnbindSystemPreviewOverrides()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemToolkit.Pin()->GetSystemViewModel();

	SystemViewModel->GetSystem().GetScalabilityPlatformSet().OnOverrideQualityLevelDelegate.Unbind();
	for(FNiagaraSystemScalabilityOverride& SystemScalabilityOverride : SystemViewModel->GetSystem().GetSystemScalabilityOverrides().Overrides)
	{
		SystemScalabilityOverride.Platforms.OnOverrideQualityLevelDelegate.Unbind();
		SystemScalabilityOverride.Platforms.OnOverrideActiveDeviceProfileDelegate.Unbind();
	}
}

void FNiagaraSystemToolkitMode_Scalability::BindEmitterUpdates()
{
	// we make sure to remove all existing multicast bindings first
	UnbindEmitterUpdates();

	// if our emitter changes properties (= platforms) or a renderer changes (= platforms), we refresh the bindings to make sure all new values get properly overridden
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemToolkit.Pin()->GetSystemViewModel();
	for(const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
		if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Standard)
		{
			EmitterHandle->GetInstance().Emitter->OnRenderersChanged().AddSP(this, &FNiagaraSystemToolkitMode_Scalability::BindEmitterPreviewOverrides);
			EmitterHandle->GetInstance().Emitter->OnPropertiesChanged().AddSP(this, &FNiagaraSystemToolkitMode_Scalability::BindEmitterPreviewOverrides);
		}
		else if ( EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Stateless )
		{
			EmitterHandle->GetStatelessEmitter()->OnRenderersChanged().AddSP(this, &FNiagaraSystemToolkitMode_Scalability::BindEmitterPreviewOverrides);
		}
	}

	BindEmitterPreviewOverrides();
}

void FNiagaraSystemToolkitMode_Scalability::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FNiagaraSystemToolkit* Toolkit)
		{
			ToolbarBuilder.BeginSection("Scalability");
			{
				FUIAction ScalabilityToggleAction(FExecuteAction::CreateRaw(Toolkit, &FNiagaraSystemToolkit::SetCurrentMode, FNiagaraSystemToolkitModeBase::DefaultModeName));
				ScalabilityToggleAction.GetActionCheckState = FGetActionCheckState::CreateLambda([Toolkit]()
				{
					return Toolkit->GetCurrentMode() == FNiagaraSystemToolkitModeBase::ScalabilityModeName ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				});
						
				ToolbarBuilder.AddToolBarButton(
					ScalabilityToggleAction,
					NAME_None, 
					LOCTEXT("ScalabilityLabel", "Scalability"),
					LOCTEXT("ScalabilityTooltip", "Turn off scalability mode."),
					FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.Scalability"),
					EUserInterfaceActionType::ToggleButton
				);
			}
			ToolbarBuilder.EndSection();		
		}		
	};
	
	ToolbarExtender->AddToolBarExtension(
    	"Asset",
    	EExtensionHook::After,
    	SystemToolkit.Pin()->GetToolkitCommands(),
    	FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, SystemToolkit.Pin().Get())
    	);
}

TSharedRef<SDockTab> FNiagaraSystemToolkitMode_Scalability::SpawnTab_ScalabilityContext(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == ScalabilityContextTabID);

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[
			SAssignNew(ScalabilityContext, SNiagaraScalabilityContext, *SystemToolkit.Pin()->GetSystemViewModel()->GetScalabilityViewModel())
		];
	
	return SpawnedTab;
}

TSharedRef<SDockTab> FNiagaraSystemToolkitMode_Scalability::SpawnTab_ResolvedScalability(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == ResolvedScalabilityTabID);

	TSharedRef<SDockTab> SpawnedTab =
		SNew(SDockTab)
		[			
			SNew(SNiagaraSystemResolvedScalabilitySettings, SystemToolkit.Pin()->GetSystemViewModel()->GetSystem(), SystemToolkit.Pin()->GetSystemViewModel())
		];

	return SpawnedTab;
}

TSharedRef<FTabManager::FLayout> FNiagaraSystemToolkitMode_Scalability::ChooseTabLayout()
{
	if(SystemToolkit.Pin()->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		return FTabManager::NewLayout("Standalone_Niagara_System_Layout_Scalability_v31")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				// Main Content Area
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				//->SetSizeCoefficient(0.75f)
				// Left area
				->Split
				(
					// Main Content Area - Left
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(.75f)
					->Split
					(
						// Inner Left
						FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
						->SetSizeCoefficient(.75f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.4f)
							->AddTab(ViewportTabID, ETabState::OpenedTab)
						)
						// Inner Right
						->Split
						(
							FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
							->Split
							(
								FTabManager::NewStack()
								->SetSizeCoefficient(0.8f)
								->AddTab(SystemOverviewTabID, ETabState::OpenedTab)
								->SetForegroundTab(SystemOverviewTabID)
							)
						)
					)
					->Split
					(
						// Inner Left Bottom
						FTabManager::NewStack()
						->SetSizeCoefficient(0.25f)
						->AddTab(CurveEditorTabID, ETabState::OpenedTab)
						->AddTab(MessageLogTabID, ETabState::OpenedTab)
						->AddTab(SequencerTabID, ETabState::OpenedTab)
						->AddTab(ScriptStatsTabID, ETabState::ClosedTab)
					)
				)
				// Right Sidebar Area
				->Split
				(
					// Top Level Right
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.35f)
						->AddTab(ScalabilityContextTabID, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.3f)
						->AddTab(ResolvedScalabilityTabID, ETabState::OpenedTab)
						->AddTab(DetailsTabID, ETabState::OpenedTab)
						->AddTab(SelectedEmitterGraphTabID, ETabState::ClosedTab)
						->AddTab(SystemScriptTabID, ETabState::ClosedTab)
						->AddTab(PreviewSettingsTabId, ETabState::ClosedTab)
						->AddTab(GeneratedCodeTabID, ETabState::ClosedTab)
						->SetForegroundTab(ResolvedScalabilityTabID)
					)
				)
			)
		);
	}
	else if(SystemToolkit.Pin()->GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		return FTabManager::NewLayout("Standalone_Niagara_Emitter_Layout_Scalability_v1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
				->SetSizeCoefficient(0.75f)
				->Split
				(
					// Top Level Left
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(.75f)
					->Split
					(
						// Inner Left Top
						FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
						->SetSizeCoefficient(0.75f)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.4f)
							->AddTab(ViewportTabID, ETabState::OpenedTab)
						)
						->Split
						(							
							FTabManager::NewStack()
							->AddTab(SystemOverviewTabID, ETabState::OpenedTab)
							->SetForegroundTab(SystemOverviewTabID)							
						)
					)
					->Split
					(
						// Inner Left Bottom
						FTabManager::NewStack()
						->SetSizeCoefficient(0.25f)
						->AddTab(CurveEditorTabID, ETabState::OpenedTab)
						->AddTab(MessageLogTabID, ETabState::OpenedTab)
						->AddTab(SequencerTabID, ETabState::OpenedTab)
						->AddTab(ScriptStatsTabID, ETabState::ClosedTab)
					)
				)
				->Split
				(
					// Top Level Right
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.35f)
						->AddTab(ScalabilityContextTabID, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.65f)
						->AddTab(DetailsTabID, ETabState::OpenedTab)
						->AddTab(SelectedEmitterGraphTabID, ETabState::ClosedTab)
						->AddTab(SystemScriptTabID, ETabState::ClosedTab)
						->AddTab(PreviewSettingsTabId, ETabState::ClosedTab)
						->AddTab(GeneratedCodeTabID, ETabState::ClosedTab)
					)
				)
			)
		);
	}

	return FTabManager::NewLayout("DummyLayout");
}

#undef LOCTEXT_NAMESPACE
