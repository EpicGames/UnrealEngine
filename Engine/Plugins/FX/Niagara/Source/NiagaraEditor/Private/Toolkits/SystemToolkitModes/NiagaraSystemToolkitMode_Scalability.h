// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"
#include "Customizations/NiagaraPlatformSetCustomization.h"

class UNiagaraSystem;
class FNiagaraSystemToolkit;

class FNiagaraSystemToolkitMode_Scalability : public FNiagaraSystemToolkitModeBase
{
public:
	FNiagaraSystemToolkitMode_Scalability(TWeakPtr<FNiagaraSystemToolkit> InSystemToolkit);

	/** FApplicationMode interface */
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	virtual void PreDeactivateMode() override;
	virtual void PostActivateMode() override;

	void ExtendToolbar();
protected:
	TSharedRef<SDockTab> SpawnTab_ScalabilityContext(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_ResolvedScalability(const FSpawnTabArgs& Args);

private:
	TSharedRef<FTabManager::FLayout> ChooseTabLayout();
	void BindEmitterUpdates();
	void UnbindEmitterUpdates();

	void BindEmitterPreviewOverrides();
	void BindSystemPreviewOverrides(UNiagaraSystem* System);
	void UnbindEmitterPreviewOverrides();
	void UnbindSystemPreviewOverrides();

	TSharedPtr<class SNiagaraScalabilityContext> ScalabilityContext; 
public:
	static const FName ScalabilityContextTabID;
	static const FName ScalabilityViewModeTabID;
	static const FName ResolvedScalabilityTabID;
	static const FName EffectTypeTabID;
};
