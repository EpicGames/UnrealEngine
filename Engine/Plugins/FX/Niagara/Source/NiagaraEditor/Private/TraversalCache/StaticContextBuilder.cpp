// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraversalCache/StaticContextBuilder.h"

#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraCommon.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEmitter.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"
#include "TraversalCache/Traversal.h"
#include "TraversalCache/TraversalBuilder.h"
#include "TraversalCache/TraversalCache.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"

namespace UE::Niagara::TraversalCache
{

void FStaticContextBuilder::CreateTopLevelScriptContext(
	const UNiagaraSystem& System,
	const FVersionedNiagaraEmitterData* EmitterData,
	const UNiagaraScript& TargetScript,
	FTopLevelScriptStaticContext& OutTopLevelScriptContext,
	FTraversalDebugData* TraversalDebugData)
{
	ENiagaraScriptUsage TargetScriptUsage = TargetScript.GetUsage();
	bool bIsSystemScript = UNiagaraScript::IsSystemScript(TargetScriptUsage);
	bool bIsEmitterScript = UNiagaraScript::IsEmitterScript(TargetScriptUsage);
	bool bIsParticleScript = UNiagaraScript::IsParticleScript(TargetScriptUsage);
	if (ensureMsgf((bIsEmitterScript == false && bIsParticleScript == false) || EmitterData != nullptr, TEXT("Must supply emitter data when creating a context for an emitter or particle script.")) == false)
	{
		return;
	}

	// Gather context from owning objects and scripts that are evaluated before the target script.
	if (bIsSystemScript)
	{
		// When handling target scripts in the system, we only have to gather the static writes from system spawn,
		// when the target script is system update.
		if (TargetScript.IsEquivalentUsage(ENiagaraScriptUsage::SystemUpdateScript))
		{
			GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *System.GetSystemSpawnScript(), TraversalDebugData);
		}
	}
	else if (bIsEmitterScript || bIsParticleScript)
	{
		// All emitter and particle scripts need both the system spawn and system update static writes, as well as static values
		// gathered from the emitter object itself.
		GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *System.GetSystemSpawnScript(), TraversalDebugData);
		GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *System.GetSystemUpdateScript(), TraversalDebugData);
		GatherContextFromEmitterObject(OutTopLevelScriptContext.GlobalContext, *EmitterData);

		if (bIsEmitterScript)
		{
			// When handling target scripts in the emitter, the only additional static writes needed are emitter spawn static write
			// when the target script is emitter update.
			if (TargetScript.IsEquivalentUsage(ENiagaraScriptUsage::EmitterUpdateScript))
			{
				GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *EmitterData->EmitterSpawnScriptProps.Script, TraversalDebugData);
			}
		}
		else // bIsParticleScript
		{
			// Particle scripts need both the emitter spawn and emitter udpate static writes.
			GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *EmitterData->EmitterSpawnScriptProps.Script, TraversalDebugData);
			GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *EmitterData->EmitterUpdateScriptProps.Script, TraversalDebugData);

			if (TargetScript.IsEquivalentUsage(ENiagaraScriptUsage::ParticleUpdateScript))
			{
				// Particle update scripts need static writes from particle spawn.
				GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *EmitterData->SpawnScriptProps.Script, TraversalDebugData);
			}
			else if (TargetScript.IsEquivalentUsage(ENiagaraScriptUsage::ParticleEventScript) || TargetScript.IsEquivalentUsage(ENiagaraScriptUsage::ParticleSimulationStageScript))
			{
				// Event scripts and simulation stage scripts need static writes from particle spawn and particle update.
				GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *EmitterData->SpawnScriptProps.Script, TraversalDebugData);
				GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *EmitterData->UpdateScriptProps.Script, TraversalDebugData);
			
				if (TargetScript.IsEquivalentUsage(ENiagaraScriptUsage::ParticleSimulationStageScript))
				{
					// Simulation stage scripts need static writes from any stage that runs before them.
					for (UNiagaraSimulationStageBase* SimulationStageBase : EmitterData->GetSimulationStages())
					{
						if (SimulationStageBase->Script == &TargetScript)
						{
							break;
						}
						else
						{
							GatherStaticParametersWrittenByScript(OutTopLevelScriptContext.GlobalContext, *SimulationStageBase->Script, TraversalDebugData);
						}
					}
				}
			}
		}
	}

	// Lastly gather the context from the target script object.
	GatherContextFromScriptObject(OutTopLevelScriptContext.GlobalContext, OutTopLevelScriptContext.FunctionInputContexts, TargetScript, TraversalDebugData);
}

void FStaticContextBuilder::AddBoolValue(FStaticContext& TargetContext, ESelectKeySource KeySource, const FNiagaraVariableBase& BoolVariable, FName NamespaceModifier, bool bValue)
{
	FSelectKey BoolKey(KeySource, BoolVariable, NamespaceModifier);
	FSelectValue BoolValue = bValue ? FSelectValue::GetBoolTrue() : FSelectValue::GetBoolFalse();
	TargetContext.Add(BoolKey, BoolValue);
}

void FStaticContextBuilder::AddEnumValue(FStaticContext& TargetContext, ESelectKeySource KeySource, const FNiagaraVariableBase& EnumVariable, FName NamespaceModifier, int32 EnumValue)
{
	FSelectKey EnumKey(KeySource, EnumVariable, NamespaceModifier);
	FSelectValue EnumSelectValue = FTraversalBuilder::CreateSelectValue(EnumVariable.GetType().GetEnum(), EnumValue);
	TargetContext.Add(EnumKey, EnumSelectValue);
}

void FStaticContextBuilder::GatherContextFromEmitterObject(FStaticContext& StaticContext, const FVersionedNiagaraEmitterData& EmitterData)
{
	AddBoolValue(StaticContext, ESelectKeySource::ExternalConstant, SYS_PARAM_EMITTER_LOCALSPACE, NAME_None, EmitterData.bLocalSpace);
	AddBoolValue(StaticContext, ESelectKeySource::ExternalConstant, SYS_PARAM_EMITTER_DETERMINISM, NAME_None, EmitterData.bDeterminism);
	AddEnumValue(StaticContext, ESelectKeySource::ExternalConstant, SYS_PARAM_EMITTER_SIMULATION_TARGET, NAME_None, (int32)EmitterData.SimTarget);
}

void FStaticContextBuilder::GatherContextFromScriptObject(FStaticContext& StaticContext, TMap<FGuid, FStaticContext>& FunctionInputContexts, const UNiagaraScript& Script, FTraversalDebugData* TraversalDebugData)
{
	AddEnumValue(StaticContext, ESelectKeySource::ExternalConstant, SYS_PARAM_SCRIPT_USAGE,
		NAME_None, (int32)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchUsage(Script.Usage));
	AddEnumValue(StaticContext, ESelectKeySource::ExternalConstant, SYS_PARAM_SCRIPT_CONTEXT,
		NAME_None, (int32)FNiagaraUtilities::ConvertScriptUsageToStaticSwitchContext(Script.Usage));

	// This is a special case to handle emitter local space being read in system scripts which really shouldn't be supported since it's not
	// defined there, but there is already content that uses this and we need to results to be consistent.
	if (Script.GetUsage() == ENiagaraScriptUsage::SystemSpawnScript || Script.GetUsage() == ENiagaraScriptUsage::SystemUpdateScript)
	{
		AddBoolValue(StaticContext, ESelectKeySource::ExternalConstant, SYS_PARAM_EMITTER_LOCALSPACE, NAME_None, false);
	}

	FTraversalCache::GetTopLevelScriptFunctionInputContexts(Script, FunctionInputContexts, TraversalDebugData);
}

void FStaticContextBuilder::GatherStaticParametersWrittenByScript(FStaticContext& StaticContext, const UNiagaraScript& SourceScript, FTraversalDebugData* TraversalDebugData)
{
	FStaticContext ScriptGlobalContext = StaticContext;
	TMap<FGuid, FStaticContext> ScriptFunctionInputContexts;

	GatherContextFromScriptObject(ScriptGlobalContext, ScriptFunctionInputContexts, SourceScript, TraversalDebugData);

	FTraversalCallingContext CallingContext(ScriptGlobalContext, ScriptFunctionInputContexts);
	CallingContext.ConnectionTraversalMode = EConnectionTraversalMode::MatchingOnly;

	TSharedPtr<const FTraversal> SourceScriptTraversal = FTraversalCache::GetTopLevelScriptTraversal(SourceScript, TraversalDebugData);
	if (SourceScriptTraversal.IsValid() && SourceScriptTraversal->CanWriteStaticAttributes(TraversalDebugData))
	{
		FStaticContext StaticWrites;
		SourceScriptTraversal->Traverse(CallingContext, StaticWrites, TraversalDebugData);

		for (const TPair<FSelectKey, FSelectValue>& StaticWrite : StaticWrites)
		{
			if (StaticWrite.Key.Source == ESelectKeySource::Attribute)
			{
				StaticContext.Add(StaticWrite);
			}
		}
	}
}

} // UE::Niagara::TraversalCache
