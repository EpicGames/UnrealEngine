// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraVariant.h"
#include "TraversalCache/TraversalNode.h"
#include "TraversalCache/TraversalShared.h"

struct FNiagaraVariableBase;
class INiagaraEditorTypeUtilities;
class UEnum;
class UNiagaraScript;
class UNiagaraSystem;
struct FVersionedNiagaraEmitterData;

namespace UE::Niagara::TraversalCache
{

class FStaticContextBuilder
{
public:
	static NIAGARAEDITOR_API void CreateTopLevelScriptContext(
		const UNiagaraSystem& System,
		const FVersionedNiagaraEmitterData* EmitterData,
		const UNiagaraScript& TargetScript,
		FTopLevelScriptStaticContext& OutTopLevelScriptContext,
		FTraversalDebugData* TraversalDebugData = nullptr);

private:
	static void AddBoolValue(FStaticContext& TargetContext, ESelectKeySource KeySource, const FNiagaraVariableBase& BoolVariable, FName NamespaceModifier, bool bValue);
	static void AddEnumValue(FStaticContext& TargetContext, ESelectKeySource KeySource, const FNiagaraVariableBase& EnumVariable, FName NamespaceModifier, int32 EnumValue);
	static void GatherContextFromEmitterObject(FStaticContext& StaticContext, const FVersionedNiagaraEmitterData& EmitterData);
	static void GatherContextFromScriptObject(FStaticContext& StaticContext, TMap<FGuid, FStaticContext>& FunctionInputContexts,const UNiagaraScript& Script, FTraversalDebugData* TraversalDebugData);
	static void GatherStaticParametersWrittenByScript(FStaticContext& StaticContext, const UNiagaraScript& SourceScript, FTraversalDebugData* TraversalDebugData);
};

} // UE::Niagara::TraversalCache