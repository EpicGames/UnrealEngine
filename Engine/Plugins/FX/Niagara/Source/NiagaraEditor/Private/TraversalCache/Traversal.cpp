// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraversalCache/Traversal.h"


#include "NiagaraConstants.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraModule.h"
#include "TraversalCache/TraversalCache.h"
#include "TraversalCache/TraversalNode.h"

namespace UE::Niagara::TraversalCache
{

void FTraversal::Traverse(const FTraversalCallingContext& CallingContext, FStaticContext& OutLocalStaticContext, FTraversalDebugData* InOutDebugData) const
{
	TraverseInternal(CallingContext, OutLocalStaticContext, InOutDebugData);
}

TSharedRef<const FTraversalData> FTraversal::TraverseWithVisitor(const FTraversalCallingContext& CallingContext, const ITraversalVisitor& Visitor, FTraversalDebugData* InOutDebugData) const
{
	FStaticContext LocalStaticContext;
	return TraverseWithVisitorInternal(CallingContext, Visitor, LocalStaticContext, InOutDebugData);
}

bool FTraversal::CanWriteStatics(FTraversalDebugData* InOutDebugData) const
{
	UE::TScopeLock ScopeLock(CanWriteStaticsCacheGuard);

	if (bCanWriteStaticsCache.IsSet() == false)
	{
		TSet<const FTraversal*> CheckedTraversals;
		bCanWriteStaticsCache = CanWriteStaticsInternal(*this, CheckedTraversals, InOutDebugData);
	}
	return bCanWriteStaticsCache.GetValue();
}

bool FTraversal::CanWriteStaticAttributes(FTraversalDebugData* InOutDebugData) const
{
	UE::TScopeLock ScopeLock(CanWriteStaticAttributesCacheGuard);

	if (bCanWriteStaticAttributesCache.IsSet() == false)
	{
		TSet<const FTraversal*> CheckedTraversals;
		bCanWriteStaticAttributesCache = CanWriteStaticAttributesInternal(*this, CheckedTraversals, InOutDebugData);
	}
	return bCanWriteStaticAttributesCache.GetValue();
}

void FTraversal::ResetCachedData() const
{
	{
		UE::TScopeLock ScopeLock(HashToTraversalDataCacheGuard);
		HashToTraversalDataCache.Empty();
	}
	{ 
		UE::TScopeLock ScopeLock(CanWriteStaticsCacheGuard);
		bCanWriteStaticsCache.Reset();
	}
	{
		UE::TScopeLock ScopeLock(CanWriteStaticAttributesCacheGuard);
		bCanWriteStaticAttributesCache.Reset();
	}
}

void FTraversal::TraverseInternal(const FTraversalCallingContext& CallingContext, FStaticContext& OutLocalStaticContext, FTraversalDebugData* InOutDebugData) const
{
	UE::TScopeLock ScopeLock(HashToTraversalDataCacheGuard);

	FTraversalCallingContext::FContextHash CallingContextHash = CallingContext.GenerateHash();
	FCachedTraversalData* CachedTraversalData = HashToTraversalDataCache.Find(CallingContextHash);
	if (CachedTraversalData != nullptr)
	{
		OutLocalStaticContext = CachedTraversalData->LocalStaticContext;
		return;
	}

	FTraversalLocalContext LocalContext;
	LocalContext.TraversalState = CallingContext.TraversalState;
	TraverseNode(CallingContext, LocalContext, TraversalRoot.Get(), nullptr, nullptr, InOutDebugData);

	CachedTraversalData = &HashToTraversalDataCache.Add(CallingContextHash);
	CachedTraversalData->LocalStaticContext = LocalContext.StaticContext;
	OutLocalStaticContext = LocalContext.StaticContext;
}

TSharedRef<const FTraversalData> FTraversal::TraverseWithVisitorInternal(const FTraversalCallingContext& CallingContext, const ITraversalVisitor& Visitor,
	FStaticContext& OutLocalStaticContext, FTraversalDebugData* InOutDebugData) const
{
	UE::TScopeLock ScopeLock(HashToTraversalDataCacheGuard);

	FTraversalCallingContext::FContextHash CallingContextHash = CallingContext.GenerateHash();
	FCachedTraversalData* CachedTraversalData = HashToTraversalDataCache.Find(CallingContextHash);
	FGuid VisitorId = Visitor.GetVisitorId();
	if (CachedTraversalData != nullptr)
	{
		TSharedRef<const FTraversalData>* CachedVisitorData = CachedTraversalData->VisitorIdToTraversalData.Find(VisitorId);
		if (CachedVisitorData != nullptr)
		{
			OutLocalStaticContext = CachedTraversalData->LocalStaticContext;
			return *CachedVisitorData;
		}
	}
	 
	TSharedRef<FTraversalData> TraversalData = Visitor.CreateTraversalData();
	FTraversalLocalContext LocalContext;
	LocalContext.TraversalState = CallingContext.TraversalState;
	TraverseNode(CallingContext, LocalContext, TraversalRoot.Get(), &Visitor, &TraversalData.Get(), InOutDebugData);

	FTraversalLocalContext UnconnectedLocalContext;
	UnconnectedLocalContext.TraversalState = SetFlag(CallingContext.TraversalState, ETraversalStateFlags::UnconnectedRoot);
	for (const TSharedRef<FTraversalNode>& UnconnectedTraversalRoot : UnconnectedTraversalRoots)
	{
		TraverseNode(CallingContext, UnconnectedLocalContext, UnconnectedTraversalRoot.Get(), &Visitor, &TraversalData.Get(), InOutDebugData);
	}

	if (InOutDebugData == nullptr || InOutDebugData->HasData() == false)
	{
		// We only cache the results when either no debug data was supplied, or when the debug data is empty.  When
		// the debug data HasData returns true, errors were encountered while traversing or building.
		if (CachedTraversalData == nullptr)
		{
			CachedTraversalData = &HashToTraversalDataCache.Add(CallingContextHash);
		}
		CachedTraversalData->VisitorIdToTraversalData.Add(VisitorId, TraversalData);
		CachedTraversalData->LocalStaticContext = LocalContext.StaticContext;
	}
	OutLocalStaticContext = LocalContext.StaticContext;

	return TraversalData;
}

void FTraversal::TraverseNode(
	const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node,
	const ITraversalVisitor* Visitor, FTraversalData* TraversalData, FTraversalDebugData* DebugData) const
{
	bool bInSet = false;
	LocalContext.TraversedNodesWithState.Add(TPair<const FTraversalNode*, ETraversalStateFlags>(&Node,  LocalContext.TraversalState), &bInSet);
	if (bInSet)
	{
		return;
	}

	TOptional<FSelectConnectionData> SelectConnectionData;
	if (Node.SelectData.IsSet())
	{
		SelectConnectionData = FSelectConnectionData();
		ResolveSelectConnectionData(CallingContext, LocalContext, Node, SelectConnectionData.GetValue(), DebugData);
	}

	if (Node.Connections.Num() > 0)
	{
		TraverseConnections(CallingContext, LocalContext, Node, SelectConnectionData, Visitor, TraversalData, DebugData);
	}

	if (Node.FunctionCallData.IsSet())
	{
		TraverseFunction(CallingContext, LocalContext, Node, Visitor, TraversalData, DebugData);
	}

	if (Node.ParameterData.IsSet())
	{
		TraverseParameters(CallingContext, LocalContext, Node, DebugData);
	}

	if (Visitor != nullptr && TraversalData != nullptr)
	{
		Visitor->VisitNode(LocalContext.TraversalState, Node, *TraversalData);
	}
}

void FTraversal::TraverseConnections(
	const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node, 
	const TOptional<FSelectConnectionData>& SelectConnectionData, const ITraversalVisitor* Visitor, FTraversalData* TraversalData, FTraversalDebugData* DebugData) const
{
	for (const FTraversalNode::FConnection& Connection : Node.Connections)
	{
		bool bIsSelectableConnection = SelectConnectionData.IsSet() && Connection.PinId != SelectConnectionData->SelectConnectionPinId;
		bool bIsSelectedConnection = bIsSelectableConnection && Connection.PinId == SelectConnectionData->SelectedConnectionPinId;

		if (SelectConnectionData.IsSet() == false ||
			CallingContext.ConnectionTraversalMode == EConnectionTraversalMode::All ||
			(CallingContext.ConnectionTraversalMode == EConnectionTraversalMode::MatchingOnly && bIsSelectedConnection))
		{
			ETraversalStateFlags ConnectionStateFlags = bIsSelectableConnection && (bIsSelectedConnection == false)
				? SetFlag(LocalContext.TraversalState, ETraversalStateFlags::CulledBySwitch)
				: LocalContext.TraversalState;
			FScopedTraversalState ConnectionScopedTraversalState(LocalContext, ConnectionStateFlags);
			TraverseNode(CallingContext, LocalContext, Connection.GetNode(), Visitor, TraversalData, DebugData);
		}
	}
}

void FTraversal::TraverseFunction(
	const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node,
	const ITraversalVisitor* Visitor, FTraversalData* TraversalData, FTraversalDebugData* DebugData) const
{
	if (Node.FunctionCallData->FunctionScriptReference.Path.IsValid() == false ||
		HasFlag(LocalContext.TraversalState, ETraversalStateFlags::UnconnectedRoot))
	{
		return;
	}

	TSharedPtr<const FTraversal> CalledFunctionTraversal = FTraversalCache::GetScriptAssetTraversal(
		Node.FunctionCallData->FunctionScriptReference.Path, Node.FunctionCallData->FunctionScriptReference.Version, DebugData);
	if (CalledFunctionTraversal.IsValid() == false)
	{
		return;
	}

	const TMap<FGuid, FStaticContext> ExternalFunctionInputStaticContexts;
	FTraversalCallingContext FunctionCallContext(CallingContext.GlobalStaticContext, ExternalFunctionInputStaticContexts);
	SetupFunctionCallTraversalContext(CallingContext, LocalContext, Node, FunctionCallContext);
	SetupFunctionCallStaticContext(CallingContext, LocalContext, Node, FunctionCallContext.FunctionCallStaticContext, DebugData);

	FStaticContext TraversedFunctionLocalStaticContext;
	if (Visitor != nullptr && TraversalData != nullptr)
	{
		TSharedRef<const FTraversalData> FunctionTraversalData = CalledFunctionTraversal->TraverseWithVisitorInternal(FunctionCallContext, *Visitor, TraversedFunctionLocalStaticContext, DebugData);
		TraversalData->CalledFunctionTraversalData.Add(FFunctionCallTraversalData(Node.SourceNodeGuid, FunctionTraversalData));
	}
	else if (CalledFunctionTraversal->CanWriteStatics(DebugData) && Node.bSourceNodeEnabled)
	{
		CalledFunctionTraversal->TraverseInternal(FunctionCallContext, TraversedFunctionLocalStaticContext, DebugData);
	}

	if (Node.bSourceNodeEnabled && HasFlag(LocalContext.TraversalState, ETraversalStateFlags::CulledBySwitch) == false)
	{
		UpdateLocalStaticContextFromFunctionStaticWrites(TraversedFunctionLocalStaticContext, Node.FunctionCallData.GetValue(), LocalContext.StaticContext);
	}
}

void FTraversal::TraverseParameters(const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node, FTraversalDebugData* DebugData) const
{
	if (Node.ParameterData->WriteParameterReferences.Num() > 0 &&
		Node.bSourceNodeEnabled &&
		HasFlag(LocalContext.TraversalState, ETraversalStateFlags::CulledBySwitch) == false &&
		HasFlag(LocalContext.TraversalState, ETraversalStateFlags::UnconnectedRoot) == false)
	{
		UpdateLocalContextFromParameterData(CallingContext, LocalContext, Node, DebugData);
	}
}

void FTraversal::SetupFunctionCallStaticContext(
	const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext, const FTraversalNode& Node,
	FStaticContext& FunctionCallStaticContext, FTraversalDebugData* DebugData)
{
	const FFunctionCallData& FunctionCallData = Node.FunctionCallData.GetValue();

	// Initialize the function context with matching external function input context which will contain static inputs set externally.
	const FStaticContext* FunctionCallInputContext = CallingContext.ExternalFunctionInputStaticContexts.Find(Node.SourceNodeGuid);
	if (FunctionCallInputContext != nullptr)
	{
		FunctionCallStaticContext.Append(*FunctionCallInputContext);
	}

	// Add any matching local per-function input context which will contain static inputs set in the calling context.
	const FStaticContext* LocalFunctionCallInputContext = LocalContext.FunctionInputStaticContexts.Find(Node.SourceNodeGuid);
	if (LocalFunctionCallInputContext != nullptr)
	{
		FunctionCallStaticContext.Append(*LocalFunctionCallInputContext);
	}

	// Add static values from inputs on the function call node, either from static switches or static inputs set directly, static inputs
	// connected to other nodes, or from propagated static switches.
	for (const FFunctionInputSelectValue& InputSelectValue : FunctionCallData.InputSelectValues)
	{
		if (InputSelectValue.ConnectionPinId.IsSet())
		{
			const FTraversalNode* ConnectedNode = Node.GetConnectedNodeByPinId(InputSelectValue.ConnectionPinId.GetValue());
			if (ConnectedNode != nullptr)
			{
				TSet<const FTraversalNode*> ResolveTraversedNodes;
				TOptional<FSelectValue> ConnectedInputValue = ResolveSelectValueForNode(CallingContext, LocalContext, *ConnectedNode, ResolveTraversedNodes, DebugData);
				if (ConnectedInputValue.IsSet())
				{
					FunctionCallStaticContext.Add(InputSelectValue.InputSelectKey, ConnectedInputValue.GetValue());
				}
			}
		}
		else if (InputSelectValue.LocalValue.IsSet())
		{
			FunctionCallStaticContext.Add(InputSelectValue.InputSelectKey, InputSelectValue.LocalValue.GetValue());
		}
		else
		{
			// If the select value wasn't supplied in the function reference or in a connected node, it's using switch propagation, so check the calling
			// context to see if it has a value, and if so, use that.
			FSelectKey CurrentKey;
			if (InputSelectValue.OptionalPropagatedNameOverride.IsSet() == false)
			{
				CurrentKey = InputSelectValue.InputSelectKey;
			}
			else
			{
				CurrentKey = FSelectKey(
					ESelectKeySource::FunctionCallNode,
					FNiagaraVariableBase(InputSelectValue.InputSelectKey.Variable.GetType(), InputSelectValue.OptionalPropagatedNameOverride.GetValue()),
					NAME_None);
			}

			TOptional<FSelectValue> CurrentValue = CallingContext.FunctionCallStaticContext.GetSelectValue(CurrentKey);
			if (CurrentValue.IsSet())
			{
				FunctionCallStaticContext.Add(InputSelectValue.InputSelectKey, CurrentValue.GetValue());
			}
		}
	}

	// Lastly set the debug state based on the function reference, if it hasn't been set externally.
	FSelectKey DebugStateKey(ESelectKeySource::ExternalConstant, SYS_PARAM_FUNCTION_DEBUG_STATE, NAME_None);
	if (CallingContext.GlobalStaticContext.Contains(DebugStateKey) == false)
	{
		if (FunctionCallData.DebugState == ENiagaraFunctionDebugState::NoDebug)
		{
			FunctionCallStaticContext.Add(DebugStateKey, FSelectValue::GetDebugStateNoDebug());
		}
		else if (FunctionCallData.DebugState == ENiagaraFunctionDebugState::Basic)
		{
			FunctionCallStaticContext.Add(DebugStateKey, FSelectValue::GetDebugStateBasic());
		}
		else
		{
			ensureMsgf(false, TEXT("Traversal encountered an unknown debug state."));
		}
	}
}

void FTraversal::SetupFunctionCallTraversalContext(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, FTraversalCallingContext& FunctionCallContext)
{
	FunctionCallContext.FunctionCallStack = CallingContext.FunctionCallStack;
	FunctionCallContext.FunctionCallStack.Add(Node.SourceNodeGuid);

	FunctionCallContext.CallingStaticContext = CallingContext.CallingStaticContext;
	FunctionCallContext.CallingStaticContext.Append(LocalContext.StaticContext);

	FunctionCallContext.TraversalState = Node.bSourceNodeEnabled
		? LocalContext.TraversalState
		: SetFlag(LocalContext.TraversalState, ETraversalStateFlags::CallerDisabled);
}

void FTraversal::UpdateLocalStaticContextFromFunctionStaticWrites(const FStaticContext& FunctionCallLocalStaticContext, const FFunctionCallData& FunctionCallData, FStaticContext& LocalStaticContext)
{
	for (const TPair<FSelectKey, FSelectValue>& FunctionCallLocalStaticPair : FunctionCallLocalStaticContext)
	{
		if (FunctionCallLocalStaticPair.Key.NamespaceModifier == NAME_None)
		{
			if (FunctionCallLocalStaticPair.Key.Source == ESelectKeySource::Attribute)
			{
				LocalStaticContext.Add(FunctionCallLocalStaticPair);
			}
		}
		else
		{
			if (FunctionCallLocalStaticPair.Key.Source == ESelectKeySource::Attribute ||
				FunctionCallLocalStaticPair.Key.Source == ESelectKeySource::ModuleOutput)
			{
				FSelectKey UpdatedKey(FunctionCallLocalStaticPair.Key.Source, FunctionCallLocalStaticPair.Key.Variable, FunctionCallData.FunctionCallName);
				LocalStaticContext.Add(UpdatedKey, FunctionCallLocalStaticPair.Value);
			}
		}
	}
}

void FTraversal::UpdateLocalContextFromParameterData(const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node, FTraversalDebugData* DebugData)
{
	for (const FParameterWrite& WriteParameterReference : Node.ParameterData->WriteParameterReferences)
	{
		if (WriteParameterReference.Parameter.GetType().IsStatic())
		{
			TOptional<FSelectValue> WriteValue;
			if (WriteParameterReference.OptionalLocalSelectValue.IsSet())
			{
				WriteValue = WriteParameterReference.OptionalLocalSelectValue;
			}
			else if (WriteParameterReference.OptionalConnectionPinId.IsSet())
			{
				const FTraversalNode* WriteConnectedNode = Node.GetConnectedNodeByPinId(WriteParameterReference.OptionalConnectionPinId.GetValue());
				if (WriteConnectedNode != nullptr)
				{
					TSet<const FTraversalNode*> ResolveTraversedNodes;
					WriteValue = ResolveSelectValueForNode(CallingContext, LocalContext, *WriteConnectedNode, ResolveTraversedNodes, DebugData);
				} 
			}
			if (WriteValue.IsSet())
			{
				if (WriteParameterReference.OptionalTargetFunctionCallNodeGuid.IsSet())
				{
					LocalContext.FunctionInputStaticContexts.FindOrAdd(WriteParameterReference.OptionalTargetFunctionCallNodeGuid.GetValue())
						.Add(FSelectKey(ESelectKeySource::ModuleInput, WriteParameterReference.Parameter, NAME_None), WriteValue.GetValue());
				}
				else
				{
					ESelectKeySource SelectKeySource = ParameterFlagsToSelectKeySource(WriteParameterReference.Flags);
					FName NamespaceModifier = WriteParameterReference.OptionalNamespaceModifier.IsSet() 
						? WriteParameterReference.OptionalNamespaceModifier.GetValue()
						: NAME_None;
					LocalContext.StaticContext.Add(FSelectKey(SelectKeySource, WriteParameterReference.Parameter, NamespaceModifier), WriteValue.GetValue());
				}
			}
		}
	}
}

void FTraversal::ResolveSelectConnectionData(const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node, FTraversal::FSelectConnectionData& OutSelectConnectionData, FTraversalDebugData* DebugData)
{
	OutSelectConnectionData.SelectConnectionPinId = Node.SelectData->SelectConnectionPinId;
	if (Node.SelectData->SelectMode == ESelectMode::None)
	{
		return;
	}

	TSet<const FTraversalNode*> ResolveTraversedNodes;
	TOptional<FSelectValue> SelectValue = ResolveSelectValueForSelectData(CallingContext, LocalContext, Node, ResolveTraversedNodes, DebugData);
	if (SelectValue.IsSet() == false)
	{
		if (DebugData != nullptr &&	HasFlag(LocalContext.TraversalState, ETraversalStateFlags::CulledBySwitch) == false)
		{
			DebugData->AddUnresolvedSelect(CallingContext.FunctionCallStack, Node.SourceNodeGuid, Node.SelectData->SelectKey);
		}
		return;
	}

	const FSelectInputData* SelectInputData = Node.SelectData->FindInputDataForSelectValue(SelectValue.GetValue());
	if (SelectInputData == nullptr)
	{
		if (DebugData != nullptr)
		{
			DebugData->AddUnresolvedSelectInput(CallingContext.FunctionCallStack, Node.SourceNodeGuid, Node.SelectData->SelectKey, SelectValue.GetValue());
		}
		return;
	}

	OutSelectConnectionData.SelectedConnectionPinId = SelectInputData->ConnectionPinId.IsSet() ? SelectInputData->ConnectionPinId.GetValue() : FGuid();
}

TOptional<FSelectValue> FTraversal::ResolveSelectValueForSelectData(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	TOptional<FSelectValue> SelectValue;
	if (Node.SelectData->SelectMode == ESelectMode::Connection)
	{
		const FTraversalNode* ConnectedNode = Node.GetConnectedNodeByPinId(Node.SelectData->SelectConnectionPinId);
		if (ConnectedNode != nullptr)
		{
			SelectValue = ResolveSelectValueForNode(CallingContext, LocalContext, *ConnectedNode, ResolveTraversedNodes, DebugData);
		}
	}
	else if (Node.SelectData->SelectMode == ESelectMode::Value)
	{
		SelectValue = LocalContext.GetSelectValue(CallingContext, Node.SelectData->SelectKey);
	}
	return SelectValue;
}

TOptional<FSelectValue> FTraversal::ResolveSelectValueForNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	if (ResolveTraversedNodes.Contains(&Node))
	{
		return TOptional<FSelectValue>();
	}
	TScopedAddToSet<FTraversalNode> ScopedAddTraversedNode(ResolveTraversedNodes, Node);

	if (Node.FunctionInputData.IsSet())
	{
		return ResolveSelectValueForFunctionInputNode(CallingContext, LocalContext, Node, ResolveTraversedNodes, DebugData);
	}
	else if (Node.SelectData.IsSet())
	{
		return ResolveSelectValueForSelectNode(CallingContext, LocalContext, Node, ResolveTraversedNodes, DebugData);
	}
	else if (Node.ParameterData.IsSet())
	{
		return ResolveSelectValueForParameterNode(CallingContext, LocalContext, Node, ResolveTraversedNodes, DebugData);
	}
	else if (Node.StaticOpData.IsSet())
	{
		return ResolveSelectValueForOpNode(CallingContext, LocalContext, Node, ResolveTraversedNodes, DebugData);
	}
	else if (Node.FunctionCallData.IsSet())
	{
		return ResolveSelectValueForFunctionCallNode(CallingContext, LocalContext, Node, ResolveTraversedNodes, DebugData);
	}

	return TOptional<FSelectValue>();
}

TOptional<FSelectValue> FTraversal::ResolveSelectValueForFunctionInputNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	TOptional<FSelectValue> InputValue;
	InputValue = LocalContext.GetSelectValue(CallingContext, Node.FunctionInputData->InputSelectKey);
	if (InputValue.IsSet() == false && Node.FunctionInputData->LocalValue.IsSet())
	{
		InputValue = Node.FunctionInputData->LocalValue;
	}

	if (InputValue.IsSet() == false && DebugData != nullptr)
	{
		DebugData->AddUnresolvedFunctionInput(CallingContext.FunctionCallStack, Node.SourceNodeGuid, Node.FunctionInputData->InputSelectKey.Variable);
	}
	return InputValue;
}

TOptional<FSelectValue> FTraversal::ResolveSelectValueForSelectNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	TOptional<FSelectValue> SelectDataValue = ResolveSelectValueForSelectData(CallingContext, LocalContext, Node, ResolveTraversedNodes, DebugData);
	if (SelectDataValue.IsSet() == false)
	{
		return TOptional<FSelectValue>();
	}

	const FSelectInputData* SelectInputData = Node.SelectData->FindInputDataForSelectValue(SelectDataValue.GetValue());
	if (SelectInputData == nullptr ||
		(SelectInputData->LocalValue.IsSet() == false && SelectInputData->ConnectionPinId.IsSet() == false))
	{
		return TOptional<FSelectValue>();
	}

	if (SelectInputData->LocalValue.IsSet())
	{
		return SelectInputData->LocalValue;
	}

	const FTraversalNode* ConnectedNode = Node.GetConnectedNodeByPinId(SelectInputData->ConnectionPinId.GetValue());
	return ConnectedNode != nullptr
		? ResolveSelectValueForNode(CallingContext, LocalContext, *ConnectedNode, ResolveTraversedNodes, DebugData)
		: TOptional<FSelectValue>();
}

TOptional<FSelectValue> FTraversal::ResolveSelectValueForParameterNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	TOptional<FSelectValue> ReadValue;
	TArray<FParameterRead> NonDiscoverReads;
	for (const FParameterRead& ReadParameterReference : Node.ParameterData->ReadParameterReferences)
	{
		if (ReadParameterReference.bIsDiscoverRead == false)
		{
			NonDiscoverReads.Add(ReadParameterReference);
		}
	}

	if (NonDiscoverReads.Num() == 1 && NonDiscoverReads[0].Parameter.GetType().IsStatic())
	{
		const FParameterRead& NonDiscoverRead = NonDiscoverReads[0];

		ESelectKeySource KeySource = ParameterFlagsToSelectKeySource(NonDiscoverRead.Flags);
		FName NamespaceModifier = NonDiscoverRead.OptionalNamespaceModifier.IsSet()
			? NonDiscoverRead.OptionalNamespaceModifier.GetValue()
			: NAME_None;
		FSelectKey StaticValueSelectKey(KeySource, NonDiscoverRead.Parameter, NamespaceModifier);
		ReadValue = LocalContext.GetSelectValue(CallingContext, StaticValueSelectKey);

		if (ReadValue.IsSet() == false && NonDiscoverRead.OptionalLocalSelectValue.IsSet())
		{
			ReadValue = NonDiscoverRead.OptionalLocalSelectValue;
		}
		if (ReadValue.IsSet() == false && NonDiscoverRead.OptionalConnectionPinId.IsSet())
		{
			const FTraversalNode* ReadDefaultNode = Node.GetConnectedNodeByPinId(NonDiscoverRead.OptionalConnectionPinId.GetValue());
			if (ReadDefaultNode != nullptr)
			{
				ReadValue = ResolveSelectValueForNode(CallingContext, LocalContext, *ReadDefaultNode, ResolveTraversedNodes, DebugData);
			}
		}
		if (ReadValue.IsSet() == false)
		{
			const FTraversalNode* ExecutionConnectionNode = Node.GetConnectedNodeByPinId(Node.ParameterData->ExecutionConnectionPinId);
			if (ExecutionConnectionNode != nullptr)
			{
				ReadValue = ResolveStaticReadValueForNode(CallingContext, LocalContext, *ExecutionConnectionNode, StaticValueSelectKey, ResolveTraversedNodes, DebugData);
			}
		}

		if (ReadValue.IsSet() == false && DebugData != nullptr)
		{
			DebugData->AddUnresolvedRead(CallingContext.FunctionCallStack, Node.SourceNodeGuid, NonDiscoverRead.Parameter);
		}

		return ReadValue;
	}
	return TOptional<FSelectValue>();
}

TOptional<FSelectValue> FTraversal::ResolveSelectValueForOpNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	TArray<int32> OpInputs;
	bool bInputUnresolved = false;
	int32 UnresolvedPinIndex = INDEX_NONE;
	for (int32 InputIndex = 0; InputIndex < Node.StaticOpData->InputData.Num() && bInputUnresolved == false; ++InputIndex)
	{
		const FStaticOpInputData& InputData = Node.StaticOpData->InputData[InputIndex];
		TOptional<int32> InputValue;
		if (InputData.LocalValue.IsSet())
		{
			InputValue = InputData.LocalValue->NumericValue;
		}
		else if (InputData.ConnectionPinId.IsSet())
		{
			const FTraversalNode* ConnectedNode = Node.GetConnectedNodeByPinId(InputData.ConnectionPinId.GetValue());
			if (ConnectedNode != nullptr)
			{
				TOptional<FSelectValue> ResolvedInputValue = ResolveSelectValueForNode(CallingContext, LocalContext, *ConnectedNode, ResolveTraversedNodes, DebugData);
				if (ResolvedInputValue.IsSet())
				{
					InputValue = ResolvedInputValue->NumericValue;
				}
			}
		}

		if (InputValue.IsSet() == false)
		{
			bInputUnresolved = true;
			UnresolvedPinIndex = InputIndex;
		}
		else
		{
			OpInputs.Add(InputValue.GetValue());
		}
	}

	if (bInputUnresolved == false)
	{
		const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(Node.StaticOpData->OpName);
		int32 OpResult = OpInfo->StaticVariableResolveFunction.Execute(OpInputs);
		return FSelectValue(OpResult, "OpResult");
	}
	else
	{
		if (DebugData != nullptr)
		{
			DebugData->AddUnresolvedStaticOp(CallingContext.FunctionCallStack, Node.SourceNodeGuid, Node.StaticOpData->OpName, UnresolvedPinIndex);
		}
		return TOptional<FSelectValue>();
	}
}

TOptional<FSelectValue> FTraversal::ResolveSelectValueForFunctionCallNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	if (Node.FunctionCallData->FunctionScriptReference.Path.IsValid() == false)
	{
		return TOptional<FSelectValue>();
	}

	TSharedPtr<const FTraversal> ResolveFunctionTraversal = FTraversalCache::GetScriptAssetTraversal(Node.FunctionCallData->FunctionScriptReference.Path,
		Node.FunctionCallData->FunctionScriptReference.Version, DebugData);
	if (ResolveFunctionTraversal.IsValid() == false)
	{
		return TOptional<FSelectValue>();
	}

	const TMap<FGuid, FStaticContext> ExternalFunctionInputStaticContexts;
	FTraversalCallingContext FunctionCallContext(CallingContext.GlobalStaticContext, ExternalFunctionInputStaticContexts);
	SetupFunctionCallTraversalContext(CallingContext, LocalContext, Node, FunctionCallContext);
	SetupFunctionCallStaticContext(CallingContext, LocalContext, Node, FunctionCallContext.FunctionCallStaticContext, DebugData);

	FTraversalLocalContext FunctionLocalContext;
	return ResolveFunctionTraversal->ResolveSelectValueForFunctionTraversal(FunctionCallContext, DebugData);
}

TOptional<FSelectValue> FTraversal::ResolveSelectValueForFunctionTraversal(const FTraversalCallingContext& FunctionCallingContext, FTraversalDebugData* DebugData) const
{
	if (TraversalRoot->Connections.Num() == 1)
	{
		TSet<const FTraversalNode*> ResolveTraversedNodes;
		FTraversalLocalContext FunctionLocalContext;
		return ResolveSelectValueForNode(FunctionCallingContext, FunctionLocalContext, TraversalRoot->Connections[0].GetNode(), ResolveTraversedNodes, DebugData);
	}
	return TOptional<FSelectValue>();
}

TOptional<FSelectValue> FTraversal::ResolveStaticReadValueForNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, FSelectKey& ReadSelectKey, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	if (ResolveTraversedNodes.Contains(&Node))
	{
		return TOptional<FSelectValue>();
	}
	TScopedAddToSet<FTraversalNode> ScopedAddTraversedNode(ResolveTraversedNodes, Node);

	if (Node.SelectData.IsSet())
	{
		return ResolveStaticReadValueForSelectNode(CallingContext, LocalContext, Node, ReadSelectKey, ResolveTraversedNodes, DebugData);
	}
	else if (Node.ParameterData.IsSet())
	{
		return ResolveStaticReadValueForParameterNode(CallingContext, LocalContext, Node, ReadSelectKey, ResolveTraversedNodes, DebugData);
	}
	else if (Node.FunctionCallData.IsSet())
	{
		return ResolveStaticReadValueForFunctionCallNode(CallingContext, LocalContext, Node, ReadSelectKey, ResolveTraversedNodes, DebugData);
	}

	return TOptional<FSelectValue>();
}

TOptional<FSelectValue> FTraversal::ResolveStaticReadValueForSelectNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, FSelectKey& ReadSelectKey, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	TOptional<FSelectValue> SelectDataValue = ResolveSelectValueForSelectData(CallingContext, LocalContext, Node, ResolveTraversedNodes, DebugData);
	if (SelectDataValue.IsSet() == false)
	{
		return TOptional<FSelectValue>();
	}

	const FSelectInputData* SelectInputData = Node.SelectData->FindInputDataForSelectValue(SelectDataValue.GetValue());
	if (SelectInputData == nullptr || SelectInputData->ConnectionPinId.IsSet() == false)
	{
		return TOptional<FSelectValue>();
	}
	
	const FTraversalNode* ConnectedNode = Node.GetConnectedNodeByPinId(SelectInputData->ConnectionPinId.GetValue());
	return ConnectedNode != nullptr 
		? ResolveStaticReadValueForNode(CallingContext, LocalContext, *ConnectedNode, ReadSelectKey, ResolveTraversedNodes, DebugData)
		: TOptional<FSelectValue>();
}

TOptional<FSelectValue> FTraversal::ResolveStaticReadValueForParameterNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, FSelectKey& ReadSelectKey, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	const FParameterWrite* MatchingWriteParameterReference = Node.ParameterData->WriteParameterReferences.FindByPredicate([&ReadSelectKey]
		(const FParameterWrite& WriteParameterReference) { return WriteParameterReference.Parameter == ReadSelectKey.Variable; });
	if (MatchingWriteParameterReference != nullptr)
	{
		if (MatchingWriteParameterReference->OptionalLocalSelectValue.IsSet())
		{
			return MatchingWriteParameterReference->OptionalLocalSelectValue;
		}
		if (MatchingWriteParameterReference->OptionalConnectionPinId.IsSet())
		{
			const FTraversalNode* WriteValueNode = Node.GetConnectedNodeByPinId(MatchingWriteParameterReference->OptionalConnectionPinId.GetValue());
			if (WriteValueNode != nullptr)
			{
				return ResolveSelectValueForNode(CallingContext, LocalContext, *WriteValueNode, ResolveTraversedNodes, DebugData);
			}
		}
	}

	const FTraversalNode* ExecutionConnectionNode = Node.GetConnectedNodeByPinId(Node.ParameterData->ExecutionConnectionPinId);
	return ExecutionConnectionNode != nullptr
		? ResolveStaticReadValueForNode(CallingContext, LocalContext, *ExecutionConnectionNode, ReadSelectKey, ResolveTraversedNodes, DebugData)
		: TOptional<FSelectValue>();
}

TOptional<FSelectValue> FTraversal::ResolveStaticReadValueForFunctionCallNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
	const FTraversalNode& Node, FSelectKey& ReadSelectKey, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData)
{
	const FTraversalNode* ExecutionConnectionNode = Node.GetConnectedNodeByPinId(Node.FunctionCallData->ExecutionConnectionPinId);
	return ExecutionConnectionNode != nullptr
		? ResolveStaticReadValueForNode(CallingContext, LocalContext, *ExecutionConnectionNode, ReadSelectKey, ResolveTraversedNodes, DebugData)
		: TOptional<FSelectValue>();
}

bool FTraversal::CanWriteStaticsInternal(const FTraversal& Traversal, TSet<const FTraversal*>& CheckedTraversals, FTraversalDebugData* DebugData)
{
	bool bInSet = false;
	CheckedTraversals.Add(&Traversal, &bInSet);
	if (bInSet)
	{
		return false;
	}

	if (Traversal.StaticVariableWrites.Num() > 0)
	{
		return true;
	}

	for (const FScriptReference& ExternalReference : Traversal.ExternalReferences)
	{
		TSharedPtr<const FTraversal> ReferencedTraversal = FTraversalCache::GetScriptAssetTraversal(ExternalReference.Path, ExternalReference.Version, DebugData);
		if (ReferencedTraversal.IsValid() && CanWriteStaticsInternal(*ReferencedTraversal.Get(), CheckedTraversals, DebugData))
		{
			return true;
		}
	}
	return false;
}

bool FTraversal::CanWriteStaticAttributesInternal(const FTraversal& Traversal, TSet<const FTraversal*>& CheckedTraversals, FTraversalDebugData* DebugData)
{
	bool bInSet = false;
	CheckedTraversals.Add(&Traversal, &bInSet);
	if (bInSet)
	{
		return false;
	}

	if (Traversal.StaticVariableWritesToAttributes.Num() > 0)
	{
		return true;
	}

	for (const FScriptReference& ExternalReference : Traversal.ExternalReferences)
	{
		TSharedPtr<const FTraversal> ReferencedTraversal = FTraversalCache::GetScriptAssetTraversal(ExternalReference.Path, ExternalReference.Version, DebugData);
		if (ReferencedTraversal.IsValid() && CanWriteStaticAttributesInternal(*ReferencedTraversal.Get(), CheckedTraversals, DebugData))
		{
			return true;
		}
	}
	return false;
}

void FTraversal::GetAllConnectedNodes(const FTraversalNode& Node, TSet<const FTraversalNode*>& OutNodes)
{
	bool bInSet = false;
	OutNodes.Add(&Node, &bInSet);
	if (bInSet == false)
	{
		for (const FTraversalNode::FConnection& Connection : Node.Connections)
		{
			GetAllConnectedNodes(Connection.GetNode(), OutNodes);
		}
	}
}

ESelectKeySource FTraversal::ParameterFlagsToSelectKeySource(EParameterFlags ParameterFlags)
{
	ESelectKeySource SelectKeySource;
	if (HasFlag(ParameterFlags, EParameterFlags::Attribute))
	{
		SelectKeySource = ESelectKeySource::Attribute;
	}
	else if (HasFlag(ParameterFlags, EParameterFlags::ModuleInput))
	{
		SelectKeySource = ESelectKeySource::ModuleInput;
	}
	else if (HasFlag(ParameterFlags, EParameterFlags::ModuleLocal))
	{
		SelectKeySource = ESelectKeySource::ModuleLocal;
	}
	else if (HasFlag(ParameterFlags, EParameterFlags::ModuleOutput))
	{
		SelectKeySource = ESelectKeySource::ModuleOutput;
	}
	else
	{
		SelectKeySource = ESelectKeySource::None;
	}
	return SelectKeySource;
}

} // UE::Niagara::TraversalCache