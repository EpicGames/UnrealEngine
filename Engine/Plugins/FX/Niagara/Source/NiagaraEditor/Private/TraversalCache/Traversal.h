// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/RecursiveMutex.h"
#include "Delegates/DelegateCombinations.h"
#include "TraversalCache/TraversalShared.h"
#include "Async/Mutex.h"

namespace UE::Niagara::TraversalCache
{

struct FFunctionCallData;
struct FTraversalDebugData;
class  FTraversalNode;

class FTraversal
{
	friend class FTraversalBuilder;

public:
	FTraversal(TSharedRef<FTraversalNode> InTraversalRoot, TArray<TSharedRef<FTraversalNode>>& InUnconnectedTraversalRoots)
		: TraversalRoot(InTraversalRoot)
		, UnconnectedTraversalRoots(InUnconnectedTraversalRoots)
	{
	}

	void Traverse(const FTraversalCallingContext&, FStaticContext& OutLocalStaticContext, FTraversalDebugData* InOutDebugData = nullptr) const;

	TSharedRef<const FTraversalData> TraverseWithVisitor(const FTraversalCallingContext& CallingContext, const ITraversalVisitor& Visitor, FTraversalDebugData* InOutDebugData = nullptr) const;

	const TMap<FName, FGuid>& GetFunctionNameToNodeGuidMap() const { return FunctionNameToNodeGuidMap; }

	bool CanWriteStatics(FTraversalDebugData* InOutDebugData = nullptr) const;

	bool CanWriteStaticAttributes(FTraversalDebugData* InOutDebugData = nullptr) const;

	const TSet<FScriptReference>& GetExternalReferences() const { return ExternalReferences; }

	void ResetCachedData() const;

private:
	struct FScopedTraversalState
	{
		FScopedTraversalState(FTraversalLocalContext& InLocalContext, ETraversalStateFlags InRequestedState)
			: LocalContext(InLocalContext)
			, OriginalState(LocalContext.TraversalState)
			, RequestedState(InRequestedState)
		{
			LocalContext.TraversalState = RequestedState;
		}

		~FScopedTraversalState()
		{
			LocalContext.TraversalState = OriginalState;
		}

	private:
		FTraversalLocalContext& LocalContext;
		ETraversalStateFlags OriginalState;
		ETraversalStateFlags RequestedState;
	};

	template<typename TItem>
	class TScopedAddToSet
	{
	public:
		TScopedAddToSet(TSet<const TItem*>& InItemSet, const TItem& InItemToAdd)
			: ItemSet(InItemSet)
			, AddItem(InItemToAdd)
		{
			ItemSet.Add(&AddItem);
		}

		~TScopedAddToSet()
		{
			ItemSet.Remove(&AddItem);
		}

	private:
		TSet<const TItem*>& ItemSet;
		const TItem& AddItem;
	};

	struct FSelectConnectionData
	{
		TOptional<FGuid> SelectConnectionPinId;
		FGuid SelectedConnectionPinId;
	};

	void TraverseInternal(const FTraversalCallingContext& CallingContext, FStaticContext& OutLocalStaticContext, FTraversalDebugData* InOutDebugData) const;

	TSharedRef<const FTraversalData> TraverseWithVisitorInternal(const FTraversalCallingContext& CallingContext, const ITraversalVisitor& Visitor,
		FStaticContext& OutLocalStaticContext, FTraversalDebugData* InOutDebugData) const;

	void TraverseNode(const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node,
		const ITraversalVisitor* Visitor, FTraversalData* TraversalData, FTraversalDebugData* DebugData) const;

	void TraverseConnections(const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node,
		const TOptional<FSelectConnectionData>& SelectConnectionData, const ITraversalVisitor* Visitor, FTraversalData* TraversalData, FTraversalDebugData* DebugData) const;

	void TraverseFunction(const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node,
		const ITraversalVisitor* Visitor, FTraversalData* TraversalData, FTraversalDebugData* DebugData) const;

	void TraverseParameters(const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node, FTraversalDebugData* DebugData) const;

	static void SetupFunctionCallStaticContext(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, FStaticContext& OutFunctionCallStaticContext, FTraversalDebugData* DebugData);

	static void SetupFunctionCallTraversalContext(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, FTraversalCallingContext& OutFunctionCallContext);
	
	static void UpdateLocalStaticContextFromFunctionStaticWrites(const FStaticContext& FunctionCallLocalStaticContext, const FFunctionCallData& FunctionCallData, FStaticContext& LocalStaticContext);

	static void UpdateLocalContextFromParameterData(const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node, FTraversalDebugData* DebugData);

	static void ResolveSelectConnectionData(const FTraversalCallingContext& CallingContext, FTraversalLocalContext& LocalContext, const FTraversalNode& Node, FTraversal::FSelectConnectionData& OutSelectConnectionData, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveSelectValueForSelectData(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveSelectValueForNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node,	TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveSelectValueForFunctionInputNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveSelectValueForSelectNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveSelectValueForParameterNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveSelectValueForOpNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveSelectValueForFunctionCallNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	TOptional<FSelectValue> ResolveSelectValueForFunctionTraversal(const FTraversalCallingContext& FunctionCallingContext, FTraversalDebugData* DebugData) const;

	static TOptional<FSelectValue> ResolveStaticReadValueForNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node,	FSelectKey& ReadSelectKey, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveStaticReadValueForSelectNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, FSelectKey& ReadSelectKey, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveStaticReadValueForParameterNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, FSelectKey& ReadSelectKey, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static TOptional<FSelectValue> ResolveStaticReadValueForFunctionCallNode(const FTraversalCallingContext& CallingContext, const FTraversalLocalContext& LocalContext,
		const FTraversalNode& Node, FSelectKey& ReadSelectKey, TSet<const FTraversalNode*>& ResolveTraversedNodes, FTraversalDebugData* DebugData);

	static bool CanWriteStaticsInternal(const FTraversal& Traversal, TSet<const FTraversal*>& CheckedTraversals, FTraversalDebugData* DebugData);

	static bool CanWriteStaticAttributesInternal(const FTraversal& Traversal, TSet<const FTraversal*>& CheckedTraversals, FTraversalDebugData* DebugData);

	static void GetAllConnectedNodes(const FTraversalNode& Node, TSet<const FTraversalNode*>& OutNodes);

	static ESelectKeySource ParameterFlagsToSelectKeySource(EParameterFlags ParameterFlags);

private:
	TSharedRef<FTraversalNode> TraversalRoot;
	TArray<TSharedRef<FTraversalNode>> UnconnectedTraversalRoots;
	TSet<FScriptReference> ExternalReferences;
	TSet<FNiagaraVariableBase> StaticVariableReads;
	TSet<FNiagaraVariableBase> StaticVariableWrites;
	TSet<FNiagaraVariableBase> StaticVariableWritesToAttributes;
	TMap<FName, FGuid> FunctionNameToNodeGuidMap;

	struct FCachedTraversalData
	{
		TMap<FGuid, TSharedRef<const FTraversalData>> VisitorIdToTraversalData;
		FStaticContext LocalStaticContext;
	};

	mutable TMap<FTraversalCallingContext::FContextHash, FCachedTraversalData> HashToTraversalDataCache;
	mutable UE::FRecursiveMutex HashToTraversalDataCacheGuard;

	mutable TOptional<bool> bCanWriteStaticsCache;
	mutable UE::FMutex CanWriteStaticsCacheGuard;

	mutable TOptional<bool> bCanWriteStaticAttributesCache;
	mutable UE::FMutex CanWriteStaticAttributesCacheGuard;	
};

} // UE::Niagara::TraversalCache

