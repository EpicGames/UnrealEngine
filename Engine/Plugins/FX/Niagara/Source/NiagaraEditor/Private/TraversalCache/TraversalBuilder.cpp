// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraversalCache/TraversalBuilder.h"

#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_Niagara.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraNode.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOp.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeReroute.h"
#include "NiagaraNodeStaticSwitch.h"
#include "NiagaraNodeWithDynamicPins.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraTypes.h"
#include "TraversalCache/Traversal.h"
#include "TraversalCache/TraversalNode.h"

namespace UE::Niagara::TraversalCache
{

namespace BuilderHelpers
{
	void SortAndFilterInputs(TArray<const UEdGraphPin*>& InputPins)
	{
		TArray<const UEdGraphPin*> ParameterMapInputPins;
		TArray<const UEdGraphPin*> OtherNonAddPins;
		for (const UEdGraphPin* InputPin : InputPins)
		{
			if (InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryMisc &&
				InputPin->PinType.PinSubCategory == UNiagaraNodeWithDynamicPins::AddPinSubCategory)
			{
				continue;
			}

			FNiagaraTypeDefinition PinTypeDefinition = UEdGraphSchema_Niagara::PinToTypeDefinition(InputPin);
			if (PinTypeDefinition.IsValid())
			{
				if (PinTypeDefinition == FNiagaraTypeDefinition::GetParameterMapDef())
				{
					ParameterMapInputPins.Add(InputPin);
				}
				else
				{
					OtherNonAddPins.Add(InputPin);
				}
			}
		}

		InputPins.Empty(ParameterMapInputPins.Num() + OtherNonAddPins.Num());
		InputPins.Append(ParameterMapInputPins);
		InputPins.Append(OtherNonAddPins);
	}

	void ResolveNamespaceModifier(FParameterReference& ParameterReference)
	{
		TArray<FString> ParameterNameParts;
		ParameterReference.Parameter.GetName().ToString().ParseIntoArray(ParameterNameParts, TEXT("."));

		FName NamespaceModifier = *ParameterNameParts[1];
		ParameterNameParts[1] = "__MODIFIER__";

		FName ResolvedParameterName = *FString::Join(ParameterNameParts, TEXT("."));
		ParameterReference.Parameter = FNiagaraVariableBase(ParameterReference.Parameter.GetType(), ResolvedParameterName);
		ParameterReference.OptionalNamespaceModifier = NamespaceModifier;
	}

	// The static traversal needs a slightly different behavior here where a value is required when the pin
	// is not connected and the default value isn't ignored.  The default schema behavior is that an empty default
	// string will result in an unallocated variable, but we need the type default, and if needs value is true
	// all the time, then connected pins and pins that don't support defaults will always allocate data which is
	// also undesired.
	FNiagaraVariable TraversalPinToNiagaraVariable(const UEdGraphPin& InPin)
	{
		bool bNeedsValue = InPin.Direction == EGPD_Input &&
			InPin.bDefaultValueIsIgnored == false &&
			InPin.LinkedTo.Num() == 0 &&
			(InPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType ||
				InPin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType);
		FNiagaraVariable Variable = UEdGraphSchema_Niagara::PinToNiagaraVariable(&InPin, bNeedsValue);
		return Variable;
	}

	class FInputHandler : public FTraversalBuilder::FGraphNodeHandler
	{
		virtual bool CanProvideFunctionInput() const override { return true; }

		virtual void GetFunctionInputData(
			const FTraversalBuilder::FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FFunctionInputData>& OutInputData,
			FTraversalBuilderDebugData* DebugData) const override
		{
			if (GraphNodeAndOutputPin.OutputPin == nullptr)
			{
				return;
			}

			const UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(&GraphNodeAndOutputPin.Node);
			if (InputNode->Input.GetType().IsStatic() == false)
			{
				return;
			}

			FFunctionInputData InputData;
			InputData.InputSelectKey = FSelectKey(ESelectKeySource::FunctionCallNode, InputNode->Input, NAME_None);
			if (InputNode->Input.IsDataAllocated())
			{
				InputData.LocalValue = FTraversalBuilder::CreateSelectValue(InputNode->Input);
			}

			OutInputData = InputData;
		}
	};

	class FMapGetHandler : public FTraversalBuilder::FGraphNodeHandler
	{
	public:
		virtual bool CanAccessParameters() const override { return true; }

		virtual void GetParameterData(
			const FTraversalBuilder::FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FParameterData>& OutParameterData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const override
		{
			if (GraphNodeAndOutputPin.OutputPin == nullptr)
			{
				return;
			}

			FParameterData ParameterData;
			OutFilteredConnectedPinIds = TArray<FGuid>();

			const UNiagaraNodeParameterMapGet* GetNode = CastChecked<UNiagaraNodeParameterMapGet>(&GraphNodeAndOutputPin.Node);

			const UEdGraphPin& ReadPin = *GraphNodeAndOutputPin.OutputPin;
			FNiagaraVariable ReadVariable = TraversalPinToNiagaraVariable(ReadPin);
			EParameterFlags ReadFlags = FTraversalBuilder::ExtractFlagsFromParameterName(ReadVariable.GetName());
			FParameterRead& ReadParameterReference = ParameterData.ReadParameterReferences.Add_GetRef(
				FParameterRead(ReadVariable, ReadFlags));

			if (HasFlag(ReadFlags, EParameterFlags::NamespaceModifier))
			{
				ResolveNamespaceModifier(ReadParameterReference);
			}

			GetReadDefaultValues(*GetNode, ReadPin, ReadVariable, ReadParameterReference, OutFilteredConnectedPinIds, DebugData);

			// Handle reads that are "discovered" but not actually read in this traversal path.
			TArray<const UEdGraphPin*> OutputPins;
			GetNode->GetOutputPins(OutputPins);
			for (const UEdGraphPin* OutputPin : OutputPins)
			{
				if (GetNode->IsAddPin(OutputPin) || OutputPin == &ReadPin)
				{
					continue;
				}

				FNiagaraVariableBase DiscoveredVariable = TraversalPinToNiagaraVariable(*OutputPin);
				if (DiscoveredVariable.IsValid())
				{
					EParameterFlags DiscoveredFlags = FTraversalBuilder::ExtractFlagsFromParameterName(DiscoveredVariable.GetName());
					FParameterRead& DiscoveredParameterReference = ParameterData.ReadParameterReferences.Add_GetRef(
						FParameterRead(DiscoveredVariable, DiscoveredFlags));
					DiscoveredParameterReference.bIsDiscoverRead = true;
				}
			}

			const UEdGraphPin* ParameterMapInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*GetNode);
			if (ParameterMapInputPin != nullptr)
			{
				ParameterData.ExecutionConnectionPinId = ParameterMapInputPin->PinId;
				OutFilteredConnectedPinIds->Add(ParameterMapInputPin->PinId);
			}

			OutParameterData = ParameterData;
		}

		static void GetReadDefaultValues(
			const UNiagaraNodeParameterMapGet& GetNode,
			const UEdGraphPin& InOutputPin,
			const FNiagaraVariable& ReadVariable,
			FParameterRead& ReadParameterReference,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData)
		{
			if (ReadVariable.IsDataInterface())
			{
				return;
			}

			const UNiagaraScriptVariable* ReadScriptVariable = GetNode.GetNiagaraGraph()->GetScriptVariable(ReadVariable);
			if (ReadScriptVariable == nullptr)
			{
				return;
			}

			if (ReadScriptVariable->DefaultMode == ENiagaraDefaultMode::Custom)
			{
				TOptional<FNiagaraVariant> PinDefaultValue;
				const UEdGraphPin* DefaultPin = GetNode.GetDefaultPin(&InOutputPin);
				if (DefaultPin != nullptr)
				{
					if (DefaultPin->LinkedTo.Num() == 0)
					{
						FNiagaraVariable ReadDefaultVariable = TraversalPinToNiagaraVariable(*DefaultPin);
						if (ReadDefaultVariable.IsDataAllocated())
						{
							TArray<uint8> ReadDefaultVariableValue;
							ReadDefaultVariableValue.Append(ReadDefaultVariable.GetData(), ReadDefaultVariable.GetSizeInBytes());
							ReadParameterReference.OptionalDefaultValue = FNiagaraVariant(ReadDefaultVariableValue);
						}
					}
					else if (DefaultPin->LinkedTo.Num() == 1)
					{
						ReadParameterReference.OptionalConnectionPinId = DefaultPin->PinId;
						OutFilteredConnectedPinIds->Add(DefaultPin->PinId);
					}
				}
			}
			else if (ReadScriptVariable->DefaultMode == ENiagaraDefaultMode::Value)
			{
				if (ReadScriptVariable->Variable.IsDataAllocated())
				{
					ReadParameterReference.OptionalDefaultValue = FNiagaraVariant(ReadScriptVariable->Variable.GetData(), ReadScriptVariable->Variable.GetSizeInBytes());
				}
				else if (ReadScriptVariable->GetDefaultValueVariant().IsValid())
				{
					ReadParameterReference.OptionalDefaultValue = ReadScriptVariable->GetDefaultValueVariant();
				}
				else if (DebugData != nullptr)
				{
					FNiagaraVariable ReadVariableCopy = ReadScriptVariable->Variable;
					FNiagaraEditorUtilities::ResetVariableToDefaultValue(ReadVariableCopy);
					FNiagaraVariant ReadVariableVariant(ReadVariableCopy.GetData(), ReadVariableCopy.GetSizeInBytes());
					ReadParameterReference.OptionalDefaultValue = ReadVariableVariant;
				}
			}
			else if (ReadScriptVariable->DefaultMode == ENiagaraDefaultMode::Binding)
			{
				ReadParameterReference.OptionalDefaultBinding = ReadScriptVariable->DefaultBinding.GetName();
			}

			if (ReadParameterReference.OptionalDefaultValue.IsSet() && FTraversalBuilder::IsValidSelectValueType(ReadVariable.GetType()))
			{
				FNiagaraVariable Temp = ReadVariable;
				Temp.SetData(ReadParameterReference.OptionalDefaultValue->GetBytes());
				ReadParameterReference.OptionalLocalSelectValue = FTraversalBuilder::CreateSelectValue(Temp);
			}
		}
	};

	class FMapSetHandler : public FTraversalBuilder::FGraphNodeHandler
	{
		virtual bool CanAccessParameters() const override { return true; }

		virtual void GetParameterData(
			const FTraversalBuilder::FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FParameterData>& OutParameterData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const override
		{
			FParameterData ParameterData;

			const UNiagaraNodeParameterMapSet* SetNode = CastChecked<UNiagaraNodeParameterMapSet>(&GraphNodeAndOutputPin.Node);
			TArray<const UEdGraphPin*> InputPins;
			SetNode->GetInputPins(InputPins);
			for (const UEdGraphPin* InputPin : InputPins)
			{
				if (SetNode->IsAddPin(InputPin))
				{
					continue;
				}

				FNiagaraVariable InputVariable = TraversalPinToNiagaraVariable(*InputPin);
				if (InputVariable.GetType() == FNiagaraTypeDefinition::GetParameterMapDef())
				{
					continue;
				}

				EParameterFlags WriteFlags = FTraversalBuilder::ExtractFlagsFromParameterName(InputVariable.GetName());
				FParameterWrite& WriteParameterReference = ParameterData.WriteParameterReferences.Add_GetRef(
					FParameterWrite(InputVariable, WriteFlags));

				if (HasFlag(WriteFlags, EParameterFlags::NamespaceModifier))
				{
					ResolveNamespaceModifier(WriteParameterReference);
				}

				TOptional<FSelectValue> WriteSelectValue;
				if (InputVariable.GetType().IsStatic() && InputPin->LinkedTo.Num() == 0 && InputVariable.IsDataAllocated())
				{
					WriteSelectValue = FTraversalBuilder::CreateSelectValue(InputVariable);
				}


				WriteParameterReference.OptionalConnectionPinId = InputPin->PinId;
				WriteParameterReference.OptionalLocalSelectValue = WriteSelectValue;
			}

			const UEdGraphPin* ParameterMapInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*SetNode);
			ParameterData.ExecutionConnectionPinId = ParameterMapInputPin != nullptr ? ParameterMapInputPin->PinId : FGuid();
			OutParameterData = ParameterData;
		}
	};

	class FFunctionCallHandler : public FTraversalBuilder::FGraphNodeHandler
	{
	public:
		virtual bool CanCallFunctionScript() const override { return true; }
		virtual void GetFunctionCallData(
			const FTraversalBuilder::FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FFunctionCallData>& OutFunctionCallData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const override
		{ 
			const UNiagaraNodeFunctionCall* FunctionCallNode = CastChecked<UNiagaraNodeFunctionCall>(&GraphNodeAndOutputPin.Node);

			FFunctionCallData FunctionCallData;
			FunctionCallData.FunctionScriptReference = FScriptReference(FSoftObjectPath(FunctionCallNode->FunctionScript), FunctionCallNode->SelectedScriptVersion);
			FunctionCallData.FunctionCallName = *FunctionCallNode->GetFunctionName();
			FunctionCallData.DebugState = FunctionCallNode->DebugState;

			UNiagaraGraph* CalledGraph = FunctionCallNode->GetCalledGraph();
			if (CalledGraph != nullptr)
			{
				TArray<UEdGraphPin*> InputPins;
				FunctionCallNode->GetInputPins(InputPins);
				for (UEdGraphPin* InputPin : InputPins)
				{
					FNiagaraVariable InputVariable = TraversalPinToNiagaraVariable(*InputPin);
					UNiagaraScriptVariable* ScriptVariable = CalledGraph->GetScriptVariable(InputPin->PersistentGuid);
					if (ScriptVariable == nullptr)
					{
						ScriptVariable = CalledGraph->GetScriptVariable(InputVariable);
					}
					if (InputVariable.GetType().IsStatic() == false && (ScriptVariable == nullptr || ScriptVariable->GetIsStaticSwitch() == false))
					{
						continue;
					}

					FFunctionInputSelectValue& InputSelectValue = FunctionCallData.InputSelectValues.AddDefaulted_GetRef();
					InputSelectValue.InputSelectKey.Source = ESelectKeySource::FunctionCallNode;
					InputSelectValue.InputSelectKey.Variable = InputVariable;

					if (InputPin->LinkedTo.Num() == 1)
					{
						InputSelectValue.ConnectionPinId = InputPin->PinId;
					}
					else
					{
						TOptional<FSelectValue> InputVariableValue;
						if (InputVariable.IsDataAllocated() && FTraversalBuilder::IsValidSelectValueType(InputVariable.GetType()))
						{
							InputVariableValue = FTraversalBuilder::CreateSelectValue(InputVariable);
						}

						if (InputVariableValue.IsSet())
						{
							InputSelectValue.LocalValue = InputVariableValue;
						}
						else if (ScriptVariable != nullptr && ScriptVariable->GetIsStaticSwitch())
						{
							// Check for propagated switches.
							const FNiagaraPropagatedVariable* PropagatedSwitch = FunctionCallNode->PropagatedStaticSwitchParameters.FindByPredicate(
								[InputVariable](const FNiagaraPropagatedVariable& PropagatedSwitch) { return PropagatedSwitch.SwitchParameter == InputVariable; });
							if (PropagatedSwitch != nullptr && PropagatedSwitch->PropagatedName.IsEmpty() == false)
							{
								InputSelectValue.OptionalPropagatedNameOverride = *PropagatedSwitch->PropagatedName;
							}
						}
					}
				}
			}

			const UEdGraphPin* ParameterMapInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*FunctionCallNode);
			FunctionCallData.ExecutionConnectionPinId = ParameterMapInputPin != nullptr ? ParameterMapInputPin->PinId : FGuid();
			OutFunctionCallData = FunctionCallData;
		}
	};

	class FStaticSwitchHandler : public FTraversalBuilder::FGraphNodeHandler
	{
	public:
		virtual bool CanSelectInputPin() const override { return true; }

		virtual void GetSelectData(
			const FTraversalBuilder::FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FSelectData>& OutSelectData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const override
		{
			FSelectData SelectData;
			OutFilteredConnectedPinIds = TArray<FGuid>();

			const UNiagaraNodeStaticSwitch* StaticSwitchNode = CastChecked<UNiagaraNodeStaticSwitch>(&GraphNodeAndOutputPin.Node);
			SelectData.SelectMode = StaticSwitchNode->IsSetByPin() ? ESelectMode::Connection : ESelectMode::Value;

			if (SelectData.SelectMode == ESelectMode::Value)
			{
				SelectData.SelectKey = GetSelectKey(*StaticSwitchNode);
			}
			else if (SelectData.SelectMode == ESelectMode::Connection && StaticSwitchNode->IsSetByPin())
			{
				SelectData.SelectConnectionPinId = StaticSwitchNode->GetSelectorPin()->PinId;
				OutFilteredConnectedPinIds->Add(SelectData.SelectConnectionPinId);
			}

			if (GraphNodeAndOutputPin.OutputPin != nullptr)
			{
				GetSelectInputData(*StaticSwitchNode, *GraphNodeAndOutputPin.OutputPin, SelectData.InputData, DebugData);
				TArray<FGuid> SelectConnectedPinIds;
				for (const FSelectInputData& SelectInputData : SelectData.InputData)
				{
					if (SelectInputData.ConnectionPinId.IsSet())
					{
						OutFilteredConnectedPinIds->Add(SelectInputData.ConnectionPinId.GetValue());
					}
				}
			}
			OutSelectData = SelectData;
		}

	private:
		FSelectKey GetSelectKey(const UNiagaraNodeStaticSwitch& StaticSwitchNode) const
		{
			if (StaticSwitchNode.SwitchTypeData.SwitchConstant != NAME_None)
			{
				const FNiagaraVariable* StaticSwitchVariable = FNiagaraConstants::FindStaticSwitchConstant(StaticSwitchNode.SwitchTypeData.SwitchConstant);
				return StaticSwitchVariable != nullptr
					? FSelectKey(ESelectKeySource::ExternalConstant, *StaticSwitchVariable, NAME_None)
					: FSelectKey(
						ESelectKeySource::ExternalConstant,
						FNiagaraVariableBase(StaticSwitchNode.GetInputType(), StaticSwitchNode.SwitchTypeData.SwitchConstant),
						NAME_None);
			}
			else
			{
				FNiagaraVariableBase StaticSwitchVariable(StaticSwitchNode.GetInputType(), StaticSwitchNode.InputParameterName);
				return FSelectKey(ESelectKeySource::FunctionCallNode, StaticSwitchVariable, NAME_None);
			}
		}

		void GetSelectInputData(const UNiagaraNodeStaticSwitch& StaticSwitchNode, const UEdGraphPin& InOutputPin, TArray<FSelectInputData>& OutInputData, FTraversalBuilderDebugData* DebugData) const
		{
			FNiagaraVariable OutputVariable = TraversalPinToNiagaraVariable(InOutputPin);
			int32 OutputIndex = StaticSwitchNode.OutputVars.IndexOfByPredicate([OutputVariable](const FNiagaraVariable& OutputVar)
			{
				// Numeric pins can have their type changed, so it's still a match if the names match but the types don't, and
				// the OutputVar is numeric.
				return OutputVar.GetName() == OutputVariable.GetName() &&
					(OutputVar.GetType() == OutputVariable.GetType() || OutputVar.GetType() == FNiagaraTypeDefinition::GetGenericNumericDef());
			});

			if (OutputIndex == INDEX_NONE)
			{
				// If we couldn't find a variable that matched by both name and type, try to match by name, but only if there is only 1 match.
				TArray<int32> OutputVarIndicesMatchedByName;
				for (int32 OutputVarIndex = 0; OutputVarIndex < StaticSwitchNode.OutputVars.Num(); OutputVarIndex++)
				{
					if (StaticSwitchNode.OutputVars[OutputVarIndex].GetName() == OutputVariable.GetName())
					{
						OutputVarIndicesMatchedByName.Add(OutputVarIndex);
					}
				}
				if (OutputVarIndicesMatchedByName.Num() == 1)
				{
					OutputIndex = OutputVarIndicesMatchedByName[0];
				}
			}

			if (OutputIndex == INDEX_NONE)
			{
				if (DebugData != nullptr)
				{
					DebugData->AddUnresolvedSelectOutput(StaticSwitchNode.NodeGuid, InOutputPin.PinId);
				}
				return;
			}

			int32 OutputNum = StaticSwitchNode.OutputVars.Num();
			TArray<int32> Options = StaticSwitchNode.GetOptionValues();
			TArray<UEdGraphPin*> InputPins;
			StaticSwitchNode.GetInputPins(InputPins);
			int32 SetPins = StaticSwitchNode.IsSetByPin() ? 1 : 0;
			if (InputPins.Num() == (StaticSwitchNode.OutputVars.Num() * Options.Num() + SetPins))
			{
				for (int32 OptionIndex = 0; OptionIndex < Options.Num(); OptionIndex++)
				{
					FSelectInputData& InputData = OutInputData.AddDefaulted_GetRef();
					InputData.SelectValue = FTraversalBuilder::CreateSelectValue(StaticSwitchNode.GetInputType(), Options[OptionIndex]);

					int32 InputPinIndex = (OptionIndex * OutputNum) + OutputIndex;
					const UEdGraphPin* InputPin = InputPins[InputPinIndex];
					if (InputPin->LinkedTo.Num() == 0)
					{
						FNiagaraVariable PinVariable = TraversalPinToNiagaraVariable(*InputPin);
						if (PinVariable.GetType().IsStatic() && PinVariable.IsDataAllocated())
						{
							InputData.LocalValue = FTraversalBuilder::CreateSelectValue(PinVariable);
						}
					}
					else if (InputPin->LinkedTo.Num() == 1)
					{
						InputData.ConnectionPinId = InputPin->PinId;
					}
				}
			}
		}
	};

	class FOpNodeHandler : public FTraversalBuilder::FGraphNodeHandler
	{
	public:
		virtual bool CanEvaluateStaticValues() const override { return true; }
		virtual void GetStaticOpData(
			const FTraversalBuilder::FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FStaticOpData>& OutStaticOpData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const override
		{
			const UNiagaraNodeOp* OpNode = CastChecked<UNiagaraNodeOp>(&GraphNodeAndOutputPin.Node);
			const FNiagaraOpInfo* OpInfo = FNiagaraOpInfo::GetOpInfo(OpNode->OpName);
			if (OpInfo == nullptr || OpInfo->bSupportsStaticResolution == false || OpInfo->StaticVariableResolveFunction.IsBound() == false)
			{
				return;
			}

			FStaticOpData StaticOpData;
			StaticOpData.OpName = OpNode->OpName;

			TArray<const UEdGraphPin*> InputPins;
			OpNode->GetInputPins(InputPins);
			for (const UEdGraphPin* InputPin : InputPins)
			{
				if (OpNode->IsAddPin(InputPin))
				{
					continue;
				}

				FNiagaraVariable InputVariable = TraversalPinToNiagaraVariable(*InputPin);
				if (InputVariable.IsValid() == false)
				{
					continue;
				}

				FStaticOpInputData& OpInput = StaticOpData.InputData.AddDefaulted_GetRef();
				if (InputVariable.GetType().IsStatic())
				{
					if (InputPin->LinkedTo.Num() == 0)
					{
						if (InputVariable.IsDataAllocated())
						{
							OpInput.LocalValue = FTraversalBuilder::CreateSelectValue(InputVariable);
						}
					}
					else if (InputPin->LinkedTo.Num() == 1)
					{
						OpInput.ConnectionPinId = InputPin->PinId;
					}
				}
			}

			OutStaticOpData = StaticOpData;
		}
	};

	class FRerouteNodeHandler : public FTraversalBuilder::FGraphNodeHandler
	{
	public:
		virtual bool IsNoop() const override { return true; }
	};
} // BuilderHelpers

FTraversalBuilder* FTraversalBuilder::Instance = nullptr;

void FTraversalBuilder::Initialize()
{
	if (bInitialized == false)
	{
		checkf(Instance == nullptr, TEXT("FTraversalBuilder initialized more than once."));
		Instance = this;
		GraphNodeHandlers.Add(UNiagaraNodeInput::StaticClass(), MakeShared<BuilderHelpers::FInputHandler>());
		GraphNodeHandlers.Add(UNiagaraNodeParameterMapGet::StaticClass(), MakeShared<BuilderHelpers::FMapGetHandler>());
		GraphNodeHandlers.Add(UNiagaraNodeParameterMapSet::StaticClass(), MakeShared<BuilderHelpers::FMapSetHandler>());
		GraphNodeHandlers.Add(UNiagaraNodeFunctionCall::StaticClass(), MakeShared<BuilderHelpers::FFunctionCallHandler>());
		GraphNodeHandlers.Add(UNiagaraNodeStaticSwitch::StaticClass(), MakeShared<BuilderHelpers::FStaticSwitchHandler>());
		GraphNodeHandlers.Add(UNiagaraNodeAssignment::StaticClass(), MakeShared<BuilderHelpers::FFunctionCallHandler>());
		GraphNodeHandlers.Add(UNiagaraNodeOp::StaticClass(), MakeShared<BuilderHelpers::FOpNodeHandler>());
		GraphNodeHandlers.Add(UNiagaraNodeReroute::StaticClass(), MakeShared<BuilderHelpers::FRerouteNodeHandler>());
		bInitialized = true;
	}
}

FTraversalBuilder::~FTraversalBuilder()
{
	Instance = nullptr;
	bInitialized = false;
}

TSharedRef<const FTraversal> FTraversalBuilder::BuildTraversal(
	const UNiagaraNodeOutput& OutputNode, FTraversalBuilderDebugData* DebugData)
{
	return GetInstance().BuildTraversalInternal(OutputNode, DebugData);
}

void FTraversalBuilder::ResolveFunctionCallStackNames(const UEdGraph& TopLevelGraph, const TArray<FGuid>& FunctionCallStack, TArray<FString>& OutFunctionCallNames)
{
	const UEdGraph* CurrentGraph = &TopLevelGraph;
	for (int32 CurrentStackIndex = 0; CurrentStackIndex < FunctionCallStack.Num(); ++CurrentStackIndex)
	{
		if (CurrentGraph == nullptr)
		{
			break;
		}

		const FGuid& FunctionCallNodeGuid = FunctionCallStack[CurrentStackIndex];
		const TObjectPtr<UEdGraphNode>* FunctionCallNodePtr = CurrentGraph->Nodes.FindByPredicate([&FunctionCallNodeGuid]
			(const UEdGraphNode* GraphNode) { return GraphNode->NodeGuid == FunctionCallNodeGuid; });
		if (FunctionCallNodePtr == nullptr)
		{
			break;
		}

		const UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(FunctionCallNodePtr->Get());
		if (FunctionCallNode == nullptr)
		{
			break;
		}

		OutFunctionCallNames.Add(FunctionCallNode->GetFunctionName());
		CurrentGraph = FunctionCallNode->GetCalledGraph();
	}
};

bool FTraversalBuilder::IsValidSelectValueType(const FNiagaraTypeDefinition& InValueType)
{
	return GetInstance().IsValidSelectValueTypeInternal(InValueType);
}

FSelectValue FTraversalBuilder::CreateSelectValue(const FNiagaraTypeDefinition& InValueType, int32 InSelectNumericValue)
{
	return GetInstance().CreateSelectValueInternal(InValueType, InSelectNumericValue);
}

FSelectValue FTraversalBuilder::CreateSelectValue(const FNiagaraVariable& InNiagaraVariable)
{
	return GetInstance().CreateSelectValueInternal(InNiagaraVariable);
}

FSelectValue FTraversalBuilder::CreateSelectValue(bool bInBoolValue)
{
	FNiagaraVariable Temp(FNiagaraTypeDefinition::GetBoolDef(), NAME_None);
	Temp.SetValue(FNiagaraBool(bInBoolValue));
	return CreateSelectValue(Temp);
}

FSelectValue FTraversalBuilder::CreateSelectValue(UEnum* Enum, int32 EnumValue)
{
	FNiagaraVariable Temp(FNiagaraTypeDefinition(Enum), NAME_None);
	FNiagaraInt32 IntValue;
	IntValue.Value = EnumValue;
	Temp.SetValue(IntValue);
	return CreateSelectValue(Temp);
}

EParameterFlags FTraversalBuilder::ExtractFlagsFromParameterName(FName ParameterName)
{
	return GetInstance().ExtractFlagsFromParameterNameInternal(ParameterName);
}

FTraversalBuilder& FTraversalBuilder::GetInstance()
{
	checkf(Instance != nullptr, TEXT("FTraversalBuilder was not initialized."));
	return *Instance;
}

TSharedRef<const FTraversal> FTraversalBuilder::BuildTraversalInternal(
	const UNiagaraNodeOutput& OutputNode, FTraversalBuilderDebugData* DebugData) const
{
	TMap<FGraphNodeAndOutputPin, TSharedRef<FTraversalNode>> TraversedOutputPins;
	TSet<const FTraversalNode*> TrimTraversedNodes;
	TOptional<TSet<TWeakPtr<FTraversalNode>>> TraversedNoopNodesWeak;
	TMap<FName, FGuid> ModuleNameToFunctionCallNodeGuid;
	TSet<const FTraversalNode*> ResolveModuleInputsTraversedNodes;

	if (DebugData != nullptr)
	{
		TraversedNoopNodesWeak = TSet<TWeakPtr<FTraversalNode>>();
	}

	auto HandleRootGraphNode = [&](const FGraphNodeAndOutputPin& RootGraphNodeAndOutputPin)
	{
		TSharedPtr<FTraversalNode> RootNode = TraverseGraphNodeFromOutputPin(RootGraphNodeAndOutputPin, TraversedOutputPins, DebugData);
		ResolveModuleInputWrites(*RootNode.Get(), ModuleNameToFunctionCallNodeGuid, ResolveModuleInputsTraversedNodes);
		return RootNode.ToSharedRef();
	};

	TSharedRef<FTraversalNode> TraversalRoot = HandleRootGraphNode(FGraphNodeAndOutputPin(OutputNode, nullptr));

	TArray<TSharedRef<FTraversalNode>> UnconnectedTraversalRoots;
	TArray<FGraphNodeAndOutputPin> UnconnectedRoots;
	GetUnconnectedRoots(OutputNode, TraversedOutputPins, UnconnectedRoots);
	for (const FGraphNodeAndOutputPin& UnconnectedRoot : UnconnectedRoots)
	{
		UnconnectedTraversalRoots.Add(HandleRootGraphNode(UnconnectedRoot));
	}

	// Trim noops after all roots are handled.
	TArray<TSharedRef<FTraversalNode>> RootsToTrim;
	RootsToTrim.Add(TraversalRoot);
	RootsToTrim.Append(UnconnectedTraversalRoots);
	for (TSharedRef<FTraversalNode>& RootToTrim : RootsToTrim)
	{
		TrimNoops(RootToTrim.Get(), TrimTraversedNodes, TraversedNoopNodesWeak.GetPtrOrNull());
	}

	if (DebugData != nullptr && TraversedNoopNodesWeak.IsSet())
	{
		// Empty this map since it will have references to the noops.
		TraversedOutputPins.Empty();
		for (TWeakPtr<FTraversalNode>& TraversedNoopNodeWeak : TraversedNoopNodesWeak.GetValue())
		{
			if (TraversedNoopNodeWeak.IsValid())
			{ 
				TSharedPtr<FTraversalNode> TraversedNoopNode = TraversedNoopNodeWeak.Pin();
				DebugData->AddUntrimmedNoop(TraversedNoopNode->SourceNodeGuid, TraversedNoopNode->SourceNodeTypeName);
			}
		}
	}

	TSharedRef<FTraversal> Traversal = MakeShared<FTraversal>(TraversalRoot, UnconnectedTraversalRoots);
	CollectAdditionalTraversalData(Traversal.Get());

	return Traversal;
}

TSharedPtr<FTraversalNode> FTraversalBuilder::TraverseGraphNodeFromOutputPin(
	const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
	TMap<FGraphNodeAndOutputPin, TSharedRef<FTraversalNode>>& TraversedOutputPins,
	FTraversalBuilderDebugData* DebugData) const
{
	if (ensureMsgf(GraphNodeAndOutputPin.OutputPin == nullptr || GraphNodeAndOutputPin.OutputPin->Direction == EGPD_Output, TEXT("OutputPin must be using the output direction.")) == false)
	{
		return nullptr;
	}

	TSharedRef<FTraversalNode>* ExistingTraversalNode = TraversedOutputPins.Find(GraphNodeAndOutputPin);
	if (ExistingTraversalNode != nullptr)
	{
		return *ExistingTraversalNode;
	}

	const UEdGraphNode& GraphNode = GraphNodeAndOutputPin.Node;
	TSharedRef<FTraversalNode> NewTraversalNode = MakeShared<FTraversalNode>();
	NewTraversalNode->SourceNodeGuid = GraphNode.NodeGuid;
	NewTraversalNode->SourceNodeTypeName = GraphNode.GetClass()->GetFName();
	NewTraversalNode->bSourceNodeEnabled = GraphNode.GetDesiredEnabledState() == ENodeEnabledState::Enabled;

	TOptional<TArray<FGuid>> FilteredConnectedPinIds;
	TraversedOutputPins.Add(GraphNodeAndOutputPin, NewTraversalNode);
	GatherNodeDataAndFilteredConnectedPinIds(GraphNodeAndOutputPin, NewTraversalNode, FilteredConnectedPinIds, DebugData);

	TArray<const UEdGraphPin*> InputPins;
	for (const UEdGraphPin* Pin : GraphNode.Pins)
	{
		if (Pin->Direction == EGPD_Input)
		{
			InputPins.Add(Pin);
		}
	}
	BuilderHelpers::SortAndFilterInputs(InputPins);
	
	for (const UEdGraphPin* InputPin : InputPins)
	{
		if (InputPin->LinkedTo.Num() > 1 && DebugData != nullptr)
		{
			TArray<TPair<FGuid, FGuid>> ConnectedNodeGuidsAndPinIds;
			for (const UEdGraphPin* LinkedPin : InputPin->LinkedTo)
			{
				TPair<FGuid, FGuid>& ConnectedNodeGuidAndPinId = ConnectedNodeGuidsAndPinIds.AddDefaulted_GetRef();
				if (LinkedPin != nullptr && LinkedPin->GetOwningNode() != nullptr)
				{
					ConnectedNodeGuidAndPinId.Key = LinkedPin->GetOwningNode()->NodeGuid;
					ConnectedNodeGuidAndPinId.Value = LinkedPin->PinId;
				}
			}

			DebugData->AddMultipleInputConnection(TPair<FGuid, FGuid>(InputPin->GetOwningNode()->NodeGuid, InputPin->PinId), ConnectedNodeGuidsAndPinIds);
		}

		if ((FilteredConnectedPinIds.IsSet() && FilteredConnectedPinIds->Contains(InputPin->PinId) == false) ||
			InputPin->LinkedTo.Num() == 0 ||
			InputPin->LinkedTo[0] == nullptr ||
			InputPin->LinkedTo[0]->GetOwningNode() == nullptr)
		{
			continue;
		}

		const UEdGraphPin* LinkedOutputPin = InputPin->LinkedTo[0];
		TSharedPtr<FTraversalNode> TraversalNodeForPin = TraverseGraphNodeFromOutputPin(
			FGraphNodeAndOutputPin(*LinkedOutputPin->GetOwningNode(), LinkedOutputPin), TraversedOutputPins, DebugData);
		if (TraversalNodeForPin.IsValid())
		{
			NewTraversalNode->Connections.Add(FTraversalNode::FConnection(InputPin->PinId, TraversalNodeForPin.ToSharedRef()));
		}
	}

	return NewTraversalNode;
}

void FTraversalBuilder::GatherNodeDataAndFilteredConnectedPinIds(const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
	TSharedRef<FTraversalNode> NewTraversalNode, TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds, FTraversalBuilderDebugData* DebugData) const
{
	const UEdGraphNode& GraphNode = GraphNodeAndOutputPin.Node;
	const TSharedRef<FTraversalBuilder::IGraphNodeHandler>* NodeHandlerPtr = GraphNodeHandlers.Find(GraphNode.GetClass());
	if (NodeHandlerPtr != nullptr)
	{
		TSharedRef<FTraversalBuilder::IGraphNodeHandler> NodeHandler = *NodeHandlerPtr;
		if (NodeHandler->IsNoop())
		{
			TArray<const UEdGraphPin*> InputPins;
			CastChecked<UNiagaraNode>(&GraphNode)->GetInputPins(InputPins);

			if (InputPins.Num() == 1)
			{
				NewTraversalNode->bIsNoop = true;
				OutFilteredConnectedPinIds = TArray<FGuid>();
				OutFilteredConnectedPinIds->Add(InputPins[0]->PinId);
			}
		}
		else
		{
			if (NodeHandler->CanProvideFunctionInput())
			{
				NodeHandler->GetFunctionInputData(GraphNodeAndOutputPin, NewTraversalNode->FunctionInputData, DebugData);
			}
			if (NodeHandler->CanAccessParameters())
			{
				NodeHandler->GetParameterData(GraphNodeAndOutputPin, NewTraversalNode->ParameterData, OutFilteredConnectedPinIds, DebugData);
			}
			if (NodeHandler->CanCallFunctionScript())
			{
				NodeHandler->GetFunctionCallData(GraphNodeAndOutputPin, NewTraversalNode->FunctionCallData, OutFilteredConnectedPinIds, DebugData);
			}
			if (NodeHandler->CanEvaluateStaticValues())
			{
				NodeHandler->GetStaticOpData(GraphNodeAndOutputPin, NewTraversalNode->StaticOpData, OutFilteredConnectedPinIds, DebugData);
			}
			if (NodeHandler->CanSelectInputPin())
			{
				NodeHandler->GetSelectData(GraphNodeAndOutputPin, NewTraversalNode->SelectData, OutFilteredConnectedPinIds, DebugData);
			}
		}
	}
}

void FTraversalBuilder::GetUnconnectedRoots(const UNiagaraNodeOutput& OutputNode, const TMap<FGraphNodeAndOutputPin, TSharedRef<FTraversalNode>>& TraversedOutputPins, TArray<FGraphNodeAndOutputPin>& OutUnconnectedRoots)
{
	TSet<const UEdGraphNode*> TraversedNodes;
	for (const TPair<FGraphNodeAndOutputPin, TSharedRef<FTraversalNode>>& TraversedOutputPin : TraversedOutputPins)
	{
		TraversedNodes.Add(&TraversedOutputPin.Key.Node);
	}

	const UEdGraph* OwningGraph = OutputNode.GetGraph();
	for (const UEdGraphNode* Node : OwningGraph->Nodes)
	{
		if (Node->IsA<UNiagaraNodeOutput>() == false && TraversedNodes.Contains(Node) == false)
		{
			bool bHasConnectedOutput = false;
			TArray<const UEdGraphPin*> OutputPins;
			for (const UEdGraphPin* Pin : Node->Pins)
			{
				if (Pin->Direction == EGPD_Output && UNiagaraNodeWithDynamicPins::IsAddPin(Pin) == false)
				{
					if (Pin->LinkedTo.Num() > 0)
					{
						bHasConnectedOutput = true;
						break;
					}
					OutputPins.Add(Pin);
				}
			}
			if (bHasConnectedOutput == false)
			{
				if (OutputPins.Num() == 0)
				{
					OutUnconnectedRoots.Add(FGraphNodeAndOutputPin(*Node, nullptr));
				}
				else
				{
					for (const UEdGraphPin* OutputPin : OutputPins)
					{
						OutUnconnectedRoots.Add(FGraphNodeAndOutputPin(*Node, OutputPin));
					}
				}
			}
		}
	}
}

void FTraversalBuilder::TrimNoops(FTraversalNode& Node, TSet<const FTraversalNode*>& TraversedNodes, TSet<TWeakPtr<FTraversalNode>>* TraversedNoopNodesWeak)
{
	bool bAlreadyInSet = false;
	TraversedNodes.Add(&Node, &bAlreadyInSet);
	if (bAlreadyInSet)
	{
		return;
	}

	TArray<FGuid> InvalidConnectionPinIds;
	for (FTraversalNode::FConnection& Connection : Node.Connections)
	{
		// Try to find a valid non-noop target connection node.
		TSharedPtr<FTraversalNode> TargetConnectionNode;
		TSharedPtr<FTraversalNode> CurrentNode = Connection.Node;
		while (CurrentNode.IsValid() && TargetConnectionNode.IsValid() == false)
		{
			if (CurrentNode->bIsNoop)
			{
				if (TraversedNoopNodesWeak != nullptr)
				{
					// If this is not null, we're tracking noops for debugging.
					TraversedNoopNodesWeak->Add(TWeakPtr<FTraversalNode>(CurrentNode));
				}

				if (CurrentNode->Connections.Num() == 0)
				{
					// No incoming connections so this noop is a dead end.
					CurrentNode.Reset();
				}
				else if (CurrentNode->Connections.Num() == 1)
				{
					// Noop has single input, so keep looking for a non-noop.
					CurrentNode = CurrentNode->Connections[0].Node;
				}
				else
				{
					// Multiple input connections are not supported, so this noop can not be trimmed.
					TargetConnectionNode = CurrentNode;
				}
			}
			else
			{
				// Non-noop found.
				TargetConnectionNode = CurrentNode;
			}
		}

		if (TargetConnectionNode.IsValid())
		{
			// If a valid target connection node was found and it's not the current connection node, replace it to trim
			// the noops.  Continue trimming from the target connection node.
			if (&Connection.Node.Get() != TargetConnectionNode.Get())
			{
				Connection.Node = TargetConnectionNode.ToSharedRef();
			}
			TrimNoops(Connection.Node.Get(), TraversedNodes, TraversedNoopNodesWeak);
		}
		else
		{
			// A valid target node was not found, so mark this connection for removal.
			InvalidConnectionPinIds.Add(Connection.PinId);
		}
	}

	if (InvalidConnectionPinIds.Num() > 0)
	{
		Node.Connections.RemoveAll([&InvalidConnectionPinIds](FTraversalNode::FConnection& Connection)
			{ return InvalidConnectionPinIds.Contains(Connection.PinId); });
	}
}

void FTraversalBuilder::ResolveModuleInputWrites(FTraversalNode& Node, TMap<FName, FGuid>& ModuleNameToFunctionCallNodeGuid, TSet<const FTraversalNode*>& TraversedNodes)
{
	bool bAlreadyInSet = false;
	TraversedNodes.Add(&Node, &bAlreadyInSet);
	if (bAlreadyInSet)
	{
		return;
	}

	if (Node.FunctionCallData.IsSet())
	{
		ModuleNameToFunctionCallNodeGuid.Add(Node.FunctionCallData->FunctionCallName, Node.SourceNodeGuid);
	}

	if (Node.ParameterData.IsSet() && Node.ParameterData->WriteParameterReferences.Num() > 0)
	{
		TArray<FParameterWrite> ResolvedWrites;
		bool bAnyParameterResolved = false;
		for (const FParameterWrite& WriteParameterReference : Node.ParameterData->WriteParameterReferences)
		{
			// Module input writes take the form [ModuleName].[InputName] so the initial traversal will flag the parameter as having an unknown namespace.
			bool bParameterResolved = false;
			if (HasFlag(WriteParameterReference.Flags, EParameterFlags::NamespaceUnknown))
			{
				FNiagaraParameterHandle WriteHandle(WriteParameterReference.Parameter.GetName());
				FGuid* NodeGuid = ModuleNameToFunctionCallNodeGuid.Find(WriteHandle.GetNamespace());
				if (NodeGuid != nullptr)
				{
					FName ResolvedParameterName = *(FNiagaraConstants::ModuleNamespaceString + TEXT(".") + WriteHandle.GetName().ToString());
					FNiagaraVariableBase ResolvedParameter(WriteParameterReference.Parameter.GetType(), ResolvedParameterName);

					EParameterFlags ResolvedFlags = WriteParameterReference.Flags;
					ResolvedFlags = ClearFlag(ResolvedFlags, EParameterFlags::NamespaceUnknown);
					ResolvedFlags = SetFlag(ResolvedFlags, EParameterFlags::ModuleInput);

					FParameterWrite& ResolvedParameterWrite = ResolvedWrites.AddDefaulted_GetRef();
					ResolvedParameterWrite = WriteParameterReference;
					ResolvedParameterWrite.Parameter = ResolvedParameter;
					ResolvedParameterWrite.Flags = ResolvedFlags;
					ResolvedParameterWrite.OptionalTargetFunctionCallNodeGuid = *NodeGuid;

					bParameterResolved = true;
					bAnyParameterResolved = true;
				}
			}

			if (bParameterResolved == false)
			{
				ResolvedWrites.Add(WriteParameterReference);
			}
		}

		if (bAnyParameterResolved)
		{
			FParameterData ResolvedParameterData = Node.ParameterData.GetValue();
			ResolvedParameterData.WriteParameterReferences = ResolvedWrites;
			Node.ParameterData = ResolvedParameterData;
		}
	}

	for (const FTraversalNode::FConnection& Connection : Node.Connections)
	{
		ResolveModuleInputWrites(Connection.Node.Get(), ModuleNameToFunctionCallNodeGuid, TraversedNodes);
	}
}

void FTraversalBuilder::CollectAdditionalTraversalData(FTraversal& Traversal)
{
	TSet<const FTraversalNode*> AllNodes;
	FTraversal::GetAllConnectedNodes(Traversal.TraversalRoot.Get(), AllNodes);

	for (const FTraversalNode* Node : AllNodes)
	{
		if (Node->FunctionCallData.IsSet())
		{
			Traversal.FunctionNameToNodeGuidMap.Add(Node->FunctionCallData->FunctionCallName, Node->SourceNodeGuid);
			if (Node->FunctionCallData->FunctionScriptReference.Path.IsValid())
			{
				Traversal.ExternalReferences.Add(Node->FunctionCallData->FunctionScriptReference);
			}
		}

		if (Node->ParameterData.IsSet())
		{
			for (const FParameterRead& ReadParameterReference : Node->ParameterData->ReadParameterReferences)
			{
				if (ReadParameterReference.Parameter.GetType().IsStatic())
				{
					Traversal.StaticVariableReads.Add(ReadParameterReference.Parameter);
				}
			}
			for (const FParameterWrite& WriteParameterReference : Node->ParameterData->WriteParameterReferences)
			{
				if (WriteParameterReference.Parameter.GetType().IsStatic())
				{
					Traversal.StaticVariableWrites.Add(WriteParameterReference.Parameter);
					if (HasFlag(WriteParameterReference.Flags, EParameterFlags::Attribute))
					{
						Traversal.StaticVariableWritesToAttributes.Add(WriteParameterReference.Parameter);
					}
				}
			}
		}
	}
}

bool FTraversalBuilder::IsValidSelectValueTypeInternal(const FNiagaraTypeDefinition& InValueType) const
{
	UE::TScopeLock ScopeLock(IsValidSelectValueTypeCacheGuard);

	bool* bIsValidType = IsValidSelectValueTypeCache.Find(InValueType);
	if (bIsValidType == nullptr)
	{
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> ValueTypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(InValueType);
		bIsValidType = &IsValidSelectValueTypeCache.Add(InValueType, ValueTypeUtilities.IsValid() && ValueTypeUtilities->CanBeSelectValue());
	}
	return *bIsValidType;
}

FSelectValue FTraversalBuilder::CreateSelectValueInternal(const FNiagaraVariable& InVariableValue) const
{
	UE::TScopeLock ScopeLock(SelectValueCacheGuard);

	const FNiagaraTypeDefinition& VariableType = InVariableValue.GetType();
	UObject* TypeObject = VariableType.IsEnum() ? (UObject*)VariableType.GetEnum() : (UObject*)VariableType.GetStruct();
	if (TypeObject == nullptr)
	{
		static FSelectValue Invalid(INDEX_NONE, NAME_None);
		return Invalid;
	}

	FSelectValueCacheKey CacheKey;
	CacheKey.TypeObjectKey = FObjectKey(TypeObject);
	CacheKey.VariableData.Append(InVariableValue.GetData(), InVariableValue.GetSizeInBytes());

	FSelectValue* SelectValue = SelectValueCache.Find(CacheKey);
	if (SelectValue == nullptr)
	{
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> ValueTypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(VariableType);
		int32 SelectNumericValue = ValueTypeUtilities->VariableToSelectNumericValue(InVariableValue);
		FName SelectValueDebugName = ValueTypeUtilities->GetDebugNameForSelectValue(VariableType, SelectNumericValue);
		SelectValue = &SelectValueCache.Add(CacheKey, FSelectValue(SelectNumericValue, SelectValueDebugName));
	}
	return *SelectValue;
}

FSelectValue FTraversalBuilder::CreateSelectValueInternal(const FNiagaraTypeDefinition& ValueType, int32 SelectNumericValue) const
{
	UE::TScopeLock ScopeLock(SelectValueCacheGuard);

	UObject* TypeObject = ValueType.IsEnum() ? (UObject*)ValueType.GetEnum() : (UObject*)ValueType.GetStruct();
	if (TypeObject == nullptr)
	{
		static FSelectValue Invalid(INDEX_NONE, NAME_None);
		return Invalid;
	}

	FSelectValueCacheKey CacheKey;
	CacheKey.TypeObjectKey = FObjectKey(TypeObject);
	CacheKey.SelectNumericValue = SelectNumericValue;

	FSelectValue* SelectValue = SelectValueCache.Find(CacheKey);
	if (SelectValue == nullptr)
	{
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> ValueTypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(ValueType);
		FName SelectValueDebugName = ValueTypeUtilities->GetDebugNameForSelectValue(ValueType, SelectNumericValue);
		SelectValue = &SelectValueCache.Add(CacheKey, FSelectValue(SelectNumericValue, SelectValueDebugName));
	}
	return *SelectValue;
}

EParameterFlags FTraversalBuilder::ExtractFlagsFromParameterNameInternal(FName ParameterName) const
{
	UE::TScopeLock ScopeLock(ExtractedParameterFlagCacheGuard);

	EParameterFlags* CachedFlags = ExtractedParameterFlagCache.Find(ParameterName);
	if (CachedFlags != nullptr)
	{
		return *CachedFlags;
	}

	EParameterFlags Flags = EParameterFlags::None;
	FNiagaraParameterHandle ParameterHandle(ParameterName);
	if (ParameterHandle.GetNamespace() == NAME_None)
	{
		Flags = EParameterFlags::InvalidParameterName;
	}
	else if (ParameterHandle.IsModuleHandle())
	{
		Flags = EParameterFlags::ModuleInput;
	}
	else if (ParameterHandle.IsLocalHandle())
	{
		Flags = EParameterFlags::ModuleLocal;
	}
	else if (ParameterHandle.IsOutputHandle())
	{
		FNiagaraParameterHandle ParameterSubHandle(ParameterHandle.GetName());
		Flags = ParameterSubHandle.GetNamespace() != NAME_None
			? EParameterFlags::ModuleOutput | EParameterFlags::NamespaceModifier
			: EParameterFlags::InvalidParameterName;
	}
	else if (ParameterHandle.IsSystemHandle() || ParameterHandle.IsEmitterHandle() || ParameterHandle.IsParticleAttributeHandle() || ParameterHandle.IsStackContextHandle())
	{
		Flags = EParameterFlags::Attribute;
		FNiagaraParameterHandle ParameterSubHandle(ParameterHandle.GetName());
		FName NamespaceModifier = ParameterSubHandle.GetNamespace();
		if (NamespaceModifier != NAME_None &&
			NamespaceModifier != FNiagaraConstants::PreviousNamespace &&
			NamespaceModifier != FNiagaraConstants::InitialNamespace &&
			NamespaceModifier != FNiagaraConstants::OwnerNamespace)
		{
			// If the attribute has a namespace modifier that's unknown, or is the module namespace
			// it has to be handled differently to support attribute module outputs correctly.
			Flags = Flags | EParameterFlags::NamespaceModifier;
			if (NamespaceModifier == FNiagaraConstants::ModuleNamespace)
			{
				Flags = Flags | EParameterFlags::ModuleOutput;
			}
		}
	}
	else if (ParameterHandle.IsEngineHandle() || ParameterHandle.IsDataInstanceHandle() ||
		ParameterHandle.IsParameterCollectionHandle() || ParameterHandle.IsUserHandle())
	{
		Flags = EParameterFlags::External;
	}
	else
	{
		Flags = EParameterFlags::NamespaceUnknown;
	}

	ExtractedParameterFlagCache.Add(ParameterName, Flags);
	return Flags;
}

} // UE::Niagara::TraversalCache