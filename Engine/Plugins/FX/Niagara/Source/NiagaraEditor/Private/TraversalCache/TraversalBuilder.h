// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"
#include "NiagaraVariant.h"
#include "NiagaraTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "TraversalCache/TraversalShared.h"
#include "Async/Mutex.h"

class UClass;
class UEdGraphNode;
class UEdGraphPin;
class UNiagaraNode;
class UNiagaraNodeOutput;

namespace UE::Niagara::TraversalCache
{

struct FFunctionCallData;
struct FFunctionInputData;
struct FParameterData;
struct FSelectData;
struct FStaticOpData;
class  FTraversal;
class  FTraversalNode;
struct FTraversalBuilderDebugData;

class FTraversalBuilder
{
public:
	struct FGraphNodeAndOutputPin
	{
		FGraphNodeAndOutputPin(const UEdGraphNode& InNode, const UEdGraphPin* InOutputPin)
			: Node(InNode)
			, OutputPin(InOutputPin)
		{
		}

		FORCEINLINE bool operator==(const FGraphNodeAndOutputPin& Other) const
		{
			return &Node == &Other.Node && OutputPin == Other.OutputPin;
		}

		friend FORCEINLINE uint32 GetTypeHash(const FGraphNodeAndOutputPin& NodeAndPin)
		{
			return HashCombine(GetTypeHash(&NodeAndPin.Node), GetTypeHash(NodeAndPin.OutputPin));
		}

		const UEdGraphNode& Node;
		const UEdGraphPin* OutputPin;
	};

	class IGraphNodeHandler
	{
	public:
		virtual ~IGraphNodeHandler() = default;

		virtual bool IsNoop() const = 0;

		virtual bool CanProvideFunctionInput() const = 0;
		virtual void GetFunctionInputData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FFunctionInputData>& OutInputData,
			FTraversalBuilderDebugData* DebugData) const = 0;

		virtual bool CanAccessParameters() const = 0;
		virtual void GetParameterData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FParameterData>& OutParameterData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const = 0;

		virtual bool CanSelectInputPin() const = 0;
		virtual void GetSelectData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FSelectData>& OutSelectData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const = 0;

		virtual bool CanCallFunctionScript() const = 0;
		virtual void GetFunctionCallData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FFunctionCallData>& OutFunctionCallData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const = 0;

		virtual bool CanEvaluateStaticValues() const = 0;
		virtual void GetStaticOpData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FStaticOpData>& OutStaticOpData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const = 0;
	};

	class FGraphNodeHandler : public IGraphNodeHandler
	{
	public:
		virtual bool IsNoop() const override { return false; }

		virtual bool CanProvideFunctionInput() const override { return false; }
		virtual void GetFunctionInputData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FFunctionInputData>& OutInputData,
			FTraversalBuilderDebugData* DebugData) const override { unimplemented(); }

		virtual bool CanAccessParameters() const override { return false; }
		virtual void GetParameterData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FParameterData>& OutParameterData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const override { unimplemented(); }

		virtual bool CanSelectInputPin() const override { return false; }
		virtual void GetSelectData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FSelectData>& OutSelectData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const override { unimplemented(); }

		virtual bool CanCallFunctionScript() const override { return false; }
		virtual void GetFunctionCallData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FFunctionCallData>& OutFunctionCallData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const override { unimplemented(); }

		virtual bool CanEvaluateStaticValues() const override { return false; }
		virtual void GetStaticOpData(
			const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
			TOptional<const FStaticOpData>& OutStaticOpData,
			TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
			FTraversalBuilderDebugData* DebugData) const override { unimplemented(); }
	};

public:
	void Initialize();

	virtual ~FTraversalBuilder();

	static TSharedRef<const FTraversal> BuildTraversal(const UNiagaraNodeOutput& OutputNode, FTraversalBuilderDebugData* DebugData = nullptr);

	static NIAGARAEDITOR_API void ResolveFunctionCallStackNames(const UEdGraph& TopLevelGraph, const TArray<FGuid>& FunctionCallStack, TArray<FString>& OutFunctionCallNames);

	static bool IsValidSelectValueType(const FNiagaraTypeDefinition& InValueType);

	static FSelectValue CreateSelectValue(const FNiagaraTypeDefinition& InValueType, int32 InSelectNumericValue);

	static FSelectValue CreateSelectValue(const FNiagaraVariable& InNiagaraVariable);

	static FSelectValue CreateSelectValue(bool bInBoolValue);

	static FSelectValue CreateSelectValue(UEnum* Enum, int32 EnumValue);

	static EParameterFlags ExtractFlagsFromParameterName(FName ParameterName);

private:
	static FTraversalBuilder& GetInstance();

	TSharedRef<const FTraversal> BuildTraversalInternal(const UNiagaraNodeOutput& OutputNode, FTraversalBuilderDebugData* DebugData) const;

	TSharedPtr<FTraversalNode> TraverseGraphNodeFromOutputPin(
		const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
		TMap<FGraphNodeAndOutputPin, TSharedRef<FTraversalNode>>& TraversedOutputPins,
		FTraversalBuilderDebugData* DebugData) const;

	void GatherNodeDataAndFilteredConnectedPinIds(
		const FGraphNodeAndOutputPin& GraphNodeAndOutputPin,
		TSharedRef<FTraversalNode> NewTraversalNode,
		TOptional<TArray<FGuid>>& OutFilteredConnectedPinIds,
		FTraversalBuilderDebugData* DebugData) const;

	static void GetUnconnectedRoots(const UNiagaraNodeOutput& OutputNode, const TMap<FGraphNodeAndOutputPin, TSharedRef<FTraversalNode>>& TraversedOutputPins, TArray<FGraphNodeAndOutputPin>& OutUnconnectedRoots);

	static void TrimNoops(FTraversalNode& Node, TSet<const FTraversalNode*>& TraversedNodes, TSet<TWeakPtr<FTraversalNode>>* TraversedNoopNodesWeak);

	static void ResolveModuleInputWrites(FTraversalNode& Node, TMap<FName, FGuid>& ModuleNameToFunctionCallNodeGuid, TSet<const FTraversalNode*>& TraversedNodes);

	bool IsValidSelectValueTypeInternal(const FNiagaraTypeDefinition& InValueType) const;

	FSelectValue CreateSelectValueInternal(const FNiagaraVariable& InVariableValue) const;
	
	FSelectValue CreateSelectValueInternal(const FNiagaraTypeDefinition& ValueType, int32 SelectNumericValue) const;

	static void CollectAdditionalTraversalData(FTraversal& Traversal);

	struct FSelectValueCacheKey
	{
		FORCEINLINE bool operator==(const FSelectValueCacheKey& Other) const
		{
			return TypeObjectKey == Other.TypeObjectKey && VariableData == Other.VariableData && SelectNumericValue == Other.SelectNumericValue;
		}

		friend FORCEINLINE uint32 GetTypeHash(const FSelectValueCacheKey& Key)
		{
			int32 Hash = HashCombine(GetTypeHash(Key.TypeObjectKey), GetTypeHash(Key.VariableData));
			return HashCombine(Hash, GetTypeHash(Key.SelectNumericValue));
		}

		FObjectKey TypeObjectKey;
		TArray<uint8> VariableData;
		int32 SelectNumericValue = INDEX_NONE;
	};

	EParameterFlags ExtractFlagsFromParameterNameInternal(FName ParameterName) const;

private:
	static FTraversalBuilder* Instance;
	bool bInitialized = false;

	TMap<TSubclassOf<UEdGraphNode>, TSharedRef<IGraphNodeHandler>> GraphNodeHandlers;

	mutable TMap<FNiagaraTypeDefinition, bool> IsValidSelectValueTypeCache;
	mutable UE::FMutex IsValidSelectValueTypeCacheGuard;

	mutable TMap<FSelectValueCacheKey, FSelectValue> SelectValueCache;
	mutable UE::FMutex SelectValueCacheGuard;

	mutable TMap<FName, EParameterFlags> ExtractedParameterFlagCache;
	mutable UE::FMutex ExtractedParameterFlagCacheGuard;
};

} // UE::Niagara::TraversalCache
