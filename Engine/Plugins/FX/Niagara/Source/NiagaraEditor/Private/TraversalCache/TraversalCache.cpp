// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraversalCache/TraversalCache.h"

#include "EdGraphSchema_Niagara.h"
#include "NiagaraCommon.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorModule.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "String/ParseTokens.h"
#include "TraversalCache/Traversal.h"
#include "TraversalCache/TraversalBuilder.h"
#include "TraversalCache/TraversalNode.h"
#include "TraversalCache/TraversalShared.h"

namespace UE::Niagara::TraversalCache
{

FTraversalCache* FTraversalCache::Instance = nullptr;

void FTraversalCache::Initialize()
{
	if (bInitialized == false)
	{
		checkf(Instance == nullptr, TEXT("FTraversalCache initialized more than once."));
		Instance = this;
		FNiagaraEditorModule::Get().OnScriptApplied().AddSP(this, &FTraversalCache::OnScriptApplied);
		if (GEditor != nullptr)
		{
			GEditor->RegisterForUndo(this);
		}
		TraversalBuilder = MakeShared<FTraversalBuilder>();
		TraversalBuilder->Initialize();
		bInitialized = true;
	}
}

FTraversalCache::~FTraversalCache()
{
	Instance = nullptr;
	bInitialized = false;
	if (GEditor != nullptr)
	{
		GEditor->UnregisterForUndo(this);
	}
}

TSharedPtr<const FTraversal> FTraversalCache::GetScriptAssetTraversal(const FSoftObjectPath& ScriptSoftObjectPath, const FGuid& ScriptVersion, FTraversalBuilderDebugData* BuilderDebugData)
{
	return GetInstance().GetScriptAssetTraversalInternal(ScriptSoftObjectPath, ScriptVersion, BuilderDebugData);
}

TSharedPtr<const FTraversal> FTraversalCache::GetScriptAssetTraversal(const FSoftObjectPath& ScriptSoftObjectPath, const FGuid& ScriptVersion, FTraversalDebugData* TraversalDebugData)
{
	TOptional<FTraversalBuilderDebugData> BuilderDebugData = TraversalDebugData != nullptr ? FTraversalBuilderDebugData() : TOptional<FTraversalBuilderDebugData>();
	TSharedPtr<const FTraversal> Traversal = FTraversalCache::GetScriptAssetTraversal(ScriptSoftObjectPath, ScriptVersion, BuilderDebugData.GetPtrOrNull());
	if (TraversalDebugData != nullptr && BuilderDebugData->HasData())
	{
		TraversalDebugData->AddBuilderDebugOutput(ScriptSoftObjectPath, ENiagaraScriptUsage::Function, FGuid(), ScriptVersion, BuilderDebugData.GetValue());
	}
	return Traversal;
}

TSharedPtr<const FTraversal> FTraversalCache::GetTopLevelScriptTraversal(const UNiagaraScript& Script, FTraversalBuilderDebugData* BuilderDebugData)
{
	return GetInstance().GetTopLevelScriptTraversalInternal(Script, BuilderDebugData);
}

TSharedPtr<const FTraversal> FTraversalCache::GetTopLevelScriptTraversal(const UNiagaraScript& Script, FTraversalDebugData* TraversalDebugData)
{
	TOptional<FTraversalBuilderDebugData> BuilderDebugData = TraversalDebugData != nullptr ? FTraversalBuilderDebugData() : TOptional<FTraversalBuilderDebugData>();
	TSharedPtr<const FTraversal> Traversal = FTraversalCache::GetTopLevelScriptTraversal(Script, BuilderDebugData.GetPtrOrNull());
	if (TraversalDebugData != nullptr && BuilderDebugData->HasData())
	{
		const UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(Script.GetSource(FGuid()));
		TraversalDebugData->AddBuilderDebugOutput(ScriptSource->NodeGraph, Script.GetUsage(), Script.GetUsageId(), FGuid(), BuilderDebugData.GetValue());
	}
	return Traversal;
}

void FTraversalCache::GetTopLevelScriptFunctionInputContexts(const UNiagaraScript& Script, TMap<FGuid, FStaticContext>& OutFunctionInputContexts, FTraversalDebugData* TraversalDebugData)
{
	GetInstance().GetTopLevelScriptFunctionInputContextsInternal(Script, OutFunctionInputContexts, TraversalDebugData);
}

void FTraversalCache::GetStackFunctionReads(
	const FTopLevelScriptStaticContext& ScriptStaticContext,
	const UNiagaraNodeFunctionCall& FunctionCallNode,
	TArray<FNiagaraVariable>& OutReads,
	TSet<FNiagaraVariable>& OutHiddenReads,
	EStackFunctionReadFilterFlags FilterFlags,
	FTraversalDebugData* TraversalDebugData)
{
	TOptional<FTraversalBuilderDebugData> BuilderDebugData = TraversalDebugData != nullptr ? FTraversalBuilderDebugData() : TOptional<FTraversalBuilderDebugData>();
	GetInstance().GetStackFunctionReadsInternal(ScriptStaticContext, FunctionCallNode, OutReads, OutHiddenReads, FilterFlags, TraversalDebugData, BuilderDebugData.GetPtrOrNull());
	if (TraversalDebugData != nullptr && BuilderDebugData->HasData())
	{
		TraversalDebugData->AddBuilderDebugOutput(FunctionCallNode.GetCalledGraph(), ENiagaraScriptUsage::Function, FGuid(), FunctionCallNode.SelectedScriptVersion, BuilderDebugData.GetValue());
	}
}

void FTraversalCache::GetStackFunctionStaticInputs(
	const FTopLevelScriptStaticContext& ScriptStaticContext,
	const UNiagaraNodeFunctionCall& FunctionCallNode,
	TArray<FNiagaraVariable>& OutStaticInputs,
	TSet<FNiagaraVariable>& OutHiddenStaticInputs,
	FTraversalDebugData* TraversalDebugData)
{
	TOptional<FTraversalBuilderDebugData> BuilderDebugData = TraversalDebugData != nullptr ? FTraversalBuilderDebugData() : TOptional<FTraversalBuilderDebugData>();
	GetInstance().GetStackFunctionStaticInputsInternal(ScriptStaticContext, FunctionCallNode, OutStaticInputs, OutHiddenStaticInputs, TraversalDebugData, BuilderDebugData.GetPtrOrNull());
	if (TraversalDebugData != nullptr && BuilderDebugData->HasData())
	{
		TraversalDebugData->AddBuilderDebugOutput(FunctionCallNode.GetCalledGraph(), ENiagaraScriptUsage::Function, FGuid(), FunctionCallNode.SelectedScriptVersion, BuilderDebugData.GetValue());
	}
}

void FTraversalCache::GetStackFunctionStaticInputPins(
	const FTopLevelScriptStaticContext& ScriptStaticContext,
	const UNiagaraNodeFunctionCall& FunctionCallNode,
	TArray<UEdGraphPin*>& OutStaticInputPins,
	TSet<UEdGraphPin*>& OutHiddenStaticInputPins,
	FTraversalDebugData* TraversalDebugData)
{
	GetInstance().GetStackFunctionStaticInputPinsInternal(ScriptStaticContext, FunctionCallNode, OutStaticInputPins, OutHiddenStaticInputPins, TraversalDebugData);
}

void FTraversalCache::InvalidateCache()
{
	GetInstance().InvalidateCacheInternal();
}

void FTraversalCache::PostUndo(bool bSuccess)
{
	// When undo/redo happens we we don't know exactly what's changed so we update this serial number so that traversals
	// can invalidate any cached data that relies on external dependencies.  The traversals themselves are only invalidated
	// when their graph change id is updated.
	TraversalCachedDataSerialNumber++;
}

void FTraversalCache::PostRedo(bool bSuccess)
{
	// When undo/redo happens we we don't know exactly what's changed so we update this serial number so that traversals
	// can invalidate any cached data that relies on external dependencies.  The traversals themselves are only invalidated
	// when their graph change id is updated.
	TraversalCachedDataSerialNumber++;
}

FTraversalCache& FTraversalCache::GetInstance()
{
	checkf(Instance != nullptr, TEXT("FTraversalCache was not initialized."));
	return *Instance;
}

const UNiagaraScript* FTraversalCache::GetScriptFromPath(const FSoftObjectPath& Path) const
{
	UE::TScopeLock ScopeLock(SoftObjectPathToScriptCacheGuard);

	const UNiagaraScript* Script = nullptr;
	TWeakObjectPtr<const UNiagaraScript>* CachedScriptPtr = SoftObjectPathToScriptCache.Find(Path);
	if (CachedScriptPtr != nullptr)
	{
		Script = CachedScriptPtr->Get();
	}

	if (Script == nullptr)
	{
		Script = Cast<UNiagaraScript>(Path.ResolveObject());
		SoftObjectPathToScriptCache.Add(Path, TWeakObjectPtr(Script));
	}

	return Script;
}

void FTraversalCache::OnScriptApplied(UNiagaraScript* InScript, FGuid InVersionGuid)
{
	FScriptTraversalCacheKey ScriptCacheKey = FScriptTraversalCacheKey::FromScriptAsset(FObjectKey(InScript), InVersionGuid);
	if (CachedTraversals.Contains(ScriptCacheKey))
	{
		TraversalCachedDataSerialNumber++;
	}
}

TSharedPtr<const FTraversal> FTraversalCache::GetScriptAssetTraversalInternal(const FSoftObjectPath& ScriptSoftObjectPath, const FGuid& ScriptVersion, FTraversalBuilderDebugData* BuilderDebugData) const
{
	UE::TScopeLock ScopeLock(CachedTraversalsGuard);

	const UNiagaraScript* Script = GetScriptFromPath(ScriptSoftObjectPath);
	if (Script == nullptr)
	{
		return TSharedPtr<const FTraversal>();
	}

	const FVersionedNiagaraScriptData* ScriptData = Script->GetScriptData(ScriptVersion);
	if (ScriptData == nullptr)
	{
		return TSharedPtr<const FTraversal>();
	}

	FScriptTraversalCacheKey CacheKey = FScriptTraversalCacheKey::FromScriptAsset(FObjectKey(Script), ScriptVersion);
	FGuid ChangeId = Script->GetBaseChangeID(ScriptVersion);

	FScriptTraversalCacheValue* CachedValue = CachedTraversals.Find(CacheKey);
	if (CachedValue != nullptr && CachedValue->GraphChangeId == ChangeId)
	{
		if (CachedValue->TraversalCachedDataSerialNumber != TraversalCachedDataSerialNumber)
		{
			// If the change id still matches, but the traversal cached data serial number has changed, it's possible that a dependent script
			// has changed, so we need to invalidate any traversal data collected with this traversal before updating to the new
			// serial number.
			CachedValue->ScriptTraversal->ResetCachedData();
			CachedValue->TraversalCachedDataSerialNumber = TraversalCachedDataSerialNumber;
		}
		return CachedValue->ScriptTraversal;
	}

	const UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(Script->GetSource(ScriptVersion));
	UNiagaraNodeOutput* OutputNode = ScriptSource->NodeGraph->FindEquivalentOutputNode(Script->GetUsage(), Script->GetUsageId());
	if (OutputNode == nullptr)
	{
		return TSharedPtr<const FTraversal>();
	}

	TSharedRef<const FTraversal> Traversal = FTraversalBuilder::BuildTraversal(*OutputNode, BuilderDebugData);
	if (BuilderDebugData == nullptr || BuilderDebugData->HasData() == false)
	{
		CachedTraversals.Add(CacheKey, FScriptTraversalCacheValue(ChangeId, TraversalCachedDataSerialNumber, Traversal));
	}
	return Traversal;
}

TSharedPtr<const FTraversal> FTraversalCache::GetTopLevelScriptTraversalInternal(const UNiagaraScript& Script, FTraversalBuilderDebugData* BuilderDebugData) const
{
	UE::TScopeLock ScopeLock(CachedTraversalsGuard);

	const UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(Script.GetSource(FGuid()));

	FScriptTraversalCacheKey CacheKey = FScriptTraversalCacheKey::FromTopLevelScript(FObjectKey(ScriptSource->NodeGraph), Script.GetUsage(), Script.GetUsageId());
	FGuid ChangeId = Script.GetBaseChangeID(FGuid());

	FScriptTraversalCacheValue* CachedValue = CachedTraversals.Find(CacheKey);
	if (CachedValue != nullptr && CachedValue->GraphChangeId == ChangeId)
	{
		if (CachedValue->TraversalCachedDataSerialNumber != TraversalCachedDataSerialNumber)
		{
			// If the change id still matches, but the traversal cached data serial number has changed, it's possible that a dependent script
			// has changed, so we need to invalidate any traversal data collected with this traversal before updating to the new
			// serial number.
			CachedValue->ScriptTraversal->ResetCachedData();
			CachedValue->TraversalCachedDataSerialNumber = TraversalCachedDataSerialNumber;
		}
		return CachedValue->ScriptTraversal;
	}

	UNiagaraNodeOutput* OutputNode = ScriptSource->NodeGraph->FindEquivalentOutputNode(Script.GetUsage(), Script.GetUsageId());
	if (OutputNode == nullptr)
	{
		return TSharedPtr<const FTraversal>();
	}

	TSharedRef<const FTraversal> Traversal = FTraversalBuilder::BuildTraversal(*OutputNode, BuilderDebugData);
	if (BuilderDebugData == nullptr || BuilderDebugData->HasData() == false)
	{
		// We only cache the results when either no debug data was supplied, or when the debug data was empty.  When
		// the debug data HasData returns true, errors were encountered while building.
		CachedTraversals.Add(CacheKey, FScriptTraversalCacheValue(ChangeId, TraversalCachedDataSerialNumber, Traversal));
	}
	return Traversal;
}

TSharedPtr<const FTraversal> FTraversalCache::GetTopLevelScriptTraversalInternal(const UNiagaraNode& StackNode, FTraversalBuilderDebugData* BuilderDebugData) const
{
	UE::TScopeLock ScopeLock(CachedTraversalsGuard);

	const UNiagaraGraph* OwningGraph = StackNode.GetNiagaraGraph();
	if (OwningGraph == nullptr)
	{
		return TSharedPtr<const FTraversal>();
	}

	const UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(StackNode);
	if (OutputNode == nullptr)
	{
		return TSharedPtr<const FTraversal>();
	}

	FScriptTraversalCacheKey CacheKey = FScriptTraversalCacheKey::FromTopLevelScript(FObjectKey(OwningGraph), OutputNode->GetUsage(), OutputNode->GetUsageId());

	FScriptTraversalCacheValue* CachedValue = CachedTraversals.Find(CacheKey);
	if (CachedValue != nullptr && CachedValue->GraphChangeId == OwningGraph->GetChangeID())
	{
		if (CachedValue->TraversalCachedDataSerialNumber != TraversalCachedDataSerialNumber)
		{
			// If the change id still matches, but the traversal cached data serial number has changed, it's possible that a dependent script
			// has changed, so we need to invalidate any traversal data collected with this traversal before updating to the new
			// serial number.
			CachedValue->ScriptTraversal->ResetCachedData();
			CachedValue->TraversalCachedDataSerialNumber = TraversalCachedDataSerialNumber;
		}
		return CachedValue->ScriptTraversal;
	}

	TSharedRef<const FTraversal> Traversal = FTraversalBuilder::BuildTraversal(*OutputNode, BuilderDebugData);
	if (BuilderDebugData == nullptr || BuilderDebugData->HasData() == false)
	{
		// We only cache the results when either no debug data was supplied, or when the debug data was empty.  When
		// the debug data HasData returns true, errors were encountered while building.
		CachedTraversals.Add(CacheKey, FScriptTraversalCacheValue(OwningGraph->GetChangeID(), TraversalCachedDataSerialNumber, Traversal));
	}
	return Traversal;
}

void FTraversalCache::GetTopLevelScriptFunctionInputContextsInternal(const UNiagaraScript& Script, TMap<FGuid, FStaticContext>& OutFunctionInputContexts, FTraversalDebugData* TraversalDebugData) const
{
	UE::TScopeLock ScopeLock(CachedParsedRapidIterationParameterDataGuard);

	// Static inputs are stored as rapid iteration parameters, and their names are mangled to keep them unique. Modules
	// will be expecting these to be in the format Module.[InputName] so the names must be unmangled, and disambiguated by
	// module node guid so that they can be gathered when creating function contexts.  Emitter rapid iteration parameters
	// may also have additional mangling for the emitter name for module inputs of the form Module.Emitter.[InputName].
	const UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(Script.GetSource(FGuid()));
	FScriptTraversalCacheKey CacheKey = FScriptTraversalCacheKey::FromTopLevelScript(FObjectKey(ScriptSource->NodeGraph), Script.GetUsage(), Script.GetUsageId());
	FParsedRapidIterationParameterData* ParsedRapidIterationParameterData = CachedParsedRapidIterationParameterData.Find(CacheKey);
	if (ParsedRapidIterationParameterData == nullptr || ParsedRapidIterationParameterData->ParameterStoreLayoutVersion != Script.RapidIterationParameters.GetLayoutVersion())
	{
		if (ParsedRapidIterationParameterData == nullptr)
		{
			ParsedRapidIterationParameterData = &CachedParsedRapidIterationParameterData.Add(CacheKey);
		}
		ParsedRapidIterationParameterData->ParameterStoreLayoutVersion = Script.RapidIterationParameters.GetLayoutVersion();
		ParsedRapidIterationParameterData->StaticFunctionInputs.Empty();

		TSharedPtr<const FTraversal> SourceScriptTraversal = GetTopLevelScriptTraversal(Script, TraversalDebugData);
		if (SourceScriptTraversal.IsValid() == false)
		{
			return;
		}

		const TMap<FName, FGuid>& FunctionNameToNodeGuidMap = SourceScriptTraversal->GetFunctionNameToNodeGuidMap();
		if (FunctionNameToNodeGuidMap.IsEmpty())
		{
			return;
		}

		for (const FNiagaraVariableWithOffset& VariableWithOffset : Script.RapidIterationParameters.ReadParameterVariables())
		{
			const FNiagaraTypeDefinition& VariableType = VariableWithOffset.GetType();
			if (VariableType.IsStatic() == false)
			{
				continue;
			}

			FNiagaraTypeDefinition NonStaticVariableType = VariableType.RemoveStaticDef();
			if (FTraversalBuilder::IsValidSelectValueType(NonStaticVariableType) == false)
			{
				continue;
			}

			FNameBuilder VariableNameBuilder(VariableWithOffset.GetName());
			TArray<FStringView> NameParts;
			UE::String::ParseTokens(VariableNameBuilder.ToView(), TEXT("."), NameParts);
			if (NameParts[0] != FNiagaraConstants::RapidIterationParametersNamespaceString)
			{
				continue;
			}

			// For system scripts we need to skip the rapid iteration namespace, but for emitter and particle scripts we need to skip the rapid iteration and the emitter namespaces. 
			int32 ModuleNameIndex;
			int32 EmitterNameIndex;
			if (Script.GetUsage() == ENiagaraScriptUsage::SystemSpawnScript || Script.GetUsage() == ENiagaraScriptUsage::SystemUpdateScript)
			{
				EmitterNameIndex = INDEX_NONE;
				ModuleNameIndex = 1;
			}
			else
			{
				EmitterNameIndex = 1;
				ModuleNameIndex = 2;
			}

			if (NameParts.Num() - ModuleNameIndex <= 1)
			{
				continue;
			}

			FName ModuleName = FName(NameParts[ModuleNameIndex]);
			const FGuid* ModuleGuid = FunctionNameToNodeGuidMap.Find(ModuleName);
			if (ModuleGuid == nullptr)
			{
				continue;
			}

			TArray<FStringView> ModuleInputNameParts = { FNiagaraConstants::ModuleNamespaceString };
			ModuleInputNameParts.Append(TArrayView<FStringView>(NameParts).Slice(ModuleNameIndex + 1, NameParts.Num() - (ModuleNameIndex + 1)));
			if (ModuleInputNameParts.Num() > 2 &&
				(Script.GetUsage() == ENiagaraScriptUsage::EmitterSpawnScript || Script.GetUsage() == ENiagaraScriptUsage::EmitterUpdateScript) &&
				ModuleInputNameParts[1] == NameParts[EmitterNameIndex])
			{
				// This handles the special case of module inputs which have the form Module.Emitter.[InputName] in the emitter script.  In this case
				// both 'Module' and 'Emitter' will have been replaced with the module name and emitter name respectively, so both parts of the name
				// must be replaced with the generic versions.
				ModuleInputNameParts[1] = FNiagaraConstants::EmitterNamespaceString;
			}

			FName ModuleInputName = *FString::Join(ModuleInputNameParts, TEXT("."));
			FNiagaraVariableBase ModuleInputVariable(VariableType, ModuleInputName);

			ParsedRapidIterationParameterData->StaticFunctionInputs.Add(FParsedStaticFunctionInput(*ModuleGuid, ModuleInputVariable, VariableWithOffset.Offset));
		}
	}

	for (const FParsedStaticFunctionInput& ParsedStaticFunctionInput : ParsedRapidIterationParameterData->StaticFunctionInputs)
	{
		FStaticContext& FunctionInputContext = OutFunctionInputContexts.FindOrAdd(ParsedStaticFunctionInput.FunctionCallNodeGuid);
		FSelectKey SelectKey(ESelectKeySource::ModuleInput, ParsedStaticFunctionInput.InputVariable, NAME_None);
		if (FunctionInputContext.Contains(SelectKey) == false)
		{
			if (ensureMsgf(ParsedStaticFunctionInput.DataOffset + ParsedStaticFunctionInput.InputVariable.GetSizeInBytes() <= Script.RapidIterationParameters.GetParameterDataArray().Num(), TEXT("Cached parameter offset not valid for parameter store.")))
			{
				FNiagaraVariable Temp = ParsedStaticFunctionInput.InputVariable;
				Temp.SetData(Script.RapidIterationParameters.GetParameterData(ParsedStaticFunctionInput.DataOffset, ParsedStaticFunctionInput.InputVariable.GetType()));
				FSelectValue SelectValue = FTraversalBuilder::CreateSelectValue(Temp);
				FunctionInputContext.Add(SelectKey, SelectValue);
			}
		}
	}
}

struct FTraversedParameter
{
	FTraversedParameter()
	{
	}

	FTraversedParameter(const FNiagaraVariableBase& InParameter, EParameterFlags InParameterReferenceFlags, bool bInParameterNodeEnabled, ETraversalStateFlags InTraversalStateFlags)
		: Parameter(InParameter)
		, ParameterReferenceFlags(InParameterReferenceFlags)
		, bParameterNodeEnabled(bInParameterNodeEnabled)
		, TraversalStateFlags(InTraversalStateFlags)
	{
	}

	FORCEINLINE bool operator==(const FTraversedParameter& Other) const
	{
		return
			Parameter == Other.Parameter &&
			ParameterReferenceFlags == Other.ParameterReferenceFlags &&
			bParameterNodeEnabled == Other.bParameterNodeEnabled &&
			TraversalStateFlags == Other.TraversalStateFlags;
	}

	FNiagaraVariableBase Parameter;
	EParameterFlags ParameterReferenceFlags = EParameterFlags::None;
	bool bParameterNodeEnabled = false;
	ETraversalStateFlags TraversalStateFlags = ETraversalStateFlags::None;
};

class FTraversalReadAndWriteParameters : public FTraversalData
{
public:
	TArray<FTraversedParameter> ReadParameters;
	TArray<FTraversedParameter> WriteParameters;
};

class FCollectReadAndWriteParameters : public ITraversalVisitor
{
public:
	FCollectReadAndWriteParameters()
	{
	}

	virtual const FGuid GetVisitorId() const override
	{
		return FGuid(0x8C395293, 0xB41843A2, 0x942ABE00, 0x9EE1BA93);
	}

	virtual TSharedRef<FTraversalData> CreateTraversalData() const override
	{
		return MakeShared<FTraversalReadAndWriteParameters>();
	}

	virtual void VisitNode(ETraversalStateFlags TraversalState, const FTraversalNode& Node, FTraversalData& TraversalData) const override
	{
		if (Node.ParameterData.IsSet() == false)
		{
			return;
		}

		FTraversalReadAndWriteParameters* ParameterData = static_cast<FTraversalReadAndWriteParameters*>(&TraversalData);
		for (const FParameterRead& ReadParameter : Node.ParameterData->ReadParameterReferences)
		{
			ParameterData->ReadParameters.AddUnique(FTraversedParameter(ReadParameter.Parameter, ReadParameter.Flags, Node.bSourceNodeEnabled, TraversalState));
		}
		for (const FParameterWrite& WriteParameter : Node.ParameterData->WriteParameterReferences)
		{
			ParameterData->WriteParameters.AddUnique(FTraversedParameter(WriteParameter.Parameter, WriteParameter.Flags, Node.bSourceNodeEnabled, TraversalState));
		}
	}
};

void FTraversalCache::GetStackFunctionReadsInternal(
	const FTopLevelScriptStaticContext& ScriptStaticContext,
	const UNiagaraNodeFunctionCall& FunctionCallNode,
	TArray<FNiagaraVariable>& OutReads,
	TSet<FNiagaraVariable>& OutHiddenReads,
	EStackFunctionReadFilterFlags FilterFlags,
	FTraversalDebugData* TraversalDebugData,
	FTraversalBuilderDebugData* BuilderDebugData) const
{
	TSharedPtr<const FTraversal> ScriptTraversal = GetTopLevelScriptTraversalInternal(FunctionCallNode, BuilderDebugData);
	if (ScriptTraversal.IsValid() == false)
	{
		return;
	}

	FTraversalCallingContext CallingContext(ScriptStaticContext.GlobalContext, ScriptStaticContext.FunctionInputContexts);
	CallingContext.ConnectionTraversalMode = EConnectionTraversalMode::All;

	FCollectReadAndWriteParameters Visitor;
	TSharedRef<const FTraversalData> TraversalData = ScriptTraversal->TraverseWithVisitor(CallingContext, Visitor, TraversalDebugData);

	const FTraversalData* FunctionCallTraversalData = TraversalData->GetCalledFunctionTraversalData(FunctionCallNode.NodeGuid);
	if (FunctionCallTraversalData == nullptr)
	{
		return;
	}

	const FTraversalReadAndWriteParameters* TraversalParameters = static_cast<const FTraversalReadAndWriteParameters*>(FunctionCallTraversalData);
	TMap<FNiagaraVariable, TArray<ETraversalStateFlags>> ReadVariableToFlagList;
	for (const FTraversedParameter& ReadParameter : TraversalParameters->ReadParameters)
	{
		if (HasFlag(ReadParameter.TraversalStateFlags, ETraversalStateFlags::UnconnectedRoot))
		{
			continue;
		}

		bool bIsInput = HasFlag(ReadParameter.ParameterReferenceFlags, EParameterFlags::ModuleInput);
		bool bIsEnabled = HasFlag(ReadParameter.TraversalStateFlags, ETraversalStateFlags::CallerDisabled) == false && ReadParameter.bParameterNodeEnabled;
		if ((HasFlag(FilterFlags, EStackFunctionReadFilterFlags::InputsOnly) == false || bIsInput) &&
			(HasFlag(FilterFlags, EStackFunctionReadFilterFlags::EnabledOnly) == false || bIsEnabled))
		{
			OutReads.AddUnique(ReadParameter.Parameter);
			TArray<ETraversalStateFlags>& ReadTraversalStateFlags = ReadVariableToFlagList.FindOrAdd(ReadParameter.Parameter);
			ReadTraversalStateFlags.Add(ReadParameter.TraversalStateFlags);
		}
	}

	for (FNiagaraVariable& Read : OutReads)
	{
		TArray<ETraversalStateFlags> ReadFlags = ReadVariableToFlagList[Read];
		bool bIsVisible = ReadFlags.ContainsByPredicate([](const ETraversalStateFlags& Flags)
		{
			return HasFlag(Flags, ETraversalStateFlags::CulledBySwitch) == false;
		});

		if (bIsVisible == false)
		{
			OutHiddenReads.Add(Read);
		}
	}
}

struct FTraversedStaticSwitchInput
{
	FTraversedStaticSwitchInput(const FSelectKey& InSelectKey, ETraversalStateFlags InTraversalStateFlags)
		: SelectKey(InSelectKey)
		, TraversalStateFlags(InTraversalStateFlags)
	{
	}

	FORCEINLINE bool operator==(const FTraversedStaticSwitchInput& Other) const
	{
		return SelectKey == Other.SelectKey && TraversalStateFlags == Other.TraversalStateFlags;
	}

	FSelectKey SelectKey;
	ETraversalStateFlags TraversalStateFlags = ETraversalStateFlags::None;
};

class FTraversalStaticSwitchInputs : public FTraversalData
{
public:
	TArray<FTraversedStaticSwitchInput> StaticSwitchInputs;
};

class FCollectStaticSwitchInputs : public ITraversalVisitor
{
public:
	FCollectStaticSwitchInputs()
	{
	}

	virtual const FGuid GetVisitorId() const override
	{
		return FGuid(0x92B6A255, 0x484B4323, 0x96B888C9, 0x39B6B8C4);
	}

	virtual TSharedRef<FTraversalData> CreateTraversalData() const override
	{
		return MakeShared<FTraversalStaticSwitchInputs>();
	}

	virtual void VisitNode(ETraversalStateFlags TraversalState, const FTraversalNode& Node, FTraversalData& TraversalData) const override
	{
		FTraversalStaticSwitchInputs* SwitchInputData = static_cast<FTraversalStaticSwitchInputs*>(&TraversalData);
		if (Node.SelectData.IsSet())
		{
			if (Node.SelectData->SelectKey.IsValid() && Node.SelectData->SelectKey.Source == ESelectKeySource::FunctionCallNode)
			{
				SwitchInputData->StaticSwitchInputs.AddUnique(FTraversedStaticSwitchInput(Node.SelectData->SelectKey, TraversalState));
			}
		}
		else if (Node.FunctionCallData.IsSet())
		{
			for (const FFunctionInputSelectValue& InputSelectValue : Node.FunctionCallData->InputSelectValues)
			{
				if (InputSelectValue.ConnectionPinId.IsSet() == false && InputSelectValue.LocalValue.IsSet() == false)
				{
					FSelectKey PropagatedSelectKey;
					if (InputSelectValue.OptionalPropagatedNameOverride.IsSet())
					{
						FNiagaraVariableBase OverrideVariable(InputSelectValue.InputSelectKey.Variable.GetType(), InputSelectValue.OptionalPropagatedNameOverride.GetValue());
						PropagatedSelectKey = FSelectKey(ESelectKeySource::FunctionCallNode, OverrideVariable, NAME_None);
					}
					else
					{
						PropagatedSelectKey = InputSelectValue.InputSelectKey;
					}
					SwitchInputData->StaticSwitchInputs.AddUnique(FTraversedStaticSwitchInput(PropagatedSelectKey, TraversalState));
				}
			}
		}
	}
};

void FTraversalCache::GetStackFunctionStaticInputsInternal(
	const FTopLevelScriptStaticContext& ScriptStaticContext,
	const UNiagaraNodeFunctionCall& FunctionCallNode,
	TArray<FNiagaraVariable>& OutStaticInputs,
	TSet<FNiagaraVariable>& OutHiddenStaticInputs,
	FTraversalDebugData* TraversalDebugData,
	FTraversalBuilderDebugData* BuilderDebugData) const
{
	TSharedPtr<const FTraversal> ScriptTraversal = GetTopLevelScriptTraversalInternal(FunctionCallNode, BuilderDebugData);
	if (ScriptTraversal.IsValid() == false)
	{
		return;
	}

	FTraversalCallingContext CallingContext(ScriptStaticContext.GlobalContext, ScriptStaticContext.FunctionInputContexts);
	CallingContext.ConnectionTraversalMode = EConnectionTraversalMode::All;

	FCollectStaticSwitchInputs Visitor;
	TSharedRef<const FTraversalData> TraversalData = ScriptTraversal->TraverseWithVisitor(CallingContext, Visitor, TraversalDebugData);

	const FTraversalData* FunctionCallTraversalData = TraversalData->GetCalledFunctionTraversalData(FunctionCallNode.NodeGuid);
	if (FunctionCallTraversalData == nullptr)
	{
		return;
	}

	const FTraversalStaticSwitchInputs* TraversalStaticSwitches = static_cast<const FTraversalStaticSwitchInputs*>(FunctionCallTraversalData);
	TMap<FNiagaraVariable, TArray<ETraversalStateFlags>> StaticInputVariableToFlagList;
	for (const FTraversedStaticSwitchInput& TraversedStaticInput : TraversalStaticSwitches->StaticSwitchInputs)
	{
		OutStaticInputs.AddUnique(TraversedStaticInput.SelectKey.Variable);
		TArray<ETraversalStateFlags>& StaticInputTraversalStateFlags = StaticInputVariableToFlagList.FindOrAdd(TraversedStaticInput.SelectKey.Variable);
		StaticInputTraversalStateFlags.Add(TraversedStaticInput.TraversalStateFlags);
	}

	for (FNiagaraVariable& StaticInput : OutStaticInputs)
	{
		TArray<ETraversalStateFlags> StaticInputFlags = StaticInputVariableToFlagList[StaticInput];
		bool bIsVisible = StaticInputFlags.ContainsByPredicate([](const ETraversalStateFlags& Flags)
		{
			return HasFlag(Flags, ETraversalStateFlags::CulledBySwitch) == false && HasFlag(Flags, ETraversalStateFlags::UnconnectedRoot) == false;
		});

		if (bIsVisible == false)
		{
			OutHiddenStaticInputs.Add(StaticInput);
		}
	}
}

void FTraversalCache::GetStackFunctionStaticInputPinsInternal(
	const FTopLevelScriptStaticContext& ScriptStaticContext,
	const UNiagaraNodeFunctionCall& FunctionCallNode,
	TArray<UEdGraphPin*>& OutStaticInputPins,
	TSet<UEdGraphPin*>& OutHiddenStaticInputPins,
	FTraversalDebugData* TraversalDebugData) const
{
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	TArray<FNiagaraVariable> StaticInputs;
	TSet<FNiagaraVariable> HiddenStaticInputs;
	GetStackFunctionStaticInputs(ScriptStaticContext, FunctionCallNode, StaticInputs, HiddenStaticInputs, TraversalDebugData);
	for (FNiagaraVariable StaticInput : StaticInputs)
	{
		FEdGraphPinType PinType = Schema->TypeDefinitionToPinType(StaticInput.GetType());
		for (UEdGraphPin* Pin : FunctionCallNode.Pins)
		{
			if (Pin->Direction != EEdGraphPinDirection::EGPD_Input)
			{
				continue;
			}
			if (Pin->PinName.IsEqual(StaticInput.GetName()) && Pin->PinType == PinType)
			{
				OutStaticInputPins.Add(Pin);
				if (HiddenStaticInputs.Contains(StaticInput))
				{
					OutHiddenStaticInputPins.Add(Pin);
				}
				break;
			}
		}
	}
}

void FTraversalCache::InvalidateCacheInternal() const
{
	{
		UE::TScopeLock ScopeLock(CachedTraversalsGuard);
		CachedTraversals.Empty();
	}
	{
		UE::TScopeLock ScopeLock(CachedParsedRapidIterationParameterDataGuard);
		CachedParsedRapidIterationParameterData.Empty();
	}
	{
		UE::TScopeLock ScopeLock(SoftObjectPathToScriptCacheGuard);
		SoftObjectPathToScriptCache.Empty();
	}
}

EStackFunctionReadFilterFlags operator|(
	EStackFunctionReadFilterFlags FlagsA,
	EStackFunctionReadFilterFlags FlagsB)
{
	return (EStackFunctionReadFilterFlags)((uint8)FlagsA | (uint8)(FlagsB));
}

EStackFunctionReadFilterFlags operator&(
	EStackFunctionReadFilterFlags FlagsA,
	EStackFunctionReadFilterFlags FlagsB)
{
	return (EStackFunctionReadFilterFlags)((uint8)FlagsA & (uint8)(FlagsB));
}

EStackFunctionReadFilterFlags operator~(
	EStackFunctionReadFilterFlags Flags)
{
	return (EStackFunctionReadFilterFlags)~(uint8)Flags;
}

} // UE::Niagara::TraversalCache