// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Misc/Guid.h"
#include "NiagaraCommon.h"
#include "NiagaraTypes.h"
#include "TraversalCache/TraversalShared.h"
#include "UObject/ObjectKey.h"
#include "Async/Mutex.h"

class UEdGraphPin;
class UNiagaraNode;
class UNiagaraNodeFunctionCall;
class UNiagaraNodeOutput;
class UNiagaraScript;

namespace UE::Niagara::TraversalCache
{

class  FTraversal;
class  FTraversalBuilder;
struct FTraversalBuilderDebugData;
struct FTraversalDebugData;

enum class EStackFunctionReadFilterFlags : uint8
{
	None = 0,
	InputsOnly = 1,
	EnabledOnly = 2
};

class FTraversalCache : public TSharedFromThis<FTraversalCache>, public FEditorUndoClient
{
public:
	void Initialize();

	virtual ~FTraversalCache();

	static TSharedPtr<const FTraversal> GetScriptAssetTraversal(const FSoftObjectPath& ScriptSoftObjectPath, const FGuid& ScriptVersion, FTraversalBuilderDebugData* BuilderDebugData = nullptr);

	static TSharedPtr<const FTraversal> GetScriptAssetTraversal(const FSoftObjectPath& ScriptSoftObjectPath, const FGuid& ScriptVersion, FTraversalDebugData* TraversalDebugData = nullptr);

	static TSharedPtr<const FTraversal> GetTopLevelScriptTraversal(const UNiagaraScript& Script, FTraversalBuilderDebugData* BuilderDebugData = nullptr);

	static TSharedPtr<const FTraversal> GetTopLevelScriptTraversal(const UNiagaraScript& Script, FTraversalDebugData* TraversalDebugData = nullptr);

	static void GetTopLevelScriptFunctionInputContexts(const UNiagaraScript& Script, TMap<FGuid, FStaticContext>& OutFunctionInputContexts, FTraversalDebugData* TraversalDebugData = nullptr);

	NIAGARAEDITOR_API static void GetStackFunctionReads(
		const FTopLevelScriptStaticContext& ScriptStaticContext,
		const UNiagaraNodeFunctionCall& FunctionCallNode,
		TArray<FNiagaraVariable>& OutReads,
		TSet<FNiagaraVariable>& OutHiddenReads,
		EStackFunctionReadFilterFlags FilterFlags = EStackFunctionReadFilterFlags::InputsOnly,
		FTraversalDebugData* TraversalDebugData = nullptr);

	NIAGARAEDITOR_API static void GetStackFunctionStaticInputs(
		const FTopLevelScriptStaticContext& ScriptStaticContext,
		const UNiagaraNodeFunctionCall& FunctionCallNode,
		TArray<FNiagaraVariable>& OutStaticInputs,
		TSet<FNiagaraVariable>& OutHiddenStaticInputs,
		FTraversalDebugData* TraversalDebugData = nullptr);

	NIAGARAEDITOR_API static void GetStackFunctionStaticInputPins(
		const FTopLevelScriptStaticContext& ScriptStaticContext,
		const UNiagaraNodeFunctionCall& FunctionCallNode,
		TArray<UEdGraphPin*>& OutStaticInputPins,
		TSet<UEdGraphPin*>& OutHiddenStaticInputPins,
		FTraversalDebugData* TraversalDebugData = nullptr);

	NIAGARAEDITOR_API static void InvalidateCache();

	//~ FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

private:
	void OnScriptApplied(UNiagaraScript* InScript, FGuid InVersionGuid);
	
	static FTraversalCache& GetInstance();

	const UNiagaraScript* GetScriptFromPath(const FSoftObjectPath& Path) const;

	TSharedPtr<const FTraversal> GetScriptAssetTraversalInternal(const FSoftObjectPath& ScriptSoftObjectPath, const FGuid& ScriptVersion, FTraversalBuilderDebugData* BuilderDebugData) const;

	TSharedPtr<const FTraversal> GetTopLevelScriptTraversalInternal(const UNiagaraScript& Script, FTraversalBuilderDebugData* BuilderDebugData) const;

	TSharedPtr<const FTraversal> GetTopLevelScriptTraversalInternal(const UNiagaraNode& StackNode, FTraversalBuilderDebugData* BuilderDebugData) const;

	void GetTopLevelScriptFunctionInputContextsInternal(const UNiagaraScript& Script, TMap<FGuid, FStaticContext>& OutFunctionInputContexts, FTraversalDebugData* TraversalDebugData) const;

	void GetStackFunctionReadsInternal(
		const FTopLevelScriptStaticContext& ScriptStaticContext,
		const UNiagaraNodeFunctionCall& FunctionCallNode,
		TArray<FNiagaraVariable>& OutInputs,
		TSet<FNiagaraVariable>& OutHiddenInputs,
		EStackFunctionReadFilterFlags FilterFlags,
		FTraversalDebugData* TraversalDebugData,
		FTraversalBuilderDebugData* BuilderDebugData) const; 

	void GetStackFunctionStaticInputsInternal(
		const FTopLevelScriptStaticContext& ScriptStaticContext,
		const UNiagaraNodeFunctionCall& FunctionCallNode,
		TArray<FNiagaraVariable>& OutStaticInputs,
		TSet<FNiagaraVariable>& OutHiddenStaticInputs,
		FTraversalDebugData* TraversalDebugData,
		FTraversalBuilderDebugData* BuilderDebugData) const;

	void GetStackFunctionStaticInputPinsInternal(
		const FTopLevelScriptStaticContext& ScriptStaticContext,
		const UNiagaraNodeFunctionCall& FunctionCallNode,
		TArray<UEdGraphPin*>& OutStaticInputPins,
		TSet<UEdGraphPin*>& OutHiddenStaticInputPins,
		FTraversalDebugData* TraversalDebugData) const;

	void InvalidateCacheInternal() const;

private:
	struct FScriptTraversalCacheKey
	{
		FScriptTraversalCacheKey()
		{
		}

		static FScriptTraversalCacheKey FromScriptAsset(
			const FObjectKey& InScriptObjectKey,
			const FGuid& InScriptVersion)
		{
			return FScriptTraversalCacheKey(
				InScriptObjectKey,
				InScriptVersion,
				ENiagaraScriptUsage::Function,
				FGuid());
		}

		static FScriptTraversalCacheKey FromTopLevelScript(
			const FObjectKey& InGraphObjectKey,
			ENiagaraScriptUsage InScriptUsage,
			const FGuid& InScriptUsageId)
		{
			return FScriptTraversalCacheKey(
				InGraphObjectKey,
				FGuid(),
				InScriptUsage,
				InScriptUsageId);
		}

		FORCEINLINE bool operator==(const FScriptTraversalCacheKey& Other) const
		{
			return 
				OwningObjectKey == Other.OwningObjectKey &&
				ScriptVersion == Other.ScriptVersion &&
				ScriptUsage == Other.ScriptUsage &&
				ScriptUsageId == Other.ScriptUsageId;
		}

		friend FORCEINLINE uint32 GetTypeHash(const FScriptTraversalCacheKey& Key)
		{
			return Key.Hash;
		}

		const FObjectKey OwningObjectKey;
		const FGuid ScriptVersion;
		const ENiagaraScriptUsage ScriptUsage = ENiagaraScriptUsage::Function;
		const FGuid ScriptUsageId;
		const int32 Hash = 0;

	private:
		FScriptTraversalCacheKey(const FObjectKey& InOwningObjectKey, const FGuid& InScriptVersion, ENiagaraScriptUsage InScriptUsage, const FGuid& InScriptUsageId)
			: OwningObjectKey(InOwningObjectKey)
			, ScriptVersion(InScriptVersion)
			, ScriptUsage(InScriptUsage)
			, ScriptUsageId(InScriptUsageId)
			, Hash(ComputeHash(OwningObjectKey, ScriptVersion, ScriptUsage, ScriptUsageId))
		{
		}

		static uint32 ComputeHash(const FObjectKey& OwningObjectKey, const FGuid& ScriptVersion, ENiagaraScriptUsage ScriptUsage, const FGuid& ScriptUsageId)
		{
			uint32 Hash = GetTypeHash(OwningObjectKey);
			Hash = HashCombine(Hash, GetTypeHash(ScriptVersion));
			Hash = HashCombine(Hash, GetTypeHash(ScriptUsage));
			Hash = HashCombine(Hash, GetTypeHash(ScriptUsageId));
			return Hash;
		}
	};

	struct FScriptTraversalCacheValue
	{
		FScriptTraversalCacheValue(FGuid InGraphChangeId, const uint32& InTraversalCachedDataSerialNumber, TSharedRef<const FTraversal> InScriptTraversal)
			: GraphChangeId(InGraphChangeId)
			, TraversalCachedDataSerialNumber(InTraversalCachedDataSerialNumber)
			, ScriptTraversal(InScriptTraversal)
		{
		}

		FGuid GraphChangeId;
		uint32 TraversalCachedDataSerialNumber;
		TSharedRef<const FTraversal> ScriptTraversal;
	};

	struct FParsedStaticFunctionInput
	{
		FParsedStaticFunctionInput(const FGuid& InFunctionCallNodeGuid, const FNiagaraVariableBase& InInputVariable, int32 InDataOffset)
			: FunctionCallNodeGuid(InFunctionCallNodeGuid)
			, InputVariable(InInputVariable)
			, DataOffset(InDataOffset)
		{
		}

		FGuid FunctionCallNodeGuid;
		FNiagaraVariableBase InputVariable;
		int32 DataOffset = INDEX_NONE;
	};

	struct FParsedRapidIterationParameterData
	{
		int32 ParameterStoreLayoutVersion = INDEX_NONE;
		TArray<FParsedStaticFunctionInput> StaticFunctionInputs;
	};

	static FTraversalCache* Instance;
	bool bInitialized = false;

	/** A serial number to track potential dependent graph changes.  We already discard cached traversals when the graph change id updates,
		but the data collected by the visitors can depend on external references which aren't captured by the graphs change id.  When this
		number is updated it signals the traversals to discard any data collected by visitors so that it can be recomputed when requested. */
	uint32 TraversalCachedDataSerialNumber = 0;

	TSharedPtr<FTraversalBuilder> TraversalBuilder;

	mutable TMap<FScriptTraversalCacheKey, FScriptTraversalCacheValue> CachedTraversals;
	mutable UE::FMutex CachedTraversalsGuard;

	mutable TMap<FScriptTraversalCacheKey, FParsedRapidIterationParameterData> CachedParsedRapidIterationParameterData;
	mutable UE::FMutex CachedParsedRapidIterationParameterDataGuard;

	mutable TMap<FSoftObjectPath, TWeakObjectPtr<const UNiagaraScript>> SoftObjectPathToScriptCache;
	mutable UE::FMutex SoftObjectPathToScriptCacheGuard;
};

EStackFunctionReadFilterFlags operator|(
	EStackFunctionReadFilterFlags FlagsA,
	EStackFunctionReadFilterFlags FlagsB);

EStackFunctionReadFilterFlags operator&(
	EStackFunctionReadFilterFlags FlagsA,
	EStackFunctionReadFilterFlags FlagsB);

EStackFunctionReadFilterFlags operator~(
	EStackFunctionReadFilterFlags Flags);

} // UE::Niagara::TraversalCache

