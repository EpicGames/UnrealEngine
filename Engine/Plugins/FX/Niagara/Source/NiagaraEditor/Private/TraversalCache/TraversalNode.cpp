// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraversalCache/TraversalNode.h"

namespace UE::Niagara::TraversalCache
{

const FSelectInputData* FSelectData::FindInputDataForSelectValue(const FSelectValue& SelectValue) const
{
	for (const FSelectInputData& SelectInputData : InputData)
	{
		if (SelectInputData.SelectValue == SelectValue)
		{
			return &SelectInputData;
		}
	}
	return nullptr;
}

const FSelectInputData* FSelectData::FindInputDataForConnectionPinId(const FGuid& ConnectionPinId) const
{
	for (const FSelectInputData& SelectInputData : InputData)
	{
		if (SelectInputData.ConnectionPinId.IsSet() && SelectInputData.ConnectionPinId.GetValue() == ConnectionPinId)
		{
			return &SelectInputData;
		}
	}
	return nullptr;
}

const FTraversalNode* FTraversalNode::GetConnectedNodeByPinId(const FGuid& PinId) const
{
	const FConnection* MatchingConnection = Connections.FindByPredicate([PinId](const FConnection& Connection) { return Connection.PinId == PinId; });
	return MatchingConnection != nullptr ? &MatchingConnection->GetNode() : nullptr;
}

} // UE::Niagara::TraversalCache