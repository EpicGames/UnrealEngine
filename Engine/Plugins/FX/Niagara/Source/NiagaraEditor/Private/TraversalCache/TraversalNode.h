// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "TraversalCache/TraversalShared.h"

namespace UE::Niagara::TraversalCache
{

struct FFunctionInputData
{
	FSelectKey InputSelectKey;
	TOptional<FSelectValue> LocalValue;
};

struct FSelectInputData
{
	FSelectValue SelectValue;
	TOptional<FSelectValue> LocalValue;
	TOptional<FGuid> ConnectionPinId;
};

struct FSelectData
{
	const FSelectInputData* FindInputDataForSelectValue(const FSelectValue& SelectValue) const;
	const FSelectInputData* FindInputDataForConnectionPinId(const FGuid& ConnectionPinId) const;

	ESelectMode SelectMode = ESelectMode::None;
	FSelectKey SelectKey;
	FGuid SelectConnectionPinId;
	TArray<FSelectInputData> InputData;
};

struct FParameterReference
{
	FParameterReference()
	{
	}

	FParameterReference(const FNiagaraVariableBase& InParameter, EParameterFlags InFlags)
		: Parameter(InParameter)
		, Flags(InFlags)
	{
	}

	FNiagaraVariableBase Parameter;
	EParameterFlags Flags = EParameterFlags::None;
	TOptional<FGuid> OptionalConnectionPinId;
	TOptional<FSelectValue> OptionalLocalSelectValue;
	TOptional<FName> OptionalNamespaceModifier;
};

struct FParameterRead : public FParameterReference
{
	FParameterRead()
	{
	}

	FParameterRead(const FNiagaraVariableBase& InParameter, EParameterFlags InFlags)
		: FParameterReference(InParameter, InFlags)
	{
	}

	bool bIsDiscoverRead = false;
	TOptional<FNiagaraVariant> OptionalDefaultValue;
	TOptional<FName> OptionalDefaultBinding;
};

struct FParameterWrite : public FParameterReference
{
	FParameterWrite()
	{
	}

	FParameterWrite(const FNiagaraVariableBase& InParameter, EParameterFlags InFlags)
		: FParameterReference(InParameter, InFlags)
	{
	}

	TOptional<FGuid> OptionalTargetFunctionCallNodeGuid;
};

struct FParameterData
{
	TArray<FParameterRead> ReadParameterReferences;
	TArray<FParameterWrite> WriteParameterReferences;
	FGuid ExecutionConnectionPinId;
};

struct FFunctionInputSelectValue
{
	FSelectKey InputSelectKey;
	TOptional<FSelectValue> LocalValue;
	TOptional<FGuid> ConnectionPinId;
	TOptional<FName> OptionalPropagatedNameOverride;
};

struct FFunctionCallData
{
	FScriptReference FunctionScriptReference;
	FName FunctionCallName;
	TArray<FFunctionInputSelectValue> InputSelectValues;
	ENiagaraFunctionDebugState DebugState = ENiagaraFunctionDebugState::NoDebug;
	FGuid ExecutionConnectionPinId;
};

struct FStaticOpInputData
{
	TOptional<FSelectValue> LocalValue;
	TOptional<FGuid> ConnectionPinId;
};

struct FStaticOpData
{
	FStaticOpData()
		: OpName(NAME_None)
	{
	}

	FName OpName;
	TArray<FStaticOpInputData> InputData;
};

class FTraversalNode
{
	friend class FTraversalBuilder;

public:
	struct FConnection
	{
		friend class FTraversalBuilder;

		FConnection(const FGuid& InPinId, TSharedRef<FTraversalNode> InNode)
			: PinId(InPinId)
			, Node(InNode)
		{
		}

		FORCEINLINE bool operator==(const FConnection& Other) const
		{
			return PinId == Other.PinId && Node == Other.Node;
		}

		const FTraversalNode& GetNode() const { return Node.Get(); }

		FGuid PinId;

	private:
		TSharedRef<FTraversalNode> Node;
	};

	const FTraversalNode* GetConnectedNodeByPinId(const FGuid& PinId) const;

	FGuid SourceNodeGuid = FGuid();
	FName SourceNodeTypeName;
	bool bSourceNodeEnabled = true;

	bool bIsNoop = false;
	TOptional<const FFunctionInputData> FunctionInputData;
	TOptional<const FSelectData> SelectData;
	TOptional<const FParameterData> ParameterData;
	TOptional<const FFunctionCallData> FunctionCallData;
	TOptional<const FStaticOpData> StaticOpData;

	TArray<FConnection> Connections;
};

} // UE::Niagara::TraversalCache