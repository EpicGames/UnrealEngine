// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraversalCache/TraversalShared.h"

#include "TraversalCache/TraversalBuilder.h"

namespace UE::Niagara::TraversalCache
{

const FSelectValue& FSelectValue::GetBoolTrue()
{
	static const FSelectValue BoolTrue = FTraversalBuilder::CreateSelectValue(true);
	return BoolTrue;
}

const FSelectValue& FSelectValue::GetBoolFalse()
{
	static const FSelectValue BoolFalse = FTraversalBuilder::CreateSelectValue(false);
	return BoolFalse;
}

const FSelectValue& FSelectValue::GetDebugStateNoDebug()
{
	static const FSelectValue DebugStateNoDebug = FTraversalBuilder::CreateSelectValue(FNiagaraTypeDefinition::GetFunctionDebugStateEnum(), (int32)ENiagaraFunctionDebugState::NoDebug);
	return DebugStateNoDebug;
}

const FSelectValue& FSelectValue::GetDebugStateBasic()
{
	static const FSelectValue DebugStateBasic = FTraversalBuilder::CreateSelectValue(FNiagaraTypeDefinition::GetFunctionDebugStateEnum(), (int32)ENiagaraFunctionDebugState::Basic);
	return DebugStateBasic;
}

TOptional<FSelectValue> FStaticContext::GetSelectValue(const FSelectKey& Key) const
{
	const FSelectValue* SelectValuePtr = Find(Key);
	return SelectValuePtr != nullptr
		? TOptional<FSelectValue>(*SelectValuePtr)
		: TOptional<FSelectValue>();
}

TOptional<FSelectValue> FTraversalLocalContext::GetSelectValue(const FTraversalCallingContext& CallingContext, const FSelectKey& SelectKey) const
{
	TOptional<FSelectValue> SelectValue;
	
	StaticContext.GetSelectValue(SelectKey);
	if (SelectValue.IsSet())
	{
		return SelectValue;
	}

	SelectValue = CallingContext.FunctionCallStaticContext.GetSelectValue(SelectKey);
	if (SelectValue.IsSet())
	{
		return SelectValue;
	}

	SelectValue = CallingContext.CallingStaticContext.GetSelectValue(SelectKey);
	if (SelectValue.IsSet())
	{
		return SelectValue;
	}

	return CallingContext.GlobalStaticContext.GetSelectValue(SelectKey);
}

const FTraversalData* FTraversalData::GetCalledFunctionTraversalData(FGuid FunctionCallNodeGuid) const
{
	for (const FFunctionCallTraversalData& FunctionCallTraversalData : CalledFunctionTraversalData)
	{
		if (FunctionCallTraversalData.FunctionCallNodeGuid == FunctionCallNodeGuid)
		{
			return &FunctionCallTraversalData.FunctionCallTraversalData.Get();
		}
	}
	return nullptr;
}

EParameterFlags operator|(
	EParameterFlags FlagsA,
	EParameterFlags FlagsB)
{
	return (EParameterFlags)((uint16)FlagsA | (uint16)FlagsB);
}

EParameterFlags operator&(
	EParameterFlags FlagsA,
	EParameterFlags FlagsB)
{
	return (EParameterFlags)((uint16)FlagsA & (uint16)FlagsB);
}

EParameterFlags operator~(
	EParameterFlags Flags)
{
	return (EParameterFlags)(~(uint16)(Flags));
}

ETraversalStateFlags operator|(
	ETraversalStateFlags FlagsA,
	ETraversalStateFlags FlagsB)
{
	return (ETraversalStateFlags)((uint8)FlagsA | (uint8)FlagsB);
}

ETraversalStateFlags operator&(
	ETraversalStateFlags FlagsA,
	ETraversalStateFlags FlagsB)
{
	return (ETraversalStateFlags)((uint8)FlagsA & (uint8)FlagsB);
}

ETraversalStateFlags operator~(
	ETraversalStateFlags Flags)
{
	return (ETraversalStateFlags)(~(uint8)Flags);
}

FTraversalCallingContext::FContextHash FTraversalCallingContext::GenerateHash() const
{
	auto AddStaticContextToHash = [&](FMD5& Hash, const FStaticContext& StaticContext)
	{
		TArray<const TPair<FSelectKey, FSelectValue>*> KeysAndValues;
		KeysAndValues.Reserve(StaticContext.Num());
		for (const TPair<FSelectKey, FSelectValue>& Pair : StaticContext)
		{
			KeysAndValues.Add(&Pair);
		}

		KeysAndValues.Sort([](const TPair<FSelectKey, FSelectValue>& PairA, const TPair<FSelectKey, FSelectValue>& PairB)
		{
			return PairA.Key.Variable.GetName().FastLess(PairB.Key.Variable.GetName());
		});

		for (const TPair<FSelectKey, FSelectValue>* Pair : KeysAndValues)
		{
			uint64 KeyNameUnstableInt = Pair->Key.Variable.GetName().ToUnstableInt();
			Hash.Update((const uint8*)&KeyNameUnstableInt, sizeof(uint64));
			Hash.Update((const uint8*)&Pair->Value.NumericValue, sizeof(int32));
		}

		int32 PairCount = StaticContext.Num();
		Hash.Update((uint8*)&PairCount, sizeof(int32));
	};

	auto AddFunctionStaticInputContextsToHash = [&](FMD5& Hash, const TMap<FGuid, FStaticContext>& FunctionStaticInputContexts)
	{
		TArray<const TPair<FGuid, FStaticContext>*> ContextPairs;
		ContextPairs.Reserve(FunctionStaticInputContexts.Num());
		for (const TPair<FGuid, FStaticContext>& ContextPair : FunctionStaticInputContexts)
		{
			ContextPairs.Add(&ContextPair);
		}

		ContextPairs.Sort([](const TPair<FGuid, FStaticContext>& PairA, const TPair<FGuid, FStaticContext>& PairB)
		{
			return PairA.Key < PairB.Key;
		});

		for (const TPair<FGuid, FStaticContext>* ContextPair : ContextPairs)
		{
			Hash.Update((const uint8*)&ContextPair->Key, sizeof(FGuid));
			AddStaticContextToHash(Hash, ContextPair->Value);
		}

		int32 ContextPairCount = FunctionStaticInputContexts.Num();
		Hash.Update((uint8*)&ContextPairCount, sizeof(int32));
	};

	FMD5 Hash;
	AddStaticContextToHash(Hash, GlobalStaticContext);
	AddFunctionStaticInputContextsToHash(Hash, ExternalFunctionInputStaticContexts);
	AddStaticContextToHash(Hash, CallingStaticContext);
	AddStaticContextToHash(Hash, FunctionCallStaticContext);
	Hash.Update((uint8*)&ConnectionTraversalMode, sizeof(EConnectionTraversalMode));
	Hash.Update((uint8*)&TraversalState, sizeof(ETraversalStateFlags));

	FContextHash ContextHash;
	Hash.Final((uint8*)&ContextHash.HashData);
	return ContextHash;
}

void FTraversalBuilderDebugData::AddMultipleInputConnection(
	const TPair<FGuid, FGuid>& TargetNodeGuidAndPinId,
	const TArray<TPair<FGuid, FGuid>>& ConnectedNodeGuidsAndPinIds)
{
	MultipleInputConnections.AddUnique(FMultipleInputConnection(TargetNodeGuidAndPinId, ConnectedNodeGuidsAndPinIds));
}

void FTraversalBuilderDebugData::AddUntrimmedNoop(const FGuid& NodeGuid, FName SourceNodeTypeName)
{
	UntrimmedNoops.AddUnique(FUntrimmedNoop(NodeGuid, SourceNodeTypeName));
}

void FTraversalBuilderDebugData::AddUnresolvedSelectOutput(const FGuid& NodeGuid, const FGuid& PinGuid)
{
	UnresolvedSelectOutputs.AddUnique(FUnresolvedSelectOutput(NodeGuid, PinGuid));
}

void FTraversalDebugData::AddUnresolvedSelect(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FSelectKey& InSelectKey)
{
	UnresolvedSelects.AddUnique(FUnresolvedSelect(InFunctionCallStack, InSourceNodeGuid, InSelectKey));
}

void FTraversalDebugData::AddUnresolvedSelectInput(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FSelectKey& InSelectKey, const FSelectValue& InResolvedSelectValue)
{
	UnresolvedSelectInputs.AddUnique(FUnresolvedSelectInput(InFunctionCallStack, InSourceNodeGuid, InSelectKey, InResolvedSelectValue));
}

void FTraversalDebugData::AddUnresolvedRead(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FNiagaraVariableBase& InReadParameter)
{
	UnresolvedReads.AddUnique(FUnresolvedRead(InFunctionCallStack, InSourceNodeGuid, InReadParameter));
}

void FTraversalDebugData::AddUnresolvedStaticOp(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, FName InOpName, int32 InUnresolvedPinIndex)
{
	UnresolvedStaticOps.AddUnique(FUnresolvedStaticOp(InFunctionCallStack, InSourceNodeGuid, InOpName, InUnresolvedPinIndex));
}

void FTraversalDebugData::AddUnresolvedFunctionInput(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FNiagaraVariableBase& InFunctionInput)
{
	UnresolvedReads.AddUnique(FUnresolvedRead(InFunctionCallStack, InSourceNodeGuid, InFunctionInput));
}

void FTraversalDebugData::AddBuilderDebugOutput(const FSoftObjectPath InGraphPath, const ENiagaraScriptUsage InScriptUsage,
	const FGuid& InScriptUsageId, const FGuid& InScriptVersion, const FTraversalBuilderDebugData& InBuilderDebugData)
{
	BuilderDebugOutputs.AddUnique(FBuilderDebugOutput(InGraphPath, InScriptUsage, InScriptUsageId, InScriptVersion, InBuilderDebugData));
}

}