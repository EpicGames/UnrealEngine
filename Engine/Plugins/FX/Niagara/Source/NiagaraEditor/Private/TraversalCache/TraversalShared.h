// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "NiagaraCommon.h"
#include "NiagaraTypes.h"
#include "NiagaraVariant.h"
#include "UObject/SoftObjectPath.h"

namespace UE::Niagara::TraversalCache
{

class FTraversalNode;

enum class ESelectMode : uint8
{
	None,
	Value,
	Connection
};

enum class ESelectKeySource : uint8
{
	None,
	ExternalConstant,
	Attribute,
	ModuleInput,
	ModuleLocal,
	ModuleOutput,
	FunctionCallNode,
};

struct FSelectKey
{
	FSelectKey()
		: Source(ESelectKeySource::None)
	{
	}

	FSelectKey(const ESelectKeySource& InSource, const FNiagaraVariableBase& InVariable, FName InNamespaceModifier)
		: Source(InSource)
		, Variable(InVariable)
		, NamespaceModifier(InNamespaceModifier)
	{
	}

	bool IsValid() const { return Source != ESelectKeySource::None && Variable.IsValid(); }

	FORCEINLINE bool operator==(const FSelectKey& Other) const
	{
		return Source == Other.Source && Variable == Other.Variable && NamespaceModifier == Other.NamespaceModifier;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FSelectKey& Key)
	{
		uint32 Hash = GetTypeHash(Key.Source);
		Hash = HashCombine(Hash, GetTypeHash(Key.Variable));
		Hash = HashCombine(Hash, GetTypeHash(Key.NamespaceModifier));
		return Hash;
	}

	ESelectKeySource Source;
	FNiagaraVariableBase Variable;
	FName NamespaceModifier;
};

struct FSelectValue
{
	FSelectValue()
		: NumericValue(0)
	{
	}

	FSelectValue(int32 InNumericValue, FName InDebugName)
		: NumericValue(InNumericValue)
		, DebugName(InDebugName)
	{
	}

	FORCEINLINE bool operator==(const FSelectValue& Other) const
	{
		return NumericValue == Other.NumericValue;
	}

	static const FSelectValue& GetBoolTrue();
	static const FSelectValue& GetBoolFalse();
	static const FSelectValue& GetDebugStateNoDebug();
	static const FSelectValue& GetDebugStateBasic();

	int32 NumericValue;
	FName DebugName;
};

struct FStaticContext : public TMap<FSelectKey, FSelectValue>
{
	TOptional<FSelectValue> GetSelectValue(const FSelectKey& Key) const;
};

struct FTopLevelScriptStaticContext
{
	FStaticContext GlobalContext;
	TMap<FGuid, FStaticContext> FunctionInputContexts;
};

enum class EParameterFlags : uint16
{
	None = 0,
	ModuleInput = 1,
	ModuleLocal = 2,
	ModuleOutput = 4,
	Attribute = 8,
	Transient = 16,
	External = 32,
	NamespaceUnknown = 64,
	NamespaceModifier = 128,
	InvalidParameterName = 4096
};

EParameterFlags operator|(
	EParameterFlags FlagsA,
	EParameterFlags FlagsB);

EParameterFlags operator&(
	EParameterFlags FlagsA,
	EParameterFlags FlagsB);

EParameterFlags operator~(
	EParameterFlags Flags);

enum class EConnectionTraversalMode
{
	All,
	MatchingOnly
};

enum class ETraversalStateFlags : uint8
{
	None = 0,
	CulledBySwitch = 1,
	CallerDisabled = 2,
	UnconnectedRoot = 4
};

ETraversalStateFlags operator|(
	ETraversalStateFlags FlagsA,
	ETraversalStateFlags FlagsB);

ETraversalStateFlags operator&(
	ETraversalStateFlags FlagsA,
	ETraversalStateFlags FlagsB);

ETraversalStateFlags operator~(
	ETraversalStateFlags Flags);

struct FTraversalCallingContext
{
	struct FContextHash
	{
		FORCEINLINE bool operator==(const FContextHash& Other) const
		{
			return
				HashData[0] == Other.HashData[0] &&
				HashData[1] == Other.HashData[1] &&
				HashData[2] == Other.HashData[2] &&
				HashData[3] == Other.HashData[3];
		}

		friend FORCEINLINE uint32 GetTypeHash(const FContextHash& Hash)
		{
			return GetArrayHash(Hash.HashData, 4);
		}

		uint32 HashData[4] = { 0, 0, 0, 0 };
	};

	FTraversalCallingContext(const FStaticContext& InGlobalStaticContext, const TMap<FGuid, FStaticContext>& InExternalFunctionInputStaticContexts)
		: GlobalStaticContext(InGlobalStaticContext)
		, ExternalFunctionInputStaticContexts(InExternalFunctionInputStaticContexts)
	{
	}

	FContextHash GenerateHash() const;

	// The static context collected from scripts processed before this traversal.
	const FStaticContext& GlobalStaticContext;

	// In a top level script traversal this contains per-function static contexts for function inputs, gathered from
	// rapid iteration parameters.  This is empty for function traversals.
	const TMap<FGuid, FStaticContext>& ExternalFunctionInputStaticContexts;

	// The local static context from the traversal that called this traversal.  Empty for top level scripts traversals.
	FStaticContext CallingStaticContext;

	// In a function traversal, this contains static context which was gathered for the function inputs.  This is empty
	// for top level script traversals.
	FStaticContext FunctionCallStaticContext;

	// The call stack of the current traversal represented by the function node ids.
	TArray<FGuid> FunctionCallStack;

	// The connection traversal mode.
	EConnectionTraversalMode ConnectionTraversalMode = EConnectionTraversalMode::All;

	// The traversal state from the calling traversal.
	ETraversalStateFlags TraversalState = ETraversalStateFlags::None;
};

struct FTraversalLocalContext
{
	TOptional<FSelectValue> GetSelectValue(const FTraversalCallingContext& CallingContext, const FSelectKey& SelectKey) const;

	FStaticContext StaticContext;
	TMap<FGuid, FStaticContext> FunctionInputStaticContexts;
	ETraversalStateFlags TraversalState = ETraversalStateFlags::None;
	TSet<TPair<const FTraversalNode*, ETraversalStateFlags>> TraversedNodesWithState;
};

class FTraversalData;

struct FFunctionCallTraversalData
{
	FFunctionCallTraversalData(const FGuid& InFunctionCallNodeGuid, TSharedRef<const FTraversalData> InFunctionCallTraversalData)
		: FunctionCallNodeGuid(InFunctionCallNodeGuid)
		, FunctionCallTraversalData(InFunctionCallTraversalData)
	{
	}
		
	FGuid FunctionCallNodeGuid;
	TSharedRef<const FTraversalData> FunctionCallTraversalData;
};

class FTraversalData
{
public:
	const FTraversalData* GetCalledFunctionTraversalData(FGuid FunctionCallNodeGuid) const;
	TArray<FFunctionCallTraversalData> CalledFunctionTraversalData;
};

class ITraversalVisitor
{
public:
	virtual ~ITraversalVisitor() = default;
	virtual const FGuid GetVisitorId() const = 0;
	virtual TSharedRef<FTraversalData> CreateTraversalData() const = 0;
	virtual void VisitNode(ETraversalStateFlags TraversalStateFlags, const FTraversalNode& Node, FTraversalData& TraversalData) const = 0;
};

struct FScriptReference
{
	FScriptReference() { }

	FScriptReference(const FSoftObjectPath& InPath, const FGuid& InVersion)
		: Path(InPath)
		, Version(InVersion)
	{
	}

	FORCEINLINE bool operator==(const FScriptReference& Other) const
	{
		return Path == Other.Path && Version == Other.Version;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FScriptReference& ScriptReference)
	{
		return HashCombine(GetTypeHash(ScriptReference.Path), GetTypeHash(ScriptReference.Version));
	}

	FSoftObjectPath Path;
	FGuid Version;
};

struct FTraversalBuilderDebugData
{
	struct FMultipleInputConnection
	{
		FMultipleInputConnection(const TPair<FGuid, FGuid> InTargetNodeGuidAndPinId, const TArray<TPair<FGuid, FGuid>>& InConnectedNodeGuidsAndPinIds)
			: TargetNodeGuidAndPinId(InTargetNodeGuidAndPinId)
			, ConnectedNodeGuidsAndPinIds(InConnectedNodeGuidsAndPinIds)
		{
		}

		FORCEINLINE bool operator==(const FMultipleInputConnection& Other) const
		{
			return
				TargetNodeGuidAndPinId == Other.TargetNodeGuidAndPinId &&
				ConnectedNodeGuidsAndPinIds == Other.ConnectedNodeGuidsAndPinIds;
		}

		TPair<FGuid, FGuid> TargetNodeGuidAndPinId;
		TArray<TPair<FGuid, FGuid>> ConnectedNodeGuidsAndPinIds;
	};

	struct FUntrimmedNoop
	{
		FUntrimmedNoop(const FGuid& InNodeGuid, FName InSourceNodeTypeName)
			: NodeGuid(InNodeGuid)
			, SourceNodeTypeName(InSourceNodeTypeName)
		{
		}

		FORCEINLINE bool operator==(const FUntrimmedNoop& Other) const
		{
			return NodeGuid == Other.NodeGuid && SourceNodeTypeName == Other.SourceNodeTypeName;
		}

		FGuid NodeGuid;
		FName SourceNodeTypeName;
	};

	struct FUnresolvedSelectOutput
	{
		FUnresolvedSelectOutput(const FGuid& InNodeGuid, const FGuid& InPinGuid)
			: NodeGuid(InNodeGuid)
			, PinGuid(InPinGuid)
		{
		}

		FORCEINLINE bool operator==(const FUnresolvedSelectOutput& Other) const
		{
			return NodeGuid == Other.NodeGuid && PinGuid == Other.PinGuid;
		}

		FGuid NodeGuid;
		FGuid PinGuid;
	};

	FORCEINLINE bool operator==(const FTraversalBuilderDebugData& Other) const
	{
		return MultipleInputConnections == Other.MultipleInputConnections &&
			UntrimmedNoops == Other.UntrimmedNoops;
	}

	void AddMultipleInputConnection(
		const TPair<FGuid, FGuid>& TargetNodeGuidAndPinId,
		const TArray<TPair<FGuid, FGuid>>& ConnectedNodeGuidsAndPinIds);

	void AddUntrimmedNoop(const FGuid& NodeGuid, FName SourceNodeTypeName);

	void AddUnresolvedSelectOutput(const FGuid& NodeGuid, const FGuid& PinGuid);

	bool HasData() const
	{
		return
			MultipleInputConnections.Num() > 0 ||
			UntrimmedNoops.Num() > 0 ||
			UnresolvedSelectOutputs.Num() > 0;
	}

	const TArray<FMultipleInputConnection>& GetMultipleInputConnections() const { return MultipleInputConnections; }

	const TArray<FUntrimmedNoop>& GetUntrimmedNoops() const { return UntrimmedNoops; }

	const TArray<FUnresolvedSelectOutput>& GetUnresolvedSelectOutputs() const { return UnresolvedSelectOutputs; }

private:
	TArray<FMultipleInputConnection> MultipleInputConnections;
	TArray<FUntrimmedNoop> UntrimmedNoops;
	TArray<FUnresolvedSelectOutput> UnresolvedSelectOutputs;
};

struct FTraversalDebugData
{
	struct FUnresolvedBase
	{
		FUnresolvedBase(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid)
			: FunctionCallStack(InFunctionCallStack)
			, SourceNodeGuid(InSourceNodeGuid)
		{
		}

		FORCEINLINE bool operator==(const FUnresolvedBase& Other) const
		{
			return
				FunctionCallStack == Other.FunctionCallStack &&
				SourceNodeGuid == Other.SourceNodeGuid;
		}

		TArray<FGuid> FunctionCallStack;
		FGuid SourceNodeGuid;
	};

	struct FUnresolvedSelect : public FUnresolvedBase
	{
		FUnresolvedSelect(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FSelectKey& InSelectKey)
			: FUnresolvedBase(InFunctionCallStack, InSourceNodeGuid)
			, SelectKey(InSelectKey)
		{
		}

		FORCEINLINE bool operator==(const FUnresolvedSelect& Other) const
		{
			return (FUnresolvedBase&)*this == (FUnresolvedBase&)Other && SelectKey == Other.SelectKey;
		}

		FSelectKey SelectKey;
	};

	struct FUnresolvedSelectInput : public FUnresolvedBase
	{
		FUnresolvedSelectInput(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FSelectKey& InSelectKey, const FSelectValue& InResolvedSelectValue)
			: FUnresolvedBase(InFunctionCallStack, InSourceNodeGuid)
			, SelectKey(InSelectKey)
			, ResolvedSelectValue(InResolvedSelectValue)
		{
		}

		FORCEINLINE bool operator==(const FUnresolvedSelectInput& Other) const
		{
			return 
				(FUnresolvedBase&)*this == (FUnresolvedBase&)Other &&
				SelectKey == Other.SelectKey &&
				ResolvedSelectValue == Other.ResolvedSelectValue;
		}

		FSelectKey SelectKey;
		FSelectValue ResolvedSelectValue;
	};

	struct FUnresolvedRead : public FUnresolvedBase
	{
		FUnresolvedRead(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FNiagaraVariableBase& InReadParameter)
			: FUnresolvedBase(InFunctionCallStack, InSourceNodeGuid)
			, ReadParameter(InReadParameter)
		{
		}

		FORCEINLINE bool operator==(const FUnresolvedRead& Other) const
		{
			return (FUnresolvedBase&)*this == (FUnresolvedBase&)Other && ReadParameter == Other.ReadParameter;
		}

		FNiagaraVariableBase ReadParameter;
	};

	struct FUnresolvedStaticOp : public FUnresolvedBase
	{
		FUnresolvedStaticOp(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, FName InOpName, int32 InUnresolvedPinIndex)
			: FUnresolvedBase(InFunctionCallStack, InSourceNodeGuid)
			, OpName(InOpName)
			, UnresolvedPinIndex(InUnresolvedPinIndex)
		{
		}

		FORCEINLINE bool operator==(const FUnresolvedStaticOp& Other) const
		{

			return
				(FUnresolvedBase&)*this == (FUnresolvedBase&)Other &&
				OpName == Other.OpName &&
				UnresolvedPinIndex == Other.UnresolvedPinIndex;
		}

		FName OpName;
		int32 UnresolvedPinIndex = INDEX_NONE;
	};

	struct FUnresolvedFunctionInput : public FUnresolvedBase
	{
		FUnresolvedFunctionInput(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FNiagaraVariableBase& InFunctionInputParameter)
			: FUnresolvedBase(InFunctionCallStack, InSourceNodeGuid)
			, FunctionInputParameter(InFunctionInputParameter)
		{
		}

		FORCEINLINE bool operator==(const FUnresolvedFunctionInput& Other) const
		{
			return (FUnresolvedBase&)*this == (FUnresolvedBase&)Other && FunctionInputParameter == Other.FunctionInputParameter;
		}

		FNiagaraVariableBase FunctionInputParameter;
	};

	struct FBuilderDebugOutput
	{
		FBuilderDebugOutput(const FSoftObjectPath InScriptPath, const ENiagaraScriptUsage InScriptUsage,
			const FGuid& InScriptUsageId, const FGuid& InScriptVersion, const FTraversalBuilderDebugData& InBuilderDebugData)
			: ScriptPath(InScriptPath)
			, ScriptUsage(InScriptUsage)
			, ScriptUsageId(InScriptUsageId)
			, ScriptVersion(InScriptVersion)
			, BuilderDebugData(InBuilderDebugData)
		{
		}


		FORCEINLINE bool operator==(const FBuilderDebugOutput& Other) const
		{
			return 
				ScriptPath == Other.ScriptPath &&
				ScriptUsage == Other.ScriptUsage && 
				ScriptUsageId == Other.ScriptUsageId && 
				ScriptVersion == Other.ScriptVersion &&
				BuilderDebugData == Other.BuilderDebugData;
		}
		
		FSoftObjectPath ScriptPath;
		ENiagaraScriptUsage ScriptUsage;
		FGuid ScriptUsageId;
		FGuid ScriptVersion;
		FTraversalBuilderDebugData BuilderDebugData;
	};

	void AddUnresolvedSelect(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FSelectKey& InSelectKey);
	void AddUnresolvedSelectInput(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FSelectKey& InSelectKey, const FSelectValue& InResolvedSelectValue);
	void AddUnresolvedRead(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FNiagaraVariableBase& InReadParameter);
	void AddUnresolvedStaticOp(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, FName InOpName, int32 InUnresolvedPinIndex);
	void AddUnresolvedFunctionInput(const TArray<FGuid>& InFunctionCallStack, const FGuid& InSourceNodeGuid, const FNiagaraVariableBase& InFunctionInput);
	void AddBuilderDebugOutput(const FSoftObjectPath InGraphPath, const ENiagaraScriptUsage InScriptUsage,
		const FGuid& InScriptUsageId, const FGuid& InScriptVersion, const FTraversalBuilderDebugData& InBuilderDebugData);

	bool HasData() const { return
		UnresolvedSelects.Num() > 0 ||
		UnresolvedSelectInputs.Num() > 0 ||
		UnresolvedReads.Num() > 0 ||
		UnresolvedStaticOps.Num() > 0 ||
		UnresolvedFunctionInputs.Num() > 0 ||
		BuilderDebugOutputs.Num() > 0; }

	const TArray<FUnresolvedSelect>& GetUnresolvedSelects() const { return UnresolvedSelects; }
	const TArray<FUnresolvedSelectInput>& GetUnresolvedSelectInputs() const { return UnresolvedSelectInputs; }
	const TArray<FUnresolvedRead>& GetUnresolvedReads() const { return UnresolvedReads; }
	const TArray<FUnresolvedStaticOp>& GetUnresolvedStaticOps() const { return UnresolvedStaticOps; }
	const TArray<FUnresolvedFunctionInput>& GetUnresolvedFunctionInputs() const { return UnresolvedFunctionInputs; }
	const TArray<FBuilderDebugOutput> GetBuilderDebugOutputs() const { return BuilderDebugOutputs; }

private:
	TArray<FUnresolvedSelect> UnresolvedSelects;
	TArray<FUnresolvedSelectInput> UnresolvedSelectInputs;
	TArray<FUnresolvedRead> UnresolvedReads;
	TArray<FUnresolvedStaticOp> UnresolvedStaticOps;
	TArray<FUnresolvedFunctionInput> UnresolvedFunctionInputs;
	TArray<FBuilderDebugOutput> BuilderDebugOutputs; 
};

template<typename TEnumFlag>
static bool HasFlag(TEnumFlag Flags, TEnumFlag FlagToCheck)
{
	return (Flags & FlagToCheck) == FlagToCheck;
}

template<typename TEnumFlag>
static TEnumFlag SetFlag(TEnumFlag Flags, TEnumFlag FlagToCheck)
{
	return Flags | FlagToCheck;
}

template<typename TEnumFlag>
static TEnumFlag ClearFlag(TEnumFlag Flags, TEnumFlag FlagToCheck)
{
	return Flags & ~FlagToCheck;
}

} // UE::Niagara::TraversalCache
