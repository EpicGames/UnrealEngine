// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraBoolTypeEditorUtilities.h"
#include "NiagaraClipboard.h"
#include "NiagaraTypes.h"
#include "NiagaraVariant.h"
#include "SNiagaraParameterEditor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"

class SNiagaraBoolParameterEditor : public SNiagaraParameterEditor
{
public:
	SLATE_BEGIN_ARGS(SNiagaraBoolParameterEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SNiagaraBoolParameterEditor::GetCheckState)
				.OnCheckStateChanged(this, &SNiagaraBoolParameterEditor::OnCheckStateChanged)
			]
		];
	}

	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetBoolStruct(), TEXT("Struct type not supported."));
		bBoolValue = ((FNiagaraBool*)Struct->GetStructMemory())->GetValue();
	}

	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) override
	{
		// Note that while bool conventionally have false = 0 and true = 1 (or any non-zero value), Niagara internally uses true == -1. Make 
		// sure to enforce this convention when setting the value in memory.
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetBoolStruct(), TEXT("Struct type not supported."));
		((FNiagaraBool*)Struct->GetStructMemory())->SetValue(bBoolValue);
	}

private:
	ECheckBoxState GetCheckState() const
	{
		return bBoolValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnCheckStateChanged(ECheckBoxState InCheckState)
	{
		bBoolValue = InCheckState == ECheckBoxState::Checked;
		ExecuteOnValueChanged();
	}

private:
	bool bBoolValue;
};

TSharedPtr<SNiagaraParameterEditor> FNiagaraEditorBoolTypeUtilities::CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const
{
	return SNew(SNiagaraBoolParameterEditor);
}

bool FNiagaraEditorBoolTypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorBoolTypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));
	return LexToString(AllocatedVariable.GetValue<FNiagaraBool>().GetValue());
}

bool FNiagaraEditorBoolTypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	bool bBoolValue = false;
	if (LexTryParseString(bBoolValue, *StringValue) || !Variable.IsDataAllocated())
	{
		FNiagaraBool BoolValue;
		BoolValue.SetValue(bBoolValue);
		Variable.SetValue<FNiagaraBool>(BoolValue);
		return true;
	}
	return false;
}

bool FNiagaraEditorBoolTypeUtilities::TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const
{
	if (InSourceType == FNiagaraTypeDefinition::GetBoolDef() && InSourceValue.GetNumBytes() == FNiagaraTypeDefinition::GetBoolDef().GetSize())
	{
		FNiagaraVariable Temp(InSourceType, NAME_None);
		Temp.SetData(InSourceValue.GetBytes());
		bool BoolValue = Temp.GetValue<FNiagaraBool>().GetValue();
		InTargetClipboardPortableValue.ValueString = LexToString(BoolValue);
		return true;
	}
	return false;
}

bool FNiagaraEditorBoolTypeUtilities::TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const
{
	bool BoolValue;
	if (InTargetType == FNiagaraTypeDefinition::GetBoolDef() &&
		LexTryParseString(BoolValue, *InSourceClipboardPortableValue.ValueString))
	{
		FNiagaraBool NiagaraBoolValue;
		NiagaraBoolValue.SetValue(BoolValue);
		FNiagaraVariable Temp(InTargetType, NAME_None);
		Temp.SetValue<FNiagaraBool>(NiagaraBoolValue);
		InTargetValue.SetBytes(Temp.GetData(), Temp.GetSizeInBytes());
		return true;
	}
	return false;
}

int32 FNiagaraEditorBoolTypeUtilities::VariableToSelectNumericValue(const FNiagaraVariable& InVariableValue) const
{
	FNiagaraTypeDefinition BaseDefinition = InVariableValue.GetType().RemoveStaticDef();
	if (ensureMsgf(BaseDefinition == FNiagaraTypeDefinition::GetBoolDef() && InVariableValue.IsDataAllocated(),
		TEXT("InVariableValue must be of type FNiagaraBool and have its data allocated to convert to a select value.")) == false)
	{
		return INDEX_NONE;
	}

	// The select values for bools differ from the underlying variant value in that it uses 1 for true and not -1.
	bool bBoolValue = InVariableValue.GetValue<FNiagaraBool>().GetValue();
	return bBoolValue ? 1 : 0;
}

FName FNiagaraEditorBoolTypeUtilities::GetDebugNameForSelectValue(const FNiagaraTypeDefinition& ValueType, int32 SelectValue) const
{
	if (SelectValue == 1)
	{
		return *LexToString(true);
	}
	if (SelectValue == 0)
	{
		return *LexToString(false);
	}
	return NAME_None;
}
