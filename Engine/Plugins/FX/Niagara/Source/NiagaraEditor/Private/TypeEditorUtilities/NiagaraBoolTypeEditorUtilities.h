// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INiagaraEditorTypeUtilities.h"

class SNiagaraParameterEditor;

/** Niagara editor utilities for the bool type. */
class FNiagaraEditorBoolTypeUtilities : public FNiagaraEditorTypeUtilities
{
public:
	//~ INiagaraEditorTypeUtilities interface.
	virtual bool CanCreateParameterEditor() const override { return true; }
	virtual TSharedPtr<SNiagaraParameterEditor> CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const override;
	virtual bool CanHandlePinDefaults() const override;
	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const override;
	virtual bool SupportsClipboardPortableValues() const { return true; }
	virtual bool TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const override;
	virtual bool TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const override;
	virtual bool CanBeSelectValue() const override { return true; }
	virtual int32 VariableToSelectNumericValue(const FNiagaraVariable& InVariableValue) const override;
	virtual FName GetDebugNameForSelectValue(const FNiagaraTypeDefinition& ValueType, int32 SelectValue) const override;
};