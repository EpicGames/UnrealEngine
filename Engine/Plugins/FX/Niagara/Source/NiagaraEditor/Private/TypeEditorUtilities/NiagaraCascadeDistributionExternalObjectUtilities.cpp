// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypeEditorUtilities/NiagaraCascadeDistributionExternalObjectUtilities.h"

#include "Distributions/DistributionFloat.h"
#include "Distributions/DistributionVector.h"
#include "Distributions/DistributionFloatConstantCurve.h"
#include "Distributions/DistributionVectorConstantCurve.h"
#include "NiagaraClipboard.h"

namespace NiagaraCascadeDistributionUtilities
{
	ERichCurveInterpMode RichCurveInterpModeFromLegacyInterpMode(EInterpCurveMode LegacyInterpMode)
	{
		switch (LegacyInterpMode)
		{
		case CIM_Constant:
			return ERichCurveInterpMode::RCIM_Constant;
		case CIM_Linear:
			return ERichCurveInterpMode::RCIM_Linear;
		case CIM_CurveAuto:
		case CIM_CurveUser:
		case CIM_CurveBreak:
		case CIM_CurveAutoClamped:
			return ERichCurveInterpMode::RCIM_Cubic;
		default:
			return ERichCurveInterpMode::RCIM_None;
		}
	}

	ERichCurveTangentMode RichCurveTangentModeFromLegacyInterpMode(EInterpCurveMode LegacyInterpMode)
	{
		switch (LegacyInterpMode)
		{
		case CIM_Linear:
		case CIM_Constant:
			return ERichCurveTangentMode::RCTM_None;
		case CIM_CurveAuto:
			return ERichCurveTangentMode::RCTM_Auto;
		case CIM_CurveUser:
			return ERichCurveTangentMode::RCTM_User;
		case CIM_CurveBreak:
			return ERichCurveTangentMode::RCTM_Break;
		case CIM_CurveAutoClamped:
			return ERichCurveTangentMode::RCTM_SmartAuto;
		default:
			return ERichCurveTangentMode::RCTM_None;
		}
	}
}

bool FNiagaraCascadeDistributionExternalObjectUtilities::TryUpdateClipboardPortableValueFromObject(const UObject& InExternalObject, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const
{
	auto AddKeyToCurve = [](FRichCurve& TargetCurve, ERichCurveInterpMode InterpMode, ERichCurveTangentMode TangentMode, float Time, float Value, float ArriveTangent, float LeaveTangent)
	{
		FKeyHandle KeyHandle = TargetCurve.AddKey(Time, Value);
		FRichCurveKey& AddedKey = TargetCurve.GetKey(KeyHandle);
		AddedKey.InterpMode = InterpMode;
		AddedKey.TangentMode = TangentMode;
		AddedKey.ArriveTangent = ArriveTangent;
		AddedKey.LeaveTangent = LeaveTangent;
	};

	const UDistributionFloatConstantCurve* FloatCurve = Cast<UDistributionFloatConstantCurve>(&InExternalObject);
	if (FloatCurve != nullptr)
	{
		FNiagaraClipboardCurveCollection CurveCollection;
		FRichCurve& Curve = CurveCollection.Curves.AddDefaulted_GetRef();
		for (const FInterpCurvePointFloat& CurvePoint : FloatCurve->ConstantCurve.Points)
		{
			ERichCurveInterpMode InterpMode = NiagaraCascadeDistributionUtilities::RichCurveInterpModeFromLegacyInterpMode(CurvePoint.InterpMode);
			ERichCurveTangentMode TangentMode = NiagaraCascadeDistributionUtilities::RichCurveTangentModeFromLegacyInterpMode(CurvePoint.InterpMode);
			AddKeyToCurve(Curve, InterpMode, TangentMode, CurvePoint.InVal, CurvePoint.OutVal, CurvePoint.ArriveTangent, CurvePoint.LeaveTangent);
		}
		InTargetClipboardPortableValue = FNiagaraClipboardPortableValue::CreateFromStructValue(CurveCollection);
		return true;
	}
	else
	{
		const UDistributionVectorConstantCurve* VectorCurve = Cast<UDistributionVectorConstantCurve>(&InExternalObject);
		if (VectorCurve != nullptr)
		{
			FNiagaraClipboardCurveCollection CurveCollection;
			FRichCurve& XCurve = CurveCollection.Curves.AddDefaulted_GetRef();
			FRichCurve& YCurve = CurveCollection.Curves.AddDefaulted_GetRef();
			FRichCurve& ZCurve = CurveCollection.Curves.AddDefaulted_GetRef();
			for (const FInterpCurvePointVector& CurvePoint : VectorCurve->ConstantCurve.Points)
			{
				ERichCurveInterpMode InterpMode = NiagaraCascadeDistributionUtilities::RichCurveInterpModeFromLegacyInterpMode(CurvePoint.InterpMode);
				ERichCurveTangentMode TangentMode = NiagaraCascadeDistributionUtilities::RichCurveTangentModeFromLegacyInterpMode(CurvePoint.InterpMode);
				AddKeyToCurve(XCurve, InterpMode, TangentMode, CurvePoint.InVal, CurvePoint.OutVal.X, CurvePoint.ArriveTangent.X, CurvePoint.LeaveTangent.X);
				AddKeyToCurve(YCurve, InterpMode, TangentMode, CurvePoint.InVal, CurvePoint.OutVal.Y, CurvePoint.ArriveTangent.Y, CurvePoint.LeaveTangent.Y);
				AddKeyToCurve(ZCurve, InterpMode, TangentMode, CurvePoint.InVal, CurvePoint.OutVal.Z, CurvePoint.ArriveTangent.Z, CurvePoint.LeaveTangent.Z);
			}
			InTargetClipboardPortableValue = FNiagaraClipboardPortableValue::CreateFromStructValue(CurveCollection);
			return true;
		}
	}
	return false;
}