// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INiagaraEditorTypeUtilities.h"

class UObject;

class FNiagaraCascadeDistributionExternalObjectUtilities : public FNiagaraEditorExternalObjectUtilities
{
public:
	virtual bool SupportsClipboardPortableValues() const override { return true; }
	virtual bool TryUpdateClipboardPortableValueFromObject(const UObject& InExternalObject, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const override;
};