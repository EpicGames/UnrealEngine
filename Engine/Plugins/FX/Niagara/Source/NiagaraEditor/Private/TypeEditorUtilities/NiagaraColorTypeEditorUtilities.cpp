// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraColorTypeEditorUtilities.h"

#include "NiagaraEditorSettings.h"
#include "SNiagaraParameterEditor.h"
#include "NiagaraTypes.h"
#include "NiagaraEditorStyle.h"
#include "Engine/Engine.h"

#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SNiagaraColorEditor.h"

class SNiagaraColorParameterEditor : public SNiagaraParameterEditor
{
public:
	SLATE_BEGIN_ARGS(SNiagaraColorParameterEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs)
	{
		SNiagaraParameterEditor::Construct(SNiagaraParameterEditor::FArguments());

		ChildSlot
		[
			SNew(SNiagaraColorEditor)
			.Color(this, &SNiagaraColorParameterEditor::GetColor)
			.OnColorChanged(this, &SNiagaraColorParameterEditor::SetColor)
			.OnBeginEditing(this, &SNiagaraColorParameterEditor::BeginEditing)
			.OnEndEditing(this, &SNiagaraColorParameterEditor::EndEditing)
			.OnCancelEditing(this, &SNiagaraColorParameterEditor::CancelEditing)
			.OnColorPickerOpened(this, &SNiagaraColorParameterEditor::ColorPickerOpened)
			.OnColorPickerClosed(this, &SNiagaraColorParameterEditor::ColorPickerClosed)
			.MinDesiredColorBlockWidth(SNiagaraParameterEditor::DefaultInputSize)
		];
	}

	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetColorStruct(), TEXT("Struct type not supported."));
		ColorValue = *((FLinearColor*)Struct->GetStructMemory());
	}

	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetColorStruct(), TEXT("Struct type not supported."));
		*((FLinearColor*)Struct->GetStructMemory()) = ColorValue;
	}

	virtual bool CanChangeContinuously() const override { return true; }

private:
	void BeginEditing()
	{
		ExecuteOnBeginValueChange();
	}

	void EndEditing()
	{
		ExecuteOnEndValueChange();
	}

	void CancelEditing(FLinearColor OriginalColor)
	{
		ColorValue = OriginalColor;
		ExecuteOnValueChanged();
	}

	void ColorPickerOpened()
	{
		SetIsEditingExclusively(true);
	}

	void ColorPickerClosed()
	{
		SetIsEditingExclusively(false);
	}

	FLinearColor GetColor() const
	{
		return ColorValue;
	}

	void SetColor(FLinearColor NewColor)
	{
		ColorValue = NewColor;
		ExecuteOnValueChanged();
	}

private:
	TSharedPtr<SColorBlock> ColorBlock;

	FLinearColor ColorValue;
};

void FNiagaraEditorColorTypeUtilities::UpdateVariableWithDefaultValue(FNiagaraVariable& Variable) const
{
	checkf(Variable.GetType().GetStruct() == FNiagaraTypeDefinition::GetColorStruct(), TEXT("Struct type not supported."));
	Variable.SetValue<FLinearColor>(FLinearColor(1, 1, 1, 1));
}

TSharedPtr<SNiagaraParameterEditor> FNiagaraEditorColorTypeUtilities::CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const
{
	return SNew(SNiagaraColorParameterEditor);
}

bool FNiagaraEditorColorTypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorColorTypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));
	return AllocatedVariable.GetValue<FLinearColor>().ToString();
}

bool FNiagaraEditorColorTypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	FLinearColor ColorValue = FLinearColor::Black;
	if (ColorValue.InitFromString(StringValue) || !Variable.IsDataAllocated())
	{
		Variable.SetValue<FLinearColor>(ColorValue);
		return true;
	}
	return false;
}

FText FNiagaraEditorColorTypeUtilities::GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	return FText::FromString(GetPinDefaultStringFromValue(AllocatedVariable));
}

FText FNiagaraEditorColorTypeUtilities::GetStackDisplayText(const FNiagaraVariable& Variable) const
{
	FLinearColor Value = Variable.GetValue<FLinearColor>();
	return FText::Format(FText::FromString("({0}, {1}, {2}, {3})"), Value.R, Value.G, Value.B, Value.A);
}
