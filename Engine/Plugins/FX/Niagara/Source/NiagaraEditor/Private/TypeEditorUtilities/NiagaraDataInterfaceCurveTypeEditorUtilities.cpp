// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceCurveTypeEditorUtilities.h"

#include "NiagaraClipboard.h"
#include "NiagaraDataInterface.h"
#include "NiagaraDataInterfaceCurve.h"
#include "NiagaraDataInterfaceVector2DCurve.h"
#include "NiagaraDataInterfaceVectorCurve.h"
#include "NiagaraDataInterfaceVector4Curve.h"
#include "NiagaraDataInterfaceColorCurve.h"
#include "NiagaraVariant.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Modules/ModuleManager.h"
#include "Curves/CurveFloat.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveLinearColor.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceCurveTypeEditorUtilities"

bool FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase::CanCreateDataInterfaceEditor() const
{
	return true;
}

TSharedPtr<SWidget> FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase::CreateDataInterfaceEditor(UObject* DataInterface, FNotifyValueChanged DataInterfaceChangedHandler) const
{
	UNiagaraDataInterfaceCurveBase* CurveDataInterface = Cast<UNiagaraDataInterfaceCurveBase>(DataInterface);
	if (CurveDataInterface != nullptr)
	{
		return SNew(SComboButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ForegroundColor(FSlateColor::UseForeground())
			.OnGetMenuContent_Raw(const_cast<FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase*>(this), &FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase::GetImportMenuContent, MakeWeakObjectPtr(CurveDataInterface), DataInterfaceChangedHandler)
			.ButtonContent()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(LOCTEXT("Import", "Import"))
			];
	}
	return TSharedPtr<SWidget>();
}

TSharedRef<SWidget> FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase::GetImportMenuContent(TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> CurveDataInterface, FNotifyValueChanged DataInterfaceChangedHandler)
{
	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase::CurveAssetSelected, CurveDataInterface, DataInterfaceChangedHandler);
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.Filter.ClassPaths.Add(GetSupportedAssetClassName());
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	return SNew(SBox)
		.WidthOverride(300.0f)
		.HeightOverride(300.f)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];
}

void FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase::CurveAssetSelected(const FAssetData& AssetData, TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> CurveDataInterfacePtr, FNotifyValueChanged DataInterfaceChangedHandler)
{
	FSlateApplication::Get().DismissAllMenus();
	UObject* CurveAsset = AssetData.GetAsset();
	if (CurveAsset != nullptr)
	{
		UNiagaraDataInterfaceCurveBase* CurveDataInterface = CurveDataInterfacePtr.Get();
		if (CurveDataInterface != nullptr)
		{
			ImportSelectedAsset(CurveAsset, CurveDataInterface);
			DataInterfaceChangedHandler.ExecuteIfBound();
		}
	}
}

bool FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase::TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const
{
	if (InSourceType.GetClass()->IsChildOf(UNiagaraDataInterfaceCurveBase::StaticClass()) &&
		InSourceValue.GetDataInterface() != nullptr &&
		InSourceValue.GetDataInterface()->IsA<UNiagaraDataInterfaceCurveBase>())
	{
		UNiagaraDataInterfaceCurveBase* CurveDataInterface = CastChecked<UNiagaraDataInterfaceCurveBase>(InSourceValue.GetDataInterface());
		FNiagaraClipboardCurveCollection CurveCollection;
		TArray<UNiagaraDataInterfaceCurveBase::FCurveData> CurveData;
		CurveDataInterface->GetCurveData(CurveData);
		for (const UNiagaraDataInterfaceCurveBase::FCurveData& CurveDataItem : CurveData)
		{
			CurveCollection.Curves.Add(*CurveDataItem.Curve);
		}
		InTargetClipboardPortableValue = FNiagaraClipboardPortableValue::CreateFromStructValue(CurveCollection);
		return true;
	}
	return false;
}

bool FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase::CanUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType) const
{
	FNiagaraClipboardCurveCollection CurveCollection;
	return InSourceClipboardPortableValue.TryUpdateStructValue(CurveCollection);
}

bool FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase::TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const
{
	if (InTargetType.GetClass()->IsChildOf(UNiagaraDataInterfaceCurveBase::StaticClass()))
	{
		FNiagaraClipboardCurveCollection CurveCollection;
		if (InSourceClipboardPortableValue.TryUpdateStructValue(CurveCollection))
		{
			UNiagaraDataInterfaceCurveBase* CurveDataInterface = NewObject<UNiagaraDataInterfaceCurveBase>(GetTransientPackage(), InTargetType.GetClass());
			TArray<UNiagaraDataInterfaceCurveBase::FCurveData> CurveData;
			CurveDataInterface->GetCurveData(CurveData);
			if (CurveData.Num() == CurveCollection.Curves.Num())
			{
				for (int32 CurveIndex = 0; CurveIndex < CurveData.Num(); CurveIndex++)
				{
					*CurveData[CurveIndex].Curve = CurveCollection.Curves[CurveIndex];
				}
				InTargetValue.SetDataInterface(CurveDataInterface);
				return true;
			}
		}
	}
	return false;
}

FTopLevelAssetPath FNiagaraDataInterfaceCurveTypeEditorUtilities::GetSupportedAssetClassName() const
{
	return UCurveFloat::StaticClass()->GetClassPathName();
}

void FNiagaraDataInterfaceCurveTypeEditorUtilities::ImportSelectedAsset(UObject* SelectedAsset, UNiagaraDataInterfaceCurveBase* CurveDataInterface) const
{
	UCurveFloat* FloatCurveAsset = Cast<UCurveFloat>(SelectedAsset);
	UNiagaraDataInterfaceCurve* FloatCurveDataInterface = Cast<UNiagaraDataInterfaceCurve>(CurveDataInterface);
	if (FloatCurveAsset != nullptr && FloatCurveDataInterface != nullptr)
	{
		FScopedTransaction ImportTransaction(LOCTEXT("ImportFloatCurvesTransaction", "Import float curve"));
		FloatCurveDataInterface->Modify();
		FloatCurveDataInterface->Curve = FloatCurveAsset->FloatCurve;
	}
}

FTopLevelAssetPath FNiagaraDataInterfaceVectorCurveTypeEditorUtilities::GetSupportedAssetClassName() const
{
	return UCurveVector::StaticClass()->GetClassPathName();
}

void FNiagaraDataInterfaceVectorCurveTypeEditorUtilities::ImportSelectedAsset(UObject* SelectedAsset, UNiagaraDataInterfaceCurveBase* CurveDataInterface) const
{
	UCurveVector* VectorCurveAsset = Cast<UCurveVector>(SelectedAsset);
	UNiagaraDataInterfaceVectorCurve* VectorCurveDataInterface = Cast<UNiagaraDataInterfaceVectorCurve>(CurveDataInterface);
	if (VectorCurveAsset != nullptr && VectorCurveDataInterface != nullptr)
	{
		FScopedTransaction ImportTransaction(LOCTEXT("ImportColorVectorTransaction", "Import vector curves"));
		VectorCurveDataInterface->Modify();
		VectorCurveDataInterface->XCurve = VectorCurveAsset->FloatCurves[0];
		VectorCurveDataInterface->YCurve = VectorCurveAsset->FloatCurves[1];
		VectorCurveDataInterface->ZCurve = VectorCurveAsset->FloatCurves[2];
	}
}

FTopLevelAssetPath FNiagaraDataInterfaceColorCurveTypeEditorUtilities::GetSupportedAssetClassName() const
{
	return UCurveLinearColor::StaticClass()->GetClassPathName();
}

void FNiagaraDataInterfaceColorCurveTypeEditorUtilities::ImportSelectedAsset(UObject* SelectedAsset, UNiagaraDataInterfaceCurveBase* CurveDataInterface) const
{
	UCurveLinearColor* ColorCurveAsset = Cast<UCurveLinearColor>(SelectedAsset);
	UNiagaraDataInterfaceColorCurve* ColorCurveDataInterface = Cast<UNiagaraDataInterfaceColorCurve>(CurveDataInterface);
	if (ColorCurveAsset != nullptr && ColorCurveDataInterface != nullptr)
	{
		FScopedTransaction ImportTransaction(LOCTEXT("ImportColorCurvesTransaction", "Import color curves"));
		ColorCurveDataInterface->Modify();
		ColorCurveDataInterface->RedCurve = ColorCurveAsset->FloatCurves[0];
		ColorCurveDataInterface->BlueCurve = ColorCurveAsset->FloatCurves[1];
		ColorCurveDataInterface->GreenCurve = ColorCurveAsset->FloatCurves[2];
		ColorCurveDataInterface->AlphaCurve = ColorCurveAsset->FloatCurves[3];
	}
}

#undef LOCTEXT_NAMESPACE