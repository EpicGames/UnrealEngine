// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INiagaraEditorTypeUtilities.h"
#include "AssetRegistry/AssetData.h"

class UNiagaraDataInterfaceCurveBase;

/** Niagara editor utilities for the UNiagaraDataInterfaceCurve type. */
class FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase : public FNiagaraEditorTypeUtilities
{
public:
	//~ FNiagaraEditorTypeUtilities interface.
	virtual bool CanCreateDataInterfaceEditor() const override;
	virtual TSharedPtr<SWidget> CreateDataInterfaceEditor(UObject* DataInterface, FNotifyValueChanged DataInterfaceChangedHandler) const override;

	virtual bool SupportsClipboardPortableValues() const { return true; }
	virtual bool TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const override;
	virtual bool CanUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType) const override;
	virtual bool TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const override;

protected:
	/** Gets the class name used for filtering the curve asset picker. */
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const = 0;

	/** Imports the selected curve asset into the supplied curve data interface. */
	virtual void ImportSelectedAsset(UObject* SelectedAsset, UNiagaraDataInterfaceCurveBase* CurveDataInterface) const = 0;

private:
	/** Gets the content for the import drop down menu. */
	TSharedRef<SWidget> GetImportMenuContent(TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> CurveDataInterface, FNotifyValueChanged DataInterfaceChangedHandler);

	/** Handles an asset being selected in the import menu. */
	void CurveAssetSelected(const FAssetData& AssetData, TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> CurveDataInterface, FNotifyValueChanged DataInterfaceChangedHandler);
};


/** Type editor utilities for float curve data interfaces. */
class FNiagaraDataInterfaceCurveTypeEditorUtilities : public FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase
{
protected:
	//~ FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase interface
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const override;
	virtual void ImportSelectedAsset(UObject* SelectedAsset, UNiagaraDataInterfaceCurveBase* CurveDataInterface) const override;
};


/** Type editor utilities for vector curve data interfaces. */
class FNiagaraDataInterfaceVectorCurveTypeEditorUtilities : public FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase
{
protected:
	//~ FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase interface
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const override;
	virtual void ImportSelectedAsset(UObject* SelectedAsset, UNiagaraDataInterfaceCurveBase* CurveDataInterface) const override;
};


/** Type editor utilities for color curve data interfaces. */
class FNiagaraDataInterfaceColorCurveTypeEditorUtilities : public FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase
{
protected:
	//~ FNiagaraDataInterfaceCurveTypeEditorUtilitiesBase interface
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const override;
	virtual void ImportSelectedAsset(UObject* SelectedAsset, UNiagaraDataInterfaceCurveBase* CurveDataInterface) const override;
};