// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypeEditorUtilities/NiagaraDistributionPropertyEditorUtilities.h"

#include "NiagaraClipboard.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Stateless/NiagaraStatelessDistribution.h"

bool FNiagaraDistributionPropertyEditorUtilities::SupportsClipboardPortableValues() const
{ 
	return true;
}

bool FNiagaraDistributionPropertyEditorUtilities::TryUpdateClipboardPortableValueFromProperty(const IPropertyHandle& InPropertyHandle, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const
{
	FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyHandle.GetProperty());
	if (StructProperty != nullptr && StructProperty->Struct->IsChildOf(FNiagaraDistributionBase::StaticStruct()))
	{
		void* ValueData;
		if (InPropertyHandle.GetValueData(ValueData) == FPropertyAccess::Success)
		{
			const FNiagaraDistributionBase* Distribution = (const FNiagaraDistributionBase*)ValueData;
			if (Distribution->IsConstant())
			{
				switch (Distribution->ChannelConstantsAndRanges.Num())
				{
				case 1:
				{
					float FloatValue = Distribution->ChannelConstantsAndRanges[0];
					InTargetClipboardPortableValue.ValueString = LexToString(FloatValue);
					return true;
				}
				case 2:
				{
					UScriptStruct* Vector2Struct = TVariantStructure<FVector2f>::Get();
					FVector2f Vector2Value(Distribution->ChannelConstantsAndRanges[0], Distribution->ChannelConstantsAndRanges[1]);
					InTargetClipboardPortableValue = FNiagaraClipboardPortableValue::CreateFromStructValue(*Vector2Struct, (uint8*)&Vector2Value);
					return true;
				}
				case 3:
				{
					UScriptStruct* Vector3Struct = TVariantStructure<FVector3f>::Get();
					FVector3f Vector3Value(Distribution->ChannelConstantsAndRanges[0], Distribution->ChannelConstantsAndRanges[1], Distribution->ChannelConstantsAndRanges[2]);
					InTargetClipboardPortableValue = FNiagaraClipboardPortableValue::CreateFromStructValue(*Vector3Struct, (uint8*)&Vector3Value);
					return true;
				}
				case 4:
				{
					if (Distribution->DisplayAsColor())
					{
						UScriptStruct* ColorStruct = TBaseStructure<FLinearColor>::Get();
						FLinearColor ColorValue(Distribution->ChannelConstantsAndRanges[0], Distribution->ChannelConstantsAndRanges[1], Distribution->ChannelConstantsAndRanges[2], Distribution->ChannelConstantsAndRanges[3]);
						InTargetClipboardPortableValue = FNiagaraClipboardPortableValue::CreateFromStructValue(*ColorStruct, (uint8*)&ColorValue);
						return true;
					}
					else
					{
						UScriptStruct* Vector4Struct = TVariantStructure<FVector4f>::Get();
						FVector4f Vector4Value(Distribution->ChannelConstantsAndRanges[0], Distribution->ChannelConstantsAndRanges[1], Distribution->ChannelConstantsAndRanges[2], Distribution->ChannelConstantsAndRanges[3]);
						InTargetClipboardPortableValue = FNiagaraClipboardPortableValue::CreateFromStructValue(*Vector4Struct, (uint8*)&Vector4Value);
						return true;
					}
				}
				}
			}
			else if (Distribution->IsCurve() || Distribution->IsGradient())
			{
				UScriptStruct* CurveCollectionStruct = FNiagaraClipboardCurveCollection::StaticStruct();
				FNiagaraClipboardCurveCollection CurveCollectionValue;
				CurveCollectionValue.Curves = Distribution->ChannelCurves;
				InTargetClipboardPortableValue = FNiagaraClipboardPortableValue::CreateFromStructValue(*CurveCollectionStruct, (uint8*)&CurveCollectionValue);
				return true;
			}
			//-TODO: Support for Array & Range
		}
	}
	return false;
}

bool FNiagaraDistributionPropertyEditorUtilities::TryUpdatePropertyFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, IPropertyHandle& InPropertyHandle) const
{
	FStructProperty* StructProperty = CastField<FStructProperty>(InPropertyHandle.GetProperty());
	if (StructProperty != nullptr && StructProperty->Struct->IsChildOf(FNiagaraDistributionBase::StaticStruct()))
	{
		void* ValueData;
		if (InPropertyHandle.GetValueData(ValueData) == FPropertyAccess::Success)
		{
			FNiagaraDistributionBase* Distribution = (FNiagaraDistributionBase*)ValueData;
			TOptional<FNiagaraDistributionBase> ParsedDistribution;

			float FloatValue;
			if (LexTryParseString(FloatValue, *InSourceClipboardPortableValue.ValueString))
			{
				ParsedDistribution = FNiagaraDistributionBase();
				ParsedDistribution->Mode = ENiagaraDistributionMode::UniformConstant;
				ParsedDistribution->ChannelConstantsAndRanges.Add(FloatValue);
			}

			if (ParsedDistribution.IsSet() == false && Distribution->GetBaseNumberOfChannels() == 2)
			{
				UScriptStruct* Vector2Struct = TVariantStructure<FVector2f>::Get();
				FVector2f Vector2Value;
				if (InSourceClipboardPortableValue.TryUpdateStructValue(*Vector2Struct, (uint8*)&Vector2Value))
				{
					ParsedDistribution = FNiagaraDistributionBase();
					ParsedDistribution->Mode = ENiagaraDistributionMode::NonUniformConstant;
					ParsedDistribution->ChannelConstantsAndRanges = { Vector2Value.X, Vector2Value.Y };
				}
			}

			if (ParsedDistribution.IsSet() == false && Distribution->GetBaseNumberOfChannels() == 3)
			{
				UScriptStruct* Vector3Struct = TVariantStructure<FVector3f>::Get();
				FVector3f Vector3Value;
				if (InSourceClipboardPortableValue.TryUpdateStructValue(*Vector3Struct, (uint8*)&Vector3Value))
				{
					ParsedDistribution = FNiagaraDistributionBase();
					ParsedDistribution->Mode = ENiagaraDistributionMode::NonUniformConstant;
					ParsedDistribution->ChannelConstantsAndRanges = { Vector3Value.X, Vector3Value.Y, Vector3Value.Z };
				}
			}

			if (ParsedDistribution.IsSet() == false && Distribution->GetBaseNumberOfChannels() == 4 && Distribution->DisplayAsColor() == false)
			{
				UScriptStruct* Vector4Struct = TVariantStructure<FVector4f>::Get();
				FVector4f Vector4Value;
				if (InSourceClipboardPortableValue.TryUpdateStructValue(*Vector4Struct, (uint8*)&Vector4Value))
				{
					ParsedDistribution = FNiagaraDistributionBase();
					ParsedDistribution->Mode = ENiagaraDistributionMode::NonUniformConstant;
					ParsedDistribution->ChannelConstantsAndRanges = { Vector4Value.X, Vector4Value.Y, Vector4Value.Z, Vector4Value.W };
				}
			}

			if (ParsedDistribution.IsSet() == false && Distribution->GetBaseNumberOfChannels() == 4 && Distribution->DisplayAsColor())
			{
				UScriptStruct* ColorStruct = TBaseStructure<FLinearColor>::Get();
				FLinearColor ColorValue;
				if (InSourceClipboardPortableValue.TryUpdateStructValue(*ColorStruct, (uint8*)&ColorValue))
				{
					ParsedDistribution = FNiagaraDistributionBase();
					ParsedDistribution->Mode = ENiagaraDistributionMode::NonUniformConstant;
					ParsedDistribution->ChannelConstantsAndRanges = { ColorValue.R, ColorValue.G, ColorValue.B, ColorValue.A };
				}
			}

			if (ParsedDistribution.IsSet() == false)
			{
				UScriptStruct* CurveCollectionStruct = FNiagaraClipboardCurveCollection::StaticStruct();
				FNiagaraClipboardCurveCollection CurveCollectionValue;
				if (InSourceClipboardPortableValue.TryUpdateStructValue(*CurveCollectionStruct, (uint8*)&CurveCollectionValue))
				{
					if (CurveCollectionValue.Curves.Num() == 1 && Distribution->AllowDistributionMode(ENiagaraDistributionMode::UniformCurve, InPropertyHandle.GetProperty()))
					{
						ParsedDistribution = FNiagaraDistributionBase();
						ParsedDistribution->Mode = ENiagaraDistributionMode::UniformCurve;
						ParsedDistribution->ChannelCurves = CurveCollectionValue.Curves;
					}
					else if (CurveCollectionValue.Curves.Num() == Distribution->GetBaseNumberOfChannels() && Distribution->AllowDistributionMode(ENiagaraDistributionMode::NonUniformCurve, InPropertyHandle.GetProperty()))
					{
						ParsedDistribution = FNiagaraDistributionBase();
						ParsedDistribution->Mode = ENiagaraDistributionMode::NonUniformCurve;
						if (Distribution->DisplayAsColor() && Distribution->AllowDistributionMode(ENiagaraDistributionMode::ColorGradient, InPropertyHandle.GetProperty()))
						{
							ParsedDistribution->Mode = ENiagaraDistributionMode::ColorGradient;
						}
						ParsedDistribution->ChannelCurves = CurveCollectionValue.Curves;
					}
				}
			}

			if (ParsedDistribution.IsSet())
			{
				const FScopedTransaction Transaction(NSLOCTEXT("DistributionPropertyEditorUtilities", "SetDistributionTransaction", "Set distribution value."));

				TArray<UObject*> OuterObjects;
				InPropertyHandle.GetOuterObjects(OuterObjects);
				for (UObject* OuterObject : OuterObjects)
				{
					OuterObject->Modify();
				}

				InPropertyHandle.NotifyPreChange();
				Distribution->Mode = ParsedDistribution->Mode;
				Distribution->ChannelConstantsAndRanges = ParsedDistribution->ChannelConstantsAndRanges;
				Distribution->ChannelCurves = ParsedDistribution->ChannelCurves;
				InPropertyHandle.NotifyPostChange(EPropertyChangeType::ValueSet);
				return true;
			}
		}
	}
	return false; 
}