// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INiagaraEditorTypeUtilities.h"

struct FNiagaraClipboardPortableValue;
class IPropertyHandle;

class FNiagaraDistributionPropertyEditorUtilities : public FNiagaraEditorPropertyUtilities
{
public:
	virtual bool SupportsClipboardPortableValues() const override;
	virtual bool TryUpdateClipboardPortableValueFromProperty(const IPropertyHandle& InPropertyHandle, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const override;
	virtual bool TryUpdatePropertyFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, IPropertyHandle& InPropertyHandle) const override;
};