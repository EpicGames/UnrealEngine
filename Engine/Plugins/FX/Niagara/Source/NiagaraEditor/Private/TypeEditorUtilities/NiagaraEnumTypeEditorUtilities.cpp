// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEnumTypeEditorUtilities.h"
#include "SNiagaraParameterEditor.h"
#include "NiagaraClipboard.h"
#include "NiagaraTypes.h"
#include "NiagaraVariant.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraVariableMetaData.h"

#include "SEnumCombo.h"
#include "Engine/Texture2D.h"
#include "Styling/AppStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Widgets/Input/SSegmentedControl.h"

class SNiagaraEnumParameterEditor : public SNiagaraParameterEditor
{
public:
	SLATE_BEGIN_ARGS(SNiagaraEnumParameterEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const UEnum* Enum, const FNiagaraInputParameterCustomization& WidgetCustomization)
	{
		SNiagaraParameterEditor::Construct(SNiagaraParameterEditor::FArguments()
			.MinimumDesiredWidth(DefaultInputSize)
			.MaximumDesiredWidth(3 * DefaultInputSize));

		if (WidgetCustomization.WidgetType == ENiagaraInputWidgetType::SegmentedButtons)
		{
			TSharedRef<SSegmentedControl<int32>> ButtonOptionsPanel =
			SNew(SSegmentedControl<int32>)
			.UniformPadding(FMargin(10, 5))
			.MaxSegmentsPerLine(WidgetCustomization.MaxSegmentsPerRow)
			.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
			//.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.Stack.SegmentedControl")
			.Value(this, &SNiagaraEnumParameterEditor::GetValue)
			.OnValueChanged(this, &SNiagaraEnumParameterEditor::SegmentedValueChanged);

			TMap<int32, FWidgetSegmentValueOverride> Overrides;
			for (const FWidgetSegmentValueOverride& WidgetOverride : WidgetCustomization.SegmentValueOverrides)
			{
				Overrides.Add(WidgetOverride.EnumIndexToOverride, WidgetOverride);
			}
			for (int i = 0; i < Enum->NumEnums() - 1; i++)
			{
				if (Enum->HasMetaData(TEXT("Hidden"), i))
				{
					continue;
				}
				FWidgetSegmentValueOverride* Override = Overrides.Find(i);
				int32 Value = static_cast<int32>(Enum->GetValueByIndex(i));
				FText DisplayName = Enum->GetDisplayNameTextByIndex(i);
				FText TooltipText = Enum->GetToolTipTextByIndex(i);
				if (TooltipText.IsEmpty())
				{
					TooltipText = DisplayName;
				}
				
				if (Override && Override->bOverrideDisplayName)
				{
					DisplayName = Override->DisplayNameOverride;
				}

				SSegmentedControl<int>::FScopedWidgetSlotArguments Slot = ButtonOptionsPanel->AddSlot(Value);
				Slot
                  .HAlign(HAlign_Center)
                  .VAlign(VAlign_Center)
                  .Text(DisplayName)
                  .ToolTip(TooltipText);

				if (Override && Override->DisplayIcon)
				{
					ImageBrushes.Add(MakeShared<FSlateImageBrush>(Override->DisplayIcon, FVector2D(16.f, 16.f)));
					Slot.Icon(ImageBrushes.Last().Get());
				}
			}
			
			ChildSlot
			[
				ButtonOptionsPanel
			];
		}
		else
		{
			ChildSlot
			[
				SNew(SEnumComboBox, Enum)
				.CurrentValue(this, &SNiagaraEnumParameterEditor::GetValue)
				.ContentPadding(FMargin(2, 0))
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.OnEnumSelectionChanged(SEnumComboBox::FOnEnumSelectionChanged::CreateSP(this, &SNiagaraEnumParameterEditor::ValueChanged))
			];
		}
	}

	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetIntStruct(), TEXT("Struct type not supported."));
		IntValue = reinterpret_cast<FNiagaraInt32*>(Struct->GetStructMemory())->Value;
	}

	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetIntStruct(), TEXT("Struct type not supported."));
		reinterpret_cast<FNiagaraInt32*>(Struct->GetStructMemory())->Value = IntValue;
	}

private:
	int32 GetValue() const
	{
		return IntValue;
	}

	void ValueChanged(int32 Value, ESelectInfo::Type SelectionType)
	{
		IntValue = Value;
		ExecuteOnValueChanged();
	}

	void SegmentedValueChanged(int32 NewValue)
	{
		IntValue = NewValue;
		ExecuteOnValueChanged();
	}

	int32 IntValue = 0;
	TArray<TSharedPtr<FSlateImageBrush>> ImageBrushes;
};

bool FNiagaraEditorEnumTypeUtilities::CanProvideDefaultValue() const
{
	return true;
}

void FNiagaraEditorEnumTypeUtilities::UpdateVariableWithDefaultValue(FNiagaraVariable& Variable) const
{
	const UEnum* Enum = Variable.GetType().GetEnum();
	checkf(Enum != nullptr, TEXT("Variable is not an enum type."));

	FNiagaraInt32 EnumIntValue;
	EnumIntValue.Value = static_cast<int32>(Enum->GetValueByIndex(0));

	Variable.SetValue<FNiagaraInt32>(EnumIntValue);
}

TSharedPtr<SNiagaraParameterEditor> FNiagaraEditorEnumTypeUtilities::CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const
{
	return SNew(SNiagaraEnumParameterEditor, ParameterType.GetEnum(), WidgetCustomization);
}

bool FNiagaraEditorEnumTypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorEnumTypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));

	const UEnum* Enum = AllocatedVariable.GetType().GetEnum();
	checkf(Enum != nullptr, TEXT("Variable is not an enum type."));

	int32 EnumValue = AllocatedVariable.GetValue<int32>();
	if(Enum->IsValidEnumValue(EnumValue))
	{
		return Enum->GetNameStringByValue(EnumValue);
	}
	else
	{
		FString ReplacementForInvalidValue = Enum->GetNameStringByIndex(0);
		UE_LOGF(LogNiagaraEditor, Error, "Error getting default value for enum pin.  Enum value %i is not supported for enum type %ls.  Using value %ls",
			EnumValue, *AllocatedVariable.GetType().GetName(), *ReplacementForInvalidValue);
		return ReplacementForInvalidValue;
	}
}

bool FNiagaraEditorEnumTypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	const UEnum* Enum = Variable.GetType().GetEnum();
	checkf(Enum != nullptr, TEXT("Variable is not an enum type."));

	FNiagaraInt32 EnumValue;
	EnumValue.Value = static_cast<int32>(Enum->GetValueByNameString(StringValue));
	if(EnumValue.Value != INDEX_NONE)
	{
		Variable.AllocateData();
		Variable.SetValue<FNiagaraInt32>(EnumValue);
		return true;
	}
	return false;
}

bool FNiagaraEditorEnumTypeUtilities::CanSetValueFromDisplayName() const
{
	return true;
}

bool FNiagaraEditorEnumTypeUtilities::SetValueFromDisplayName(const FText& TextValue, FNiagaraVariable& Variable) const
{
	const UEnum* Enum = Variable.GetType().GetEnum();
	checkf(Enum != nullptr, TEXT("Variable is not an enum type."));

	int32 FoundIndex = INDEX_NONE;
	for (int32 i = 0; i < Enum->NumEnums(); i++)
	{
		if (TextValue.CompareTo(Enum->GetDisplayNameTextByIndex(i)) == 0)
		{
			FoundIndex = i;
			break;
		}
	}

	if (FoundIndex != INDEX_NONE)
	{
		FNiagaraInt32 EnumValue;
		EnumValue.Value = static_cast<int32>(Enum->GetValueByIndex(FoundIndex));
		Variable.AllocateData();
		Variable.SetValue<FNiagaraInt32>(EnumValue);
		return true;
	}
	return false;
}

FText FNiagaraEditorEnumTypeUtilities::GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate search text for an unallocated variable."));

	const UEnum* Enum = AllocatedVariable.GetType().GetEnum();
	checkf(Enum != nullptr, TEXT("Variable is not an enum type."));

	const int32 EnumNameIndex = Enum->GetIndexByValue(AllocatedVariable.GetValue<int32>());
	return Enum->GetDisplayNameTextByIndex(EnumNameIndex);
}

FText FNiagaraEditorEnumTypeUtilities::GetStackDisplayText(const FNiagaraVariable& Variable) const
{
	return Variable.GetType().GetEnum()->GetDisplayNameTextByValue(Variable.GetValue<int32>());
}

bool FNiagaraEditorEnumTypeUtilities::TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const
{
	if (InSourceType.IsEnum() && InSourceValue.GetNumBytes() == FNiagaraTypeDefinition::GetIntDef().GetSize())
	{
		FNiagaraVariable Temp(InSourceType, NAME_None);
		Temp.SetData(InSourceValue.GetBytes());
		UEnum* ValueEnum = InSourceType.GetEnum();
		InTargetClipboardPortableValue.ValueString = ValueEnum->GetNameStringByValue(Temp.GetValue<FNiagaraInt32>().Value);
		return true;
	}
	return false;
}

bool FNiagaraEditorEnumTypeUtilities::TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const
{
	if (InTargetType.IsEnum())
	{
		int32 EnumValue = (int32)InTargetType.GetEnum()->GetValueByNameString(InSourceClipboardPortableValue.ValueString);
		if (EnumValue != INDEX_NONE)
		{
			FNiagaraInt32 NiagaraIntValue;
			NiagaraIntValue.Value = EnumValue;
			FNiagaraVariable Temp(InTargetType, NAME_None);
			Temp.SetValue<FNiagaraInt32>(NiagaraIntValue);
			InTargetValue.SetBytes(Temp.GetData(), Temp.GetSizeInBytes());
			return true;
		}
	}
	return false;
}

int32 FNiagaraEditorEnumTypeUtilities::VariableToSelectNumericValue(const FNiagaraVariable& InVariableValue) const
{
	if (ensureMsgf(InVariableValue.GetType().IsEnum() && InVariableValue.IsDataAllocated(),
		TEXT("InVariableValue must be an enum type and must have its data allocated to convert to a select value.")) == false)
	{
		return INDEX_NONE;
	}

	// We store enums by their value in FNiagaraVariables and parameter store, but the select value is the index.
	UEnum* Enum = InVariableValue.GetType().GetEnum();
	return Enum->GetIndexByValue(InVariableValue.GetValue<FNiagaraInt32>().Value);
}

FName FNiagaraEditorEnumTypeUtilities::GetDebugNameForSelectValue(const FNiagaraTypeDefinition& ValueType, int32 SelectValue) const
{
	UEnum* Enum = ValueType.GetEnum();
	FName Name = Enum->GetNameByIndex(SelectValue);
	FText DisplayName = Enum->GetDisplayNameTextByIndex(SelectValue);
	if (DisplayName.IsEmptyOrWhitespace())
	{
		return Name;
	}

	FString NameString = Name.ToString();
	FString DisplayNameString = DisplayName.ToString();
	return NameString.Equals(DisplayNameString)	? Name : *FString::Printf(TEXT("%s (%s)"), *DisplayNameString, *NameString);
}
