// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraFloatTypeEditorUtilities.h"

#include "EdGraphSchema_Niagara.h"
#include "GraphEditorSettings.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorStyle.h"
#include "SNiagaraParameterEditor.h"
#include "NiagaraClipboard.h"
#include "NiagaraTypes.h"
#include "NiagaraVariant.h"
#include "Math/UnitConversion.h"
#include "Widgets/SNiagaraNumericDropDown.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SVolumeControl.h"

void SNiagaraFloatParameterEditor::Construct(const FArguments& InArgs, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization)
{
	ValueAttribute = InArgs._Value;
	OnValueChangedEvent = InArgs._OnValueChanged;
	OnBeginValueChangeEvent = InArgs._OnBeginValueChange;
	OnEndValueChangeEvent = InArgs._OnEndValueChange;
	SNiagaraParameterEditor::Construct(SNiagaraParameterEditor::FArguments()
	                                   .MinimumDesiredWidth(DefaultInputSize)
	                                   .MaximumDesiredWidth(DefaultInputSize));

	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
	const UNiagaraEditorSettings* NiagaraSettings = GetDefault<UNiagaraEditorSettings>();

	if (WidgetCustomization.WidgetType == ENiagaraInputWidgetType::Slider)
	{
		float MinValue = WidgetCustomization.bHasMinValue ? WidgetCustomization.MinValue : 0;
		float MaxValue = WidgetCustomization.bHasMaxValue ? WidgetCustomization.MaxValue : 1;
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(UEdGraphSchema_Niagara::GetTypeColor(FNiagaraTypeDefinition::GetFloatDef()))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"))
			]
			+ SHorizontalBox::Slot().AutoWidth()
			[
				SNew(SBox).WidthOverride(100.0f)
				[
					SNew(SSlider)
					.MinValue(MinValue)
					.MaxValue(MaxValue)
					.Value(this, &SNiagaraFloatParameterEditor::GetSliderValue)
					.OnValueChanged_Lambda([this, WidgetCustomization](float NewVal)
					{
						SliderValue = NewVal;
						FloatValue = NewVal;
						if (WidgetCustomization.bHasStepWidth && WidgetCustomization.StepWidth != 0)
						{
							FloatValue = FMath::RoundToFloat(NewVal / WidgetCustomization.StepWidth) * WidgetCustomization.StepWidth;
						}
						ExecuteOnValueChanged();
						OnValueChangedEvent.ExecuteIfBound(FloatValue);
					})
					.OnMouseCaptureBegin(this, &SNiagaraFloatParameterEditor::BeginSliderMovement)
					.OnMouseCaptureEnd(this, &SNiagaraFloatParameterEditor::ExecuteOnEndValueChange)
				]
			]
			+ SHorizontalBox::Slot().AutoWidth()
			[
				SNew(SBox).WidthOverride(75.0f)
				[
					SNew(SNumericEntryBox<float>)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.MinValue(MinValue)
					.MaxValue(MaxValue)
					.Value(this, &SNiagaraFloatParameterEditor::GetValue)
					.OnValueChanged(this, &SNiagaraFloatParameterEditor::ValueChanged)
					.OnValueCommitted(this, &SNiagaraFloatParameterEditor::ValueCommitted)
					.TypeInterface(GetTypeInterface<float>(DisplayUnit))
					.AllowSpin(false)
					.Delta(WidgetCustomization.bHasStepWidth ? WidgetCustomization.StepWidth : 0)
				]
			]
		];
	}
	else if (WidgetCustomization.WidgetType == ENiagaraInputWidgetType::Volume)
	{
		ChildSlot
		[
			SNew(SVolumeControl)
			.Volume(this, &SNiagaraFloatParameterEditor::GetSliderValue)
			.Muted(this, &SNiagaraFloatParameterEditor::IsMuted)
			.OnVolumeChanged_Lambda([this](float NewVal)
			{
				SliderValue = NewVal;
				FloatValue = NewVal;
				ExecuteOnValueChanged();
				OnValueChangedEvent.ExecuteIfBound(NewVal);
			})
			.OnMuteChanged_Lambda([this](bool NewVal)
			{
				bMuted = NewVal;
				FloatValue = bMuted ? 0 : SliderValue;
				ExecuteOnValueChanged();
				OnValueChangedEvent.ExecuteIfBound(FloatValue);
			})
		];
	}
	else if (WidgetCustomization.WidgetType == ENiagaraInputWidgetType::NumericDropdown && WidgetCustomization.InputDropdownValues.Num() > 0)
	{
		TArray<SNiagaraNumericDropDown<float>::FNamedValue> DropDownValues;
		for (const FWidgetNamedInputValue& Value : WidgetCustomization.InputDropdownValues)
		{
			DropDownValues.Add(SNiagaraNumericDropDown<float>::FNamedValue(Value.Value, Value.DisplayName.IsEmpty() ? FText::AsNumber(Value.Value) : Value.DisplayName, Value.Tooltip));
		}

		ChildSlot
		[
			SNew(SNiagaraNumericDropDown<float>)
			.DropDownValues(DropDownValues)
			.bShowNamedValue(true)
			.MinDesiredValueWidth(75)
			.PillType(FNiagaraTypeDefinition::GetFloatDef())
			.Value_Lambda([this]() 
			{ 
				return FloatValue;
			})
			.OnValueChanged(this, &SNiagaraFloatParameterEditor::ValueChanged)
		];
	}
	else
	{
		TOptional<float> MinValue;
		TOptional<float> MaxValue;
		if (WidgetCustomization.bHasMinValue)
		{
			MinValue = WidgetCustomization.MinValue;
		}
		if (WidgetCustomization.bHasMaxValue)
		{
			MaxValue = WidgetCustomization.MaxValue;
		}
		
		ChildSlot
		[
			SNew(SNumericEntryBox<float>)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			.MinValue(MinValue)
			.MaxValue(MaxValue)
			.MinSliderValue(MinValue)
			.MaxSliderValue(MaxValue)
			.Value(this, &SNiagaraFloatParameterEditor::GetValue)
			.OnValueChanged(this, &SNiagaraFloatParameterEditor::ValueChanged)
			.OnValueCommitted(this, &SNiagaraFloatParameterEditor::ValueCommitted)
			.OnBeginSliderMovement(this, &SNiagaraFloatParameterEditor::BeginSliderMovement)
			.OnEndSliderMovement(this, &SNiagaraFloatParameterEditor::EndSliderMovement)
			.TypeInterface(GetTypeInterface<float>(DisplayUnit))
			.AllowSpin(true)
			.BroadcastValueChangesPerKey(!NiagaraSettings->GetUpdateStackValuesOnCommitOnly() && !WidgetCustomization.bBroadcastValueChangesOnCommitOnly)
			.LabelPadding(FMargin(3))
			.Delta(WidgetCustomization.bHasStepWidth ? WidgetCustomization.StepWidth : 0)
			.LabelLocation(SNumericEntryBox<float>::ELabelLocation::Inside)
			.Label()
			[
				SNumericEntryBox<float>::BuildNarrowColorLabel(Settings->FloatPinTypeColor)
			]
		];
	}
}

void SNiagaraFloatParameterEditor::UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct)
{
	checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetFloatStruct(), TEXT("Struct type not supported."));
	FloatValue = reinterpret_cast<FNiagaraFloat*>(Struct->GetStructMemory())->Value;
	if (!bMuted)
	{
		SliderValue = FloatValue;
	}
}

void SNiagaraFloatParameterEditor::UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct)
{
	checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetFloatStruct(), TEXT("Struct type not supported."));
	reinterpret_cast<FNiagaraFloat*>(Struct->GetStructMemory())->Value = FloatValue;
}

void SNiagaraFloatParameterEditor::BeginSliderMovement()
{
	ExecuteOnBeginValueChange();
	OnBeginValueChangeEvent.ExecuteIfBound();
}

void SNiagaraFloatParameterEditor::EndSliderMovement(float Value)
{
	ExecuteOnEndValueChange();
	OnEndValueChangeEvent.ExecuteIfBound(Value);
}

TOptional<float> SNiagaraFloatParameterEditor::GetValue() const
{
	return ValueAttribute.Get(FloatValue);
}

float SNiagaraFloatParameterEditor::GetSliderValue() const
{
	return ValueAttribute.Get(SliderValue);
}

bool SNiagaraFloatParameterEditor::IsMuted() const
{
	return bMuted;
}

void SNiagaraFloatParameterEditor::ValueChanged(float Value)
{
	FloatValue = Value;
	ExecuteOnValueChanged();
	OnValueChangedEvent.ExecuteIfBound(Value);
}

void SNiagaraFloatParameterEditor::ValueCommitted(float Value, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		ValueChanged(Value);
	}
}

TSharedPtr<SNiagaraParameterEditor> FNiagaraEditorFloatTypeUtilities::CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const
{
	return SNew(SNiagaraFloatParameterEditor, DisplayUnit, WidgetCustomization);
}

bool FNiagaraEditorFloatTypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorFloatTypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));
	return LexToString(AllocatedVariable.GetValue<FNiagaraFloat>().Value);
}

bool FNiagaraEditorFloatTypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	FNiagaraFloat FloatValue;
	if (LexTryParseString(FloatValue.Value, *StringValue) || !Variable.IsDataAllocated())
	{
		Variable.SetValue<FNiagaraFloat>(FloatValue);
		return true;
	}
	return false;
}

FText FNiagaraEditorFloatTypeUtilities::GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	return FText::FromString(GetPinDefaultStringFromValue(AllocatedVariable));
}

bool FNiagaraEditorFloatTypeUtilities::TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const
{
	if (InSourceType == FNiagaraTypeDefinition::GetFloatDef() && InSourceValue.GetNumBytes() == FNiagaraTypeDefinition::GetFloatDef().GetSize())
	{
		FNiagaraVariable Temp(InSourceType, NAME_None);
		Temp.SetData(InSourceValue.GetBytes());
		float FloatValue = Temp.GetValue<FNiagaraFloat>().Value;
		InTargetClipboardPortableValue.ValueString = LexToString(FloatValue);
		return true;
	}
	return false;
}

bool FNiagaraEditorFloatTypeUtilities::TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const
{
	float FloatValue;
	if (InTargetType == FNiagaraTypeDefinition::GetFloatDef() &&
		LexTryParseString(FloatValue, *InSourceClipboardPortableValue.ValueString))
	{
		FNiagaraFloat NiagaraFloatValue;
		NiagaraFloatValue.Value = FloatValue;
		FNiagaraVariable Temp(InTargetType, NAME_None);
		Temp.SetValue<FNiagaraFloat>(NiagaraFloatValue);
		InTargetValue.SetBytes(Temp.GetData(), Temp.GetSizeInBytes());
		return true;
	}
	return false;
}
