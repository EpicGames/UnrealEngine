// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraTypes.h"
#include "SNiagaraParameterEditor.h"

/** Niagara editor utilities for the float type. */
class FNiagaraEditorFloatTypeUtilities : public FNiagaraEditorTypeUtilities
{
public:
	//~ INiagaraEditorTypeUtilities interface.
	virtual bool CanCreateParameterEditor() const override { return true; }
	virtual TSharedPtr<SNiagaraParameterEditor> CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const override;
	virtual bool CanHandlePinDefaults() const override;
	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const override;
	virtual FText GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual FText GetStackDisplayText(const FNiagaraVariable& Variable) const override { return FText::AsNumber(Variable.GetValue<float>()); };
	virtual bool SupportsClipboardPortableValues() const override { return true; }
	virtual bool TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& OutTargetClipboardPortableValue) const override;
	virtual bool TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const override;
};

class SNiagaraFloatParameterEditor : public SNiagaraParameterEditor
{
	DECLARE_DELEGATE_OneParam(FOnValueChanged, float);
	
public:
	SLATE_BEGIN_ARGS(SNiagaraFloatParameterEditor) { }
		SLATE_ATTRIBUTE(float, Value)
		SLATE_EVENT( FOnValueChanged, OnValueChanged )
		SLATE_EVENT( FSimpleDelegate, OnBeginValueChange )
		SLATE_EVENT( FOnValueChanged, OnEndValueChange )
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization);
	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) override;
	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) override;
	virtual bool CanChangeContinuously() const override { return true; }
private:
	void BeginSliderMovement();
	void EndSliderMovement(float Value);
	TOptional<float> GetValue() const;
	float GetSliderValue() const;
	bool IsMuted() const;	
	void ValueChanged(float Value);
	void ValueCommitted(float Value, ETextCommit::Type CommitInfo);

	float FloatValue = 0;
	float SliderValue = 0;
	bool bMuted = false;

	TAttribute<float> ValueAttribute;
	FOnValueChanged OnValueChangedEvent;
	FSimpleDelegate OnBeginValueChangeEvent;
	FOnValueChanged OnEndValueChangeEvent;
};