// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraIntegerTypeEditorUtilities.h"

#include "EdGraphSchema_Niagara.h"
#include "GraphEditorSettings.h"
#include "NiagaraEditorSettings.h"
#include "SNiagaraParameterEditor.h"
#include "NiagaraClipboard.h"
#include "NiagaraTypes.h"
#include "NiagaraVariant.h"
#include "NiagaraEditorStyle.h"
#include "Widgets/SNiagaraNumericDropDown.h"

#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSlider.h"

void SNiagaraIntegerParameterEditor::Construct(const FArguments& InArgs, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization)
{
	ValueAttribute = InArgs._Value;
	OnValueChangedEvent = InArgs._OnValueChanged;
	OnBeginValueChangeEvent = InArgs._OnBeginValueChange;
	OnEndValueChangeEvent = InArgs._OnEndValueChange;
	SNiagaraParameterEditor::Construct(SNiagaraParameterEditor::FArguments()
		.MinimumDesiredWidth(DefaultInputSize)
		.MaximumDesiredWidth(DefaultInputSize));

	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();

	if (WidgetCustomization.WidgetType == ENiagaraInputWidgetType::Slider)
	{
		float MinValue = WidgetCustomization.bHasMinValue ? WidgetCustomization.MinValue : 0;
		float MaxValue = WidgetCustomization.bHasMaxValue ? WidgetCustomization.MaxValue : 1;
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(UEdGraphSchema_Niagara::GetTypeColor(FNiagaraTypeDefinition::GetIntDef()))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"))
			]
			+ SHorizontalBox::Slot().AutoWidth()
			[
				SNew(SBox).WidthOverride(100.0f)
				[
					SNew(SSlider)
					.MinValue(MinValue)
					.MaxValue(MaxValue)
					.Value(this, &SNiagaraIntegerParameterEditor::GetSliderValue)
					.OnValueChanged_Lambda([this, WidgetCustomization](float NewVal)
					{
						SliderValue = NewVal;

						// slider only works with float, so we round for the actual parameter value
						int32 Resolution = WidgetCustomization.bHasStepWidth && WidgetCustomization.StepWidth >= 1 ? WidgetCustomization.StepWidth : 1;
						IntValue = FMath::RoundToInt(NewVal / Resolution) * Resolution;
						ExecuteOnValueChanged();
						OnValueChangedEvent.ExecuteIfBound(IntValue);
					})
					.OnMouseCaptureBegin(this, &SNiagaraIntegerParameterEditor::ExecuteOnBeginValueChange)
					.OnMouseCaptureEnd(this, &SNiagaraIntegerParameterEditor::ExecuteOnEndValueChange)
				]
			]
			+ SHorizontalBox::Slot().AutoWidth()
			[
				SNew(SBox).WidthOverride(75.0f)
				[
					SNew(SNumericEntryBox<int32>)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.MinValue(MinValue)
					.MaxValue(MaxValue)
					.Value(this, &SNiagaraIntegerParameterEditor::GetValue)
					.OnValueChanged(this, &SNiagaraIntegerParameterEditor::ValueChanged)
					.OnValueCommitted(this, &SNiagaraIntegerParameterEditor::ValueCommitted)
					.TypeInterface(GetTypeInterface<int32>(DisplayUnit))
					.AllowSpin(false)
					.Delta(WidgetCustomization.bHasStepWidth ? WidgetCustomization.StepWidth : 0)
				]
			]
		];
	}
	else if (WidgetCustomization.WidgetType == ENiagaraInputWidgetType::NumericDropdown && WidgetCustomization.InputDropdownValues.Num() > 0)
	{
		TArray<SNiagaraNumericDropDown<int32>::FNamedValue> DropDownValues;
		for (const FWidgetNamedInputValue& Value : WidgetCustomization.InputDropdownValues)
		{
			DropDownValues.Add(SNiagaraNumericDropDown<int32>::FNamedValue(Value.Value, Value.DisplayName.IsEmpty() ? FText::AsNumber(Value.Value) : Value.DisplayName, Value.Tooltip));
		}

		ChildSlot
		[
			SNew(SNiagaraNumericDropDown<int32>)
			.DropDownValues(DropDownValues)
			.bShowNamedValue(true)
			.MinDesiredValueWidth(75)
			.PillType(FNiagaraTypeDefinition::GetIntDef())
			.Value_Lambda([this]() 
			{ 
				return IntValue;
			})
			.OnValueChanged_Lambda([this](int32 NewVal)
			{
				IntValue = NewVal;
				ExecuteOnValueChanged();
				OnValueChangedEvent.ExecuteIfBound(NewVal);
			})
		];
	}
	else if(WidgetCustomization.WidgetType == ENiagaraInputWidgetType::EnumStyle && WidgetCustomization.EnumStyleDropdownValues.Num() > 0)
	{
		TArray<SNiagaraNumericDropDown<int32>::FNamedValue> DropDownValues;
		for(int32 EnumStyleValueIndex = 0; EnumStyleValueIndex < WidgetCustomization.EnumStyleDropdownValues.Num(); EnumStyleValueIndex++)
		{
			FText DisplayName = WidgetCustomization.EnumStyleDropdownValues[EnumStyleValueIndex].DisplayName;
			FText Tooltip = WidgetCustomization.EnumStyleDropdownValues[EnumStyleValueIndex].Tooltip;
					
			DropDownValues.Add(SNiagaraNumericDropDown<int32>::FNamedValue(EnumStyleValueIndex, DisplayName, Tooltip));
		}
		
		ChildSlot
		[
			SNew(SNiagaraNumericDropDown<int32>)
			.DropDownValues(DropDownValues)
			.bAllowTyping(false)
			.bShowNamedValue(true)
			.MinDesiredValueWidth(75)
			.PillType(FNiagaraTypeDefinition::GetIntDef())
			.Value_Lambda([this]() 
			{ 
				return IntValue;
			})
			.OnValueChanged_Lambda([this](int32 NewVal)
			{
				IntValue = NewVal;
				ExecuteOnValueChanged();
				OnValueChangedEvent.ExecuteIfBound(NewVal);
			})
		];
	}
	else
	{
		TOptional<int32> MinValue;
		TOptional<int32> MaxValue;
		if (WidgetCustomization.bHasMinValue)
		{
			MinValue = WidgetCustomization.MinValue;
		}
		if (WidgetCustomization.bHasMaxValue)
		{
			MaxValue = WidgetCustomization.MaxValue;
		}
		
		ChildSlot
		[
			SNew(SNumericEntryBox<int32>)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			.MinValue(MinValue)
			.MaxValue(MaxValue)
			.MinSliderValue(MinValue)
			.MaxSliderValue(MaxValue)
			.Value(this, &SNiagaraIntegerParameterEditor::GetValue)
			.OnValueChanged(this, &SNiagaraIntegerParameterEditor::ValueChanged)
			.OnValueCommitted(this, &SNiagaraIntegerParameterEditor::ValueCommitted)
			.OnBeginSliderMovement(this, &SNiagaraIntegerParameterEditor::BeginSliderMovement)
			.OnEndSliderMovement(this, &SNiagaraIntegerParameterEditor::EndSliderMovement)
			.TypeInterface(GetTypeInterface<int32>(DisplayUnit))
			.AllowSpin(true)
			.BroadcastValueChangesPerKey(!GetDefault<UNiagaraEditorSettings>()->GetUpdateStackValuesOnCommitOnly() && !WidgetCustomization.bBroadcastValueChangesOnCommitOnly)
			.LabelPadding(FMargin(3))
			.LabelLocation(SNumericEntryBox<int32>::ELabelLocation::Inside)
			.Label()
			[
				SNumericEntryBox<int32>::BuildNarrowColorLabel(Settings->IntPinTypeColor)
			]
		];
	}
}

void SNiagaraIntegerParameterEditor::UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct)
{
	checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetIntStruct(), TEXT("Struct type not supported."));
	IntValue = reinterpret_cast<FNiagaraInt32*>(Struct->GetStructMemory())->Value;
	SliderValue = IntValue;
}

void SNiagaraIntegerParameterEditor::UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct)
{
	checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetIntStruct(), TEXT("Struct type not supported."));
	reinterpret_cast<FNiagaraInt32*>(Struct->GetStructMemory())->Value = IntValue;
}

void SNiagaraIntegerParameterEditor::BeginSliderMovement()
{
	ExecuteOnBeginValueChange();
	OnBeginValueChangeEvent.ExecuteIfBound();
}

void SNiagaraIntegerParameterEditor::EndSliderMovement(int32 Value)
{
	ExecuteOnEndValueChange();
	OnEndValueChangeEvent.ExecuteIfBound(Value);
}

TOptional<int32> SNiagaraIntegerParameterEditor::GetValue() const
{
	return ValueAttribute.Get(IntValue);
}

float SNiagaraIntegerParameterEditor::GetSliderValue() const
{
    return ValueAttribute.Get(SliderValue);
}

void SNiagaraIntegerParameterEditor::ValueChanged(int32 Value)
{
	IntValue = Value;
	ExecuteOnValueChanged();
	OnValueChangedEvent.ExecuteIfBound(Value);
}

void SNiagaraIntegerParameterEditor::ValueCommitted(int32 Value, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		ValueChanged(Value);
	}
}

TSharedPtr<SNiagaraParameterEditor> FNiagaraEditorIntegerTypeUtilities::CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const
{
	return SNew(SNiagaraIntegerParameterEditor, DisplayUnit, WidgetCustomization);
}

bool FNiagaraEditorIntegerTypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorIntegerTypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));
	return LexToString(AllocatedVariable.GetValue<FNiagaraInt32>().Value);
}

bool FNiagaraEditorIntegerTypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	FNiagaraInt32 IntegerValue;
	if(LexTryParseString(IntegerValue.Value, *StringValue) || !Variable.IsDataAllocated())
	{
		Variable.SetValue<FNiagaraInt32>(IntegerValue);
		return true;
	}
	return false;
}

FText FNiagaraEditorIntegerTypeUtilities::GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	return FText::FromString(GetPinDefaultStringFromValue(AllocatedVariable));
}

bool FNiagaraEditorIntegerTypeUtilities::TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const
{
	if (InSourceType == FNiagaraTypeDefinition::GetIntDef() && InSourceValue.GetNumBytes() == FNiagaraTypeDefinition::GetIntDef().GetSize())
	{
		FNiagaraVariable Temp(InSourceType, NAME_None);
		Temp.SetData(InSourceValue.GetBytes());
		int32 IntValue = Temp.GetValue<FNiagaraInt32>().Value;
		InTargetClipboardPortableValue.ValueString = LexToString(IntValue);
		return true;
	}
	return false;
}

bool FNiagaraEditorIntegerTypeUtilities::TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const
{
	int32 IntValue;
	if (InTargetType == FNiagaraTypeDefinition::GetIntDef() &&
		LexTryParseString(IntValue, *InSourceClipboardPortableValue.ValueString))
	{
		FNiagaraInt32 NiagaraIntValue;
		NiagaraIntValue.Value = IntValue;
		FNiagaraVariable Temp(InTargetType, NAME_None);
		Temp.SetValue<FNiagaraInt32>(NiagaraIntValue);
		InTargetValue.SetBytes(Temp.GetData(), Temp.GetSizeInBytes());
		return true;
	}
	return false;
}

int32 FNiagaraEditorIntegerTypeUtilities::VariableToSelectNumericValue(const FNiagaraVariable& InVariableValue) const
{
	FNiagaraTypeDefinition BaseDefinition = InVariableValue.GetType().RemoveStaticDef();
	if (ensureMsgf(BaseDefinition == FNiagaraTypeDefinition::GetIntDef() && InVariableValue.IsDataAllocated(),
		TEXT("InVariableValue must be an FNiagaraInt32 and must have its data allocated to convert to a select value.")) == false)
	{
		return INDEX_NONE;
	}
	return InVariableValue.GetValue<FNiagaraInt32>().Value;
}

FName FNiagaraEditorIntegerTypeUtilities::GetDebugNameForSelectValue(const FNiagaraTypeDefinition& ValueType, int32 SelectValue) const
{
	return *LexToString(SelectValue);
}
