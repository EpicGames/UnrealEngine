// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INiagaraEditorTypeUtilities.h"
#include "SNiagaraParameterEditor.h"

class FNiagaraEditorIntegerTypeUtilities : public FNiagaraEditorTypeUtilities
{
public:
	//~ INiagaraEditorTypeUtilities interface.
	virtual bool CanCreateParameterEditor() const override { return true; }
	virtual TSharedPtr<SNiagaraParameterEditor> CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const override;
	virtual bool CanHandlePinDefaults() const override;
	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const override;
	virtual FText GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual bool SupportsClipboardPortableValues() const override { return true; }
	virtual bool TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const override;
	virtual bool TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const override;
	virtual bool CanBeSelectValue() const override { return true; }
	virtual int32 VariableToSelectNumericValue(const FNiagaraVariable& InVariableValue) const override;
	virtual FName GetDebugNameForSelectValue(const FNiagaraTypeDefinition& ValueType, int32 SelectValue) const override;
};

class SNiagaraIntegerParameterEditor : public SNiagaraParameterEditor
{
	DECLARE_DELEGATE_OneParam(FOnValueChanged, int32);
	
public:
	SLATE_BEGIN_ARGS(SNiagaraIntegerParameterEditor) { }
		SLATE_ATTRIBUTE(int32, Value)
		SLATE_EVENT( FOnValueChanged, OnValueChanged )
		SLATE_EVENT( FSimpleDelegate, OnBeginValueChange )
		SLATE_EVENT( FOnValueChanged, OnEndValueChange )
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization);
	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) override;
	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) override;
	virtual bool CanChangeContinuously() const override { return true; }
private:
	void BeginSliderMovement();
	void EndSliderMovement(int32 Value);
	TOptional<int32> GetValue() const;
	float GetSliderValue() const;
	void ValueChanged(int32 Value);
	void ValueCommitted(int32 Value, ETextCommit::Type CommitInfo);

	int32 IntValue = 0;
	float SliderValue = 0;

	TAttribute<int32> ValueAttribute;
	FOnValueChanged OnValueChangedEvent;
	FSimpleDelegate OnBeginValueChangeEvent;
	FOnValueChanged OnEndValueChangeEvent;




	
};