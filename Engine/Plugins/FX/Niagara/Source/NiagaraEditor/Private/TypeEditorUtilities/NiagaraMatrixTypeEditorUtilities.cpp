// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMatrixTypeEditorUtilities.h"
#include "NiagaraTypes.h"

void FNiagaraEditorMatrixTypeUtilities::UpdateVariableWithDefaultValue(FNiagaraVariable& Variable) const
{
	// This is implemented directly here to avoid alignment issues with the default initialization path for structs which
	// uses pointer assignment rather than a mem-copy.
	checkf(Variable.GetType().GetStruct() == FNiagaraTypeDefinition::GetMatrix4Struct(), TEXT("Struct type not supported."));
	FMatrix44f DefaultMatrix(FMatrix44f::Identity);
	Variable.SetData((uint8*)&DefaultMatrix);
}