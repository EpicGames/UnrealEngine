// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INiagaraEditorTypeUtilities.h"

class SNiagaraParameterEditor;

/** Niagara editor utilities for the bool type. */
class FNiagaraEditorMatrixTypeUtilities : public FNiagaraEditorTypeUtilities
{
public:
	//~ INiagaraEditorTypeUtilities interface.
	virtual bool CanProvideDefaultValue() const override { return true; }
	virtual void UpdateVariableWithDefaultValue(FNiagaraVariable& Variable) const override;
};