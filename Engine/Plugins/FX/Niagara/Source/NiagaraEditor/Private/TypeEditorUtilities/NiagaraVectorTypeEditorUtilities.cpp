// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraVectorTypeEditorUtilities.h"

#include "NiagaraEditorSettings.h"
#include "SNiagaraParameterEditor.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraTypes.h"

#include "Misc/DefaultValueHelper.h"
#include "Widgets/Input/SVectorInputBox.h"

class SNiagaraVectorParameterEditorBase : public SNiagaraParameterEditor
{
public:
	SLATE_BEGIN_ARGS(SNiagaraVectorParameterEditorBase) { }
		SLATE_ARGUMENT(int32, ComponentCount)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization)
	{
		SNiagaraParameterEditor::Construct(SNiagaraParameterEditor::FArguments()
			.MinimumDesiredWidth(DefaultInputSize * InArgs._ComponentCount)
			.MaximumDesiredWidth(DefaultInputSize * InArgs._ComponentCount));

		ChildSlot
		[
			ConstructVectorWidget(InArgs._ComponentCount, DisplayUnit, WidgetCustomization)
		];
	}

	virtual bool CanChangeContinuously() const override { return true; }
	
protected:
	virtual float GetValue(int32 Index) const = 0;

	virtual void SetValue(int32 Index, float Value) = 0;

private:
	typedef SNumericVectorInputBox<float, UE::Math::TVector2<float>, 2> SNumericVectorInputBox2;
	typedef SNumericVectorInputBox<float, UE::Math::TVector<float>, 3> SNumericVectorInputBox3;
	typedef SNumericVectorInputBox<float, UE::Math::TVector4<float>, 4> SNumericVectorInputBox4;

	template<typename TArgs, typename TVector>
	TArgs CreateBaseArgs(EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization)
	{
		TOptional<TVector> MinValue;
		TOptional<TVector> MaxValue;
		if (WidgetCustomization.bHasMinValue)
		{
			MinValue = TVector(WidgetCustomization.MinValue);
		}
		if (WidgetCustomization.bHasMaxValue)
		{
			MaxValue = TVector(WidgetCustomization.MaxValue);
		}

		return TArgs()
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			.AllowSpin(true)
			.bColorAxisLabels(true)
			.SpinDelta(WidgetCustomization.bHasStepWidth ? WidgetCustomization.StepWidth : 0)
			.OnBeginSliderMovement(this, &SNiagaraVectorParameterEditorBase::BeginSliderMovement)
			.OnEndSliderMovement(this, &SNiagaraVectorParameterEditorBase::EndSliderMovement)
			.TypeInterface(MakeShareable(new TNumericUnitTypeInterface<float>(DisplayUnit)))
			.MinVector(MinValue)
			.MaxVector(MaxValue)
			.MinSliderVector(MinValue)
			.MaxSliderVector(MaxValue)
			.X(this, &SNiagaraVectorParameterEditorBase::GetValueInternal, 0)
			.OnXChanged(this, &SNiagaraVectorParameterEditorBase::ValueChanged, 0)
			.OnXCommitted(this, &SNiagaraVectorParameterEditorBase::ValueCommitted, 0)
			.Y(this, &SNiagaraVectorParameterEditorBase::GetValueInternal, 1)
			.OnYChanged(this, &SNiagaraVectorParameterEditorBase::ValueChanged, 1)
			.OnYCommitted(this, &SNiagaraVectorParameterEditorBase::ValueCommitted, 1);
	}

	template<typename TArguments>
	void SetZArgs(TArguments& Args)
	{
		Args.Z(this, &SNiagaraVectorParameterEditorBase::GetValueInternal, 2)
			.OnZChanged(this, &SNiagaraVectorParameterEditorBase::ValueChanged, 2)
			.OnZCommitted(this, &SNiagaraVectorParameterEditorBase::ValueCommitted, 2);
	}

	template<typename TArguments>
	void SetWArgs(TArguments& Args)
	{
		Args.W(this, &SNiagaraVectorParameterEditorBase::GetValueInternal, 3)
			.OnWChanged(this, &SNiagaraVectorParameterEditorBase::ValueChanged, 3)
			.OnWCommitted(this, &SNiagaraVectorParameterEditorBase::ValueCommitted, 3);
	}

	TSharedRef<SWidget> ConstructVectorWidget(int32 ChannelCount, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization)
	{
		if (ChannelCount == 2)
		{
			SNumericVectorInputBox2::FArguments Vector2Args = CreateBaseArgs<SNumericVectorInputBox2::FArguments, FVector2f>(DisplayUnit, WidgetCustomization);
			return SArgumentNew(Vector2Args, SNumericVectorInputBox2);
		}
		if (ChannelCount == 3)
		{
			SNumericVectorInputBox3::FArguments Vector3Args = CreateBaseArgs<SNumericVectorInputBox3::FArguments, FVector3f>(DisplayUnit, WidgetCustomization);
			SetZArgs(Vector3Args);
			return SArgumentNew(Vector3Args, SNumericVectorInputBox3);
		}
		if (ChannelCount == 4)
		{
			SNumericVectorInputBox4::FArguments Vector4Args = CreateBaseArgs<SNumericVectorInputBox4::FArguments, FVector4f>(DisplayUnit, WidgetCustomization);
			SetZArgs(Vector4Args);
			SetWArgs(Vector4Args);
			return SArgumentNew(Vector4Args, SNumericVectorInputBox4);
		}
		return SNullWidget::NullWidget;
	}

	void BeginSliderMovement()
	{
		ExecuteOnBeginValueChange();
	}

	void EndSliderMovement(float Value)
	{
		ExecuteOnEndValueChange();
	}

	TOptional<float> GetValueInternal(int32 Index) const
	{
		return TOptional<float>(GetValue(Index));
	}

	void ValueChanged(float Value, int32 Index)
	{
		SetValue(Index, Value);
		ExecuteOnValueChanged();
	}

	void ValueCommitted(float Value, ETextCommit::Type CommitInfo, int32 Index)
	{
		if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
		{
			ValueChanged(Value, Index);
		}
	}
};


class SNiagaraVector2ParameterEditor : public SNiagaraVectorParameterEditorBase
{
public:
	SLATE_BEGIN_ARGS(SNiagaraVector2ParameterEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization)
	{
		SNiagaraVectorParameterEditorBase::Construct(
			SNiagaraVectorParameterEditorBase::FArguments()
			.ComponentCount(2), DisplayUnit, WidgetCustomization);
	}

	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetVec2Struct(), TEXT("Struct type not supported."));
		VectorValue = *((FVector2f*)Struct->GetStructMemory());
	}

	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetVec2Struct(), TEXT("Struct type not supported."));
		*((FVector2f*)Struct->GetStructMemory()) = VectorValue;
	}

protected:
	virtual float GetValue(int32 Index) const override
	{
		return VectorValue[Index];
	}

	virtual void SetValue(int32 Index, float Value) override
	{
		VectorValue[Index] = Value;
	}

private:
	FVector2f VectorValue;
};

TSharedPtr<SNiagaraParameterEditor> FNiagaraEditorVector2TypeUtilities::CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const
{
	return SNew(SNiagaraVector2ParameterEditor, DisplayUnit, WidgetCustomization);
}

bool FNiagaraEditorVector2TypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorVector2TypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));

	// NOTE: We can not use ToString() here since the vector pin control doesn't use the standard 'X=0,Y=0' syntax. The string parsing breaks without the parentesis which are not added by default.
	FVector2f Value = AllocatedVariable.GetValue<FVector2f>();
	return FString::Printf(TEXT("(X=%3.3f, Y=%3.3f)"), Value.X, Value.Y);
}

bool FNiagaraEditorVector2TypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	FVector2f VectorValue = FVector2f::ZeroVector;
	if (VectorValue.InitFromString(StringValue) || !Variable.IsDataAllocated())
	{
		Variable.SetValue<FVector2f>(VectorValue);
		return true;
	}
	return false;
}

FText FNiagaraEditorVector2TypeUtilities::GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	return FText::FromString(GetPinDefaultStringFromValue(AllocatedVariable));
}

FText FNiagaraEditorVector2TypeUtilities::GetStackDisplayText(const FNiagaraVariable& Variable) const
{
	FVector2f Value = Variable.GetValue<FVector2f>();
	return FText::Format(FText::FromString("({0}, {1})"), Value.X, Value.Y);
}

class SNiagaraVector3ParameterEditor : public SNiagaraVectorParameterEditorBase
{
public:
	SLATE_BEGIN_ARGS(SNiagaraVector3ParameterEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization)
	{
		SNiagaraVectorParameterEditorBase::Construct(
			SNiagaraVectorParameterEditorBase::FArguments()
			.ComponentCount(3), DisplayUnit, WidgetCustomization);
	}

	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetVec3Struct() || Struct->GetStruct() == FNiagaraTypeDefinition::GetPositionStruct(), TEXT("Struct type not supported."));	// LWC_TODO: Support for FVector3d likely required here.
		VectorValue = *((FVector3f*)Struct->GetStructMemory());
	}

	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetVec3Struct() || Struct->GetStruct() == FNiagaraTypeDefinition::GetPositionStruct(), TEXT("Struct type not supported."));
		*((FVector3f*)Struct->GetStructMemory()) = VectorValue;
	}

protected:
	virtual float GetValue(int32 Index) const override
	{
		return VectorValue[Index];
	}

	virtual void SetValue(int32 Index, float Value) override
	{
		VectorValue[Index] = Value;
	}

private:
	FVector3f VectorValue;
};

TSharedPtr<SNiagaraParameterEditor> FNiagaraEditorVector3TypeUtilities::CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const
{
	return SNew(SNiagaraVector3ParameterEditor, DisplayUnit, WidgetCustomization);
}

bool FNiagaraEditorVector3TypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorVector3TypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));

	// NOTE: We can not use ToString() here since the vector pin control doesn't use the standard 'X=0,Y=0,Z=0' syntax.
	FVector3f Value = AllocatedVariable.GetValue<FVector3f>();
	return FString::Printf(TEXT("%3.3f,%3.3f,%3.3f"), Value.X, Value.Y, Value.Z);
}

bool FNiagaraEditorVector3TypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	// NOTE: We can not use InitFromString() here since the vector pin control doesn't use the standard 'X=0,Y=0,Z=0' syntax.
	FVector3f Value = FVector3f::ZeroVector;
	if (FDefaultValueHelper::ParseVector(StringValue, Value) || !Variable.IsDataAllocated())
	{
		Variable.SetValue<FVector3f>((FVector3f)Value);
		return true;
	}
	return false;
}

FText FNiagaraEditorVector3TypeUtilities::GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	return FText::FromString(GetPinDefaultStringFromValue(AllocatedVariable));
}

FText FNiagaraEditorVector3TypeUtilities::GetStackDisplayText(const FNiagaraVariable& Variable) const
{
	FVector3f Value = Variable.GetValue<FVector3f>();
	return FText::Format(FText::FromString("({0}, {1}, {2})"), Value.X, Value.Y, Value.Z);
}

class SNiagaraVector4ParameterEditor : public SNiagaraVectorParameterEditorBase
{
public:
	SLATE_BEGIN_ARGS(SNiagaraVector4ParameterEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization)
	{
		SNiagaraVectorParameterEditorBase::Construct(
			SNiagaraVectorParameterEditorBase::FArguments()
			.ComponentCount(4), DisplayUnit, WidgetCustomization);
	}

	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetVec4Struct(), TEXT("Struct type not supported."));
		VectorValue = *((FVector4f*)Struct->GetStructMemory());
	}

	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetVec4Struct(), TEXT("Struct type not supported."));
		*((FVector4f*)Struct->GetStructMemory()) = VectorValue;
	}

protected:
	virtual float GetValue(int32 Index) const override
	{
		return VectorValue[Index];
	}

	virtual void SetValue(int32 Index, float Value) override
	{
		VectorValue[Index] = Value;
	}

private:
	FVector4f VectorValue;
};

TSharedPtr<SNiagaraParameterEditor> FNiagaraEditorVector4TypeUtilities::CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const
{
	return SNew(SNiagaraVector4ParameterEditor, DisplayUnit, WidgetCustomization);
}

bool FNiagaraEditorVector4TypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorVector4TypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));
	// NOTE: We can not use ToString() here since the vector pin control doesn't use the standard 'X=0,Y=0,Z=0,W=0' syntax.
	FVector4f Value = AllocatedVariable.GetValue<FVector4f>();
	return FString::Printf(TEXT("%3.3f,%3.3f,%3.3f,%3.3f"), Value.X, Value.Y, Value.Z, Value.W);
}

bool FNiagaraEditorVector4TypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	// NOTE: We can not use InitFromString() here since the vector pin control doesn't use the standard 'X=0,Y=0,Z=0,W=0' syntax.
	FVector4f Value(0, 0, 0, 0);
	if (FDefaultValueHelper::ParseVector4(StringValue, Value) || !Variable.IsDataAllocated())
	{
		Variable.SetValue<FVector4f>(Value);
		return true;
	}
	return false;
}


FText FNiagaraEditorVector4TypeUtilities::GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	return FText::FromString(GetPinDefaultStringFromValue(AllocatedVariable));
}

FText FNiagaraEditorVector4TypeUtilities::GetStackDisplayText(const FNiagaraVariable& Variable) const
{
	FVector4f Value = Variable.GetValue<FVector4f>();
	return FText::Format(FText::FromString("({0}, {1}, {2}, {3})"), Value.X, Value.Y, Value.Z, Value.W);
}

class SNiagaraQuatParameterEditor : public SNiagaraVectorParameterEditorBase
{
public:
	SLATE_BEGIN_ARGS(SNiagaraQuatParameterEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization)
	{
		SNiagaraVectorParameterEditorBase::Construct(
			SNiagaraVectorParameterEditorBase::FArguments()
			.ComponentCount(4), DisplayUnit, WidgetCustomization);
	}

	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetQuatStruct(), TEXT("Struct type not supported."));
		VectorValue = *((FQuat4f*)Struct->GetStructMemory());
	}

	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) override
	{
		checkf(Struct->GetStruct() == FNiagaraTypeDefinition::GetQuatStruct(), TEXT("Struct type not supported."));
		*((FQuat4f*)Struct->GetStructMemory()) = VectorValue;
	}

protected:
	virtual float GetValue(int32 Index) const override
	{
		switch (Index)
		{
		case 0:
			return VectorValue.X;
		case 1:
			return VectorValue.Y;
		case 2:
			return VectorValue.Z;
		case 3:
			return VectorValue.W;
		}
		return 0.0f;
	}

	virtual void SetValue(int32 Index, float Value) override
	{
		switch (Index)
		{
		case 0:
			VectorValue.X = Value;
			break;
		case 1:
			VectorValue.Y = Value;
			break;
		case 2:
			VectorValue.Z = Value;
			break;
		case 3:
			VectorValue.W = Value;
			break;
		}
	}

private:
	FQuat4f VectorValue;
};

TSharedPtr<SNiagaraParameterEditor> FNiagaraEditorQuatTypeUtilities::CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const
{
	return SNew(SNiagaraQuatParameterEditor, DisplayUnit, WidgetCustomization);
}

bool FNiagaraEditorQuatTypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorQuatTypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));
	// NOTE: We can not use ToString() here since the vector pin control doesn't use the standard 'X=0,Y=0,Z=0,W=0' syntax.
	FQuat4f Value = AllocatedVariable.GetValue<FQuat4f>();
	return FString::Printf(TEXT("%3.3f,%3.3f,%3.3f,%3.3f"), Value.X, Value.Y, Value.Z, Value.W);
}

bool FNiagaraEditorQuatTypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	// NOTE: We can not use InitFromString() here since the vector pin control doesn't use the standard 'X=0,Y=0,Z=0,W=0' syntax.
	FVector4f Value(0, 0, 0, 0);
	if (FDefaultValueHelper::ParseVector4(StringValue, Value) || !Variable.IsDataAllocated())
	{
		FQuat4f Quat(Value.X, Value.Y, Value.Z, Value.W);
		Variable.SetValue<FQuat4f>(Quat);
		return true;
	}
	return false;
}

FText FNiagaraEditorQuatTypeUtilities::GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	return FText::FromString(GetPinDefaultStringFromValue(AllocatedVariable));
}

FText FNiagaraEditorQuatTypeUtilities::GetStackDisplayText(const FNiagaraVariable& Variable) const
{
	FQuat4f Value = Variable.GetValue<FQuat4f>();
	return FText::Format(FText::FromString("({0}, {1}, {2}, {3})"), Value.X, Value.Y, Value.Z, Value.W);
}

void FNiagaraEditorQuatTypeUtilities::UpdateVariableWithDefaultValue(FNiagaraVariable& Variable) const
{
	checkf(Variable.GetType().GetStruct() == FNiagaraTypeDefinition::GetQuatStruct(), TEXT("Struct type not supported."));
	Variable.SetValue<FQuat4f>(FQuat4f(0, 0, 0, 1));
}

bool FNiagaraEditorNiagaraIDTypeUtilities::CanHandlePinDefaults() const
{
	return true;
}

FString FNiagaraEditorNiagaraIDTypeUtilities::GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const
{
	checkf(AllocatedVariable.IsDataAllocated(), TEXT("Can not generate a default value string for an unallocated variable."));
	FNiagaraID Value = AllocatedVariable.GetValue<FNiagaraID>();
	return FString::Printf(TEXT("%i,%i"), Value.Index, Value.AcquireTag);
}

bool FNiagaraEditorNiagaraIDTypeUtilities::SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const
{
	if (StringValue == TEXT("0"))
	{
		// Special case handling of 0 default which is specified in niagara constants and is already present in assets.
		Variable.SetValue(FNiagaraID());
		return true;
	}
	else
	{
		TArray<FString> ValueParts;
		StringValue.ParseIntoArray(ValueParts, TEXT(","));
		if (ValueParts.Num() == 2)
		{
			int32 Index;
			int32 AcquireTag;
			if (LexTryParseString(Index, *ValueParts[0]) && LexTryParseString(AcquireTag, *ValueParts[1]))
			{
				FNiagaraID Value(Index, AcquireTag);
				Variable.SetValue(Value);
				return true;
			}
		}
	}
	return false;
}
