// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraVariableMetaData.h"

class SNiagaraParameterEditor;

/** Niagara editor utilities for the FVector2D type. */
class FNiagaraEditorVector2TypeUtilities : public FNiagaraEditorTypeUtilities
{
public:
	//~ INiagaraEditorTypeUtilities interface.
	virtual bool CanCreateParameterEditor() const override { return true; }
	virtual TSharedPtr<SNiagaraParameterEditor> CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const override;
	virtual bool CanHandlePinDefaults() const override;
	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const override;
	virtual FText GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual FText GetStackDisplayText(const FNiagaraVariable& Variable) const override;
};

/** Niagara editor utilities for the FVector type. */
class FNiagaraEditorVector3TypeUtilities : public FNiagaraEditorTypeUtilities
{
public:
	//~ INiagaraEditorTypeUtilities interface.
	virtual bool CanCreateParameterEditor() const override { return true; }
	virtual TSharedPtr<SNiagaraParameterEditor> CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const override;
	virtual bool CanHandlePinDefaults() const override;
	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const override;
	virtual FText GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual FText GetStackDisplayText(const FNiagaraVariable& Variable) const override;
};

/** Niagara editor utilities for the FVector4 type. */
class FNiagaraEditorVector4TypeUtilities : public FNiagaraEditorTypeUtilities
{
public:
	//~ INiagaraEditorTypeUtilities interface.
	virtual bool CanCreateParameterEditor() const override { return true; }
	virtual TSharedPtr<SNiagaraParameterEditor> CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const override;
	virtual bool CanHandlePinDefaults() const override;
	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const override;
	virtual FText GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual FText GetStackDisplayText(const FNiagaraVariable& Variable) const override;
};

/** Niagara editor utilities for the FVector4 type. */
class FNiagaraEditorQuatTypeUtilities : public FNiagaraEditorTypeUtilities
{
public:
	//~ INiagaraEditorTypeUtilities interface.
	virtual bool CanProvideDefaultValue() const override { return true; }
	virtual void UpdateVariableWithDefaultValue(FNiagaraVariable& Variable) const override;
	virtual bool CanCreateParameterEditor() const override { return true; }
	virtual TSharedPtr<SNiagaraParameterEditor> CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const override;
	virtual bool CanHandlePinDefaults() const override;
	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const override;
	virtual FText GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual FText GetStackDisplayText(const FNiagaraVariable& Variable) const override;
};

/** Niagara editor utilities for the FNiagaraID type. */
class FNiagaraEditorNiagaraIDTypeUtilities : public FNiagaraEditorTypeUtilities
{
public:
	//~ INiagaraEditorTypeUtilities interface.
	virtual bool CanHandlePinDefaults() const override;
	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const override;
	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const override;
};