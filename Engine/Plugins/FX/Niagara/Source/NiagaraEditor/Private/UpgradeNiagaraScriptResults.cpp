// Copyright Epic Games, Inc. All Rights Reserved.

#include "UpgradeNiagaraScriptResults.h"

#include "NiagaraClipboard.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraNodeFunctionCall.h"
#include "Logging/StructuredLog.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UpgradeNiagaraScriptResults)

namespace NiagaraScriptResults
{
	template<typename T>
	TArray<T*> GetStackEntries(UNiagaraStackViewModel* StackViewModel, bool bRefresh = false)
	{
		TArray<T*> Results;
		TArray<UNiagaraStackEntry*> EntriesToCheck;
		if (UNiagaraStackEntry* RootEntry = StackViewModel->GetRootEntry())
		{
			if (bRefresh)
			{
				RootEntry->RefreshChildren();
			}
			RootEntry->GetUnfilteredChildren(EntriesToCheck);
		}
		while (EntriesToCheck.Num() > 0)
		{
			UNiagaraStackEntry* Entry = EntriesToCheck.Pop();
			if (T* ItemToCheck = Cast<T>(Entry))
			{
				Results.Add(ItemToCheck);
			}
			Entry->GetUnfilteredChildren(EntriesToCheck);
		}
		return Results;
	}

	template<typename T>
	T GetValue(const UNiagaraClipboardFunctionInput* Input)
	{
		T Value;
		if (Input == nullptr || Input->InputType.GetSize() != sizeof(T) || Input->Local.Num() != sizeof(T))
		{
			FMemory::Memzero(Value);
		}
		else
		{
			FMemory::Memcpy(&Value, Input->Local.GetData(), sizeof(T));
		}
		return Value;
	}

	template<typename T>
	void SetValue(UNiagaraPythonScriptModuleInput* ModuleInput, T Data)
	{
		TArray<uint8> LocalData;
		LocalData.SetNumZeroed(sizeof(T));
		FMemory::Memcpy(LocalData.GetData(), &Data, sizeof(T));
		const UNiagaraClipboardFunctionInput* Input = ModuleInput->Input;
		ModuleInput->Input = UNiagaraClipboardFunctionInput::CreateLocalValue(ModuleInput, Input->InputName, Input->InputType, Input->bEditConditionValue, LocalData);
	}
}

bool UNiagaraPythonScriptModuleInput::IsSet() const
{
	return Input && Input->InputType.IsValid();
}

bool UNiagaraPythonScriptModuleInput::IsLocalValue() const
{
	return IsSet() && Input->ValueMode == ENiagaraClipboardFunctionInputValueMode::Local;
}

bool UNiagaraPythonScriptModuleInput::IsLinkedValue() const
{
	return IsSet() && Input->ValueMode == ENiagaraClipboardFunctionInputValueMode::Linked;
}

float UNiagaraPythonScriptModuleInput::AsFloat() const
{
	if (IsSet() && Input->InputType == FNiagaraTypeDefinition::GetFloatDef())
	{
		FNiagaraVariable LocalInput;
		return NiagaraScriptResults::GetValue<float>(Input);
	}
	return 0;
}

int32 UNiagaraPythonScriptModuleInput::AsInt() const
{
	if (IsSet() && Input->InputType == FNiagaraTypeDefinition::GetIntDef())
	{
		return NiagaraScriptResults::GetValue<int32>(Input);
	}
	if (IsSet() && Input->InputType.IsEnum())
	{
		return NiagaraScriptResults::GetValue<int32>(Input);
	}
	return 0;
}

bool UNiagaraPythonScriptModuleInput::AsBool() const
{
	if (IsSet() && Input->InputType == FNiagaraTypeDefinition::GetBoolDef())
	{
		if (Input->Local.Num() != sizeof(FNiagaraBool))
		{
			return false;
		}
		const FNiagaraBool* BoolStruct = reinterpret_cast<const FNiagaraBool*>(Input->Local.GetData());
		return BoolStruct->GetValue();
	}
	return false;
}

FVector2D UNiagaraPythonScriptModuleInput::AsVec2() const
{
	if (IsSet() && Input->InputType == FNiagaraTypeDefinition::GetVec2Def())
	{
		return NiagaraScriptResults::GetValue<FVector2D>(Input);
	}
	return FVector2D();
}

FVector UNiagaraPythonScriptModuleInput::AsVec3() const
{
	if (IsSet() && Input->InputType == FNiagaraTypeDefinition::GetVec3Def())
	{
		return FVector(NiagaraScriptResults::GetValue<FVector3f>(Input));
	}
	return FVector();
}

FVector4 UNiagaraPythonScriptModuleInput::AsVec4() const
{
	if (IsSet() && Input->InputType == FNiagaraTypeDefinition::GetVec4Def())
	{
		return FVector4(NiagaraScriptResults::GetValue<FVector4f>(Input));
	}
	return FVector4();
}

FLinearColor UNiagaraPythonScriptModuleInput::AsColor() const
{
	if (IsSet() && Input->InputType == FNiagaraTypeDefinition::GetColorDef())
	{
		return NiagaraScriptResults::GetValue<FLinearColor>(Input);
	}
	return FLinearColor();
}

FQuat UNiagaraPythonScriptModuleInput::AsQuat() const
{
	if (IsSet() && Input->InputType == FNiagaraTypeDefinition::GetQuatDef())
	{
		return NiagaraScriptResults::GetValue<FQuat>(Input);
	}
	return FQuat();
}

FString UNiagaraPythonScriptModuleInput::AsEnum() const
{
	if (IsSet() && Input->InputType.IsEnum())
	{
		int32 Value = NiagaraScriptResults::GetValue<int32>(Input);
		return Input->InputType.GetEnum()->GetNameStringByValue(Value);
	}
	return FString();
}

FString UNiagaraPythonScriptModuleInput::AsLinkedValue() const
{
	if (IsLinkedValue())
	{
		return Input->Linked.GetName().ToString();
	}
	return FString();
}

UUpgradeNiagaraScriptResults::UUpgradeNiagaraScriptResults()
{
	DummyInput = NewObject<UNiagaraPythonScriptModuleInput>();
}

void UUpgradeNiagaraScriptResults::Init()
{
	// if some of the old inputs are missing in the new inputs we still bring them over, otherwise they can't be set from the script
	for (UNiagaraPythonScriptModuleInput* OldInput : OldInputs)
	{
		UNiagaraPythonScriptModuleInput* NewInput = GetNewInput(OldInput->Input->InputName);
		if (NewInput == nullptr)
		{
			UNiagaraPythonScriptModuleInput* ScriptInput = NewObject<UNiagaraPythonScriptModuleInput>();
			ScriptInput->Input = OldInput->Input;
			NewInputs.Add(ScriptInput);
		}
	}
}

UNiagaraPythonScriptModuleInput* UUpgradeNiagaraScriptResults::GetOldInput(const FString& InputName)
{
	for (UNiagaraPythonScriptModuleInput* ModuleInput : OldInputs)
	{
		if (ModuleInput->Input->InputName == FName(InputName))
		{
			return ModuleInput;
		}
	}
	return DummyInput;
}

void UUpgradeNiagaraScriptResults::SetFloatInput(const FString& InputName, float Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType == FNiagaraTypeDefinition::GetFloatDef())
	{
		NiagaraScriptResults::SetValue(ModuleInput, Value);
	}
}

void UUpgradeNiagaraScriptResults::SetIntInput(const FString& InputName, int32 Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType == FNiagaraTypeDefinition::GetIntDef())
	{
		NiagaraScriptResults::SetValue(ModuleInput, Value);
	}
}

void UUpgradeNiagaraScriptResults::SetBoolInput(const FString& InputName, bool Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType == FNiagaraTypeDefinition::GetBoolDef())
	{
		ModuleInput->Input = UNiagaraClipboardEditorScriptingUtilities::CreateBoolLocalValueInput(ModuleInput, FName(InputName), false, false, Value);
	}
}

void UUpgradeNiagaraScriptResults::SetVec2Input(const FString& InputName, FVector2D Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType == FNiagaraTypeDefinition::GetVec2Def())
	{
		NiagaraScriptResults::SetValue(ModuleInput, FVector2f(Value));
	}
}

void UUpgradeNiagaraScriptResults::SetVec3Input(const FString& InputName, FVector Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType == FNiagaraTypeDefinition::GetVec3Def())
	{
		NiagaraScriptResults::SetValue(ModuleInput, FVector3f(Value));
	}
}

void UUpgradeNiagaraScriptResults::SetVec4Input(const FString& InputName, FVector4 Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType == FNiagaraTypeDefinition::GetVec4Def())
	{
		NiagaraScriptResults::SetValue(ModuleInput, FVector4f(Value));
	}
}

void UUpgradeNiagaraScriptResults::SetColorInput(const FString& InputName, FLinearColor Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType == FNiagaraTypeDefinition::GetColorDef())
	{
		NiagaraScriptResults::SetValue(ModuleInput, Value);
	}
}

void UUpgradeNiagaraScriptResults::SetQuatInput(const FString& InputName, FQuat Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType == FNiagaraTypeDefinition::GetQuatDef())
	{
		NiagaraScriptResults::SetValue(ModuleInput, FQuat4f(Value));
	}
}

void UUpgradeNiagaraScriptResults::SetEnumInput(const FString& InputName, FString Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType.IsEnum())
	{
		int32 EnumValue = ModuleInput->Input->InputType.GetEnum()->GetValueByNameString(Value, EGetByNameFlags::ErrorIfNotFound | EGetByNameFlags::CheckAuthoredName);
		NiagaraScriptResults::SetValue(ModuleInput, EnumValue);
	}
}

void UUpgradeNiagaraScriptResults::SetEnumInputFromInt(const FString& InputName, int32 Value)
{
	UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName));
	if (ModuleInput && ModuleInput->Input->InputType.IsEnum())
	{
		if (!ModuleInput->Input->InputType.GetEnum()->IsValidEnumValue(Value))
		{
			UE_LOGFMT(LogNiagaraEditor, Error, "Value {Value} is not a valid enum value for input {InputName}", Value, InputName);
		}
		NiagaraScriptResults::SetValue(ModuleInput, Value);
	}
}
void UUpgradeNiagaraScriptResults::SetLinkedInput(const FString& InputName, FString Value)
{
	if (UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName)))
	{
		const UNiagaraClipboardFunctionInput* CurrentInput = ModuleInput->Input;
		FNiagaraVariableBase LinkedParameter = FNiagaraVariableBase(CurrentInput->InputType, *Value);
		ModuleInput->Input = UNiagaraClipboardFunctionInput::CreateLinkedValue(ModuleInput, CurrentInput->InputName, CurrentInput->InputType, CurrentInput->bEditConditionValue, LinkedParameter);
	}
}

void UUpgradeNiagaraScriptResults::SetNewInput(const FString& InputName, UNiagaraPythonScriptModuleInput* Value)
{
	for (int i = 0; i < NewInputs.Num(); i++)
	{
		UNiagaraPythonScriptModuleInput* ModuleInput = NewInputs[i];
		UNiagaraClipboardFunctionInput* FunctionInput = const_cast<UNiagaraClipboardFunctionInput*>(ModuleInput->Input.Get());
		if (Value && FunctionInput && FunctionInput->InputName == InputName )
		{
			if (Value->IsSet() && FunctionInput->InputType == Value->Input->InputType)
			{
				FunctionInput->Data = Value->Input->Data;
				FunctionInput->Dynamic = Value->Input->Dynamic;
				FunctionInput->Expression = Value->Input->Expression;
				FunctionInput->Linked = Value->Input->Linked;
				FunctionInput->Local = Value->Input->Local;
				FunctionInput->ValueMode = Value->Input->ValueMode;
			}
			else
			{
				ResetToDefault(InputName);
			}
			return;
		}
	}
}

void UUpgradeNiagaraScriptResults::ResetToDefault(const FString& InputName)
{
	if (UNiagaraPythonScriptModuleInput* ModuleInput = GetNewInput(FName(InputName)))
	{
		const UNiagaraClipboardFunctionInput* CurrentInput = ModuleInput->Input;
		ModuleInput->Input = UNiagaraClipboardFunctionInput::CreateDefaultInputValue(ModuleInput, CurrentInput->InputName, CurrentInput->InputType);
	}
}

TArray<UNiagaraClipboardFunctionInput*> UUpgradeNiagaraScriptResults::CopyOldInputs() const
{
	return CopyInputs(OldInputs);
}

TArray<UNiagaraClipboardFunctionInput*> UUpgradeNiagaraScriptResults::CopyNewInputs() const
{
	return CopyInputs(NewInputs);
}

TArray<UNiagaraClipboardFunctionInput*> UUpgradeNiagaraScriptResults::CopyInputs(const TArray<TObjectPtr<UNiagaraPythonScriptModuleInput>>& Source) const
{
	TArray<UNiagaraClipboardFunctionInput*> Result;
	for (const UNiagaraPythonScriptModuleInput* Input : Source)
	{
		if (Input->Input)
		{
			if (UNiagaraClipboardFunctionInput* NewInput = Cast<UNiagaraClipboardFunctionInput>(StaticDuplicateObject(Input->Input, GetTransientPackage())))
			{
				NewInput->CopyValuesFrom(Input->Input);
				Result.Add(NewInput);
			}
		}
	}
	return Result;
}

UNiagaraPythonScriptModuleInput* UUpgradeNiagaraScriptResults::GetNewInput(const FName& InputName) const
{
	for (UNiagaraPythonScriptModuleInput* ModuleInput : NewInputs)
	{
		if (ModuleInput->Input && ModuleInput->Input->InputName == InputName)
		{
			return ModuleInput;
		}
	}
	return nullptr;
}

void UNiagaraPythonModule::Init(UNiagaraStackModuleItem* InModuleItem)
{
	ModuleItem = InModuleItem;
}

UNiagaraStackModuleItem* UNiagaraPythonModule::GetObject() const
{
	return ModuleItem;
}

void UNiagaraPythonEmitter::Init(TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterViewModel)
{
	EmitterViewModel = InEmitterViewModel;
}

UNiagaraEmitter* UNiagaraPythonEmitter::GetObject()
{
	return EmitterViewModel->GetEmitterViewModel()->GetEmitter().Emitter;
}

FVersionedNiagaraEmitterData UNiagaraPythonEmitter::GetProperties() const
{
	if (EmitterViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData())
	{
		return *EmitterViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData();
	}
	return FVersionedNiagaraEmitterData();
}

void UNiagaraPythonEmitter::SetProperties(FVersionedNiagaraEmitterData Data)
{
	if (EmitterViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData())
	{
		FVersionedNiagaraEmitterData* EmitterData = EmitterViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData();
		for (TFieldIterator<FProperty> PropertyIterator(FVersionedNiagaraEmitterData::StaticStruct()); PropertyIterator; ++PropertyIterator)
		{
			if (PropertyIterator->HasAllPropertyFlags(CPF_Edit))
			{
				void* Dest = PropertyIterator->ContainerPtrToValuePtr<void>(EmitterData);
				void* Src = PropertyIterator->ContainerPtrToValuePtr<void>(&Data);
				PropertyIterator->CopyCompleteValue(Dest, Src);
			}
		}
	}
}

TArray<UNiagaraPythonModule*> UNiagaraPythonEmitter::GetModules() const
{
	TArray<UNiagaraPythonModule*> Result;

	if (UNiagaraStackViewModel* StackViewModel = EmitterViewModel->GetEmitterStackViewModel())
	{
		TArray<UNiagaraStackModuleItem*> StackModuleItems =	NiagaraScriptResults::GetStackEntries<UNiagaraStackModuleItem>(StackViewModel);
		for (UNiagaraStackModuleItem* ModuleItem : StackModuleItems)
		{
			UNiagaraPythonModule* Module = NewObject<UNiagaraPythonModule>();
			Module->Init(ModuleItem);
			Result.Add(Module);
		}
	}

	return Result;
}

bool UNiagaraPythonEmitter::HasModule(const FString& ModuleName) const
{
	return GetModule(ModuleName)->GetObject() != nullptr;
}

UNiagaraPythonModule* UNiagaraPythonEmitter::GetModule(const FString& ModuleName) const
{
	UNiagaraPythonModule* Module = NewObject<UNiagaraPythonModule>();
	if (UNiagaraStackViewModel* StackViewModel = EmitterViewModel->GetEmitterStackViewModel())
	{
		TArray<UNiagaraStackModuleItem*> StackModuleItems =	NiagaraScriptResults::GetStackEntries<UNiagaraStackModuleItem>(StackViewModel);
		for (UNiagaraStackModuleItem* ModuleItem : StackModuleItems)
		{
			if (ModuleItem->GetModuleNode().GetFunctionName() == ModuleName)
			{
				Module->Init(ModuleItem);
				break;
			}
		}
	}
	return Module;
}

void UUpgradeNiagaraEmitterContext::Init(UNiagaraPythonEmitter* InOldEmitter, UNiagaraPythonEmitter* InNewEmitter)
{
	OldEmitter = InOldEmitter;
	NewEmitter = InNewEmitter;

	UpgradeVersionData.Empty();

	if (!IsValid())
	{
		return;
	}
	
	FVersionedNiagaraEmitterData* SourceData = OldEmitter->EmitterViewModel->GetEmitterHandle()->GetInstance().GetEmitterData();
	FVersionedNiagaraEmitterData* TargetData = NewEmitter->EmitterViewModel->GetEmitterHandle()->GetInstance().GetEmitterData();
	FVersionedNiagaraEmitter SourceParent = SourceData->GetParent();
	FVersionedNiagaraEmitter TargetParent = TargetData->GetParent();
	FNiagaraAssetVersion SourceVersion = SourceParent.GetEmitterData()->Version;
	FNiagaraAssetVersion TargetVersion = TargetParent.GetEmitterData()->Version;

	// gather script required to execute
	for (const FNiagaraAssetVersion& Version : SourceParent.Emitter->GetAllAvailableVersions())
	{
		if (SourceVersion <= Version && Version <= TargetVersion)
		{
			FVersionedNiagaraEmitterData* ParentData = SourceParent.Emitter->GetEmitterData(Version.VersionGuid);
			UpgradeVersionData.Add(ParentData);
		}
	}
}

bool UUpgradeNiagaraEmitterContext::IsValid() const
{
	return OldEmitter && OldEmitter->IsValid() && NewEmitter && NewEmitter->IsValid();
}

const TArray<FVersionedNiagaraEmitterData*>& UUpgradeNiagaraEmitterContext::GetUpgradeData() const
{
	return UpgradeVersionData;
}



