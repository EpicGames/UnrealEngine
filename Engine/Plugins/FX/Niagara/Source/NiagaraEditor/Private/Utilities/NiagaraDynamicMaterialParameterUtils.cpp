// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDynamicMaterialParameterUtils.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionDynamicParameter.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraEmitterInstance.h"

TArray<UMaterial*> NiagaraDynamicMaterialParameterUtils::GetMaterialsFromRenderers(TArrayView<UNiagaraRendererProperties* const> Renderers, const FNiagaraEmitterInstance* EmitterInstance)
{
	TArray<UMaterial*> ResultMaterials;
	for (UNiagaraRendererProperties* RenderProperties : Renderers)
	{
		TArray<UMaterialInterface*> UsedMaterialInterfaces;
		RenderProperties->GetUsedMaterials(EmitterInstance, UsedMaterialInterfaces);
		for (UMaterialInterface* UsedMaterialInterface : UsedMaterialInterfaces)
		{
			if (UsedMaterialInterface != nullptr)
			{
				UMaterial* UsedMaterial = UsedMaterialInterface->GetBaseMaterial();
				if (UsedMaterial != nullptr)
				{
					ResultMaterials.AddUnique(UsedMaterial);
					break;
				}
			}
		}
	}
	return ResultMaterials;
}

TOptional<float> NiagaraDynamicMaterialParameterUtils::GetDynamicParameterDefaultValue(TArrayView<UMaterial* const> Materials, int32 InParameterIndex, int32 InParameterChannel)
{
	check(InParameterChannel >= 0 && InParameterChannel < 4);

	for (UMaterial* Material : Materials)
	{
		if (Material == nullptr)
		{
			continue;
		}

		TArray<UMaterialExpression*> Expressions;
		Material->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpression>(Expressions);
		for (UMaterialExpression* Expression : Expressions)
		{
			const UMaterialExpressionDynamicParameter* DynamicParameterExpression = Cast<UMaterialExpressionDynamicParameter>(Expression);
			if (DynamicParameterExpression == nullptr || DynamicParameterExpression->ParameterIndex != static_cast<uint32>(InParameterIndex))
			{
				continue;
			}

			return DynamicParameterExpression->DefaultValue.Component(InParameterChannel);
		}
	}

	return TOptional<float>();
}
