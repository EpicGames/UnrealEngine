// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UMaterial;
class UNiagaraRendererProperties;
class FNiagaraEmitterInstance;

namespace NiagaraDynamicMaterialParameterUtils
{
	/**
	 * Collects unique base UMaterial* objects from an array of renderer properties.
	 * Materials are resolved via UMaterialInterface::GetBaseMaterial and deduplicated.
	 */
	TArray<UMaterial*> GetMaterialsFromRenderers(TArrayView<UNiagaraRendererProperties* const> Renderers, const FNiagaraEmitterInstance* EmitterInstance);

	/**
	 * Searches the given materials for a UMaterialExpressionDynamicParameter node
	 * whose ParameterIndex matches InParameterIndex, and returns the default value
	 * for InParameterChannel (0=R, 1=G, 2=B, 3=A) from the node's DefaultValue.
	 */
	TOptional<float> GetDynamicParameterDefaultValue(TArrayView<UMaterial* const> Materials, int32 InParameterIndex, int32 InParameterChannel);
}
