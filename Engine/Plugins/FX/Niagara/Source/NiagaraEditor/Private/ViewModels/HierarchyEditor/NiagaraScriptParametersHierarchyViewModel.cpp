// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"

#include "NiagaraConstants.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptSource.h"
#include "ScopedTransaction.h"
#include "Logging/StructuredLog.h"
#include "ViewModels/NiagaraScriptViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScriptParametersHierarchyViewModel)

#define LOCTEXT_NAMESPACE "NiagaraScriptParameterHierarchyEditor"

void UNiagaraHierarchyScriptParameter::Initialize(const UNiagaraScriptVariable& InParameterScriptVariable)
{
	if(InParameterScriptVariable.Metadata.GetVariableGuid().IsValid() == false)
	{
		UE_LOGFMT(LogNiagaraEditor, Warning, "Invalid hierarchy script parameter initialization. No valid guid found. This hierarchy element will be deleted on next refresh.");
	}
	
	SetIdentity(FHierarchyElementIdentity({InParameterScriptVariable.Metadata.GetVariableGuid()}, {}));
}

bool UNiagaraHierarchyScriptParameter::IsValid() const
{
	return GetScriptVariable() != nullptr;
}

FString UNiagaraHierarchyScriptParameter::ToString() const
{
	TOptional<FNiagaraVariable> Variable = GetVariable();

	if(Variable.IsSet())
	{
		return Variable.GetValue().GetName().ToString();
	}
	
	return TEXT("Invalid"); 
}

FText UNiagaraHierarchyScriptParameter::GetTooltip() const
{
	if(UNiagaraScriptVariable* ScriptVariable = GetScriptVariable())
	{
		return ScriptVariable->Metadata.Description; 
	}

	return FText::GetEmpty();
}

UNiagaraScriptVariable* UNiagaraHierarchyScriptParameter::GetScriptVariable() const
{
	UNiagaraGraph* OwningGraph = GetTypedOuter<UNiagaraGraph>();
	if(ensureMsgf(OwningGraph != nullptr, TEXT("When retrieving the script variable matching this hierarchy parameter, there should always be an owning graph")))
	{
		// If the script variable does not exist, this is an indicator it has been deleted and this hierarchy parameter should be deleted
		return OwningGraph->GetScriptVariable(GetPersistentIdentity().Guids[0]);
	}

	return nullptr;
}

TOptional<FNiagaraVariable> UNiagaraHierarchyScriptParameter::GetVariable() const
{
	if(UNiagaraScriptVariable* ScriptVariable = GetScriptVariable())
	{
		return ScriptVariable->Variable;
	}
	
	return TOptional<FNiagaraVariable>(); 
}

void UNiagaraScriptParametersHierarchyViewModel::Initialize(TSharedRef<FNiagaraScriptViewModel> InScriptViewModel)
{
	ScriptViewModelWeak = InScriptViewModel;

	Cast<UNiagaraScriptSource>(ScriptViewModelWeak.Pin()->GetStandaloneScript().GetScriptData()->GetSource())->NodeGraph->OnParametersChanged().AddUObject(this, &UNiagaraScriptParametersHierarchyViewModel::OnParametersChanged);
	UDataHierarchyViewModelBase::Initialize();

	UNiagaraHierarchyScriptParameterRefreshContext* ScriptParameterRefreshContext = NewObject<UNiagaraHierarchyScriptParameterRefreshContext>(this);
	ScriptParameterRefreshContext->SetNiagaraGraph(Cast<UNiagaraScriptSource>(InScriptViewModel->GetStandaloneScript().GetScriptData()->GetSource())->NodeGraph);
	SetRefreshContext(ScriptParameterRefreshContext);
}

TSharedRef<FNiagaraScriptViewModel> UNiagaraScriptParametersHierarchyViewModel::GetScriptViewModel() const
{
	TSharedPtr<FNiagaraScriptViewModel> ScriptViewModelPinned = ScriptViewModelWeak.Pin();
	checkf(ScriptViewModelPinned.IsValid(), TEXT("Script view model destroyed before parameters hierarchy view model."));
	return ScriptViewModelPinned.ToSharedRef();
}


UHierarchyRoot* UNiagaraScriptParametersHierarchyViewModel::GetHierarchyRoot() const
{
	const TArray<FVersionedNiagaraScriptWeakPtr>& Scripts = GetScriptViewModel()->GetScripts();
	if(ensure(Scripts.Num() >= 1 && Scripts[0].Pin().Script != nullptr) == false)
	{
		return nullptr;
	}

	FVersionedNiagaraScriptData* ScriptData = Scripts[0].Pin().GetScriptData();
	if(ensure(ScriptData != nullptr) == false)
	{
		return nullptr;
	}

	return Cast<UNiagaraScriptSource>(ScriptData->GetSource())->NodeGraph->GetScriptParameterHierarchyRoot();
}

UObject* UNiagaraScriptParametersHierarchyViewModel::GetOuterForSourceRoot() const
{
	// The graph always needs to be valid as the outer
	if(UNiagaraGraph* Graph = Cast<UNiagaraScriptSource>(ScriptViewModelWeak.Pin()->GetStandaloneScript().GetScriptData()->GetSource())->NodeGraph)
	{
		return Graph;
	}

	checkNoEntry();
	return nullptr;
}

TSubclassOf<UHierarchyCategory> UNiagaraScriptParametersHierarchyViewModel::GetCategoryDataClass() const
{
	return UNiagaraHierarchyScriptCategory::StaticClass();
}

TSharedPtr<FHierarchyElementViewModel> UNiagaraScriptParametersHierarchyViewModel::CreateCustomViewModelForElement(UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> Parent)
{
	if(UNiagaraHierarchyScriptParameter* Item = Cast<UNiagaraHierarchyScriptParameter>(Element))
	{
		return MakeShared<FNiagaraHierarchyScriptParameterViewModel>(Item, Parent.ToSharedRef(), this);
	}
	if(UHierarchyCategory* Category = Cast<UHierarchyCategory>(Element))
	{
		return MakeShared<FNiagaraHierarchyScriptCategoryViewModel>(Category, Parent.ToSharedRef(), this);
	}
	else if(UHierarchyRoot* Root = Cast<UHierarchyRoot>(Element))
	{
		// If the root is the hierarchy root, we know it's for the hierarchy. If not, it's the transient source root
		bool bIsForHierarchy = GetHierarchyRoot() == Element;
		return MakeShared<FNiagaraHierarchyScriptRootViewModel>(Root, this, bIsForHierarchy);
	}

	return nullptr;
}

void UNiagaraScriptParametersHierarchyViewModel::PrepareSourceItems(UHierarchyRoot* SourceRoot, TSharedPtr<FHierarchyRootViewModel> SourceRootViewModel)
{
	UNiagaraScriptSourceBase* SourceBase = GetScriptViewModel()->GetStandaloneScript().GetScriptData()->GetSource();
	UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(SourceBase);
	const TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>& ScriptVariableMap = ScriptSource->NodeGraph->GetAllMetaData();

	for(const auto& ScriptVariablePair : ScriptVariableMap)
	{
		// We only want to be able to organize module inputs & static switches 
		if(ScriptVariablePair.Key.IsInNameSpace(FNiagaraConstants::ModuleNamespace) == false && ScriptVariablePair.Value->GetIsStaticSwitch() == false)
		{
			continue;
		}
		
		UNiagaraScriptVariable* ScriptVariable = ScriptVariablePair.Value;
		
		UNiagaraHierarchyScriptParameter* ScriptParameterHierarchyObject = NewObject<UNiagaraHierarchyScriptParameter>(SourceRoot, NAME_None, RF_Transient);
		ScriptParameterHierarchyObject->Initialize(*ScriptVariable);
		SourceRoot->GetChildrenMutable().Add(ScriptParameterHierarchyObject);
	}
}

void UNiagaraScriptParametersHierarchyViewModel::SetupCommands()
{
	Super::SetupCommands();
}

TSharedRef<FHierarchyDragDropOp> UNiagaraScriptParametersHierarchyViewModel::CreateDragDropOp(const FDataHierarchyDragDropContext& DragContext)
{
	TSharedRef<FNiagaraHierarchyScriptParameterDragDropOp> DragDropOp = MakeShared<FNiagaraHierarchyScriptParameterDragDropOp>(DragContext);
	DragDropOp->Construct();
	return DragDropOp;
}

void UNiagaraScriptParametersHierarchyViewModel::FinalizeInternal()
{
	if(ScriptViewModelWeak.IsValid())
	{
		// If this is called during Undo, it's possible the Graph does no longer exist
		if(UNiagaraGraph* Graph = Cast<UNiagaraScriptSource>(ScriptViewModelWeak.Pin()->GetStandaloneScript().GetScriptData()->GetSource())->NodeGraph)
		{
			Graph->OnParametersChanged().RemoveAll(this);
		}
	}
}

void UNiagaraScriptParametersHierarchyViewModel::OnParametersChanged(TOptional<TInstancedStruct<FNiagaraParametersChangedData>> ParametersChangedData)
{
	if(ParametersChangedData.IsSet())
	{
		TInstancedStruct<FNiagaraParametersChangedData> ChangedData = ParametersChangedData.GetValue();
		if(ChangedData.GetScriptStruct() == FNiagaraParameterRenamedData::StaticStruct())
		{
			const FNiagaraParameterRenamedData& RenamedData = ChangedData.Get<FNiagaraParameterRenamedData>();
			const UNiagaraScriptVariable* OldScriptVariable = RenamedData.OldScriptVariable.Get();
			const UNiagaraScriptVariable* NewScriptVariable = RenamedData.NewScriptVariable.Get();
			if (ensureMsgf(OldScriptVariable && NewScriptVariable, TEXT("UNiagaraScriptParametersHierarchyViewModel::OnParametersChanged - OldVariable(%p) NewVariable(%p)"), OldScriptVariable, NewScriptVariable))
			{
				if(OldScriptVariable->GetIsStaticSwitch() && NewScriptVariable->GetIsStaticSwitch())
				{
					TArray<UNiagaraHierarchyScriptParameter*> AllHierarchyScriptParameters;
					HierarchyRoot->GetChildrenOfType<UNiagaraHierarchyScriptParameter>(AllHierarchyScriptParameters, true);

					UNiagaraHierarchyScriptParameter** FoundHierarchyScriptParameter = AllHierarchyScriptParameters.FindByPredicate([Guid = OldScriptVariable->Metadata.GetVariableGuid()](UNiagaraHierarchyScriptParameter* Candidate)
					{
						return Candidate->GetPersistentIdentity().Guids[0] == Guid;
					});

					if(FoundHierarchyScriptParameter)
					{
						HierarchyRoot->Modify();
						(*FoundHierarchyScriptParameter)->Initialize(*NewScriptVariable);
					}
				}
			}
		}
	}
	ForceFullRefresh();
}

bool UNiagaraScriptParametersHierarchyViewModel::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	return Super::MatchesContext(InContext, TransactionObjectContexts) || TransactionObjectContexts.ContainsByPredicate([](TPair<UObject*, FTransactionObjectEvent> ContextPair)
	{
		return ContextPair.Key->IsA<UNiagaraGraph>() || ContextPair.Key->IsA<UNiagaraScriptVariable>();
	});
}

TSharedRef<SWidget> FNiagaraHierarchyScriptParameterDragDropOp::CreateCustomDecorator() const
{
	TArray<FDraggedElementEntry> TopLevelEntries = GetDragContext().GetTopLevelEntries();

	if(TopLevelEntries.Num() == 1 && TopLevelEntries[0].Element.IsValid())
	{
		if(const UNiagaraHierarchyScriptParameter* ScriptParameter = Cast<UNiagaraHierarchyScriptParameter>(TopLevelEntries[0].Element.Pin()->GetData()))
		{
			TOptional<FNiagaraVariable> Variable = ScriptParameter->GetVariable();
			if(Variable.IsSet())
			{
				return FNiagaraParameterUtilities::GetParameterWidget(Variable.GetValue(), false, false);
			}
		}
	}

	TSharedRef<SVerticalBox> MultiDecorator = SNew(SVerticalBox);
	for(const FDraggedElementEntry& Entry : TopLevelEntries)
	{
		if(Entry.Element.IsValid())
		{
			if(const UNiagaraHierarchyScriptParameter* ScriptParameter = Cast<UNiagaraHierarchyScriptParameter>(Entry.Element.Pin()->GetData()))
			{
				TOptional<FNiagaraVariable> Variable = ScriptParameter->GetVariable();
				if(Variable.IsSet())
				{
					MultiDecorator->AddSlot()
					.AutoHeight()
					.Padding(1.f)
					[
						FNiagaraParameterUtilities::GetParameterWidget(Variable.GetValue(), false, false)
					];
				}
			}
		}
	}

	return MultiDecorator;
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchyScriptCategoryViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone)
{
	if(IsEditableByUser().bResult == false)
	{
		return false;
	}
	
	FResultWithUserFeedback CanPerformActionResults = FHierarchyCategoryViewModel::CanDropOnInternal(DraggedElement, ItemDropZone);

	// We allow dropping parameters above/below categories
	if(ItemDropZone != EItemDropZone::OntoItem && DraggedElement->GetData()->IsA<UNiagaraHierarchyScriptParameter>())
	{
		CanPerformActionResults.UserFeedback = FText::GetEmpty();
		CanPerformActionResults.bResult = true;
	}

	return CanPerformActionResults;
}

bool FNiagaraHierarchyScriptParameterViewModel::DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const
{
	// GetScriptVariable checks the owning graph if it still exists
	const UNiagaraScriptVariable* ScriptVariable = Cast<UNiagaraHierarchyScriptParameter>(GetData())->GetScriptVariable();
	
	if(ScriptVariable == nullptr)
	{
		return false;
	}	

	// We make sure that the variable not only still exists but also qualifies for the hierarchy (namespace can change for example)
	if(ScriptVariable->GetIsStaticSwitch() == false && ScriptVariable->Variable.IsInNameSpace(FNiagaraConstants::ModuleNamespace) == false)
	{
		return false;
	}
	
	return true;
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchyScriptParameterViewModel::CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType)
{
	// We support nested inputs up to 1 layer deep
	if(InHierarchyElementType->IsChildOf<UNiagaraHierarchyScriptParameter>())
	{
		if(Parent.IsValid() && Parent.Pin()->GetData()->IsA<UNiagaraHierarchyScriptParameter>())
		{
			FResultWithUserFeedback CanContainResult(false);
			FText BaseMessage = LOCTEXT("InputOnInputNestedChildTooDeep", "Can't nest new input {0} under input {1}.\nChildren inputs can only have one layer of depth! Add it to the parent input!");
			CanContainResult.UserFeedback = FText::FormatOrdered(BaseMessage, InHierarchyElementType->GetDisplayNameText(), ToStringAsText());
			return CanContainResult;
		}
				
		return true;
	}
	
	return false;
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchyScriptParameterViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedItem, EItemDropZone ItemDropZone)
{
	// if the input isn't editable, we don't allow any drops on/above/below the item.
	// Even though it technically works, the merge process will only re-add the item at the end and not preserve order so there is no point in allowing dropping above/below
	if(IsEditableByUser().bResult == false)
	{
		return false;
	}

	FResultWithUserFeedback AllowDrop(false);
	
	TSharedPtr<FHierarchyElementViewModel> TargetDropItem = AsShared();

	// we only allow drops if some general conditions are fulfilled
	if(DraggedItem->GetData() != TargetDropItem->GetData() &&
		(!DraggedItem->HasParent(TargetDropItem, false) || ItemDropZone != EItemDropZone::OntoItem)  &&
		!TargetDropItem->HasParent(DraggedItem, true))
	{
		if(ItemDropZone == EItemDropZone::OntoItem)
		{
			// We support nested inputs
			if(DraggedItem->GetData()->IsA<UNiagaraHierarchyScriptParameter>() && TargetDropItem->GetData()->IsA<UNiagaraHierarchyScriptParameter>())
			{
				// But not if the dragged input already has a child
				if(DraggedItem->GetData()->DoesOneChildExist<UNiagaraHierarchyScriptParameter>())
				{
					FText BaseMessage = LOCTEXT("DraggedItemHasChild_NotAllowed", "Can't nest input {0} under input {1}.\nRemove children of {2} first. ");
					AllowDrop.UserFeedback = FText::FormatOrdered(BaseMessage, DraggedItem->ToStringAsText(), TargetDropItem->ToStringAsText(), DraggedItem->ToStringAsText());
					AllowDrop.bResult = false;
					return AllowDrop;
				}
				
				// And only up to 1 layer if we are going to create a new child input
				if(TargetDropItem->GetParent().Pin()->GetData<UNiagaraHierarchyScriptParameter>())
				{
					FText BaseMessage = LOCTEXT("DroppingInputOnInputNestedChildTooDeep", "Can't nest input {0} under input {1}.\nChildren inputs can only have one layer of depth!");
					AllowDrop.UserFeedback = FText::FormatOrdered(BaseMessage, DraggedItem->ToStringAsText(), TargetDropItem->ToStringAsText());
					AllowDrop.bResult = false;
					return AllowDrop;
				}
				
				FText BaseMessage = LOCTEXT("DroppingInputOnInputNestedChild", "This will nest input {0} under input {1}");
				AllowDrop.UserFeedback = FText::FormatOrdered(BaseMessage, DraggedItem->ToStringAsText(), TargetDropItem->ToStringAsText());
				AllowDrop.bResult = true;
				return AllowDrop;
			}
		}
		else
		{
			// if the dragged item is an input, we generally allow above/below, even for nested child inputs
			if(DraggedItem->GetData()->IsA<UNiagaraHierarchyScriptParameter>())
			{
				AllowDrop.bResult = true;
			}
			// if the dragged item is a category, we generally allow putting it above/below other parameters, but not above/below child parameters
			else if(DraggedItem->GetData()->IsA<UHierarchyCategory>())
			{
				AllowDrop.bResult = true;
				if(TargetDropItem->GetData()->IsA(UNiagaraHierarchyScriptParameter::StaticClass()) && TargetDropItem->GetParent().Pin()->GetData()->IsA(UNiagaraHierarchyScriptParameter::StaticClass()))
				{
					AllowDrop.bResult = false;
				}
			}
		}
	}

	return AllowDrop;
}

#undef LOCTEXT_NAMESPACE
