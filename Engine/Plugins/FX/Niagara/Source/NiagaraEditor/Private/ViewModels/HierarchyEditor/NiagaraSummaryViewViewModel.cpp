// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"

#include "EdGraphSchema_Niagara.h"
#include "GraphEditAction.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraScriptMergeManager.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSystem.h"
#include "NiagaraSimulationStageBase.h"
#include "SDropTarget.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ScopedTransaction.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ToolMenu.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSummaryViewViewModel)

#define LOCTEXT_NAMESPACE "NiagaraSummaryViewHierarchyEditor"

bool GetIsFromBaseEmitter(const FVersionedNiagaraEmitter& Emitter, FHierarchyElementIdentity SummaryItemIdentity)
{
	TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
	return MergeManager->DoesSummaryItemExistInBase(Emitter, SummaryItemIdentity);
}

void UNiagaraHierarchyModule::Initialize(const UNiagaraNodeFunctionCall& InModuleNode)
{
	FHierarchyElementIdentity ModuleIdentity;
	ModuleIdentity.Guids.Add(InModuleNode.NodeGuid);
	SetIdentity(ModuleIdentity);
}

void UNiagaraHierarchyModuleInput::Initialize(const UNiagaraNodeFunctionCall& InModuleNode, FGuid InputGuid)
{
	FHierarchyElementIdentity InputIdentity;
	InputIdentity.Guids.Add(InModuleNode.NodeGuid);
	InputIdentity.Guids.Add(InputGuid);
	SetIdentity(InputIdentity);
}

void UNiagaraHierarchyAssignmentInput::Initialize(const UNiagaraNodeAssignment& AssignmentNode, FName AssignmentTarget)
{
	FHierarchyElementIdentity InputIdentity;
	InputIdentity.Guids.Add(AssignmentNode.NodeGuid);
	InputIdentity.Names.Add(AssignmentTarget);
	SetIdentity(InputIdentity);
}

void UNiagaraHierarchyEmitterProperties::Initialize(const FVersionedNiagaraEmitter& Emitter)
{
	FHierarchyElementIdentity InputIdentity;
	InputIdentity.Names.Add(FName(Emitter.Emitter->GetUniqueEmitterName()));
	InputIdentity.Names.Add("Category");
	InputIdentity.Names.Add("Properties");
	SetIdentity(InputIdentity);
}

void UNiagaraHierarchyRenderer::Initialize(const UNiagaraRendererProperties& Renderer)
{
	FHierarchyElementIdentity RendererIdentity;
	RendererIdentity.Guids.Add(Renderer.GetMergeId());
	SetIdentity(RendererIdentity);
}

void UNiagaraHierarchyEventHandler::Initialize(const FNiagaraEventScriptProperties& EventHandler)
{
	FHierarchyElementIdentity EventHandlerIdentity;
	EventHandlerIdentity.Guids.Add(EventHandler.Script->GetUsageId());
	EventHandlerIdentity.Guids.Add(EventHandler.SourceEmitterID);
	SetIdentity(EventHandlerIdentity);
}

void UNiagaraHierarchyEventHandlerProperties::Initialize(const FNiagaraEventScriptProperties& EventHandler)
{
	SetIdentity(MakeIdentity(EventHandler));
}

FHierarchyElementIdentity UNiagaraHierarchyEventHandlerProperties::MakeIdentity(const FNiagaraEventScriptProperties& EventHandler)
{
	FHierarchyElementIdentity Identity;
	Identity.Guids.Add(EventHandler.Script->GetUsageId());
	Identity.Names.Add(TEXT("Category"));
	Identity.Names.Add(TEXT("Properties"));
	return Identity;
}

void UNiagaraHierarchySimStage::Initialize(const UNiagaraSimulationStageBase& SimStage)
{
	FHierarchyElementIdentity SimStageIdentity;
	SimStageIdentity.Guids.Add(SimStage.GetMergeId());
	SetIdentity(SimStageIdentity);
}

void UNiagaraHierarchySimStageProperties::Initialize(const UNiagaraSimulationStageBase& SimStage)
{
	SetIdentity(MakeIdentity(SimStage));
}

FHierarchyElementIdentity UNiagaraHierarchySimStageProperties::MakeIdentity(const UNiagaraSimulationStageBase& SimStage)
{
	FHierarchyElementIdentity SimStagePropertiesIdentity;
	SimStagePropertiesIdentity.Guids.Add(SimStage.GetMergeId());
	SimStagePropertiesIdentity.Names.Add(FName("Category"));
	SimStagePropertiesIdentity.Names.Add("Properties");
	return SimStagePropertiesIdentity;
}

void UNiagaraHierarchyObjectProperty::Initialize(FGuid ObjectGuid, FString PropertyName)
{
	FHierarchyElementIdentity PropertyIdentity;
	PropertyIdentity.Guids.Add(ObjectGuid);
	PropertyIdentity.Names.Add(FName(PropertyName));
	SetIdentity(PropertyIdentity);
}

void UNiagaraSummaryViewViewModel::Initialize(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	EmitterViewModelWeak = EmitterViewModel;
	EmitterViewModel->OnScriptGraphChanged().AddUObject(this, &UNiagaraSummaryViewViewModel::OnScriptGraphChanged);
	//-TODO:Stateless: Do we need stateless support here?
	if (EmitterViewModel->GetEmitter().Emitter)
	{
		EmitterViewModel->GetEmitter().Emitter->OnRenderersChanged().AddUObject(this, &UNiagaraSummaryViewViewModel::OnRenderersChanged);
		EmitterViewModel->GetEmitter().Emitter->OnSimStagesChanged().AddUObject(this, &UNiagaraSummaryViewViewModel::OnSimStagesChanged);
		EmitterViewModel->GetEmitter().Emitter->OnEventHandlersChanged().AddUObject(this, &UNiagaraSummaryViewViewModel::OnEventHandlersChanged);

		UDataHierarchyViewModelBase::Initialize();
	}
}

void UNiagaraSummaryViewViewModel::FinalizeInternal()
{
	GetEmitterViewModel()->OnScriptGraphChanged().RemoveAll(this);

	if(GetEmitterViewModel()->GetEmitter().Emitter != nullptr)
	{
		GetEmitterViewModel()->GetEmitter().Emitter->OnRenderersChanged().RemoveAll(this);
		GetEmitterViewModel()->GetEmitter().Emitter->OnSimStagesChanged().RemoveAll(this);
		GetEmitterViewModel()->GetEmitter().Emitter->OnEventHandlersChanged().RemoveAll(this);
	}
}

TSharedRef<FNiagaraEmitterViewModel> UNiagaraSummaryViewViewModel::GetEmitterViewModel() const
{
	TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = EmitterViewModelWeak.Pin();
	checkf(EmitterViewModel.IsValid(), TEXT("Emitter view model destroyed before summary hierarchy view model."));
	return EmitterViewModel.ToSharedRef();
}

TWeakObjectPtr<UNiagaraNodeFunctionCall> FNiagaraFunctionViewModel::GetFunctionCallNode() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	return SummaryViewModel->GetFunctionCallNode(GetData()->GetPersistentIdentity().Guids[0]);
}

void FNiagaraFunctionViewModel::OnScriptApplied(UNiagaraScript* NiagaraScript, FGuid Guid)
{
	if(GetFunctionCallNode()->FunctionScript == NiagaraScript)
	{
		RefreshChildrenInputs(true);
		SyncViewModelsToData();
	}
}

void FNiagaraFunctionViewModel::ClearCache() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	SummaryViewModel->ClearFunctionCallNodeCache(GetData()->GetPersistentIdentity().Guids[0]);
}

FString FNiagaraFunctionViewModel::ToString() const
{
	if(GetFunctionCallNode().IsValid())
	{
		return GetFunctionCallNode()->GetNodeTitle(ENodeTitleType::ListView).ToString();
	}

	return TEXT("Unknown");
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraFunctionViewModel::IsEditableByUser()
{
	if(bIsDynamicInput)
	{
		FResultWithUserFeedback CanEditResults(false);
		CanEditResults.UserFeedback = LOCTEXT("DynamicInputCantBeDragged", "You can not drag entire Dynamic Inputs. Either drag the entire module input, or individual inputs of the Dynamic Input");
		return CanEditResults;
	}
	
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("ModuleIsFromBaseEmitter", "This module was added in the parent emitter and can not be edited.") : FText::GetEmpty();
	return CanEditResults;
}

bool FNiagaraFunctionViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

void FNiagaraFunctionViewModel::Initialize()
{
	if(GetFunctionCallNode().IsValid())
	{
		OnScriptAppliedHandle = FNiagaraEditorModule::Get().OnScriptApplied().AddLambda([this](UNiagaraScript* Script, FGuid ScriptVersion)
		{
			if(GetFunctionCallNode()->FunctionScript == Script)
			{
				SyncViewModelsToData();
			}
		});

		// determine whether this represents a dynamic input or a module by checking if the output pin of this node is a parameter map.
		UEdGraphPin* OutputPin = GetFunctionCallNode()->GetOutputPin(0);
		bIsDynamicInput = UEdGraphSchema_Niagara::PinToTypeDefinition(OutputPin) != FNiagaraTypeDefinition::GetParameterMapDef();
	}
}

FNiagaraFunctionViewModel::~FNiagaraFunctionViewModel()
{
	if(OnScriptAppliedHandle.IsValid())
	{
		FNiagaraEditorModule::Get().OnScriptApplied().Remove(OnScriptAppliedHandle);
		OnScriptAppliedHandle.Reset();
	}
}

void FNiagaraFunctionViewModel::RefreshChildrenDataInternal()
{
	RefreshChildrenInputs(false);
}

void FNiagaraFunctionViewModel::RefreshChildrenInputs(bool bClearCache)
{
	TWeakObjectPtr<UNiagaraNodeFunctionCall> FunctionNodeWeak = GetFunctionCallNode();
	if(FunctionNodeWeak.IsValid())
	{
		UNiagaraNodeFunctionCall* FunctionNode = FunctionNodeWeak.Get();
		UNiagaraNodeAssignment* AsAssignmentNode = Cast<UNiagaraNodeAssignment>(FunctionNode);
		
		if(UNiagaraGraph* AssetGraph = FunctionNode->GetCalledGraph())
		{
			// if it's not an assignment node, it's a module node
			if(AsAssignmentNode == nullptr)
			{
				TArray<FNiagaraVariable> Variables;
				AssetGraph->GetAllVariables(Variables);

				TMap<FGuid, FNiagaraVariable> VariableGuidMap;
				TMap<FGuid, FNiagaraVariableMetaData> VariableGuidMetadataMap;
				for(const FNiagaraVariable& Variable : Variables)
				{
					// we create an input for most top level static switches & module inputs
					bool bIsModuleInput = Variable.IsInNameSpace(FNiagaraConstants::ModuleNamespaceString);
					TOptional<bool> bIsStaticSwitchInputOptional = AssetGraph->IsStaticSwitch(Variable);
					if(!bIsModuleInput && !bIsStaticSwitchInputOptional.Get(false))
					{
						continue;
					}
					
					TOptional<FNiagaraVariableMetaData> VariableMetaData = AssetGraph->GetMetaData(Variable);
					// we don't show inline edit condition attributes
					if(VariableMetaData->bInlineEditConditionToggle == false)
					{
						VariableGuidMap.Add(VariableMetaData->GetVariableGuid(), Variable);
						VariableGuidMetadataMap.Add(VariableMetaData->GetVariableGuid(), VariableMetaData.GetValue());
					}
				}
				
				TArray<FGuid> VariableGuids;
				VariableGuidMap.GenerateKeyArray(VariableGuids);
				VariableGuids.Sort([&](const FGuid& GuidA, const FGuid& GuidB)
				{
					if(VariableGuidMetadataMap[GuidA].bAdvancedDisplay != VariableGuidMetadataMap[GuidB].bAdvancedDisplay)
					{
						if(VariableGuidMetadataMap[GuidA].bAdvancedDisplay)
						{
							return false;
						}
						else
						{
							return true;
						}
					}
					
					return VariableGuidMetadataMap[GuidA].GetEditorSortPriority_DEPRECATED() < VariableGuidMetadataMap[GuidB].GetEditorSortPriority_DEPRECATED();
				});
				
				for(const FGuid& VariableGuid : VariableGuids)
				{
					FHierarchyElementIdentity SearchedChildIdentity;
					SearchedChildIdentity.Guids.Add(FunctionNode->NodeGuid);
					SearchedChildIdentity.Guids.Add(VariableGuid);
					const bool bChildExists = GetData()->GetChildren().ContainsByPredicate([SearchedChildIdentity](UHierarchyElement* CandidateChild)
					{
						return CandidateChild->GetPersistentIdentity() == SearchedChildIdentity;
					});
				
					if(bChildExists == false)
					{
						UNiagaraHierarchyModuleInput* ModuleInput = GetDataMutable()->AddChild<UNiagaraHierarchyModuleInput>();
						ModuleInput->Initialize(*FunctionNode, VariableGuid);
					}
				}
			}
			else
			{
				UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(FunctionNode);

				for(const FNiagaraVariable& Variable : AssignmentNode->GetAssignmentTargets())
				{
					const bool bChildExists = GetData()->GetChildren().ContainsByPredicate([Variable, bClearCache, this](UHierarchyElement* Candidate)
					{
						if(UNiagaraHierarchyAssignmentInput* AssignmentInput = Cast<UNiagaraHierarchyAssignmentInput>(Candidate))
						{
							return Variable.GetName() == AssignmentInput->GetPersistentIdentity().Names[0];
						}

						return false;						
					});

					if(bChildExists == false)
					{
						UNiagaraHierarchyAssignmentInput* AssignmentInput = GetDataMutable()->AddChild<UNiagaraHierarchyAssignmentInput>();
						AssignmentInput->Initialize(*AsAssignmentNode, Variable.GetName());
					}
				}
			}
		}
	}
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraFunctionViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> NiagaraHierarchyItemViewModelBase, EItemDropZone ItemDropZone)
{
	if(IsEditableByUser().bResult == false)
	{
		return false;
	}
	
	// we don't allow any items to be added directly onto the module as it's self managing
	if(ItemDropZone == EItemDropZone::OntoItem)
	{
		FResultWithUserFeedback Results(false);
		Results.UserFeedback = LOCTEXT("CanDropOnModuleDragMessage", "You can not add any items to a module directly. Please create a category, which can contain arbitrary items.");
		return Results;
	}
	
	return FHierarchyItemViewModel::CanDropOnInternal(NiagaraHierarchyItemViewModelBase, ItemDropZone);
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraModuleInputViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedItem, EItemDropZone ItemDropZone)
{
	// if the input isn't editable, we don't allow any drops on/above/below the item.
	// Even though it technically works, the merge process will only re-add the item at the end and not preserve order so there is no point in allowing dropping above/below
	if(IsEditableByUser().bResult == false)
	{
		return false;
	}

	FResultWithUserFeedback AllowDrop(false);
	
	TSharedPtr<FHierarchyElementViewModel> TargetDropItem = AsShared();

	// we only allow drops if some general conditions are fulfilled
	if(DraggedItem->GetData() != TargetDropItem->GetData() &&
		(!DraggedItem->HasParent(TargetDropItem, false) || ItemDropZone != EItemDropZone::OntoItem)  &&
		!TargetDropItem->HasParent(DraggedItem, true))
	{
		if(ItemDropZone == EItemDropZone::OntoItem)
		{
			// if the current input doesn't have a parent input, we allow dropping other inputs onto it
			if(DraggedItem->GetData()->IsA<UNiagaraHierarchyModuleInput>() && TargetDropItem->GetData()->IsA<UNiagaraHierarchyModuleInput>() && TargetDropItem->GetParent().Pin()->GetData<UNiagaraHierarchyModuleInput>() == nullptr)
			{
				if(DraggedItem->GetData()->GetChildren().Num() > 0)
				{
					FText BaseMessage = LOCTEXT("DroppingInputOnInputWillEmptyChildren", "Input {0} has child inputs. Dropping the input here will remove these children as we only allow nested inputs one level deep.");
					AllowDrop.UserFeedback = FText::FormatOrdered(BaseMessage, DraggedItem->ToStringAsText());
					AllowDrop.bResult = true;
				}
				else
				{
					FText BaseMessage = LOCTEXT("DroppingInputOnInputNestedChild", "This will nest input {0} under input {1}");
					AllowDrop.UserFeedback = FText::FormatOrdered(BaseMessage, DraggedItem->ToStringAsText(), TargetDropItem->ToStringAsText());
					AllowDrop.bResult = true;
				}
			}
		}
		else
		{
			// if the dragged item is an input, we generally allow above/below, even for nested child inputs
			if(DraggedItem->GetData()->IsA<UNiagaraHierarchyModuleInput>())
			{
				AllowDrop.bResult = true;
			}
			else
			{
				// we use default logic only if there is no parent input. Nested children are not allowed to contain anything but other inputs.
				if(TargetDropItem->GetParent().Pin()->GetData<UNiagaraHierarchyModuleInput>() == nullptr)
				{
					AllowDrop = FHierarchyItemViewModel::CanDropOnInternal(DraggedItem, ItemDropZone);
				}
			}
		}
	}

	return AllowDrop;
}

void FNiagaraModuleInputViewModel::PostAddFixup()
{
	// If our new parent is an input, we empty out our own inputs as we only allow 1 layer of child inputs
	if(Parent.Pin()->GetData()->IsA<UNiagaraHierarchyModuleInput>())
	{
		GetDataMutable()->GetChildrenMutable().Empty();
	}
}

void FNiagaraModuleInputViewModel::AppendDynamicContextMenuForSingleElement(UToolMenu* ToolMenu)
{
	FUIAction Action;
	Action.ExecuteAction = FExecuteAction::CreateSP(this, &FNiagaraModuleInputViewModel::AddNativeChildrenInputs);
	Action.CanExecuteAction = FCanExecuteAction::CreateSP(this, &FNiagaraModuleInputViewModel::CanAddNativeChildrenInputs);
	Action.IsActionVisibleDelegate = FIsActionButtonVisible::CreateSP(this, &FNiagaraModuleInputViewModel::CanAddNativeChildrenInputs);
	
	ToolMenu->AddMenuEntry("Dynamic",
		FToolMenuEntry::InitMenuEntry(FName("Add Children Inputs"),LOCTEXT("AddChildrenInputsMenuLabel", "Add Children Inputs"),
		LOCTEXT("AddChildrenInputsMenuTooltip", "Add children inputs of this input as child inputs."), FSlateIcon(),Action));
}

TWeakObjectPtr<UNiagaraNodeFunctionCall> FNiagaraModuleInputViewModel::GetModuleNode() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	return SummaryViewModel->GetFunctionCallNode(GetData()->GetPersistentIdentity().Guids[0]);
}

TOptional<FInputData> FNiagaraModuleInputViewModel::GetInputData() const
{
	if(!InputDataCache.IsSet())
	{
		InputDataCache = FindInputDataInternal();
	}

	return InputDataCache;
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraModuleInputViewModel::CanHaveChildren() const
{
	// we generally allow children inputs in the source view
	if(IsForHierarchy() == false)
	{
		return true;
	}

	return true;
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraModuleInputViewModel::CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType)
{
	// We support nested inputs up to 1 layer deep
	if(InHierarchyElementType->IsChildOf<UNiagaraHierarchyModuleInput>())
	{
		if(Parent.IsValid() && Parent.Pin()->GetData()->IsA<UNiagaraHierarchyModuleInput>())
		{
			FResultWithUserFeedback CanContainResult(false);
			FText BaseMessage = LOCTEXT("InputOnInputNestedChildTooDeep", "Can't nest new input {0} under input {1}.\nChildren inputs can only have one layer of depth! Add it to the parent input!");
			CanContainResult.UserFeedback = FText::FormatOrdered(BaseMessage, InHierarchyElementType->GetDisplayNameText(), ToStringAsText());
			return CanContainResult;
		}
				
		return true;
	}
	
	return false;
}

FString FNiagaraModuleInputViewModel::ToString() const
{
	TOptional<FInputData> InputData = GetInputData();
	if(InputData.IsSet())
	{
		return InputData->InputName.ToString();
	}

	return FHierarchyItemViewModel::ToString();
}

TArray<FString> FNiagaraModuleInputViewModel::GetSearchTerms() const
{
	TArray<FString> SearchTerms;
	SearchTerms.Add(ToString());

	FText DisplayNameOverride = GetData<UNiagaraHierarchyModuleInput>()->GetDisplayNameOverride();
	if(DisplayNameOverride.IsEmpty() == false)
	{
		SearchTerms.Add(DisplayNameOverride.ToString());
	}
	
	return SearchTerms;
}

bool FNiagaraModuleInputViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

void FNiagaraModuleInputViewModel::ClearCache() const
{
	InputDataCache.Reset();
}

void FNiagaraModuleInputViewModel::RefreshChildDynamicInputs(bool bClearCache)
{
	TOptional<FInputData> InputData = GetInputData();
	if(InputData.IsSet())
	{
		UNiagaraNodeFunctionCall* DynamicInputNode = FNiagaraStackGraphUtilities::FindDynamicInputNodeForInput(*GetModuleNode().Get(), InputData->InputName);

		if(DynamicInputNode)
		{
			FHierarchyElementIdentity DynamicInputIdentity;
			DynamicInputIdentity.Guids.Add(DynamicInputNode->NodeGuid);
			const bool bChildExists = GetDataMutable()->GetChildrenMutable().ContainsByPredicate([DynamicInputIdentity, bClearCache](UHierarchyElement* CandidateChild)
				{
					return CandidateChild->GetPersistentIdentity() == DynamicInputIdentity;
				});
				
			if(bChildExists == false)
			{
				UNiagaraHierarchyModule* DynamicInputHierarchyModule = GetDataMutable()->AddChild<UNiagaraHierarchyModule>();
				DynamicInputHierarchyModule->Initialize(*DynamicInputNode);
			}			
		}
	}
}

FText FNiagaraModuleInputViewModel::GetSummaryInputNameOverride() const
{
	return GetData<UNiagaraHierarchyModuleInput>()->GetDisplayNameOverride();
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraModuleInputViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("ModuleInputIsFromBaseEmitter", "This input was added in the parent emitter and can not be edited.") : FText::GetEmpty();

	if(bIsForHierarchy && CanEditResults.bResult == true)
	{
		if(Parent.Pin()->GetData()->IsA<UNiagaraHierarchyModule>())
		{
			CanEditResults.bResult = false;
			CanEditResults.UserFeedback = LOCTEXT("ModuleCanOnlyBeEditedDirectly", "This input can not be modified as it is inherent part of its parent module. Add this input separately if you want to modify it.");
		}
	}
	
	return CanEditResults;
}

void FNiagaraModuleInputViewModel::RefreshChildrenDataInternal()
{
	RefreshChildDynamicInputs(false);
}

TOptional<FInputData> FNiagaraModuleInputViewModel::FindInputDataInternal() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	return SummaryViewModel->GetInputData(*GetData<UNiagaraHierarchyModuleInput>());
}

void FNiagaraModuleInputViewModel::AddNativeChildrenInputs()
{
	if(IsForHierarchy() == false)
	{
		return;
	}
	
	if(GetModuleNode().IsValid() && GetInputData().IsSet())
	{
		TArray<FHierarchyElementIdentity> ChildIdentities = GetNativeChildInputIdentities();
		TMap<FHierarchyElementIdentity, int32> ChildSortOrderMap;

		if(UNiagaraGraph* Graph = GetModuleNode()->GetCalledGraph())
		{
			TArray<FNiagaraVariable> Variables;
			Graph->GetAllVariables(Variables);

			for(const FNiagaraVariable& Variable : Variables)
			{
				if(UNiagaraScriptVariable* ScriptVariable = Graph->GetScriptVariable(Variable))
				{
					if(!ScriptVariable->Metadata.GetParentAttribute_DEPRECATED().IsNone() && ScriptVariable->Metadata.GetParentAttribute_DEPRECATED().IsEqual(GetInputData()->InputName))
					{
						FHierarchyElementIdentity ChildIdentity;
						ChildIdentity.Guids.Add(GetData()->GetPersistentIdentity().Guids[0]);
						ChildIdentity.Guids.Add(ScriptVariable->Metadata.GetVariableGuid());
						ChildSortOrderMap.Add(ChildIdentity, ScriptVariable->Metadata.GetEditorSortPriority_DEPRECATED());
					}
				}
			}
		}

		for(const FHierarchyElementIdentity& ChildIdentity : ChildIdentities)
		{
			if(FindViewModelForChild(ChildIdentity, false) == nullptr)
			{
				TSharedPtr<FHierarchyElementViewModel> ViewModel = HierarchyViewModel->GetHierarchyRootViewModel()->FindViewModelForChild(ChildIdentity, true);
				if(ViewModel.IsValid())
				{
					ReparentToThis(ViewModel);
				}
				else
				{
					UNiagaraHierarchyModuleInput* ModuleInput = GetDataMutable()->AddChild<UNiagaraHierarchyModuleInput>();
					ModuleInput->Initialize(*GetModuleNode().Get(), ChildIdentity.Guids[1]);
				}
			}
		}

		SyncViewModelsToData();
		
		TArray<TSharedPtr<FNiagaraModuleInputViewModel>> ChildInputs;
		GetChildrenViewModelsForType<UNiagaraHierarchyModuleInput, FNiagaraModuleInputViewModel>(ChildInputs);

		auto SortChildrenInputs = [&](UHierarchyElement& ItemA, UHierarchyElement& ItemB)
		{
			int32 SortOrderA = ChildSortOrderMap[ItemA.GetPersistentIdentity()];
			int32 SortOrderB = ChildSortOrderMap[ItemB.GetPersistentIdentity()];

			return SortOrderA < SortOrderB;
		};
		
		GetDataMutable()->SortChildren(SortChildrenInputs, false);
		SyncViewModelsToData();
		
		HierarchyViewModel->OnHierarchyChanged(TInstancedStruct<FHierarchyChangedPayload>::Make("Native children inputs added"));
	}
}

bool FNiagaraModuleInputViewModel::CanAddNativeChildrenInputs() const
{
	if(IsForHierarchy() == false)
	{
		return false;
	}
	
	return GetNativeChildInputIdentities().Num() > 0;
}

TArray<FHierarchyElementIdentity> FNiagaraModuleInputViewModel::GetNativeChildInputIdentities() const
{
	TArray<FHierarchyElementIdentity> ChildIdentities;
	if(GetModuleNode().IsValid())
	{
		if(UNiagaraGraph* Graph = GetModuleNode()->GetCalledGraph())
		{
			TArray<FNiagaraVariable> Variables;
			Graph->GetAllVariables(Variables);

			for(const FNiagaraVariable& Variable : Variables)
			{
				if(UNiagaraScriptVariable* ScriptVariable = Graph->GetScriptVariable(Variable))
				{
					if(!ScriptVariable->Metadata.GetParentAttribute_DEPRECATED().IsNone() && ScriptVariable->Metadata.GetParentAttribute_DEPRECATED().IsEqual(GetInputData()->InputName))
					{
						FHierarchyElementIdentity ChildIdentity;
						ChildIdentity.Guids.Add(GetData()->GetPersistentIdentity().Guids[0]);
						ChildIdentity.Guids.Add(ScriptVariable->Metadata.GetVariableGuid());
						ChildIdentities.Add(ChildIdentity);
					}
				}
			}
		}
	}

	return ChildIdentities;
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraAssignmentInputViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedItem, EItemDropZone ItemDropZone)
{
	// if the input isn't editable, we don't allow any drops on/above/below the item.
	// Even though it technically works, the merge process will only re-add the item at the end and not preserve order.
	if(IsEditableByUser().bResult == false)
	{
		return false;
	}
	
	return FHierarchyItemViewModel::CanDropOnInternal(DraggedItem, ItemDropZone);
}

TWeakObjectPtr<UNiagaraNodeAssignment> FNiagaraAssignmentInputViewModel::GetAssignmentNode() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	return Cast<UNiagaraNodeAssignment>(SummaryViewModel->GetFunctionCallNode(GetData()->GetPersistentIdentity().Guids[0]));
}

TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData> FNiagaraAssignmentInputViewModel::GetInputData() const
{
	if(!InputDataCache.IsSet())
	{
		InputDataCache = FindInputDataInternal();
	}

	return InputDataCache;
}

FString FNiagaraAssignmentInputViewModel::ToString() const
{
	TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData> InputData = GetInputData();
	if(GetInputData().IsSet())
	{
		return InputData->InputName.ToString();
	}

	return FHierarchyItemViewModel::ToString();
}

TArray<FString> FNiagaraAssignmentInputViewModel::GetSearchTerms() const
{
	TArray<FString> SearchTerms;
	SearchTerms.Add(ToString());	
	return SearchTerms;
}

bool FNiagaraAssignmentInputViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

void FNiagaraAssignmentInputViewModel::ClearCache() const
{
	InputDataCache.Reset();
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	SummaryViewModel->ClearFunctionCallNodeCache(GetData()->GetPersistentIdentity().Guids[0]);
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraAssignmentInputViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("ModuleInputIsFromBaseEmitter", "This input was added in the parent emitter and can not be edited.") : FText::GetEmpty();

	if(bIsForHierarchy && CanEditResults.bResult == true)
	{
		if(Parent.Pin()->GetData()->IsA<UNiagaraHierarchyModule>())
		{
			CanEditResults.bResult = false;
			CanEditResults.UserFeedback = LOCTEXT("ModuleCanOnlyBeEditedDirectly", "This input can not be modified as it is inherent part of its parent module. Add this input separately if you want to modify it.");
		}
	}
	
	return CanEditResults;
}

TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData> FNiagaraAssignmentInputViewModel::FindInputDataInternal() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	
	if(GetAssignmentNode().IsValid())
	{
		return FNiagaraStackGraphUtilities::FindAssignmentInputData(*GetAssignmentNode().Get(), GetData()->GetPersistentIdentity().Names[0], SummaryViewModel->GetEmitterViewModel());
	}

	return TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData>();
}

bool FNiagaraHierarchySummaryCategoryViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchySummaryCategoryViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("CategoryIsFromBaseEmitter", "This category was added in the parent emitter and can not be edited. You can add new items.") : FText::GetEmpty();
	return CanEditResults;
}

bool FNiagaraHierarchyPropertyViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

bool FNiagaraHierarchyPropertyViewModel::DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const
{
	UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
	TMap<FGuid, UObject*> PropertyObjectMap = ViewModel->GetObjectsForProperties();
	return PropertyObjectMap.Contains(GetData()->GetPersistentIdentity().Guids[0]);
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchyPropertyViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("ObjectPropertyIsFromBaseEmitter", "This property was added in the parent emitter and can not be edited.") : FText::GetEmpty();
	return CanEditResults;
}

FString FNiagaraHierarchyRendererViewModel::ToString() const
{
	UNiagaraRendererProperties* RendererProperties = GetRendererProperties();
	return RendererProperties != nullptr ? RendererProperties->GetWidgetDisplayName().ToString() : FString(); 
}

bool FNiagaraHierarchyRendererViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

void FNiagaraHierarchyRendererViewModel::RefreshChildrenDataInternal()
{
	TArray<UHierarchyElement*> NewChildren;
	for (TFieldIterator<FProperty> PropertyIterator(GetRendererProperties()->GetClass(), EFieldIteratorFlags::ExcludeSuper); PropertyIterator; ++PropertyIterator)
	{
		if(PropertyIterator->HasAnyPropertyFlags(CPF_Edit))
		{
			FString PropertyName = (*PropertyIterator)->GetName();

			FHierarchyElementIdentity PropertyIdentity;
			PropertyIdentity.Guids.Add(GetRendererProperties()->GetMergeId());
			PropertyIdentity.Names.Add(FName(PropertyName));
			
			auto* FoundItem = GetDataMutable()->GetChildrenMutable().FindByPredicate([PropertyIdentity](UHierarchyElement* Candidate)
			{
				return Candidate->GetPersistentIdentity() == PropertyIdentity;
			});

			UNiagaraHierarchyObjectProperty* RendererProperty = nullptr;
			if(FoundItem == nullptr)
			{
				RendererProperty = GetDataMutable()->AddChild<UNiagaraHierarchyObjectProperty>();
				RendererProperty->Initialize(GetRendererProperties()->GetMergeId(), PropertyName);
			}
			else
			{
				RendererProperty = CastChecked<UNiagaraHierarchyObjectProperty>(*FoundItem);
			}

			NewChildren.Add(RendererProperty);
		}
	}

	GetDataMutable()->GetChildrenMutable().Empty();
	GetDataMutable()->GetChildrenMutable().Append(NewChildren);
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchyRendererViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("RendererIsFromBaseEmitter", "This renderer was added in the parent emitter and can not be edited.") : FText::GetEmpty();
	return CanEditResults;
}

FString FNiagaraHierarchyEmitterPropertiesViewModel::ToString() const
{
	return TEXT("Emitter Properties");
}

bool FNiagaraHierarchyEmitterPropertiesViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchyEmitterPropertiesViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("EmitterPropertiesIsFromBaseEmitter", "These emitter properties were added in the parent emitter and can not be edited.") : FText::GetEmpty();
	return CanEditResults;
}

FString FNiagaraHierarchyEventHandlerViewModel::ToString() const
{
	FNiagaraEventScriptProperties* ScriptProperties = GetEventScriptProperties();
	if(ScriptProperties != nullptr)
	{
		return ScriptProperties->SourceEventName.ToString();
	}

	return FString();
}

FNiagaraEventScriptProperties* FNiagaraHierarchyEventHandlerViewModel::GetEventScriptProperties() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	FGuid UsageID = GetData()->GetPersistentIdentity().Guids[0];
	
	for(FNiagaraEventScriptProperties& ScriptProperties : SummaryViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData()->EventHandlerScriptProps)
	{
		if(ScriptProperties.Script->GetUsageId() == UsageID)
		{
			return &ScriptProperties;
		}
	}
	
	return nullptr;
}

bool FNiagaraHierarchyEventHandlerViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

void FNiagaraHierarchyEventHandlerViewModel::RefreshChildrenDataInternal()
{
	TArray<UHierarchyElement*> NewChildren;

	// First we add the properties item
	FHierarchyElementIdentity PropertiesIdentity = UNiagaraHierarchyEventHandlerProperties::MakeIdentity(*GetEventScriptProperties());	

	auto* FoundProperties = GetDataMutable()->GetChildrenMutable().FindByPredicate([PropertiesIdentity](UHierarchyElement* Candidate)
	{
		return Candidate->GetPersistentIdentity() == PropertiesIdentity;
	});

	UNiagaraHierarchyEventHandlerProperties* PropertiesCategory = nullptr;
	if(FoundProperties == nullptr)
	{
		PropertiesCategory = GetDataMutable()->AddChild<UNiagaraHierarchyEventHandlerProperties>();
		PropertiesCategory->Initialize(*GetEventScriptProperties());
	}
	else
	{
		PropertiesCategory = CastChecked<UNiagaraHierarchyEventHandlerProperties>(*FoundProperties);
	}

	NewChildren.Add(PropertiesCategory);

	// Then we go through all modules of that sim stage
	UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
	TArray<UNiagaraNodeFunctionCall*> EventHandlerModules = FNiagaraStackGraphUtilities::FindModuleNodesForEventHandler(*GetEventScriptProperties(), ViewModel->GetEmitterViewModel());
	
	for(UNiagaraNodeFunctionCall* EventHandlerModule : EventHandlerModules)
	{
		UNiagaraHierarchyModule* HierarchyEventHandlerModule = nullptr;
		FHierarchyElementIdentity ModuleIdentity;
		ModuleIdentity.Guids.Add(EventHandlerModule->NodeGuid);
		auto* FoundHierarchySimStageModule = GetDataMutable()->GetChildrenMutable().FindByPredicate([ModuleIdentity](UHierarchyElement* Candidate)
		{
			return Candidate->GetPersistentIdentity() == ModuleIdentity;
		});
		
		if(FoundHierarchySimStageModule == nullptr)		
		{
			HierarchyEventHandlerModule = GetDataMutable()->AddChild<UNiagaraHierarchyModule>();
			HierarchyEventHandlerModule->Initialize(*EventHandlerModule);
		}
		else
		{
			HierarchyEventHandlerModule = CastChecked<UNiagaraHierarchyModule>(*FoundHierarchySimStageModule);
		}
		
		NewChildren.Add(HierarchyEventHandlerModule);
	}
	
	GetDataMutable()->GetChildrenMutable().Empty();
	GetDataMutable()->GetChildrenMutable().Append(NewChildren);
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchyEventHandlerViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("EventHandlerIsFromBaseEmitter", "This event handler was added in the parent emitter and can not be edited.") : FText::GetEmpty();
	return CanEditResults;
}

UNiagaraRendererProperties* FNiagaraHierarchyRendererViewModel::GetRendererProperties() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	FGuid RendererGuid = GetData()->GetPersistentIdentity().Guids[0];

	UNiagaraRendererProperties* MatchingRenderer = nullptr;
	const TArray<UNiagaraRendererProperties*> RendererProperties = SummaryViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetRenderers();
	for(UNiagaraRendererProperties* Renderer : RendererProperties)
	{
		if(Renderer->GetMergeId() == RendererGuid)
		{
			MatchingRenderer = Renderer;
			break;
		}
	}
	
	return MatchingRenderer;
}

FString FNiagaraHierarchyEventHandlerPropertiesViewModel::ToString() const
{
	return GetEventScriptProperties()->SourceEventName.ToString().Append(" Properties");
}

FNiagaraEventScriptProperties* FNiagaraHierarchyEventHandlerPropertiesViewModel::GetEventScriptProperties() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	FGuid UsageID = GetData()->GetPersistentIdentity().Guids[0];
	
	for(FNiagaraEventScriptProperties& ScriptProperties : SummaryViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData()->EventHandlerScriptProps)
	{
		if(ScriptProperties.Script->GetUsageId() == UsageID)
		{
			return &ScriptProperties;
		}
	}
	
	return nullptr;
}

bool FNiagaraHierarchyEventHandlerPropertiesViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

void FNiagaraHierarchyEventHandlerPropertiesViewModel::RefreshChildrenDataInternal()
{
	TArray<UHierarchyElement*> NewPropertiesChildren;

	// todo (me) while this works, the stack needs to access the correct FStructOnScope that points to the FNiagaraEventScriptProperties
	// That can be made to work correctly, but the EventScriptProperties are heavily customized and introduce UI issues
	// Potentially solvable by registering the same customization but skipping for now
	
	// for (TFieldIterator<FProperty> PropertyIterator(StaticStruct<FNiagaraEventScriptProperties>(), EFieldIteratorFlags::IncludeSuper); PropertyIterator; ++PropertyIterator)
	// {
	// 	if(PropertyIterator->HasAnyPropertyFlags(CPF_Edit))
	// 	{
	// 		FString PropertyName = (*PropertyIterator)->GetName();
	//
	// 		FNiagaraHierarchyIdentity PropertyIdentity;
	// 		PropertyIdentity.Guids.Add(GetEventScriptProperties()->Script->GetUsageId());
	// 		PropertyIdentity.Names.Add(FName(PropertyName));
	// 		
	// 		UHierarchyElement** FoundPropertyItem = GetDataMutable()->GetChildrenMutable().FindByPredicate([PropertyIdentity](UHierarchyElement* Candidate)
	// 		{
	// 			return Candidate->GetPersistentIdentity() == PropertyIdentity;
	// 		});
	//
	// 		UNiagaraHierarchyObjectProperty* EventHandlerProperty = nullptr;
	// 		if(FoundPropertyItem == nullptr)
	// 		{
	// 			EventHandlerProperty = GetDataMutable()->AddChild<UNiagaraHierarchyObjectProperty>();
	// 			EventHandlerProperty->Initialize(GetEventScriptProperties()->Script->GetUsageId(), PropertyName);
	// 		}
	// 		else
	// 		{
	// 			EventHandlerProperty = CastChecked<UNiagaraHierarchyObjectProperty>(*FoundPropertyItem);
	// 		}
	//
	// 		NewPropertiesChildren.Add(EventHandlerProperty);
	// 	}
	// }

	GetDataMutable()->GetChildrenMutable().Empty();
	GetDataMutable()->GetChildrenMutable().Append(NewPropertiesChildren);
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchyEventHandlerPropertiesViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("EventHandlerPropertiesIsFromBaseEmitter", "This property item was added in the parent emitter and can not be edited.") : FText::GetEmpty();
	return CanEditResults;
}

FString FNiagaraHierarchySimStageViewModel::ToString() const
{
	UNiagaraSimulationStageBase* SimStage = GetSimStage();
	return SimStage != nullptr ? SimStage->SimulationStageName.ToString() : FString(); 
}

UNiagaraSimulationStageBase* FNiagaraHierarchySimStageViewModel::GetSimStage() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	FGuid SimStageGuid = GetData()->GetPersistentIdentity().Guids[0];

	UNiagaraSimulationStageBase* MatchingSimStage = nullptr;
	const TArray<UNiagaraSimulationStageBase*> SimStages = SummaryViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetSimulationStages();
	for(UNiagaraSimulationStageBase* SimStage : SimStages)
	{
		if(SimStage->GetMergeId() == SimStageGuid)
		{
			MatchingSimStage = SimStage;
			break;
		}
	}
	
	return MatchingSimStage;
}

bool FNiagaraHierarchySimStageViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

void FNiagaraHierarchySimStageViewModel::RefreshChildrenDataInternal()
{
	TArray<UHierarchyElement*> NewChildren;

	// First we add the properties item
	FHierarchyElementIdentity PropertiesIdentity;
	PropertiesIdentity.Guids.Add(GetSimStage()->GetMergeId());
	PropertiesIdentity.Names.Add(FName("Category"));
	PropertiesIdentity.Names.Add(FName("Properties"));

	auto* FoundProperties = GetDataMutable()->GetChildrenMutable().FindByPredicate([PropertiesIdentity](UHierarchyElement* Candidate)
	{
		return Candidate->GetPersistentIdentity() == PropertiesIdentity;
	});

	UNiagaraHierarchySimStageProperties* PropertiesCategory = nullptr;
	if(FoundProperties == nullptr)
	{
		PropertiesCategory = GetDataMutable()->AddChild<UNiagaraHierarchySimStageProperties>();
		PropertiesCategory->Initialize(*GetSimStage());
	}
	else
	{
		PropertiesCategory = CastChecked<UNiagaraHierarchySimStageProperties>(*FoundProperties);
	}

	NewChildren.Add(PropertiesCategory);

	// Then we go through all modules of that sim stage
	UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
	TArray<UNiagaraNodeFunctionCall*> SimStageModules = FNiagaraStackGraphUtilities::FindModuleNodesForSimulationStage(*GetSimStage(), ViewModel->GetEmitterViewModel());

	for(UNiagaraNodeFunctionCall* SimStageModule : SimStageModules)
	{
		UNiagaraHierarchyModule* HierarchySimStageModule = nullptr;
		FHierarchyElementIdentity ModuleIdentity;
		ModuleIdentity.Guids.Add(SimStageModule->NodeGuid);
		auto* FoundHierarchySimStageModule = GetDataMutable()->GetChildrenMutable().FindByPredicate([ModuleIdentity](UHierarchyElement* Candidate)
		{
			return Candidate->GetPersistentIdentity() == ModuleIdentity;
		});
		
		if(FoundHierarchySimStageModule == nullptr)		
		{
			HierarchySimStageModule = GetDataMutable()->AddChild<UNiagaraHierarchyModule>();
			HierarchySimStageModule->Initialize(*SimStageModule);
		}
		else
		{
			HierarchySimStageModule = CastChecked<UNiagaraHierarchyModule>(*FoundHierarchySimStageModule);
		}
		
		NewChildren.Add(HierarchySimStageModule);
	}
	
	GetDataMutable()->GetChildrenMutable().Empty();
	GetDataMutable()->GetChildrenMutable().Append(NewChildren);
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchySimStageViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("SimStageIsFromBaseEmitter", "This simulation stage was added in the parent emitter and can not be edited.") : FText::GetEmpty();
	return CanEditResults;
}

FString FNiagaraHierarchySimStagePropertiesViewModel::ToString() const
{
	return GetSimStage()->SimulationStageName.ToString().Append(" Properties");
}

UNiagaraSimulationStageBase* FNiagaraHierarchySimStagePropertiesViewModel::GetSimStage() const
{
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = GetHierarchyViewModel();
	UNiagaraSummaryViewViewModel* SummaryViewModel = CastChecked<UNiagaraSummaryViewViewModel>(ViewModel.Get());
	FGuid SimStageGuid = GetData()->GetPersistentIdentity().Guids[0];

	UNiagaraSimulationStageBase* MatchingSimStage = nullptr;
	const TArray<UNiagaraSimulationStageBase*> SimStages = SummaryViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetSimulationStages();
	for(UNiagaraSimulationStageBase* SimStage : SimStages)
	{
		if(SimStage->GetMergeId() == SimStageGuid)
		{
			MatchingSimStage = SimStage;
			break;
		}
	}
	
	return MatchingSimStage;
}

bool FNiagaraHierarchySimStagePropertiesViewModel::IsFromBaseEmitter() const
{
	if(!IsFromBaseEmitterCache.IsSet())
	{
		UNiagaraSummaryViewViewModel* ViewModel = Cast<UNiagaraSummaryViewViewModel>(GetHierarchyViewModel());
		IsFromBaseEmitterCache = GetIsFromBaseEmitter(ViewModel->GetEmitterViewModel()->GetEmitter(), GetData()->GetPersistentIdentity());
	}

	return IsFromBaseEmitterCache.GetValue();
}

void FNiagaraHierarchySimStagePropertiesViewModel::RefreshChildrenDataInternal()
{
	TArray<UHierarchyElement*> NewPropertiesChildren;

	for (TFieldIterator<FProperty> PropertyIterator(GetSimStage()->GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIterator; ++PropertyIterator)
	{
		if(PropertyIterator->HasAnyPropertyFlags(CPF_Edit))
		{
			FString PropertyName = (*PropertyIterator)->GetName();

			FHierarchyElementIdentity PropertyIdentity;
			PropertyIdentity.Guids.Add(GetSimStage()->GetMergeId());
			PropertyIdentity.Names.Add(FName(PropertyName));
			
			auto* FoundPropertyItem = GetDataMutable()->GetChildrenMutable().FindByPredicate([PropertyIdentity](UHierarchyElement* Candidate)
			{
				return Candidate->GetPersistentIdentity() == PropertyIdentity;
			});

			UNiagaraHierarchyObjectProperty* SimStageProperty = nullptr;
			if(FoundPropertyItem == nullptr)
			{
				SimStageProperty = GetDataMutable()->AddChild<UNiagaraHierarchyObjectProperty>();
				SimStageProperty->Initialize(GetSimStage()->GetMergeId(), PropertyName);
			}
			else
			{
				SimStageProperty = CastChecked<UNiagaraHierarchyObjectProperty>(*FoundPropertyItem);
			}

			NewPropertiesChildren.Add(SimStageProperty);
		}
	}

	GetDataMutable()->GetChildrenMutable().Empty();
	GetDataMutable()->GetChildrenMutable().Append(NewPropertiesChildren);
}

FHierarchyElementViewModel::FResultWithUserFeedback FNiagaraHierarchySimStagePropertiesViewModel::IsEditableByUser()
{
	FResultWithUserFeedback CanEditResults(IsFromBaseEmitter() == false);
	CanEditResults.UserFeedback = CanEditResults.bResult == false ? LOCTEXT("RendererPropertiesIsFromBaseEmitter", "This renderer's properties were added in the parent emitter and can not be edited.") : FText::GetEmpty();
	return CanEditResults;
}

TSharedRef<SWidget> FNiagaraHierarchyInputParameterHierarchyDragDropOp::CreateCustomDecorator() const
{
	TArray<FDraggedElementEntry> TopLevelEntries = GetDragContext().GetTopLevelEntries();

	if(TopLevelEntries.Num() == 1 && TopLevelEntries[0].Element.IsValid())
	{
		TSharedPtr<FNiagaraModuleInputViewModel> InputViewModel = StaticCastSharedPtr<FNiagaraModuleInputViewModel>(TopLevelEntries[0].Element.Pin());
		if(TOptional<FInputData> InputData = InputViewModel->GetInputData())
		{
			return FNiagaraParameterUtilities::GetParameterWidget(FNiagaraVariable(InputData->Type, InputData->InputName), false, false);
		}
		return SNullWidget::NullWidget;
	}

	TSharedRef<SVerticalBox> MultiDecorator = SNew(SVerticalBox);
	for(const FDraggedElementEntry& Entry : TopLevelEntries)
	{
		if(Entry.Element.IsValid())
		{
			TSharedPtr<FNiagaraModuleInputViewModel> InputViewModel = StaticCastSharedPtr<FNiagaraModuleInputViewModel>(Entry.Element.Pin());
			if(TOptional<FInputData> InputData = InputViewModel->GetInputData())
			{
				MultiDecorator->AddSlot()
				.AutoHeight()
				.Padding(1.f)
				[
					FNiagaraParameterUtilities::GetParameterWidget(FNiagaraVariable(InputData->Type, InputData->InputName), false, false)
				];
			}
		}
	}

	return MultiDecorator;
}

FString FNiagaraHierarchyPropertyViewModel::ToString() const
{
	return GetData()->GetPersistentIdentity().Names[0].ToString();
}

UHierarchyRoot* UNiagaraSummaryViewViewModel::GetHierarchyRoot() const
{
	UHierarchyRoot* RootItem = GetEmitterViewModel()->GetEditorData().GetSummaryRoot();

	ensure(RootItem != nullptr);
	return RootItem;
}

TSharedPtr<FHierarchyElementViewModel> UNiagaraSummaryViewViewModel::CreateCustomViewModelForElement(UHierarchyElement* ItemBase, TSharedPtr<FHierarchyElementViewModel> Parent)
{	
	if(UNiagaraHierarchyModuleInput* SummaryViewItem = Cast<UNiagaraHierarchyModuleInput>(ItemBase))
	{
		return MakeShared<FNiagaraModuleInputViewModel>(SummaryViewItem, Parent.ToSharedRef(), this);
	}
	else if(UNiagaraHierarchyModule* Module = Cast<UNiagaraHierarchyModule>(ItemBase))
	{
		return MakeShared<FNiagaraFunctionViewModel>(Module, Parent.ToSharedRef(), this);
	}
	else if(UNiagaraHierarchyRenderer* Renderer = Cast<UNiagaraHierarchyRenderer>(ItemBase))
	{
		return MakeShared<FNiagaraHierarchyRendererViewModel>(Renderer, Parent.ToSharedRef(), this);
	}
	else if(UNiagaraHierarchyEmitterProperties* EmitterProperties = Cast<UNiagaraHierarchyEmitterProperties>(ItemBase))
	{
		return MakeShared<FNiagaraHierarchyEmitterPropertiesViewModel>(EmitterProperties, Parent.ToSharedRef(), this);
	}
	else if(UNiagaraHierarchyEventHandler* EventHandler = Cast<UNiagaraHierarchyEventHandler>(ItemBase))
	{
		return MakeShared<FNiagaraHierarchyEventHandlerViewModel>(EventHandler, Parent.ToSharedRef(), this);
	}
	else if(UNiagaraHierarchyEventHandlerProperties* EventHandlerProperties = Cast<UNiagaraHierarchyEventHandlerProperties>(ItemBase))
	{
		return MakeShared<FNiagaraHierarchyEventHandlerPropertiesViewModel>(EventHandlerProperties, Parent.ToSharedRef(), this);
	}
	else if(UNiagaraHierarchySimStage* SimStage = Cast<UNiagaraHierarchySimStage>(ItemBase))
	{
		return MakeShared<FNiagaraHierarchySimStageViewModel>(SimStage, Parent.ToSharedRef(), this);
	}
	else if(UNiagaraHierarchySimStageProperties* SimStageProperties = Cast<UNiagaraHierarchySimStageProperties>(ItemBase))
	{
		return MakeShared<FNiagaraHierarchySimStagePropertiesViewModel>(SimStageProperties, Parent.ToSharedRef(), this);
	}
	else if(UNiagaraHierarchyObjectProperty* ObjectProperty = Cast<UNiagaraHierarchyObjectProperty>(ItemBase))
	{
		return MakeShared<FNiagaraHierarchyPropertyViewModel>(ObjectProperty, Parent.ToSharedRef(), this);
	}
	else if(UNiagaraHierarchyAssignmentInput* AssignmentInput = Cast<UNiagaraHierarchyAssignmentInput>(ItemBase))
	{
		return MakeShared<FNiagaraAssignmentInputViewModel>(AssignmentInput, Parent.ToSharedRef(), this);
	}
	else if(UHierarchyCategory* Category = Cast<UHierarchyCategory>(ItemBase))
	{
		return MakeShared<FNiagaraHierarchySummaryCategoryViewModel>(Category, Parent.ToSharedRef(), this);
	}
	
	return nullptr;
}

void UNiagaraSummaryViewViewModel::PrepareSourceItems(UHierarchyRoot* SourceRoot, TSharedPtr<FHierarchyRootViewModel> SourceRootViewModel)
{
	TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = EmitterViewModelWeak.Pin();

	TArray<UHierarchyElement*> NewItems;
	TArray<UHierarchySection*> Sections;
	
	FunctionCallCache.Empty();

	const FName EmitterSpawnSectionName = "Emitter Spawn";
	const FName EmitterUpdateSectionName = "Emitter Update";
	const FName ParticleSpawnSectionName = "Particle Spawn";
	const FName ParticleUpdateSectionName = "Particle Update";
	const FName EventsSectionName = "Events";
	const FName SimStagesSectionName = "Sim Stages";
	const FName RenderersSectionName = "Renderers";
	
	TMap<ENiagaraScriptUsage, FName> ScriptUsageSectionNameMap;
	ScriptUsageSectionNameMap.Add(ENiagaraScriptUsage::EmitterSpawnScript, EmitterSpawnSectionName);
	ScriptUsageSectionNameMap.Add(ENiagaraScriptUsage::EmitterUpdateScript, EmitterUpdateSectionName);
	ScriptUsageSectionNameMap.Add(ENiagaraScriptUsage::ParticleSpawnScript, ParticleSpawnSectionName);
	ScriptUsageSectionNameMap.Add(ENiagaraScriptUsage::ParticleUpdateScript, ParticleUpdateSectionName);

	TMap<FName, FName> ScriptUsageSectionIconMap;
	ScriptUsageSectionIconMap.Add(ScriptUsageSectionNameMap[ENiagaraScriptUsage::EmitterSpawnScript], "NiagaraEditor.Emitter.SpawnIcon");
	ScriptUsageSectionIconMap.Add(ScriptUsageSectionNameMap[ENiagaraScriptUsage::EmitterUpdateScript], "NiagaraEditor.Emitter.UpdateIcon");
	ScriptUsageSectionIconMap.Add(ScriptUsageSectionNameMap[ENiagaraScriptUsage::ParticleSpawnScript], "NiagaraEditor.Particle.SpawnIcon");
	ScriptUsageSectionIconMap.Add(ScriptUsageSectionNameMap[ENiagaraScriptUsage::ParticleUpdateScript], "NiagaraEditor.Particle.UpdateIcon");
	ScriptUsageSectionIconMap.Add(EventsSectionName, "NiagaraEditor.EventIcon");
	ScriptUsageSectionIconMap.Add(SimStagesSectionName, "NiagaraEditor.SimulationStageIcon");
	ScriptUsageSectionIconMap.Add(RenderersSectionName, "NiagaraEditor.RenderIcon");
	
	auto CreateAndInitializeSectionData = [this, SourceRoot, &Sections](FName SectionName) -> UHierarchySection*
	{
		UHierarchySection* Section = nullptr;

		auto* FoundHierarchySection = SourceRoot->GetSectionDataMutable().FindByPredicate([SectionName](UHierarchySection* Candidate)
		{
			return Candidate->GetSectionName().IsEqual(SectionName);
		});

		if(FoundHierarchySection == nullptr)
		{
			Section = NewObject<UHierarchySection>(SourceRoot, GetSectionDataClass());
			Section->SetSectionName(SectionName);
			SourceRoot->GetSectionDataMutable().Add(Section);
		}
		else
		{
			Section = *FoundHierarchySection;
		}

		ensure(Section);
		Sections.Add(Section);
		return Section;
	};
	
	for(auto& SectionNamePair : ScriptUsageSectionNameMap)
	{
		CreateAndInitializeSectionData(SectionNamePair.Value);
	}
	
	CreateAndInitializeSectionData(EventsSectionName);
	CreateAndInitializeSectionData(SimStagesSectionName);
	CreateAndInitializeSectionData(RenderersSectionName);

	// We create the view models now
	SourceRootViewModel->SyncViewModelsToData();
	
	TMap<FName, TSharedPtr<FHierarchySectionViewModel>> SectionViewModelMap;
	TMap<FName, int32> SectionElementCountMap;
	for(TSharedPtr<FHierarchySectionViewModel> SectionViewModel : SourceRootViewModel->GetSectionViewModels())
	{
		SectionViewModelMap.Add(SectionViewModel->GetSectionName(), SectionViewModel);
	}

	// Now we set their images
	for(const auto& SectionViewModelMapEntry : SectionViewModelMap)
	{
		SectionViewModelMapEntry.Value->SetSectionImage(FNiagaraEditorStyle::Get().GetBrush(ScriptUsageSectionIconMap[SectionViewModelMapEntry.Key]));
	}
	
	// we keep track of renderers & sim stages here for the same reasons
	TArray<UNiagaraHierarchyRenderer*> HierarchyRenderers;
	TArray<UNiagaraHierarchySimStage*> HierarchySimStages;
	TArray<UNiagaraHierarchyEventHandler*> HierarchyEventHandlers;

	TArray<UNiagaraNodeFunctionCall*> SimStageModules = FNiagaraStackGraphUtilities::GetAllSimStagesModuleNodes(EmitterViewModel.ToSharedRef());
	TArray<UNiagaraNodeFunctionCall*> EventHandlerModules = FNiagaraStackGraphUtilities::GetAllEventHandlerModuleNodes(EmitterViewModel.ToSharedRef());

	FHierarchyElementIdentity EmitterPropertiesIdentity;
	EmitterPropertiesIdentity.Guids.Add(EmitterViewModel->GetEmitter().Version);
	EmitterPropertiesIdentity.Names.Add("Category");
	EmitterPropertiesIdentity.Names.Add("Properties");

	UNiagaraHierarchyEmitterProperties* EmitterProperties =  SourceRoot->AddChild<UNiagaraHierarchyEmitterProperties>();
	EmitterProperties->Initialize(EmitterViewModel->GetEmitter());

	NewItems.Add(EmitterProperties);
	
	// We create hierarchy modules here. We attempt to maintain as many previous elements as possible in order to maintain UI state
	for(UNiagaraNodeFunctionCall* ModuleNode : FNiagaraStackGraphUtilities::GetAllModuleNodes(EmitterViewModel.ToSharedRef()))
	{
		// we skip over sim stage modules here as we want to add them to their respective sim stage group items instead
		if(SimStageModules.Contains(ModuleNode) || EventHandlerModules.Contains(ModuleNode))
		{
			continue;
		}
		
		ENiagaraScriptUsage ScriptUsage = FNiagaraStackGraphUtilities::GetOutputNodeUsage(*ModuleNode);

		UNiagaraHierarchyModule* HierarchyModule = SourceRoot->AddChild<UNiagaraHierarchyModule>();
		HierarchyModule->Initialize(*ModuleNode);		
		NewItems.Add(HierarchyModule);
		
		if(TSharedPtr<FHierarchySectionViewModel> SectionViewModel = SectionViewModelMap[ScriptUsageSectionNameMap[ScriptUsage]])
		{
			FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchyModule->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
			SectionAssociation->Section = Cast<UHierarchySection>(SectionViewModel->GetData());
			SectionElementCountMap.FindOrAdd(SectionViewModel->GetSectionName())++;
		}
	}
	
	for(const FNiagaraEventScriptProperties& ScriptPropertiesItem : EmitterViewModel->GetEmitter().GetEmitterData()->GetEventHandlers())
	{
		UNiagaraHierarchyEventHandler* HierarchyEventHandler = nullptr;
		FHierarchyElementIdentity EventHandlerIdentity;
		EventHandlerIdentity.Guids.Add(ScriptPropertiesItem.Script->GetUsageId());
		EventHandlerIdentity.Guids.Add(ScriptPropertiesItem.SourceEmitterID);
		
		auto* FoundItem = SourceRoot->GetChildrenMutable().FindByPredicate([EventHandlerIdentity](UHierarchyElement* ItemBase)
		{
			return ItemBase->GetPersistentIdentity() == EventHandlerIdentity;
		});
		
		if(FoundItem == nullptr)		
		{
			HierarchyEventHandler = SourceRoot->AddChild<UNiagaraHierarchyEventHandler>();
			HierarchyEventHandler->Initialize(ScriptPropertiesItem);
		}
		else
		{
			HierarchyEventHandler = CastChecked<UNiagaraHierarchyEventHandler>(*FoundItem);
		}
		
		NewItems.Add(HierarchyEventHandler);
		HierarchyEventHandlers.Add(HierarchyEventHandler);
		
		if(TSharedPtr<FHierarchySectionViewModel> SectionViewModel = SectionViewModelMap[EventsSectionName])
		{
			FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchyEventHandler->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
			SectionAssociation->Section = Cast<UHierarchySection>(SectionViewModel->GetData());
			SectionElementCountMap.FindOrAdd(SectionViewModel->GetSectionName())++;
		}
	}

	// We add sim stages here
	for(UNiagaraSimulationStageBase* SimStage : EmitterViewModel->GetEmitter().GetEmitterData()->GetSimulationStages())
	{
		UNiagaraHierarchySimStage* HierarchySimStage = nullptr;
		FHierarchyElementIdentity SimStageID;
		SimStageID.Guids.Add(SimStage->GetMergeId());

		auto* FoundItem = SourceRoot->GetChildrenMutable().FindByPredicate([SimStageID](UHierarchyElement* ItemBase)
		{
			return ItemBase->GetPersistentIdentity() == SimStageID;
		});
		
		if(FoundItem == nullptr)		
		{
			HierarchySimStage = SourceRoot->AddChild<UNiagaraHierarchySimStage>();
			HierarchySimStage->Initialize(*SimStage);
		}
		else
		{
			HierarchySimStage = CastChecked<UNiagaraHierarchySimStage>(*FoundItem);
		}
		
		NewItems.Add(HierarchySimStage);
		HierarchySimStages.Add(HierarchySimStage);
		
		if(TSharedPtr<FHierarchySectionViewModel> SectionViewModel = SectionViewModelMap[SimStagesSectionName])
		{
			FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchySimStage->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
			SectionAssociation->Section = Cast<UHierarchySection>(SectionViewModel->GetData());
			SectionElementCountMap.FindOrAdd(SectionViewModel->GetSectionName())++;
		}
	}
	
	// We create hierarchy renderers here
	for(UNiagaraRendererProperties* RendererProperties : EmitterViewModel->GetEmitter().GetEmitterData()->GetRenderers())
	{
		UNiagaraHierarchyRenderer* HierarchyRenderer = nullptr;
		FHierarchyElementIdentity RendererIdentity;
		RendererIdentity.Guids.Add(RendererProperties->GetMergeId());

		auto* FoundItem = SourceRoot->GetChildrenMutable().FindByPredicate([RendererIdentity](UHierarchyElement* ItemBase)
		{
			return ItemBase->GetPersistentIdentity() == RendererIdentity;
		});
		
		if(FoundItem == nullptr)		
		{
			HierarchyRenderer = SourceRoot->AddChild<UNiagaraHierarchyRenderer>();
			HierarchyRenderer->Initialize(*RendererProperties);
		}
		else
		{
			HierarchyRenderer = CastChecked<UNiagaraHierarchyRenderer>(*FoundItem);
		}
		
		NewItems.Add(HierarchyRenderer);
		HierarchyRenderers.Add(HierarchyRenderer);
		
		if(TSharedPtr<FHierarchySectionViewModel> SectionViewModel = SectionViewModelMap[RenderersSectionName])
		{
			FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchyRenderer->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
			SectionAssociation->Section = Cast<UHierarchySection>(SectionViewModel->GetData());
			SectionElementCountMap.FindOrAdd(SectionViewModel->GetSectionName())++;
		}
	}

	// If any section has no actual elements, we delete it again
	TArray<TSharedPtr<FHierarchySectionViewModel>> TmpSectionViewModels = SourceRootViewModel->GetSectionViewModels();
	for(auto It = TmpSectionViewModels.CreateIterator(); It; ++It)
	{
		FName SectionName = (*It)->GetSectionName();
		if(SectionElementCountMap.Contains(SectionName) == false || SectionElementCountMap[SectionName] == 0)
		{
			(*It)->Delete();
		}
	}
	
	SourceRootViewModel->SyncViewModelsToData();
}

void UNiagaraSummaryViewViewModel::SetupCommands()
{
	// no custom commands yet
}

TSharedRef<FHierarchyDragDropOp> UNiagaraSummaryViewViewModel::CreateDragDropOp(const FDataHierarchyDragDropContext& DragContext)
{
	// Only use the specialized op when all top-level entries are module inputs
	bool bAllModuleInputs = !DragContext.IsEmpty();
	for(const FDraggedElementEntry& Entry : DragContext.GetTopLevelEntries())
	{
		if(!Entry.Element.IsValid() || !Entry.Element.Pin()->GetData()->IsA<UNiagaraHierarchyModuleInput>())
		{
			bAllModuleInputs = false;
			break;
		}
	}

	if(bAllModuleInputs)
	{
		TSharedRef<FNiagaraHierarchyInputParameterHierarchyDragDropOp> DragDropOp = MakeShared<FNiagaraHierarchyInputParameterHierarchyDragDropOp>(DragContext);
		DragDropOp->Construct();
		return DragDropOp;
	}

	TSharedRef<FHierarchyDragDropOp> DragDropOp = MakeShared<FHierarchyDragDropOp>(DragContext);
	DragDropOp->Construct();
	return DragDropOp;
}

TArray<TTuple<UClass*, FOnGetDetailCustomizationInstance>> UNiagaraSummaryViewViewModel::GetInstanceCustomizations()
{
	return {};
}

TMap<FGuid, UObject*> UNiagaraSummaryViewViewModel::GetObjectsForProperties()
{
	TMap<FGuid, UObject*> GuidToObjectMap;
	for(UNiagaraRendererProperties* RendererProperties : GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetRenderers())
	{
		GuidToObjectMap.Add(RendererProperties->GetMergeId(), RendererProperties);
	}

	for(UNiagaraSimulationStageBase* SimulationStage : GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetSimulationStages())
	{
		GuidToObjectMap.Add(SimulationStage->GetMergeId(), SimulationStage);
	}

	return GuidToObjectMap;
}

void UNiagaraSummaryViewViewModel::OnScriptGraphChanged(const FEdGraphEditAction& Action, const UNiagaraScript& Script)
{
	// since OnScriptGraphChanged will be called many times in a row, we avoid refreshing too much by requesting a full refresh for next frame instead
	// it will request a single refresh regardless of how often this is called until the refresh is done
	if((Action.Action & EEdGraphActionType::GRAPHACTION_RemoveNode) != 0)
	{
 		for(const UEdGraphNode* RemovedNode : Action.Nodes)
		{
			if(FunctionCallCache.Contains(RemovedNode->NodeGuid))
			{
				FunctionCallCache.Remove(RemovedNode->NodeGuid);
			}
		}
	}
	
	RequestFullRefreshNextFrame();
}

void UNiagaraSummaryViewViewModel::OnRenderersChanged()
{
	RequestFullRefreshNextFrame();
}

void UNiagaraSummaryViewViewModel::OnSimStagesChanged()
{
	RequestFullRefreshNextFrame();
}

void UNiagaraSummaryViewViewModel::OnEventHandlersChanged()
{
	RequestFullRefreshNextFrame();
}

UNiagaraNodeFunctionCall* UNiagaraSummaryViewViewModel::GetFunctionCallNode(const FGuid& NodeIdentity)
{
	if(FunctionCallCache.Contains(NodeIdentity))
	{
		if(FunctionCallCache[NodeIdentity].IsValid())
		{
			return FunctionCallCache[NodeIdentity].Get();
		}
		else
		{
			FunctionCallCache.Remove(NodeIdentity);
		}
	}

	if(UNiagaraNodeFunctionCall* FoundFunctionCall = FNiagaraStackGraphUtilities::FindFunctionCallNode(NodeIdentity, GetEmitterViewModel()))
	{
		FunctionCallCache.Add(NodeIdentity, FoundFunctionCall);
		return FoundFunctionCall;
	}

	return nullptr;
}

void UNiagaraSummaryViewViewModel::ClearFunctionCallNodeCache(const FGuid& NodeIdentity)
{
	if(FunctionCallCache.Contains(NodeIdentity))
	{
		FunctionCallCache.Remove(NodeIdentity);
	}
}

TOptional<FInputData> UNiagaraSummaryViewViewModel::GetInputData(const UNiagaraHierarchyModuleInput& Input)
{
	if(UNiagaraNodeFunctionCall* FunctionCall = GetFunctionCallNode(Input.GetPersistentIdentity().Guids[0]))
	{
		if(UNiagaraGraph* CalledGraph = FunctionCall->GetCalledGraph())
		{
			if(UNiagaraScriptVariable* MatchingScriptVariable = CalledGraph->GetScriptVariable(Input.GetPersistentIdentity().Guids[1]))
			{
				FInputData InputData;
				InputData.InputName = MatchingScriptVariable->Variable.GetName();
				InputData.Type = MatchingScriptVariable->Variable.GetType();
				InputData.MetaData = MatchingScriptVariable->Metadata;
				InputData.bIsStatic = FNiagaraParameterHandle(InputData.InputName).IsModuleHandle() ? false : true;
				InputData.FunctionCallNode = FunctionCall;
				InputData.ChildrenInputGuids = CalledGraph->GetChildScriptVariableGuidsForInput(MatchingScriptVariable->Metadata.GetVariableGuid());
				return InputData;
			}			
		}
	}

	return TOptional<FInputData>();
}

void SNiagaraHierarchyModule::Construct(const FArguments& InArgs, TSharedPtr<FNiagaraFunctionViewModel> InModuleViewModel)
{
	ModuleViewModel = InModuleViewModel;
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f)
		[
			SNew(SImage)
			.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.DynamicInput"))
			.ToolTipText(LOCTEXT("DynamicInputIconTooltip", "Dynamic Inputs can not be dragged directly into the hierarchy. Please use the entire module input or individual inputs beneath."))
			.Visibility(InModuleViewModel->IsDynamicInput() ? EVisibility::Visible : EVisibility::Collapsed)
		]
		+ SHorizontalBox::Slot()
		[
			SAssignNew(InlineEditableTextBlock, SInlineEditableTextBlock)
			.Text(this, &SNiagaraHierarchyModule::GetModuleDisplayName)
			.IsReadOnly(true)
		]
	];
}

FText SNiagaraHierarchyModule::GetModuleDisplayName() const
{
	if(ModuleViewModel.IsValid())
	{
		if(ModuleViewModel.Pin()->GetFunctionCallNode().IsValid())
		{
			return ModuleViewModel.Pin()->GetFunctionCallNode()->GetNodeTitle(ENodeTitleType::ListView);
		}
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
