// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/HierarchyEditor/NiagaraUserParametersHierarchyViewModel.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "SDropTarget.h"
#include "Customizations/NiagaraScriptVariableCustomization.h"
#include "ViewModels/NiagaraSystemViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraUserParametersHierarchyViewModel)

#define LOCTEXT_NAMESPACE "NiagaraUserParametersHierarchyEditor"

void UNiagaraHierarchyUserParameter::Initialize(UNiagaraScriptVariable& InUserParameterScriptVariable)
{
	UserParameterScriptVariable = &InUserParameterScriptVariable;
	SetIdentity(FHierarchyElementIdentity({InUserParameterScriptVariable.Metadata.GetVariableGuid()}, {}));
}

UObject* FNiagaraHierarchyUserParameterViewModel::GetObjectForEditing()
{
	UNiagaraHierarchyUserParameter* HierarchyUserParameter = Cast<UNiagaraHierarchyUserParameter>(GetDataMutable<UNiagaraHierarchyUserParameter>());
	FNiagaraVariable ContainedVariable = HierarchyUserParameter->GetUserParameter();
	UNiagaraUserParametersHierarchyViewModel* UserParametersHierarchyViewModel = Cast<UNiagaraUserParametersHierarchyViewModel>(GetHierarchyViewModel());
	UserParametersHierarchyViewModel->GetSystemViewModel()->GetSystem().GetExposedParameters().RedirectUserVariable(ContainedVariable);
	TObjectPtr<UNiagaraScriptVariable> ScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(ContainedVariable, UserParametersHierarchyViewModel->GetSystemViewModel());
	return ScriptVariable.Get();
}

bool FNiagaraHierarchyUserParameterViewModel::DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const
{
	const UNiagaraHierarchyUserParameterRefreshContext* UserParameterRefreshContext = CastChecked<UNiagaraHierarchyUserParameterRefreshContext>(Context);
	const UNiagaraSystem* System = UserParameterRefreshContext->GetSystem();
	return FNiagaraEditorUtilities::UserParameters::FindScriptVariableForUserParameter(GetData()->GetPersistentIdentity().Guids[0], *System) != nullptr;
}

TSharedRef<FNiagaraSystemViewModel> UNiagaraUserParametersHierarchyViewModel::GetSystemViewModel() const
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModelPinned = SystemViewModelWeak.Pin();
	checkf(SystemViewModelPinned.IsValid(), TEXT("System view model destroyed before user parameters hierarchy view model."));
	return SystemViewModelPinned.ToSharedRef();
}

void UNiagaraUserParametersHierarchyViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModelWeak = InSystemViewModel;
	UDataHierarchyViewModelBase::Initialize();

	UNiagaraHierarchyUserParameterRefreshContext* UserParameterRefreshContext = NewObject<UNiagaraHierarchyUserParameterRefreshContext>(this);
	UserParameterRefreshContext->SetSystem(&InSystemViewModel->GetSystem());

	SetRefreshContext(UserParameterRefreshContext);
	
	UNiagaraSystemEditorData& SystemEditorData = InSystemViewModel->GetEditorData();
	SystemEditorData.OnUserParameterScriptVariablesSynced().AddUObject(this, &UNiagaraUserParametersHierarchyViewModel::ForceFullRefresh);
}

void UNiagaraUserParametersHierarchyViewModel::FinalizeInternal()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
	if(SystemViewModel.IsValid())
	{
		GetSystemViewModel()->GetSystem().GetExposedParameters().RemoveAllOnChangedHandlers(this);
		UNiagaraSystemEditorData* SystemEditorData = Cast<UNiagaraSystemEditorData>(GetSystemViewModel()->GetSystem().GetEditorData());
		SystemEditorData->OnUserParameterScriptVariablesSynced().RemoveAll(this);
	}
}

TSharedRef<SWidget> FNiagaraUserParameterHierarchyDragDropOp::CreateCustomDecorator() const
{
	TArray<FDraggedElementEntry> TopLevelEntries = GetDragContext().GetTopLevelEntries();

	if(TopLevelEntries.Num() == 1 && TopLevelEntries[0].Element.IsValid())
	{
		if(const UNiagaraHierarchyUserParameter* UserParameter = Cast<UNiagaraHierarchyUserParameter>(TopLevelEntries[0].Element.Pin()->GetData()))
		{
			return FNiagaraParameterUtilities::GetParameterWidget(UserParameter->GetUserParameter(), true, false);
		}
	}

	TSharedRef<SVerticalBox> MultiDecorator = SNew(SVerticalBox);
	for(const FDraggedElementEntry& Entry : TopLevelEntries)
	{
		if(Entry.Element.IsValid())
		{
			if(const UNiagaraHierarchyUserParameter* UserParameter = Cast<UNiagaraHierarchyUserParameter>(Entry.Element.Pin()->GetData()))
			{
				MultiDecorator->AddSlot()
				.AutoHeight()
				.Padding(1.f)
				[
					FNiagaraParameterUtilities::GetParameterWidget(UserParameter->GetUserParameter(), true, false)
				];
			}
		}
	}

	return MultiDecorator;
}

UHierarchyRoot* UNiagaraUserParametersHierarchyViewModel::GetHierarchyRoot() const
{
	UHierarchyRoot* RootItem = GetSystemViewModel()->GetEditorData().UserParameterHierarchy;

	ensure(RootItem != nullptr);
	return RootItem;
}

TSharedPtr<FHierarchyElementViewModel> UNiagaraUserParametersHierarchyViewModel::CreateCustomViewModelForElement(UHierarchyElement* ItemBase, TSharedPtr<FHierarchyElementViewModel> Parent)
{
	if(UNiagaraHierarchyUserParameter* UserParameter = Cast<UNiagaraHierarchyUserParameter>(ItemBase))
	{
		return MakeShared<FNiagaraHierarchyUserParameterViewModel>(UserParameter, Parent.ToSharedRef(), this);
	}
	
	return nullptr;
}

void UNiagaraUserParametersHierarchyViewModel::PrepareSourceItems(UHierarchyRoot* SourceRoot, TSharedPtr<FHierarchyRootViewModel>)
{	
	TArray<FNiagaraVariable> UserParameters;
	GetSystemViewModel()->GetSystem().GetExposedParameters().GetUserParameters(UserParameters);

	// we ensure we have one data child per user parameter, and we get rid of deleted ones
	for(FNiagaraVariable& UserParameter : UserParameters)
	{
		FNiagaraVariable RedirectedUserParameter = UserParameter;
		GetSystemViewModel()->GetSystem().GetExposedParameters().RedirectUserVariable(RedirectedUserParameter);
		TObjectPtr<UNiagaraScriptVariable> ScriptVariable = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(RedirectedUserParameter, GetSystemViewModel());
		
		// since the source items are transient we need to create them here and keep them around until the end of the tool's lifetime
		UNiagaraHierarchyUserParameter* UserParameterHierarchyObject = NewObject<UNiagaraHierarchyUserParameter>(SourceRoot);
		UserParameterHierarchyObject->Initialize(*ScriptVariable.Get());
		SourceRoot->GetChildrenMutable().Add(UserParameterHierarchyObject);
	}
}

void UNiagaraUserParametersHierarchyViewModel::SetupCommands()
{
	// no custom commands yet
}

TSharedRef<FHierarchyDragDropOp> UNiagaraUserParametersHierarchyViewModel::CreateDragDropOp(const FDataHierarchyDragDropContext& DragContext)
{
	TSharedRef<FNiagaraUserParameterHierarchyDragDropOp> DragDropOp = MakeShared<FNiagaraUserParameterHierarchyDragDropOp>(DragContext);
	DragDropOp->Construct();
	return DragDropOp;
}

TArray<TTuple<UClass*, FOnGetDetailCustomizationInstance>> UNiagaraUserParametersHierarchyViewModel::GetInstanceCustomizations()
{
	return {{UNiagaraScriptVariable::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraScriptVariableHierarchyDetails::MakeInstance)}};
}

#undef LOCTEXT_NAMESPACE
