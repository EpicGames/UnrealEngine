// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraBakerViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/SNiagaraBakerWidget.h"
#include "NiagaraBakerRenderer.h"
#include "NiagaraComponent.h"
#include "NiagaraPlatformSet.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "NiagaraWorldManager.h"

#include "NiagaraDataInterfaceRenderTarget2D.h"

#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "AssetToolsModule.h"
#include "EngineModule.h"
#include "LegacyScreenPercentageDriver.h"
#include "PackageTools.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"

#include "Factories/Texture2dFactoryNew.h"

#define LOCTEXT_NAMESPACE "NiagaraBakerViewModel"

FNiagaraBakerViewModel::FNiagaraBakerViewModel()
{
}

FNiagaraBakerViewModel::~FNiagaraBakerViewModel()
{
}

void FNiagaraBakerViewModel::Initialize(TWeakPtr<FNiagaraSystemViewModel> InWeakSystemViewModel)
{
	WeakSystemViewModel = InWeakSystemViewModel;

	UNiagaraSystem* NiagaraSystem = WeakSystemViewModel.Pin()->GetPreviewComponent()->GetAsset();
	BakerRenderer.Reset(new FNiagaraBakerRenderer(NiagaraSystem));
}

void FNiagaraBakerViewModel::SetDisplayTimeFromNormalized(float NormalizeTime)
{
	if ( Widget )
	{
		if (const UNiagaraBakerSettings* GeneratedSettings = GetBakerGeneratedSettings())
		{
			const float RelativeTime = FMath::Clamp(NormalizeTime, 0.0f, 1.0f) * GeneratedSettings->DurationSeconds;
			Widget->SetPreviewRelativeTime(RelativeTime);
		}
		else if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
		{
			const float RelativeTime = FMath::Clamp(NormalizeTime, 0.0f, 1.0f) * BakerSettings->DurationSeconds;
			Widget->SetPreviewRelativeTime(RelativeTime);
		}
	}
}

TSharedPtr<class SWidget> FNiagaraBakerViewModel::GetWidget()
{
	if (!Widget.IsValid())
	{
		Widget = SNew(SNiagaraBakerWidget)
			.WeakViewModel(this->AsShared());
	}

	return Widget;
}

void FNiagaraBakerViewModel::TogglePlaybackLooping()
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("ToggleLooping", "Toggle Looping"));
		BakerSettings->Modify();
		BakerSettings->bPreviewLooping = !BakerSettings->bPreviewLooping;
	}
}

bool FNiagaraBakerViewModel::IsPlaybackLooping() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->bPreviewLooping : false;
}

bool FNiagaraBakerViewModel::ShowRenderComponentOnly() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->bRenderComponentOnly : false;
}

void FNiagaraBakerViewModel::ToggleRenderComponentOnly()
{
	if ( UNiagaraBakerSettings* BakerSettings = GetBakerSettings() )
	{
		const FScopedTransaction Transaction(LOCTEXT("SetRenderComponentOnly", "Set Render Component Only"));
		BakerSettings->Modify();
		BakerSettings->bRenderComponentOnly = !BakerSettings->bRenderComponentOnly;
	}
}

bool FNiagaraBakerViewModel::PreviewLoopedOutput() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->bPreviewLoopedOutput : false;
}

void FNiagaraBakerViewModel::TogglePreviewLoopedOutput()
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetRenderComponentOnly", "Set Render Component Only"));
		BakerSettings->Modify();
		BakerSettings->bPreviewLoopedOutput = !BakerSettings->bPreviewLoopedOutput;
	}
}

void FNiagaraBakerViewModel::SetCameraSettingsIndex(int CamerSettingsIndex)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetCameraSettingsIndex", "Set Camera Settings Index"));
		BakerSettings->Modify();
		BakerSettings->CurrentCameraIndex = FMath::Clamp(CamerSettingsIndex, 0, BakerSettings->CameraSettings.Num() - 1);
	}
}

bool FNiagaraBakerViewModel::IsCameraSettingIndex(int CamerSettingsIndex) const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->CurrentCameraIndex == CamerSettingsIndex : false;
}

void FNiagaraBakerViewModel::AddCameraBookmark()
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("AddCameraBookmark", "Add Camera Bookmark"));
		BakerSettings->Modify();
		FNiagaraBakerCameraSettings CameraSettings = BakerSettings->GetCurrentCamera();
		BakerSettings->CameraSettings.Emplace(CameraSettings);
	}
}

void FNiagaraBakerViewModel::RemoveCameraBookmark(int32 CameraIndex)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		if (CameraIndex >= int32(ENiagaraBakerViewMode::Num) && CameraIndex < BakerSettings->CameraSettings.Num())
		const FScopedTransaction Transaction(LOCTEXT("RemoveCameraBookmark", "Remove Camera Bookmark"));
		BakerSettings->Modify();
		BakerSettings->CameraSettings.RemoveAt(CameraIndex);
		BakerSettings->CurrentCameraIndex = FMath::Clamp(BakerSettings->CurrentCameraIndex, 0, BakerSettings->CameraSettings.Num() - 1);
	}
}

FText FNiagaraBakerViewModel::GetCurrentCameraModeText() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? GetCameraSettingsText(BakerSettings->CurrentCameraIndex) : FText::GetEmpty();
}

FName FNiagaraBakerViewModel::GetCurrentCameraModeIconName() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? GetCameraSettingsIconName(BakerSettings->CurrentCameraIndex) : NAME_None;
}

FSlateIcon FNiagaraBakerViewModel::GetCurrentCameraModeIcon() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? GetCameraSettingsIcon(BakerSettings->CurrentCameraIndex) : FSlateIcon();
}

FText FNiagaraBakerViewModel::GetCameraSettingsText(int32 CameraSettingsIndex) const
{
	if ( UNiagaraBakerSettings* BakerSettings = GetBakerSettings() )
	{
		if (BakerSettings->CameraSettings.IsValidIndex(CameraSettingsIndex))
		{
			FText ModeText;
			switch (BakerSettings->CameraSettings[CameraSettingsIndex].ViewMode)
			{
				case ENiagaraBakerViewMode::Perspective:	ModeText = LOCTEXT("Perspective", "Perspective"); break;
				case ENiagaraBakerViewMode::OrthoFront:		ModeText = LOCTEXT("OrthoFront", "Front"); break;
				case ENiagaraBakerViewMode::OrthoBack:		ModeText = LOCTEXT("OrthoBack", "Back"); break;
				case ENiagaraBakerViewMode::OrthoLeft:		ModeText = LOCTEXT("OrthoLeft", "Left"); break;
				case ENiagaraBakerViewMode::OrthoRight:		ModeText = LOCTEXT("OrthoRight", "Right"); break;
				case ENiagaraBakerViewMode::OrthoTop:		ModeText = LOCTEXT("OrthoTop", "Top"); break;
				case ENiagaraBakerViewMode::OrthoBottom:	ModeText = LOCTEXT("OrthoBottom", "Bottom"); break;
				default:									ModeText = LOCTEXT("Error", "Error"); break;
			}

			if (CameraSettingsIndex < int32(ENiagaraBakerViewMode::Num))
			{
				return ModeText;
			}
			else
			{
				return FText::Format(LOCTEXT("CameraBookmarkFormat", "Bookmark {0} - {1}"), FText::AsNumber(CameraSettingsIndex - int32(ENiagaraBakerViewMode::Num)), ModeText);
			}
		}
	}
	return FText::GetEmpty();
}

FName FNiagaraBakerViewModel::GetCameraSettingsIconName(int32 CameraSettingsIndex) const
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		static FName PerspectiveIcon("EditorViewport.Perspective");
		static FName TopIcon("EditorViewport.Top");
		static FName LeftIcon("EditorViewport.Left");
		static FName FrontIcon("EditorViewport.Front");
		static FName BottomIcon("EditorViewport.Bottom");
		static FName RightIcon("EditorViewport.Right");
		static FName BackIcon("EditorViewport.Back");

		if (BakerSettings->CameraSettings.IsValidIndex(CameraSettingsIndex))
		{
			switch (BakerSettings->CameraSettings[CameraSettingsIndex].ViewMode)
			{
				case ENiagaraBakerViewMode::Perspective:	return PerspectiveIcon;
				case ENiagaraBakerViewMode::OrthoFront:		return FrontIcon;
				case ENiagaraBakerViewMode::OrthoBack:		return BackIcon;
				case ENiagaraBakerViewMode::OrthoLeft:		return LeftIcon;
				case ENiagaraBakerViewMode::OrthoRight:		return RightIcon;
				case ENiagaraBakerViewMode::OrthoTop:		return TopIcon;
				case ENiagaraBakerViewMode::OrthoBottom:	return BottomIcon;
				default:									return PerspectiveIcon;
			}
		}
	}
	return NAME_None;
}

FSlateIcon FNiagaraBakerViewModel::GetCameraSettingsIcon(int32 CameraSettingsIndex) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), GetCameraSettingsIconName(CameraSettingsIndex));
}

bool FNiagaraBakerViewModel::IsCurrentCameraPerspective() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->GetCurrentCamera().ViewMode == ENiagaraBakerViewMode::Perspective : true;
}

FVector FNiagaraBakerViewModel::GetCurrentCameraLocation() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->GetCurrentCamera().ViewportLocation : FVector::ZeroVector;
}

void FNiagaraBakerViewModel::SetCurrentCameraLocation(const FVector Value)
{
	if ( UNiagaraBakerSettings* BakerSettings = GetBakerSettings() )
	{
		BakerSettings->GetCurrentCamera().ViewportLocation = Value;
	}
}

FRotator FNiagaraBakerViewModel::GetCurrentCameraRotation() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->GetCurrentCamera().ViewportRotation : FRotator::ZeroRotator;
}

void FNiagaraBakerViewModel::SetCurrentCameraRotation(const FRotator Value) const
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		BakerSettings->GetCurrentCamera().ViewportRotation = Value;
	}
}

float FNiagaraBakerViewModel::GetCameraFOV() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->GetCurrentCamera().FOV : 0.0f;
}

void FNiagaraBakerViewModel::SetCameraFOV(float InFOV)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetCameraFOV", "Set Camera FOV"));
		BakerSettings->Modify();
		BakerSettings->GetCurrentCamera().FOV = InFOV;
	}
}

float FNiagaraBakerViewModel::GetCameraOrbitDistance() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->GetCurrentCamera().OrbitDistance : 0.0f;
}

void FNiagaraBakerViewModel::SetCameraOrbitDistance(float InOrbitDistance)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetCameraOrbitDistance", "Set Camera Orbit Distance"));
		BakerSettings->Modify();
		BakerSettings->GetCurrentCamera().OrbitDistance = InOrbitDistance;
	}
}

float FNiagaraBakerViewModel::GetCameraOrthoWidth() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->GetCurrentCamera().OrthoWidth : 0.0f;
}

void FNiagaraBakerViewModel::SetCameraOrthoWidth(float InOrthoWidth)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetCameraOrthoWidth", "Set Camera Ortho Width"));
		BakerSettings->Modify();
		BakerSettings->GetCurrentCamera().OrthoWidth = InOrthoWidth;
	}
}

void FNiagaraBakerViewModel::ToggleCameraAspectRatioEnabled()
{
	if ( UNiagaraBakerSettings* BakerSettings = GetBakerSettings() )
	{
		const FScopedTransaction Transaction(LOCTEXT("ToggleUseCameraAspectRatio", "Toggle Use Camera Aspect Ratio"));
		BakerSettings->Modify();
		BakerSettings->GetCurrentCamera().bUseAspectRatio = !BakerSettings->GetCurrentCamera().bUseAspectRatio;
	}
}

bool FNiagaraBakerViewModel::IsCameraAspectRatioEnabled() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->GetCurrentCamera().bUseAspectRatio : false;
}

float FNiagaraBakerViewModel::GetCameraAspectRatio() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->GetCurrentCamera().AspectRatio : 0.0f;
}

void FNiagaraBakerViewModel::SetCameraAspectRatio(float InAspectRatio)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetCameraAspectRatio", "Set Camera Aspect Ratio"));
		BakerSettings->Modify();
		BakerSettings->GetCurrentCamera().AspectRatio = InAspectRatio;
	}
}

void FNiagaraBakerViewModel::ResetCurrentCamera()
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("ResetCamera", "Reset Camera"));
		BakerSettings->Modify();
		BakerSettings->GetCurrentCamera().ResetToDefault();
	}
}

bool FNiagaraBakerViewModel::IsBakeQualityLevel(FName QualityLevel) const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->BakeQualityLevel == QualityLevel : false;
}

void FNiagaraBakerViewModel::SetBakeQualityLevel(FName QualityLevel)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("BakeQualityLevel", "Set Bake Quality Level"));
		BakerSettings->Modify();
		BakerSettings->BakeQualityLevel = QualityLevel;
	}
}

bool FNiagaraBakerViewModel::IsSimTickRate(int TickRate) const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->FramesPerSecond == TickRate : false;
}

int FNiagaraBakerViewModel::GetSimTickRate() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->FramesPerSecond : 60;
}

void FNiagaraBakerViewModel::SetSimTickRate(int TickRate)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		TickRate = FMath::Max(TickRate, 1);
		if (BakerSettings->FramesPerSecond != TickRate)
		{
			const FScopedTransaction Transaction(LOCTEXT("SetSimTickRate", "SetSimTickRate"));
			BakerSettings->Modify();
			BakerSettings->FramesPerSecond = TickRate;
		}
	}
}

bool FNiagaraBakerViewModel::LockToSimulationFrameRate() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->bLockToSimulationFrameRate : true;
}

void FNiagaraBakerViewModel::ToggleLockToSimulationFrameRate()
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const FScopedTransaction Transaction(LOCTEXT("SetAllowMultiTick", "SetAllowMultiTick"));
		BakerSettings->Modify();
		BakerSettings->bLockToSimulationFrameRate = BakerSettings->bLockToSimulationFrameRate ? 0 : 1;
	}
}

void FNiagaraBakerViewModel::AddOutput(UClass* Class)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		UNiagaraBakerOutput* NewOutput = NewObject<UNiagaraBakerOutput>(BakerSettings, Class);

		const FScopedTransaction Transaction(LOCTEXT("AddOutput", "Add Output"));
		BakerSettings->Modify();
		BakerSettings->Outputs.Add(NewOutput);

		CurrentOutputIndex = BakerSettings->Outputs.Num() - 1;
		OnCurrentOutputChanged.Broadcast();
	}
}

void FNiagaraBakerViewModel::RemoveCurrentOutput()
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		if (BakerSettings->Outputs.IsValidIndex(CurrentOutputIndex))
		{
			const FScopedTransaction Transaction(LOCTEXT("RemoveOutput", "Remove Output"));
			BakerSettings->Modify();
			BakerSettings->Outputs[CurrentOutputIndex]->MarkAsGarbage();
			BakerSettings->Outputs.RemoveAt(CurrentOutputIndex);
			CurrentOutputIndex = FMath::Max(0, CurrentOutputIndex - 1);
			OnCurrentOutputChanged.Broadcast();
		}
	}
}

bool FNiagaraBakerViewModel::CanRemoveCurrentOutput() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings && BakerSettings->Outputs.IsValidIndex(CurrentOutputIndex);
}

UNiagaraBakerOutput* FNiagaraBakerViewModel::GetCurrentOutput() const
{
	if ( UNiagaraBakerSettings* BakerSettings = GetBakerSettings() )
	{
		if ( BakerSettings->Outputs.IsValidIndex(CurrentOutputIndex) )
		{
			return BakerSettings->Outputs[CurrentOutputIndex];
		}
	}
	return nullptr;
}

void FNiagaraBakerViewModel::SetCurrentOutputIndex(int32 OutputIndex)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		if (BakerSettings->Outputs.Num() > 0)
		{
			CurrentOutputIndex = FMath::Clamp(OutputIndex, 0, BakerSettings->Outputs.Num() - 1);
		}
		else
		{
			CurrentOutputIndex = 0;
		}
		OnCurrentOutputChanged.Broadcast();
	}
}

FText FNiagaraBakerViewModel::GetOutputText(int32 OutputIndex) const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	if ( BakerSettings && BakerSettings->Outputs.IsValidIndex(OutputIndex) )
	{
		if(BakerSettings->Outputs[OutputIndex])
		{
			return FText::FromString(BakerSettings->Outputs[OutputIndex]->OutputName);
		}
	}
	return FText::GetEmpty();
}

FText FNiagaraBakerViewModel::GetCurrentOutputText() const
{
	return GetOutputText(CurrentOutputIndex);
}

int FNiagaraBakerViewModel::GetCurrentOutputNumFrames() const
{
	if ( UNiagaraBakerSettings* BakerSettings = GetBakerSettings() )
	{
		return BakerSettings->GetOutputNumFrames(CurrentOutputIndex);
	}
	return 1;
}

FNiagaraBakerOutputFrameIndices FNiagaraBakerViewModel::GetCurrentOutputFrameIndices(float RelativeTime) const
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		return BakerSettings->GetOutputFrameIndices(CurrentOutputIndex, RelativeTime);
	}
	return FNiagaraBakerOutputFrameIndices();
}

float FNiagaraBakerViewModel::GetTimelineStart() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->StartSeconds : 0.0f;
}

void FNiagaraBakerViewModel::SetTimelineStart(float Value)
{
	if ( UNiagaraBakerSettings* BakerSettings = GetBakerSettings() )
	{
		Value = FMath::Max(Value, 0.0f);
		if ( BakerSettings->StartSeconds != Value )
		{
			const float EndSeconds = BakerSettings->StartSeconds + BakerSettings->DurationSeconds;
			const FScopedTransaction Transaction(LOCTEXT("StartSeconds", "StartSeconds"));
			BakerSettings->Modify();
			BakerSettings->StartSeconds = Value;
			BakerSettings->DurationSeconds = FMath::Max(EndSeconds - Value, 0.0f);
		}
	}
}

float FNiagaraBakerViewModel::GetDurationSeconds() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->DurationSeconds : 0.0f;
}

float FNiagaraBakerViewModel::GetTimelineEnd() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->StartSeconds + BakerSettings->DurationSeconds : 0.0f;
}

void FNiagaraBakerViewModel::SetTimelineEnd(float Value)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		Value = FMath::Max(Value - BakerSettings->StartSeconds, 0.0f);
		if (BakerSettings->DurationSeconds != Value)
		{
			const FScopedTransaction Transaction(LOCTEXT("EndSeconds", "EndSeconds"));
			BakerSettings->Modify();
			BakerSettings->DurationSeconds = Value;
		}
	}
}

int32 FNiagaraBakerViewModel::GetFramesOnX() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->FramesPerDimension.X : 1;
}

void FNiagaraBakerViewModel::SetFramesOnX(int32 Value)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		if (BakerSettings->FramesPerDimension.X != Value)
		{
			const FScopedTransaction Transaction(LOCTEXT("FramesPerDimension", "FramesPerDimension"));
			BakerSettings->Modify();
			BakerSettings->FramesPerDimension.X = Value;

			FPropertyChangedEvent PropertyEvent(UNiagaraBakerSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UNiagaraBakerSettings, FramesPerDimension)));
			BakerSettings->PostEditChangeProperty(PropertyEvent);
		}
	}
}

int32 FNiagaraBakerViewModel::GetFramesOnY() const
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	return BakerSettings ? BakerSettings->FramesPerDimension.Y : 1;
}

void FNiagaraBakerViewModel::SetFramesOnY(int32 Value)
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		if (BakerSettings->FramesPerDimension.Y != Value)
		{
			const FScopedTransaction Transaction(LOCTEXT("FramesPerDimension", "FramesPerDimension"));
			BakerSettings->Modify();
			BakerSettings->FramesPerDimension.Y = Value;

			FPropertyChangedEvent PropertyEvent(UNiagaraBakerSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UNiagaraBakerSettings, FramesPerDimension)));
			BakerSettings->PostEditChangeProperty(PropertyEvent);
		}
	}
}

FNiagaraBakerFeedbackContext FNiagaraBakerViewModel::RenderBaker()
{
	FNiagaraBakerFeedbackContext FeedbackContext;

	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	UNiagaraSystem* NiagaraSystem = BakerRenderer->GetNiagaraSystem();
	if ( BakerSettings == nullptr || NiagaraSystem == nullptr )
	{
		FeedbackContext.Errors.Emplace(TEXT("Nothing to bake as we don't have valid settings or system to bake with."));
		return FeedbackContext;
	}

	if ( BakerSettings->Outputs.Num() == 0 )
	{
		NiagaraSystem->SetBakerGeneratedSettings(nullptr);
		FeedbackContext.Errors.Emplace(TEXT("Nothing to bake as we have no outputs."));
		return FeedbackContext;
	}

	// Create output renderers
	TArray<TUniquePtr<FNiagaraBakerOutputRenderer>> OutputRenderers;
	OutputRenderers.Reserve(BakerSettings->Outputs.Num());

	bool bHasValidRenderer = false;
	for ( int iOutput=0; iOutput < BakerSettings->Outputs.Num(); ++iOutput)
	{
		UNiagaraBakerOutput* Output = BakerSettings->Outputs[iOutput];
		OutputRenderers.Emplace(FNiagaraBakerRenderer::GetOutputRenderer(Output->GetClass()));
		if ( FNiagaraBakerOutputRenderer* OutputRenderer = OutputRenderers.Last().Get() )
		{
			bHasValidRenderer |= OutputRenderer->BeginBake(FeedbackContext, Output);
		}
	}

	// Early out if we have no valid renderers
	if ( !bHasValidRenderer )
	{
		NiagaraSystem->SetBakerGeneratedSettings(nullptr);
		FeedbackContext.Errors.Emplace(TEXT("Nothing to bake as all outputs will not genreate any data."));
		return FeedbackContext;
	}

	// Render frames
	{
		const int32 TotalFrames = BakerSettings->FramesPerDimension.X * BakerSettings->FramesPerDimension.Y;
		const float FrameDeltaSeconds = BakerSettings->DurationSeconds / float(TotalFrames);

		FScopedSlowTask SlowTask(TotalFrames);
		SlowTask.MakeDialog();

		if (BakerSettings->BakeQualityLevel.IsNone() == false)
		{
			const UNiagaraSettings* NiagaraSettings = GetDefault<UNiagaraSettings>();
			for (int32 i=0; i < NiagaraSettings->QualityLevels.Num(); ++i)
			{
				if (FName(NiagaraSettings->QualityLevels[i].ToString()) == BakerSettings->BakeQualityLevel)
				{
					FNiagaraPlatformSet::SetNiagaraQualityLevelOverride(i);
					break;
				}
			}
		}

		for ( int32 iFrame=0; iFrame < TotalFrames; ++iFrame )
		{
			SlowTask.EnterProgressFrame(1, FText::Format(LOCTEXT("BakingFormat", "Baking Frame ({0} / {1})"), iFrame + 1, TotalFrames));

			// Tick to right location
			const float FrameTime = BakerSettings->StartSeconds + (float(iFrame) * FrameDeltaSeconds);
			BakerRenderer->SetAbsoluteTime(FrameTime);

			// Capture frames
			for ( int32 i=0; i < OutputRenderers.Num(); ++i )
			{
				if (OutputRenderers[i].IsValid())
				{
					OutputRenderers[i]->BakeFrame(FeedbackContext, BakerSettings->Outputs[i], iFrame, *BakerRenderer.Get());
				}
			}
		}

		if (BakerSettings->BakeQualityLevel.IsNone() == false)
		{
			FNiagaraPlatformSet::ClearNiagaraQualityLevelOverride();
		}
	}

	// Ensure we flush everything from the render thread
	if ( UWorld* World = BakerRenderer->GetWorld() )
	{
		if (FNiagaraWorldManager* WorldManager = FNiagaraWorldManager::Get(World))
		{
			WorldManager->FlushComputeAndDeferredQueues(true);
		}
	}

	// Complete bake
	{
		FScopedSlowTask SlowTask(OutputRenderers.Num());
		SlowTask.MakeDialog();

		for (int32 i = 0; i < OutputRenderers.Num(); ++i)
		{
			if (OutputRenderers[i].IsValid())
			{
				SlowTask.EnterProgressFrame(1, FText::Format(LOCTEXT("FinishBakeFormat", "Finish Bake for Output '{0}'"), FText::FromString(BakerSettings->Outputs[i]->OutputName)));

				OutputRenderers[i]->EndBake(FeedbackContext, BakerSettings->Outputs[i]);
			}
		}
	}

	// Duplicate and set as generated data
	NiagaraSystem->SetBakerGeneratedSettings(DuplicateObject<UNiagaraBakerSettings>(BakerSettings, NiagaraSystem));

	// GC to cleanup any transient objects we created or ones we have ditched
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	return FeedbackContext;
}

#undef LOCTEXT_NAMESPACE
