// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"
#include "NiagaraBakerRenderer.h"
#include "SceneView.h"
#include "Templates/SharedPointer.h"

#include "NiagaraBakerSettings.h"

class UNiagaraBakerSettings;
struct FSlateIcon;

enum class ENiagaraBakerColorChannel : uint8
{
	Red,
	Green,
	Blue,
	Alpha,
	Num
};

class FNiagaraBakerViewModel : public TSharedFromThis<FNiagaraBakerViewModel>
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnCurrentOutputChanged);

	FNiagaraBakerViewModel();
	~FNiagaraBakerViewModel();

	void Initialize(TWeakPtr<class FNiagaraSystemViewModel> WeakSystemViewModel);

	TSharedPtr<class SWidget> GetWidget();

	FNiagaraBakerFeedbackContext RenderBaker();

	void SetDisplayTimeFromNormalized(float NormalizeTime);

	FNiagaraBakerRenderer& GetBakerRenderer() const { check(BakerRenderer.IsValid()); return *BakerRenderer.Get(); }
	UNiagaraBakerSettings* GetBakerSettings() const { return BakerRenderer->GetBakerSettings(); }
	const UNiagaraBakerSettings* GetBakerGeneratedSettings() { return BakerRenderer->GetBakerGeneratedSettings(); }

	bool IsChannelEnabled(ENiagaraBakerColorChannel Channel) const { return bColorChannelEnabled[int(Channel)]; }
	void ToggleChannelEnabled(ENiagaraBakerColorChannel Channel) { bColorChannelEnabled[int(Channel)] = !bColorChannelEnabled[int(Channel)]; }
	void SetChannelEnabled(ENiagaraBakerColorChannel Channel, bool bEnabled) { bColorChannelEnabled[int(Channel)] = bEnabled; }

	void TogglePlaybackLooping();
	bool IsPlaybackLooping() const;

	bool ShowRealtimePreview() const { return bShowRealtimePreview; }
	void ToggleRealtimePreview() { bShowRealtimePreview = !bShowRealtimePreview; }

	bool ShowBakedView() const { return bShowBakedView; }
	void ToggleBakedView() { bShowBakedView = !bShowBakedView; }

	bool IsCheckerboardEnabled() const { return bCheckerboardEnabled; }
	void ToggleCheckerboardEnabled() { bCheckerboardEnabled = !bCheckerboardEnabled; }

	bool ShowInfoText() const { return bShowInfoText; }
	void ToggleInfoText() { bShowInfoText = !bShowInfoText; }

	bool ShowRenderComponentOnly() const;
	void ToggleRenderComponentOnly();

	bool PreviewLoopedOutput() const;
	void TogglePreviewLoopedOutput();

	void SetCameraSettingsIndex(int CamerSettingsIndex);
	bool IsCameraSettingIndex(int CamerSettingsIndex) const;

	void AddCameraBookmark();
	void RemoveCameraBookmark(int32 CameraIndex);

	FText GetCurrentCameraModeText() const;
	FName GetCurrentCameraModeIconName() const;
	FSlateIcon GetCurrentCameraModeIcon() const;

	FText GetCameraSettingsText(int32 CameraSettingsIndex) const;
	FName GetCameraSettingsIconName(int32 CameraSettingsIndex) const;
	FSlateIcon GetCameraSettingsIcon(int32 CameraSettingsIndex) const;

	bool IsCurrentCameraPerspective() const;
	FVector GetCurrentCameraLocation() const;
	void SetCurrentCameraLocation(const FVector Value);
	FRotator GetCurrentCameraRotation() const;
	void SetCurrentCameraRotation(const FRotator Value) const;

	float GetCameraFOV() const;
	void SetCameraFOV(float InFOV);

	float GetCameraOrbitDistance() const;
	void SetCameraOrbitDistance(float InOrbitDistance);

	float GetCameraOrthoWidth() const;
	void SetCameraOrthoWidth(float InOrthoWidth);

	void ToggleCameraAspectRatioEnabled();
	bool IsCameraAspectRatioEnabled() const;
	float GetCameraAspectRatio() const;
	void SetCameraAspectRatio(float InAspectRatio);

	void ResetCurrentCamera();

	bool IsBakeQualityLevel(FName QualityLevel) const;
	void SetBakeQualityLevel(FName QualityLevel);

	bool IsSimTickRate(int TickRate) const;
	int GetSimTickRate() const;
	void SetSimTickRate(int TickRate);

	bool LockToSimulationFrameRate() const;
	void ToggleLockToSimulationFrameRate();

	void AddOutput(UClass* Class);
	void RemoveCurrentOutput();
	bool CanRemoveCurrentOutput() const;
	bool IsCurrentOutputIndex(int32 OutputIndex) const { return OutputIndex == CurrentOutputIndex; }
	UNiagaraBakerOutput* GetCurrentOutput() const;
	int32 GetCurrentOutputIndex() const { return CurrentOutputIndex; }
	void SetCurrentOutputIndex(int32 OutputIndex);
	FText GetOutputText(int32 OutputIndex) const;
	FText GetCurrentOutputText() const;
	int GetCurrentOutputNumFrames() const;
	FNiagaraBakerOutputFrameIndices GetCurrentOutputFrameIndices(float RelativeTime) const;

	float GetTimelineStart() const;
	void SetTimelineStart(float Value);

	float GetDurationSeconds() const;
	
	float GetTimelineEnd() const;
	void SetTimelineEnd(float Value);

	int32 GetFramesOnX() const;
	void SetFramesOnX(int32 Value);
	int32 GetFramesOnY() const;
	void SetFramesOnY(int32 Value);

	float GetPlaybackRate() const { return PlaybackRate; }
	void SetPlaybackRate(float Value) { PlaybackRate = FMath::Clamp(Value, 0.0f, 10.0f); }

	FOnCurrentOutputChanged OnCurrentOutputChanged;

private:
	TWeakPtr<FNiagaraSystemViewModel> WeakSystemViewModel;
	TSharedPtr<class SNiagaraBakerWidget> Widget;
	TUniquePtr<FNiagaraBakerRenderer> BakerRenderer;

	int32 CurrentOutputIndex = 0;

	bool bShowRealtimePreview = true;
	bool bShowBakedView = true;
	bool bCheckerboardEnabled = true;		//-TODO: Move to Baker Settings?
	bool bShowInfoText = true;				//-TODO: Remove later
	bool bColorChannelEnabled[int(ENiagaraBakerColorChannel::Num)] = {true, true, true, false};

	float PlaybackRate = 1.0f;
};
