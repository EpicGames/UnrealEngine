// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraCollectionParameterViewModel.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraTypes.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraDataInterface.h"

#define LOCTEXT_NAMESPACE "CollectionParameterViewModel"

FNiagaraCollectionParameterViewModel::FNiagaraCollectionParameterViewModel(const FNiagaraVariable& Variable, UNiagaraParameterCollectionInstance* CollectionInstance, ENiagaraParameterEditMode ParameterEditMode)
	: FNiagaraParameterViewModel(ParameterEditMode)
	, CollectionInst(CollectionInstance)
	, Parameter(Variable)
{
	if (Variable.IsDataInterface())
	{
		DefaultValueType = INiagaraParameterViewModel::EDefaultValueType::DataInterface;
	}
	else if (Variable.IsUObject())
	{
		DefaultValueType = INiagaraParameterViewModel::EDefaultValueType::ObjectAsset;
	}
	else
	{
		DefaultValueType = INiagaraParameterViewModel::EDefaultValueType::Struct;
	}

	RefreshParameterValue();
}

void FNiagaraCollectionParameterViewModel::Reset()
{
	OnNameChanged().RemoveAll(this);
	OnTypeChanged().RemoveAll(this);
	OnDefaultValueChanged().RemoveAll(this);
}

FNiagaraVariable FNiagaraCollectionParameterViewModel::GetVariable() const
{
	return Parameter;
}

FName FNiagaraCollectionParameterViewModel::GetName() const
{
	if (CollectionInst && CollectionInst->GetParent())
	{
		return CollectionInst->GetParent()->FriendlyNameFromParameterName(Parameter.GetName());
	}
	else
	{
		return FName();
	}
}

FText FNiagaraCollectionParameterViewModel::GetTypeDisplayName() const
{
	return FText::Format(LOCTEXT("TypeTextFormat", "Type: {0}"), Parameter.GetType().GetNameText());
}

void FNiagaraCollectionParameterViewModel::NameTextCommitted(const FText& Name, ETextCommit::Type CommitInfo)
{
	FName NewName = CollectionInst->GetParent()->ParameterNameFromFriendlyString(Name.ToString());

	FName OldName = Parameter.GetName();
	if (!Parameter.GetName().IsEqual(NewName, ENameCase::CaseSensitive))
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("EditInputName", "Edit input name"));
		Parameter.SetName(NewName);
		OnNameChangedDelegate.Broadcast(OldName, NewName);
	}
}

bool FNiagaraCollectionParameterViewModel::VerifyNodeNameTextChanged(const FText& NewText, FText& OutErrorMessage)
{
	// Disallow empty names
	if (NewText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("NPCNameEmptyWarn", "Cannot have empty name!");
		return false;
	}

	FName NewName = CollectionInst->GetParent()->ParameterNameFromFriendlyString(NewText.ToString());

	for (FNiagaraVariable& Var : CollectionInst->GetParent()->GetParameters())
	{
		if (Var.GetName() == NewName)
		{
			OutErrorMessage = FText::Format(LOCTEXT("NPCNameConflictWarn", "\"{0}\" is already the name of another parameter in this collection."), NewText);
			return false;
		}
	}

	return true;
}

FNiagaraTypeDefinition FNiagaraCollectionParameterViewModel::GetType() const
{
	return Parameter.GetType();
}

bool FNiagaraCollectionParameterViewModel::CanChangeParameterType()const
{
	return CollectionInst->IsDefaultInstance();
}

bool FNiagaraCollectionParameterViewModel::IsEditingEnabled()const
{
	return IsProvided() == ECheckBoxState::Checked && !CollectionInst->IsExternallyDriven(Parameter);
}

bool FNiagaraCollectionParameterViewModel::CanChangeSortOrder() const
{
	return false;
}

int32 FNiagaraCollectionParameterViewModel::GetSortOrder() const
{
	return 0;
}

void FNiagaraCollectionParameterViewModel::SetSortOrder(int32 SortOrder)
{
	check(0);
}

bool FNiagaraCollectionParameterViewModel::CanRenameParameter()const
{
	return !CollectionInst->IsExternallyDriven(Parameter) && CollectionInst->IsDefaultInstance();
}

FText FNiagaraCollectionParameterViewModel::GetTooltip() const
{
	if (CollectionInst->IsExternallyDriven(Parameter))
	{
		return FText::Format(LOCTEXT("NPCExternallyDrivenTip", "\"{0}\" can not be edited as it is being driven from the parent MaterialParameterCollection"), GetNameText());
	}

	return FNiagaraParameterViewModel::GetTooltip();
}

bool FNiagaraCollectionParameterViewModel::IsOptional()const
{
	return !CollectionInst->IsExternallyDriven(Parameter) && !CollectionInst->IsDefaultInstance();
}

ECheckBoxState FNiagaraCollectionParameterViewModel::IsProvided()const
{
	ECheckBoxState State = CollectionInst->OverridesParameter(Parameter) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	return State;
}

void FNiagaraCollectionParameterViewModel::SetProvided(ECheckBoxState CheckboxState)
{
	check(!CollectionInst->IsDefaultInstance());//All values are always provided for default instances.
	
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("ChangeProvideParameter", "Provide Parameter Change"));
		CollectionInst->Modify();

		if (CheckboxState == ECheckBoxState::Checked)
		{
			CollectionInst->SetOverridesParameter(Parameter, true);
		}
		else
		{
			CollectionInst->SetOverridesParameter(Parameter, false);
			CollectionInst->SyncWithCollectionParameters({ Parameter.GetName() });
		}
	}

	OnProvidedChangedDelegate.Broadcast();
}

void FNiagaraCollectionParameterViewModel::SelectedTypeChanged(TSharedPtr<FNiagaraTypeDefinition> Item, ESelectInfo::Type SelectionType)
{
	if (Item.IsValid() && Parameter.GetType() != *Item)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("EditParameterType", "Edit type"));
		Parameter.SetType(*Item);
		OnTypeChangedDelegate.Broadcast();
	}
}

INiagaraParameterViewModel::EDefaultValueType FNiagaraCollectionParameterViewModel::GetDefaultValueType()
{
	return DefaultValueType;
}

TSharedRef<FStructOnScope> FNiagaraCollectionParameterViewModel::GetDefaultValueStruct()
{
	return ParameterValue.ToSharedRef();
}

UNiagaraDataInterface* FNiagaraCollectionParameterViewModel::GetDefaultValueDataInterface()
{
	return CollectionInst->GetParameterStore().GetDataInterface(Parameter);
}

TObjectPtr<UObject> FNiagaraCollectionParameterViewModel::GetDefaultValueObjectAsset()
{
	return CollectionInst->GetParameterStore().GetUObject(Parameter);
}

void FNiagaraCollectionParameterViewModel::SetDefaultValueObjectAsset(TObjectPtr<UObject> Object)
{
	CollectionInst->GetParameterStore().SetUObject(Object, Parameter);
}

void FNiagaraCollectionParameterViewModel::NotifyDefaultValuePropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!IsOptional() || IsProvided() == ECheckBoxState::Checked)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("EditParameterValueProperty", "Edit parameter value"));
		CollectionInst->Modify();
		switch (DefaultValueType)
		{

			case INiagaraParameterViewModel::EDefaultValueType::Struct:
				CollectionInst->GetParameterStore().SetParameterData(ParameterValue->GetStructMemory(), Parameter);
				break;

			case INiagaraParameterViewModel::EDefaultValueType::DataInterface:
				if (UNiagaraDataInterface* Interface = CollectionInst->GetParameterStore().GetDataInterface(Parameter))
				{
					Interface->Modify();
				}
				break;

			case INiagaraParameterViewModel::EDefaultValueType::ObjectAsset:
			default:
				checkNoEntry();
				break;
		}
		OnDefaultValueChangedDelegate.Broadcast();
	}
}

void FNiagaraCollectionParameterViewModel::NotifyBeginDefaultValueChange()
{
	if (!IsOptional() || IsProvided() == ECheckBoxState::Checked)
	{
		GEditor->BeginTransaction(LOCTEXT("BeginEditParameterValue", "Edit parameter value"));
		CollectionInst->Modify();
	}
}

void FNiagaraCollectionParameterViewModel::NotifyEndDefaultValueChange()
{
	if (GEditor->IsTransactionActive())
	{
		GEditor->EndTransaction();
	}
}

static bool DataMatches(const uint8* DataA, const uint8* DataB, int32 Length)
{
	for (int32 i = 0; i < Length; ++i)
	{
		if (DataA[i] != DataB[i])
		{
			return false;
		}
	}
	return true;
}

void FNiagaraCollectionParameterViewModel::NotifyDefaultValueChanged()
{
	if (!IsOptional() || IsProvided() == ECheckBoxState::Checked)
	{
		if (!Parameter.GetType().IsDataInterface() && !Parameter.GetType().IsUObject())
		{
			const uint8* ParamData = CollectionInst->GetParameterStore().GetParameterData(Parameter);
			check(ParamData);
			if (DataMatches(ParamData, ParameterValue->GetStructMemory(), Parameter.GetSizeInBytes()) == false)
			{
				FScopedTransaction ScopedTransaction(LOCTEXT("EditParameterValue", "Edit parameter value"));
				CollectionInst->Modify();
				CollectionInst->GetParameterStore().SetParameterData(ParameterValue->GetStructMemory(), Parameter);
			}
		}
		OnDefaultValueChangedDelegate.Broadcast();
	}
}

FNiagaraCollectionParameterViewModel::FOnNameChanged& FNiagaraCollectionParameterViewModel::OnNameChanged()
{
	return OnNameChangedDelegate;
}

void FNiagaraCollectionParameterViewModel::RefreshParameterValue()
{
	FNiagaraTypeDefinition Type = Parameter.GetType();

	if (!Type.IsDataInterface() && !Type.IsUObject())
	{
		ParameterValue = MakeShareable(new FStructOnScope(Parameter.GetType().GetStruct()));

		const uint8* ParamData = CollectionInst->GetParameterStore().GetParameterData(Parameter);
		//Param data can be null if this is a parameter view that is not provided in an instance
		if (ParamData)
		{
			FMemory::Memcpy(ParameterValue->GetStructMemory(), ParamData, Parameter.GetSizeInBytes());
			OnDefaultValueChangedDelegate.Broadcast();
		}
	}
}

#undef LOCTEXT_NAMESPACE 
