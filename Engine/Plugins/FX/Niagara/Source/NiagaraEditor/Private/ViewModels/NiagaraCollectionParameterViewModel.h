// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraParameterViewModel.h"
#include "NiagaraParameterCollection.h"

/** A view model for a script parameter. */
class FNiagaraCollectionParameterViewModel : public FNiagaraParameterViewModel
{
public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNameChanged, FName /*OldName*/, FName /*NewName*/);

public:

	FNiagaraCollectionParameterViewModel(const FNiagaraVariable& Variable, UNiagaraParameterCollectionInstance* CollectionInstance, ENiagaraParameterEditMode ParameterEditMode);

	void Reset();

	//~ INiagaraParameterViewmodel interface.
	virtual FNiagaraVariable GetVariable() const override;
	virtual FName GetName() const override;
	virtual void NameTextCommitted(const FText& Name, ETextCommit::Type CommitInfo) override;
	virtual bool VerifyNodeNameTextChanged(const FText& NewText, FText& OutErrorMessage) override;
	virtual FText GetTypeDisplayName() const override;
	virtual FNiagaraTypeDefinition GetType() const override;
	virtual void SelectedTypeChanged(TSharedPtr<FNiagaraTypeDefinition> Item, ESelectInfo::Type SelectionType) override;
	virtual EDefaultValueType GetDefaultValueType() override;
	virtual TSharedRef<FStructOnScope> GetDefaultValueStruct() override;
	virtual UNiagaraDataInterface* GetDefaultValueDataInterface() override;
	virtual TObjectPtr<UObject> GetDefaultValueObjectAsset() override;
	virtual void SetDefaultValueObjectAsset(TObjectPtr<UObject>) override;
	virtual void NotifyDefaultValuePropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void NotifyBeginDefaultValueChange() override;
	virtual void NotifyEndDefaultValueChange() override;
	virtual void NotifyDefaultValueChanged() override;
	virtual bool CanChangeSortOrder() const override;
	virtual int32 GetSortOrder() const override;
	virtual void SetSortOrder(int32 SortOrder) override;
	virtual bool CanRenameParameter()const override;
	virtual FText GetTooltip() const override;


	virtual bool CanChangeParameterType()const override;
	virtual bool IsEditingEnabled()const override;

	virtual bool IsOptional()const override;
	virtual ECheckBoxState IsProvided()const override;
	virtual void SetProvided(ECheckBoxState CheckboxState) override;

	/** Gets a multicast delegate which is called whenever the name of this parameter changes. */
	FOnNameChanged& OnNameChanged();

private:
	/** Refreshes the parameter value struct from the variable data. */
	void RefreshParameterValue();

private:

	/** The type of default value this parameter is providing. */
	EDefaultValueType DefaultValueType;

	/** A struct representing the value of the variable. */
	TSharedPtr<FStructOnScope> ParameterValue;

	/** A multicast delegate which is called whenever the name of the parameter changes. */
	FOnNameChanged OnNameChangedDelegate;

	UNiagaraParameterCollectionInstance* CollectionInst;
	FNiagaraVariable Parameter;
};


