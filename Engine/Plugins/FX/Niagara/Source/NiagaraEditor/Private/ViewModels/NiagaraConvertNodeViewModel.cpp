// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraConvertNodeViewModel.h"
#include "NiagaraNodeConvert.h"
#include "NiagaraConvertPinViewModel.h"
#include "NiagaraConvertPinSocketViewModel.h"
#include "EdGraphSchema_Niagara.h"

#define LOCTEXT_NAMESPACE "NiagaraConvertNodeViewModel"

FNiagaraConvertNodeViewModel::FNiagaraConvertNodeViewModel(UNiagaraNodeConvert& InConvertNode)
	: ConvertNode(InConvertNode)
	, bPinViewModelsNeedRefresh(true)
	, bConnectionViewModelsNewRefresh(true)
{
}

const TArray<TSharedRef<FNiagaraConvertPinViewModel>>& FNiagaraConvertNodeViewModel::GetInputPinViewModels()
{
	if (bPinViewModelsNeedRefresh)
	{
		RefreshPinViewModels();
	}
	return InputPinViewModels;
}

const TArray<TSharedRef<FNiagaraConvertPinViewModel>>& FNiagaraConvertNodeViewModel::GetOutputPinViewModels()
{
	if (bPinViewModelsNeedRefresh)
	{
		RefreshPinViewModels();
	}
	return OutputPinViewModels;
}

const TArray<TSharedRef<FNiagaraConvertConnectionViewModel>> FNiagaraConvertNodeViewModel::GetConnectionViewModels()
{
	if (bConnectionViewModelsNewRefresh)
	{
		RefreshConnectionViewModels();
	}
	return ConnectionViewModels;
}

TSharedPtr<FNiagaraConvertPinSocketViewModel> FNiagaraConvertNodeViewModel::GetDraggedSocketViewModel() const
{
	return DraggedSocketViewModel;
}

void FNiagaraConvertNodeViewModel::SetDraggedSocketViewModel(TSharedPtr<FNiagaraConvertPinSocketViewModel> DraggedSocket)
{
	DraggedSocketViewModel = DraggedSocket;
}

bool FNiagaraConvertNodeViewModel::IsSocketConnected(TSharedRef<const FNiagaraConvertPinSocketViewModel> Socket) const
{
	TSharedPtr<FNiagaraConvertPinViewModel> SocketPinViewModel = Socket->GetOwnerPinViewModel();
	if (SocketPinViewModel.IsValid())
	{
		TArray<FName> SocketPath = Socket->GetPath();
		for (FNiagaraConvertConnection& Connection : ConvertNode.GetConnections())
		{
			if (Socket->GetDirection() == EGPD_Input)
			{
				if (SocketPinViewModel->GetPinId() == Connection.SourcePinId && SocketPath == Connection.SourcePath)
				{
					return true;
				}
			}
			else // EGPD_Output
			{
				if (SocketPinViewModel->GetPinId() == Connection.DestinationPinId && SocketPath == Connection.DestinationPath)
				{
					return true;
				}
			}
		}
	}
	return false;
}

void FNiagaraConvertNodeViewModel::GetConnectedSockets(TSharedRef<const FNiagaraConvertPinSocketViewModel> Socket, TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& ConnectedSockets)
{
	TSharedPtr<FNiagaraConvertPinViewModel> SocketPinViewModel = Socket->GetOwnerPinViewModel();
	if (SocketPinViewModel.IsValid())
	{
		TArray<FName> SocketPath = Socket->GetPath();
		EEdGraphPinDirection ConnectedPinDirection = Socket->GetDirection() == EGPD_Input ? EGPD_Output : EGPD_Input;
		for (FNiagaraConvertConnection& Connection : ConvertNode.GetConnections())
		{
			FGuid ConnectedPinId;
			TArray<FName>* ConnectedPinPath;
			if (Socket->GetDirection() == EGPD_Input)
			{
				ConnectedPinId = Connection.DestinationPinId;
				ConnectedPinPath = &Connection.DestinationPath;
			}
			else // EGPD_Output
			{
				ConnectedPinId = Connection.SourcePinId;
				ConnectedPinPath = &Connection.SourcePath;
			}

			TSharedPtr<FNiagaraConvertPinSocketViewModel> ConnectedSocket = GetSocket(ConnectedPinId, *ConnectedPinPath, ConnectedPinDirection);
			if (ConnectedSocket.IsValid())
			{
				ConnectedSockets.Add(ConnectedSocket.ToSharedRef());
			}
		}
	}
}

bool FNiagaraConvertNodeViewModel::CanConnectSockets(TSharedRef<FNiagaraConvertPinSocketViewModel> SocketA, TSharedRef<FNiagaraConvertPinSocketViewModel> SocketB, FText& ConnectionMessage, bool& bMessageIsWarning)
{
	bMessageIsWarning = false;
	if (SocketA->GetOwnerConvertNodeViewModel().Get() != this || SocketB->GetOwnerConvertNodeViewModel().Get() != this)
	{
		ConnectionMessage = LOCTEXT("DifferentConvertNodeConnectionMessage", "Can only connect pins from the same convert node.");
		return false;
	}

	if (SocketA->GetDirection() == SocketB->GetDirection())
	{
		ConnectionMessage = LOCTEXT("SamePinDirectionConnectionMessage", "Can only connect pins with different directions.");
		return false;
	}

	if (SocketA->GetOwnerPinViewModel().IsValid() == false || SocketB->GetOwnerPinViewModel().IsValid() == false)
	{
		ConnectionMessage = LOCTEXT("InvalidPinStateConnectionMessage", "Can not connect due to invalid pin state.");
		return false;
	}

	FNiagaraTypeDefinition TypeA = SocketA->GetTypeDefinition();
	FNiagaraTypeDefinition TypeB = SocketB->GetTypeDefinition();
	if (false == FNiagaraTypeDefinition::TypesAreAssignable(TypeA, TypeB))
	{
		ConnectionMessage = FText::Format(LOCTEXT("InvalidPinTypeConnectionMessage", "Cannot connect types: {0} to {1}"), TypeA.GetNameText(), TypeB.GetNameText());
		return false;
	}

	if (FNiagaraTypeDefinition::IsLossyConversion(TypeA, TypeB))
	{
		bMessageIsWarning = true;
		ConnectionMessage = FText::Format(LOCTEXT("ConnectFormatLossy", "Possible lossy conversion {0} to {1}. Are you sure?"), TypeA.GetNameText(), TypeB.GetNameText());
	}
	else
	{
		ConnectionMessage = FText::Format(LOCTEXT("ConnectFormat", "Connect '{0}' to '{1}'"), SocketA->GetDisplayPathText(), SocketB->GetDisplayPathText());
	}

	return true;
}

bool IsParent(const TArray<FName>& PossibleParentPath, const TArray<FName> SrcPath)
{
	int32 SrcIdx = 0;
	for (int32 i = 0; i < PossibleParentPath.Num(); i++)
	{
		if (SrcPath.Num() > SrcIdx)
		{
			if (PossibleParentPath[i] == SrcPath[SrcIdx])
			{
				SrcIdx++;
				continue;
			}

			if (PossibleParentPath[i] == FName())
			{
				continue;
			}

			if (SrcPath[SrcIdx] == FName())
			{
				SrcIdx++;
				continue;
			}

			if (PossibleParentPath[i] != SrcPath[SrcIdx])
			{
				return false;
			}
		}
	}

	if (SrcPath.Num() == SrcIdx)
	{
		return false;
	}
	else
	{
		return true;
	}
}

void FNiagaraConvertNodeViewModel::ConnectSockets(TSharedRef<FNiagaraConvertPinSocketViewModel> SocketA, TSharedRef<FNiagaraConvertPinSocketViewModel> SocketB)
{
	TSharedRef<FNiagaraConvertPinSocketViewModel> InputSocket = SocketA->GetDirection() == EGPD_Input ? SocketA : SocketB;
	TSharedRef<FNiagaraConvertPinSocketViewModel> OutputSocket = SocketA->GetDirection() == EGPD_Output ? SocketA : SocketB;

	FGuid InputPinId = InputSocket->GetOwnerPinViewModel()->GetPinId();
	TArray<FName> InputPath = InputSocket->GetPath();

	FGuid OutputPinId = OutputSocket->GetOwnerPinViewModel()->GetPinId();
	TArray<FName> OutputPath = OutputSocket->GetPath();

	ConvertNode.Modify();

	TArray<FNiagaraConvertConnection>& Connections = ConvertNode.GetConnections();

	// Remove any existing connections to the same destination.
	auto RemovePredicate = [&](FNiagaraConvertConnection& Connection)
	{
		return Connection.DestinationPinId == OutputPinId && Connection.DestinationPath == OutputPath;
	};
	Connections.RemoveAll(RemovePredicate);

	// Remove any connections higher or lower in the traversal order affecting this ooutput property.
	// For instance, connecting the X property of a Vector3 will cause the direct Vector3->Vector3 connection one level higher to be removed.
	// Alternately, if you only have the float->float X property connection, a higher-level vector3->vector3 connection will trump it and cause the old x connection to be removed.
	for (int32 i = Connections.Num() - 1; i >= 0; i--)
	{
		FNiagaraConvertConnection& Connection = Connections[i];
		if (Connection.DestinationPinId == OutputPinId && IsParent(Connection.DestinationPath, OutputPath))
		{
			Connections.RemoveAt(i);
		}
		else if (Connection.DestinationPinId == OutputPinId && IsParent(OutputPath, Connection.DestinationPath))
		{
			Connections.RemoveAt(i);
		}
	}

	// Add this new connection
	Connections.Add(FNiagaraConvertConnection(InputPinId, InputPath, OutputPinId, OutputPath));
	ConvertNode.MarkNodeRequiresSynchronization(__FUNCTION__, true);
	InvalidateConnectionViewModels();
}

void FNiagaraConvertNodeViewModel::DisconnectSocket(TSharedRef<FNiagaraConvertPinSocketViewModel> Socket)
{
	TSharedPtr<FNiagaraConvertPinViewModel> OwnerPin = Socket->GetOwnerPinViewModel();
	if (OwnerPin.IsValid())
	{
		ConvertNode.Modify();

		TArray<FName> Path = Socket->GetPath();
		auto RemovePredicate = [&](FNiagaraConvertConnection& Connection)
		{
			return (Connection.SourcePinId == OwnerPin->GetPinId() && Connection.SourcePath == Path) ||
				(Connection.DestinationPinId == OwnerPin->GetPinId() && Connection.DestinationPath == Path);
		};

		ConvertNode.GetConnections().RemoveAll(RemovePredicate); 
		ConvertNode.MarkNodeRequiresSynchronization(__FUNCTION__, true);
		InvalidateConnectionViewModels();
	}
}

void FNiagaraConvertNodeViewModel::DisconnectSockets(TSharedRef<FNiagaraConvertPinSocketViewModel> SocketA, TSharedRef<FNiagaraConvertPinSocketViewModel> SocketB)
{
	TSharedPtr<FNiagaraConvertPinViewModel> OwnerPinA = SocketA->GetOwnerPinViewModel();
	TSharedPtr<FNiagaraConvertPinViewModel> OwnerPinB = SocketB->GetOwnerPinViewModel();
	if (OwnerPinA.IsValid() && OwnerPinB.IsValid())
	{
		ConvertNode.Modify();

		TArray<FName> PathA = SocketA->GetPath();
		TArray<FName> PathB = SocketB->GetPath();

		auto RemovePredicate = [&](FNiagaraConvertConnection& Connection)
		{
			bool SourceMatchesA = Connection.SourcePinId == OwnerPinA->GetPinId() && Connection.SourcePath == PathA;
			bool SourceMatchesB = Connection.SourcePinId == OwnerPinB->GetPinId() && Connection.SourcePath == PathB;
			bool DestinationMatchesA = Connection.DestinationPinId == OwnerPinA->GetPinId() && Connection.DestinationPath == PathA;
			bool DestinationMatchesB = Connection.DestinationPinId == OwnerPinB->GetPinId() && Connection.DestinationPath == PathB;
			return (SourceMatchesA && DestinationMatchesB) || (SourceMatchesB && DestinationMatchesA);
		};

		ConvertNode.GetConnections().RemoveAll(RemovePredicate);
		ConvertNode.MarkNodeRequiresSynchronization(__FUNCTION__, true);
		InvalidateConnectionViewModels();
	}
}

void FNiagaraConvertNodeViewModel::RefreshPinViewModels()
{
	InputPinViewModels.Empty();
	OutputPinViewModels.Empty();

	FPinCollectorArray InputPins;
	FPinCollectorArray OutputPins;
	ConvertNode.GetInputPins(InputPins);
	ConvertNode.GetOutputPins(OutputPins);

	for (UEdGraphPin* InputPin : InputPins)
	{
		if (InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType || 
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType || 
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum || 
			InputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum)
		{
			InputPinViewModels.Add(MakeShareable(new FNiagaraConvertPinViewModel(this->AsShared(), *InputPin)));
		}
	}

	for (UEdGraphPin* OutputPin : OutputPins)
	{
		if (OutputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticType ||
			OutputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryStaticEnum || 
			OutputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryType ||
			OutputPin->PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryEnum)
		{																			 
			OutputPinViewModels.Add(MakeShareable(new FNiagaraConvertPinViewModel(this->AsShared(), *OutputPin)));
		}
	}

	bPinViewModelsNeedRefresh = false;
}

void FNiagaraConvertNodeViewModel::InvalidateConnectionViewModels()
{
	bConnectionViewModelsNewRefresh = true;
}

void FNiagaraConvertNodeViewModel::RefreshConnectionViewModels()
{
	ConnectionViewModels.Empty();

	for (const FNiagaraConvertConnection& Connection : ConvertNode.GetConnections())
	{
		TSharedPtr<FNiagaraConvertPinSocketViewModel> SourceSocket = GetSocket(Connection.SourcePinId, Connection.SourcePath, EGPD_Input);
		TSharedPtr<FNiagaraConvertPinSocketViewModel> DestinationSocket = GetSocket(Connection.DestinationPinId, Connection.DestinationPath, EGPD_Output);
		if (SourceSocket.IsValid() && DestinationSocket.IsValid())
		{
			ConnectionViewModels.Add(MakeShareable(new FNiagaraConvertConnectionViewModel(SourceSocket.ToSharedRef(), DestinationSocket.ToSharedRef())));
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Invalid connection!");
			//if (SourceSocket.IsValid() == false)
			//	SourceSocket = GetSocket(Connection.SourcePinId, Connection.SourcePath, EGPD_Input);
			//if (DestinationSocket.IsValid() == false)
			//	DestinationSocket = GetSocket(Connection.DestinationPinId, Connection.DestinationPath, EGPD_Output);
		}
	}

	bConnectionViewModelsNewRefresh = false;
}

TSharedPtr<FNiagaraConvertPinSocketViewModel> GetSocketByPathRecursive(const TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& SocketViewModels, const TArray<FName>& Path, int32 PathIndex)
{
	for (const TSharedRef<FNiagaraConvertPinSocketViewModel>& SocketViewModel : SocketViewModels)
	{
		if (SocketViewModel->GetPath() == Path)
		{
			return SocketViewModel;
		}
		else
		{
			TSharedPtr<FNiagaraConvertPinSocketViewModel> ChildSocket = GetSocketByPathRecursive(SocketViewModel->GetChildSockets(), Path, PathIndex);
			if (ChildSocket.IsValid())
			{
				return ChildSocket;
			}
		}
	}
	return TSharedPtr<FNiagaraConvertPinSocketViewModel>();
}

TSharedPtr<FNiagaraConvertPinSocketViewModel> FNiagaraConvertNodeViewModel::GetSocket(FGuid PinId, const TArray<FName> Path, EEdGraphPinDirection Direction)
{
	const TArray<TSharedRef<FNiagaraConvertPinViewModel>> PinViewModels = Direction == EGPD_Input
		? GetInputPinViewModels()
		: GetOutputPinViewModels();

	TSharedPtr<FNiagaraConvertPinViewModel> PathPinViewModel;
	for (TSharedRef<FNiagaraConvertPinViewModel> PinViewModel : PinViewModels)
	{
		if (PinViewModel->GetPinId() == PinId)
		{
			PathPinViewModel = PinViewModel;
			break;
		}
	}

	if (PathPinViewModel.IsValid())
	{
		return GetSocketByPathRecursive(PathPinViewModel->GetSocketViewModels(), Path, 0);
	}
	else
	{
		return TSharedPtr<FNiagaraConvertPinSocketViewModel>();
	}
}

bool FNiagaraConvertNodeViewModel::IsWiringShown() const
{
	return ConvertNode.IsWiringShown();
}

void FNiagaraConvertNodeViewModel::RecordChildrenShowing(bool bIsShowingChildren, FGuid PinId, const TArray<FName>& Path)
{
	FNiagaraConvertPinRecord Record;
	Record.Path = Path;
	Record.PinId = PinId;
	if (bIsShowingChildren)
	{
		ConvertNode.AddExpandedRecord(Record);
	}
	else
	{
		ConvertNode.RemoveExpandedRecord(Record);
	}
}

bool FNiagaraConvertNodeViewModel::AreChildrenShowing(FGuid PinId, const TArray<FName>& Path)
{
	FNiagaraConvertPinRecord Record;
	Record.Path = Path;
	Record.PinId = PinId;
	return ConvertNode.HasExpandedRecord(Record);
}



#undef LOCTEXT_NAMESPACE