// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"

#include "NiagaraTypes.h"

class UNiagaraNodeConvert;
class FNiagaraConvertPinViewModel;
class FNiagaraConvertPinSocketViewModel;

/** A view model for an inner connection in a Niagara convert node. */
class FNiagaraConvertConnectionViewModel
{
public:
	const TSharedRef<FNiagaraConvertPinSocketViewModel> SourceSocket;
	const TSharedRef<FNiagaraConvertPinSocketViewModel> DestinationSocket;

	/** Creates a connection view model with a source and destination socket. */
	FNiagaraConvertConnectionViewModel(TSharedRef<FNiagaraConvertPinSocketViewModel> InSourceSocket, TSharedRef<FNiagaraConvertPinSocketViewModel> InDestinationSocket)
		: SourceSocket(InSourceSocket)
		, DestinationSocket(InDestinationSocket)
	{
	}
};

/** A view model for connection UI of the Niagara convert node. */
class FNiagaraConvertNodeViewModel : public TSharedFromThis<FNiagaraConvertNodeViewModel>
{
public:
	FNiagaraConvertNodeViewModel(UNiagaraNodeConvert& InConvertNode);

	/** Gets the view models for the input pins. */
	const TArray<TSharedRef<FNiagaraConvertPinViewModel>>& GetInputPinViewModels();

	/** Gets the view models for the output pins. */
	const TArray<TSharedRef<FNiagaraConvertPinViewModel>>& GetOutputPinViewModels();

	/** Gets the view models for the connections. */
	const TArray<TSharedRef<FNiagaraConvertConnectionViewModel>> GetConnectionViewModels();

	/** Gets the view model for the currently dragged socket, if there is one. */
	TSharedPtr<FNiagaraConvertPinSocketViewModel> GetDraggedSocketViewModel() const;

	/** Sets the view model for the currently dragged socket. */
	void SetDraggedSocketViewModel(TSharedPtr<FNiagaraConvertPinSocketViewModel> DraggedSocket);

	/** Determines whether two sockets can be connected giving a message about the connection. */
	bool CanConnectSockets(TSharedRef<FNiagaraConvertPinSocketViewModel> SocketA, TSharedRef<FNiagaraConvertPinSocketViewModel> SocketB, FText& ConnectionMessage, bool& bIsWarningMessage);

	/** Connects two socket view models. */
	void ConnectSockets(TSharedRef<FNiagaraConvertPinSocketViewModel> SocketA, TSharedRef<FNiagaraConvertPinSocketViewModel> SocketB);

	/** Disconnects a socket from all other connected sockets. */
	void DisconnectSocket(TSharedRef<FNiagaraConvertPinSocketViewModel> Socket);

	/** Disconnects a socket from a specific socket. */
	void DisconnectSockets(TSharedRef<FNiagaraConvertPinSocketViewModel> SocketA, TSharedRef<FNiagaraConvertPinSocketViewModel> SocketB);

	/** Returns whether or not a socket is connected. */
	bool IsSocketConnected(TSharedRef<const FNiagaraConvertPinSocketViewModel> Socket) const;

	/** Gets an array of sockets connected to a specific socket. */
	void GetConnectedSockets(TSharedRef<const FNiagaraConvertPinSocketViewModel> Socket, TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& ConnectedSockets);

	/** Do we show any of the switchboard UI?*/
	bool IsWiringShown() const;

	/** Store off whether or not this socket is expanded.*/
	void RecordChildrenShowing(bool bIsShowingChildren, FGuid PinId, const TArray<FName>& Path);

	/** Query whether or not htis socket is expanded.*/
	bool AreChildrenShowing(FGuid PinId, const TArray<FName>& Path);

private:
	/** Rebuilds the pin view models. */
	void RefreshPinViewModels();

	/** Marks the connection view models for rebuilding. */
	void InvalidateConnectionViewModels();

	/** Rebuilds the connection view models. */
	void RefreshConnectionViewModels();

	/** Gets a socket by it's pin id, path, and direction. */
	TSharedPtr<FNiagaraConvertPinSocketViewModel> GetSocket(FGuid PinId, const TArray<FName> Path, EEdGraphPinDirection Direction);

private:
	/** The convert node that this view model represents. */
	UNiagaraNodeConvert& ConvertNode;

	/** Whether or not the pin view models need to be rebuilt before use. */
	bool bPinViewModelsNeedRefresh;

	/** Whether or not the connection view models need to be rebuilt before use. */
	bool bConnectionViewModelsNewRefresh;

	/** The input pin view models. */
	TArray<TSharedRef<FNiagaraConvertPinViewModel>> InputPinViewModels;

	/** The output pin view models. */
	TArray<TSharedRef<FNiagaraConvertPinViewModel>> OutputPinViewModels;

	/** The connection view models. */
	TArray<TSharedRef<FNiagaraConvertConnectionViewModel>> ConnectionViewModels;

	/** The view model for the currently dragged socket if there is one. */
	TSharedPtr<FNiagaraConvertPinSocketViewModel> DraggedSocketViewModel;
};