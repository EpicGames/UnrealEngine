// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraConvertPinSocketViewModel.h"
#include "NiagaraConvertPinViewModel.h"
#include "NiagaraConvertNodeViewModel.h"
#include "NiagaraNodeConvert.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NiagaraConvertPinSocketViewModel.h"

FNiagaraConvertPinSocketViewModel::FNiagaraConvertPinSocketViewModel(
	TSharedRef<FNiagaraConvertPinViewModel> InOwnerPinViewModel, 
	TSharedPtr<FNiagaraConvertPinSocketViewModel> InOwnerPinSocketViewModel,
	FName InName, 
	FName InDisplayName,
	const FNiagaraTypeDefinition& InTypeDef,
	EEdGraphPinDirection InDirection,
	int32 InTypeTraversalDepth)
{
	OwnerPinViewModel = InOwnerPinViewModel;
	OwnerPinSocketViewModel = InOwnerPinSocketViewModel;
	Name = InName;
	DisplayName = InDisplayName;
	Direction = InDirection;
	TypeDefinition = InTypeDef;
	ConstructPathText();
	ConstructDisplayPathText();
	bIsConnectedNeedsRefresh = true;
	TypeTraversalDepth = InTypeTraversalDepth;
	bIsShowingChildren = false;
	AbsoluteConnectionPosition = FVector2D(-FLT_MAX, -FLT_MAX);
	AbsoluteDragPosition = FVector2D(-FLT_MAX, -FLT_MAX);
	if (TypeTraversalDepth <= 0)
	{
		bIsSocketShown = true;
	}
	else
	{
		bIsSocketShown = false;
	}

	ExpandedImage = FCoreStyle::Get().GetBrush("TreeArrow_Expanded");
	CollapsedImage = FCoreStyle::Get().GetBrush("TreeArrow_Collapsed");

	// We persist the expansion state in the root node. This means that we need to properly synchronize state with our parents.
	TSharedPtr<FNiagaraConvertNodeViewModel>  ConvertVM = GetOwnerConvertNodeViewModel();
	if (ConvertVM->AreChildrenShowing(GetOwnerPinViewModel()->GetPinId(), GetPath()))
	{
		bIsSocketShown = true;
		bIsShowingChildren = true;
		ExpandParents();
	}

	TSharedPtr<FNiagaraConvertPinSocketViewModel> Parent = GetOwnerPinSocketViewModel();
	if (Parent.IsValid() && Parent->bIsShowingChildren)
	{
		bIsSocketShown = true;
	}
}

FName FNiagaraConvertPinSocketViewModel::GetName() const
{
	return Name;
}

FName FNiagaraConvertPinSocketViewModel::GetDisplayName() const
{
	return DisplayName;
}

TArray<FName> FNiagaraConvertPinSocketViewModel::GetPath() const
{
	TArray<FName> PathNames;
	const FNiagaraConvertPinSocketViewModel* CurrentSocket = this;
	int32 SocketLevel = TypeTraversalDepth;
	while (CurrentSocket != nullptr)
	{
		if (CurrentSocket->GetName() != FName()/* || TypeTraversalDepth == SocketLevel*/)
		{
			PathNames.Insert(CurrentSocket->GetName(), 0);
		}
		CurrentSocket = CurrentSocket->GetOwnerPinSocketViewModel().Get();
		SocketLevel--;
	}
	return PathNames;
}

TArray<FName> FNiagaraConvertPinSocketViewModel::GetDisplayPath() const
{
	TArray<FName> PathNames;
	const FNiagaraConvertPinSocketViewModel* CurrentSocket = this;
	int32 SocketLevel = TypeTraversalDepth;
	while (CurrentSocket != nullptr)
	{
		if (CurrentSocket->GetName() != FName() || TypeTraversalDepth == SocketLevel)
		{
			PathNames.Insert(CurrentSocket->GetDisplayName(), 0);
		}
		CurrentSocket = CurrentSocket->GetOwnerPinSocketViewModel().Get();
		SocketLevel--;
	}
	return PathNames;
}


FText FNiagaraConvertPinSocketViewModel::GetPathText() const
{
	return PathText;
}

FText FNiagaraConvertPinSocketViewModel::GetDisplayPathText() const
{
	return DisplayPathText;
}

EEdGraphPinDirection FNiagaraConvertPinSocketViewModel::GetDirection() const
{
	return Direction;
}

bool FNiagaraConvertPinSocketViewModel::GetIsConnected() const
{
	if (bIsConnectedNeedsRefresh)
	{
		RefreshIsConnected();
	}
	return bIsConnected;
}

bool FNiagaraConvertPinSocketViewModel::CanBeConnected() const
{
	return true;
}

const FNiagaraTypeDefinition& FNiagaraConvertPinSocketViewModel::GetTypeDefinition() const
{
	return TypeDefinition;
}


FMargin FNiagaraConvertPinSocketViewModel::GetChildSlotMargin() const
{
	return GetSocketVisibility() == EVisibility::Visible ?  FMargin(0.0f, 0.0f, 0.0f, 5.0f) : FMargin();
}

FMargin FNiagaraConvertPinSocketViewModel::GetSlotMargin() const 
{
	return GetSocketVisibility() == EVisibility::Visible ? FMargin(0.0f, 0.0f, 0.0f, 1.0f) : FMargin();
}

FReply FNiagaraConvertPinSocketViewModel::ExpandButtonClicked()
{
	bIsShowingChildren = !bIsShowingChildren;
	TSharedPtr<FNiagaraConvertNodeViewModel>  ConvertVM = GetOwnerConvertNodeViewModel();
	ConvertVM->RecordChildrenShowing(bIsShowingChildren, GetOwnerPinViewModel()->GetPinId(), GetPath());

	for (TSharedRef<FNiagaraConvertPinSocketViewModel>& ChildVM : ChildSockets)
	{
		ChildVM->SetSocketShown(bIsShowingChildren);
	}
	return FReply::Handled();
}

FReply FNiagaraConvertPinSocketViewModel::OnMouseDoubleClick(const FGeometry & Geom, const FPointerEvent & PtrEvent)
{
	ExpandButtonClicked();
	return FReply::Handled();
}

void FNiagaraConvertPinSocketViewModel::SetSocketShown(bool bInShown)
{
	bIsSocketShown = bInShown;
	CollapseChildren();
	ExpandParents();
}

void FNiagaraConvertPinSocketViewModel::CollapseChildren()
{
	// If we are collapsing the socket, then all expanded children need to also be fully collapsed.
	if (bIsSocketShown == false && bIsShowingChildren == true)
	{
		TSharedPtr<FNiagaraConvertNodeViewModel>  ConvertVM = GetOwnerConvertNodeViewModel();
		bIsShowingChildren = bIsSocketShown;
		ConvertVM->RecordChildrenShowing(bIsShowingChildren, GetOwnerPinViewModel()->GetPinId(), GetPath());

		for (TSharedRef<FNiagaraConvertPinSocketViewModel>& ChildVM : ChildSockets)
		{
			ChildVM->SetSocketShown(bIsSocketShown);
		}
	}
}

void FNiagaraConvertPinSocketViewModel::ExpandParents()
{
	// If we are expanding the socket, just in case we should make sure that all parent sockets are also expanded and
	// their show children is set.
	if (bIsSocketShown == true)
	{
		TSharedPtr<FNiagaraConvertNodeViewModel>  ConvertVM = GetOwnerConvertNodeViewModel();
		TSharedPtr<FNiagaraConvertPinSocketViewModel> Parent = GetOwnerPinSocketViewModel();
		while (Parent.IsValid())
		{
			Parent->bIsSocketShown = true;
			if (Parent->bIsShowingChildren == false)
			{
				Parent->bIsShowingChildren = true;
				ConvertVM->RecordChildrenShowing(Parent->bIsShowingChildren, Parent->GetOwnerPinViewModel()->GetPinId(), Parent->GetPath());
			}

			Parent = Parent->GetOwnerPinSocketViewModel();
		}
	}
}

const FSlateBrush* FNiagaraConvertPinSocketViewModel::GetExpansionBrush() const
{
	if (bIsShowingChildren)
	{
		return ExpandedImage;
	}
	else
	{
		return CollapsedImage;
	}
}

EVisibility FNiagaraConvertPinSocketViewModel::GetExpansionBrushVisibility() const
{
	return ChildSockets.Num() > 0 ? EVisibility::Visible : EVisibility::Hidden;
}

FMargin FNiagaraConvertPinSocketViewModel::GetSocketPadding() const
{
	return FMargin(10 * TypeTraversalDepth, 0.0f, 0.0f, 0.0f);
}

FText FNiagaraConvertPinSocketViewModel::GetDisplayNameText() const
{
	return FText::FromName(DisplayName);
}

EVisibility FNiagaraConvertPinSocketViewModel::GetSocketVisibility() const
{
	TSharedPtr<FNiagaraConvertNodeViewModel>  ConvertVM = GetOwnerConvertNodeViewModel();

	return bIsSocketShown && ConvertVM->IsWiringShown()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

EVisibility FNiagaraConvertPinSocketViewModel::GetSocketIconVisibility() const
{
	return GetSocketVisibility();
}

EVisibility FNiagaraConvertPinSocketViewModel::GetSocketTextVisibility() const
{
	return GetSocketVisibility();
}

FVector2D FNiagaraConvertPinSocketViewModel::GetAbsoluteConnectionPosition() const
{
	return AbsoluteConnectionPosition;
}

void FNiagaraConvertPinSocketViewModel::SetAbsoluteConnectionPosition(FVector2D Position)
{
	AbsoluteConnectionPosition = Position;
}

const TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& FNiagaraConvertPinSocketViewModel::GetChildSockets() const
{
	return ChildSockets;
}

void FNiagaraConvertPinSocketViewModel::SetChildSockets(const TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& InChildSockets)
{
	ChildSockets = InChildSockets;
}

TSharedPtr<FNiagaraConvertPinSocketViewModel> FNiagaraConvertPinSocketViewModel::GetOwnerPinSocketViewModel() const
{
	return OwnerPinSocketViewModel.Pin();
}

TSharedPtr<FNiagaraConvertPinViewModel> FNiagaraConvertPinSocketViewModel::GetOwnerPinViewModel() const
{
	return OwnerPinViewModel.Pin();
}

TSharedPtr<FNiagaraConvertNodeViewModel> FNiagaraConvertPinSocketViewModel::GetOwnerConvertNodeViewModel() const
{
	TSharedPtr<FNiagaraConvertPinViewModel> PinnedOwnerPinViewModel = OwnerPinViewModel.Pin();
	if (PinnedOwnerPinViewModel.IsValid())
	{
		return PinnedOwnerPinViewModel->GetOwnerConvertNodeViewModel();
	}
	return TSharedPtr<FNiagaraConvertNodeViewModel>();
}

void FNiagaraConvertPinSocketViewModel::SetIsBeingDragged(bool bInIsBeingDragged)
{
	bIsBeingDragged = bInIsBeingDragged;
	TSharedPtr<FNiagaraConvertNodeViewModel> OwnerConvertNodeViewModel = GetOwnerConvertNodeViewModel();
	if (OwnerConvertNodeViewModel.IsValid())
	{
		if (bIsBeingDragged)
		{
			OwnerConvertNodeViewModel->SetDraggedSocketViewModel(this->AsShared());
		}
		else
		{
			if (OwnerConvertNodeViewModel->GetDraggedSocketViewModel().Get() == this)
			{
				OwnerConvertNodeViewModel->SetDraggedSocketViewModel(TSharedPtr<FNiagaraConvertPinSocketViewModel>());
			}
		}
	}
}

FVector2D FNiagaraConvertPinSocketViewModel::GetAbsoluteDragPosition() const
{
	return AbsoluteDragPosition;
}

void FNiagaraConvertPinSocketViewModel::SetAbsoluteDragPosition(FVector2D InAbsoluteDragPosition)
{
	AbsoluteDragPosition = InAbsoluteDragPosition;
}

bool FNiagaraConvertPinSocketViewModel::CanConnect(TSharedRef<FNiagaraConvertPinSocketViewModel> OtherSocket, FText& ConnectionMessage, bool& bMessageIsWarning)
{
	TSharedPtr<FNiagaraConvertNodeViewModel> OwnerConvertNodeViewModel = GetOwnerConvertNodeViewModel();
	if (OwnerConvertNodeViewModel.IsValid())
	{
		return OwnerConvertNodeViewModel->CanConnectSockets(OtherSocket, this->AsShared(), ConnectionMessage, bMessageIsWarning);
	}
	else
	{
		bMessageIsWarning = false;
		ConnectionMessage = LOCTEXT("InvalidSocketConnectionMessage", "Can not connect because socket is in an invalid state.");
		return false;
	}
}

void FNiagaraConvertPinSocketViewModel::GetConnectedSockets(TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& ConnectedSockets)
{
	TSharedPtr<FNiagaraConvertNodeViewModel> OwnerConvertNodeViewModel = GetOwnerConvertNodeViewModel();
	if (OwnerConvertNodeViewModel.IsValid())
	{
		OwnerConvertNodeViewModel->GetConnectedSockets(this->AsShared(), ConnectedSockets);
	}
}

void FNiagaraConvertPinSocketViewModel::Connect(TSharedRef<FNiagaraConvertPinSocketViewModel> OtherSocket)
{
	TSharedPtr<FNiagaraConvertNodeViewModel> OwnerConvertNodeViewModel = GetOwnerConvertNodeViewModel();
	if (OwnerConvertNodeViewModel.IsValid())
	{
		FText ConnectionMessage;
		bool bWarning = false;
		if (OwnerConvertNodeViewModel->CanConnectSockets(OtherSocket, this->AsShared(), ConnectionMessage, bWarning))
		{
			OwnerConvertNodeViewModel->ConnectSockets(OtherSocket, this->AsShared());
			bIsConnectedNeedsRefresh = true;
		}
	}
}

void FNiagaraConvertPinSocketViewModel::DisconnectAll()
{
	FScopedTransaction DisconnectSpecificTransaction(LOCTEXT("DisconnectAllTransaction", "Break all inner links for pin."));
	TSharedPtr<FNiagaraConvertNodeViewModel> OwnerConvertNodeViewModel = GetOwnerConvertNodeViewModel();
	if (OwnerConvertNodeViewModel.IsValid())
	{
		OwnerConvertNodeViewModel->DisconnectSocket(this->AsShared());
	}
}

void FNiagaraConvertPinSocketViewModel::DisconnectSpecific(TSharedRef<FNiagaraConvertPinSocketViewModel> OtherSocket)
{
	FScopedTransaction DisconnectSpecificTransaction(LOCTEXT("DisconnectSpecificTransaction", "Break specific inner link for pin."));
	TSharedPtr<FNiagaraConvertNodeViewModel> OwnerConvertNodeViewModel = GetOwnerConvertNodeViewModel();
	if (OwnerConvertNodeViewModel.IsValid())
	{
		OwnerConvertNodeViewModel->DisconnectSockets(this->AsShared(), OtherSocket);
	}
}

void FNiagaraConvertPinSocketViewModel::ConstructPathText()
{
	TArray<FName> PathNames = GetPath();
	TArray<FString> PathStrings;
	for (const FName& PathName : PathNames)
	{
		PathStrings.Add(PathName.ToString());
	}
	PathText = FText::FromString(FString::Join(PathStrings, TEXT(".")));
}

void FNiagaraConvertPinSocketViewModel::ConstructDisplayPathText()
{
	TArray<FName> PathNames = GetDisplayPath();
	TArray<FString> PathStrings;
	for (const FName& PathName : PathNames)
	{
		if (PathName != FName())
		{
			PathStrings.Add(PathName.ToString());
		}
	}
	DisplayPathText = FText::FromString(FString::Join(PathStrings, TEXT(".")));
}

void FNiagaraConvertPinSocketViewModel::RefreshIsConnected() const
{
	TSharedPtr<FNiagaraConvertNodeViewModel> OwnerConvertNodeViewModel = GetOwnerConvertNodeViewModel();
	bIsConnected = OwnerConvertNodeViewModel.IsValid() && OwnerConvertNodeViewModel->IsSocketConnected(this->AsShared());
}

#undef LOCTEXT_NAMESPACE