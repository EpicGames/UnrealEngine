// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "EdGraph/EdGraphNode.h"
#include "Templates/SharedPointer.h"
#include "EdGraph/EdGraphNode.h"
#include "Layout/Visibility.h"
#include "Layout/Margin.h"
#include "Input/Reply.h"
#include "NiagaraCommon.h"

class FNiagaraConvertPinViewModel;
class FNiagaraConvertNodeViewModel;
struct FSlateBrush;

/** A view model for a connectible socket representing a property on a pin on a convert node. */
class FNiagaraConvertPinSocketViewModel : public TSharedFromThis<FNiagaraConvertPinSocketViewModel>
{
public:
	FNiagaraConvertPinSocketViewModel(
		TSharedRef<FNiagaraConvertPinViewModel> InOwnerPinViewModel, 
		TSharedPtr<FNiagaraConvertPinSocketViewModel> InOwnerPinSocketViewModel, 
		FName InName, 
		FName InDisplayName,
		const FNiagaraTypeDefinition& InTypeDef,
		EEdGraphPinDirection InDirection,
		int32 InTypeTraversalDepth);

	/** Gets the name of this socket which is the name of the property it represents. */
	FName GetName() const;

	/** Gets the display name of this socket which is the name of the property it represents*/
	FName GetDisplayName() const;

	/** Gets the path to this socket from the pin using the socket names. */
	TArray<FName> GetPath() const;

	/** Gets the path to this socket at text. */
	FText GetPathText() const;

	/** Gets the display path to this socket from the pin using the socket display names. */
	TArray<FName> GetDisplayPath() const;

	/** Gets the display path to this socket at text. */
	FText GetDisplayPathText() const;

	/** Gets the direction of this socket. */
	EEdGraphPinDirection GetDirection() const;

	/** Gets whether or not this socket is connected. */
	bool GetIsConnected() const;

	/** Gets whether or not this socket can be connected. */
	bool CanBeConnected() const;

	/** Get the type associated with this pin.*/
	const FNiagaraTypeDefinition& GetTypeDefinition() const;
	
	/** Gets the visibility of the connection icon for this socket. */
	EVisibility GetSocketIconVisibility() const;

	/** Gets the visibility of the connection text for this socket. */
	EVisibility GetSocketTextVisibility() const;

	/** Gets the visibility of the socket overall.*/
	EVisibility GetSocketVisibility() const;

	/** Gets the connection position for this socket in absolute coordinate space. */
	FVector2D GetAbsoluteConnectionPosition() const;

	/** Sets the connection position for this socket in absolute coordinate space. */
	void SetAbsoluteConnectionPosition(FVector2D Position);

	/** Gets the child sockets for this socket. */
	const TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& GetChildSockets() const;

	/** Sets the child sockets for this socket. */
	void SetChildSockets(const TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& InChildSockets);

	/** Gets the view model for this socket that owns this socket view model.  May be null if this
		is a root socket. */
	TSharedPtr<FNiagaraConvertPinSocketViewModel> GetOwnerPinSocketViewModel() const;

	/** Gets the view model for the pin that owns this socket view model. */
	TSharedPtr<FNiagaraConvertPinViewModel> GetOwnerPinViewModel() const;

	/** Gets the view model for the convert node that owns the pin that owns this socket view model. */
	TSharedPtr<FNiagaraConvertNodeViewModel> GetOwnerConvertNodeViewModel() const;

	/** Sets whether or not this socket is being dragged. */
	void SetIsBeingDragged(bool bInIsBeingDragged);

	/** Gets the absolute position of this socket when it is being dragged. */
	FVector2D GetAbsoluteDragPosition() const;

	/** Sets the absolute position of this socket when it is being dragged. */
	void SetAbsoluteDragPosition(FVector2D InAbsoluteDragPosition);

	/** Gets whether or not this socket can be connected to another socket and provides a message about the connection. */
	bool CanConnect(TSharedRef<FNiagaraConvertPinSocketViewModel> OtherSocket, FText& ConnectionMessage, bool& bMessageIsWarning);

	/** Gets all sockets which are connected to this socket. */
	void GetConnectedSockets(TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& ConnectedSockets);

	/** Connects this socket to another socket. */
	void Connect(TSharedRef<FNiagaraConvertPinSocketViewModel> OtherSocket);

	/** Disconnects all connections to this socket. */
	void DisconnectAll();

	/** Disconnects this socket for a specific socket. */
	void DisconnectSpecific(TSharedRef<FNiagaraConvertPinSocketViewModel> OtherSocket);
	
	/** Margin for the child slot.*/
	FMargin GetChildSlotMargin() const;

	/** Margin for the overall slot.*/
	FMargin GetSlotMargin() const;

	/** Toggles expansion of children.*/
	FReply ExpandButtonClicked();

	/** Meant to toggle expansion of children.*/
	FReply OnMouseDoubleClick(const FGeometry & Geom, const FPointerEvent & PtrEvent);

	/** Get which brush we should show Expanded/Collapsed?*/
	const FSlateBrush* GetExpansionBrush() const;

	/** Get whether or not to show the expansion brush*/
	EVisibility GetExpansionBrushVisibility() const;

	/** Get the spacing based on tree level*/
	FMargin GetSocketPadding() const;

	/** Get this socket's display name (non-hierarchical)*/
	FText GetDisplayNameText() const;

	/** Helper to show this socket and handle making children and parents align.*/
	void SetSocketShown(bool bInShown);

private:
	/** Builds text representing the path to this socket from it's owning pin. */
	void ConstructPathText();

	/** Builds text representing the display path to this socket from it's owning pin. */
	void ConstructDisplayPathText();

	/** Refreshes whether or not this pin is connected. */
	void RefreshIsConnected() const;

	/** Make all the parents expanded.*/
	void ExpandParents();

	/** Make all of the children hidden.*/
	void CollapseChildren();

private:
	/** The pin view model that owns this socket view model. */
	TWeakPtr<FNiagaraConvertPinViewModel> OwnerPinViewModel;

	/** The socket view model which owns this view model if it's not a root socket. */
	TWeakPtr<FNiagaraConvertPinSocketViewModel> OwnerPinSocketViewModel;

	/** The name of this socket from the property it represents. */
	FName Name;

	/** The display name of this socket from the property it represents. */
	FName DisplayName;

	/** The direction of this socket. */
	EEdGraphPinDirection Direction;

	/** The child sockets for this socket. */
	TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>> ChildSockets;

	/** The path to this socket as text. */
	FText PathText;

	/** The display path to this socket as text. */
	FText DisplayPathText;

	/** The connection position of this socket in absolute space. */
	FVector2D AbsoluteConnectionPosition;

	/** Whether or not the bIsConnected variable should be refreshed before it is used. */
	bool bIsConnectedNeedsRefresh;

	/** Whether or not this socket is connected. */
	mutable bool bIsConnected;

	/** Whether or not this socket is being dragged. */
	bool bIsBeingDragged;

	/** The absolute drag position of this socket if it is being dragged. */
	FVector2D AbsoluteDragPosition;

	/** If considering the type as a Struct, does this pin represent immediate properties of the type (value 0), or what degree of nesting this type refers to.*/
	int32 TypeTraversalDepth;

	/** The type definition associated with this node.*/
	FNiagaraTypeDefinition TypeDefinition;


	/** Is this socket shown? This must match the parent's bIsShowingChildren.*/
	bool bIsSocketShown;
	
	/** Do we show the expanded icon? All children's bIsSocketShown should match this value.*/
	bool bIsShowingChildren;
	
	/** Expanded arrow image*/
	const FSlateBrush* ExpandedImage;
	
	/** Collapsed arrow image*/
	const FSlateBrush* CollapsedImage;
};
