// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraConvertPinViewModel.h"
#include "NiagaraNodeConvert.h"
#include "NiagaraConvertNodeViewModel.h"
#include "NiagaraConvertPinSocketViewModel.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraCommon.h"

FNiagaraConvertPinViewModel::FNiagaraConvertPinViewModel(TSharedRef<FNiagaraConvertNodeViewModel> InOwnerConvertNodeViewModel, UEdGraphPin& InGraphPin)
	: OwnerConvertNodeViewModel(InOwnerConvertNodeViewModel)
	, GraphPin(InGraphPin)
	, bSocketViewModelsNeedRefresh(true)
{
}

FGuid FNiagaraConvertPinViewModel::GetPinId() const
{
	return GraphPin.PinId;
}

UEdGraphPin& FNiagaraConvertPinViewModel::GetGraphPin()
{
	return GraphPin;
}

const TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& FNiagaraConvertPinViewModel::GetSocketViewModels()
{
	if (bSocketViewModelsNeedRefresh)
	{
		RefreshSocketViewModels();
	}
	return SocketViewModels;
}

TSharedPtr<FNiagaraConvertNodeViewModel> FNiagaraConvertPinViewModel::GetOwnerConvertNodeViewModel()
{
	return OwnerConvertNodeViewModel.Pin();
}

void GenerateSocketViewModelsRecursive(const UEdGraphSchema_Niagara* Schema, TSharedPtr<FNiagaraConvertNodeViewModel> NodeViewModel, TSharedRef<FNiagaraConvertPinViewModel> OwnerPinViewModel, TSharedPtr<FNiagaraConvertPinSocketViewModel> OwnerPinSocketViewModel, EEdGraphPinDirection Direction, const FNiagaraTypeDefinition& TypeDef, TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& SocketViewModels, int32 TypeTraversalDepth = 0)
{
	int32 NumProperties = 0;
	int32 NumStructProperties = 0;
	const UStruct* Struct = TypeDef.GetStruct();
	for (TFieldIterator<FProperty> PropertyIterator(Struct); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;
		if (Property != nullptr)
		{
			NumProperties++;
			FStructProperty* StructProperty = CastField<FStructProperty>(Property);
			if (StructProperty != nullptr)
			{
				NumStructProperties++;
			}
		}
	}

	// We need to add in a pin for root nodes for types that aren't simple types so
	// that you can route the overall value rather than all the individual pieces.
	// Note that when we do this, we'll need to keep track and add in child view models in later code.
	bool bAddedValueParent = false;
	if (TypeTraversalDepth == 0 && NumProperties > 1)
	{
		UEdGraphPin& Pin = OwnerPinViewModel->GetGraphPin();
		FName Name = FName();
		FName DisplayName = TEXT("Value");

		TSharedRef<FNiagaraConvertPinSocketViewModel> SocketViewModel = MakeShareable(new FNiagaraConvertPinSocketViewModel(OwnerPinViewModel, OwnerPinSocketViewModel, Name, DisplayName, TypeDef, Direction, TypeTraversalDepth));
		SocketViewModels.Add(SocketViewModel);
		OwnerPinSocketViewModel = SocketViewModel;
		bAddedValueParent = true;
		TypeTraversalDepth++;
	}

	TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>> OwnerChildSocketViewModels;

	// Now iterate all properties and create their sockets recursively.
	for (TFieldIterator<FProperty> PropertyIterator(Struct); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;
		if (Schema->IsValidNiagaraPropertyType(Property))
		{
			FNiagaraTypeDefinition ChildTypeDef = Schema->GetTypeDefForProperty(Property);
			TSharedRef<FNiagaraConvertPinSocketViewModel> SocketViewModel = MakeShareable(new FNiagaraConvertPinSocketViewModel(OwnerPinViewModel, OwnerPinSocketViewModel, Property->GetFName(), FName(*Property->GetDisplayNameText().ToString()), ChildTypeDef, Direction, TypeTraversalDepth));

			FStructProperty* StructProperty = CastField<FStructProperty>(Property);
			if (StructProperty != nullptr)
			{
				TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>> ChildSocketViewModels;
				GenerateSocketViewModelsRecursive(Schema, NodeViewModel, OwnerPinViewModel, SocketViewModel, Direction, ChildTypeDef, ChildSocketViewModels, TypeTraversalDepth + 1);
				SocketViewModel->SetChildSockets(ChildSocketViewModels);
			}

			if (bAddedValueParent)
			{
				OwnerChildSocketViewModels.Add(SocketViewModel);
			}
			else
			{
				SocketViewModels.Add(SocketViewModel);
			}
		}
	}

	// Finish up the root pin (if created earlier) by setting the accumulated children.
	if (bAddedValueParent)
	{
		OwnerPinSocketViewModel->SetChildSockets(OwnerChildSocketViewModels);
	}
}

void FNiagaraConvertPinViewModel::RefreshSocketViewModels()
{
	SocketViewModels.Empty();
	const UEdGraphSchema_Niagara* Schema = Cast<UEdGraphSchema_Niagara>(GraphPin.GetSchema());
	GenerateSocketViewModelsRecursive(Schema, GetOwnerConvertNodeViewModel(), this->AsShared(), TSharedPtr<FNiagaraConvertPinSocketViewModel>(), GraphPin.Direction, Schema->PinToTypeDefinition(&GraphPin), SocketViewModels);
	bSocketViewModelsNeedRefresh = false;
}