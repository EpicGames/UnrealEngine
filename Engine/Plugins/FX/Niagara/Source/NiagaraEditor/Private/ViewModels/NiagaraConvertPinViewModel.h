// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/SharedPointer.h"

class FNiagaraConvertNodeViewModel;
class FNiagaraConvertPinSocketViewModel;
class UEdGraphPin;

/** A view model for a pin in a convert node. */
class FNiagaraConvertPinViewModel : public TSharedFromThis<FNiagaraConvertPinViewModel>
{
public:
	FNiagaraConvertPinViewModel(TSharedRef<FNiagaraConvertNodeViewModel> InOwnerConvertNodeViewModel, UEdGraphPin& InGraphPin);

	/** Gets the id for the pin. */
	FGuid GetPinId() const;

	/** Gets the graph pin which is represented by this view model. */
	UEdGraphPin& GetGraphPin();

	/** Gets the root socket view models for the socket tree which represents the properties on the type this pin represents. */
	const TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& GetSocketViewModels();

	/** Gets the convert node view model which owns this pin view model. */
	TSharedPtr<FNiagaraConvertNodeViewModel> GetOwnerConvertNodeViewModel();

private:
	/** Rebuilds the socket view models. */
	void RefreshSocketViewModels();

private:
	/** The convert node view model which owns this view model. */
	TWeakPtr<FNiagaraConvertNodeViewModel> OwnerConvertNodeViewModel;

	/** The graph pin which this view model represents. */
	UEdGraphPin& GraphPin;

	/** When true the socket view models need to be rebuilt. */
	bool bSocketViewModelsNeedRefresh;

	/** The root socket view models for the socket tree which represents the properties on the type this pin represents. */
	TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>> SocketViewModels;
};
