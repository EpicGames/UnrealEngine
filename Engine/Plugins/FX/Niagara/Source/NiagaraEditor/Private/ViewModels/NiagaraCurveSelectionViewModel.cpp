// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraCurveSelectionViewModel.h"

#include "EdGraphSchema_Niagara.h"
#include "Misc/AxisDisplayInfo.h"
#include "NiagaraDataInterfaceCurveBase.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraParameterStore.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScript.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraTypes.h"
#include "NiagaraSimulationStageBase.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraPlaceholderDataInterfaceManager.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraCurveSelectionViewModel)

bool FNiagaraCurveSelectionTreeNodeDataId::operator==(const FNiagaraCurveSelectionTreeNodeDataId& Other) const
{
	return UniqueName == Other.UniqueName &&
		Guid == Other.Guid &&
		Object == Other.Object;
}

FNiagaraCurveSelectionTreeNodeDataId FNiagaraCurveSelectionTreeNodeDataId::FromUniqueName(FName UniqueName)
{
	FNiagaraCurveSelectionTreeNodeDataId Identifier;
	Identifier.UniqueName = UniqueName;
	return Identifier;
}

FNiagaraCurveSelectionTreeNodeDataId FNiagaraCurveSelectionTreeNodeDataId::FromGuid(FGuid Guid)
{
	FNiagaraCurveSelectionTreeNodeDataId Identifier;
	Identifier.Guid = Guid;
	return Identifier;
}

FNiagaraCurveSelectionTreeNodeDataId FNiagaraCurveSelectionTreeNodeDataId::FromObject(UObject* Object)
{
	FNiagaraCurveSelectionTreeNodeDataId Identifier;
	Identifier.Object = Object;
	return Identifier;
}

FNiagaraCurveSelectionTreeNode::FNiagaraCurveSelectionTreeNode()
	: NodeUniqueId(FGuid::NewGuid())
	, CurveDataInterface(nullptr)
	, Curve(nullptr)
	, bIsParameter(false)
	, bShowInTree(false)
	, bIsExpanded(false)
	, bIsEnabled(true)
{
}

FGuid FNiagaraCurveSelectionTreeNode::GetNodeUniqueId() const
{
	return NodeUniqueId;
}

const FNiagaraCurveSelectionTreeNodeDataId& FNiagaraCurveSelectionTreeNode::GetDataId() const
{
	return DataId;
}

void FNiagaraCurveSelectionTreeNode::SetDataId(const FNiagaraCurveSelectionTreeNodeDataId& InDataId)
{
	DataId = InDataId;
}

FText FNiagaraCurveSelectionTreeNode::GetDisplayName() const
{
	return DisplayName;
}

void FNiagaraCurveSelectionTreeNode::SetDisplayName(FText InDisplayName)
{
	DisplayName = InDisplayName;
}

FText FNiagaraCurveSelectionTreeNode::GetSecondDisplayName() const
{
	return SecondDisplayName;
}

void FNiagaraCurveSelectionTreeNode::SetSecondDisplayName(FText InSecondDisplayName)
{
	SecondDisplayName = InSecondDisplayName;
}

FText FNiagaraCurveSelectionTreeNode::GetToolTipText() const
{
	return ToolTipText;
}

void FNiagaraCurveSelectionTreeNode::SetToolTipText(FText InToolTipText)
{
	ToolTipText = InToolTipText;
}

ENiagaraCurveSelectionNodeStyleMode FNiagaraCurveSelectionTreeNode::GetStyleMode() const
{
	return StyleMode;
}

FName FNiagaraCurveSelectionTreeNode::GetExecutionCategory() const
{
	return ExecutionCategory;
}

FName FNiagaraCurveSelectionTreeNode::GetExecutionSubcategory() const
{
	return ExecutionSubcategory;
}

bool FNiagaraCurveSelectionTreeNode::GetIsParameter() const
{
	return bIsParameter;
}

void FNiagaraCurveSelectionTreeNode::SetStyle(ENiagaraCurveSelectionNodeStyleMode InStyleMode, FName InExecutionCategory, FName InExecutionSubcategory, bool bInIsParameter)
{
	StyleMode = InStyleMode;
	ExecutionCategory = InExecutionCategory;
	ExecutionSubcategory = InExecutionSubcategory;
	bIsParameter = bInIsParameter;
}

TSharedPtr<FNiagaraCurveSelectionTreeNode> FNiagaraCurveSelectionTreeNode::GetParent() const
{
	return ParentWeak.Pin();
}

void FNiagaraCurveSelectionTreeNode::SetParent(TSharedPtr<FNiagaraCurveSelectionTreeNode> InParent)
{
	ParentWeak = InParent;
}

const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>>& FNiagaraCurveSelectionTreeNode::GetChildNodes() const
{
	return ChildNodes;
}

void FNiagaraCurveSelectionTreeNode::SetChildNodes(const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> InChildNodes)
{
	for (TSharedRef<FNiagaraCurveSelectionTreeNode> ChildNode : ChildNodes)
	{
		ChildNode->SetParent(TSharedPtr<FNiagaraCurveSelectionTreeNode>());
	}
	ChildNodes = InChildNodes;
	for (TSharedRef<FNiagaraCurveSelectionTreeNode> ChildNode : ChildNodes)
	{
		ChildNode->SetParent(this->AsShared());
	}
}

TSharedPtr<FNiagaraCurveSelectionTreeNode> FNiagaraCurveSelectionTreeNode::FindNodeWithDataId(const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>>& Nodes, FNiagaraCurveSelectionTreeNodeDataId DataId)
{
	const TSharedRef<FNiagaraCurveSelectionTreeNode>* NodeWithDataIdPtr = Nodes.FindByPredicate(
		[&DataId](const TSharedRef<FNiagaraCurveSelectionTreeNode>& Node) { return Node.Get().GetDataId() == DataId; });

	return NodeWithDataIdPtr != nullptr ? *NodeWithDataIdPtr : TSharedPtr<FNiagaraCurveSelectionTreeNode>();
}

TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> FNiagaraCurveSelectionTreeNode::GetCurveDataInterface() const
{
	return CurveDataInterface;
}

FRichCurve* FNiagaraCurveSelectionTreeNode::GetCurve() const
{
	return Curve;
}

FName FNiagaraCurveSelectionTreeNode::GetCurveName() const
{
	return CurveName;
}

FLinearColor FNiagaraCurveSelectionTreeNode::GetCurveColor() const
{
	return CurveColor;
}

bool FNiagaraCurveSelectionTreeNode::GetCurveIsReadOnly() const
{
	return Curve != nullptr && GetIsEnabledAndParentIsEnabled() == false;
}

void FNiagaraCurveSelectionTreeNode::SetCurveDataInterface(UNiagaraDataInterfaceCurveBase* InCurveDataInterface)
{
	CurveDataInterface = InCurveDataInterface;
}

void FNiagaraCurveSelectionTreeNode::SetPlaceholderDataInterfaceHandle(TSharedPtr<FNiagaraPlaceholderDataInterfaceHandle> InPlaceholderDataInterfaceHandle)
{
	PlaceholderDataInterfaceHandle = InPlaceholderDataInterfaceHandle;
}

void FNiagaraCurveSelectionTreeNode::SetCurveData(UNiagaraDataInterfaceCurveBase* InCurveDataInterface, FRichCurve* InCurve, FName InCurveName, FLinearColor InCurveColor)
{
	CurveDataInterface = InCurveDataInterface;
	Curve = InCurve;
	CurveName = InCurveName;
	CurveColor = InCurveColor;
}

const TOptional<FObjectKey>& FNiagaraCurveSelectionTreeNode::GetDisplayedObjectKey() const
{
	return DisplayedObjectKey;
}

void FNiagaraCurveSelectionTreeNode::SetDisplayedObjectKey(FObjectKey InDisplayedObjectKey)
{
	DisplayedObjectKey = InDisplayedObjectKey;
}

FSimpleMulticastDelegate& FNiagaraCurveSelectionTreeNode::GetOnCurveChanged()
{
	return OnCurveChangedDelegate;
}

bool FNiagaraCurveSelectionTreeNode::GetShowInTree() const
{
	return bShowInTree;
}

void FNiagaraCurveSelectionTreeNode::SetShowInTree(bool bInShowInTree)
{
	bShowInTree = bInShowInTree;
}

bool FNiagaraCurveSelectionTreeNode::GetIsExpanded() const
{
	return bIsExpanded;
}

void FNiagaraCurveSelectionTreeNode::SetIsExpanded(bool bInIsExpanded)
{
	bIsExpanded = bInIsExpanded;
}

bool FNiagaraCurveSelectionTreeNode::GetIsEnabled() const
{
	return bIsEnabled;
}

void FNiagaraCurveSelectionTreeNode::SetIsEnabled(bool bInIsEnabled)
{
	bIsEnabled = bInIsEnabled;
}

bool FNiagaraCurveSelectionTreeNode::GetIsEnabledAndParentIsEnabled() const
{
	if (bIsEnabledAndParentIsEnabledCache.IsSet() == false)
	{
		bIsEnabledAndParentIsEnabledCache = bIsEnabled && (GetParent().IsValid() == false || GetParent()->GetIsEnabledAndParentIsEnabled());
	}
	return bIsEnabledAndParentIsEnabledCache.GetValue();
}

const TArray<int32>& FNiagaraCurveSelectionTreeNode::GetSortIndices() const
{
	return SortIndices;
}

void FNiagaraCurveSelectionTreeNode::UpdateSortIndices(int32 Index)
{
	SortIndices.Empty();

	TSharedPtr<FNiagaraCurveSelectionTreeNode> Parent = GetParent();
	if (Parent.IsValid())
	{
		SortIndices.Append(Parent->GetSortIndices());
	}

	SortIndices.Add(Index);

	int32 ChildIndex = 0;
	for (const TSharedRef<FNiagaraCurveSelectionTreeNode>& Child : ChildNodes)
	{
		Child->UpdateSortIndices(++ChildIndex);
	}
};

void FNiagaraCurveSelectionTreeNode::ResetCachedEnabledState()
{
	bIsEnabledAndParentIsEnabledCache.Reset();
	for (const TSharedRef<FNiagaraCurveSelectionTreeNode>& Child : ChildNodes)
	{
		Child->ResetCachedEnabledState();
	}
}

void FNiagaraCurveSelectionTreeNode::NotifyCurveChanged()
{
	OnCurveChangedDelegate.Broadcast();
}

void UNiagaraCurveSelectionViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModelWeak = InSystemViewModel;
	InSystemViewModel->GetSystem().GetExposedParameters().AddOnChangedHandler(
		FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraCurveSelectionViewModel::UserParametersChanged));
	RootCurveSelectionTreeNode = MakeShared<FNiagaraCurveSelectionTreeNode>();
	bHandlingInternalCurveChanged = false;
	bRefreshPending = true;
}

void UNiagaraCurveSelectionViewModel::Finalize()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
	if (SystemViewModel.IsValid())
	{
		SystemViewModel->GetSystem().GetExposedParameters().RemoveAllOnChangedHandlers(this);
	}
	SystemViewModelWeak.Reset();
}

const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>>& UNiagaraCurveSelectionViewModel::GetRootNodes()
{
	return RootCurveSelectionTreeNode->GetChildNodes();
}

void UNiagaraCurveSelectionViewModel::FocusAndSelectCurveDataInterface(UNiagaraDataInterfaceCurveBase& CurveDataInterface)
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
	if (SystemViewModel.IsValid())
	{
		FGuid IdOfNodeToSelect;
		TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> TreeNodesToCheck;
		TreeNodesToCheck.Append(RootCurveSelectionTreeNode->GetChildNodes());
		while (IdOfNodeToSelect.IsValid() == false && TreeNodesToCheck.Num() != 0)
		{
			TSharedRef<FNiagaraCurveSelectionTreeNode> TreeNodeToCheck = TreeNodesToCheck[0];
			TreeNodesToCheck.RemoveAt(0);

			if (TreeNodeToCheck->GetCurveDataInterface() == &CurveDataInterface)
			{
				IdOfNodeToSelect = TreeNodeToCheck->GetNodeUniqueId();
			}
			else
			{
				TreeNodesToCheck.Append(TreeNodeToCheck->GetChildNodes());
			}
		}

		if(IdOfNodeToSelect.IsValid())
		{
			SystemViewModel->FocusTab(FNiagaraSystemToolkitModeBase::CurveEditorTabID);
			OnRequestSelectNodeDelegate.Broadcast(IdOfNodeToSelect);
		}
	}
}

TSharedRef<FNiagaraCurveSelectionTreeNode> UNiagaraCurveSelectionViewModel::CreateNodeForCurveDataInterface(const FNiagaraCurveSelectionTreeNodeDataId& DataId, UNiagaraDataInterfaceCurveBase& CurveDataInterface, bool bIsParameter) const
{
	TSharedRef<FNiagaraCurveSelectionTreeNode> DataInterfaceNode = MakeShared<FNiagaraCurveSelectionTreeNode>();
	DataInterfaceNode->SetDataId(DataId);
	DataInterfaceNode->SetStyle(ENiagaraCurveSelectionNodeStyleMode::DataInterface, NAME_None, NAME_None, bIsParameter);
	DataInterfaceNode->SetCurveDataInterface(&CurveDataInterface);
	DataInterfaceNode->SetDisplayedObjectKey(FObjectKey(&CurveDataInterface));
	DataInterfaceNode->SetShowInTree(true);

	TArray<UNiagaraDataInterfaceCurveBase::FCurveData> InterfaceCurveData;
	CurveDataInterface.GetCurveData(InterfaceCurveData);

	if (InterfaceCurveData.Num() == 1)
	{
		DataInterfaceNode->SetCurveData(&CurveDataInterface, InterfaceCurveData[0].Curve, InterfaceCurveData[0].Name, InterfaceCurveData[0].Color);
		DataInterfaceNode->GetOnCurveChanged().AddUObject(this, &UNiagaraCurveSelectionViewModel::DataInterfaceCurveChanged, TWeakObjectPtr<UNiagaraDataInterfaceCurveBase>(&CurveDataInterface));
	}
	else
	{
		TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> CurveNodes;
		int32 CurveComponentIndex = 0;
		for (UNiagaraDataInterfaceCurveBase::FCurveData& CurveData : InterfaceCurveData)
		{
			FText CurveDisplayName;
			FText CurveToolTipText;
			FLinearColor CurveDisplayColor;

			EAxisList::Type CurveAxis = FNiagaraEditorUtilities::VectorComponentToAxis(InterfaceCurveData.Num(), CurveComponentIndex);
			if (CurveAxis != EAxisList::None)
			{
				CurveDisplayName = AxisDisplayInfo::GetAxisDisplayNameShort(CurveAxis);
				CurveToolTipText = AxisDisplayInfo::GetAxisDisplayName(CurveAxis);
				CurveDisplayColor = AxisDisplayInfo::GetAxisColor(CurveAxis);
			}
			else
			{
				CurveDisplayName = FText::FromName(CurveData.Name);
				CurveDisplayColor = CurveData.Color;
			}

			TSharedRef<FNiagaraCurveSelectionTreeNode> CurveNode = MakeShared<FNiagaraCurveSelectionTreeNode>();
			CurveNode->SetDataId(FNiagaraCurveSelectionTreeNodeDataId::FromUniqueName(CurveData.Name));
			CurveNode->SetDisplayName(CurveDisplayName);
			CurveNode->SetToolTipText(CurveToolTipText);
			CurveNode->SetStyle(ENiagaraCurveSelectionNodeStyleMode::CurveComponent, NAME_None, NAME_None, false);
			CurveNode->SetCurveData(&CurveDataInterface, CurveData.Curve, CurveData.Name, CurveDisplayColor);
			CurveNode->SetDisplayedObjectKey(FObjectKey(&CurveDataInterface));
			CurveNode->SetShowInTree(true);
			CurveNode->GetOnCurveChanged().AddUObject(this, &UNiagaraCurveSelectionViewModel::DataInterfaceCurveChanged, TWeakObjectPtr<UNiagaraDataInterfaceCurveBase>(&CurveDataInterface));
			CurveNodes.Add(CurveNode);
			CurveComponentIndex++;
		}
		DataInterfaceNode->SetChildNodes(CurveNodes);
	}
	return DataInterfaceNode;
}

TSharedPtr<FNiagaraCurveSelectionTreeNode> UNiagaraCurveSelectionViewModel::CreateNodeForUserParameters(TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes, UNiagaraSystem& System) const
{
	TSharedPtr<FNiagaraCurveSelectionTreeNode> NewUserNode;
	if (System.GetExposedParameters().Num() > 0)
	{
		TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldUserChildNodes;
		TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> NewUserChildNodes;

		FNiagaraCurveSelectionTreeNodeDataId UserDataId = FNiagaraCurveSelectionTreeNodeDataId::FromUniqueName(FNiagaraConstants::UserNamespace);
		TSharedPtr<FNiagaraCurveSelectionTreeNode> OldUserNode = FNiagaraCurveSelectionTreeNode::FindNodeWithDataId(OldParentChildNodes, UserDataId);
		if (OldUserNode.IsValid())
		{
			OldUserChildNodes = OldUserNode->GetChildNodes();
		}

		TArray<FNiagaraVariable> UserParameters;
		System.GetExposedParameters().GetParameters(UserParameters);

		for (const FNiagaraVariable& UserParameter : UserParameters)
		{
			if (UserParameter.IsDataInterface() && UserParameter.GetType().GetClass()->IsChildOf<UNiagaraDataInterfaceCurveBase>())
			{
				UNiagaraDataInterfaceCurveBase* CurveDataInterface = Cast<UNiagaraDataInterfaceCurveBase>(System.GetExposedParameters().GetDataInterface(UserParameter));
				if (CurveDataInterface != nullptr)
				{
					FNiagaraCurveSelectionTreeNodeDataId UserParameterDataId = FNiagaraCurveSelectionTreeNodeDataId::FromObject(CurveDataInterface);
					TSharedPtr<FNiagaraCurveSelectionTreeNode> UserParameterNode = FNiagaraCurveSelectionTreeNode::FindNodeWithDataId(OldUserChildNodes, UserParameterDataId);
					if (UserParameterNode.IsValid() == false)
					{
						UserParameterNode = CreateNodeForCurveDataInterface(UserParameterDataId, *CurveDataInterface, true);
					}
					UserParameterNode->SetDisplayName(FText::FromName(UserParameter.GetName()));
					UserParameterNode->SetSecondDisplayName(CurveDataInterface->GetClass()->GetDisplayNameText());
					NewUserChildNodes.Add(UserParameterNode.ToSharedRef());
				}
			}
		}

		if (NewUserChildNodes.Num() > 0)
		{
			if (OldUserNode.IsValid())
			{
				NewUserNode = OldUserNode;
			}
			else
			{
				NewUserNode = MakeShared<FNiagaraCurveSelectionTreeNode>();
				NewUserNode->SetDataId(UserDataId);
				NewUserNode->SetDisplayName(FText::FromName(FNiagaraConstants::UserNamespace));
				NewUserNode->SetStyle(ENiagaraCurveSelectionNodeStyleMode::Script, UNiagaraStackEntry::FExecutionCategoryNames::System, UNiagaraStackEntry::FExecutionSubcategoryNames::Settings, false);
				NewUserNode->SetDisplayedObjectKey(FObjectKey(&System));
			}
			NewUserNode->SetChildNodes(NewUserChildNodes);
		}
	}
	return NewUserNode;
}

TSharedPtr<FNiagaraCurveSelectionTreeNode> UNiagaraCurveSelectionViewModel::CreateNodeForFunction(
	const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes,
	UNiagaraNodeFunctionCall& FunctionCallNode, UNiagaraStackEditorData& StackEditorData,
	FName ExecutionCategory, FName ExecutionSubCategory,
	FName InputName, bool bIsParameterDynamicInput,
	FCompileConstantResolver& ConstantResolver, FGuid& OwningEmitterHandleId) const
{
	TSharedPtr<FNiagaraCurveSelectionTreeNode> NewFunctionNode;

	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldFunctionChildNodes;
	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> NewFunctionChildNodes;

	FNiagaraCurveSelectionTreeNodeDataId FunctionDataId = FNiagaraCurveSelectionTreeNodeDataId::FromObject(&FunctionCallNode);
	TSharedPtr<FNiagaraCurveSelectionTreeNode> OldFunctionNode = FNiagaraCurveSelectionTreeNode::FindNodeWithDataId(OldParentChildNodes, FunctionDataId);
	if (OldFunctionNode.IsValid())
	{
		OldFunctionChildNodes = OldFunctionNode->GetChildNodes();
	}

	bool bIsParameterInput = FunctionCallNode.IsA<UNiagaraNodeAssignment>();

	// Handle dynamic inputs and collect override curve data interfaces by name.
	TMap<FName, UNiagaraDataInterfaceCurveBase*> InputNameToOverrideCurveDataInterface;
	UNiagaraNodeParameterMapSet* OverrideNode = FNiagaraStackGraphUtilities::GetStackFunctionOverrideNode(FunctionCallNode);
	if (OverrideNode != nullptr)
	{
		TArray<UEdGraphPin*> OverridePins = FNiagaraStackGraphUtilities::GetOverridePinsForFunction(*OverrideNode, FunctionCallNode);
		for (UEdGraphPin* OverridePin : OverridePins)
		{
			if (OverridePin->LinkedTo.Num() == 1)
			{
				FNiagaraParameterHandle InputHandle(OverridePin->PinName);
				UNiagaraNodeFunctionCall* InputFunctionCallNode = Cast<UNiagaraNodeFunctionCall>(OverridePin->LinkedTo[0]->GetOwningNode());
				if (InputFunctionCallNode != nullptr)
				{
					TSharedPtr<FNiagaraCurveSelectionTreeNode> FunctionInputNode = CreateNodeForFunction(
						OldFunctionChildNodes, *InputFunctionCallNode, StackEditorData, 
						NAME_None, NAME_None, InputHandle.GetName(), bIsParameterInput,
						ConstantResolver, OwningEmitterHandleId);
					if(FunctionInputNode.IsValid())
					{
						NewFunctionChildNodes.Add(FunctionInputNode.ToSharedRef());
					}
				}
				else
				{
					UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(OverridePin->LinkedTo[0]->GetOwningNode());
					if (InputNode != nullptr && InputNode->GetDataInterface() != nullptr && InputNode->GetDataInterface()->IsA<UNiagaraDataInterfaceCurveBase>())
					{
						InputNameToOverrideCurveDataInterface.Add(InputHandle.GetName(), CastChecked<UNiagaraDataInterfaceCurveBase>(InputNode->GetDataInterface()));
					}
				}
			}
		}
	}

	TArray<FNiagaraVariable> InputVariables;
	TSet<FNiagaraVariable> HiddenVariables;
	FNiagaraStackGraphUtilities::GetStackFunctionInputs(FunctionCallNode, InputVariables, HiddenVariables, ConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly);
	for (FNiagaraVariable InputVariable : InputVariables)
	{
		if (HiddenVariables.Contains(InputVariable))
		{
			continue;
		}

		if (InputVariable.IsValid() && InputVariable.GetType().IsDataInterface() && InputVariable.GetType().GetClass()->IsChildOf(UNiagaraDataInterfaceCurveBase::StaticClass()))
		{
			UNiagaraDataInterfaceCurveBase* DataInterfaceToDisplay = nullptr;
			TSharedPtr<FNiagaraPlaceholderDataInterfaceHandle> PlaceholderDataInterfaceHandle;

			TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
			FNiagaraParameterHandle InputHandle(InputVariable.GetName());
			UNiagaraDataInterfaceCurveBase** OverrideCurveDataInterfacePtr = InputNameToOverrideCurveDataInterface.Find(InputHandle.GetName());
			if (OverrideCurveDataInterfacePtr != nullptr)
			{
				PlaceholderDataInterfaceHandle = SystemViewModel->GetPlaceholderDataInterfaceManager()->GetPlaceholderDataInterface(OwningEmitterHandleId, FunctionCallNode, InputHandle);
				if(PlaceholderDataInterfaceHandle.IsValid())
				{
					// If there is an active placeholder data interface, display and edit it to keep other views consistent.  Changes to it will be copied to the target data interface
					// by the placeholder manager.
					DataInterfaceToDisplay = CastChecked<UNiagaraDataInterfaceCurveBase>(PlaceholderDataInterfaceHandle->GetDataInterface());
				}
				else
				{
					DataInterfaceToDisplay = *OverrideCurveDataInterfacePtr;
				}
			}
			else
			{
				PlaceholderDataInterfaceHandle = SystemViewModel->GetPlaceholderDataInterfaceManager()->GetOrCreatePlaceholderDataInterface(
					OwningEmitterHandleId, FunctionCallNode, InputHandle, InputVariable.GetType().GetClass());
				DataInterfaceToDisplay = CastChecked<UNiagaraDataInterfaceCurveBase>(PlaceholderDataInterfaceHandle->GetDataInterface());
			}

			FNiagaraCurveSelectionTreeNodeDataId InputCurveDataInterfaceDataId = FNiagaraCurveSelectionTreeNodeDataId::FromObject(DataInterfaceToDisplay);
			TSharedPtr<FNiagaraCurveSelectionTreeNode> FunctionInputNode = FNiagaraCurveSelectionTreeNode::FindNodeWithDataId(OldFunctionChildNodes, InputCurveDataInterfaceDataId);
			if (FunctionInputNode.IsValid() == false)
			{
				FunctionInputNode = CreateNodeForCurveDataInterface(InputCurveDataInterfaceDataId, *DataInterfaceToDisplay, bIsParameterInput);
			}

			FunctionInputNode->SetPlaceholderDataInterfaceHandle(PlaceholderDataInterfaceHandle);
			FunctionInputNode->SetDisplayName(FText::FromName(InputHandle.GetName()));
			FunctionInputNode->SetSecondDisplayName(DataInterfaceToDisplay->GetClass()->GetDisplayNameText());

			NewFunctionChildNodes.Add(FunctionInputNode.ToSharedRef());
		}
	}

	if (NewFunctionChildNodes.Num() > 0)
	{
		if (OldFunctionNode.IsValid())
		{
			NewFunctionNode = OldFunctionNode;
		}
		else
		{
			NewFunctionNode = MakeShared<FNiagaraCurveSelectionTreeNode>();
			NewFunctionNode->SetDataId(FunctionDataId);
			NewFunctionNode->SetDisplayedObjectKey(FObjectKey(&FunctionCallNode));

			if (InputName != NAME_None)
			{
				NewFunctionNode->SetStyle(ENiagaraCurveSelectionNodeStyleMode::DynamicInput, ExecutionCategory, ExecutionSubCategory, bIsParameterDynamicInput);
			}
			else
			{
				NewFunctionNode->SetStyle(ENiagaraCurveSelectionNodeStyleMode::Module, ExecutionCategory, ExecutionSubCategory, bIsParameterDynamicInput);
			}
		}

		FText FunctionDisplayName = FunctionCallNode.GetNodeTitle(ENodeTitleType::ListView);
		if(InputName != NAME_None)
		{
			NewFunctionNode->SetDisplayName(FText::FromName(InputName));
			NewFunctionNode->SetSecondDisplayName(FunctionDisplayName);
		}
		else
		{
			const FText* DisplayName = StackEditorData.GetStackEntryDisplayName(FNiagaraStackGraphUtilities::StackKeys::GenerateStackModuleEditorDataKey(FunctionCallNode));
			if(DisplayName != nullptr)
			{
				NewFunctionNode->SetDisplayName(*DisplayName);
				NewFunctionNode->SetSecondDisplayName(FunctionDisplayName);
			}
			else
			{
				NewFunctionNode->SetDisplayName(FunctionDisplayName);
				NewFunctionNode->SetSecondDisplayName(FText());
			}
		}

		NewFunctionNode->SetIsEnabled(FunctionCallNode.GetDesiredEnabledState() != ENodeEnabledState::Disabled);
		NewFunctionNode->SetShowInTree(InputName == NAME_None || (NewFunctionChildNodes.Num() != 1 || NewFunctionChildNodes[0]->GetStyleMode() != ENiagaraCurveSelectionNodeStyleMode::DataInterface));
		NewFunctionNode->SetChildNodes(NewFunctionChildNodes);
	}
	return NewFunctionNode;
}

TSharedPtr<FNiagaraCurveSelectionTreeNode> UNiagaraCurveSelectionViewModel::CreateNodeForScript(
	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes,
	UNiagaraScript& Script, FString ScriptDisplayName, UNiagaraStackEditorData& StackEditorData,
	FName ExecutionCategory, FName ExecutionSubcategory,
	const FNiagaraEmitterHandle* OwningEmitterHandle) const
{
	TSharedPtr<FNiagaraCurveSelectionTreeNode> NewScriptNode;

	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldScriptChildNodes;
	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> NewScriptChildNodes;

	FNiagaraCurveSelectionTreeNodeDataId ScriptDataId = FNiagaraCurveSelectionTreeNodeDataId::FromObject(&Script);
	TSharedPtr<FNiagaraCurveSelectionTreeNode> OldScriptNode = FNiagaraCurveSelectionTreeNode::FindNodeWithDataId(OldParentChildNodes, ScriptDataId);
	if (OldScriptNode != nullptr)
	{
		OldScriptChildNodes = OldScriptNode->GetChildNodes();
	}

	UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(Script.GetLatestSource());
	if (ScriptSource != nullptr)
	{
		UNiagaraNodeOutput* OutputNode = ScriptSource->NodeGraph->FindEquivalentOutputNode(Script.GetUsage(), Script.GetUsageId());
		if (OutputNode != nullptr)
		{
			FCompileConstantResolver ConstantResolver = OwningEmitterHandle == nullptr
				? FCompileConstantResolver(&SystemViewModelWeak.Pin()->GetSystem(), Script.GetUsage())
				: FCompileConstantResolver(OwningEmitterHandle->GetInstance(), Script.GetUsage());
			FGuid OwningEmitterHandleId = OwningEmitterHandle != nullptr ? OwningEmitterHandle->GetId() : FGuid();

			TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> ScriptGroups;
			FNiagaraStackGraphUtilities::GetStackNodeGroups(*OutputNode, ScriptGroups);

			for (const FNiagaraStackGraphUtilities::FStackNodeGroup& ScriptGroup : ScriptGroups)
			{
				UNiagaraNodeFunctionCall* ModuleFunctionCallNode = Cast<UNiagaraNodeFunctionCall>(ScriptGroup.EndNode);
				if (ModuleFunctionCallNode != nullptr)
				{
					TSharedPtr<FNiagaraCurveSelectionTreeNode> ModuleNode = CreateNodeForFunction(
						OldScriptChildNodes, *ModuleFunctionCallNode, StackEditorData,
						ExecutionCategory, ExecutionSubcategory, NAME_None, false, ConstantResolver, OwningEmitterHandleId);
					if (ModuleNode.IsValid())
					{
						NewScriptChildNodes.Add(ModuleNode.ToSharedRef());
					}
				}
			}
		}
	}
	
	if (NewScriptChildNodes.Num() > 0)
	{
		if (OldScriptNode.IsValid())
		{
			NewScriptNode = OldScriptNode;
		}
		else
		{
			NewScriptNode = MakeShared<FNiagaraCurveSelectionTreeNode>();
			NewScriptNode->SetDataId(ScriptDataId);
			NewScriptNode->SetDisplayName(FText::FromString(ScriptDisplayName));
			NewScriptNode->SetStyle(ENiagaraCurveSelectionNodeStyleMode::Script, ExecutionCategory, ExecutionSubcategory, false);
			NewScriptNode->SetDisplayedObjectKey(FObjectKey(&Script));
		}
		NewScriptNode->SetChildNodes(NewScriptChildNodes);
	}
	return NewScriptNode;
}

TSharedPtr<FNiagaraCurveSelectionTreeNode> UNiagaraCurveSelectionViewModel::CreateNodeForSystem(TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes, UNiagaraSystem& System) const
{
	TSharedPtr<FNiagaraCurveSelectionTreeNode> NewSystemNode;

	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldSystemChildNodes;
	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> NewSystemChildNodes;

	FNiagaraCurveSelectionTreeNodeDataId SystemDataId = FNiagaraCurveSelectionTreeNodeDataId::FromUniqueName(System.GetFName());
	TSharedPtr<FNiagaraCurveSelectionTreeNode> OldSystemNode = FNiagaraCurveSelectionTreeNode::FindNodeWithDataId(OldParentChildNodes, SystemDataId);
	if (OldSystemNode.IsValid())
	{
		OldSystemChildNodes = OldSystemNode->GetChildNodes();
	}

	TSharedPtr<FNiagaraCurveSelectionTreeNode> UserNode = CreateNodeForUserParameters(OldSystemChildNodes, System);
	if (UserNode.IsValid())
	{
		NewSystemChildNodes.Add(UserNode.ToSharedRef());
	}

	UNiagaraStackEditorData& StackEditorData = CastChecked<UNiagaraSystemEditorData>(System.GetEditorData())->GetStackEditorData();

	TSharedPtr<FNiagaraCurveSelectionTreeNode> SystemSpawnNode = CreateNodeForScript(
		OldSystemChildNodes, *System.GetSystemSpawnScript(), TEXT("System Spawn"), StackEditorData,
		UNiagaraStackEntry::FExecutionCategoryNames::System, UNiagaraStackEntry::FExecutionSubcategoryNames::Spawn,
		nullptr);
	if (SystemSpawnNode.IsValid())
	{
		NewSystemChildNodes.Add(SystemSpawnNode.ToSharedRef());
	}

	TSharedPtr<FNiagaraCurveSelectionTreeNode> SystemUpdateNode = CreateNodeForScript(
		OldSystemChildNodes, *System.GetSystemUpdateScript(), TEXT("System Update"), StackEditorData,
		UNiagaraStackEntry::FExecutionCategoryNames::System, UNiagaraStackEntry::FExecutionSubcategoryNames::Update,
		nullptr);
	if (SystemUpdateNode.IsValid())
	{
		NewSystemChildNodes.Add(SystemUpdateNode.ToSharedRef());
	}

	if (NewSystemChildNodes.Num() > 0)
	{
		if (OldSystemNode.IsValid())
		{
			NewSystemNode = OldSystemNode;
		}
		else
		{
			NewSystemNode = MakeShared<FNiagaraCurveSelectionTreeNode>();
			NewSystemNode->SetDataId(SystemDataId);
			NewSystemNode->SetSecondDisplayName(FText::FromName(FNiagaraConstants::SystemNamespace));
			NewSystemNode->SetStyle(ENiagaraCurveSelectionNodeStyleMode::TopLevelObject, UNiagaraStackEntry::FExecutionCategoryNames::System, UNiagaraStackEntry::FExecutionSubcategoryNames::Settings, false);
			NewSystemNode->SetDisplayedObjectKey(FObjectKey(&System));
			NewSystemNode->SetShowInTree(true);
			NewSystemNode->SetIsExpanded(true);

			// When creating a node for this system the first time, bind the changed delegate for the stack editor data so the
			// nodes can be refreshed when the display names are overridden.
			StackEditorData.OnPersistentDataChanged().AddUObject(const_cast<UNiagaraCurveSelectionViewModel*>(this), &UNiagaraCurveSelectionViewModel::StackEditorDataChanged);
		}

		NewSystemNode->SetDisplayName(FText::FromName(System.GetFName()));
		NewSystemNode->SetChildNodes(NewSystemChildNodes);
	}
	else if(OldSystemNode.IsValid())
	{
		// Unbind the stack editor data change since this old system node is no longer used.
		StackEditorData.OnPersistentDataChanged().RemoveAll(this);
	}
	return NewSystemNode;
}

TSharedPtr<FNiagaraCurveSelectionTreeNode> UNiagaraCurveSelectionViewModel::CreateNodeForEmitter(TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes, const FNiagaraEmitterHandle& EmitterHandle) const
{
	TSharedPtr<FNiagaraCurveSelectionTreeNode> NewEmitterNode;

	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldEmitterChildNodes;
	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> NewEmitterChildNodes;

	FNiagaraCurveSelectionTreeNodeDataId EmitterDataId = FNiagaraCurveSelectionTreeNodeDataId::FromGuid(EmitterHandle.GetId());
	TSharedPtr<FNiagaraCurveSelectionTreeNode> OldEmitterNode = FNiagaraCurveSelectionTreeNode::FindNodeWithDataId(OldParentChildNodes, EmitterDataId);
	if (OldEmitterNode.IsValid())
	{
		OldEmitterChildNodes = OldEmitterNode->GetChildNodes();
	}

	// Early out for stateless emitters
	FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
	if (EmitterData == nullptr)
	{
		ensureMsgf(OldEmitterNode.IsValid(), TEXT("Old emitter node was valid and new one is not, this is not expected"));
		return NewEmitterNode;
	}

	UNiagaraStackEditorData& StackEditorData = CastChecked<UNiagaraEmitterEditorData>(EmitterData->GetEditorData())->GetStackEditorData();
	TSharedPtr<FNiagaraCurveSelectionTreeNode> EmitterSpawnNode = CreateNodeForScript(
		OldEmitterChildNodes, *EmitterData->EmitterSpawnScriptProps.Script, TEXT("Emitter Spawn"), StackEditorData,
		UNiagaraStackEntry::FExecutionCategoryNames::Emitter, UNiagaraStackEntry::FExecutionSubcategoryNames::Spawn,
		&EmitterHandle);
	if (EmitterSpawnNode.IsValid())
	{
		NewEmitterChildNodes.Add(EmitterSpawnNode.ToSharedRef());
	}

	TSharedPtr<FNiagaraCurveSelectionTreeNode> EmitterUpdateNode = CreateNodeForScript(
		OldEmitterChildNodes, *EmitterData->EmitterUpdateScriptProps.Script, TEXT("Emitter Update"), StackEditorData,
		UNiagaraStackEntry::FExecutionCategoryNames::Emitter, UNiagaraStackEntry::FExecutionSubcategoryNames::Update,
		&EmitterHandle);
	if (EmitterUpdateNode.IsValid())
	{
		NewEmitterChildNodes.Add(EmitterUpdateNode.ToSharedRef());
	}

	TSharedPtr<FNiagaraCurveSelectionTreeNode> ParticleSpawnNode = CreateNodeForScript(
		OldEmitterChildNodes, *EmitterData->SpawnScriptProps.Script, TEXT("Particle Spawn"), StackEditorData,
		UNiagaraStackEntry::FExecutionCategoryNames::Particle, UNiagaraStackEntry::FExecutionSubcategoryNames::Spawn,
		&EmitterHandle);
	if (ParticleSpawnNode.IsValid())
	{
		NewEmitterChildNodes.Add(ParticleSpawnNode.ToSharedRef());
	}

	TSharedPtr<FNiagaraCurveSelectionTreeNode> ParticleUpdateNode = CreateNodeForScript(
		OldEmitterChildNodes, *EmitterData->UpdateScriptProps.Script, TEXT("Particle Update"), StackEditorData,
		UNiagaraStackEntry::FExecutionCategoryNames::Particle, UNiagaraStackEntry::FExecutionSubcategoryNames::Update,
		&EmitterHandle);
	if (ParticleUpdateNode.IsValid())
	{
		NewEmitterChildNodes.Add(ParticleUpdateNode.ToSharedRef());
	}

	for (const FNiagaraEventScriptProperties& EventScriptProps : EmitterData->GetEventHandlers())
	{
		TSharedPtr<FNiagaraCurveSelectionTreeNode> EmitterEventNode = CreateNodeForScript(
			OldEmitterChildNodes, *EventScriptProps.Script, FString::Printf(TEXT("Event Handler - %s"), *EventScriptProps.SourceEventName.ToString()), StackEditorData,
			UNiagaraStackEntry::FExecutionCategoryNames::Particle, UNiagaraStackEntry::FExecutionSubcategoryNames::Event,
			&EmitterHandle);
		if (EmitterEventNode.IsValid())
		{
			NewEmitterChildNodes.Add(EmitterEventNode.ToSharedRef());
		}
	}

	for (UNiagaraSimulationStageBase* SimulationStage : EmitterData->GetSimulationStages())
	{
		TSharedPtr<FNiagaraCurveSelectionTreeNode> EmitterSimulationStageNode = CreateNodeForScript(
			OldEmitterChildNodes, *SimulationStage->Script, FString::Printf(TEXT("Simulation Stage - %s"), *SimulationStage->SimulationStageName.ToString()), StackEditorData,
			UNiagaraStackEntry::FExecutionCategoryNames::Particle, UNiagaraStackEntry::FExecutionSubcategoryNames::SimulationStage,
			&EmitterHandle);
		if (EmitterSimulationStageNode.IsValid())
		{
			NewEmitterChildNodes.Add(EmitterSimulationStageNode.ToSharedRef());
		}
	}

	if (NewEmitterChildNodes.Num() > 0)
	{
		if (OldEmitterNode.IsValid())
		{
			NewEmitterNode = OldEmitterNode;
		}
		else
		{
			NewEmitterNode = MakeShared<FNiagaraCurveSelectionTreeNode>();
			NewEmitterNode->SetDataId(EmitterDataId);
			NewEmitterNode->SetSecondDisplayName(FText::FromName(FNiagaraConstants::EmitterNamespace));
			NewEmitterNode->SetStyle(ENiagaraCurveSelectionNodeStyleMode::TopLevelObject, UNiagaraStackEntry::FExecutionCategoryNames::Emitter, UNiagaraStackEntry::FExecutionSubcategoryNames::Settings, false);
			NewEmitterNode->SetDisplayedObjectKey(FObjectKey(EmitterHandle.GetInstance().Emitter));
			NewEmitterNode->SetShowInTree(true);
			NewEmitterNode->SetIsExpanded(true);

			// When creating a node for this emitter the first time, bind the changed delegate for the stack editor data so the
			// nodes can be refreshed when the display names are overriden.
			StackEditorData.OnPersistentDataChanged().AddUObject(const_cast<UNiagaraCurveSelectionViewModel*>(this), &UNiagaraCurveSelectionViewModel::StackEditorDataChanged);
		}

		NewEmitterNode->SetIsEnabled(EmitterHandle.GetIsEnabled());
		NewEmitterNode->SetDisplayName(FText::FromName(EmitterHandle.GetName()));
		NewEmitterNode->SetChildNodes(NewEmitterChildNodes);
	}
	else if (OldEmitterNode.IsValid())
	{
		// Unbind the stack editor data change since this old emitter node is no longer used.
		StackEditorData.OnPersistentDataChanged().RemoveAll(this);
	}
	return NewEmitterNode;
}

void UNiagaraCurveSelectionViewModel::Refresh()
{
	if (bHandlingInternalCurveChanged)
	{
		return;
	}

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldRootChildNodes = RootCurveSelectionTreeNode->GetChildNodes();
	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> NewRootChildNodes;

	if (SystemViewModel.IsValid())
	{
		UNiagaraSystem& System = SystemViewModel->GetSystem();
		TSharedPtr<FNiagaraCurveSelectionTreeNode> SystemNode = CreateNodeForSystem(OldRootChildNodes, System);
		if (SystemNode.IsValid())
		{
			NewRootChildNodes.Add(SystemNode.ToSharedRef());
		}

		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
		{
			// During transactions Emitters may not have their EditorData set if they were created during the transaction; skip if this is true.
			FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
			if (EmitterHandle && EmitterHandle->GetEmitterData() && EmitterHandle->GetEmitterData()->GetEditorData())
			{
				TSharedPtr<FNiagaraCurveSelectionTreeNode> EmitterNode = CreateNodeForEmitter(OldRootChildNodes, *EmitterHandle);
				if (EmitterNode.IsValid())
				{
					NewRootChildNodes.Add(EmitterNode.ToSharedRef());
				}
			}
		}
	}

	RootCurveSelectionTreeNode->SetChildNodes(NewRootChildNodes);
	RootCurveSelectionTreeNode->UpdateSortIndices(0);
	RootCurveSelectionTreeNode->ResetCachedEnabledState();

	bRefreshPending = false;
	OnRefreshedDelegate.Broadcast();
}

void UNiagaraCurveSelectionViewModel::RefreshDeferred()
{
	bRefreshPending = true;
}

void UNiagaraCurveSelectionViewModel::Tick()
{
	if (bRefreshPending)
	{
		Refresh();
	}
}

FSimpleMulticastDelegate& UNiagaraCurveSelectionViewModel::OnRefreshed()
{
	return OnRefreshedDelegate;
}

UNiagaraCurveSelectionViewModel::FOnRequestSelectNode& UNiagaraCurveSelectionViewModel::OnRequestSelectNode()
{
	return OnRequestSelectNodeDelegate;
}

void UNiagaraCurveSelectionViewModel::DataInterfaceCurveChanged(TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> ChangedCurveDataInterfaceWeak) const
{
	UNiagaraDataInterfaceCurveBase* ChangedCurveDataInterface = ChangedCurveDataInterfaceWeak.Get();
	if (ChangedCurveDataInterface != nullptr)
	{
		ChangedCurveDataInterface->UpdateLUT();
		TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
		if (SystemViewModel.IsValid())
		{
			TGuardValue<bool> RefreshGuard(bHandlingInternalCurveChanged, true);
			TArray<UObject*> ChangedObjects = { ChangedCurveDataInterface };
			SystemViewModel->NotifyDataObjectChanged(ChangedObjects, ENiagaraDataObjectChange::Changed);
		}
	}
}

void UNiagaraCurveSelectionViewModel::StackEditorDataChanged()
{
	if(SystemViewModelWeak.IsValid())
	{
		bRefreshPending = true;
	}
}

void UNiagaraCurveSelectionViewModel::UserParametersChanged()
{
	if (SystemViewModelWeak.IsValid())
	{
		bRefreshPending = true;
	}
}
