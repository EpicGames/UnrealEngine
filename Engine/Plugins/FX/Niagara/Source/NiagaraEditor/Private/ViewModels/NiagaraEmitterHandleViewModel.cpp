// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitterHandle.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "NiagaraScriptGraphViewModel.h"
#include "NiagaraScriptSource.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraScriptOutputCollectionViewModel.h"
#include "Algo/Find.h"
#include "ScopedTransaction.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"
#include "ViewModels/Stack/NiagaraStackRenderItemGroup.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "NiagaraMessages.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "Widgets/SNiagaraDebugger.h"

#define LOCTEXT_NAMESPACE "EmitterHandleViewModel"

FNiagaraEmitterHandleViewModel::FNiagaraEmitterHandleViewModel(bool bInIsForDataProcessingOnly)
	: EmitterHandleIndex(INDEX_NONE)
	, EmitterHandle(nullptr)
	, EmitterViewModel(MakeShared<FNiagaraEmitterViewModel>(bInIsForDataProcessingOnly))
	, EmitterStackViewModel(NewObject<UNiagaraStackViewModel>(GetTransientPackage()))
	, bIsRenamePending(false)
{
}

bool FNiagaraEmitterHandleViewModel::IsValid() const
{
	if(EmitterHandleIndex != INDEX_NONE && EmitterHandle != nullptr)
	{
		TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = OwningSystemViewModelWeak.Pin();
		if (SystemViewModel.IsValid() && SystemViewModel->IsValid())
		{
			UNiagaraSystem& System = SystemViewModel->GetSystem();
			return EmitterHandleIndex < System.GetNumEmitters() && &System.GetEmitterHandle(EmitterHandleIndex) == EmitterHandle;
		}
	}
	return false;
}

void FNiagaraEmitterHandleViewModel::Cleanup()
{
	EmitterViewModel->Cleanup();
	if (EmitterStackViewModel != nullptr)
	{
		EmitterStackViewModel->Finalize();
		EmitterStackViewModel = nullptr;
	}
}

void FNiagaraEmitterHandleViewModel::GetRendererEntries(TArray<UNiagaraStackEntry*>& InRenderingEntries)
{
	InRenderingEntries.Empty();
	UNiagaraStackRoot* StackRoot = Cast<UNiagaraStackRoot>(EmitterStackViewModel->GetRootEntry());
	if (StackRoot)
	{
		TArray<UNiagaraStackEntry*> Children;
		UNiagaraStackRenderItemGroup* RendererGroup = StackRoot->GetRenderGroup();
		if (RendererGroup != nullptr)
		{
			RendererGroup->GetUnfilteredChildren(Children);
			for (UNiagaraStackEntry* Child : Children)
			{
				if (UNiagaraStackRendererItem* RendererItem = Cast<UNiagaraStackRendererItem>(Child))
				{
					InRenderingEntries.Add(Child);
				}
			}
		}
	}
}

TSharedRef<FNiagaraSystemViewModel> FNiagaraEmitterHandleViewModel::GetOwningSystemViewModel() const
{
	TSharedPtr<FNiagaraSystemViewModel> OwningSystemViewModelPinned = OwningSystemViewModelWeak.Pin();
	checkf(OwningSystemViewModelPinned.IsValid(), TEXT("Owning system view model was destroyed before child handle view model."));
	return OwningSystemViewModelPinned.ToSharedRef();
}

FGuid FNiagaraEmitterHandleViewModel::AddMessage(UNiagaraMessageData* NewMessage, const FGuid& InNewGuid /*= FGuid()*/) const
{
	if (ensureMsgf(EmitterHandle != nullptr, TEXT("EmitterHandleViewModel had a null EmitterHandle!")))
	{
		const FGuid NewGuid = InNewGuid.IsValid() ? InNewGuid : FGuid::NewGuid();
		
		EmitterHandle->GetInstance().Emitter->GetMessageStore().AddMessage(NewGuid, NewMessage);
		return NewGuid;
	}
	return FGuid();
}

void FNiagaraEmitterHandleViewModel::RemoveMessage(const FGuid& MessageKey) const
{
	if (ensureMsgf(EmitterHandle != nullptr, TEXT("EmitterHandleViewModel had a null EmitterHandle!")))
	{
		EmitterHandle->GetInstance().Emitter->GetMessageStore().RemoveMessage(MessageKey);
	}
}

FNiagaraEmitterHandleViewModel::~FNiagaraEmitterHandleViewModel()
{
	Cleanup();
}


void FNiagaraEmitterHandleViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InOwningSystemViewModel, int32 InEmitterHandleIndex, TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> InSimulation)
{
	OwningSystemViewModelWeak = InOwningSystemViewModel;
	EmitterHandleIndex = InEmitterHandleIndex;
	EmitterHandle = &InOwningSystemViewModel->GetSystem().GetEmitterHandle(InEmitterHandleIndex);
	FVersionedNiagaraEmitter Emitter = EmitterHandle != nullptr ? EmitterHandle->GetInstance() : FVersionedNiagaraEmitter();
	EmitterViewModel->Initialize(Emitter, InSimulation);
	EmitterViewModel->OnEmitterSelectionRequested().BindSP(this, &FNiagaraEmitterHandleViewModel::RequestEmitterSelection);
	EmitterStackViewModel->InitializeWithViewModels(InOwningSystemViewModel, this->AsShared(), FNiagaraStackViewModelOptions(false, true));
}

void FNiagaraEmitterHandleViewModel::Reset()
{
	EmitterStackViewModel->Reset();
	OwningSystemViewModelWeak.Reset();
	EmitterHandleIndex = INDEX_NONE;
	EmitterHandle = nullptr;
	EmitterViewModel->OnEmitterSelectionRequested().Unbind();
	EmitterViewModel->Reset();
}

void FNiagaraEmitterHandleViewModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (EmitterStackViewModel != nullptr)
	{
		Collector.AddReferencedObject(EmitterStackViewModel);
	}
}

void FNiagaraEmitterHandleViewModel::SetSimulation(TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> InSimulation)
{
	EmitterViewModel->SetSimulation(InSimulation);
}

FGuid FNiagaraEmitterHandleViewModel::GetId() const
{
	if (EmitterHandle)
	{
		return EmitterHandle->GetId();
	}
	return FGuid();
}

FText FNiagaraEmitterHandleViewModel::GetErrorText() const
{
	switch (EmitterViewModel->GetLatestCompileStatus())
	{
	case ENiagaraScriptCompileStatus::NCS_Unknown:
	case ENiagaraScriptCompileStatus::NCS_BeingCreated:
		return LOCTEXT("NiagaraEmitterHandleCompileStatusUnknown", "Needs compilation & refresh.");
	case ENiagaraScriptCompileStatus::NCS_UpToDate:
		return LOCTEXT("NiagaraEmitterHandleCompileStatusUpToDate", "Compiled");
	default:
		return LOCTEXT("NiagaraEmitterHandleCompileStatusError", "Error! Needs compilation & refresh.");
	}
}

FSlateColor FNiagaraEmitterHandleViewModel::GetErrorTextColor() const
{
	switch (EmitterViewModel->GetLatestCompileStatus())
	{
	case ENiagaraScriptCompileStatus::NCS_Unknown:
	case ENiagaraScriptCompileStatus::NCS_BeingCreated:
		return FSlateColor(FLinearColor::Yellow);
	case ENiagaraScriptCompileStatus::NCS_UpToDate:
		return FSlateColor(FLinearColor::Green);
	default:
		return FSlateColor(FLinearColor::Red);
	}
}

EVisibility FNiagaraEmitterHandleViewModel::GetErrorTextVisibility() const
{
	return EmitterViewModel->GetLatestCompileStatus() != ENiagaraScriptCompileStatus::NCS_UpToDate ? EVisibility::Visible : EVisibility::Collapsed;
}

FName FNiagaraEmitterHandleViewModel::GetName() const
{
	if (EmitterHandle)
	{
		return EmitterHandle->GetName();
	}
	return FName();
}

void FNiagaraEmitterHandleViewModel::SetName(FName InName)
{
	if (EmitterHandle && EmitterHandle->GetName().IsEqual(InName, ENameCase::CaseSensitive, false))
	{
		return;
	}

	if (EmitterHandle)
	{
		FScopedTransaction ScopedTransaction(NSLOCTEXT("NiagaraEmitterEditor", "EditEmitterNameTransaction", "Edit emitter name"));
		UNiagaraSystem& System = GetOwningSystemViewModel()->GetSystem();
		System.Modify();
		EmitterHandle->SetName(InName, System);
		OnPropertyChangedDelegate.Broadcast();
		OnNameChangedDelegate.Broadcast();
	}
}

FText FNiagaraEmitterHandleViewModel::GetNameText() const
{
	if (EmitterHandle)
	{
		return FText::FromName(EmitterHandle->GetName());
	}
	return FText();
}

bool FNiagaraEmitterHandleViewModel::CanRenameEmitter() const
{
	return GetOwningSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset;
}

void FNiagaraEmitterHandleViewModel::OnNameTextComitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	SetName(*InText.ToString());
}

bool FNiagaraEmitterHandleViewModel::VerifyNameTextChanged(const FText& NewText, FText& OutErrorMessage)
{
	static const FString ReservedNames[] =
	{
		TEXT("Particles"),
		TEXT("Local"),
		TEXT("Engine"),
		TEXT("Transient"),
		TEXT("User"),
		TEXT("Emitter"),
		TEXT("Module"),
		TEXT("NPC")
	};

	FString NewString = NewText.ToString();
	if (NewText.IsEmptyOrWhitespace())
	{
		OutErrorMessage = NSLOCTEXT("NiagaraEmitterEditor", "NiagaraInputNameEmptyWarn", "Cannot have empty name!");
		return false;
	}
	else if (Algo::Find(ReservedNames, NewString) != nullptr)
	{
		OutErrorMessage = FText::Format(NSLOCTEXT("NiagaraEmitterEditor", "NiagaraInputNameReservedWarn", "Cannot use reserved name \"{0}\"!"), NewText);
		return false;
	}
	return true;
}

bool FNiagaraEmitterHandleViewModel::GetIsEnabled() const
{
	if (EmitterHandle)
	{
		return EmitterHandle->GetIsEnabled();
	}
	return false;
}

bool FNiagaraEmitterHandleViewModel::SetIsEnabled(bool bInIsEnabled, bool bRequestRecompile)
{
	if (EmitterHandle && EmitterHandle->GetIsEnabled() != bInIsEnabled)
	{
		FScopedTransaction ScopedTransaction(NSLOCTEXT("NiagaraEmitterEditor", "EditEmitterEnabled", "Change emitter enabled state"));
		GetOwningSystemViewModel()->GetSystem().Modify();
		EmitterHandle->SetIsEnabled(bInIsEnabled, GetOwningSystemViewModel()->GetSystem(), bRequestRecompile);
		OnPropertyChangedDelegate.Broadcast();

		return true;
	}

	return false;
}

bool FNiagaraEmitterHandleViewModel::GetIsIsolated() const
{
	return EmitterHandle != nullptr && EmitterHandle->IsIsolated();
}

void FNiagaraEmitterHandleViewModel::SetIsIsolated(bool bInIsIsolated)
{
	bool bWasIsolated = GetIsIsolated();
	bool bStateChanged = bWasIsolated != bInIsIsolated;

	TArray<FGuid> EmitterIds;

	if(bStateChanged)
	{		
		if(bInIsIsolated)
		{
			EmitterIds.Add(EmitterHandle->GetId());
		}

		GetOwningSystemViewModel()->IsolateEmitters(EmitterIds);
	}
}

void FNiagaraEmitterHandleViewModel::ToggleIsIsolated()
{
	GetOwningSystemViewModel()->ToggleEmitterIsolation(AsShared());
}

void FNiagaraEmitterHandleViewModel::RequestCaptureThumbnail()
{
	GetOwningSystemViewModel()->OnEmitterThumbnailRequested().ExecuteIfBound(GetEmitterHandle()->GetId());
}

ENiagaraSystemViewModelEditMode FNiagaraEmitterHandleViewModel::GetOwningSystemEditMode() const
{
	return GetOwningSystemViewModel()->GetEditMode();
}

ECheckBoxState FNiagaraEmitterHandleViewModel::GetIsEnabledCheckState() const
{
	if (EmitterHandle)
	{
		return EmitterHandle->GetIsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void FNiagaraEmitterHandleViewModel::OnIsEnabledCheckStateChanged(ECheckBoxState InCheckState)
{
	SetIsEnabled(InCheckState == ECheckBoxState::Checked);
}

FNiagaraEmitterHandle* FNiagaraEmitterHandleViewModel::GetEmitterHandle()
{
	return EmitterHandle;
}

TSharedRef<FNiagaraEmitterViewModel> FNiagaraEmitterHandleViewModel::GetEmitterViewModel()
{
	return EmitterViewModel;
}

UNiagaraStackViewModel* FNiagaraEmitterHandleViewModel::GetEmitterStackViewModel()
{
	return EmitterStackViewModel;
}

bool FNiagaraEmitterHandleViewModel::GetIsRenamePending() const
{
	return bIsRenamePending;
}

void FNiagaraEmitterHandleViewModel::SetIsRenamePending(bool bInIsRenamePending)
{
	bIsRenamePending = bInIsRenamePending;
}

void FNiagaraEmitterHandleViewModel::RequestEmitterSelection(bool bClearSelection)
{
	GetOwningSystemViewModel()->GetSelectionViewModel()->UpdateSelectedEntries({GetEmitterStackViewModel()->GetRootEntry()}, {}, bClearSelection);
}

void FNiagaraEmitterHandleViewModel::BeginDebugEmitter()
{

#if WITH_NIAGARA_DEBUGGER
	if (EmitterHandle)
		SNiagaraDebugger::InvokeDebugger(*EmitterHandle);
#endif
}

FNiagaraEmitterHandleViewModel::FOnPropertyChanged& FNiagaraEmitterHandleViewModel::OnPropertyChanged()
{
	return OnPropertyChangedDelegate;
}

FNiagaraEmitterHandleViewModel::FOnNameChanged& FNiagaraEmitterHandleViewModel::OnNameChanged()
{
	return OnNameChangedDelegate;
}

#undef LOCTEXT_NAMESPACE
