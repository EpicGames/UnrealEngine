// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterInstance.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraScriptSourceBase.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "NiagaraScriptGraphViewModel.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraScriptSource.h"
#include "NiagaraEmitterEditorData.h"
#include "ViewModels/NiagaraParameterEditMode.h"
#include "NiagaraGraph.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstance.h"

#include "ScopedTransaction.h"
#include "IContentBrowserSingleton.h"
#include "Framework/Application/SlateApplication.h"
#include "ContentBrowserModule.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/SWindow.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"

#define LOCTEXT_NAMESPACE "EmitterEditorViewModel"

template<> TMap<UNiagaraEmitter*, TArray<FNiagaraEmitterViewModel*>> TNiagaraViewModelManager<UNiagaraEmitter, FNiagaraEmitterViewModel>::ObjectsToViewModels{};

const FText FNiagaraEmitterViewModel::StatsFormat = NSLOCTEXT("NiagaraEmitterViewModel", "StatsFormat", "{0} Particles | {1} ms | {2} MB | {3}");
const FText FNiagaraEmitterViewModel::StatsParticleCountFormat = NSLOCTEXT("NiagaraEmitterViewModel", "StatsParticleCountFormat", "{0} Particles");
const FText FNiagaraEmitterViewModel::ParticleDisabledDueToScalability = NSLOCTEXT("NiagaraEmitterViewModel", "ParticleDisabledDueToScalability", "Disabled due to scalability settings (Current Effects Quality: {0}). See Scalability options for valid range. ");

namespace NiagaraCommands
{
	static FAutoConsoleVariable EmitterStatsFormat(TEXT("Niagara.EmitterStatsFormat"), 1, TEXT("0 shows the particles count, ms, mb and state. 1 shows particles count."));
}

FNiagaraEmitterViewModel::FNiagaraEmitterViewModel(bool bInIsForDataProcessingOnly)
	: EmitterWeakPtr(FVersionedNiagaraEmitterWeakPtr(nullptr, FGuid()))
	, SharedScriptViewModel(MakeShareable(new FNiagaraScriptViewModel(LOCTEXT("SharedDisplayName", "Graph"), ENiagaraParameterEditMode::EditAll, bInIsForDataProcessingOnly)))
	, SummaryViewHierarchyViewModel(NewObject<UNiagaraSummaryViewViewModel>())
	, bUpdatingSelectionInternally(false)
	, ExecutionStateEnum(StaticEnum<ENiagaraExecutionState>())
{	
}

void FNiagaraEmitterViewModel::Cleanup()
{
	if (NewParentWindow.IsValid())
	{
		NewParentWindow.Reset();
	}

	if (EmitterWeakPtr.Emitter.IsValid())
	{
		EmitterWeakPtr.Emitter->OnEmitterVMCompiled().RemoveAll(this);
		EmitterWeakPtr.Emitter->OnPropertiesChanged().RemoveAll(this);
	}

	if (SharedScriptViewModel.IsValid())
	{
		SharedScriptViewModel->GetGraphViewModel()->GetNodeSelection()->OnSelectedObjectsChanged().RemoveAll(this);
		SharedScriptViewModel.Reset();
	}

	if(SummaryViewHierarchyViewModel != nullptr)
	{
		SummaryViewHierarchyViewModel->Finalize();
		SummaryViewHierarchyViewModel = nullptr;
	}

	if(EmitterWeakPtr.IsValid())
	{
		GetEditorData().OnPersistentDataChanged().RemoveAll(this);
	}

	RemoveScriptEventHandlers();
}

bool FNiagaraEmitterViewModel::Initialize(const FVersionedNiagaraEmitter& InEmitter, TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> InSimulation)
{
	SetEmitter(InEmitter);
	SetSimulation(InSimulation);
	SummaryViewHierarchyViewModel->Initialize(AsShared());
	SummaryViewHierarchyViewModel->GetOnHierarchyChangedDelegate().AddSP(this, &FNiagaraEmitterViewModel::OnSummaryViewHierarchyChanged);
	GetEditorData().OnSummaryViewStateChanged().AddSP(this, &FNiagaraEmitterViewModel::UpdateSelectionOnSummaryViewStateChanged);
	return true;
}

void FNiagaraEmitterViewModel::Reset()
{
	SetEmitter(FVersionedNiagaraEmitter());
	SummaryViewHierarchyViewModel->GetOnHierarchyChangedDelegate().RemoveAll(this);
	SetSimulation(nullptr);
}

INiagaraParameterDefinitionsSubscriber* FNiagaraEmitterViewModel::GetParameterDefinitionsSubscriber()
{
	return GetEmitter().Emitter;
}

FNiagaraEmitterViewModel::~FNiagaraEmitterViewModel()
{
	Cleanup();
	UnregisterViewModelWithMap(RegisteredHandle);

	//UE_LOGF(LogNiagaraEditor, Warning, "Deleting Emitter view model %p", this);
}

void FNiagaraEmitterViewModel::SetEmitter(FVersionedNiagaraEmitter InEmitter)
{
	if (EmitterWeakPtr.IsValid())
	{
		EmitterWeakPtr.Emitter->OnEmitterVMCompiled().RemoveAll(this);
		EmitterWeakPtr.Emitter->OnEmitterGPUCompiled().RemoveAll(this);
		EmitterWeakPtr.Emitter->OnPropertiesChanged().RemoveAll(this);
		UnregisterViewModelWithMap(RegisteredHandle);
	}

	UnregisterViewModelWithMap(RegisteredHandle);

	RemoveScriptEventHandlers();

	EmitterWeakPtr = InEmitter.ToWeakPtr();

	if (EmitterWeakPtr.IsValid())
	{
		EmitterWeakPtr.Emitter->OnEmitterVMCompiled().AddSP(this, &FNiagaraEmitterViewModel::OnVMCompiled);
		EmitterWeakPtr.Emitter->OnPropertiesChanged().AddSP(this, &FNiagaraEmitterViewModel::OnEmitterPropertiesChanged);
		EmitterWeakPtr.Emitter->OnEmitterGPUCompiled().AddSP(this, &FNiagaraEmitterViewModel::OnGPUCompiled);
	}

	AddScriptEventHandlers();

	RegisteredHandle = RegisterViewModelWithMap(InEmitter.Emitter, this);

	check(SharedScriptViewModel.IsValid());
	SharedScriptViewModel->SetScripts(InEmitter);

	OnEmitterChanged().Broadcast();
}

TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> FNiagaraEmitterViewModel::GetSimulation() const
{
	return Simulation;
}

void FNiagaraEmitterViewModel::SetSimulation(TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> InSimulation)
{
	Simulation = InSimulation;
}

FVersionedNiagaraEmitter FNiagaraEmitterViewModel::GetEmitter()
{
	return EmitterWeakPtr.ResolveWeakPtr();
}

bool FNiagaraEmitterViewModel::HasParentEmitter() const
{
	return EmitterWeakPtr.IsValid() && EmitterWeakPtr.GetEmitterData()->GetParent().Emitter != nullptr;
}

FVersionedNiagaraEmitter FNiagaraEmitterViewModel::GetParentEmitter() const
{
	FVersionedNiagaraEmitterData* EmitterData = EmitterWeakPtr.GetEmitterData();
	return EmitterData ? EmitterData->GetParent() : FVersionedNiagaraEmitter();
}

FText FNiagaraEmitterViewModel::GetParentNameText() const
{
	if (EmitterWeakPtr.IsValid() && EmitterWeakPtr.GetEmitterData()->GetParent().Emitter != nullptr)
	{
		return FText::FromString(EmitterWeakPtr.GetEmitterData()->GetParent().Emitter->GetName());
	}
	return FText();
}

FText FNiagaraEmitterViewModel::GetParentPathNameText() const
{
	if (EmitterWeakPtr.IsValid() && EmitterWeakPtr.GetEmitterData()->GetParent().Emitter != nullptr)
	{
		return FText::FromString(EmitterWeakPtr.GetEmitterData()->GetParent().Emitter->GetPathName());
	}
	return FText();
}

void FNiagaraEmitterViewModel::CreateNewParentWindow(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.SelectionMode = ESelectionMode::SingleToggle;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.Filter.ClassPaths.Add(UNiagaraEmitter::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetsActivated.BindSP(this, &FNiagaraEmitterViewModel::UpdateParentEmitter, EmitterHandleViewModel);

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	TSharedRef<SWidget> AssetPicker = ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);

	const FText TitleText = LOCTEXT("NewParentEmitter", "New Parent Emitter");
	// Create the window to pick the class
	SAssignNew(NewParentWindow, SWindow)
		.Title(TitleText)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(700.f, 300.f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);
	NewParentWindow->SetContent(AssetPicker);
	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
	if (RootWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(NewParentWindow.ToSharedRef(), RootWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(NewParentWindow.ToSharedRef());
	}
}

void FNiagaraEmitterViewModel::UpdateParentEmitter(const TArray<FAssetData> & ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod, TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("RemoveParentEmitterTransaction", "Remove Parent Emitter"));
	if ((ActivationMethod == EAssetTypeActivationMethod::DoubleClicked || ActivationMethod == EAssetTypeActivationMethod::Opened) && ActivatedAssets.Num() == 1)
	{
		if (EmitterWeakPtr.IsValid())
		{
			FAssetData NewParentData = ActivatedAssets[0];
			UNiagaraEmitter* NewParentPtr = Cast<UNiagaraEmitter>(NewParentData.GetAsset());
			UNiagaraEmitter* Emitter = EmitterWeakPtr.Emitter.Get();
			if (NewParentPtr && NewParentPtr != Emitter)
			{
				Emitter->PreEditChange(nullptr);
				EmitterWeakPtr.GetEmitterData()->Reparent(FVersionedNiagaraEmitter(NewParentPtr, NewParentPtr->GetExposedVersion().VersionGuid));
				Emitter->MergeChangesFromParent();
				Emitter->PostEditChange();
				EmitterHandleViewModel->GetOwningSystemViewModel()->ResetStack();
				NewParentWindow->RequestDestroyWindow();
			}
		}
	}
}

void FNiagaraEmitterViewModel::RemoveParentEmitter()
{
	FScopedTransaction ScopedTransaction(LOCTEXT("RemoveParentEmitterTransaction", "Remove Parent Emitter"));
	EmitterWeakPtr.Emitter->Modify();
	EmitterWeakPtr.GetEmitterData()->RemoveParent();
	OnParentRemovedDelegate.Broadcast();
}

FText FNiagaraEmitterViewModel::GetStatsText() const
{
	if (Simulation.IsValid())
	{
		FNiagaraEmitterInstancePtr SimInstance = Simulation.Pin();
		if (SimInstance.IsValid())
		{
			static const FNumberFormattingOptions FractionalFormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(3)
				.SetMaximumFractionalDigits(3);

			FNiagaraEmitterInstanceImpl* StatefulSimInstance = SimInstance->AsStateful();
			const bool bSystemCompiling = SimInstance->GetParentSystemInstance()->GetSystem()->HasOutstandingCompilationRequests();
			if (bSystemCompiling || (StatefulSimInstance && !StatefulSimInstance->IsReadyToRun()))
			{
				return LOCTEXT("PendingCompile", "Compilation in progress...");
			}

			const FNiagaraEmitterHandle& Handle = SimInstance->GetEmitterHandle();
			if (FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData())
			{
				if (Handle.IsValid() == false)
				{
					return LOCTEXT("InvalidHandle", "Invalid handle");
				}
				
				if (Handle.GetIsEnabled() == false)
				{
					return LOCTEXT("DisabledSimulation", "Emitter is not enabled. Excluded from code.");
				}

				if (!Handle.GetEmitterData()->IsValid())
				{
					return LOCTEXT("InvalidInstance", "Invalid Emitter! May have compile errors.");
				}

 				int32 QualityLevel = FNiagaraPlatformSet::GetQualityLevel();
  				if (!EmitterData->IsAllowedByScalability())
  				{
					return FText::Format(ParticleDisabledDueToScalability, FNiagaraPlatformSet::GetQualityLevelText(QualityLevel));
  				}

				if (NiagaraCommands::EmitterStatsFormat->GetInt() == 1)
				{
					return FText::Format(StatsParticleCountFormat, FText::AsNumber(SimInstance->GetNumParticles()));
				}
				else
				{
					constexpr double Megabyte = 1024 * 1024;
					const float TickTimeMS = FPlatformTime::ToMilliseconds(SimInstance->GetTickTimeCycles());
					return FText::Format(StatsFormat,
						FText::AsNumber(SimInstance->GetNumParticles()),
						FText::AsNumber(TickTimeMS, &FractionalFormatOptions),
						FText::AsNumber(static_cast<double>(SimInstance->GetTotalBytesUsed()) / Megabyte, &FractionalFormatOptions),
						ExecutionStateEnum->GetDisplayNameTextByValue(static_cast<int32>(SimInstance->GetExecutionState())));
				}
			}
		}
	}
	else if(!EmitterWeakPtr.GetEmitterData()->IsReadyToRun())
	{
		return LOCTEXT("SimulationNotReady", "Preparing simulation...");
	}
	
	return LOCTEXT("NoActivePreview", "No running preview...");
}

TSharedRef<FNiagaraScriptViewModel> FNiagaraEmitterViewModel::GetSharedScriptViewModel()
{
	return SharedScriptViewModel.ToSharedRef();
}

UNiagaraSummaryViewViewModel* FNiagaraEmitterViewModel::GetSummaryHierarchyViewModel()
{
	return SummaryViewHierarchyViewModel;
}

const UNiagaraEmitterEditorData& FNiagaraEmitterViewModel::GetEditorData() const
{
	if (EmitterWeakPtr.Emitter.IsValid())
	{
		return *CastChecked<UNiagaraEmitterEditorData>(EmitterWeakPtr.GetEmitterData()->GetEditorData());
	}
	//-TODO:Stateless: We don't have UNiagaraEmitterEditorData
	return *GetDefault<UNiagaraEmitterEditorData>();
}

UNiagaraEmitterEditorData& FNiagaraEmitterViewModel::GetEditorData()
{
	if (EmitterWeakPtr.IsValid())
	{
		return *CastChecked<UNiagaraEmitterEditorData>(EmitterWeakPtr.GetEmitterData()->GetEditorData());
	}
	//-TODO:Stateless: We don't have UNiagaraEmitterEditorData
	return *GetMutableDefault<UNiagaraEmitterEditorData>();
}

void FNiagaraEmitterViewModel::GetEmitterMessageStores(TArray<FNiagaraMessageSourceAndStore>& OutMessageStores)
{
	if (EmitterWeakPtr.IsValid())
	{
		OutMessageStores.Add(FNiagaraMessageSourceAndStore(*EmitterWeakPtr.Emitter, EmitterWeakPtr.Emitter->GetMessageStore()));
		TArray<UNiagaraScript*> Scripts;
		EmitterWeakPtr.GetEmitterData()->GetScripts(Scripts, false);
		for (UNiagaraScript* Script : Scripts)
		{
			FNiagaraEditorUtilities::GetScriptMessageStores(Script, OutMessageStores);
		}
	}
}

void FNiagaraEmitterViewModel::AddEventHandler(FNiagaraEventScriptProperties& EventScriptProperties, bool bResetGraphForOutput /*= false*/)
{
	EventScriptProperties.Script = NewObject<UNiagaraScript>(GetEmitter().Emitter, MakeUniqueObjectName(GetEmitter().Emitter, UNiagaraScript::StaticClass(), "EventScript"), EObjectFlags::RF_Transactional);
	EventScriptProperties.Script->SetUsage(ENiagaraScriptUsage::ParticleEventScript);
	EventScriptProperties.Script->SetUsageId(FGuid::NewGuid());
	EventScriptProperties.Script->SetLatestSource(GetSharedScriptViewModel()->GetGraphViewModel()->GetScriptSource());
	EmitterWeakPtr.Emitter->AddEventHandler(EventScriptProperties, EmitterWeakPtr.Version);
	if (bResetGraphForOutput)
	{
		FNiagaraStackGraphUtilities::ResetGraphForOutput(*GetSharedScriptViewModel()->GetGraphViewModel()->GetGraph(), ENiagaraScriptUsage::ParticleEventScript, EventScriptProperties.Script->GetUsageId());
	}
}

void FNiagaraEmitterViewModel::OnGPUCompiled(FVersionedNiagaraEmitter InEmitter)
{
	// Flush compilation logs just like the CPU script compiled.
	OnVMCompiled(InEmitter);
}

void FNiagaraEmitterViewModel::OnVMCompiled(FVersionedNiagaraEmitter InEmitter)
{
	FVersionedNiagaraEmitter Emitter = EmitterWeakPtr.ResolveWeakPtr();
	if (InEmitter == Emitter)
	{
		TArray<ENiagaraScriptCompileStatus> CompileStatuses;
		TArray<FString> CompileErrors;
		TArray<FString> CompilePaths;
		TArray<TPair<ENiagaraScriptUsage, int32> > Usages;

		ENiagaraScriptCompileStatus AggregateStatus = ENiagaraScriptCompileStatus::NCS_UpToDate;
		FString AggregateErrors;

		TArray<UNiagaraScript*> Scripts;
		FVersionedNiagaraEmitterData* EmitterData = Emitter.GetEmitterData();
		EmitterData->GetScripts(Scripts, true);

		int32 EventsFound = 0;
		for (int32 i = 0; i < Scripts.Num(); i++)
		{
			UNiagaraScript* Script = Scripts[i];
			if (Script != nullptr && Script->GetVMExecutableData().IsValid())
			{
				CompileStatuses.Add(Script->GetVMExecutableData().LastCompileStatus);
				CompileErrors.Add(Script->GetVMExecutableData().ErrorMsg);
				CompilePaths.Add(Script->GetPathName());

				if (Script->GetUsage() == ENiagaraScriptUsage::ParticleEventScript)
				{
					Usages.Add(TPair<ENiagaraScriptUsage, int32>(Script->GetUsage(), EventsFound));
					EventsFound++;
				}
				else
				{
					Usages.Add(TPair<ENiagaraScriptUsage, int32>(Script->GetUsage(), 0));
				}
			}
			else
			{
				CompileStatuses.Add(ENiagaraScriptCompileStatus::NCS_Unknown);
				CompileErrors.Add(TEXT("Invalid script pointer!"));
				CompilePaths.Add(TEXT("Unknown..."));
				Usages.Add(TPair<ENiagaraScriptUsage, int32>(ENiagaraScriptUsage::Function, 0));
			}

		}

		// Now handle any GPU sims..
		TArray<UNiagaraScript*> GPUScripts;
		
		// Going ahead and making an array for future proofing..
		{
			UNiagaraScript* GPUScript = EmitterData->GetGPUComputeScript();
			GPUScripts.Add(GPUScript);
		}

		// Push in GPU script errors into the other scripts
		for (UNiagaraScript* GPUScript : GPUScripts)
		{
			if (GPUScript)
			{
				const FNiagaraShaderScript* ShaderScript = GPUScript->GetRenderThreadScript();
				if (!ShaderScript)
					return;

				for (int32 i = 0; i < Scripts.Num(); i++)
				{
					if (CompileStatuses.Num() > i && CompileErrors.Num() > i && Scripts[i] != nullptr && GPUScript->ContainsUsage(Scripts[i]->Usage))
					{
						const TArray<FString>& NewErrors = ShaderScript->GetCompileErrors();
						bool bErrors = false;
						for (const FString& ErrorStr : NewErrors)
						{
							if (ErrorStr.Contains(TEXT("err0r")))
							{
								CompileErrors[i] += ErrorStr + TEXT("\n");
								bErrors = true;
							}
						}

						if (bErrors)
						{
							CompileStatuses[i] = ENiagaraScriptCompileStatus::NCS_Error;
						}
					}
				}

			}
		}

		for (int32 i = 0; i < CompileStatuses.Num(); i++)
		{
			AggregateStatus = FNiagaraEditorUtilities::UnionCompileStatus(AggregateStatus, CompileStatuses[i]);
			AggregateErrors += CompilePaths[i] + TEXT(" ") + FNiagaraEditorUtilities::StatusToText(CompileStatuses[i]).ToString() + TEXT("\n");
			AggregateErrors += CompileErrors[i] + TEXT("\n");
		}
		
		if (SharedScriptViewModel.IsValid())
		{
			SharedScriptViewModel->UpdateCompileStatus(AggregateStatus, AggregateErrors, CompileStatuses, CompileErrors, CompilePaths, Scripts);
		}
	}
	OnScriptCompiled().Broadcast(nullptr, FGuid());
}

ENiagaraScriptCompileStatus FNiagaraEmitterViewModel::GetLatestCompileStatus()
{
	check(SharedScriptViewModel.IsValid());
	ENiagaraScriptCompileStatus UnionStatus = SharedScriptViewModel->GetLatestCompileStatus();

	FVersionedNiagaraEmitterData* EmitterData = EmitterWeakPtr.GetEmitterData();
	if (EmitterData && EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
	{
		if (UnionStatus != ENiagaraScriptCompileStatus::NCS_Dirty)
		{
			if (!EmitterData->GetGPUComputeScript()->AreScriptAndSourceSynchronized())
			{
				UnionStatus = ENiagaraScriptCompileStatus::NCS_Dirty;
			}
		}
	}
	return UnionStatus;
}


FNiagaraEmitterViewModel::FOnEmitterChanged& FNiagaraEmitterViewModel::OnEmitterChanged()
{
	return OnEmitterChangedDelegate;
}

FNiagaraEmitterViewModel::FOnPropertyChanged& FNiagaraEmitterViewModel::OnPropertyChanged()
{
	return OnPropertyChangedDelegate;
}

FNiagaraEmitterViewModel::FOnScriptCompiled& FNiagaraEmitterViewModel::OnScriptCompiled()
{
	return OnScriptCompiledDelegate;
}

FNiagaraEmitterViewModel::FOnParentRemoved& FNiagaraEmitterViewModel::OnParentRemoved()
{
	return OnParentRemovedDelegate;
}

FNiagaraEmitterViewModel::FOnScriptGraphChanged& FNiagaraEmitterViewModel::OnScriptGraphChanged()
{
	return OnScriptGraphChangedDelegate;
}

FNiagaraEmitterViewModel::FOnScriptParameterStoreChanged& FNiagaraEmitterViewModel::OnScriptParameterStoreChanged()
{
	return OnScriptParameterStoreChangedDelegate;
}

FNiagaraEmitterViewModel::FOnEmitterSelectionRequested& FNiagaraEmitterViewModel::OnEmitterSelectionRequested()
{
	return OnEmitterSelectionRequestedDelegate;
}

void FNiagaraEmitterViewModel::AddScriptEventHandlers()
{
	if (FVersionedNiagaraEmitterData* EmitterData = EmitterWeakPtr.GetEmitterData())
	{
		TArray<UNiagaraScript*> Scripts;
		EmitterData->GetScripts(Scripts, false);
		for (UNiagaraScript* Script : Scripts)
		{
			UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(Script->GetLatestSource());
			FDelegateHandle OnGraphChangedHandle = ScriptSource->NodeGraph->AddOnGraphChangedHandler(
				FOnGraphChanged::FDelegate::CreateThreadSafeSP(this->AsShared(), &FNiagaraEmitterViewModel::ScriptGraphChanged, (const UNiagaraScript*)Script));
			FDelegateHandle OnGraphNeedRecompileHandle = ScriptSource->NodeGraph->AddOnGraphNeedsRecompileHandler(
				FOnGraphChanged::FDelegate::CreateThreadSafeSP(this->AsShared(), &FNiagaraEmitterViewModel::ScriptGraphChanged, (const UNiagaraScript*)Script));

			ScriptToOnGraphChangedHandleMap.Add(FObjectKey(Script), OnGraphChangedHandle);
			ScriptToRecompileHandleMap.Add(FObjectKey(Script), OnGraphNeedRecompileHandle);

			FDelegateHandle OnParameterStoreChangedHandle = Script->RapidIterationParameters.AddOnChangedHandler(
				FNiagaraParameterStore::FOnChanged::FDelegate::CreateThreadSafeSP(
					this->AsShared(), &FNiagaraEmitterViewModel::ScriptParameterStoreChanged, (const FNiagaraParameterStore*)&Script->RapidIterationParameters, (const UNiagaraScript*)Script));
			ScriptToOnParameterStoreChangedHandleMap.Add(FObjectKey(Script), OnParameterStoreChangedHandle);
		}
	}
}

void FNiagaraEmitterViewModel::RemoveScriptEventHandlers()
{
	if (FVersionedNiagaraEmitterData* EmitterData = EmitterWeakPtr.GetEmitterData())
	{
		TArray<UNiagaraScript*> Scripts;
		EmitterData->GetScripts(Scripts, false);
		for (UNiagaraScript* Script : Scripts)
		{
			FDelegateHandle* OnGraphChangedHandle = ScriptToOnGraphChangedHandleMap.Find(FObjectKey(Script));
			if (OnGraphChangedHandle != nullptr)
			{
				UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(Script->GetLatestSource());
				ScriptSource->NodeGraph->RemoveOnGraphChangedHandler(*OnGraphChangedHandle);
			}

			FDelegateHandle* OnGraphRecompileHandle = ScriptToRecompileHandleMap.Find(FObjectKey(Script));
			if (OnGraphRecompileHandle != nullptr)
			{
				UNiagaraScriptSource* ScriptSource = CastChecked<UNiagaraScriptSource>(Script->GetLatestSource());
				ScriptSource->NodeGraph->RemoveOnGraphNeedsRecompileHandler(*OnGraphRecompileHandle);
			}

			FDelegateHandle* OnParameterStoreChangedHandle = ScriptToOnParameterStoreChangedHandleMap.Find(FObjectKey(Script));
			if (OnParameterStoreChangedHandle != nullptr)
			{
				Script->RapidIterationParameters.RemoveOnChangedHandler(*OnParameterStoreChangedHandle);
			}
		}
	}

	ScriptToOnGraphChangedHandleMap.Empty();
	ScriptToRecompileHandleMap.Empty();
	ScriptToOnParameterStoreChangedHandleMap.Empty();
}

void FNiagaraEmitterViewModel::ScriptGraphChanged(const FEdGraphEditAction& InAction, const UNiagaraScript* OwningScript)
{
	OnScriptGraphChangedDelegate.Broadcast(InAction, *OwningScript);
}

void FNiagaraEmitterViewModel::ScriptParameterStoreChanged(const FNiagaraParameterStore* ChangedParameterStore, const UNiagaraScript* OwningScript)
{
	OnScriptParameterStoreChangedDelegate.Broadcast(*ChangedParameterStore, *OwningScript);
}

void FNiagaraEmitterViewModel::OnEmitterPropertiesChanged()
{
	// Check that these are valid since post edit changed is called on objects even when they've been deleted as a result of undo/redo.
	if (SharedScriptViewModel.IsValid() && EmitterWeakPtr.Emitter.IsValid())
	{
		// When the properties change we reset the scripts on the script view model because gpu/cpu or interpolation may have changed.
		SharedScriptViewModel->SetScripts(EmitterWeakPtr.ResolveWeakPtr());
		OnPropertyChangedDelegate.Broadcast();
	}
}

void FNiagaraEmitterViewModel::OnSummaryViewHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload)
{
	GetEditorData().OnPersistentDataChanged().Broadcast();
}

void FNiagaraEmitterViewModel::UpdateSelectionOnSummaryViewStateChanged() const
{
	if(GetEditorData().ShouldShowSummaryView())
	{
		OnEmitterSelectionRequestedDelegate.ExecuteIfBound(false);
	}
}

FString FNiagaraEmitterViewModel::GetReferencerName() const
{
	return TEXT("EmitterViewModel");
}

void FNiagaraEmitterViewModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	if(SummaryViewHierarchyViewModel != nullptr)
	{
		Collector.AddReferencedObject(SummaryViewHierarchyViewModel);
	}
}

#undef LOCTEXT_NAMESPACE
