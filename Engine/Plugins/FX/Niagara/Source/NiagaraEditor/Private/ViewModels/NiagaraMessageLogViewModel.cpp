// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMessageLogViewModel.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "NiagaraMessages.h"
#include "NiagaraScriptSourceBase.h"

FNiagaraMessageLogViewModel::FNiagaraMessageLogViewModel(const FName& InMessageLogName, const FGuid& InMessageLogGuidKey, TSharedPtr<class SWidget>& OutMessageLogWidget)
	: MessageLogGuidKey(InMessageLogGuidKey)
{
	FNiagaraMessageManager* MessageManager = FNiagaraMessageManager::Get();
	TArray<FName> MessageTopicsToSubscribe;
	MessageTopicsToSubscribe.Add(FNiagaraMessageTopics::CompilerTopicName);
	MessageTopicsToSubscribe.Append(MessageManager->GetAdditionalMessageLogTopics());
	
	FNiagaraMessageManager::Get()->SubscribeToAssetMessagesByTopic(
		  FText::FromString("MessageLogViewModel")
		, InMessageLogGuidKey
		, MessageTopicsToSubscribe
		, MessageManagerRegistrationKey
	).BindRaw(this, &FNiagaraMessageLogViewModel::RefreshMessageLog);

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	// Reuse any existing log, or create a new one (that is not held onto by the message log system)
	auto CreateMessageLogListing = [&MessageLogModule](const FName& LogName)->TSharedRef<IMessageLogListing> {
		FMessageLogInitializationOptions LogOptions;
		// Show Pages so that user is never allowed to clear log messages
		LogOptions.bShowPages = false;
		LogOptions.bShowFilters = false;
		LogOptions.bAllowClear = false;
		LogOptions.MaxPageCount = 1;

		if (MessageLogModule.IsRegisteredLogListing(LogName))
		{
			return MessageLogModule.GetLogListing(LogName);
		}
		else
		{
			return  MessageLogModule.CreateLogListing(LogName, LogOptions);
		}
	};

	MessageLogListing = CreateMessageLogListing(InMessageLogName);
	OutMessageLogWidget = MessageLogModule.CreateLogListingWidget(MessageLogListing.ToSharedRef());
}

FNiagaraMessageLogViewModel::~FNiagaraMessageLogViewModel()
{
	FNiagaraMessageManager* MessageManager = FNiagaraMessageManager::Get();
	MessageManager->Unsubscribe(FText::FromString("MessageLogViewModel"), MessageLogGuidKey, MessageManagerRegistrationKey);
	MessageManager->ClearAssetMessages(MessageLogGuidKey);
}

void FNiagaraMessageLogViewModel::RefreshMessageLog(const TArray<TSharedRef<const INiagaraMessage>>& InNewMessages)
{
	MessageLogListing->ClearMessages();
	TArray<TSharedRef<FTokenizedMessage>> NewTokenizedMessages;
	for (const TSharedRef<const INiagaraMessage>& NewMessage : InNewMessages)
	{
		NewTokenizedMessages.Add(NewMessage->GenerateTokenizedMessage());
	}
	MessageLogListing->AddMessages(NewTokenizedMessages);
}

void FNiagaraMessageLogViewModel::SetMessageLogGuidKey(const FGuid& InViewedAssetObjectKey)
{
	MessageLogGuidKey = InViewedAssetObjectKey;
}
