// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageLogListing.h"
#include "Logging/TokenizedMessage.h"
#include "NiagaraMessageManager.h"
#include "Templates/SharedPointer.h"

class FNiagaraMessageLogViewModel : public TSharedFromThis<FNiagaraMessageLogViewModel>
{
public:
	FNiagaraMessageLogViewModel(const FName& InMessageLogName, const FGuid& InMessageLogGuidKey, TSharedPtr<class SWidget>& OutMessageLogWidget );

	~FNiagaraMessageLogViewModel();

	void RefreshMessageLog(const TArray<TSharedRef<const INiagaraMessage>>& InNewMessages);

	void SetMessageLogGuidKey(const FGuid& InMessageLogGuidKey);

private:
	TSharedPtr<IMessageLogListing> MessageLogListing;
	FGuid MessageLogGuidKey;
	FGuid MessageManagerRegistrationKey;
};
