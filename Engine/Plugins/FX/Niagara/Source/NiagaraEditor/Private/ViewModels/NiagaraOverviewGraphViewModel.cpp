// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "Framework/Commands/GenericCommands.h"
#include "ScopedTransaction.h"
#include "EdGraphUtilities.h"
#include "HAL/PlatformApplicationMisc.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraOverviewNode.h"
#include "NiagaraObjectSelection.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "NiagaraSystem.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "NiagaraStackEditorData.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"

#define LOCTEXT_NAMESPACE "NiagaraOverviewGraphViewModel"

FNiagaraOverviewGraphViewModel::FNiagaraOverviewGraphViewModel()
	: Commands(MakeShareable(new FUICommandList()))
	, NodeSelection(MakeShareable(new FNiagaraObjectSelection()))
	, bUpdatingSystemSelectionFromGraph(false)
	, bUpdatingGraphSelectionFromSystem(false)
{
}

FNiagaraOverviewGraphViewModel::~FNiagaraOverviewGraphViewModel()
{
	if(bIsForDataProcessingOnly == false)
	{
		GEditor->UnregisterForUndo(this);
	}

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModelPinned = SystemViewModel.Pin();
	if (SystemViewModelPinned.IsValid())
	{
		if (SystemViewModelPinned->GetSelectionViewModel() != nullptr)
		{
			SystemViewModelPinned->GetSelectionViewModel()->OnEntrySelectionChanged().RemoveAll(this);
		}
	}
}

void FNiagaraOverviewGraphViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModel = InSystemViewModel;
	OverviewGraph = InSystemViewModel->GetEditorData().GetSystemOverviewGraph();
	bIsForDataProcessingOnly = InSystemViewModel->GetIsForDataProcessingOnly();

	if(bIsForDataProcessingOnly == false)
	{
		SetupCommands();
		GEditor->RegisterForUndo(this);
	}

	NodeSelection->OnSelectedObjectsChanged().AddSP(this, &FNiagaraOverviewGraphViewModel::GraphSelectionChanged);
	InSystemViewModel->GetSelectionViewModel()->OnEntrySelectionChanged().AddSP(this, &FNiagaraOverviewGraphViewModel::SystemSelectionChanged);

	UpdateOverviewGraphNodes();
}

FText FNiagaraOverviewGraphViewModel::GetDisplayName() const
{
	if (DisplayNameCache.IsSet() == false)
	{
		DisplayNameCache = GetDisplayNameInternal();
	}
	return DisplayNameCache.GetValue();
}

UEdGraph* FNiagaraOverviewGraphViewModel::GetGraph() const
{
	if (SystemViewModel.IsValid())
	{
		return SystemViewModel.Pin()->GetEditorData().GetSystemOverviewGraph();
	}
	return nullptr;
}

TSharedRef<FUICommandList> FNiagaraOverviewGraphViewModel::GetCommands()
{
	return Commands;
}

TSharedRef<FNiagaraObjectSelection> FNiagaraOverviewGraphViewModel::GetNodeSelection()
{
	return NodeSelection;
}

const FNiagaraGraphViewSettings& FNiagaraOverviewGraphViewModel::GetViewSettings() const
{
	return GetSystemViewModel()->GetEditorData().GetSystemOverviewGraphViewSettings();
}

void FNiagaraOverviewGraphViewModel::SetViewSettings(const FNiagaraGraphViewSettings& InOverviewGraphViewSettings)
{
	GetSystemViewModel()->GetEditorData().SetSystemOverviewGraphViewSettings(InOverviewGraphViewSettings);
}

FNiagaraOverviewGraphViewModel::FOnNodesPasted& FNiagaraOverviewGraphViewModel::OnNodesPasted()
{
	return OnNodesPastedDelegate;
}

TSharedRef<FNiagaraSystemViewModel> FNiagaraOverviewGraphViewModel::GetSystemViewModel()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModelPinned = SystemViewModel.Pin();
	checkf(SystemViewModelPinned.IsValid(), TEXT("System view model destroyed before overview graph view model."));
	return SystemViewModelPinned.ToSharedRef();
}

const TSharedRef<FNiagaraSystemViewModel> FNiagaraOverviewGraphViewModel::GetSystemViewModel() const
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModelPinned = SystemViewModel.Pin();
	checkf(SystemViewModelPinned.IsValid(), TEXT("System view model destroyed before overview graph view model."));
	return SystemViewModelPinned.ToSharedRef();
}

void FNiagaraOverviewGraphViewModel::UpdateOverviewGraphNodes()
{
	for(UEdGraphNode* Node : OverviewGraph->Nodes)
	{
		if(UNiagaraOverviewNode* OverviewNode = Cast<UNiagaraOverviewNode>(Node))
		{
			OverviewNode->UpdateStatus();
		}
	}
}

void FNiagaraOverviewGraphViewModel::SetupCommands()
{
	Commands->MapAction(
		FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::SelectAllNodes));

	Commands->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::DeleteSelectedNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::CanDeleteNodes));

	Commands->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::CopySelectedNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::CanCopyNodes));

	Commands->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::CutSelectedNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::CanCutNodes));

	Commands->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::PasteNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::CanPasteNodes));

	Commands->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::DuplicateNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::CanDuplicateNodes));

	Commands->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::RenameNode),
		FCanExecuteAction::CreateRaw(this, &FNiagaraOverviewGraphViewModel::CanRenameNode));
}

void FNiagaraOverviewGraphViewModel::SelectAllNodes()
{
	UEdGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		TArray<UObject*> AllNodes;
		Graph->GetNodesOfClass<UObject>(AllNodes);
		TSet<UObject*> AllNodeSet;
		AllNodeSet.Append(AllNodes);
		NodeSelection->SetSelectedObjects(AllNodeSet);
	}
}

void FNiagaraOverviewGraphViewModel::DeleteSelectedNodes()
{
	UEdGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		const FScopedTransaction Transaction(FGenericCommands::Get().Delete->GetDescription());
		Graph->Modify();

		TSet<UObject*> ObjectsToDelete = NodeSelection->GetSelectedObjectsResolved();
		NodeSelection->ClearSelectedObjects();

		TSet<FGuid> EmitterGuidsToDelete;
		for (UObject* ObjectToDelete : ObjectsToDelete)
		{
			UEdGraphNode* NodeToDelete = Cast<UEdGraphNode>(ObjectToDelete);
			if (NodeToDelete != nullptr && NodeToDelete->CanUserDeleteNode())
			{
				NodeToDelete->Modify();
				UNiagaraOverviewNode* OverviewNodeToDelete = Cast<UNiagaraOverviewNode>(NodeToDelete);
				if (OverviewNodeToDelete != nullptr)
				{
					// For emitter nodes, just collect their ids and let the system view model delete them.  It will update the overview graph.
					EmitterGuidsToDelete.Add(OverviewNodeToDelete->GetEmitterHandleGuid());
				}
				else
				{
					// Other types of nodes can be deleted directly.
					NodeToDelete->DestroyNode();
				}
			}
		}
		//we have checked SystemViewModel is valid as this is a requisite for Graph to not be null.
		SystemViewModel.Pin()->DeleteEmitters(EmitterGuidsToDelete);
	}
}

bool FNiagaraOverviewGraphViewModel::CanDeleteNodes() const
{
	UEdGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		for (UObject* SelectedNode : NodeSelection->GetSelectedObjectsResolved())
		{
			UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode);
			if (SelectedGraphNode != nullptr && SelectedGraphNode->CanUserDeleteNode())
			{
				return true;
			}
		}
	}
	return false;
}

void FNiagaraOverviewGraphViewModel::CutSelectedNodes()
{
	// Collect nodes which can not be delete or duplicated so they can be reselected.
	TSet<UObject*> CanBeDuplicatedAndDeleted;
	TSet<UObject*> CanNotBeDuplicatedAndDeleted;
	for (UObject* SelectedNode : NodeSelection->GetSelectedObjectsResolved())
	{
		UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode);
		if (SelectedGraphNode != nullptr)
		{
			if (SelectedGraphNode->CanDuplicateNode() && SelectedGraphNode->CanUserDeleteNode())
			{
				CanBeDuplicatedAndDeleted.Add(SelectedNode);
			}
			else
			{
				CanNotBeDuplicatedAndDeleted.Add(SelectedNode);
			}
		}
	}

	// Select the nodes which can be copied and deleted, copy and delete them, and then restore the ones which couldn't be copied or deleted.
	NodeSelection->SetSelectedObjects(CanBeDuplicatedAndDeleted);
	CopySelectedNodes();
	DeleteSelectedNodes();
	NodeSelection->SetSelectedObjects(CanNotBeDuplicatedAndDeleted);
}

bool FNiagaraOverviewGraphViewModel::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void FNiagaraOverviewGraphViewModel::CopySelectedNodes()
{
	TSet<UObject*> NodesToCopy;
	for (UObject* SelectedNode : NodeSelection->GetSelectedObjectsResolved())
	{
		UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode);
		if (SelectedGraphNode != nullptr)
		{
			if (SelectedGraphNode->CanDuplicateNode())
			{
				SelectedGraphNode->PrepareForCopying();
				NodesToCopy.Add(SelectedNode);
			}
		}
	}

	FString ExportedText;
	FEdGraphUtilities::ExportNodesToText(NodesToCopy, ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

bool FNiagaraOverviewGraphViewModel::CanCopyNodes() const
{
	UEdGraph* Graph = GetGraph();

	// Copying from emitters is unsupported. An emitter overview node only references an emitter handle ID and a transient system.
	// Since the system is transient, the emitter handle isn't always valid for duplication. Moreover the duplication would circumvent emitter inheritance. 
	// We don't store enough information on the node to find the emitter asset in question, so adding the emitter with the correct hierarchy would require a larger refactor to properly support.
	if (GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{		
		return false;
	}

	if (Graph != nullptr)
	{
		for (UObject* SelectedNode : NodeSelection->GetSelectedObjectsResolved())
		{
			UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode);
			if (SelectedGraphNode != nullptr && SelectedGraphNode->CanDuplicateNode())
			{
				return true;
			}
		}
	}
	return false;
}


void FNiagaraOverviewGraphViewModel::PasteNodes()
{
	UEdGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		const FScopedTransaction Transaction(FGenericCommands::Get().Paste->GetDescription());;
		Graph->Modify();

		NodeSelection->ClearSelectedObjects();

		// Grab the text to paste from the clipboard.
		FString TextToImport;
		FPlatformApplicationMisc::ClipboardPaste(TextToImport);

		// Import the nodes
		TSet<UEdGraphNode*> PastedNodes;
		FEdGraphUtilities::ImportNodesFromText(Graph, TextToImport, PastedNodes);

		TArray<FNiagaraSystemViewModel::FEmitterHandleToDuplicate> EmitterHandlesToDuplicate;
 		for (UEdGraphNode* PastedNode : PastedNodes)
 		{
			PastedNode->CreateNewGuid();
			UNiagaraOverviewNode* OverviewNode = Cast<UNiagaraOverviewNode>(PastedNode);
			
			if(OverviewNode != nullptr)
			{
				if (OverviewNode->GetOwningSystem() == nullptr)
				{
					// Nodes pasted from emitters have no owning system, and will be invalid, so they are destroyed here instead.
					FNiagaraEditorUtilities::InfoWithToastAndLog(LOCTEXT("PasteFromEmitterAsset", "Cannot paste emitters from emitter assets. Please use Add Emitter from the right click menu instead."));
					OverviewNode->DestroyNode();
				}
				else if	(OverviewNode->GetEmitterHandleGuid().IsValid())
				{
					FNiagaraSystemViewModel::FEmitterHandleToDuplicate EmitterHandleToDuplicate;
					EmitterHandleToDuplicate.SystemPath = OverviewNode->GetOwningSystem()->GetPathName();
					EmitterHandleToDuplicate.EmitterHandleId = OverviewNode->GetEmitterHandleGuid();
					EmitterHandleToDuplicate.OverviewNode = OverviewNode;
					EmitterHandlesToDuplicate.Add(EmitterHandleToDuplicate);
				}
			}
 		}
		// Make the overview graph aware of the pasted nodes, so it can position them correctly.
		OnNodesPastedDelegate.Broadcast(PastedNodes);

		GetSystemViewModel()->DuplicateEmitters(EmitterHandlesToDuplicate);
	}
}

bool FNiagaraOverviewGraphViewModel::CanPasteNodes() const
{
	UEdGraph* Graph = GetGraph();
	if (Graph == nullptr)
	{
		return false;
	}

	// Cannot paste into emitter asset
	if (GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		return false;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FEdGraphUtilities::CanImportNodesFromText(Graph, ClipboardContent);
}

void FNiagaraOverviewGraphViewModel::DuplicateNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

bool FNiagaraOverviewGraphViewModel::CanDuplicateNodes() const
{
	return CanCopyNodes();
}

void FNiagaraOverviewGraphViewModel::RenameNode()
{
	if (GetGraph())
	{
		for (UObject* SelectedNode : NodeSelection->GetSelectedObjectsResolved())
		{
			UNiagaraOverviewNode* SelectedOverviewNode = Cast<UNiagaraOverviewNode>(SelectedNode);
			if (SelectedOverviewNode != nullptr)
			{
				SelectedOverviewNode->RequestRename();
				return;
			}
		}
	}
}

bool FNiagaraOverviewGraphViewModel::CanRenameNode() const
{
	if (GetGraph())
	{
		TSet<UObject*> SelectedNodes = NodeSelection->GetSelectedObjectsResolved();
		if (SelectedNodes.Num() == 1)
		{
			for (UObject* SelectedNode : SelectedNodes)
			{
				UNiagaraOverviewNode* SelectedOverviewNode = Cast<UNiagaraOverviewNode>(SelectedNode);
				if (SelectedOverviewNode != nullptr && SelectedOverviewNode->GetCanRenameNode())
				{
					return true;
				}
			}
		}
	}

	return false;
}

FText FNiagaraOverviewGraphViewModel::GetDisplayNameInternal() const
{
	ensureMsgf(SystemViewModel.IsValid(), TEXT("SystemViewModel was not initialized before InitDisplayName!"));
	TSharedPtr<FNiagaraSystemViewModel> WeakSystemViewModel = SystemViewModel.Pin();
	ENiagaraSystemViewModelEditMode SystemEditMode = WeakSystemViewModel->GetEditMode();
	if (SystemEditMode == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> EditedEmitterHandleViewModels = WeakSystemViewModel->GetEmitterHandleViewModels();
		if (ensureMsgf(EditedEmitterHandleViewModels.Num() > 0, TEXT("SystemViewModel did not have any EmitterHandleViewModels! Cannot get currently edited Emitter's Name.")))
		{
			return FText::FromName(WeakSystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterHandle()->GetName());
		}
	}
	else if (SystemEditMode == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		return FText::FromString(WeakSystemViewModel->GetSystem().GetName());
	}
	else
	{
		ensureMsgf(false, TEXT("Encountered unexpected SystemViewModel edit mode!"));
	}
	return LOCTEXT("NiagaraOverview_FallbackTitlebar", "Niagara");
}

void FNiagaraOverviewGraphViewModel::GraphSelectionChanged()
{
	if (bUpdatingGraphSelectionFromSystem == false)
	{
		TGuardValue<bool> UpdateGuard(bUpdatingSystemSelectionFromGraph, true);
		bool bClearCurrentSelection = FSlateApplication::Get().GetModifierKeys().IsControlDown() == false;

		UNiagaraStackEntry* RootEntry = GetSystemViewModel()->GetSystemStackViewModel()->GetRootEntry();
		UNiagaraStackCommentCollection* CommentCollection = nullptr;

		// we refresh the comment collection here to sync it with the current node selection
		if(UNiagaraStackRoot* AsRoot = Cast<UNiagaraStackRoot>(RootEntry))
		{
			CommentCollection = AsRoot->GetCommentCollection();
			if(CommentCollection != nullptr)
			{
				CommentCollection->RefreshChildren();
			}
		}
		
		TArray<UNiagaraStackEntry*> EntriesToSelect;
		TArray<UNiagaraStackEntry*> EntriesToDeselect;

		TArray<FGuid> SelectedGuids;
		TArray<TObjectPtr<UNiagaraStackObject>> NewTempEntries;

		for (UObject* SelectedObject : NodeSelection->GetSelectedObjectsResolved())
		{
			if (UNiagaraOverviewNode* SelectedOverviewNode = Cast<UNiagaraOverviewNode>(SelectedObject))
			{
				SelectedGuids.AddUnique(SelectedOverviewNode->GetEmitterHandleGuid());
			}
			// we find the stack object mapped to the comment node we have selected in the graph to select it in the stack;
			// this should always exist at this point as we have synced above
			else if(UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(SelectedObject))
			{
				if(CommentCollection == nullptr)
				{
					continue;
				}
				
				if (UNiagaraStackObject* CommentStackObject = CommentCollection->FindStackObjectForCommentNode(CommentNode))
				{
					EntriesToSelect.Add(CommentStackObject);
				}
			}
		}

		UNiagaraStackEntry* SystemRootEntry = GetSystemViewModel()->GetSystemStackViewModel()->GetRootEntry();
		if (SelectedGuids.Contains(FGuid()))
		{
			EntriesToSelect.Add(SystemRootEntry);
		}
		else
		{
			EntriesToDeselect.Add(SystemRootEntry);
		}

		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : GetSystemViewModel()->GetEmitterHandleViewModels())
		{
			UNiagaraStackEntry* EmitterRootEntry = EmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry();
			if (SelectedGuids.Contains(EmitterHandleViewModel->GetId()))
			{
				EntriesToSelect.Add(EmitterRootEntry);
			}
			else
			{
				EntriesToDeselect.Add(EmitterRootEntry);
			}
		}

		GetSystemViewModel()->GetSelectionViewModel()->UpdateSelectedEntries(EntriesToSelect, EntriesToDeselect, bClearCurrentSelection);
	}
}

void FNiagaraOverviewGraphViewModel::SystemSelectionChanged()
{
	if (bUpdatingSystemSelectionFromGraph == false)
	{
		TGuardValue<bool> UpdateGuard(bUpdatingGraphSelectionFromSystem, true);

		TArray<UObject*> SelectedNodes;
		TArray<UNiagaraOverviewNode*> OverviewNodes;
		OverviewGraph->GetNodesOfClass<UNiagaraOverviewNode>(OverviewNodes);
		TArray<UNiagaraStackEntry*> SelectedEntries;
		GetSystemViewModel()->GetSelectionViewModel()->GetSelectedEntries(SelectedEntries);
		for (UNiagaraOverviewNode* OverviewNode : OverviewNodes)
		{
			if (OverviewNode->GetEmitterHandleGuid().IsValid())
			{
				TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleviewModel = GetSystemViewModel()->GetEmitterHandleViewModelById(OverviewNode->GetEmitterHandleGuid());
				if (EmitterHandleviewModel.IsValid() && SelectedEntries.Contains(EmitterHandleviewModel->GetEmitterStackViewModel()->GetRootEntry()))
				{
					SelectedNodes.Add(OverviewNode);
				}
			}
			else
			{
				if (SelectedEntries.Contains(GetSystemViewModel()->GetSystemStackViewModel()->GetRootEntry()))
				{
					SelectedNodes.Add(OverviewNode);
				}
			}
		}
		NodeSelection->SetSelectedObjects(SelectedNodes);
	}
}

void FNiagaraOverviewGraphViewModel::PostUndo(bool bSuccess)
{
	// This is neccessary to have the graph editor respond correctly to data changes due to undo.
	// Validate the OverviewGraph exists before notifying as we may be entering PostUndo from a transaction that predates the OverviewGraph.
	if (OverviewGraph.IsValid())
	{
		OverviewGraph->NotifyGraphChanged();
	}
}

#undef LOCTEXT_NAMESPACE // NiagaraScriptGraphViewModel
