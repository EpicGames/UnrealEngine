// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterCollectionAssetViewModel.h"
#include "NiagaraParameterCollection.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraCollectionParameterViewModel.h"

#include "Editor.h"

#include "ScopedTransaction.h"
#include "EditorSupportDelegates.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "Math/NumericLimits.h"

#include "NiagaraDataInterface.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptInputCollection"

template<> TMap<UNiagaraParameterCollection*, TArray<FNiagaraParameterCollectionAssetViewModel*>> TNiagaraViewModelManager<UNiagaraParameterCollection, FNiagaraParameterCollectionAssetViewModel>::ObjectsToViewModels{};

FNiagaraParameterCollectionAssetViewModel::FNiagaraParameterCollectionAssetViewModel(UNiagaraParameterCollection* InCollection, FText InDisplayName, ENiagaraParameterEditMode InParameterEditMode)
	: FNiagaraParameterCollectionViewModel(InParameterEditMode)
	, DisplayName(InDisplayName)
	, Collection(InCollection)
	, Instance(InCollection->GetDefaultInstance())
{
	check(Collection && Instance);

	RegisteredHandle = RegisterViewModelWithMap(Collection, this);
	GEditor->RegisterForUndo(this);

	ExternalChangeHandle = Collection->OnCollectionChangedDelegate.AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::OnCollectionChangedExternally);
	ParameterValuesChangedHandle = Instance->ParameterValuesChanged().AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::RefreshParameterViewModels);

	RefreshParameterViewModels();
}

FNiagaraParameterCollectionAssetViewModel::FNiagaraParameterCollectionAssetViewModel(UNiagaraParameterCollectionInstance* InInstance, FText InDisplayName, ENiagaraParameterEditMode InParameterEditMode)
	: FNiagaraParameterCollectionViewModel(InParameterEditMode)
	, DisplayName(InDisplayName)
	, Collection(InInstance->GetParent())
	, Instance(InInstance)
{
	check(Instance);

	GEditor->RegisterForUndo(this);

	if (Collection)
	{
		RegisteredHandle = RegisterViewModelWithMap(Collection, this);
		ExternalChangeHandle = Collection->OnCollectionChangedDelegate.AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::OnCollectionChangedExternally);
	}

	ParameterValuesChangedHandle = Instance->ParameterValuesChanged().AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::RefreshParameterViewModels);

	RefreshParameterViewModels();
}

FNiagaraParameterCollectionAssetViewModel::~FNiagaraParameterCollectionAssetViewModel()
{
	for (int32 i = 0; i < ParameterViewModels.Num(); i++)
	{
		FNiagaraCollectionParameterViewModel* ParameterViewModel = (FNiagaraCollectionParameterViewModel*)(&ParameterViewModels[i].Get());
		if (ParameterViewModel != nullptr)
		{
			ParameterViewModel->OnNameChanged().RemoveAll(this);
			ParameterViewModel->OnTypeChanged().RemoveAll(this);
			ParameterViewModel->OnDefaultValueChanged().RemoveAll(this);
		}
	}
	ParameterViewModels.Empty();

	GEditor->UnregisterForUndo(this);
	UnregisterViewModelWithMap(RegisteredHandle);

	if (Collection)
	{
		Collection->OnCollectionChangedDelegate.Remove(ExternalChangeHandle);
	}

	Instance->ParameterValuesChanged().Remove(ParameterValuesChangedHandle);
}

void FNiagaraParameterCollectionAssetViewModel::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	if (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraParameterCollectionInstance, Collection))
	{
		GEditor->BeginTransaction(LOCTEXT("ChangeNPCInstanceParent", "Change Parent"));

		if (Collection)
		{
			Collection->OnCollectionChangedDelegate.Remove(ExternalChangeHandle);
			UnregisterViewModelWithMap(RegisteredHandle);
		}

		Instance->ParameterValuesChanged().Remove(ParameterValuesChangedHandle);
		Instance->Empty();
	}
}

void FNiagaraParameterCollectionAssetViewModel::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraParameterCollectionInstance, Collection))
	{
		//Instance has changed parent;
		check(!Instance->IsDefaultInstance());
		Collection = Instance->GetParent();

		if (Collection)
		{
			RegisteredHandle = RegisterViewModelWithMap(Collection, this);
			ExternalChangeHandle = Collection->OnCollectionChangedDelegate.AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::OnCollectionChangedExternally);
		}

		ParameterValuesChangedHandle = Instance->ParameterValuesChanged().AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::RefreshParameterViewModels);

		RefreshParameterViewModels();
		GEditor->EndTransaction();
	}
}

FText FNiagaraParameterCollectionAssetViewModel::GetDisplayName() const
{
	return DisplayName;
}

FName FNiagaraParameterCollectionAssetViewModel::GenerateNewName(FNiagaraTypeDefinition Type)const
{
	check(Collection && Instance);

	FName ProposedName = FName(Type.GetName());
	TSet<FName> Existing;
	Existing.Reserve(ParameterViewModels.Num());
	for (TSharedRef<INiagaraParameterViewModel> ParameterViewModel : ParameterViewModels)
	{
		Existing.Add(ParameterViewModel->GetName());
	}

	return Collection->ConditionalAddFullNamespace(FNiagaraUtilities::GetUniqueName(ProposedName, Existing));
}

void FNiagaraParameterCollectionAssetViewModel::AddParameter(TSharedPtr<FNiagaraTypeDefinition> ParameterType)
{
	check(Collection && Instance);

	FScopedTransaction ScopedTransaction(LOCTEXT("AddNPCParameter", "Add Parameter"));
	Collection->Modify();

	FNiagaraTypeDefinition Type = *ParameterType.Get();
	FName NewName = GenerateNewName(Type);

	int32 ParamIdx = Collection->AddParameter(NewName, Type);

	//TODO: It'd be nice to be able to get a default value for types in runtime code and do this inside the parameter store itself.
	if (!Type.IsDataInterface() && !Type.IsUObject())
	{
		TArray<uint8> DefaultData;
		FNiagaraEditorUtilities::GetTypeDefaultValue(Type, DefaultData);
		Instance->GetParameterStore().SetParameterData(DefaultData.GetData(), Collection->GetParameters()[ParamIdx]);
	}

	CollectionChanged(false);
	RefreshParameterViewModels();

	for (TSharedRef<INiagaraParameterViewModel> ParameterViewModel : ParameterViewModels)
	{
		if (ParameterViewModel->GetName() == NewName)
		{
			GetSelection().SetSelectedObject(ParameterViewModel);
			break;
		}
	}	
}

void FNiagaraParameterCollectionAssetViewModel::UpdateOpenInstances()
{
	TArray<TSharedPtr<FNiagaraParameterCollectionAssetViewModel>> ViewModels;
	if (FNiagaraParameterCollectionAssetViewModel::GetAllViewModelsForObject(Collection, ViewModels))
	{
		for (TSharedPtr<FNiagaraParameterCollectionAssetViewModel> ViewModel : ViewModels)
		{
			if (ViewModel.Get() != this)
			{
				//This is not sufficient. 
				//If we rename a parameter for example, we'll lose any overrides it had. Improve this.
				ViewModel->RefreshParameterViewModels();
			}
		}
	}
}

void FNiagaraParameterCollectionAssetViewModel::UpdateParameterSelectionFromSearch(const FText& InSearchText)
{
	GetSelection().ClearSelectedObjects();

	SearchText = InSearchText;

	RefreshParameterViewModels();
	
	if(!SearchText.IsEmpty())
	{
		TArray<TSharedRef<INiagaraParameterViewModel>> MatchingParameters;
		for(TSharedRef<INiagaraParameterViewModel> Parameter : ParameterViewModels)
		{
			if(Parameter->GetName().ToString().Contains(SearchText.ToString()))
			{
				MatchingParameters.Add(Parameter);
			}
		}

		GetSelection().SetSelectedObjects(MatchingParameters);
	}
}

void FNiagaraParameterCollectionAssetViewModel::RemoveParameter(FNiagaraVariable& Parameter)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("RemoveNPCParameter", "Remove Parameter"));
	Collection->RemoveParameter(Parameter);

	CollectionChanged(true);

	RefreshParameterViewModels();
}

void FNiagaraParameterCollectionAssetViewModel::PostUndo(bool bSuccess)
{
	Collection = Instance->GetParent();
	CollectionChanged(true);

	RefreshParameterViewModels();
}

void FNiagaraParameterCollectionAssetViewModel::DeleteSelectedParameters()
{
	check(Collection && Instance);
	if (GetSelection().GetSelectedObjects().Num() > 0)
	{
		TSet<FNiagaraVariable> VarsToDelete;
		for (TSharedRef<INiagaraParameterViewModel> Parameter : GetSelection().GetSelectedObjects())
		{
			VarsToDelete.Add(Parameter->GetVariable());
		}
		GetSelection().ClearSelectedObjects();

		DeleteParameters(VarsToDelete.Array());
	}
}

void FNiagaraParameterCollectionAssetViewModel::DeleteParameters(TArray<FNiagaraVariable> ParametersToDelete)
{
	check(Collection && Instance);

	TSet<FNiagaraVariableBase> ResolvedParametersToDelete;
	for(const FNiagaraVariableBase& Parameter : ParametersToDelete)
	{
		FNiagaraVariableBase ToDelete(Parameter.GetType(), Collection->ConditionalAddFullNamespace(Parameter.GetName()));
		ResolvedParametersToDelete.Add(ToDelete);
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("DeleteNPCParameter", "Delete Parameter"));
	for (FNiagaraVariable ParamToDelete : ResolvedParametersToDelete)
	{
		Collection->RemoveParameter(ParamToDelete);
	}

	CollectionChanged(true);
	RefreshParameterViewModels();
}

const TArray<TSharedRef<INiagaraParameterViewModel>>& FNiagaraParameterCollectionAssetViewModel::GetParameters()
{
	return ParameterViewModels;
}

EVisibility FNiagaraParameterCollectionAssetViewModel::GetAddButtonVisibility()const
{
	return Instance->IsDefaultInstance() ? EVisibility::Visible : EVisibility::Hidden;
}

void FNiagaraParameterCollectionAssetViewModel::CollectionChanged(bool bRecompile)
{
	for (TObjectIterator<UNiagaraParameterCollectionInstance> It; It; ++It)
	{
		if (It->Collection == Collection)
		{
			It->SyncWithCollection();
		}
	}

	//Refresh any existing view models that might be showing changed instances.
	UpdateOpenInstances();

// 	//Refresh any nodes that are referencing this collection.
// 	for (TObjectIterator<UNiagaraNodeParameterCollection> It; It; ++It)
// 	{
// 		if (It->GetReferencedAsset() == Collection)
// 		{
// 			It->RefreshFromExternalChanges();
// 		}
// 	}

	if (bRecompile)
	{
		//Update any active systems that are using this collection.
		FNiagaraSystemUpdateContext Context(Collection, true);
	}
}

TSharedPtr<FNiagaraCollectionParameterViewModel> FNiagaraParameterCollectionAssetViewModel::CreateParameterViewModel(const FNiagaraVariable& Var)
{
	TSharedPtr<FNiagaraCollectionParameterViewModel> ParameterViewModel = MakeShareable(new FNiagaraCollectionParameterViewModel(Var, Instance, ParameterEditMode));

	ParameterViewModel->OnNameChanged().AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::OnParameterNameChanged, Var);
	ParameterViewModel->OnTypeChanged().AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::OnParameterTypeChanged, Var);
	ParameterViewModel->OnDefaultValueChanged().AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::OnParameterValueChangedInternal, ParameterViewModel.ToSharedRef());
	ParameterViewModel->OnProvidedChanged().AddRaw(this, &FNiagaraParameterCollectionAssetViewModel::OnParameterProvidedChanged, Var);

	return ParameterViewModel;
}

void FNiagaraParameterCollectionAssetViewModel::RefreshParameterViewModels()
{
	if (!Collection)
	{
		ParameterViewModels.Empty();
		OnCollectionChangedDelegate.Broadcast();
		return;
	}

	for (int32 i = 0; i < ParameterViewModels.Num(); i++)
	{
		FNiagaraCollectionParameterViewModel* ParameterViewModel = (FNiagaraCollectionParameterViewModel*)(&ParameterViewModels[i].Get());
		if (ParameterViewModel != nullptr)
		{
			ParameterViewModel->Reset();
		}
	}

	ParameterViewModels.Empty();

	bool bSearchTextIsEmpty = SearchText.IsEmpty();

	TArray<FString> SearchTerms;
	if(bSearchTextIsEmpty == false)
	{
		SearchText.ToString().ParseIntoArray(SearchTerms, TEXT(" "));
	}
	for (int32 i = 0; i < Collection->GetParameters().Num(); ++i)
	{
		FNiagaraVariable& Var = Collection->GetParameters()[i];

		if(bSearchTextIsEmpty == false)
		{
			bool bMatchFound = false;
			for(const FString& SearchTerm : SearchTerms)
			{
				if(Var.GetName().ToString().Contains(SearchTerm))
				{
					bMatchFound = true;
					break;
				}
			}

			if(!bMatchFound)
			{
				continue;
			}
		}
		
		ParameterViewModels.Add(CreateParameterViewModel(Var).ToSharedRef());
	}

	OnCollectionChangedDelegate.Broadcast();
}

void FNiagaraParameterCollectionAssetViewModel::RefreshParameterViewModels(TConstArrayView<FName> ChangedParameters)
{
	bool bChangeMade = false;

	for (TSharedRef<INiagaraParameterViewModel>& ParameterViewModel : ParameterViewModels)
	{
		FNiagaraVariable Var = ParameterViewModel->GetVariable();
		if (ChangedParameters.Contains(Var.GetName()))
		{
			if (FNiagaraCollectionParameterViewModel* CastParameterViewModel = (FNiagaraCollectionParameterViewModel*)(&ParameterViewModel.Get()))
			{
				CastParameterViewModel->Reset();
			}

			ParameterViewModel = CreateParameterViewModel(Var).ToSharedRef();

			bChangeMade = true;
		}
	}

	if (bChangeMade)
	{
		OnCollectionChangedDelegate.Broadcast();
	}
}

bool FNiagaraParameterCollectionAssetViewModel::SupportsType(const FNiagaraTypeDefinition& Type) const
{
	return Type != FNiagaraTypeDefinition::GetGenericNumericDef() && !Type.IsInternalType();
}

void FNiagaraParameterCollectionAssetViewModel::OnParameterNameChanged(FName OldName, FName NewName, FNiagaraVariable ParameterVariable)
{
	//How can we update any other open instances here?

	int32 Index = Collection->IndexOfParameter(ParameterVariable);
	check(Index != INDEX_NONE);

	FName ParamName = Collection->ConditionalAddFullNamespace(GetParameters()[Index]->GetName());
	Collection->GetParameters()[Index].SetName(ParamName);
	Instance->RenameParameter(ParameterVariable, ParamName);
	CollectionChanged(false);

	RefreshParameterViewModels();
}

void FNiagaraParameterCollectionAssetViewModel::SetAllParametersEditingEnabled(bool bInEnabled)
{
	for (TSharedRef<INiagaraParameterViewModel>& ParameterViewModel : ParameterViewModels)
	{
		ParameterViewModel->SetEditingEnabled(bInEnabled);
	}
}

void FNiagaraParameterCollectionAssetViewModel::SetAllParametersTooltipOverrides(const FText& Override)
{
	for (TSharedRef<INiagaraParameterViewModel>& ParameterViewModel : ParameterViewModels)
	{
		ParameterViewModel->SetTooltipOverride(Override);
	}
}

TSharedPtr<INiagaraParameterViewModel> FNiagaraParameterCollectionAssetViewModel::GetParameterViewModel(const FName& Name)
{
	for (TSharedRef<INiagaraParameterViewModel>& ParameterViewModel : ParameterViewModels)
	{
		if (ParameterViewModel->GetName() == Name)
		{
			return TSharedPtr<INiagaraParameterViewModel>(ParameterViewModel);
		}
	}
	return TSharedPtr<INiagaraParameterViewModel>();
}

void FNiagaraParameterCollectionAssetViewModel::OnParameterTypeChanged(FNiagaraVariable ParameterVariable)
{
	int32 Index = Collection->IndexOfParameter(ParameterVariable);
	check(Index != INDEX_NONE);

	Collection->Modify();

	Collection->GetDefaultInstance()->RemoveParameter(ParameterVariable);

	FNiagaraTypeDefinition Type = ParameterViewModels[Index]->GetType();
	Collection->GetParameters()[Index].SetType(Type);

	Collection->GetDefaultInstance()->AddParameter(Collection->GetParameters()[Index]);
	
	//TODO: It'd be nice to be able to get a default value for types in runtime code and do this inside the parameter store itself.
	if (!Type.IsDataInterface() && !Type.IsUObject())
	{
		TArray<uint8> DefaultData;
		FNiagaraEditorUtilities::GetTypeDefaultValue(Type, DefaultData);
		Collection->GetDefaultInstance()->GetParameterStore().SetParameterData(DefaultData.GetData(), Collection->GetParameters()[Index]);
	}

	CollectionChanged(true);

	RefreshParameterViewModels();
}

void FNiagaraParameterCollectionAssetViewModel::OnParameterProvidedChanged(FNiagaraVariable ParameterVariable)
{
	//int32 Index = Collection->IndexOfParameter(ParameterVariable);
	//check(Index != INDEX_NONE);

	RefreshParameterViewModels();
	//CollectionChanged(false);

	// Restore the value from the Collection if we are no longer overriding it.  Note that this will override the value that is in the instance
	// so we don't currently have a way to store an unused override in the instance...something that would be nice to have.
	if (Instance && Collection && !Instance->OverridesParameter(ParameterVariable))
	{
		if (const uint8* CollectionParameterValue = Collection->GetDefaultInstance()->GetParameterStore().GetParameterData(ParameterVariable))
		{
			Instance->GetParameterStore().SetParameterData(CollectionParameterValue, ParameterVariable);
			Instance->GetParameterStore().Tick();
		}
	}
}

void FNiagaraParameterCollectionAssetViewModel::OnParameterValueChangedInternal(TSharedRef<FNiagaraCollectionParameterViewModel> ChangedParameter)
{
	//restart any systems using this collection.
	FNiagaraSystemUpdateContext UpdateContext(Collection, true);

	OnParameterValueChanged().Broadcast({ ChangedParameter->GetName() });

	if (Collection->GetDefaultInstance() == Instance)
	{
		Collection->OnParameterValuesEdited.Broadcast({ ChangedParameter->GetVariable().GetName() });
	}
	
	//Push the change to anyone already bound.
	Instance->GetParameterStore().Tick();
}

void FNiagaraParameterCollectionAssetViewModel::OnCollectionChangedExternally()
{
	CollectionChanged(true);
	RefreshParameterViewModels();
}

#undef LOCTEXT_NAMESPACE // "NiagaraScriptInputCollectionViewModel"
