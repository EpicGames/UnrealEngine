// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraParameterCollectionViewModel.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"

class SNiagaraParameterCollection;
class UNiagaraParameterCollection;
class UNiagaraParameterCollectionInstance;
class FNiagaraCollectionParameterViewModel;

/** A view model for Niagara Parameter Collection assets. */
class FNiagaraParameterCollectionAssetViewModel : 
	public FNiagaraParameterCollectionViewModel, 
	public FEditorUndoClient, 
	public TSharedFromThis<FNiagaraParameterCollectionAssetViewModel>, 
	public FNotifyHook,
	public TNiagaraViewModelManager<UNiagaraParameterCollection, FNiagaraParameterCollectionAssetViewModel>
{
public:
	FNiagaraParameterCollectionAssetViewModel(UNiagaraParameterCollection* InCollection, FText InDisplayName, ENiagaraParameterEditMode InParameterEditMode);
	FNiagaraParameterCollectionAssetViewModel(UNiagaraParameterCollectionInstance* InInstance, FText InDisplayName, ENiagaraParameterEditMode InParameterEditMode);

	~FNiagaraParameterCollectionAssetViewModel();

	//~ INiagaraParameterCollectionViewModel interface
	virtual FText GetDisplayName() const override;
	virtual void AddParameter(TSharedPtr<FNiagaraTypeDefinition> ParameterType) override;
	virtual void RemoveParameter(FNiagaraVariable& Parameter);
	virtual void DeleteSelectedParameters() override;
	virtual void DeleteParameters(TArray<FNiagaraVariable> ParametersToDelete) override;
	virtual const TArray<TSharedRef<INiagaraParameterViewModel>>& GetParameters() override;
	virtual EVisibility GetAddButtonVisibility()const override;

	void UpdateOpenInstances();

	void UpdateParameterSelectionFromSearch(const FText& SearchText);
	
	//~ FNotifyHook interface
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange)override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)override;

	/** Rebuilds the parameter view models. */
	virtual void RefreshParameterViewModels() override;

	void CollectionChanged(bool bRecompile);

	/** Gets the parameter view model associated with a given Id.*/
	TSharedPtr<INiagaraParameterViewModel> GetParameterViewModel(const FName& Name);

	/** Sets all parameter view models editable state to the input value.*/
	void SetAllParametersEditingEnabled(bool bInEnabled);

	/** Sets the tooltip overrides on all parameters.*/
	void SetAllParametersTooltipOverrides(const FText& Override);
	
	//~ FEditorUndoClient
	void PostUndo(bool bSuccess);
	void PostRedo(bool bSuccess) { PostUndo(bSuccess); }

protected:
	//~ FNiagaraParameterCollectionViewModel interface.
	virtual bool SupportsType(const FNiagaraTypeDefinition& Type) const override;

	void RefreshParameterViewModels(TConstArrayView<FName> ChangedParameters);

private:	
	/** Handles when the name on a parameter changes. */
	void OnParameterNameChanged(FName OldName, FName NewName, FNiagaraVariable ParameterVariable);

	/** Handles when the type on a parameter changes. */
	void OnParameterTypeChanged(FNiagaraVariable ParameterVariable);
	
	/** Handled when an optional parameter is toggled between provided and not. */
	void OnParameterProvidedChanged(FNiagaraVariable ParameterVariable);

	void OnParameterValueChangedInternal(TSharedRef<FNiagaraCollectionParameterViewModel> ChangedParameter);

	/** Called if the Collection is changed external to the UI and needs refereshing */
	void OnCollectionChangedExternally();

	FName GenerateNewName(FNiagaraTypeDefinition Type)const;

private:
	TSharedPtr<FNiagaraCollectionParameterViewModel> CreateParameterViewModel(const FNiagaraVariable& Var);

	/** The parameter view models. */
	TArray<TSharedRef<INiagaraParameterViewModel>> ParameterViewModels;

	/** The display name for the view model. */
	FText DisplayName;

	UNiagaraParameterCollection* Collection;
	UNiagaraParameterCollectionInstance* Instance;

	FText SearchText;
	
	TNiagaraViewModelManager<UNiagaraParameterCollection, FNiagaraParameterCollectionAssetViewModel>::Handle RegisteredHandle;

	FDelegateHandle ExternalChangeHandle;
	FDelegateHandle ParameterValuesChangedHandle;
};

