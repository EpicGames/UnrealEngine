// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterCollectionViewModel.h"
#include "NiagaraParameterViewModel.h"
#include "NiagaraTypes.h"

#define LOCTEXT_NAMESPACE "NiagaraParameterCollectionViewModel"


void INiagaraParameterCollectionViewModel::SortViewModels(TArray<TSharedRef<INiagaraParameterViewModel>>& InOutViewModels)
{
	auto SortVars = [](const TSharedRef<INiagaraParameterViewModel>& A, const TSharedRef<INiagaraParameterViewModel>& B)
	{
		if (A->GetSortOrder() < B->GetSortOrder())
		{
			return true;
		}
		else if (A->GetSortOrder() > B->GetSortOrder())
		{
			return false;
		}

		//If equal priority, sort lexicographically.
		return A->GetName().ToString() < B->GetName().ToString();
	};
	InOutViewModels.Sort(SortVars);
}

TArray<TSharedPtr<FNiagaraTypeDefinition>> INiagaraParameterCollectionViewModel::GetAvailableTypesSorted()
{
	TArray<TSharedPtr<FNiagaraTypeDefinition>> AvailableTypes = GetAvailableTypes();
	FText::FSortPredicate SortPredicate;
	AvailableTypes.Sort([SortPredicate](const TSharedPtr<FNiagaraTypeDefinition>& A, const TSharedPtr<FNiagaraTypeDefinition>& B)
    {
        return SortPredicate(A->GetNameText(), B->GetNameText());
    });
	return AvailableTypes;
}

FNiagaraParameterCollectionViewModel::FNiagaraParameterCollectionViewModel(ENiagaraParameterEditMode InParameterEditMode)
	: ParameterEditMode(InParameterEditMode)
	, bNeedsRefresh(false)
	, bIsExpanded(true)
{
}

bool FNiagaraParameterCollectionViewModel::GetIsExpanded() const
{
	return bIsExpanded;
}

void FNiagaraParameterCollectionViewModel::SetIsExpanded(bool bInIsExpanded)
{
	if (bIsExpanded != bInIsExpanded)
	{
		bIsExpanded = bInIsExpanded;
		OnExpandedChangedDelegate.Broadcast();
	}
}

EVisibility FNiagaraParameterCollectionViewModel::GetAddButtonVisibility() const
{
	return ParameterEditMode == ENiagaraParameterEditMode::EditAll ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FNiagaraParameterCollectionViewModel::GetAddButtonText() const
{
	return LOCTEXT("AddButtonText", "Add Parameter");
}

bool FNiagaraParameterCollectionViewModel::CanDeleteParameters() const
{
	return ParameterEditMode == ENiagaraParameterEditMode::EditAll;
}

void FNiagaraParameterCollectionViewModel::Tick(float DeltaTime)
{
	if (bNeedsRefresh)
	{
		RefreshParameterViewModels();
		bNeedsRefresh = false;
	}
}

bool FNiagaraParameterCollectionViewModel::IsTickable() const
{
	return true;
}

TStatId FNiagaraParameterCollectionViewModel::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraParameterCollectionViewModel, STATGROUP_Tickables);
}

TSet<FName> FNiagaraParameterCollectionViewModel::GetParameterNames()
{
	TSet<FName> ParameterNames;
	for (TSharedRef<INiagaraParameterViewModel> Parameter : GetParameters())
	{
		ParameterNames.Add(Parameter->GetName());
	}
	return ParameterNames;
}

const TArray<TSharedPtr<FNiagaraTypeDefinition>>& FNiagaraParameterCollectionViewModel::GetAvailableTypes()
{
	if (AvailableTypes.IsValid() == false)
	{
		RefreshAvailableTypes();
	}
	return *AvailableTypes;
}

FText FNiagaraParameterCollectionViewModel::GetTypeDisplayName(TSharedPtr<FNiagaraTypeDefinition> Type) const
{
	return Type->GetStruct()->GetDisplayNameText();
}

void FNiagaraParameterCollectionViewModel::RefreshAvailableTypes()
{
	if (AvailableTypes.IsValid() == false)
	{
		AvailableTypes = MakeShareable(new TArray<TSharedPtr<FNiagaraTypeDefinition>>());
	}

	AvailableTypes->Empty();
	TArray<FNiagaraTypeDefinition> AllowedParameterTypes;
	FNiagaraEditorUtilities::GetAllowedParameterTypes(AllowedParameterTypes);
	for (const FNiagaraTypeDefinition& RegisteredType : AllowedParameterTypes)
	{
		if (SupportsType(RegisteredType))
		{
			AvailableTypes->Add(MakeShareable(new FNiagaraTypeDefinition(RegisteredType)));
		}
	}
}

void FNiagaraParameterCollectionViewModel::NotifyParameterChangedExternally(FName ParameterName)
{
	for (TSharedRef<INiagaraParameterViewModel> ParameterViewModel : GetParameters())
	{
		if (ParameterViewModel->GetName() == ParameterName)
		{
			ParameterViewModel->NotifyDefaultValueChanged();
		}
	}
}

#undef LOCTEXT_NAMESPACE // NiagaraParameterCollectionViewModel