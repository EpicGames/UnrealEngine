// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraObjectSelection.h"
#include "NiagaraParameterViewModel.h"
#include "Layout/Visibility.h"
#include "TickableEditorObject.h"

struct FNiagaraTypeDefinition;

/** A niagara selection for parameter view models. */
class FParameterSelection : public TNiagaraSelection<TSharedRef<INiagaraParameterViewModel>>
{
};

/** Defines the view model for the parameter collection editor. */
class INiagaraParameterCollectionViewModel
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnCollectionChanged);
	DECLARE_MULTICAST_DELEGATE(FOnExpandedChanged);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnParameterValueChanged, TConstArrayView<FName>);

public:
	virtual ~INiagaraParameterCollectionViewModel() = default;

	/** Gets the display name for the parameter collection. */
	virtual FText GetDisplayName() const = 0;

	/** Gets whether or not the collection UI is expanded. */
	virtual bool GetIsExpanded() const = 0;

	/** Sets whether the collection UI is expanded. */
	virtual void SetIsExpanded(bool bInIsExpanded) = 0;

	/** Gets the visibility of the add parameter button .*/
	virtual EVisibility GetAddButtonVisibility() const = 0;

	/** Gets the dest displayed next to the add parameter button .*/
	virtual FText GetAddButtonText() const = 0;

	/** Adds a new parameter to the collection. */
	virtual void AddParameter(TSharedPtr<FNiagaraTypeDefinition> ParameterType) = 0;

	/** Returns whether or not parameters can be deleted from the collection. */
	virtual bool CanDeleteParameters() const = 0;

	/** Deletes the currently selected parameters. */
	virtual void DeleteSelectedParameters() = 0;

	virtual void DeleteParameters(TArray<FNiagaraVariable> ParametersToDelete) = 0;

	/** Gets the parameter view models. */
	virtual const TArray<TSharedRef<INiagaraParameterViewModel>>& GetParameters() = 0;

	/** Gets the available types which can be used with the parameter view models. */
	virtual const TArray<TSharedPtr<FNiagaraTypeDefinition>>& GetAvailableTypes() = 0;

	/** Gets the available types which can be used with the parameter view models, sorted ascending by name. */
	TArray<TSharedPtr<FNiagaraTypeDefinition>> GetAvailableTypesSorted();

	/** Gets the display name for the provided type. */
	virtual FText GetTypeDisplayName(TSharedPtr<FNiagaraTypeDefinition> Type) const = 0;

	/** Gets the currently selected parameter view models. */
	virtual FParameterSelection& GetSelection() = 0;

	/** Gets the currently selected parameter view models. */
	virtual const FParameterSelection& GetSelection() const = 0;

	/** Gets a multicast delegate which is called whenever the collection of view models changes. (This is not called when an individual parameter's value changes. */
	virtual FOnCollectionChanged& OnCollectionChanged() = 0;

	/** Gets a multicast delegate which is called whenever the expanded state of the control is changed. */
	virtual FOnExpandedChanged& OnExpandedChanged() = 0;

	/** Gets a multicast delegate which is called whenever the value of one of the parameters in the collection changes. */
	virtual FOnParameterValueChanged& OnParameterValueChanged() = 0;

	/** Rebuilds the parameter view models. */
	virtual void RefreshParameterViewModels()  = 0;

	/** Notifies the parameter collection that a parameter was changed externally. */
	virtual void NotifyParameterChangedExternally(FName ParameterName) = 0;

	/** Helper function to sort view models via their name and sort order.*/
	static void SortViewModels(TArray<TSharedRef<INiagaraParameterViewModel>>& InOutViewModels);
};

/** Base class for parameter collection view models.  Partially implements the parameter collection interface with
	behavior common to all view models. */
class FNiagaraParameterCollectionViewModel : public INiagaraParameterCollectionViewModel, public FTickableEditorObject
{
public:
	FNiagaraParameterCollectionViewModel(ENiagaraParameterEditMode InParameterEditMode);

	virtual ~FNiagaraParameterCollectionViewModel() = default;

	//~ INiagaraParameterCollectionViewModel interface.
	virtual bool GetIsExpanded() const override;
	virtual void SetIsExpanded(bool bInIsExpanded) override;
	virtual EVisibility GetAddButtonVisibility() const override;
	virtual FText GetAddButtonText() const override;
	virtual bool CanDeleteParameters() const override;
	virtual const TArray<TSharedPtr<FNiagaraTypeDefinition>>& GetAvailableTypes() override;
	virtual FText GetTypeDisplayName(TSharedPtr<FNiagaraTypeDefinition> Type) const override;
	virtual FParameterSelection& GetSelection() { return ParameterSelection; }
	virtual const FParameterSelection& GetSelection() const { return ParameterSelection; }
	virtual FOnCollectionChanged& OnCollectionChanged() override { return OnCollectionChangedDelegate; }
	virtual FOnExpandedChanged& OnExpandedChanged() override { return OnExpandedChangedDelegate; }
	virtual FOnParameterValueChanged& OnParameterValueChanged() override { return OnParameterValueChangedDelegate; }
	virtual void NotifyParameterChangedExternally(FName ParameterName) override;

	//~ FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;

protected:
	/** Gets a set containing the names of the parameters. */
	TSet<FName> GetParameterNames();

	/** Allows derived classes to filter the available types displayed by the UI. */
	virtual bool SupportsType(const FNiagaraTypeDefinition& Type) const { return true; }

private:
	/** Refreshes the list of available types. */
	void RefreshAvailableTypes();

protected:
	/** The currently selected parameters. */
	FParameterSelection ParameterSelection;

	/** A multicast delegate which is called whenever the parameter collection is changed. */
	FOnCollectionChanged OnCollectionChangedDelegate;

	/** A multicast delegate which is called whenever the UI expanded state changes. */
	FOnExpandedChanged OnExpandedChangedDelegate;

	/** A multicast delegate which is called whenever the value of one of the parameters in the collection changes. */
	FOnParameterValueChanged OnParameterValueChangedDelegate;

	/** Gets the edit mode for parameters in this collection. */
	const ENiagaraParameterEditMode ParameterEditMode;

	/** Refresh the UI next frames. */
	bool bNeedsRefresh;

private:
	/** An array of available types for parameters. */
	TSharedPtr<TArray<TSharedPtr<FNiagaraTypeDefinition>>> AvailableTypes;

	/** Whether or not the UI is expanded. */
	bool bIsExpanded;
};
