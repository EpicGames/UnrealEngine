// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraParameterDefinitionsPanelViewModel.h"

#include "NiagaraClipboard.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraScriptVariable.h"
#include "ScopedTransaction.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraStandaloneScriptViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"

#define LOCTEXT_NAMESPACE "NiagaraParameterDefinitionsPanelViewModel"

///////////////////////////////////////////////////////////////////////////////
/// Shared View Model Loctexts												///
///////////////////////////////////////////////////////////////////////////////
const static FText Loctext_AddDefinitionsAssetTransaction = LOCTEXT("AddDefinitionsAssetTransaction", "Add Definitions");
const static FText Loctext_RemoveDefinitionsAssetTransaction = LOCTEXT("RemoveDefinitionsAssetTransaction", "Unlink all Parameters From Definitions");
const static FText Loctext_SubscribeAllParametersToDefinitionsTransaction = LOCTEXT("SubscribeAllParametersToDefinitionsTransaction", "Link all Parameters to Definitions");

const static FText Loctext_SubscribeAllParametersToDefinitionsToolTip = LOCTEXT("SubscribeAllParametersToDefinitionsToolTip", "Link all Parameters to Definitions.");
const static FText Loctext_CannotSubscribeAllParametersToDefinitionsToolTip_NoCandidates = LOCTEXT("CannotSubscribeALlParametersToDefinitionsToolTip_NoCandidates", "Cannot link all Parameters to Definitions; No Parameters have a name and type match to a Definition.");

const static FText Loctext_UnsubscribeFromParameterDefinitionsToolTip = LOCTEXT("UnsubscribeFromParameterDefinitionsToolTip", "Unlink all Parameters from Definitions.");

///////////////////////////////////////////////////////////////////////////////
/// Base Parameter Definitions Panel View Model									///
///////////////////////////////////////////////////////////////////////////////

const TArray<UNiagaraScriptVariable*> INiagaraParameterDefinitionsPanelViewModel::GetEditableScriptVariablesWithName(const FName ParameterName) const
{
	TArray<UNiagaraScriptVariable*> EditableScriptVariables;
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		UNiagaraScriptVariable* ParameterScriptVariable = Graph->GetScriptVariable(ParameterName);
		if (ParameterScriptVariable != nullptr)
		{
			EditableScriptVariables.Add(ParameterScriptVariable);
		}
	}
	return EditableScriptVariables;
}

const TArray<FNiagaraGraphParameterReference> INiagaraParameterDefinitionsPanelViewModel::GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const
{
	TArray<FNiagaraGraphParameterReference> ParameterReferences;
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		const FNiagaraGraphParameterReferenceCollection* GraphParameterReferenceCollectionPtr = Graph->GetParameterReferenceMap().Find(Item.GetVariable());
		if (GraphParameterReferenceCollectionPtr)
		{
			ParameterReferences.Append(GraphParameterReferenceCollectionPtr->ParameterReferences);
		}
	}
	return ParameterReferences;
}


///////////////////////////////////////////////////////////////////////////////
/// Script Toolkit Parameter Definitions Panel View Model						///
///////////////////////////////////////////////////////////////////////////////

FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::FNiagaraScriptToolkitParameterDefinitionsPanelViewModel(const TSharedPtr<FNiagaraStandaloneScriptViewModel>& InScriptViewModel)
{
	ScriptViewModel = InScriptViewModel;
	VariableObjectSelection = ScriptViewModel->GetVariableSelection();
}

void FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::Cleanup()
{
	ScriptViewModel->GetOnSubscribedParameterDefinitionsChangedDelegate().RemoveAll(this);
}

void FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::Init(const FScriptToolkitUIContext& InUIContext)
{
	UIContext = InUIContext;

	ScriptViewModel->GetOnSubscribedParameterDefinitionsChangedDelegate().AddSP(this, &FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::Refresh);
}

const TArray<UNiagaraGraph*> FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::GetEditableGraphsConst() const
{
	TArray<UNiagaraGraph*> EditableGraphs;
	if (UNiagaraGraph* Graph = ScriptViewModel->GetGraphViewModel()->GetGraph())
	{
		EditableGraphs.Add(Graph);
	}
	return EditableGraphs;
}

const TArray<UNiagaraParameterDefinitions*> FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::GetAvailableParameterDefinitionsAssets(bool bSkipSubscribedParameterDefinitions) const
{
	return ScriptViewModel->GetAvailableParameterDefinitions(bSkipSubscribedParameterDefinitions);
}

const TArray<UNiagaraParameterDefinitions*> FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::GetParameterDefinitionsAssets() const
{
	return ScriptViewModel->GetSubscribedParameterDefinitions();
}

void FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const
{
	FScopedTransaction AddTransaction(Loctext_AddDefinitionsAssetTransaction);
	ScriptViewModel->GetStandaloneScript().Script->Modify();
	ScriptViewModel->SubscribeToParameterDefinitions(NewParameterDefinitions);
	Refresh();
	UIContext.RefreshParameterPanel();
	UIContext.RefreshSelectionDetailsViewPanel();
}

void FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::RemoveParameterDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete) const
{
	FScopedTransaction RemoveTransaction(Loctext_RemoveDefinitionsAssetTransaction);
	ScriptViewModel->GetStandaloneScript().Script->Modify();
	ScriptViewModel->UnsubscribeFromParameterDefinitions(CategoryToDelete.ParameterDefinitionsUniqueId);
	Refresh();
	UIContext.RefreshParameterPanel();
	UIContext.RefreshSelectionDetailsViewPanel();
}

bool FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::GetCanRemoveParameterDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete, FText& OutCanUnsubscribeLibraryToolTip) const
{
	if (CategoryToDelete.ParameterDefinitionsUniqueId.IsValid() == false)
	{
		OutCanUnsubscribeLibraryToolTip = FText::FromString("");
		return false;
	}

	OutCanUnsubscribeLibraryToolTip = Loctext_UnsubscribeFromParameterDefinitionsToolTip;
	return true;
}

const UNiagaraParameterDefinitions* FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::FindSubscribedParameterDefinitionsById(const FGuid& DefinitionsId) const
{
	return ScriptViewModel->FindSubscribedParameterDefinitionsById(DefinitionsId);
}

void FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::SubscribeAllParametersToDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe) const
{
	FScopedTransaction SubscribeAllParametersTransaction(Loctext_SubscribeAllParametersToDefinitionsTransaction);
	ScriptViewModel->GetStandaloneScript().Script->Modify();
	ScriptViewModel->SubscribeAllParametersToDefinitions(CategoryToSubscribe.ParameterDefinitionsUniqueId);

	Refresh();
	UIContext.RefreshParameterPanel();
	UIContext.RefreshSelectionDetailsViewPanel();
}

bool FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::GetCanSubscribeAllParametersToDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe, FText& OutCanSubscribeParametersToolTip) const
{
	const UNiagaraParameterDefinitions* ParameterDefinitions = FindSubscribedParameterDefinitionsById(CategoryToSubscribe.ParameterDefinitionsUniqueId);
	if (ParameterDefinitions == nullptr)
	{
		ensureMsgf(false, TEXT("Tried to set all parameters to link to definition but could not find definition asset by ID!"));
		return false;
	}

	const TArray<UNiagaraScriptVariable*> LibraryScriptVars = ParameterDefinitions->GetParametersConst();
	for (const UNiagaraScriptVariable* ScriptVar : ScriptViewModel->GetAllScriptVars())
	{
		const FNiagaraVariable& CandidateVariable = ScriptVar->Variable;
		if (LibraryScriptVars.ContainsByPredicate([&CandidateVariable](const UNiagaraScriptVariable* LibraryScriptVar){ return LibraryScriptVar->Variable == CandidateVariable; }))
		{
			OutCanSubscribeParametersToolTip = Loctext_SubscribeAllParametersToDefinitionsToolTip;
			return true;
		}
	}

	OutCanSubscribeParametersToolTip = Loctext_CannotSubscribeAllParametersToDefinitionsToolTip_NoCandidates;
	return false;
}

void FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::OnParameterItemSelected(const FNiagaraParameterDefinitionsPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const
{
	for (UNiagaraParameterDefinitions* ParameterDefinitions : GetParameterDefinitionsAssets())
	{ 
		if (UNiagaraScriptVariable* SelectedScriptVariable = ParameterDefinitions->GetScriptVariable(SelectedItem.GetVariable()))
		{
			VariableObjectSelection->SetSelectedObject(SelectedScriptVariable);
			return;
		}
	}
}

FReply FNiagaraScriptToolkitParameterDefinitionsPanelViewModel::OnParameterItemsDragged(const TArray<FNiagaraParameterDefinitionsPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const
{
	if (DraggedItems.Num() == 1)
	{
		const FNiagaraParameterDefinitionsPanelItem& DraggedItem = DraggedItems[0];
		const FNiagaraVariable& DraggedItemVariable = DraggedItem.GetVariable();
		
		return FNiagaraScriptToolkitParameterPanelUtilities::CreateDragEventForParameterItem(
			DraggedItem,
			MouseEvent,
			GetGraphParameterReferencesForItem(DraggedItem),
			nullptr
		);
	}

	return FReply::Handled();
}

///////////////////////////////////////////////////////////////////////////////
/// System Toolkit Parameter Definitions Panel View Model					///
///////////////////////////////////////////////////////////////////////////////

FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::FNiagaraSystemToolkitParameterDefinitionsPanelViewModel(const TSharedPtr<FNiagaraSystemViewModel>& InSystemViewModel)
{
	SystemViewModel = InSystemViewModel;
	SystemGraphSelectionViewModelWeak = SystemViewModel->GetSystemGraphSelectionViewModel();
}

void FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::Cleanup()
{
	SystemViewModel->GetOnSubscribedParameterDefinitionsChangedDelegate().RemoveAll(this);
}

void FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::Init(const FSystemToolkitUIContext& InUIContext)
{
	UIContext = InUIContext;

	SystemViewModel->GetOnSubscribedParameterDefinitionsChangedDelegate().AddSP(this, &FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::Refresh);
}

const TArray<UNiagaraGraph*> FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::GetEditableGraphsConst() const
{
	return SystemViewModel->GetSelectedGraphs();
}

const TArray<UNiagaraParameterDefinitions*> FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::GetAvailableParameterDefinitionsAssets(bool bSkipSubscribedParameterDefinitions) const
{
	return SystemViewModel->GetAvailableParameterDefinitions(bSkipSubscribedParameterDefinitions);
}

const TArray<UNiagaraParameterDefinitions*> FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::GetParameterDefinitionsAssets() const
{
	return SystemViewModel->GetSubscribedParameterDefinitions();
}

void FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const
{
	FScopedTransaction AddTransaction(Loctext_AddDefinitionsAssetTransaction);
	SystemViewModel->SubscribeToParameterDefinitions(NewParameterDefinitions);
	Refresh();
	UIContext.RefreshParameterPanel();
}

void FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::RemoveParameterDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete) const
{
	FScopedTransaction RemoveTransaction(Loctext_RemoveDefinitionsAssetTransaction);
	SystemViewModel->UnsubscribeFromParameterDefinitions(CategoryToDelete.ParameterDefinitionsUniqueId);
	Refresh();
	UIContext.RefreshParameterPanel();
}

bool FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::GetCanRemoveParameterDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete, FText& OutCanUnsubscribeLibraryToolTip) const
{
	if (CategoryToDelete.ParameterDefinitionsUniqueId.IsValid() == false)
	{
		OutCanUnsubscribeLibraryToolTip = FText::FromString("");
		return false;
	}

	OutCanUnsubscribeLibraryToolTip = Loctext_UnsubscribeFromParameterDefinitionsToolTip;
	return true;
}

const UNiagaraParameterDefinitions* FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::FindSubscribedParameterDefinitionsById(const FGuid& DefinitionsId) const
{
	return SystemViewModel->FindSubscribedParameterDefinitionsById(DefinitionsId);
}

void FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::SubscribeAllParametersToDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe) const
{
	FScopedTransaction SubscribeAllParametersTransaction(Loctext_SubscribeAllParametersToDefinitionsTransaction);
	SystemViewModel->GetSystem().Modify();
	SystemViewModel->SubscribeAllParametersToDefinitions(CategoryToSubscribe.ParameterDefinitionsUniqueId);
	
	Refresh();
	UIContext.RefreshParameterPanel();
}

bool FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::GetCanSubscribeAllParametersToDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe, FText& OutCanSubscribeParametersToolTip) const
{
	const UNiagaraParameterDefinitions* ParameterDefinitions = FindSubscribedParameterDefinitionsById(CategoryToSubscribe.ParameterDefinitionsUniqueId);
	if (ParameterDefinitions == nullptr)
	{
		ensureMsgf(false, TEXT("Tried to set all parameters to subscribe to definition but could not find definition asset by ID!"));
		return false;
	}

	const TArray<UNiagaraScriptVariable*> LibraryScriptVars = ParameterDefinitions->GetParametersConst();
	for (const UNiagaraScriptVariable* ScriptVar : SystemViewModel->GetAllScriptVars())
	{
		const FNiagaraVariable& CandidateVariable = ScriptVar->Variable;
		if (LibraryScriptVars.ContainsByPredicate([&CandidateVariable](const UNiagaraScriptVariable* LibraryScriptVar) { return LibraryScriptVar->Variable == CandidateVariable; }))
		{
			OutCanSubscribeParametersToolTip = Loctext_SubscribeAllParametersToDefinitionsToolTip;
			return true;
		}
	}

	OutCanSubscribeParametersToolTip = Loctext_CannotSubscribeAllParametersToDefinitionsToolTip_NoCandidates;
	return false;
}

FReply FNiagaraSystemToolkitParameterDefinitionsPanelViewModel::OnParameterItemsDragged(const TArray<FNiagaraParameterDefinitionsPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const
{
	if (DraggedItems.Num() == 1)
	{
		const FNiagaraParameterDefinitionsPanelItem& DraggedItem = DraggedItems[0];
		return FNiagaraSystemToolkitParameterPanelUtilities::CreateDragEventForParameterItem(
			DraggedItem,
			MouseEvent,
			GetGraphParameterReferencesForItem(DraggedItem),
			nullptr
		);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE // "FNiagaraParameterDefinitionsPanelViewModel"
