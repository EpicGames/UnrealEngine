// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraParameterDefinitionsSubscriberViewModel.h"

#include "Misc/Paths.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorData.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptSourceBase.h"
#include "NiagaraScriptVariable.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"


void INiagaraParameterDefinitionsSubscriberViewModel::SubscribeToParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions)
{
	GetParameterDefinitionsSubscriber()->SubscribeToParameterDefinitions(NewParameterDefinitions);
}

void INiagaraParameterDefinitionsSubscriberViewModel::UnsubscribeFromParameterDefinitions(const FGuid& ParameterDefinitionsToRemoveUniqueId)
{
	UNiagaraParameterDefinitions* DefinitionsToRemove = FindSubscribedParameterDefinitionsById(ParameterDefinitionsToRemoveUniqueId);
	if (DefinitionsToRemove != nullptr)
	{
		DefinitionsToRemove->GetOnParameterDefinitionsChanged().RemoveAll(this);
	}

	// Call UnsubscribeFromParameterDefinitions() even if they could not be found to use assert logic there.
	GetParameterDefinitionsSubscriber()->UnsubscribeFromParameterDefinitions(ParameterDefinitionsToRemoveUniqueId);
}

void INiagaraParameterDefinitionsSubscriberViewModel::SynchronizeWithParameterDefinitions(FSynchronizeWithParameterDefinitionsArgs Args /*= FSynchronizeWithParameterDefinitionsArgs()*/)
{
	GetParameterDefinitionsSubscriber()->SynchronizeWithParameterDefinitions(Args);
}

void INiagaraParameterDefinitionsSubscriberViewModel::SynchronizeScriptVarWithParameterDefinitions(UNiagaraScriptVariable* ScriptVarToSynchronize, bool bForce)
{
	FSynchronizeWithParameterDefinitionsArgs Args = FSynchronizeWithParameterDefinitionsArgs();
	Args.SpecificDestScriptVarIds.Add(ScriptVarToSynchronize->Metadata.GetVariableGuid());
	Args.bForceGatherDefinitions = bForce;
	Args.bForceSynchronizeParameters = bForce;
	SynchronizeWithParameterDefinitions(Args); //@todo(ng) error if we failed to find the var
}

void INiagaraParameterDefinitionsSubscriberViewModel::SubscribeAllParametersToDefinitions(const FGuid& DefinitionsUniqueId)
{
	const UNiagaraParameterDefinitions* LibraryToSynchronize = FindSubscribedParameterDefinitionsById(DefinitionsUniqueId);
	if (LibraryToSynchronize == nullptr)
	{
		ensureMsgf(false, TEXT("Tried to link all name matching parameters to definition but failed to find definition asset by ID!"));
		return;
	}

	const TArray<UNiagaraScriptVariable*>& LibraryScriptVars = LibraryToSynchronize->GetParametersConst();

	// Iterate each UNiagaraScriptVariable and find Library script vars with name matches to synchronize.
	bool bNeedSynchronize = false;
	for (UNiagaraScriptVariable* DestScriptVar : GetAllScriptVars())
	{
		// If the parameter is already subscribed, continue.
		if (DestScriptVar->GetIsSubscribedToParameterDefinitions())
		{
			continue;
		}

		const FName& DestScriptVarName = DestScriptVar->Variable.GetName();
		if (UNiagaraScriptVariable* const* LibraryScriptVarPtr = LibraryScriptVars.FindByPredicate([&DestScriptVarName](const UNiagaraScriptVariable* LibraryScriptVar) { return LibraryScriptVar->Variable.GetName() == DestScriptVarName; }))
		{
			// We found a library (source) script var with a name match. Mark the destination script var as synchronizing with a parameter definitions and overwrite the metadata by hand so that the unique ID guid matches with its source.
			UNiagaraScriptVariable* LibraryScriptVar = *LibraryScriptVarPtr;
			DestScriptVar->Modify();
			DestScriptVar->SetIsSubscribedToParameterDefinitions(true);
			DestScriptVar->Metadata.SetVariableGuid(LibraryScriptVar->Metadata.GetVariableGuid());
			bNeedSynchronize = true;
		}
	}

	if (bNeedSynchronize)
	{
		SynchronizeWithParameterDefinitions();
	}
}

void INiagaraParameterDefinitionsSubscriberViewModel::SetParameterIsSubscribedToDefinitions(const FGuid& ScriptVarId, bool bIsSynchronizing)
{
	// Iterate each UNiagaraScriptVariable and find Library script vars with id matches to synchronize.
	TArray<UNiagaraScriptVariable*> ScriptVars = GetAllScriptVars();

	UNiagaraScriptVariable* const* ScriptVarPtr = ScriptVars.FindByPredicate([&ScriptVarId](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Metadata.GetVariableGuid() == ScriptVarId; });
	if (ScriptVarPtr == nullptr)
	{
		ensureMsgf(false, TEXT("Tried to link parameter to definition but failed to find parameter by Id!"));
		return;
	}
	UNiagaraScriptVariable* ScriptVar = *ScriptVarPtr;

	auto TrySynchronizeScriptVarToLibraries = [this](UNiagaraScriptVariable* ScriptVar, TArray<UNiagaraParameterDefinitions*> ParameterDefinitions)->UNiagaraParameterDefinitions* /*FoundLibrary*/ {
		for (UNiagaraParameterDefinitions* ParameterDefinitionsItr : ParameterDefinitions)
		{
			for (const UNiagaraScriptVariable* LibraryScriptVar : ParameterDefinitionsItr->GetParametersConst())
			{
				if (LibraryScriptVar->Variable == ScriptVar->Variable)
				{	
					ScriptVar->Modify();
					ScriptVar->SetIsSubscribedToParameterDefinitions(true);
					ScriptVar->SetIsOverridingParameterDefinitionsDefaultValue(false);
					ScriptVar->Metadata.SetVariableGuid(LibraryScriptVar->Metadata.GetVariableGuid());
					ScriptVar->Metadata.Description = LibraryScriptVar->Metadata.Description;

					// The parameter may be in a UNiagaraEditorParametersAdapter; Only apply changes to the underlying graph is there is one to apply to.
					if (ScriptVar->GetOuter()->IsA<UNiagaraGraph>())
					{
						FNiagaraStackGraphUtilities::SynchronizeVariableToLibraryAndApplyToGraph(ScriptVar);
					}
					return ParameterDefinitionsItr;
				}
			}
		}
		return nullptr;
	};

	if (bIsSynchronizing == false)
	{
		ScriptVar->Modify();
		ScriptVar->SetIsSubscribedToParameterDefinitions(false);
	}
	else /*bIsSynchronizing == true*/
	{
		const bool bSkipSubscribed = true;
		if (const UNiagaraParameterDefinitions* FoundSubscribedLibrary = TrySynchronizeScriptVarToLibraries(ScriptVar, GetSubscribedParameterDefinitions()))
		{
			return;
		}
		else if (UNiagaraParameterDefinitions* FoundAvailableLibrary = TrySynchronizeScriptVarToLibraries(ScriptVar, GetAvailableParameterDefinitions(bSkipSubscribed)))
		{
			SubscribeToParameterDefinitions(FoundAvailableLibrary);
			return;
		}

		ensureMsgf(false, TEXT("Tried to set parameter to synchronize with parameter definitions but definition parameter could not be found! Definition cache out of date!"));
	}
}

void INiagaraParameterDefinitionsSubscriberViewModel::SetParameterIsOverridingLibraryDefaultValue(const FGuid& ScriptVarId, bool bIsOverriding)
{
	// Iterate each UNiagaraScriptVariable and find Library script vars with id matches to synchronize.
	TArray<UNiagaraScriptVariable*> ScriptVars = GetAllScriptVars();

	UNiagaraScriptVariable* const* ScriptVarPtr = ScriptVars.FindByPredicate([&ScriptVarId](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Metadata.GetVariableGuid() == ScriptVarId; });
	if (ScriptVarPtr == nullptr)
	{
		ensureMsgf(false, TEXT("Tried to link parameter to definition but failed to find parameter by Id!"));
		return;
	}
	UNiagaraScriptVariable* ScriptVar = *ScriptVarPtr;

	if (ScriptVar->GetIsSubscribedToParameterDefinitions() == false)
	{
		ensureMsgf(false, TEXT("Tried to set parameter to override definition default value but it was linked to a definition!"));
		return;
	}
	
	ScriptVar->SetIsOverridingParameterDefinitionsDefaultValue(bIsOverriding);
	if (bIsOverriding == false)
	{
		FNiagaraStackGraphUtilities::SynchronizeVariableToLibraryAndApplyToGraph(ScriptVar);
	}
}

TArray<UNiagaraParameterDefinitions*> INiagaraParameterDefinitionsSubscriberViewModel::GetSubscribedParameterDefinitions()
{
	return FNiagaraEditorUtilities::DowncastParameterDefinitionsBaseArray(GetParameterDefinitionsSubscriber()->GetSubscribedParameterDefinitions());
}

TArray<UNiagaraParameterDefinitions*> INiagaraParameterDefinitionsSubscriberViewModel::GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions)
{
	TArray<UNiagaraParameterDefinitions*> OutParameterDefinitions = FNiagaraEditorUtilities::GetAllParameterDefinitions();
	if (bSkipSubscribedParameterDefinitions)
	{
		const TArray<FParameterDefinitionsSubscription>& Subscriptions = GetParameterDefinitionsSubscriber()->GetParameterDefinitionsSubscriptions();
		for (int32 Idx = OutParameterDefinitions.Num() - 1; Idx > -1; --Idx)
		{
			const UNiagaraParameterDefinitions* Definition = OutParameterDefinitions[Idx];
			if (Subscriptions.ContainsByPredicate([Definition](const FParameterDefinitionsSubscription& Subscription) { return Subscription.Definitions == Definition; }))
			{
				OutParameterDefinitions.RemoveAtSwap(Idx);
			}
		}
	}
	return OutParameterDefinitions;
}

TArray<UNiagaraScriptVariable*> INiagaraParameterDefinitionsSubscriberViewModel::GetAllScriptVars()
{
	TArray<UNiagaraScriptVariable*> OutScriptVars;
	for (UNiagaraScriptSourceBase* SourceScript : GetParameterDefinitionsSubscriber()->GetAllSourceScripts())
	{
		TArray<TObjectPtr<UNiagaraScriptVariable>> TempScriptVars;
		CastChecked<UNiagaraScriptSource>(SourceScript)->NodeGraph->GetAllMetaData().GenerateValueArray(TempScriptVars);
		OutScriptVars.Append(TempScriptVars);
	}
	for (UNiagaraEditorParametersAdapterBase* EditorOnlyParametersAdapter : GetParameterDefinitionsSubscriber()->GetEditorOnlyParametersAdapters())
	{
		OutScriptVars.Append(CastChecked<UNiagaraEditorParametersAdapter>(EditorOnlyParametersAdapter)->GetParameters());
	}

	return OutScriptVars;
}

UNiagaraParameterDefinitions* INiagaraParameterDefinitionsSubscriberViewModel::FindSubscribedParameterDefinitionsById(const FGuid& LibraryId)
{
	UNiagaraParameterDefinitionsBase* FoundParameterDefinitions = GetParameterDefinitionsSubscriber()->FindSubscribedParameterDefinitionsById(LibraryId);
	if (FoundParameterDefinitions != nullptr)
	{
		return CastChecked<UNiagaraParameterDefinitions>(FoundParameterDefinitions);
	}
	return nullptr;
}

UNiagaraScriptVariable* INiagaraParameterDefinitionsSubscriberViewModel::FindScriptVarById(const FGuid& ScriptVarId)
{
	UNiagaraScriptVariable* const* ScriptVarPtr = GetAllScriptVars().FindByPredicate([&ScriptVarId](const UNiagaraScriptVariable* ScriptVar){ return ScriptVar->Metadata.GetVariableGuid() == ScriptVarId; });
	return ScriptVarPtr != nullptr ? *ScriptVarPtr : nullptr;
}

UNiagaraScriptVariable* INiagaraParameterDefinitionsSubscriberViewModel::FindSubscribedParameterDefinitionsScriptVarByName(const FName& ScriptVarName)
{
	for (UNiagaraParameterDefinitions* ParameterDefinitions : GetSubscribedParameterDefinitions())
	{
		UNiagaraScriptVariable* const* FoundScriptVarPtr = ParameterDefinitions->GetParametersConst().FindByPredicate([ScriptVarName](const UNiagaraScriptVariable* ScriptVar){ return ScriptVar->Variable.GetName() == ScriptVarName; });
		if (FoundScriptVarPtr != nullptr)
		{
			return *FoundScriptVarPtr;
		}
	}
	return nullptr;
}

FOnSubscribedParameterDefinitionsChanged& INiagaraParameterDefinitionsSubscriberViewModel::GetOnSubscribedParameterDefinitionsChangedDelegate()
{
	return GetParameterDefinitionsSubscriber()->GetOnSubscribedParameterDefinitionsChangedDelegate();
}

FString INiagaraParameterDefinitionsSubscriberViewModel::GetSourceObjectPackagePathName()
{
	return FPaths::GetPath(GetParameterDefinitionsSubscriber()->GetSourceObjectPathName());
}
