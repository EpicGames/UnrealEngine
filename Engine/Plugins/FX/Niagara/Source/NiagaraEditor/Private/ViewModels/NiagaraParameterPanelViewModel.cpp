// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraParameterPanelViewModel.h"

#include "EdGraphSchema_Niagara.h"
#include "NiagaraActions.h"
#include "NiagaraClipboard.h"
#include "NiagaraConstants.h"
#include "NiagaraDataInterfaceUtilities.h"
#include "NiagaraEditorData.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraScriptGraphViewModel.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemScriptViewModel.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraTypes.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraSystemEditorDocumentsViewModel.h"
#include "ViewModels/NiagaraSystemGraphSelectionViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/HierarchyEditor/NiagaraUserParametersHierarchyViewModel.h"
#include "Widgets/SNiagaraDebugger.h"
#include "Widgets/SNiagaraParameterMenu.h"
#include "Widgets/SNiagaraParameterPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "Widgets/SNullWidget.h"
#define LOCTEXT_NAMESPACE "NiagaraParameterPanelViewModel"

template<> TMap<UNiagaraScript*, TArray<FNiagaraScriptToolkitParameterPanelViewModel*>> TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptToolkitParameterPanelViewModel>::ObjectsToViewModels{};
template<> TMap<UNiagaraSystem*, TArray<FNiagaraSystemToolkitParameterPanelViewModel*>> TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemToolkitParameterPanelViewModel>::ObjectsToViewModels{};


TArray<FNiagaraParameterPanelCategory> FNiagaraSystemToolkitParameterPanelViewModel::DefaultCategories;
TArray<FNiagaraParameterPanelCategory> FNiagaraSystemToolkitParameterPanelViewModel::DefaultAdvancedCategories;
TArray<FNiagaraParameterPanelCategory> FNiagaraSystemToolkitParameterPanelViewModel::UserCategories;
TArray<FNiagaraParameterPanelCategory> FNiagaraSystemToolkitParameterPanelViewModel::DefaultScriptCategories;
TArray<FNiagaraParameterPanelCategory> FNiagaraSystemToolkitParameterPanelViewModel::DefaultAdvancedScriptCategories;

TArray<FNiagaraParameterPanelCategory> FNiagaraScriptToolkitParameterPanelViewModel::DefaultCategories;
TArray<FNiagaraParameterPanelCategory> FNiagaraScriptToolkitParameterPanelViewModel::DefaultAdvancedCategories;
TArray<FNiagaraParameterPanelCategory> FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::DefaultCategories;

///////////////////////////////////////////////////////////////////////////////
/// System Toolkit Parameter Panel Utilities								///
///////////////////////////////////////////////////////////////////////////////

FReply FNiagaraSystemToolkitParameterPanelUtilities::CreateDragEventForParameterItem(const FNiagaraParameterPanelItemBase& DraggedItem, const FPointerEvent& MouseEvent, const TArray<FNiagaraGraphParameterReference>& GraphParameterReferencesForItem, const TSharedPtr<TArray<FName>>& ParametersWithNamespaceModifierRenamePending)
{
	const static FText TooltipFormat = LOCTEXT("Parameters", "Name: {0} \nType: {1}");

	//@todo(ng) refactor drag action to not carry around the reference collection; graph ptr goes unused.
	FNiagaraGraphParameterReferenceCollection ReferenceCollection = FNiagaraGraphParameterReferenceCollection(true);
	ReferenceCollection.ParameterReferences = GraphParameterReferencesForItem;
	const TArray<FNiagaraGraphParameterReferenceCollection> ReferenceCollectionArray = { ReferenceCollection };
	const FText Name = FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(DraggedItem.GetVariable().GetName());
	const FText ToolTip = FText::Format(TooltipFormat, Name, DraggedItem.GetVariable().GetType().GetNameText());
	TSharedPtr<FEdGraphSchemaAction> ItemDragAction = MakeShared<FNiagaraParameterAction>(DraggedItem.ScriptVariable, FText::GetEmpty(), Name, ToolTip, 0, FText(), 0/*SectionID*/);
	TSharedPtr<FNiagaraParameterDragOperation> DragOperation = MakeShared<FNiagaraParameterDragOperation>(ItemDragAction);
	DragOperation->SetupDefaults();
	DragOperation->Construct();
	return FReply::Handled().BeginDragDrop(DragOperation.ToSharedRef());
}


///////////////////////////////////////////////////////////////////////////////
/// Script Toolkit Parameter Panel Utilities								///
///////////////////////////////////////////////////////////////////////////////

FReply FNiagaraScriptToolkitParameterPanelUtilities::CreateDragEventForParameterItem(const FNiagaraParameterPanelItemBase& DraggedItem, const FPointerEvent& MouseEvent, const TArray<FNiagaraGraphParameterReference>& GraphParameterReferencesForItem, const TSharedPtr<TArray<FName>>& ParametersWithNamespaceModifierRenamePending)
{
	const static FText TooltipFormat = LOCTEXT("Parameters", "Name: {0} \nType: {1}");

	//@todo(ng) refactor drag action to not carry around the reference collection; graph ptr goes unused.
	FNiagaraGraphParameterReferenceCollection ReferenceCollection = FNiagaraGraphParameterReferenceCollection(true);
	ReferenceCollection.ParameterReferences = GraphParameterReferencesForItem;
	const TArray<FNiagaraGraphParameterReferenceCollection> ReferenceCollectionArray = { ReferenceCollection };
	const FText Name = FNiagaraParameterUtilities::FormatParameterNameForTextDisplay(DraggedItem.GetVariable().GetName());
	const FText ToolTip = FText::Format(TooltipFormat, Name, DraggedItem.GetVariable().GetType().GetNameText());
	TSharedPtr<FEdGraphSchemaAction> ItemDragAction = MakeShared<FNiagaraParameterAction>(DraggedItem.ScriptVariable, FText::GetEmpty(), Name, ToolTip, 0, FText(), 0/*SectionID*/);
	TSharedPtr<FNiagaraParameterGraphDragOperation> DragOperation = FNiagaraParameterGraphDragOperation::New(ItemDragAction);
	DragOperation->SetAltDrag(MouseEvent.IsAltDown());
	DragOperation->SetCtrlDrag(MouseEvent.IsLeftControlDown() || MouseEvent.IsRightControlDown());
	return FReply::Handled().BeginDragDrop(DragOperation.ToSharedRef());
}

///////////////////////////////////////////////////////////////////////////////
/// Generic Parameter Panel Utilities										///
///////////////////////////////////////////////////////////////////////////////

bool FNiagaraParameterPanelUtilities::GetCanSetParameterNamespaceAndToolTipForScriptOrSystem(const FNiagaraParameterPanelItem& ItemToModify, const FName NewNamespace, FText& OutCanSetParameterNamespaceToolTip)
{
	if (ItemToModify.ScriptVariable->GetIsSubscribedToParameterDefinitions())
	{
		OutCanSetParameterNamespaceToolTip = LOCTEXT("ParameterPanelViewModel_ChangeParameterNamespace_ParameterDefinitions", "Cannot change Parameter namespace: Parameters linked to Parameter Definitions may only have their namespace changed in the source Parameter Definitions asset.");
		return false;
	}
	else if (ItemToModify.bExternallyReferenced)
	{
		OutCanSetParameterNamespaceToolTip = LOCTEXT("ParameterPanelViewModel_ChangeParameterNamespace_ExternallyReferenced", "Cannot change Parameter namespace: Parameter is from an externally referenced script and can't be directly edited.");
		return false;
	}
	return true;
}

bool FNiagaraParameterPanelUtilities::GetCanSetParameterNamespaceModifierAndToolTipForScriptOrSystem(const TArray<FNiagaraParameterPanelItem>& CachedViewedItems, const FNiagaraParameterPanelItem& ItemToModify, const FName NamespaceModifier, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip)
{
	if (FNiagaraParameterUtilities::TestCanSetSpecificNamespaceModifierWithMessage(ItemToModify.GetVariable().GetName(), NamespaceModifier, OutCanSetParameterNamespaceModifierToolTip) == false)
	{
		return false;
	}

	if (bDuplicateParameter == false)
	{
		if (ItemToModify.ScriptVariable->GetIsSubscribedToParameterDefinitions())
		{
			OutCanSetParameterNamespaceModifierToolTip = LOCTEXT("ParameterPanelViewModel_ChangeParameterNamespaceModifier_ParameterDefinitions", "Cannot change Parameter namespace modifier: Parameters from Parameter Definitions may only have their namespace modifier changed in the source Parameter Definitions asset.");
			return false;
		}
		else if (ItemToModify.bExternallyReferenced)
		{
			OutCanSetParameterNamespaceModifierToolTip = LOCTEXT("CantChangeNamespaceModifierExternallyReferenced", "Parameter is from an externally referenced script and can't be directly edited.");
			return false;
		}
		else if (NamespaceModifier != NAME_None)
		{
			FName NewName = FNiagaraParameterUtilities::SetSpecificNamespaceModifier(ItemToModify.GetVariable().GetName(), NamespaceModifier);
			if (CachedViewedItems.ContainsByPredicate([NewName](const FNiagaraParameterPanelItem& CachedViewedItem) {return CachedViewedItem.GetVariable().GetName() == NewName; }))
			{
				OutCanSetParameterNamespaceModifierToolTip = LOCTEXT("CantChangeNamespaceModifierAlreadyExits", "Can't set this namespace modifier because it would create a parameter that already exists.");
				return false;
			}
		}
	}

	return true;
}

bool FNiagaraParameterPanelUtilities::GetCanSetParameterCustomNamespaceModifierAndToolTipForScriptOrSystem(const FNiagaraParameterPanelItem& ItemToModify, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip)
{
	if (FNiagaraParameterUtilities::TestCanSetCustomNamespaceModifierWithMessage(ItemToModify.GetVariable().GetName(), OutCanSetParameterNamespaceModifierToolTip) == false)
	{
		return false;
	}

	if (bDuplicateParameter == false) 
	{
		if (ItemToModify.bExternallyReferenced)
		{
			OutCanSetParameterNamespaceModifierToolTip = LOCTEXT("CantChangeNamespaceModifierExternallyReferenced", "Parameter is from an externally referenced script and can't be directly edited.");
			return false;
		}
		else if (ItemToModify.ScriptVariable->GetIsSubscribedToParameterDefinitions())
		{
			OutCanSetParameterNamespaceModifierToolTip = LOCTEXT("ParameterPanelViewModel_ChangeParameterCustomNamespace_ParameterDefinitions", "Cannot set Parameter custom namespace: Parameters linked to Parameter Definitions may only have a custom namespace set in the source Parameter Definitions asset.");
			return false;
		}
	}

	return true;
}

///////////////////////////////////////////////////////////////////////////////
/// Immutable Parameter Panel View Model									///
///////////////////////////////////////////////////////////////////////////////

void INiagaraImmutableParameterPanelViewModel::RefreshFull(bool bDoCategoryExpansion) const
{
	OnRequestRefreshDelegate.ExecuteIfBound(bDoCategoryExpansion);
}

void INiagaraImmutableParameterPanelViewModel::RefreshFullNextTick(bool bDoCategoryExpansion) const
{
	OnRequestRefreshNextTickDelegate.ExecuteIfBound(bDoCategoryExpansion);
}

void INiagaraImmutableParameterPanelViewModel::PostUndo(bool bSuccess)
{
	Refresh();
}

void INiagaraImmutableParameterPanelViewModel::CopyParameterReference(const FNiagaraParameterPanelItemBase& ItemToCopy) const
{
	FPlatformApplicationMisc::ClipboardCopy(*ItemToCopy.GetVariable().GetName().ToString());
}

bool INiagaraImmutableParameterPanelViewModel::GetCanCopyParameterReferenceAndToolTip(const FNiagaraParameterPanelItemBase& ItemToCopy, FText& OutCanCopyParameterToolTip) const
{
	OutCanCopyParameterToolTip = LOCTEXT("CopyReferenceToolTip", "Copy a string reference for this parameter to the clipboard.\nThis reference can be used in expressions and custom HLSL nodes.");
	return true;
}

void INiagaraImmutableParameterPanelViewModel::CopyParameterMetaData(const FNiagaraParameterPanelItemBase ItemToCopy) const
{
	const TArray<UNiagaraScriptVariable*> ScriptVariables = GetEditableScriptVariablesWithName(ItemToCopy.GetVariable().GetName());
	if (ScriptVariables.Num() > 0 && ScriptVariables[0] != nullptr)
	{
		UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
		ClipboardContent->ScriptVariables.Add({*ScriptVariables[0]});
		FNiagaraEditorModule::Get().GetClipboard().SetClipboardContent(ClipboardContent);
	}
}

bool INiagaraImmutableParameterPanelViewModel::GetCanCopyParameterMetaDataAndToolTip(const FNiagaraParameterPanelItemBase& ItemToCopy, FText& OutCanCopyToolTip) const
{
	if (GetEditableScriptVariablesWithName(ItemToCopy.GetVariable().GetName()).Num() > 0)
	{
		return true;
	}
	return false;
}


///////////////////////////////////////////////////////////////////////////////
///	Parameter Panel View Model												///
///////////////////////////////////////////////////////////////////////////////

INiagaraParameterPanelViewModel::INiagaraParameterPanelViewModel()
{
	ActiveSectionIndex = -1;
}

INiagaraParameterPanelViewModel::~INiagaraParameterPanelViewModel()
{
	// Clean up transient UNiagaraScriptVariables used as intermediate representations.
	for (auto It = TransientParameterToScriptVarMap.CreateIterator(); It; ++It)
	{
		UNiagaraScriptVariable* ScriptVar = It.Value();
		ScriptVar->RemoveFromRoot();
		ScriptVar = nullptr;
	}
}

void INiagaraParameterPanelViewModel::RefreshFull(bool bDoCategoryExpansion) const
{
	INiagaraImmutableParameterPanelViewModel::RefreshFull(bDoCategoryExpansion);

	TSharedPtr<INiagaraParameterPanelViewModel> MainVM = MainParameterPanelViewModel.Pin();
	if (MainVM.IsValid())
	{
		MainVM->RefreshFull(bDoCategoryExpansion);
	}
}

bool INiagaraParameterPanelViewModel::CanAddType(const FNiagaraTypeDefinition& InType, TOptional<FNiagaraParameterPanelCategory> Category)
{
	return !(
		InType == FNiagaraTypeDefinition::GetParameterMapDef()
		|| InType == FNiagaraTypeDefinition::GetGenericNumericDef()
		|| InType.IsInternalType()
		);
}

bool INiagaraParameterPanelViewModel::GetCanDeleteParameterAndToolTip(const FNiagaraParameterPanelItem& ItemToDelete, FText& OutCanDeleteParameterToolTip) const
{
	if (ItemToDelete.bExternallyReferenced)
	{
		//@todo(ng) revise loctexts
		OutCanDeleteParameterToolTip = LOCTEXT("CantDeleteSelected_External", "This parameter is referenced in an external script and cannot be deleted.");
		return false;
	}
	OutCanDeleteParameterToolTip = LOCTEXT("DeleteSelected", "Delete the selected parameter.");
	return true;
}

void INiagaraParameterPanelViewModel::PasteParameterMetaData(const TArray<FNiagaraParameterPanelItem> SelectedItems)
{
	const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent();
	if (ClipboardContent == nullptr || ClipboardContent->ScriptVariables.Num() != 1 )
	{
		return;
	}

	TArray<UNiagaraScriptVariable*> TargetScriptVariables;
	for (const FNiagaraParameterPanelItem& Item : SelectedItems)
	{
		
		TargetScriptVariables.Append(GetEditableScriptVariablesWithName(Item.GetVariable().GetName()));
	}

	if (TargetScriptVariables.Num() > 0)
	{
		FScopedTransaction PasteMetadataTransaction(LOCTEXT("PasteMetadataTransaction", "Paste parameter metadata"));
		for (UNiagaraScriptVariable* TargetScriptVariable : TargetScriptVariables)
		{
			TargetScriptVariable->Modify();
			TargetScriptVariable->Metadata.CopyUserEditableMetaData(ClipboardContent->ScriptVariables[0].ScriptVariable->Metadata);
			TargetScriptVariable->PostEditChange();
		}
	}
}

bool INiagaraParameterPanelViewModel::GetCanPasteParameterMetaDataAndToolTip(FText& OutCanPasteToolTip)
{
	const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent();
	if (ClipboardContent == nullptr || ClipboardContent->ScriptVariables.Num() != 1)
	{
		OutCanPasteToolTip = LOCTEXT("CantPasteMetaDataToolTip_Invalid", "Cannot Paste: There is not any parameter metadata to paste in the clipboard.");
		return false;
	}
	OutCanPasteToolTip = LOCTEXT("PasteMetaDataToolTip", "Paste the parameter metadata from the system clipboard to the selected parameters.");
	return true;
}

void INiagaraParameterPanelViewModel::DuplicateParameters(const TArray<FNiagaraParameterPanelItem> ItemsToDuplicate) 
{
	bool bAnyChange = false;
	FScopedTransaction Transaction(LOCTEXT("DuplicateParameter(s)Transaction", "Duplicate parameter(s)"));

	for(const FNiagaraParameterPanelItem& ItemToDuplicate : ItemsToDuplicate)
	{
		TSet<FName> ParameterNames;
		for (const FNiagaraParameterPanelItem& CachedViewedItem : CachedViewedItems)
		{
			ParameterNames.Add(CachedViewedItem.GetVariable().GetName());
		}
		
		const FName NewUniqueName = FNiagaraUtilities::GetUniqueName(ItemToDuplicate.GetVariable().GetName(), ParameterNames);
		const bool bRequestRename = true;
		const bool bMakeUniqueName = true;
		AddParameter(FNiagaraVariable(ItemToDuplicate.GetVariable().GetType(), NewUniqueName), FNiagaraParameterPanelCategory(ItemToDuplicate.NamespaceMetaData), bRequestRename, bMakeUniqueName);
		bAnyChange = true;
	}

	if(!bAnyChange)
	{
		Transaction.Cancel();
	}
		
}

bool INiagaraParameterPanelViewModel::GetCanDebugParameters(const TArray<FNiagaraParameterPanelItem>& ItemsToDebug) const
{
	for (const FNiagaraParameterPanelItem& ItemToDebug : ItemsToDebug)
	{
		const FNiagaraVariableBase& Var = ItemToDebug.GetVariable();
		if (Var.IsInNameSpace(FNiagaraConstants::UserNamespaceString) ||
			Var.IsInNameSpace(FNiagaraConstants::SystemNamespaceString) ||
			Var.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString) ||
			Var.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString))
		{
			return true;
		}
	}
	return false;
}

bool INiagaraParameterPanelViewModel::GetCanDuplicateParameterAndToolTip(const TArray<FNiagaraParameterPanelItem>& ItemsToDuplicate, FText& OutCanDuplicateParameterToolTip) const
{
	for(const FNiagaraParameterPanelItem& ItemToDuplicate : ItemsToDuplicate)
	{
		if (ItemToDuplicate.NamespaceMetaData.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingName))
		{
			OutCanDuplicateParameterToolTip = LOCTEXT("ParameterPanelViewModel_DuplicateParameter_PreventEditingName", "A parameter can not be duplicated because it does not support editing its name.");
			return false;
		}
	}
	
	OutCanDuplicateParameterToolTip = LOCTEXT("ParameterPanellViewModel_DuplicateParameter", "Create new parameters with the same types as the selected parameters.");
	return true;
}

bool INiagaraParameterPanelViewModel::GetCanRenameParameterAndToolTip(const FNiagaraParameterPanelItem& ItemToRename, const FText& NewVariableNameText, bool bCheckEmptyNameText, FText& OutCanRenameParameterToolTip) const
{
	if (ItemToRename.ScriptVariable->GetIsSubscribedToParameterDefinitions())
	{
		OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelViewModel_RenameParameter_ParameterDefinition", "Cannot rename Parameter: Parameters linked to Parameter Definitions may only be renamed in the source Parameter Definitions asset.");
		return false;
	}
	else if (ItemToRename.bExternallyReferenced)
	{
		OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelViewModel_RenameParameter_ExternallyReferenced", "Cannot rename Parameter: Parameter is from an externally referenced script and can't be directly edited.");
		return false;
	}
	else if (ItemToRename.NamespaceMetaData.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingName))
	{
		OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelViewModel_RenameParameter_NamespaceMetaData", "Cannot rename Parameter: The namespace of this Parameter does not allow renaming.");
		return false;
	}

	const FString NewVariableNameString = NewVariableNameText.ToString();
	if(ItemToRename.GetVariable().GetName().ToString() != NewVariableNameString)
	{ 
		if (CachedViewedItems.ContainsByPredicate([NewVariableNameString](const FNiagaraParameterPanelItem& Item) {return Item.GetVariable().GetName().ToString() == NewVariableNameString; }))
		{
			OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelViewModel_RenameParameter_NameAlias", "Cannot Rename Parameter: A Parameter with this name already exists.");
			return false;
		}
	}

	if (bCheckEmptyNameText && NewVariableNameText.IsEmptyOrWhitespace())
	{
		// The incoming name text will contain the namespace even if the parameter name entry is empty, so make a parameter handle to split out the name.
		const FNiagaraParameterHandle NewVariableNameHandle = FNiagaraParameterHandle(FName(*NewVariableNameString));
		if (NewVariableNameHandle.GetName().IsNone())
		{
			OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelViewModel_RenameParameter_NameNone", "Parameter must have a name.");
			return false;
		}
	}

	OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelViewModel_RenameParameter_CreatedInSystem", "Rename this Parameter and all usages in the System and Emitters.");
	return true;
}

bool INiagaraParameterPanelViewModel::GetCanSubscribeParameterToLibraryAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const bool bSubscribing, FText& OutCanSubscribeParameterToolTip) const
{
	if (ItemToModify.bExternallyReferenced)
	{
		OutCanSubscribeParameterToolTip = LOCTEXT("ParameterPanelViewModel_CannotSubscribeParameterToDefinition_ExternallyReferenced", "Cannot subscribe Parameter to Parameter Definitions: Parameter is from and externally referenced script and cannot be directly edited.");
		return false;
	}
	else if (ItemToModify.DefinitionMatchState == EParameterDefinitionMatchState::NoMatchingDefinitions)
	{
		OutCanSubscribeParameterToolTip = LOCTEXT("ParameterPanelViewModel_CannotSubscribeParameterToDefinition_NotNameAliased", "Cannot subscribe Parameter to Parameter Definitions: Parameter name does not match any names in available Parameter Definitions.");
		return false;
	}
	else if (ItemToModify.DefinitionMatchState == EParameterDefinitionMatchState::MatchingDefinitionNameButNotType)
	{
		OutCanSubscribeParameterToolTip = LOCTEXT("ParameterPanelViewModel_CannotSubscribeParameterToDefinition_TypeMismatch", "Cannot subscribe Parameter to Parameter Definitions: Parameter name matches Parameter Definition, but Parameter type does not match Parameter Definition.");
		return false;
	}
	else if (ItemToModify.DefinitionMatchState == EParameterDefinitionMatchState::MatchingMoreThanOneDefinition)
	{
		OutCanSubscribeParameterToolTip = LOCTEXT("ParameterPanelViewModel_CannotSubscribeParameterToDefinition_MultipleDefinitions", "Cannot subscribe Parameter to Parameter Definitions: More than one Parameter Definition is specifying a Parameter with this name and type.");
		return false;
	}
	else if (ItemToModify.DefinitionMatchState == EParameterDefinitionMatchState::MatchingOneDefinition)
	{
		if (bSubscribing)
		{
			OutCanSubscribeParameterToolTip = LOCTEXT("ParameterPanelViewModel_SubscribeParameterToDefinition", "Set this Parameter to automatically update its default value and metadata from a Parameter Definition.");
			return true;
		}
		else
		{
			OutCanSubscribeParameterToolTip = LOCTEXT("ParameterPanelViewModel_UnsubscribeParameterFromDefinition", "Unsubscribe this Parameter from the a Parameter Definition and do not synchronize the default value and metadata.");
			return true;
		}
	}
	
	ensureMsgf(false, TEXT("Tried to get whether a parameter could be linked to a definition but the parameter name match state was not known!"));
	return false;
}

void INiagaraParameterPanelViewModel::SetParameterNamespace(const FNiagaraParameterPanelItem ItemToModify, FNiagaraNamespaceMetadata NewNamespaceMetaData, bool bDuplicateParameter) 
{
	FName NewName = FNiagaraParameterUtilities::ChangeNamespace(ItemToModify.GetVariable().GetName(), NewNamespaceMetaData);
	if (NewName != NAME_None)
	{
		bool bParameterExists = CachedViewedItems.ContainsByPredicate([NewName](const FNiagaraParameterPanelItem& CachedViewedItem) {return CachedViewedItem.GetVariable().GetName() == NewName; });
		if (bDuplicateParameter)
		{
			FName NewUniqueName;
			if (bParameterExists)
			{
				TSet<FName> ParameterNames;
				for (const FNiagaraParameterPanelItem& CachedViewedItem : CachedViewedItems)
				{
					ParameterNames.Add(CachedViewedItem.GetVariable().GetName());
				}
				NewUniqueName = FNiagaraUtilities::GetUniqueName(NewName, ParameterNames);
			}
			else
			{
				NewUniqueName = NewName;
			}
			FScopedTransaction Transaction(LOCTEXT("DuplicateParameterToNewNamespaceTransaction", "Duplicate parameter to new namespace"));
			const bool bRequestRename = false;
			const bool bMakeUniqueName = true;
			AddParameter(FNiagaraVariable(ItemToModify.GetVariable().GetType(), NewUniqueName), FNiagaraParameterPanelCategory(NewNamespaceMetaData), bRequestRename, bMakeUniqueName);
		}
		else if (bParameterExists == false)
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeNamespaceTransaction", "Change namespace"));
			RenameParameter(ItemToModify, NewName);
		}
	}
}

void INiagaraParameterPanelViewModel::SetParameterNamespaceModifier(const FNiagaraParameterPanelItem ItemToModify, const FName NewNamespaceModifier, bool bDuplicateParameter) 
{
	FName NewName = FNiagaraParameterUtilities::SetSpecificNamespaceModifier(ItemToModify.GetVariable().GetName(), NewNamespaceModifier);
	if (NewName != NAME_None)
	{
		TSet<FName> ParameterNames;
		for (const FNiagaraParameterPanelItem& CachedViewedItem : CachedViewedItems)
		{
			ParameterNames.Add(CachedViewedItem.GetVariable().GetName());
		}

		bool bParameterExists = ParameterNames.Contains(NewName);
		if (bDuplicateParameter)
		{
			FName NewUniqueName;
			if (bParameterExists)
			{
				NewUniqueName = FNiagaraUtilities::GetUniqueName(NewName, ParameterNames);
			}
			else
			{
				NewUniqueName = NewName;
			}
			FScopedTransaction Transaction(LOCTEXT("DuplicateParameterToWithCustomNamespaceModifierTransaction", "Duplicate parameter with custom namespace modifier"));
			const bool bRequestRename = false;
			const bool bMakeUniqueName = true;
			AddParameter(FNiagaraVariable(ItemToModify.GetVariable().GetType(), NewUniqueName), FNiagaraParameterPanelCategory(ItemToModify.NamespaceMetaData), bRequestRename, bMakeUniqueName);
		}
		else if (bParameterExists == false)
		{
			FScopedTransaction Transaction(LOCTEXT("SetCustomNamespaceModifierTransaction", "Set custom namespace modifier"));
			RenameParameter(ItemToModify, NewName);
		}
	}
}

void INiagaraParameterPanelViewModel::ChangeParameterType(const TArray<FNiagaraParameterPanelItem> ItemsToModify, const FNiagaraTypeDefinition NewType) 
{
	ensureMsgf(false, TEXT("type change not supported for this view"));
}

bool INiagaraParameterPanelViewModel::GetCanChangeParameterType(const TArray<FNiagaraParameterPanelItem>& ItemsToChange, FText& OutTooltip) const
{
	for (const FNiagaraParameterPanelItem& ItemToChange : ItemsToChange)
	{
		if (ItemToChange.bExternallyReferenced)
		{
			OutTooltip = LOCTEXT("CantChangeSelectedType_External", "A parameter is referenced in an external script and its type cannot be changed.");
			return false;
		}
		
		for(const UNiagaraGraph* Graph : GetEditableGraphsConst())
		{
			if(UNiagaraScriptVariable* ScriptVariable = Graph->GetScriptVariable(ItemToChange.GetVariable()))
			{
				if(ScriptVariable->GetIsStaticSwitch())
				{
					OutTooltip = LOCTEXT("CantChangeSelectedType_StaticSwitch", "A parameter is a static switch parameter and its type cannot be changed here.");
					return false;
				}

				if(ItemToChange.GetVariable().GetType().IsStatic())
				{
					OutTooltip = LOCTEXT("CantChangeSelectedType_StaticParameter", "A parameter is static and its type cannot be changed.");
					return false;
				}

				if(ScriptVariable->GetIsSubscribedToParameterDefinitions())
				{
					OutTooltip = LOCTEXT("CantChangeSelectedType_SubscribedParameter", "A parameter is subscribed to a parameter definition. The parameter can only be changed in the parameter definition asset.");
					return false;
				}
			}
		}	
	}	

	OutTooltip = LOCTEXT("CanChangeSelectedType", "Change the type of selected parameters.\nPin connections will get fixed up as much as possible while leaving orphaned pins for connections that can't be maintained.");
	return true;
}

void INiagaraParameterPanelViewModel::GetChangeTypeSubMenu(FMenuBuilder& MenuBuilder, TArray<FNiagaraParameterPanelItem> Items) 
{
	TArray<FNiagaraTypeDefinition> FilteredTypes;
	TArray<FNiagaraTypeDefinition> AllowedTypes;
	FNiagaraEditorUtilities::GetAllowedParameterTypes(AllowedTypes);
	for (const FNiagaraTypeDefinition& RegisteredType : AllowedTypes)
	{
		// only allow basic types for now
		if (RegisteredType.IsDataInterface() || RegisteredType.IsEnum() || RegisteredType.IsUObject() || FNiagaraTypeRegistry::GetRegisteredPayloadTypes().Contains(RegisteredType) || RegisteredType.IsInternalType())
		{
			continue;
		}
		FilteredTypes.Add(RegisteredType);
	}
	FilteredTypes.Sort([](const FNiagaraTypeDefinition& Lhs, const FNiagaraTypeDefinition& Rhs)
	{
		return Lhs.GetNameText().CompareTo(Rhs.GetNameText()) < 0;
	});
	
	for (const FNiagaraTypeDefinition& Type : FilteredTypes)
	{
		MenuBuilder.AddMenuEntry(
			Type.GetNameText(),
			LOCTEXT("ChangeTypeActionTooltip", "Change the type of this variable. Replaced all pins in the graph and metadata with the selected type."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &INiagaraParameterPanelViewModel::ChangeParameterType, Items, Type))); 
	}
}

void INiagaraParameterPanelViewModel::SetParameterCustomNamespaceModifier(const FNiagaraParameterPanelItem ItemToModify, bool bDuplicateParameter) 
{
	if (bDuplicateParameter == false && ItemToModify.bExternallyReferenced)
	{
		return;
	}
	TSet<FName> ParameterNames;
	for (const FNiagaraParameterPanelItem& CachedViewedItem : CachedViewedItems)
	{
		ParameterNames.Add(CachedViewedItem.GetVariable().GetName());
	}
	FName NewName = FNiagaraParameterUtilities::SetCustomNamespaceModifier(ItemToModify.GetVariable().GetName(), ParameterNames);

	if (NewName == NAME_None)
	{
		return;
	}

	if (bDuplicateParameter)
	{
		bool bParameterExists = ParameterNames.Contains(NewName);
		FName NewUniqueName;
		if (bParameterExists)
		{
			NewUniqueName = FNiagaraUtilities::GetUniqueName(NewName, ParameterNames);
		}
		else
		{
			NewUniqueName = NewName;
		}
		FScopedTransaction Transaction(LOCTEXT("DuplicateParameterToWithCustomNamespaceModifierTransaction", "Duplicate parameter with custom namespace modifier"));
		const bool bRequestRename = false;
		const bool bMakeUniqueName = true;
		AddParameter(FNiagaraVariable(ItemToModify.GetVariable().GetType(), NewUniqueName), FNiagaraParameterPanelCategory(ItemToModify.NamespaceMetaData), bRequestRename, bMakeUniqueName);
		OnNotifyParameterPendingNamespaceModifierRenameDelegate.ExecuteIfBound(NewUniqueName);
	}
	else
	{
		if (ItemToModify.GetVariable().GetName() != NewName)
		{
			FScopedTransaction Transaction(LOCTEXT("SetCustomNamespaceModifierTransaction", "Set custom namespace modifier"));
			RenameParameter(ItemToModify, NewName);
		}
		OnNotifyParameterPendingNamespaceModifierRenameDelegate.ExecuteIfBound(NewName);
	}
}


void INiagaraParameterPanelViewModel::GetChangeNamespaceSubMenu(FMenuBuilder& MenuBuilder, bool bDuplicateParameter, FNiagaraParameterPanelItem Item) 
{
	TArray<FNiagaraParameterUtilities::FChangeNamespaceMenuData> MenuData;
	FNiagaraParameterUtilities::GetChangeNamespaceMenuData(Item.GetVariable().GetName(), GetParameterContext(), MenuData);

	FText CanChangeToolTip;
	bool bCanChange = true;
	if (bDuplicateParameter == false)
	{
		bCanChange = GetCanSetParameterNamespaceAndToolTip(Item, FName(), CanChangeToolTip);
	}

	for (const FNiagaraParameterUtilities::FChangeNamespaceMenuData& MenuDataItem : MenuData)
	{
		bool bCanChangeThisNamespace = bCanChange && MenuDataItem.bCanChange;
		FText CanChangeThisNamespaceToolTip = bCanChange ? MenuDataItem.CanChangeToolTip : CanChangeToolTip;
		if (bCanChangeThisNamespace && bDuplicateParameter == false)
		{
			// Check for an existing duplicate by name.
			FName NewName = FNiagaraParameterUtilities::ChangeNamespace(Item.GetVariable().GetName(), MenuDataItem.Metadata);
			if (CachedViewedItems.ContainsByPredicate([NewName](const FNiagaraParameterPanelItem& Item) {return Item.GetVariable().GetName() == NewName; }))
			{
				bCanChangeThisNamespace = false;
				CanChangeThisNamespaceToolTip = LOCTEXT("CantMoveAlreadyExits", "Can not move to this namespace because a parameter with this name already exists.");
			}
		}

		FUIAction Action = FUIAction(
			FExecuteAction::CreateSP(this, &INiagaraParameterPanelViewModel::SetParameterNamespace, Item, MenuDataItem.Metadata, bDuplicateParameter),
			FCanExecuteAction::CreateLambda([bCanChangeThisNamespace]() { return bCanChangeThisNamespace; }));

		TSharedRef<SWidget> MenuItemWidget = FNiagaraParameterUtilities::CreateNamespaceMenuItemWidget(MenuDataItem.NamespaceParameterName, CanChangeThisNamespaceToolTip);
		MenuBuilder.AddMenuEntry(Action, MenuItemWidget, NAME_None, CanChangeThisNamespaceToolTip);
	}
}

void INiagaraParameterPanelViewModel::GetChangeNamespaceModifierSubMenu(FMenuBuilder& MenuBuilder, bool bDuplicateParameter, FNiagaraParameterPanelItem Item) 
{
	TArray<FName> OptionalNamespaceModifiers;
	FNiagaraParameterUtilities::GetOptionalNamespaceModifiers(Item.GetVariable().GetName(), GetParameterContext(), OptionalNamespaceModifiers);

	for (const FName OptionalNamespaceModifier : OptionalNamespaceModifiers)
	{
		FText SetToolTip;
		bool bCanSetNamespaceModifier = GetCanSetParameterNamespaceModifierAndToolTip(Item, OptionalNamespaceModifier, bDuplicateParameter, SetToolTip);
		MenuBuilder.AddMenuEntry(
			FText::FromName(OptionalNamespaceModifier),
			SetToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &INiagaraParameterPanelViewModel::SetParameterNamespaceModifier, Item, OptionalNamespaceModifier, bDuplicateParameter),
				FCanExecuteAction::CreateLambda([bCanSetNamespaceModifier]() {return bCanSetNamespaceModifier; })));
	}

	FText SetCustomToolTip;
	bool bCanSetCustomNamespaceModifier = GetCanSetParameterCustomNamespaceModifierAndToolTip(Item, bDuplicateParameter, SetCustomToolTip);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CustomNamespaceModifier", "Custom..."),
		SetCustomToolTip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &INiagaraParameterPanelViewModel::SetParameterCustomNamespaceModifier, Item, bDuplicateParameter),
			FCanExecuteAction::CreateLambda([bCanSetCustomNamespaceModifier] {return bCanSetCustomNamespaceModifier; })));

	FText SetNoneToolTip;
	bool bCanSetEmptyNamespaceModifier = GetCanSetParameterNamespaceModifierAndToolTip(Item, FName(NAME_None), bDuplicateParameter, SetNoneToolTip);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NoneNamespaceModifier", "Clear"),
		SetNoneToolTip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &INiagaraParameterPanelViewModel::SetParameterNamespaceModifier, Item, FName(NAME_None), bDuplicateParameter),
			FCanExecuteAction::CreateLambda([bCanSetEmptyNamespaceModifier] {return bCanSetEmptyNamespaceModifier; })));
}

void INiagaraParameterPanelViewModel::OnParameterItemActivated(const FNiagaraParameterPanelItem& ActivatedItem) const
{
	ActivatedItem.RequestRename();
}

const TArray<FNiagaraParameterPanelItem>& INiagaraParameterPanelViewModel::GetCachedViewedParameterItems() const
{
	return CachedViewedItems;
}


void INiagaraParameterPanelViewModel::SelectParameterItemByName(const FName ParameterName, const bool bRequestRename)
{
	if (bRequestRename)
	{
		OnNotifyParameterPendingRenameDelegate.ExecuteIfBound(ParameterName);
	}
	else
	{
		OnSelectParameterItemByNameDelegate.ExecuteIfBound(ParameterName);
	}

	// Propagate to the selection object (if found)
	UNiagaraScriptVariable* FoundScriptVariable = nullptr;
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		if (UNiagaraScriptVariable* ScriptVariable = Graph->GetScriptVariable(ParameterName))
		{
			FoundScriptVariable = ScriptVariable;
			break;
		}
	}

	if (IncludeViewItemsInSelectParameterItem() && !FoundScriptVariable)
	{
		const TArray < FNiagaraParameterPanelItem>& CachedItems = GetCachedViewedParameterItems();
		for (const FNiagaraParameterPanelItem& Item : CachedItems)
		{
			if (Item.GetVariable().GetName() == ParameterName)
			{
				FoundScriptVariable = (UNiagaraScriptVariable*)Item.ScriptVariable;
				break;
			}
		}
	}


	const TSharedPtr<FNiagaraObjectSelection>& Selection = GetVariableObjectSelection();
	if (Selection.IsValid() && FoundScriptVariable)
	{
		Selection->SetSelectedObject(FoundScriptVariable);
	}


	OnINiagaraParameterPanelViewModelSelectionChanged(FoundScriptVariable);

	TSharedPtr<INiagaraParameterPanelViewModel> MainVM = MainParameterPanelViewModel.Pin();
	if (MainVM.IsValid())
	{
		MainVM->SelectParameterItemByName(ParameterName, bRequestRename);
	}
}

///////////////////////////////////////////////////////////////////////////////
/// System Toolkit Parameter Panel View Model								///
///////////////////////////////////////////////////////////////////////////////

FNiagaraSystemToolkitParameterPanelViewModel::FNiagaraSystemToolkitParameterPanelViewModel(const TSharedPtr<FNiagaraSystemViewModel>& InSystemViewModel/*, const TWeakPtr<FNiagaraSystemGraphSelectionViewModel>& InSystemGraphSelectionViewModelWeak*/)
{
	SystemViewModel = InSystemViewModel;
	SystemGraphSelectionViewModelWeak = InSystemViewModel->GetSystemGraphSelectionViewModel();
	SystemScriptGraph = SystemViewModel->GetSystemScriptViewModel()->GetGraphViewModel()->GetGraph();
}

void FNiagaraSystemToolkitParameterPanelViewModel::Cleanup()
{
	UnregisterViewModelWithMap(RegisteredHandle);

	UNiagaraSystem& System = SystemViewModel->GetSystem();

	auto GetGraphFromScript = [](UNiagaraScript* Script)->UNiagaraGraph* {
		return CastChecked<UNiagaraScriptSource>(Script->GetLatestSource())->NodeGraph;
	};

	TSet<UNiagaraGraph*> GraphsToRemoveCallbacks;
	GraphsToRemoveCallbacks.Add(GetGraphFromScript(System.GetSystemSpawnScript()));
	GraphsToRemoveCallbacks.Add(GetGraphFromScript(System.GetSystemUpdateScript()));
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
		FVersionedNiagaraEmitterData* EmitterData = EmitterHandle ? EmitterHandle->GetEmitterData() : nullptr;
		if (EmitterData)
		{
			TArray<UNiagaraScript*> EmitterScripts;
			const bool bCompilableOnly = false;
			EmitterData->GetScripts(EmitterScripts, bCompilableOnly);
			for (UNiagaraScript* EmitterScript : EmitterScripts)
			{
				GraphsToRemoveCallbacks.Add(GetGraphFromScript(EmitterScript));
			}
		}
	}

	for (UNiagaraGraph* Graph : GraphsToRemoveCallbacks)
	{
		if (FDelegateHandle* OnGraphChangedHandle = GraphIdToOnGraphChangedHandleMap.Find(Graph->GetUniqueID()))
		{
			Graph->RemoveOnGraphChangedHandler(*OnGraphChangedHandle);
		}
	}

	if (SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		if(SystemGraphSelectionViewModelWeak.IsValid())
		{
			SystemGraphSelectionViewModelWeak.Pin()->GetOnSelectedEmitterScriptGraphsRefreshedDelegate().RemoveAll(this);
		}

		if(SystemViewModel->GetSelectionViewModel())
		{
			SystemViewModel->GetSelectionViewModel()->OnSystemIsSelectedChanged().RemoveAll(this);
		}
	}

	if (SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		System.GetExposedParameters().RemoveOnChangedHandler(UserParameterStoreChangedHandle);
	}

	SystemViewModel->OnSystemCompiled().RemoveAll(this);
	SystemViewModel->OnParameterRemovedExternally().RemoveAll(this);
	SystemViewModel->OnParameterRenamedExternally().RemoveAll(this);

	SystemViewModel->GetOnSubscribedParameterDefinitionsChangedDelegate().RemoveAll(this);
	SystemViewModel->OnEmitterHandleViewModelsChanged().RemoveAll(this);

}


void FNiagaraSystemToolkitParameterPanelViewModel::Init(const FSystemToolkitUIContext& InUIContext)
{
	UIContext = InUIContext;
	UNiagaraSystem& System = SystemViewModel->GetSystem();

	VariableObjectSelection = MakeShared< FNiagaraObjectSelection>();
	
	ActiveSectionIndex = 0;
	LastActiveSystemSectionIdx = ActiveSectionIndex;
	
	// Define the sections
	ActiveSystemIdx = Sections.Emplace(FGuid(0x7B4AFB34, 0xD0DF4618, 0x9A05349E, 0x361CE735), LOCTEXT("SectionSystemActive", "Active Overview"), LOCTEXT("SectionActiveSystemDesc", "Displays parameters that are context-sensitive to the selected items in the System Overview."));
	ActiveScriptIdx = Sections.Emplace(FGuid(0x6E11F3EA, 0x5E314E82, 0xA64FEA41, 0x4F1DB891), LOCTEXT("SectionScriptActive", "Active Module"), LOCTEXT("SectionActiveScriptDesc", "Displays parameters that are context-sensitive to the active Scratch module document. Use this for renaming/editing local parameters to the module."));

	// Go through all the settings and propagate with the default values if not already defined.
	UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
	if (Settings)
	{
		bool bSaveConfig = false;
		bool bAdded = false;

		FNiagaraParameterPanelSectionStorage& ScriptStorage = Settings->FindOrAddParameterPanelSectionStorage(Sections[ActiveScriptIdx].SectionId, bAdded);
		if (bAdded)
		{
			ScriptStorage.ExpandedCategories.Emplace(FNiagaraEditorGuids::ModuleNamespaceMetaDataGuid);
			bSaveConfig = true;
		}

		FNiagaraParameterPanelSectionStorage& SystemStorage = Settings->FindOrAddParameterPanelSectionStorage(Sections[ActiveSystemIdx].SectionId, bAdded);
		if (bAdded)
		{
			SystemStorage.ExpandedCategories.Emplace(FNiagaraEditorGuids::SystemNamespaceMetaDataGuid);
			bSaveConfig = true;
		}

		if (bSaveConfig)
		{
			Settings->SaveConfig();
		}
	}

	// Init bindings
	// Bind OnChanged() and OnNeedsRecompile() callbacks for all script source graphs.
	auto GetGraphFromScript = [](UNiagaraScript* Script)->UNiagaraGraph* {
		return CastChecked<UNiagaraScriptSource>(Script->GetLatestSource())->NodeGraph;
	};

	TSet<UNiagaraGraph*> GraphsToAddCallbacks;
	GraphsToAddCallbacks.Add(GetGraphFromScript(System.GetSystemSpawnScript()));
	GraphsToAddCallbacks.Add(GetGraphFromScript(System.GetSystemUpdateScript()));
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		if (FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle())
		{
			if (EmitterHandle->GetEmitterData() == nullptr)
			{
				continue;
			}

			TArray<UNiagaraScript*> EmitterScripts;
			const bool bCompilableOnly = false;
			EmitterHandle->GetEmitterData()->GetScripts(EmitterScripts, bCompilableOnly);
			for (UNiagaraScript* EmitterScript : EmitterScripts)
			{
				GraphsToAddCallbacks.Add(GetGraphFromScript(EmitterScript));
			}
		}
	}

	for (UNiagaraGraph* Graph : GraphsToAddCallbacks)
	{
		FDelegateHandle OnGraphChangedHandle = Graph->AddOnGraphChangedHandler(
			FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::OnGraphChanged));
		GraphIdToOnGraphChangedHandleMap.Add(Graph->GetUniqueID(), OnGraphChangedHandle);
	}

	// Bind OnChanged() callback for system edit mode active emitter handle selection changing.
	if (SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		if(ensureMsgf(SystemGraphSelectionViewModelWeak.IsValid(), TEXT("SystemGraphSelectionViewModel was null for System edit mode!")))
		{
			SystemGraphSelectionViewModelWeak.Pin()->GetOnSelectedEmitterScriptGraphsRefreshedDelegate().AddSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::RefreshNextTick);
		}
		
		SystemViewModel->GetSelectionViewModel()->OnSystemIsSelectedChanged().AddSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::Refresh);
	}

	// Bind OnChanged() for System Exposed Parameters (User Parameters).
	if (SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		UserParameterStoreChangedHandle = System.GetExposedParameters().AddOnChangedHandler(
			FNiagaraParameterStore::FOnChanged::FDelegate::CreateSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::RefreshNextTick));
	}

	// Bind OnChanged() bindings for compilation and external parameter modifications.
	SystemViewModel->OnSystemCompiled().AddSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::RefreshNextTick);
	SystemViewModel->OnParameterRemovedExternally().AddSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::OnParameterRemovedExternally);
	SystemViewModel->OnParameterRenamedExternally().AddSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::OnParameterRenamedExternally);

	// Bind OnChanged() bindings for parameter definitions.
	SystemViewModel->GetOnSubscribedParameterDefinitionsChangedDelegate().AddSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::Refresh);

	// Bind OnChanged() bindings for emitter handles changing.
	SystemViewModel->OnEmitterHandleViewModelsChanged().AddSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::ReconcileOnGraphChangedBindings);

	// Init default categories
	if (DefaultCategories.Num() == 0)
	{
		TArray<FNiagaraNamespaceMetadata> NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetAllNamespaceMetadata();
		for (const FNiagaraNamespaceMetadata& NamespaceMetadatum : NamespaceMetadata)
		{
			if (NamespaceMetadatum.Options.Contains(ENiagaraNamespaceMetadataOptions::HideInSystem) == false)
			{
				if (NamespaceMetadatum.Options.Contains(ENiagaraNamespaceMetadataOptions::AdvancedInSystem))
				{
					DefaultAdvancedCategories.Add(NamespaceMetadatum);
				}
				else
				{
					DefaultCategories.Add(NamespaceMetadatum);
					DefaultAdvancedCategories.Add(NamespaceMetadatum);
				}
			}
		}
	}
	if (UserCategories.Num() == 0)
	{
		TArray<FNiagaraNamespaceMetadata> NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetAllNamespaceMetadata();
		for (const FNiagaraNamespaceMetadata& NamespaceMetadatum : NamespaceMetadata)
		{
			if (NamespaceMetadatum.GetGuid() == FNiagaraEditorGuids::UserNamespaceMetaDataGuid)
				UserCategories.Add(NamespaceMetadatum);
		}
	}
	// Init Default Script categories
	if (DefaultScriptCategories.Num() == 0)
	{
		TArray<FNiagaraNamespaceMetadata> NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetAllNamespaceMetadata();
		for (const FNiagaraNamespaceMetadata& NamespaceMetadatum : NamespaceMetadata)
		{
			if (NamespaceMetadatum.Options.Contains(ENiagaraNamespaceMetadataOptions::HideInScript) == false)
			{
				if (NamespaceMetadatum.Options.Contains(ENiagaraNamespaceMetadataOptions::AdvancedInScript))
				{
					DefaultAdvancedScriptCategories.Add(NamespaceMetadatum);
				}
				else
				{
					DefaultScriptCategories.Add(NamespaceMetadatum);
					DefaultAdvancedScriptCategories.Add(NamespaceMetadatum);
				}
			}
		}
	}

	RegisteredHandle = RegisterViewModelWithMap(&System, this);
}


void FNiagaraSystemToolkitParameterPanelViewModel::OnINiagaraParameterPanelViewModelSelectionChanged(UNiagaraScriptVariable* InVar)
{

	if (InVar)
	{
		SelectedVariable = InVar->Variable;
	}
	else
	{
		VariableObjectSelection->ClearSelectedObjects();
		SelectedVariable = FNiagaraVariable();
	}
	
	InvalidateCachedDependencies();
}

void FNiagaraSystemToolkitParameterPanelViewModel::OnParameterItemSelected(const FNiagaraParameterPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const
{
	SelectedVariable = SelectedItem.GetVariable();
	const FGuid SelectedVariableGuid = SelectedItem.GetVariableMetaData().GetVariableGuid();
	bool bFound = false;
	
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptVariables;
		Graph->GetAllMetaData().GenerateValueArray(ScriptVariables);

		for(TObjectPtr<UNiagaraScriptVariable> ScriptVariable : ScriptVariables)
		{
			if(ScriptVariable->Metadata.GetVariableGuid() == SelectedVariableGuid)
			{
				VariableObjectSelection->SetSelectedObject(ScriptVariable);
				bFound = true;
				break;
			}
		}

		if(bFound)
		{
			break;
		}
	}

	if (!bFound)
	{
		VariableObjectSelection->ClearSelectedObjects();
	}

	InvalidateCachedDependencies();
}

bool FNiagaraSystemToolkitParameterPanelViewModel::IsVariableSelected(const FNiagaraVariableBase& InVar) const
{
	if (VariableObjectSelection.IsValid())
	{
		for (UObject* Obj : VariableObjectSelection->GetSelectedObjectsResolved())
		{
			UNiagaraScriptVariable* ScriptVar = Cast<UNiagaraScriptVariable>(Obj);
			if (ScriptVar && ScriptVar->Variable.IsEquivalent(InVar, false))
			{
				return true;
			}
		}

	}
	if (IncludeViewItemsInSelectParameterItem() && InVar == static_cast<const FNiagaraVariableBase&>(SelectedVariable))
	{
		return true;
	}

	return false;
}
const TArray<UNiagaraScriptVariable*> FNiagaraSystemToolkitParameterPanelViewModel::GetEditableScriptVariablesWithName(const FName ParameterName) const
{
	TArray<UNiagaraScriptVariable*> EditableScriptVariables;
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		UNiagaraScriptVariable* ParameterScriptVariable = Graph->GetScriptVariable(ParameterName);
		if (ParameterScriptVariable != nullptr)
		{
			EditableScriptVariables.Add(ParameterScriptVariable);
		}
	}
	return EditableScriptVariables;
}
bool FNiagaraSystemToolkitParameterPanelViewModel::GetSectionEnabled(FText Section) const
{
	if (ActiveScriptIdx != -1 && Sections.IsValidIndex(ActiveScriptIdx) && Section.IdenticalTo(Sections[ActiveScriptIdx].DisplayName))
	{
		TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchDocumentVM = SystemViewModel->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
		return ScratchDocumentVM.IsValid();
	}

	return true;
}

bool  FNiagaraSystemToolkitParameterPanelViewModel::ShouldRouteThroughScratchParameterMap(const FNiagaraParameterPanelCategory* Category, const FNiagaraVariableBase* NewVariable)
{
	bool bForceScript = false;
	if (ActiveSectionIndex == ActiveScriptIdx && ActiveScriptIdx != -1)
	{
		bForceScript = true;
	}
	if (NewVariable != nullptr && bForceScript)
	{
		const FGuid NamespaceId = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(NewVariable->GetName()).GetGuid();
		/*if (NamespaceId == FNiagaraEditorGuids::UserNamespaceMetaDataGuid)
			bForceScript = false;
		if (NamespaceId == FNiagaraEditorGuids::UserNamespaceMetaDataGuid)
			bForceScript = false;
		if (NamespaceId == FNiagaraEditorGuids::UserNamespaceMetaDataGuid)
			bForceScript = false;*/
	}

	return bForceScript;
}

void FNiagaraSystemToolkitParameterPanelViewModel::AddParameter(FNiagaraVariable NewVariable, const FNiagaraParameterPanelCategory Category, const bool bRequestRename, const bool bMakeUniqueName) 
{
	TGuardValue<bool> AddParameterRefreshGuard(bIsAddingParameter, true);
	bool bSuccess = false;
	UNiagaraSystem& System = SystemViewModel->GetSystem();

	bool bForceScript = ShouldRouteThroughScratchParameterMap(&Category, &NewVariable);

	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchDocumentVM = SystemViewModel->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
	if (ScratchDocumentVM.IsValid() && bForceScript)
	{
		ScratchDocumentVM->GetParameterPanelViewModel()->AddParameter(NewVariable, Category, bRequestRename, bMakeUniqueName);
		RefreshNextTick();
		return;
	}
	
	{

		FScopedTransaction AddTransaction(LOCTEXT("AddSystemParameterTransaction", "Add parameter to system."));
		System.Modify();
		const FGuid NamespaceId = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(NewVariable.GetName()).GetGuid();
		if (NamespaceId == FNiagaraEditorGuids::UserNamespaceMetaDataGuid)
		{
			bSuccess = FNiagaraEditorUtilities::AddParameter(NewVariable, System.GetExposedParameters(), System, nullptr);
		}
		else
		{
			UNiagaraEditorParametersAdapter* EditorParametersAdapter = SystemViewModel->GetEditorOnlyParametersAdapter();
			TArray<TObjectPtr<UNiagaraScriptVariable>>& EditorOnlyScriptVars = EditorParametersAdapter->GetParameters();
			bool bNewScriptVarAlreadyExists = EditorOnlyScriptVars.ContainsByPredicate([&NewVariable](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Variable == NewVariable; });

			// unless the namespace prevents name changes we make sure the new parameter has a unique name  
			if (bMakeUniqueName && !Category.NamespaceMetaData.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingName))
			{
				TSet<FName> Names;
				for (const auto& ScriptVar : EditorOnlyScriptVars)
				{
					Names.Add(ScriptVar->Variable.GetName());
				}
				NewVariable.SetName(FNiagaraUtilities::GetUniqueName(NewVariable.GetName(), Names));
				bNewScriptVarAlreadyExists = false;
			}

			if (bNewScriptVarAlreadyExists == false)
			{
				EditorParametersAdapter->Modify();
				UNiagaraScriptVariable* NewScriptVar = NewObject<UNiagaraScriptVariable>(EditorParametersAdapter, FName(), RF_Transactional);
				NewScriptVar->Init(NewVariable, FNiagaraVariableMetaData());
				NewScriptVar->SetIsStaticSwitch(false);
				NewScriptVar->SetIsSubscribedToParameterDefinitions(false);
				EditorOnlyScriptVars.Add(NewScriptVar);
				bSuccess = true;

				// Check if the new parameter has the same name and type as an existing parameter definition, and if so, link to the definition automatically.
				FNiagaraParameterDefinitionsUtilities::TrySubscribeScriptVarToDefinitionByName(NewScriptVar, SystemViewModel.Get());
			}
		}
	}

	if (bSuccess)
	{
		Refresh();
		SelectParameterItemByName(NewVariable.GetName(), bRequestRename);
	}
}

void FNiagaraSystemToolkitParameterPanelViewModel::FindOrAddParameter(FNiagaraVariable Variable, const FNiagaraParameterPanelCategory Category) 
{
	TGuardValue<bool> AddParameterRefreshGuard(bIsAddingParameter, true);
	bool bSuccess = false;
	UNiagaraSystem& System = SystemViewModel->GetSystem();

	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchDocumentVM = SystemViewModel->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
	if (ScratchDocumentVM.IsValid())
	{
		ScratchDocumentVM->GetParameterPanelViewModel()->FindOrAddParameter(Variable, Category);
		return;
	}

	FScopedTransaction AddTransaction(LOCTEXT("FindOrAddAddSystemParameterTransaction", "Add parameter to script."));
	const FGuid NamespaceId = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(Variable.GetName()).GetGuid();
	if (NamespaceId == FNiagaraEditorGuids::UserNamespaceMetaDataGuid)
	{
		if(System.GetExposedParameters().FindParameterVariable(Variable) == nullptr)
		{
			AddParameter(Variable, Category, false, false);
		}
	}
	else
	{
		UNiagaraEditorParametersAdapter* EditorParametersAdapter = SystemViewModel->GetEditorOnlyParametersAdapter();
		TArray<TObjectPtr<UNiagaraScriptVariable>>& EditorOnlyScriptVars = EditorParametersAdapter->GetParameters();
		bool bNewScriptVarAlreadyExists = EditorOnlyScriptVars.ContainsByPredicate([&Variable](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Variable == Variable; });
		
		if (bNewScriptVarAlreadyExists == false)
		{
			AddParameter(Variable, Category, false, false);
		}
	}

	if (bSuccess)
	{
		Refresh();
	}
	else
	{
		AddTransaction.Cancel();
	}
	
	SelectParameterItemByName(Variable.GetName(), false);
}

void FNiagaraSystemToolkitParameterPanelViewModel::SetActiveSection(int32 InSection)
{
	INiagaraParameterPanelViewModel::SetActiveSection(InSection);

	// We want to cache this off so that we can remember what you were last on when the primary document was active.
	if (SystemViewModel->GetDocumentViewModel()->IsPrimaryDocumentActive())
	{
		LastActiveSystemSectionIdx = ActiveSectionIndex;
	}
	else
	{
		TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchDocumentVM = SystemViewModel->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
		if (ScratchDocumentVM.IsValid())
		{
			TWeakPtr<INiagaraParameterPanelViewModel> WeakVM = StaticCastSharedRef< INiagaraParameterPanelViewModel>(AsShared());
			ScratchDocumentVM->GetParameterPanelViewModel()->SetMainParameterPanelViewModel(WeakVM);
		}
	}
}

bool FNiagaraSystemToolkitParameterPanelViewModel::GetCanAddParametersToCategory(FNiagaraParameterPanelCategory Category) const
{
	// If in emitter edit mode, don't support anything other than Emitter level and below
	if (SystemViewModel && SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		if (Category.NamespaceMetaData.GetGuid() == FNiagaraEditorGuids::SystemNamespaceMetaDataGuid || 
			Category.NamespaceMetaData.GetGuid() == FNiagaraEditorGuids::UserNamespaceMetaDataGuid)
		{
			return false;
		}
	}
	return GetEditableGraphsConst().Num() > 0 && Category.NamespaceMetaData.GetGuid() != FNiagaraEditorGuids::StaticSwitchNamespaceMetaDataGuid;
}

void FNiagaraSystemToolkitParameterPanelViewModel::DeleteParameters(const TArray<FNiagaraParameterPanelItem>& ItemsToDelete) 
{
	bool bAnyChange = false;
	FScopedTransaction RemoveParameterTransaction(LOCTEXT("RemoveParameter", "Removed Parameter(s)"));

	// Handle deleting parameters from the active script differently than from the system overview being active.
	// This will allow us to behave as users expect.
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchDocumentVM = SystemViewModel->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
	if (ScratchDocumentVM.IsValid())
	{
		for (const FNiagaraParameterPanelItem& ItemToDelete : ItemsToDelete)
		{
			if (ItemToDelete.bExternallyReferenced)
			{
				continue;
			}

			for (UNiagaraGraph* Graph : GetEditableGraphs())
			{
				UNiagaraScriptVariable* SelectedScriptVariable = Graph->GetScriptVariable(ItemToDelete.GetVariable());
				if (SelectedScriptVariable && VariableObjectSelection->GetSelectedObjects().Contains(SelectedScriptVariable))
				{
					VariableObjectSelection->ClearSelectedObjects();
				}
				Graph->RemoveParameter(ItemToDelete.GetVariable());

				if (!bAnyChange)
				{
					bAnyChange = true;
				}
			}
		}
	}
	else
	{
		for (const FNiagaraParameterPanelItem& ItemToDelete : ItemsToDelete)
		{
			if (ItemToDelete.bExternallyReferenced)
			{
				continue;
			}

			if (SystemViewModel->DeleteParameter(ItemToDelete.GetVariable()))
			{
				bAnyChange = true;
			}
		}
	}

	if(bAnyChange)
	{
		Refresh();
	}
	else
	{
		RemoveParameterTransaction.Cancel();
	}
}

void FNiagaraSystemToolkitParameterPanelViewModel::RenameParameter(const FNiagaraParameterPanelItem& ItemToRename, const FName NewName) 
{
	if (ensureMsgf(ItemToRename.bExternallyReferenced == false, TEXT("Can not modify an externally referenced parameter.")) == false)
	{
		return;
	}
	else if (ItemToRename.GetVariable().GetName() == NewName)
	{
		return;
	}

	bool bRenameSuccess = false;
	FScopedTransaction RenameParameterTransaction(LOCTEXT("RenameParameter", "Rename parameter"));

	// Forward renaming of script variables to the active script if it is selected.
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchDocumentVM = SystemViewModel->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
	if (ScratchDocumentVM.IsValid())
	{
		bRenameSuccess = ScratchDocumentVM->RenameParameter(ItemToRename.GetVariable(), NewName);
	}
	// Otherwise rename via the system.
	else
	{
		bRenameSuccess = SystemViewModel->RenameParameter(ItemToRename.GetVariable(), NewName, ENiagaraGetGraphParameterReferencesMode::SelectedGraphs);
	}
	
	if (bRenameSuccess)
	{
		Refresh();
		const bool bRequestRename = false;
		SelectParameterItemByName(NewName, bRequestRename);
	}
}

FReply FNiagaraSystemToolkitParameterPanelViewModel::OnParameterItemsDragged(const TArray<FNiagaraParameterPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const
{
	if (OnGetParametersWithNamespaceModifierRenamePendingDelegate.IsBound() == false)
	{
		ensureMsgf(false, TEXT("OnGetParametersWithNamespaceModifierRenamePendingDelegate was not bound when handling parameter drag in parameter panel view model! "));
		return FReply::Handled();
	}

	if(DraggedItems.Num() == 1)
	{ 
		const FNiagaraParameterPanelItem& DraggedItem = DraggedItems[0];
		return FNiagaraSystemToolkitParameterPanelUtilities::CreateDragEventForParameterItem(
			DraggedItem,
			MouseEvent,
			GetGraphParameterReferencesForItem(DraggedItem),
			OnGetParametersWithNamespaceModifierRenamePendingDelegate.Execute()
		);
	}

	return FReply::Handled();
}

TSharedPtr<SWidget> FNiagaraSystemToolkitParameterPanelViewModel::CreateContextMenuForItems(const TArray<FNiagaraParameterPanelItem>& Items, const TSharedPtr<FUICommandList>& ToolkitCommands)
{
	// Only create context menus when a single item is selected.
	if (Items.Num() == 1)
	{
		const FNiagaraParameterPanelItem& SelectedItem = Items[0];
		if (SelectedItem.ScriptVariable->GetIsStaticSwitch())
		{
			// Static switches do not have context menu actions for System Toolkits.
			return SNullWidget::NullWidget;
		}

		// Create a menu with all relevant operations.
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, ToolkitCommands);
		MenuBuilder.BeginSection("Edit", LOCTEXT("EditMenuHeader", "Edit"));
		{
			FText CopyReferenceToolTip;
			GetCanCopyParameterReferenceAndToolTip(SelectedItem, CopyReferenceToolTip);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy, NAME_None, LOCTEXT("CopyReference", "Copy Reference"), CopyReferenceToolTip);

			FText DeleteToolTip;
			GetCanDeleteParameterAndToolTip(SelectedItem, DeleteToolTip);
			MenuBuilder.AddMenuEntry(FNiagaraParameterPanelCommands::Get().DeleteItem, NAME_None, TAttribute<FText>(), DeleteToolTip, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"));

			FText RenameToolTip;
			GetCanRenameParameterAndToolTip(SelectedItem, FText(), false, RenameToolTip);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("Rename", "Rename"), RenameToolTip);


			MenuBuilder.AddMenuSeparator();

			MenuBuilder.AddSubMenu(
				LOCTEXT("ChangeNamespace", "Change Namespace"),
				LOCTEXT("ChangeNamespaceToolTip", "Select a new namespace for the selected parameter."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::GetChangeNamespaceSubMenu, false, SelectedItem));

			MenuBuilder.AddSubMenu(
				LOCTEXT("ChangeNamespaceModifier", "Change Namespace Modifier"),
				LOCTEXT("ChangeNamespaceModifierToolTip", "Edit the namespace modifier for the selected parameter."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::GetChangeNamespaceModifierSubMenu, false, SelectedItem));
			
			MenuBuilder.AddMenuSeparator();
			bool bCanDebugParameter = GetCanDebugParameters(Items);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DebugParameter", "Watch Parameter In Niagara Debugger"),
				LOCTEXT("DebugParameterToolTip", "Open the Niagara Debugger and add this Parameter to the list of tracked parameters.\r\nParticles parameters will show up alongside each particle.\r\nSystem, Emitter, and User parameters will be attached to the System in the world."), 
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::DebugParameters, Items),
					FCanExecuteAction::CreateLambda([bCanDebugParameter]() {return bCanDebugParameter; })));

			MenuBuilder.AddMenuSeparator();

			FText DuplicateToolTip;
			bool bCanDuplicateParameter = GetCanDuplicateParameterAndToolTip(Items, DuplicateToolTip);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DuplicateParameter", "Duplicate"),
				DuplicateToolTip,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"),
				FUIAction(
					FExecuteAction::CreateSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::DuplicateParameters, Items),
					FCanExecuteAction::CreateLambda([bCanDuplicateParameter](){return bCanDuplicateParameter;})));

			MenuBuilder.AddSubMenu(
				LOCTEXT("DuplicateToNewNamespace", "Duplicate to Namespace"),
				LOCTEXT("DuplicateToNewNamespaceToolTip", "Duplicate this parameter to a new namespace."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::GetChangeNamespaceSubMenu, true, SelectedItem));

			MenuBuilder.AddSubMenu(
				LOCTEXT("DuplicateWithNewNamespaceModifier", "Duplicate with Namespace Modifier"),
				LOCTEXT("DupilcateWithNewNamespaceModifierToolTip", "Duplicate this parameter with a different namespace modifier."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::GetChangeNamespaceModifierSubMenu, true, SelectedItem));
		}
		MenuBuilder.EndSection();
		return MenuBuilder.MakeWidget();
	}
	// More than one item selected, do not return a context menu.
	return SNullWidget::NullWidget;
}

FNiagaraParameterUtilities::EParameterContext FNiagaraSystemToolkitParameterPanelViewModel::GetParameterContext() const
{
	TSharedPtr<class FNiagaraScratchPadScriptViewModel> ScratchPadScriptVM = SystemViewModel->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
	if (ScratchPadScriptVM.IsValid())
	{
		return FNiagaraParameterUtilities::EParameterContext::Script;
	}
	
	return FNiagaraParameterUtilities::EParameterContext::System;
}

void FNiagaraSystemToolkitParameterPanelViewModel::DebugParameters(const TArray<FNiagaraParameterPanelItem> Items) const
{
	TArray<FNiagaraVariableBase> Attributes;
	for (const FNiagaraParameterPanelItem& Item : Items)
	{
		Attributes.Emplace(Item.GetVariable());
	}

	const TArray<FGuid> EmitterHandles = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	UNiagaraSystem* System = &SystemViewModel->GetSystem();
	if (System)
	{
		TArray< FNiagaraEmitterHandle> Handles;
		for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
		{
			if (EmitterHandles.Contains(Handle.GetId() ))
				Handles.Add(Handle);
		}
#if WITH_NIAGARA_DEBUGGER
		SNiagaraDebugger::InvokeDebugger(System, Handles, Attributes);
#endif
	}
}

const TArray<FNiagaraParameterPanelCategory>& FNiagaraSystemToolkitParameterPanelViewModel::GetDefaultCategories() const
{
	// Show advanced in Active/Common case. Only show User when in user mode.
	bool bForceScript = false;
	bool bForceSystem = false;
	
	if (ActiveSectionIndex == ActiveScriptIdx && ActiveScriptIdx != -1)
		bForceScript= true;
	else if (ActiveSectionIndex == ActiveSystemIdx && ActiveSystemIdx != -1)
		bForceSystem = true;
	
	if (bForceScript)
	{
		CachedCurrentCategories = FNiagaraSystemToolkitParameterPanelViewModel::DefaultAdvancedScriptCategories;
	}
	else
	{
		CachedCurrentCategories = FNiagaraSystemToolkitParameterPanelViewModel::DefaultAdvancedCategories;
	}
	
	return CachedCurrentCategories;
}

FMenuAndSearchBoxWidgets FNiagaraSystemToolkitParameterPanelViewModel::GetParameterMenu(FNiagaraParameterPanelCategory Category) 
{
	const bool bRequestRename = true;
	const bool bSkipSubscribedLibraries = false;
	const bool bMakeUniqueName = true;

	// Collect names of all parameters the System already owns to cull from the parameter add menu.
	TSet<FName> AdditionalCulledParameterNames;
	for (UNiagaraScriptVariable* EditorOnlyScriptVar : SystemViewModel->GetEditorOnlyParametersAdapter()->GetParameters())
	{
		AdditionalCulledParameterNames.Add(EditorOnlyScriptVar->Variable.GetName());
	}

	TSharedPtr<SNiagaraAddParameterFromPanelMenu> MenuWidget = SAssignNew(ParameterMenuWidget, SNiagaraAddParameterFromPanelMenu)
		.Graphs(GetEditableGraphsConst())
		.AvailableParameterDefinitions(SystemViewModel->GetAvailableParameterDefinitions(bSkipSubscribedLibraries))
		.SubscribedParameterDefinitions(SystemViewModel->GetSubscribedParameterDefinitions())
		.OnNewParameterRequested(this, &FNiagaraSystemToolkitParameterPanelViewModel::AddParameter, Category, bRequestRename, bMakeUniqueName)
		.OnSpecificParameterRequested(this, &FNiagaraSystemToolkitParameterPanelViewModel::FindOrAddParameter, Category)
		.OnAddScriptVar(this, &FNiagaraSystemToolkitParameterPanelViewModel::AddScriptVariable)
		.OnAddParameterDefinitions(this, &FNiagaraSystemToolkitParameterPanelViewModel::AddParameterDefinitions)
		.OnAllowMakeType(this, &INiagaraParameterPanelViewModel::CanAddType, TOptional<FNiagaraParameterPanelCategory>(Category))
		.NamespaceId(Category.NamespaceMetaData.GetGuid())
		.ShowNamespaceCategory(false)
		.ShowGraphParameters(false)
		.AutoExpandMenu(false)
		.AdditionalCulledParameterNames(AdditionalCulledParameterNames);

	ParameterMenuSearchBoxWidget = MenuWidget->GetSearchBox();
	FMenuAndSearchBoxWidgets MenuAndSearchBoxWidgets;
	MenuAndSearchBoxWidgets.MenuWidget = MenuWidget;
	MenuAndSearchBoxWidgets.MenuSearchBoxWidget = ParameterMenuSearchBoxWidget;
	return MenuAndSearchBoxWidgets;
}

FReply FNiagaraSystemToolkitParameterPanelViewModel::HandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation) 
{
	TSharedPtr<FNiagaraParameterGraphDragOperation> ParameterGraphDragDropOperation = StaticCastSharedPtr<FNiagaraParameterGraphDragOperation>(DragDropOperation);
	if (ParameterGraphDragDropOperation.IsValid() == false)
	{
		return FReply::Handled();
	}

	TSharedPtr<FEdGraphSchemaAction> SourceAction = ParameterGraphDragDropOperation->GetSourceAction();
	if (SourceAction.IsValid() == false)
	{
		return FReply::Handled();
	}

	TSharedPtr<FNiagaraParameterAction> SourceParameterAction = StaticCastSharedPtr<FNiagaraParameterAction>(SourceAction);
	if (SourceParameterAction.IsValid() == false)
	{
		return FReply::Handled();
	}

	AddScriptVariable(SourceParameterAction->GetScriptVar());
	return FReply::Handled();
}

bool FNiagaraSystemToolkitParameterPanelViewModel::GetCanHandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation) const
{
	if (DragDropOperation->IsOfType<FNiagaraParameterGraphDragOperation>() == false)
	{
		return false;
	}
	TSharedPtr<FNiagaraParameterGraphDragOperation> ParameterGraphDragDropOperation = StaticCastSharedPtr<FNiagaraParameterGraphDragOperation>(DragDropOperation);

	const TSharedPtr<FEdGraphSchemaAction>& SourceAction = ParameterGraphDragDropOperation->GetSourceAction();
	if (SourceAction.IsValid() == false)
	{
		return false;
	}

	if (SourceAction->GetTypeId() != FNiagaraEditorStrings::FNiagaraParameterActionId)
	{
		return false;
	}
	const TSharedPtr<FNiagaraParameterAction>& SourceParameterAction = StaticCastSharedPtr<FNiagaraParameterAction>(SourceAction);

	const UNiagaraScriptVariable* ScriptVar = SourceParameterAction->GetScriptVar();
	if (ScriptVar == nullptr)
	{
		return false;
	}

	// Do not allow trying to create a new parameter from the drop action if that parameter name/type pair already exists.
	const FNiagaraVariable& Parameter = ScriptVar->Variable;
	if (SystemViewModel->GetAllScriptVars().ContainsByPredicate([Parameter](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Variable == Parameter; }))
	{
		return false;
	}

	return true;
}



void FNiagaraSystemToolkitParameterPanelViewModel::RefreshDueToActiveDocumentChanged()
{
	// We want to recall the section you were last in when you swapped between Primary and Scratch documents, which
	// will invoke a full parameter refresh.
	if (SystemViewModel->GetDocumentViewModel()->IsPrimaryDocumentActive())
	{
		SetActiveSection(LastActiveSystemSectionIdx);
	}
	else
	{
		SetActiveSection(ActiveScriptIdx);
	}
}

bool FNiagaraSystemToolkitParameterPanelViewModel::GetCanSetParameterNamespaceAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NewNamespace, FText& OutCanSetParameterNamespaceToolTip) const
{
	return FNiagaraParameterPanelUtilities::GetCanSetParameterNamespaceAndToolTipForScriptOrSystem(ItemToModify, NewNamespace, OutCanSetParameterNamespaceToolTip);
}

bool FNiagaraSystemToolkitParameterPanelViewModel::GetCanSetParameterNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NamespaceModifier, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const
{
	return FNiagaraParameterPanelUtilities::GetCanSetParameterNamespaceModifierAndToolTipForScriptOrSystem(CachedViewedItems, ItemToModify, NamespaceModifier, bDuplicateParameter, OutCanSetParameterNamespaceModifierToolTip);
}

bool FNiagaraSystemToolkitParameterPanelViewModel::GetCanSetParameterCustomNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const
{
	return FNiagaraParameterPanelUtilities::GetCanSetParameterCustomNamespaceModifierAndToolTipForScriptOrSystem(ItemToModify, bDuplicateParameter, OutCanSetParameterNamespaceModifierToolTip);
}

TSharedRef<SWidget> FNiagaraSystemToolkitParameterPanelViewModel::CreateAddParameterMenuForAssignmentNode(UNiagaraNodeAssignment* AssignmentNode, const TSharedPtr<SComboButton>& AddButton) 
{
	auto AddParameterLambda = [this, AssignmentNode](FNiagaraVariable NewParameter) {
		// If an assignment target is already setting the associated parameter, make a unique name here so that the assignment target is valid.
		// Because the unique name is made before calling FNiagaraSystemToolkitParameterPanelViewModel::AddParameter, pass bMakeUniqueName = false.
		if (AssignmentNode->GetAssignmentTargets().Contains(NewParameter))
		{
			TSet<FName> Names;
			UNiagaraEditorParametersAdapter* EditorParametersAdapter = SystemViewModel->GetEditorOnlyParametersAdapter();
			TArray<TObjectPtr<UNiagaraScriptVariable>>& EditorOnlyScriptVars = EditorParametersAdapter->GetParameters();
			for (UNiagaraScriptVariable* ScriptVar : EditorOnlyScriptVars)
			{
				Names.Add(ScriptVar->Variable.GetName());
			}
			NewParameter.SetName(FNiagaraUtilities::GetUniqueName(NewParameter.GetName(), Names));
		}

		const FString VarDefaultValue = FNiagaraConstants::GetAttributeDefaultValue(NewParameter);
		FNiagaraParameterPanelCategory TempCategory = FNiagaraParameterPanelCategory(FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(NewParameter.GetName()));
		const bool bRequestRename = true;
		const bool bMakeUniqueName = false;
		AddParameter(NewParameter, TempCategory, bRequestRename, bMakeUniqueName);
		AssignmentNode->AddParameter(NewParameter, VarDefaultValue);
	};

	auto AddExistingParameterLambda = [this, AssignmentNode](const FNiagaraVariable& NewParameter) {
		// If an assignment target is already setting the associated parameter, we simply don't do anything
		if (AssignmentNode->GetAssignmentTargets().Contains(NewParameter))
		{
			return;
		}

		const FString VarDefaultValue = FNiagaraConstants::GetAttributeDefaultValue(NewParameter);
		FNiagaraParameterPanelCategory TempCategory = FNiagaraParameterPanelCategory(FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(NewParameter.GetName()));
		FindOrAddParameter(NewParameter, TempCategory);
		AssignmentNode->AddParameter(NewParameter, VarDefaultValue);
	};

	auto AddScriptVarLambda = [this, AssignmentNode](const UNiagaraScriptVariable* NewScriptVar) {
		const FString VarDefaultValue = FNiagaraConstants::GetAttributeDefaultValue(NewScriptVar->Variable);
		AddScriptVariable(NewScriptVar);
		AssignmentNode->AddParameter(NewScriptVar->Variable, VarDefaultValue);
	};

	// Collect args for add menu widget construct
	TArray<UNiagaraGraph*> InGraphs = { AssignmentNode->GetNiagaraGraph() };
	UNiagaraSystem* OwningSystem = AssignmentNode->GetTypedOuter<UNiagaraSystem>();

	const bool bSkipSubscribedLibraries = false;
	const bool bIsParameterRead = true;
	FGuid NamespaceId;

	// Collect names of all parameters the assignment node already assigns to cull from the parameter add menu.
	TSet<FName> AdditionalCulledParameterNames;
	for (const FNiagaraVariable& AssignmentTargetVar : AssignmentNode->GetAssignmentTargets())
	{
		AdditionalCulledParameterNames.Add(AssignmentTargetVar.GetName());
	}

	TSharedRef<SNiagaraAddParameterFromPanelMenu> MenuWidget = SNew(SNiagaraAddParameterFromPanelMenu)
		.Graphs(InGraphs)
		.AvailableParameterDefinitions(SystemViewModel->GetAvailableParameterDefinitions(bSkipSubscribedLibraries))
		.SubscribedParameterDefinitions(SystemViewModel->GetSubscribedParameterDefinitions())
		.OnNewParameterRequested_Lambda(AddParameterLambda)
		.OnSpecificParameterRequested_Lambda(AddExistingParameterLambda)
		.OnAddScriptVar_Lambda(AddScriptVarLambda)
		.OnAddParameterDefinitions(this, &FNiagaraSystemToolkitParameterPanelViewModel::AddParameterDefinitions)
		.OnAllowMakeType(this, &INiagaraParameterPanelViewModel::CanAddType, TOptional<FNiagaraParameterPanelCategory>())
		.AllowCreatingNew(true)
		.NamespaceId(FNiagaraEditorUtilities::GetNamespaceIdForUsage(FNiagaraStackGraphUtilities::GetOutputNodeUsage(*AssignmentNode)))
		.ShowNamespaceCategory(false)
		.ShowGraphParameters(false)
		.AutoExpandMenu(false)
		.AdditionalCulledParameterNames(AdditionalCulledParameterNames)
		.AssignmentNode(AssignmentNode);

	AddButton->SetMenuContentWidgetToFocus(MenuWidget->GetSearchBox());
	return MenuWidget;
}

TArray<FNiagaraVariable> FNiagaraSystemToolkitParameterPanelViewModel::GetEditableStaticSwitchParameters() const
{
	TArray<FNiagaraVariable> OutStaticSwitchParameters;
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		OutStaticSwitchParameters.Append(Graph->FindStaticSwitchInputs());
	}
	return OutStaticSwitchParameters;
}

const TArray<FNiagaraGraphParameterReference> FNiagaraSystemToolkitParameterPanelViewModel::GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const
{
	const FNiagaraParameterPanelItem& ParameterPanelItem = static_cast<const FNiagaraParameterPanelItem&>(Item);
	bool bGetReferencesAcrossAllGraphs = ParameterPanelItem.bExternallyReferenced == false;

	ENiagaraGetGraphParameterReferencesMode GetGraphParameterReferencesMode = bGetReferencesAcrossAllGraphs ? ENiagaraGetGraphParameterReferencesMode::AllGraphs : ENiagaraGetGraphParameterReferencesMode::SelectedGraphs;
	return SystemViewModel->GetGraphParameterReferences(Item.GetVariable(), GetGraphParameterReferencesMode);
}

const TArray<UNiagaraParameterDefinitions*> FNiagaraSystemToolkitParameterPanelViewModel::GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions) const
{
	return SystemViewModel->GetAvailableParameterDefinitions(bSkipSubscribedParameterDefinitions);
}

void FNiagaraSystemToolkitParameterPanelViewModel::UpdateCategoryExpansionState(const FNiagaraParameterPanelCategory& Category, bool bIsExpanded)
{
	// Update the cached expansion state for the active section if it's valid.
	if (Sections.IsValidIndex(ActiveSectionIndex))
	{
		UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
		bool bAdded = false;
		FNiagaraParameterPanelSectionStorage& Storage = Settings->FindOrAddParameterPanelSectionStorage(Sections[ActiveSectionIndex].SectionId, bAdded);
		if (bIsExpanded)
		{
			Storage.ExpandedCategories.AddUnique(Category.NamespaceMetaData.GetGuid());
		}
		else
		{
			Storage.ExpandedCategories.Remove(Category.NamespaceMetaData.GetGuid());
		}
		Settings->SaveConfig();
	}
}

bool FNiagaraSystemToolkitParameterPanelViewModel::IsCategoryExpandedByDefault(const FNiagaraParameterPanelCategory& Category) const 
{
	// Pull data from the settings for categories being expanded or collapsed
	if (Sections.IsValidIndex(ActiveSectionIndex))
	{
		UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
		bool bAdded = false;
		FNiagaraParameterPanelSectionStorage& Storage = Settings->FindOrAddParameterPanelSectionStorage(Sections[ActiveSectionIndex].SectionId, bAdded);
		if (Storage.ExpandedCategories.Contains(Category.NamespaceMetaData.GetGuid()))
		{
			return true;
		}
	}
	return false;
}

const TArray<UNiagaraGraph*> FNiagaraSystemToolkitParameterPanelViewModel::GetAllGraphsConst() const
{
	return SystemViewModel->GetAllGraphs();
}

TArray<UNiagaraGraph*> FNiagaraSystemToolkitParameterPanelViewModel::GetEditableGraphs() const
{
	return SystemViewModel->GetSelectedGraphs();
}

void INiagaraParameterPanelViewModel::SetActiveSection(int32 InSection)
{
	int32 OldActiveSectionIndex = ActiveSectionIndex;

	if (Sections.IsValidIndex(InSection))
	{
		ActiveSectionIndex = InSection;
	}

	// Need to update UI if sections change
	if (OldActiveSectionIndex != ActiveSectionIndex)
	{
		RefreshFullNextTick(true);
	}
}

void INiagaraParameterPanelViewModel::SetActiveSection(FText & InSection)
{
	for (int32 Idx = 0; Idx <Sections.Num(); Idx++)
	{
		if (Sections[Idx].DisplayName.EqualTo(InSection))
		{
			SetActiveSection(Idx);
			return;
		}
	}
}

FText INiagaraParameterPanelViewModel::GetTooltipForSection(FText& InSection) const
{
	for (int32 i = 0; i < Sections.Num(); i++)
	{
		if (Sections[i].DisplayName.EqualTo(InSection))
			return Sections[i].Description;
	}

	return FText::GetEmpty();
}
TArray<FNiagaraParameterPanelItem> FNiagaraSystemToolkitParameterPanelViewModel::GetViewedParameterItems() const
{
	bool bForceScript = false;
	bool bForceSystem = false;
	if (ActiveSectionIndex == ActiveScriptIdx && ActiveScriptIdx != -1)
		bForceScript = true;
	else if (ActiveSectionIndex == ActiveSystemIdx && ActiveSystemIdx != -1)
		bForceSystem = true;

	// On the first time opening the parameter panel view model we are not guaranteed to call GetDefaultCategories() before GetViewedParameterItems(). 
	// We require CachedCurrentCategories being set as this is used to filter out parameter items that are being viewed. If CachedCurrentCategories 
	// is not set, call GetDefaultCategories() to initialize it. 
	if (CachedCurrentCategories.Num() == 0)
	{
		GetDefaultCategories();
	}


	TMap<FNiagaraVariable, FNiagaraParameterPanelItem> VisitedParameterToItemMap;
	TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>> ParameterToScriptVariableMap;
	TArray<UNiagaraGraph*> EditableGraphs = GetEditableGraphsConst();
	TArray<UNiagaraGraph*> AllGraphs = GetAllGraphsConst();
	
	TArray<UNiagaraGraph*> GraphsToCheck;
	
	UNiagaraSystemSelectionViewModel* SelectionViewModel = SystemViewModel->GetSelectionViewModel();
	if(SelectionViewModel->GetSystemIsSelected() == false && SelectionViewModel->GetSelectedEmitterHandleIds().Num() == 0)
	{	
		GraphsToCheck = AllGraphs;
	}
	else
	{
		GraphsToCheck = EditableGraphs;
	}
	
	// Collect all metadata to be packaged with the FNiagaraParameterPanelItems.
	for (const UNiagaraGraph* GraphToCheck : GraphsToCheck)
	{
		ParameterToScriptVariableMap.Append(GraphToCheck->GetAllMetaData());
	}
	ParameterToScriptVariableMap.Append(TransientParameterToScriptVarMap);

	// Helper lambda to get all FNiagaraVariable parameters from a UNiagaraParameterCollection as FNiagaraParameterPanelItemArgs.
	auto CollectParamStore = [this, &ParameterToScriptVariableMap, &VisitedParameterToItemMap](const FNiagaraParameterStore* ParamStore){
		TArray<FNiagaraVariable> Vars;
		ParamStore->GetParameters(Vars);
		for (const FNiagaraVariable& Var : Vars)
		{
			UNiagaraScriptVariable* ScriptVar = FNiagaraEditorUtilities::UserParameters::GetScriptVariableForUserParameter(Var, SystemViewModel);

			FNiagaraParameterPanelItem Item;
			Item.ScriptVariable = ScriptVar;
			Item.NamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(Var.GetName());
			
			// Determine whether the item is name aliasing a parameter definition's parameter.
			Item.DefinitionMatchState = FNiagaraParameterDefinitionsUtilities::GetDefinitionMatchStateForParameter(ScriptVar->Variable);

			VisitedParameterToItemMap.Add(Var, Item);
		}
	};

	// Collect user parameters from system.
	if (bForceSystem)
	{
		CollectParamStore(&SystemViewModel->GetSystem().GetExposedParameters());
	}

	if (bForceSystem)
	{
		// Collect parameters added to the system asset.
		for (UNiagaraScriptVariable* EditorOnlyScriptVar : SystemViewModel->GetEditorOnlyParametersAdapter()->GetParameters())
		{
			ParameterToScriptVariableMap.Add(EditorOnlyScriptVar->Variable, EditorOnlyScriptVar);

			const FNiagaraVariable& Var = EditorOnlyScriptVar->Variable;
			FNiagaraParameterPanelItem Item;
			Item.ScriptVariable = EditorOnlyScriptVar;
			Item.NamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(Var.GetName());

			// Determine whether the item is name aliasing a parameter definition's parameter.
			Item.DefinitionMatchState = FNiagaraParameterDefinitionsUtilities::GetDefinitionMatchStateForParameter(EditorOnlyScriptVar->Variable);

			VisitedParameterToItemMap.Add(Var, Item);
		}
	}

	if (bForceSystem)
	{
		// Collect parameters for all emitters.
		TArray<FNiagaraVariable> VisitedInvalidParameters;

		// -For each applicable graph perform a parameter map history traversal and record each unique visited parameter.
		for (const UNiagaraGraph* GraphToCheck : GraphsToCheck)
		{
			TArray<UNiagaraNodeOutput*> OutputNodes;
			GraphToCheck->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);
			for (UNiagaraNodeOutput* OutputNode : OutputNodes)
			{
				UNiagaraNode* NodeToTraverse = OutputNode;
				if (OutputNode->GetUsage() == ENiagaraScriptUsage::SystemSpawnScript || OutputNode->GetUsage() == ENiagaraScriptUsage::SystemUpdateScript)
				{
					// Traverse past the emitter nodes, otherwise the system scripts will pick up all of the emitter and particle script parameters.
					UEdGraphPin* InputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*NodeToTraverse);
					while (NodeToTraverse != nullptr && InputPin != nullptr && InputPin->LinkedTo.Num() == 1 &&
						(NodeToTraverse->IsA<UNiagaraNodeOutput>() || NodeToTraverse->IsA<UNiagaraNodeEmitter>()))
					{
						NodeToTraverse = Cast<UNiagaraNode>(InputPin->LinkedTo[0]->GetOwningNode());
						InputPin = NodeToTraverse != nullptr ? FNiagaraStackGraphUtilities::GetParameterMapInputPin(*NodeToTraverse) : nullptr;
					}
				}

				if (NodeToTraverse == nullptr)
				{
					continue;
				}

				bool bIgnoreDisabled = true;
				FNiagaraParameterMapHistoryBuilder Builder;
				FVersionedNiagaraEmitter GraphOwningEmitter = GraphToCheck->GetOwningEmitter();
				FCompileConstantResolver ConstantResolver = GraphOwningEmitter.Emitter != nullptr
					? FCompileConstantResolver(GraphOwningEmitter, OutputNode->GetUsage())
					: FCompileConstantResolver();
				 
				Builder.SetIgnoreDisabled(bIgnoreDisabled);
				*Builder.ConstantResolver = ConstantResolver;
				FName StageName;
				ENiagaraScriptUsage StageUsage = OutputNode->GetUsage();
				if (StageUsage == ENiagaraScriptUsage::ParticleSimulationStageScript && GraphOwningEmitter.Emitter)
				{
					UNiagaraSimulationStageBase* Base = GraphOwningEmitter.GetEmitterData()->GetSimulationStageById(OutputNode->GetUsageId());
					if (Base)
					{
						StageName = Base->GetStackContextReplacementName();
					}
				}
				Builder.BeginUsage(StageUsage, StageName);
				NodeToTraverse->BuildParameterMapHistory(Builder, true, true);
				Builder.EndUsage();

				if (Builder.Histories.Num() != 1)
				{
					// We should only have traversed one emitter (have not visited more than one NiagaraNodeEmitter.)
					ensureMsgf(false, TEXT("Encountered '%d' parameter map history when collecting parameters for system parameter panel view model, we expect only 1!"), Builder.Histories.Num());
				}
				if (Builder.Histories.Num() == 0)
				{
					continue;
				}

				// Get all UNiagaraScriptVariables of visited graphs in the ParameterToScriptVariableMap so that generated items are in sync.
				TSet<UNiagaraGraph*> VisitedExternalGraphs;
				for (const UEdGraphPin* MapPin : Builder.Histories[0].MapPinHistory)
				{
					const UNiagaraNodeFunctionCall* MapPinOuterFunctionCallNode = Cast<UNiagaraNodeFunctionCall>(MapPin->GetOuter());
					if (MapPinOuterFunctionCallNode != nullptr)
					{
						UNiagaraGraph* VisitedExternalGraph = MapPinOuterFunctionCallNode->GetCalledGraph();
						if (VisitedExternalGraph != nullptr)
						{
							VisitedExternalGraphs.Add(VisitedExternalGraph);
						}
					}
				}
				for (const UNiagaraGraph* VisitedExternalGraph : VisitedExternalGraphs)
				{
					ParameterToScriptVariableMap.Append(VisitedExternalGraph->GetAllMetaData());
				}

				auto AddReadReference = [GraphToCheck](const FNiagaraParameterMapHistory::FReadHistory& History, FNiagaraParameterPanelItem& Item)
				{
					FNiagaraParameterReferencePath Source;
					Source.SourceObject = GraphToCheck;
					Source.ModuleName = History.ReadPin.ModuleName;
					if (Source.ModuleName == NAME_None && History.ReadPin.Pin->LinkedTo.Num() > 0)
					{
						// use the name of the module input for usages as stack function inputs
						Source.ModuleName = History.ReadPin.Pin->LinkedTo[0]->PinName;

						// find the name of the module this belongs to when used in a dynamic input
						TArray<UEdGraphNode*> NodesToCheck;
						NodesToCheck.Add(History.ReadPin.Pin->LinkedTo[0]->GetOwningNode());
						int32 NodeIndex = 0;
						while (NodesToCheck.IsValidIndex(NodeIndex))
						{
							if (UNiagaraNode* Node = Cast<UNiagaraNode>(NodesToCheck[NodeIndex]))
							{
								if (UNiagaraNodeFunctionCall* FunctionNode = Cast<UNiagaraNodeFunctionCall>(Node))
								{
									const bool bIsModuleFunctionScript = FunctionNode->FunctionScript && FunctionNode->FunctionScript->IsModuleScript();
									if (FunctionNode->IsNodeEnabled() && bIsModuleFunctionScript)
									{
										// Found the module
										FString CurrentName = Source.ModuleName.ToString();
										if (!CurrentName.StartsWith(FunctionNode->GetFunctionName()))
										{
											Source.ModuleName = FName("[" + FunctionNode->GetFunctionName() + "] " + CurrentName);
										}
										break;
									}
								}
								if (UEdGraphPin* OutputPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*Node))
								{
									for (UEdGraphPin* LinkedPin : OutputPin->LinkedTo)
									{
										NodesToCheck.AddUnique(LinkedPin->GetOwningNode());
									}
								}
							}
							NodeIndex++;
						}
					}
					Item.AddToReadCount(Source);
				};
				auto AddWriteReference = [GraphToCheck](const FNiagaraParameterMapHistory::FModuleScopedPin& History, FNiagaraParameterPanelItem& Item)
				{
					if (Cast<UNiagaraNodeParameterMapGet>(History.Pin->GetOwningNode()))
					{
						// ignore writes from map get nodes, as it's just the default value pin
						return;
					}
					Item.AddToWriteCount(FNiagaraParameterReferencePath(GraphToCheck, History.ModuleName));
				};

				const TArray<FName>& CustomIterationSourceNamespaces = Builder.Histories[0].IterationNamespaceOverridesEncountered;
				for (int32 VariableIndex = 0; VariableIndex < Builder.Histories[0].Variables.Num(); VariableIndex++)
				{
					const FNiagaraVariable& Var = Builder.Histories[0].Variables[VariableIndex];
					// If this variable has already been visited and does not have a valid namespace then skip it.
					if (VisitedInvalidParameters.Contains(Var))
					{
						continue;
					}

					TArray<TNiagaraParameterMapHistory<FNiagaraCompilationGraphBridge>::FReadHistory> ReadHistory = Builder.Histories[0].PerVariableReadHistory[VariableIndex];
					TArray<TModuleScopedPin<UEdGraphPin>> WriteHistory = Builder.Histories[0].PerVariableWriteHistory[VariableIndex];
					FNiagaraParameterPanelItem* ItemPtr = VisitedParameterToItemMap.Find(Var);
					// Add newly found variables
					if (ItemPtr == nullptr)
					{
						// This variable has not been registered, prepare the FNiagaraParameterPanelItem.
						// -First make sure the variable namespace is in a valid category. If not, skip it.
						FNiagaraNamespaceMetadata CandidateNamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(Var.GetName());
						if (CachedCurrentCategories.Contains(FNiagaraParameterPanelCategory(CandidateNamespaceMetaData)) == false)
						{
							VisitedInvalidParameters.Add(Var);
							continue;
						}

						// -Lookup the script variable.
						TObjectPtr<UNiagaraScriptVariable> const* ScriptVarPtr = ParameterToScriptVariableMap.Find(Var);
						UNiagaraScriptVariable* ScriptVar;
						if (ScriptVarPtr != nullptr)
						{
							ScriptVar = *ScriptVarPtr;
						}
						else
						{
							// Create a new UNiagaraScriptVariable to represent this parameter for the lifetime of the ParameterPanelViewModel.
							ScriptVar = NewObject<UNiagaraScriptVariable>(GetTransientPackage());
							ScriptVar->AddToRoot();
							ScriptVar->Init(Var, FNiagaraVariableMetaData());
							TransientParameterToScriptVarMap.Add(Var, ScriptVar);
						}

						FNiagaraParameterPanelItem Item = FNiagaraParameterPanelItem();
						Item.ScriptVariable = ScriptVar;
						Item.NamespaceMetaData = CandidateNamespaceMetaData;
						Item.bExternallyReferenced = false;

						// -Determine whether the parameter is from a custom stack context.
						Item.bSourcedFromCustomStackContext = false;
						for (const FName CustomIterationNamespace : CustomIterationSourceNamespaces)
						{
							if (Var.IsInNameSpace(CustomIterationNamespace))
							{
								Item.bSourcedFromCustomStackContext = true;
								break;
							}
						}

						// Determine whether the item is name aliasing a parameter definition's parameter.
						Item.DefinitionMatchState = FNiagaraParameterDefinitionsUtilities::GetDefinitionMatchStateForParameter(Item.ScriptVariable->Variable);

						ItemPtr = &VisitedParameterToItemMap.Add(Var, Item);
					}

					if (!ItemPtr->bExternallyReferenced)
					{
						bool bVarOnlyInTopLevelGraph = true;
						if (!bForceScript)
						{
							for (FNiagaraParameterMapHistory::FModuleScopedPin& WritePin : WriteHistory)
							{
								UEdGraphNode* VariableOwningNode = WritePin.Pin->GetOwningNode();
								bVarOnlyInTopLevelGraph &= AllGraphs.Contains(static_cast<const UNiagaraGraph*>(VariableOwningNode->GetGraph()));
							}
							for (FNiagaraParameterMapHistory::FReadHistory& ReadPins : ReadHistory)
							{
								UEdGraphNode* VariableOwningNode = ReadPins.ReadPin.Pin->GetOwningNode();
								bVarOnlyInTopLevelGraph &= !AllGraphs.Contains(static_cast<const UNiagaraGraph*>(VariableOwningNode->GetGraph()));
							}
						}

						ItemPtr->bExternallyReferenced = !bVarOnlyInTopLevelGraph;
					}

					// -Increment the reference count.
					for (const FNiagaraParameterMapHistory::FReadHistory& History : ReadHistory)
					{
						AddReadReference(History, *ItemPtr);
					}
					for (const FNiagaraParameterMapHistory::FModuleScopedPin& History : WriteHistory)
					{
						AddWriteReference(History, *ItemPtr);
					}
				}
			}
		}

		// Stateless emitter usage
		for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterViewModel : SystemViewModel->GetEmitterHandleViewModels())
		{
			FNiagaraEmitterHandle* EmitterHandle = EmitterViewModel->GetEmitterHandle();
			UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandle && EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Stateless ? EmitterHandle->GetStatelessEmitter() : nullptr;
			if (StatelessEmitter == nullptr)
			{
				continue;
			}

			if (SelectionViewModel->GetSelectedEmitterHandleIds().Num() != 0)
			{
				if (!SelectionViewModel->GetSelectedEmitterHandleIds().Contains(EmitterHandle->GetId()))
				{
					continue;
				}
			}

			StatelessEmitter->GatherSimulationAttributeReferences(
				[&](FName OwnerName, const FNiagaraVariableBase& Variable, bool bRead, bool bWrite)
				{
					if (VisitedInvalidParameters.Contains(Variable))
					{
						return;
					}

					FNiagaraParameterPanelItem* ItemPtr = VisitedParameterToItemMap.Find(Variable);
					if (ItemPtr == nullptr)
					{
						UNiagaraScriptVariable* ScriptVariable = NewObject<UNiagaraScriptVariable>(GetTransientPackage());
						ScriptVariable->AddToRoot();
						ScriptVariable->Init(Variable, FNiagaraVariableMetaData());
						TransientParameterToScriptVarMap.Add(Variable, ScriptVariable);

						FNiagaraNamespaceMetadata CandidateNamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(Variable.GetName());
						if (CachedCurrentCategories.Contains(FNiagaraParameterPanelCategory(CandidateNamespaceMetaData)) == false)
						{
							VisitedInvalidParameters.Add(Variable);
							return;
						}

						FNiagaraParameterPanelItem Item;
						Item.ScriptVariable = ScriptVariable;
						Item.NamespaceMetaData = CandidateNamespaceMetaData;
						ItemPtr = &VisitedParameterToItemMap.Add(Variable, Item);
					}

					if (bRead)
					{
						ItemPtr->AddToReadCount(FNiagaraParameterReferencePath(StatelessEmitter, OwnerName));
					}
					if (bWrite)
					{
						ItemPtr->AddToWriteCount(FNiagaraParameterReferencePath(StatelessEmitter, OwnerName));
					}
				}
			);
		}

		// Add active renderers usage variables to the counts
		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterVMS : SystemViewModel->GetEmitterHandleViewModels())
		{
			if (!EmitterVMS.Get().IsValid() || !EmitterVMS.Get().GetIsEnabled() || !EmitterVMS.Get().GetEmitterHandle())
			{
				continue;
			}

			FNiagaraEmitterHandle* EmitterHandle = EmitterVMS.Get().GetEmitterHandle();
			UObject* SourceObject = EmitterHandle->GetEmitterBase();
			if (const FVersionedNiagaraEmitterData* EmitterData = EmitterVMS.Get().GetEmitterHandle()->GetEmitterData())
			{
				if (UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(EmitterData->GraphSource))
				{
					SourceObject = ScriptSource->NodeGraph;
				}
			}

			FNiagaraAliasContext ResolveAliasesContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript);
			ResolveAliasesContext.ChangeEmitterNameToEmitter(EmitterHandle->GetUniqueInstanceName());

			EmitterHandle->ForEachEnabledRendererWithIndex(
				[&](const UNiagaraRendererProperties* RenderProperties, int32 RendererIndex)
				{
					TArray<FNiagaraVariable> BoundAttributes = RenderProperties->GetBoundAttributes();

					// Find attributes first
					for (const FNiagaraVariableAttributeBinding* AttributeBinding : RenderProperties->GetAttributeBindings())
					{
						const FNiagaraVariable BoundAttribute = RenderProperties->GetBoundAttribute(AttributeBinding);
						if (BoundAttribute.IsValid() == false)
						{
							continue;
						}
						BoundAttributes.RemoveSwap(BoundAttribute, EAllowShrinking::No);

						const FNiagaraVariable ResolvedAttribute = FNiagaraUtilities::ResolveAliases(BoundAttribute, ResolveAliasesContext);
						if (FNiagaraParameterPanelItem* ItemPtr = VisitedParameterToItemMap.Find(ResolvedAttribute))
						{
							const FName SourceModuleName(RenderProperties->GetWidgetDisplayName().ToString() + " (" + AttributeBinding->GetDataSetBindableVariable().GetName().ToString() + " Binding)");
							ItemPtr->AddToReadCount(FNiagaraParameterReferencePath(SourceObject, SourceModuleName));
						}
					}

					// Add remaining bindings
					for (const FNiagaraVariable& BoundAttribute : BoundAttributes)
					{
						if (BoundAttribute.IsValid() == false)
						{
							continue;
						}

						const FNiagaraVariable ResolvedAttribute = FNiagaraUtilities::ResolveAliases(BoundAttribute, ResolveAliasesContext);
						if (FNiagaraParameterPanelItem* ItemPtr = VisitedParameterToItemMap.Find(ResolvedAttribute))
						{
							const FName SourceModuleName(*RenderProperties->GetWidgetDisplayName().ToString());
							ItemPtr->AddToReadCount(FNiagaraParameterReferencePath(SourceObject, SourceModuleName));
						}
					}
				}
			);
		}

		// Find all data interfaces bindings
		{
			TSet<UNiagaraDataInterface*> DiscoveredDIs;
			const auto FindDataInterfaceBindings =
				[&](const UObject* Context, UNiagaraDataInterface* DataInterface, const FNiagaraVariableBase& Variable)
				{
					if (!DataInterface || DiscoveredDIs.Contains(DataInterface))
					{
						return true;
					}
					DiscoveredDIs.Add(DataInterface);

					for (TFieldIterator<FStructProperty> PropertyIt(DataInterface->GetClass()); PropertyIt; ++PropertyIt)
					{
						FStructProperty* StructProperty = *PropertyIt;
						if (!StructProperty || !StructProperty->Struct)
						{
							continue;
						}

						if (StructProperty->Struct->IsChildOf(FNiagaraUserParameterBinding::StaticStruct()))
						{
							FNiagaraUserParameterBinding* ParameterBinding = StructProperty->ContainerPtrToValuePtr<FNiagaraUserParameterBinding>(DataInterface);
							if (ParameterBinding->Parameter.IsValid())
							{
								if (FNiagaraParameterPanelItem* ItemPtr = VisitedParameterToItemMap.Find(ParameterBinding->Parameter))
								{
									ItemPtr->AddToReadCount(FNiagaraParameterReferencePath(Context, Variable.GetName()));
								}
							}
						}
						else if (StructProperty->Struct->IsChildOf(FNiagaraParameterBinding::StaticStruct()))
						{
							FNiagaraParameterBinding* ParameterBinding = StructProperty->ContainerPtrToValuePtr<FNiagaraParameterBinding>(DataInterface);
							if (ParameterBinding->ResolvedParameter.IsValid())
							{
								if (FNiagaraParameterPanelItem* ItemPtr = VisitedParameterToItemMap.Find(ParameterBinding->ResolvedParameter))
								{
									ItemPtr->AddToReadCount(FNiagaraParameterReferencePath(Context, Variable.GetName()));
								}
							}
						}
					}
					return true;
				};

			{
				const FNiagaraParameterStore& UserParameterStore = SystemViewModel->GetSystem().GetExposedParameters();
				for (const FNiagaraVariableWithOffset& UserVariable : UserParameterStore.ReadParameterVariables())
				{
					if (UserVariable.GetType().IsDataInterface() == false)
					{
						continue;
					}
					FindDataInterfaceBindings(&SystemViewModel->GetSystem(), UserParameterStore.GetDataInterface(UserVariable.Offset), UserVariable);
				}
			}

			FNiagaraDataInterfaceUtilities::ForEachDataInterface(
				&SystemViewModel->GetSystem(),
				[&](const FNiagaraDataInterfaceUtilities::FDataInterfaceUsageContext& UsageContext)
				{
					FindDataInterfaceBindings(UsageContext.OwnerObject, UsageContext.DataInterface, UsageContext.Variable);
					return true;
				}
			);
		}
	}
	else if (bForceScript)
	{
		TSharedPtr<class FNiagaraScratchPadScriptViewModel> ScratchPadScriptVM = SystemViewModel->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
		if (ScratchPadScriptVM.IsValid())
		{
			TSharedPtr<INiagaraParameterPanelViewModel> ParamPanelForScriptVM = ScratchPadScriptVM->GetParameterPanelViewModel();
			if (ParamPanelForScriptVM.IsValid())
			{
				CachedViewedItems = ParamPanelForScriptVM->GetViewedParameterItems();
				return CachedViewedItems;
			}
		}
	}

	// Refresh the CachedViewedItems and return that as the latest array of viewed items.
	VisitedParameterToItemMap.GenerateValueArray(CachedViewedItems);
	return CachedViewedItems;
}

TArray<TWeakObjectPtr<UNiagaraGraph>> FNiagaraSystemToolkitParameterPanelViewModel::GetEditableEmitterScriptGraphs() const
{
	if (SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset && ensureMsgf(SystemGraphSelectionViewModelWeak.IsValid(), TEXT("SystemGraphSelectionViewModel was null for System edit mode!")))
	{
		return SystemGraphSelectionViewModelWeak.Pin()->GetSelectedEmitterScriptGraphs();
	}
	else
	{
		TArray<TWeakObjectPtr<UNiagaraGraph>> EditableEmitterScriptGraphs;
		if (FVersionedNiagaraEmitterData* EmitterData = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterHandle()->GetEmitterData())
		{
			EditableEmitterScriptGraphs.Add(static_cast<UNiagaraScriptSource*>(EmitterData->GraphSource)->NodeGraph);
		}
		return EditableEmitterScriptGraphs;
	}
}

TArray<FNiagaraEmitterHandle*> FNiagaraSystemToolkitParameterPanelViewModel::GetEditableEmitterHandles() const
{
	TArray<FNiagaraEmitterHandle*> EditableEmitterHandles;
	const TArray<FGuid>& SelectedEmitterHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();

	const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& EmitterHandleViewModels = SystemViewModel->GetEmitterHandleViewModels();
	for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
	{
		if (SelectedEmitterHandleIds.Contains(EmitterHandleViewModel->GetId()))
		{
			EditableEmitterHandles.Add(EmitterHandleViewModel->GetEmitterHandle());
		}
	}
	return EditableEmitterHandles;
}

void FNiagaraSystemToolkitParameterPanelViewModel::AddScriptVariable(const UNiagaraScriptVariable* NewScriptVar) 
{
	if (NewScriptVar == nullptr)
	{
		ensureMsgf(false, TEXT("Encounted null script variable when adding parameter!"));
		return;
	}

	TGuardValue<bool> AddParameterRefreshGuard(bIsAddingParameter, true);
	bool bSuccess = false;
	UNiagaraSystem& System = SystemViewModel->GetSystem();

	FScopedTransaction AddTransaction(LOCTEXT("AddSystemParameterTransaction", "Add parameter to system."));
	System.Modify();
	const FGuid NamespaceId = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(NewScriptVar->Variable.GetName()).GetGuid();
	if (NamespaceId == FNiagaraEditorGuids::UserNamespaceMetaDataGuid)
	{
		FNiagaraVariable NewParameter = FNiagaraVariable(NewScriptVar->Variable);
		bSuccess = FNiagaraEditorUtilities::AddParameter(NewParameter, System.GetExposedParameters(), System, nullptr);
	}
	else
	{
		UNiagaraEditorParametersAdapter* EditorParametersAdapter = SystemViewModel->GetEditorOnlyParametersAdapter();
		TArray<TObjectPtr<UNiagaraScriptVariable>>& EditorOnlyScriptVars = EditorParametersAdapter->GetParameters();
		const FGuid& NewScriptVarId = NewScriptVar->Metadata.GetVariableGuid();
		bool bNewScriptVarAlreadyExists = EditorOnlyScriptVars.ContainsByPredicate([&NewScriptVarId](const UNiagaraScriptVariable* ScriptVar){ return ScriptVar->Metadata.GetVariableGuid() == NewScriptVarId; });
		if (bNewScriptVarAlreadyExists == false)
		{
			EditorParametersAdapter->Modify();
			UNiagaraScriptVariable* DupeNewScriptVar = CastChecked<UNiagaraScriptVariable>(StaticDuplicateObject(NewScriptVar, EditorParametersAdapter, FName()));
			DupeNewScriptVar->SetFlags(RF_Transactional);
			EditorOnlyScriptVars.Add(DupeNewScriptVar);
			bSuccess = true;
		}
	}

	if (bSuccess)
	{
		if (SystemViewModel->GetSystemStackViewModel())
			SystemViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();
		Refresh();
		const bool bRequestRename = false;
		SelectParameterItemByName(NewScriptVar->Variable.GetName(), bRequestRename);
	}
}

void FNiagaraSystemToolkitParameterPanelViewModel::AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const
{
	FScopedTransaction AddTransaction(LOCTEXT("AddParameterDefinitions", "Add parameter definitions."));
	SystemViewModel->GetSystem().Modify();
	SystemViewModel->SubscribeToParameterDefinitions(NewParameterDefinitions);
}

void FNiagaraSystemToolkitParameterPanelViewModel::RemoveParameterDefinitions(const FGuid& ParameterDefinitionsToRemoveId) const
{
	FScopedTransaction RemoveTransaction(LOCTEXT("RemoveParameterDefinitions", "Remove parameter definitions."));
	SystemViewModel->GetSystem().Modify();
	SystemViewModel->UnsubscribeFromParameterDefinitions(ParameterDefinitionsToRemoveId);
	Refresh();
	UIContext.RefreshParameterDefinitionsPanel();
}

void FNiagaraSystemToolkitParameterPanelViewModel::OnGraphChanged(const struct FEdGraphEditAction& InAction) const
{
	RefreshNextTick();
}

void FNiagaraSystemToolkitParameterPanelViewModel::OnParameterRenamedExternally(const FNiagaraVariableBase& InOldVar, const FNiagaraVariableBase& InNewVar, UNiagaraEmitter* InOptionalEmitter)
{
	// See if this was the last reference to that parameter being renamed, if so, we need to update to a full rename and rename all locations where it was used that are downstream, like renderer bindings.

	// Emitter & Particle namespaces are just for the ones actively being worked on.
	if (InOldVar.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString) ||
		InOldVar.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString))
	{
		const TArray<FGuid>& SelectedEmitterHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();

		// Note that we might have multiple selections and don't know explicitly which one changed, so we have to go through all independently and examine them.
		if (SelectedEmitterHandleIds.Num() > 0)
		{
			const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& EmitterHandleViewModels = SystemViewModel->GetEmitterHandleViewModels();
			for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
			{
				if (SelectedEmitterHandleIds.Contains(EmitterHandleViewModel->GetId()))
				{
					bool bFound = false;
					FVersionedNiagaraEmitter VersionedEmitter = EmitterHandleViewModel->GetEmitterHandle()->GetInstance();
					UNiagaraGraph* Graph = static_cast<UNiagaraScriptSource*>(VersionedEmitter.GetEmitterData()->GraphSource)->NodeGraph;
					if (Graph)
					{
						const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& RefMap = Graph->GetParameterReferenceMap();
						const FNiagaraGraphParameterReferenceCollection* Coll = RefMap.Find(InOldVar);
						if (Coll)
						{
							bFound = true;
							break;
						}
					}

					if (!bFound)
					{
						if (const FNiagaraParameterPanelItem* FoundItemPtr = CachedViewedItems.FindByPredicate([&InOldVar](const FNiagaraParameterPanelItem& Item) { return static_cast<const FNiagaraVariableBase&>(Item.GetVariable()) == InOldVar; }))
						{
							RenameParameter(*FoundItemPtr, InNewVar.GetName());
						}
					}
				}
			}
		}		
	}
	// User and System need to be checked for all graphs as they could be used anywhere.
	else if (InOldVar.IsInNameSpace(FNiagaraConstants::UserNamespaceString) ||
			 InOldVar.IsInNameSpace(FNiagaraConstants::SystemNamespaceString))
	{
		TArray<UNiagaraGraph*> Graphs;
		Graphs.Add(SystemScriptGraph.Get());

		const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& EmitterHandleViewModels = SystemViewModel->GetEmitterHandleViewModels();
		for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
		{
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandleViewModel->GetEmitterHandle()->GetEmitterData();
			UNiagaraGraph* Graph = EmitterData ? CastChecked<UNiagaraScriptSource>(EmitterData->GraphSource)->NodeGraph : nullptr;
			if (Graph)
			{ 
				Graphs.Add(Graph);
			}
		}

		bool bFound = false;
		for (UNiagaraGraph* Graph : Graphs)
		{
			const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& RefMap = Graph->GetParameterReferenceMap();
			const FNiagaraGraphParameterReferenceCollection* Coll = RefMap.Find(InOldVar);
			if (Coll)
			{
				bFound = true;
				break;
			}
		}

		if (!bFound)
		{
			if (const FNiagaraParameterPanelItem* FoundItemPtr = CachedViewedItems.FindByPredicate([&InOldVar](const FNiagaraParameterPanelItem& Item) { return static_cast<const FNiagaraVariableBase&>(Item.GetVariable()) == InOldVar; }))
			{
				RenameParameter(*FoundItemPtr, InNewVar.GetName());
			}
		}
	}

	Refresh();
}

void FNiagaraSystemToolkitParameterPanelViewModel::OnParameterRemovedExternally(const FNiagaraVariableBase& InOldVar,  UNiagaraEmitter* InOptionalEmitter)
{
	// See if this was the last reference to that parameter being renamed, if so, we need to update to a full rename and rename all locations where it was used that are downstream, like renderer bindings.

	// Emitter & Particle namespaces are just for the ones actively being worked on.
	if (InOldVar.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString) ||
		InOldVar.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString))
	{
		const TArray<FGuid>& SelectedEmitterHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();

		// Note that we might have multiple selections and don't know explicitly which one changed, so we have to go through all independently and examine them.
		if (SelectedEmitterHandleIds.Num() > 0)
		{
			const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& EmitterHandleViewModels = SystemViewModel->GetEmitterHandleViewModels();
			for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
			{
				if (SelectedEmitterHandleIds.Contains(EmitterHandleViewModel->GetId()))
				{
					bool bFound = false;
					FVersionedNiagaraEmitter VersionedEmitter = EmitterHandleViewModel->GetEmitterHandle()->GetInstance();
					UNiagaraGraph* Graph = static_cast<UNiagaraScriptSource*>(VersionedEmitter.GetEmitterData()->GraphSource)->NodeGraph;
					if (Graph)
					{
						const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& RefMap = Graph->GetParameterReferenceMap();
						const FNiagaraGraphParameterReferenceCollection* Coll = RefMap.Find(InOldVar);
						if (Coll)
						{
							bFound = true;
							break;
						}
					}

					if (!bFound)
					{
						VersionedEmitter.Emitter->HandleVariableRemoved(InOldVar, true, VersionedEmitter.Version);
					}
				}
			}
		}
	}
	// User and System need to be checked for all graphs as they could be used anywhere.
	else if (InOldVar.IsInNameSpace(FNiagaraConstants::UserNamespaceString) ||
		InOldVar.IsInNameSpace(FNiagaraConstants::SystemNamespaceString))
	{		
		TArray<UNiagaraGraph*> Graphs;
		Graphs.Add(SystemScriptGraph.Get());

		const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& EmitterHandleViewModels = SystemViewModel->GetEmitterHandleViewModels();
		for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
		{
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandleViewModel->GetEmitterHandle()->GetEmitterData();
			if (EmitterData && EmitterData->GraphSource)
			{
				if (UNiagaraGraph* Graph = static_cast<UNiagaraScriptSource*>(EmitterData->GraphSource)->NodeGraph)
				{
					Graphs.Add(Graph);
				}
			}
		}

		bool bFound = false;
		for (UNiagaraGraph* Graph : Graphs)
		{
			const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& RefMap = Graph->GetParameterReferenceMap();
			const FNiagaraGraphParameterReferenceCollection* Coll = RefMap.Find(InOldVar);
			if (Coll)
			{
				bFound = true;
				break;
			}
		}

		if (!bFound)
		{
			SystemViewModel->GetSystem().HandleVariableRemoved(InOldVar, true);
		}
	}

	Refresh();
}

void FNiagaraSystemToolkitParameterPanelViewModel::ReconcileOnGraphChangedBindings()
{
	UNiagaraSystem& System = SystemViewModel->GetSystem();

	auto GetGraphFromScript = [](UNiagaraScript* Script)->UNiagaraGraph* {
		return CastChecked<UNiagaraScriptSource>(Script->GetLatestSource())->NodeGraph;
	};

	TSet<uint32> UnvisitedGraphIds;
	GraphIdToOnGraphChangedHandleMap.GetKeys(UnvisitedGraphIds);
	UnvisitedGraphIds.Remove(GetGraphFromScript(System.GetSystemSpawnScript())->GetUniqueID());
	UnvisitedGraphIds.Remove(GetGraphFromScript(System.GetSystemUpdateScript())->GetUniqueID());

	TSet<UNiagaraGraph*> GraphsToAddCallbacks;
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		if (FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle())
		{
			TArray<UNiagaraScript*> EmitterScripts;
			const bool bCompilableOnly = false;
			if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandle->GetEmitterData())
			{
				EmitterData->GetScripts(EmitterScripts, bCompilableOnly);
				for (UNiagaraScript* EmitterScript : EmitterScripts)
				{
					GraphsToAddCallbacks.Add(GetGraphFromScript(EmitterScript));
				}
			}
		}
	}

	for (UNiagaraGraph* Graph : GraphsToAddCallbacks)
	{
		UnvisitedGraphIds.Remove(Graph->GetUniqueID());
		if (GraphIdToOnGraphChangedHandleMap.Find(Graph->GetUniqueID()) == nullptr)
		{
			FDelegateHandle OnGraphChangedHandle = Graph->AddOnGraphChangedHandler(
				FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraSystemToolkitParameterPanelViewModel::OnGraphChanged));
			GraphIdToOnGraphChangedHandleMap.Add(Graph->GetUniqueID(), OnGraphChangedHandle);
		}
	}

	for (const uint32& UnvisitedGraphId : UnvisitedGraphIds)
	{
		GraphIdToOnGraphChangedHandleMap.Remove(UnvisitedGraphId);
	}
}

///////////////////////////////////////////////////////////////////////////////
/// Script Toolkit Parameter Panel View Model								///
///////////////////////////////////////////////////////////////////////////////

FNiagaraScriptToolkitParameterPanelViewModel::FNiagaraScriptToolkitParameterPanelViewModel(TSharedPtr<FNiagaraScriptViewModel> InScriptViewModel)
{
	ScriptViewModel = InScriptViewModel;
	VariableObjectSelection = ScriptViewModel->GetVariableSelection();

	RegisteredHandle = RegisterViewModelWithMap(InScriptViewModel->GetStandaloneScript().Script, this);
}

void FNiagaraScriptToolkitParameterPanelViewModel::Cleanup()
{
	UnregisterViewModelWithMap(RegisteredHandle);

	UNiagaraGraph* NiagaraGraph = static_cast<UNiagaraGraph*>(ScriptViewModel->GetGraphViewModel()->GetGraph());
	NiagaraGraph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
	NiagaraGraph->RemoveOnGraphNeedsRecompileHandler(OnGraphNeedsRecompileHandle);
	NiagaraGraph->OnSubObjectSelectionChanged().Remove(OnSubObjectSelectionHandle);

	ScriptViewModel->GetOnSubscribedParameterDefinitionsChangedDelegate().RemoveAll(this);
}

void FNiagaraScriptToolkitParameterPanelViewModel::Init(const FScriptToolkitUIContext& InUIContext)
{
	UIContext = InUIContext;

	// Init bindings
	UNiagaraGraph* NiagaraGraph = static_cast<UNiagaraGraph*>(ScriptViewModel->GetGraphViewModel()->GetGraph());
	OnGraphChangedHandle = NiagaraGraph->AddOnGraphChangedHandler(
		FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::OnGraphChanged));
	OnGraphNeedsRecompileHandle = NiagaraGraph->AddOnGraphNeedsRecompileHandler(
		FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::OnGraphChanged));
	OnSubObjectSelectionHandle = NiagaraGraph->OnSubObjectSelectionChanged().AddSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::OnGraphSubObjectSelectionChanged);

	ScriptViewModel->GetOnSubscribedParameterDefinitionsChangedDelegate().AddSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::Refresh);

	// Init default categories
	if (DefaultCategories.Num() == 0)
	{
		TArray<FNiagaraNamespaceMetadata> NamespaceMetadata = GetDefault<UNiagaraEditorSettings>()->GetAllNamespaceMetadata();
		for (const FNiagaraNamespaceMetadata& NamespaceMetadatum : NamespaceMetadata)
		{
			if (NamespaceMetadatum.Options.Contains(ENiagaraNamespaceMetadataOptions::HideInScript) == false)
			{
				if (NamespaceMetadatum.Options.Contains(ENiagaraNamespaceMetadataOptions::AdvancedInScript))
				{
					DefaultAdvancedCategories.Add(NamespaceMetadatum);
				}
				else
				{
					DefaultCategories.Add(NamespaceMetadatum);
					DefaultAdvancedCategories.Add(NamespaceMetadatum);
				}
			}
		}
	}
}

void FNiagaraScriptToolkitParameterPanelViewModel::AddParameter(FNiagaraVariable NewVariable, const FNiagaraParameterPanelCategory Category, const bool bRequestRename, const bool bMakeUniqueName) 
{
	TGuardValue<bool> AddParameterRefreshGuard(bIsAddingParameter, true);
	bool bSuccess = false;

	if(bMakeUniqueName)
	{ 
		TSet<FName> Names;
		for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
		{
			for (auto It = Graph->GetParameterReferenceMap().CreateConstIterator(); It; ++It)
			{
				Names.Add(It.Key().GetName());
			}
		}
		const FName NewUniqueName = FNiagaraUtilities::GetUniqueName(NewVariable.GetName(), Names);
		NewVariable.SetName(NewUniqueName);
	}

	FScopedTransaction AddTransaction(LOCTEXT("AddScriptParameterTransaction", "Add parameter to script."));
	for (UNiagaraGraph* Graph : GetEditableGraphs())
	{
		Graph->Modify();
		UNiagaraScriptVariable* NewScriptVar = Graph->AddParameter(NewVariable);
		bSuccess = true;

		// Check if the new parameter has the same name and type as an existing parameter definition, and if so, link to the definition automatically.
		FNiagaraParameterDefinitionsUtilities::TrySubscribeScriptVarToDefinitionByName(NewScriptVar, ScriptViewModel.Get());
	}

	if (bSuccess)
	{
		Refresh();
		SelectParameterItemByName(NewVariable.GetName(), bRequestRename);
	}
}

void FNiagaraScriptToolkitParameterPanelViewModel::FindOrAddParameter(FNiagaraVariable Variable, const FNiagaraParameterPanelCategory Category) 
{
	TGuardValue<bool> AddParameterRefreshGuard(bIsAddingParameter, true);
	bool bSuccess = false;

	FScopedTransaction AddTransaction(LOCTEXT("FindOrAddScriptParameterTransaction", "Add parameter to script."));
	for (UNiagaraGraph* Graph : GetEditableGraphs())
	{
		Graph->Modify();
		UNiagaraScriptVariable* NewScriptVar = Graph->AddParameter(Variable);
		bSuccess = true;

		// Check if the new parameter has the same name and type as an existing parameter definition, and if so, link to the definition automatically.
		FNiagaraParameterDefinitionsUtilities::TrySubscribeScriptVarToDefinitionByName(NewScriptVar, ScriptViewModel.Get());
	}

	if (bSuccess)
	{
		Refresh();
		SelectParameterItemByName(Variable.GetName(), false);
	}
	else
	{
		AddTransaction.Cancel();
	}
}

bool FNiagaraScriptToolkitParameterPanelViewModel::GetCanAddParametersToCategory(FNiagaraParameterPanelCategory Category) const
{
	return GetEditableGraphsConst().Num() > 0 && Category.NamespaceMetaData.GetGuid() != FNiagaraEditorGuids::StaticSwitchNamespaceMetaDataGuid;
}

void FNiagaraScriptToolkitParameterPanelViewModel::DeleteParameters(const TArray<FNiagaraParameterPanelItem>& ItemsToDelete) 
{
	bool bAnyChange = false;
	FScopedTransaction RemoveParametersWithPins(LOCTEXT("RemoveParametersWithPins", "Removed parameter(s) and referenced pins"));
	for(const FNiagaraParameterPanelItem& ItemToDelete : ItemsToDelete)
	{
		if (ItemToDelete.bExternallyReferenced)
		{
			continue;
		}			

		for (UNiagaraGraph* Graph : GetEditableGraphs())
		{
			UNiagaraScriptVariable* SelectedScriptVariable = Graph->GetScriptVariable(ItemToDelete.GetVariable());
			if (SelectedScriptVariable && VariableObjectSelection->GetSelectedObjects().Contains(SelectedScriptVariable))
			{
				VariableObjectSelection->ClearSelectedObjects();
			}
			Graph->Modify();
			Graph->RemoveParameter(ItemToDelete.GetVariable());
			
			if (!bAnyChange)
			{
				bAnyChange = true;
			}
		}
	}

	if(bAnyChange)
	{
		Refresh();
		UIContext.RefreshSelectionDetailsViewPanel();
	}
	else
	{
		RemoveParametersWithPins.Cancel();
	}
}

void FNiagaraScriptToolkitParameterPanelViewModel::RenameParameter(const FNiagaraParameterPanelItem& ItemToRename, const FName NewName) 
{
	if (ensureMsgf(ItemToRename.bExternallyReferenced == false, TEXT("Can not modify an externally referenced parameter.")) == false)
	{
		return;
	}
	else if (ItemToRename.GetVariable().GetName() == NewName)
	{
		return;
	}

	RenameParameter(ItemToRename.ScriptVariable, NewName);
}

void FNiagaraScriptToolkitParameterPanelViewModel::ChangeParameterType(const TArray<FNiagaraParameterPanelItem> ItemsToModify, const FNiagaraTypeDefinition NewType) 
{
	FScopedTransaction Transaction(LOCTEXT("ChangeParameterTypeTransaction", "Change parameter type"));

	TArray<FNiagaraVariable> VariablesToModify;

	for (const FNiagaraParameterPanelItem& PanelItem : ItemsToModify)
	{
		VariablesToModify.Add(PanelItem.GetVariable());
	}
	
	ScriptViewModel->GetStandaloneScript().Script->Modify();
	
	for (UNiagaraGraph* Graph : GetEditableGraphs())
	{
		Graph->Modify();
		Graph->ChangeParameterType(VariablesToModify, NewType, true);
	}

	UIContext.RefreshSelectionDetailsViewPanel();
	Refresh();
}

void FNiagaraScriptToolkitParameterPanelViewModel::RenameParameter(const UNiagaraScriptVariable* ScriptVarToRename, const FName NewName) 
{
	FScopedTransaction RenameTransaction(LOCTEXT("RenameParameterTransaction", "Rename parameter"));

	bool bSuccess = ScriptViewModel->RenameParameter(ScriptVarToRename->Variable, NewName);

	if (bSuccess)
	{
		Refresh();
		const bool bRequestRename = false;
		SelectParameterItemByName(NewName, bRequestRename);
	}
}

void FNiagaraScriptToolkitParameterPanelViewModel::RenameParameter(const FNiagaraVariable& VariableToRename, const FName NewName) 
{
	for (UNiagaraGraph* Graph : GetEditableGraphs())
	{
		if (TObjectPtr<UNiagaraScriptVariable> const* FoundScriptVarToRenamePtr = Graph->GetAllMetaData().Find(VariableToRename))
		{
			RenameParameter(*FoundScriptVarToRenamePtr, NewName);
			return;
		}
	}
}

void FNiagaraScriptToolkitParameterPanelViewModel::DuplicateParameters(const TArray<FNiagaraParameterPanelItem> ItemsToDuplicate) 
{
	FScopedTransaction Transaction(LOCTEXT("DuplicateParameterTransaction", "Duplicate parameter"));
	TGuardValue<bool> AddParameterRefreshGuard(bIsAddingParameter, true);
	bool bSuccess = false;

	FName SingleVariableName = NAME_None;
	for(const FNiagaraParameterPanelItem& ItemToDuplicate : ItemsToDuplicate)
	{
		TSet<FName> Names;
		for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
		{
			for (auto It = Graph->GetParameterReferenceMap().CreateConstIterator(); It; ++It)
			{
				Names.Add(It.Key().GetName());
			}
		}
		const FName NewUniqueName = FNiagaraUtilities::GetUniqueName(ItemToDuplicate.GetVariable().GetName(), Names);

		if(ItemsToDuplicate.Num() == 1 && SingleVariableName == NAME_None)
		{
			SingleVariableName = NewUniqueName;
		}
		
		FNiagaraVariable NewVariable(ItemToDuplicate.GetVariable().GetType(), NewUniqueName);
		FNiagaraVariableMetaData ParameterMetadata = ItemToDuplicate.ScriptVariable ? ItemToDuplicate.ScriptVariable->Metadata : FNiagaraVariableMetaData();
		ParameterMetadata.CreateNewGuid();
		
		for (UNiagaraGraph* Graph : GetEditableGraphs())
		{
			Graph->Modify();
			Graph->AddParameter(NewVariable, ParameterMetadata, false, false);
			bSuccess = true;
		}
	}

	if (bSuccess)
	{
		Refresh();

		if(SingleVariableName != NAME_None)
		{
			SelectParameterItemByName(SingleVariableName, true);
		}
	}
	else
	{
		Transaction.Cancel();
	}
}

FReply FNiagaraScriptToolkitParameterPanelViewModel::OnParameterItemsDragged(const TArray<FNiagaraParameterPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const
{
	if (OnGetParametersWithNamespaceModifierRenamePendingDelegate.IsBound() == false)
	{
		ensureMsgf(false, TEXT("OnGetParametersWithNamespaceModifierRenamePendingDelegate was not bound when handling parameter drag in parameter panel view model! "));
		return FReply::Handled();
	}

	if (DraggedItems.Num() == 1)
	{
		const FNiagaraParameterPanelItemBase& DraggedItem = DraggedItems[0];
		return FNiagaraScriptToolkitParameterPanelUtilities::CreateDragEventForParameterItem(
			DraggedItem,
			MouseEvent,
			GetGraphParameterReferencesForItem(DraggedItem),
			OnGetParametersWithNamespaceModifierRenamePendingDelegate.Execute()
		);
	}

	return FReply::Handled();
}

TSharedPtr<SWidget> FNiagaraScriptToolkitParameterPanelViewModel::CreateContextMenuForItems(const TArray<FNiagaraParameterPanelItem>& Items, const TSharedPtr<FUICommandList>& ToolkitCommands)
{
	if(Items.Num() == 0)
	{
		return nullptr;
	}
	
	// Create a menu with all relevant operations.
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, ToolkitCommands);
	
	const FNiagaraParameterPanelItem& SelectedItem = Items[0];
	const FNiagaraParameterPanelItemBase& SelectedItemBase = Items[0];

	FUIAction SingleItemAction;
	bool bSingleItemSelected = Items.Num() == 1;
	SingleItemAction.CanExecuteAction = FCanExecuteAction::CreateLambda([=]()
	{
		return bSingleItemSelected;
	});
	
	// helper lambda to add copy/paste metadata actions.
	auto AddMetaDataContextMenuEntries = [this, &MenuBuilder, &Items, &SelectedItem, &SelectedItemBase, SingleItemAction]() {
		FText CopyParameterMetaDataToolTip;
		const bool bCanCopyParameterMetaData = GetCanCopyParameterMetaDataAndToolTip(SelectedItem, CopyParameterMetaDataToolTip) && SingleItemAction.CanExecute();
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyParameterMetadata", "Copy Metadata"),
			CopyParameterMetaDataToolTip,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
			FUIAction(
				FExecuteAction::CreateSP(this, &INiagaraImmutableParameterPanelViewModel::CopyParameterMetaData, SelectedItemBase),
				FCanExecuteAction::CreateLambda([bCanCopyParameterMetaData]() { return bCanCopyParameterMetaData; })));

		FText PasteParameterMetaDataToolTip;
		const bool bCanPasteParameterMetaData = GetCanPasteParameterMetaDataAndToolTip(PasteParameterMetaDataToolTip) && SingleItemAction.CanExecute();
		MenuBuilder.AddMenuEntry(
			LOCTEXT("PasteParameterMetadata", "Paste Metadata"),
			PasteParameterMetaDataToolTip,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
			FUIAction(
				FExecuteAction::CreateSP(this, &INiagaraParameterPanelViewModel::PasteParameterMetaData, Items),
				FCanExecuteAction::CreateLambda([bCanPasteParameterMetaData]() { return bCanPasteParameterMetaData; })));
	};

	MenuBuilder.BeginSection("Edit", LOCTEXT("EditMenuHeader", "Edit"));
	{
		if (SelectedItem.ScriptVariable->GetIsStaticSwitch())
		{
			// Only allow modifying metadata for static switches.
			AddMetaDataContextMenuEntries();
		}
		else
		{ 
			FText CopyReferenceToolTip;
			GetCanCopyParameterReferenceAndToolTip(SelectedItem, CopyReferenceToolTip);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy, NAME_None, LOCTEXT("CopyReference", "Copy Reference"), CopyReferenceToolTip);

			FText DeleteToolTip;
			GetCanDeleteParameterAndToolTip(SelectedItem, DeleteToolTip);
			MenuBuilder.AddMenuEntry(FNiagaraParameterPanelCommands::Get().DeleteItem, NAME_None, TAttribute<FText>(), DeleteToolTip, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"));

			FText RenameToolTip;
			GetCanRenameParameterAndToolTip(SelectedItem, FText(), false, RenameToolTip);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("Rename", "Rename"), RenameToolTip);


			MenuBuilder.AddMenuSeparator();

			AddMetaDataContextMenuEntries();


			MenuBuilder.AddMenuSeparator();
			
			FText CanChangeTooltip;
			bool bCanChangeType = GetCanChangeParameterType(Items, CanChangeTooltip);
			FUIAction CanChangeTypeAction;
			CanChangeTypeAction.CanExecuteAction = FCanExecuteAction::CreateLambda([bCanChangeType]()
			{
				return bCanChangeType;
			});
			
			MenuBuilder.AddSubMenu(
				LOCTEXT("ChangeType", "Change Type"),
				CanChangeTooltip,
				FNewMenuDelegate::CreateSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::GetChangeTypeSubMenu, Items),
				CanChangeTypeAction, NAME_None, EUserInterfaceActionType::Button);

			MenuBuilder.AddSubMenu(
				LOCTEXT("ChangeNamespace", "Change Namespace"),
				LOCTEXT("ChangeNamespaceToolTip", "Select a new namespace for the selected parameter."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::GetChangeNamespaceSubMenu, false, SelectedItem),
				SingleItemAction, NAME_None, EUserInterfaceActionType::None);

			MenuBuilder.AddSubMenu(
				LOCTEXT("ChangeNamespaceModifier", "Change Namespace Modifier"),
				LOCTEXT("ChangeNamespaceModifierToolTip", "Edit the namespace modifier for the selected parameter."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::GetChangeNamespaceModifierSubMenu, false, SelectedItem),
				SingleItemAction, NAME_None, EUserInterfaceActionType::None);

			MenuBuilder.AddMenuSeparator();

			FText DuplicateToolTip;
			bool bCanDuplicateParameter = GetCanDuplicateParameterAndToolTip(Items, DuplicateToolTip);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DuplicateParameter", "Duplicate"),
				DuplicateToolTip,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"),
				FUIAction(
					FExecuteAction::CreateSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::DuplicateParameters, Items),
					FCanExecuteAction::CreateLambda([bCanDuplicateParameter]() {return bCanDuplicateParameter; })));

			MenuBuilder.AddSubMenu(
				LOCTEXT("DuplicateToNewNamespace", "Duplicate to Namespace"),
				LOCTEXT("DuplicateToNewNamespaceToolTip", "Duplicate this parameter to a new namespace."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::GetChangeNamespaceSubMenu, true, SelectedItem),
				SingleItemAction, NAME_None, EUserInterfaceActionType::None);

			MenuBuilder.AddSubMenu(
				LOCTEXT("DuplicateWithNewNamespaceModifier", "Duplicate with Namespace Modifier"),
				LOCTEXT("DupilcateWithNewNamespaceModifierToolTip", "Duplicate this parameter with a different namespace modifier."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraScriptToolkitParameterPanelViewModel::GetChangeNamespaceModifierSubMenu, true, SelectedItem),
				SingleItemAction, NAME_None, EUserInterfaceActionType::None);
		}
	}
	
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

FNiagaraParameterUtilities::EParameterContext FNiagaraScriptToolkitParameterPanelViewModel::GetParameterContext() const
{
	return FNiagaraParameterUtilities::EParameterContext::Script;
}

void FNiagaraScriptToolkitParameterPanelViewModel::OnParameterItemSelected(const FNiagaraParameterPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const
{
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		if (UNiagaraScriptVariable* SelectedScriptVariable = Graph->GetScriptVariable(SelectedItem.GetVariable()))
		{
			VariableObjectSelection->SetSelectedObject(SelectedScriptVariable);
			return;
		}
	}
}

const TArray<FNiagaraParameterPanelCategory>& FNiagaraScriptToolkitParameterPanelViewModel::GetDefaultCategories() const
{
// 	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
// 	const bool bShowAdvanced = NiagaraEditorSettings->GetDisplayAdvancedParameterPanelCategories();
// 	if (bShowAdvanced)
// 	{
// 		CachedCurrentCategories = FNiagaraScriptToolkitParameterPanelViewModel::DefaultAdvancedCategories;
// 		return FNiagaraScriptToolkitParameterPanelViewModel::DefaultAdvancedCategories;
// 	}
	// For now always show advanced.

	CachedCurrentCategories = FNiagaraScriptToolkitParameterPanelViewModel::DefaultAdvancedCategories;
	return CachedCurrentCategories;
}

FMenuAndSearchBoxWidgets FNiagaraScriptToolkitParameterPanelViewModel::GetParameterMenu(FNiagaraParameterPanelCategory Category) 
{
	const bool bRequestRename = true;
	const bool bSkipSubscribedLibraries = false;
	const bool bMakeUniqueName = true;

	TSharedPtr<SNiagaraAddParameterFromPanelMenu> MenuWidget = SAssignNew(ParameterMenuWidget, SNiagaraAddParameterFromPanelMenu)
		.Graphs(GetEditableGraphsConst())
		.AvailableParameterDefinitions(ScriptViewModel->GetAvailableParameterDefinitions(bSkipSubscribedLibraries))
 		.SubscribedParameterDefinitions(ScriptViewModel->GetSubscribedParameterDefinitions())
		.OnNewParameterRequested(this, &FNiagaraScriptToolkitParameterPanelViewModel::AddParameter, Category, bRequestRename, bMakeUniqueName)
		.OnSpecificParameterRequested(this, &FNiagaraScriptToolkitParameterPanelViewModel::FindOrAddParameter, Category)
		.OnAddScriptVar(this, &FNiagaraScriptToolkitParameterPanelViewModel::AddScriptVariable)
		.OnAddParameterDefinitions(this, &FNiagaraScriptToolkitParameterPanelViewModel::AddParameterDefinitions)
		.OnAllowMakeType(this, &INiagaraParameterPanelViewModel::CanAddType, TOptional<FNiagaraParameterPanelCategory>(Category))
		.NamespaceId(Category.NamespaceMetaData.GetGuid())
		.ShowNamespaceCategory(false)
		.ShowGraphParameters(false)
		.AutoExpandMenu(false)
		.CullParameterActionsAlreadyInGraph(true);

	ParameterMenuSearchBoxWidget = MenuWidget->GetSearchBox();
	FMenuAndSearchBoxWidgets MenuAndSearchBoxWidgets;
	MenuAndSearchBoxWidgets.MenuWidget = MenuWidget;
	MenuAndSearchBoxWidgets.MenuSearchBoxWidget = ParameterMenuSearchBoxWidget;

	return MenuAndSearchBoxWidgets;
}

FReply FNiagaraScriptToolkitParameterPanelViewModel::HandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation) 
{
	TSharedPtr<FNiagaraParameterGraphDragOperation> ParameterGraphDragDropOperation = StaticCastSharedPtr<FNiagaraParameterGraphDragOperation>(DragDropOperation);
	if (ParameterGraphDragDropOperation.IsValid() == false)
	{
		return FReply::Handled();
	}

	TSharedPtr<FEdGraphSchemaAction> SourceAction = ParameterGraphDragDropOperation->GetSourceAction();
	if (SourceAction.IsValid() == false)
	{
		return FReply::Handled();
	}

	TSharedPtr<FNiagaraParameterAction> SourceParameterAction = StaticCastSharedPtr<FNiagaraParameterAction>(SourceAction);
	if (SourceParameterAction.IsValid() == false)
	{
		return FReply::Handled();
	}

	AddScriptVariable(SourceParameterAction->GetScriptVar());
	return FReply::Handled();
}

bool FNiagaraScriptToolkitParameterPanelViewModel::GetCanHandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation) const
{
	if (DragDropOperation->IsOfType<FNiagaraParameterGraphDragOperation>() == false)
	{
		return false;
	}
	TSharedPtr<FNiagaraParameterGraphDragOperation> ParameterGraphDragDropOperation = StaticCastSharedPtr<FNiagaraParameterGraphDragOperation>(DragDropOperation);

	const TSharedPtr<FEdGraphSchemaAction>& SourceAction = ParameterGraphDragDropOperation->GetSourceAction();
	if (SourceAction.IsValid() == false)
	{
		return false;
	}

	if (SourceAction->GetTypeId() != FNiagaraEditorStrings::FNiagaraParameterActionId)
	{
		return false;
	}
	const TSharedPtr<FNiagaraParameterAction>& SourceParameterAction = StaticCastSharedPtr<FNiagaraParameterAction>(SourceAction);

	const UNiagaraScriptVariable* ScriptVar = SourceParameterAction->GetScriptVar();
	if (ScriptVar == nullptr)
	{
		return false;
	}

	// Do not allow trying to create a new parameter from the drop action if that parameter name/type pair already exists.
	const FNiagaraVariable& Parameter = ScriptVar->Variable;
	if (ScriptViewModel->GetAllScriptVars().ContainsByPredicate([Parameter](const UNiagaraScriptVariable* ScriptVar) { return ScriptVar->Variable == Parameter; }))
	{
		return false;
	}

	return true;
}

bool FNiagaraScriptToolkitParameterPanelViewModel::GetCanSetParameterNamespaceAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NewNamespace, FText& OutCanSetParameterNamespaceToolTip) const
{
	return FNiagaraParameterPanelUtilities::GetCanSetParameterNamespaceAndToolTipForScriptOrSystem(ItemToModify, NewNamespace, OutCanSetParameterNamespaceToolTip);
}

bool FNiagaraScriptToolkitParameterPanelViewModel::GetCanSetParameterNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NamespaceModifier, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const
{
	return FNiagaraParameterPanelUtilities::GetCanSetParameterNamespaceModifierAndToolTipForScriptOrSystem(CachedViewedItems, ItemToModify, NamespaceModifier, bDuplicateParameter, OutCanSetParameterNamespaceModifierToolTip);
}

bool FNiagaraScriptToolkitParameterPanelViewModel::GetCanSetParameterCustomNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const
{
	return FNiagaraParameterPanelUtilities::GetCanSetParameterCustomNamespaceModifierAndToolTipForScriptOrSystem(ItemToModify, bDuplicateParameter, OutCanSetParameterNamespaceModifierToolTip);
}

void FNiagaraScriptToolkitParameterPanelViewModel::SetParameterIsOverridingLibraryDefaultValue(const FNiagaraParameterPanelItem ItemToModify, const bool bOverriding) 
{
	if (ensureMsgf(ItemToModify.bExternallyReferenced == false, TEXT("Cannot modify an externally referenced parameter.")) == false)
	{
		return;
	}

	const FText TransactionText = bOverriding ? LOCTEXT("OverrideParameterDefinitionDefaulValue", "Override Parameter Definition Default") : LOCTEXT("UseParameterDefinitionDefaultValue", "Stop Overriding Parameter Definition Default");
	FScopedTransaction OverrideTransaction(TransactionText);
	ScriptViewModel->GetStandaloneScript().Script->Modify();
	ScriptViewModel->SetParameterIsOverridingLibraryDefaultValue(ItemToModify.ScriptVariable->Metadata.GetVariableGuid(), bOverriding);
	Refresh();
	UIContext.RefreshParameterDefinitionsPanel();
	UIContext.RefreshSelectionDetailsViewPanel();
}

TArray<UNiagaraGraph*> FNiagaraScriptToolkitParameterPanelViewModel::GetEditableGraphs() const
{
	TArray<UNiagaraGraph*> EditableGraphs;
	if (UNiagaraGraph* Graph = ScriptViewModel->GetGraphViewModel()->GetGraph())
	{
		EditableGraphs.Add(Graph);
	}
	return EditableGraphs;
}

const TArray<UNiagaraScriptVariable*> FNiagaraScriptToolkitParameterPanelViewModel::GetEditableScriptVariablesWithName(const FName ParameterName) const
{
	TArray<UNiagaraScriptVariable*> EditableScriptVariables;
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		UNiagaraScriptVariable* ParameterScriptVariable = Graph->GetScriptVariable(ParameterName);
		if (ParameterScriptVariable != nullptr)
		{
			EditableScriptVariables.Add(ParameterScriptVariable);
		}
	}
	return EditableScriptVariables;
}

TArray<FNiagaraVariable> FNiagaraScriptToolkitParameterPanelViewModel::GetEditableStaticSwitchParameters() const
{
	TArray<FNiagaraVariable> OutStaticSwitchParameters;
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		OutStaticSwitchParameters.Append(Graph->FindStaticSwitchInputs());
	}
	return OutStaticSwitchParameters;
}

const TArray<FNiagaraGraphParameterReference> FNiagaraScriptToolkitParameterPanelViewModel::GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const
{
	TArray<FNiagaraGraphParameterReference> ParameterReferences;
	for (const UNiagaraGraph* Graph : GetEditableGraphsConst())
	{
		const FNiagaraGraphParameterReferenceCollection* GraphParameterReferenceCollectionPtr = Graph->GetParameterReferenceMap().Find(Item.GetVariable());
		if (GraphParameterReferenceCollectionPtr)
		{
			ParameterReferences.Append(GraphParameterReferenceCollectionPtr->ParameterReferences);
		}
	}
	return ParameterReferences;
}

const TArray<UNiagaraParameterDefinitions*> FNiagaraScriptToolkitParameterPanelViewModel::GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions) const
{
	return ScriptViewModel->GetAvailableParameterDefinitions(bSkipSubscribedParameterDefinitions);
}

TArray<FNiagaraParameterPanelItem> FNiagaraScriptToolkitParameterPanelViewModel::GetViewedParameterItems() const
{
	// On the first time opening the parameter panel view model we are not guaranteed to call GetDefaultCategories() before GetViewedParameterItems(). 
	// We require CachedCurrentCategories being set as this is used to filter out parameter items that are being viewed. If CachedCurrentCategories 
	// is not set, call GetDefaultCategories() to initialize it. 
	if (CachedCurrentCategories.Num() == 0)
	{
		GetDefaultCategories();
	}

	TMap<FNiagaraVariable, FNiagaraParameterPanelItem> VisitedParameterToItemMap;
	TArray<FNiagaraVariable> VisitedInvalidParameters;

	TArray<UNiagaraGraph*> EditableGraphs = GetEditableGraphsConst();
	for(UNiagaraGraph* Graph : EditableGraphs)
	{
		const TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>& ParameterMetaData = Graph->GetAllMetaData();
		const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& ReferenceCollection = Graph->GetParameterReferenceMap();

		for(const auto& ParameterMetaDataPair : ParameterMetaData)
		{
			const FNiagaraVariable Parameter = ParameterMetaDataPair.Key;
			const UNiagaraScriptVariable* ScriptVariable = ParameterMetaDataPair.Value;

			// If this variable has already been visited and does not have a valid namespace then skip it.
			if (VisitedInvalidParameters.Contains(Parameter))
			{
				continue;
			}

			// -Now make sure the variable namespace is in a valid category. If not, skip it.
			FNiagaraNamespaceMetadata CandidateNamespaceMetaData;
			if (ScriptVariable->GetIsStaticSwitch())
			{
				CandidateNamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForId(FNiagaraEditorGuids::StaticSwitchNamespaceMetaDataGuid);
			}
			else
			{
				CandidateNamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(Parameter.GetName());
			}

			if (CachedCurrentCategories.Contains(FNiagaraParameterPanelCategory(CandidateNamespaceMetaData)) == false)
			{
				VisitedInvalidParameters.Add(Parameter);
				continue;
			}

			FNiagaraParameterPanelItem Item;
			Item.ScriptVariable = ScriptVariable;
			Item.NamespaceMetaData = CandidateNamespaceMetaData;

			// Determine whether the item is name aliasing a parameter definition's parameter.
			Item.DefinitionMatchState = FNiagaraParameterDefinitionsUtilities::GetDefinitionMatchStateForParameter(Parameter);

			// Set the reference count. It is possible a parameter has no entry in the reference collection,
			// for example a static switch that was added via propagation but then removed again will have no references. There might be other exceptions, too.
			if(ReferenceCollection.Contains(Parameter))
			{
				for (const FNiagaraGraphParameterReference& Reference : ReferenceCollection[Parameter].ParameterReferences)
				{
					if (Cast<UNiagaraNodeParameterMapGet>(Reference.Value.Get()))
					{
						Item.ReadReferenceCount++;
					}
					else
					{
						Item.WriteReferenceCount++;
					}
				}
			}

			VisitedParameterToItemMap.Add(Parameter, Item);
		}
	}

	// Refresh the CachedViewedItems and return that as the latest array of viewed items.
	VisitedParameterToItemMap.GenerateValueArray(CachedViewedItems);
	return CachedViewedItems;
}

void FNiagaraScriptToolkitParameterPanelViewModel::AddScriptVariable(const UNiagaraScriptVariable* NewScriptVar) 
{
	bool bSuccess = false;
	FScopedTransaction AddTransaction(LOCTEXT("AddScriptParameterTransaction", "Add parameter to script."));
	for (UNiagaraGraph* Graph : GetEditableGraphs())
	{
		Graph->Modify();
		Graph->AddParameter(NewScriptVar);
		bSuccess = true;
	}

	if (bSuccess)
	{
		Refresh();
		const bool bRequestRename = false;
		SelectParameterItemByName(NewScriptVar->Variable.GetName(), bRequestRename);
	}
}

void FNiagaraScriptToolkitParameterPanelViewModel::AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const
{
	FScopedTransaction AddTransaction(LOCTEXT("AddParameterDefinitions", "Add parameter definitions."));
	ScriptViewModel->GetStandaloneScript().Script->Modify();
	ScriptViewModel->SubscribeToParameterDefinitions(NewParameterDefinitions);
	UIContext.RefreshParameterDefinitionsPanel();
}

void FNiagaraScriptToolkitParameterPanelViewModel::RemoveParameterDefinitions(const FGuid& ParameterDefinitionsToRemoveId) const
{
	FScopedTransaction RemoveTransaction(LOCTEXT("RemoveParameterDefinitions", "Remove parameter definitions."));
	ScriptViewModel->GetStandaloneScript().Script->Modify();
	ScriptViewModel->UnsubscribeFromParameterDefinitions(ParameterDefinitionsToRemoveId);
	Refresh();
	UIContext.RefreshParameterDefinitionsPanel();
}

void FNiagaraScriptToolkitParameterPanelViewModel::OnGraphChanged(const struct FEdGraphEditAction& InAction) const
{
	RefreshNextTick();
}

void FNiagaraScriptToolkitParameterPanelViewModel::OnGraphSubObjectSelectionChanged(const UObject* Obj) const
{
	OnParameterPanelViewModelExternalSelectionChangedDelegate.Broadcast(Obj);
}

///////////////////////////////////////////////////////////////////////////////
/// Parameter Definitions Toolkit Parameter Panel View Model				///
///////////////////////////////////////////////////////////////////////////////

FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::FNiagaraParameterDefinitionsToolkitParameterPanelViewModel(UNiagaraParameterDefinitions* InParameterDefinitions, const TSharedPtr<FNiagaraObjectSelection>& InObjectSelection)
{
	ParameterDefinitionsWeak = InParameterDefinitions;
	VariableObjectSelection = InObjectSelection;
}

void FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::Init(const FParameterDefinitionsToolkitUIContext& InUIContext)
{
	UIContext = InUIContext;

	ParameterDefinitionsWeak.Get()->GetOnParameterDefinitionsChanged().AddSP(this, &FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::Refresh);

	// Init default categories
	if (DefaultCategories.Num() == 0)
	{
		DefaultCategories.Add(FNiagaraEditorUtilities::GetNamespaceMetaDataForId(FNiagaraEditorGuids::SystemNamespaceMetaDataGuid));
		DefaultCategories.Add(FNiagaraEditorUtilities::GetNamespaceMetaDataForId(FNiagaraEditorGuids::EmitterNamespaceMetaDataGuid));
		DefaultCategories.Add(FNiagaraEditorUtilities::GetNamespaceMetaDataForId(FNiagaraEditorGuids::ParticleAttributeNamespaceMetaDataGuid));
	}
}

void FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::Cleanup()
{
	if (ParameterDefinitionsWeak.IsValid())
	{
		ParameterDefinitionsWeak.Get()->GetOnParameterDefinitionsChanged().RemoveAll(this);
	}
}

void FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::AddParameter(FNiagaraVariable NewVariable, const FNiagaraParameterPanelCategory Category, const bool bRequestRename, const bool bMakeUniqueName) 
{
	TGuardValue<bool> AddParameterRefreshGuard(bIsAddingParameter, true);

	if (bMakeUniqueName)
	{
		TSet<FName> Names;
		for (const UNiagaraScriptVariable* ScriptVar : ParameterDefinitionsWeak.Get()->GetParametersConst())
		{
			Names.Add(ScriptVar->Variable.GetName());
		}
		const FName NewUniqueName = FNiagaraUtilities::GetUniqueName(NewVariable.GetName(), Names);
		NewVariable.SetName(NewUniqueName);
	}
	
	ParameterDefinitionsWeak.Get()->AddParameter(NewVariable);
	SelectParameterItemByName(NewVariable.GetName(), bRequestRename);

}

void FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::FindOrAddParameter(FNiagaraVariable Variable, const FNiagaraParameterPanelCategory Category) 
{
	TGuardValue<bool> AddParameterRefreshGuard(bIsAddingParameter, true);

	if(!ParameterDefinitionsWeak.Get()->HasParameter(Variable))
	{
		ParameterDefinitionsWeak.Get()->FindOrAddParameter(Variable);
	}
	
	SelectParameterItemByName(Variable.GetName(), false);
}

bool FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetCanAddParametersToCategory(FNiagaraParameterPanelCategory Category) const
{
	return true;
}

void FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::DeleteParameters(const TArray<FNiagaraParameterPanelItem>& ItemsToDelete) 
{
	FScopedTransaction RemoveParameter(LOCTEXT("ParameterDefinitionsToolkitParameterPanelViewModel_RemoveParameter", "Removed Parameter(s)"));

	for(const FNiagaraParameterPanelItem& ItemToDelete : ItemsToDelete)
	{
		ParameterDefinitionsWeak.Get()->RemoveParameter(ItemToDelete.GetVariable());
	}
	VariableObjectSelection->ClearSelectedObjects();
	UIContext.RefreshSelectionDetailsViewPanel();
}

void FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::RenameParameter(const FNiagaraParameterPanelItem& ItemToRename, const FName NewName) 
{
	FScopedTransaction RenameParameter(LOCTEXT("ParameterDefinitionsToolkitParameterPanelViewModel_RenameParameter", "Rename Parameter"));
	ParameterDefinitionsWeak.Get()->RenameParameter(ItemToRename.GetVariable(), NewName);
}

TSharedPtr<SWidget> FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::CreateContextMenuForItems(const TArray<FNiagaraParameterPanelItem>& Items, const TSharedPtr<FUICommandList>& ToolkitCommands)
{
	// Only create context menus when a single item is selected.
	if (Items.Num() == 1)
	{
		const FNiagaraParameterPanelItem& SelectedItem = Items[0];

		// Create a menu with all relevant operations.
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, ToolkitCommands);
		MenuBuilder.BeginSection("Edit", LOCTEXT("EditMenuHeader", "Edit"));
		{
			FText CopyReferenceToolTip;
			GetCanCopyParameterReferenceAndToolTip(SelectedItem, CopyReferenceToolTip);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy, NAME_None, LOCTEXT("CopyReference", "Copy Reference"), CopyReferenceToolTip);

			FText DeleteToolTip;
			GetCanDeleteParameterAndToolTip(SelectedItem, DeleteToolTip);
			MenuBuilder.AddMenuEntry(FNiagaraParameterPanelCommands::Get().DeleteItem, NAME_None, TAttribute<FText>(), DeleteToolTip, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"));

			FText RenameToolTip;
			GetCanRenameParameterAndToolTip(SelectedItem, FText(), false, RenameToolTip);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("Rename", "Rename"), RenameToolTip);


			MenuBuilder.AddMenuSeparator();

			MenuBuilder.AddSubMenu(
				LOCTEXT("ChangeNamespace", "Change Namespace"),
				LOCTEXT("ChangeNamespaceToolTip", "Select a new namespace for the selected parameter."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetChangeNamespaceSubMenu, false, SelectedItem));

			MenuBuilder.AddSubMenu(
				LOCTEXT("ChangeNamespaceModifier", "Change Namespace Modifier"),
				LOCTEXT("ChangeNamespaceModifierToolTip", "Edit the namespace modifier for the selected parameter."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetChangeNamespaceModifierSubMenu, false, SelectedItem));


			MenuBuilder.AddMenuSeparator();

			FText DuplicateToolTip;
			bool bCanDuplicateParameter = GetCanDuplicateParameterAndToolTip(Items, DuplicateToolTip);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DuplicateParameter", "Duplicate"),
				DuplicateToolTip,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"),
				FUIAction(
					FExecuteAction::CreateSP(this, &FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::DuplicateParameters, Items),
					FCanExecuteAction::CreateLambda([bCanDuplicateParameter]() {return bCanDuplicateParameter; })));
			
			MenuBuilder.AddSubMenu( 
				LOCTEXT("DuplicateToNewNamespace", "Duplicate to Namespace"),
				LOCTEXT("DuplicateToNewNamespaceToolTip", "Duplicate this parameter to a new namespace."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetChangeNamespaceSubMenu, true, SelectedItem));

			MenuBuilder.AddSubMenu(
				LOCTEXT("DuplicateWithNewNamespaceModifier", "Duplicate with Namespace Modifier"),
				LOCTEXT("DupilcateWithNewNamespaceModifierToolTip", "Duplicate this parameter with a different namespace modifier."),
				FNewMenuDelegate::CreateSP(this, &FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetChangeNamespaceModifierSubMenu, true, SelectedItem));
		}
		MenuBuilder.EndSection();
		return MenuBuilder.MakeWidget();
	}
	// More than one item selected, do not return a context menu.
	return SNullWidget::NullWidget;
}

FNiagaraParameterUtilities::EParameterContext FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetParameterContext() const
{
	return FNiagaraParameterUtilities::EParameterContext::Definitions;
}

const TArray<UNiagaraScriptVariable*> FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetEditableScriptVariablesWithName(const FName ParameterName) const
{
	const TArray<UNiagaraScriptVariable*>& LibraryParameters = ParameterDefinitionsWeak.Get()->GetParametersConst();
	return LibraryParameters.FilterByPredicate([ParameterName](const UNiagaraScriptVariable* ScriptVariable){return ScriptVariable->Variable.GetName() == ParameterName;});
}

const TArray<FNiagaraGraphParameterReference> FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const
{
	return TArray<FNiagaraGraphParameterReference>();
}

const TArray<UNiagaraParameterDefinitions*> FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions) const
{
	// NOTE: Parameter library toolkit does not subscribe to parameter libraries directly and as such returns an empty array.
	return TArray<UNiagaraParameterDefinitions*>();
}

TArray<FNiagaraVariable> FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetEditableStaticSwitchParameters() const
{
	// Parameter libraries do not have static switch parameters.
	return TArray<FNiagaraVariable>();
}

TArray<FNiagaraParameterPanelItem> FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetViewedParameterItems() const
{
	CachedViewedItems.Reset();
	for (const UNiagaraScriptVariable* const& ScriptVar : ParameterDefinitionsWeak.Get()->GetParametersConst())
	{
		FNiagaraParameterPanelItem& Item = CachedViewedItems.AddDefaulted_GetRef();
		Item.ScriptVariable = ScriptVar;
		Item.NamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(ScriptVar->Variable.GetName());
		Item.ReadReferenceCount = 1;
		Item.DefinitionMatchState = EParameterDefinitionMatchState::MatchingOneDefinition;
	}
	return CachedViewedItems;
}

bool FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetCanRenameParameterAndToolTip(const FNiagaraParameterPanelItem& ItemToRename, const FText& NewVariableNameText, bool bCheckEmptyNameText, FText& OutCanRenameParameterToolTip) const
{
	const FName NewVariableName = FName(*NewVariableNameText.ToString());
	if(ItemToRename.GetVariable().GetName() != NewVariableName)
	{ 
		if (CachedViewedItems.ContainsByPredicate([NewVariableName](const FNiagaraParameterPanelItem& Item) {return Item.GetVariable().GetName() == NewVariableName; }))
		{
			OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelLibraryViewModel_RenameParameter_NameAlias", "Cannot Rename Parameter: A Parameter with this name already exists in this definition asset.");
			return false;
		}
		else if (FNiagaraParameterDefinitionsUtilities::GetNumParametersReservedForName(NewVariableName) > 0)
		{
			OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelLibraryViewModel_RenameParameter_ReservedDefinitionName", "Cannot Rename Parameter: A Parameter with this name already exists in another definition asset.");
			return false;
		}
	}

	if (bCheckEmptyNameText && NewVariableNameText.IsEmptyOrWhitespace())
	{
		// The incoming name text will contain the namespace even if the parameter name entry is empty, so make a parameter handle to split out the name.
		const FNiagaraParameterHandle NewVariableNameHandle = FNiagaraParameterHandle(NewVariableName);
		if (NewVariableNameHandle.GetName().IsNone())
		{
			OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelLibraryViewModel_RenameParameter_NameNone", "Parameter must have a name.");
			return false;
		}
	}

	OutCanRenameParameterToolTip = LOCTEXT("ParameterPanelViewModel_RenameParameter_CreatedInDefinition", "Rename this Parameter for all Systems, Emitters and Scripts using this Definition.");
	return true;
}

bool FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetCanSetParameterNamespaceAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NewNamespace, FText& OutCanSetParameterNamespaceToolTip) const
{
	return true;
}

bool FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetCanSetParameterNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NamespaceModifier, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const
{
	if (FNiagaraParameterUtilities::TestCanSetSpecificNamespaceModifierWithMessage(ItemToModify.GetVariable().GetName(), NamespaceModifier, OutCanSetParameterNamespaceModifierToolTip) == false)
	{
		return false;
	}

	if (bDuplicateParameter == false)
	{
		if (NamespaceModifier != NAME_None)
		{
			FName NewName = FNiagaraParameterUtilities::SetSpecificNamespaceModifier(ItemToModify.GetVariable().GetName(), NamespaceModifier);
			if (CachedViewedItems.ContainsByPredicate([NewName](const FNiagaraParameterPanelItem& CachedViewedItem) {return CachedViewedItem.GetVariable().GetName() == NewName; }))
			{
				OutCanSetParameterNamespaceModifierToolTip = LOCTEXT("CantChangeNamespaceModifierAlreadyExits", "Can't set this namespace modifier because it would create a parameter that already exists.");
				return false;
			}
		}
	}

	return true;
}

bool FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetCanSetParameterCustomNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const
{
	if (FNiagaraParameterUtilities::TestCanSetCustomNamespaceModifierWithMessage(ItemToModify.GetVariable().GetName(), OutCanSetParameterNamespaceModifierToolTip) == false)
	{
		return false;
	}

	return true;
}

const TArray<FNiagaraParameterPanelCategory>& FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetDefaultCategories() const
{
	return FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::DefaultCategories;
}

FMenuAndSearchBoxWidgets FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetParameterMenu(FNiagaraParameterPanelCategory Category) 
{
	const bool bRequestRename = true;
	const bool bMakeUniqueName = true;

	TSharedPtr<SNiagaraAddParameterFromPanelMenu> MenuWidget = SAssignNew(ParameterMenuWidget, SNiagaraAddParameterFromPanelMenu)
	.OnNewParameterRequested(this, &FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::AddParameter, Category, bRequestRename, bMakeUniqueName)
	.OnSpecificParameterRequested(this, &FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::FindOrAddParameter, Category)
	.OnAllowMakeType(this, &INiagaraParameterPanelViewModel::CanAddType, TOptional<FNiagaraParameterPanelCategory>(Category))
	.NamespaceId(Category.NamespaceMetaData.GetGuid())
	.ShowNamespaceCategory(false)
	.ShowGraphParameters(false)
	.AutoExpandMenu(false);

	ParameterMenuSearchBoxWidget = MenuWidget->GetSearchBox();
	FMenuAndSearchBoxWidgets MenuAndSearchBoxWidgets;
	MenuAndSearchBoxWidgets.MenuWidget = MenuWidget;
	MenuAndSearchBoxWidgets.MenuSearchBoxWidget = ParameterMenuSearchBoxWidget;
	return MenuAndSearchBoxWidgets;
}

FReply FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::HandleDragDropOperation(TSharedPtr<FDragDropOperation> DropOperation) 
{
	ensureMsgf(false, TEXT("Tried to handle drag drop op in parameter definitions parameter panel viewmodel!"));
	return FReply::Handled();
}

bool FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::GetCanHandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation) const
{
	return false;
}

void FNiagaraParameterDefinitionsToolkitParameterPanelViewModel::OnParameterItemSelected(const FNiagaraParameterPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const
{
	if (UNiagaraScriptVariable* SelectedScriptVariable = ParameterDefinitionsWeak.Get()->GetScriptVariable(SelectedItem.GetVariable()))
	{
		VariableObjectSelection->SetSelectedObject(SelectedScriptVariable);
	}
}

#undef LOCTEXT_NAMESPACE // "FNiagaraParameterPanelViewModel"
