// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraParameterViewModel.h"

FNiagaraParameterViewModel::FNiagaraParameterViewModel(ENiagaraParameterEditMode InParameterEditMode)
	: ParameterEditMode(InParameterEditMode), 
	bIsEditingEnabled(true)
{
}

bool FNiagaraParameterViewModel::CanRenameParameter() const
{
	return ParameterEditMode == ENiagaraParameterEditMode::EditAll;
}

FText FNiagaraParameterViewModel::GetNameText() const
{
	return FText::FromName(GetName());
}

bool FNiagaraParameterViewModel::CanChangeParameterType() const
{
	return ParameterEditMode == ENiagaraParameterEditMode::EditAll;
}

bool FNiagaraParameterViewModel::CanChangeSortOrder() const
{
	return ParameterEditMode == ENiagaraParameterEditMode::EditAll;
}

FNiagaraParameterViewModel::FOnDefaultValueChanged& FNiagaraParameterViewModel::OnDefaultValueChanged()
{
	return OnDefaultValueChangedDelegate;
}

FText FNiagaraParameterViewModel::GetTooltip() const 
{
	if (TooltipOverride.IsEmpty())
	{
		return GetNameText();
	}
	else
	{
		return TooltipOverride;
	}
}

void FNiagaraParameterViewModel::SetTooltipOverride(const FText& InTooltipOverride)
{
	TooltipOverride = InTooltipOverride;
}
