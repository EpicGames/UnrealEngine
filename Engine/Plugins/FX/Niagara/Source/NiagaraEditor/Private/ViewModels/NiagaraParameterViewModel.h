// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/NiagaraParameterEditMode.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/Delegate.h"
#include "Styling/SlateTypes.h"
#include "UObject/StructOnScope.h"
#include "Internationalization/Text.h"

struct FNiagaraTypeDefinition;
struct FNiagaraVariable;
class UNiagaraDataInterface;

/** Defines the view model for a parameter in the parameter collection editor. */
class INiagaraParameterViewModel
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnDefaultValueChanged);
	DECLARE_MULTICAST_DELEGATE(FOnTypeChanged);
	DECLARE_MULTICAST_DELEGATE(FOnProvidedChanged);

public:
	// Defines the type of default value this parameter provides.
	enum class EDefaultValueType
	{
		None,
		Struct,
		DataInterface,
		ObjectAsset,
	};

public:
	virtual ~INiagaraParameterViewModel() = default;

	virtual FNiagaraVariable GetVariable() const = 0;
	
	/** Gets the name of the parameter. */
	virtual FName GetName() const = 0;

	/** Gets whether or not this parameter can be renamed. */
	virtual bool CanRenameParameter() const = 0;

	/** Gets the text representation of the name of the parameter. */
	virtual FText GetNameText() const = 0;

	/** Handles a name text change being committed from the UI. */
	virtual void NameTextCommitted(const FText& Name, ETextCommit::Type CommitInfo) = 0;

	/** Handles verification of an in-progress variable name change in the UI.*/
	virtual bool VerifyNodeNameTextChanged(const FText& NewText, FText& OutErrorMessage) = 0;

	/** Gets the display name for the parameter's type. */
	virtual FText GetTypeDisplayName() const = 0;

	/** Gets whether or not the type of this parameter can be changed. */
	virtual bool CanChangeParameterType() const = 0;

	/** Gets the type of the parameter. */
	virtual FNiagaraTypeDefinition GetType() const = 0;

	/** Handles the parameter type being changed from the UI. */
	virtual void SelectedTypeChanged(TSharedPtr<FNiagaraTypeDefinition> Item, ESelectInfo::Type SelectionType) = 0;

	/** Gets the type of default value this view model provides. */
	virtual EDefaultValueType GetDefaultValueType() = 0;

	/** Gets the struct representing the default value for the parameter. */
	virtual TSharedRef<FStructOnScope> GetDefaultValueStruct() = 0;

	/** Gets the data interface representing the default value for the parameter. */
	virtual UNiagaraDataInterface* GetDefaultValueDataInterface() = 0;

	/** Gets the object reference representing the default value for the parameter. */
	virtual TObjectPtr<UObject> GetDefaultValueObjectAsset() = 0;
	/** Sets the object reference representing the default value for the parameter. */
	virtual void SetDefaultValueObjectAsset(TObjectPtr<UObject>) = 0;

	/** Called to notify the parameter view model that a property on the default value has been changed by the UI. */
	virtual void NotifyDefaultValuePropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent) = 0;

	/** Called to notify the parameter view model that a change to the default value has begun. */
	virtual void NotifyBeginDefaultValueChange() = 0;

	/** Called to notify the parameter view model that a change to the default value has ended. */
	virtual void NotifyEndDefaultValueChange() = 0;

	/** Called to notify the parameter view model that the default value has been changed by the UI. */
	virtual void NotifyDefaultValueChanged() = 0;

	/** Get a multicast delegate which is called whenever the default value of the parameter changes. */
	virtual FOnDefaultValueChanged& OnDefaultValueChanged() = 0;

	/** Gets a multicast delegate which is called whenever the type of this parameter changes. */
	virtual FOnTypeChanged& OnTypeChanged() = 0;

	/** Gests a multicast delegate which is called when an optional parameter has it's provided state changed. */
	virtual FOnProvidedChanged& OnProvidedChanged() = 0;

	/** Gets whether or not this parameter is editable in this context.*/
	virtual bool IsEditingEnabled() const = 0;

	/** Sets whether or not this parameter is editable in this context.*/
	virtual void SetEditingEnabled(bool bEnabled) = 0;

	/** Gets the tooltip when hovering over this parameter.*/
	virtual FText GetTooltip() const = 0;

	/** Sets the state override tooltip text for this parameter. If this is set to empty string, we just use the name.*/
	virtual void SetTooltipOverride(const FText& InTooltipOverride) = 0;

	/** Gets the state override tooltip text for this parameter. If you want this to be invalid, set the tooltip override to empty string.*/
	virtual const FText& GetTooltipOverride() const = 0;

	/** Whether or not the sort order should be adjustable. */
	virtual bool CanChangeSortOrder() const = 0;

	/** The current sort order.*/
	virtual int32 GetSortOrder() const = 0;
	
	/** Set the current sort order*/
	virtual void SetSortOrder(int32 SortOrder) = 0;

	/** If this parameter is optional. */
	virtual bool IsOptional()const = 0;

	/** If an optional parameter is provided. */
	virtual ECheckBoxState IsProvided()const = 0;

	/** Changes the provided state for an optional parameter. */
	virtual void SetProvided(ECheckBoxState CheckboxState) = 0;
};

/** Base class for parameter view models.  Partially implements the parameter interface with
behavior common to all view models. */
class FNiagaraParameterViewModel : public INiagaraParameterViewModel
{
public:
	FNiagaraParameterViewModel(ENiagaraParameterEditMode ParameterEditMode);

	//~ INiagaraParameterViewModel interface
	virtual bool CanRenameParameter() const override;
	virtual FText GetNameText() const override;
	virtual bool CanChangeParameterType() const override;
	virtual bool CanChangeSortOrder() const override;
	virtual FOnDefaultValueChanged& OnDefaultValueChanged() override;
	virtual FOnTypeChanged& OnTypeChanged() override { return OnTypeChangedDelegate; }
	virtual FOnProvidedChanged& OnProvidedChanged() override { return OnProvidedChangedDelegate; }
	virtual bool IsEditingEnabled() const override { return bIsEditingEnabled; }
	virtual void SetEditingEnabled(bool bEnabled) override { bIsEditingEnabled = bEnabled; }
	virtual FText GetTooltip() const override;
	virtual void SetTooltipOverride(const FText& InTooltipOverride) override;
	virtual const FText& GetTooltipOverride() const override { return TooltipOverride; }
	virtual bool IsOptional()const override { return false; }
	virtual ECheckBoxState IsProvided()const override { return ECheckBoxState::Checked; }
	virtual void SetProvided(ECheckBoxState CheckboxState) override { /*Do nothing;*/ }
protected:
	/** Defines the edit mode for this parameter. */
	ENiagaraParameterEditMode ParameterEditMode;

	/** A multicast delegate which is called whenever the default value changes. */
	FOnDefaultValueChanged OnDefaultValueChangedDelegate;

	/** A multicast delegate which is called whenever the type of the parameter changes. */
	FOnTypeChanged OnTypeChangedDelegate;

	/** A multicast delegate which is called whenever an optional parameter is toggled between provided and not. */
	FOnProvidedChanged OnProvidedChangedDelegate;
	
	/** Whether or not editing this view model is enabled.*/
	bool bIsEditingEnabled;

	/** Override of the default tooltip specified externally.*/
	FText TooltipOverride;
};
