// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraPlaceholderDataInterfaceManager.h"

#include "NiagaraDataInterface.h"
#include "NiagaraEmitter.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "NiagaraNodeFunctionCall.h"

FNiagaraPlaceholderDataInterfaceManager::FNiagaraPlaceholderDataInterfaceManager(TSharedRef<FNiagaraSystemViewModel> InOwningSystemViewModel)
{
	OwningSystemViewModelWeak = InOwningSystemViewModel;
}

TSharedRef<FNiagaraPlaceholderDataInterfaceHandle> FNiagaraPlaceholderDataInterfaceManager::GetOrCreatePlaceholderDataInterface(
	const FGuid& OwningEmitterHandleId,
	UNiagaraNodeFunctionCall& OwningFunctionCall,
	const FNiagaraParameterHandle& InputHandle,
	const UClass* DataInterfaceClass)
{
	FPlaceholderDataInterfaceInfo* PlaceholderDataInterfaceInfo = GetPlaceholderDataInterfaceInfo(OwningEmitterHandleId, OwningFunctionCall, InputHandle);
	if (PlaceholderDataInterfaceInfo == nullptr)
	{
		PlaceholderDataInterfaceInfo = &PlaceholderDataInterfaceInfos.AddDefaulted_GetRef();
		PlaceholderDataInterfaceInfo->OwningEmitterHandleId = OwningEmitterHandleId;
		PlaceholderDataInterfaceInfo->OwningFunctionCall = &OwningFunctionCall;
		PlaceholderDataInterfaceInfo->InputHandle = InputHandle;
	}

	if (PlaceholderDataInterfaceInfo->PlaceholderDataInterface.IsValid() == false)
	{
		// @todo add a transient uproperty to keep the placeholder data interfaces in their correct owner objects
		PlaceholderDataInterfaceInfo->PlaceholderDataInterface = NewObject<UNiagaraDataInterface>(&OwningFunctionCall, DataInterfaceClass, NAME_None, RF_Transactional | RF_Transient);
		PlaceholderDataInterfaceInfo->PlaceholderDataInterface->OnChanged().AddSP(
			this, &FNiagaraPlaceholderDataInterfaceManager::PlaceholderDataInterfaceChanged, PlaceholderDataInterfaceInfo->PlaceholderDataInterface);
	}

	TSharedPtr<FNiagaraPlaceholderDataInterfaceHandle> PlaceholderDataInterfaceHandle = PlaceholderDataInterfaceInfo->PlaceholderDataInterfaceHandleWeak.Pin();
	if (PlaceholderDataInterfaceHandle.IsValid() == false || PlaceholderDataInterfaceHandle->DataInterface.HasSameIndexAndSerialNumber(PlaceholderDataInterfaceInfo->PlaceholderDataInterface) == false)
	{
		PlaceholderDataInterfaceHandle = MakeShareable<FNiagaraPlaceholderDataInterfaceHandle>(new FNiagaraPlaceholderDataInterfaceHandle(*this, *PlaceholderDataInterfaceInfo->PlaceholderDataInterface, 
			FSimpleDelegate::CreateSP(this->AsShared(), &FNiagaraPlaceholderDataInterfaceManager::PlaceholderHandleDeleted, PlaceholderDataInterfaceInfo->PlaceholderDataInterface)));
		PlaceholderDataInterfaceInfo->PlaceholderDataInterfaceHandleWeak = PlaceholderDataInterfaceHandle;
	}

	return PlaceholderDataInterfaceHandle.ToSharedRef();
}

TSharedPtr<FNiagaraPlaceholderDataInterfaceHandle> FNiagaraPlaceholderDataInterfaceManager::GetPlaceholderDataInterface(
	const FGuid& OwningEmitterHandleId,
	UNiagaraNodeFunctionCall& OwningFunctionCall,
	const FNiagaraParameterHandle& InputHandle)
{
	FPlaceholderDataInterfaceInfo* PlaceholderDataInterfaceInfo = GetPlaceholderDataInterfaceInfo(OwningEmitterHandleId, OwningFunctionCall, InputHandle);
	bool bHasValidPlaceholderAndHandle = 
		PlaceholderDataInterfaceInfo != nullptr &&
		PlaceholderDataInterfaceInfo->PlaceholderDataInterface.IsValid() &&
		PlaceholderDataInterfaceInfo->PlaceholderDataInterfaceHandleWeak.IsValid() &&
		PlaceholderDataInterfaceInfo->PlaceholderDataInterfaceHandleWeak.Pin()->DataInterface.HasSameIndexAndSerialNumber(PlaceholderDataInterfaceInfo->PlaceholderDataInterface);
	return bHasValidPlaceholderAndHandle
		? PlaceholderDataInterfaceInfo->PlaceholderDataInterfaceHandleWeak.Pin()
		: TSharedPtr<FNiagaraPlaceholderDataInterfaceHandle>();
}

bool FNiagaraPlaceholderDataInterfaceManager::TryGetOwnerInformation(UNiagaraDataInterface* InDataInterface, FGuid& OutOwningEmitterHandleId, UNiagaraNodeFunctionCall*& OutOwningFunctionCallNode)
{
	for (const FPlaceholderDataInterfaceInfo& PlaceholderDataInterfaceInfo : PlaceholderDataInterfaceInfos)
	{
		if (PlaceholderDataInterfaceInfo.PlaceholderDataInterface == InDataInterface)
		{
			OutOwningEmitterHandleId = PlaceholderDataInterfaceInfo.OwningEmitterHandleId;
			OutOwningFunctionCallNode = PlaceholderDataInterfaceInfo.OwningFunctionCall.Get();
			return true;
		}
	}
	return false;
}

void FNiagaraPlaceholderDataInterfaceManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (FPlaceholderDataInterfaceInfo& PlaceholderDataInterfaceInfo : PlaceholderDataInterfaceInfos)
	{
		Collector.AddReferencedObject(PlaceholderDataInterfaceInfo.PlaceholderDataInterface);
	}
}

FNiagaraPlaceholderDataInterfaceManager::FPlaceholderDataInterfaceInfo* FNiagaraPlaceholderDataInterfaceManager::GetPlaceholderDataInterfaceInfo(
	const FGuid& OwningEmitterHandleId,
	UNiagaraNodeFunctionCall& OwningFunctionCall,
	const FNiagaraParameterHandle& InputHandle)
{
	return PlaceholderDataInterfaceInfos.FindByPredicate([OwningEmitterHandleId, &OwningFunctionCall, InputHandle](const FPlaceholderDataInterfaceInfo& PlaceholderDataInterfaceInfo)
		{
			return
				PlaceholderDataInterfaceInfo.OwningEmitterHandleId == OwningEmitterHandleId &&
				PlaceholderDataInterfaceInfo.OwningFunctionCall.Get() == &OwningFunctionCall &&
				PlaceholderDataInterfaceInfo.InputHandle == InputHandle;
		});
}

void FNiagaraPlaceholderDataInterfaceManager::PlaceholderDataInterfaceChanged(TWeakObjectPtr<UNiagaraDataInterface> PlaceholderDataInterfaceWeak)
{
	UNiagaraDataInterface* PlaceholderDataInterface = PlaceholderDataInterfaceWeak.Get();
	if (PlaceholderDataInterface == nullptr)
	{
		return;
	}

	FPlaceholderDataInterfaceInfo* PlaceholderDataInterfaceInfo = PlaceholderDataInterfaceInfos.FindByPredicate([PlaceholderDataInterface](const FPlaceholderDataInterfaceInfo& PlaceholderDataInterfaceInfo) {
		return PlaceholderDataInterfaceInfo.PlaceholderDataInterface == PlaceholderDataInterface; });

	if (PlaceholderDataInterfaceInfo != nullptr && PlaceholderDataInterfaceInfo->OwningFunctionCall.IsValid())
	{
		UNiagaraDataInterface* OverrideDataInterface = nullptr;
		if (PlaceholderDataInterfaceInfo->OwningFunctionCall->GetNiagaraGraph()->GetChangeID() == PlaceholderDataInterfaceInfo->OverrideDataInterfaceGraphChangeId)
		{
			// We can only use the cached override data interface if the graph hasn't been modified since it was retrieved.
			OverrideDataInterface = PlaceholderDataInterfaceInfo->CachedOverrideDataInterface.Get();
		}

		if (OverrideDataInterface == nullptr)
		{
			FNiagaraParameterHandle AliasedInputParameterHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(PlaceholderDataInterfaceInfo->InputHandle, PlaceholderDataInterfaceInfo->OwningFunctionCall.Get());
			FNiagaraVariable ParameterVariable(FNiagaraTypeDefinition(PlaceholderDataInterface->GetClass()), PlaceholderDataInterfaceInfo->InputHandle.GetParameterHandleString());
			TOptional<FNiagaraVariableMetaData> ParameterMetaData = PlaceholderDataInterfaceInfo->OwningFunctionCall->GetNiagaraGraph()->GetMetaData(ParameterVariable);
			FGuid ParameterVariableGuid = ParameterMetaData.IsSet() ? ParameterMetaData->GetVariableGuid() : FGuid();
			UEdGraphPin& OverridePin = FNiagaraStackGraphUtilities::GetOrCreateStackFunctionInputOverridePin(
				*PlaceholderDataInterfaceInfo->OwningFunctionCall.Get(), AliasedInputParameterHandle,
				FNiagaraTypeDefinition(PlaceholderDataInterface->GetClass()), ParameterVariableGuid, FGuid());

			if (OverridePin.LinkedTo.Num() == 1)
			{
				// If the override pin is linked try to get an existing override data interface.
				UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(OverridePin.LinkedTo[0]->GetOwningNode());
				if (InputNode != nullptr && InputNode->GetDataInterface() != nullptr)
				{
					OverrideDataInterface = InputNode->GetDataInterface();
				}
			}

			if(OverrideDataInterface == nullptr)
			{
				// If no override data interface could be found, create a new one.
				if (OverridePin.LinkedTo.Num() > 0)
				{
					FNiagaraStackGraphUtilities::RemoveNodesForStackFunctionInputOverridePin(OverridePin);
				}
				FNiagaraStackGraphUtilities::SetDataInterfaceValueForFunctionInput(OverridePin, PlaceholderDataInterface->GetClass(), AliasedInputParameterHandle.GetParameterHandleString().ToString(), OverrideDataInterface);
				FNiagaraStackGraphUtilities::RelayoutGraph(*PlaceholderDataInterfaceInfo->OwningFunctionCall->GetNiagaraGraph());
			}

			PlaceholderDataInterfaceInfo->CachedOverrideDataInterface = OverrideDataInterface;
			PlaceholderDataInterfaceInfo->OverrideDataInterfaceGraphChangeId = PlaceholderDataInterfaceInfo->OwningFunctionCall->GetNiagaraGraph()->GetChangeID();
		}

		OverrideDataInterface->Modify();
		PlaceholderDataInterface->CopyTo(OverrideDataInterface);
		TArray<UObject*> ChangedObjects;
		ChangedObjects.Add(OverrideDataInterface);
		OwningSystemViewModelWeak.Pin()->NotifyDataObjectChanged(ChangedObjects, ENiagaraDataObjectChange::Changed);
	}
}

void FNiagaraPlaceholderDataInterfaceManager::PlaceholderHandleDeleted(TWeakObjectPtr<UNiagaraDataInterface> PlaceholderDataInterfaceWeak)
{
	int32 PlaceholderDataInterfaceInfoIndex = PlaceholderDataInterfaceInfos.IndexOfByPredicate([PlaceholderDataInterfaceWeak](const FPlaceholderDataInterfaceInfo& PlaceholderDataInterfaceInfo) {
		return PlaceholderDataInterfaceInfo.PlaceholderDataInterface.HasSameIndexAndSerialNumber(PlaceholderDataInterfaceWeak); });

	if (PlaceholderDataInterfaceInfoIndex != INDEX_NONE)
	{
		if(PlaceholderDataInterfaceWeak.IsValid())
		{
			PlaceholderDataInterfaceWeak->OnChanged().RemoveAll(this);
		}
		PlaceholderDataInterfaceInfos.RemoveAtSwap(PlaceholderDataInterfaceInfoIndex);
	}
}

FNiagaraPlaceholderDataInterfaceHandle::~FNiagaraPlaceholderDataInterfaceHandle()
{
	OnDeleted.ExecuteIfBound();
}

UNiagaraDataInterface* FNiagaraPlaceholderDataInterfaceHandle::GetDataInterface() const
{
	return DataInterface.Get();
}

FNiagaraPlaceholderDataInterfaceHandle::FNiagaraPlaceholderDataInterfaceHandle(FNiagaraPlaceholderDataInterfaceManager& InOwningManager, UNiagaraDataInterface& InDataInterface, FSimpleDelegate InOnDeleted)
{
	OwningManager = &InOwningManager;
	DataInterface = &InDataInterface;
	OnDeleted = InOnDeleted;
}