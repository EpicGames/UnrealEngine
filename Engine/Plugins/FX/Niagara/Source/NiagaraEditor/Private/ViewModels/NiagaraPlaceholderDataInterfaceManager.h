// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"

class UNiagaraDataInterface;
class UNiagaraEmitter;
class UNiagaraNodeFunctionCall;
class FNiagaraSystemViewModel;

class FNiagaraPlaceholderDataInterfaceHandle;

/**
 * Manages placeholder data interfaces which are created when a module had a data interface input, but it has
 * not been overridden in the stack.  When a placeholder data interfaces is edited, an override data interface
 * will be created and any changes to the placeholder will be copied to the override while the placeholder is
 * still active in the UI.  Lifetime for placeholders is tracked by the handles they're returned with.
 */
class FNiagaraPlaceholderDataInterfaceManager : public FGCObject, public TSharedFromThis<FNiagaraPlaceholderDataInterfaceManager>
{
private:
	struct FPlaceholderDataInterfaceInfo
	{
		FGuid OwningEmitterHandleId;
		TWeakObjectPtr<UNiagaraNodeFunctionCall> OwningFunctionCall;
		FNiagaraParameterHandle InputHandle;
		TWeakObjectPtr<UNiagaraDataInterface> PlaceholderDataInterface;
		TWeakObjectPtr<UNiagaraDataInterface> CachedOverrideDataInterface;
		FGuid OverrideDataInterfaceGraphChangeId;
		TWeakPtr<FNiagaraPlaceholderDataInterfaceHandle> PlaceholderDataInterfaceHandleWeak;
	};

public:
	FNiagaraPlaceholderDataInterfaceManager(TSharedRef<FNiagaraSystemViewModel> InOwningSystemViewModel);

	/** 
	* Gets or creates a placeholder data interface for the specified input on the specified function.  This should only be called
	* once it's been determined that the stack graph doesn't already have an override data interface defined.  The handle returned by this 
	* function is used to track the lifetime of the placeholder data interface and must be referenced while the placeholder data
	* interface is in use.
	*/
	TSharedRef<FNiagaraPlaceholderDataInterfaceHandle> GetOrCreatePlaceholderDataInterface(
		const FGuid& OwningEmitterHandleId,
		UNiagaraNodeFunctionCall& OwningFunctionCall,
		const FNiagaraParameterHandle& InputHandle,
		const UClass* DataInterfaceClass);

	/*
	* Gets an existing placeholder data interface for the specified input on the specified function if it exists.  If a place holder
	* is not active for the input the pointer to the handle will be invalid.  The handle returned by this function is used to track
	* the lifetime of the placeholder data interface and must be referenced while the placeholder data interface is in use.
	*/
	TSharedPtr<FNiagaraPlaceholderDataInterfaceHandle> GetPlaceholderDataInterface(
		const FGuid& OwningEmitterHandleId,
		UNiagaraNodeFunctionCall& OwningFunctionCall,
		const FNiagaraParameterHandle& InputHandle);

	/*
	* Tries to get owner information for a managed data interfaces. 
	*/
	bool TryGetOwnerInformation(UNiagaraDataInterface* InDataInterface, FGuid& OutOwningEmitterHandleId, UNiagaraNodeFunctionCall*& OutOwningFunctionCallNode);

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraPlaceholderDataInterfaceManager");
	}

private:
	FPlaceholderDataInterfaceInfo* GetPlaceholderDataInterfaceInfo(const FGuid& OwningEmitterHandleId, UNiagaraNodeFunctionCall& OwningFunctionCall, const FNiagaraParameterHandle& InputHandle);
	void PlaceholderDataInterfaceChanged(TWeakObjectPtr<UNiagaraDataInterface> PlaceholderDataInterfaceWeak);
	void PlaceholderHandleDeleted(TWeakObjectPtr<UNiagaraDataInterface> PlaceholderDataInterfaceWeak);

private:
	TWeakPtr<FNiagaraSystemViewModel> OwningSystemViewModelWeak;
	TArray<FPlaceholderDataInterfaceInfo> PlaceholderDataInterfaceInfos;
};

/* A handle to a placeholder data interface which is used to track the lifetime of the placeholder object.  When this handle is destructed the placeholder
* data interface will no longer be protected from garbage collection and will no longer be watched for changes. */
class FNiagaraPlaceholderDataInterfaceHandle
{

public:
	~FNiagaraPlaceholderDataInterfaceHandle();

	/** Gets the data interface referenced by this handle. */
	UNiagaraDataInterface* GetDataInterface() const;

private:
	FNiagaraPlaceholderDataInterfaceHandle(FNiagaraPlaceholderDataInterfaceManager& InOwningManager, UNiagaraDataInterface& InDataInterface, FSimpleDelegate InOnDeleted);

private:
	friend FNiagaraPlaceholderDataInterfaceManager;

	FNiagaraPlaceholderDataInterfaceManager* OwningManager;
	TWeakObjectPtr<UNiagaraDataInterface> DataInterface;
	FSimpleDelegate OnDeleted;
};