// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "UObject/Linker.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraScriptSource.h"
#include "ViewModels/NiagaraScratchPadUtilities.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"

#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "FileHelpers.h"
#include "ScopedTransaction.h"
#include "Framework/Commands/UICommandList.h"

#define LOCTEXT_NAMESPACE "NiagaraScratchPadScriptViewModel"



FNiagaraScratchPadScriptViewModel::FNiagaraScratchPadScriptViewModel(bool bInIsForDataProcessingOnly)
	: FNiagaraScriptViewModel(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &FNiagaraScratchPadScriptViewModel::GetDisplayNameInternal)), ENiagaraParameterEditMode::EditAll, bInIsForDataProcessingOnly)
	, bIsPendingRename(false)
	, bIsPinned(false)
	, EditorHeight(300)
	, bHasPendingChanges(false)
{
}

FNiagaraScratchPadScriptViewModel::~FNiagaraScratchPadScriptViewModel()
{
	if (EditScript.Script != nullptr)
	{
		if (EditScript.Script->GetLatestSource() != nullptr)
		{
			UNiagaraScriptSource* EditScriptSource = CastChecked<UNiagaraScriptSource>(EditScript.Script->GetLatestSource());
			if(EditScriptSource->NodeGraph != nullptr)
			{
				EditScriptSource->NodeGraph->RemoveOnGraphNeedsRecompileHandler(OnGraphNeedsRecompileHandle);
			}
		}
		EditScript.Script->OnPropertyChanged().RemoveAll(this);
		EditScript.Script = nullptr;
	}
}

void FNiagaraScratchPadScriptViewModel::Initialize(UNiagaraScript* Script, UNiagaraScript* InEditScript, TWeakPtr<FNiagaraSystemViewModel> InSystemViewModel)
{
	OriginalScript = Script;
	SystemViewModel = InSystemViewModel;

	// Copy over the old edited graph, as this initialize might be incoming from inheritance changes and we want to maintain
	// the edits to the old graph we were working on.
	if (InEditScript)
	{
		EditScript.Script = InEditScript;
		SetScript(EditScript);
	}
	else
	{
		EditScript.Script = CastChecked<UNiagaraScript>(StaticDuplicateObject(Script, GetTransientPackage()));
		SetScript(EditScript);
	}
	LastAppliedChangeID = GetScriptChangeID();
	
	UNiagaraScriptSource* EditScriptSource = CastChecked<UNiagaraScriptSource>(EditScript.Script->GetLatestSource());
	OnGraphNeedsRecompileHandle = EditScriptSource->NodeGraph->AddOnGraphNeedsRecompileHandler(FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraScratchPadScriptViewModel::OnScriptGraphChanged));
	EditScript.Script->OnPropertyChanged().AddSP(this, &FNiagaraScratchPadScriptViewModel::OnScriptPropertyChanged);
	ParameterPanelCommands = MakeShared<FUICommandList>();

	ParameterPaneViewModel = MakeShared<FNiagaraScriptToolkitParameterPanelViewModel>(this->AsShared());
	ExternalSelectionChangedDelegate = ParameterPaneViewModel->GetOnExternalSelectionChangedDelegate().AddRaw(this, &FNiagaraScratchPadScriptViewModel::OnGraphSubObjectSelectionChanged);
	FScriptToolkitUIContext UIContext = FScriptToolkitUIContext(
		FSimpleDelegate::CreateSP(ParameterPaneViewModel.ToSharedRef(), &INiagaraImmutableParameterPanelViewModel::Refresh),
		FSimpleDelegate(), //@todo(ng) skip binding refresh parameter definitions panel as scratchpad does not have a parameter definitions panel currently
		FSimpleDelegate::CreateSP(GetVariableSelection(), &FNiagaraObjectSelection::Refresh)
	);
	ParameterPaneViewModel->Init(UIContext);

	ScriptHierarchyViewModel = NewObject<UNiagaraScriptParametersHierarchyViewModel>();
	ScriptHierarchyViewModel->Initialize(AsShared());

	ScriptHierarchyViewModel->GetOnHierarchyChangedDelegate().AddSP(this, &FNiagaraScratchPadScriptViewModel::OnHierarchyChanged);
}

TSharedPtr<IAssetReferenceFilter> FNiagaraScratchPadScriptViewModel::MakeAssetReferenceFilter() const
{
	if (OriginalScript && GEditor)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAsset(OriginalScript);
		return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
	}

	return {};
}

bool FNiagaraScratchPadScriptViewModel::IsValid() const
{
	return EditScript.Script != nullptr && GraphViewModel->GetScriptSource() != nullptr;
}

void FNiagaraScratchPadScriptViewModel::OnGraphSubObjectSelectionChanged(const UObject* Obj)
{
	OnGraphSelectionChanged().Broadcast(Obj);
}

void FNiagaraScratchPadScriptViewModel::Finalize()
{
	// This pointer needs to be reset manually here because there is a shared ref cycle.
	if (ParameterPaneViewModel.IsValid())
	{
		ParameterPaneViewModel->GetOnExternalSelectionChangedDelegate().Remove(ExternalSelectionChangedDelegate);
		ParameterPaneViewModel.Reset();
	}

	if(ScriptHierarchyViewModel != nullptr)
	{
		ScriptHierarchyViewModel->Finalize();
		ScriptHierarchyViewModel = nullptr;
	}
}

void FNiagaraScratchPadScriptViewModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(EditScript.Script);
	Collector.AddReferencedObject(ScriptHierarchyViewModel);
}

UNiagaraScript* FNiagaraScratchPadScriptViewModel::GetOriginalScript() const
{
	return OriginalScript;
}

const FVersionedNiagaraScript& FNiagaraScratchPadScriptViewModel::GetEditScript() const
{
	return EditScript;
}

TArray<UNiagaraGraph*> FNiagaraScratchPadScriptViewModel::GetEditableGraphs() 
{
	TArray<UNiagaraGraph*> EditableGraphs;
	if (UNiagaraGraph* Graph = GetGraphViewModel()->GetGraph())
	{
		EditableGraphs.Add(Graph);
	}
	return EditableGraphs;
}

UNiagaraScriptParametersHierarchyViewModel* FNiagaraScratchPadScriptViewModel::GetHierarchyViewModel()
{
	return ScriptHierarchyViewModel;
}

TSharedPtr<INiagaraParameterPanelViewModel> FNiagaraScratchPadScriptViewModel::GetParameterPanelViewModel() const
{
	return ParameterPaneViewModel;
}

TSharedPtr<FUICommandList> FNiagaraScratchPadScriptViewModel::GetParameterPanelCommands() const
{
	return ParameterPanelCommands;
}

FText FNiagaraScratchPadScriptViewModel::GetToolTip() const
{
	return FText::Format(LOCTEXT("ScratchPadScriptToolTipFormat", "Description: {0}{1}"),
		EditScript.GetScriptData()->Description.IsEmptyOrWhitespace() ? LOCTEXT("NoDescription", "(none)") : EditScript.GetScriptData()->Description,
		bHasPendingChanges ? LOCTEXT("HasPendingChangesStatus", "\n* Has pending changes to apply") : FText());
}

bool FNiagaraScratchPadScriptViewModel::GetIsPendingRename() const
{
	return bIsPendingRename;
}

void FNiagaraScratchPadScriptViewModel::SetIsPendingRename(bool bInIsPendingRename)
{
	bIsPendingRename = bInIsPendingRename;
}

void FNiagaraScratchPadScriptViewModel::SetScriptName(FText InScriptName)
{
	FString NewName = FNiagaraUtilities::SanitizeNameForObjectsAndPackages(InScriptName.ToString());
	if (OriginalScript->GetName() != NewName)
	{
		FScopedTransaction RenameTransaction(LOCTEXT("RenameScriptTransaction", "Rename scratch pad script."));

		FName NewUniqueName = FNiagaraEditorUtilities::GetUniqueObjectName<UNiagaraScript>(OriginalScript->GetOuter(), *NewName);
		
		if(OriginalScript->HasAnyFlags(RF_Transactional) == false)
		{
			OriginalScript->SetFlags(RF_Transactional);
		}
		
		OriginalScript->Modify();
		OriginalScript->Rename(*NewUniqueName.ToString(), nullptr, REN_DontCreateRedirectors);

		TArray<UNiagaraNodeFunctionCall*> ReferencingFunctionCallNodes;
		FNiagaraEditorUtilities::GetReferencingFunctionCallNodes(OriginalScript, ReferencingFunctionCallNodes);
		for(UNiagaraNodeFunctionCall* ReferencingFunctionCallNode : ReferencingFunctionCallNodes)
		{
			FNiagaraScratchPadUtilities::FixFunctionInputsFromFunctionScriptRename(*ReferencingFunctionCallNode, NewUniqueName);
			ReferencingFunctionCallNode->MarkNodeRequiresSynchronization(TEXT("ScratchPad script renamed"), true);
		}

		OnRenamedDelegate.Broadcast();
	}
}

bool FNiagaraScratchPadScriptViewModel::GetIsPinned() const
{
	return bIsPinned;
}

void FNiagaraScratchPadScriptViewModel::SetIsPinned(bool bInIsPinned)
{
	if (bIsPinned != bInIsPinned)
	{
		bIsPinned = bInIsPinned;
		OnPinnedChangedDelegate.Broadcast();
	}
}

void FNiagaraScratchPadScriptViewModel::TransferFromOldWhenDoingApply(TSharedPtr< FNiagaraScratchPadScriptViewModel> InOldScriptVM)
{
	// Copy over has pending changes, as this might be incoming from inheritance changes and we want to maintain
	// the edits to the old graph we were working on.
	if (InOldScriptVM.IsValid())
	{
		bHasPendingChanges = InOldScriptVM->bHasPendingChanges;
	}
}


float FNiagaraScratchPadScriptViewModel::GetEditorHeight() const
{
	return EditorHeight;
}

void FNiagaraScratchPadScriptViewModel::SetEditorHeight(float InEditorHeight)
{
	EditorHeight = InEditorHeight;
}

bool FNiagaraScratchPadScriptViewModel::HasUnappliedChanges() const
{
	return bHasPendingChanges || (LastAppliedChangeID.IsValid() && LastAppliedChangeID != GetScriptChangeID());
}

void FNiagaraScratchPadScriptViewModel::ApplyChanges()
{
	ResetLoaders(OriginalScript->GetOutermost()); // Make sure that we're not going to get invalid version number linkers into the package we are going into. 

	OriginalScript = Cast<UNiagaraScript>(StaticDuplicateObject(EditScript.Script, OriginalScript->GetOuter(), OriginalScript->GetFName(), RF_AllFlags, OriginalScript->GetClass()));
	bHasPendingChanges = false;
	LastAppliedChangeID = GetScriptChangeID();

	TArray<UNiagaraNodeFunctionCall*> FunctionCallNodesToRefresh;
	FNiagaraEditorUtilities::GetReferencingFunctionCallNodes(OriginalScript, FunctionCallNodesToRefresh);

	for (UNiagaraNodeFunctionCall* FunctionCallNodeToRefresh : FunctionCallNodesToRefresh)
	{
		FunctionCallNodeToRefresh->RefreshFromExternalChanges();
		FunctionCallNodeToRefresh->MarkNodeRequiresSynchronization(TEXT("ScratchPadChangesApplied"), true);
	}

	TSharedRef<FNiagaraScriptViewModel> ViewModel = AsShared();
	TSharedRef<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel = StaticCastSharedRef<FNiagaraScratchPadScriptViewModel>(ViewModel);
	OnHasUnappliedChangesChangedDelegate.Broadcast();
	OnChangesAppliedDelegate.Broadcast(ScratchPadScriptViewModel);
}

void FNiagaraScratchPadScriptViewModel::ApplyChangesAndSave()
{
	ApplyChanges();

	const bool bCheckDirty = false;
	const bool bPromptToSave = false;
	FEditorFileUtils::PromptForCheckoutAndSave({ OriginalScript->GetPackage() }, bCheckDirty, bPromptToSave);
}

void FNiagaraScratchPadScriptViewModel::DiscardChanges()
{
	OnRequestDiscardChangesDelegate.ExecuteIfBound();
}

FNiagaraScratchPadScriptViewModel::FOnRenamed& FNiagaraScratchPadScriptViewModel::OnRenamed()
{
	return OnRenamedDelegate;
}

FNiagaraScratchPadScriptViewModel::FOnPinnedChanged& FNiagaraScratchPadScriptViewModel::OnPinnedChanged()
{
	return OnPinnedChangedDelegate;
}

FNiagaraScratchPadScriptViewModel::FOnHasUnappliedChangesChanged& FNiagaraScratchPadScriptViewModel::OnHasUnappliedChangesChanged()
{
	return OnHasUnappliedChangesChangedDelegate;
}

FNiagaraScratchPadScriptViewModel::FOnChangesApplied& FNiagaraScratchPadScriptViewModel::OnChangesApplied()
{
	return OnChangesAppliedDelegate;
}

FSimpleDelegate& FNiagaraScratchPadScriptViewModel::OnRequestDiscardChanges()
{
	return OnRequestDiscardChangesDelegate;
}

void FNiagaraScratchPadScriptViewModel::OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload)
{
	if (bHasPendingChanges == false)
	{
		bHasPendingChanges = true;
		OnHasUnappliedChangesChangedDelegate.Broadcast();
	}
}

FText FNiagaraScratchPadScriptViewModel::GetDisplayNameInternal() const
{
	return FText::FromString(OriginalScript->GetName());
}

void FNiagaraScratchPadScriptViewModel::OnScriptGraphChanged(const FEdGraphEditAction &Action)
{
	if(bHasPendingChanges == false)
	{
		bHasPendingChanges = true;
		OnHasUnappliedChangesChangedDelegate.Broadcast();
	}
}

void FNiagaraScratchPadScriptViewModel::OnScriptPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (bHasPendingChanges == false)
	{
		bHasPendingChanges = true;
		OnHasUnappliedChangesChangedDelegate.Broadcast();
	}
}

FGuid FNiagaraScratchPadScriptViewModel::GetScriptChangeID() const
{
	if (Source.IsValid() && Source->NodeGraph)
	{
		return Source->NodeGraph->GetChangeID();
	}
	return FGuid();
}

FNiagaraScratchPadScriptViewModel::FOnNodeIDFocusRequested& FNiagaraScratchPadScriptViewModel::OnNodeIDFocusRequested()
{
	return OnNodeIDFocusRequestedDelegate;
}
FNiagaraScratchPadScriptViewModel::FOnPinIDFocusRequested& FNiagaraScratchPadScriptViewModel::OnPinIDFocusRequested()
{
	return OnPinIDFocusRequestedDelegate;
}

void FNiagaraScratchPadScriptViewModel::RaisePinFocusRequested(FNiagaraScriptIDAndGraphFocusInfo* InFocusInfo)
{
	OnNodeIDFocusRequestedDelegate.Broadcast(InFocusInfo);
}
void FNiagaraScratchPadScriptViewModel::RaiseNodeFocusRequested(FNiagaraScriptIDAndGraphFocusInfo* InFocusInfo)
{
	OnPinIDFocusRequestedDelegate.Broadcast(InFocusInfo);
}

#undef LOCTEXT_NAMESPACE