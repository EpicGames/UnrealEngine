// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScratchPadUtilities.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraEditorUtilities.h"

void FNiagaraScratchPadUtilities::FixFunctionInputsFromFunctionScriptRename(UNiagaraNodeFunctionCall& FunctionCallNode, FName NewScriptName)
{
	FString OldFunctionName = FunctionCallNode.GetFunctionName();
	if (OldFunctionName == NewScriptName.ToString())
	{
		return;
	}

	FunctionCallNode.Modify();
	FunctionCallNode.SuggestName(FString());
	const FString NewFunctionName = FunctionCallNode.GetFunctionName();
	UNiagaraSystem* System = FunctionCallNode.GetTypedOuter<UNiagaraSystem>();
	FVersionedNiagaraEmitter Emitter = FunctionCallNode.GetNiagaraGraph()->GetOwningEmitter();
	FNiagaraStackGraphUtilities::RenameReferencingParameters(System, Emitter, FunctionCallNode, OldFunctionName, NewFunctionName);
}

void FNiagaraScratchPadUtilities::FixExternalScratchPadScriptsForEmitter(UNiagaraSystem& SourceSystem, const FVersionedNiagaraEmitter& TargetEmitter)
{
	if ( TargetEmitter.Emitter == nullptr )
	{
		return;
	}

	UNiagaraSystem* TargetSystem = TargetEmitter.Emitter->GetTypedOuter<UNiagaraSystem>();
	FVersionedNiagaraEmitterData* EmitterData = TargetEmitter.GetEmitterData();
	if (EmitterData && TargetSystem != &SourceSystem)
	{
		UNiagaraScriptSource* EmitterScriptSource = Cast<UNiagaraScriptSource>(EmitterData->GraphSource);
		if (ensureMsgf(EmitterScriptSource != nullptr, TEXT("Emitter script source was null. Target Emitter: %s"), *TargetEmitter.Emitter->GetPathName()))
		{
			// Find function call nodes which reference external scratch pad scripts.
			TArray<UNiagaraNodeFunctionCall*> AllFunctionCallNodes;
			EmitterScriptSource->NodeGraph->GetNodesOfClass<UNiagaraNodeFunctionCall>(AllFunctionCallNodes);

			TArray<UNiagaraNodeFunctionCall*> ExternalScratchFunctionCallNodes;
			for (UNiagaraNodeFunctionCall* FunctionCallNode : AllFunctionCallNodes)
			{
				if (FunctionCallNode->IsA<UNiagaraNodeAssignment>() == false &&
					FunctionCallNode->FunctionScript != nullptr &&
					FunctionCallNode->FunctionScript->IsAsset() == false)
				{
					UNiagaraSystem* ScriptOwningSystem = FunctionCallNode->FunctionScript->GetTypedOuter<UNiagaraSystem>();
					if (ScriptOwningSystem == &SourceSystem)
					{
						ExternalScratchFunctionCallNodes.Add(FunctionCallNode);
					}
				}
			}

			// Determine the destination for the copies of the external scratch pad scripts.
			UObject* TargetScratchPadScriptOuter = nullptr;
			using ScriptArrayType = decltype(EmitterData->ScratchPads->Scripts);
			ScriptArrayType* TargetScratchPadScriptArray = nullptr;
			if (TargetSystem != nullptr)
			{
				TargetScratchPadScriptOuter = TargetSystem;
				TargetScratchPadScriptArray = &TargetSystem->ScratchPadScripts;
			}
			else
			{
				TargetScratchPadScriptOuter = TargetEmitter.Emitter;
				TargetScratchPadScriptArray = &EmitterData->ScratchPads->Scripts;
			}

			// Fix up the external scratch scripts.
			if (ensureMsgf(TargetScratchPadScriptOuter != nullptr && TargetScratchPadScriptArray != nullptr,
				TEXT("Failed to find target outer and array for fixing up  external scratch pad scripts.  Target Emitter: %s"), *TargetEmitter.Emitter->GetPathName()))
			{
				// Collate the function call nodes by the scratch script they call.
				TMap<UNiagaraScript*, TArray<UNiagaraNodeFunctionCall*>> ExternalScratchPadScriptToFunctionCallNodes;
				for (UNiagaraNodeFunctionCall* ExternalScratchFunctionCallNode : ExternalScratchFunctionCallNodes)
				{
					TArray<UNiagaraNodeFunctionCall*>& FunctionCallNodesForScript = ExternalScratchPadScriptToFunctionCallNodes.FindOrAdd(ExternalScratchFunctionCallNode->FunctionScript);
					FunctionCallNodesForScript.Add(ExternalScratchFunctionCallNode);
				}

				// Collect the current scratch pad script names to fix up duplicates.
				TSet<FName> ExistingScratchPadScriptNames;
				for (UNiagaraScript* ExistingScratchPadScript : (*TargetScratchPadScriptArray))
				{
					ExistingScratchPadScriptNames.Add(*ExistingScratchPadScript->GetName());
				}

				// Copy the scripts, rename them, and fix up the function calls.
				for (TPair<UNiagaraScript*, TArray<UNiagaraNodeFunctionCall*>> ExternalScratchPadScriptFunctionCallNodesPair : ExternalScratchPadScriptToFunctionCallNodes)
				{
					UNiagaraScript* ExternalScratchPadScript = ExternalScratchPadScriptFunctionCallNodesPair.Key;
					TArray<UNiagaraNodeFunctionCall*>& FunctionCallNodes = ExternalScratchPadScriptFunctionCallNodesPair.Value;

					FName TargetName = *ExternalScratchPadScript->GetName();
					FName UniqueTargetName = FNiagaraEditorUtilities::GetUniqueObjectName<UNiagaraScript>(TargetScratchPadScriptOuter, TargetName.ToString());
					UNiagaraScript* TargetScratchPadScriptCopy = CastChecked<UNiagaraScript>(StaticDuplicateObject(ExternalScratchPadScript, TargetScratchPadScriptOuter, UniqueTargetName));
					TargetScratchPadScriptArray->Add(TargetScratchPadScriptCopy);

					for (UNiagaraNodeFunctionCall* FunctionCallNode : FunctionCallNodes)
					{
						FunctionCallNode->FunctionScript = TargetScratchPadScriptCopy;
						FixFunctionInputsFromFunctionScriptRename(*FunctionCallNode, UniqueTargetName);
						FunctionCallNode->MarkNodeRequiresSynchronization(TEXT("Fix externally referenced scratch pad scripts."), true);
					}

					ExistingScratchPadScriptNames.Add(UniqueTargetName);
				}
			}
		}
	}
}
