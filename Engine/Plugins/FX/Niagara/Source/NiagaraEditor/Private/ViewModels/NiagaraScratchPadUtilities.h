// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"

class UNiagaraSystem;
class UNiagaraEmitter;
class UNiagaraNodeFunctionCall;

namespace FNiagaraScratchPadUtilities
{
	/** Fixes parameter map pins and rapid iteration parameters which are used to configure inputs for modules and dynamic inputs after 
		the scratch pad script a function call node references is renamed. */
	void FixFunctionInputsFromFunctionScriptRename(UNiagaraNodeFunctionCall& FunctionCallNode, FName NewScriptName);

	/** Fixes function call nodes in an emitter which reference scratch pad scripts from another system which can happen when copying
		and pasting emitters between systems, and when saving system emitters as assets. */
	void FixExternalScratchPadScriptsForEmitter(UNiagaraSystem& SourceSystem, const FVersionedNiagaraEmitter& TargetEmitter);
}