// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "UObject/UObjectIterator.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/NiagaraSystemEditorDocumentsViewModel.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"
#include "NiagaraSystem.h"
#include "NiagaraScriptSource.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraEditorModule.h"
#include "NiagaraClipboard.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraScriptFactoryNew.h"

#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Misc/PackageName.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Misc/MessageDialog.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/PlatformApplicationMisc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScratchPadViewModel)

#define LOCTEXT_NAMESPACE "NiagaraScratchPadViewModel"

void UNiagaraScratchPadViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModelWeak = InSystemViewModel;
	ObjectSelection = MakeShared<FNiagaraObjectSelection>();
	RefreshScriptViewModels();
	if (ScriptViewModels.Num() > 0)
	{
		SetActiveScriptViewModel(ScriptViewModels[0]);
	}
	AvailableUsages = { ENiagaraScriptUsage::DynamicInput, ENiagaraScriptUsage::Module };
}

void UNiagaraScratchPadViewModel::Finalize()
{
	for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScriptViewModel : ScriptViewModels)
	{
		TearDownScriptViewModel(ScriptViewModel);
	}
	ScriptViewModels.Empty();
	PinnedScriptViewModels.Empty();
	EditScriptViewModels.Empty();
	ActiveScriptViewModel.Reset();
}

void GetOuterAndTargetScripts(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, UObject*& OutOuter, TArray<TObjectPtr<UNiagaraScript>>*& OutTargetScripts)
{
	if (SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		OutOuter = &SystemViewModel->GetSystem();
		OutTargetScripts = &SystemViewModel->GetSystem().ScratchPadScripts;
	}
	else
	{
		if (SystemViewModel->GetSystem().GetEmitterHandles().Num() == 1)
		{
			FVersionedNiagaraEmitter TargetEmitter = SystemViewModel->GetSystem().GetEmitterHandles()[0].GetInstance();
			FVersionedNiagaraEmitterData* EmitterData = TargetEmitter.GetEmitterData();
			OutOuter = EmitterData->ScratchPads;
			OutTargetScripts = &EmitterData->ScratchPads->Scripts;
		}
		else
		{
			OutOuter = nullptr;
			OutTargetScripts = nullptr;
		}
	}
}

void UpdateChangeId(TSharedRef<FNiagaraSystemViewModel> SystemViewModel)
{
	if (SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		if (SystemViewModel->GetSystem().GetEmitterHandles().Num() == 1)
		{
			UNiagaraEmitter* TargetEmitter = SystemViewModel->GetSystem().GetEmitterHandles()[0].GetInstance().Emitter;
			TargetEmitter->NotifyScratchPadScriptsChanged();
		}
	}
}

void UNiagaraScratchPadViewModel::RefreshScriptViewModels()
{
	TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>> OldScriptViewModelsRemaining = ScriptViewModels;
	ScriptViewModels.Empty();
	bHasUnappliedChangesCache.Reset();

	bool bViewModelsChanged = false;

	UObject* ScriptOuter;
	TArray<TObjectPtr<UNiagaraScript>>* TargetScripts;
	GetOuterAndTargetScripts(GetSystemViewModel(), ScriptOuter, TargetScripts);

	TSharedPtr<FNiagaraScratchPadScriptViewModel> ForceSetActiveDocumentAtEnd;

	TArray<UNiagaraGraph*> GraphsToClose;
	TArray<UNiagaraGraph*> GraphsToOpen;
	if (ScriptOuter != nullptr && TargetScripts != nullptr)
	{
		TSharedPtr<FNiagaraScratchPadScriptViewModel> ActiveScratchViewModel = GetSystemViewModel()->GetDocumentViewModel()->GetActiveScratchPadViewModelIfSet();
		for (int32 i = 0; i < (*TargetScripts).Num(); i++)
		{
			UNiagaraScript* ScratchPadScript = (*TargetScripts)[i];
			TSharedPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModel;

			if (GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
			{
				// System scripts don't ever merge, so we can safely just take the scripts or reuse view models.
				TSharedRef<FNiagaraScratchPadScriptViewModel>* OldScriptViewModel = OldScriptViewModelsRemaining.FindByPredicate(
					[ScratchPadScript](TSharedRef<FNiagaraScratchPadScriptViewModel> ScriptViewModel) { return ScriptViewModel->GetOriginalScript() == ScratchPadScript; });
				if (OldScriptViewModel != nullptr)
				{
					ScriptViewModel = *OldScriptViewModel;
					OldScriptViewModelsRemaining.Remove(ScriptViewModel.ToSharedRef());
				}
				else
				{
					ScriptViewModel = CreateAndSetupScriptviewModel(ScratchPadScript, nullptr);
					bViewModelsChanged = true;
				}
			}
			else
			{
				// Emitter scripts may already be opened for edit but not yet applied so mergers can refresh and would wipe out the base scripts. These need to be kept up to date.
				// Because the merge could wipe out view models, we can't just reuse the old ones, but we can steal their graphs.
				// We need to match by name as indices are not stable.
				TSharedRef<FNiagaraScratchPadScriptViewModel>* OldScriptViewModelRef = OldScriptViewModelsRemaining.FindByPredicate(
					[ScratchPadScript](TSharedRef<FNiagaraScratchPadScriptViewModel> ScriptViewModel) { return ScriptViewModel->GetOriginalScript()->GetFName() == ScratchPadScript->GetFName(); });


				if (OldScriptViewModelRef != nullptr)
				{
					TSharedPtr<FNiagaraScratchPadScriptViewModel> OldScriptViewModel = (*OldScriptViewModelRef);
					// Pass the baton of the script to the new generation. Make sure to clear out references in the previous generation.
					ScriptViewModel = CreateAndSetupScriptviewModel(ScratchPadScript, OldScriptViewModel->GetEditScript().Script);
					// This will copy over apply state and anything else that would otherwise be lost in the handoff
					ScriptViewModel->TransferFromOldWhenDoingApply(OldScriptViewModel);
				
					// We need to keep track of the active document and reset again after we have the full array built up.
					if (ActiveScratchViewModel == OldScriptViewModel)
					{
						ForceSetActiveDocumentAtEnd = ScriptViewModel;
					}

					// We now need to make sure that all the UI/UX is transferred to the right objects for event handlers/etc.
					GetSystemViewModel()->GetDocumentViewModel()->SwapEditableScripts(OldScriptViewModel, ScriptViewModel);
					// Clear out the old to make sure it isn't touching anything
					OldScriptViewModel->SetScript(FVersionedNiagaraScript());
					OldScriptViewModelsRemaining.Remove(OldScriptViewModel.ToSharedRef());
					OldScriptViewModel.Reset();
				}
				else
				{
					ScriptViewModel = CreateAndSetupScriptviewModel(ScratchPadScript, nullptr);
				}
				bViewModelsChanged = true;
			}
			ScriptViewModels.Add(ScriptViewModel.ToSharedRef());
		}
	}

	if (OldScriptViewModelsRemaining.Num() > 0)
	{
		for (TSharedRef<FNiagaraScratchPadScriptViewModel> OldScriptViewModel : OldScriptViewModelsRemaining)
		{
			TearDownScriptViewModel(OldScriptViewModel);
		}
		bViewModelsChanged = true;
	}

	bool bEditViewModelsChanged = false;
	if (ActiveScriptViewModel.IsValid() && ScriptViewModels.Contains(ActiveScriptViewModel.ToSharedRef()) == false)
	{
		bool bRefreshEditScriptViewModels = false;
		ResetActiveScriptViewModelInternal(bRefreshEditScriptViewModels);
		bEditViewModelsChanged = true;
	}

	TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>> OldPinnedScriptViewModels = PinnedScriptViewModels;
	PinnedScriptViewModels.Empty();
	for (TSharedRef<FNiagaraScratchPadScriptViewModel> OldPinnedScriptViewModel : OldPinnedScriptViewModels)
	{
		// Remove pinned view models which are no longer valid, but add them one at a time to maintain the pin order.
		if (ScriptViewModels.Contains(OldPinnedScriptViewModel))
		{
			PinnedScriptViewModels.Add(OldPinnedScriptViewModel);
		}
		else
		{
			bEditViewModelsChanged = true;
		}
	}

	if (bEditViewModelsChanged)
	{
		RefreshEditScriptViewModels();
	}

	{
		for (int32 i = 0; i < ScriptViewModels.Num(); i++)
		{
			TArray<TSharedPtr<FNiagaraScriptViewModel>> ExistingViewModels;
			TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>::GetAllViewModelsForObject(ScriptViewModels[i]->GetEditScript().Script, ExistingViewModels);

			(ensureMsgf(ExistingViewModels.Num() == 1, TEXT("Missing or duplicate script view models detected.  Can not create script details customization.")));
		}
	}
	
	if (bViewModelsChanged)
	{
		OnScriptViewModelsChangedDelegate.Broadcast();
	}

	if (ForceSetActiveDocumentAtEnd.IsValid())
	{
		FocusScratchPadScriptViewModel(ForceSetActiveDocumentAtEnd.ToSharedRef());
	}
}

void UNiagaraScratchPadViewModel::ApplyScratchPadChanges()
{
	for(const TSharedRef<FNiagaraScratchPadScriptViewModel>& ScriptViewModel : ScriptViewModels)
	{
		if(ScriptViewModel->HasUnappliedChanges())
		{
			ScriptViewModel->ApplyChanges();
		}
	}
}

const TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>>& UNiagaraScratchPadViewModel::GetScriptViewModels() const
{
	return ScriptViewModels;
}

const TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>>& UNiagaraScratchPadViewModel::GetEditScriptViewModels() const
{
	return EditScriptViewModels;
}

TSharedPtr<FNiagaraScratchPadScriptViewModel> UNiagaraScratchPadViewModel::GetViewModelForScript(UNiagaraScript* InScript)
{
	TSharedRef<FNiagaraScratchPadScriptViewModel>* ViewModelForScript = ScriptViewModels.FindByPredicate([InScript](TSharedRef<FNiagaraScratchPadScriptViewModel>& ScriptViewModel) { return ScriptViewModel->GetOriginalScript() == InScript; });
	if (ViewModelForScript != nullptr)
	{
		return *ViewModelForScript;
	}
	return TSharedPtr<FNiagaraScratchPadScriptViewModel>();
}

TSharedPtr<FNiagaraScratchPadScriptViewModel> UNiagaraScratchPadViewModel::GetViewModelForScript(FName InScriptName)
{
	TSharedRef<FNiagaraScratchPadScriptViewModel>* ViewModelForScript = ScriptViewModels.FindByPredicate([InScriptName](TSharedRef<FNiagaraScratchPadScriptViewModel>& ScriptViewModel) { return ScriptViewModel->GetOriginalScript()->GetFName().IsEqual(InScriptName, ENameCase::IgnoreCase, false); });
	if (ViewModelForScript != nullptr)
	{
		return *ViewModelForScript;
	}
	return TSharedPtr<FNiagaraScratchPadScriptViewModel>();
}



TSharedPtr<FNiagaraScratchPadScriptViewModel> UNiagaraScratchPadViewModel::GetViewModelForEditScript(UNiagaraScript* InEditScript)
{
	TSharedRef<FNiagaraScratchPadScriptViewModel>* ViewModelForEditScript = ScriptViewModels.FindByPredicate([InEditScript](TSharedRef<FNiagaraScratchPadScriptViewModel>& ScriptViewModel) { return ScriptViewModel->GetEditScript().Script == InEditScript; });
	if (ViewModelForEditScript != nullptr)
	{
		return *ViewModelForEditScript;
	}
	return TSharedPtr<FNiagaraScratchPadScriptViewModel>();
}

const TArray<ENiagaraScriptUsage>& UNiagaraScratchPadViewModel::GetAvailableUsages() const
{
	return AvailableUsages;
}

FText UNiagaraScratchPadViewModel::GetDisplayNameForUsage(ENiagaraScriptUsage InUsage) const
{
	switch (InUsage)
	{
	case ENiagaraScriptUsage::DynamicInput:
		return LOCTEXT("DynamicInputDisplayName", "Dynamic Inputs");
	case ENiagaraScriptUsage::Module:
		return LOCTEXT("ModuleDisplayName", "Modules");
	case ENiagaraScriptUsage::Function:
		return LOCTEXT("FunctionDisplayName", "Functions");
	default: 
		return LOCTEXT("InvalidUsageDisplayName", "Invalid");
	}
}

TSharedRef<FNiagaraObjectSelection> UNiagaraScratchPadViewModel::GetObjectSelection()
{
	return ObjectSelection.ToSharedRef();
}

TSharedPtr<FNiagaraScratchPadScriptViewModel> UNiagaraScratchPadViewModel::GetActiveScriptViewModel()
{
	return ActiveScriptViewModel;
}

void UNiagaraScratchPadViewModel::SetActiveScriptViewModel(TSharedRef<FNiagaraScratchPadScriptViewModel> InActiveScriptViewModel )
{
	if (ensureMsgf(ScriptViewModels.Contains(InActiveScriptViewModel), TEXT("Can only set an active view model from this scratch pad view model.")))
	{
		ActiveScriptViewModel = InActiveScriptViewModel;
		const FVersionedNiagaraScript& EditScript = ActiveScriptViewModel->GetEditScript();
		ObjectSelection->SetSelectedObject(EditScript.Script, &EditScript.Version);
		RefreshEditScriptViewModels();
		OnActiveScriptChangedDelegate.Broadcast();
	}
}

void UNiagaraScratchPadViewModel::FocusScratchPadScriptViewModel(TSharedRef<FNiagaraScratchPadScriptViewModel> InScriptViewModel)
{
	if (ensureMsgf(ScriptViewModels.Contains(InScriptViewModel), TEXT("Can only focus a view model from this scratch pad view model.")))
	{
		SetActiveScriptViewModel(InScriptViewModel);
		GetSystemViewModel()->GetDocumentViewModel()->OpenChildScript(InScriptViewModel->GetGraphViewModel()->GetGraph());
	}
}

void UNiagaraScratchPadViewModel::ResetActiveScriptViewModel()
{
	bool bRefreshEditScriptViewModels = true;
	ResetActiveScriptViewModelInternal(bRefreshEditScriptViewModels);
}

void UNiagaraScratchPadViewModel::ResetActiveScriptViewModelInternal(bool bRefreshEditScriptViewModels)
{
	if (ActiveScriptViewModel.IsValid())
	{
		ActiveScriptViewModel.Reset();
		ObjectSelection->ClearSelectedObjects();

		if (bRefreshEditScriptViewModels)
		{
			RefreshEditScriptViewModels();
		}

		OnActiveScriptChangedDelegate.Broadcast();
	}
}

void UNiagaraScratchPadViewModel::CopyActiveScript()
{
	if (ActiveScriptViewModel.IsValid())
	{
		UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
		ClipboardContent->Scripts.Add(CastChecked<UNiagaraScript>(StaticDuplicateObject(ActiveScriptViewModel->GetOriginalScript(), ClipboardContent)));
		FNiagaraEditorModule::Get().GetClipboard().SetClipboardContent(ClipboardContent);
	}
}

void GetScriptsFromClipboard(TArray<const UNiagaraScript*>& OutScripts, const TArray<ENiagaraScriptUsage>& AvailableUsages)
{
	const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent();
	if (ClipboardContent != nullptr)
	{
		OutScripts.Append(ClipboardContent->Scripts);
	}
	else
	{
		FString ClipboardString;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardString);
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (ClipboardString.IsEmpty() == false && ClipboardString.Len() < NAME_SIZE && AssetRegistry != nullptr)
		{
			TArray<FAssetData> Assets;
			FAssetData AssetFoundByObjectPath = AssetRegistry->GetAssetByObjectPath(FSoftObjectPath(ClipboardString));
			if (AssetFoundByObjectPath.IsValid())
			{
				Assets.Add(AssetFoundByObjectPath);
			}
			else
			{
				AssetRegistry->GetAssetsByPackageName(*ClipboardString, Assets);
			}

			for (const FAssetData& Asset : Assets)
			{
				UNiagaraScript* Script = Cast<UNiagaraScript>(Asset.GetAsset());
				if (Script != nullptr && AvailableUsages.Contains(Script->GetUsage()))
				{
					OutScripts.Add(Script);
				}
			}
		}
	}
}

bool UNiagaraScratchPadViewModel::CanPasteScript() const
{
	TArray<const UNiagaraScript*> ClipboardScripts;
	GetScriptsFromClipboard(ClipboardScripts, AvailableUsages);
	return ClipboardScripts.Num() > 0;
}

void UNiagaraScratchPadViewModel::PasteScript()
{
	TArray<const UNiagaraScript*> ClipboardScripts;
	GetScriptsFromClipboard(ClipboardScripts, AvailableUsages);
	if (ClipboardScripts.Num() > 0)
	{
		FScopedTransaction Transaction(LOCTEXT("PasteScratchPadScriptTransaction", "Paste the scripts from the system clipboard."));
		TSharedPtr<FNiagaraScratchPadScriptViewModel> PastedScriptViewModel;
		for(const UNiagaraScript* ClipboardScript : ClipboardScripts)
		{
			PastedScriptViewModel = CreateNewScriptAsDuplicate(ClipboardScript);
		}
		SetActiveScriptViewModel(PastedScriptViewModel.ToSharedRef());
	}
}

void UNiagaraScratchPadViewModel::DeleteActiveScript()
{
	if (ActiveScriptViewModel.IsValid())
	{
		UNiagaraScript* ActiveScript = ActiveScriptViewModel->GetOriginalScript();
		FScopedTransaction DeleteTransaction(LOCTEXT("DeleteScratchPadScriptTransaction", "Delete scratch pad script."));
		for (TObjectIterator<UNiagaraNodeFunctionCall> It; It; ++It)
		{
			UNiagaraNodeFunctionCall* FunctionCallNode = *It;
			if (FunctionCallNode->FunctionScript == ActiveScript)
			{
				FunctionCallNode->Modify();
				FunctionCallNode->FunctionScript = nullptr;
			}
		}

		// Close any open tabs
		GetSystemViewModel()->GetDocumentViewModel()->CloseChildScript(ActiveScriptViewModel->GetGraphViewModel()->GetGraph());

		UObject* ScriptOuter;
		TArray<TObjectPtr<UNiagaraScript>>* TargetScripts;
		GetOuterAndTargetScripts(GetSystemViewModel(), ScriptOuter, TargetScripts);

		ScriptOuter->Modify();
		TargetScripts->Remove(ActiveScript);

		OnScriptDeletedDelegate.Broadcast();
		RefreshScriptViewModels();
		UpdateChangeId(GetSystemViewModel());
	}
}

FName GetUniqueScriptName(UObject* Outer, const FString& CandidateName)
{
	return FNiagaraEditorUtilities::GetUniqueObjectName<UNiagaraScript>(Outer, CandidateName);
}

TSharedPtr<FNiagaraScratchPadScriptViewModel> UNiagaraScratchPadViewModel::CreateNewScript(ENiagaraScriptUsage InScriptUsage, ENiagaraScriptUsage InTargetSupportedUsage, FNiagaraTypeDefinition InOutputType)
{
	UObject* ScriptOuter;
	TArray<TObjectPtr<UNiagaraScript>>* TargetScripts;
	GetOuterAndTargetScripts(GetSystemViewModel(), ScriptOuter, TargetScripts);

	// we have to make sure we can record the script outer into the transaction buffer. If not, undoing the creation of this script will keep the (deleted/undone) script in the outer
	if(ScriptOuter->HasAnyFlags(RF_Transactional) == false)
	{
		ScriptOuter->SetFlags(RF_Transactional);
	}
	ScriptOuter->Modify();
	
	UNiagaraScript* NewScript = nullptr;
	switch (InScriptUsage)
	{
	case ENiagaraScriptUsage::DynamicInput:
	{
		UNiagaraScript* DefaultDynamicInput = Cast<UNiagaraScript>(GetDefault<UNiagaraEditorSettings>()->DefaultDynamicInputScript.TryLoad());
		if (DefaultDynamicInput != nullptr &&
			Cast<UNiagaraScriptSource>(DefaultDynamicInput->GetLatestSource()) != nullptr &&
			Cast<UNiagaraScriptSource>(DefaultDynamicInput->GetLatestSource())->NodeGraph != nullptr)
		{
			NewScript = CastChecked<UNiagaraScript>(StaticDuplicateObject(DefaultDynamicInput, ScriptOuter, GetUniqueScriptName(ScriptOuter, TEXT("ScratchDynamicInput"))));
			TArray<UNiagaraNodeOutput*> OutputNodes;
			CastChecked<UNiagaraScriptSource>(NewScript->GetLatestSource())->NodeGraph->GetNodesOfClass(OutputNodes);
			if (OutputNodes.Num() == 1)
			{
				if (InOutputType.IsValid())
				{
					UNiagaraNodeOutput* DynamicInputOutputNode = OutputNodes[0];
					
					// Break pin lins before changing outputs to prevent old linked inputs from being retained as orphaned.
					DynamicInputOutputNode->BreakAllNodeLinks();

					// Add the new output and then refresh by notifying that the outputs property changed.
					DynamicInputOutputNode->Outputs.Empty();
					DynamicInputOutputNode->Outputs.Add(FNiagaraVariable(InOutputType, "Output"));
					FPropertyChangedEvent OutputsChangedEvent(FindFieldChecked<FProperty>(UNiagaraNodeOutput::StaticClass(), GET_MEMBER_NAME_CHECKED(UNiagaraNodeOutput, Outputs)));
					DynamicInputOutputNode->PostEditChangeProperty(OutputsChangedEvent);
				}
			}
		}
		break;
	}
	case ENiagaraScriptUsage::Module:
	{
		UNiagaraScript* DefaultModule = Cast<UNiagaraScript>(GetDefault<UNiagaraEditorSettings>()->DefaultModuleScript.TryLoad());
		if (DefaultModule != nullptr &&
			Cast<UNiagaraScriptSource>(DefaultModule->GetLatestSource()) != nullptr &&
			Cast<UNiagaraScriptSource>(DefaultModule->GetLatestSource())->NodeGraph != nullptr)
		{
			NewScript = CastChecked<UNiagaraScript>(StaticDuplicateObject(DefaultModule, ScriptOuter, GetUniqueScriptName(ScriptOuter, TEXT("ScratchModule"))));
		}
		break;
	}
	}

	if (NewScript == nullptr)
	{
		NewScript = NewObject<UNiagaraScript>(ScriptOuter);
		NewScript->Usage = InScriptUsage;
		UNiagaraScriptFactoryNew::InitializeScript(NewScript);
	}

	NewScript->ClearFlags(RF_Public | RF_Standalone);
	ScriptOuter->Modify();
	TargetScripts->Add(NewScript);
	NewScript->GetLatestScriptData()->ModuleUsageBitmask |= (1 << (int32)InTargetSupportedUsage);
	RefreshScriptViewModels();
	UpdateChangeId(GetSystemViewModel());

	return GetViewModelForScript(NewScript);
}

TSharedPtr<FNiagaraScratchPadScriptViewModel> UNiagaraScratchPadViewModel::CreateNewScriptAsDuplicate(const UNiagaraScript* ScriptToDuplicate)
{
	UObject* ScriptOuter;
	TArray<TObjectPtr<UNiagaraScript>>* TargetScripts;
	GetOuterAndTargetScripts(GetSystemViewModel(), ScriptOuter, TargetScripts);

	UNiagaraScript* NewScript = CastChecked<UNiagaraScript>(StaticDuplicateObject(ScriptToDuplicate, ScriptOuter, GetUniqueScriptName(ScriptOuter, *ScriptToDuplicate->GetFName().ToString())));
	NewScript->ClearFlags(RF_Public | RF_Standalone);
	ScriptOuter->Modify();
	TargetScripts->Add(NewScript);
	RefreshScriptViewModels();
	UpdateChangeId(GetSystemViewModel());

	return GetViewModelForScript(NewScript);
}

void UNiagaraScratchPadViewModel::CreateAssetFromActiveScript()
{
	if (ActiveScriptViewModel.IsValid())
	{
		if (ActiveScriptViewModel->HasUnappliedChanges())
		{
			EAppReturnType::Type DialogResult = FMessageDialog::Open(EAppMsgType::YesNoCancel, EAppReturnType::Cancel,
				LOCTEXT("ScriptHasUnappliedChangesMessage", "The selected scratch pad script has unapplied changes.\nWould you like to apply the changes before saving?\n"),
				LOCTEXT("ScriptHasUnappliedchangesLabel", "Apply Changes?"));
			if (DialogResult == EAppReturnType::Cancel)
			{
				return;
			}
			else if(DialogResult == EAppReturnType::Yes)
			{
				ActiveScriptViewModel->ApplyChanges();
			}
		}

		UNiagaraScript* ScriptToCopy = ActiveScriptViewModel->GetOriginalScript();
		const FString StartingPath = FPackageName::GetLongPackagePath(ScriptToCopy->GetOutermost()->GetName());
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		UNiagaraScript* NewAssetScript = Cast<UNiagaraScript>(AssetToolsModule.Get().DuplicateAssetWithDialogAndTitle(
			ScriptToCopy->GetName(), StartingPath, ScriptToCopy, LOCTEXT("CreateScriptAssetTitle", "Create Script As")));
		if (NewAssetScript != nullptr)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAssetScript);
		}
	}
}

void UNiagaraScratchPadViewModel::OpenEditorForActive()
{
	if (ActiveScriptViewModel.IsValid())
	{
		UEdGraph* Graph = ActiveScriptViewModel->GetGraphViewModel()->GetGraph();
		if (Graph)
		{
			GetSystemViewModel()->GetDocumentViewModel()->OpenChildScript(Graph);
		}
	}
}

bool UNiagaraScratchPadViewModel::CanSelectNextUsageForActiveScript()
{
	if (ActiveScriptViewModel.IsValid())
	{
		TArray<UNiagaraNodeFunctionCall*> ReferencingFunctionCallNodes;
		FNiagaraEditorUtilities::GetReferencingFunctionCallNodes(ActiveScriptViewModel->GetOriginalScript(), ReferencingFunctionCallNodes);
		return ReferencingFunctionCallNodes.Num() > 0;
	}
	return false;
}

void UNiagaraScratchPadViewModel::SelectNextUsageForActiveScript()
{
	if (ActiveScriptViewModel.IsValid())
	{
		UNiagaraScript* OriginalScript = ActiveScriptViewModel->GetOriginalScript();
		TArray<UNiagaraStackEntry*> ReferencingSelectableEntries;
		if (OriginalScript->GetUsage() == ENiagaraScriptUsage::Module)
		{
			for (TObjectIterator<UNiagaraStackModuleItem> It; It; ++It)
			{
				UNiagaraStackModuleItem* ModuleItem = *It;
				if (ModuleItem->GetModuleNode().FunctionScript == OriginalScript)
				{
					ReferencingSelectableEntries.Add(ModuleItem);
				}
			}
		}
		else if (OriginalScript->GetUsage() == ENiagaraScriptUsage::DynamicInput)
		{
			for (TObjectIterator<UNiagaraStackFunctionInput> It; It; ++It)
			{
				UNiagaraStackFunctionInput* FunctionInput = *It;
				if (FunctionInput->GetValueMode() == UNiagaraStackFunctionInput::EValueMode::Dynamic &&
					FunctionInput->GetDynamicInputNode()->FunctionScript == OriginalScript)
				{
					UNiagaraStackModuleItem* OwningModuleItem = FunctionInput->GetTypedOuter<UNiagaraStackModuleItem>();
					if (OwningModuleItem != nullptr)
					{
						ReferencingSelectableEntries.Add(OwningModuleItem);
					}
				}
			}
		}

		if (ReferencingSelectableEntries.Num() > 0)
		{
			int32 CurrentIndex = INDEX_NONE;
			TArray<UNiagaraStackEntry*> SelectedEntries;
			GetSystemViewModel()->GetSelectionViewModel()->GetSelectedEntries(SelectedEntries);
			if (SelectedEntries.Num() == 1)
			{
				CurrentIndex = ReferencingSelectableEntries.IndexOfByKey(SelectedEntries[0]);
			}

			CurrentIndex++;
			if (CurrentIndex >= ReferencingSelectableEntries.Num())
			{
				CurrentIndex = 0;
			}

			TArray<UNiagaraStackEntry*> NewSelectedEntries;
			NewSelectedEntries.Add(ReferencingSelectableEntries[CurrentIndex]);
			GetSystemViewModel()->GetSelectionViewModel()->UpdateSelectedEntries(NewSelectedEntries, SelectedEntries, true);
		}
	}
}

bool UNiagaraScratchPadViewModel::HasUnappliedChanges() const
{
	if (bHasUnappliedChangesCache.IsSet() == false)
	{
		bool bHasUnappliedChanges = false;
		for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScriptViewModel : ScriptViewModels)
		{
			bHasUnappliedChanges |= ScriptViewModel->HasUnappliedChanges();
		}
		bHasUnappliedChangesCache = bHasUnappliedChanges;
	}
	return bHasUnappliedChangesCache.GetValue();
}

UNiagaraScratchPadViewModel::FOnScriptViewModelsChanged& UNiagaraScratchPadViewModel::OnScriptViewModelsChanged()
{
	return OnScriptViewModelsChangedDelegate;
}

UNiagaraScratchPadViewModel::FOnScriptViewModelsChanged& UNiagaraScratchPadViewModel::OnEditScriptViewModelsChanged()
{
	return OnEditScriptViewModelsChangedDelegate;
}

UNiagaraScratchPadViewModel::FOnActiveScriptChanged& UNiagaraScratchPadViewModel::OnActiveScriptChanged()
{
	return OnActiveScriptChangedDelegate;
}

UNiagaraScratchPadViewModel::FOnScriptRenamed& UNiagaraScratchPadViewModel::OnScriptRenamed()
{
	return OnScriptRenamedDelegate;
}

UNiagaraScratchPadViewModel::FOnScriptDeleted& UNiagaraScratchPadViewModel::OnScriptDeleted()
{
	return OnScriptDeletedDelegate;
}

TSharedRef<FNiagaraSystemViewModel> UNiagaraScratchPadViewModel::GetSystemViewModel()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
	checkf(SystemViewModel.IsValid(), TEXT("SystemViewModel destroyed before scratch pad view model."));
	return SystemViewModel.ToSharedRef();
}

TSharedRef<FNiagaraScratchPadScriptViewModel> UNiagaraScratchPadViewModel::CreateAndSetupScriptviewModel(UNiagaraScript* ScratchPadScript, UNiagaraScript* ScratchPadEditScript)
{
	TSharedRef<FNiagaraScratchPadScriptViewModel> ScriptViewModel = MakeShared<FNiagaraScratchPadScriptViewModel>(GetSystemViewModel()->GetIsForDataProcessingOnly());
	ScriptViewModel->Initialize(ScratchPadScript, ScratchPadEditScript, SystemViewModelWeak);
	ScriptViewModel->GetGraphViewModel()->GetNodeSelection()->OnSelectedObjectsChanged().AddUObject(this, &UNiagaraScratchPadViewModel::ScriptGraphNodeSelectionChanged, TWeakPtr<FNiagaraScratchPadScriptViewModel>(ScriptViewModel));
	ScriptViewModel->OnRenamed().AddUObject(this, &UNiagaraScratchPadViewModel::ScriptViewModelScriptRenamed, TWeakPtr<FNiagaraScratchPadScriptViewModel>(ScriptViewModel));
	ScriptViewModel->OnGraphSelectionChanged().AddUObject(this, &UNiagaraScratchPadViewModel::ScriptViewModelGraphSelectionChanged, TWeakPtr<FNiagaraScratchPadScriptViewModel>(ScriptViewModel));
	ScriptViewModel->OnPinnedChanged().AddUObject(this, &UNiagaraScratchPadViewModel::ScriptViewModelPinnedChanged, TWeakPtr<FNiagaraScratchPadScriptViewModel>(ScriptViewModel));
	ScriptViewModel->OnHasUnappliedChangesChanged().AddUObject(this, &UNiagaraScratchPadViewModel::ScriptViewModelHasUnappliedChangesChanged);
	ScriptViewModel->OnChangesApplied().AddUObject(this, &UNiagaraScratchPadViewModel::ScriptViewModelChangesApplied);
	ScriptViewModel->OnRequestDiscardChanges().BindUObject(this, &UNiagaraScratchPadViewModel::ScriptViewModelRequestDiscardChanges, TWeakPtr<FNiagaraScratchPadScriptViewModel>(ScriptViewModel));
	ScriptViewModel->GetVariableSelection()->OnSelectedObjectsChanged().AddUObject(this, &UNiagaraScratchPadViewModel::ScriptViewModelVariableSelectionChanged, TWeakPtr<FNiagaraScratchPadScriptViewModel>(ScriptViewModel));
	return ScriptViewModel;
}

void UNiagaraScratchPadViewModel::TearDownScriptViewModel(TSharedRef<FNiagaraScratchPadScriptViewModel> InScriptViewModel)
{
	InScriptViewModel->GetGraphViewModel()->GetNodeSelection()->OnSelectedObjectsChanged().RemoveAll(this);
	InScriptViewModel->OnRenamed().RemoveAll(this);
	InScriptViewModel->OnGraphSelectionChanged().RemoveAll(this);
	InScriptViewModel->OnPinnedChanged().RemoveAll(this);
	InScriptViewModel->OnHasUnappliedChangesChanged().RemoveAll(this);
	InScriptViewModel->OnChangesApplied().RemoveAll(this);
	InScriptViewModel->OnRequestDiscardChanges().Unbind();
	InScriptViewModel->GetVariableSelection()->OnSelectedObjectsChanged().RemoveAll(this);
	InScriptViewModel->Finalize();
}

void UNiagaraScratchPadViewModel::RefreshEditScriptViewModels()
{
	EditScriptViewModels.Empty();
	EditScriptViewModels.Append(PinnedScriptViewModels);
	if (ActiveScriptViewModel.IsValid())
	{
		EditScriptViewModels.AddUnique(ActiveScriptViewModel.ToSharedRef());
	}
	OnEditScriptViewModelsChangedDelegate.Broadcast();
}

void UNiagaraScratchPadViewModel::ScriptGraphNodeSelectionChanged(TWeakPtr<FNiagaraScratchPadScriptViewModel> InScriptViewModelWeak)
{
	TSharedPtr<FNiagaraScratchPadScriptViewModel> InScriptViewModel = InScriptViewModelWeak.Pin();
	if (InScriptViewModel.IsValid())
	{
		TSet<UObject*> SelectedNodes = InScriptViewModel->GetGraphViewModel()->GetNodeSelection()->GetSelectedObjectsResolved();
		if (SelectedNodes.Num() > 0)
		{
			ObjectSelection->SetSelectedObjects(SelectedNodes);
		}
		else if (ActiveScriptViewModel.IsValid())
		{
			const FVersionedNiagaraScript& EditScript = ActiveScriptViewModel->GetEditScript();
			ObjectSelection->SetSelectedObject(EditScript.Script, &EditScript.Version);
		}
		else
		{
			ObjectSelection->ClearSelectedObjects();
		}
	}
}

void UNiagaraScratchPadViewModel::ScriptViewModelScriptRenamed(TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak)
{
	UpdateChangeId(GetSystemViewModel());
	OnScriptRenamed().Broadcast();
}

void UNiagaraScratchPadViewModel::ScriptViewModelGraphSelectionChanged(const UObject* Obj, TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak)
{
	const UNiagaraScriptVariable* Var = Cast<UNiagaraScriptVariable>(Obj);
	if (Var)
	{
		GetSystemViewModel()->GetParameterPanelViewModel()->SelectParameterItemByName(Var->Variable.GetName(), false);
	}
}

void UNiagaraScratchPadViewModel::ScriptViewModelPinnedChanged(TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak)
{
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModel = ScriptViewModelWeak.Pin();
	bool bPinnedCollectionChanged = false;
	if (ScriptViewModel.IsValid())
	{
		if (ScriptViewModel->GetIsPinned())
		{
			PinnedScriptViewModels.AddUnique(ScriptViewModel.ToSharedRef());
			if (ActiveScriptViewModel != ScriptViewModel)
			{
				SetActiveScriptViewModel(ScriptViewModel.ToSharedRef());
			}
			else
			{
				RefreshEditScriptViewModels();
			}
		}
		else
		{
			PinnedScriptViewModels.Remove(ScriptViewModel.ToSharedRef());
			if (PinnedScriptViewModels.Num() == 0)
			{
				if (ActiveScriptViewModel.IsValid() == false)
				{
					// When unpinning the last script, and there is no active script set it as the active script so that it remains displayed in the UI.
					SetActiveScriptViewModel(ScriptViewModel.ToSharedRef());
				}	
			}
			else
			{
				if (ActiveScriptViewModel == ScriptViewModel)
				{
					SetActiveScriptViewModel(PinnedScriptViewModels.Last());
				}
				else
				{
					RefreshEditScriptViewModels();
				}
			}
		}
	}
}

void UNiagaraScratchPadViewModel::ScriptViewModelHasUnappliedChangesChanged()
{
	bHasUnappliedChangesCache.Reset();
}

void UNiagaraScratchPadViewModel::ScriptViewModelChangesApplied(TSharedRef<FNiagaraScratchPadScriptViewModel> ViewModel)
{
	UpdateChangeId(GetSystemViewModel());
	FNiagaraEditorModule::Get().ScriptApplied(ViewModel->GetOriginalScript(), FGuid());
}

void UNiagaraScratchPadViewModel::ScriptViewModelRequestDiscardChanges(TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak)
{
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModel = ScriptViewModelWeak.Pin();
	if (ScriptViewModel.IsValid() && ScriptViewModel->HasUnappliedChanges())
	{
		EAppReturnType::Type DialogResult = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::No,
			LOCTEXT("DiscardChangesMessage", "Are you sure you want to discard changes?\nThis operation can not be undone."),
			LOCTEXT("DiscardChangesTitle", "Discard Changes?"));
		if (DialogResult == EAppReturnType::No)
		{
			return;
		}

		int32 ViewModelIndex = ScriptViewModels.IndexOfByKey(ScriptViewModel.ToSharedRef());
		if (ensureMsgf(ViewModelIndex != INDEX_NONE, TEXT("Active script view model wasn't in script view model collection!")))
		{
			TSharedRef<FNiagaraScratchPadScriptViewModel> NewScriptViewModel = CreateAndSetupScriptviewModel(ScriptViewModel->GetOriginalScript(), nullptr);
			ScriptViewModels[ViewModelIndex] = NewScriptViewModel;

			int32 PinnedViewModelIndex = PinnedScriptViewModels.IndexOfByKey(ScriptViewModel.ToSharedRef());
			if (PinnedViewModelIndex != INDEX_NONE)
			{
				PinnedScriptViewModels[PinnedViewModelIndex] = NewScriptViewModel;
			}

			OnScriptViewModelsChangedDelegate.Broadcast();
			SetActiveScriptViewModel(NewScriptViewModel);
		}
	}
}

void UNiagaraScratchPadViewModel::ScriptViewModelVariableSelectionChanged(TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak)
{
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModel = ScriptViewModelWeak.Pin();
	if (ScriptViewModel.IsValid())
	{
		ObjectSelection->SetSelectedObjects(ScriptViewModel->GetVariableSelection()->GetSelectedObjectsResolved());
	}
}

#undef LOCTEXT_NAMESPACE
