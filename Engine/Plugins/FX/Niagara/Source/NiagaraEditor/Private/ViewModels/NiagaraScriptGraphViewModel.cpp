// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptGraphViewModel.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeInput.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ScopedTransaction.h"
#include "EdGraphUtilities.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "NiagaraClipboard.h"
#include "NiagaraEditorModule.h"
#include "NiagaraNodeReroute.h"
#include "NiagaraScriptVariable.h"

#include "Misc/Base64.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "NiagaraNotificationWidgetProvider.h"
#include "Misc/Base64.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptGraphViewModel"

FNiagaraScriptGraphViewModel::FNiagaraScriptGraphViewModel(TAttribute<FText> InDisplayName, bool bInIsForDataProcessingOnly)
	: DisplayName(InDisplayName)
	, Commands(MakeShareable(new FUICommandList()))
	, NodeSelection(MakeShareable(new FNiagaraObjectSelection()))
	, PreviewMovieViewModel(MakeShared<FNiagaraPreviewMovieViewModel>())
	, bIsForDataProcessingOnly(bInIsForDataProcessingOnly)
{
	if (bIsForDataProcessingOnly == false)
	{
		SetupCommands();
		GEditor->RegisterForUndo(this);
	}

	ErrorColor = FAppStyle::GetColor("ErrorReporting.BackgroundColor");

	PreviewMovieViewModel->Initialize();
}

FNiagaraScriptGraphViewModel::~FNiagaraScriptGraphViewModel()
{
	if (bIsForDataProcessingOnly == false && GEditor != nullptr)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void FNiagaraScriptGraphViewModel::SetScriptSource(UNiagaraScriptSource* InScriptSrc)
{
	NodeSelection->ClearSelectedObjects();
	ScriptSource = InScriptSrc;
	OnGraphChangedDelegate.Broadcast();
}

FText FNiagaraScriptGraphViewModel::GetDisplayName() const
{
	return DisplayName.Get();
}

void FNiagaraScriptGraphViewModel::SetDisplayName(FText NewName)
{
	DisplayName.Set(NewName);
}

UNiagaraScriptSource* FNiagaraScriptGraphViewModel::GetScriptSource()
{
	return ScriptSource.Get();
}

UNiagaraGraph* FNiagaraScriptGraphViewModel::GetGraph() const
{
	if (ScriptSource.IsValid() && ScriptSource != nullptr)
	{
		return ScriptSource->NodeGraph;
	}
	return nullptr;
}

TSharedRef<FUICommandList> FNiagaraScriptGraphViewModel::GetCommands()
{
	return Commands;
}

TSharedRef<FNiagaraObjectSelection> FNiagaraScriptGraphViewModel::GetNodeSelection()
{
	return NodeSelection;
}

FNiagaraScriptGraphViewModel::FOnNodesPasted& FNiagaraScriptGraphViewModel::OnNodesPasted()
{
	return OnNodesPastedDelegate;
}

FNiagaraScriptGraphViewModel::FOnGraphChanged& FNiagaraScriptGraphViewModel::OnGraphChanged()
{
	return OnGraphChangedDelegate;
}

EVisibility FNiagaraScriptGraphViewModel::GetGraphErrorTextVisible() const
{
	return ErrorMsg.Len() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FNiagaraScriptGraphViewModel::GetGraphErrorText() const
{
	return LOCTEXT("GraphErrorText", "ERROR");
}

FSlateColor FNiagaraScriptGraphViewModel::GetGraphErrorColor() const
{
	return ErrorColor;
}

FText FNiagaraScriptGraphViewModel::GetGraphErrorMsgToolTip() const
{
	return FText::FromString(ErrorMsg);
}

void FNiagaraScriptGraphViewModel::SetErrorTextToolTip(FString ErrorMsgToolTip)
{
	ErrorMsg = ErrorMsgToolTip;
}

TSharedPtr<FNiagaraPreviewMovieViewModel> FNiagaraScriptGraphViewModel::GetPreviewMovieViewModel()
{
	return PreviewMovieViewModel;
}

void FNiagaraScriptGraphViewModel::PostUndo(bool bSuccess)
{
	// The graph may have been deleted as a result of the undo operation so make sure it's valid
	// before using it.
	UNiagaraGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		Graph->NotifyGraphChanged();
	}
}

void FNiagaraScriptGraphViewModel::SetupCommands()
{
	Commands->MapAction(
		FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::SelectAllNodes));

	Commands->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::DeleteSelectedNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::CanDeleteNodes));

	Commands->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::CopySelectedNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::CanCopyNodes));

	Commands->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::CutSelectedNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::CanCutNodes));

	Commands->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::PasteNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::CanPasteNodes));

	Commands->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::DuplicateNodes),
		FCanExecuteAction::CreateRaw(this, &FNiagaraScriptGraphViewModel::CanDuplicateNodes));
}

void FNiagaraScriptGraphViewModel::SelectAllNodes()
{
	UNiagaraGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		TArray<UObject*> AllNodes;
		Graph->GetNodesOfClass<UObject>(AllNodes);
		TSet<UObject*> AllNodeSet;
		AllNodeSet.Append(AllNodes);
		NodeSelection->SetSelectedObjects(AllNodeSet);
	}
}

void FNiagaraScriptGraphViewModel::DeleteSelectedNodes()
{
	UNiagaraGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		const FScopedTransaction Transaction(FGenericCommands::Get().Delete->GetDescription());
		Graph->Modify();

		TSet<UObject*> NodesToDelete = NodeSelection->GetSelectedObjectsResolved();
		NodeSelection->ClearSelectedObjects();

		for (UObject* NodeToDelete : NodesToDelete)
		{
			UEdGraphNode* GraphNodeToDelete = Cast<UEdGraphNode>(NodeToDelete);
			if (GraphNodeToDelete != nullptr && GraphNodeToDelete->CanUserDeleteNode())
			{
				GraphNodeToDelete->Modify();
				GraphNodeToDelete->DestroyNode();
			}
		}
	}
}

bool FNiagaraScriptGraphViewModel::CanDeleteNodes() const
{
	UNiagaraGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		for (UObject* SelectedNode : NodeSelection->GetSelectedObjectsResolved())
		{
			UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode);
			if (SelectedGraphNode != nullptr && SelectedGraphNode->CanUserDeleteNode())
			{
				return true;
			}
		}
	}
	return false;
}

void FNiagaraScriptGraphViewModel::CutSelectedNodes()
{
	// Collect nodes which can not be delete or duplicated so they can be reselected.
	TSet<UObject*> CanBeDuplicatedAndDeleted;
	TSet<UObject*> CanNotBeDuplicatedAndDeleted;
	for (UObject* SelectedNode : NodeSelection->GetSelectedObjectsResolved())
	{
		UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode);
		if (SelectedGraphNode != nullptr)
		{
			if (SelectedGraphNode->CanDuplicateNode() && SelectedGraphNode->CanUserDeleteNode())
			{
				CanBeDuplicatedAndDeleted.Add(SelectedNode);
			}
			else
			{
				CanNotBeDuplicatedAndDeleted.Add(SelectedNode);
			}
		}
	}

	// Select the nodes which can be copied and deleted, copy and delete them, and then restore the ones which couldn't be copied or deleted.
	NodeSelection->SetSelectedObjects(CanBeDuplicatedAndDeleted);
	CopySelectedNodes();
	DeleteSelectedNodes();
	NodeSelection->SetSelectedObjects(CanNotBeDuplicatedAndDeleted);
}

bool FNiagaraScriptGraphViewModel::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void FNiagaraScriptGraphViewModel::CopySelectedNodes()
{
	UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();

	// we put all nodes we want to copy into the clipboard content. We also cache a set of nodes for more performant lookup later.
	TSet<UObject*> CachedNodes;
	for (UObject* SelectedNode : NodeSelection->GetSelectedObjectsResolved())
	{
		UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode);
		if (SelectedGraphNode != nullptr)
		{
			if (SelectedGraphNode->CanDuplicateNode())
			{
				SelectedGraphNode->PrepareForCopying();
				CachedNodes.Add(SelectedGraphNode);
			}
		}
	}

	FString ExportedNodesText;
	FEdGraphUtilities::ExportNodesToText(CachedNodes, ExportedNodesText);

	ClipboardContent->ExportedNodes = FBase64::Encode(ExportedNodesText);
	
	// we then attempt to find all variables referenced by those nodes to copy the script variables in our clipboard
	TSet<FNiagaraVariable> Variables;
	const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& ParameterReferenceMap = GetGraph()->GetParameterReferenceMap();
	for(auto& VariableData : ParameterReferenceMap)
	{
		for(auto& ParameterReference : VariableData.Value.ParameterReferences)
		{
			if(CachedNodes.Find(ParameterReference.Value.Get()))
			{
				Variables.Add(VariableData.Key);
				break;
			}
		}
	}	

	for(FNiagaraVariable& Var : Variables)
	{
		UNiagaraScriptVariable* ScriptVariable = GetGraph()->GetScriptVariable(Var);

		if(ScriptVariable)
		{
			UNiagaraScriptVariable* CopiedVariable = Cast<UNiagaraScriptVariable>(StaticDuplicateObject(ScriptVariable, ClipboardContent));
			ClipboardContent->ScriptVariables.AddUnique({*CopiedVariable});
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Log, "Variable %ls was encountered as parameter during a copy nodes operation, but the corresponding script variable could not be found.", *(Var.GetName().ToString()));
		}
	}
	
	FNiagaraEditorModule::Get().GetClipboard().SetClipboardContent(ClipboardContent);
}

bool FNiagaraScriptGraphViewModel::CanCopyNodes() const
{
	UNiagaraGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		for (UObject* SelectedNode : NodeSelection->GetSelectedObjectsResolved())
		{
			UEdGraphNode* SelectedGraphNode = Cast<UEdGraphNode>(SelectedNode);
			if (SelectedGraphNode != nullptr && SelectedGraphNode->CanDuplicateNode())
			{
				return true;
			}
		}
	}
	return false;
}


void FNiagaraScriptGraphViewModel::PasteNodes()
{
	UEdGraph* Graph = GetGraph();
	if (Graph == nullptr)
	{
		return;
	}
	
	const FScopedTransaction Transaction(FGenericCommands::Get().Paste->GetDescription());;
	Graph->Modify();

	NodeSelection->ClearSelectedObjects();

	const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent();
	
	if (ClipboardContent == nullptr)
	{
		return;
	}
	
	TSet<UEdGraphNode*> PastedNodes;
	FString ExportedNodesText;
	FBase64::Decode(ClipboardContent->ExportedNodes, ExportedNodesText);
	FEdGraphUtilities::ImportNodesFromText(Graph, ExportedNodesText, PastedNodes);
	
	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		PastedNode->CreateNewGuid();
		UNiagaraNode* Node = Cast<UNiagaraNode>(PastedNode);
		if (Node)
			Node->MarkNodeRequiresSynchronization(__FUNCTION__, false);
	}
	
	Graph->NotifyGraphChanged();
	UNiagaraGraph* NiagaraGraph = CastChecked<UNiagaraGraph>(Graph);
	
	// NotifyGraphChanged will invalidate caches, so we are guaranteed to get the updated reference map after pasting nodes
	const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& ParameterReferenceMap = NiagaraGraph->GetParameterReferenceMap();

	// Create a UNiagaraScriptVariable instance for every entry in the parameter map for which there is no existing script variable. 
	TArray<FNiagaraVariable> VarsToAdd;
	for (auto& ParameterToReferences : ParameterReferenceMap)
	{
		UNiagaraScriptVariable* Variable = NiagaraGraph->GetScriptVariable(ParameterToReferences.Key.GetName());
		if (Variable == nullptr)
		{
			VarsToAdd.Add(ParameterToReferences.Key);
		}
	}

	TArray<FNiagaraVariable> StaticSwitchInputs = NiagaraGraph->FindStaticSwitchInputs();

	for(const FNiagaraClipboardScriptVariable& ClipboardScriptVariable : ClipboardContent->ScriptVariables)
	{
		const UNiagaraScriptVariable* ScriptVariable = ClipboardScriptVariable.ScriptVariable;

		/** This should generally never be nullptr at this point, but:
		*   When a script variable can't be properly deserialized due to faulty object (not variable!) names (such as "Module.Normalize Thickness" instead of "NiagaraScriptVariable_2")
		*   It will be nullptr so we just skip it to avoid a crash */
		if(ScriptVariable == nullptr)
		{
			continue;
		}
		
		// we make sure here that we only copy over script variables if they didn't exist before. We inform the user so they know.
		if(UNiagaraScriptVariable* ExistingScriptVariable = NiagaraGraph->GetScriptVariable(ScriptVariable->Variable))
		{
			// if we have the variable already exists and change IDs are the same, we assume we are pasting the same content multiple times.
			// in that case, we can simply ignore the variable
			if(ExistingScriptVariable->GetChangeId() == ClipboardScriptVariable.OriginalChangeId)
			{
				continue;
			}

			// however, if the existing variable differs somehow, we notify the user that meta data copy-over is being skipped.
			FNotificationInfo Info(FText::GetEmpty());
			Info.ExpireDuration = 5.0f;
			Info.bFireAndForget = true;

			// we override the notification by providing our own widget
			TSharedRef<SWidget> NotificationWidget = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SNiagaraParameterNameTextBlock)
				.ParameterText(FText::FromName(ExistingScriptVariable->Variable.GetName()))
				.IsReadOnly(true)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SkippedPastingMetaDataForDuplicateVariable",	"Skipped pasting meta data for parameter.\nParameter already existed."))
			];
			
			TSharedRef<FNiagaraParameterNotificationWidgetProvider> WidgetProvider = MakeShared<FNiagaraParameterNotificationWidgetProvider>(NotificationWidget);
			Info.ContentWidget = WidgetProvider;
		
			FSlateNotificationManager::Get().AddNotification(Info);
		}
		else
		{
			// we create the new parameter + script variable combo here and copy over metadata
			UNiagaraScriptVariable* NewScriptVariable = NiagaraGraph->AddParameter(ScriptVariable->Variable, StaticSwitchInputs.Contains(ScriptVariable->Variable));
			NewScriptVariable->InitFrom(ScriptVariable);
			// we restore the change ID here to ensure that further paste attempts will check against the correct ID.
			NewScriptVariable->SetChangeId(ClipboardScriptVariable.OriginalChangeId);
		}
	}
	

	// fixup pasted nodes includes adding the propagated static switch values to the graph. Since we already added these above, they won't have an effect.
	// but since this is needed elsewhere, we keep it in there
	FNiagaraEditorUtilities::FixUpPastedNodes(Graph, PastedNodes);

	OnNodesPastedDelegate.Broadcast(PastedNodes);

	TSet<UObject*> PastedObjects;
	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		PastedObjects.Add(PastedNode);
	}

	NodeSelection->SetSelectedObjects(PastedObjects);
	NiagaraGraph->NotifyGraphNeedsRecompile();
}

bool FNiagaraScriptGraphViewModel::CanPasteNodes() const
{
	UNiagaraGraph* Graph = GetGraph();
	if (Graph == nullptr)
	{
		return false;
	}

	const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent();
	FString ExportedNodesText;
	if(ClipboardContent)
	{
		FBase64::Decode(ClipboardContent->ExportedNodes, ExportedNodesText);
	}
	return ClipboardContent ? FEdGraphUtilities::CanImportNodesFromText(Graph, ExportedNodesText) : false;
}

void FNiagaraScriptGraphViewModel::DuplicateNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

bool FNiagaraScriptGraphViewModel::CanDuplicateNodes() const
{
	return CanCopyNodes();
}

#undef LOCTEXT_NAMESPACE // NiagaraScriptGraphViewModel
