// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "NiagaraObjectSelection.h"
#include "EditorUndoClient.h"
#include "Styling/SlateColor.h"
#include "Framework/Commands/UICommandList.h"

class UNiagaraScriptSource;
class UNiagaraGraph;

/** A view model for editing a niagara script in a graph editor. */
class FNiagaraScriptGraphViewModel : public TSharedFromThis<FNiagaraScriptGraphViewModel>, public FEditorUndoClient
{
public:
	/** A multicast delegate which is called when nodes are pasted in the graph which supplies the pasted nodes. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodesPasted, const TSet<UEdGraphNode*>&);

	DECLARE_MULTICAST_DELEGATE(FOnGraphChanged);

public:
	/** Create a new view model with the supplied  display name. */
	FNiagaraScriptGraphViewModel(TAttribute<FText> InDisplayName, bool bInIsForDataProcessingOnly);

	~FNiagaraScriptGraphViewModel();

	/** Sets this view model to a new script. */
	void SetScriptSource(UNiagaraScriptSource* InScriptSource);

	/** Gets the display text for this graph. */
	FText GetDisplayName() const;

	/** Sets the display text for this graph. */
	void SetDisplayName(FText NewName);

	/** Gets the script displayed and edited by this view model. */
	UNiagaraScriptSource* GetScriptSource();

	/** Gets the graph which is used to edit, view, and compile the script. */
	UNiagaraGraph* GetGraph() const;

	/** Gets commands used for editing the graph. */
	TSharedRef<FUICommandList> GetCommands();

	/** Gets the currently selected graph nodes. */
	TSharedRef<FNiagaraObjectSelection> GetNodeSelection();

	/** Sets the currently selected graph nodes. */
	void SetSelectedNodes(const TSet<UObject*>& InSelectedNodes);

	/** Clears the currently selected graph nodes. */
	void ClearSelectedNodes();

	/** Gets a multicast delegate which is called any time nodes are pasted in the graph. */
	FOnNodesPasted& OnNodesPasted();

	/** Gets a multicast delegate which is called whenever the graph object is changed to a different graph. */
	FOnGraphChanged& OnGraphChanged();

	EVisibility GetGraphErrorTextVisible() const;
	FText GetGraphErrorText() const;
	FSlateColor GetGraphErrorColor() const;
	FText GetGraphErrorMsgToolTip() const;

	void SetErrorTextToolTip(FString ErrorMsgToolTip);

	TSharedPtr<FNiagaraPreviewMovieViewModel> GetPreviewMovieViewModel();

	//~ FEditorUndoClient interface.
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

private:
	void SetupCommands();

	void SelectAllNodes();
	void DeleteSelectedNodes();
	bool CanDeleteNodes() const;
	void CutSelectedNodes();
	bool CanCutNodes() const;
	void CopySelectedNodes();
	bool CanCopyNodes() const;
	void PasteNodes();
	bool CanPasteNodes() const;
	void DuplicateNodes();
	bool CanDuplicateNodes() const;

private:

	/** The script being view and edited by this view model. */
	TWeakObjectPtr<UNiagaraScriptSource> ScriptSource;

	/** The display name for the script graph. */
	TAttribute<FText> DisplayName;

	/** Commands for editing the graph. */
	TSharedRef<FUICommandList> Commands;

	/** The set of nodes objects currently selected in the graph. */
	TSharedRef<FNiagaraObjectSelection> NodeSelection;
	
	/** A multicast delegate which is called whenever nodes are pasted into the graph. */
	FOnNodesPasted OnNodesPastedDelegate;

	/** A multicast delegate which is called whenever the graph object is changed to a different graph. */
	FOnGraphChanged OnGraphChangedDelegate;

	/** Used to report errors on the node */
	FString ErrorMsg;
	
	/** Used to set the error color */
	FSlateColor ErrorColor;

	TSharedPtr<FNiagaraPreviewMovieViewModel> PreviewMovieViewModel;

	/** Whether or not this view model is going to be used for data processing only and will not be shown in the UI. */
	bool bIsForDataProcessingOnly;
};
