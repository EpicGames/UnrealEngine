// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptInputCollectionViewModel.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraTypes.h"
#include "NiagaraScriptParameterViewModel.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeInput.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraDataInterface.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptInputCollection"

FText DisplayNameFormat = NSLOCTEXT("ScriptInputCollection", "DisplayNameFormat", "{0} Inputs");

FNiagaraScriptInputCollectionViewModel::FNiagaraScriptInputCollectionViewModel(TAttribute<FText> InDisplayName, ENiagaraParameterEditMode InParameterEditMode)
	: FNiagaraParameterCollectionViewModel(InParameterEditMode)
	, DisplayName(InDisplayName)
	, bCanHaveNumericParameters(true)
{
}

FNiagaraScriptInputCollectionViewModel::~FNiagaraScriptInputCollectionViewModel()
{
	for (int32 i = 0; i < ParameterViewModels.Num(); i++)
	{
		FNiagaraScriptParameterViewModel* ParameterViewModel = (FNiagaraScriptParameterViewModel*)(&ParameterViewModels[i].Get());
		if (ParameterViewModel != nullptr)
		{
			ParameterViewModel->Reset();
			ParameterViewModel->OnNameChanged().RemoveAll(this);
			ParameterViewModel->OnTypeChanged().RemoveAll(this);
			ParameterViewModel->OnDefaultValueChanged().RemoveAll(this);
		}
	}
	ParameterViewModels.Empty();

	if (Graph.IsValid())
	{
		Graph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
		Graph->RemoveOnGraphNeedsRecompileHandler(OnRecompileHandle);
	}
}

void FNiagaraScriptInputCollectionViewModel::SetScripts(TArray<FVersionedNiagaraScript>& InScripts)
{
	if (Graph.IsValid())
	{
		Graph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
		Graph->RemoveOnGraphNeedsRecompileHandler(OnRecompileHandle);
	}

	Scripts.Empty();
	for (FVersionedNiagaraScript& Script : InScripts)
	{
		Scripts.Add(Script.ToWeakPtr());
		check(Script.Script->GetSource(Script.Version) == InScripts[0].Script->GetSource(InScripts[0].Version));
	}

	if (InScripts.Num() != 0 && Scripts[0].Script.IsValid() && Scripts[0].Script->GetSource(Scripts[0].Version))
	{
		Graph = Cast<UNiagaraScriptSource>(Scripts[0].Script->GetSource(Scripts[0].Version))->NodeGraph;
		OnGraphChangedHandle = Graph->AddOnGraphChangedHandler(
			FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraScriptInputCollectionViewModel::OnGraphChanged));
		OnRecompileHandle = Graph->AddOnGraphNeedsRecompileHandler(
			FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraScriptInputCollectionViewModel::OnGraphChanged));
		bCanHaveNumericParameters = Scripts[0].Script->IsStandaloneScript();
	}
	else
	{
		Graph = nullptr;
		bCanHaveNumericParameters = true;
	}

	RefreshParameterViewModels();
}

FText FNiagaraScriptInputCollectionViewModel::GetDisplayName() const 
{
	return FText::Format(DisplayNameFormat, DisplayName.Get());
}

FVector2D GetNewNodeLocation(UNiagaraGraph* Graph, UNiagaraNode* NewInputNode, float VerticalNodeOffset, float HorizontalNodeOffset)
{
	FVector2D PlacementLocation = FVector2D::ZeroVector;
	TArray<UNiagaraNodeInput*> InputNodes;
	Graph->GetNodesOfClass(InputNodes);
	if (InputNodes.Num() > 1)
	{
		// If there are input nodes, try to put it under the lowest one.
		UNiagaraNodeInput* LowestNode = nullptr;
		for (UNiagaraNodeInput* InputNode : InputNodes)
		{
			if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter && InputNode != NewInputNode && (LowestNode == nullptr || InputNode->NodePosY > LowestNode->NodePosY))
			{
				LowestNode = InputNode;
			}
		}
		
		if (LowestNode)
		{
			PlacementLocation = FVector2D(
				LowestNode->NodePosX,
				LowestNode->NodePosY + VerticalNodeOffset);
		}
		else
		{
			PlacementLocation = FVector2D(0.0f, 0.0f);
		}
	}
	else
	{
		TArray<UNiagaraNode*> Nodes;
		Graph->GetNodesOfClass(Nodes);
		if (Nodes.Num() > 0)
		{
			// If there are other nodes, try to put it to the left of the leftmost one.
			UNiagaraNode* LeftmostNode = nullptr;
			for (UNiagaraNode* Node : Nodes)
			{
				if (Node != NewInputNode && (LeftmostNode == nullptr || Node->NodePosX < LeftmostNode->NodePosX))
				{
					LeftmostNode = Node;
				}
			}
			check(LeftmostNode);
			PlacementLocation = FVector2D(
				LeftmostNode->NodePosX - HorizontalNodeOffset,
				LeftmostNode->NodePosY);
		}
	}
	return PlacementLocation;
}

void FNiagaraScriptInputCollectionViewModel::AddParameter(TSharedPtr<FNiagaraTypeDefinition> ParameterType)
{
	if (Graph.IsValid())
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("AddScriptInput", "Add script input"));
		Graph->Modify();
		FGraphNodeCreator<UNiagaraNodeInput> InputNodeCreator(*Graph);
		UNiagaraNodeInput* InputNode = InputNodeCreator.CreateNode();

		FNiagaraEditorUtilities::InitializeParameterInputNode(*InputNode, *ParameterType.Get(), Graph.Get());

		FVector2D PlacementLocation = GetNewNodeLocation(Graph.Get(), InputNode, 100, 150);
		InputNode->NodePosX = PlacementLocation.X;
		InputNode->NodePosY = PlacementLocation.Y;

		InputNodeCreator.Finalize();

		// The CreateNode notified the graph has changed, but changing the name and type will also need to signal the graph changed event...
		// We need to do this because NiagaraSystemScriptView model is listening for these changes to update the bindings table. This 
		// will also cause the RefreshParameterViewModels in our own graph changed handler.
		Graph->NotifyGraphChanged();

		for (TSharedRef<INiagaraParameterViewModel> ParameterViewModel : ParameterViewModels)
		{
			if (ParameterViewModel->GetName() == InputNode->Input.GetName())
			{
				GetSelection().SetSelectedObject(ParameterViewModel);
				break;
			}
		}
	}
}

void FNiagaraScriptInputCollectionViewModel::DeleteSelectedParameters()
{
	if (GetSelection().GetSelectedObjects().Num() > 0)
	{
		TSet<FNiagaraVariable> VariablesToDelete;
		for (TSharedRef<INiagaraParameterViewModel> InputParameter : GetSelection().GetSelectedObjects())
		{
			VariablesToDelete.Add(InputParameter->GetVariable());
		}
		GetSelection().ClearSelectedObjects();
		DeleteParameters(VariablesToDelete.Array());		
	}
}

void FNiagaraScriptInputCollectionViewModel::DeleteParameters(TArray<FNiagaraVariable> ParametersToDelete)
{
	if (Graph.IsValid())
	{
		FScopedTransaction ScopedTransaction(NSLOCTEXT("NiagaraEmitterInputEditor", "DeletedSelectedNodes", "Delete selected nodes"));
		Graph->Modify();

		TArray<UNiagaraNodeInput*> InputNodes;
		Graph->GetNodesOfClass(InputNodes);
		for (UNiagaraNodeInput* InputNode : InputNodes)
		{
			if (ParametersToDelete.Contains(InputNode->Input))
			{
				InputNode->Modify();
				InputNode->DestroyNode();
			}
		}
	}
}

const TArray<TSharedRef<INiagaraParameterViewModel>>& FNiagaraScriptInputCollectionViewModel::GetParameters()
{
	return ParameterViewModels;
}

void FNiagaraScriptInputCollectionViewModel::RefreshParameterViewModels()
{
	for (int32 i = 0; i < ParameterViewModels.Num(); i++)
	{
		FNiagaraScriptParameterViewModel* ParameterViewModel = (FNiagaraScriptParameterViewModel*)(&ParameterViewModels[i].Get());
		if (ParameterViewModel != nullptr)
		{
			ParameterViewModel->Reset();
			ParameterViewModel->OnNameChanged().RemoveAll(this);
			ParameterViewModel->OnTypeChanged().RemoveAll(this);
			ParameterViewModel->OnDefaultValueChanged().RemoveAll(this);
		}
	}

	ParameterViewModels.Empty();

	TArray<UNiagaraNodeInput*> InputNodes;

	if (Graph.IsValid())
	{
		UNiagaraGraph::FFindInputNodeOptions Options;
		Options.bSort = true;
		Graph->FindInputNodes(InputNodes, Options);
	}

	TSet<FName> AddedInputNames;
	for (UNiagaraNodeInput* InputNode : InputNodes)
	{
		if (0 == Scripts.Num())
		{
			continue;
		}

		// We can have multiple input nodes in the graph for each unique input name so make sure we only add one of each.
		if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter && AddedInputNames.Contains(InputNode->Input.GetName()) == false)
		{
			FNiagaraVariable& GraphVariable = InputNode->Input;
			TSharedPtr<FNiagaraScriptParameterViewModel> ParameterViewModel;
			if (GraphVariable.GetType().GetScriptStruct() != nullptr)
			{
				FNiagaraVariable* EmitterVariable = nullptr;
				UNiagaraScript* Script = nullptr;
				for (FVersionedNiagaraScriptWeakPtr& ScriptWeakPtr : Scripts)
				{
					if (!ScriptWeakPtr.Script.IsValid() || !ScriptWeakPtr.Script->GetVMExecutableData().IsValid())
					{
						continue;
					}
					if (EmitterVariable != nullptr)
					{
						break;
					}

					for (FNiagaraVariable& EmitterVariableToCheck : ScriptWeakPtr.Script->GetVMExecutableData().Parameters.Parameters)
					{

						// @TODO We should check ID's here, but its possible that the 
						// EmitterVariable is invalid and we may not have a great way to 
						// recover without also changing the VM.
						if (EmitterVariableToCheck.GetName() == GraphVariable.GetName())
						{
							EmitterVariable = &EmitterVariableToCheck;
							Script = ScriptWeakPtr.Script.Get();
							break;
						}
					}
				}
				ParameterViewModel = MakeShareable(new FNiagaraScriptParameterViewModel(GraphVariable, *InputNode, EmitterVariable, Script, ParameterEditMode));
				
			}
			else if (InputNode->Input.IsDataInterface())
			{
				UNiagaraDataInterface* EmitterDataInterface = InputNode->GetDataInterface();
				for (FVersionedNiagaraScriptWeakPtr& ScriptWeakPtr : Scripts)
				{
					if (EmitterDataInterface != nullptr)
					{
						break;
					}

					for (FNiagaraScriptDataInterfaceInfo& DataInterfaceInfoItem : ScriptWeakPtr.Script->GetCachedDefaultDataInterfaces())
					{
						if (DataInterfaceInfoItem.Name == InputNode->Input.GetName())
						{
							EmitterDataInterface = DataInterfaceInfoItem.DataInterface;
							break;
						}
					}
				}
				ParameterViewModel = MakeShareable(new FNiagaraScriptParameterViewModel(GraphVariable, *InputNode, EmitterDataInterface, ParameterEditMode));
			}
			else if (InputNode->Input.IsUObject())
			{
				UObject* ObjectAsset = InputNode->GetObjectAsset();
				ParameterViewModel = MakeShareable(new FNiagaraScriptParameterViewModel(GraphVariable, *InputNode, ObjectAsset, ParameterEditMode));
			}

			if (ParameterViewModel)
			{
				ParameterViewModel->OnNameChanged().AddSP(this, &FNiagaraScriptInputCollectionViewModel::OnParameterNameChanged, TWeakObjectPtr<UNiagaraNodeInput>(InputNode));
				ParameterViewModel->OnTypeChanged().AddSP(this, &FNiagaraScriptInputCollectionViewModel::OnParameterTypeChanged, &GraphVariable);
				ParameterViewModel->OnDefaultValueChanged().AddSP(this, &FNiagaraScriptInputCollectionViewModel::OnParameterValueChangedInternal, ParameterViewModel.ToSharedRef());
				ParameterViewModels.Add(ParameterViewModel.ToSharedRef());
				AddedInputNames.Add(InputNode->Input.GetName());
			}
		}
	}

	OnCollectionChangedDelegate.Broadcast();
}

bool FNiagaraScriptInputCollectionViewModel::SupportsType(const FNiagaraTypeDefinition& Type) const
{
	//if (Type.IsUObject() && Type.IsDataInterface() == false)
	//{
	//	// Don't allow generic objects as script inputs.
	//	return false;
	//}

	if (Type.IsStatic())
	{
		return true;
	}

	if (Type.IsInternalType())
	{
		return false;
	}

	if (Scripts.Num() == 1 && Scripts[0].Script.IsValid())
	{
		// We only support parameter map inputs for dynamic inputs and modules, with the ability to create data interfaces as needed for defaults.
		if (Scripts[0].Script->GetUsage() == ENiagaraScriptUsage::DynamicInput || Scripts[0].Script->GetUsage() == ENiagaraScriptUsage::Module)
		{
			if (Type != FNiagaraTypeDefinition::GetParameterMapDef() && !Type.IsDataInterface() && !Type.IsUObject())
			{
				return false;
			}
		}
	}

	return bCanHaveNumericParameters || Type != FNiagaraTypeDefinition::GetGenericNumericDef();
}

void FNiagaraScriptInputCollectionViewModel::OnGraphChanged(const struct FEdGraphEditAction& InAction)
{
	bNeedsRefresh = true;
}

void FNiagaraScriptInputCollectionViewModel::OnParameterNameChanged(FName OldName, FName NewName, TWeakObjectPtr<UNiagaraNodeInput> InputNodeWeak)
{
	UNiagaraNodeInput* InputNode = InputNodeWeak.Get();
	if (InputNode != nullptr)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("EditInputName", "Edit input name"));
		InputNode->OnRenameNode(NewName.ToString());
	}
}

void FNiagaraScriptInputCollectionViewModel::SetAllParametersEditingEnabled(bool bInEnabled)
{
	for (TSharedRef<INiagaraParameterViewModel>& ParameterViewModel : ParameterViewModels)
	{
		ParameterViewModel->SetEditingEnabled(bInEnabled);
	}
}

void FNiagaraScriptInputCollectionViewModel::SetAllParametersTooltipOverrides(const FText& Override)
{
	for (TSharedRef<INiagaraParameterViewModel>& ParameterViewModel : ParameterViewModels)
	{
		ParameterViewModel->SetTooltipOverride(Override);
	}
}

TSharedPtr<INiagaraParameterViewModel> FNiagaraScriptInputCollectionViewModel::GetParameterViewModel(const FName& Name)
{
	for (TSharedRef<INiagaraParameterViewModel>& ParameterViewModel : ParameterViewModels)
	{
		if (ParameterViewModel->GetName() == Name)
		{
			return TSharedPtr<INiagaraParameterViewModel>(ParameterViewModel);
		}
	}
	return TSharedPtr<INiagaraParameterViewModel>();
}


void FNiagaraScriptInputCollectionViewModel::OnParameterTypeChanged(FNiagaraVariable* ParameterVariable)
{
	TArray<UNiagaraNodeInput*> InputNodes;
	TArray<UNiagaraNodeInput*> InputNodesToUpdate;

	if (Graph.IsValid())
	{
		Graph->GetNodesOfClass(InputNodes);
	}
	
	for (UNiagaraNodeInput* InputNode : InputNodes)
	{
		if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter && InputNode->Input.GetName() == ParameterVariable->GetName())
		{
			InputNodesToUpdate.Add(InputNode);
		}
	}

	if (InputNodesToUpdate.Num() > 0)
	{
		// Reinitialize the first node found.
		UNiagaraNodeInput* FirstNodeToUpdate = InputNodesToUpdate[0];
		FirstNodeToUpdate->Modify();
		TSet<FName> EmptyNames;
		FNiagaraEditorUtilities::InitializeParameterInputNode(*FirstNodeToUpdate, ParameterVariable->GetType(), nullptr, FirstNodeToUpdate->Input.GetName());
		FirstNodeToUpdate->NotifyInputTypeChanged();

		// Then copy that nodes input variable to the others so that they all have the same id and data object pointers.
		for (int32 i = 1; i < InputNodesToUpdate.Num(); ++i)
		{
			UNiagaraNodeInput* InputNodeToUpdate = InputNodesToUpdate[i];
			InputNodeToUpdate->Modify();
			InputNodeToUpdate->Input = FirstNodeToUpdate->Input;
			InputNodeToUpdate->SetDataInterface(FirstNodeToUpdate->GetDataInterface());
			InputNodeToUpdate->SetObjectAsset(FirstNodeToUpdate->GetObjectAsset());
			InputNodeToUpdate->NotifyInputTypeChanged();
		}
	}

	// Synchronize script variables...
	for (FVersionedNiagaraScriptWeakPtr& VersionedScript : Scripts)
	{
		if (!VersionedScript.Script.IsValid() || !VersionedScript.Script->GetVMExecutableData().IsValid())
		{
			continue;
		}
		for (FNiagaraVariable& EmitterVariableToCheck : VersionedScript.Script->GetVMExecutableData().Parameters.Parameters)
		{
			if (EmitterVariableToCheck.GetName() == ParameterVariable->GetName() && ParameterVariable != &EmitterVariableToCheck)
			{
				EmitterVariableToCheck = *ParameterVariable;
				break;
			}
		}
	}

}

void FNiagaraScriptInputCollectionViewModel::OnParameterValueChangedInternal(TSharedRef<FNiagaraScriptParameterViewModel> ChangedParameter)
{
	// Since we potentially have multiple input nodes that point to the exact same underlying input variable, we need to make sure that 
	// all of them are synchronized in their values or else we might end up with confusion when we finally compile. 
	if (ChangedParameter->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::Struct || 
		ChangedParameter->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::DataInterface ||
		ChangedParameter->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::ObjectAsset)
	{
		TArray<UNiagaraNodeInput*> InputNodes;
		TArray<UNiagaraNodeInput*> InputNodesToUpdate;

		if (Graph.IsValid())
		{
			Graph->GetNodesOfClass(InputNodes);
		}

		for (UNiagaraNodeInput* InputNode : InputNodes)
		{
			// Copy the value to all nodes with the matching id to cover both the case where we're updating a graph variable
			// and the case where we're updating a compiled variable.
			if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter && InputNode->Input.GetName() == ChangedParameter->GetName())
			{
				if (ensureMsgf(InputNode->Input.GetType() == ChangedParameter->GetType(), TEXT("Can not propagate variable values when the types don't match.")))
				{
					InputNodesToUpdate.Add(InputNode);
				}
			}
		}

		if (ChangedParameter->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::Struct)
		{
			for (UNiagaraNodeInput* InputNodeToUpdate : InputNodesToUpdate)
			{
				InputNodeToUpdate->Modify();
				InputNodeToUpdate->Input.AllocateData();
				InputNodeToUpdate->Input.SetData(ChangedParameter->GetDefaultValueStruct()->GetStructMemory());
			}
			
			// Synchronize script variables...
			for (FVersionedNiagaraScriptWeakPtr& VersionedScript : Scripts)
			{
				if (!VersionedScript.Script.IsValid() || !VersionedScript.Script->GetVMExecutableData().IsValid())
				{
					continue;
				}

				for (FNiagaraVariable& EmitterVariableToCheck : VersionedScript.Script->GetVMExecutableData().Parameters.Parameters)
				{
					if (EmitterVariableToCheck.GetName() == ChangedParameter->GetName())
					{
						EmitterVariableToCheck = InputNodesToUpdate[0]->Input;
						break;
					}
				}
			}
		}
		else if (ChangedParameter->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::DataInterface)
		{
			UNiagaraDataInterface* DataInterface = ChangedParameter->GetDefaultValueDataInterface();
			if (DataInterface != nullptr)
			{
				TSet<UNiagaraDataInterface*> DataInterfacesToUpdate;
				for (UNiagaraNodeInput* InputNodeToUpdate : InputNodesToUpdate)
				{
					DataInterfacesToUpdate.Add(InputNodeToUpdate->GetDataInterface());
				}

				for (UNiagaraDataInterface* DataInterfaceToUpdate : DataInterfacesToUpdate)
				{
					if (DataInterfaceToUpdate->Equals(DataInterface) == false)
					{
						DataInterfaceToUpdate->Modify();
						DataInterface->CopyTo(DataInterfaceToUpdate);
					}
				}

				// Synchronize script variables...
				for (FVersionedNiagaraScriptWeakPtr& VersionedScript : Scripts)
				{
					if (!VersionedScript.Script.IsValid() || !VersionedScript.Script->GetVMExecutableData().IsValid())
					{
						continue;
					}
					for (FNiagaraScriptDataInterfaceInfo& Info : VersionedScript.Script->GetCachedDefaultDataInterfaces())
					{
						if (Info.Name == ChangedParameter->GetName() && DataInterface != Info.DataInterface)
						{
							DataInterface->CopyTo(Info.DataInterface);
							break;
						}
					}
				}
			}
		}
		else if (ChangedParameter->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::ObjectAsset)
		{
			TObjectPtr<UObject> NewObject = ChangedParameter->GetDefaultValueObjectAsset();

			for (UNiagaraNodeInput* InputNodeToUpdate : InputNodesToUpdate)
			{
				InputNodeToUpdate->Modify();
				InputNodeToUpdate->SetObjectAsset(NewObject);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	OnParameterValueChanged().Broadcast({ ChangedParameter->GetName() });
}

#undef LOCTEXT_NAMESPACE // "NiagaraScriptInputCollection"
