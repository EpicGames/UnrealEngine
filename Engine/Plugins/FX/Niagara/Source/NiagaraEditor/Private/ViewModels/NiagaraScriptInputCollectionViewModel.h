// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraParameterCollectionViewModel.h"
#include "NiagaraParameterViewModel.h"

class UNiagaraScript;
class UNiagaraGraph;
class FNiagaraScriptParameterViewModel;
struct FNiagaraTypeDefinition;
class UNiagaraEmitter;

/** A parameter collection view model for script input parameters. */
class FNiagaraScriptInputCollectionViewModel : public FNiagaraParameterCollectionViewModel, public TSharedFromThis<FNiagaraScriptInputCollectionViewModel>
{
public:
	FNiagaraScriptInputCollectionViewModel(TAttribute<FText> InDisplayName, ENiagaraParameterEditMode InParameterEditMode);

	~FNiagaraScriptInputCollectionViewModel();

	/** Sets the view model to a new script. */
	void SetScripts(TArray<FVersionedNiagaraScript>& InScripts);

	//~ INiagaraParameterCollectionViewModel interface
	virtual FText GetDisplayName() const override;
	virtual void AddParameter(TSharedPtr<FNiagaraTypeDefinition> ParameterType) override;
	virtual void DeleteSelectedParameters() override;
	virtual void DeleteParameters(TArray<FNiagaraVariable> ParametersToDelete) override;
	virtual const TArray<TSharedRef<INiagaraParameterViewModel>>& GetParameters() override;

	/** Rebuilds the parameter view models. */
	virtual void RefreshParameterViewModels() override;

	/** Gets the parameter view model associated with a given Id.*/
	TSharedPtr<INiagaraParameterViewModel> GetParameterViewModel(const FName& Name);
	
	/** Sets all parameter view models editable state to the input value.*/
	void SetAllParametersEditingEnabled(bool bInEnabled);

	/** Sets the tooltip overrides on all parameters.*/
	void SetAllParametersTooltipOverrides(const FText& Override);

protected:
	//~ FNiagaraParameterCollectionViewModel interface.
	virtual bool SupportsType(const FNiagaraTypeDefinition& Type) const override;

private:
	/** Handles when the graph for the script changes. */
	void OnGraphChanged(const struct FEdGraphEditAction& InAction);

	/** Handles when the name on a parameter changes. */
	void OnParameterNameChanged(FName OldName, FName NewName, TWeakObjectPtr<UNiagaraNodeInput> InputNodeWeak);

	/** Handles when the type on a parameter changes. */
	void OnParameterTypeChanged(FNiagaraVariable* ParameterVariable);

	/** Handles when the default value on a parameter changes. */
	void OnParameterValueChangedInternal(TSharedRef<FNiagaraScriptParameterViewModel> ChangedParameter);

private:
	/** The parameter view models. */
	TArray<TSharedRef<INiagaraParameterViewModel>> ParameterViewModels;

	/** The script which provides the input parameters viewed and edited by this view model. */
	TArray<FVersionedNiagaraScriptWeakPtr> Scripts;

	/** The graph which owns the non-compiled input parameters viewed and edited by this view model. */
	TWeakObjectPtr<UNiagaraGraph> Graph;

	/** The display name for the view model. */
	TAttribute<FText> DisplayName;

	/** The handle to the graph changed delegate. */
	FDelegateHandle OnGraphChangedHandle;
	FDelegateHandle OnRecompileHandle;

	/** Whether or not generic numeric type parameters are supported as inputs and outputs for this script. */
	bool bCanHaveNumericParameters;
};