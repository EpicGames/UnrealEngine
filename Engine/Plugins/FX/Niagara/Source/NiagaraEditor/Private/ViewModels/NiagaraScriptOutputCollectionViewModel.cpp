// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptOutputCollectionViewModel.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraTypes.h"
#include "NiagaraScriptParameterViewModel.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraEmitter.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptOutputCollection"

FNiagaraScriptOutputCollectionViewModel::FNiagaraScriptOutputCollectionViewModel(ENiagaraParameterEditMode InParameterEditMode)
	: FNiagaraParameterCollectionViewModel(InParameterEditMode)
	, DisplayName(LOCTEXT("DisplayName", "Outputs"))
	, bCanHaveNumericParameters(true)
{
}

FNiagaraScriptOutputCollectionViewModel::~FNiagaraScriptOutputCollectionViewModel()
{
	for (int32 i = 0; i < ParameterViewModels.Num(); i++)
	{
		FNiagaraScriptParameterViewModel* ParameterViewModel = (FNiagaraScriptParameterViewModel*)(&ParameterViewModels[i].Get());
		if (ParameterViewModel != nullptr)
		{
			ParameterViewModel->Reset();
			ParameterViewModel->OnNameChanged().RemoveAll(this);
			ParameterViewModel->OnTypeChanged().RemoveAll(this);
		}
	}

	if (Graph.IsValid())
	{
		Graph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
		Graph->RemoveOnGraphNeedsRecompileHandler(OnRecompileHandle);
	}
}

void FNiagaraScriptOutputCollectionViewModel::SetScripts(TArray<FVersionedNiagaraScript>& InScripts)
{
	// Remove callback on previously held graph.
	if (Graph.IsValid())
	{
		Graph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
		Graph->RemoveOnGraphNeedsRecompileHandler(OnRecompileHandle);
	}

	Scripts.Empty();
	for (FVersionedNiagaraScript& VersionedScript : InScripts)
	{
		Scripts.Add(VersionedScript.ToWeakPtr());
		check(VersionedScript.Script->GetSource(VersionedScript.Version) == InScripts[0].Script->GetSource(InScripts[0].Version));
	}

	OutputNode = nullptr;
	if (Scripts.Num() > 0 && Scripts[0].Script.IsValid() && Scripts[0].Script->GetSource(Scripts[0].Version))
	{
		Graph = Cast<UNiagaraScriptSource>(Scripts[0].Script->GetSource(Scripts[0].Version))->NodeGraph;
		OnGraphChangedHandle = Graph->AddOnGraphChangedHandler(
			FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraScriptOutputCollectionViewModel::OnGraphChanged));
		OnRecompileHandle = Graph->AddOnGraphNeedsRecompileHandler(
			FOnGraphChanged::FDelegate::CreateSP(this, &FNiagaraScriptOutputCollectionViewModel::OnGraphChanged));
		bCanHaveNumericParameters = Scripts[0].Script->IsStandaloneScript();

		if (Scripts.Num() == 1)
		{
			TArray<UNiagaraNodeOutput*> OutputNodes;
			Graph->GetNodesOfClass(OutputNodes);
			if (OutputNodes.Num() == 1)
			{
				OutputNode = OutputNodes[0];
			}
		}
		else
		{
			OutputNode = Graph->FindOutputNode(ENiagaraScriptUsage::ParticleSpawnScript);
		}
	}
	else
	{
		Graph = nullptr;
		OutputNode = nullptr;
		bCanHaveNumericParameters = true;
	}

	RefreshParameterViewModels();
}

FText FNiagaraScriptOutputCollectionViewModel::GetDisplayName() const 
{
	return DisplayName;
}

void FNiagaraScriptOutputCollectionViewModel::AddParameter(TSharedPtr<FNiagaraTypeDefinition> ParameterType)
{
	if (Graph.IsValid() == false)
	{
		return;
	}
	
	TArray<UNiagaraNodeOutput*> OutputNodes;
	Graph->FindOutputNodes(OutputNodes);
	if (OutputNodes.Num() == 0)
	{
		return;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("AddScriptOutput", "Add script output"));
	FName OutputName = FNiagaraUtilities::GetUniqueName(FName(TEXT("NewOutput")), GetParameterNames());
	for (UNiagaraNodeOutput* OtherOutputNode : OutputNodes)
	{
		OtherOutputNode->Modify();
		OtherOutputNode->Outputs.Add(FNiagaraVariable(*ParameterType.Get(), OutputName));

		OtherOutputNode->NotifyOutputVariablesChanged();
	}
	Graph->MarkGraphRequiresSynchronization(TEXT("Ouptut parameter added"));
	OnOutputParametersChangedDelegate.Broadcast();
}

bool FNiagaraScriptOutputCollectionViewModel::CanDeleteParameters() const
{
	return GetSelection().GetSelectedObjects().Num() > 0;
}

void FNiagaraScriptOutputCollectionViewModel::DeleteSelectedParameters()
{
	if (Graph.IsValid() == false)
	{
		return;
	}
	
	if(GetSelection().GetSelectedObjects().Num() > 0)
	{
		TSet<FNiagaraVariable> VariablesToDelete;
		for (TSharedRef<INiagaraParameterViewModel> OutputParameter : GetSelection().GetSelectedObjects())
		{
			VariablesToDelete.Add(OutputParameter->GetVariable());
		}
		GetSelection().ClearSelectedObjects();
		
		DeleteParameters(VariablesToDelete.Array());
	}
}

void FNiagaraScriptOutputCollectionViewModel::DeleteParameters(TArray<FNiagaraVariable> ParametersToDelete)
{
	TArray<UNiagaraNodeOutput*> OutputNodes;
	Graph->FindOutputNodes(OutputNodes);
	if (OutputNodes.Num() == 0)
	{
		return;
	}

	auto DeletePredicate = [&ParametersToDelete] (FNiagaraVariable& OutputVariable)
	{ 
		return ParametersToDelete.Contains(OutputVariable);
	};
	
	FScopedTransaction ScopedTransaction(LOCTEXT("DeletedSelectedNodes", "Delete selected nodes"));
	for (UNiagaraNodeOutput* OtherOutputNode : OutputNodes)
	{
		OtherOutputNode->Modify();
		OtherOutputNode->Outputs.RemoveAll(DeletePredicate);
		OtherOutputNode->NotifyOutputVariablesChanged();
	}
	OnOutputParametersChangedDelegate.Broadcast();
}

const TArray<TSharedRef<INiagaraParameterViewModel>>& FNiagaraScriptOutputCollectionViewModel::GetParameters()
{
	return ParameterViewModels;
}

void FNiagaraScriptOutputCollectionViewModel::RefreshParameterViewModels()
{
	for (int32 i = 0; i < ParameterViewModels.Num(); i++)
	{
		FNiagaraScriptParameterViewModel* ParameterViewModel = (FNiagaraScriptParameterViewModel*)(&ParameterViewModels[i].Get());
		if (ParameterViewModel != nullptr)
		{
			ParameterViewModel->Reset();
			ParameterViewModel->OnNameChanged().RemoveAll(this);
			ParameterViewModel->OnTypeChanged().RemoveAll(this);
		}
	}
	ParameterViewModels.Empty();

	if (OutputNode.IsValid())
	{
		for (FNiagaraVariable& OutputVariable : OutputNode->Outputs)
		{
			TSharedRef<FNiagaraScriptParameterViewModel> ParameterViewModel = MakeShareable(new FNiagaraScriptParameterViewModel(OutputVariable, *OutputNode, nullptr, nullptr, ParameterEditMode));
			ParameterViewModel->OnNameChanged().AddSP(this, &FNiagaraScriptOutputCollectionViewModel::OnParameterNameChanged, &OutputVariable);
			ParameterViewModel->OnTypeChanged().AddSP(this, &FNiagaraScriptOutputCollectionViewModel::OnParameterTypeChanged, &OutputVariable);
			ParameterViewModels.Add(ParameterViewModel);
		}
	}

	OnCollectionChangedDelegate.Broadcast();
}

bool FNiagaraScriptOutputCollectionViewModel::SupportsType(const FNiagaraTypeDefinition& Type) const
{
	if (Type.IsInternalType())
	{
		return false;
	}

	if (Scripts.Num() == 1 && Scripts[0].Script.IsValid())
	{
		// We only support parameter map outputs for modules.
		if (Scripts[0].Script->GetUsage() == ENiagaraScriptUsage::Module)
		{
			if (Type != FNiagaraTypeDefinition::GetParameterMapDef())
			{
				return false;
			}
		}
	}
	return Type.GetScriptStruct() != nullptr && (bCanHaveNumericParameters || Type != FNiagaraTypeDefinition::GetGenericNumericDef());
}

FNiagaraScriptOutputCollectionViewModel::FOnOutputParametersChanged& FNiagaraScriptOutputCollectionViewModel::OnOutputParametersChanged()
{
	return OnOutputParametersChangedDelegate;
}

void FNiagaraScriptOutputCollectionViewModel::OnGraphChanged(const struct FEdGraphEditAction& InAction)
{
	bNeedsRefresh = true;
}

void FNiagaraScriptOutputCollectionViewModel::OnParameterNameChanged(FName OldName, FName NewName, FNiagaraVariable* ParameterVariable)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("EditOutputName", "Edit output name"));

	if (OutputNode.IsValid() == false)
	{
		return;
	}

	ParameterVariable->SetName(NewName);

	// Make sure to edit the pin name so that when we regenerate the node the 
	// links are preserved.
	UEdGraphPin* FoundPin = OutputNode->FindPin(OldName, EEdGraphPinDirection::EGPD_Input);
	if (FoundPin)
	{
		FoundPin->Modify();
		FoundPin->PinName = NewName;
	}

	
	OutputNode->NotifyOutputVariablesChanged();

	// Now sync the others in the graph to this one...
	TArray<UNiagaraNodeOutput*> OutputNodes;
	Graph->GetNodesOfClass(OutputNodes);
	for (UNiagaraNodeOutput* GraphOutputNode : OutputNodes)
	{
		if (GraphOutputNode == OutputNode.Get())
		{
			continue;
		} 
		
		GraphOutputNode->Modify();
		GraphOutputNode->Outputs = OutputNode->Outputs;
		GraphOutputNode->NotifyOutputVariablesChanged();
	}
	OnOutputParametersChangedDelegate.Broadcast();
}

void FNiagaraScriptOutputCollectionViewModel::OnParameterTypeChanged(FNiagaraVariable* ParameterVariable)
{
	if (OutputNode.IsValid() == false)
	{
		return;
	}
	OutputNode->NotifyOutputVariablesChanged();

	// Now sync the others in the graph to this one...
	TArray<UNiagaraNodeOutput*> OutputNodes;
	Graph->GetNodesOfClass(OutputNodes);
	for (UNiagaraNodeOutput* GraphOutputNode : OutputNodes)
	{
		if (GraphOutputNode == OutputNode.Get())
		{
			continue;
		}

		GraphOutputNode->Modify();
		GraphOutputNode->Outputs = OutputNode->Outputs;
		GraphOutputNode->NotifyOutputVariablesChanged();
	}

	OnOutputParametersChangedDelegate.Broadcast();
}

void FNiagaraScriptOutputCollectionViewModel::OnParameterValueChangedInternal(FNiagaraVariable* ParameterVariable)
{
	if (OutputNode.IsValid() == false)
	{
		return;
	}
	OutputNode->NotifyOutputVariablesChanged();
	OnOutputParametersChangedDelegate.Broadcast();

	// Now sync the others in the graph to this one...
	TArray<UNiagaraNodeOutput*> OutputNodes;
	Graph->GetNodesOfClass(OutputNodes);
	for (UNiagaraNodeOutput* GraphOutputNode : OutputNodes)
	{
		if (GraphOutputNode == OutputNode.Get())
		{
			continue;
		}

		GraphOutputNode->Modify();
		GraphOutputNode->Outputs = OutputNode->Outputs;
		GraphOutputNode->NotifyOutputVariablesChanged();
	}

	OnParameterValueChanged().Broadcast({ ParameterVariable->GetName() });
}

#undef LOCTEXT_NAMESPACE // "NiagaraScriptInputCollection"
