// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraParameterCollectionViewModel.h"

class UNiagaraScript;
class UNiagaraGraph;
class UNiagaraNodeOutput;
class UNiagaraEmitter;
struct FNiagaraTypeDefinition;

/** A paramter collection view model for script outputs. */
class FNiagaraScriptOutputCollectionViewModel : public FNiagaraParameterCollectionViewModel, public TSharedFromThis<FNiagaraScriptOutputCollectionViewModel>
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnOutputParametersChanged);

public:
	FNiagaraScriptOutputCollectionViewModel(ENiagaraParameterEditMode InParameterEditMode);

	~FNiagaraScriptOutputCollectionViewModel();

	/** Sets this view model to a new script. */
	void SetScripts(TArray<FVersionedNiagaraScript>& InScript);


	//~ INiagaraParameterCollectionViewModel interface
	virtual FText GetDisplayName() const override;
	virtual void AddParameter(TSharedPtr<FNiagaraTypeDefinition> ParameterType) override;
	virtual bool CanDeleteParameters() const override;
	virtual void DeleteSelectedParameters() override;
	virtual void DeleteParameters(TArray<FNiagaraVariable> ParametersToDelete) override;
	virtual const TArray<TSharedRef<INiagaraParameterViewModel>>& GetParameters() override;

	/** Rebuilds the parameter view models. */
	void RefreshParameterViewModels();

	/** Gets a multicast delegate which is called whenever the output parameter collection is changed, or when
		a paramter in the collection is changed. */
	FOnOutputParametersChanged& OnOutputParametersChanged();

protected:
	//~ FNiagaraParameterCollectionViewModel interface
	virtual bool SupportsType(const FNiagaraTypeDefinition& Type) const override;

private:
	/** Handles the graph changing .*/
	void OnGraphChanged(const struct FEdGraphEditAction& InAction);

	/** Handles the name on a parameter changing .*/
	void OnParameterNameChanged(FName OldName, FName NewName, FNiagaraVariable* ParameterVariable);

	/** Handles the type on a parameter changing .*/
	void OnParameterTypeChanged(FNiagaraVariable* ParameterVariable);

	/** Handles the value of a parameter changing .*/
	void OnParameterValueChangedInternal(FNiagaraVariable* ParameterVariable);

private:
	/** The view models for the output parameters. */
	TArray<TSharedRef<INiagaraParameterViewModel>> ParameterViewModels;

	/** The script which provides the output node which owns the output parameters. */
	TArray<FVersionedNiagaraScriptWeakPtr> Scripts;

	/** The graph which owns the output node which owns the output paramters. */
	TWeakObjectPtr<UNiagaraGraph> Graph;

	/** The output node which owns the output parameters. */
	TWeakObjectPtr<UNiagaraNodeOutput> OutputNode;

	/** The display name for the parameter collection. */
	FText DisplayName;

	/** The handle to the graph changed delegate needed for removing. */
	FDelegateHandle OnGraphChangedHandle;
	FDelegateHandle OnRecompileHandle;

	/** A multicast delegate which is called whenever the output parameter collection is changed, or when
	a paramter in the collection is changed. */
	FOnOutputParametersChanged OnOutputParametersChangedDelegate;

	/** Whether or not generic numeric type parameters are supported as inputs and outputs for this script. */
	bool bCanHaveNumericParameters;
};