// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScriptParameterViewModel.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraTypes.h"
#include "NiagaraNodeInput.h"
#include "NiagaraGraph.h"
#include "ScopedTransaction.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "ScriptParameterViewModel"

FNiagaraScriptParameterViewModel::FNiagaraScriptParameterViewModel(FNiagaraVariable& InGraphVariable, UObject& InGraphVariableOwner, FNiagaraVariable* InCompiledVariable, UObject* InCompiledVariableOwner, ENiagaraParameterEditMode ParameterEditMode)
	: FNiagaraParameterViewModel(ParameterEditMode)
	, GraphVariable(&InGraphVariable)
	, CompiledVariable(InCompiledVariable)
	, CompiledVariableOwner(InCompiledVariableOwner)
{
	GraphVariableOwner = &InGraphVariableOwner;
	bool bUsingCompiledVariable = InCompiledVariable != nullptr;
	checkf(bUsingCompiledVariable == false || InCompiledVariableOwner != nullptr, TEXT("When using a compiled variable, it's owner must not be null"));
	DefaultValueType = INiagaraParameterViewModel::EDefaultValueType::Struct;
	RefreshParameterValue();
	DebugName = GraphVariable->GetName().ToString();
	//UE_LOGF(LogNiagaraEditor, Log, "Create %p Var %ls", this, *DebugName);
}

FNiagaraScriptParameterViewModel::FNiagaraScriptParameterViewModel(FNiagaraVariable& InGraphVariable, UObject& InGraphVariableOwner, UObject* InValueObject, ENiagaraParameterEditMode ParameterEditMode)
	: FNiagaraParameterViewModel(ParameterEditMode)
	, GraphVariable(&InGraphVariable)
{
	GraphVariableOwner = &InGraphVariableOwner;
	DebugName = GraphVariable->GetName().ToString();
	if (InGraphVariable.IsDataInterface())
	{
		ValueDataInterface = Cast<UNiagaraDataInterface>(InValueObject);
		DefaultValueType = INiagaraParameterViewModel::EDefaultValueType::DataInterface;
	}
	else
	{
		ValueObjectAsset = InValueObject;
		DefaultValueType = INiagaraParameterViewModel::EDefaultValueType::ObjectAsset;
	}

	//UE_LOGF(LogNiagaraEditor, Log, "Create %p Var %ls", this, *DebugName);
}

FNiagaraScriptParameterViewModel::~FNiagaraScriptParameterViewModel()
{
	//UE_LOGF(LogNiagaraEditor, Log, "Delete %p Var %ls", this, *DebugName);
}

FNiagaraVariable FNiagaraScriptParameterViewModel::GetVariable() const
{
	check(GraphVariable);
	return *GraphVariable;
}

void FNiagaraScriptParameterViewModel::Reset()
{
	GraphVariableOwner = nullptr;
	GraphVariable = nullptr;
	CompiledVariable = nullptr;
	CompiledVariableOwner = nullptr;
	ValueVariable = nullptr;
	ValueVariableOwner = nullptr;
	ValueDataInterface = nullptr;
	ValueObjectAsset = nullptr;
}


FName FNiagaraScriptParameterViewModel::GetName() const
{
	if (GraphVariable == nullptr)
	{
		return NAME_None;
	}
	return GraphVariable->GetName();
}

FText FNiagaraScriptParameterViewModel::GetTypeDisplayName() const
{
	if (GraphVariable == nullptr)
	{
		return FText();
	}
	return FText::Format(LOCTEXT("TypeTextFormat", "Type: {0}"), GraphVariable->GetType().GetNameText());
}

void FNiagaraScriptParameterViewModel::NameTextCommitted(const FText& Name, ETextCommit::Type CommitInfo)
{
	check(GraphVariable);
	FName NewName = *Name.ToString();
	FName OldName = GraphVariable->GetName();
	if (!OldName.IsEqual(NewName, ENameCase::CaseSensitive))
	{
		OnNameChangedDelegate.Broadcast(OldName, NewName);
	}
}

bool FNiagaraScriptParameterViewModel::VerifyNodeNameTextChanged(const FText& NewText, FText& OutErrorMessage)
{
	UNiagaraNode* OwningNode = Cast<UNiagaraNode>(GraphVariableOwner);
	return OwningNode == nullptr || FNiagaraEditorUtilities::VerifyNameChangeForInputOrOutputNode(*OwningNode, GraphVariable->GetName(), NewText.ToString(), OutErrorMessage);
}

FNiagaraTypeDefinition FNiagaraScriptParameterViewModel::GetType() const
{
	check(GraphVariable);
	return GraphVariable->GetType();
}

bool FNiagaraScriptParameterViewModel::CanChangeSortOrder() const
{
	return Cast<UNiagaraNodeInput>(GraphVariableOwner.Get()) != nullptr && FNiagaraParameterViewModel::CanChangeSortOrder();
}

int32 FNiagaraScriptParameterViewModel::GetSortOrder() const
{
	UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(GraphVariableOwner.Get());
	if (InputNode != nullptr)
	{
		return InputNode->CallSortPriority;
	}
	else
	{
		return 0;
	}
}

void FNiagaraScriptParameterViewModel::SetSortOrder(int32 SortOrder)
{
	UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(GraphVariableOwner.Get());
	if (InputNode != nullptr && SortOrder != InputNode->CallSortPriority)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("EditInputSortPriority", "Edit input sort priority"));
		InputNode->Modify();
		InputNode->CallSortPriority = SortOrder;
	}
}

void FNiagaraScriptParameterViewModel::SelectedTypeChanged(TSharedPtr<FNiagaraTypeDefinition> Item, ESelectInfo::Type SelectionType)
{
	if (Item.IsValid() && GraphVariable && GraphVariable->GetType() != *Item)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("EditInputType", "Edit input type"));
		GraphVariableOwner->Modify();
		GraphVariable->SetType(*Item);
		FNiagaraEditorUtilities::ResetVariableToDefaultValue(*GraphVariable);
		OnTypeChangedDelegate.Broadcast();
	}
}

INiagaraParameterViewModel::EDefaultValueType FNiagaraScriptParameterViewModel::GetDefaultValueType()
{
	return DefaultValueType;
}

TSharedRef<FStructOnScope> FNiagaraScriptParameterViewModel::GetDefaultValueStruct()
{
	return ParameterValue.ToSharedRef();
}

UNiagaraDataInterface* FNiagaraScriptParameterViewModel::GetDefaultValueDataInterface()
{
	return ValueDataInterface;
}

TObjectPtr<UObject> FNiagaraScriptParameterViewModel::GetDefaultValueObjectAsset()
{
	return ValueObjectAsset;
}

void FNiagaraScriptParameterViewModel::SetDefaultValueObjectAsset(TObjectPtr<UObject> Object)
{
	ValueObjectAsset = Object;
	OnDefaultValueChangedDelegate.Broadcast();
}

void FNiagaraScriptParameterViewModel::NotifyDefaultValuePropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (DefaultValueType == INiagaraParameterViewModel::EDefaultValueType::Struct)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("EditParameterValueProperty", "Edit parameter value"));
		ValueVariableOwner->Modify();
		ValueVariable->SetData(ParameterValue->GetStructMemory());
	}
	OnDefaultValueChangedDelegate.Broadcast();
}

void FNiagaraScriptParameterViewModel::NotifyBeginDefaultValueChange()
{
	GEditor->BeginTransaction(LOCTEXT("BeginEditParameterValue", "Edit parameter value"));
	ValueVariableOwner->Modify();
}

void FNiagaraScriptParameterViewModel::NotifyEndDefaultValueChange()
{
	if (GEditor->IsTransactionActive())
	{
		GEditor->EndTransaction();
	}
}

void FNiagaraScriptParameterViewModel::NotifyDefaultValueChanged()
{
	if (ValueVariable != nullptr && ParameterValue.IsValid() && !FNiagaraEditorUtilities::DataMatches(*ValueVariable, *ParameterValue.Get()))
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("EditParameterValue", "Edit parameter value"));
		ValueVariableOwner->Modify();
		ValueVariable->SetData(ParameterValue->GetStructMemory());
	}
	OnDefaultValueChangedDelegate.Broadcast();
}

FNiagaraScriptParameterViewModel::FOnNameChanged& FNiagaraScriptParameterViewModel::OnNameChanged()
{
	return OnNameChangedDelegate;
}

void FNiagaraScriptParameterViewModel::RefreshParameterValue()
{
	check(GraphVariable);
	if (CompiledVariable != nullptr && CompiledVariable->GetType().GetStruct() == GraphVariable->GetType().GetStruct())
	{
		// If the compiled variable is valid and the same type as the graph variable we will update that value to allow for changes to be
		// seen in the simulation without compiling.
		ValueVariable = CompiledVariable;
		ValueVariableOwner = CompiledVariableOwner;
	}
	else
	{
		// If the compiled variable is null or the type of the graph variable has been changed,
		// then we edit the value in the graph variable so that the value is of the correct type and can be edited.
		ValueVariable = GraphVariable;
		ValueVariableOwner = GraphVariableOwner.Get();
	}
	ParameterValue = MakeShareable(new FStructOnScope(GraphVariable->GetType().GetStruct()));
	ValueVariable->AllocateData();
	ValueVariable->CopyTo(ParameterValue->GetStructMemory());
	OnDefaultValueChangedDelegate.Broadcast();
}

#undef LOCTEXT_NAMESPACE // "ScriptParameterViewModel"