// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraParameterViewModel.h"

class UNiagaraNodeInput;
struct FNiagaraVariable;

/** A view model for a script parameter. */
class FNiagaraScriptParameterViewModel : public FNiagaraParameterViewModel
{
public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNameChanged, FName /*Old*/, FName/*New*/);

public:
	/*
	 * Create a new script parameter view model.
	 * @param InGraphVariable The variable which is owned by the graph which provides the data for this parameter.
	 * @param InGraphVariableOwner The UObject that owns the graph variable, for property undo transactions.
	 * @param InCompiledVariable An optional compiled version of the variable.  When this version is valid changes to
	 *        parameter value will be made to the compiled variable instead of the graph variable.
	 * @param InCompiledVariableOwner The UObject that owns the compiled variable.  This object must be provided if the compiled variable is provided.
	 */
	FNiagaraScriptParameterViewModel(FNiagaraVariable& InGraphVariable, UObject& InGraphVariableOwner, FNiagaraVariable* InCompiledVariable, UObject* InCompiledVariableOwner, ENiagaraParameterEditMode ParameterEditMode);

	/*
	* Create a new script parameter view model.
	* @param InGraphVariable The variable which is owned by the graph which provides the data for this parameter.
	* @param InGraphVariableOwner The UObject that owns the graph variable, for property undo transactions.
	* @param InValueObject The UObject which is providing the value for this parameter.
	*/
	FNiagaraScriptParameterViewModel(FNiagaraVariable& InGraphVariable, UObject& InGraphVariableOwner, UObject* InValueObject, ENiagaraParameterEditMode ParameterEditMode);

	virtual ~FNiagaraScriptParameterViewModel();

	//~ INiagaraParameterViewmodel interface.
	virtual FNiagaraVariable GetVariable() const override;
	virtual FName GetName() const override;
	virtual void NameTextCommitted(const FText& Name, ETextCommit::Type CommitInfo) override;
	virtual bool VerifyNodeNameTextChanged(const FText& NewText, FText& OutErrorMessage) override;
	virtual FText GetTypeDisplayName() const override;
	virtual FNiagaraTypeDefinition GetType() const override;
	virtual void SelectedTypeChanged(TSharedPtr<FNiagaraTypeDefinition> Item, ESelectInfo::Type SelectionType) override;
	virtual EDefaultValueType GetDefaultValueType() override;
	virtual TSharedRef<FStructOnScope> GetDefaultValueStruct() override;
	virtual UNiagaraDataInterface* GetDefaultValueDataInterface() override;
	virtual TObjectPtr<UObject> GetDefaultValueObjectAsset() override;
	virtual void SetDefaultValueObjectAsset(TObjectPtr<UObject>) override;
	virtual void NotifyDefaultValuePropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void NotifyBeginDefaultValueChange() override;
	virtual void NotifyEndDefaultValueChange() override;
	virtual void NotifyDefaultValueChanged() override;
	virtual bool CanChangeSortOrder() const override;
	virtual int32 GetSortOrder() const override;
	virtual void SetSortOrder(int32 SortOrder) override;

	/** Gets a multicast delegate which is called whenever the name of this parameter changes. */
	FOnNameChanged& OnNameChanged();

	void Reset();

private:
	/** Refreshes the parameter value struct from the variable data. */
	void RefreshParameterValue();

private:
	/** The graph variable which is being displayed and edited by this view model. */
	FNiagaraVariable* GraphVariable = nullptr;

	/** The owning UObject of the graph variable. */
	TWeakObjectPtr<UObject> GraphVariableOwner;

	/** An optional compiled version of the graph variable. */
	FNiagaraVariable* CompiledVariable = nullptr;

	/** The owner of the optional compiled version of the graph variable. */
	UObject* CompiledVariableOwner = nullptr;

	/** The variable currently being used to display and edit the value of the parameter. */
	FNiagaraVariable* ValueVariable = nullptr;

	/** The owner of the value variable. */
	UObject* ValueVariableOwner = nullptr;

	/** The data interface which is providing the parameter value. */
	UNiagaraDataInterface* ValueDataInterface = nullptr;

	/** The object asset which is providing the parameter value. */
	TObjectPtr<UObject> ValueObjectAsset = nullptr;

	/** The type of default value this parameter is providing. */
	EDefaultValueType DefaultValueType;

	/** A struct representing the value of the variable. */
	TSharedPtr<FStructOnScope> ParameterValue;

	/** A multicast delegate which is called whenever the name of the parameter changes. */
	FOnNameChanged OnNameChangedDelegate;

	FString DebugName;
};