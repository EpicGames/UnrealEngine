// Copyright Epic Games, Inc. All Rights Reserved.

//-TODO: Some platforms require an offline compile to get instruction counts, this needs to be handled

#include "NiagaraScriptStatsViewModel.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "NiagaraScript.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"

#include "Async/Async.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "NiagaraShader.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Types/SlateEnums.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraScriptStatsViewModel)

#define LOCTEXT_NAMESPACE "NiagaraScriptStatsViewModel"

namespace NiagaraScriptStatsLocal
{
	static const FName GScriptSourceName(TEXT("Script Source"));
	static FString GCompilingString(TEXT("Compiling..."));

	static void GetVMScriptStatus(TConstArrayView<UNiagaraScript*> NiagaraScripts, bool& bOutIsCompiling, bool& bOutHasError, FString& OutStatus)
	{
		bool bHasError = false;
		for ( UNiagaraScript* Script : NiagaraScripts )
		{
			if ( !Script->IsReadyToRun(ENiagaraSimTarget::CPUSim) )
			{
				bOutIsCompiling = true;
				bOutHasError = false;
				OutStatus = GCompilingString;
				return;
			}
			bHasError |= Script->GetVMExecutableData().LastCompileStatus == ENiagaraScriptCompileStatus::NCS_Error;
		}

		bOutIsCompiling = false;
		bOutHasError = bHasError;

		OutStatus.Reset();
		if (bHasError)
		{
			for (UNiagaraScript* Script : NiagaraScripts)
			{
				const FNiagaraVMExecutableData& VMData = Script->GetVMExecutableData();
				if ( VMData.ErrorMsg.Len() > 0 )
				{
					OutStatus.Appendf(TEXT("%s = %s\n"), *Script->GetName(), *VMData.ErrorMsg);
				}
			}
		}
		else
		{
			for (UNiagaraScript* Script : NiagaraScripts)
			{
				const FNiagaraVMExecutableData& VMData = Script->GetVMExecutableData();
				if (!OutStatus.IsEmpty())
				{
					OutStatus.AppendChar(TEXT('\n'));
				}
				OutStatus.Appendf(TEXT("VM: %s\n"), *Script->GetName());
				if (VMData.LastOpCount > 0)
				{
					OutStatus.Appendf(TEXT("- Op Count = %u"), VMData.LastOpCount);
				}
				else
				{
					OutStatus.Append(TEXT("- Data missing please force a recompile"));
				}
			}
		}
	}

	FColor ScriptStatusToColor(bool bIsCompiling, bool bHasError)
	{
		if (bIsCompiling)
		{
			return FColor::Yellow;
		}
		return bHasError ? FColor::Orange : FColor::Green;
	}

	bool ScriptStatusToWrapText(bool bIsCompiling, bool bHasError)
	{
		return bHasError;
	}

	// Deletion that handles that we may have in flight render commands but need to delete on the GT
	struct FNiagaraScriptSafeDelete
	{
		void operator()(FNiagaraShaderScript* Ptr) const
		{
			Ptr->CancelCompilation();
			FNiagaraShaderMap::RemovePendingScript(Ptr);

			ENQUEUE_RENDER_COMMAND(ScriptSafeDelete)(
				[RT_Ptr=Ptr](FRHICommandListImmediate& RHICmdList)
				{
					AsyncTask(ENamedThreads::GameThread, [GT_Ptr=RT_Ptr]() { delete GT_Ptr; });
				}
			);
		}
	};

	struct FNiagaraScriptStats
	{
		FNiagaraScriptStats(const FGuid& InGuid, const FNiagaraEmitterHandle& EmitterHandle, EShaderPlatform ShaderPlatform)
		{
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();

			EmitterId = InGuid;
			bIsGPU = EmitterData && EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim;
		}

		void UpdateStatus(EShaderPlatform ShaderPlatform, const FNiagaraEmitterHandle& EmitterHandle)
		{
			if ( bIsCompiling == false )
			{
				return;
			}

			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
			if (EmitterHandle.GetEmitterMode() != ENiagaraEmitterMode::Standard || EmitterData == nullptr)
			{
				ResultsString = TEXT("N/A");
				bIsCompiling = false;
				return;
			}

			if ( bIsGPU )
			{
				UpdateStatusGPU(ShaderPlatform, EmitterData);
			}
			else
			{
				TArray<UNiagaraScript*> NiagaraScripts;
				EmitterData->GetScripts(NiagaraScripts);
				GetVMScriptStatus(MakeArrayView(NiagaraScripts), bIsCompiling, bHasError, ResultsString);
			}
		}

		void UpdateStatusGPU(EShaderPlatform ShaderPlatform, FVersionedNiagaraEmitterData* EmitterData)
		{
			bHasError = false;

			// Can we kick off our cache yet?
			if ( ShaderScripts.Num() == 0 )
			{
				bool bScriptsValid = true;
				EmitterData->ForEachScript(
					[&](UNiagaraScript* NiagaraScript)
					{
						if ( !NiagaraScript->GetVMExecutableDataCompilationId().CompilerVersionID.IsValid() || !NiagaraScript->GetVMExecutableDataCompilationId().BaseScriptCompileHash.IsValid() )
						{
							bScriptsValid = false;
						}
					}
				);

				if (!bScriptsValid)
				{
					return;
				}

				EmitterData->ForEachScript(
					[&](UNiagaraScript* NiagaraScript)
					{
						TArray<TUniquePtr<FNiagaraShaderScript>> NewResources;
						NiagaraScript->CacheResourceShadersForCooking(ShaderPlatform, NewResources);
						if (NewResources.Num() > 0)
						{
							ensure(NewResources.Num() == 1);
							FNiagaraShaderScript* ShaderScript = NewResources[0].Release();
							ShaderScripts.Emplace(ShaderScript);
						}
					}
				);
				return;
			}

			// Scripts are compiling
			for ( const auto& ShaderScript : ShaderScripts)
			{
				if ( !ShaderScript->IsCompilationFinished() )
				{
					return;
				}
				bHasError |= ShaderScript->GetCompileErrors().Num() > 0;
			}

			// We have finished compiling, build results information
			bIsCompiling = false;

			ResultsString.Reset();
			if ( bHasError )
			{
				for (const auto& ShaderScript : ShaderScripts)
				{
					for ( const FString& ErrorString :  ShaderScript->GetCompileErrors() )
					{
						ResultsString.Append(ErrorString);
						ResultsString.AppendChar(TEXT('\n'));
					}
				}
			}
			else
			{
				for (const auto& ShaderScript : ShaderScripts)
				{
					TConstArrayView<FSimulationStageMetaData> SimulationStageMetaData = ShaderScript->GetBaseVMScript()->GetSimulationStageMetaData();
					for (int32 iSimStageIndex=0; iSimStageIndex < SimulationStageMetaData.Num(); ++iSimStageIndex)
					{
						FNiagaraShaderRef Shader = ShaderScript->GetShaderGameThread(iSimStageIndex);
						if ( Shader.IsValid() )
						{
							if (!ResultsString.IsEmpty())
							{
								ResultsString.AppendChar(TEXT('\n'));
							}
							ResultsString.Appendf(TEXT("GPU: %s"), *SimulationStageMetaData[iSimStageIndex].SimulationStageName.ToString());
							if (Shader->GetNumInstructions() > 0)
							{
								ResultsString.Appendf(TEXT("\n- Num Instructions = %d"), Shader->GetNumInstructions());
							}
#if WITH_EDITORONLY_DATA
							TStringBuilder<2048> ShaderStatsBuilder;
							const TArray<FGenericShaderStat> ShaderStats = ShaderScript->GetGameThreadShaderMap()->GetShaderStatistics(Shader->GetType(ShaderScript->GetGameThreadShaderMap()->GetPointerTable()));
							for (const auto& ShaderStat : ShaderStats)
							{
								ShaderStatsBuilder << TEXT("\n- ") << ShaderStat.StatName << TEXT(" = ");
								Visit([&ShaderStatsBuilder](auto& StoredValue) { ShaderStatsBuilder << StoredValue; }, ShaderStat.Value);
							}
							ResultsString.Append(ShaderStatsBuilder.ToString());
#endif // WITH_EDITORONLY_DATA
						}
					}
				}
			}
		}

		void CancelCompilation()
		{
			for (auto& Script : ShaderScripts)
			{
				Script->CancelCompilation();
			}
		}

		const FGuid& GetEmitterId() const { return EmitterId; }

		FColor GetCellColor() const { return ScriptStatusToColor(bIsCompiling, bHasError); }

		FString GetCellString() const { return bIsCompiling ? GCompilingString : ResultsString; }

		bool GetCellWrapText() const { return ScriptStatusToWrapText(bIsCompiling, bHasError); }

	private:
		FGuid EmitterId;
		bool bIsCompiling = true;
		bool bIsGPU = false;
		bool bHasError = false;
		FString ResultsString;
		TArray<TUniquePtr<FNiagaraShaderScript, FNiagaraScriptSafeDelete>> ShaderScripts;
	};
}

/////////////////////////////////////////////////////////////////////////////////////

UNiagaraScripStatsViewModelSettings::UNiagaraScripStatsViewModelSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EnabledPlatforms.Add(static_cast<int32>(GMaxRHIShaderPlatform));
}

/////////////////////////////////////////////////////////////////////////////////////

class SNiagaraScriptStatsViewRow : public SMultiColumnTableRow<TSharedPtr<FGuid>>
{
public:
	SLATE_BEGIN_ARGS(SNiagaraScriptStatsViewRow) {}
		SLATE_ARGUMENT(TSharedPtr<FGuid>, PtrRowID)
		SLATE_ARGUMENT(TWeakPtr<FNiagaraScriptStatsViewModel>, WeakViewModel)
	SLATE_END_ARGS()

	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		PtrRowID = InArgs._PtrRowID;
		WeakViewModel = InArgs._WeakViewModel;

		SMultiColumnTableRow<TSharedPtr<FGuid>>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			InOwnerTableView
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		const EHorizontalAlignment HAlign = EHorizontalAlignment::HAlign_Left;
		const EVerticalAlignment VALign = EVerticalAlignment::VAlign_Center;

		return SNew(SBox)
			.Padding(FMargin(4, 2, 4, 2))
			.HAlign(HAlign)
			.VAlign(VALign)
			[
				SNew(STextBlock)
				.ColorAndOpacity(this, &SNiagaraScriptStatsViewRow::GetCellColor, ColumnName)
				.Text(this, &SNiagaraScriptStatsViewRow::GetCellText, ColumnName, false)
				.AutoWrapText(this, &SNiagaraScriptStatsViewRow::GetCellWrapText, ColumnName)
				.ShadowOffset(FVector2D::UnitVector)
				.ShadowColorAndOpacity(FLinearColor::Black)
			];
	}

	FText GetCellText(const FName ColumnName, const bool bToolTip) const
	{
		auto ViewModel = WeakViewModel.Pin();
		if ( ViewModel.IsValid() && PtrRowID.IsValid() )
		{
			auto CellDetails = ViewModel->GetCellDetails(ColumnName, *PtrRowID);
			return CellDetails.CellText;
		}
		return FText();
	}

	FSlateColor GetCellColor(const FName ColumnName) const
	{
		auto ViewModel = WeakViewModel.Pin();
		if (ViewModel.IsValid() && PtrRowID.IsValid())
		{
			auto CellDetails = ViewModel->GetCellDetails(ColumnName, *PtrRowID);
			return CellDetails.CellColor;
		}

		return FLinearColor::White;
	}

	bool GetCellWrapText(const FName ColumnName) const
	{
		auto ViewModel = WeakViewModel.Pin();
		if (ViewModel.IsValid() && PtrRowID.IsValid())
		{
			auto CellDetails = ViewModel->GetCellDetails(ColumnName, *PtrRowID);
			return CellDetails.CellWrapText;
		}

		return false;
	}

private:
	TSharedPtr<FGuid>						PtrRowID;
	TWeakPtr<FNiagaraScriptStatsViewModel>	WeakViewModel;
};

/////////////////////////////////////////////////////////////////////////////////////

class SNiagaraScriptStatsListView : public SListView<TSharedPtr<FGuid>>
{
public:
	SLATE_BEGIN_ARGS(SNiagaraScriptStatsListView) {}
		SLATE_ARGUMENT(TWeakPtr<FNiagaraScriptStatsViewModel>, WeakViewModel)
		SLATE_ARGUMENT(TSharedPtr<SHeaderRow>, HeaderRow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		WeakViewModel = InArgs._WeakViewModel;

		auto ViewModel = WeakViewModel.Pin();
		check(ViewModel != nullptr);

		SListView::Construct(SListView::FArguments()
			.ListItemsSource(ViewModel->GetGridRowIDs())
			.OnGenerateRow(this, &SNiagaraScriptStatsListView::MakeRowWidget)
			.Visibility(EVisibility::Visible)
			.HeaderRow(InArgs._HeaderRow)
		);
	}

	TSharedRef<ITableRow> MakeRowWidget(const TSharedPtr<FGuid> PtrRowID, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		return SNew(SNiagaraScriptStatsViewRow, OwnerTable)
			.PtrRowID(PtrRowID)
			.WeakViewModel(WeakViewModel);
	}

private:
	TWeakPtr<FNiagaraScriptStatsViewModel>	WeakViewModel;
};

/////////////////////////////////////////////////////////////////////////////////////

class SNiagaraScriptStatsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraScriptStatsWidget) {}
		SLATE_ARGUMENT(TWeakPtr<FNiagaraScriptStatsViewModel>, WeakViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		WeakViewModel = InArgs._WeakViewModel;
		PlatformColumnHeader = SNew(SHeaderRow);

		auto ViewModel = WeakViewModel.Pin();
		if (!ViewModel.IsValid())
		{
			return;
		}

		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SComboButton)
						.ComboButtonStyle(FAppStyle::Get(), "ToolbarComboButton")
						.ForegroundColor(FLinearColor::White)
						.ContentPadding(0)
						.OnGetMenuContent(this, &SNiagaraScriptStatsWidget::MakePlatformsWidget)
						.ButtonContent()
						[
							SNew(SHorizontalBox)
							// Icon
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("LevelEditor.Tabs.StatsViewer"))
							]
							// Text
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0, 0, 2, 0)
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
								.Text(LOCTEXT("PlatformsButton", "Platforms"))
							]
						]
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.OnClicked(this, &SNiagaraScriptStatsWidget::OnForceRecompile)
						.Text(LOCTEXT("Recompile", "Recompile"))
						.ToolTipText(LOCTEXT("RecompileTooltip", "Forces a recompile to run"))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.OnClicked(this, &SNiagaraScriptStatsWidget::OnCopyToClipboard)
						.Text(LOCTEXT("CopyToCSV", "Copy To CSV"))
						.ToolTipText(LOCTEXT("CopyToCSVTooltip", "Copies the data to the clipboard, formatted as CSV"))
					]
				]
			]
			// this will contain the stats grid
			+ SVerticalBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(SScrollBox)
						.Orientation(Orient_Vertical)
						+ SScrollBox::Slot()
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.VAlign(VAlign_Fill)
							.AutoHeight()
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Fill)
								.FillWidth(1.f)
								.Padding(5, 0)
								[
									SNew(SScrollBox)
									.Orientation(Orient_Horizontal)
									+ SScrollBox::Slot()
									[
										SAssignNew(StatsInfoList, SNiagaraScriptStatsListView)
										.WeakViewModel(WeakViewModel)
										.HeaderRow(PlatformColumnHeader)
									]
								]
							]
						]
					]
				]
			]
		];
	}

	TSharedRef<SWidget> MakePlatformsWidget()
	{
		FMenuBuilder Builder(false, nullptr);

		TSharedPtr<FNiagaraScriptStatsViewModel> ViewModel = WeakViewModel.Pin();
		if (ViewModel.IsValid())
		{
			for (const auto& Details : ViewModel->GetShaderPlatformDetails())
			{
				auto SPCheckState = [WeakViewModel=WeakViewModel, Details=Details]()
				{
					auto ViewModel = WeakViewModel.Pin();
					if (ViewModel.IsValid())
					{
						return ViewModel->IsEnabled(Details.ShaderPlatform) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}

					return ECheckBoxState::Unchecked;
				};

				auto SPCheckStateChanged = [WeakViewModel=WeakViewModel, Details=Details, Owner=this](const ECheckBoxState NewState)
				{
					auto ViewModel = WeakViewModel.Pin();
					if (ViewModel.IsValid())
					{
						const bool bEnabled = NewState == ECheckBoxState::Checked;
						ViewModel->SetEnabled(Details.ShaderPlatform, bEnabled);
						UNiagaraScripStatsViewModelSettings* Settings = GetMutableDefault<UNiagaraScripStatsViewModelSettings>();
						if (bEnabled)
						{
							Owner->AddColumn(Details.DisplayName);
							Settings->EnabledPlatforms.AddUnique(Details.ShaderPlatform);
						}
						else
						{
							Owner->RemoveColumn(Details.DisplayName);
							Settings->EnabledPlatforms.Remove(Details.ShaderPlatform);
						}
						Settings->SaveConfig();
					}

					Owner->RequestRefresh();
				};

				auto PlatformWidget = SNew(SCheckBox)
					.OnCheckStateChanged_Lambda(SPCheckStateChanged)
					.IsChecked_Lambda(SPCheckState)
					.Content()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "RichTextBlock.Bold")
						.Text(FText::FromName(Details.DisplayName))
						.Margin(FMargin(2.0f, 2.0f, 4.0f, 2.0f))
					];
				Builder.AddMenuEntry(FUIAction(), PlatformWidget);
			}
		}

		return Builder.MakeWidget();
	}

	TSharedRef<ITableRow> MakeRowWidget(const TSharedPtr<FGuid> PtrRowID, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		return SNew(SNiagaraScriptStatsViewRow, OwnerTable)
			.PtrRowID(PtrRowID)
			.WeakViewModel(WeakViewModel);
	}

	void AddColumn(const FName& ColumnName)
	{
		FLinearColor Color = FLinearColor::Gray;

		FText ColumnText = FText::FromName(ColumnName);
		const auto ColumnArgs = SHeaderRow::Column(ColumnName)
			.DefaultLabel(ColumnText)
			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
			.ManualWidth(this, &SNiagaraScriptStatsWidget::GetColumnSize, ColumnName)
			.HeaderContent()
			[
				SNew(STextBlock)
				.ColorAndOpacity(Color)
				.Text(ColumnText)
			];

		PlatformColumnHeader->AddColumn(ColumnArgs);
	}

	void RemoveColumn(const FName& ColumnName)
	{
		PlatformColumnHeader->RemoveColumn(ColumnName);
	}

	void RequestRefresh()
	{
		StatsInfoList->RequestListRefresh();
	}

	float GetColumnSize(const FName ColumnName) const
	{
		if (ColumnName == NiagaraScriptStatsLocal::GScriptSourceName)
		{
			return 300.0f;
		}
		return 200.0f;
	}
	
	FReply OnForceRecompile()
	{
		auto ViewModel = WeakViewModel.Pin();
		if (ViewModel.IsValid())
		{
			ViewModel->OnForceRecompile();
		}
		return FReply::Handled();
	}

	FReply OnCopyToClipboard() const
	{
		auto ViewModel = WeakViewModel.Pin();
		if (ViewModel.IsValid())
		{
			FString ClipboardString;

			for (const TSharedPtr<FGuid>& RowID : *ViewModel->GetGridRowIDs())
			{
				for (const SHeaderRow::FColumn& Column : PlatformColumnHeader->GetColumns())
				{
					const auto CellDetails = ViewModel->GetCellDetails(Column.ColumnId, *RowID);
					ClipboardString.Append(CellDetails.CellText.ToString());
					ClipboardString.Append("\n");
				}
			}
			FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);
		}
		return FReply::Handled();
	}

private:
	TSharedPtr<SHeaderRow> PlatformColumnHeader;
	TWeakPtr<FNiagaraScriptStatsViewModel> WeakViewModel;
	TSharedPtr<SNiagaraScriptStatsListView> StatsInfoList;
};

/////////////////////////////////////////////////////////////////////////////////////

FNiagaraScriptStatsViewModel::FNiagaraScriptStatsViewModel()
{
	BuildShaderPlatformDetails();
}

FNiagaraScriptStatsViewModel::~FNiagaraScriptStatsViewModel()
{
	CancelCompilations();
}

void FNiagaraScriptStatsViewModel::Initialize(TWeakPtr<FNiagaraSystemViewModel> InWeakSystemViewModel)
{
	WeakSystemViewModel = InWeakSystemViewModel;
		
	if (auto SystemViewModel = WeakSystemViewModel.Pin())
	{
		SystemViewModel->OnEmitterHandleViewModelsChanged().AddSP(this, &FNiagaraScriptStatsViewModel::RefreshView);
		SystemViewModel->GetSelectionViewModel()->OnSystemIsSelectedChanged().AddSP(this, &FNiagaraScriptStatsViewModel::RefreshView);
		SystemViewModel->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().AddSP(this, &FNiagaraScriptStatsViewModel::RefreshView);
		SystemViewModel->OnSystemCompiled().AddSP(this, &FNiagaraScriptStatsViewModel::OnSystemCompiled);
	}

	RefreshView();

	Widget = SNew(SNiagaraScriptStatsWidget)
		.WeakViewModel(this->AsShared());

	Widget->AddColumn(NiagaraScriptStatsLocal::GScriptSourceName);

	for (auto Details : ShaderPlatforms )
	{
		if (Details.bEnabled)
		{
			Widget->AddColumn(Details.DisplayName);
		}
	}
}

void FNiagaraScriptStatsViewModel::RefreshView()
{
	RowIDs.Reset();
	if (auto SystemViewModel = WeakSystemViewModel.Pin())
	{
		UNiagaraSystem* NiagaraSystem = &SystemViewModel->GetSystem();

		RowIDs.Emplace(MakeShareable(new FGuid(NiagaraSystem->GetAssetGuid())));

		for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles())
		{
			if (EmitterHandle.IsValid() && EmitterHandle.GetIsEnabled())
			{
				if (UNiagaraEmitter* NiagaraEmitter = EmitterHandle.GetInstance().Emitter)
				{
					RowIDs.Emplace(MakeShareable(new FGuid(EmitterHandle.GetId())));
				}
			}
		}
	}

	if (Widget.IsValid())
	{
		Widget->RequestRefresh();
	}
}

TSharedPtr<class SWidget> FNiagaraScriptStatsViewModel::GetWidget()
{
	return Widget;
}

bool FNiagaraScriptStatsViewModel::IsEnabled(EShaderPlatform SP) const
{
	if (const FShaderPlatformDetails* FoundDetails = ShaderPlatforms.FindByPredicate([&SP](const FShaderPlatformDetails& InDetails) { return InDetails.ShaderPlatform == SP; }))
	{
		return FoundDetails->bEnabled;
	}
	return false;
}

void FNiagaraScriptStatsViewModel::SetEnabled(EShaderPlatform SP, bool bEnabled)
{
	if (FShaderPlatformDetails* FoundDetails = ShaderPlatforms.FindByPredicate([&SP](const FShaderPlatformDetails& InDetails) { return InDetails.ShaderPlatform == SP; }))
	{
		FoundDetails->bEnabled = bEnabled;
	}
}

const TArray<TSharedPtr<FGuid>>* FNiagaraScriptStatsViewModel::GetGridRowIDs() const
{
	return &RowIDs;
}

FNiagaraScriptStatsViewModel::FGridCellDetails FNiagaraScriptStatsViewModel::GetCellDetails(const FName& ColumnName, const FGuid& RowID)
{
	FGridCellDetails Details;
	Details.CellText = FText::FromString(FString("Invalid"));
	Details.CellColor = FColorList::Red;
	Details.CellWrapText = false;

	// Invalid Model?
	auto SystemViewModel = WeakSystemViewModel.Pin();
	if (!SystemViewModel.IsValid())
	{
		return Details;
	}

	// System GUID?
	UNiagaraSystem* NiagaraSystem = &SystemViewModel->GetSystem();
	if ( RowID == NiagaraSystem->GetAssetGuid() )
	{
		if (ColumnName == NiagaraScriptStatsLocal::GScriptSourceName)
		{
			Details.CellText = FText::FromString(FString(TEXT("System")));
			Details.CellColor = FColorList::White;
		}
		else
		{
			TArray<UNiagaraScript*> NiagaraScripts;
			NiagaraScripts.Add(NiagaraSystem->GetSystemSpawnScript());
			NiagaraScripts.Add(NiagaraSystem->GetSystemUpdateScript());

			bool bIsCompiling = false;
			bool bHasError = false;
			FString ResultsString;
			NiagaraScriptStatsLocal::GetVMScriptStatus(MakeArrayView(NiagaraScripts), bIsCompiling, bHasError, ResultsString);

			Details.CellText = FText::FromString(ResultsString);
			Details.CellColor = NiagaraScriptStatsLocal::ScriptStatusToColor(bIsCompiling, bHasError);
			Details.CellWrapText = NiagaraScriptStatsLocal::ScriptStatusToWrapText(bIsCompiling, bHasError);
			return Details;
		}
	}
	
	// Failed to find emitter?
	const int32 EmitterHandleIndex = NiagaraSystem->GetEmitterHandles().IndexOfByPredicate([&RowID](const FNiagaraEmitterHandle& InHandle) { return InHandle.GetId() == RowID; });
	if (EmitterHandleIndex == INDEX_NONE)
	{
		return Details;
	}

	// This column in special as it's the script source name
	if (ColumnName == NiagaraScriptStatsLocal::GScriptSourceName)
	{
		Details.CellText = FText::FromString(NiagaraSystem->GetEmitterHandle(EmitterHandleIndex).GetUniqueInstanceName());
		Details.CellColor = FColorList::White;
		return Details;
	}

	// Need to look up compilation results
	const EShaderPlatform ShaderPlatform = ColumnNameToShaderPlatform(ColumnName);
	if ( ShaderPlatform == SP_NumPlatforms )
	{
		return Details;
	}

	auto& AllEmitterScripts = PerPlatformScripts.FindOrAdd(ShaderPlatform);
	const FNiagaraEmitterHandle& EmitterHandle = NiagaraSystem->GetEmitterHandle(EmitterHandleIndex);

	auto* EmitterScriptsPtr = AllEmitterScripts.FindByPredicate([&RowID](const auto& InScripts) { return InScripts->GetEmitterId() == RowID; });
	if (EmitterScriptsPtr == nullptr)
	{
		EmitterScriptsPtr = &AllEmitterScripts.AddDefaulted_GetRef();
		EmitterScriptsPtr->Reset(new NiagaraScriptStatsLocal::FNiagaraScriptStats(RowID, EmitterHandle, ShaderPlatform));
	}
	NiagaraScriptStatsLocal::FNiagaraScriptStats* EmitterScripts = EmitterScriptsPtr->Get();

	EmitterScripts->UpdateStatus(ShaderPlatform, EmitterHandle);

	Details.CellText = FText::FromString(EmitterScripts->GetCellString());
	Details.CellColor = EmitterScripts->GetCellColor();
	Details.CellWrapText = EmitterScripts->GetCellWrapText();
	return Details;
}

EShaderPlatform FNiagaraScriptStatsViewModel::ColumnNameToShaderPlatform(const FName& ColumnName) const
{
	for (const FShaderPlatformDetails& Details : ShaderPlatforms)
	{
		if ( Details.DisplayName == ColumnName )
		{
			return Details.ShaderPlatform;
		}
	}

	ensureMsgf(false, TEXT("Failed to find ShaderPlatform for ColumnName %s"), *ColumnName.ToString());
	return SP_NumPlatforms;
}

void FNiagaraScriptStatsViewModel::BuildShaderPlatformDetails()
{
	TArray<EShaderPlatform, TInlineAllocator<SP_StaticPlatform_Last + 1>> StaticShaderPlatforms;
#if PLATFORM_WINDOWS
	StaticShaderPlatforms.Add(SP_PCD3D_SM5);
	StaticShaderPlatforms.Add(SP_PCD3D_SM6);
	StaticShaderPlatforms.Add(SP_PCD3D_ES3_1);
#endif
	StaticShaderPlatforms.Add(SP_VULKAN_SM5);
	StaticShaderPlatforms.Add(SP_VULKAN_SM6);
	StaticShaderPlatforms.Add(SP_OPENGL_ES3_1_ANDROID); 
	StaticShaderPlatforms.Add(SP_VULKAN_ES3_1_ANDROID);
	StaticShaderPlatforms.Add(SP_VULKAN_SM5_ANDROID);
	StaticShaderPlatforms.Add(SP_METAL_SM5);
    StaticShaderPlatforms.Add(SP_METAL_SM6);
	StaticShaderPlatforms.Add(SP_METAL_ES3_1_IOS);
	StaticShaderPlatforms.Add(SP_METAL_SM5_IOS);
	StaticShaderPlatforms.Add(SP_METAL_SM6_IOS);

	TArray<EShaderPlatform, TInlineAllocator<SP_StaticPlatform_Last + 1>> ValidShaderPlatforms;
	for (EShaderPlatform StaticShaderPlatform : StaticShaderPlatforms)
	{
		if (FDataDrivenShaderPlatformInfo::IsValid(StaticShaderPlatform))
		{
			ValidShaderPlatforms.Add(StaticShaderPlatform);
		}
	}

	for (int32 iPlatform = SP_StaticPlatform_First; iPlatform <= SP_StaticPlatform_Last; ++iPlatform)
	{
		const EShaderPlatform ShaderPlatform = (EShaderPlatform)iPlatform;
		if ( FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform) )
		{
			ValidShaderPlatforms.Add(ShaderPlatform);
		}
	}

	// Ensure we can generate shaders for the above
	ShaderPlatforms.Reset(ValidShaderPlatforms.Num());

	ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
	const UNiagaraScripStatsViewModelSettings* Settings = GetDefault<UNiagaraScripStatsViewModelSettings>();
	for (const EShaderPlatform ShaderPlatform : ValidShaderPlatforms)
	{
		const FName ShaderFormat = LegacyShaderPlatformToShaderFormat(ShaderPlatform);
		if (TPM.FindShaderFormat(ShaderFormat) == nullptr)
		{
			continue;
		}

		FShaderPlatformDetails& Details = ShaderPlatforms.AddDefaulted_GetRef();
		Details.ShaderPlatform = ShaderPlatform;
		Details.ShaderFormatName = LegacyShaderPlatformToShaderFormat(ShaderPlatform);
		Details.ShaderPlatformName = ShaderPlatformToPlatformName(ShaderPlatform);
		Details.DisplayName = Details.ShaderFormatName;
		Details.bEnabled = Settings->EnabledPlatforms.Contains(ShaderPlatform);
	}

	ShaderPlatforms.Sort(
		[](const FShaderPlatformDetails& lhs, const FShaderPlatformDetails& rhs)
		{
			return lhs.DisplayName.LexicalLess(rhs.DisplayName);
		}
	);
}

void FNiagaraScriptStatsViewModel::CancelCompilations()
{
	for (auto it = PerPlatformScripts.CreateIterator(); it; ++it)
	{
		for (auto& EmitterScripts : it.Value())
		{
			EmitterScripts->CancelCompilation();
		}
	}

	PerPlatformScripts.Empty();
}

void FNiagaraScriptStatsViewModel::OnForceRecompile()
{
	CancelCompilations();

	if (auto SystemViewModel = WeakSystemViewModel.Pin())
	{
		UNiagaraSystem* NiagaraSystem = &SystemViewModel->GetSystem();
		NiagaraSystem->UNiagaraSystem::RequestCompile(true);
	}
}

void FNiagaraScriptStatsViewModel::OnSystemCompiled()
{
	PerPlatformScripts.Empty();
	RefreshView();
}

#undef LOCTEXT_NAMESPACE
