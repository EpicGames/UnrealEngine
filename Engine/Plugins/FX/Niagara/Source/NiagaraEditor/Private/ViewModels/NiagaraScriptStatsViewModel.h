// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "NiagaraScriptStatsViewModel.generated.h"

class FNiagaraSystemViewModel;
enum EShaderPlatform : uint16;

namespace NiagaraScriptStatsLocal
{
	struct FNiagaraScriptStats;
}

UCLASS(hidecategories = Object, config = EditorPerProjectUserSettings)
class UNiagaraScripStatsViewModelSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, config, Category = Options)
	TArray<int32> EnabledPlatforms;
};

class FNiagaraScriptStatsViewModel : public TSharedFromThis<FNiagaraScriptStatsViewModel>
{
	struct FShaderPlatformDetails
	{
		EShaderPlatform	ShaderPlatform;
		FName			ShaderFormatName;
		FName			ShaderPlatformName;
		FName			DisplayName;
		bool			bEnabled = false;
	};

	struct FGridCellDetails
	{
		FText CellText;
		FLinearColor CellColor = FColor::White;
		bool CellWrapText = false;
	};

public:
	FNiagaraScriptStatsViewModel();
	~FNiagaraScriptStatsViewModel();

	void Initialize(TWeakPtr<FNiagaraSystemViewModel> WeakSystemViewModel);
	void RefreshView();

	TSharedPtr<class SWidget> GetWidget();

	TConstArrayView<FShaderPlatformDetails> GetShaderPlatformDetails() const { return MakeArrayView(ShaderPlatforms); }

	bool IsEnabled(EShaderPlatform SP) const;
	void SetEnabled(EShaderPlatform SP, bool bEnabled);

	const TArray<TSharedPtr<FGuid>>* GetGridRowIDs() const;

	FGridCellDetails GetCellDetails(const FName& ColumnName, const FGuid& RowID);

	EShaderPlatform ColumnNameToShaderPlatform(const FName& ColumnName) const;

	void CancelCompilations();

	void OnForceRecompile();

	void OnSystemCompiled();

protected:
	void BuildShaderPlatformDetails();

private:
	TWeakPtr<FNiagaraSystemViewModel> WeakSystemViewModel;
	TArray<FShaderPlatformDetails> ShaderPlatforms;
	TArray<TSharedPtr<FGuid>> RowIDs;

	TSharedPtr<class SNiagaraScriptStatsWidget> Widget;

	TMap<EShaderPlatform, TArray<TUniquePtr<NiagaraScriptStatsLocal::FNiagaraScriptStats>>> PerPlatformScripts;
};
