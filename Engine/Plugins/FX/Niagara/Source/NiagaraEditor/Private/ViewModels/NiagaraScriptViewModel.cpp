// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraScriptViewModel.h"

#include "Editor.h"
#include "NiagaraEmitter.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeInput.h"
#include "NiagaraScript.h"
#include "NiagaraScriptGraphViewModel.h"
#include "NiagaraScriptInputCollectionViewModel.h"
#include "NiagaraScriptOutputCollectionViewModel.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "ViewModels/TNiagaraViewModelManager.h"

template<> TMap<UNiagaraScript*, TArray<FNiagaraScriptViewModel*>> TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>::ObjectsToViewModels{};

FNiagaraScriptViewModel::FNiagaraScriptViewModel(TAttribute<FText> DisplayName, ENiagaraParameterEditMode InParameterEditMode, bool bInIsForDataProcessingOnly)
	: InputCollectionViewModel(MakeShareable(new FNiagaraScriptInputCollectionViewModel(DisplayName, InParameterEditMode)))
	, OutputCollectionViewModel(MakeShareable(new FNiagaraScriptOutputCollectionViewModel(InParameterEditMode)))
	, GraphViewModel(MakeShareable(new FNiagaraScriptGraphViewModel(DisplayName, bInIsForDataProcessingOnly)))
	, VariableSelection(MakeShareable(new FNiagaraObjectSelection()))
	, bUpdatingSelectionInternally(false)
	, LastCompileStatus(ENiagaraScriptCompileStatus::NCS_Unknown)
	, bIsForDataProcessingOnly(bInIsForDataProcessingOnly)
{
	InputCollectionViewModel->GetSelection().OnSelectedObjectsChanged().AddRaw(this, &FNiagaraScriptViewModel::InputViewModelSelectionChanged);
	InputCollectionViewModel->OnParameterValueChanged().AddRaw(this, &FNiagaraScriptViewModel::InputParameterValueChanged);
	OutputCollectionViewModel->OnParameterValueChanged().AddRaw(this, &FNiagaraScriptViewModel::OutputParameterValueChanged);
	GraphViewModel->GetNodeSelection()->OnSelectedObjectsChanged().AddRaw(this, &FNiagaraScriptViewModel::GraphViewModelSelectedNodesChanged);
	if (bIsForDataProcessingOnly == false)
	{
		GEditor->RegisterForUndo(this);
	}
}

void FNiagaraScriptViewModel::OnVMScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion)
{
	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		FVersionedNiagaraScript VersionedScript = Scripts[i].Pin();
		if (VersionedScript.Script == InScript && InScript->IsStandaloneScript())
		{
			if (InScript->GetVMExecutableData().IsValid() == false)
			{
				continue;
			}

			LastCompileStatus = InScript->GetVMExecutableData().LastCompileStatus;

			if (InScript->GetVMExecutableData().LastCompileStatus == ENiagaraScriptCompileStatus::NCS_Error)
			{
				const FString& ErrorMsg = InScript->GetVMExecutableData().ErrorMsg;
				GraphViewModel->SetErrorTextToolTip(ErrorMsg + "\n(These same errors are also in the log)");
			}
			else
			{
				GraphViewModel->SetErrorTextToolTip("");
			}

			InputCollectionViewModel->RefreshParameterViewModels();
			OutputCollectionViewModel->RefreshParameterViewModels();
		}
	}
}

void FNiagaraScriptViewModel::OnGPUScriptCompiled(UNiagaraScript*, const FGuid&)
{
	// Do nothing in base implementation
}

bool FNiagaraScriptViewModel::IsGraphDirty(FGuid VersionGuid) const
{
	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		if (!Scripts[i].Script.IsValid())
		{
			continue;
		}

		if (Scripts[i].Script->IsCompilable() && !Scripts[i].Script->AreScriptAndSourceSynchronized(VersionGuid))
		{
			return true;
		}
	}
	return false;
}


FNiagaraScriptViewModel::~FNiagaraScriptViewModel()
{
	InputCollectionViewModel->GetSelection().OnSelectedObjectsChanged().RemoveAll(this);
	GraphViewModel->GetNodeSelection()->OnSelectedObjectsChanged().RemoveAll(this);

	if (Source.IsValid() && Source != nullptr)
	{
		UNiagaraGraph* Graph = Source->NodeGraph;
		if (Graph != nullptr)
		{
			Graph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
		}
	}

	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		if (Scripts[i].Script.IsValid())
		{
			Scripts[i].Script->OnVMScriptCompiled().RemoveAll(this);
			Scripts[i].Script->OnGPUScriptCompiled().RemoveAll(this);
		}
	}

	if (bIsForDataProcessingOnly == false && GEditor != nullptr)
	{
		GEditor->UnregisterForUndo(this);
	}

	for (TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>::Handle RegisteredHandle : RegisteredHandles)
	{
		UnregisterViewModelWithMap(RegisteredHandle);
	}
	//UE_LOGF(LogNiagaraEditor, Warning, "Deleting script view model %p", this);

}

INiagaraParameterDefinitionsSubscriber* FNiagaraScriptViewModel::GetParameterDefinitionsSubscriber()
{
	checkf(bIsForDataProcessingOnly, TEXT("Tried to get parameter definitions subscriber for scriptviewmodel not being used strictly for data processing! When not for data processing, this must be a standalone or scratch scriptviewmodel!"));
	checkf(Scripts.Num() == 1, TEXT("Tried to get parameter definitions subscriber for scriptviewmodel but there was more than one script!"));
	return &Scripts[0];
}

FText FNiagaraScriptViewModel::GetDisplayName() const
{
	return GraphViewModel->GetDisplayName();
}

const TArray<FVersionedNiagaraScriptWeakPtr>& FNiagaraScriptViewModel::GetScripts() const
{
	return Scripts;
}

void FNiagaraScriptViewModel::SetScripts(UNiagaraScriptSource* InScriptSource, TArray<FVersionedNiagaraScript>& InScripts)
{
	// Remove the graph changed handler on the child.
	if (Source.IsValid())
	{
		UNiagaraGraph* Graph = Source->NodeGraph;
		if (Graph != nullptr)
		{
			Graph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
		}
	}
	else
	{
		Source = nullptr;
	}

	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		if (Scripts[i].Script.IsValid())
		{
			Scripts[i].Script->OnVMScriptCompiled().RemoveAll(this);
			Scripts[i].Script->OnGPUScriptCompiled().RemoveAll(this);
		}
	}

	for (TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>::Handle RegisteredHandle : RegisteredHandles)
	{
		UnregisterViewModelWithMap(RegisteredHandle);
	}
	RegisteredHandles.Empty();

	Scripts.Empty();
	for (FVersionedNiagaraScript& VersionedScript : InScripts)
	{
		int32 i = Scripts.Add(VersionedScript.ToWeakPtr());
		check(VersionedScript.Script->GetSource(VersionedScript.Version) == InScriptSource);
		Scripts[i].Script->OnVMScriptCompiled().AddSP(this, &FNiagaraScriptViewModel::OnVMScriptCompiled);
		Scripts[i].Script->OnGPUScriptCompiled().AddSP(this, &FNiagaraScriptViewModel::OnGPUScriptCompiled);
	}
	Source = InScriptSource;

	InputCollectionViewModel->SetScripts(InScripts);
	OutputCollectionViewModel->SetScripts(InScripts);
	GraphViewModel->SetScriptSource(Source.Get());

	// Guess at initial compile status
	LastCompileStatus = ENiagaraScriptCompileStatus::NCS_UpToDate;

	CompileStatuses.Empty();
	CompileErrors.Empty();
	CompilePaths.Empty();

	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		FString Message;
		FVersionedNiagaraScript VersionedScript = Scripts[i].Pin();
		UNiagaraScript* Script = VersionedScript.Script;
		check(Script);
		ENiagaraScriptCompileStatus ScriptStatus = Script->GetLastCompileStatus();
		if (Script->IsCompilable() && Script->GetVMExecutableData().IsValid() && !Script->GetVMExecutableData().ByteCode.HasByteCode()) // This is either a brand new script or failed in the past. Since we create a default working script, assume invalid.
		{
			Message = TEXT("Please recompile for full error stack.");
			GraphViewModel->SetErrorTextToolTip(Message);
		}
		else // Possibly warnings previously, but still compiled. It *could* have been dirtied somehow, but we assume that it is up-to-date.
		{
			if (Script->IsCompilable() && Script->AreScriptAndSourceSynchronized())
			{
				LastCompileStatus = FNiagaraEditorUtilities::UnionCompileStatus(LastCompileStatus, Script->GetLastCompileStatus());
			}
			else if (Script->IsCompilable())
			{
				//DO nothing
				ScriptStatus = ENiagaraScriptCompileStatus::NCS_UpToDate;
			}
			else
			{
				LastCompileStatus = FNiagaraEditorUtilities::UnionCompileStatus(LastCompileStatus, ENiagaraScriptCompileStatus::NCS_Error);
				ScriptStatus = ENiagaraScriptCompileStatus::NCS_Error;
				Message = TEXT("Please recompile for full error stack.");
				GraphViewModel->SetErrorTextToolTip(Message);
			}
		}

		CompilePaths.Add(Script->GetPathName());
		CompileErrors.Add(Message);
		CompileStatuses.Add(ScriptStatus);
		RegisteredHandles.Add(RegisterViewModelWithMap(Script, this));
	}

	CompileTypes.SetNum(CompileStatuses.Num());
	for (int32 i = 0; i < CompileStatuses.Num(); i++)
	{
		CompileTypes[i].Key = Scripts[i].Script->GetUsage();
		CompileTypes[i].Value = Scripts[i].Script->GetUsageId();
	}
}

void FNiagaraScriptViewModel::SetScripts(FVersionedNiagaraEmitter InEmitter)
{
	FVersionedNiagaraEmitterData* EmitterData = InEmitter.GetEmitterData();
	if (EmitterData == nullptr)
	{
		TArray<FVersionedNiagaraScript> EmptyScripts;
		SetScripts(nullptr, EmptyScripts);
	}
	else
	{
		TArray<UNiagaraScript*> InScripts;
		EmitterData->GetScripts(InScripts);

		TArray<FVersionedNiagaraScript> EmitterScripts;
		for (UNiagaraScript* Script : InScripts)
		{
			EmitterScripts.AddDefaulted_GetRef().Script = Script;
		}
		
		SetScripts(Cast<UNiagaraScriptSource>(EmitterData->GraphSource), EmitterScripts);
	}
}

void FNiagaraScriptViewModel::SetScript(FVersionedNiagaraScript InScript)
{
	TArray<FVersionedNiagaraScript> InScripts;
	UNiagaraScriptSource* InSource = nullptr;
	if (InScript.Script)
	{
		InScripts.Add(InScript);
		InSource = Cast<UNiagaraScriptSource>(InScript.Script->GetSource(InScript.Version));
	}
	SetScripts(InSource, InScripts);
}

void FNiagaraScriptViewModel::MarkAllDirty(FString Reason)
{
	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		auto VersionedScript = Scripts[i];
		if (VersionedScript.Script.IsValid() && VersionedScript.Script->IsCompilable())
		{
			VersionedScript.Script->MarkScriptAndSourceDesynchronized(Reason, VersionedScript.Version);
		}
	}
}

TSharedRef<FNiagaraScriptInputCollectionViewModel> FNiagaraScriptViewModel::GetInputCollectionViewModel()
{
	return InputCollectionViewModel;
}


TSharedRef<FNiagaraScriptOutputCollectionViewModel> FNiagaraScriptViewModel::GetOutputCollectionViewModel()
{
	return OutputCollectionViewModel;
}

TSharedRef<FNiagaraScriptGraphViewModel> FNiagaraScriptViewModel::GetGraphViewModel()
{
	return GraphViewModel;
}

TSharedRef<FNiagaraObjectSelection> FNiagaraScriptViewModel::GetVariableSelection()
{
	return VariableSelection;
}

FVersionedNiagaraScript FNiagaraScriptViewModel::GetStandaloneScript()
{
	if (Scripts.Num() == 1)
	{
		FVersionedNiagaraScript ScriptOutput = Scripts[0].Pin();
		if (ScriptOutput.Script && ScriptOutput.Script->IsStandaloneScript())
		{
			return ScriptOutput;
		}
	}
	return FVersionedNiagaraScript();
}

bool FNiagaraScriptViewModel::RenameParameter(const FNiagaraVariable TargetParameter, const FName NewName)
{
	GetStandaloneScript().Script->Modify();
	UNiagaraGraph* Graph = GetGraphViewModel()->GetGraph();
	
	Graph->Modify();
	if (Graph->RenameParameter(TargetParameter, NewName))
	{
		UNiagaraScriptVariable* RenamedScriptVar = Graph->GetScriptVariable(NewName);

		// Check if the rename will give the same name and type as an existing parameter definition, and if so, link to the definition automatically.
		FNiagaraParameterDefinitionsUtilities::TrySubscribeScriptVarToDefinitionByName(RenamedScriptVar, this);
	}

	return true;
}

void FNiagaraScriptViewModel::UpdateCompileStatus(ENiagaraScriptCompileStatus InAggregateCompileStatus, const FString& InAggregateCompileErrorString,
	const TArray<ENiagaraScriptCompileStatus>& InCompileStatuses, const TArray<FString>& InCompileErrors, const TArray<FString>& InCompilePaths,
	const TArray<UNiagaraScript*>& InCompileSources)
{
	if (Source.IsValid() && Source != nullptr)
	{
		CompileStatuses = InCompileStatuses;
		CompileErrors = InCompileErrors;
		CompilePaths = InCompilePaths;

		CompileTypes.SetNum(CompileStatuses.Num());
		for (int32 i = 0; i < CompileStatuses.Num(); i++)
		{
			CompileTypes[i].Key = InCompileSources[i]->GetUsage();
			CompileTypes[i].Value = InCompileSources[i]->GetUsageId();
		}

		LastCompileStatus = InAggregateCompileStatus;
		
		if (LastCompileStatus == ENiagaraScriptCompileStatus::NCS_Error)
		{
			GraphViewModel->SetErrorTextToolTip(InAggregateCompileErrorString + "\n(These same errors are also in the log)");
		}
		else
		{
			GraphViewModel->SetErrorTextToolTip("");
		}

		InputCollectionViewModel->RefreshParameterViewModels();
		OutputCollectionViewModel->RefreshParameterViewModels();
	}
}

ENiagaraScriptCompileStatus FNiagaraScriptViewModel::GetScriptCompileStatus(ENiagaraScriptUsage InUsage, FGuid InUsageId) const
{
	ENiagaraScriptCompileStatus Status = ENiagaraScriptCompileStatus::NCS_Unknown;
	for (int32 i = 0; i < CompileTypes.Num(); i++)
	{
		if (UNiagaraScript::IsEquivalentUsage(CompileTypes[i].Key, InUsage) && CompileTypes[i].Value == InUsageId)
		{
			return CompileStatuses[i];
		}
	}
	return Status;
}

FText FNiagaraScriptViewModel::GetScriptErrors(ENiagaraScriptUsage InUsage, FGuid InUsageId) const
{
	FText Text = FText::GetEmpty();
	for (int32 i = 0; i < CompileTypes.Num(); i++)
	{
		if (UNiagaraScript::IsEquivalentUsage(CompileTypes[i].Key, InUsage) && CompileTypes[i].Value == InUsageId)
		{
			return FText::FromString(CompileErrors[i]);
		}
	}
	return Text;
}

UNiagaraScript* FNiagaraScriptViewModel::GetContainerScript(ENiagaraScriptUsage InUsage, FGuid InUsageId) 
{
	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		if (Scripts[i].Script->ContainsUsage(InUsage) && Scripts[i].Script->GetUsageId() == InUsageId)
		{
			return Scripts[i].Script.Get();
		}
	}
	return nullptr;
}

UNiagaraScript* FNiagaraScriptViewModel::GetScript(ENiagaraScriptUsage InUsage, FGuid InUsageId) 
{
	for (int32 i = 0; i < Scripts.Num(); i++)
	{
		if (Scripts[i].Script->IsEquivalentUsage(InUsage) && Scripts[i].Script->GetUsageId() == InUsageId)
		{
			return Scripts[i].Script.Get();
		}
	}
	return nullptr;
}

void FNiagaraScriptViewModel::CompileStandaloneScript(bool bForceCompile)
{
	if (Source.IsValid() && Scripts.Num() == 1 && Scripts[0].Script.IsValid())
	{
		FVersionedNiagaraScriptWeakPtr VersionedScript = Scripts[0];
		if (VersionedScript.Script->IsStandaloneScript() && VersionedScript.Script->IsCompilable())
		{
			VersionedScript.SynchronizeWithParameterDefinitions();
			VersionedScript.Script->RequestCompile(VersionedScript.Version, bForceCompile);
		}
		else if (!VersionedScript.Script->IsCompilable())
		{
			// do nothing
		}
		else
		{
			ensure(0); // Should not call this for a script that isn't standalone.
		}
	}
	else
	{
		ensure(0); // Should not call this for a script that isn't standalone.
	}
}


ENiagaraScriptCompileStatus FNiagaraScriptViewModel::GetLatestCompileStatus(FGuid VersionGuid)
{
	if (GraphViewModel->GetGraph() && IsGraphDirty(VersionGuid))
	{
		return ENiagaraScriptCompileStatus::NCS_Dirty;
	}
	return LastCompileStatus;
}

void FNiagaraScriptViewModel::RefreshNodes()
{
	if (GraphViewModel->GetGraph())
	{
		TArray<UNiagaraNode*> NiagaraNodes;
		GraphViewModel->GetGraph()->GetNodesOfClass<UNiagaraNode>(NiagaraNodes);
		for (UNiagaraNode* NiagaraNode : NiagaraNodes)
		{
			if (NiagaraNode->RefreshFromExternalChanges())
			{
				for (int32 i = 0; i < Scripts.Num(); i++)
				{
					auto VersionedScript = Scripts[i];
					if (VersionedScript.Script.IsValid() && VersionedScript.Script->IsCompilable())
					{
						VersionedScript.Script->MarkScriptAndSourceDesynchronized(TEXT("Nodes manually refreshed"), VersionedScript.Version);
					}
				}
			}
		}
	}
}

void FNiagaraScriptViewModel::PostUndo(bool bSuccess)
{
	InputCollectionViewModel->RefreshParameterViewModels();
	OutputCollectionViewModel->RefreshParameterViewModels();
}

void FNiagaraScriptViewModel::GraphViewModelSelectedNodesChanged()
{
	if (bUpdatingSelectionInternally == false)
	{
		bUpdatingSelectionInternally = true;
		{
			TSet<FName> SelectedInputIds;
			for (UObject* SelectedObject : GraphViewModel->GetNodeSelection()->GetSelectedObjectsResolved())
			{
				UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(SelectedObject);
				if (InputNode != nullptr)
				{
					SelectedInputIds.Add(InputNode->Input.GetName());
				}
			}

			TSet<TSharedRef<INiagaraParameterViewModel>> ParametersToSelect;
			for (TSharedRef<INiagaraParameterViewModel> Parameter : InputCollectionViewModel->GetParameters())
			{
				if (SelectedInputIds.Contains(Parameter->GetName()))
				{
					ParametersToSelect.Add(Parameter);
				}
			}

			InputCollectionViewModel->GetSelection().SetSelectedObjects(ParametersToSelect);
		}
		bUpdatingSelectionInternally = false;
	}
}

void FNiagaraScriptViewModel::InputViewModelSelectionChanged()
{
	if (bUpdatingSelectionInternally == false)
	{
		bUpdatingSelectionInternally = true;
		{
			TSet<FName> SelectedInputIds;
			for (TSharedRef<INiagaraParameterViewModel> SelectedParameter : InputCollectionViewModel->GetSelection().GetSelectedObjects())
			{
				SelectedInputIds.Add(SelectedParameter->GetName());
			}

			TArray<UNiagaraNodeInput*> InputNodes;
			if (GraphViewModel->GetGraph())
			{
				GraphViewModel->GetGraph()->GetNodesOfClass(InputNodes);
			}
			TSet<UObject*> NodesToSelect;
			for (UNiagaraNodeInput* InputNode : InputNodes)
			{
				if (SelectedInputIds.Contains(InputNode->Input.GetName()))
				{
					NodesToSelect.Add(InputNode);
				}
			}

			GraphViewModel->GetNodeSelection()->SetSelectedObjects(NodesToSelect);
		}
		bUpdatingSelectionInternally = false;
	}
}

void FNiagaraScriptViewModel::InputParameterValueChanged(TConstArrayView<FName> ParameterName)
{
	MarkAllDirty(TEXT("Input parameter value changed"));
}

void FNiagaraScriptViewModel::OutputParameterValueChanged(TConstArrayView<FName> ParameterName)
{
	MarkAllDirty(TEXT("Output parameter value changed"));
}
