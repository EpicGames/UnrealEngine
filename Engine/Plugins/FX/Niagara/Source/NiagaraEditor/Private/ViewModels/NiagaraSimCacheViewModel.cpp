// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "NiagaraComponent.h"
#include "NiagaraSimCache.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraEditorSimCacheUtils.h"
#include "NiagaraDebuggerCommon.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Serialization/MemoryReader.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/SNiagaraSimCacheTreeView.h"
#include "UObject/Package.h"
#include "Editor.h"
#include "FileHelpers.h"

#define LOCTEXT_NAMESPACE "NiagaraSimCacheViewModel"

namespace NiagaraSimCacheViewModelPrivate
{
	bool bNamespaceGroupingEnabled = true;
	static FAutoConsoleVariableRef CVarNamespaceGroupingEnabled(
		TEXT("fx.Niagara.SimCacheViewer.NamespaceGroupingEnabled"),
		bNamespaceGroupingEnabled,
		TEXT("When enabled namespaces will be part of the attribute tree view."),
		ECVF_Default
	);

	TOptional<bool> FindComponentFilteredState(FName ComponentName, TArray<FNiagaraSimCacheViewModel::FComponentInfo> Components)
	{
		for (const FNiagaraSimCacheViewModel::FComponentInfo& Component : Components)
		{
			if (Component.Name == ComponentName)
			{
				return Component.bIsFiltered;
			}
		}
		return {};
	}

	void TraverseStructItems(const FString& StructPath, const FNiagaraTypeDefinition& StructTypeDef, const TFunction<void(const FString&, const FNiagaraTypeDefinition&)>& ItemCallback)
	{
		UScriptStruct* Struct = StructTypeDef.GetScriptStruct();
		if (Struct == nullptr)
		{
			return;
		}

		for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = *PropertyIt;
			const FString PropertyPath = StructPath + "." + Property->GetName();

			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (UScriptStruct* FriendlyStruct = FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProperty->Struct, ENiagaraStructConversion::Simulation))
				{
					TraverseStructItems(PropertyPath, FNiagaraTypeDefinition(FriendlyStruct), ItemCallback);
				}
			}
			if (Property->IsA(FIntProperty::StaticClass()))
			{
				ItemCallback(PropertyPath, FNiagaraTypeDefinition::GetIntDef());
			}
			else if (Property->IsA(FFloatProperty::StaticClass()))
			{
				ItemCallback(PropertyPath, FNiagaraTypeDefinition::GetFloatDef());
			}
			else if (Property->IsA(FBoolProperty::StaticClass()))
			{
				ItemCallback(PropertyPath, FNiagaraTypeDefinition::GetBoolDef());
			}
		}
	}

	namespace NamespaceGroupedTree
	{
		void AddItemToTreeInternal(TWeakPtr<SNiagaraSimCacheTreeView> WeakOwnerTreeView, TArray<TSharedRef<FNiagaraSimCacheTreeItem>>& ComponentTreeItems, FString ComponentPath, const FNiagaraTypeDefinition ComponentType)
		{
			TSharedPtr<FNiagaraSimCacheTreeItem> ParentItem = ComponentTreeItems[0];

			FString CurrentPath;

			TArray<FString> ComponentPathParts;
			ComponentPath.ParseIntoArray(ComponentPathParts, TEXT("."));

			for (int32 i = 0; i < ComponentPathParts.Num(); ++i)
			{
				const FString PathPart = ComponentPathParts[i];
				if (CurrentPath.Len() > 0)
				{
					CurrentPath.AppendChar('.');
				}
				CurrentPath.Append(PathPart);

				TSharedRef<FNiagaraSimCacheTreeItem>* CurrentPathItem = ComponentTreeItems.FindByPredicate(
					[&CurrentPath](const TSharedPtr<FNiagaraSimCacheTreeItem>& Item)
					{
						return Item->GetFilterName() == CurrentPath;
					}
				);

				if (CurrentPathItem == nullptr)
				{
					TSharedRef<FNiagaraSimCacheComponentTreeItem> NewItem = MakeShared<FNiagaraSimCacheComponentTreeItem>(WeakOwnerTreeView);
					NewItem->SetDisplayName(FText::FromString(PathPart));
					NewItem->SetFilterName(CurrentPath);

					const bool bIsLastPart = i == ComponentPathParts.Num() - 1;
					if (bIsLastPart)
					{
						NewItem->TypeDef = ComponentType;
					}

					ComponentTreeItems.Emplace(NewItem);
					CurrentPathItem = &ComponentTreeItems.Last();
				}
				ParentItem->AddChild(*CurrentPathItem);
				ParentItem = *CurrentPathItem;
			}
		}

		void AddItemToTree(TWeakPtr<SNiagaraSimCacheTreeView> WeakOwnerTreeView, TArray<TSharedRef<FNiagaraSimCacheTreeItem>>& ComponentTreeItems, const FNiagaraSimCacheVariable& Variable)
		{
			const FNiagaraTypeDefinition VariableTypeDef = Variable.Variable.GetType();
			const FString VariableName = Variable.Variable.GetName().ToString();

			AddItemToTreeInternal(WeakOwnerTreeView, ComponentTreeItems, VariableName, VariableTypeDef);
			if (!VariableTypeDef.IsEnum() && !FNiagaraTypeDefinition::IsScalarDefinition(VariableTypeDef))
			{
				TraverseStructItems(
					VariableName,
					VariableTypeDef,
					[&WeakOwnerTreeView, &ComponentTreeItems](const FString& StructPath, const FNiagaraTypeDefinition& StructTypeDef)
					{
						AddItemToTreeInternal(WeakOwnerTreeView, ComponentTreeItems, StructPath, StructTypeDef);
					}
				);
			}
		}
	} //namespace NamespaceGroupedTree

	namespace TypeGroupedTree
	{
		TSharedRef<FNiagaraSimCacheTreeItem> AddItemToTreeInternal(TWeakPtr<SNiagaraSimCacheTreeView> WeakOwnerTreeView, const TArray<TSharedRef<FNiagaraSimCacheTreeItem>>& ComponentTreeItems, FString ComponentPath, const FNiagaraTypeDefinition ComponentType)
		{
			TSharedPtr<FNiagaraSimCacheTreeItem> ParentItem = ComponentTreeItems.Last();

			TSharedRef<FNiagaraSimCacheComponentTreeItem> NewItem = MakeShared<FNiagaraSimCacheComponentTreeItem>(WeakOwnerTreeView);
			NewItem->SetFilterName(ComponentPath);
			NewItem->TypeDef = ComponentType;

			if (ComponentTreeItems.Num() > 1)
			{
				const FString DisplayPath = ComponentPath.Mid(ParentItem->GetFilterName().Len() + 1);
				NewItem->SetDisplayName(FText::FromString(DisplayPath));
			}
			else
			{
				NewItem->SetDisplayName(FText::FromString(ComponentPath));
			}

			ParentItem->AddChild(NewItem);
			return NewItem;
		}

		void AddItemToTree(TWeakPtr<SNiagaraSimCacheTreeView> WeakOwnerTreeView, TArray<TSharedRef<FNiagaraSimCacheTreeItem>>& ComponentTreeItems, const FNiagaraSimCacheVariable& Variable)
		{
			const FNiagaraTypeDefinition VariableTypeDef = Variable.Variable.GetType();
			const FString VariableName = Variable.Variable.GetName().ToString();

			TSharedRef<FNiagaraSimCacheTreeItem> ParentItem = AddItemToTreeInternal(WeakOwnerTreeView, ComponentTreeItems, VariableName, VariableTypeDef);
			if (!VariableTypeDef.IsEnum() && !FNiagaraTypeDefinition::IsScalarDefinition(VariableTypeDef))
			{
				ComponentTreeItems.Add(ParentItem);
				TraverseStructItems(
					VariableName,
					VariableTypeDef,
					[&WeakOwnerTreeView, &ComponentTreeItems, &ParentItem](const FString& StructPath, const FNiagaraTypeDefinition& StructTypeDef)
					{
						AddItemToTreeInternal(WeakOwnerTreeView, ComponentTreeItems, StructPath, StructTypeDef);
					}
				);
				ComponentTreeItems.Pop();
			}
		}
	} //namespace TypeGroupedTree

} //namespace NiagaraSimCacheViewModelPrivate

FNiagaraSimCacheViewModel::FNiagaraSimCacheViewModel()
{
}

FNiagaraSimCacheViewModel::~FNiagaraSimCacheViewModel()
{
	UNiagaraSimCache::OnCacheEndWrite.RemoveAll(this);
	bDelegatesAdded = false;
	SimCache = nullptr;
	PreviewComponent = nullptr;
}

void FNiagaraSimCacheViewModel::Initialize(TWeakObjectPtr<UNiagaraSimCache> InSimCache) 
{
	if (bDelegatesAdded == false)
	{
		bDelegatesAdded = true;
		UNiagaraSimCache::OnCacheEndWrite.AddSP(this, &FNiagaraSimCacheViewModel::OnCacheModified);
	}

	if(InSimCache.IsValid())
	{
		SimCache = InSimCache.Get();
	}
	
	UpdateComponentInfos();
	UpdateCachedFrame();
	SetupPreviewComponentAndInstance();
	
	OnSimCacheChangedDelegate.Broadcast();
	OnViewDataChangedDelegate.Broadcast(true);
}

void FNiagaraSimCacheViewModel::SetupPreviewComponentAndInstance()
{
	UNiagaraSystem* System = SimCache ? SimCache->GetSystem(true) : nullptr;

	if(SimCache && System)
	{
		PreviewComponent = NewObject<UNiagaraComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		PreviewComponent->CastShadow = 1;
		PreviewComponent->bCastDynamicShadow = 1;
		PreviewComponent->SetAllowScalability(false);
		PreviewComponent->SetAsset(System);
		PreviewComponent->SetForceSolo(true);
		PreviewComponent->SetAgeUpdateMode(ENiagaraAgeUpdateMode::DesiredAge);
		PreviewComponent->SetCanRenderWhileSeeking(false);
		PreviewComponent->Activate(true);
		PreviewComponent->SetSimCache(SimCache);
		PreviewComponent->SetRelativeLocation(FVector::ZeroVector);
		PreviewComponent->SetDesiredAge(SimCache->GetStartSeconds());
	}
}

TArrayView<FNiagaraSimCacheViewModel::FComponentInfo> FNiagaraSimCacheViewModel::GetMutableSelectedComponentInfos()
{
	if (SelectionMode == ESelectionMode::SystemInstance)
	{
		return SystemComponentInfos;
	}
	else if (SelectionMode == ESelectionMode::Emitter)
	{
		if (TArray<FComponentInfo>* ComponentInfos = EmitterComponentInfos.Find(SelectedEmitterName))
		{
			return *ComponentInfos;
		}
	}
	return MakeArrayView<FComponentInfo>(nullptr, 0);
}

TConstArrayView<FNiagaraSimCacheViewModel::FComponentInfo> FNiagaraSimCacheViewModel::GetSelectedComponentInfos() const
{
	return const_cast<FNiagaraSimCacheViewModel*>(this)->GetMutableSelectedComponentInfos();
}

FText FNiagaraSimCacheViewModel::GetComponentText(const FName ComponentName, const int32 InstanceIndex) const
{
	const FComponentInfo* ComponentInfo = GetSelectedComponentInfos().FindByPredicate([ComponentName](const FComponentInfo& FoundInfo) { return FoundInfo.Name == ComponentName; });

	if (ComponentInfo)
	{
		if (InstanceIndex >= 0 && InstanceIndex < NumInstances)
		{
			if (ComponentInfo->bIsFloat)
			{
				const float Value = FloatComponents[(ComponentInfo->ComponentOffset * NumInstances) + InstanceIndex];
				return FText::AsNumber(Value);
			}
			else if (ComponentInfo->bIsHalf)
			{
				const FFloat16 Value = HalfComponents[(ComponentInfo->ComponentOffset * NumInstances) + InstanceIndex];
				return FText::AsNumber(Value.GetFloat());
			}
			else if (ComponentInfo->bIsInt32)
			{
				const int32 Value = Int32Components[(ComponentInfo->ComponentOffset * NumInstances) + InstanceIndex];
				if (ComponentInfo->bShowAsBool)
				{
					return Value == 0 ? LOCTEXT("False", "False") : LOCTEXT("True", "True");
				}
				else if (ComponentInfo->Enum != nullptr)
				{
					return ComponentInfo->Enum->GetDisplayNameTextByValue(Value);
				}
				else
				{
					return FText::AsNumber(Value);
				}
			}
		}
	}
	return LOCTEXT("Error", "Error");
}

bool FNiagaraSimCacheViewModel::CompareComponent(int32 ComponentIndex, int32 LhsInstance, int32 RhsInstance, bool bAscending) const
{
	bool bResult = bAscending ? LhsInstance < RhsInstance : LhsInstance > RhsInstance;

	TConstArrayView<FNiagaraSimCacheViewModel::FComponentInfo> ComponentInfos = GetSelectedComponentInfos();
	if (ComponentInfos.IsValidIndex(ComponentIndex) && LhsInstance >= 0 && LhsInstance < NumInstances && RhsInstance >= 0 && RhsInstance < NumInstances)
	{
		const FComponentInfo* ComponentInfo = &ComponentInfos[ComponentIndex];
		if (ComponentInfo->bIsFloat)
		{
			const float LhsValue = FloatComponents[(ComponentInfo->ComponentOffset * NumInstances) + LhsInstance];
			const float RhsValue = FloatComponents[(ComponentInfo->ComponentOffset * NumInstances) + RhsInstance];
			if ( LhsValue != RhsValue )
			{
				bResult = bAscending ? LhsValue < RhsValue : LhsValue > RhsValue;
			}
		}
		else if (ComponentInfo->bIsHalf)
		{
			const float LhsValue = HalfComponents[(ComponentInfo->ComponentOffset * NumInstances) + LhsInstance].GetFloat();
			const float RhsValue = HalfComponents[(ComponentInfo->ComponentOffset * NumInstances) + RhsInstance].GetFloat();
			if (LhsValue != RhsValue)
			{
				bResult = bAscending ? LhsValue < RhsValue : LhsValue > RhsValue;
			}
		}
		else if (ComponentInfo->bIsInt32)
		{
			const int32 LhsValue = Int32Components[(ComponentInfo->ComponentOffset * NumInstances) + LhsInstance];
			const int32 RhsValue = Int32Components[(ComponentInfo->ComponentOffset * NumInstances) + RhsInstance];
			if (ComponentInfo->bShowAsBool)
			{
				const int32 bLhsValue = LhsValue != 0;
				const int32 bRhsValue = RhsValue != 0;
				if (bLhsValue != bRhsValue)
				{
					bResult = bAscending ? bLhsValue < bRhsValue : bLhsValue > bRhsValue;
				}
			}
			else
			{
				if (LhsValue != RhsValue)
				{
					bResult = bAscending ? LhsValue < RhsValue : LhsValue > RhsValue;
				}
			}
		}
	}
	return bResult;
}

int32 FNiagaraSimCacheViewModel::GetNumFrames() const
{
	
	return SimCache ? SimCache->GetNumFrames() : 0;
}

void FNiagaraSimCacheViewModel::SetFrameIndex(const int32 InFrameIndex)
{
	FrameIndex = InFrameIndex;
	UpdateCachedFrame();
	if(PreviewComponent && SimCache)
	{
		const float Duration = SimCache->GetDurationSeconds();
		const int NumFrames = SimCache->GetNumFrames();
		const float StartSeconds = SimCache->GetStartSeconds();

		const float NormalizedFrame = FMath::Clamp( NumFrames == 0 ? 0.0f : float(InFrameIndex) / float(NumFrames - 1), 0.0f, 1.0f );
		const float DesiredAge = FMath::Clamp(StartSeconds + (Duration * NormalizedFrame), StartSeconds, StartSeconds + Duration);
		
		PreviewComponent->Activate();
		PreviewComponent->SetDesiredAge(DesiredAge);
	}
	OnViewDataChangedDelegate.Broadcast(false);
}

FText FNiagaraSimCacheViewModel::GetSelectedText() const
{
	if (SimCache && SimCache->IsCacheValid())
	{
		switch (SelectionMode)
		{
			case ESelectionMode::SystemInstance:
				return LOCTEXT("SystemInstance", "System Instance");

			case ESelectionMode::Emitter:
				return FText::FromName(SelectedEmitterName);

			case ESelectionMode::DataInterface:
				if ( SelectedDataInterface.IsValid() )
				{
					// Data interfaces can have some long names, i.e. SetVariablesxxx.Emitter.DIName, so let's limit to something sensible
					constexpr int32 MaxStringLength = 32;
					FString DataInterfaceName = SelectedDataInterface.GetName().ToString();
					if (DataInterfaceName.Len() > MaxStringLength)
					{
						DataInterfaceName.RightChopInline(DataInterfaceName.Len() - MaxStringLength, EAllowShrinking::No);
						DataInterfaceName.InsertAt(0, TEXT("..."));
					}
					return FText::FromString(DataInterfaceName);
				}
				break;

			case ESelectionMode::DebugData:
				return LOCTEXT("DebugData", "Debug Data");
		}
	}
	return FText();
}

const UObject* FNiagaraSimCacheViewModel::GetSelectedDataInterfaceStorage() const
{
	if (SelectionMode == ESelectionMode::DataInterface && SimCache)
	{
		return SimCache->GetDataInterfaceStorageObject(SelectedDataInterface);
	}
	return nullptr;
}

void FNiagaraSimCacheViewModel::SetSelectedSystemInstance()
{
	SelectionMode			= ESelectionMode::SystemInstance;
	SelectedEmitterName		= NAME_None;
	SelectedDataInterface	= FNiagaraVariableBase();

	RefreshFromSelectionChanged();
}

void FNiagaraSimCacheViewModel::SetSelectedEmitter(FName EmitterName)
{
	SelectionMode			= ESelectionMode::Emitter;
	SelectedEmitterName		= EmitterName;
	SelectedDataInterface	= FNiagaraVariableBase();

	RefreshFromSelectionChanged();
}

void FNiagaraSimCacheViewModel::SetSelectedDataInterface(FNiagaraVariableBase DIVariable)
{
	SelectionMode			= ESelectionMode::DataInterface;
	SelectedEmitterName		= NAME_None;
	SelectedDataInterface	= DIVariable;

	RefreshFromSelectionChanged();
}

void FNiagaraSimCacheViewModel::SetSelectedDebugData()
{
	SelectionMode			= ESelectionMode::DebugData;
	SelectedEmitterName		= NAME_None;
	SelectedDataInterface	= FNiagaraVariableBase();

	RefreshFromSelectionChanged();
}

bool FNiagaraSimCacheViewModel::IsComponentFilterActive() const
{
	return SelectionMode == ESelectionMode::SystemInstance || SelectionMode == ESelectionMode::Emitter;
}

bool FNiagaraSimCacheViewModel::IsComponentFiltered(FName ComponentName) const
{
	for (const FComponentInfo& ComponentInfo : GetSelectedComponentInfos())
	{
		if (ComponentInfo.Name == ComponentName)
		{
			return ComponentInfo.bIsFiltered;
		}
	}
	return false;
}

bool FNiagaraSimCacheViewModel::IsComponentFiltered(const FString& ComponentNameString) const
{
	return IsComponentFiltered(FName(*ComponentNameString));
}

void FNiagaraSimCacheViewModel::SetComponentFiltered(const FString& ComponentNameString, bool bFiltered)
{
	const FName ComponentName(*ComponentNameString);
	for (FComponentInfo& ComponentInfo : GetMutableSelectedComponentInfos())
	{
		if (ComponentInfo.Name == ComponentName)
		{
			ComponentInfo.bIsFiltered = bFiltered;
			OnViewDataChangedDelegate.Broadcast(true);
			return;
		}
	}
}

void FNiagaraSimCacheViewModel::ToggleComponentFiltered(const FString& ComponentNameString)
{
	const FName ComponentName(*ComponentNameString);
	for (FComponentInfo& ComponentInfo : GetMutableSelectedComponentInfos())
	{
		if (ComponentInfo.Name == ComponentName)
		{
			ComponentInfo.bIsFiltered = !ComponentInfo.bIsFiltered;
			OnViewDataChangedDelegate.Broadcast(true);
			return;
		}
	}
}

void FNiagaraSimCacheViewModel::SetAllComponentFiltered(bool bFiltered)
{
	for (FComponentInfo& ComponentInfo : GetMutableSelectedComponentInfos())
	{
		ComponentInfo.bIsFiltered = bFiltered;
	}
	OnViewDataChangedDelegate.Broadcast(true);
}

void FNiagaraSimCacheViewModel::RefreshSelection()
{
	bool bSelectionValid = false;
	if (SimCache && SimCache->IsCacheValid())
	{
		switch (SelectionMode)
		{
			case ESelectionMode::SystemInstance:
				bSelectionValid = true;
				break;

			case ESelectionMode::Emitter:
				bSelectionValid = SimCache->GetEmitterIndex(SelectedEmitterName) != INDEX_NONE;
				break;

			case ESelectionMode::DataInterface:
				bSelectionValid = SimCache->GetDataInterfaceStorageObject(SelectedDataInterface) != nullptr;
				break;

			case ESelectionMode::DebugData:
				bSelectionValid = SimCache->GetDebugData() != nullptr;
				break;
		}
	}

	if (!bSelectionValid)
	{
		SetSelectedSystemInstance();
	}

	UpdateCachedFrame();
	UpdateCurrentEntries();
	OnBufferChangedDelegate.Broadcast();
	OnViewDataChangedDelegate.Broadcast(true);
}

void FNiagaraSimCacheViewModel::RefreshFromSelectionChanged()
{
	UpdateCachedFrame();
	UpdateCurrentEntries();
	OnBufferChangedDelegate.Broadcast();
	OnViewDataChangedDelegate.Broadcast(true);
}

bool FNiagaraSimCacheViewModel::IsCacheValid() const
{
	return SimCache ? SimCache->IsCacheValid() : false;
}

int32 FNiagaraSimCacheViewModel::GetNumEmitterLayouts() const
{
	return SimCache ? SimCache->GetNumEmitters() : 0;
}

FName FNiagaraSimCacheViewModel::GetEmitterLayoutName(const int32 Index) const
{
	return SimCache ? SimCache->GetEmitterName(Index) : NAME_None;
}

const UNiagaraSimCacheDebugData* FNiagaraSimCacheViewModel::GetCacheDebugData() const
{
	return SimCache ? SimCache->GetDebugData() : nullptr;
}

FNiagaraSimCacheViewModel::FOnViewDataChanged& FNiagaraSimCacheViewModel::OnViewDataChanged()
{
	return OnViewDataChangedDelegate;
}

FNiagaraSimCacheViewModel::FOnSimCacheChanged& FNiagaraSimCacheViewModel::OnSimCacheChanged()
{
	return OnSimCacheChangedDelegate;
}

FNiagaraSimCacheViewModel::FOnBufferChanged& FNiagaraSimCacheViewModel::OnBufferChanged()
{
	return OnBufferChangedDelegate;
}

void FNiagaraSimCacheViewModel::OnCacheModified(UNiagaraSimCache* InSimCache)
{
	if ( SimCache )
	{
		if ( SimCache == InSimCache )
		{
			SetFrameIndex(0);
			UpdateComponentInfos();
			UpdateCachedFrame();
			OnSimCacheChangedDelegate.Broadcast();
			OnViewDataChangedDelegate.Broadcast(true);
		}
	}
}

void FNiagaraSimCacheViewModel::UpdateCachedFrame()
{
	NumInstances = 0;
	FloatComponents.Empty();
	HalfComponents.Empty();
	Int32Components.Empty();
	
	if (SimCache == nullptr)
	{
		return;
	}

	if ( FrameIndex < 0 || FrameIndex >= SimCache->GetNumFrames() )
	{
		return;
	}

	// Determine if we need to read attributes from the cache
	TOptional<int32> EmitterIndex;
	switch (SelectionMode)
	{
		case ESelectionMode::SystemInstance:
			NumInstances = 1;
			EmitterIndex = INDEX_NONE;
			break;

		case ESelectionMode::Emitter:
		{
			const int32 FoundEmitterIndex = SimCache->GetEmitterIndex(SelectedEmitterName);
			if (FoundEmitterIndex != INDEX_NONE)
			{
				EmitterIndex = FoundEmitterIndex;
				NumInstances = SimCache->GetEmitterNumInstances(FoundEmitterIndex, FrameIndex);
			}
			break;
		}

		case ESelectionMode::DataInterface:
		case ESelectionMode::DebugData:
			NumInstances = 1;
			break;
	}

	// Read attributes
	if (EmitterIndex.IsSet())
	{
		const FName EmitterName = EmitterIndex.GetValue() == INDEX_NONE ? NAME_None : SimCache->GetEmitterName(EmitterIndex.GetValue());

		SimCache->ForEachEmitterAttribute(
			EmitterIndex.GetValue(),
			[&](const FNiagaraSimCacheVariable& Variable)
			{
				// Pull in data
				SimCache->ReadAttribute(FloatComponents, HalfComponents, Int32Components, Variable.Variable.GetName(), EmitterName, FrameIndex);

				return true;
			}
		);
	}
}

void FNiagaraSimCacheViewModel::UpdateComponentInfos()
{
	// Save filter state
	TArray<FComponentInfo>					TempSystemComponentInfos;
	TMap<FName, TArray<FComponentInfo>>		TempEmitterComponentInfos;
	Swap(TempSystemComponentInfos, SystemComponentInfos);
	Swap(TempEmitterComponentInfos, EmitterComponentInfos);

	// Clear data
	SystemComponentInfos.Empty();
	EmitterComponentInfos.Empty();
	FoundFloatComponents = 0;
	FoundHalfComponents = 0;
	FoundInt32Components = 0;
	if (SimCache == nullptr)
	{
		return;
	}
	
	SimCache->ForEachEmitterAttribute(INDEX_NONE,
		[&](const FNiagaraSimCacheVariable& Variable)
		{
			// Build component info
			const FNiagaraTypeDefinition& TypeDef = Variable.Variable.GetType();
			if (TypeDef.IsEnum())
			{
				FComponentInfo& ComponentInfo = SystemComponentInfos.AddDefaulted_GetRef();
				ComponentInfo.Name = Variable.Variable.GetName();
				ComponentInfo.ComponentOffset = FoundInt32Components++;
				ComponentInfo.bIsInt32 = true;
				ComponentInfo.Enum = TypeDef.GetEnum();
			}
			else
			{
				BuildComponentInfos(Variable.Variable.GetName(), TypeDef.GetScriptStruct(), SystemComponentInfos);
			}

			return true;
		}
	);

	for (int32 iEmitter=0; iEmitter < SimCache->GetNumEmitters(); ++iEmitter)
	{
		TArray<FComponentInfo>& CurrentComponentInfos = EmitterComponentInfos.Add(SimCache->GetEmitterName(iEmitter));

		FoundFloatComponents = 0;
		FoundHalfComponents = 0;
		FoundInt32Components = 0;
		
		SimCache->ForEachEmitterAttribute(
			iEmitter,
			[&](const FNiagaraSimCacheVariable& Variable)
			{
				// Build component info
				const FNiagaraTypeDefinition& TypeDef = Variable.Variable.GetType();
				if (TypeDef.IsEnum())
				{
					FComponentInfo& ComponentInfo = CurrentComponentInfos.AddDefaulted_GetRef();
					ComponentInfo.Name = Variable.Variable.GetName();
					ComponentInfo.ComponentOffset = FoundInt32Components++;
					ComponentInfo.bIsInt32 = true;
					ComponentInfo.Enum = TypeDef.GetEnum();
				}
				else
				{
					BuildComponentInfos(Variable.Variable.GetName(), TypeDef.GetScriptStruct(), CurrentComponentInfos);
				}

				return true;
			}
		);
	}

	// Restore filter state
	for (FComponentInfo& ComponentInfo : SystemComponentInfos)
	{
		ComponentInfo.bIsFiltered = NiagaraSimCacheViewModelPrivate::FindComponentFilteredState(ComponentInfo.Name, TempSystemComponentInfos).Get(true);
	}

	for (auto EmitterIt=EmitterComponentInfos.CreateIterator(); EmitterIt; ++EmitterIt)
	{
		TArray<FComponentInfo>* TempEmitterComponents = TempEmitterComponentInfos.Find(EmitterIt->Key);
		if (TempEmitterComponents == nullptr)
		{
			continue;
		}

		for (FComponentInfo& ComponentInfo : EmitterIt->Value)
		{
			ComponentInfo.bIsFiltered = NiagaraSimCacheViewModelPrivate::FindComponentFilteredState(ComponentInfo.Name, *TempEmitterComponents).Get(true);
		}
	}
}

void FNiagaraSimCacheViewModel::BuildTreeItemChildren(TSharedPtr<FNiagaraSimCacheTreeItem> TreeItem, TWeakPtr<SNiagaraSimCacheTreeView> OwningTreeView)
{
	if (!TreeItem || !SimCache)
	{
		return;
	}

	TOptional<int32> CacheEmitterIndex;
	switch (TreeItem->GetType())
	{
		case ENiagaraSimCacheOverviewItemType::System:
			CacheEmitterIndex = INDEX_NONE;
			break;

		case ENiagaraSimCacheOverviewItemType::Emitter:
		{
			const FName EmitterName = TreeItem->GetEmitterName();
			const int32 EmitterIndex = SimCache->GetEmitterIndex(EmitterName);
			if (EmitterIndex != INDEX_NONE)
			{
				CacheEmitterIndex = EmitterIndex;
			}
			break;
		}
	}
	
	if (CacheEmitterIndex.IsSet())
	{
		TArray<TSharedRef<FNiagaraSimCacheTreeItem>> ComponentTreeItems;
		ComponentTreeItems.Emplace(TreeItem.ToSharedRef());

		SimCache->ForEachEmitterAttribute(
			CacheEmitterIndex.GetValue(),
			[OwningTreeView, &ComponentTreeItems](const FNiagaraSimCacheVariable& Variable)
			{
				if (NiagaraSimCacheViewModelPrivate::bNamespaceGroupingEnabled)
				{
					NiagaraSimCacheViewModelPrivate::NamespaceGroupedTree::AddItemToTree(OwningTreeView, ComponentTreeItems, Variable);
				}
				else
				{
					NiagaraSimCacheViewModelPrivate::TypeGroupedTree::AddItemToTree(OwningTreeView, ComponentTreeItems, Variable);
				}
				return true;
			}
		);
	}
}

void FNiagaraSimCacheViewModel::BuildEntries(TWeakPtr<SNiagaraSimCacheTreeView> OwningTreeView)
{
	RootEntries.Empty();
	BufferEntries.Empty();
	
	const TSharedRef<FNiagaraSimCacheTreeItem> SharedSystemTreeItem = MakeShared<FNiagaraSimCacheTreeItem>(OwningTreeView);
	const TSharedRef<FNiagaraSimCacheOverviewSystemItem> SharedSystemBufferItem = MakeShared<FNiagaraSimCacheOverviewSystemItem>();

	if(IsCacheValid())
	{
		SharedSystemTreeItem->SetDisplayName(LOCTEXT("SystemInstance", "System Instance"));
		SharedSystemBufferItem->SetDisplayName(LOCTEXT("SystemInstance", "System Instance"));
	}
	else
	{
		SharedSystemTreeItem->SetDisplayName(LOCTEXT("InvalidCache", "Invalid Cache"));
		SharedSystemBufferItem->SetDisplayName(LOCTEXT("InvalidCache", "Invalid Cache"));
	}

	RootEntries.Add(SharedSystemTreeItem);
	BufferEntries.Add(SharedSystemBufferItem);

	if(!IsCacheValid())
	{
		UpdateCurrentEntries();
		return;
	}
		
	BuildTreeItemChildren(SharedSystemTreeItem, OwningTreeView);
	
	for(int32 i = 0; i < GetNumEmitterLayouts(); i++)
	{
		const TSharedRef<FNiagaraSimCacheEmitterTreeItem> CurrentEmitterItem = MakeShared<FNiagaraSimCacheEmitterTreeItem>(OwningTreeView);
		const TSharedRef<FNiagaraSimCacheOverviewEmitterItem> CurrentEmitterBufferItem = MakeShared<FNiagaraSimCacheOverviewEmitterItem>();
		const FName EmitterName = GetEmitterLayoutName(i);
		
		CurrentEmitterItem->SetDisplayName(FText::FromName(EmitterName));
		CurrentEmitterBufferItem->SetDisplayName(FText::FromName(EmitterName));
		
		CurrentEmitterItem->SetEmitterName(EmitterName);
		CurrentEmitterBufferItem->SetEmitterName(EmitterName);

		RootEntries.Add(CurrentEmitterItem);
		BufferEntries.Add(CurrentEmitterBufferItem);

		BuildTreeItemChildren(CurrentEmitterItem, OwningTreeView);
	}
	if (SimCache)
	{
		for (const FNiagaraVariableBase& Var : SimCache->GetStoredDataInterfaces())
		{
			const TSharedRef<FNiagaraSimCacheDataInterfaceTreeItem> CurrentDataInterfaceItem = MakeShared<FNiagaraSimCacheDataInterfaceTreeItem>(OwningTreeView);
			const TSharedRef<FNiagaraSimCacheOverviewDataInterfaceItem> CurrentDataInterfaceBufferItem = MakeShared<FNiagaraSimCacheOverviewDataInterfaceItem>();
		
			CurrentDataInterfaceItem->SetDisplayName(FText::FromName(Var.GetName()));
			CurrentDataInterfaceItem->DataInterfaceReference = Var;
			CurrentDataInterfaceBufferItem->SetDisplayName(FText::FromName(Var.GetName()));
			CurrentDataInterfaceBufferItem->DataInterfaceReference = Var;

			RootEntries.Add(CurrentDataInterfaceItem);
			BufferEntries.Add(CurrentDataInterfaceBufferItem);
		}

		if (SimCache->GetDebugData() != nullptr)
		{
			const TSharedRef<FNiagaraSimCacheDebugDataTreeItem> TreeItem = MakeShared<FNiagaraSimCacheDebugDataTreeItem>(OwningTreeView);
			const TSharedRef<FNiagaraSimCacheOverviewDebugDataItem> DataItem = MakeShared<FNiagaraSimCacheOverviewDebugDataItem>();
			const FText DisplayNameText = LOCTEXT("DebugData", "Debug Data");
			TreeItem->SetDisplayName(DisplayNameText);
			DataItem->SetDisplayName(DisplayNameText);

			RootEntries.Add(TreeItem);
			BufferEntries.Add(DataItem);
		}
	}

	UpdateCurrentEntries();
}

void FNiagaraSimCacheViewModel::UpdateCurrentEntries()
{
	SelectedRootEntries.Empty();
	switch (SelectionMode)
	{
		case ESelectionMode::SystemInstance:
			SelectedRootEntries.Add(RootEntries[0]);
			return;

		case ESelectionMode::Emitter:
		{
			TSharedRef<FNiagaraSimCacheTreeItem>* EmitterItem = RootEntries.FindByPredicate(
				[EmitterName=SelectedEmitterName](const TSharedRef<FNiagaraSimCacheTreeItem>& TreeItem)
				{
					return (TreeItem->GetType() == ENiagaraSimCacheOverviewItemType::Emitter) && (TreeItem->GetEmitterName() == EmitterName);
				}
			);

			if (EmitterItem)
			{
				SelectedRootEntries.Add(*EmitterItem);
			}
			return;
		}

		default:
			//-TODO: DO we need to do something here?
			break;
	}
}

TArray<TSharedRef<FNiagaraSimCacheTreeItem>>* FNiagaraSimCacheViewModel::GetSelectedRootEntries()
{
	return &SelectedRootEntries;
}

TArray<TSharedRef<FNiagaraSimCacheOverviewItem>>* FNiagaraSimCacheViewModel::GetBufferEntries()
{
	return &BufferEntries;
}

bool FNiagaraSimCacheViewModel::CanCopyActiveToClipboard() const
{
	return IsCacheValid() && (SelectionMode == ESelectionMode::SystemInstance || SelectionMode == ESelectionMode::Emitter);
}

void FNiagaraSimCacheViewModel::CopyActiveToClipboard() const
{
	if (!CanCopyActiveToClipboard())
	{
		return;
	}

	FString ClipboardString;
	TConstArrayView<FComponentInfo> ComponentInfos = GetSelectedComponentInfos();

	ClipboardString.Append(TEXT("Instance"));
	for ( int iComponent=0; iComponent < ComponentInfos.Num(); ++iComponent)
	{
		ClipboardString.AppendChar(TEXT(','));
		ComponentInfos[iComponent].Name.AppendString(ClipboardString);
	}

	for (int32 iInstance=0; iInstance < NumInstances; ++iInstance)
	{
		ClipboardString.AppendChar(TEXT('\n'));
		ClipboardString.AppendInt(iInstance);

		for (int iComponent = 0; iComponent < ComponentInfos.Num(); ++iComponent)
		{
			ClipboardString.AppendChar(TEXT(','));
			ClipboardString.Append(GetComponentText(ComponentInfos[iComponent].Name, iInstance).ToString());
		}
	}
	FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);
}

bool FNiagaraSimCacheViewModel::CanExportToDisk() const
{
	return IsCacheValid();
}

void FNiagaraSimCacheViewModel::ExportToDisk() const
{
	if (!IsCacheValid())
	{
		return;
	}
	FNiagaraEditorSimCacheUtils::ExportToDisk(SimCache.Get());
}

bool FNiagaraSimCacheViewModel::IsCreateAssetVisible() const
{
	if (SimCache)
	{
		return SimCache->IsAsset() == false;
	}
	return false;
}

void FNiagaraSimCacheViewModel::CreateAsset()
{
	if (SimCache == nullptr)
	{
		return;
	}

	SimCache->SetFlags(RF_Public | RF_Standalone);

	TArray<UObject*> AssetsToSave = { SimCache };
	TArray<UObject*> SavedAssets;
	FEditorFileUtils::SaveAssetsAs(AssetsToSave, SavedAssets);

	if (SavedAssets.Num() == 0 || SavedAssets[0] == nullptr || SavedAssets[0] == AssetsToSave[0])
	{
		SimCache->ClearFlags(RF_Public | RF_Standalone);
		return;
	}

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	AssetEditorSubsystem->CloseAllEditorsForAsset(AssetsToSave[0]);
	AssetEditorSubsystem->OpenEditorForAssets_Advanced(SavedAssets, EToolkitMode::Standalone);
}

bool FNiagaraSimCacheViewModel::CanRemoveDebugData() const
{
	return SimCache && SimCache->GetDebugData() != nullptr;
}

void FNiagaraSimCacheViewModel::RemoveDebugData()
{
	if (SimCache == nullptr)
	{
		return;
	}

	SimCache->RemoveDebugData();
}

void FNiagaraSimCacheViewModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	if(SimCache != nullptr)
	{
		Collector.AddReferencedObject(SimCache);
	}

	if(PreviewComponent != nullptr)
	{
		Collector.AddReferencedObject(PreviewComponent);
	}
}

void FNiagaraSimCacheViewModel::BuildComponentInfos(const FName Name, const UScriptStruct* Struct, TArray<FComponentInfo>& InComponentInfos)
{
	int32 NumProperties = 0;
	for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		++NumProperties;
	}

	for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		const FName PropertyName = NumProperties > 1 ? FName(*FString::Printf(TEXT("%s.%s"), *Name.ToString(), *Property->GetName())) : Name;
		if (Property->IsA(FFloatProperty::StaticClass()))
		{
			FComponentInfo& ComponentInfo = InComponentInfos.AddDefaulted_GetRef();
			ComponentInfo.Name = PropertyName;
			ComponentInfo.ComponentOffset = FoundFloatComponents++;
			ComponentInfo.bIsFloat = true;
		}
		else if (Property->IsA(FUInt16Property::StaticClass()))
		{
			FComponentInfo& ComponentInfo = InComponentInfos.AddDefaulted_GetRef();
			ComponentInfo.Name = PropertyName;
			ComponentInfo.ComponentOffset = FoundHalfComponents++;
			ComponentInfo.bIsHalf = true;
		}
		else if (Property->IsA(FIntProperty::StaticClass()))
		{
			FComponentInfo& ComponentInfo = InComponentInfos.AddDefaulted_GetRef();
			ComponentInfo.Name = PropertyName;
			ComponentInfo.ComponentOffset = FoundInt32Components++;
			ComponentInfo.bIsInt32 = true;
			ComponentInfo.bShowAsBool = (NumProperties == 1) && (Struct == FNiagaraTypeDefinition::GetBoolStruct());
		}
		else if (Property->IsA(FBoolProperty::StaticClass()))
		{
			FComponentInfo& ComponentInfo = InComponentInfos.AddDefaulted_GetRef();
			ComponentInfo.Name = PropertyName;
			ComponentInfo.ComponentOffset = FoundInt32Components++;
			ComponentInfo.bIsInt32 = true;
			ComponentInfo.bShowAsBool = true;
		}
		else if (Property->IsA(FEnumProperty::StaticClass()))
		{
			FComponentInfo& ComponentInfo = InComponentInfos.AddDefaulted_GetRef();
			ComponentInfo.Name = PropertyName;
			ComponentInfo.ComponentOffset = FoundInt32Components++;
			ComponentInfo.bIsInt32 = true;
			ComponentInfo.Enum = CastFieldChecked<FEnumProperty>(Property)->GetEnum();
		}
		else if (Property->IsA(FStructProperty::StaticClass()))
		{
			const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Property);
			BuildComponentInfos(PropertyName, FNiagaraTypeHelper::FindNiagaraFriendlyTopLevelStruct(StructProperty->Struct, ENiagaraStructConversion::Simulation), InComponentInfos);
		}
		else
		{
			// Fail
		}
	}
}

#undef LOCTEXT_NAMESPACE
