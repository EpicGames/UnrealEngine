// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStandaloneScriptViewModel.h"
#include "NiagaraMessageManager.h"
#include "NiagaraMessages.h"
#include "NiagaraMessageUtilities.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraScriptSource.h"
#include "NiagaraParameterDefinitions.h"

#include "Editor.h"
#include "Editor/EditorEngine.h"

FNiagaraStandaloneScriptViewModel::FNiagaraStandaloneScriptViewModel(
	FText DisplayName,
	ENiagaraParameterEditMode InParameterEditMode,
	TSharedPtr<FNiagaraMessageLogViewModel> InNiagaraMessageLogViewModel,
	const FGuid& InSourceScriptObjKey,
	bool bInIsForDataProcessingOnly
)
	: FNiagaraScriptViewModel(DisplayName, InParameterEditMode, bInIsForDataProcessingOnly)
	, NiagaraMessageLogViewModel(InNiagaraMessageLogViewModel)
	, ScriptMessageLogGuidKey(InSourceScriptObjKey)
{
}

void FNiagaraStandaloneScriptViewModel::Initialize(FVersionedNiagaraScript& InScript, const FVersionedNiagaraScript& InSourceScript)
{
	SetScript(InScript);
	SourceScript = InSourceScript;

	SendLastCompileMessages(SourceScript);
}

TSharedPtr<IAssetReferenceFilter> FNiagaraStandaloneScriptViewModel::MakeAssetReferenceFilter() const
{
	if (SourceScript.Script && GEditor)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAsset(SourceScript.Script);
		return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
	}

	return {};
}

FVersionedNiagaraScript FNiagaraStandaloneScriptViewModel::GetStandaloneScript()
{
	checkf(Scripts.Num() == 1, TEXT("StandaloneScriptViewModel did not have exactly one script!"));
	return Scripts[0].Pin();
}

const FVersionedNiagaraScript FNiagaraStandaloneScriptViewModel::GetStandaloneScript() const
{
	return const_cast<FNiagaraStandaloneScriptViewModel*>(this)->GetStandaloneScript();
}

INiagaraParameterDefinitionsSubscriber* FNiagaraStandaloneScriptViewModel::GetParameterDefinitionsSubscriber()
{
	checkf(Scripts.Num() == 1, TEXT("StandaloneScriptViewModel did not have exactly one script!"));
	return &Scripts[0];
}

void FNiagaraStandaloneScriptViewModel::OnVMScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion)
{
	FNiagaraScriptViewModel::OnVMScriptCompiled(InScript, ScriptVersion);
	SendLastCompileMessages(FVersionedNiagaraScript(InScript, ScriptVersion));
}

void FNiagaraStandaloneScriptViewModel::SendLastCompileMessages(const FVersionedNiagaraScript& InScript)
{
	FNiagaraMessageManager* MessageManager = FNiagaraMessageManager::Get();
	MessageManager->ClearAssetMessagesForTopic(ScriptMessageLogGuidKey, FNiagaraMessageTopics::CompilerTopicName);

	int32 ErrorCount = 0;
	int32 WarningCount = 0;

	const TArray<FNiagaraCompileEvent>& CurrentCompileEvents = InScript.Script->GetVMExecutableData().LastCompileEvents;

	// Iterate from back to front to avoid reordering the events when they are queued
	for (int i = CurrentCompileEvents.Num() - 1; i >= 0; --i)
	{
		const FNiagaraCompileEvent& CompileEvent = CurrentCompileEvents[i];
		if (CompileEvent.Severity == FNiagaraCompileEventSeverity::Error)
		{
			ErrorCount++;
		}
		else if (CompileEvent.Severity == FNiagaraCompileEventSeverity::Warning)
		{
			WarningCount++;
		}

		MessageManager->AddMessageJob(MakeUnique<FNiagaraMessageJobCompileEvent>(CompileEvent, TWeakObjectPtr<const UNiagaraScript>(InScript.Script), InScript.Version, TOptional<const FString>(), SourceScript.Script->GetPathName()), ScriptMessageLogGuidKey);
	}

	const FText PostCompileSummaryText = FNiagaraMessageUtilities::MakePostCompileSummaryText(FText::FromString("Script"), GetLatestCompileStatus(), WarningCount, ErrorCount);
	TSharedRef<const FNiagaraMessageText> PostCompileSummaryMessage =  MakeShared<FNiagaraMessageText>(PostCompileSummaryText, EMessageSeverity::Info, FNiagaraMessageTopics::CompilerTopicName);
	MessageManager->AddMessage(PostCompileSummaryMessage, ScriptMessageLogGuidKey);
}
