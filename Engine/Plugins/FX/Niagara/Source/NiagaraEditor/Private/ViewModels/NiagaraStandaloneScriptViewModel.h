// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/NiagaraParameterEditMode.h"
#include "ViewModels/NiagaraParameterDefinitionsSubscriberViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"

class FNiagaraMessageLogViewModel;
class UNiagaraScript;
class UNiagaraScriptVariable;


class FNiagaraStandaloneScriptViewModel : public FNiagaraScriptViewModel
{
public:
	NIAGARAEDITOR_API FNiagaraStandaloneScriptViewModel(
		FText DisplayName,
		ENiagaraParameterEditMode InParameterEditMode,
		TSharedPtr<FNiagaraMessageLogViewModel> InNiagaraMessageLogViewModel,
		const FGuid& InMessageLogGuidKey,
		bool bInIsForDataProcessingOnly
	);

	NIAGARAEDITOR_API void Initialize(FVersionedNiagaraScript& InScript, const FVersionedNiagaraScript& InSourceScript);

	TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter() const override;

	//~ Begin INiagaraParameterDefinitionsSubscriberViewModel Interface
protected:
	NIAGARAEDITOR_API virtual INiagaraParameterDefinitionsSubscriber* GetParameterDefinitionsSubscriber() override;
	//~ End NiagaraParameterDefinitionsSubscriberViewModel Interface

public:
	NIAGARAEDITOR_API virtual FVersionedNiagaraScript GetStandaloneScript() override;
	NIAGARAEDITOR_API const FVersionedNiagaraScript GetStandaloneScript() const;

private:
	NIAGARAEDITOR_API virtual void OnVMScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion) override;

	/** Sends messages to FNiagaraMessageManager for all compile events from the last compile. */
	void SendLastCompileMessages(const FVersionedNiagaraScript& InScript);

	TSharedPtr<FNiagaraMessageLogViewModel> NiagaraMessageLogViewModel;
	FVersionedNiagaraScript SourceScript;
	const FGuid ScriptMessageLogGuidKey;
};
