// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraStatelessEmitterTemplateViewModel.h"

#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "Stateless/NiagaraStatelessEmitterTranslator.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessModule.h"

#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "NiagaraStatelessEmitterTemplateViewModel"

FNiagaraStatelessEmitterTemplateViewModel::FNiagaraStatelessEmitterTemplateViewModel(UNiagaraStatelessEmitterTemplate* InEmitterTemplate)
	: EmitterTemplate(InEmitterTemplate)
{
	FObjectDuplicationParameters DuplicateParameters(InEmitterTemplate, GetTransientPackage());
	DuplicateParameters.FlagMask = InEmitterTemplate->GetFlags() & ~(RF_Standalone | RF_Public);
	DuplicateParameters.ApplyFlags = RF_Transactional | RF_Transient;
	EmitterTemplateForEdit.Reset(Cast<UNiagaraStatelessEmitterTemplate>(StaticDuplicateObjectEx(DuplicateParameters)));

	FEditorDelegates::PostUndoRedo.AddRaw(this, &FNiagaraStatelessEmitterTemplateViewModel::PostUndoRedo);

	BuildOutputVariables();
}

FNiagaraStatelessEmitterTemplateViewModel::~FNiagaraStatelessEmitterTemplateViewModel()
{
	FEditorDelegates::PostUndoRedo.RemoveAll(this);
}

void FNiagaraStatelessEmitterTemplateViewModel::PostUndoRedo()
{
	OnTemplateChangedDelegate.Broadcast();
}

void FNiagaraStatelessEmitterTemplateViewModel::NotifyPreChange(FProperty* PropertyAboutToChange)
{
}

void FNiagaraStatelessEmitterTemplateViewModel::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	EmitterTemplate->ModifyTemplate(EmitterTemplateForEdit.Get());

	BuildOutputVariables();

	OnTemplateChangedDelegate.Broadcast();
}

TConstArrayView<FNiagaraStatelessEmitterTemplateViewModel::FOutputVariable> FNiagaraStatelessEmitterTemplateViewModel::GetOututputVariables() const
{
	return OutputVariables;
}

TPair<ENiagaraStatelessFeatureMask, ENiagaraStatelessFeatureMask> FNiagaraStatelessEmitterTemplateViewModel::GetFeatureMaskRange() const
{
	ENiagaraStatelessFeatureMask MinMask = ENiagaraStatelessFeatureMask::All;
	ENiagaraStatelessFeatureMask MaxMask = ENiagaraStatelessFeatureMask::All;

	if (EmitterTemplate->GetSimulationShader().IsNull())
	{
		MinMask &= ~ENiagaraStatelessFeatureMask::ExecuteGPU;
		MaxMask &= ~ENiagaraStatelessFeatureMask::ExecuteGPU;
	}

	for (UClass* ModuleClass : EmitterTemplate->GetModules())
	{
		UNiagaraStatelessModule* Module = ModuleClass ? Cast<UNiagaraStatelessModule>(ModuleClass->GetDefaultObject()) : nullptr;
		if (Module == nullptr)
		{
			continue;
		}

		const ENiagaraStatelessFeatureMask ModuleMask = Module->GetFeatureMask();
		MinMask &= ModuleMask;
		if ( Module->CanDisableModule() == false )
		{
			MaxMask &= ModuleMask;
		}
	}

	return {MinMask, MaxMask};
}

FString FNiagaraStatelessEmitterTemplateViewModel::GenerateComputeTemplateHLSL() const
{
	FString HlslOutput;
	FNiagaraStatelessEmitterTranslator::TranslateToCompute(HlslOutput, EmitterTemplate.Get());
	return MoveTemp(HlslOutput);
}

void FNiagaraStatelessEmitterTemplateViewModel::BuildOutputVariables()
{
	auto FindOrAddOutputVariable =
		[this](const FNiagaraVariableBase& Variable) -> FOutputVariable&
		{
			for (FOutputVariable& Existing : OutputVariables)
			{
				if (Existing.Variable == Variable)
				{
					return Existing;
				}
			}
			FOutputVariable& NewOutput = OutputVariables.AddDefaulted_GetRef();
			NewOutput.Variable = Variable;
			return NewOutput;
		};

	// Gather output variables
	OutputVariables.Reset();
	for (const FNiagaraVariableBase& Variable : EmitterTemplate->GetImplicitVariables())
	{
		FOutputVariable& Output = FindOrAddOutputVariable(Variable);
		Output.bIsImplicit = true;
	}

	for (const FNiagaraVariableBase& Variable : EmitterTemplate->GetModuleVariables())
	{
		FOutputVariable& Output = FindOrAddOutputVariable(Variable);
		Output.bIsModule = true;
	}

	for (const FNiagaraVariableBase& Variable : EmitterTemplate->GetShaderOutputVariables())
	{
		FOutputVariable& Output = FindOrAddOutputVariable(Variable);
		Output.bIsShader = true;
	}

	Algo::Sort(
		OutputVariables,
		[](const FOutputVariable& Lhs, const FOutputVariable& Rhs)
		{
			return Lhs.Variable.GetName().LexicalLess(Rhs.Variable.GetName());
		}
	);
}

#undef LOCTEXT_NAMESPACE
