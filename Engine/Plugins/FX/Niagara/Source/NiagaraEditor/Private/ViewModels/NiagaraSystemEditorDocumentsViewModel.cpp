// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraSystemEditorDocumentsViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraSystemScriptViewModel.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"
#include "NiagaraSystem.h"
#include "NiagaraScriptSource.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraEditorModule.h"
#include "NiagaraClipboard.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraScriptFactoryNew.h"
#include "NiagaraGraph.h"

#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Misc/PackageName.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Misc/MessageDialog.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/PlatformApplicationMisc.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "NiagaraScriptSource.h"
#include "Widgets/NiagaraScratchScriptEditor.h"
#include "Toolkits/NiagaraSystemToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSystemEditorDocumentsViewModel)

#define LOCTEXT_NAMESPACE "NiagaraScratchPadViewModel"

void UNiagaraSystemEditorDocumentsViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModelWeak = InSystemViewModel;
}

void UNiagaraSystemEditorDocumentsViewModel::Finalize()
{
	DocumentManager.Reset();
	TabManager.Reset();
	ActiveDocumentTabScriptViewModel.Reset();
}

TArray<UNiagaraGraph*> UNiagaraSystemEditorDocumentsViewModel::GetAllGraphsForActiveScriptDocument()
{
	if (ActiveDocumentTabScriptViewModel.IsValid())
	{
		return ActiveDocumentTabScriptViewModel->GetEditableGraphs();
	}
	else if (ActiveDocumentTab.IsValid())
	{
		if (ActiveDocumentTab.Pin()->GetLayoutIdentifier().TabType != PrimaryDocumentTabId)
		{
			ActiveDocumentTabScriptViewModel = GetActiveScratchPadViewModelIfSet();

			if (ActiveDocumentTabScriptViewModel.IsValid())
			{
				return ActiveDocumentTabScriptViewModel->GetEditableGraphs();
			}
		}
	}
	return TArray<UNiagaraGraph*>();
}

TArray<UNiagaraGraph*> UNiagaraSystemEditorDocumentsViewModel::GetEditableGraphsForActiveScriptDocument()
{
	if (ActiveDocumentTabScriptViewModel.IsValid())
	{
		return ActiveDocumentTabScriptViewModel->GetEditableGraphs();
	}
	else if (ActiveDocumentTab.IsValid())
	{
		if (ActiveDocumentTab.Pin()->GetLayoutIdentifier().TabType != PrimaryDocumentTabId)
		{
			ActiveDocumentTabScriptViewModel = GetActiveScratchPadViewModelIfSet();
			
			if (ActiveDocumentTabScriptViewModel.IsValid())
			{
				return ActiveDocumentTabScriptViewModel->GetEditableGraphs();
			}
		}	
	}
	
	return TArray<UNiagaraGraph*>();
}

TArray<UNiagaraGraph*> UNiagaraSystemEditorDocumentsViewModel::GetAllGraphsForPrimaryDocument()
{
	TArray<UNiagaraGraph*> OutGraphs;

	// Helper lambda to null check graph weak object ptrs before adding them to the retval array.
	auto AddToOutGraphsChecked = [&OutGraphs](const TWeakObjectPtr<UNiagaraGraph>& WeakGraph) {
		UNiagaraGraph* Graph = WeakGraph.Get();
		if (Graph == nullptr)
		{
			ensureMsgf(false, TEXT("Encountered null graph when gathering editable graphs for system parameter panel viewmodel!"));
			return;
		}
		OutGraphs.Add(Graph);
	};
	TSharedRef<FNiagaraSystemViewModel> SystemViewModel = GetSystemViewModel();
	if (SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		TArray<UNiagaraGraph*> Graphs;
		AddToOutGraphsChecked(SystemViewModel->GetSystemScriptViewModel()->GetGraphViewModel()->GetGraph());

		const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& EmitterHandleViewModels = SystemViewModel->GetEmitterHandleViewModels();
		for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
		{
			FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
			if (EmitterHandle == nullptr)
			{
				continue;
			}
			UNiagaraGraph* Graph = Cast<UNiagaraScriptSource>(EmitterHandle->GetEmitterData()->GraphSource)->NodeGraph;
			if (Graph)
			{
				OutGraphs.Add(Graph);
			}
		}
	}
	else
	{
		FNiagaraEmitterHandle* EmitterHandle = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterHandle();
		if (EmitterHandle != nullptr)
		{
			OutGraphs.Add(Cast<UNiagaraScriptSource>(EmitterHandle->GetEmitterData()->GraphSource)->NodeGraph);
		}
	}
	return OutGraphs;
}

TArray<UNiagaraGraph*> UNiagaraSystemEditorDocumentsViewModel::GetEditableGraphsForPrimaryDocument()
{
	
	TArray<UNiagaraGraph*> EditableGraphs;

	// Helper lambda to null check graph weak object ptrs before adding them to the retval array.
	auto AddToEditableGraphChecked = [&EditableGraphs](const TWeakObjectPtr<UNiagaraGraph>& WeakGraph) {
		UNiagaraGraph* Graph = WeakGraph.Get();
		if (Graph == nullptr)
		{
			ensureMsgf(false, TEXT("Encountered null graph when gathering editable graphs for system parameter panel viewmodel!"));
			return;
		}
		EditableGraphs.Add(Graph);
	};

	TSharedRef<FNiagaraSystemViewModel> SystemViewModel = GetSystemViewModel();
	if (SystemViewModel->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset && ensureMsgf(SystemGraphSelectionVMWeak.IsValid(), TEXT("SystemGraphSelectionViewModel was null for System edit mode!")))
	{
		for (const TWeakObjectPtr<UNiagaraGraph>& WeakGraph : SystemGraphSelectionVMWeak.Pin()->GetSelectedEmitterScriptGraphs())
		{
			AddToEditableGraphChecked(WeakGraph);
		}
		AddToEditableGraphChecked(SystemViewModel->GetSystemScriptViewModel()->GetGraphViewModel()->GetGraph());
	}
	else
	{
		if (FVersionedNiagaraEmitterData* EmitterData = SystemViewModel->GetEmitterHandleViewModels()[0]->GetEmitterHandle()->GetEmitterData())
		{
			EditableGraphs.Add(CastChecked<UNiagaraScriptSource>(EmitterData->GraphSource)->NodeGraph);
		}
	}

	return EditableGraphs;
}

void UNiagaraSystemEditorDocumentsViewModel::OpenChildScript(UEdGraph* InGraph)
{
	TArray< TSharedPtr<SDockTab> > Results;
	if (!FindOpenTabsContainingDocument(InGraph, Results))
	{
		OpenDocument(InGraph, FDocumentTracker::EOpenDocumentCause::ForceOpenNewDocument);
	}
	else if (Results.Num() > 0)
	{
		if (Results[0].IsValid() && TabManager.IsValid())
		{
			TabManager->DrawAttention(Results[0].ToSharedRef());
			Results[0]->ActivateInParent(ETabActivationCause::SetDirectly);
			SetActiveDocumentTab(Results[0]);
		}
	}
}

void UNiagaraSystemEditorDocumentsViewModel::SwapEditableScripts(TSharedPtr < class FNiagaraScratchPadScriptViewModel> OldScriptViewModel, TSharedPtr < class FNiagaraScratchPadScriptViewModel> NewScriptViewModel)
{
	// In the event of an inheritance update, we might be called upon to do a hotswap of the currently editable scratch scripts,
	// which are never overwritten by the parent. However, they were on graphs that need to be propagated to the 
	// documents that are currently open if already in use for editing.
	if (OldScriptViewModel.IsValid())
	{
		TArray< TSharedPtr<SDockTab> > Results;
		UEdGraph* OldGraph = OldScriptViewModel->GetGraphViewModel()->GetGraph();
		if (FindOpenTabsContainingDocument(OldGraph, Results))
		{
			for (TSharedPtr<SDockTab> Tab : Results)
			{
				if (Tab.IsValid() && Tab->GetLayoutIdentifier().TabType == TEXT("Document"))
				{
					TSharedRef<SNiagaraScratchPadScriptEditor> GraphEditor = StaticCastSharedRef<SNiagaraScratchPadScriptEditor>(Tab->GetContent());
					GraphEditor->SetViewModel(NewScriptViewModel);

					if (ActiveDocumentTabScriptViewModel == OldScriptViewModel)
					{
						ActiveDocumentTabScriptViewModel = NewScriptViewModel;
					}
				}
			}
		}
	}

}

void UNiagaraSystemEditorDocumentsViewModel::CleanInvalidTabs() const
{
	if(DocumentManager.IsValid())
	{
		DocumentManager->CleanInvalidTabs();
	}
}


void UNiagaraSystemEditorDocumentsViewModel::CloseChildScript(UEdGraph* InGraph)
{
	CloseDocumentTab(InGraph);
}


TSharedRef<FNiagaraSystemViewModel> UNiagaraSystemEditorDocumentsViewModel::GetSystemViewModel() const
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
	checkf(SystemViewModel.IsValid(), TEXT("SystemViewModel destroyed before system editor document view model."));
	return SystemViewModel.ToSharedRef();
}

void UNiagaraSystemEditorDocumentsViewModel::InitializePreTabManager(TSharedPtr<FNiagaraSystemToolkit> InToolkit)
{
	DocumentManager = MakeShareable(new FDocumentTracker);
	DocumentManager->Initialize(InToolkit);


	// Register the document factories
	{

		TSharedRef<FDocumentTabFactory> GraphEditorFactory = MakeShared<FDocumentTabFactory_NiagaraScratchPad>(
			InToolkit,
			FDocumentTabFactory_NiagaraScratchPad::FOnCreateGraphEditorWidget::CreateUObject(this, &UNiagaraSystemEditorDocumentsViewModel::CreateGraphEditorWidget)
		);

		// Also store off a reference to the grapheditor factory so we can find all the tabs spawned by it later.
		GraphEditorTabFactoryPtr = GraphEditorFactory;
		DocumentManager->RegisterDocumentFactory(GraphEditorFactory);
	}
}

void UNiagaraSystemEditorDocumentsViewModel::InitializePostTabManager(TSharedPtr<FNiagaraSystemToolkit> InToolkit)
{
	TabManager = InToolkit->GetTabManager();

	if (DocumentManager.IsValid())
	{
		DocumentManager->SetTabManager(InToolkit->GetTabManager().ToSharedRef());
	}
	SystemGraphSelectionVMWeak = InToolkit->GetSystemGraphSelectionViewModel();
}

void UNiagaraSystemEditorDocumentsViewModel::DrawAttentionToPrimaryDocument()
{
	// Flash the system overivew in the UI
	TSharedPtr<SDockTab>  ActiveTab = GetActiveDocumentTab().Pin();
	if (ActiveTab.IsValid())
	{
		if (ActiveTab->GetLayoutIdentifier().TabType != PrimaryDocumentTabId && TabManager.IsValid())
		{
			TSharedPtr<SDockTab> FoundPrimaryDocumentTab = TabManager->FindExistingLiveTab(PrimaryDocumentTabId);
			if (FoundPrimaryDocumentTab.IsValid())
			{
				FoundPrimaryDocumentTab->DrawAttention();
				FoundPrimaryDocumentTab->ActivateInParent(ETabActivationCause::SetDirectly);
			}
		}
	}
}


TSharedRef<SNiagaraScratchPadScriptEditor> UNiagaraSystemEditorDocumentsViewModel::CreateGraphEditorWidget(TSharedRef<FTabInfo> InTabInfo, UEdGraph* InGraph)
{
	TSharedRef<FNiagaraScratchPadScriptViewModel> GraphViewModel = MakeShared<FNiagaraScratchPadScriptViewModel>(false);

	for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScratchPadViewModel : GetSystemViewModel()->GetScriptScratchPadViewModel()->GetScriptViewModels())
	{
		if (ScratchPadViewModel->GetGraphViewModel()->GetGraph() == InGraph)
		{
			GraphViewModel = ScratchPadViewModel;
			break;
		}
	}

	TSharedRef<SNiagaraScratchPadScriptEditor> Editor = SNew(SNiagaraScratchPadScriptEditor, GraphViewModel);
	// We add a tag here to identify what the tab contains. The tab type doesn't work for that purpose as it can be invalid once a tab is activated
	Editor->SetTag(FName("ScratchPadScriptEditor"));
	return Editor;
}



TSharedPtr<SDockTab> UNiagaraSystemEditorDocumentsViewModel::OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);
	if (DocumentManager.IsValid())
	{
		return DocumentManager->OpenDocument(Payload, Cause);
	}
	else
	{
		return nullptr;
	}
}

void UNiagaraSystemEditorDocumentsViewModel::NavigateTab(FDocumentTracker::EOpenDocumentCause InCause)
{
	OpenDocument(nullptr, InCause);
}

void UNiagaraSystemEditorDocumentsViewModel::CloseDocumentTab(const UObject* DocumentID)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);
	if (DocumentManager.IsValid())
	{
		DocumentManager->CloseTab(Payload);
		DocumentManager->CleanInvalidTabs();
	}
}

// Finds any open tabs containing the specified document and adds them to the specified array; returns true if at least one is found
bool UNiagaraSystemEditorDocumentsViewModel::FindOpenTabsContainingDocument(const UObject* DocumentID, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results)
{
	int32 StartingCount = Results.Num();

	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);
	if (DocumentManager.IsValid())
	{
		DocumentManager->FindMatchingTabs(Payload, /*inout*/ Results);
	}

	// Did we add anything new?
	return (StartingCount != Results.Num());
}

void UNiagaraSystemEditorDocumentsViewModel::SetActiveDocumentTab(TSharedPtr<SDockTab> Tab)
{
	if(ActiveDocumentTab != Tab)
	{
		ActiveDocumentTab = Tab;
		ActiveDocumentTabScriptViewModel = GetActiveScratchPadViewModelIfSet();

		ActiveDocChangedDelegate.Broadcast(Tab);

		// We need to update the parameter panel view model with new parameters potentially
		INiagaraParameterPanelViewModel* PanelVM = GetSystemViewModel()->GetParameterPanelViewModel();
		if (PanelVM)
		{
			PanelVM->RefreshNextTick();
		}
	}
}

bool UNiagaraSystemEditorDocumentsViewModel::IsPrimaryDocumentActive() const
{
	TSharedPtr<SDockTab> Tab = ActiveDocumentTab.Pin();
	if (Tab.IsValid() && (Tab->GetLayoutIdentifier().TabType == PrimaryDocumentTabId || PrimaryDocumentTabId.IsNone()))
	{
		return true;
	}
	return false;
}


TSharedPtr<FNiagaraScratchPadScriptViewModel> UNiagaraSystemEditorDocumentsViewModel::GetScratchPadViewModelFromGraph(FNiagaraSystemViewModel* InSysViewModel, UEdGraph* InTargetGraph)
{
	if (InSysViewModel && InTargetGraph)
	{
		UNiagaraScript* Script = InTargetGraph->GetTypedOuter<UNiagaraScript>();
		UNiagaraScratchPadViewModel* ScratchViewModel = InSysViewModel->GetScriptScratchPadViewModel();
		if (ScratchViewModel && Script)
		{
			for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScratchScriptViewModel : ScratchViewModel->GetScriptViewModels())
			{
				if(ScratchScriptViewModel->IsValid() == false)
				{
					continue;
				}
				
				UNiagaraScript* EditScript = ScratchScriptViewModel->GetGraphViewModel()->GetScriptSource()->GetTypedOuter<UNiagaraScript>();
				if (EditScript == Script)
				{
					return ScratchScriptViewModel;
				}
				else if (ScratchScriptViewModel->GetOriginalScript() == Script)
				{
					return ScratchScriptViewModel;
				}
			}
		}
	}
	return nullptr;
}


TSharedPtr<FNiagaraScratchPadScriptViewModel> UNiagaraSystemEditorDocumentsViewModel::GetActiveScratchPadViewModelIfSet()
{
	TSharedPtr<SDockTab> Tab = ActiveDocumentTab.Pin();
	// When this is called during tab construction, we can't rely on the TabID to be initialized to check if this is a scratch pad tab
	if (Tab.IsValid() && Tab->GetContent()->GetTag() == "ScratchPadScriptEditor")
	{
		TSharedRef<SNiagaraScratchPadScriptEditor> GraphEditor = StaticCastSharedRef<SNiagaraScratchPadScriptEditor>(Tab->GetContent());
		if (GraphEditor->GetGraphEditor())
		{
			TSharedPtr<FNiagaraSystemViewModel> SystemVM = SystemViewModelWeak.Pin();
			return GetScratchPadViewModelFromGraph(SystemVM.Get(), GraphEditor->GetGraphEditor()->GetCurrentGraph());
		}
	}

	return nullptr;
}


#undef LOCTEXT_NAMESPACE
