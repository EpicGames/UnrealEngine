// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraSystemGraphSelectionViewModel.h"

#include "Misc/Guid.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptSource.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"


void FNiagaraSystemGraphSelectionViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModelWeak = InSystemViewModel;
	InSystemViewModel->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().AddSP(this, &FNiagaraSystemGraphSelectionViewModel::RefreshSelectedEmitterScriptGraphs);
	InSystemViewModel->OnEmitterHandleViewModelsChanged().AddSP(this, &FNiagaraSystemGraphSelectionViewModel::RefreshSelectedEmitterScriptGraphs);
	RefreshSelectedEmitterScriptGraphs();
}

const TArray<TWeakObjectPtr<UNiagaraGraph>> FNiagaraSystemGraphSelectionViewModel::GetSelectedEmitterScriptGraphs() const
{
	return SelectedEmitterScriptGraphs; 
}

void FNiagaraSystemGraphSelectionViewModel::RefreshSelectedEmitterScriptGraphs()
{
	SelectedEmitterScriptGraphs.Reset();

	const TArray<FGuid>& SelectedEmitterHandleIds = GetSystemSelectionViewModel()->GetSelectedEmitterHandleIds();

	if (SelectedEmitterHandleIds.Num() > 0)
	{
		const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& EmitterHandleViewModels = GetSystemViewModel()->GetEmitterHandleViewModels();
		for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
		{
			if (SelectedEmitterHandleIds.Contains(EmitterHandleViewModel->GetId()))
			{
				if (FVersionedNiagaraEmitterData* EmitterData = EmitterHandleViewModel->GetEmitterHandle()->GetEmitterData())
				{
					SelectedEmitterScriptGraphs.Add(static_cast<UNiagaraScriptSource*>(EmitterData->GraphSource)->NodeGraph);
				}
			}
		}
	}

	OnSelectedEmitterScriptGraphsRefreshedDelegate.Broadcast();
}

TSharedRef<FNiagaraSystemViewModel> FNiagaraSystemGraphSelectionViewModel::GetSystemViewModel()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
	checkf(SystemViewModel.IsValid(), TEXT("Owning system viewmodel destroyed before system graph selection viewmodel."));
	return SystemViewModel.ToSharedRef();
}

UNiagaraSystemSelectionViewModel* FNiagaraSystemGraphSelectionViewModel::GetSystemSelectionViewModel()
{
	return GetSystemViewModel()->GetSelectionViewModel();
}
