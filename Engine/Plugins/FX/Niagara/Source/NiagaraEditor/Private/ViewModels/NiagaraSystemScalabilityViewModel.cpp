// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraSystemScalabilityViewModel.h"

#include "ViewModels/NiagaraSystemViewModel.h"
#include "NiagaraSystem.h"
#include "ISequencerModule.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSystemScalabilityViewModel)

#define LOCTEXT_NAMESPACE "NiagaraSystemScalabilityViewModel"

UNiagaraSystemScalabilityViewModel::UNiagaraSystemScalabilityViewModel()
	: SystemViewModel(nullptr)
{
}

void UNiagaraSystemScalabilityViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModel = InSystemViewModel;

	PreviewPlatforms = MakeShared<FNiagaraPlatformSet>(FNiagaraPlatformSet::CreateQualityLevelMask(3));
}

bool UNiagaraSystemScalabilityViewModel::IsValid() const
{
	return SystemViewModel != nullptr;
}

bool UNiagaraSystemScalabilityViewModel::IsActive() const
{
	return GetSystemViewModel().Pin()->GetWorkflowMode() == FNiagaraSystemToolkitModeBase::ScalabilityModeName;
}

void UNiagaraSystemScalabilityViewModel::UpdatePreviewDeviceProfile(UDeviceProfile* DeviceProfile)
{
	if(DeviceProfile)
	{
		PreviewDeviceProfile = DeviceProfile;
	}
	else
	{
		PreviewDeviceProfile.Reset();
	}

	FNiagaraPlatformSet::InvalidateCachedData();
	GetSystemViewModel().Pin()->GetSystem().UpdateScalability();

	//Ensure we properly reset the preview system.
	GetSystemViewModel().Pin()->ResetSystem(FNiagaraSystemViewModel::ETimeResetMode::AllowResetTime, FNiagaraSystemViewModel::EMultiResetMode::ResetThisInstance, FNiagaraSystemViewModel::EReinitMode::ReinitializeSystem);
}

void UNiagaraSystemScalabilityViewModel::UpdatePreviewQualityLevel(int32 QualityLevel)
{
	PreviewPlatforms->QualityLevelMask = 0;
	PreviewPlatforms->SetEnabledForEffectQuality(QualityLevel, true);

	FNiagaraPlatformSet::InvalidateCachedData();
	GetSystemViewModel().Pin()->GetSystem().UpdateScalability();

	//Ensure we properly reset the preview system.
	GetSystemViewModel().Pin()->ResetSystem(FNiagaraSystemViewModel::ETimeResetMode::AllowResetTime, FNiagaraSystemViewModel::EMultiResetMode::ResetThisInstance, FNiagaraSystemViewModel::EReinitMode::ReinitializeSystem);
}

bool UNiagaraSystemScalabilityViewModel::IsPlatformActive(const FNiagaraPlatformSet& PlatformSet)
{
	TOptional<TObjectPtr<UDeviceProfile>> DeviceProfile = GetPreviewDeviceProfile();
	UDeviceProfileManager& DeviceProfileManager = UDeviceProfileManager::Get();
	int32 PreviewQualityLevel = FNiagaraPlatformSet::QualityLevelFromMask(PreviewPlatforms->QualityLevelMask);
	return PlatformSet.IsEnabled(DeviceProfile.Get(DeviceProfileManager.GetActiveProfile()), PreviewQualityLevel, true).bIsActive;
}

// void UNiagaraSystemScalabilityViewModel::NavigateToScalabilityProperty(UObject* Object, FName PropertyName)
// {
// 	TArray<UNiagaraOverviewNode*> OverviewNodes;
// 	GetSystemViewModel().Pin()->GetOverviewGraphViewModel()->GetGraph()->GetNodesOfClass<UNiagaraOverviewNode>(OverviewNodes);
// 	
// 	for(UNiagaraOverviewNode* Node : OverviewNodes)
// 	{
// 		if(FNiagaraEmitterHandle* EmitterHandle = Node->TryGetEmitterHandle())
// 		{
// 			if(Object == EmitterHandle->GetInstance())
// 			{
// 				GetSystemViewModel().Pin()->GetOverviewGraphViewModel()->GetNodeSelection()->SetSelectedObject(Node);
// 				break;
// 			}
// 		}
// 		else if(Object == Node->GetOwningSystem())
// 		{
// 			GetSystemViewModel().Pin()->GetOverviewGraphViewModel()->GetNodeSelection()->SetSelectedObject(Node);
// 			break;
// 		}
// 	}
//
// 	ScalabilityPropertySelectedDelegate.Broadcast(PropertyName);
// }

#undef LOCTEXT_NAMESPACE // NiagaraSystemScalabilityViewModel

