// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemScriptViewModel.h"

#include "NiagaraEmitterHandle.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeInput.h"
#include "NiagaraScriptInputCollectionViewModel.h"
#include "NiagaraSystem.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"

#include "Editor.h"
#include "Editor/EditorEngine.h"

FNiagaraSystemScriptViewModel::FNiagaraSystemScriptViewModel(bool bInIsForDataProcessingOnly)
	: FNiagaraScriptViewModel(NSLOCTEXT("SystemScriptViewModel", "GraphName", "System"), ENiagaraParameterEditMode::EditAll, bInIsForDataProcessingOnly)
{
}

void FNiagaraSystemScriptViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModelWeak = InSystemViewModel;

	UNiagaraSystem* System = GetSystem();
	if (ensure(System))
	{
		SetScript(FVersionedNiagaraScript(System->GetSystemSpawnScript()));
		System->OnSystemCompiled().AddSP(this, &FNiagaraSystemScriptViewModel::OnSystemVMCompiled);
	}
}

FNiagaraSystemScriptViewModel::~FNiagaraSystemScriptViewModel()
{
	if (UNiagaraSystem* System = GetSystem())
	{
		System->OnSystemCompiled().RemoveAll(this);
	}
}

TSharedPtr<IAssetReferenceFilter> FNiagaraSystemScriptViewModel::MakeAssetReferenceFilter() const
{
	if (const UNiagaraSystem* System = GetSystem())
	{
		if (GEditor)
		{
			FAssetReferenceFilterContext AssetReferenceFilterContext;
			AssetReferenceFilterContext.AddReferencingAsset(System);
			return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
		}
	}

	return {};
}

void FNiagaraSystemScriptViewModel::OnSystemVMCompiled(UNiagaraSystem* InSystem)
{
	{
		UNiagaraSystem* System = GetSystem();
		if (!ensure(System) || InSystem != System)
		{
			return;
		}
	}

	TArray<ENiagaraScriptCompileStatus> InCompileStatuses;
	TArray<FString> InCompileErrors;
	TArray<FString> InCompilePaths;
	TArray<TPair<ENiagaraScriptUsage, int32> > InUsages;

	ENiagaraScriptCompileStatus AggregateStatus = ENiagaraScriptCompileStatus::NCS_UpToDate;
	FString AggregateErrors;

	TArray<UNiagaraScript*> SystemScripts;
	TArray<bool> ScriptsEnabled;
	SystemScripts.Add(InSystem->GetSystemSpawnScript());
	SystemScripts.Add(InSystem->GetSystemUpdateScript());
	ScriptsEnabled.Add(true);
	ScriptsEnabled.Add(true);
	
	for (const FNiagaraEmitterHandle& Handle : InSystem->GetEmitterHandles())
	{
		//-TODO:Stateless: Do we need stateless support here?
		FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData();
		if (EmitterData == nullptr)
		{
			continue;
		}
		int32 NumScripts = SystemScripts.Num();
		EmitterData->GetScripts(SystemScripts, true);
		for (; NumScripts < SystemScripts.Num(); NumScripts++)
		{
			if (Handle.GetIsEnabled())
			{
				ScriptsEnabled.Add(true);
			}
			else
			{
				ScriptsEnabled.Add(false);
			}
		}
	}

	check(ScriptsEnabled.Num() == SystemScripts.Num());

	int32 EventsFound = 0;
	for (int32 i = 0; i < SystemScripts.Num(); i++)
	{
		UNiagaraScript* Script = SystemScripts[i];
		if (Script != nullptr && Script->GetVMExecutableData().IsValid() && ScriptsEnabled[i])
		{
			InCompileStatuses.Add(Script->GetVMExecutableData().LastCompileStatus);
			InCompileErrors.Add(Script->GetVMExecutableData().ErrorMsg);
			InCompilePaths.Add(Script->GetPathName());

			if (Script->GetUsage() == ENiagaraScriptUsage::ParticleEventScript)
			{
				InUsages.Add(TPair<ENiagaraScriptUsage, int32>(Script->GetUsage(), EventsFound));
				EventsFound++;
			}
			else
			{
				InUsages.Add(TPair<ENiagaraScriptUsage, int32>(Script->GetUsage(), 0));
			}
		}
		else if (Script != nullptr && ScriptsEnabled[i] == false)
		{
			InCompileStatuses.Add(ENiagaraScriptCompileStatus::NCS_UpToDate);
			InCompileErrors.Add(FString());
			InCompilePaths.Add(Script->GetPathName());

			if (Script->GetUsage() == ENiagaraScriptUsage::ParticleEventScript)
			{
				InUsages.Add(TPair<ENiagaraScriptUsage, int32>(Script->GetUsage(), EventsFound));
				EventsFound++;
			}
			else
			{
				InUsages.Add(TPair<ENiagaraScriptUsage, int32>(Script->GetUsage(), 0));
			}
		}
		else
		{
			InCompileStatuses.Add(ENiagaraScriptCompileStatus::NCS_Unknown);
			InCompileErrors.Add(TEXT("Invalid script pointer!"));
			InCompilePaths.Add(TEXT("Unknown..."));
			InUsages.Add(TPair<ENiagaraScriptUsage, int32>(ENiagaraScriptUsage::Function, 0));
		}
	}

	for (int32 i = 0; i < InCompileStatuses.Num(); i++)
	{
		AggregateStatus = FNiagaraEditorUtilities::UnionCompileStatus(AggregateStatus, InCompileStatuses[i]);
		AggregateErrors += InCompilePaths[i] + TEXT(" ") + FNiagaraEditorUtilities::StatusToText(InCompileStatuses[i]).ToString() + TEXT("\n");
		AggregateErrors += InCompileErrors[i] + TEXT("\n");
	}

	UpdateCompileStatus(AggregateStatus, AggregateErrors, InCompileStatuses, InCompileErrors, InCompilePaths, SystemScripts);

	LastCompileStatus = AggregateStatus;

	if (OnSystemCompiledDelegate.IsBound())
	{
		OnSystemCompiledDelegate.Broadcast();
	}
}

FNiagaraSystemScriptViewModel::FOnSystemCompiled& FNiagaraSystemScriptViewModel::OnSystemCompiled()
{
	return OnSystemCompiledDelegate;
}

void FNiagaraSystemScriptViewModel::CompileSystem(bool bForce)
{
	UNiagaraSystem* System = GetSystem();
	if (ensure(System))
	{
		System->RequestCompile(bForce);
	}
}

ENiagaraScriptCompileStatus FNiagaraSystemScriptViewModel::GetLatestCompileStatus(FGuid VersionGuid)
{
	UNiagaraSystem* System = GetSystem();
	if (ensure(System))
	{
		TArray<UNiagaraScript*> SystemScripts;
		SystemScripts.Add(System->GetSystemSpawnScript());
		SystemScripts.Add(System->GetSystemUpdateScript());

		for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
		{
			if (Handle.GetIsEnabled())
			{
				Handle.GetEmitterData()->GetScripts(SystemScripts, true);
			}
		}

		bool bDirty = false;
		for (int32 i = 0; i < SystemScripts.Num(); i++)
		{
			if (!SystemScripts[i])
			{
				continue;
			}

			if (SystemScripts[i]->IsCompilable() && !SystemScripts[i]->AreScriptAndSourceSynchronized(VersionGuid))
			{
				bDirty = true;
				break;
			}
		}

		if (bDirty)
		{
			return ENiagaraScriptCompileStatus::NCS_Dirty;
		}
	}

	return LastCompileStatus;
}

