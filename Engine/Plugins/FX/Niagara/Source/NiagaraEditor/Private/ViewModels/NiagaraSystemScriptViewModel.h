// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"

class FNiagaraSystemViewModel;
class FNiagaraScriptViewModel;
class UNiagaraSystem;

/** View model which manages the System script. */
class FNiagaraSystemScriptViewModel : public FNiagaraScriptViewModel
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnSystemCompiled)

public:
	explicit FNiagaraSystemScriptViewModel(bool bInIsForDataProcessingOnly);

	void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

	virtual ~FNiagaraSystemScriptViewModel() override;

	TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter() const override;

	FOnSystemCompiled& OnSystemCompiled();

	void CompileSystem(bool bForce);
	
	virtual ENiagaraScriptCompileStatus GetLatestCompileStatus(FGuid VersionGuid = FGuid()) override;

private:

	void OnSystemVMCompiled(UNiagaraSystem* InSystem);

	FORCEINLINE UNiagaraSystem* GetSystem() const
	{
		if (const TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin())
		{
			return &SystemViewModel->GetSystem();
		}

		return nullptr;
	}

private:
	/** The System who's script is getting viewed and edited by this view model. */
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModelWeak;

	FOnSystemCompiled OnSystemCompiledDelegate;
};
