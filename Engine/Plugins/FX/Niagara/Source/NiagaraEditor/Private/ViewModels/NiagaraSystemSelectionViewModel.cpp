// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"
#include "ViewModels/Stack/NiagaraStackSelection.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraSystemEditorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraSystemSelectionViewModel)

FGuid StackEntryToEmitterGuid(UNiagaraStackEntry* StackEntry)
{
	if (StackEntry->GetEmitterViewModel().IsValid())
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = StackEntry->GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(StackEntry->GetEmitterViewModel()->GetEmitter());
		if (EmitterHandleViewModel.IsValid())
		{
			return EmitterHandleViewModel->GetId();
		}
	}
	return FGuid();
}

UNiagaraSystemSelectionViewModel::FSelectionEntry::FSelectionEntry(UNiagaraStackEntry* SelectedEntry)
	: Entry(SelectedEntry)
	, EmitterGuid(StackEntryToEmitterGuid(SelectedEntry))
	, EditorDataKey(SelectedEntry->GetStackEditorDataKey())
{
}

void UNiagaraSystemSelectionViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModelWeak = InSystemViewModel;

	StackSelection = NewObject<UNiagaraStackSelection>(this);
	StackSelection->Initialize(UNiagaraStackEntry::FRequiredEntryData(
		InSystemViewModel,
		TSharedPtr<FNiagaraEmitterViewModel>(),
		UNiagaraStackEntry::FExecutionCategoryNames::System,
		UNiagaraStackEntry::FExecutionSubcategoryNames::Settings,
		InSystemViewModel->GetEditorData().GetStackEditorData()));

	SelectionStackViewModel = NewObject<UNiagaraStackViewModel>(this);
	SelectionStackViewModel->InitializeWithRootEntry(StackSelection);

	bSystemIsSelected = false;
	bRefreshIsPending = false;
}

void UNiagaraSystemSelectionViewModel::Finalize()
{
	SystemViewModelWeak.Reset();

	if (StackSelection != nullptr)
	{
		StackSelection->Finalize();
		StackSelection = nullptr;
	}

	if (SelectionStackViewModel != nullptr)
	{
		SelectionStackViewModel->Finalize();
		SelectionStackViewModel = nullptr;
	}
}

bool UNiagaraSystemSelectionViewModel::ContainsEntry(UNiagaraStackEntry* StackEntry) const
{
	for (const FSelectionEntry& SelectionEntry : SelectionEntries)
	{
		if (SelectionEntry.Entry == StackEntry)
		{
			return true;
		}
	}
	return false;
}

void UNiagaraSystemSelectionViewModel::GetSelectedEntries(TArray<UNiagaraStackEntry*>& OutSelectedEntries) const
{
	for (const FSelectionEntry& SelectionEntry : SelectionEntries)
	{
		OutSelectedEntries.Add(SelectionEntry.Entry.Get());
	}
}

bool UNiagaraSystemSelectionViewModel::GetSystemIsSelected() const
{
	return bSystemIsSelected;
}

const TArray<FGuid>& UNiagaraSystemSelectionViewModel::GetSelectedEmitterHandleIds() const
{
	return SelectedEmitterHandleIds;
}

void UNiagaraSystemSelectionViewModel::UpdateSelectedEntries(const TArray<UNiagaraStackEntry*> InSelectedEntries, const TArray<UNiagaraStackEntry*> InDeselectedEntries, bool bClearCurrentSelection)
{
	bool bSelectionChanged = false;
	if (bClearCurrentSelection)
	{
		SelectionEntries.Empty();
		bSelectionChanged = true;
	}
	else
	{
		for (UNiagaraStackEntry* DeselectedEntry : InDeselectedEntries)
		{
			if (ContainsEntry(DeselectedEntry))
			{
				RemoveEntryFromSelectionInternal(DeselectedEntry);
				bSelectionChanged = true;
			}
		}
	}

	for (UNiagaraStackEntry* SelectedEntry : InSelectedEntries)
	{
		if (ContainsEntry(SelectedEntry) == false)
		{
			SelectionEntries.Add(FSelectionEntry(SelectedEntry));
			bSelectionChanged = true;
		}
	}

	if (bSelectionChanged)
	{
		UpdateExternalSelectionState();
	}
}

UNiagaraStackViewModel* UNiagaraSystemSelectionViewModel::GetSelectionStackViewModel()
{
	return SelectionStackViewModel;
}

void  UNiagaraSystemSelectionViewModel::EmptySelection()
{
	if (SelectionEntries.Num() > 0)
	{
		SelectionEntries.Empty();
		UpdateExternalSelectionState();
	}
}

void UNiagaraSystemSelectionViewModel::RemoveEntriesFromSelection(const TArray<UNiagaraStackEntry*>& InEntriesToRemove)
{
	TArray<UNiagaraStackEntry*> EntriesToDeselect;
	for (FSelectionEntry& SelectionEntry : SelectionEntries)
	{
		if (InEntriesToRemove.Contains(SelectionEntry.Entry.Get()))
		{
			EntriesToDeselect.Add(SelectionEntry.Entry.Get());
		}
	}

	bool bSelectionChanged = EntriesToDeselect.Num() > 0;
	for (UNiagaraStackEntry* EntryToDeselect : EntriesToDeselect)
	{
		RemoveEntryFromSelectionInternal(EntryToDeselect);
	}

	if (bSelectionChanged)
	{
		UpdateExternalSelectionState();
	}
}

void UNiagaraSystemSelectionViewModel::RemoveEntryFromSelectionByDisplayedObject(const UObject* InObject)
{
	TArray<UNiagaraStackEntry*> EntriesToDeselect;
	for (FSelectionEntry& SelectionEntry : SelectionEntries)
	{
		if (SelectionEntry.Entry->GetDisplayedObject() == InObject)
		{
			EntriesToDeselect.Add(SelectionEntry.Entry.Get());
		}
	}

	bool bSelectionChanged = EntriesToDeselect.Num() > 0;
	for (UNiagaraStackEntry* EntryToDeselect : EntriesToDeselect)
	{
		RemoveEntryFromSelectionInternal(EntryToDeselect);
	}

	if (bSelectionChanged)
	{
		UpdateExternalSelectionState();
	}
}

void UNiagaraSystemSelectionViewModel::AddEntriesToSelectionByDisplayedObjectsDeferred(const TArray<const UObject*>& InObjects)
{
	for (const UObject* InObject : InObjects)
	{
		DeferredDisplayedObjectKeysToAddToSelection.Add(FObjectKey(InObject));
	}
}

void UNiagaraSystemSelectionViewModel::AddEntryToSelectionByDisplayedObjectDeferred(const UObject* InObject)
{
	TArray<const UObject*> InObjects;
	InObjects.Add(InObject);
	AddEntriesToSelectionByDisplayedObjectsDeferred(InObjects);
}

void UNiagaraSystemSelectionViewModel::AddEntriesToSelectionByDisplayedObjectKeysDeferred(const TArray<FObjectKey>& InObjectKeys)
{
	DeferredDisplayedObjectKeysToAddToSelection.Append(InObjectKeys);
}

void UNiagaraSystemSelectionViewModel::AddEntryToSelectionByDisplayedObjectKeyDeferred(const FObjectKey& InObjectKey)
{
	DeferredDisplayedObjectKeysToAddToSelection.Add(InObjectKey);
}

void UNiagaraSystemSelectionViewModel::AddEntryToSelectionBySelectionIdDeferred(const FGuid& InSelectionId)
{
	DeferredSelectionIdsToAddToSelection.Add(InSelectionId);
}

void CollectGroupAndItemEntries(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, TMap<FGuid, TMap<FString, UNiagaraStackEntry*>>& OutEmitterGuidToEntryKeyToEntryMap)
{
	auto CollectFromStackViewModel = [](FGuid EmitterGuid, UNiagaraStackViewModel* StackViewModel, TMap<FGuid, TMap<FString, UNiagaraStackEntry*>>& OutEmitterGuidToEntryKeyToEntryMap)
	{
		TArray<UNiagaraStackEntry*> EntriesToCheck;
		EntriesToCheck.Add(StackViewModel->GetRootEntry());

		TMap<FString, UNiagaraStackEntry*>& EntryKeyToEntryMap = OutEmitterGuidToEntryKeyToEntryMap.FindOrAdd(EmitterGuid);
		while (EntriesToCheck.Num() > 0)
		{
			UNiagaraStackEntry* Entry = EntriesToCheck[EntriesToCheck.Num() - 1];
			EntriesToCheck.RemoveAt(EntriesToCheck.Num() - 1);

			if (Entry->IsA<UNiagaraStackItemGroup>() || Entry->IsA<UNiagaraStackItem>() || Entry->IsA<UNiagaraStackRoot>())
			{
				EntryKeyToEntryMap.Add(Entry->GetStackEditorDataKey(), Entry);
			}

			Entry->GetUnfilteredChildren(EntriesToCheck);
		}
	};

	CollectFromStackViewModel(FGuid(), SystemViewModel->GetSystemStackViewModel(), OutEmitterGuidToEntryKeyToEntryMap);
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
	{
		CollectFromStackViewModel(EmitterHandleViewModel->GetId(), EmitterHandleViewModel->GetEmitterStackViewModel(), OutEmitterGuidToEntryKeyToEntryMap);
	}
}

bool UNiagaraSystemSelectionViewModel::Refresh()
{
	bool bSelectionChanged = false;

	TMap<FGuid, TMap<FString, UNiagaraStackEntry*>> EmitterGuidToEntryKeyToEntryMap;
	TArray<FSelectionEntry> NewSelectionEntries;
	for (FSelectionEntry& SelectionEntry : SelectionEntries)
	{
		if (SelectionEntry.Entry.IsValid() == false || SelectionEntry.Entry->IsFinalized())
		{
			// If the entry is no longer valid, try to reselect it by it's emitter guid and stack editor data key.
			if (EmitterGuidToEntryKeyToEntryMap.Num() == 0)
			{
				// This is expensive, so only do it when necessary.
				CollectGroupAndItemEntries(GetSystemViewModel(), EmitterGuidToEntryKeyToEntryMap);
			}
			TMap<FString, UNiagaraStackEntry*>* EntryKeyToEntryMap = EmitterGuidToEntryKeyToEntryMap.Find(SelectionEntry.EmitterGuid);
			if (EntryKeyToEntryMap != nullptr)
			{
				UNiagaraStackEntry** Entry = EntryKeyToEntryMap->Find(SelectionEntry.EditorDataKey);
				if (Entry != nullptr)
				{
					NewSelectionEntries.Add(FSelectionEntry(*Entry));
				}
			}
			bSelectionChanged = true;
		}
		else
		{
			NewSelectionEntries.Add(SelectionEntry);
		}
	}

	if (bSelectionChanged)
	{
		SelectionEntries = NewSelectionEntries;
		UpdateExternalSelectionState();
	}
	else
	{
		StackSelection->RefreshChildren();
	}
	return bSelectionChanged;
}

void UNiagaraSystemSelectionViewModel::RefreshDeferred()
{
	bRefreshIsPending = true;
}

UNiagaraSystemSelectionViewModel::FOnSelectionChanged& UNiagaraSystemSelectionViewModel::OnEntrySelectionChanged()
{
	return OnEntrySelectionChangedDelegate;
}

UNiagaraSystemSelectionViewModel::FOnSelectionChanged& UNiagaraSystemSelectionViewModel::OnEmitterHandleIdSelectionChanged()
{
	return OnEmitterHandleIdSelectionChangedDelegate;
}

UNiagaraSystemSelectionViewModel::FOnSelectionChanged& UNiagaraSystemSelectionViewModel::OnSystemIsSelectedChanged()
{
	return OnSystemIsSelectedChangedDelegate;
}

template<typename TEntryKey, typename TTryGetKeyFromEntry>
void GatherAllKeysForEntry(UNiagaraStackEntry* StackEntry, TTryGetKeyFromEntry TryGetKeyFromEntry, TArray<TEntryKey>& OutEntryKeys)
{
	TEntryKey EntryKey;
	if (TryGetKeyFromEntry(StackEntry, EntryKey))
	{
		OutEntryKeys.Add(EntryKey);
	}

	TArray<UNiagaraStackEntry*> Children;
	StackEntry->GetUnfilteredChildren(Children);
	for (UNiagaraStackEntry* ChildEntry : Children)
	{
		GatherAllKeysForEntry(ChildEntry, TryGetKeyFromEntry, OutEntryKeys);
	}
}

template<typename TEntryKey, typename TTryGetKeyFromEntry>
void FindStackGroupsAndItemsForEntryKeysRecursive(UNiagaraStackEntry* StackEntry, const TArray<TEntryKey>& EntryKeysToFind, TTryGetKeyFromEntry TryGetKeyFromEntry, TArray<UNiagaraStackEntry*>& OutFoundStackEntries)
{
	if (StackEntry->IsA<UNiagaraStackItemGroup>())
	{
		TEntryKey EntryKey;
		if (TryGetKeyFromEntry(StackEntry, EntryKey) && EntryKeysToFind.Contains(EntryKey))
		{
			OutFoundStackEntries.Add(StackEntry);
		}
	}
	else if (StackEntry->IsA<UNiagaraStackItem>())
	{
		TArray<TEntryKey> ItemEntryKeys;
		GatherAllKeysForEntry(StackEntry, TryGetKeyFromEntry, ItemEntryKeys);
		if (EntryKeysToFind.ContainsByPredicate([&ItemEntryKeys](const TEntryKey& EntryKey) { return ItemEntryKeys.Contains(EntryKey); }))
		{
			OutFoundStackEntries.Add(StackEntry);
		}
	}

	if (StackEntry->IsA<UNiagaraStackItem>() == false)
	{
		TArray<UNiagaraStackEntry*> Children;
		StackEntry->GetUnfilteredChildren(Children);
		for (UNiagaraStackEntry* ChildEntry : Children)
		{
			FindStackGroupsAndItemsForEntryKeysRecursive(ChildEntry, EntryKeysToFind, TryGetKeyFromEntry, OutFoundStackEntries);
		}
	}
}

void UNiagaraSystemSelectionViewModel::Tick()
{
	bool bSelectionChanged = false;

	TArray<UNiagaraStackEntry*> FoundStackEntries;
	TSharedRef<FNiagaraSystemViewModel> SystemViewModel = GetSystemViewModel();

	if(DeferredDisplayedObjectKeysToAddToSelection.Num() > 0)
	{
		auto TryGetObjectKeyFromStackEntry = [](const UNiagaraStackEntry* StackEntry, FObjectKey& OutObjectKey)
		{
			if (StackEntry->GetDisplayedObject() != nullptr)
			{
				OutObjectKey = FObjectKey(StackEntry->GetDisplayedObject());
				return true;
			}
			OutObjectKey = FObjectKey();
			return false;
		};
		
		FindStackGroupsAndItemsForEntryKeysRecursive(SystemViewModel->GetSystemStackViewModel()->GetRootEntry(),
			DeferredDisplayedObjectKeysToAddToSelection, TryGetObjectKeyFromStackEntry, FoundStackEntries);

		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterViewModel : SystemViewModel->GetEmitterHandleViewModels())
		{
			FindStackGroupsAndItemsForEntryKeysRecursive(EmitterViewModel->GetEmitterStackViewModel()->GetRootEntry(),
				DeferredDisplayedObjectKeysToAddToSelection, TryGetObjectKeyFromStackEntry, FoundStackEntries);
		}

		DeferredDisplayedObjectKeysToAddToSelection.Empty();
	}

	if (DeferredSelectionIdsToAddToSelection.Num() > 0)
	{
		auto TryGetSelectionIdFromStackEntry = [](const UNiagaraStackEntry* StackEntry, FGuid& OutSelectionId)
		{ 
			OutSelectionId = StackEntry->GetSelectionId();
			return OutSelectionId.IsValid();
		};

		FindStackGroupsAndItemsForEntryKeysRecursive(SystemViewModel->GetSystemStackViewModel()->GetRootEntry(),
			DeferredSelectionIdsToAddToSelection, TryGetSelectionIdFromStackEntry, FoundStackEntries);

		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterViewModel : SystemViewModel->GetEmitterHandleViewModels())
		{
			FindStackGroupsAndItemsForEntryKeysRecursive(EmitterViewModel->GetEmitterStackViewModel()->GetRootEntry(),
				DeferredSelectionIdsToAddToSelection, TryGetSelectionIdFromStackEntry, FoundStackEntries);
		}

		DeferredSelectionIdsToAddToSelection.Empty();
	}

	for (UNiagaraStackEntry* FoundStackEntry : FoundStackEntries)
	{
		if (ContainsEntry(FoundStackEntry) == false)
		{
			SelectionEntries.Add(FSelectionEntry(FoundStackEntry));
			bSelectionChanged = true;
		}
	}

	bool bHasBeenRefreshed = false;
	if (bRefreshIsPending)
	{
		bHasBeenRefreshed = Refresh();
		bRefreshIsPending = false;
	}

	if (bSelectionChanged && bHasBeenRefreshed == false)
	{
		UpdateExternalSelectionState();
	}
}

TSharedRef<FNiagaraSystemViewModel> UNiagaraSystemSelectionViewModel::GetSystemViewModel()
{
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = SystemViewModelWeak.Pin();
	checkf(SystemViewModel.IsValid(), TEXT("Owning system view model destroyed before system overview view model."));
	return SystemViewModel.ToSharedRef();
}

void UNiagaraSystemSelectionViewModel::RemoveEntryFromSelectionInternal(UNiagaraStackEntry* DeselectedEntry)
{
	SelectionEntries.RemoveAll([DeselectedEntry](const FSelectionEntry& SelectionEntry) { return SelectionEntry.Entry == DeselectedEntry; });
}

void UNiagaraSystemSelectionViewModel::UpdateExternalSelectionState()
{
	bool bWasSystemSelected = bSystemIsSelected;
	bSystemIsSelected = false;
	TArray<FGuid> OldSelectedEmitterHandleIds = SelectedEmitterHandleIds;
	SelectedEmitterHandleIds.Empty();
	TArray<FVersionedNiagaraEmitter> SelectedEmitters;
	for (const FSelectionEntry& SelectionEntry : SelectionEntries)
	{
		if (SelectionEntry.Entry->GetEmitterViewModel().IsValid())
		{
			SelectedEmitters.AddUnique(SelectionEntry.Entry->GetEmitterViewModel()->GetEmitter());
		}
		else
		{
			bSystemIsSelected = true;
		}
	}

	int32 NumRemoved = 0;
	for (const FVersionedNiagaraEmitter& SelectedEmitter : SelectedEmitters)
	{
		//-TODO:Stateless: Do we need stateless support here?
		if (SelectedEmitter.GetEmitterData() == nullptr)
		{
			continue;
		}
		FGuid EmitterId = GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(SelectedEmitter)->GetId();
		NumRemoved += OldSelectedEmitterHandleIds.Remove(EmitterId);
		SelectedEmitterHandleIds.Add(EmitterId);
	}

	// If there were ids left over, or we added more than we removed than the id selection has changed.
	bool bEmitterHandleIdSelectionChanged = OldSelectedEmitterHandleIds.Num() != 0 || SelectedEmitterHandleIds.Num() != NumRemoved;
	bool bSystemSelectionChanged = bWasSystemSelected != bSystemIsSelected;

	// Check the new selection to make sure that there are no entries which are owned by other entries and if there are
	// remove them.
	TArray<UNiagaraStackEntry*> StackSelectionEntries;
	for (const FSelectionEntry& SelectionEntry : SelectionEntries)
	{
		bool bIsChildEntry = false;
		UNiagaraStackEntry* EntryInOuterChain = SelectionEntry.Entry->GetTypedOuter<UNiagaraStackEntry>();
		while (EntryInOuterChain != nullptr)
		{
			if (ContainsEntry(EntryInOuterChain))
			{
				bIsChildEntry = true;
				EntryInOuterChain = nullptr;
			}
			else
			{
				EntryInOuterChain = EntryInOuterChain->GetTypedOuter<UNiagaraStackEntry>();
			}
		}

		if (bIsChildEntry == false)
		{
			StackSelectionEntries.Add(SelectionEntry.Entry.Get());
		}
	}
	
	StackSelection->SetSelectedEntries(StackSelectionEntries);
	OnEntrySelectionChangedDelegate.Broadcast();
	if (bEmitterHandleIdSelectionChanged)
	{
		OnEmitterHandleIdSelectionChangedDelegate.Broadcast();
	}
	if(bSystemSelectionChanged)
	{
		OnSystemIsSelectedChangedDelegate.Broadcast();
	}
}
