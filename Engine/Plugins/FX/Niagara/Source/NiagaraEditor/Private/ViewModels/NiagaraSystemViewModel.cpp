// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraSystemViewModel.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "EdGraphSchema_NiagaraSystemOverview.h"
#include "EdGraphUtilities.h"
#include "Editor.h"
#include "EditorSupportDelegates.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ISequencerModule.h"
#include "Math/NumericLimits.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "Sequencer/NiagaraSequence/MovieSceneNiagaraEmitterTrack.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterfaceCurveBase.h"
#include "NiagaraEditorData.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraEmitterFactoryNew.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraEmitterInstanceImpl.h"
#include "NiagaraGraph.h"
#include "NiagaraMessageManager.h"
#include "NiagaraMessageUtilities.h"
#include "NiagaraMessages.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNotificationWidgetProvider.h"
#include "NiagaraOverviewNode.h"
#include "NiagaraParameterDefinitionsSubscriber.h"
#include "NiagaraRecentAndFavoritesManager.h"
#include "NiagaraScriptGraphViewModel.h"
#include "NiagaraScriptRuntimeData.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "Sequencer/NiagaraSequence/NiagaraSequence.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraSystemScriptViewModel.h"
#include "ScopedTransaction.h"
#include "ViewModels/NiagaraCurveSelectionViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "ViewModels/NiagaraPlaceholderDataInterfaceManager.h"
#include "ViewModels/NiagaraScratchPadUtilities.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraSystemEditorDocumentsViewModel.h"
#include "ViewModels/NiagaraSystemGraphSelectionViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "ViewModels/HierarchyEditor/NiagaraUserParametersHierarchyViewModel.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "Widgets/Notifications/SNotificationList.h"

//-TODO:Stateless:
#include "Stateless/NiagaraStatelessEmitter.h"
#include "NiagaraSpriteRendererProperties.h"
//-TODO:Stateless:

DECLARE_CYCLE_STAT(TEXT("Niagara - SystemViewModel - CompileSystem"), STAT_NiagaraEditor_SystemViewModel_CompileSystem, STATGROUP_NiagaraEditor);

#define LOCTEXT_NAMESPACE "NiagaraSystemViewModel"

template<> TMap<UNiagaraSystem*, TArray<FNiagaraSystemViewModel*>> TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::ObjectsToViewModels{};

FNiagaraSystemViewModel::FNiagaraSystemViewModel()
	: System(nullptr)
	, PreviewComponent(nullptr)
	, SystemInstance(nullptr)
	, NiagaraSequence(nullptr)
	, bSettingSequencerTimeDirectly(false)
	, bForceAutoCompileOnce(false)
	, bSupportCompileForEdit(false)
	, bUpdatingEmittersFromSequencerDataChange(false)
	, bUpdatingSequencerFromEmitterDataChange(false)
	, bUpdatingSystemSelectionFromSequencer(false)
	, bUpdatingSequencerSelectionFromSystem(false)
	, bResetingSequencerTracks(false)
	, EditorSettings(GetMutableDefault<UNiagaraEditorSettings>())
	, bResetRequestPending(false)
	, bCompilePendingCompletion(false)
	, SystemStackViewModel(nullptr)
	, EditorDocumentsViewModel(nullptr)
	, SelectionViewModel(nullptr)
	, ScriptScratchPadViewModel(nullptr)
	, bPendingAssetMessagesChanged(true)
{
}

void FNiagaraSystemViewModel::Initialize(UNiagaraSystem& InSystem, FNiagaraSystemViewModelOptions InOptions)
{
	System = &InSystem;

	RegisteredHandle = RegisterViewModelWithMap(System, this);

	bCanModifyEmittersFromTimeline = InOptions.bCanModifyEmittersFromTimeline;
	bCanAutoCompile = InOptions.bCanAutoCompile;
	bCanSimulate = InOptions.bCanSimulate;
	EditMode = InOptions.EditMode;
	OnGetSequencerAddMenuContent = InOptions.OnGetSequencerAddMenuContent;
	SystemMessageLogGuidKey = InOptions.MessageLogGuid;
	bIsForDataProcessingOnly = InOptions.bIsForDataProcessingOnly;

	if (bIsForDataProcessingOnly == false)
	{
		GEditor->RegisterForUndo(this);
	}

	SystemChangedDelegateHandle = System->OnSystemPostEditChange().AddSP(this, &FNiagaraSystemViewModel::SystemChanged);

	SelectionViewModel = NewObject<UNiagaraSystemSelectionViewModel>(GetTransientPackage());
	SelectionViewModel->Initialize(this->AsShared());
	SelectionViewModel->OnEmitterHandleIdSelectionChanged().AddSP(this, &FNiagaraSystemViewModel::SystemSelectionChanged);

	SystemScriptViewModel = MakeShared<FNiagaraSystemScriptViewModel>(bIsForDataProcessingOnly);
	SystemScriptViewModel->Initialize(this->AsShared());

	OverviewGraphViewModel = MakeShared<FNiagaraOverviewGraphViewModel>();
	OverviewGraphViewModel->Initialize(this->AsShared());

	SystemStackViewModel = NewObject<UNiagaraStackViewModel>(GetTransientPackage());
	SystemStackViewModel->OnStructureChanged().AddSP(this, &FNiagaraSystemViewModel::StackViewModelStructureChanged);

	EditorDocumentsViewModel = NewObject< UNiagaraSystemEditorDocumentsViewModel >(GetTransientPackage());
	EditorDocumentsViewModel->Initialize(this->AsShared());

	ScriptScratchPadViewModel = NewObject<UNiagaraScratchPadViewModel>(GetTransientPackage());
	ScriptScratchPadViewModel->Initialize(this->AsShared());
	ScriptScratchPadViewModel->OnScriptRenamed().AddSP(this, &FNiagaraSystemViewModel::ScratchPadScriptsChanged);
	ScriptScratchPadViewModel->OnScriptDeleted().AddSP(this, &FNiagaraSystemViewModel::ScratchPadScriptsChanged);

	CurveSelectionViewModel = NewObject<UNiagaraCurveSelectionViewModel>(GetTransientPackage());
	CurveSelectionViewModel->Initialize(this->AsShared());

	ScalabilityViewModel = NewObject<UNiagaraSystemScalabilityViewModel>(GetTransientPackage());
	ScalabilityViewModel->Initialize(this->AsShared());

	UserParameterPanelViewModel = MakeShared<FNiagaraUserParameterPanelViewModel>();
	
	UserParametersHierarchyViewModel = NewObject<UNiagaraUserParametersHierarchyViewModel>(GetTransientPackage());
	UserParametersHierarchyViewModel->Initialize(this->AsShared());
	
	PlaceholderDataInterfaceManager = MakeShared<FNiagaraPlaceholderDataInterfaceManager>(this->AsShared());

	SystemGraphSelectionViewModel = MakeShared<FNiagaraSystemGraphSelectionViewModel>();
	SystemGraphSelectionViewModel->Initialize(this->AsShared());

	PreviewMovieViewModel = MakeShared<FNiagaraPreviewMovieViewModel>();
	PreviewMovieViewModel->Initialize();
	
	bSupportCompileForEdit = !bIsForDataProcessingOnly && (EditMode == ENiagaraSystemViewModelEditMode::SystemAsset || EditMode == ENiagaraSystemViewModelEditMode::EmitterAsset);
	bIsCompileForEdit = false;
	if (bSupportCompileForEdit)
	{
		bIsCompileForEdit = InOptions.bCompileForEdit;
		System->SetCompileForEdit(bIsCompileForEdit);
	}
	
	if (bIsForDataProcessingOnly == false)
	{
		SetupPreviewComponentAndInstance();
		SetupSequencer();
	}

	OnScriptAppliedDelegateHandle = FNiagaraEditorModule::Get().OnScriptApplied().AddLambda([this](UNiagaraScript* Script, FGuid VersionGuid)
	{
		for(UNiagaraGraph* Graph : GetAllGraphs())
		{
			TArray<UNiagaraNodeFunctionCall*> FunctionCallNodes;
			Graph->GetNodesOfClass(FunctionCallNodes);

			for(UNiagaraNodeFunctionCall* FunctionCall : FunctionCallNodes)
			{
				if(FunctionCall->FunctionScript == Script)
				{		
					ResetSystem(ETimeResetMode::AllowResetTime, EMultiResetMode::AllowResetAllInstances, EReinitMode::ReinitializeSystem);
					return;
				}
			}
		}
	});
	
	RefreshAll();
	AddSystemEventHandlers();
}

bool FNiagaraSystemViewModel::IsValid() const
{
	return System != nullptr;
}

void FNiagaraSystemViewModel::DumpToText(FString& ExportText)
{
	TSet<UObject*> ExportObjs;
	ExportObjs.Add(System);
	FEdGraphUtilities::ExportNodesToText(ExportObjs, ExportText);
}

void FNiagaraSystemViewModel::SetParameterPanelViewModel(TSharedPtr<INiagaraParameterPanelViewModel> InVM) 
{
	if (ParameterPanelViewModel.IsValid())
	{
		ParameterPanelViewModel.Pin()->GetOnInvalidateCachedDependencies().RemoveAll(this);
	}

	ParameterPanelViewModel = InVM;

	if (InVM.IsValid())
	{
		InVM->GetOnInvalidateCachedDependencies().AddRaw(this, &FNiagaraSystemViewModel::InvalidateCachedParams);
	}
}

void FNiagaraSystemViewModel::InvalidateCachedParams()
{
	GetSystemStackViewModel()->InvalidateCachedParameterUsage();
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleVM : GetEmitterHandleViewModels())
	{
		UNiagaraStackViewModel* EmitterStackViewModel = EmitterHandleVM->GetEmitterStackViewModel();
		if (EmitterStackViewModel)
		{
			EmitterStackViewModel->InvalidateCachedParameterUsage();
		}
	}
}

void FNiagaraSystemViewModel::Cleanup()
{
	if (SystemInstance)
	{
		SystemInstance->OnInitialized().RemoveAll(this);
		SystemInstance->OnReset().RemoveAll(this);
		SystemInstance = nullptr;
	}

	if (PreviewComponent)
	{
		PreviewComponent->OnSystemInstanceChanged().RemoveAll(this);
		PreviewComponent->DeactivateImmediate();
		PreviewComponent = nullptr;
	}

	if(bIsForDataProcessingOnly == false)
	{
		GEditor->UnregisterForUndo(this);
	}

	if (System)
	{
		if (bSupportCompileForEdit && bIsCompileForEdit)
		{
			System->SetCompileForEdit(false);
			System->InvalidateActiveCompiles();
			System->RequestCompile(false);
		}
		System->OnSystemPostEditChange().Remove(SystemChangedDelegateHandle);
	}

	UNiagaraEditorSettings::OnSettingsChanged().RemoveAll(this);
	
	// Make sure that we clear out all of our event handlers
	UnregisterViewModelWithMap(RegisteredHandle);

	for (TSharedRef<FNiagaraEmitterHandleViewModel>& HandleRef : EmitterHandleViewModels)
	{
		HandleRef->OnPropertyChanged().RemoveAll(this);
		HandleRef->OnNameChanged().RemoveAll(this);
		HandleRef->GetEmitterViewModel()->OnPropertyChanged().RemoveAll(this);
		HandleRef->GetEmitterViewModel()->OnScriptCompiled().RemoveAll(this);
		HandleRef->GetEmitterViewModel()->OnScriptGraphChanged().RemoveAll(this);
		HandleRef->GetEmitterViewModel()->OnScriptParameterStoreChanged().RemoveAll(this);
		HandleRef->GetEmitterStackViewModel()->OnStructureChanged().RemoveAll(this);
		HandleRef->Cleanup();
	}
	EmitterHandleViewModels.Empty();

	if (Sequencer.IsValid())
	{
		Sequencer->OnMovieSceneDataChanged().RemoveAll(this);
		Sequencer->OnGlobalTimeChanged().RemoveAll(this);
		Sequencer->GetSelectionChangedTracks().RemoveAll(this);
		Sequencer->GetSelectionChangedSections().RemoveAll(this);
		Sequencer.Reset();
	}

	RemoveSystemEventHandlers();
	SystemScriptViewModel.Reset();

	if (SystemStackViewModel != nullptr)
	{
		SystemStackViewModel->OnStructureChanged().RemoveAll(this);
		SystemStackViewModel->Finalize();
		SystemStackViewModel = nullptr;
	}

	if (EditorDocumentsViewModel != nullptr)
	{
		EditorDocumentsViewModel->Finalize();
		EditorDocumentsViewModel = nullptr;
	}

	if (SelectionViewModel != nullptr)
	{
		SelectionViewModel->OnEmitterHandleIdSelectionChanged().RemoveAll(this);
		SelectionViewModel->Finalize();
		SelectionViewModel = nullptr;
	}

	if (ScriptScratchPadViewModel != nullptr)
	{
		ScriptScratchPadViewModel->OnScriptRenamed().RemoveAll(this);
		ScriptScratchPadViewModel->OnScriptDeleted().RemoveAll(this);
		ScriptScratchPadViewModel->Finalize();
		ScriptScratchPadViewModel = nullptr;
	}

	if (CurveSelectionViewModel != nullptr)
	{
		CurveSelectionViewModel->Finalize();
		CurveSelectionViewModel = nullptr;
	}

	if(ScalabilityViewModel != nullptr)
	{
		ScalabilityViewModel = nullptr;
	}
	
	if(UserParametersHierarchyViewModel != nullptr)
	{
		UserParametersHierarchyViewModel->Finalize();
		UserParametersHierarchyViewModel = nullptr;
	}

	if(UserParameterPanelViewModel.IsValid())
	{
		UserParameterPanelViewModel.Reset();
	}

	if (PlaceholderDataInterfaceManager.IsValid())
	{
		PlaceholderDataInterfaceManager.Reset();
	}

	if (SystemGraphSelectionViewModel != nullptr)
	{
		SystemGraphSelectionViewModel = nullptr;
	}

	if (ParameterPanelViewModel.IsValid())
	{
		ParameterPanelViewModel.Pin()->GetOnInvalidateCachedDependencies().RemoveAll(this);
	}

	FNiagaraEditorModule::Get().OnScriptApplied().Remove(OnScriptAppliedDelegateHandle);

	System = nullptr;
}

FNiagaraSystemViewModel::~FNiagaraSystemViewModel()
{
	Cleanup();
}

INiagaraParameterDefinitionsSubscriber* FNiagaraSystemViewModel::GetParameterDefinitionsSubscriber()
{
	if(EditMode == ENiagaraSystemViewModelEditMode::SystemAsset)
	{ 
		return System;
	}
	else /**EditMode == ENiagaraSystemViewModelEditMode::EmitterAsset */
	{
		return GetEmitterHandleViewModels()[0]->GetEmitterHandle()->GetInstance().Emitter;
	}
}

FText FNiagaraSystemViewModel::GetDisplayName() const
{
	return FText::FromString(System->GetName());
}

const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& FNiagaraSystemViewModel::GetEmitterHandleViewModels() const
{
	return EmitterHandleViewModels;
}

TSharedPtr<FNiagaraEmitterHandleViewModel> FNiagaraSystemViewModel::GetEmitterHandleViewModelById(FGuid InEmitterHandleId)
{
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
	{
		if (EmitterHandleViewModel->GetId() == InEmitterHandleId)
		{
			return EmitterHandleViewModel;
		}
	}
	return TSharedPtr<FNiagaraEmitterHandleViewModel>();
}

TSharedPtr<FNiagaraEmitterHandleViewModel> FNiagaraSystemViewModel::GetEmitterHandleViewModelForEmitter(const FVersionedNiagaraEmitter& InEmitter) const
{
	if (FVersionedNiagaraEmitterData* InEmitterData = InEmitter.GetEmitterData())
	{
		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
		{
			// we compare the pointer to the emitter data here because depending on the versioning status,
			// two guids might point to the same or to different emitters 
			if (InEmitterData == EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter().GetEmitterData())
			{
				return EmitterHandleViewModel;
			}
		}
	}
	return TSharedPtr<FNiagaraEmitterHandleViewModel>();
}

TSharedPtr<FNiagaraSystemScriptViewModel> FNiagaraSystemViewModel::GetSystemScriptViewModel()
{
	return SystemScriptViewModel;
}

void FNiagaraSystemViewModel::SetCompileForEdit(bool bNewState)
{
	if (bSupportCompileForEdit && bIsCompileForEdit != bNewState)
	{
		check(System->GetCompileForEdit() == bIsCompileForEdit);
		bIsCompileForEdit = bNewState;

		System->SetCompileForEdit(bNewState);
		System->InvalidateActiveCompiles();
		System->RequestCompile(false);
	}
}

void FNiagaraSystemViewModel::CompileSystem(bool bForce)
{
	if (EditMode == ENiagaraSystemViewModelEditMode::EmitterDuringMerge && bForce == false)
	{
		return;
	}
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_SystemViewModel_CompileSystem);
	check(SystemScriptViewModel.IsValid());
	SystemScriptViewModel->CompileSystem(bForce);
	bCompilePendingCompletion = true;
	InvalidateCachedCompileStatus();

	if (SystemStackViewModel)
	{
		SystemStackViewModel->RequestValidationUpdate();
		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel :GetEmitterHandleViewModels())
		{
			EmitterHandleViewModel->GetEmitterStackViewModel()->RequestValidationUpdate();
		}
	}
}

ENiagaraScriptCompileStatus FNiagaraSystemViewModel::GetLatestCompileStatus() const
{
	return LatestCompileStatusCache.IsSet()
		? LatestCompileStatusCache.GetValue()
		: ENiagaraScriptCompileStatus::NCS_Unknown;
}

UNiagaraSystemEditorData& FNiagaraSystemViewModel::GetEditorData() const
{
	return *CastChecked<UNiagaraSystemEditorData>(GetSystem().GetEditorData(), ECastCheckedType::NullChecked);
}

UNiagaraComponent* FNiagaraSystemViewModel::GetPreviewComponent()
{
	return PreviewComponent;
}

TSharedPtr<ISequencer> FNiagaraSystemViewModel::GetSequencer()
{
	return Sequencer;
}

UNiagaraSystem& FNiagaraSystemViewModel::GetSystem() const
{
	checkf(System != nullptr, TEXT("System view model not initialized before use."));
	return *System;
}

bool FNiagaraSystemViewModel::GetCanModifyEmittersFromTimeline() const
{
	return bCanModifyEmittersFromTimeline;
}

/** Gets the current editing mode for this system view model. */
ENiagaraSystemViewModelEditMode FNiagaraSystemViewModel::GetEditMode() const
{
	return EditMode;
}

FName FNiagaraSystemViewModel::GetWorkflowMode() const
{
	if(OnGetWorkflowModeDelegate.IsBound())
	{
		return OnGetWorkflowModeDelegate.Execute();
	}
	return NAME_None;	
}

FOnGetWorkflowMode& FNiagaraSystemViewModel::OnGetWorkflowMode()
{
	return OnGetWorkflowModeDelegate;
}

void FNiagaraSystemViewModel::SetWorkflowMode(FName WorkflowMode)
{
	if(OnChangeWorkflowModeDelegate.IsBound())
	{
		OnChangeWorkflowModeDelegate.Execute(WorkflowMode);
	}
}

FNiagaraSystemViewModel::FOnChangeWorkflowMode& FNiagaraSystemViewModel::OnChangeWorkflowMode()
{
	return OnChangeWorkflowModeDelegate;
}

FNiagaraSystemViewModel::FOnEmitterThumbnailRequested& FNiagaraSystemViewModel::OnEmitterThumbnailRequested()
{
	return OnEmitterThumbnailRequestedDelegate;
}

TSharedPtr<FNiagaraEmitterHandleViewModel> FNiagaraSystemViewModel::AddEmitterFromAssetData(const FAssetData& AssetData)
{
	UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(AssetData.GetAsset());
	if (Emitter != nullptr)
	{
		FNiagaraEditorModule::Get().GetRecentsManager()->EmitterUsed(*Emitter);
		return AddEmitter(*Emitter, Emitter->GetExposedVersion().VersionGuid);
	}

	return nullptr;
}

TSharedPtr<FNiagaraEmitterHandleViewModel> FNiagaraSystemViewModel::AddEmitter(UNiagaraEmitter& Emitter, FGuid EmitterVersion)
{
	// Reset view models before modifying the emitter handle list to prevent accessing deleted data.
	ResetEmitterHandleViewModelsAndTracks();

	// When editing an emitter asset the system is a placeholder and we don't want to make adding an emitter to it undoable.
	bool bSystemIsPlaceholder = EditMode == ENiagaraSystemViewModelEditMode::EmitterAsset;
	if (false == bSystemIsPlaceholder && false == bIsForDataProcessingOnly)
	{
		GEditor->BeginTransaction(LOCTEXT("AddEmitter", "Add emitter"));
	}

	const FGuid NewEmitterHandleId = FNiagaraEditorUtilities::AddEmitterToSystem(GetSystem(), Emitter, EmitterVersion, EditMode != ENiagaraSystemViewModelEditMode::EmitterDuringMerge);

	if (false == bSystemIsPlaceholder && false == bIsForDataProcessingOnly)
	{
		GEditor->EndTransaction();
	}

	if (GetSystem().GetNumEmitters() == 1 && EditorSettings->GetAutoPlay() && Sequencer.IsValid())
	{
		// When adding a new emitter to an empty system start playing.
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
	}

	RefreshAll();

	if (bSystemIsPlaceholder == false)
	{
		GetEmitterHandleViewModelById(NewEmitterHandleId)->GetEmitterViewModel()->GetEditorData().SetShowSummaryView(Emitter.GetEmitterData(EmitterVersion)->AddEmitterDefaultViewState == ENiagaraEmitterDefaultSummaryState::Summary ? true : false);
	}

	
	TRange<float> SystemPlaybackRange = GetEditorData().GetPlaybackRange();
	TRange<float> EmitterPlaybackRange = GetEmitterHandleViewModelById(NewEmitterHandleId)->GetEmitterViewModel()->GetEditorData().GetPlaybackRange();
	TRange<float> NewSystemPlaybackRange = TRange<float>(
		FMath::Min(SystemPlaybackRange.GetLowerBoundValue(), EmitterPlaybackRange.GetLowerBoundValue()),
		FMath::Max(SystemPlaybackRange.GetUpperBoundValue(), EmitterPlaybackRange.GetUpperBoundValue()));

	GetEditorData().SetPlaybackRange(NewSystemPlaybackRange);

	TGuardValue<bool> UpdateGuard(bUpdatingSequencerFromEmitterDataChange, true);

	if (NiagaraSequence)
	{
		FFrameTime NewStartFrame = NewSystemPlaybackRange.GetLowerBoundValue() * NiagaraSequence->GetMovieScene()->GetTickResolution();
		int32 NewDuration = (NewSystemPlaybackRange.Size<float>() * NiagaraSequence->GetMovieScene()->GetTickResolution()).FrameNumber.Value;

		NiagaraSequence->GetMovieScene()->SetPlaybackRange(NewStartFrame.RoundToFrame(), NewDuration);
	}

	TSharedPtr<FNiagaraEmitterHandleViewModel> NewEmitterHandleViewModel = GetEmitterHandleViewModelById(NewEmitterHandleId);
	TArray<UNiagaraStackEntry*> SelectedStackEntries;
	SelectedStackEntries.Add(NewEmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry());
	SelectionViewModel->UpdateSelectedEntries(SelectedStackEntries, TArray<UNiagaraStackEntry*>(), true);

	bForceAutoCompileOnce = true;

	return NewEmitterHandleViewModel;
}

TSharedPtr<FNiagaraEmitterHandleViewModel> FNiagaraSystemViewModel::AddEmptyEmitter()
{
	// Otherwise we use an actually empty emitter
	UNiagaraEmitter* EmptyEmitter = NewObject<UNiagaraEmitter>(GetTransientPackage());
	bool bAddDefaultModulesAndRenderers = false;
	UNiagaraEmitterFactoryNew::InitializeEmitter(EmptyEmitter, bAddDefaultModulesAndRenderers);
	FName EmptyEmitterName = FNiagaraEditorUtilities::GetUniqueObjectName<UNiagaraEmitter>(GetTransientPackage(), TEXT("Empty"));
	EmptyEmitter->SetUniqueEmitterName(EmptyEmitterName.ToString());
	EmptyEmitter->SetFlags(RF_Transactional);
	EmptyEmitter->bIsInheritable = false;
	return AddEmitter(FVersionedNiagaraEmitter(EmptyEmitter, FGuid()));
}

TSharedPtr<FNiagaraEmitterHandleViewModel> FNiagaraSystemViewModel::AddStatelessEmitter()
{
	UNiagaraSystem& NiagaraSystem = GetSystem();

	auto SetupStatelessEmitter =
		[](UNiagaraStatelessEmitter* StatelessEmitter, FName EmitterName)
		{
			StatelessEmitter->SetUniqueEmitterName(*EmitterName.ToString());

			StatelessEmitter->AddSpawnInfo();
			if (UClass* TemplateClass = FindObject<UClass>(nullptr, TEXT("/Script/Niagara.NiagaraStatelessEmitterDefault")))
			{
				StatelessEmitter->SetEmitterTemplate(Cast<UNiagaraStatelessEmitterTemplate>(TemplateClass->GetDefaultObject()));
			}
			{
				FSoftObjectPath DefaultMaterial(TEXT("Material'/Niagara/DefaultAssets/DefaultSpriteMaterial.DefaultSpriteMaterial'"));

				UNiagaraSpriteRendererProperties* NewRenderer = NewObject<UNiagaraSpriteRendererProperties>(StatelessEmitter, "Renderer", RF_Transactional);
				NewRenderer->Material = Cast<UMaterialInterface>(DefaultMaterial.TryLoad());
				StatelessEmitter->AddRenderer(NewRenderer, FGuid());// EmitterData->Version.VersionGuid);
			}
		};

	// This method uses the combined standard / stateless setup
	static bool bPureStatelessAdd = false;
	if (bPureStatelessAdd == false)
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> NewViewModel = AddEmptyEmitter();
		if (FNiagaraEmitterHandle* EmitterHandle = NewViewModel->GetEmitterHandle())
		{
			UNiagaraStatelessEmitter* NewEmitter = NewObject<UNiagaraStatelessEmitter>(&NiagaraSystem, "StatelessEmitter", RF_Transactional);
			SetupStatelessEmitter(NewEmitter, EmitterHandle->GetName());

			EmitterHandle->SetStatelessEmitter(NewEmitter);
			EmitterHandle->SetEmitterMode(NiagaraSystem, ENiagaraEmitterMode::Stateless);
		}
		return NewViewModel;
	}

	// This method is pure stateless only which might result in compilation / graph issue that need ironing out
	UNiagaraStatelessEmitter* NewEmitter = nullptr;
	{
		TSet<FName> EmitterHandleNames;
		for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem.GetEmitterHandles())
		{
			EmitterHandleNames.Add(EmitterHandle.GetName());
		}
		FName NewEmitterName = FName("Lightweight");
		NewEmitterName = FNiagaraUtilities::GetUniqueName(NewEmitterName, EmitterHandleNames);
		NewEmitter = NewObject<UNiagaraStatelessEmitter>(&NiagaraSystem, *NewEmitterName.ToString(), RF_Transactional);
		SetupStatelessEmitter(NewEmitter, NewEmitterName);
	}

	ResetEmitterHandleViewModelsAndTracks();

	GEditor->BeginTransaction(LOCTEXT("AddEmitter", "Add emitter"));
	FNiagaraEmitterHandle EmitterHandle = FNiagaraEmitterHandle(*NewEmitter);
	{
		FNiagaraEditorUtilities::KillSystemInstances(NiagaraSystem);
		NiagaraSystem.Modify();
		NiagaraSystem.AddEmitterHandleDirect(EmitterHandle);
		GetEditorData().SynchronizeOverviewGraphWithSystem(NiagaraSystem);
	}
	GEditor->EndTransaction();
	const FGuid NewEmitterHandleId = EmitterHandle.GetId();

	if (GetSystem().GetNumEmitters() == 1 && EditorSettings->GetAutoPlay() && Sequencer.IsValid())
	{
		// When adding a new emitter to an empty system start playing.
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
	}

	RefreshAll();
	
	TSharedPtr<FNiagaraEmitterHandleViewModel> NewEmitterHandleViewModel = GetEmitterHandleViewModelById(NewEmitterHandleId);
	return NewEmitterHandleViewModel;
}
TSharedPtr<FNiagaraEmitterHandleViewModel> FNiagaraSystemViewModel::AddEmitter(const FVersionedNiagaraEmitter& VersionedEmitter)
{
	return AddEmitter(*VersionedEmitter.Emitter, VersionedEmitter.Version);
}

TSharedPtr<FNiagaraEmitterHandleViewModel> FNiagaraSystemViewModel::AddMinimalEmitter()
{
	FSoftObjectPath MinimalEmitter = GetDefault<UNiagaraEditorSettings>()->DefaultEmptyEmitter;
	if(MinimalEmitter.IsValid() && MinimalEmitter.IsAsset())
	{
		if(UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(MinimalEmitter.TryLoad()))
		{
			return AddEmitterFromAssetData(FAssetData(Emitter));
		}
	}

	return AddEmptyEmitter();
}

void FNiagaraSystemViewModel::DuplicateEmitters(TArray<FEmitterHandleToDuplicate> EmitterHandlesToDuplicate)
{
	if (EmitterHandlesToDuplicate.Num() <= 0)
	{
		return;
	}

	// Kill all system instances and reset view models before modifying the emitter handle list to prevent accessing deleted data.
	ResetEmitterHandleViewModelsAndTracks();
	FNiagaraEditorUtilities::KillSystemInstances(GetSystem());
	const FScopedTransaction DeleteTransaction(EmitterHandlesToDuplicate.Num() == 1
		? LOCTEXT("DuplicateEmitter", "Duplicate emitter")
		: LOCTEXT("DuplicateEmitters", "Duplicate emitters"));

	TSet<FName> EmitterHandleNames;
	for (const FNiagaraEmitterHandle& EmitterHandle : GetSystem().GetEmitterHandles())
	{
		EmitterHandleNames.Add(EmitterHandle.GetName());
	}

	GetSystem().Modify();

	TMap<FGuid, UNiagaraSystem*> NewHandleToOriginalSystemMap;
	for (FEmitterHandleToDuplicate& EmitterHandleToDuplicate : EmitterHandlesToDuplicate)
	{
		UNiagaraSystem* SourceSystem = nullptr;
		FNiagaraEmitterHandle HandleToDuplicate;
		for (TObjectIterator<UNiagaraSystem> OtherSystemIt; OtherSystemIt; ++OtherSystemIt)
		{
			UNiagaraSystem* OtherSystem = *OtherSystemIt;
			if (OtherSystem->GetPathName() == EmitterHandleToDuplicate.SystemPath)
			{
				for (const FNiagaraEmitterHandle& EmitterHandle : OtherSystem->GetEmitterHandles())
				{
					if (EmitterHandle.GetId() == EmitterHandleToDuplicate.EmitterHandleId)
					{
						SourceSystem = OtherSystem;
						HandleToDuplicate = EmitterHandle;
						break;
					}
				}
			}

			if (SourceSystem != nullptr && HandleToDuplicate.IsValid())
			{
				break;
			}
		}

		if (SourceSystem != nullptr && HandleToDuplicate.IsValid())
		{
			const FNiagaraEmitterHandle& EmitterHandle = GetSystem().DuplicateEmitterHandle(HandleToDuplicate, FNiagaraUtilities::GetUniqueName(HandleToDuplicate.GetName(), EmitterHandleNames));
			FNiagaraScratchPadUtilities::FixExternalScratchPadScriptsForEmitter(*SourceSystem, EmitterHandle.GetInstance());
			EmitterHandleNames.Add(EmitterHandle.GetName());
			NewHandleToOriginalSystemMap.Add(EmitterHandle.GetId(), SourceSystem);
			if (EmitterHandleToDuplicate.OverviewNode)
			{
				EmitterHandleToDuplicate.OverviewNode->Initialize(&GetSystem(), EmitterHandle.GetId());
			}
		}
	}

	FNiagaraStackGraphUtilities::RebuildEmitterNodes(GetSystem());
	GetEditorData().SynchronizeOverviewGraphWithSystem(GetSystem());
	RefreshAll();
	bForceAutoCompileOnce = true;

	// We copy over referenced User Parameters now
	FNiagaraUserRedirectionParameterStore& TargetParameterStore = GetSystem().GetExposedParameters();

	// we have to keep track of parameters we have already added but renamed.
	// i.e. if we copy paste 2 emitters into a new system and both reference the same parameter, but there was already one with matching name and mismatched type
	// we create a new parameter with unique name _once_ that both emitters should point to.
	TMap<FNiagaraVariable, FNiagaraVariable> RenamedUserParameters;
	
	for(auto& Element : NewHandleToOriginalSystemMap)
	{
		// if the source system of the current emitter is the system we are currently editing, we skip the copy paste of user parameters
		if(Element.Value == &GetSystem())
		{
			continue;
		}
		
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = GetEmitterHandleViewModelById(Element.Key);
		TArray<FNiagaraVariable> ReferencedUserParameters = FNiagaraEditorUtilities::UserParameters::GetReferencedUserParametersFromEmitter(EmitterHandleViewModel->GetEmitterViewModel());

		UNiagaraSystem* OriginalSystem = Element.Value;
		
		for(const FNiagaraVariable& UserParameter : ReferencedUserParameters)
		{
			// in case we already have a parameter with the same name & type, we simply skip the parameter by default.
			// if the user wants to add a new User Parameter, he can do that too.
			if(TargetParameterStore.FindParameterVariable(UserParameter, false) != nullptr)
			{
				continue;
			}
			// if we have a parameter with the same name but a different type, we need to create a new parameter with a different name, copy the data and fixup the references
			// if we have already added a new unique user parameter with the same name, we reuse that parameter
			else if(TargetParameterStore.FindParameterVariable(UserParameter, true) != nullptr)
			{
				TArray<FNiagaraVariable> ExistingUserParameters;
				TargetParameterStore.GetUserParameters(ExistingUserParameters);

				TSet<FName> ExistingUserParameterNames;
				for(const FNiagaraVariable& ExistingUserParameter : ExistingUserParameters)
				{
					FNiagaraVariable RedirectedUserParameter = ExistingUserParameter;
					TargetParameterStore.RedirectUserVariable(RedirectedUserParameter);
					ExistingUserParameterNames.Add(RedirectedUserParameter.GetName());
				}

				bool bParameterWasRenamed = RenamedUserParameters.Contains(UserParameter);

				if(!bParameterWasRenamed)
				{
					FNiagaraVariable NewUserParameter = FNiagaraVariable(UserParameter.GetType(), FNiagaraUtilities::GetUniqueName(UserParameter.GetName(), ExistingUserParameterNames));
					TargetParameterStore.AddParameter(NewUserParameter);
					OriginalSystem->GetExposedParameters().CopyParameterData(TargetParameterStore, UserParameter, NewUserParameter);
					FNiagaraEditorUtilities::FNiagaraActionScopeConfig Config(Element.Key);
					FNiagaraEditorUtilities::UserParameters::ReplaceUserParameterReferences(AsShared(), UserParameter, NewUserParameter, Config);
					RenamedUserParameters.Add(UserParameter, NewUserParameter);
				}
				else
				{
					FNiagaraVariable RenamedUserParameter = RenamedUserParameters[UserParameter];
					FNiagaraEditorUtilities::FNiagaraActionScopeConfig Config(Element.Key);
					FNiagaraEditorUtilities::UserParameters::ReplaceUserParameterReferences(AsShared(), UserParameter, RenamedUserParameter, Config);
				}
			}
			// if the parameter doesn't exist at all, we just add it, copy the data and we're done
			else
			{
				const FNiagaraVariableWithOffset* ExistingUserParameter = OriginalSystem->GetExposedParameters().FindParameterVariable(UserParameter);
				if(ensure(ExistingUserParameter != nullptr))
				{
					TargetParameterStore.AddParameter(UserParameter);
					OriginalSystem->GetExposedParameters().CopyParameterData(GetSystem().GetExposedParameters(), UserParameter);
				}
			}
		}
	}

	if(RenamedUserParameters.Num() > 0)
	{
		TSharedRef<SVerticalBox> WidgetContent = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock).Text(LOCTEXT("RenamedParametersDuringPasteNotificationStart", "Renamed the following User Parameters on the new emitters.\n"))
		];
		
		for(auto& RenamedVariable : RenamedUserParameters)
		{
			WidgetContent->AddSlot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					FNiagaraParameterUtilities::GetParameterWidget(RenamedVariable.Key, false, false)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(3.f)
				[
					SNew(STextBlock).Text(LOCTEXT("RenamedParametersDuringPasteParameterConnector", "to"))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					FNiagaraParameterUtilities::GetParameterWidget(RenamedVariable.Value, false, false)
				]
			];
		}
		
		TSharedRef<FNiagaraParameterNotificationWidgetProvider> WidgetProvider = MakeShared<FNiagaraParameterNotificationWidgetProvider>(WidgetContent);
		FNotificationInfo Info(WidgetProvider);
		Info.WidthOverride = FOptionalSize();
		Info.ExpireDuration = 5.f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

FGuid FNiagaraSystemViewModel::GetMessageLogGuid() const
{
	return SystemMessageLogGuidKey.IsSet() ? SystemMessageLogGuidKey.GetValue() : FGuid();
}

FGuid FNiagaraSystemViewModel::AddMessage(UNiagaraMessageData* NewMessage) const
{
	FGuid NewGuid = FGuid::NewGuid();
	UNiagaraSystem& ViewedSystem = GetSystem();
	if (EditMode == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		ViewedSystem.GetMessageStore().AddMessage(NewGuid, static_cast<UNiagaraMessageDataBase*>(NewMessage));
	}
	else if (EditMode == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		if (ensureMsgf(EmitterHandleViewModels.Num() == 1, TEXT("There was not exactly 1 EmitterHandleViewModel for the SystemViewModel in Emitter edit mode!")))
		{
			EmitterHandleViewModels[0]->AddMessage(NewMessage, NewGuid);
		}
	}
	else
	{
		checkf(false, TEXT("New system viewmodel edit mode defined! Must implemented AddMessage() for this edit mode!"));
		return FGuid();
	}

	bPendingAssetMessagesChanged = true;
	return NewGuid;
}

void FNiagaraSystemViewModel::RemoveMessage(const FGuid& MessageKey) const
{
	UNiagaraSystem& ViewedSystem = GetSystem();
	if (EditMode == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		ViewedSystem.GetMessageStore().RemoveMessage(MessageKey);
	}
	else if (EditMode == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		if (ensureMsgf(EmitterHandleViewModels.Num() == 1, TEXT("There was not exactly 1 EmitterHandleViewModel for the SystemViewModel in Emitter edit mode!")))
		{
			EmitterHandleViewModels[0]->RemoveMessage(MessageKey);
		}
	}
	else
	{
		checkf(false, TEXT("New system viewmodel edit mode defined! Must implemented RemoveMessage() for this edit mode!"));
	}

	bPendingAssetMessagesChanged = true;
}

FGuid FNiagaraSystemViewModel::AddStackMessage(UNiagaraMessageData* NewMessage, UNiagaraNodeFunctionCall* TargetFunctionCallNode) const
{
	FGuid NewGuid = FGuid::NewGuid();
	TargetFunctionCallNode->GetMessageStore().AddMessage(NewGuid, NewMessage);
	bPendingAssetMessagesChanged = true;
	return NewGuid;
}

NIAGARAEDITOR_API void FNiagaraSystemViewModel::RemoveStackMessage(const FGuid& MessageKey, UNiagaraNodeFunctionCall* TargetFunctionCallNode) const
{
	TargetFunctionCallNode->GetMessageStore().RemoveMessage(MessageKey);
	bPendingAssetMessagesChanged = true;
}

void FNiagaraSystemViewModel::DismissMessageAndRefresh(FNiagaraMessageSourceAndStore MessageSourceAndStore, FGuid MessageKey)
{
	UObject* MessageSource = MessageSourceAndStore.GetSource();
	if (MessageSource != nullptr)
	{
		MessageSource->Modify();
		MessageSourceAndStore.GetStore()->DismissMessage(MessageKey);
		bPendingAssetMessagesChanged = true;
	}
}

void FNiagaraSystemViewModel::DeleteEmitters(TSet<FGuid> EmitterHandleIdsToDelete)
{
	if (EmitterHandleIdsToDelete.Num() > 0)
	{
		// Reset view models before modifying the emitter handle list to prevent accessing deleted data.
		ResetEmitterHandleViewModelsAndTracks();
		FNiagaraEditorUtilities::RemoveEmittersFromSystemByEmitterHandleId(GetSystem(), EmitterHandleIdsToDelete);
		RefreshAll();
		bForceAutoCompileOnce = true;
	}
}

TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> FNiagaraSystemViewModel::GetPinnedEmitterHandles()
{
	return PinnedEmitterHandles;
}

void FNiagaraSystemViewModel::SetEmitterPinnedState(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleModel, bool bPinnedState)
{
	if (bPinnedState)
	{
		PinnedEmitterHandles.AddUnique(EmitterHandleModel);
	}
	else
	{
		PinnedEmitterHandles.Remove(EmitterHandleModel);
	}
	OnPinnedChangedDelegate.Broadcast();
}

FNiagaraSystemViewModel::FOnPinnedEmittersChanged& FNiagaraSystemViewModel::GetOnPinnedEmittersChanged()
{
	return OnPinnedChangedDelegate;
}

bool FNiagaraSystemViewModel::GetIsEmitterPinned(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleModel)
{
	return PinnedEmitterHandles.ContainsByPredicate([=](TSharedRef<FNiagaraEmitterHandleViewModel> Model) {return Model==EmitterHandleModel;});
}


FNiagaraSystemViewModel::FOnEmitterHandleViewModelsChanged& FNiagaraSystemViewModel::OnEmitterHandleViewModelsChanged()
{
	return OnEmitterHandleViewModelsChangedDelegate;
}

FNiagaraSystemViewModel::FOnPostSequencerTimeChange& FNiagaraSystemViewModel::OnPostSequencerTimeChanged()
{
	return OnPostSequencerTimeChangeDelegate;
}

FNiagaraSystemViewModel::FOnSystemCompiled& FNiagaraSystemViewModel::OnSystemCompiled()
{
	return OnSystemCompiledDelegate;
}

void FNiagaraSystemViewModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (PreviewComponent != nullptr)
	{
		Collector.AddReferencedObject(PreviewComponent);
	}
	if (NiagaraSequence != nullptr)
	{
		Collector.AddReferencedObject(NiagaraSequence);
	}
	if (SystemStackViewModel != nullptr)
	{
		Collector.AddReferencedObject(SystemStackViewModel);
	}
	if (EditorDocumentsViewModel != nullptr)
	{
		Collector.AddReferencedObject(EditorDocumentsViewModel);
	}
	if (SelectionViewModel != nullptr)
	{
		Collector.AddReferencedObject(SelectionViewModel);
	}
	if (ScriptScratchPadViewModel != nullptr)
	{
		Collector.AddReferencedObject(ScriptScratchPadViewModel);
	}
	if (CurveSelectionViewModel != nullptr)
	{
		Collector.AddReferencedObject(CurveSelectionViewModel);
	}
	if (ScalabilityViewModel != nullptr)
	{
		Collector.AddReferencedObject(ScalabilityViewModel);
	}
	if (UserParametersHierarchyViewModel != nullptr)
	{
		Collector.AddReferencedObject(UserParametersHierarchyViewModel);
	}
}

void FNiagaraSystemViewModel::PostUndo(bool bSuccess)
{
	// Reset system stack and emitter handle view models to prevent accessing invalid data if they were in the undo operation.
	System->KillAllActiveCompilations();
	ResetStack();
	System->RequestCompile(false);
}

bool FNiagaraSystemViewModel::WaitingOnCompilation() const
{
	return GetSystem().HasOutstandingCompilationRequests(true);
}

void FNiagaraSystemViewModel::Tick(float DeltaTime)
{
	if (System == nullptr)
	{
		// If the system pointer is no longer valid, this system view model has been cleaned up and is invalid
		// and is about to be destroyed so don't run tick logic.
		return;
	}

	if (bCompilePendingCompletion && GetSystem().HasOutstandingCompilationRequests() == false)
	{
		bCompilePendingCompletion = false;
		OnSystemCompiled().Broadcast();
	}

	if (!SystemScriptViewModel.IsValid())
	{
		return;
	}

	TickCompileStatus();

	bool bAutoCompileThisFrame = GetDefault<UNiagaraEditorSettings>()->GetAutoCompile() && bCanAutoCompile && GetLatestCompileStatus() == ENiagaraScriptCompileStatus::NCS_Dirty;
	if ((bForceAutoCompileOnce || bAutoCompileThisFrame) && !WaitingOnCompilation())
	{
		CompileSystem(false);
		bForceAutoCompileOnce = false;
	}

	if (bResetRequestPending)
	{
		ResetSystem(ETimeResetMode::AllowResetTime, EMultiResetMode::AllowResetAllInstances, EReinitMode::ReinitializeSystem);
	}

	if (EmitterIdsRequiringSequencerTrackUpdate.Num() > 0)
	{
		UpdateSequencerTracksForEmitters(EmitterIdsRequiringSequencerTrackUpdate);
		EmitterIdsRequiringSequencerTrackUpdate.Empty();
	}

	if (SystemStackViewModel != nullptr)
	{
		SystemStackViewModel->Tick();
	}

	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
	{
		EmitterHandleViewModel->GetEmitterStackViewModel()->Tick();
	}

	if (SelectionViewModel != nullptr)
	{
		SelectionViewModel->Tick();
	}

	if (CurveSelectionViewModel != nullptr)
	{
		CurveSelectionViewModel->Tick();
	}

	if (bPendingAssetMessagesChanged)
	{
		bPendingAssetMessagesChanged = false;
		RefreshAssetMessages();
	}
}

void FNiagaraSystemViewModel::NotifyPreSave()
{
	if (System->HasOutstandingCompilationRequests(true))
	{
		UE_LOGF(LogNiagaraEditor, Log, "System %ls has pending compile jobs. Waiting for that code to complete before Saving..", *System->GetName());
		System->WaitForCompilationComplete(true);
	}

	// we want to save without compile for edit turned on for best results, but make sure to turn in back on again after save is done
	if (bSupportCompileForEdit && bIsCompileForEdit)
	{
		check(System->GetCompileForEdit());
		System->SetCompileForEdit(false);
		System->RequestCompile(false);
		System->WaitForCompilationComplete(true);
	}

	OnPreSaveDelegate.Broadcast();
}

void FNiagaraSystemViewModel::NotifyPostSave()
{
	// we compile for edit again after having it turned off for PreSave. This should fetch DDC data and must not update the ChangeID
	if (bSupportCompileForEdit && bIsCompileForEdit)
	{
		check(!System->GetCompileForEdit());
		System->SetCompileForEdit(true);
		System->RequestCompile(false);
		System->WaitForCompilationComplete(true);
	}
}

void FNiagaraSystemViewModel::NotifyPreClose()
{
	if (GetSystem().HasOutstandingCompilationRequests(true))
	{
		UE_LOGF(LogNiagaraEditor, Log, "System %ls has pending compile jobs. Waiting for that code to complete before Closing..", *GetSystem().GetName());
		GetSystem().WaitForCompilationComplete(true);
	}
	OnPreCloseDelegate.Broadcast();
}

FSimpleMulticastDelegate& FNiagaraSystemViewModel::OnPreSave()
{
	return OnPreSaveDelegate;
}

FNiagaraSystemViewModel::FOnPreClose& FNiagaraSystemViewModel::OnPreClose()
{
	return OnPreCloseDelegate;
}

FNiagaraSystemViewModel::FOnRequestFocusTab& FNiagaraSystemViewModel::OnRequestFocusTab()
{
	return OnRequestFocusTabDelegate;
}

void FNiagaraSystemViewModel::FocusTab(FName TabName, bool bDrawAttention)
{
	OnRequestFocusTabDelegate.Broadcast(TabName, bDrawAttention);
}

TSharedPtr<FUICommandList> FNiagaraSystemViewModel::GetToolkitCommands()
{
	return ToolkitCommands.Pin();
}

void FNiagaraSystemViewModel::SetToolkitCommands(const TSharedRef<FUICommandList>& InToolkitCommands)
{
	ToolkitCommands = InToolkitCommands;
}

const TArray<FNiagaraStackModuleData>& FNiagaraSystemViewModel::GetStackModuleDataByModuleEntry(const UNiagaraStackEntry* ModuleEntry)
{
	FGuid EmitterHandleId = FGuid();
	FVersionedNiagaraEmitter Emitter;
	if (ModuleEntry->GetEmitterViewModel().IsValid())
	{
		Emitter = ModuleEntry->GetEmitterViewModel()->GetEmitter();
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = GetEmitterHandleViewModelForEmitter(Emitter);
		if (ensureMsgf(EmitterHandleViewModel.IsValid(), TEXT("Failed to get the emitter handle view model for emitter %s while getting stack module data."), *Emitter.Emitter->GetPathName()))
		{
			EmitterHandleId = EmitterHandleViewModel->GetId();
		}
	}

	TArray<FNiagaraStackModuleData>* StackModuleData = GuidToCachedStackModuleData.Find(EmitterHandleId);
	return StackModuleData != nullptr ? *StackModuleData : BuildAndCacheStackModuleData(EmitterHandleId, Emitter);
}

const TArray<FNiagaraStackModuleData>& FNiagaraSystemViewModel::GetStackModuleDataByEmitterHandleId(FGuid EmitterHandleId)
{
	TArray<FNiagaraStackModuleData>* StackModuleData = GuidToCachedStackModuleData.Find(EmitterHandleId);
	if (StackModuleData != nullptr)
	{
		return *StackModuleData;
	}

	FVersionedNiagaraEmitter Emitter;
	if (EmitterHandleId.IsValid())
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = GetEmitterHandleViewModelById(EmitterHandleId);
		if (ensureMsgf(EmitterHandleViewModel.IsValid(), TEXT("Failed to get the emitter handle view model for emitter handle id %s while getting stack module data."), *EmitterHandleId.ToString(EGuidFormats::DigitsWithHyphens)))
		{
			Emitter = EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter();
		}
	}

	return BuildAndCacheStackModuleData(EmitterHandleId, Emitter);
}

const TArray<FNiagaraStackModuleData>& FNiagaraSystemViewModel::BuildAndCacheStackModuleData(FGuid EmitterHandleId, const FVersionedNiagaraEmitter& Emitter)
{
	TArray<FNiagaraStackModuleData>& StackModuleData = GuidToCachedStackModuleData.Add(EmitterHandleId);
	TArray<UNiagaraScript*> OrderedScripts;
	GetOrderedScriptsForEmitter(Emitter, OrderedScripts);
	for(UNiagaraScript* OrderedScript : OrderedScripts)
	{
		BuildStackModuleData(OrderedScript, EmitterHandleId, StackModuleData);
	}
	return StackModuleData;
}

TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> FNiagaraSystemViewModel::GetSelectedEmitterHandleViewModels()
{
	TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> SelectedEmitterHandleViewModels;
	const TArray<FGuid>& SelectedEmitterHandleIds = GetSelectionViewModel()->GetSelectedEmitterHandleIds();

	for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
	{
		if (SelectedEmitterHandleIds.Contains(EmitterHandleViewModel->GetId()))
		{
			SelectedEmitterHandleViewModels.Add(EmitterHandleViewModel);
		}
	}
	return SelectedEmitterHandleViewModels;
}

UNiagaraEditorParametersAdapter* FNiagaraSystemViewModel::GetEditorOnlyParametersAdapter() const
{
	if (EditMode == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		return CastChecked<UNiagaraEditorParametersAdapter>(System->GetEditorParameters());
	}
	/** else EditMode == ENiagaraSystemViewModelEditMode::EmitterAsset */
	FVersionedNiagaraEmitterData* EmitterData = GetEmitterHandleViewModels()[0]->GetEmitterHandle()->GetEmitterData();
	return EmitterData ? CastChecked<UNiagaraEditorParametersAdapter>(EmitterData->GetEditorParameters()) : nullptr;
}

void FNiagaraSystemViewModel::GetOrderedScriptsForEmitterHandleId(FGuid EmitterHandleId, TArray<UNiagaraScript*>& OutScripts)
{
	FVersionedNiagaraEmitter Emitter;
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = GetEmitterHandleViewModelById(EmitterHandleId);
	if (EmitterHandleViewModel.IsValid())
	{
		Emitter = EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter();
	}
	GetOrderedScriptsForEmitter(Emitter, OutScripts);
}

void FNiagaraSystemViewModel::GetOrderedScriptsForEmitter(const FVersionedNiagaraEmitter& Emitter, TArray<UNiagaraScript*>& OutScripts)
{
	OutScripts.Add(System->GetSystemSpawnScript());
	OutScripts.Add(System->GetSystemUpdateScript());
	FVersionedNiagaraEmitterData* EmitterData = Emitter.GetEmitterData();
	if (EmitterData != nullptr)
	{
		OutScripts.Add(EmitterData->EmitterSpawnScriptProps.Script);
		OutScripts.Add(EmitterData->EmitterUpdateScriptProps.Script);
		OutScripts.Add(EmitterData->SpawnScriptProps.Script);
		OutScripts.Add(EmitterData->UpdateScriptProps.Script);
		for (UNiagaraSimulationStageBase* SimulationStage : EmitterData->GetSimulationStages())
		{
			if (SimulationStage->bEnabled)
			{
				OutScripts.Add(SimulationStage->Script);
			}
		}
	}
}

UNiagaraStackViewModel* FNiagaraSystemViewModel::GetSystemStackViewModel()
{
	return SystemStackViewModel;
}

UNiagaraSystemSelectionViewModel* FNiagaraSystemViewModel::GetSelectionViewModel()
{
	return SelectionViewModel;
}

UNiagaraScratchPadViewModel* FNiagaraSystemViewModel::GetScriptScratchPadViewModel()
{
	return ScriptScratchPadViewModel;
}

UNiagaraCurveSelectionViewModel* FNiagaraSystemViewModel::GetCurveSelectionViewModel()
{
	return CurveSelectionViewModel;
}

UNiagaraSystemScalabilityViewModel* FNiagaraSystemViewModel::GetScalabilityViewModel()
{
	return ScalabilityViewModel;
}

UNiagaraUserParametersHierarchyViewModel* FNiagaraSystemViewModel::GetUserParametersHierarchyViewModel()
{
	return UserParametersHierarchyViewModel;
}

TArray<float> FNiagaraSystemViewModel::OnGetPlaybackSpeeds() const
{
	return GetDefault<UNiagaraEditorSettings>()->GetPlaybackSpeeds();
}

TStatId FNiagaraSystemViewModel::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraSystemViewModel, STATGROUP_Tickables);
}

void FNiagaraSystemViewModel::OnModifiedIndirectly(UMovieSceneSignedObject* MovieSceneSignedObject)
{
	SequencerMovieSceneModified(Cast<UMovieScene>(MovieSceneSignedObject));
}

void FNiagaraSystemViewModel::OnModifiedDirectly(UMovieSceneSignedObject* MovieSceneSignedObject)
{
	SequencerMovieSceneModified(Cast<UMovieScene>(MovieSceneSignedObject));
}

TSharedRef<FNiagaraPlaceholderDataInterfaceManager> FNiagaraSystemViewModel::GetPlaceholderDataInterfaceManager()
{
	return PlaceholderDataInterfaceManager.ToSharedRef();
}

bool FNiagaraSystemViewModel::GetIsForDataProcessingOnly() const
{
	return bIsForDataProcessingOnly;
}

void FNiagaraSystemViewModel::AddParameter(const FNiagaraVariableBase& InUserParameter)
{
	FScopedTransaction RenameParameterTransaction(LOCTEXT("AddParameterTransaction", "Add parameter"));
	System->Modify();
	System->GetExposedParameters().AddParameter(InUserParameter);
}

bool FNiagaraSystemViewModel::RenameParameter(const FNiagaraVariable TargetParameter, const FName NewName, ENiagaraGetGraphParameterReferencesMode RenameScopeMode)
{
	FScopedTransaction RenameParameterTransaction(LOCTEXT("RenameParameter", "Rename parameter"));

	System->Modify();

	bool bExposedParametersRename = false;
	bool bEditorOnlyParametersRename = false;
	bool bAssignmentNodeRename = false;

	// Rename the parameter if it's in the parameter store for user parameters.
	if (System->GetExposedParameters().IndexOf(TargetParameter) != INDEX_NONE)
	{
		FNiagaraParameterStore* ExposedParametersStore = &System->GetExposedParameters();
		TArray<FNiagaraVariable> OwningParameters;
		ExposedParametersStore->GetParameters(OwningParameters);
		if (OwningParameters.ContainsByPredicate([NewName](const FNiagaraVariable& Variable) { return Variable.GetName() == NewName; }))
		{
			// If the parameter store already has a parameter with this name, remove the old parameter to prevent collisions.
			ExposedParametersStore->RemoveParameter(TargetParameter);
		}
		else
		{
			// Otherwise it's safe to rename.
			GetEditorData().RenameUserScriptVariable(TargetParameter, NewName);
			ExposedParametersStore->RenameParameter(TargetParameter, NewName);
		}

		FNiagaraVariable NewVariable = TargetParameter;
		NewVariable.SetName(NewName);

		FNiagaraEditorUtilities::FNiagaraActionScopeConfig Config;
		Config.bIncludeSystem = true;
		Config.bIncludeEmitters = true;
		FNiagaraEditorUtilities::UserParameters::ReplaceUserParameterReferences(AsShared(), TargetParameter, NewVariable, Config);
		
		bExposedParametersRename = true;
	}

	// Look for set parameters nodes or linked inputs which reference this parameter and rename if so.
	for (const FNiagaraGraphParameterReference& ParameterReference : GetGraphParameterReferences(TargetParameter, RenameScopeMode))
	{
		UNiagaraNode* ReferenceNode = Cast<UNiagaraNode>(ParameterReference.Value);
		if (ReferenceNode != nullptr)
		{
			UNiagaraNodeAssignment* OwningAssignmentNode = ReferenceNode->GetTypedOuter<UNiagaraNodeAssignment>();
			if (OwningAssignmentNode != nullptr)
			{
				// If this is owned by a set variables node and it's not locked, update the assignment target on the assignment node.
				bAssignmentNodeRename |= FNiagaraStackGraphUtilities::TryRenameAssignmentTarget(*OwningAssignmentNode, TargetParameter, NewName);
			}
			else
			{
				// Otherwise if the reference node is a get node it's for a linked input so we can just update pin name.
				UNiagaraNodeParameterMapGet* ReferenceGetNode = Cast<UNiagaraNodeParameterMapGet>(ReferenceNode);
				if (ReferenceGetNode != nullptr)
				{
					if (ReferenceGetNode->Pins.ContainsByPredicate([&ParameterReference](UEdGraphPin* Pin) { return Pin->PersistentGuid == ParameterReference.Key; }))
					{
						ReferenceGetNode->GetNiagaraGraph()->RenameParameter(TargetParameter, NewName, true);
					}
				}
			}
		}
	}

	// Rename the parameter if it is owned directly as an editor only parameter.
	UNiagaraEditorParametersAdapter* EditorParametersAdapter = GetEditorOnlyParametersAdapter();
	if (auto* ScriptVariablePtr = EditorParametersAdapter->GetParameters().FindByPredicate([&TargetParameter](const UNiagaraScriptVariable* ScriptVariable) { return ScriptVariable->Variable.GetName() == TargetParameter.GetName(); }))
	{
		EditorParametersAdapter->Modify();
		UNiagaraScriptVariable* ScriptVariable = *ScriptVariablePtr;
		ScriptVariable->Modify();
		ScriptVariable->Variable.SetName(NewName);
		ScriptVariable->UpdateChangeId();
		bEditorOnlyParametersRename = true;

		// Check if the rename will give the same name and type as an existing parameter definition, and if so, link to the definition automatically.
		FNiagaraParameterDefinitionsUtilities::TrySubscribeScriptVarToDefinitionByName(ScriptVariable, this);
	}

	// Handle renaming any renderer properties that might match.
	if (bExposedParametersRename | bEditorOnlyParametersRename | bAssignmentNodeRename)
	{
		if (TargetParameter.IsInNameSpace(FNiagaraConstants::ParticleAttributeNamespaceString) || TargetParameter.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString))
		{
			for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : GetSelectedEmitterHandleViewModels())
			{
				FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
				EmitterHandle->GetInstance().Emitter->HandleVariableRenamed(TargetParameter, FNiagaraVariableBase(TargetParameter.GetType(), NewName), true, EmitterHandle->GetInstance().Version);
				EmitterHandleViewModel->GetEmitterStackViewModel()->RequestRefreshDeferred();
			}
		}
		else
		{
			System->HandleVariableRenamed(TargetParameter, FNiagaraVariableBase(TargetParameter.GetType(), NewName), true);
		}

		// changing a parameter name can result in conflicts between the runtime data and the parameter stores that we've just
		// changed.  Currently the safest way to deal with that is a recompile of the system
		CompileSystem(false);

		GetSystemStackViewModel()->RequestRefreshDeferred();
		return true;
	}

	return false;
}

bool FNiagaraSystemViewModel::DeleteParameter(const FNiagaraVariableBase Parameter)
{
	FScopedTransaction RenameParameterTransaction(LOCTEXT("DeleteParameter", "Delete parameter"));

	System->Modify();

	// Delete the parameter if it's in the parameter store for user parameters.
	if (System->GetExposedParameters().IndexOf(Parameter) != INDEX_NONE)
	{
		FNiagaraEditorUtilities::FNiagaraActionScopeConfig Config;
		Config.bIncludeSystem = true;
		Config.bIncludeEmitters = true;
		FNiagaraEditorUtilities::UserParameters::DeleteUserParameterReferences(AsShared(), Parameter, Config);
		
		System->GetExposedParameters().RemoveParameter(Parameter);
	}
	else
	{
		if (UNiagaraEditorParametersAdapter* EditorParametersAdapter = GetEditorOnlyParametersAdapter())
		{
			auto MatchingParameter = [&Parameter](const UNiagaraScriptVariable* ScriptVariable) -> bool
			{
				return ScriptVariable && ScriptVariable->Variable.IsEquivalent(Parameter);
			};

			EditorParametersAdapter->Modify();
			EditorParametersAdapter->GetParameters().RemoveAll(MatchingParameter);
		}
	}

	System->HandleVariableRemoved(Parameter, true);

	return true;
}

bool FNiagaraSystemViewModel::SetParameterDefaultValue(const FNiagaraVariableBase& InUserParameter, const FNiagaraVariant& InDefaultValue)
{
	int32 ParameterOffset = System->GetExposedParameters().IndexOf(InUserParameter);
	if (ParameterOffset == INDEX_NONE)
	{
		return false;
	}

	const FNiagaraTypeDefinition& ParameterType = InUserParameter.GetType();
	if (ParameterType.IsDataInterface() && (InDefaultValue.GetDataInterface() == nullptr || InDefaultValue.GetDataInterface()->GetClass() != ParameterType.GetClass()))
	{
		return false;
	}
	if (ParameterType.GetStruct() != nullptr && InDefaultValue.GetNumBytes() != ParameterType.GetSize())
	{
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("SetUserParameterDefaultValueTransaction", "Set User Parameter Default"));
	System->Modify();
	if (ParameterType.IsDataInterface())
	{
		UNiagaraDataInterface* DataInterface = System->GetExposedParameters().GetDataInterface(ParameterOffset);
		DataInterface->Modify();
		InDefaultValue.GetDataInterface()->CopyTo(DataInterface);
	}
	else if (ParameterType.IsUObject())
	{
		System->GetExposedParameters().SetUObject(InDefaultValue.GetUObject(), ParameterOffset);
	}
	else
	{
		if (InUserParameter.GetType() == FNiagaraTypeDefinition::GetPositionDef() && InDefaultValue.GetNumBytes() == sizeof(FVector))
		{
			FVector* PositionVector = (FVector*)InDefaultValue.GetBytes();
			System->GetExposedParameters().SetPositionParameterValue(*PositionVector, InUserParameter.GetName());
		}
		else
		{
			System->GetExposedParameters().SetParameterData(InDefaultValue.GetBytes(), ParameterOffset, InDefaultValue.GetNumBytes());
		}
	}
	return true;
}

TArray<UNiagaraGraph*> FNiagaraSystemViewModel::GetAllGraphs()
{
	TArray<UNiagaraGraph*> OutGraphs;

	// Helper lambda to null check graph weak object ptrs before adding them to the retval array.
	auto AddToOutGraphsChecked = [&OutGraphs](const TWeakObjectPtr<UNiagaraGraph>& WeakGraph) {
		UNiagaraGraph* Graph = WeakGraph.Get();
		if (Graph == nullptr)
		{
			ensureMsgf(false, TEXT("Encountered null graph when gathering editable graphs for system parameter panel viewmodel!"));
			return;
		}
		OutGraphs.Add(Graph);
	};

	if (GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		TArray<UNiagaraGraph*> Graphs;
		AddToOutGraphsChecked(GetSystemScriptViewModel()->GetGraphViewModel()->GetGraph());

		for (const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
		{
			FNiagaraEmitterHandle* EmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle ? EmitterHandle->GetEmitterData() : nullptr;
			if (EmitterData && EmitterData->GraphSource)
			{
				if (UNiagaraGraph* Graph = Cast<UNiagaraScriptSource>(EmitterData->GraphSource)->NodeGraph)
				{
					OutGraphs.Add(Graph);
				}
			}
		}
	}
	else
	{
		FNiagaraEmitterHandle* EmitterHandle = GetEmitterHandleViewModels()[0]->GetEmitterHandle();
		FVersionedNiagaraEmitterData* EmitterData = EmitterHandle ? EmitterHandle->GetEmitterData() : nullptr;
		if (EmitterData && EmitterData->GraphSource)
		{
			if (UNiagaraGraph* Graph = Cast<UNiagaraScriptSource>(EmitterData->GraphSource)->NodeGraph)
			{
				OutGraphs.Add(Graph);
			}
		}
	}

	TArray<UNiagaraGraph*> EdGraphs = GetDocumentViewModel()->GetEditableGraphsForActiveScriptDocument();
	for (UNiagaraGraph* Graph : EdGraphs)
	{
		OutGraphs.AddUnique(Graph);
	}
	return OutGraphs;
}

TArray<UNiagaraGraph*> FNiagaraSystemViewModel::GetSelectedGraphs()
{
	TArray<UNiagaraGraph*> SelectedGraphs;

	// Helper lambda to null check graph weak object ptrs before adding them to the retval array.
	auto AddToSelectedGraphsChecked = [&SelectedGraphs](const TWeakObjectPtr<UNiagaraGraph>& WeakGraph) {
		UNiagaraGraph* Graph = WeakGraph.Get();
		if (Graph == nullptr)
		{
			ensureMsgf(false, TEXT("Encountered null graph when gathering editable graphs for system parameter panel viewmodel!"));
			return;
		}
		SelectedGraphs.Add(Graph);
	};

	if (GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		for (const TWeakObjectPtr<UNiagaraGraph>& WeakGraph : SystemGraphSelectionViewModel->GetSelectedEmitterScriptGraphs())
		{
			AddToSelectedGraphsChecked(WeakGraph);
		}
		AddToSelectedGraphsChecked(GetSystemScriptViewModel()->GetGraphViewModel()->GetGraph());
	}
	else
	{
		if (FVersionedNiagaraEmitterData* EmitterData = GetEmitterHandleViewModels()[0]->GetEmitterHandle()->GetEmitterData())
		{
			SelectedGraphs.Add(CastChecked<UNiagaraScriptSource>(EmitterData->GraphSource)->NodeGraph);
		}
	}

	TArray<UNiagaraGraph*> EdGraphs = GetDocumentViewModel()->GetEditableGraphsForActiveScriptDocument();
	for (UNiagaraGraph* Graph : EdGraphs)
	{
		SelectedGraphs.AddUnique(Graph);
	}
	return SelectedGraphs;
}

TArray<FNiagaraGraphParameterReference> FNiagaraSystemViewModel::GetGraphParameterReferences(const FNiagaraVariable& Parameter, ENiagaraGetGraphParameterReferencesMode Mode)
{
	TArray<UNiagaraGraph*> TargetGraphs;
	if (Mode == ENiagaraGetGraphParameterReferencesMode::AllGraphs)
	{
		TargetGraphs = GetAllGraphs();
	}
	else if (Mode == ENiagaraGetGraphParameterReferencesMode::SelectedGraphs)
	{
		TargetGraphs = GetSelectedGraphs();
	}
	else
	{
		ensureMsgf(false, TEXT("Encountered unexpected ENiagaraGetGraphParameterReferencesMode value!"));
		return TArray<FNiagaraGraphParameterReference>();
	}

	// -For each selected graph perform a parameter map history traversal and collect all graph parameter references associated with the target FNiagaraParameterPanelItem.
	TArray<FNiagaraGraphParameterReference> GraphParameterReferences;
	for (const UNiagaraGraph* Graph : TargetGraphs)
	{
		TArray<UNiagaraNodeOutput*> OutputNodes;
		Graph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);
		for (UNiagaraNodeOutput* OutputNode : OutputNodes)
		{
			UNiagaraNode* NodeToTraverse = OutputNode;
			if (OutputNode->GetUsage() == ENiagaraScriptUsage::SystemSpawnScript || OutputNode->GetUsage() == ENiagaraScriptUsage::SystemUpdateScript)
			{
				// Traverse past the emitter nodes, otherwise the system scripts will pick up all of the emitter and particle script parameters.
				UEdGraphPin* InputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*NodeToTraverse);
				while (NodeToTraverse != nullptr && InputPin != nullptr && InputPin->LinkedTo.Num() == 1 &&
					(NodeToTraverse->IsA<UNiagaraNodeOutput>() || NodeToTraverse->IsA<UNiagaraNodeEmitter>()))
				{
					NodeToTraverse = Cast<UNiagaraNode>(InputPin->LinkedTo[0]->GetOwningNode());
					InputPin = NodeToTraverse != nullptr ? FNiagaraStackGraphUtilities::GetParameterMapInputPin(*NodeToTraverse) : nullptr;
				}
			}

			if (NodeToTraverse == nullptr)
			{
				continue;
			}

			bool bIgnoreDisabled = true;
			FNiagaraParameterMapHistoryBuilder Builder;
			FVersionedNiagaraEmitter GraphOwningEmitter = Graph->GetOwningEmitter();
			FCompileConstantResolver ConstantResolver = GraphOwningEmitter.Emitter != nullptr
				? FCompileConstantResolver(GraphOwningEmitter, OutputNode->GetUsage())
				: FCompileConstantResolver();

			Builder.SetIgnoreDisabled(bIgnoreDisabled);
			*Builder.ConstantResolver = ConstantResolver;
			FName StageName;
			ENiagaraScriptUsage StageUsage = OutputNode->GetUsage();
			if (StageUsage == ENiagaraScriptUsage::ParticleSimulationStageScript && GraphOwningEmitter.Emitter)
			{
				UNiagaraSimulationStageBase* Base = GraphOwningEmitter.GetEmitterData()->GetSimulationStageById(OutputNode->GetUsageId());
				if (Base)
				{
					StageName = Base->GetStackContextReplacementName();
				}
			}
			Builder.BeginUsage(StageUsage, StageName);
			NodeToTraverse->BuildParameterMapHistory(Builder, true, false);
			Builder.EndUsage();

			if (Builder.Histories.Num() != 1)
			{
				// We should only have traversed one emitter (have not visited more than one NiagaraNodeEmitter.)
				UE_LOGF(LogNiagaraEditor, Warning, "Encountered more than one parameter map history when collecting graph parameter reference collections for system parameter panel view model!");
			}
			if (Builder.Histories.Num() == 0)
			{
				continue;
			}

			const TArray<FName>& CustomIterationSourceNamespaces = Builder.Histories[0].IterationNamespaceOverridesEncountered;
			for (int32 VariableIndex = 0; VariableIndex < Builder.Histories[0].Variables.Num(); VariableIndex++)
			{
				if (Parameter == Builder.Histories[0].Variables[VariableIndex])
				{
					for (const FNiagaraParameterMapHistory::FReadHistory& ReadHistory : Builder.Histories[0].PerVariableReadHistory[VariableIndex])
					{
						if (ReadHistory.ReadPin.Pin->GetOwningNode() != nullptr)
						{
							GraphParameterReferences.Add(FNiagaraGraphParameterReference(ReadHistory.ReadPin.Pin->PersistentGuid, ReadHistory.ReadPin.Pin->GetOwningNode()));
						}
					}

					for (const FNiagaraParameterMapHistory::FModuleScopedPin& Write : Builder.Histories[0].PerVariableWriteHistory[VariableIndex])
					{
						if (Write.Pin->GetOwningNode() != nullptr)
						{
							GraphParameterReferences.Add(FNiagaraGraphParameterReference(Write.Pin->PersistentGuid, Write.Pin->GetOwningNode()));
						}
					}
				}
			}
		}
	}
	return GraphParameterReferences;
}

void FNiagaraSystemViewModel::GetSystemMessageStores(TArray<FNiagaraMessageSourceAndStore>& OutMessageStores)
{
	OutMessageStores.Add(FNiagaraMessageSourceAndStore(*System, System->GetMessageStore()));
	FNiagaraEditorUtilities::GetScriptMessageStores(System->GetSystemSpawnScript(), OutMessageStores);
	FNiagaraEditorUtilities::GetScriptMessageStores(System->GetSystemUpdateScript(), OutMessageStores);
}

void FNiagaraSystemViewModel::RefreshAssetMessagesDeferred()
{
	bPendingAssetMessagesChanged = true;
}

void FNiagaraSystemViewModel::SendLastCompileMessageJobs() const
{
	if (SystemMessageLogGuidKey.IsSet() == false)
	{
		return;
	}

	struct FNiagaraScriptAndOwningScriptNameString
	{
		FNiagaraScriptAndOwningScriptNameString(const UNiagaraScript* InScript, const FString& InOwningScriptNameString)
			: Script(InScript)
			, OwningScriptNameString(InOwningScriptNameString)
		{
		}

		const UNiagaraScript* Script;
		const FString OwningScriptNameString;
	};

	FNiagaraMessageManager* MessageManager = FNiagaraMessageManager::Get();
	int32 ErrorCount = 0;
	int32 WarningCount = 0;

	TArray<FNiagaraScriptAndOwningScriptNameString> ScriptsToGetCompileEventsFrom;
	ScriptsToGetCompileEventsFrom.Add(FNiagaraScriptAndOwningScriptNameString(GetSystem().GetSystemSpawnScript(), GetSystem().GetName()));
	ScriptsToGetCompileEventsFrom.Add(FNiagaraScriptAndOwningScriptNameString(GetSystem().GetSystemUpdateScript(), GetSystem().GetName()));
	const TArray<FNiagaraEmitterHandle> EmitterHandles = GetSystem().GetEmitterHandles();
	for (const FNiagaraEmitterHandle& Handle : EmitterHandles)
	{
		FVersionedNiagaraEmitter EmitterInSystem = Handle.GetInstance();
		//-TODO:Stateless: Do we need stateless support here?
		if (FVersionedNiagaraEmitterData* EmitterData = EmitterInSystem.GetEmitterData())
		{
			TArray<UNiagaraScript*> EmitterScripts;
			EmitterData->GetScripts(EmitterScripts, false);
			for (UNiagaraScript* EmitterScript : EmitterScripts)
			{
				ScriptsToGetCompileEventsFrom.Add(FNiagaraScriptAndOwningScriptNameString(EmitterScript, EmitterInSystem.Emitter->GetUniqueEmitterName()));
			}
		}
	}

	// Clear out existing compile event messages.
	MessageManager->ClearAssetMessagesForTopic(SystemMessageLogGuidKey.GetValue(), FNiagaraMessageTopics::CompilerTopicName);

	// Make new messages and messages jobs from the compile.
	// Iterate from back to front to avoid reordering the events when they are queued.
	for (int i = ScriptsToGetCompileEventsFrom.Num()-1; i >=0; --i)
	{
		const FNiagaraScriptAndOwningScriptNameString& ScriptInfo = ScriptsToGetCompileEventsFrom[i];
		const TArray<FNiagaraCompileEvent>& CurrentCompileEvents = ScriptInfo.Script->GetVMExecutableData().LastCompileEvents;
		for (int j = CurrentCompileEvents.Num() - 1; j >= 0; --j)
		{
			const FNiagaraCompileEvent& CompileEvent = CurrentCompileEvents[j];
			if (CompileEvent.Severity == FNiagaraCompileEventSeverity::Error)
			{
				ErrorCount++;
			}
			else if (CompileEvent.Severity == FNiagaraCompileEventSeverity::Warning)
			{
				WarningCount++;
			}

			MessageManager->AddMessageJob(MakeUnique<FNiagaraMessageJobCompileEvent>(CompileEvent, MakeWeakObjectPtr(const_cast<UNiagaraScript*>(ScriptInfo.Script)), FGuid(), ScriptInfo.OwningScriptNameString), SystemMessageLogGuidKey.GetValue());
		}

		// Check if there are any GPU compile errors and if so push them.
		const FNiagaraShaderScript* ShaderScript = ScriptInfo.Script->GetRenderThreadScript();
		if (ShaderScript != nullptr && ShaderScript->IsCompilationFinished() &&
			ScriptInfo.Script->Usage == ENiagaraScriptUsage::ParticleGPUComputeScript)
		{
			const TArray<FString>& GPUCompileErrors = ShaderScript->GetCompileErrors();
			for (const FString& String : GPUCompileErrors)
			{
				FNiagaraCompileEventSeverity Severity = FNiagaraCompileEventSeverity::Warning;
				if (String.Contains(TEXT("err0r")))
				{
					Severity = FNiagaraCompileEventSeverity::Error;
					ErrorCount++;
				}
				else
				{
					WarningCount++;
				}
				FNiagaraCompileEvent CompileEvent = FNiagaraCompileEvent(Severity, String);
				MessageManager->AddMessageJob(MakeUnique<FNiagaraMessageJobCompileEvent>(CompileEvent, MakeWeakObjectPtr(const_cast<UNiagaraScript*>(ScriptInfo.Script)), FGuid(),
					ScriptInfo.OwningScriptNameString), SystemMessageLogGuidKey.GetValue());
			}
		}
	}

	const FText PostCompileSummaryText = FNiagaraMessageUtilities::MakePostCompileSummaryText(FText::FromString("System"), GetLatestCompileStatus(), WarningCount, ErrorCount);
	MessageManager->AddMessage(MakeShared<FNiagaraMessageText>(PostCompileSummaryText, EMessageSeverity::Info, FNiagaraMessageTopics::CompilerTopicName), SystemMessageLogGuidKey.GetValue());
}

void FNiagaraSystemViewModel::InvalidateCachedCompileStatus()
{
	LatestCompileStatusCache.Reset();
	ScriptsToCheckForStatus.Empty();
	ScriptCompileStatuses.Empty();
}

void FNiagaraSystemViewModel::TickCompileStatus()
{
	// Checking the compile status is expensive which is why it's updated on tick, and multiple scripts
	// are time sliced across multiple frames.
	if (WaitingOnCompilation())
	{
		// When compiling the compile status is always unknown.
		InvalidateCachedCompileStatus();
	}
	else
	{
		if (LatestCompileStatusCache.IsSet() == false)
		{
			bool bCompileStatusRefreshed = false;
			if (ScriptsToCheckForStatus.Num() > 0)
			{
				// If there are still scripts to check for status do that...
				UNiagaraScript* ScriptToCheck = ScriptsToCheckForStatus.Last();
				ScriptsToCheckForStatus.RemoveAt(ScriptsToCheckForStatus.Num() - 1);

				if (ScriptToCheck->AreScriptAndSourceSynchronized() == false)
				{
					// If any script is not synchronized the entire system is considered dirty and we no longer need
					// to check other script statuses.
					LatestCompileStatusCache = ENiagaraScriptCompileStatus::NCS_Dirty;
					bCompileStatusRefreshed = true;
					ScriptsToCheckForStatus.Empty();
					ScriptCompileStatuses.Empty();
				}
				else
				{
					ENiagaraScriptCompileStatus LastScriptCompileStatus = ScriptToCheck->GetLastCompileStatus();
					if (LastScriptCompileStatus == ENiagaraScriptCompileStatus::NCS_Unknown ||
						LastScriptCompileStatus == ENiagaraScriptCompileStatus::NCS_Dirty ||
						LastScriptCompileStatus == ENiagaraScriptCompileStatus::NCS_BeingCreated)
					{
						// If the script doesn't have a vaild last compile status assume that it's dirty and by extension
						// so is the system and we can ignore the other statuses.
						LatestCompileStatusCache = ENiagaraScriptCompileStatus::NCS_Dirty;
						bCompileStatusRefreshed = true;
						ScriptsToCheckForStatus.Empty();
						ScriptCompileStatuses.Empty();
					}
					else
					{
						// Otherwise save it for processing once all scripts are done.
						ScriptCompileStatuses.Add(LastScriptCompileStatus);
					}
				}
			}
			else
			{
				// Otherwise check for status results.
				if (ScriptCompileStatuses.Num() > 0)
				{
					// If there are statues from scripts figure out the system status.
					if (ScriptCompileStatuses.Contains(ENiagaraScriptCompileStatus::NCS_Error))
					{
						// If any individual script has an error the system is considered to have an error.
						LatestCompileStatusCache = ENiagaraScriptCompileStatus::NCS_Error;
						bCompileStatusRefreshed = true;
					}
					else if (ScriptCompileStatuses.Contains(ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings) ||
						ScriptCompileStatuses.Contains(ENiagaraScriptCompileStatus::NCS_ComputeUpToDateWithWarnings))
					{
						// If there were no errors and any individual script has a warning the system is considered to have a warning.
						LatestCompileStatusCache = ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings;
						bCompileStatusRefreshed = true;
					}
					else
					{
						LatestCompileStatusCache = ENiagaraScriptCompileStatus::NCS_UpToDate;
						bCompileStatusRefreshed = true;
					}
					ScriptCompileStatuses.Empty();
				}
				else
				{
					// If there is no current status, no scripts to check, and no status results then
					// collect up the scripts to check deferred.
					ScriptsToCheckForStatus.Add(System->GetSystemSpawnScript());
					ScriptsToCheckForStatus.Add(System->GetSystemUpdateScript());
					for (const FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
					{
						//-TODO:Stateless: Do we need stateless support here?
						if (EmitterHandle.GetIsEnabled() && EmitterHandle.GetEmitterData())
						{
							EmitterHandle.GetEmitterData()->GetScripts(ScriptsToCheckForStatus, true);
						}
					}
				}
			}

			if (bCompileStatusRefreshed)
			{
				SendLastCompileMessageJobs();
				RefreshStackViewModels();
				bPendingAssetMessagesChanged = true;
			}
		}
	}
}

void FNiagaraSystemViewModel::SetupPreviewComponentAndInstance()
{
	if (bCanSimulate)
	{
		PreviewComponent = NewObject<UNiagaraComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		PreviewComponent->CastShadow = 1;
		PreviewComponent->bCastDynamicShadow = 1;
		PreviewComponent->SetAllowScalability(false);
		PreviewComponent->SetAsset(System);
		PreviewComponent->SetForceSolo(true);
		PreviewComponent->SetAgeUpdateMode(ENiagaraAgeUpdateMode::DesiredAge);
		PreviewComponent->SetSeekDelta((float)GetEditorData().GetPlaybackFrameRate().AsInterval());
		PreviewComponent->SetLockDesiredAgeDeltaTimeToSeekDelta(GetEditorData().GetLockPlaybackFrameRate());
		PreviewComponent->SetCanRenderWhileSeeking(false);
		PreviewComponent->Activate(true);

		FTransform OwnerTransform = GetEditorData().GetOwnerTransform();
		PreviewComponent->SetRelativeTransform(OwnerTransform);

		PreviewComponent->OnSystemInstanceChanged().AddRaw(this, &FNiagaraSystemViewModel::PreviewComponentSystemInstanceChanged);
		PreviewComponentSystemInstanceChanged();
	}
}

void FNiagaraSystemViewModel::RefreshAll()
{
	if (GetSystem().HasOutstandingCompilationRequests() == false)
	{
		// Data changes which require full refreshes can cause stability problems when resetting the accompanying system, especially
		// when used in conjunction with warm up.  This is a temporary fix for these crashes.
		CompileSystem(false);
	}
	ResetSystem(ETimeResetMode::AllowResetTime, EMultiResetMode::AllowResetAllInstances, EReinitMode::ReinitializeSystem);
	RefreshEmitterHandleViewModels();
	RefreshSequencerTracks();
	ClearSystemStats();
	ClearEmitterStats();
	InvalidateCachedCompileStatus();
	ScriptScratchPadViewModel->RefreshScriptViewModels();
	CurveSelectionViewModel->Refresh();
	SystemStackViewModel->InitializeWithViewModels(this->AsShared(), TSharedPtr<FNiagaraEmitterHandleViewModel>(), FNiagaraStackViewModelOptions(true, false));
	SelectionViewModel->Refresh();
	// clean invalid tabs has to be called after view models have refreshed as invalid tabs can be determined by view model state
	EditorDocumentsViewModel->CleanInvalidTabs();
	bPendingAssetMessagesChanged = true;
}

void FNiagaraSystemViewModel::ResetStack()
{
	SystemStackViewModel->Reset();
	ResetEmitterHandleViewModelsAndTracks();
	RefreshAll();
	GetDefault<UEdGraphSchema_NiagaraSystemOverview>()->ForceVisualizationCacheClear();
}

void FNiagaraSystemViewModel::NotifyDataObjectChanged(TArray<UObject*> ChangedObjects, ENiagaraDataObjectChange ChangeType)
{
	if (ChangedObjects.Num() == 1 && ChangedObjects[0]->IsA<UNiagaraSystem>())
	{
		// we do nothing on system changes here, because they will trigger a compile and reset on their own, depending on the changed property
		return;
	}

	bool bRefreshCurveSelectionViewModel = false;
	bool bRefreshSystemDICompiledData = false;
	if(ChangedObjects.Num() != 0)
	{
		for(UObject* ChangedObject : ChangedObjects)
		{
			UNiagaraDataInterface* ChangedDataInterface = Cast<UNiagaraDataInterface>(ChangedObject);
			if (ChangedDataInterface)
			{
				UpdateCompiledDataInterfaces(ChangedDataInterface);
				bRefreshSystemDICompiledData = true;
			}


			if(ChangedObject->IsA<UNiagaraDataInterfaceCurveBase>() && ChangeType != ENiagaraDataObjectChange::Changed)
			{
				bRefreshCurveSelectionViewModel = true;
			}

			if(UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(ChangedObject))
			{
				for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
				{
					if (EmitterHandleViewModel->GetEmitterViewModel()->GetEmitter().Emitter == Emitter)
					{
						UNiagaraStackViewModel* StackViewModel = EmitterHandleViewModel->GetEmitterStackViewModel();
						if(ensure(StackViewModel))
						{
							StackViewModel->RequestRefreshDeferred();
						}
					}
				}
			}
		}
	}
	else
	{
		bRefreshCurveSelectionViewModel = true;
	}

	if(bRefreshSystemDICompiledData)
	{
		System->OnCompiledDataInterfaceChanged();		
	}

	if (bRefreshCurveSelectionViewModel)
	{
		CurveSelectionViewModel->Refresh();
	}

	ResetSystem(ETimeResetMode::AllowResetTime, EMultiResetMode::AllowResetAllInstances, EReinitMode::ReinitializeSystem);
}

void FNiagaraSystemViewModel::NotifyObjectAssetChanged(UNiagaraNode& OwningNode, FName VariableReadName, UObject* NewValue)
{
	UNiagaraSystem& NiagaraSystem = GetSystem();
	const FNiagaraEmitterHandle* EmitterHandle;
	TArray<const UNiagaraScript*> CompiledScripts;
	FNiagaraStackGraphUtilities::GetEmitterHandleAndCompiledScriptsForStackNode(NiagaraSystem, OwningNode, EmitterHandle, CompiledScripts);
	if (ensureMsgf(CompiledScripts.Num() > 0, TEXT("Could not find compiled scripts for UObject input node.")))
	{
		FGuid EmitterHandleId;
		FNameBuilder EmitterNamespace;
		if ( EmitterHandle != nullptr )
		{
			EmitterHandle->GetName().ToString(EmitterNamespace);
			EmitterNamespace.AppendChar('.');
			EmitterHandleId = EmitterHandle->GetId();
		}

		for (const UNiagaraScript* CompiledScript : CompiledScripts)
		{
			for (const FNiagaraScriptUObjectCompileInfo& CompileInfo : CompiledScript->GetCachedDefaultUObjects())
			{
				if (CompileInfo.Variable.GetName() != VariableReadName)
				{
					continue;
				}

				// If we have writes validate that the writes are for this compile info
				// For example if we had a system which used the same two emitters from a parent we can have a node which has the same Set name
				// so here we just validate that the compile info is for this emitter as we will have two infos with the same Set name
				if (EmitterHandle != nullptr)
				{
					bool bIsThisEmitter = false;
					for (FName WriteName : CompileInfo.RegisteredParameterMapWrites)
					{
						FNameBuilder TempName;
						WriteName.ToString(TempName);
						if (TempName.ToView().StartsWith(EmitterNamespace.ToView()))
						{
							bIsThisEmitter = true;
						}
					}
					if (!bIsThisEmitter)
					{
						continue;
					}
				}

				const_cast<FNiagaraScriptUObjectCompileInfo&>(CompileInfo).Object = NewValue;

				const FNiagaraScriptRuntimeData* ScriptRuntimeData = NiagaraSystem.GetScriptRuntimeData(EmitterHandle, *CompiledScript).Get();
				if (ScriptRuntimeData == nullptr)
				{
					continue;
				}

				for (const FNiagaraResolvedUObjectInfo& ResolvedInfo : ScriptRuntimeData->GetResolvedUObjects())
				{
					if (CompileInfo.RegisteredParameterMapWrites.Contains(ResolvedInfo.ResolvedVariable.GetName()))
					{
						const_cast<FNiagaraResolvedUObjectInfo&>(ResolvedInfo).Object = NewValue;
					}
				}
			}
		}
	}

	System->OnCompiledUObjectChanged();

	ResetSystem(ETimeResetMode::AllowResetTime, EMultiResetMode::AllowResetAllInstances, EReinitMode::ReinitializeSystem);
}

void FNiagaraSystemViewModel::IsolateEmitters(TArray<FGuid> EmitterHandlesIdsToIsolate)
{
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandle : EmitterHandleViewModels)
	{
		EmitterHandle->GetEmitterHandle()->SetIsolated(false);
	}

	bool bAnyEmitterIsolated = false;
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandle : EmitterHandleViewModels)
	{
		if (EmitterHandlesIdsToIsolate.Contains(EmitterHandle->GetId()))
		{
			bAnyEmitterIsolated = true;
			EmitterHandle->GetEmitterHandle()->SetIsolated(true);
		}
	}

	SetSystenIsolationEnabled(bAnyEmitterIsolated);
}

void FNiagaraSystemViewModel::CacheIsolatedEmitterState()
{
	CachedEmitterIsolationState.Empty();
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandle : EmitterHandleViewModels)
	{
		if(EmitterHandle->GetEmitterHandle()->IsIsolated())
		{
			CachedEmitterIsolationState.Add(EmitterHandle->GetEmitterHandle()->GetId());
		}
	}
}

void FNiagaraSystemViewModel::RestoreIsolatedEmitterState()
{
	IsolateEmitters(CachedEmitterIsolationState);
	CachedEmitterIsolationState.Empty();
}

void FNiagaraSystemViewModel::SetSystenIsolationEnabled(bool bIsolationEnabled)
{
	UNiagaraSystem* NiagaraSystem = &GetSystem();
	NiagaraSystem->SetIsolateEnabled(bIsolationEnabled);

	for (TObjectIterator<UNiagaraComponent> It; It; ++It)
	{
		UNiagaraComponent* Comp = *It;
		if (::IsValid(Comp) && Comp->IsRegistered() && Comp->GetAsset() == NiagaraSystem)
		{
			Comp->MarkRenderStateDirty();
		}
	}
}

void FNiagaraSystemViewModel::EnableEmitters(TArray<FGuid> EmitterHandlesIdsToEnable)
{
	FScopedTransaction Transaction(LOCTEXT("EnableSelectedEmitters", "Enable selected emitters"));
	GetSystem().Modify();

	bool bAnyEmittersEnabled = false;

	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandle : EmitterHandleViewModels)
	{
		if (EmitterHandlesIdsToEnable.Contains(EmitterHandle->GetId()))
		{
			const bool bEnabled = EmitterHandle->GetEmitterHandle()->SetIsEnabled(true, GetSystem(), false);
			if (bEnabled)
			{
				EmitterHandlePropertyChanged(EmitterHandle->GetId());
			}
			bAnyEmittersEnabled = bAnyEmittersEnabled || bEnabled;
		}
	}

	if (bAnyEmittersEnabled)
	{
		GetSystem().RequestCompile(false);
	}
	else
	{
		Transaction.Cancel();
	}
}

void FNiagaraSystemViewModel::DisableEmitters(TArray<FGuid> EmitterHandlesIdsToDisable)
{
	FScopedTransaction Transaction(LOCTEXT("DisableSelectedEmitters", "Disabled selected emitters"));
	GetSystem().Modify();
	
	bool bAnyEmittersDisabled = false;
	
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandle : EmitterHandleViewModels)
	{
		if(EmitterHandlesIdsToDisable.Contains(EmitterHandle->GetId()))
		{
			const bool bDisabled = EmitterHandle->GetEmitterHandle()->SetIsEnabled(false, GetSystem(), false);
			if (bDisabled)
			{
				EmitterHandlePropertyChanged(EmitterHandle->GetId());
			}
			bAnyEmittersDisabled = bAnyEmittersDisabled || bDisabled;
		}
	}

	if(bAnyEmittersDisabled)
	{
		GetSystem().RequestCompile(false);
	}
	else
	{
		Transaction.Cancel();		
	}
}

void FNiagaraSystemViewModel::ToggleEmitterIsolation(TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandle)
{
	InEmitterHandle->GetEmitterHandle()->SetIsolated(!InEmitterHandle->GetEmitterHandle()->IsIsolated());

	bool bAnyEmitterIsolated = false;
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandle : EmitterHandleViewModels)
	{
		if (EmitterHandle->GetEmitterHandle()->IsIsolated())
		{
			bAnyEmitterIsolated = true;
			break;
		}
	}

	SetSystenIsolationEnabled(bAnyEmitterIsolated);
}

void FNiagaraSystemViewModel::ResetEmitterHandleViewModelsAndTracks()
{
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
	{
		EmitterHandleViewModel->Reset();
	}

	if (NiagaraSequence)
	{
		TGuardValue<bool> UpdateGuard(bResetingSequencerTracks, true);
		TArray<UMovieSceneTrack*> Tracks = NiagaraSequence->GetMovieScene()->GetTracks();
		for (UMovieSceneTrack* Track : Tracks)
		{
			if (Track != nullptr)
			{
				NiagaraSequence->GetMovieScene()->RemoveTrack(*Track);
			}
		}
	}
}

void FNiagaraSystemViewModel::RefreshEmitterHandleViewModels()
{
	TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> OldViewModels = EmitterHandleViewModels;
	EmitterHandleViewModels.Empty();
	GuidToCachedStackModuleData.Empty();

	// Map existing view models to the real instances that now exist. Reuse if we can. Create a new one if we cannot.
	TArray<FGuid> ValidEmitterHandleIds;
	int32 i;
	for (i = 0; i < GetSystem().GetNumEmitters(); ++i)
	{
		FNiagaraEmitterHandle* EmitterHandle = &GetSystem().GetEmitterHandle(i);
		FNiagaraEmitterInstancePtr Simulation = SystemInstance ? SystemInstance->GetSimulationForHandle(*EmitterHandle) : nullptr;
		ValidEmitterHandleIds.Add(EmitterHandle->GetId());

		TSharedPtr<FNiagaraEmitterHandleViewModel> ViewModel;
		bool bAdd = OldViewModels.Num() <= i;
		if (bAdd)
		{
			ViewModel = MakeShared<FNiagaraEmitterHandleViewModel>(bIsForDataProcessingOnly);
			ViewModel->OnNameChanged().AddRaw(this, &FNiagaraSystemViewModel::EmitterHandleNameChanged);
			ViewModel->GetEmitterViewModel()->OnPropertyChanged().AddRaw(this, &FNiagaraSystemViewModel::EmitterPropertyChanged);
			ViewModel->GetEmitterViewModel()->OnScriptCompiled().AddRaw(this, &FNiagaraSystemViewModel::ScriptCompiled);
			ViewModel->GetEmitterViewModel()->OnScriptParameterStoreChanged().AddRaw(this, &FNiagaraSystemViewModel::EmitterParameterStoreChanged);
			ViewModel->GetEmitterStackViewModel()->OnStructureChanged().AddRaw(this, &FNiagaraSystemViewModel::StackViewModelStructureChanged);
		}
		else
		{
			ViewModel = OldViewModels[i];

			// These delegate bindings include the emitter handle id which may have changed since we're reusing existing view models, so remove the binding here.
			ViewModel->OnPropertyChanged().RemoveAll(this);
			ViewModel->GetEmitterViewModel()->OnScriptGraphChanged().RemoveAll(this);
		}

		// Add delegate bindings which require the emitter handle id passed as an additional parameter.
		ViewModel->OnPropertyChanged().AddRaw(this, &FNiagaraSystemViewModel::EmitterHandlePropertyChanged, EmitterHandle->GetId());
		ViewModel->GetEmitterViewModel()->OnScriptGraphChanged().AddRaw(this, &FNiagaraSystemViewModel::EmitterScriptGraphChanged, EmitterHandle->GetId());

		EmitterHandleViewModels.Add(ViewModel.ToSharedRef());
		ViewModel->Initialize(this->AsShared(), i, Simulation);
	}

	check(EmitterHandleViewModels.Num() == GetSystem().GetNumEmitters());

	// Clear out any old view models that may still be left around.
	for (; i < OldViewModels.Num(); i++)
	{
		TSharedRef<FNiagaraEmitterHandleViewModel> ViewModel = OldViewModels[i];
		ViewModel->OnPropertyChanged().RemoveAll(this);
		ViewModel->OnNameChanged().RemoveAll(this);
		ViewModel->GetEmitterViewModel()->OnPropertyChanged().RemoveAll(this);
		ViewModel->GetEmitterViewModel()->OnScriptCompiled().RemoveAll(this);
		ViewModel->GetEmitterViewModel()->OnScriptGraphChanged().RemoveAll(this);
		ViewModel->GetEmitterViewModel()->OnScriptParameterStoreChanged().RemoveAll(this);
		ViewModel->GetEmitterStackViewModel()->OnStructureChanged().RemoveAll(this);
		ViewModel->Reset();
	}

	bool bAnyEmitterIsolated = false;
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandle : EmitterHandleViewModels)
	{
		if (EmitterHandle->GetIsIsolated())
		{
			bAnyEmitterIsolated = true;
		}
	}
	SetSystenIsolationEnabled(bAnyEmitterIsolated);

	OnEmitterHandleViewModelsChangedDelegate.Broadcast();
}

void PopulateChildMovieSceneFoldersFromNiagaraFolders(const UNiagaraSystemEditorFolder* NiagaraFolder, UMovieSceneFolder* MovieSceneFolder, const TMap<FGuid, UMovieSceneNiagaraEmitterTrack*>& EmitterHandleIdToTrackMap)
{
	for (const UNiagaraSystemEditorFolder* ChildNiagaraFolder : NiagaraFolder->GetChildFolders())
	{
		UMovieSceneFolder* MatchingMovieSceneFolder = nullptr;
		for (UMovieSceneFolder* ChildMovieSceneFolder : MovieSceneFolder->GetChildFolders())
		{
			if (ChildMovieSceneFolder->GetFolderName() == ChildNiagaraFolder->GetFolderName())
			{
				MatchingMovieSceneFolder = ChildMovieSceneFolder;
			}
		}

		if (MatchingMovieSceneFolder == nullptr)
		{
			MatchingMovieSceneFolder = NewObject<UMovieSceneFolder>(MovieSceneFolder, ChildNiagaraFolder->GetFolderName(), RF_Transactional);
			MatchingMovieSceneFolder->SetFolderName(ChildNiagaraFolder->GetFolderName());
			MovieSceneFolder->AddChildFolder(MatchingMovieSceneFolder);
		}

		PopulateChildMovieSceneFoldersFromNiagaraFolders(ChildNiagaraFolder, MatchingMovieSceneFolder, EmitterHandleIdToTrackMap);
	}

	for (const FGuid& ChildEmitterHandleId : NiagaraFolder->GetChildEmitterHandleIds())
	{
		UMovieSceneNiagaraEmitterTrack* const* TrackPtr = EmitterHandleIdToTrackMap.Find(ChildEmitterHandleId);
		if (TrackPtr != nullptr && MovieSceneFolder->GetChildTracks().Contains(*TrackPtr) == false)
		{
			MovieSceneFolder->AddChildTrack(*TrackPtr);
		}
	}
}

void FNiagaraSystemViewModel::RefreshSequencerTracks()
{
	TGuardValue<bool> UpdateGuard(bUpdatingSequencerFromEmitterDataChange, true);

	if (Sequencer.IsValid())
	{
		TArray<UMovieSceneTrack*> Tracks = NiagaraSequence->GetMovieScene()->GetTracks();
		for (UMovieSceneTrack* Track : Tracks)
		{
			if (Track != nullptr)
			{
				NiagaraSequence->GetMovieScene()->RemoveTrack(*Track);
			}
		}

		TMap<FGuid, UMovieSceneNiagaraEmitterTrack*> EmitterHandleIdToTrackMap;
		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
		{
			UMovieSceneNiagaraEmitterTrack* EmitterTrack = Cast<UMovieSceneNiagaraEmitterTrack>(NiagaraSequence->GetMovieScene()->AddTrack(UMovieSceneNiagaraEmitterTrack::StaticClass()));
			EmitterTrack->SetFlags(RF_Transient);
			EmitterTrack->Initialize(*this, EmitterHandleViewModel, NiagaraSequence->GetMovieScene()->GetTickResolution());
			EmitterHandleIdToTrackMap.Add(EmitterHandleViewModel->GetId(), EmitterTrack);
		}

		UMovieScene* MovieScene = NiagaraSequence->GetMovieScene();
		MovieScene->EmptyRootFolders();

		const UNiagaraSystemEditorData& SystemEditorData = GetEditorData();
		UNiagaraSystemEditorFolder& RootFolder = SystemEditorData.GetRootFolder();
		for (const UNiagaraSystemEditorFolder* RootChildFolder : RootFolder.GetChildFolders())
		{
			UMovieSceneFolder* MovieSceneRootFolder = NewObject<UMovieSceneFolder>(MovieScene, RootChildFolder->GetFolderName(), RF_Transactional | RF_Transient);
			MovieSceneRootFolder->SetFolderName(RootChildFolder->GetFolderName());
			MovieScene->AddRootFolder(MovieSceneRootFolder);
			PopulateChildMovieSceneFoldersFromNiagaraFolders(RootChildFolder, MovieSceneRootFolder, EmitterHandleIdToTrackMap);
		}
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}

	// Since we just rebuilt all of the sequencer tracks, these updates don't need to be done.
	EmitterIdsRequiringSequencerTrackUpdate.Empty();
}

void FNiagaraSystemViewModel::UpdateSequencerTracksForEmitters(const TArray<FGuid>& EmitterIdsRequiringUpdate)
{
	if (Sequencer.IsValid())
	{
		TGuardValue<bool> UpdateGuard(bUpdatingSequencerFromEmitterDataChange, true);
		for (UMovieSceneTrack* Track : NiagaraSequence->GetMovieScene()->GetTracks())
		{
			UMovieSceneNiagaraEmitterTrack* EmitterTrack = CastChecked<UMovieSceneNiagaraEmitterTrack>(Track);
			if (EmitterIdsRequiringUpdate.Contains(EmitterTrack->GetEmitterHandleViewModel()->GetId()))
			{
				EmitterTrack->UpdateTrackFromEmitterGraphChange(NiagaraSequence->GetMovieScene()->GetTickResolution());
			}
		}

		// We use 'Unknown' here because this is a generic Refresh function with no context as to what changed.
		// Also, using 'MovieSceneStructureItemsChanged' will clear the selection
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::Unknown);
	}
}

UMovieSceneNiagaraEmitterTrack* FNiagaraSystemViewModel::GetTrackForHandleViewModel(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	if (NiagaraSequence)
	{
		for (UMovieSceneTrack* Track : NiagaraSequence->GetMovieScene()->GetTracks())
		{
			UMovieSceneNiagaraEmitterTrack* EmitterTrack = CastChecked<UMovieSceneNiagaraEmitterTrack>(Track);
			if (EmitterTrack->GetEmitterHandleViewModel() == EmitterHandleViewModel)
			{
				return EmitterTrack;
			}
		}
	}
	return nullptr;
}

void FNiagaraSystemViewModel::SetupSequencer()
{
	if (EditMode == ENiagaraSystemViewModelEditMode::EmitterDuringMerge || !FSlateApplicationBase::IsInitialized())
	{
		// we don't need a sequencer when merging emitters or if we're in a commandlet with no slate application
		return;
	}

	NiagaraSequence = NewObject<UNiagaraSequence>(GetTransientPackage());
	UMovieScene* MovieScene = NewObject<UMovieScene>(NiagaraSequence, FName("Niagara System MovieScene"), RF_Transactional | RF_Transient);

	MovieScene->SetDisplayRate(GetEditorData().GetPlaybackFrameRate());
	MovieScene->SetEvaluationType(GetEditorData().GetLockPlaybackFrameRate() ? EMovieSceneEvaluationType::FrameLocked : EMovieSceneEvaluationType::WithSubFrames);

	MovieSceneEventHandler.Unlink();
	MovieScene->UMovieSceneSignedObject::EventHandlers.Link(MovieSceneEventHandler, this);

	NiagaraSequence->Initialize(this, MovieScene);

	FFrameTime StartTime = GetEditorData().GetPlaybackRange().GetLowerBoundValue() * MovieScene->GetTickResolution();
	int32      Duration  = (GetEditorData().GetPlaybackRange().Size<float>() * MovieScene->GetTickResolution()).FrameNumber.Value;

	MovieScene->SetPlaybackRange(StartTime.RoundToFrame(), Duration);

	FMovieSceneEditorData& EditorData = NiagaraSequence->GetMovieScene()->GetEditorData();
	float ViewTimeOffset = .1f;
	EditorData.WorkStart = GetEditorData().GetPlaybackRange().GetLowerBoundValue() - ViewTimeOffset;
	EditorData.WorkEnd = GetEditorData().GetPlaybackRange().GetUpperBoundValue() + ViewTimeOffset;
	EditorData.ViewStart = EditorData.WorkStart;
	EditorData.ViewEnd = EditorData.WorkEnd;

	FSequencerViewParams ViewParams(TEXT("NiagaraSequencerSettings"));
	{
		ViewParams.UniqueName = "NiagaraSequenceEditor";
		ViewParams.OnGetAddMenuContent = OnGetSequencerAddMenuContent;
		ViewParams.OnGetPlaybackSpeeds = ISequencer::FOnGetPlaybackSpeeds::CreateRaw(this, &FNiagaraSystemViewModel::OnGetPlaybackSpeeds);
	}

	FSequencerInitParams SequencerInitParams;
	{
		SequencerInitParams.ViewParams = ViewParams;
		SequencerInitParams.RootSequence = NiagaraSequence;
		SequencerInitParams.bEditWithinLevelEditor = false;
		SequencerInitParams.ToolkitHost = nullptr;
	}

	UNiagaraEditorSettings::OnSettingsChanged().AddRaw(this, &FNiagaraSystemViewModel::SnapToNextSpeed);
	
	ISequencerModule &SequencerModule = FModuleManager::LoadModuleChecked< ISequencerModule >("Sequencer");
	Sequencer = SequencerModule.CreateSequencer(SequencerInitParams);
	Sequencer->OnMovieSceneDataChanged().AddRaw(this, &FNiagaraSystemViewModel::SequencerDataChanged);
	Sequencer->OnGlobalTimeChanged().AddRaw(this, &FNiagaraSystemViewModel::SequencerTimeChanged);
	Sequencer->GetSelectionChangedTracks().AddRaw(this, &FNiagaraSystemViewModel::SequencerTrackSelectionChanged);
	Sequencer->GetSelectionChangedSections().AddRaw(this, &FNiagaraSystemViewModel::SequencerSectionSelectionChanged);
	Sequencer->SetPlaybackStatus(GetSystem().GetNumEmitters() > 0 && EditorSettings->GetAutoPlay()
		? EMovieScenePlayerStatus::Playing
		: EMovieScenePlayerStatus::Stopped);
}

void FNiagaraSystemViewModel::SnapToNextSpeed(const FString& PropertyName, const UNiagaraEditorSettings* Settings)
{
	// we update the speed in any case
	if (Sequencer.IsValid())
	{
		Sequencer->SnapToClosestPlaybackSpeed();
	}
}

void FNiagaraSystemViewModel::IsolateSelectedEmitters()
{
	const TArray<FGuid> EmitterHandles = GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	IsolateEmitters(EmitterHandles);
}

void FNiagaraSystemViewModel::ToggleIsolateEmitterAndSelectedEmitters(FGuid OriginEmitter)
{
	TArray<FGuid> SelectedEmitterHandles = GetSelectionViewModel()->GetSelectedEmitterHandleIds();

	if(SelectedEmitterHandles.Contains(OriginEmitter) == false)
	{
		SelectedEmitterHandles.Empty();
		SelectedEmitterHandles.Add(OriginEmitter);
	}

	bool bAnyIsolationMismatch = false;

	bool bContainsIsolatedEmitter = false;
	bool bContainsNonIsolatedEmitter = false;
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandle : EmitterHandleViewModels)
	{
		if (SelectedEmitterHandles.Contains(EmitterHandle->GetId()))
		{
			if(EmitterHandle->GetEmitterHandle()->IsIsolated())
			{
				bContainsIsolatedEmitter |= true;
			}
			else
			{
				bContainsNonIsolatedEmitter |= true;
			}

			if(bContainsNonIsolatedEmitter && bContainsIsolatedEmitter)
			{
				bAnyIsolationMismatch = true;
				break;
			}
		}
	}

	// if emitter isolation state is mismatched, it means we want to isolate all currently selected emitters 
	if(bAnyIsolationMismatch)
	{
		IsolateEmitters(SelectedEmitterHandles);
	}
	// if all currently selected emitters have the same state, we want to toggle it instead
	else
	{
		// all emitters are isolated, which means we want to turn emitter isolation off
		if(bContainsIsolatedEmitter)
		{
			IsolateEmitters({});
		}
		// all emitters aren't isolated, so we want to turn emitter isolation on
		else if(bContainsNonIsolatedEmitter)
		{
			IsolateEmitters(SelectedEmitterHandles);
		}
	}
}

void FNiagaraSystemViewModel::EnableSelectedEmitters()
{
	const TArray<FGuid> EmitterHandles = GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	EnableEmitters(EmitterHandles);
}

void FNiagaraSystemViewModel::DisableSelectedEmitters()
{
	const TArray<FGuid> EmitterHandles = GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	DisableEmitters(EmitterHandles);
}

void FNiagaraSystemViewModel::ResetSystem()
{
	ResetSystem(ETimeResetMode::AllowResetTime, EMultiResetMode::ResetThisInstance, EReinitMode::ResetSystem);
}

void FNiagaraSystemViewModel::ResetSystem(ETimeResetMode TimeResetMode, EMultiResetMode MultiResetMode, EReinitMode ReinitMode)
{
	bool bResetAge = false;
	if (Sequencer.IsValid())
	{
		// important to note here that EMovieScenePlayerStatus::Stopped means the simulation is *paused* in sequencer.
		bResetAge = TimeResetMode == ETimeResetMode::AllowResetTime && (Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing || (Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Stopped && EditorSettings->GetResimulateOnChangeWhilePaused() == false));
		if (bResetAge)
		{
			TGuardValue<bool> Guard(bSettingSequencerTimeDirectly, true);
			if (Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing)
			{
				Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Paused);
				Sequencer->SetGlobalTime(0);
				Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
			}
			else
			{
				Sequencer->SetGlobalTime(0);
				Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
			}
		}
	}

	FNiagaraSystemUpdateContext UpdateContext;

	UpdateContext.GetPostWork().BindLambda(
		[ReinitMode, bResetAge](UNiagaraComponent* Component)
		{
			if (ReinitMode == EReinitMode::ResetSystem && bResetAge && Component->GetAgeUpdateMode() == ENiagaraAgeUpdateMode::DesiredAge)
			{
				Component->SetDesiredAge(0);
			}
		}
	);

	if (MultiResetMode == EMultiResetMode::ResetThisInstance)
	{
		// Resetting the single preview instance is only valid when the change causing the reset hasn't invalidated any cached simulation data.
		// Examples of these changes are bounds changes or preview changes such as resetting on loop or changing the preview quality level.
		if (PreviewComponent != nullptr)
		{
			UpdateContext.AddSoloComponent(PreviewComponent, ReinitMode == EReinitMode::ReinitializeSystem);
		}
	}
	else
	{
		UNiagaraSystem* NiagaraSystem = &GetSystem();
		UpdateContext.Add(NiagaraSystem, ReinitMode == EReinitMode::ReinitializeSystem);
	}

	UpdateContext.CommitUpdate();

	if (EditMode == ENiagaraSystemViewModelEditMode::EmitterAsset && MultiResetMode == EMultiResetMode::AllowResetAllInstances && EditorSettings->GetResetDependentSystemsWhenEditingEmitters())
	{
		FNiagaraEditorUtilities::ResetSystemsThatReferenceSystemViewModel(*this);
	}

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	bResetRequestPending = false;
}

void FNiagaraSystemViewModel::RequestResetSystem()
{
	bResetRequestPending = true;
}

void GetCompiledScriptsAndEmitterNameFromInputNode(UNiagaraNode& StackNode, UNiagaraSystem& OwningSystem, TArray<UNiagaraScript*>& OutCompiledScripts, FString& OutEmitterName)
{
	OutEmitterName = FString();
	UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(StackNode);
	if (OutputNode != nullptr)
	{
		if (OutputNode->GetUsage() == ENiagaraScriptUsage::SystemSpawnScript)
		{
			OutCompiledScripts.Add(OwningSystem.GetSystemSpawnScript());
		}
		else if (OutputNode->GetUsage() == ENiagaraScriptUsage::SystemUpdateScript)
		{
			OutCompiledScripts.Add(OwningSystem.GetSystemUpdateScript());
		}
		else
		{
			const FNiagaraEmitterHandle* OwningEmitterHandle = OwningSystem.GetEmitterHandles().FindByPredicate([&StackNode](const FNiagaraEmitterHandle& EmitterHandle)
			{
				return CastChecked<UNiagaraScriptSource>(EmitterHandle.GetEmitterData()->GraphSource)->NodeGraph == StackNode.GetNiagaraGraph();
			});

			if(OwningEmitterHandle != nullptr)
			{
				OutEmitterName = OwningEmitterHandle->GetInstance().Emitter->GetUniqueEmitterName();
				switch (OutputNode->GetUsage())
				{
				case ENiagaraScriptUsage::EmitterSpawnScript:
					OutCompiledScripts.Add(OwningSystem.GetSystemSpawnScript());
					break;
				case ENiagaraScriptUsage::EmitterUpdateScript:
					OutCompiledScripts.Add(OwningSystem.GetSystemUpdateScript());
					break;
				case ENiagaraScriptUsage::ParticleSpawnScript:
				case ENiagaraScriptUsage::ParticleUpdateScript:
				case ENiagaraScriptUsage::ParticleEventScript:
				case ENiagaraScriptUsage::ParticleSimulationStageScript:
					//TODO extract emitterdata into one var
					OutCompiledScripts.Add(OwningEmitterHandle->GetEmitterData()->GetScript(OutputNode->GetUsage(), OutputNode->GetUsageId()));
					if (OwningEmitterHandle->GetEmitterData()->SimTarget == ENiagaraSimTarget::GPUComputeSim)
					{
						OutCompiledScripts.Add(OwningEmitterHandle->GetEmitterData()->GetScript(ENiagaraScriptUsage::ParticleGPUComputeScript, FGuid()));
					}
					break;
				}
			}
		}
	}
}

void FNiagaraSystemViewModel::UpdateCompiledDataInterfaces(UNiagaraDataInterface* ChangedDataInterface)
{
	UNiagaraDataInterface* ResolvedCompileDataInterface = FNiagaraEditorUtilities::GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(GetSystem(), *ChangedDataInterface);
	if (ResolvedCompileDataInterface != nullptr)
	{
		ResolvedCompileDataInterface->Modify(false);
		ChangedDataInterface->CopyTo(ResolvedCompileDataInterface);
	}
	else
	{
		// If the data interface wasn't owned by a script, try to find it in the exposed parameter data interfaces.
		const FNiagaraVariableBase* FoundExposedDataInterface = GetSystem().GetExposedParameters().FindVariableFromDataInterface(ChangedDataInterface);
		if (FoundExposedDataInterface != nullptr)
		{
			GetSystem().GetExposedParameters().OnInterfaceChange();
		}
	}
}

void FNiagaraSystemViewModel::EmitterHandlePropertyChanged(FGuid OwningEmitterHandleId)
{
	// When the emitter handle changes, refresh the System scripts emitter nodes and the sequencer tracks just in case the
	// property that changed was the handles emitter.
	if (bUpdatingEmittersFromSequencerDataChange == false && NiagaraSequence)
	{
		TGuardValue<bool> UpdateGuard(bUpdatingSequencerFromEmitterDataChange, true);
		for (UMovieSceneTrack* Track : NiagaraSequence->GetMovieScene()->GetTracks())
		{
			UMovieSceneNiagaraEmitterTrack* EmitterTrack = CastChecked<UMovieSceneNiagaraEmitterTrack>(Track);
			if (EmitterTrack->GetEmitterHandleViewModel()->GetId() == OwningEmitterHandleId)
			{
				EmitterTrack->UpdateTrackFromEmitterGraphChange(NiagaraSequence->GetMovieScene()->GetTickResolution());
			}
		}
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}

	// Refresh the overview nodes and the emitter stacks just in case the emitter enabled state changed.
	GetEditorData().SynchronizeOverviewGraphWithSystem(GetSystem());
	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = GetEmitterHandleViewModelById(OwningEmitterHandleId);
	if (EmitterHandleViewModel.IsValid())
	{
		EmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry()->RefreshChildren();
	}

	ResetSystem(ETimeResetMode::AllowResetTime, EMultiResetMode::ResetThisInstance, EReinitMode::ReinitializeSystem);
}

void FNiagaraSystemViewModel::EmitterHandleNameChanged()
{
	GetDefault<UEdGraphSchema_NiagaraSystemOverview>()->ForceVisualizationCacheClear();
	CompileSystem(false);
	CurveSelectionViewModel->Refresh();
}

void FNiagaraSystemViewModel::EmitterPropertyChanged()
{
	ResetSystem(ETimeResetMode::AllowResetTime, EMultiResetMode::AllowResetAllInstances, EReinitMode::ReinitializeSystem);
}

void FNiagaraSystemViewModel::ScriptCompiled(UNiagaraScript*, const FGuid&)
{
	bCompilePendingCompletion = true;
	//ReInitializeSystemInstances();
}

void FNiagaraSystemViewModel::SystemParameterStoreChanged(const FNiagaraParameterStore* ChangedParameterStore, const UNiagaraScript* OwningScript)
{
	UpdateSimulationFromParameterChange();
}

void FNiagaraSystemViewModel::EmitterScriptGraphChanged(const FEdGraphEditAction& InAction, const UNiagaraScript& OwningScript, FGuid OwningEmitterHandleId)
{
	if (bUpdatingEmittersFromSequencerDataChange == false)
	{
		EmitterIdsRequiringSequencerTrackUpdate.AddUnique(OwningEmitterHandleId);
	}
	// Remove from cache on graph change
	GuidToCachedStackModuleData.Remove(OwningEmitterHandleId);
	InvalidateCachedCompileStatus();

	// Do a deferred refresh when responding to graph changes since we may be mid change and the graph could be invalid.
	CurveSelectionViewModel->RefreshDeferred();

	bPendingAssetMessagesChanged = true;
}

void FNiagaraSystemViewModel::SystemScriptGraphChanged(const FEdGraphEditAction& InAction)
{
	GuidToCachedStackModuleData.Empty();
	InvalidateCachedCompileStatus();
	// Do a deferred refresh when responding to graph changes since we may be mid change and the graph could be invalid.
	CurveSelectionViewModel->RefreshDeferred();
	bPendingAssetMessagesChanged = true;
}

void FNiagaraSystemViewModel::EmitterParameterStoreChanged(const FNiagaraParameterStore& ChangedParameterStore, const UNiagaraScript& OwningScript)
{
	if (bUpdatingEmittersFromSequencerDataChange == false)
	{
		if (EmitterIdsRequiringSequencerTrackUpdate.Num() > 0)
		{
			UpdateSequencerTracksForEmitters(EmitterIdsRequiringSequencerTrackUpdate);
		}
		else if (Sequencer.IsValid())
		{
			TGuardValue<bool> UpdateGuard(bUpdatingSequencerFromEmitterDataChange, true);
			for (UMovieSceneTrack* Track : NiagaraSequence->GetMovieScene()->GetTracks())
			{
				UMovieSceneNiagaraEmitterTrack* EmitterTrack = CastChecked<UMovieSceneNiagaraEmitterTrack>(Track);
				EmitterTrack->UpdateTrackFromEmitterParameterChange(NiagaraSequence->GetMovieScene()->GetTickResolution());
			}
			
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
	UpdateSimulationFromParameterChange();
}

void FNiagaraSystemViewModel::UpdateSimulationFromParameterChange()
{
	if (EditorSettings->GetResetSimulationOnChange())
	{
		/* Calling RequestResetSystem here avoids reentrancy into ResetSystem() when we edit the system parameter store on
		** UNiagaraComponent::Activate() as we always call PrepareRapidIterationParameters().  */
		RequestResetSystem();
	}
	else
	{
		if (Sequencer && Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Stopped)
		{
			// TODO: Update the view when paused and reset on change is turned off.
		}
	}
}

void PopulateNiagaraFoldersFromMovieSceneFolders(TArrayView<UMovieSceneFolder* const> MovieSceneFolders, const TArray<UMovieSceneTrack*>& MovieSceneTracks, UNiagaraSystemEditorFolder* ParentFolder)
{
	TArray<FName> ValidFolderNames;
	for (UMovieSceneFolder* MovieSceneFolder : MovieSceneFolders)
	{
		ValidFolderNames.Add(MovieSceneFolder->GetFolderName());
		UNiagaraSystemEditorFolder* MatchingNiagaraFolder = nullptr;
		for (UNiagaraSystemEditorFolder* ChildNiagaraFolder : ParentFolder->GetChildFolders())
		{
			CA_ASSUME(ChildNiagaraFolder != nullptr);
			if (ChildNiagaraFolder->GetFolderName() == MovieSceneFolder->GetFolderName())
			{
				MatchingNiagaraFolder = ChildNiagaraFolder;
				break;
			}
		}

		if (MatchingNiagaraFolder == nullptr)
		{
			MatchingNiagaraFolder = NewObject<UNiagaraSystemEditorFolder>(ParentFolder, MovieSceneFolder->GetFolderName(), RF_Transactional);
			MatchingNiagaraFolder->SetFolderName(MovieSceneFolder->GetFolderName());
			ParentFolder->AddChildFolder(MatchingNiagaraFolder);
		}

		PopulateNiagaraFoldersFromMovieSceneFolders(MovieSceneFolder->GetChildFolders(), MovieSceneFolder->GetChildTracks(), MatchingNiagaraFolder);
	}

	TArray<UNiagaraSystemEditorFolder*> ChildNiagaraFolders = ParentFolder->GetChildFolders();
	for (UNiagaraSystemEditorFolder* ChildNiagaraFolder : ChildNiagaraFolders)
	{
		if (ValidFolderNames.Contains(ChildNiagaraFolder->GetFolderName()) == false)
		{
			ParentFolder->RemoveChildFolder(ChildNiagaraFolder);
		}
	}

	TArray<FGuid> ValidEmitterHandleIds;
	for (UMovieSceneTrack* MovieSceneTrack : MovieSceneTracks)
	{
		UMovieSceneNiagaraEmitterTrack* NiagaraEmitterTrack = Cast<UMovieSceneNiagaraEmitterTrack>(MovieSceneTrack);
		if (NiagaraEmitterTrack != nullptr)
		{
			FGuid EmitterHandleId = NiagaraEmitterTrack->GetEmitterHandleViewModel()->GetId();
			ValidEmitterHandleIds.Add(EmitterHandleId);
			if (ParentFolder->GetChildEmitterHandleIds().Contains(EmitterHandleId) == false)
			{
				ParentFolder->AddChildEmitterHandleId(EmitterHandleId);
			}
		}
	}

	TArray<FGuid> ChildEmitterHandleIds = ParentFolder->GetChildEmitterHandleIds();
	for (FGuid& ChildEmitterHandleId : ChildEmitterHandleIds)
	{
		if (ValidEmitterHandleIds.Contains(ChildEmitterHandleId) == false)
		{
			ParentFolder->RemoveChildEmitterHandleId(ChildEmitterHandleId);
		}
	}
}

void FNiagaraSystemViewModel::SequencerMovieSceneModified(const UMovieScene* MovieScene)
{
	if (MovieScene == nullptr)
	{
		return;
	}

	UNiagaraSystemEditorData& SystemEditorData = GetEditorData();
	if (SystemEditorData.GetPlaybackFrameRate() != MovieScene->GetDisplayRate())
	{
		SystemEditorData.SetPlaybackFrameRate(MovieScene->GetDisplayRate());
		if (PreviewComponent != nullptr)
		{
			PreviewComponent->SetSeekDelta((float)SystemEditorData.GetPlaybackFrameRate().AsInterval());
		}
	}

	bool bMovieSceneIsFrameLocked = MovieScene->GetEvaluationType() == EMovieSceneEvaluationType::FrameLocked;
	if (SystemEditorData.GetLockPlaybackFrameRate() != bMovieSceneIsFrameLocked)
	{
		SystemEditorData.SetLockPlaybackFrameRate(bMovieSceneIsFrameLocked);
		if (PreviewComponent != nullptr)
		{
			PreviewComponent->SetLockDesiredAgeDeltaTimeToSeekDelta(bMovieSceneIsFrameLocked);
		}
	}
}

void FNiagaraSystemViewModel::SequencerDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	if (bUpdatingSequencerFromEmitterDataChange == false && GIsTransacting == false && NiagaraSequence)
	{
		TGuardValue<bool> UpdateGuard(bUpdatingEmittersFromSequencerDataChange, true);

		GetEditorData().Modify();
		TRange<FFrameNumber> FramePlaybackRange = NiagaraSequence->GetMovieScene()->GetPlaybackRange();
		float StartTimeSeconds = NiagaraSequence->GetMovieScene()->GetTickResolution().AsSeconds(FramePlaybackRange.GetLowerBoundValue());
		float EndTimeSeconds = NiagaraSequence->GetMovieScene()->GetTickResolution().AsSeconds(FramePlaybackRange.GetUpperBoundValue());
		GetEditorData().SetPlaybackRange(TRange<float>(StartTimeSeconds, EndTimeSeconds));

		TSet<FGuid> VaildTrackEmitterHandleIds;
		TArray<FEmitterHandleToDuplicate> EmittersToDuplicate;
		TArray<TTuple<TSharedPtr<FNiagaraEmitterHandleViewModel>, FName>> EmitterHandlesToRename;

		bool bRefreshAllTracks = false;
		for (UMovieSceneTrack* Track : NiagaraSequence->GetMovieScene()->GetTracks())
		{
			UMovieSceneNiagaraEmitterTrack* EmitterTrack = CastChecked<UMovieSceneNiagaraEmitterTrack>(Track);
			if (EmitterTrack->GetEmitterHandleViewModel().IsValid())
			{
				if (EmitterTrack->GetAllSections().Num() == 0 && EmitterTrack->GetSectionsWereModified())
				{
					// If there are no sections and the section collection was modified, the section was deleted so skip adding
					// it's handle to the valid collection so that it's emitter is deleted.
					continue;
				}

				VaildTrackEmitterHandleIds.Add(EmitterTrack->GetEmitterHandleViewModel()->GetId());
				EmitterTrack->UpdateEmitterHandleFromTrackChange(NiagaraSequence->GetMovieScene()->GetTickResolution());
				EmitterTrack->GetEmitterHandleViewModel()->GetEmitterViewModel()->GetEditorData().Modify();
				EmitterTrack->GetEmitterHandleViewModel()->GetEmitterViewModel()->GetEditorData().SetPlaybackRange(GetEditorData().GetPlaybackRange());
				if (EmitterTrack->GetDisplayName().ToString() != EmitterTrack->GetEmitterHandleViewModel()->GetNameText().ToString())
				{
					EmitterHandlesToRename.Add(TTuple<TSharedPtr<FNiagaraEmitterHandleViewModel>, FName>(EmitterTrack->GetEmitterHandleViewModel(), *EmitterTrack->GetDisplayName().ToString()));
				}

				if (EmitterTrack->GetAllSections().Num() > 1)
				{
					// If a section was duplicated, force a refresh to remove the duplicated section since that's not currently supported.
					// TODO: Detect duplicated sections and create a duplicate emitter with the correct values.
					bRefreshAllTracks = true;
				}
			}
			else
			{
				if (EmitterTrack->GetEmitterHandleId().IsValid())
				{
					// The emitter handle is invalid, but the track has a valid Id, most probably because of a copy/paste event
					FEmitterHandleToDuplicate EmitterHandleToDuplicate;
					EmitterHandleToDuplicate.SystemPath = EmitterTrack->GetSystemPath();
					EmitterHandleToDuplicate.EmitterHandleId = EmitterTrack->GetEmitterHandleId();
					EmittersToDuplicate.AddUnique(EmitterHandleToDuplicate);
				}
			}
		}

		bRefreshAllTracks |= EmitterHandlesToRename.Num() > 0;

		for (TTuple<TSharedPtr<FNiagaraEmitterHandleViewModel>, FName>& EmitterHandletoRename : EmitterHandlesToRename)
		{
			EmitterHandletoRename.Get<0>()->SetName(EmitterHandletoRename.Get<1>());
		}

		TSet<FGuid> AllEmitterHandleIds;
		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
		{
			AllEmitterHandleIds.Add(EmitterHandleViewModel->GetId());
		}

		TSet<FGuid> RemovedEmitterHandleIds = AllEmitterHandleIds.Difference(VaildTrackEmitterHandleIds);
		if (RemovedEmitterHandleIds.Num() > 0)
		{
			if (bCanModifyEmittersFromTimeline)
			{
				DeleteEmitters(RemovedEmitterHandleIds);
			}
			else
			{
				bRefreshAllTracks = true;
			}
		}

		if (EmittersToDuplicate.Num() > 0)
		{
			if (bCanModifyEmittersFromTimeline)
			{
				DuplicateEmitters(EmittersToDuplicate);
			}
			else
			{
				bRefreshAllTracks = true;
			}
		}

		TArray<UMovieSceneTrack*> RootTracks;
		TArrayView<UMovieSceneFolder* const> RootFolders = NiagaraSequence->GetMovieScene()->GetRootFolders();
		if (RootFolders.Num() != 0 || GetEditorData().GetRootFolder().GetChildFolders().Num() != 0)
		{
			PopulateNiagaraFoldersFromMovieSceneFolders(RootFolders, RootTracks, &GetEditorData().GetRootFolder());
		}

		if (bRefreshAllTracks)
		{
			RefreshSequencerTracks();
		}
	}
}

void FNiagaraSystemViewModel::SequencerTimeChanged()
{
	if (!PreviewComponent || !SystemInstance || !SystemInstance->GetAreDataInterfacesInitialized() || !Sequencer.IsValid())
	{
		return;
	}
	EMovieScenePlayerStatus::Type CurrentStatus = Sequencer->GetPlaybackStatus();
	float CurrentSequencerTime = Sequencer->GetGlobalTime().AsSeconds();
	if (SystemInstance != nullptr)
	{
		// Avoid reentrancy if we're setting the time directly.
		if (bSettingSequencerTimeDirectly == false && CurrentSequencerTime != PreviousSequencerTime)
		{
			// Skip the first update after going from stopped to playing or from playing to stopped because snapping in sequencer may have made
			// the time reverse by a small amount, and sending that update to the System will reset it unnecessarily.
			bool bStartedPlaying = CurrentStatus == EMovieScenePlayerStatus::Playing && PreviousSequencerStatus != EMovieScenePlayerStatus::Playing;
			bool bEndedPlaying = CurrentStatus != EMovieScenePlayerStatus::Playing && PreviousSequencerStatus == EMovieScenePlayerStatus::Playing;

			bool bUpdateDesiredAge = bStartedPlaying == false;
			bool bResetSystemInstance = SystemInstance->IsComplete();

			if (bUpdateDesiredAge)
			{
				// NOTE: Logic copied from FMovieSceneNiagaraSystemTrackTemplate
				// Add a quarter of a frame offset here to push the desired age into the middle of the frame since it will be automatically rounded
				// down to the nearest seek delta.  This prevents a situation where float rounding results in a value which is just slightly less than
				// the frame boundary, which results in a skipped simulation frame.
				const float FrameOffset = PreviewComponent->GetLockDesiredAgeDeltaTimeToSeekDelta() ? (PreviewComponent->GetSeekDelta() / 4.0f) : 0.0f;
				float DesiredAge = CurrentSequencerTime + FrameOffset;

				if (CurrentStatus == EMovieScenePlayerStatus::Playing)
				{
					PreviewComponent->SetDesiredAge(FMath::Max(DesiredAge, 0.0f));
				}
				else
				{
					PreviewComponent->SeekToDesiredAge(FMath::Max(DesiredAge, 0.0f));
				}
			}

			if (bResetSystemInstance)
			{
				// We don't want to reset the current time if we're scrubbing.
				if (CurrentStatus == EMovieScenePlayerStatus::Playing)
				{
					ResetSystem(ETimeResetMode::AllowResetTime, EMultiResetMode::ResetThisInstance, EReinitMode::ReinitializeSystem);
				}
				else
				{
					ResetSystem(ETimeResetMode::KeepCurrentTime, EMultiResetMode::ResetThisInstance, EReinitMode::ResetSystem);
				}
			}
		}
	}

	PreviousSequencerStatus = CurrentStatus;
	PreviousSequencerTime = CurrentSequencerTime;

	OnPostSequencerTimeChangeDelegate.Broadcast();
}

void FNiagaraSystemViewModel::SystemSelectionChanged()
{
	if (bUpdatingSystemSelectionFromSequencer == false)
	{
		UpdateSequencerFromEmitterHandleSelection();
	}
}

void FNiagaraSystemViewModel::SequencerTrackSelectionChanged(TArray<UMovieSceneTrack*> SelectedTracks)
{
	if (bUpdatingSequencerSelectionFromSystem == false && bResetingSequencerTracks == false)
	{
		UpdateEmitterHandleSelectionFromSequencer();
	}
}

void FNiagaraSystemViewModel::SequencerSectionSelectionChanged(TArray<UMovieSceneSection*> SelectedSections)
{
	if (bUpdatingSequencerSelectionFromSystem == false && bResetingSequencerTracks == false)
	{
		UpdateEmitterHandleSelectionFromSequencer();
	}
}

void FNiagaraSystemViewModel::UpdateEmitterHandleSelectionFromSequencer()
{
	if (!Sequencer.IsValid())
	{
		return;
	}
	
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
	{
		if (!EmitterHandleViewModel->GetEmitterHandle() || ::IsValid(EmitterHandleViewModel->GetEmitterHandle()->GetInstance().Emitter) == false)
		{
			// we can run into pending kill emitters here when undoing an emitter add transaction. This method gets called as part of sequencer's PostUndo() refresh broadcast, which happens BEFORE this view model's PostUndo() could clean everything up.
			return;
		}
	}

	TArray<UMovieSceneTrack*> SelectedTracks;
	TArray<UMovieSceneSection*> SelectedSections;
	Sequencer->GetSelectedTracks(SelectedTracks);
	Sequencer->GetSelectedSections(SelectedSections);
	if (SelectedTracks.Num() == 0 && SelectedSections.Num() == 0)
	{
		return;
	}

	TArray<FGuid> NewSelectedEmitterHandleIds;

	for (UMovieSceneTrack* SelectedTrack : SelectedTracks)
	{
		UMovieSceneNiagaraEmitterTrack* SelectedEmitterTrack = Cast<UMovieSceneNiagaraEmitterTrack>(SelectedTrack);
		if (SelectedEmitterTrack != nullptr && SelectedEmitterTrack->GetEmitterHandleViewModel().IsValid())
		{
			NewSelectedEmitterHandleIds.AddUnique(SelectedEmitterTrack->GetEmitterHandleViewModel()->GetId());
		}
	}

	for (UMovieSceneSection* SelectedSection : SelectedSections)
	{
		UMovieSceneNiagaraEmitterSectionBase* SelectedEmitterSection = Cast<UMovieSceneNiagaraEmitterSectionBase>(SelectedSection);
		if (SelectedEmitterSection != nullptr && SelectedEmitterSection->GetEmitterHandleViewModel().IsValid())
		{
			NewSelectedEmitterHandleIds.AddUnique(SelectedEmitterSection->GetEmitterHandleViewModel()->GetId());
		}
	}

	TGuardValue<bool> UpdateGuard(bUpdatingSystemSelectionFromSequencer, true);
	if (SelectionViewModel != nullptr)
	{
		TArray<UNiagaraStackEntry*> EntriesToSelect;
		TArray<UNiagaraStackEntry*> EntriesToDeselect;
		UNiagaraStackEntry* SystemRootEntry = SystemStackViewModel->GetRootEntry();
		if (NewSelectedEmitterHandleIds.Num() > 0 && GetEditorData().GetOwningSystemIsPlaceholder() == false)
		{
			EntriesToSelect.Add(SystemRootEntry);
		}
		else
		{
			EntriesToDeselect.Add(SystemRootEntry);
		}
		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
		{
			UNiagaraStackEntry* EmitterRootEntry = EmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry();
			if (NewSelectedEmitterHandleIds.Contains(EmitterHandleViewModel->GetId()))
			{
				EntriesToSelect.Add(EmitterRootEntry);
			}
			else
			{
				EntriesToDeselect.Add(EmitterRootEntry);
			}
		}
		bool bClearCurrentSelection = FSlateApplication::Get().GetModifierKeys().IsControlDown() == false;
		SelectionViewModel->UpdateSelectedEntries(EntriesToSelect, EntriesToDeselect, bClearCurrentSelection);
	}
}

void FNiagaraSystemViewModel::UpdateSequencerFromEmitterHandleSelection()
{
	if (!Sequencer.IsValid())
	{
		return;
	}
	TGuardValue<bool> UpdateGuard(bUpdatingSequencerSelectionFromSystem, true);
	Sequencer->EmptySelection();
	for (FGuid SelectedEmitterHandleId : SelectionViewModel->GetSelectedEmitterHandleIds())
	{
		for (UMovieSceneTrack* Track : NiagaraSequence->GetMovieScene()->GetTracks())
		{
			UMovieSceneNiagaraEmitterTrack* EmitterTrack = Cast<UMovieSceneNiagaraEmitterTrack>(Track);
			if (EmitterTrack != nullptr && EmitterTrack->GetEmitterHandleViewModel()->GetId() == SelectedEmitterHandleId)
			{
				Sequencer->SelectTrack(EmitterTrack);
			}
		}
	}
}

void FNiagaraSystemViewModel::SystemInstanceReset()
{
	SystemInstanceInitialized();
}

void FNiagaraSystemViewModel::PreviewComponentSystemInstanceChanged()
{
	FNiagaraSystemInstance* OldSystemInstance = SystemInstance;
	FNiagaraSystemInstanceControllerPtr SystemInstanceController = PreviewComponent->GetSystemInstanceController();
	SystemInstance = SystemInstanceController.IsValid() ? SystemInstanceController->GetSoloSystemInstance() : nullptr;
	if (SystemInstance != OldSystemInstance)
	{
		if (SystemInstance != nullptr)
		{
			SystemInstance->OnInitialized().AddRaw(this, &FNiagaraSystemViewModel::SystemInstanceInitialized);
			SystemInstance->OnReset().AddRaw(this, &FNiagaraSystemViewModel::SystemInstanceReset);
		}
		else
		{
			for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
			{
				if (EmitterHandleViewModel->GetEmitterHandle())
				{
					EmitterHandleViewModel->SetSimulation(nullptr);
				}
			}
		}
	}
}

void FNiagaraSystemViewModel::SystemInstanceInitialized()
{
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
	{
		if (EmitterHandleViewModel->IsValid())
		{
			EmitterHandleViewModel->SetSimulation(SystemInstance->GetSimulationForHandle(*EmitterHandleViewModel->GetEmitterHandle()));
		}
	}
}

void FNiagaraSystemViewModel::UpdateEmitterFixedBounds()
{
	for (TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : EmitterHandleViewModels)
	{
		// if we are an emitter asset we don't require pre-selection
		// if we are a system asset instead we filter out unselected emitters
		if (EditMode != ENiagaraSystemViewModelEditMode::EmitterAsset && SelectionViewModel->GetSelectedEmitterHandleIds().Contains(EmitterHandleViewModel->GetId()) == false)
		{
			continue;
		}
		FNiagaraEmitterHandle* SelectedEmitterHandle = EmitterHandleViewModel->GetEmitterHandle();
		check(SelectedEmitterHandle);
		for (const FNiagaraEmitterInstanceRef& EmitterInst : SystemInstance->GetEmitters())
		{
			if (&EmitterInst->GetEmitterHandle() == SelectedEmitterHandle && !EmitterInst->IsComplete())
			{
				if (FNiagaraEmitterInstanceImpl* EmitterInstImpl = EmitterInst->AsStateful())
				{
					EmitterInstImpl->CalculateFixedBounds(PreviewComponent->GetComponentToWorld().Inverse());
				}
			}
		}
	}
	PreviewComponent->MarkRenderTransformDirty();
	ResetSystem(ETimeResetMode::KeepCurrentTime, EMultiResetMode::ResetThisInstance, EReinitMode::ResetSystem);
}

void FNiagaraSystemViewModel::UpdateSystemFixedBounds()
{
	// early out as we only allow system fixed bounds update on system assets
	if(GetEditMode() != ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		return;
	}

	if(SystemInstance != nullptr)
	{
		GetSystem().Modify();
		
		GetSystem().bFixedBounds = true;
		GetSystem().SetFixedBounds(SystemInstance->GetLocalBounds());

		PreviewComponent->MarkRenderTransformDirty();
		ResetSystem(ETimeResetMode::KeepCurrentTime, EMultiResetMode::ResetThisInstance, EReinitMode::ResetSystem);
	}
}

ENiagaraPerformanceStateScopeMode FNiagaraSystemViewModel::GetPerformanceStateScopeMode() const
{
	return System->GetPerformanceStateScopeMode();
}

void FNiagaraSystemViewModel::ClearSystemStats()
{
#if STATS
	System->GetStatData().ClearStatCaptures();
#endif
}

void FNiagaraSystemViewModel::ClearEmitterStats()
{
#if STATS
	for (const FNiagaraEmitterHandle& Handle : System->GetEmitterHandles())
	{
		if (FVersionedNiagaraEmitterData* EmitterData = Handle.GetEmitterData())
		{
			EmitterData->GetStatData().ClearStatCaptures();
		}
	}
#endif
}

// this switches the viewed emitter version in the editor, for the version selector of the referenced parent version see SNiagaraOverviewStackNode::SwitchToVersion
bool FNiagaraSystemViewModel::ChangeEmitterVersion(const FVersionedNiagaraEmitter& Emitter, const FGuid& NewVersion)
{
	if (System->ChangeEmitterVersion(Emitter, NewVersion))
	{
		ResetStack();
		System->RequestCompile(false);
		return true;
	}
	return false;
}

void FNiagaraSystemViewModel::AddSystemEventHandlers()
{
	if (System != nullptr)
	{
		TArray<UNiagaraScript*> Scripts;
		Scripts.Add(System->GetSystemSpawnScript());
		Scripts.Add(System->GetSystemUpdateScript());

		for (UNiagaraScript* Script : Scripts)
		{
			if (Script != nullptr)
			{
				FDelegateHandle OnParameterStoreChangedHandle = Script->RapidIterationParameters.AddOnChangedHandler(
					FNiagaraParameterStore::FOnChanged::FDelegate::CreateThreadSafeSP(
						this->AsShared(), &FNiagaraSystemViewModel::SystemParameterStoreChanged, (const FNiagaraParameterStore*)&Script->RapidIterationParameters, (const UNiagaraScript*)Script));
				ScriptToOnParameterStoreChangedHandleMap.Add(FObjectKey(Script), OnParameterStoreChangedHandle);
			}
		}

		UserParameterStoreChangedHandle = System->GetExposedParameters().AddOnChangedHandler(
			FNiagaraParameterStore::FOnChanged::FDelegate::CreateThreadSafeSP(
				this->AsShared(), &FNiagaraSystemViewModel::SystemParameterStoreChanged, (const FNiagaraParameterStore*)&System->GetExposedParameters(), (const UNiagaraScript*)nullptr));

		SystemScriptGraphChangedHandle = SystemScriptViewModel->GetGraphViewModel()->GetGraph()->AddOnGraphChangedHandler(
			FOnGraphChanged::FDelegate::CreateSP(this->AsShared(), &FNiagaraSystemViewModel::SystemScriptGraphChanged));
		SystemScriptGraphNeedsRecompileHandle = SystemScriptViewModel->GetGraphViewModel()->GetGraph()->AddOnGraphNeedsRecompileHandler(
			FOnGraphChanged::FDelegate::CreateSP(this->AsShared(), &FNiagaraSystemViewModel::SystemScriptGraphChanged));
	}
}

void FNiagaraSystemViewModel::RemoveSystemEventHandlers()
{
	if (System != nullptr)
	{
		TArray<UNiagaraScript*> Scripts;
		Scripts.Add(System->GetSystemSpawnScript());
		Scripts.Add(System->GetSystemUpdateScript());

		for (UNiagaraScript* Script : Scripts)
		{
			if (Script != nullptr)
			{
				FDelegateHandle* OnParameterStoreChangedHandle = ScriptToOnParameterStoreChangedHandleMap.Find(FObjectKey(Script));
				if (OnParameterStoreChangedHandle != nullptr)
				{
					Script->RapidIterationParameters.RemoveOnChangedHandler(*OnParameterStoreChangedHandle);
				}
			}
		}

		System->GetExposedParameters().RemoveOnChangedHandler(UserParameterStoreChangedHandle);
		if (SystemScriptViewModel.IsValid())
		{
			SystemScriptViewModel->GetGraphViewModel()->GetGraph()->RemoveOnGraphChangedHandler(SystemScriptGraphChangedHandle);
			SystemScriptViewModel->GetGraphViewModel()->GetGraph()->RemoveOnGraphNeedsRecompileHandler(SystemScriptGraphNeedsRecompileHandle);
		}
	}

	ScriptToOnParameterStoreChangedHandleMap.Empty();
	UserParameterStoreChangedHandle.Reset();
	SystemScriptGraphChangedHandle.Reset();
	SystemScriptGraphNeedsRecompileHandle.Reset();
}

void FNiagaraSystemViewModel::BuildStackModuleData(UNiagaraScript* Script, FGuid InEmitterHandleId, TArray<FNiagaraStackModuleData>& OutStackModuleData)
{
	UNiagaraNodeOutput* OutputNode = FNiagaraEditorUtilities::GetScriptOutputNode(*Script);
	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackGroups;
	FNiagaraStackGraphUtilities::GetStackNodeGroups((UNiagaraNode&)*OutputNode, StackGroups);

	int StackIndex = 0;
	if (StackGroups.Num() > 2)
	{
		for (int i = 1; i < StackGroups.Num() - 1; i++)
		{
			FNiagaraStackGraphUtilities::FStackNodeGroup& StackGroup = StackGroups[i];
			StackIndex = i - 1;
			TArray<UNiagaraNode*> GroupNodes;
			StackGroup.GetAllNodesInGroup(GroupNodes);
			UNiagaraNodeFunctionCall * ModuleNode = Cast<UNiagaraNodeFunctionCall>(StackGroup.EndNode);
			if (ModuleNode != nullptr)
			{
				ENiagaraScriptUsage Usage = Script->GetUsage();
				FGuid UsageId = Script->GetUsageId();
				int32 Index = StackIndex;
				FNiagaraStackModuleData ModuleData = { ModuleNode, Usage, UsageId, Index, InEmitterHandleId };
				OutStackModuleData.Add(ModuleData);
			}
		}
	}

}

void FNiagaraSystemViewModel::SystemChanged(UNiagaraSystem* ChangedSystem)
{
	if (GIsTransacting == false)
	{
		check(System == ChangedSystem);
		RefreshAll();
	}
}

void FNiagaraSystemViewModel::StackViewModelStructureChanged(ENiagaraStructureChangedFlags Flags)
{
	if ((Flags & StructureChanged) != 0 && SelectionViewModel != nullptr)
	{
		SelectionViewModel->RefreshDeferred();
	}
}

void FNiagaraSystemViewModel::ScratchPadScriptsChanged()
{
	SystemStackViewModel->GetRootEntry()->RefreshChildren();
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
	{
		EmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry()->RefreshChildren();
	}
}

void FNiagaraSystemViewModel::RefreshStackViewModels()
{
	if (SystemStackViewModel)
	{
		SystemStackViewModel->RequestRefreshDeferred();
	}
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : EmitterHandleViewModels)
	{
		EmitterHandleViewModel->GetEmitterStackViewModel()->RequestRefreshDeferred();
	}
}

void FNiagaraSystemViewModel::RefreshAssetMessages()
{
	FNiagaraMessageManager* MessageManager = FNiagaraMessageManager::Get();
	for (const FObjectKey& MessageSourceObjectKey : MessageSourceObjectKeys)
	{
		MessageManager->ClearAssetMessagesForObject(SystemMessageLogGuidKey.GetValue(), MessageSourceObjectKey);
	}
	MessageSourceObjectKeys.Reset();

	auto PublishMessages = [this, &MessageManager](const FNiagaraMessageSourceAndStore& MessageSourceAndStore) {
		for (auto It = MessageSourceAndStore.GetStore()->GetMessages().CreateConstIterator(); It; ++It)
		{
			const FGuid& MessageKey = It.Key();
			if (MessageSourceAndStore.GetStore()->IsMessageDismissed(MessageKey))
			{
				continue;
			}

			const UNiagaraMessageData* MessageData = static_cast<UNiagaraMessageData*>(It.Value());
			FGenerateNiagaraMessageInfo GenerateNiagaraMessageInfo = FGenerateNiagaraMessageInfo();

			TArray<FObjectKey> AssociatedObjectKeys = { FObjectKey(MessageSourceAndStore.GetSource()) };
			GenerateNiagaraMessageInfo.SetAssociatedObjectKeys(AssociatedObjectKeys);

			if (MessageData->GetAllowDismissal())
			{
				GenerateNiagaraMessageInfo.SetDismissHandler(FSimpleDelegate::CreateSP(this, &FNiagaraSystemViewModel::DismissMessageAndRefresh, MessageSourceAndStore, It.Key()));
			}

			TSharedRef<const INiagaraMessage> Message = MessageData->GenerateNiagaraMessage(GenerateNiagaraMessageInfo);
			MessageManager->AddMessage(Message, SystemMessageLogGuidKey.GetValue());
		}
	};

	auto PublishScriptMessages = [&](const TArray<UNiagaraScript*>& Scripts)->TArray<FObjectKey> /* FunctionCallNodeObjectKeys */ {
		TArray<FObjectKey> NewObjectKeys;
		for(UNiagaraScript* Script : Scripts)
		{
			UNiagaraNodeOutput* OutputNode = FNiagaraEditorUtilities::GetScriptOutputNode(*Script);
			if (OutputNode == nullptr)
			{
				// The output node may be null if the target script is a GPU compute script that is not in use.
				continue;
			}
			TArray<UNiagaraNodeFunctionCall*> FunctionCallNodes;
			FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*OutputNode, FunctionCallNodes);
			for(UNiagaraNodeFunctionCall* FunctionCallNode : FunctionCallNodes)
			{
				PublishMessages(FNiagaraMessageSourceAndStore(*FunctionCallNode, FunctionCallNode->GetMessageStore()));
				NewObjectKeys.Add(FObjectKey(FunctionCallNode));
			}
		}
		return NewObjectKeys;
	};

	UNiagaraSystem& ViewedSystem = GetSystem();
	if (EditMode == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		PublishMessages(FNiagaraMessageSourceAndStore(ViewedSystem, ViewedSystem.GetMessageStore()));
		MessageSourceObjectKeys.Add(FObjectKey(&ViewedSystem));

		TArray<UNiagaraScript*> Scripts;
		Scripts.Add(ViewedSystem.GetSystemSpawnScript());
		Scripts.Add(ViewedSystem.GetSystemUpdateScript());
		for (const FNiagaraEmitterHandle& EmitterHandle : ViewedSystem.GetEmitterHandles())
		{
			FVersionedNiagaraEmitter VersionedEmitter = EmitterHandle.GetInstance();
			UNiagaraEmitter* Emitter = VersionedEmitter.Emitter.Get();
			//-TODO:Stateless: Do we need stateless support here?
			if (Emitter == nullptr)
			{
				continue;
			}	
			PublishMessages(FNiagaraMessageSourceAndStore(*Emitter, Emitter->GetMessageStore()));
			MessageSourceObjectKeys.Add(FObjectKey(Emitter));
			VersionedEmitter.GetEmitterData()->GetScripts(Scripts, false);
		}
		MessageSourceObjectKeys.Append(PublishScriptMessages(Scripts));

	}
	else if (EditMode == ENiagaraSystemViewModelEditMode::EmitterAsset)
	{
		const TArray<FNiagaraEmitterHandle>& EmitterHandles = ViewedSystem.GetEmitterHandles();
		if (ensureMsgf(EmitterHandles.Num() == 1, TEXT("There was not exactly 1 Emitter Handle for the SystemViewModel in Emitter edit mode!")))
		{
			FVersionedNiagaraEmitter VersionedEmitter = EmitterHandles[0].GetInstance();
			//-TODO:Stateless: Do we need stateless support here?
			if (UNiagaraEmitter* Emitter = VersionedEmitter.Emitter.Get())
			{
				PublishMessages(FNiagaraMessageSourceAndStore(*Emitter, Emitter->GetMessageStore()));
				MessageSourceObjectKeys.Add(FObjectKey(Emitter));

				TArray<UNiagaraScript*> Scripts;
				VersionedEmitter.GetEmitterData()->GetScripts(Scripts, false);
				MessageSourceObjectKeys.Append(PublishScriptMessages(Scripts));
			}
		}
	}
	else
	{
		checkf(false, TEXT("New system viewmodel edit mode defined! Must implemented RefreshAssetMessages() for this edit mode!"));
	}
}

TSharedPtr<FNiagaraOverviewGraphViewModel> FNiagaraSystemViewModel::GetOverviewGraphViewModel() const
{
	return OverviewGraphViewModel;
}

#undef LOCTEXT_NAMESPACE // NiagaraSystemViewModel
