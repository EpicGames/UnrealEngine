// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/NiagaraUserParameterPanelViewModel.h"
#include "CoreMinimal.h"
#include "ViewModels/NiagaraSystemViewModel.h"

template<> TMap<UNiagaraSystem*, TArray<FNiagaraUserParameterPanelViewModel*>> TNiagaraViewModelManager<UNiagaraSystem, FNiagaraUserParameterPanelViewModel>::ObjectsToViewModels{};

FNiagaraUserParameterPanelViewModel::~FNiagaraUserParameterPanelViewModel()
{
	UnregisterViewModelWithMap(RegisteredHandle);
}

void FNiagaraUserParameterPanelViewModel::Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModelWeak = InSystemViewModel;
	RegisteredHandle = RegisterViewModelWithMap(&(SystemViewModelWeak.Pin()->GetSystem()), this);
}

void FNiagaraUserParameterPanelViewModel::Refresh() const
{
	OnRefreshRequestedDelegate.ExecuteIfBound();
}
