// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraScript.h"
#include "NiagaraTypes.h"
#include "NiagaraEditorModule.h"
#include "NiagaraConstants.h"

FNiagaraParameterHandle::FNiagaraParameterHandle() 
{
}

FNiagaraParameterHandle::FNiagaraParameterHandle(FName InParameterHandleName)
	: ParameterHandleName(InParameterHandleName)
{
	int32 DotIndex;
	FNameBuilder ParameterHandleBuilder(ParameterHandleName);
	FStringView ParameterHandleString(ParameterHandleBuilder);

	if (ParameterHandleString.FindChar(TEXT('.'), DotIndex))
	{
		Name = FName(ParameterHandleString.RightChop(DotIndex + 1));
		Namespace = FName(ParameterHandleString.Left(DotIndex));
	}
	else
	{
		Name = ParameterHandleName;
	}
}

FNiagaraParameterHandle::FNiagaraParameterHandle(FName InNamespace, FName InName)
	: Name(InName)
	, Namespace(InNamespace)
{
	FNameBuilder ParameterHandleBuilder(Namespace);
	ParameterHandleBuilder.AppendChar(TCHAR('.'));
	InName.AppendString(ParameterHandleBuilder);

	ParameterHandleName = FName(ParameterHandleBuilder);
}

bool FNiagaraParameterHandle::operator==(const FNiagaraParameterHandle& Other) const
{
	return ParameterHandleName == Other.ParameterHandleName;
}

FNiagaraParameterHandle FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(const FNiagaraParameterHandle& ModuleParameterHandle, const UNiagaraNodeFunctionCall* ModuleNode)
{
	return ModuleParameterHandle.IsModuleHandle()
		? FNiagaraParameterHandle(*ModuleNode->GetFunctionName(), ModuleParameterHandle.GetName())
		: ModuleParameterHandle;
}

FNiagaraParameterHandle FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(const FName FullInputName, const FName FunctionName)
{
	FNameBuilder ParameterHandleBuilder(FullInputName);
	FStringView ParamHandleString(ParameterHandleBuilder);

	const int32 ModuleNamespaceLen = FNiagaraConstants::ModuleNamespaceString.Len();

	if (ParamHandleString.Len() > ModuleNamespaceLen
		&& ParamHandleString[ModuleNamespaceLen] == TCHAR('.')
		&& ParamHandleString.StartsWith(FNiagaraConstants::ModuleNamespaceString))
	{
		return FNiagaraParameterHandle(FunctionName, FName(ParamHandleString.RightChop(ModuleNamespaceLen + 1)));
	}

	return FNiagaraParameterHandle(FullInputName);
}

FNiagaraParameterHandle FNiagaraParameterHandle::CreateEngineParameterHandle(const FNiagaraVariable& SystemVariable)
{
	return FNiagaraParameterHandle(SystemVariable.GetName());
}

FNiagaraParameterHandle FNiagaraParameterHandle::CreateEmitterParameterHandle(const FNiagaraVariable& EmitterVariable)
{
	return FNiagaraParameterHandle(FNiagaraConstants::EmitterNamespace, EmitterVariable.GetName());
}

FNiagaraParameterHandle FNiagaraParameterHandle::CreateParticleAttributeParameterHandle(const FName InName)
{
	return FNiagaraParameterHandle(FNiagaraConstants::ParticleAttributeNamespace, InName);
}

FNiagaraParameterHandle FNiagaraParameterHandle::CreateModuleParameterHandle(const FName InName)
{
	return FNiagaraParameterHandle(FNiagaraConstants::ModuleNamespace, InName);
}

FNiagaraParameterHandle FNiagaraParameterHandle::CreateInitialParameterHandle(const FNiagaraParameterHandle& Handle)
{
	FNameBuilder NameBuilder;
	NameBuilder.Append(FNiagaraConstants::InitialPrefix);
	NameBuilder.AppendChar(TCHAR('.'));
	Handle.GetName().AppendString(NameBuilder);

	return FNiagaraParameterHandle(Handle.GetNamespace(), FName(FStringView(NameBuilder)));
}

bool FNiagaraParameterHandle::IsValid() const 
{
	return ParameterHandleName.IsNone() == false;
}

const FName FNiagaraParameterHandle::GetParameterHandleString() const 
{
	return ParameterHandleName;
}

const FName FNiagaraParameterHandle::GetName() const 
{
	return Name;
}

const FName FNiagaraParameterHandle::GetNamespace() const
{
	return Namespace;
}

const TArray<FName> FNiagaraParameterHandle::GetHandleParts() const
{
	if (HandlePartsCache.Num() == 0)
	{
		FString HandleString = ParameterHandleName.ToString();
		TArray<FString> HandlePartStrings;
		HandleString.ParseIntoArray(HandlePartStrings, TEXT("."));
		for (const FString& HandlePartString : HandlePartStrings)
		{
			HandlePartsCache.Add(*HandlePartString);
		}
	}
	return HandlePartsCache;
}

bool FNiagaraParameterHandle::IsUserHandle() const
{
	return Namespace == FNiagaraConstants::UserNamespace;
}

bool FNiagaraParameterHandle::IsEngineHandle() const
{
	return Namespace == FNiagaraConstants::EngineNamespace;
}

bool FNiagaraParameterHandle::IsSystemHandle() const
{
	return Namespace == FNiagaraConstants::SystemNamespace;
}

bool FNiagaraParameterHandle::IsEmitterHandle() const
{
	return Namespace == FNiagaraConstants::EmitterNamespace;
}

bool FNiagaraParameterHandle::IsParticleAttributeHandle() const
{
	return Namespace == FNiagaraConstants::ParticleAttributeNamespace;
}

bool FNiagaraParameterHandle::IsModuleHandle() const
{
	return Namespace == FNiagaraConstants::ModuleNamespace;
}

bool FNiagaraParameterHandle::IsOutputHandle() const
{
	return Namespace == FNiagaraConstants::OutputNamespace;
}

bool FNiagaraParameterHandle::IsLocalHandle() const
{
	return Namespace == FNiagaraConstants::LocalNamespace;
}

bool FNiagaraParameterHandle::IsParameterCollectionHandle() const
{
	return Namespace == FNiagaraConstants::ParameterCollectionNamespace;
}

bool FNiagaraParameterHandle::IsReadOnlyHandle() const
{
	return
		Namespace == FNiagaraConstants::UserNamespace ||
		Namespace == FNiagaraConstants::EngineNamespace ||
		Namespace == FNiagaraConstants::ParameterCollectionNamespace ||
		Namespace == FNiagaraConstants::ModuleNamespace;
}

bool FNiagaraParameterHandle::IsTransientHandle() const
{
	return Namespace == FNiagaraConstants::TransientNamespace;
}

bool FNiagaraParameterHandle::IsDataInstanceHandle() const
{
	return Namespace == FNiagaraConstants::DataInstanceNamespace;
}

bool FNiagaraParameterHandle::IsStackContextHandle() const
{
	return Namespace == FNiagaraConstants::StackContextNamespace;
}

