// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackClipboardUtilities.h"
#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "NiagaraEditorModule.h"
#include "NiagaraClipboard.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NiagaraStackClipboardUtilities"

bool FNiagaraStackClipboardUtilities::TestCanCutSelectionWithMessage(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutCanCutMessage)
{
	if (SelectedEntries.Num() == 1)
	{
		return SelectedEntries[0]->TestCanCutWithMessage(OutCanCutMessage);
	}

	UClass* EntryClass = nullptr;
	bool bAnySupportCut = false;
	bool bAnyCanCut = false;
	bool bClassesMatch = true;
	for (UNiagaraStackEntry* SelectedEntry : SelectedEntries)
	{
		if (EntryClass == nullptr)
		{
			EntryClass = SelectedEntry->GetClass();
		}
		else
		{
			if (SelectedEntry->GetClass() != EntryClass)
			{
				bClassesMatch = false;
			}
		}
		if (SelectedEntry->SupportsCut())
		{
			bAnySupportCut = true;
			FText UnusedMessage;
			bAnyCanCut |= SelectedEntry->TestCanCutWithMessage(UnusedMessage);
		}
	}

	if (bClassesMatch == false)
	{
		OutCanCutMessage = LOCTEXT("CantCutMixedSelection", "Can only Cut selections of similar items, e.g. modules, inputs, renderers, etc.");
		return false;
	}

	if (bAnySupportCut)
	{
		if (bAnyCanCut)
		{
			OutCanCutMessage = LOCTEXT("CutSelection", "Cut the selected items.");
			return true;
		}
		else
		{
			OutCanCutMessage = LOCTEXT("CantCutSelection", "None of the items in the selection can be cut.");
			return false;
		}
	}
	else
	{
		OutCanCutMessage = FText();
		return false;
	}
}

void FNiagaraStackClipboardUtilities::CutSelection(const TArray<UNiagaraStackEntry*>& SelectedEntries)
{
	FText TransactionMessage;
	if (SelectedEntries.Num() == 1)
	{
		TransactionMessage = SelectedEntries[0]->GetCutTransactionText();
	}
	if (TransactionMessage.IsEmptyOrWhitespace())
	{
		TransactionMessage = LOCTEXT("CutSelectionTransaction", "Cut the niagara selection");
	}

	FScopedTransaction CutTransaction(TransactionMessage);
	UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
	for (UNiagaraStackEntry* SelectedEntry : SelectedEntries)
	{
		SelectedEntry->CopyForCut(ClipboardContent);
	}

	TArray<UNiagaraStackEntry*> EntriesToRemove = SelectedEntries;
	SelectedEntries[0]->GetSystemViewModel()->GetSelectionViewModel()->RemoveEntriesFromSelection(SelectedEntries);
	for (UNiagaraStackEntry* EntryToRemove : EntriesToRemove)
	{
		EntryToRemove->RemoveForCut();
	}
	FNiagaraEditorModule::Get().GetClipboard().SetClipboardContent(ClipboardContent);
}

bool FNiagaraStackClipboardUtilities::TestCanCopySelectionWithMessage(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutCanCopyMessage)
{
	if (SelectedEntries.Num() == 1)
	{
		return SelectedEntries[0]->TestCanCopyWithMessage(OutCanCopyMessage);
	}

	UClass* EntryClass = nullptr;
	bool bAnySupportCopy = false;
	bool bAnyCanCopy = false;
	bool bClassesMatch = true;
	for (UNiagaraStackEntry* SelectedEntry : SelectedEntries)
	{
		if (EntryClass == nullptr)
		{
			EntryClass = SelectedEntry->GetClass();
		}
		else
		{
			if (SelectedEntry->GetClass() != EntryClass)
			{
				bClassesMatch = false;
			}
		}
		if (SelectedEntry->SupportsCopy())
		{
			bAnySupportCopy = true;
			FText UnusedMessage;
			bAnyCanCopy |= SelectedEntry->TestCanCopyWithMessage(UnusedMessage);
		}
	}

	if (bClassesMatch == false)
	{
		OutCanCopyMessage = LOCTEXT("CantCopyMixedSelection", "Can only copy selections of similar items, e.g. modules, inputs, renderers, etc.");
		return false;
	}

	if (bAnySupportCopy)
	{
		if (bAnyCanCopy)
		{
			OutCanCopyMessage = LOCTEXT("CopySelection", "Copy the selected items.");
			return true;
		}
		else
		{
			OutCanCopyMessage = LOCTEXT("CantCopySelection", "None of the items in the selection can be copied.");
			return false;
		}
	}
	else
	{
		OutCanCopyMessage = FText();
		return false;
	}
}

void FNiagaraStackClipboardUtilities::CopySelection(const TArray<UNiagaraStackEntry*>& SelectedEntries)
{
	UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
	for (UNiagaraStackEntry* SelectedEntry : SelectedEntries)
	{
		SelectedEntry->Copy(ClipboardContent);
	}
	FNiagaraEditorModule::Get().GetClipboard().SetClipboardContent(ClipboardContent);
}

bool FNiagaraStackClipboardUtilities::TestCanPasteSelectionWithMessage(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutCanPasteMessage)
{
	if (SelectedEntries.Num() > 1)
	{
		OutCanPasteMessage = LOCTEXT("CantPasteToMultipleTargets", "Can not paste with multiple rows selected.");
		return false;
	}
	if (SelectedEntries.Num() == 1 && SelectedEntries[0]->SupportsPaste())
	{
		const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent();
		if (ClipboardContent != nullptr)
		{
			return SelectedEntries[0]->TestCanPasteWithMessage(ClipboardContent, OutCanPasteMessage);
		}
	}
	return false;
}

void FNiagaraStackClipboardUtilities::PasteSelection(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutPasteWarning)
{
	if (SelectedEntries.Num() == 1)
	{
		const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent();

		FText TransactionMessage = ClipboardContent? SelectedEntries[0]->GetPasteTransactionText(ClipboardContent) : FText();
		if (TransactionMessage.IsEmptyOrWhitespace())
		{
			TransactionMessage = LOCTEXT("PasteSelectionTransaction", "Paste to niagara stack.");
		}

		FScopedTransaction PasteTransaction(TransactionMessage);
		if (ClipboardContent != nullptr)
		{
			// If the incoming paste is not targeting a specific location in the stack but the stack group header, auto fixup the paste indices.
			if (SelectedEntries[0]->IsA<UNiagaraStackScriptItemGroup>())
			{
				ClipboardContent->bFixupPasteIndexForScriptDependenciesInStack = true;
			}

			SelectedEntries[0]->Paste(ClipboardContent, OutPasteWarning);
		}
	}
}

bool FNiagaraStackClipboardUtilities::TestCanDeleteSelectionWithMessage(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutCanDeleteMessage)
{
	if (SelectedEntries.Num() == 1)
	{
		if (SelectedEntries[0]->SupportsDelete())
		{
			return SelectedEntries[0]->TestCanDeleteWithMessage(OutCanDeleteMessage);
		}
		return false;
	}
	else
	{
		bool bAnySupportDelete = false;
		bool bAnyCanDelete = false;
		for (UNiagaraStackEntry* SelectedEntry : SelectedEntries)
		{
			if (SelectedEntry->SupportsDelete())
			{
				bAnySupportDelete = true;
				FText UnusedMessage;
				if (SelectedEntry->TestCanDeleteWithMessage(UnusedMessage))
				{
					bAnyCanDelete = true;
					break;
				}
			}
		}

		if (bAnyCanDelete)
		{
			OutCanDeleteMessage = LOCTEXT("DeleteMulti", "Delete items in the selection which can be deleted.");
			return true;
		}
		else
		{
			if (bAnySupportDelete)
			{
				// Only provide a message if an item in the selection actually supports being deleted.
				OutCanDeleteMessage = LOCTEXT("CantDeleteMulti", "None of the selected items can be deleted.");
			}
			return false;
		}
	}
}

void FNiagaraStackClipboardUtilities::CopyNote(const UNiagaraStackEntry* StackEntry)
{
	if(StackEntry->HasStackNoteData())
	{
		UNiagaraClipboardContent* NewClipboardContent = UNiagaraClipboardContent::Create();
		NewClipboardContent->StackNote = StackEntry->GetStackNoteData();

		FNiagaraEditorModule::Get().GetClipboard().SetClipboardContent(NewClipboardContent);
	}
}

void FNiagaraStackClipboardUtilities::DeleteSelection(const TArray<UNiagaraStackEntry*>& SelectedEntries)
{
	FText TransactionMessage;
	if (SelectedEntries.Num() == 1)
	{
		TransactionMessage = SelectedEntries[0]->GetDeleteTransactionText();
	}
	if (TransactionMessage.IsEmptyOrWhitespace())
	{
		TransactionMessage = LOCTEXT("DeleteSelectionTransaction", "Delete the niagara selection");
	}

	TArray<UNiagaraStackEntry*> EntriesToDelete;
	for (UNiagaraStackEntry* SelectedEntry : SelectedEntries)
	{
		FText DeleteMessage;
		if (SelectedEntry->SupportsDelete() && SelectedEntry->TestCanDeleteWithMessage(DeleteMessage))
		{
			EntriesToDelete.Add(SelectedEntry);
		}
	}

	if (EntriesToDelete.Num() > 0)
	{
		EntriesToDelete[0]->GetSystemViewModel()->GetSelectionViewModel()->RemoveEntriesFromSelection(EntriesToDelete);
		const FScopedTransaction DeleteTransaction(TransactionMessage);
		for (UNiagaraStackEntry* EntryToDelete : EntriesToDelete)
		{
			EntryToDelete->Delete();
		}
	}
}

#undef LOCTEXT_NAMESPACE