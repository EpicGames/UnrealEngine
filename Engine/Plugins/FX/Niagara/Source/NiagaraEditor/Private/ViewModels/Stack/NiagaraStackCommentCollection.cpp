// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackCommentCollection.h"

#include "EdGraphNode_Comment.h"
#include "NiagaraObjectSelection.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackCommentCollection)

void UNiagaraStackCommentCollection::Initialize(FRequiredEntryData InRequiredEntryData)
{
	const FString StackKey = TEXT("CommentCollection");
	Super::Initialize(InRequiredEntryData, StackKey);
}

UNiagaraStackObject* UNiagaraStackCommentCollection::FindStackObjectForCommentNode(UEdGraphNode_Comment* CommentNode) const
{
	TArray<UNiagaraStackObject*> CommentStackEntries;
	GetUnfilteredChildrenOfType(CommentStackEntries);

	UNiagaraStackObject** FoundObject = CommentStackEntries.FindByPredicate([CommentNode](UNiagaraStackObject* CandidateStackObject)
	{
		return CandidateStackObject->GetObject() == CommentNode;
	});

	if(FoundObject)
	{
		return *FoundObject;
	}

	return nullptr;
}

void UNiagaraStackCommentCollection::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	TSet<UObject*> SelectedNodes = GetSystemViewModel()->GetOverviewGraphViewModel()->GetNodeSelection()->GetSelectedObjectsResolved();

	for (UObject* SelectedNode : SelectedNodes)
	{
		if(UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(SelectedNode))
		{
			UNiagaraStackObject* StackObject = FindCurrentChildOfTypeByPredicate<UNiagaraStackObject>(CurrentChildren, [&](UNiagaraStackObject* CandidateStackObject)
			{
				return CandidateStackObject->GetObject() == CommentNode;
			});
			
			if(StackObject == nullptr)
			{
				bool bIsInTopLevelObject = true;
				bool bHideTopLevelCategories = false;
				StackObject = NewObject<UNiagaraStackObject>(this);
				StackObject->Initialize(CreateDefaultChildRequiredData(), CommentNode, bIsInTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey(), nullptr);
			}
			
			NewChildren.Add(StackObject);
		}
	}
}
