// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackEmitterPropertiesGroup.h"

#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraScriptSource.h"
#include "ScopedTransaction.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackEmitterSettingsGroup.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "NiagaraSimulationStageBase.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackEmitterPropertiesGroup)

#define LOCTEXT_NAMESPACE "StackEmitterProperties"

FNiagaraStackEmitterStageAddUtilities::FNiagaraStackEmitterStageAddUtilities(TSharedPtr<FNiagaraEmitterViewModel> InEmitterViewModel, FOnItemAdded InOnItemAdded, bool bInAllowEventHandlers, bool bInAllowSimulationStages)
	: FNiagaraStackItemGroupAddUtilities(LOCTEXT("AddUtilitiesName", "Stage"), EAddMode::AddFromAction, true, true)
	, EmitterViewModelWeak(TWeakPtr<FNiagaraEmitterViewModel>(InEmitterViewModel))
	, OnItemAdded(InOnItemAdded)
	, bAllowEventHandlers(bInAllowEventHandlers)
	, bAllowSimulationStages(bInAllowSimulationStages)
{
}

void FNiagaraStackEmitterStageAddUtilities::GenerateAddActions(TArray<TSharedRef<INiagaraStackItemGroupAddAction>>& OutAddActions, const FNiagaraStackItemGroupAddOptions& AddProperties) const
{
	if (bAllowEventHandlers)
	{
		TArray<FString> EventsCategories = { "Events" };
		OutAddActions.Add(MakeShared<FAddEmitterStageAction>(
			EStageAddMode::Event,
			nullptr,
			EventsCategories,
			LOCTEXT("AddEventActionDisplayName", "Event Handler"),
			LOCTEXT("AddEventActionDescription", "Add an event handler to this emitter."),
			LOCTEXT("AddEventActionKeywords", "Event")));
	}

	if (bAllowSimulationStages)
	{
		TArray<FString> SimulationStageCategories = { "Simulation Stages" };
		const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
		TArray<UClass*> SimulationStageClasses;
		GetDerivedClasses(UNiagaraSimulationStageBase::StaticClass(), SimulationStageClasses);
		for (UClass* SimulationStageClass : SimulationStageClasses)
		{
			if (NiagaraEditorSettings->IsVisibleClass(SimulationStageClass))
			{
				OutAddActions.Add(MakeShared<FAddEmitterStageAction>(
					EStageAddMode::SimulationStage,
					SimulationStageClass,
					SimulationStageCategories,
					SimulationStageClass->GetDisplayNameText(),
					FText::FromString(SimulationStageClass->GetDescription()),
					FText::FromString(SimulationStageClass->GetName())));
			}
		}
	}
}

void FNiagaraStackEmitterStageAddUtilities::ExecuteAddAction(TSharedRef<INiagaraStackItemGroupAddAction> AddAction, int32 TargetIndex)
{
	TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = EmitterViewModelWeak.Pin();
	if (EmitterViewModel.IsValid() == false)
	{
		return;
	}

	FVersionedNiagaraEmitter VersionedEmitter = EmitterViewModel->GetEmitter();
	UNiagaraEmitter* Emitter = VersionedEmitter.Emitter;
	UNiagaraScriptSource* Source = EmitterViewModel->GetSharedScriptViewModel()->GetGraphViewModel()->GetScriptSource();
	UNiagaraGraph* Graph = EmitterViewModel->GetSharedScriptViewModel()->GetGraphViewModel()->GetGraph();

	// The stack should not have been created if any of these are null, so bail out if it happens somehow rather than try to handle all of these cases.
	checkf(Emitter != nullptr && Source != nullptr && Graph != nullptr, TEXT("Stack created for invalid emitter or graph."));

	TSharedRef<FAddEmitterStageAction> AddEmitterStageAction = StaticCastSharedRef<FAddEmitterStageAction>(AddAction);
	FGuid AddedEventHandlerId;
	UNiagaraSimulationStageBase* AddedSimulationStage = nullptr;
	if (AddEmitterStageAction->AddMode == EStageAddMode::Event)
	{
		// since this is potentially modifying live data we need to kill off any existing instances that might be in flight before we make
		// the change.
		if (const UNiagaraSystem* EmitterSystem = Cast<UNiagaraSystem>(Emitter->GetOuter()))
		{
			FNiagaraEditorUtilities::KillSystemInstances(*EmitterSystem);
		}

		FScopedTransaction ScopedTransaction(LOCTEXT("AddNewEventHandlerTransaction", "Add new event handler"));

		Emitter->Modify();
		FNiagaraEventScriptProperties EventScriptProperties;
		EventScriptProperties.Script = NewObject<UNiagaraScript>(Emitter, MakeUniqueObjectName(Emitter, UNiagaraScript::StaticClass(), "EventScript"), EObjectFlags::RF_Transactional);
		EventScriptProperties.Script->SetUsage(ENiagaraScriptUsage::ParticleEventScript);
		EventScriptProperties.Script->SetUsageId(FGuid::NewGuid());
		EventScriptProperties.Script->SetLatestSource(Source);
		Emitter->AddEventHandler(EventScriptProperties, VersionedEmitter.Version);
		FNiagaraStackGraphUtilities::ResetGraphForOutput(*Graph, ENiagaraScriptUsage::ParticleEventScript, EventScriptProperties.Script->GetUsageId());
		AddedEventHandlerId = EventScriptProperties.Script->GetUsageId();
	}
	else if (AddEmitterStageAction->AddMode == EStageAddMode::SimulationStage && AddEmitterStageAction->SimulationStageClass != nullptr)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("AddNewSimulationStagesTransaction", "Add new simulation stage"));

		Emitter->Modify();
		AddedSimulationStage = NewObject<UNiagaraSimulationStageBase>(Emitter, AddEmitterStageAction->SimulationStageClass, NAME_None, RF_Transactional);
		AddedSimulationStage->Script = NewObject<UNiagaraScript>(AddedSimulationStage, MakeUniqueObjectName(AddedSimulationStage, UNiagaraScript::StaticClass(), "SimulationStage"), EObjectFlags::RF_Transactional);
		AddedSimulationStage->Script->SetUsage(ENiagaraScriptUsage::ParticleSimulationStageScript);
		AddedSimulationStage->Script->SetUsageId(AddedSimulationStage->GetMergeId());
		AddedSimulationStage->Script->SetLatestSource(Source);
		Emitter->AddSimulationStage(AddedSimulationStage, VersionedEmitter.Version);
		if (TargetIndex != INDEX_NONE)
		{
			Emitter->MoveSimulationStageToIndex(AddedSimulationStage, TargetIndex, VersionedEmitter.Version);
		}
		FNiagaraStackGraphUtilities::ResetGraphForOutput(*Graph, ENiagaraScriptUsage::ParticleSimulationStageScript, AddedSimulationStage->Script->GetUsageId());
	}

	// Set the emitter here so that the internal state of the view model is updated.
	// TODO: Move the logic for managing additional scripts into the emitter view model or script view model.
	TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> Simulation = EmitterViewModel->GetSimulation();
	EmitterViewModel->Reset();
	EmitterViewModel->Initialize(VersionedEmitter, Simulation);

	OnItemAdded.ExecuteIfBound(AddedEventHandlerId, AddedSimulationStage);
}

UNiagaraStackEmitterPropertiesGroup::UNiagaraStackEmitterPropertiesGroup()
	: PropertiesItem(nullptr)
{
}

void UNiagaraStackEmitterPropertiesGroup::Initialize(FRequiredEntryData InRequiredEntryData)
{
	FText DisplayName = LOCTEXT("EmitterPropertiesGroupName", "Properties");
	FText Tooltip = LOCTEXT("EmitterPropertiesTooltip", "Properties which are set per Emitter.");
	AddUtilities = MakeShared<FNiagaraStackEmitterStageAddUtilities>(InRequiredEntryData.EmitterViewModel.ToSharedRef(),
		FNiagaraStackEmitterStageAddUtilities::FOnItemAdded::CreateUObject(this, &UNiagaraStackEmitterPropertiesGroup::ItemAddedFromUtilities));
	Super::Initialize(InRequiredEntryData, DisplayName, Tooltip, AddUtilities.Get());
}

const FSlateBrush* UNiagaraStackEmitterPropertiesGroup::GetIconBrush() const
{
	return FAppStyle::Get().GetBrush("Icons.Details");
}

bool UNiagaraStackEmitterPropertiesGroup::SupportsSecondaryIcon() const
{
	if(IsFinalized() == false && GetEmitterViewModel().IsValid())
	{
		// we don't want to display the CPU/GPU icons in collapsed mode as we have a separate toggle for it
		if(GetEmitterViewModel()->GetEditorData().ShouldShowSummaryView() == true)
		{
			return false;
		}
	}

	return true;
}

const FSlateBrush* UNiagaraStackEmitterPropertiesGroup::GetSecondaryIconBrush() const
{
	if (IsFinalized() == false && GetEmitterViewModel().IsValid())
	{
		FVersionedNiagaraEmitterData* EmitterData = GetEmitterViewModel()->GetEmitter().GetEmitterData();
		//-TODO:Stateless: Do we need a stateless icon?
		if (EmitterData == nullptr)
		{
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.GPUIcon");
		}
		if (EmitterData->SimTarget == ENiagaraSimTarget::CPUSim)
		{
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.CPUIcon");
		}
		if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.GPUIcon");
		}
	}
	return FAppStyle::GetBrush("NoBrush");
}

void UNiagaraStackEmitterPropertiesGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (PropertiesItem == nullptr)
	{
		PropertiesItem = NewObject<UNiagaraStackEmitterPropertiesItem>(this);
		PropertiesItem->Initialize(CreateDefaultChildRequiredData());
	}
	NewChildren.Add(PropertiesItem);

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

void UNiagaraStackEmitterPropertiesGroup::ItemAddedFromUtilities(FGuid AddedEventHandlerId, UNiagaraSimulationStageBase* AddedSimulationStage)
{
	OnRequestFullRefresh().Broadcast();

	GetSystemViewModel()->GetSelectionViewModel()->EmptySelection();
	if (AddedEventHandlerId.IsValid())
	{
		GetSystemViewModel()->GetSelectionViewModel()->AddEntryToSelectionBySelectionIdDeferred(AddedEventHandlerId);
	}
	else if (AddedSimulationStage != nullptr)
	{
		GetSystemViewModel()->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectKeyDeferred(FObjectKey(AddedSimulationStage));
	}
}

#undef LOCTEXT_NAMESPACE
