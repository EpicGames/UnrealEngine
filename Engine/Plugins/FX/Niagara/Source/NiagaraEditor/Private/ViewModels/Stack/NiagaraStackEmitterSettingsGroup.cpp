// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackEmitterSettingsGroup.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackSummaryViewInputCollection.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterEditorData.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "NiagaraScriptMergeManager.h"
#include "NiagaraEmitterDetailsCustomization.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraSystem.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Styling/AppStyle.h"
#include "IDetailTreeNode.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraSettings.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackEmitterSettingsGroup)

#define LOCTEXT_NAMESPACE "UNiagaraStackEmitterItemGroup"

void UNiagaraStackEmitterPropertiesItem::Initialize(FRequiredEntryData InRequiredEntryData)
{
	Super::Initialize(InRequiredEntryData, TEXT("EmitterProperties"));
	EmitterWeakPtr = GetEmitterViewModel()->GetEmitter().ToWeakPtr();
	//-TODO:Stateless:
	if (EmitterWeakPtr.IsValid())
	//-TODO:Stateless:
	{
		EmitterWeakPtr.Emitter->OnPropertiesChanged().AddUObject(this, &UNiagaraStackEmitterPropertiesItem::EmitterPropertiesChanged);
	}
}

void UNiagaraStackEmitterPropertiesItem::FinalizeInternal()
{
	if (EmitterWeakPtr.IsValid())
	{
		EmitterWeakPtr.Emitter->OnPropertiesChanged().RemoveAll(this);
	}
	Super::FinalizeInternal();
}

FText UNiagaraStackEmitterPropertiesItem::GetDisplayName() const
{
	return LOCTEXT("EmitterPropertiesName", "Emitter Properties");
}

FText UNiagaraStackEmitterPropertiesItem::GetTooltipText() const
{
	return LOCTEXT("EmitterPropertiesTooltip", "Properties that are handled per Emitter. These cannot change at runtime.");
}

bool UNiagaraStackEmitterPropertiesItem::TestCanResetToBaseWithMessage(FText& OutCanResetToBaseMessage) const
{
	if (GetEmitterViewModel().IsValid() == false)
	{
		bCanResetToBaseCache = false;
	}
	else if (bCanResetToBaseCache.IsSet() == false)
	{
		FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetParentEmitter();
		if (BaseEmitter.GetEmitterData() != nullptr && EmitterWeakPtr.Emitter != BaseEmitter.Emitter)
		{
			TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
			bCanResetToBaseCache = MergeManager->IsEmitterEditablePropertySetDifferentFromBase(EmitterWeakPtr.ResolveWeakPtr(), BaseEmitter);
		}
		else
		{
			bCanResetToBaseCache = false;
		}
	}
	if (bCanResetToBaseCache.GetValue())
	{
		OutCanResetToBaseMessage = LOCTEXT("CanResetToBase", "Reset the emitter properties to the state defined by the parent emitter.");
		return true;
	}
	else
	{
		OutCanResetToBaseMessage = LOCTEXT("CanNotResetToBase", "No parent to reset to, or not different from parent.");
		return false;
	}
}

void UNiagaraStackEmitterPropertiesItem::ResetToBase()
{
	FText Unused;
	if (TestCanResetToBaseWithMessage(Unused))
	{
		FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetParentEmitter();
		TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
		MergeManager->ResetEmitterEditablePropertySetToBase(EmitterWeakPtr.ResolveWeakPtr(), BaseEmitter);
	}
}

const FSlateBrush* UNiagaraStackEmitterPropertiesItem::GetIconBrush() const
{
	if (FVersionedNiagaraEmitterData* EmitterData = EmitterWeakPtr.GetEmitterData())
	{
		if (EmitterData->SimTarget == ENiagaraSimTarget::CPUSim)
		{
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.CPUIcon");
		}
		if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
		{
			return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.GPUIcon");
		}
	}
	return FAppStyle::GetBrush("NoBrush");
}

void UNiagaraStackEmitterPropertiesItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (EmitterObject == nullptr)
	{
		EmitterObject = NewObject<UNiagaraStackObject>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(), FExecutionCategoryNames::Emitter, NAME_None, GetStackEditorData());
		if (EmitterWeakPtr.IsValid())
		{
			bool bIsTopLevelObject = true;
			bool bHideTopLevelCategories = false;
			EmitterObject->Initialize(RequiredEntryData, EmitterWeakPtr.Emitter.Get(), bIsTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey());
			EmitterObject->RegisterInstancedCustomPropertyLayout(UNiagaraEmitter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraEmitterDetails::MakeInstance));
		}
	}

	NewChildren.Add(EmitterObject);
	bCanResetToBaseCache.Reset();
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
	RefreshIssues(NewIssues);
}

UNiagaraStackEntry::FStackIssueFixDelegate UNiagaraStackEmitterPropertiesItem::GetUpgradeVersionFix()
{
	return FStackIssueFixDelegate::CreateLambda([this]()
	{
		FGuid NewVersion = GetEmitterViewModel()->GetParentEmitter().Emitter->GetExposedVersion().VersionGuid;
		FNiagaraEditorUtilities::SwitchParentEmitterVersion(GetEmitterViewModel().ToSharedRef(), GetSystemViewModel(), NewVersion);
	});
}

void UNiagaraStackEmitterPropertiesItem::RefreshIssues(TArray<FStackIssue>& NewIssues)
{
	FNiagaraStackGraphUtilities::CheckForDeprecatedEmitterVersion(GetEmitterViewModel(), GetStackEditorDataKey(), GetUpgradeVersionFix(), NewIssues);
	
	FVersionedNiagaraEmitterData* ActualEmitterData = GetEmitterViewModel()->GetEmitter().GetEmitterData();
	if (ActualEmitterData)
	{
		if (ActualEmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim && ActualEmitterData->CalculateBoundsMode == ENiagaraEmitterCalculateBoundMode::Dynamic)
		{
			bool bAddError = true;

			UNiagaraSystem& Sys = GetSystemViewModel()->GetSystem();
			if (Sys.bFixedBounds)
			{
				bAddError = false;
			}

			if (bAddError)
			{
				FStackIssue MissingRequiredFixedBoundsModuleError(
					EStackIssueSeverity::Warning,
					LOCTEXT("RequiredFixedBoundsWarningFormat", "The emitter is GPU and is using dynamic bounds mode.\r\nPlease update the Emitter or System properties otherwise bounds may be incorrect."),
					LOCTEXT("MissingFixedBounds", "Missing fixed bounds."),
					GetStackEditorDataKey(),
					false);

				NewIssues.Add(MissingRequiredFixedBoundsModuleError);
			}
		}

		if ( ActualEmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim && ActualEmitterData->bGpuAlwaysRunParticleUpdateScript )
		{
			const FText FixGpuInterpolatedDisabledText = LOCTEXT("FixGpuInterpolatedDisabled", "Fix GPU emitter to not run particle update script on particle spawn");
			FVersionedNiagaraEmitterWeakPtr WeakEmitter = GetEmitterViewModel()->GetEmitter().ToWeakPtr();

			NewIssues.Emplace(
				EStackIssueSeverity::Warning,
				LOCTEXT("GpuNoInterpolatedIncorrect", "GPU is incorrectly running particle update script on particle spawn."),
				LOCTEXT("GpuNoInterpolatedIncorrectLong", "Due to a previous GPU codegen issue both particle spawn & update scripts were running on particle spawn when interpolated spawning is disabled.  This is now fixed for new content to match CPU behavior where only particle spawn will run when interpolated spawning is disabled."),
				GetStackEditorDataKey(),
				true,
				FStackIssueFix(
					FixGpuInterpolatedDisabledText,
					FStackIssueFixDelegate::CreateLambda(
						[WeakEmitter, FixGpuInterpolatedDisabledText]()
						{
							FVersionedNiagaraEmitter PinnedEmitter = WeakEmitter.ResolveWeakPtr();
							if (PinnedEmitter.Emitter)
							{
								FScopedTransaction ScopedTransaction(FixGpuInterpolatedDisabledText);

								PinnedEmitter.Emitter->Modify();
								PinnedEmitter.GetEmitterData()->bGpuAlwaysRunParticleUpdateScript = false;
								UNiagaraSystem::RequestCompileForEmitter(PinnedEmitter);
							}
						}
					)
				)
			);
		}

		if (ActualEmitterData->UsesInterpolatedSpawning() == false)
		{
			const FText InterpolatedSpawningDisabledInfoFix = LOCTEXT("InterpolatedSpawningDisabledInfoFix", "Run update script on spawn");
			FVersionedNiagaraEmitterWeakPtr WeakEmitter = GetEmitterViewModel()->GetEmitter().ToWeakPtr();

			NewIssues.Emplace(
				EStackIssueSeverity::Info,
				LOCTEXT("InterpolatedSpawningDisabledInfo", "Interpolated spawning disabled, particle update will not run on the first frame."),
				LOCTEXT("InterpolatedSpawningDisabledDetailed", "When interpolated spawning is disabled only the spawn script runs on the first frame."),
				GetStackEditorDataKey(),
				true,
				FStackIssueFix(
					InterpolatedSpawningDisabledInfoFix,
					FStackIssueFixDelegate::CreateLambda(
						[WeakEmitter, InterpolatedSpawningDisabledInfoFix]()
						{
							FVersionedNiagaraEmitter PinnedEmitter = WeakEmitter.ResolveWeakPtr();
							if (PinnedEmitter.Emitter)
							{
								FScopedTransaction ScopedTransaction(InterpolatedSpawningDisabledInfoFix);

								PinnedEmitter.Emitter->Modify();
								PinnedEmitter.GetEmitterData()->InterpolatedSpawnMode = ENiagaraInterpolatedSpawnMode::RunUpdateScript;
								UNiagaraSystem::RequestCompileForEmitter(PinnedEmitter);
							}
						}
					)
				)
			);
		}

		UNiagaraSystem& System = GetSystemViewModel()->GetSystem();
		TWeakPtr<FNiagaraSystemViewModel> WeakSysViewModel = GetSystemViewModel();
		if (System.NeedsWarmup())
		{
			const UNiagaraSettings* NiagaraSettings = GetDefault<UNiagaraSettings>();
			float WarmupDelta = System.GetWarmupTickDelta();
			if (NiagaraSettings->bLimitDeltaTime && NiagaraSettings->MaxDeltaTimePerTick < WarmupDelta)
			{
				TArray<FStackIssueFix> Fixes;

				float MaxEmitterDt = NiagaraSettings->MaxDeltaTimePerTick;
				//This emitter does not allow ticks with a delta time so large.
				FText FixDescriptionReduceWarmupDt = LOCTEXT("FixWarmupDeltaTime", "Reduce System Warmup Delta Time");
				Fixes.Emplace(
					FixDescriptionReduceWarmupDt,
					FStackIssueFixDelegate::CreateLambda([=]()
						{
							if (auto Pinned = WeakSysViewModel.Pin())
							{
								FScopedTransaction ScopedTransaction(FixDescriptionReduceWarmupDt);
								Pinned->GetSystem().Modify();
								Pinned->GetSystem().SetWarmupTickDelta(MaxEmitterDt);
								Pinned->RefreshAll();
							}
						}));

				FStackIssue WarmupDeltaTimeExceedsEmitterDeltaTimeWarning(
					EStackIssueSeverity::Warning,
					LOCTEXT("WarmupDeltaTimeExceedsEmitterDeltaTimeWarningSummary", "System Warmup Delta Time Exceeds Emitter Max Delta Time."),
					LOCTEXT("WarmupDeltaTimeExceedsEmitterDeltaTimeWarningText", "Max Tick Delta Time is smaller than the System's Warmup Delta Time. This could cause unintended results during warmup for this emitter.\nThe max tick delta time can be changed in the Niagara settings."),
					GetStackEditorDataKey(),
					false,
					Fixes);

				NewIssues.Add(WarmupDeltaTimeExceedsEmitterDeltaTimeWarning);
			}
		}

		// check for any emitters which are exclusively using Emitter sourced renderers and dynamic bounds.  Currently our bounds calculators don't support
		// emitter sources and so no valid bounds will be generated
		if (ActualEmitterData->SimTarget == ENiagaraSimTarget::CPUSim && ActualEmitterData->CalculateBoundsMode == ENiagaraEmitterCalculateBoundMode::Dynamic)
		{
			UNiagaraSystem& Sys = GetSystemViewModel()->GetSystem();
			if (!Sys.bFixedBounds)
			{
				bool bHasParticleSourcedRenderer = false;
				ActualEmitterData->ForEachEnabledRenderer([&](UNiagaraRendererProperties* RendererProperties)
				{
					if (RendererProperties && RendererProperties->GetCurrentSourceMode() == ENiagaraRendererSourceDataMode::Particles)
					{
						bHasParticleSourcedRenderer = true;
					}
				});

				if (!bHasParticleSourcedRenderer)
				{
					FStackIssue EmitterSourcedEmitterRequiredFixedBoundsError(
						EStackIssueSeverity::Warning,
						LOCTEXT("EmitterSourcedWarningFormat", "The emitter is using dynamic bounds mode but only using Emitter sourced renderers.\r\nPlease update the Emitter or System properties otherwise bounds may be incorrect."),
						LOCTEXT("EmitterSourcedWarning", "Missing fixed bounds."),
						GetStackEditorDataKey(),
						false);

					NewIssues.Add(EmitterSourcedEmitterRequiredFixedBoundsError);
				}
			}
		}
	}

}

void UNiagaraStackEmitterPropertiesItem::EmitterPropertiesChanged()
{
	if (IsFinalized() == false)
	{
		// Undo/redo can cause objects to disappear and reappear which can prevent safe removal of delegates
		// so guard against receiving an event when finalized here.
		bCanResetToBaseCache.Reset();
		RefreshChildren();

		if (EmitterWeakPtr.IsValid())
		{
			GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(EmitterWeakPtr.ResolveWeakPtr()).Get()->GetEmitterStackViewModel()->RequestValidationUpdate();
		}
	}
}

void UNiagaraStackEmitterSummaryItem::Initialize(FRequiredEntryData InRequiredEntryData)
{
	Super::Initialize(InRequiredEntryData, TEXT("EmitterParameters"));
	Emitter = GetEmitterViewModel()->GetEmitter().ToWeakPtr();
}

FText UNiagaraStackEmitterSummaryItem::GetDisplayName() const
{
	return LOCTEXT("EmitterSummaryName", "Emitter Summary");
}

FText UNiagaraStackEmitterSummaryItem::GetTooltipText() const
{
	return LOCTEXT("EmitterSummaryTooltip", "Subset of parameters from the stack, summarized here for easier access.");
}

FText UNiagaraStackEmitterSummaryItem::GetIconText() const
{
	return FText::FromString(FString(TEXT("\xf0ca")/* fa-list-ul */));
}

void UNiagaraStackEmitterSummaryItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (SummaryViewCollection == nullptr)
	{
		SummaryViewCollection = NewObject<UNiagaraStackSummaryViewCollection>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(), FExecutionCategoryNames::Emitter, NAME_None, GetStackEditorData());
		SummaryViewCollection->Initialize(RequiredEntryData, Emitter, GetStackEditorDataKey());
	}

	NewChildren.Add(SummaryViewCollection);
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

void UNiagaraStackEmitterSummaryItem::ToggleShowAdvancedInternal()
{
	// this will cause section data & filtered children to be refreshed while caching the last active section
	SummaryViewCollection->RefreshForAdvancedToggle();
}

TSharedPtr<IDetailTreeNode> GetSummarySectionsPropertyNode(const TArray<TSharedRef<IDetailTreeNode>>& Nodes)
{
	TArray<TSharedRef<IDetailTreeNode>> ChildrenToCheck;
	for (TSharedRef<IDetailTreeNode> Node : Nodes)
	{
		if (Node->GetNodeType() == EDetailNodeType::Item)
		{
			TSharedPtr<IPropertyHandle> NodePropertyHandle = Node->CreatePropertyHandle();
			if (NodePropertyHandle.IsValid() && NodePropertyHandle->GetProperty()->GetFName() == UNiagaraEmitterEditorData::PrivateMemberNames::SummarySections)
			{
				return Node;
			}
		}

		TArray<TSharedRef<IDetailTreeNode>> Children;
		Node->GetChildren(Children);
		ChildrenToCheck.Append(Children);
	}
	if (ChildrenToCheck.Num() == 0)
	{
		return nullptr;
	}
	return GetSummarySectionsPropertyNode(ChildrenToCheck);
}

void UNiagaraStackEmitterSummaryItem::OnEditButtonClicked()
{
	if (GetEmitterViewModel().IsValid())
	{
		GetSystemViewModel()->FocusTab(FNiagaraSystemToolkitModeBase::EmitterSummaryViewEditorTabID, true);
	}
}

TOptional<FText> UNiagaraStackEmitterSummaryItem::GetEditModeButtonText() const
{
	return LOCTEXT("EditSummaryViewButtonLabel", "Edit Summary");
}

TOptional<FText> UNiagaraStackEmitterSummaryItem::GetEditModeButtonTooltip() const
{
	return LOCTEXT("EditSummaryViewButtonTooltip", "Summons the Summary Editor that lets you define what inputs, renderers and properties (and more) should be displayed when looking at the Emitter Summary.");
}

UNiagaraStackEmitterSummaryGroup::UNiagaraStackEmitterSummaryGroup()
	: SummaryItem(nullptr)
{
}

void UNiagaraStackEmitterSummaryGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (SummaryItem == nullptr)
	{
		SummaryItem = NewObject<UNiagaraStackEmitterSummaryItem>(this);
		SummaryItem->Initialize(CreateDefaultChildRequiredData());
	}
	NewChildren.Add(SummaryItem);

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

FText UNiagaraStackEmitterSummaryGroup::GetIconText() const
{
	return FText::FromString(FString(TEXT("\xf0ca")/* fa-list-ul */));
}

#undef LOCTEXT_NAMESPACE

