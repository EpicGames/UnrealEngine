// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackEntry.h"

#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "ViewModels/Stack/NiagaraStackNote.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraScriptMergeManager.h"
#include "Misc/SecureHash.h"
#include "Internationalization/Text.h"
#include "ScopedTransaction.h"
#include "DataHierarchyViewModelBase.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"
#include "ViewModels/Stack/NiagaraStackClipboardUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackEntry)

class UNiagaraStackItemGroup;
const FName UNiagaraStackEntry::FExecutionCategoryNames::System = TEXT("System");
const FName UNiagaraStackEntry::FExecutionCategoryNames::Emitter = TEXT("Emitter");
const FName UNiagaraStackEntry::FExecutionCategoryNames::StatelessEmitter = TEXT("StatelessEmitter");
const FName UNiagaraStackEntry::FExecutionCategoryNames::Particle = TEXT("Particle");
const FName UNiagaraStackEntry::FExecutionCategoryNames::Render = TEXT("Render");

const FName UNiagaraStackEntry::FExecutionSubcategoryNames::Settings = TEXT("Settings");
const FName UNiagaraStackEntry::FExecutionSubcategoryNames::Spawn = TEXT("Spawn");
const FName UNiagaraStackEntry::FExecutionSubcategoryNames::Update = TEXT("Update");
const FName UNiagaraStackEntry::FExecutionSubcategoryNames::Event = TEXT("Event");
const FName UNiagaraStackEntry::FExecutionSubcategoryNames::SimulationStage = TEXT("Simulation Stage");
const FName UNiagaraStackEntry::FExecutionSubcategoryNames::Render = TEXT("Render");

#define LOCTEXT_NAMESPACE "UNiagaraStackEntry"

namespace
{
	// Build a locale-independent hash input for an FText. LOCTEXT-derived text carries a stable
	// (Namespace, Key) pair that survives translation - we prefer that so two LOCTEXT entries
	// that translate to the same string (or even share an English source string) still produce
	// distinct identifiers. Source-string fallback covers FText::FromString / FText::Format
	// results that have no key; ToString() is a final fallback for FText with neither.
	FString MakeStableIdentifierInput(const FText& Text)
	{
		const TOptional<FString> Namespace = FTextInspector::GetNamespace(Text);
		const TOptional<FString> Key = FTextInspector::GetKey(Text);
		if (Key.IsSet() && !Key->IsEmpty())
		{
			return FString::Printf(TEXT("%s::%s"),
				Namespace.IsSet() ? **Namespace : TEXT(""),
				**Key);
		}

		if (const FString* Source = FTextInspector::GetSourceString(Text))
		{
			return *Source;
		}

		return Text.ToString();
	}
}

UNiagaraStackEntry::FStackIssueFix::FStackIssueFix() : Style(EStackIssueFixStyle::Fix)
{
}

UNiagaraStackEntry::FStackIssueFix::FStackIssueFix(FText InDescription, FStackIssueFixDelegate InFixDelegate, EStackIssueFixStyle InFixStyle)
	: Description(InDescription)
	, FixDelegate(InFixDelegate)
	, Style(InFixStyle)
	, UniqueIdentifier(FMD5::HashAnsiString(*MakeStableIdentifierInput(InDescription)))
{
	checkf(Description.IsEmptyOrWhitespace() == false, TEXT("Description can not be empty."));
	checkf(InFixDelegate.IsBound(), TEXT("Fix delegate must be bound."));
}

bool UNiagaraStackEntry::FStackIssueFix::IsValid() const
{
	return FixDelegate.IsBound();
}

const FText& UNiagaraStackEntry::FStackIssueFix::GetDescription() const
{
	return Description;
}

void UNiagaraStackEntry::FStackIssueFix::SetFixDelegate(const FStackIssueFixDelegate& InFixDelegate)
{
	FixDelegate = InFixDelegate;
}

const FString& UNiagaraStackEntry::FStackIssueFix::GetUniqueIdentifier() const
{
	return UniqueIdentifier;
}

const UNiagaraStackEntry::FStackIssueFixDelegate& UNiagaraStackEntry::FStackIssueFix::GetFixDelegate() const
{
	return FixDelegate;
}

UNiagaraStackEntry::EStackIssueFixStyle UNiagaraStackEntry::FStackIssueFix::GetStyle() const
{
	return Style;
}

UNiagaraStackEntry::FStackIssue::FStackIssue() 
	: Severity(EStackIssueSeverity::None)
	, DismissHandler(nullptr)
{
}

UNiagaraStackEntry::FStackIssue::FStackIssue(EStackIssueSeverity InSeverity, FText InShortDescription, FText InLongDescription, FString InStackEditorDataKey, bool bInCanBeDismissed, const TArray<FStackIssueFix>& InFixes, const FSimpleDelegate& InDismissHandler)
	: Severity(InSeverity)
	, ShortDescription(InShortDescription)
	, LongDescription(InLongDescription)
	, UniqueIdentifier(FMD5::HashAnsiString(*FString::Printf(TEXT("%s-%s-%s"), *InStackEditorDataKey, *MakeStableIdentifierInput(InShortDescription), *MakeStableIdentifierInput(InLongDescription))))
	, bCanBeDismissed(bInCanBeDismissed)
	, Fixes(InFixes)
	, DismissHandler(InDismissHandler)
{
	checkf(ShortDescription.IsEmptyOrWhitespace() == false, TEXT("Short description can not be empty."));
	//checkf(LongDescription.IsEmptyOrWhitespace() == false, TEXT("Long description can not be empty."));
	//if (LongDescription.IsEmptyOrWhitespace())
	//{
	//	LongDescription = ShortDescription;
	//}
	checkf(InStackEditorDataKey.IsEmpty() == false, TEXT("Stack editor data key can not be empty."));
}

UNiagaraStackEntry::FStackIssue::FStackIssue(EStackIssueSeverity InSeverity, FText InShortDescription, FText InLongDescription, FString InStackEditorDataKey, bool bInCanBeDismissed, FStackIssueFix InFix)
	: FStackIssue(InSeverity, InShortDescription, InLongDescription, InStackEditorDataKey, bInCanBeDismissed, TArray<FStackIssueFix>({InFix }))
{
}

UNiagaraStackEntry::FStackIssue::FStackIssue(EStackIssueSeverity InSeverity, FText InShortDescription, FText InLongDescription, FString InStackEditorDataKey, bool bInCanBeDismissed)
	: FStackIssue(InSeverity, InShortDescription, InLongDescription, InStackEditorDataKey, bInCanBeDismissed, TArray<FStackIssueFix>())
{
}

bool UNiagaraStackEntry::FStackIssue::IsValid()
{
	return UniqueIdentifier.IsEmpty() == false;
}

EStackIssueSeverity UNiagaraStackEntry::FStackIssue::GetSeverity() const
{
	return Severity;
}

const FText& UNiagaraStackEntry::FStackIssue::GetShortDescription() const
{
	return ShortDescription;
}

const FText& UNiagaraStackEntry::FStackIssue::GetLongDescription() const
{
	return LongDescription;
}

bool UNiagaraStackEntry::FStackIssue::GetCanBeDismissed() const
{
	return bCanBeDismissed;
}

const FSimpleDelegate& UNiagaraStackEntry::FStackIssue::GetDismissHandler() const
{
	return DismissHandler;
}

const FString& UNiagaraStackEntry::FStackIssue::GetUniqueIdentifier() const
{
	return UniqueIdentifier;
}

const TArray<UNiagaraStackEntry::FStackIssueFix>& UNiagaraStackEntry::FStackIssue::GetFixes() const
{
	return Fixes;
}

bool UNiagaraStackEntry::FStackIssue::GetIsExpandedByDefault() const
{
	return bIsExpandedByDefault;
}

void UNiagaraStackEntry::FStackIssue::InsertFix(int32 InsertionIdx, const UNiagaraStackEntry::FStackIssueFix& Fix)
{
	Fixes.Insert(Fix, InsertionIdx);
}

void UNiagaraStackEntry::FStackIssue::SetIsExpandedByDefault(bool InExpanded)
{
	bIsExpandedByDefault = InExpanded;
}


UNiagaraStackEntry::UNiagaraStackEntry()
	: bFilterChildrenPending(false)
	, IndentLevel(0)
	, bIsFinalized(false)
	, bIsSearchResult(false)
	, bOwnerIsEnabled(true)
{
}

void UNiagaraStackEntry::Initialize(FRequiredEntryData InRequiredEntryData, FString InStackEditorDataKey)
{
	checkf(bIsFinalized == false, TEXT("Can not initialize an entry after it has been finalized."));
	SystemViewModel = InRequiredEntryData.SystemViewModel;
	EmitterViewModel = InRequiredEntryData.EmitterViewModel;
	ExecutionCategoryName = InRequiredEntryData.ExecutionCategoryName;
	ExecutionSubcategoryName = InRequiredEntryData.ExecutionSubcategoryName;
	StackEditorData = InRequiredEntryData.StackEditorData;
	StackEditorDataKey = InStackEditorDataKey;
}

void UNiagaraStackEntry::Finalize()
{
	if (ensureMsgf(IsFinalized() == false, TEXT("Can not finalize a stack entry more than once.")) == false)
	{
		return;
	}

	FinalizeInternal();
	checkf(bIsFinalized, TEXT("Parent FinalizeInternal not called from overriden FinalizeInternal"));

	SystemViewModel.Reset();
	EmitterViewModel.Reset();
	StackEditorData = nullptr;

	for (UNiagaraStackEntry* Child : Children)
	{
		if(Child->GetOuter() == this)
		{
			Child->Finalize();
		}
	}
	Children.Empty();

	if(StackNote)
	{
		StackNote->Finalize();
		StackNote = nullptr;
	}

	for (UNiagaraStackEntry* ErrorChild : ErrorChildren)
	{
		ErrorChild->Finalize();
	}
	ErrorChildren.Empty();
}

bool UNiagaraStackEntry::IsFinalized() const
{
	return bIsFinalized;
}

FText UNiagaraStackEntry::GetDisplayName() const
{
	return FText();
}

TOptional<FText> UNiagaraStackEntry::GetAlternateDisplayName() const
{
	return AlternateDisplayName;
}

UObject* UNiagaraStackEntry::GetDisplayedObject() const
{
	return nullptr;
}

FGuid UNiagaraStackEntry::GetSelectionId() const
{
	return FGuid();
}

UNiagaraStackEditorData& UNiagaraStackEntry::GetStackEditorData() const
{
	checkf(StackEditorData, TEXT("StackEditorData is nullptr the calling code needs to check for finalized or not"));
	return *StackEditorData;
}

FString UNiagaraStackEntry::GetStackEditorDataKey() const
{
	return StackEditorDataKey;
}

FText UNiagaraStackEntry::GetTooltipText() const
{
	return FText();
}

bool UNiagaraStackEntry::UsesCustomTooltipWidget() const
{
	return false;
}

TSharedPtr<IToolTip> UNiagaraStackEntry::GetCustomTooltipWidget() const
{
	return nullptr;
}

bool UNiagaraStackEntry::GetCanExpand() const
{
	return true;
}

bool UNiagaraStackEntry::GetCanExpandInOverview() const
{
	return false;
}

bool UNiagaraStackEntry::IsExpandedByDefault() const
{
	return true;
}

bool UNiagaraStackEntry::KeepExpanderIndentation() const
{
	return false;
}

bool UNiagaraStackEntry::GetIsExpanded() const
{
	if (GetShouldShowInStack() == false || GetCanExpand() == false)
	{
		// Entries that aren't visible, or can't expand are always expanded.
		return true;
	}

	if (bIsExpandedCache.IsSet() == false)
	{
		bIsExpandedCache = StackEditorData->GetStackEntryIsExpanded(GetStackEditorDataKey(), IsExpandedByDefault());
	}
	return bIsExpandedCache.GetValue();
}

void UNiagaraStackEntry::SetIsExpanded(bool bInExpanded)
{
	if (StackEditorData && GetCanExpand())
	{
		StackEditorData->SetStackEntryIsExpanded(GetStackEditorDataKey(), bInExpanded);
	}
	bIsExpandedCache.Reset();

	ExpansionChangedDelegate.Broadcast();
}

void UNiagaraStackEntry::SetIsExpanded_Recursive(bool bInExpanded)
{
	if (StackEditorData && GetCanExpand())
	{
		StackEditorData->SetStackEntryIsExpanded(GetStackEditorDataKey(), bInExpanded);
	}

	TArray<UNiagaraStackEntry*> UnfilteredChildren;
	GetUnfilteredChildren(UnfilteredChildren);
	for (UNiagaraStackEntry* Child : UnfilteredChildren)
	{
		Child->SetIsExpanded_Recursive(bInExpanded);
	}

	bIsExpandedCache.Reset();
	ExpansionChangedDelegate.Broadcast();
}

bool UNiagaraStackEntry::GetIsExpandedInOverview() const
{
	if (GetCanExpandInOverview() == false)
	{
		// Entries that can't expand are always expanded.
		return true;
	}

	if (bIsExpandedInOverviewCache.IsSet() == false)
	{
		bIsExpandedInOverviewCache = StackEditorData->GetStackEntryIsExpandedInOverview(GetStackEditorDataKey(), IsExpandedByDefault());
	}
	return bIsExpandedInOverviewCache.GetValue();
}

void UNiagaraStackEntry::SetIsExpandedInOverview(bool bInExpanded)
{
	if (StackEditorData && GetCanExpandInOverview())
	{
		StackEditorData->SetStackEntryIsExpandedInOverview(GetStackEditorDataKey(), bInExpanded);
	}

	bIsExpandedInOverviewCache.Reset();
	ExpansionInOverviewChangedDelegate.Broadcast();	
}

bool UNiagaraStackEntry::GetIsEnabled() const
{
	return true;
}

bool UNiagaraStackEntry::GetOwnerIsEnabled() const
{
	return bOwnerIsEnabled;
}

FName UNiagaraStackEntry::GetExecutionCategoryName() const
{
	return ExecutionCategoryName;
}

FName UNiagaraStackEntry::GetExecutionSubcategoryName() const
{
	return ExecutionSubcategoryName;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackEntry::GetStackRowStyle() const
{
	return EStackRowStyle::None;
}

bool UNiagaraStackEntry::GetShouldShowInStack() const
{
	return true;
}

bool UNiagaraStackEntry::GetShouldShowInOverview() const
{
	return true;
}

void UNiagaraStackEntry::GetFilteredChildren(TArray<UNiagaraStackEntry*>& OutFilteredChildren) const
{
	if (bFilterChildrenPending)
	{
		bFilterChildrenPending = false;

		FilteredChildren.Empty();
		FilteredChildren.Append(ErrorChildren);
		if(StackNote != nullptr && StackNote->GetShouldShowInStack())
		{
			FilteredChildren.Add(StackNote);
		}
		
		for (UNiagaraStackEntry* Child : Children)
		{
			bool bPassesFilter = true;
			for (const FOnFilterChild& ChildFilter : ChildFilters)
			{
				if (ChildFilter.Execute(*Child) == false)
				{
					bPassesFilter = false;
					break;
				}
			}

			if (bPassesFilter)
			{
				FilteredChildren.Add(Child);
			}
		}
	}
	OutFilteredChildren.Append(FilteredChildren);
}

void UNiagaraStackEntry::GetCustomFilteredChildren(TArray<UNiagaraStackEntry*>& OutFilteredChildren, const TArray<FOnFilterChild>& CustomChildFilters) const
{
	for (UNiagaraStackEntry* Child : Children)
	{
		bool bPassesFilter = true;
		for (const FOnFilterChild& ChildFilter : CustomChildFilters)
		{
			if (ChildFilter.Execute(*Child) == false)
			{
				bPassesFilter = false;
				break;
			}
		}

		if (bPassesFilter)
		{
			OutFilteredChildren.Add(Child);
		}
	}
}

void UNiagaraStackEntry::GetUnfilteredChildren(TArray<UNiagaraStackEntry*>& OutUnfilteredChildren) const
{
	OutUnfilteredChildren.Append(ErrorChildren);
	OutUnfilteredChildren.Append(Children);
}

FDelegateHandle UNiagaraStackEntry::AddChildFilter(FOnFilterChild ChildFilter)
{
	ChildFilters.Add(ChildFilter);
	StructureChangedDelegate.Broadcast(ENiagaraStructureChangedFlags::FilteringChanged);
	return ChildFilters.Last().GetHandle();
}

void UNiagaraStackEntry::RemoveChildFilter(FDelegateHandle FilterHandle)
{
	ChildFilters.RemoveAll([=](const FOnFilterChild& ChildFilter) { return ChildFilter.GetHandle() == FilterHandle; });
	StructureChangedDelegate.Broadcast(ENiagaraStructureChangedFlags::FilteringChanged);
}

TSharedRef<FNiagaraSystemViewModel> UNiagaraStackEntry::GetSystemViewModel() const
{
	TSharedPtr<FNiagaraSystemViewModel> PinnedSystemViewModel = SystemViewModel.Pin();
	checkf(PinnedSystemViewModel.IsValid(), TEXT("Base stack entry not initialized or system view model was already deleted."));
	return PinnedSystemViewModel.ToSharedRef();
}

TSharedPtr<FNiagaraEmitterViewModel> UNiagaraStackEntry::GetEmitterViewModel() const
{
	return EmitterViewModel.Pin();
}

TSharedPtr<FNiagaraSystemViewModel> UNiagaraStackEntry::GetSystemViewModelPtr() const
{
	return SystemViewModel.Pin();
}

UNiagaraStackNote* UNiagaraStackEntry::GetStackNote()
{
	return StackNote;
}

UNiagaraStackEntry::FOnExpansionChanged& UNiagaraStackEntry::OnExpansionChanged()
{
	return ExpansionChangedDelegate;
}

UNiagaraStackEntry::FOnExpansionChanged& UNiagaraStackEntry::OnExpansionInOverviewChanged()
{
	return ExpansionInOverviewChangedDelegate;
}

UNiagaraStackEntry::FOnStructureChanged& UNiagaraStackEntry::OnStructureChanged()
{
	return StructureChangedDelegate;
}

UNiagaraStackEntry::FOnDataObjectModified& UNiagaraStackEntry::OnDataObjectModified()
{
	return DataObjectModifiedDelegate;
}

UNiagaraStackEntry::FOnRequestFullRefresh& UNiagaraStackEntry::OnRequestFullRefresh()
{
	return RequestFullRefreshDelegate;
}

const UNiagaraStackEntry::FOnRequestFullRefresh& UNiagaraStackEntry::OnRequestFullRefreshDeferred() const
{
	return RequestFullRefreshDeferredDelegate;
}

UNiagaraStackEntry::FOnRequestFullRefresh& UNiagaraStackEntry::OnRequestFullRefreshDeferred()
{
	return RequestFullRefreshDeferredDelegate;
}

UNiagaraStackEntry::FOnAlternateDisplayNameChanged& UNiagaraStackEntry::OnAlternateDisplayNameChanged()
{
	return AlternateDisplayNameChangedDelegate;
}

int32 UNiagaraStackEntry::GetIndentLevel() const
{
	return IndentLevel;
}

void UNiagaraStackEntry::GetSearchItems(TArray<UNiagaraStackEntry::FStackSearchItem>& SearchItems) const
{
	SearchItems.Add({FName("DisplayName"), GetDisplayName()}); 
}

UObject* UNiagaraStackEntry::GetExternalAsset() const
{
	return nullptr;
}

bool UNiagaraStackEntry::CanDrag() const
{
	return false;
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackEntry::CanDrop(const FDropRequest& DropRequest)
{
	TOptional<FDropRequestResponse> Response = CanDropInternal(DropRequest);
	if (Response.IsSet())
	{
		return Response;
	}
	else
	{
		return OnRequestCanDropDelegate.IsBound() 
			? OnRequestCanDropDelegate.Execute(*this, DropRequest)
			: TOptional<FDropRequestResponse>();
	}
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackEntry::Drop(const FDropRequest& DropRequest)
{
	TOptional<FDropRequestResponse> Response = DropInternal(DropRequest);
	if (Response.IsSet())
	{
		return Response;
	}
	else
	{
		return OnRequestDropDelegate.IsBound()
			? OnRequestDropDelegate.Execute(*this, DropRequest)
			: TOptional<FDropRequestResponse>();
	}
}

void UNiagaraStackEntry::SetOnRequestCanDrop(FOnRequestDrop InOnRequestCanDrop)
{
	OnRequestCanDropDelegate = InOnRequestCanDrop;
}

void UNiagaraStackEntry::SetOnRequestDrop(FOnRequestDrop InOnRequestDrop)
{
	OnRequestDropDelegate = InOnRequestDrop;
}

const bool UNiagaraStackEntry::GetIsSearchResult() const
{
	return bIsSearchResult;
}

void UNiagaraStackEntry::SetIsSearchResult(bool bInIsSearchResult)
{
	bIsSearchResult = bInIsSearchResult;
}

bool UNiagaraStackEntry::HasBaseEmitter() const
{
	// TODO (me) the is valid check is a temp fix as it can happen that it gets deleted for some reason which leads to a crash here
	if (!SystemViewModel.IsValid() || GetSystemViewModel()->GetIsForDataProcessingOnly())
	{
		// If the model is just for data processing we don't want to go through the whole merge procedure for all the stack entries and just treat all entries as non-inherited.
		return false;
	}
	
	if (bHasBaseEmitterCache.IsSet() == false)
	{
		const UNiagaraEmitter* BaseEmitter = GetEmitterViewModel().IsValid() ? GetEmitterViewModel()->GetParentEmitter().Emitter.Get() : nullptr;
		bHasBaseEmitterCache = BaseEmitter != nullptr;
	}
	return bHasBaseEmitterCache.GetValue();
}

bool UNiagaraStackEntry::HasIssuesOrAnyChildHasIssues() const
{
	return GetCollectedIssueData().HasAnyIssues();
}

bool UNiagaraStackEntry::HasUsagesOrAnyChildHasUsages() const
{
	return GetCollectedUsageData().bHasReferencedParameterRead || GetCollectedUsageData().bHasReferencedParameterWrite;
}

void UNiagaraStackEntry::GetRecursiveUsages(bool& bRead, bool& bWrite) const
{
	bRead = GetCollectedUsageData().bHasReferencedParameterRead;
	bWrite = GetCollectedUsageData().bHasReferencedParameterWrite;
}

int32 UNiagaraStackEntry::GetTotalNumberOfInfoIssues() const
{
	return GetCollectedIssueData().TotalNumberOfInfoIssues;
}

int32 UNiagaraStackEntry::GetTotalNumberOfWarningIssues() const
{
	return GetCollectedIssueData().TotalNumberOfWarningIssues;
}

int32 UNiagaraStackEntry::GetTotalNumberOfErrorIssues() const
{
	return GetCollectedIssueData().TotalNumberOfErrorIssues;
}

const TArray<UNiagaraStackEntry::FStackIssue>& UNiagaraStackEntry::GetIssues() const
{
	return StackIssues;
}

const TArray<UNiagaraStackEntry*>& UNiagaraStackEntry::GetAllChildrenWithIssues() const
{
	return GetCollectedIssueData().ChildrenWithIssues;
}

void UNiagaraStackEntry::AddValidationIssue(EStackIssueSeverity Severity, const FText& SummaryText, const FText& Description, bool bCanBeDismissed, const TArray<FNiagaraValidationFix>& Fixes, const TArray<FNiagaraValidationFix>& Links)
{
	TArray<FStackIssueFix> StackFixes;
	for (const FNiagaraValidationFix& Fix : Fixes)
	{
		StackFixes.Add(FStackIssueFix(Fix.Description, Fix.FixDelegate, EStackIssueFixStyle::Fix));
	}
	for (const FNiagaraValidationFix& Link : Links)
	{
		StackFixes.Add(FStackIssueFix(Link.Description, Link.FixDelegate, EStackIssueFixStyle::Link));
	}

	FStackIssue NewIssue(Severity, SummaryText, Description, GetStackEditorDataKey(), bCanBeDismissed, StackFixes);
	for (const FStackIssue& ExistingIssue : ExternalStackIssues)
	{
		if (ExistingIssue.GetUniqueIdentifier().Equals(NewIssue.GetUniqueIdentifier()))
		{
			return;
		}
	}
	ExternalStackIssues.Add(NewIssue);
	RefreshChildren();
}

void UNiagaraStackEntry::AddExternalIssue(EStackIssueSeverity Severity, const FText& SummaryText, const FText& Description, bool bCanBeDismissed)
{
	FStackIssue NewIssue(Severity, SummaryText, Description, GetStackEditorDataKey(), bCanBeDismissed);
	for (const FStackIssue& ExistingIssue : ExternalStackIssues)
	{
		if (ExistingIssue.GetUniqueIdentifier().Equals(NewIssue.GetUniqueIdentifier()))
		{
			return;
		}
	}
	ExternalStackIssues.Add(NewIssue);
	RefreshChildren();
}

void UNiagaraStackEntry::ClearExternalIssues()
{
	if (ExternalStackIssues.Num() > 0)
	{
		ExternalStackIssues.Empty();
		Cast<UNiagaraStackEntry>(GetOuter())->RefreshChildren();
	}
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackEntry::CanDropInternal(const FDropRequest& DropRequest)
{
	return TOptional<FDropRequestResponse>();
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackEntry::DropInternal(const FDropRequest& DropRequest)
{
	return TOptional<FDropRequestResponse>();
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackEntry::ChildRequestCanDropInternal(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest)
{
	return TOptional<FDropRequestResponse>();
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackEntry::ChildRequestDropInternal(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest)
{
	return TOptional<FDropRequestResponse>();
}

void UNiagaraStackEntry::ChildStructureChangedInternal()
{
}

void UNiagaraStackEntry::FinalizeInternal()
{
	bIsFinalized = true;
}

void UNiagaraStackEntry::RefreshChildren()
{
	checkf(bIsFinalized == false, TEXT("Can not refresh children on an entry after it has been finalized."));
	//-TODO:Stateless: Do we need stateless support here?
	//checkf(SystemViewModel.IsValid(), TEXT("Base stack entry not initialized."));
	if (SystemViewModel.IsValid() == false)
	{
		return;
	}

	for (UNiagaraStackEntry* Child : Children)
	{
		Child->OnExpansionChanged().RemoveAll(this);
		Child->OnExpansionInOverviewChanged().RemoveAll(this);
		Child->OnStructureChanged().RemoveAll(this);
		Child->OnDataObjectModified().RemoveAll(this);
		Child->OnRequestFullRefresh().RemoveAll(this);
		Child->OnRequestFullRefreshDeferred().RemoveAll(this);
		UNiagaraStackEntry* OuterOwner = Cast<UNiagaraStackEntry>(Child->GetOuter());
		if (OuterOwner == this)
		{
			Child->SetOnRequestCanDrop(FOnRequestDrop());
			Child->SetOnRequestDrop(FOnRequestDrop());
		}
	}
	for (UNiagaraStackErrorItem* ErrorChild : ErrorChildren)
	{
		ErrorChild->OnExpansionChanged().RemoveAll(this);
		ErrorChild->OnExpansionInOverviewChanged().RemoveAll(this);
		ErrorChild->OnStructureChanged().RemoveAll(this);
		ErrorChild->OnDataObjectModified().RemoveAll(this);
		ErrorChild->OnRequestFullRefresh().RemoveAll(this);
		ErrorChild->OnRequestFullRefreshDeferred().RemoveAll(this);
		ErrorChild->OnIssueModified().RemoveAll(this);
	}

	bHasBaseEmitterCache.Reset();

	TArray<UNiagaraStackEntry*> NewChildren;
	TArray<FStackIssue> NewStackIssues;

	InvalidateFilteredChildren();
	RefreshChildrenInternal(Children, NewChildren, NewStackIssues);

	// If any of the current children were not moved to the new children collection, and they're owned by this entry than finalize them since
	// they weren't reused.
	for (UNiagaraStackEntry* Child : Children)
	{
		if (!NewChildren.Contains(Child) && Child->GetOuter() == this)
		{
			Child->Finalize();
		}
	}

	Children.Empty(NewChildren.Num());
	TSet<UNiagaraStackEntry*> HandledChildren;
	for (UNiagaraStackEntry* NewChild : NewChildren)
	{
		bool bAlreadyHandled = false;
		HandledChildren.Add(NewChild, &bAlreadyHandled);
		if (ensureMsgf(bAlreadyHandled == false, TEXT("Duplicate child %s (%s) in entry %s (%s)."),
			*NewChild->GetDisplayName().ToString(), *NewChild->GetClass()->GetName(), *GetDisplayName().ToString(), *GetClass()->GetName()))
		{
			Children.Add(NewChild);
		}
	}

	for (UNiagaraStackEntry* Child : Children)
	{
		UNiagaraStackEntry* OuterOwner = Cast<UNiagaraStackEntry>(Child->GetOuter());
		Child->IndentLevel = GetChildIndentLevel() + (Child->IsSemanticChild() ? 1 : 0);
		Child->bOwnerIsEnabled = OuterOwner == nullptr || (OuterOwner->GetIsEnabled() && OuterOwner->GetOwnerIsEnabled());
		Child->RefreshChildren();
		Child->OnStructureChanged().AddUObject(this, &UNiagaraStackEntry::ChildStructureChanged);
		Child->OnExpansionChanged().AddUObject(this, &UNiagaraStackEntry::ChildExpansionChanged);
		Child->OnExpansionInOverviewChanged().AddUObject(this, &UNiagaraStackEntry::ChildExpansionInOverviewChanged);
		Child->OnDataObjectModified().AddUObject(this, &UNiagaraStackEntry::ChildDataObjectModified);
		Child->OnRequestFullRefresh().AddUObject(this, &UNiagaraStackEntry::ChildRequestFullRefresh);
		Child->OnRequestFullRefreshDeferred().AddUObject(this, &UNiagaraStackEntry::ChildRequestFullRefreshDeferred);
		if (OuterOwner == this)
		{
			Child->SetOnRequestCanDrop(FOnRequestDrop::CreateUObject(this, &UNiagaraStackEntry::ChildRequestCanDrop));
			Child->SetOnRequestDrop(FOnRequestDrop::CreateUObject(this, &UNiagaraStackEntry::ChildRequestDrop));
		}
	}

	if(HasStackNoteData())
	{		
		if(StackNote == nullptr)
		{
			StackNote = NewObject<UNiagaraStackNote>(this);
			StackNote->Initialize(CreateDefaultChildRequiredData(), GetStackEditorDataKey());
			StackNote->IndentLevel = GetIndentLevel();
			StackNote->OnNoteChanged().BindUObject(this, &UNiagaraStackEntry::SetStackNoteData);
		}
		
		StackNote->RefreshChildren();
	}
	else
	{
		if(StackNote != nullptr)
		{
			StackNote->OnNoteChanged().Unbind();
			StackNote->Finalize();
			StackNote = nullptr;
		}
	}
	
	// Stack issues refresh
	NewStackIssues.RemoveAll([this](const FStackIssue& Issue) { return Issue.GetCanBeDismissed() && GetStackEditorData().GetDismissedStackIssueIds().Contains(Issue.GetUniqueIdentifier()); }); 

	StackIssues.Empty();
	StackIssues.Append(NewStackIssues);
	StackIssues.Append(ExternalStackIssues);
	RefreshStackErrorChildren();
	for (UNiagaraStackErrorItem* ErrorChild : ErrorChildren)
	{
		ErrorChild->IndentLevel = GetChildIndentLevel();
		ErrorChild->RefreshChildren();
		ErrorChild->OnStructureChanged().AddUObject(this, &UNiagaraStackEntry::ChildStructureChanged);
		ErrorChild->OnExpansionChanged().AddUObject(this, &UNiagaraStackEntry::ChildExpansionChanged);
		ErrorChild->OnExpansionInOverviewChanged().AddUObject(this, &UNiagaraStackEntry::ChildExpansionInOverviewChanged);
		ErrorChild->OnDataObjectModified().AddUObject(this, &UNiagaraStackEntry::ChildDataObjectModified);
		ErrorChild->OnRequestFullRefresh().AddUObject(this, &UNiagaraStackEntry::ChildRequestFullRefresh);
		ErrorChild->OnRequestFullRefreshDeferred().AddUObject(this, &UNiagaraStackEntry::ChildRequestFullRefreshDeferred);
		ErrorChild->OnIssueModified().AddUObject(this, &UNiagaraStackEntry::IssueModified);
	}

	//-TODO:Stateless: Do we need stateless support here?
	if (StackEditorData)
	{
		const FText* NewAlternateName = StackEditorData->GetStackEntryDisplayName(StackEditorDataKey);
		if (NewAlternateName != nullptr && NewAlternateName->IsEmptyOrWhitespace() == false)
		{
			if (AlternateDisplayName.IsSet() == false || NewAlternateName->IdenticalTo(AlternateDisplayName.GetValue()) == false)
			{
				AlternateDisplayName = *NewAlternateName;
				AlternateDisplayNameChangedDelegate.Broadcast();
			}
		}
		else if (AlternateDisplayName.IsSet())
		{
			AlternateDisplayName.Reset();
			AlternateDisplayNameChangedDelegate.Broadcast();
		}
	}

	PostRefreshChildrenInternal();

	StructureChangedDelegate.Broadcast(ENiagaraStructureChangedFlags::StructureChanged);
}

void UNiagaraStackEntry::RefreshFilteredChildren()
{
	InvalidateFilteredChildren();
	for (UNiagaraStackEntry* Child : Children)
	{
		Child->OnStructureChanged().RemoveAll(this);
		Child->RefreshFilteredChildren();
		Child->OnStructureChanged().AddUObject(this, &UNiagaraStackEntry::ChildStructureChanged);
	}

	StructureChangedDelegate.Broadcast(ENiagaraStructureChangedFlags::FilteringChanged);
}

void UNiagaraStackEntry::RefreshStackErrorChildren()
{
	// keep the error entries that are already built
	TArray<UNiagaraStackErrorItem*> NewErrorChildren;
	for (int i = 0; i < StackIssues.Num(); i++)
	{
		const FStackIssue& Issue = StackIssues[i];
		UNiagaraStackErrorItem* ErrorEntry = nullptr;
		TObjectPtr<UNiagaraStackErrorItem>* Found = ErrorChildren.FindByPredicate(
			[&](UNiagaraStackErrorItem* CurrentChild) { return CurrentChild->GetStackIssue().GetUniqueIdentifier() == Issue.GetUniqueIdentifier(); });
		if (Found == nullptr)
		{
			ErrorEntry = NewObject<UNiagaraStackErrorItem>(this);
			ErrorEntry->Initialize(CreateDefaultChildRequiredData(), Issue, GetStackEditorDataKey());
		}
		else
		{
			ErrorEntry = *Found;
			ErrorEntry->SetStackIssue(Issue); // we found the entry by id but we want to properly refresh the subentries of the issue (specifically its fixes), too
		}
		if (i > 0)
		{
			// If there is more than one issue, only expand the first by default and collapse the others
			ErrorEntry->SetIsExpandedByDefault(false);
		}
		if (ensureMsgf(NewErrorChildren.Contains(ErrorEntry) == false,
			TEXT("Duplicate stack issue rows detected, this is caused by two different issues generating the same unique id. Issue Short description: %s Issue Long description: %s.  This issue will not be shown in the UI."),
			*Issue.GetShortDescription().ToString(),
			*Issue.GetLongDescription().ToString()))
		{
			NewErrorChildren.Add(ErrorEntry);
		}
	}

	// If any of the current error children were not moved to the new error children collection than finalize them since
	// they weren't reused.
	for (UNiagaraStackEntry* ErrorChild : ErrorChildren)
	{
		if(NewErrorChildren.Contains(ErrorChild) == false)
		{
			ErrorChild->Finalize();
		}
	}

	ErrorChildren.Empty();
	ErrorChildren.Append(NewErrorChildren);
}

void UNiagaraStackEntry::IssueModified()
{
	if (bIsFinalized == false)
	{
		// Fixing an issue may have caused this entry to be deleted and finalized, so don't refresh in that case.
		RefreshChildren();
	}
}

void UNiagaraStackEntry::InvalidateFilteredChildren()
{
	FilteredChildren.Empty();
	CachedCollectedIssueData.Reset();
	CachedCollectedUsageData.Reset();
	bFilterChildrenPending = true;
}

const UNiagaraStackEntry::FCollectedIssueData& UNiagaraStackEntry::GetCollectedIssueData() const
{
	if(CachedCollectedIssueData.IsSet() == false)
	{
		CachedCollectedIssueData = FCollectedIssueData();
	
		TArray<UNiagaraStackEntry*> RefreshedFilteredChildren;
		GetFilteredChildren(RefreshedFilteredChildren);

		for (UNiagaraStackEntry* ChildStackEntry : RefreshedFilteredChildren)
		{
			CachedCollectedIssueData->TotalNumberOfInfoIssues += ChildStackEntry->GetTotalNumberOfInfoIssues();
			CachedCollectedIssueData->TotalNumberOfWarningIssues += ChildStackEntry->GetTotalNumberOfWarningIssues();
			CachedCollectedIssueData->TotalNumberOfErrorIssues += ChildStackEntry->GetTotalNumberOfErrorIssues();
			
			if (ChildStackEntry->GetIssues().Num() > 0)
			{
				CachedCollectedIssueData->ChildrenWithIssues.Add(ChildStackEntry);
			}
			CachedCollectedIssueData->ChildrenWithIssues.Append(ChildStackEntry->GetAllChildrenWithIssues());

		}

		for (const FStackIssue& Issue : StackIssues)
		{
			if (Issue.GetSeverity() == EStackIssueSeverity::Info)
			{
				CachedCollectedIssueData->TotalNumberOfInfoIssues++;
			}
			else if (Issue.GetSeverity() == EStackIssueSeverity::Warning)
			{
				CachedCollectedIssueData->TotalNumberOfWarningIssues++;
			}
			else if (Issue.GetSeverity() == EStackIssueSeverity::Error)
			{
				CachedCollectedIssueData->TotalNumberOfErrorIssues++;
			}
		}
	}

	return CachedCollectedIssueData.GetValue();
}


const UNiagaraStackEntry::FCollectedUsageData& UNiagaraStackEntry::GetCollectedUsageData() const
{
	if (CachedCollectedUsageData.IsSet() == false)
	{
		CachedCollectedUsageData = FCollectedUsageData();

		TArray<UNiagaraStackEntry*> RefreshedChildren;
		GetUnfilteredChildren(RefreshedChildren);

		for (UNiagaraStackEntry* ChildStackEntry : RefreshedChildren)
		{
			if (ChildStackEntry->GetCollectedUsageData().bHasReferencedParameterRead)
				CachedCollectedUsageData.GetValue().bHasReferencedParameterRead = true;
			if (ChildStackEntry->GetCollectedUsageData().bHasReferencedParameterWrite)
				CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite = true;

		}
	}

	return CachedCollectedUsageData.GetValue();
}

void UNiagaraStackEntry::InvalidateCollectedUsage()
{
	// Make sure to use the unfiltered children instead of filtered as the info we need is only found in certain, usually filtered children.
	CachedCollectedUsageData.Reset();
	TArray<UNiagaraStackEntry*> RefreshedFilteredChildren;
	GetUnfilteredChildren(RefreshedFilteredChildren);

	for (UNiagaraStackEntry* ChildStackEntry : RefreshedFilteredChildren)
	{
		ChildStackEntry->InvalidateCollectedUsage();
	}
}

void UNiagaraStackEntry::BeginDestroy()
{
	ensureMsgf(HasAnyFlags(RF_ClassDefaultObject) || bIsFinalized, TEXT("Stack entry being destroyed but it was not finalized."));
	Super::BeginDestroy();
}

void UNiagaraStackEntry::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
}

void UNiagaraStackEntry::PostRefreshChildrenInternal()
{
}

UNiagaraStackEntry::FRequiredEntryData UNiagaraStackEntry::CreateDefaultChildRequiredData() const
{
	return FRequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(), GetExecutionCategoryName(), GetExecutionSubcategoryName(), GetStackEditorData());
}

int32 UNiagaraStackEntry::GetChildIndentLevel() const
{
	return GetShouldShowInStack() ? GetIndentLevel() + 1 : GetIndentLevel();
}

void UNiagaraStackEntry::ChildStructureChanged(ENiagaraStructureChangedFlags Info)
{
	InvalidateFilteredChildren();
	ChildStructureChangedInternal();
	StructureChangedDelegate.Broadcast(Info);
}

void UNiagaraStackEntry::ChildExpansionChanged()
{
	ExpansionChangedDelegate.Broadcast();
}

void UNiagaraStackEntry::ChildExpansionInOverviewChanged()
{
	ExpansionInOverviewChangedDelegate.Broadcast();
}

void UNiagaraStackEntry::ChildDataObjectModified(TArray<UObject*> ChangedObjects, ENiagaraDataObjectChange ChangeType)
{
	DataObjectModifiedDelegate.Broadcast(ChangedObjects, ChangeType);
}

void UNiagaraStackEntry::ChildRequestFullRefresh()
{
	RequestFullRefreshDelegate.Broadcast();
}

void UNiagaraStackEntry::ChildRequestFullRefreshDeferred()
{
	RequestFullRefreshDeferredDelegate.Broadcast();
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackEntry::ChildRequestCanDrop(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest)
{
	TOptional<FDropRequestResponse> Response = ChildRequestCanDropInternal(TargetChild, DropRequest);
	if (Response.IsSet())
	{
		return Response;
	}
	else
	{
		return OnRequestCanDropDelegate.IsBound()
			? OnRequestCanDropDelegate.Execute(TargetChild, DropRequest)
			: TOptional<FDropRequestResponse>();
	}
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackEntry::ChildRequestDrop(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest)
{
	TOptional<FDropRequestResponse> Response = ChildRequestDropInternal(TargetChild, DropRequest);
	if (Response.IsSet())
	{
		return Response;
	}
	else
	{
		return OnRequestDropDelegate.IsBound()
			? OnRequestDropDelegate.Execute(TargetChild, DropRequest)
			: TOptional<FDropRequestResponse>();
	}
}

bool UNiagaraStackEntry::GetIsRenamePending() const
{
	return SupportsRename() && GetStackEditorData().GetStackEntryIsRenamePending(StackEditorDataKey);
}

void UNiagaraStackEntry::SetIsRenamePending(bool bIsRenamePending)
{
	if (SupportsRename())
	{
		GetStackEditorData().SetStackEntryIsRenamePending(StackEditorDataKey, bIsRenamePending);
	}
}

void UNiagaraStackEntry::OnRenamed(FText NewName)
{
	if (SupportsRename())
	{
		if (!NewName.EqualTo(AlternateDisplayName.Get(FText::GetEmpty())))
		{
			if (NewName.IsEmptyOrWhitespace() || NewName.EqualTo(GetDisplayName()))
			{
				AlternateDisplayName.Reset();
			}
			else
			{
				AlternateDisplayName = NewName;
			}

			FScopedTransaction ScopedTransaction(NSLOCTEXT("NiagaraStackEntry", "RenameModule", "Rename Module"));
			
			GetStackEditorData().Modify();
			GetStackEditorData().SetStackEntryDisplayName(GetStackEditorDataKey(), AlternateDisplayName.Get(FText::GetEmpty()));

			AlternateDisplayNameChangedDelegate.Broadcast();
		}
	}
}

bool UNiagaraStackEntry::HasStackNoteData() const
{
	return GetStackEditorData().HasStackNote(GetStackEditorDataKey());
}

FNiagaraStackNoteData UNiagaraStackEntry::GetStackNoteData() const
{
	return GetStackEditorData().GetStackNote(GetStackEditorDataKey()).Get(FNiagaraStackNoteData());
}

void UNiagaraStackEntry::SetStackNoteData(FNiagaraStackNoteData InStackNoteData)
{
	if(InStackNoteData.IsValid())
	{
		GetStackEditorData().AddOrReplaceStackNote(GetStackEditorDataKey(), InStackNoteData);
	}
	else
	{
		GetStackEditorData().DeleteStackNote(GetStackEditorDataKey());	
	}
	
	RefreshChildren();
}

void UNiagaraStackEntry::DeleteStackNoteData()
{
	if(HasStackNoteData())
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteStackNoteTransaction", "Deleted Stack Note"));
		GetStackEditorData().Modify();
		
		GetStackEditorData().DeleteStackNote(GetStackEditorDataKey());
		RefreshChildren();
	}
}

void UNiagaraStackEntry::AddStackNote()
{
	if(SupportsStackNotes())
	{
		if(HasStackNoteData() == false)
		{
			FScopedTransaction Transaction(LOCTEXT("NewStackNoteAdded", "Added Stack Note"));
			GetStackEditorData().Modify();
		
			FNiagaraStackNoteData NewStackNote;
			NewStackNote.MessageHeader = LOCTEXT("DefaultStackNoteHeader", "Note");
			NewStackNote.Message = LOCTEXT("DefaultStackNoteMessage", "Text");
			NewStackNote.Color = GetDefault<UNiagaraEditorSettings>()->GetDefaultNoteColor();

			GetStackEditorData().AddOrReplaceStackNote(GetStackEditorDataKey(), NewStackNote);
			RefreshChildren();
		
			if(UNiagaraStackNote* Note = GetStackNote())
			{
				Note->SetIsRenamePending(true);
			}
		}
	}
}

void UNiagaraStackEntry::ToggleStackNoteInlineDisplay()
{
	if(GetStackNote() != nullptr)
	{
		GetStackNote()->ToggleInlineDisplay();
	}
}

void UNiagaraStackEntry::CopyStackNote()
{
	FNiagaraStackClipboardUtilities::CopyNote(this);
}

void UNiagaraStackEntry::PasteStackNote()
{
	if(SupportsStackNotes())
	{
		if(const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent())
		{
			if(ClipboardContent->StackNote.IsValid())
			{
				FScopedTransaction Transaction(LOCTEXT("PasteNoteTransaction", "Note pasted"));
				GetStackEditorData().Modify();
				
				SetStackNoteData(ClipboardContent->StackNote);

				// if the note we pasted is not inlined, we expand the stack entry in order to show the note
				if(GetStackNoteData().bInlineNote == false)
				{
					SetIsExpanded(true);
				}
			}
		}
	}
}

FHierarchyElementIdentity UNiagaraStackEntry::DetermineSummaryIdentity() const
{
	return FHierarchyElementIdentity();
}

bool UNiagaraStackEntry::IsInSummaryView() const
{
	if(GetEmitterViewModel().IsValid())
	{
		if(SupportsSummaryView())
		{
			FHierarchyElementIdentity Identity = DetermineSummaryIdentity();
			if(Identity.IsValid())
			{
				return GetEmitterViewModel()->GetSummaryHierarchyViewModel()->GetHierarchyRootViewModel()->FindViewModelForChild(Identity, true) != nullptr;
			}
		}
	}

	return false;
}

bool UNiagaraStackEntry::IsAnyParentInSummaryView() const
{
	if(GetEmitterViewModel().IsValid())
	{
		for(UNiagaraStackEntry* Parent = Cast<UNiagaraStackEntry>(GetOuter()); Parent != nullptr; Parent = Cast<UNiagaraStackEntry>(Parent->GetOuter()))
		{
			if(Parent->IsInSummaryView())
			{
				return true;
			}
		}
	}

	return false;
}

bool UNiagaraStackEntry::IsAnyChildInSummaryView(bool bRecursive) const
{
	if(GetEmitterViewModel().IsValid())
	{
		TArray<UNiagaraStackEntry*> AllChildren;
		GetUnfilteredChildren(AllChildren);
		for(UNiagaraStackEntry* Child : AllChildren)
		{
			if(Child->IsInSummaryView())
			{
				return true;
			}
		}

		if(bRecursive)
		{
			for(UNiagaraStackEntry* Child : AllChildren)
			{
				if(Child->IsAnyChildInSummaryView(bRecursive))
				{
					return true;
				}
			}			
		}		
	}

	return false;
}

bool UNiagaraStackEntry::ExistsInParentEmitterSummary() const
{
	if(GetEmitterViewModel().IsValid())
	{
		if(GetEmitterViewModel()->GetParentEmitter().Emitter != nullptr)
		{
			FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetParentEmitter();
			FHierarchyElementIdentity Identity = DetermineSummaryIdentity();
			if(BaseEmitter.GetEmitterData() != nullptr && Identity.IsValid())
			{
				return Cast<UNiagaraEmitterEditorData>(BaseEmitter.GetEmitterData()->GetEditorData())->GetSummaryRoot()->FindChildWithIdentity(Identity, true) != nullptr;
			}
		}
	}

	return false;
}

void UNiagaraStackSpacer::Initialize(FRequiredEntryData InRequiredEntryData, float InSpacerHeight, TAttribute<bool> InShouldShowInStack, FString InOwningStackItemEditorDataKey)
{
	Super::Initialize(InRequiredEntryData, InOwningStackItemEditorDataKey + TEXT("Spacer"));
	SpacerHeight = InSpacerHeight;
	ShouldShowInStack = InShouldShowInStack;
}

#undef LOCTEXT_NAMESPACE
