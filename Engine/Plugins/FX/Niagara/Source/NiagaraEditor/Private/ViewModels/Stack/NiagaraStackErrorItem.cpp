// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "NiagaraStackEditorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackErrorItem)

#define LOCTEXT_NAMESPACE "NiagaraStackErrorItem"
//UNiagaraStackErrorItem

void UNiagaraStackErrorItem::Initialize(FRequiredEntryData InRequiredEntryData, FStackIssue InStackIssue, FString InStackEditorDataKey)
{
	FString ErrorStackEditorDataKey = FString::Printf(TEXT("%s-Error-%s"), *InStackEditorDataKey, *InStackIssue.GetUniqueIdentifier());
	Super::Initialize(InRequiredEntryData, ErrorStackEditorDataKey);
	StackIssue = InStackIssue;
	EntryStackEditorDataKey = InStackEditorDataKey;
}

void UNiagaraStackErrorItem::SetStackIssue(const FStackIssue& InStackIssue)
{
	StackIssue = InStackIssue;
}

FText UNiagaraStackErrorItem::GetDisplayName() const
{
	return StackIssue.GetShortDescription();
}

EStackIssueSeverity UNiagaraStackErrorItem::GetIssueSeverity() const
{
	return StackIssue.GetSeverity();
}

bool UNiagaraStackErrorItem::IsExpandedByDefault() const
{
	return bIsExpandedByDefault && StackIssue.GetIsExpandedByDefault();
}

void UNiagaraStackErrorItem::SetIsExpandedByDefault(bool bIsExpanded)
{
	bIsExpandedByDefault = bIsExpanded;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackErrorItem::GetStackRowStyle() const
{
	return EStackRowStyle::StackIssue;
}

UNiagaraStackErrorItem::FOnIssueNotify& UNiagaraStackErrorItem::OnIssueModified()
{
	return IssueModifiedDelegate;
}

void UNiagaraStackErrorItem::GetSearchItems(TArray<FStackSearchItem>& SearchItems) const
{
	SearchItems.Add({ FName("ErrorShortDescription"), StackIssue.GetShortDescription() });
	SearchItems.Add({ FName("ErrorLongDescription"), StackIssue.GetLongDescription() });
}

void UNiagaraStackErrorItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	for (UNiagaraStackEntry* Child : CurrentChildren)
	{
		UNiagaraStackErrorItemFix* FixChild = Cast<UNiagaraStackErrorItemFix>(Child);
		if (FixChild != nullptr)
		{
			FixChild->OnIssueFixed().RemoveAll(this);
		}
	}
	
	if (!StackIssue.GetLongDescription().IsEmptyOrWhitespace())
	{
		// long description
		UNiagaraStackErrorItemLongDescription* ErrorEntryLongDescription = FindCurrentChildOfTypeByPredicate<UNiagaraStackErrorItemLongDescription>(CurrentChildren,
			[&](UNiagaraStackErrorItemLongDescription* CurrentChild) { return true; });
		if (ErrorEntryLongDescription == nullptr)
		{
			ErrorEntryLongDescription = NewObject<UNiagaraStackErrorItemLongDescription>(this);
			ErrorEntryLongDescription->Initialize(CreateDefaultChildRequiredData(), StackIssue, GetStackEditorDataKey());
		}


		NewChildren.Add(ErrorEntryLongDescription);
	}

	// fixes
	for (int i = 0; i < StackIssue.GetFixes().Num(); i++)
	{
		UNiagaraStackEntry::FStackIssueFix CurrentFix = StackIssue.GetFixes()[i];
		UNiagaraStackErrorItemFix* ErrorEntryFix = FindCurrentChildOfTypeByPredicate<UNiagaraStackErrorItemFix>(CurrentChildren,
			[&](UNiagaraStackErrorItemFix* CurrentChild) { return CurrentChild->GetStackIssueFix().GetUniqueIdentifier() == CurrentFix.GetUniqueIdentifier(); });
		if (ErrorEntryFix == nullptr)
		{
			ErrorEntryFix = NewObject<UNiagaraStackErrorItemFix>(this);
			ErrorEntryFix->Initialize(CreateDefaultChildRequiredData(), StackIssue, CurrentFix, EntryStackEditorDataKey);
		}
		else
		{
			ErrorEntryFix->SetFixDelegate(CurrentFix.GetFixDelegate());
		}
		if (ensureMsgf(NewChildren.Contains(ErrorEntryFix) == false,
			TEXT("Duplicate stack issue fix rows detected. This is caused by two different issue fixes with the same description which is used to generate their unique ID. Issue Fix description: %s.  This issue fix will not be shown in the UI."),
			*CurrentFix.GetDescription().ToString()))
		{
			ErrorEntryFix->OnIssueFixed().AddUObject(this, &UNiagaraStackErrorItem::IssueFixed);
			NewChildren.Add(ErrorEntryFix);
		}
	}
	// dismiss button
	if (StackIssue.GetCanBeDismissed())
	{
		UNiagaraStackErrorItemDismiss* ErrorEntryDismiss = FindCurrentChildOfTypeByPredicate<UNiagaraStackErrorItemDismiss>(CurrentChildren,
			[&](UNiagaraStackErrorItemDismiss* CurrentChild) { return true; });
		if (ErrorEntryDismiss == nullptr)
		{
			ErrorEntryDismiss = NewObject<UNiagaraStackErrorItemDismiss>(this);
			ErrorEntryDismiss->Initialize(CreateDefaultChildRequiredData(), StackIssue, EntryStackEditorDataKey);
		}
		ErrorEntryDismiss->OnIssueFixed().AddUObject(this, &UNiagaraStackErrorItem::IssueFixed);
		NewChildren.Add(ErrorEntryDismiss);
	}
}

void UNiagaraStackErrorItem::IssueFixed()
{
	OnIssueModified().Broadcast();
}

//UNiagaraStackErrorItemLongDescription
void UNiagaraStackErrorItemLongDescription::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStackEntry::FStackIssue InStackIssue, FString InStackEditorDataKey)
{
	FString ErrorStackEditorDataKey = FString::Printf(TEXT("Long-%s"), *InStackEditorDataKey);
	Super::Initialize(InRequiredEntryData, ErrorStackEditorDataKey);
	StackIssue = InStackIssue;
}

FText UNiagaraStackErrorItemLongDescription::GetDisplayName() const
{
	return StackIssue.GetLongDescription();
}

EStackIssueSeverity UNiagaraStackErrorItemLongDescription::GetIssueSeverity() const
{
	return StackIssue.GetSeverity();
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackErrorItemLongDescription::GetStackRowStyle() const
{
	return EStackRowStyle::StackIssue;
}

//UNiagaraStackErrorItemFix

void UNiagaraStackErrorItemFix::Initialize(FRequiredEntryData InRequiredEntryData, FStackIssue InStackIssue, FStackIssueFix InIssueFix, FString InStackEditorDataKey)
{
	FString ErrorStackEditorDataKey = FString::Printf(TEXT("Fix-%s"), *InStackEditorDataKey);
	Super::Initialize(InRequiredEntryData, ErrorStackEditorDataKey);
	StackIssue = InStackIssue;
	IssueFix = InIssueFix;
}

FReply UNiagaraStackErrorItemFix::OnTryFixError()
{
	// Request a deferred refresh before running the fix since this entry might be finalized during the fix and it's delegate will be lost.
	OnRequestFullRefreshDeferred().Broadcast();
	IssueFix.GetFixDelegate().ExecuteIfBound();
	OnIssueFixed().Broadcast();
	return FReply::Handled();
}

FText UNiagaraStackErrorItemFix::GetDisplayName() const
{
	return IssueFix.GetDescription();
}

EStackIssueSeverity UNiagaraStackErrorItemFix::GetIssueSeverity() const
{
	return StackIssue.GetSeverity();
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackErrorItemFix::GetStackRowStyle() const
{
	return EStackRowStyle::StackIssue;
}

FText UNiagaraStackErrorItemFix::GetFixButtonText() const
{
	return LOCTEXT("FixIssue", "Fix issue");
}

UNiagaraStackErrorItem::FOnIssueNotify& UNiagaraStackErrorItemFix::OnIssueFixed()
{
	return IssueFixedDelegate;
}

void UNiagaraStackErrorItemFix::SetFixDelegate(const FStackIssueFixDelegate& InFixDelegate)
{
	IssueFix.SetFixDelegate(InFixDelegate);
}

//UNiagaraStackErrorItemDismiss

void UNiagaraStackErrorItemDismiss::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStackEntry::FStackIssue InStackIssue, FString InStackEditorDataKey)
{
	FString ErrorStackEditorDataKey = FString::Printf(TEXT("Dismiss-%s"), *InStackEditorDataKey);
	UNiagaraStackEntry::Initialize(InRequiredEntryData, ErrorStackEditorDataKey);
	StackIssue = InStackIssue;
	IssueFix = FStackIssueFix(
		StackIssue.GetSeverity() == EStackIssueSeverity::Info ?
			LOCTEXT("DismissNote", "Dismiss note") :
			LOCTEXT("DismissError", "Dismiss the issue without fixing (I know what I'm doing)"),
		FStackIssueFixDelegate::CreateUObject(this, &UNiagaraStackErrorItemDismiss::DismissIssue));
}

void UNiagaraStackErrorItemDismiss::DismissIssue()
{
	if (StackIssue.GetDismissHandler().IsBound())
	{
		StackIssue.GetDismissHandler().Execute();
	}
	else
	{
		GetStackEditorData().Modify();
		GetStackEditorData().DismissStackIssue(StackIssue.GetUniqueIdentifier());
	}
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackErrorItemDismiss::GetStackRowStyle() const
{
	return EStackRowStyle::StackIssue;
}

FText UNiagaraStackErrorItemDismiss::GetFixButtonText() const
{
	return StackIssue.GetSeverity() == EStackIssueSeverity::Info ? LOCTEXT("DismissNoteButtonText", "Dismiss") : LOCTEXT("DismissIssueButtonText", "Dismiss issue");
}

#undef LOCTEXT_NAMESPACE
