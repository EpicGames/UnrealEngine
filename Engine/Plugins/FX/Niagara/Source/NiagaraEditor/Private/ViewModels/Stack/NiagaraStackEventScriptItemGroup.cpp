// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackEventScriptItemGroup.h"
#include "NiagaraEmitter.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeFunctionCall.h"
#include "EdGraphSchema_Niagara.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptMergeManager.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Customizations/NiagaraEventScriptPropertiesCustomization.h"

#include "Internationalization/Internationalization.h"
#include "ScopedTransaction.h"
#include "IDetailTreeNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackEventScriptItemGroup)

#define LOCTEXT_NAMESPACE "UNiagaraStackEventScriptItemGroup"

void UNiagaraStackEventWrapper::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);

	if (FVersionedNiagaraEmitterData* EmitterData = EmitterWeakPtr.GetEmitterData())
	{
		EmitterData->EventHandlerScriptProps = EventHandlerScriptProps;
		EmitterWeakPtr.Emitter->PostEditChangeVersionedProperty(PropertyChangedEvent, EmitterWeakPtr.Version);
	}
}

void UNiagaraStackEventHandlerPropertiesItem::Initialize(FRequiredEntryData InRequiredEntryData, FGuid InEventScriptUsageId)
{
	FString EventStackEditorDataKey = FString::Printf(TEXT("Event-%s-Properties"), *InEventScriptUsageId.ToString(EGuidFormats::DigitsWithHyphens));
	Super::Initialize(InRequiredEntryData, EventStackEditorDataKey);

	EventScriptUsageId = InEventScriptUsageId;

	EmitterWeakPtr = GetEmitterViewModel()->GetEmitter().ToWeakPtr();
	EmitterWeakPtr.Emitter->OnPropertiesChanged().AddUObject(this, &UNiagaraStackEventHandlerPropertiesItem::EventHandlerPropertiesChanged);
}

void UNiagaraStackEventHandlerPropertiesItem::FinalizeInternal()
{
	if (EmitterWeakPtr.IsValid())
	{
		EmitterWeakPtr.Emitter->OnPropertiesChanged().RemoveAll(this);
	}
	Super::FinalizeInternal();
}

FText UNiagaraStackEventHandlerPropertiesItem::GetDisplayName() const
{
	return LOCTEXT("EventHandlerPropertiesDisplayName", "Event Handler Properties");
}

bool UNiagaraStackEventHandlerPropertiesItem::TestCanResetToBaseWithMessage(FText& OutCanResetToBaseMessage) const
{
	if (bCanResetToBaseCache.IsSet() == false)
	{
		if (HasBaseEventHandler())
		{
			FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetParentEmitter();
			if (BaseEmitter.Emitter != nullptr && EmitterWeakPtr.Emitter != BaseEmitter.Emitter)
			{
				TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
				bCanResetToBaseCache = MergeManager->IsEventHandlerPropertySetDifferentFromBase(EmitterWeakPtr.ResolveWeakPtr(), BaseEmitter, EventScriptUsageId);
			}
			else
			{
				bCanResetToBaseCache = false;
			}
		}
		else
		{
			bCanResetToBaseCache = false;
		}
	}
	if (bCanResetToBaseCache.GetValue())
	{
		OutCanResetToBaseMessage = LOCTEXT("CanResetToBase", "Reset the event handler properties to the state defined by the parent emitter.");
		return true;
	}
	else
	{
		OutCanResetToBaseMessage = LOCTEXT("CanNotResetToBase", "No parent to reset to, or not different from parent.");
		return false;
	}
}

void UNiagaraStackEventHandlerPropertiesItem::ResetToBase()
{
	FText Unused;
	if (TestCanResetToBaseWithMessage(Unused) && EmitterWeakPtr.IsValid())
	{
		FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetParentEmitter();
		TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
		MergeManager->ResetEventHandlerPropertySetToBase(EmitterWeakPtr.ResolveWeakPtr(), BaseEmitter, EventScriptUsageId);
		RefreshChildren();
	}
}

void UNiagaraStackEventHandlerPropertiesItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (EmitterObject == nullptr && EmitterWeakPtr.GetEmitterData())
	{
		EventWrapper = NewObject<UNiagaraStackEventWrapper>(this, NAME_None, RF_Transactional);
		EventWrapper->EventHandlerScriptProps = EmitterWeakPtr.GetEmitterData()->EventHandlerScriptProps;
		EventWrapper->EmitterWeakPtr = EmitterWeakPtr;

		EmitterObject = NewObject<UNiagaraStackObject>(this);
		bool bIsTopLevelObject = true;
		bool bHideTopLevelCategories = false;
		EmitterObject->Initialize(CreateDefaultChildRequiredData(), EventWrapper, bIsTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey());
		EmitterObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraEventScriptProperties::StaticStruct()->GetFName(), 
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraEventScriptPropertiesCustomization::MakeInstance, 
			TWeakObjectPtr<UNiagaraSystem>(&GetSystemViewModel()->GetSystem()), GetEmitterViewModel()->GetEmitter().ToWeakPtr()));
		EmitterObject->SetOnFilterDetailNodes(FNiagaraStackObjectShared::FOnFilterDetailNodes::CreateUObject(this, &UNiagaraStackEventHandlerPropertiesItem::FilterEmitterStackObjectRootTreeNodes));
	}

	FVersionedNiagaraEmitterData* EmitterData = GetEmitterViewModel()->GetEmitter().GetEmitterData();
	if (EmitterData && (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim))
	{
		TArray<FStackIssueFix> IssueFixes;
		IssueFixes.Emplace(
			LOCTEXT("SetCpuSimulationFix", "Set CPU simulation"),
			FStackIssueFixDelegate::CreateLambda(
				[WeakEmitter = GetEmitterViewModel()->GetEmitter().ToWeakPtr()]()
		{
			if (WeakEmitter.IsValid())
			{
				FScopedTransaction Transaction(LOCTEXT("SetCpuSimulation", "Set Cpu Simulation"));
				WeakEmitter.Emitter.Get()->Modify();
				WeakEmitter.GetEmitterData()->SimTarget = ENiagaraSimTarget::CPUSim;
				UNiagaraSystem::RequestCompileForEmitter(WeakEmitter.ResolveWeakPtr());
			}
		}
		)
		);

		NewIssues.Emplace(
			EStackIssueSeverity::Error,
			LOCTEXT("EventHandlersNotSupportedOnCPU", "Event Handlers are not supported on GPU"),
			LOCTEXT("EventHandlersNotSupportedOnCPULong", "Event Handlers are currently not supported on GPU, please disable or remove."),
			GetStackEditorDataKey(),
			false,
			IssueFixes
		);
	}

	NewChildren.Add(EmitterObject);

	bCanResetToBaseCache.Reset();
	bHasBaseEventHandlerCache.Reset();

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

FHierarchyElementIdentity UNiagaraStackEventHandlerPropertiesItem::DetermineSummaryIdentity() const
{
	FHierarchyElementIdentity Identity;
	Identity.Guids.Add(GetEventScriptUsageId());
	Identity.Names.Add("Category");
	Identity.Names.Add("Properties");
	return Identity;
}

void UNiagaraStackEventHandlerPropertiesItem::EventHandlerPropertiesChanged()
{
	bCanResetToBaseCache.Reset();
}

TSharedPtr<IDetailTreeNode> GetEventHandlerArrayPropertyNode(const TArray<TSharedRef<IDetailTreeNode>>& Nodes)
{
	TArray<TSharedRef<IDetailTreeNode>> ChildrenToCheck;
	for (TSharedRef<IDetailTreeNode> Node : Nodes)
	{
		if (Node->GetNodeType() == EDetailNodeType::Item)
		{
			TSharedPtr<IPropertyHandle> NodePropertyHandle = Node->CreatePropertyHandle();
			if (NodePropertyHandle.IsValid() && NodePropertyHandle->GetProperty()->GetFName() == UNiagaraEmitter::PrivateMemberNames::EventHandlerScriptProps)
			{
				return Node;
			}
		}

		TArray<TSharedRef<IDetailTreeNode>> Children;
		Node->GetChildren(Children);
		ChildrenToCheck.Append(Children);
	}
	if (ChildrenToCheck.Num() == 0)
	{
		return nullptr;
	}
	return GetEventHandlerArrayPropertyNode(ChildrenToCheck);
}

void UNiagaraStackEventHandlerPropertiesItem::FilterEmitterStackObjectRootTreeNodes(const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes)
{
	TSharedPtr<IDetailTreeNode> EventHandlerArrayPropertyNode = GetEventHandlerArrayPropertyNode(InSourceNodes);
	if (EventHandlerArrayPropertyNode.IsValid())
	{
		TArray<TSharedRef<IDetailTreeNode>> EventHandlerArrayItemNodes;
		EventHandlerArrayPropertyNode->GetChildren(EventHandlerArrayItemNodes);
		for (TSharedRef<IDetailTreeNode> EventHandlerArrayItemNode : EventHandlerArrayItemNodes)
		{
			TSharedPtr<IPropertyHandle> EventHandlerArrayItemPropertyHandle = EventHandlerArrayItemNode->CreatePropertyHandle();
			if (EventHandlerArrayItemPropertyHandle.IsValid())
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(EventHandlerArrayItemPropertyHandle->GetProperty());
				if (StructProperty != nullptr && StructProperty->Struct == FNiagaraEventScriptProperties::StaticStruct())
				{
					TArray<void*> RawData;
					EventHandlerArrayItemPropertyHandle->AccessRawData(RawData);
					if (RawData.Num() == 1)
					{
						FNiagaraEventScriptProperties* EventScriptProperties = static_cast<FNiagaraEventScriptProperties*>(RawData[0]);
						if (EventScriptProperties->Script->GetUsageId() == EventScriptUsageId)
						{
							EventHandlerArrayItemNode->GetChildren(OutFilteredNodes);
							return;
						}
					}
				}
			}
		}
	}
}

bool UNiagaraStackEventHandlerPropertiesItem::HasBaseEventHandler() const
{
	if (bHasBaseEventHandlerCache.IsSet() == false)
	{
		FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetParentEmitter();
		if (BaseEmitter.Emitter != nullptr && EmitterWeakPtr.Emitter.Get() != BaseEmitter.Emitter)
		{
			TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
			bHasBaseEventHandlerCache = MergeManager->HasBaseEventHandler(BaseEmitter, EventScriptUsageId);
		}
		else
		{
			bHasBaseEventHandlerCache = false;
		}
	}
	return bHasBaseEventHandlerCache.GetValue();
}

void UNiagaraStackEventScriptItemGroup::Initialize(
	FRequiredEntryData InRequiredEntryData,
	TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
	ENiagaraScriptUsage InScriptUsage,
	FGuid InScriptUsageId,
	FGuid InEventSourceEmitterId)
{
	EventSourceEmitterId = InEventSourceEmitterId;
	FText ToolTip = LOCTEXT("EventGroupTooltip", "Determines how this Emitter responds to incoming events. There can be more than one event handler stage per Emitter.");
	FText TempDisplayName = FText::Format(LOCTEXT("TempDisplayNameFormat", "Event Handler - {0}"), FText::FromString(InScriptUsageId.ToString(EGuidFormats::DigitsWithHyphens)));
	Super::Initialize(InRequiredEntryData, TempDisplayName, ToolTip, InScriptViewModel, InScriptUsage, InScriptUsageId);
}

void UNiagaraStackEventScriptItemGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	bHasBaseEventHandlerCache.Reset();

	FVersionedNiagaraEmitterData* EmitterData = GetEmitterViewModel()->GetEmitter().GetEmitterData();

	const FNiagaraEventScriptProperties* EventScriptProperties = EmitterData->GetEventHandlers().FindByPredicate(
		[this](const FNiagaraEventScriptProperties& InEventScriptProperties) { return InEventScriptProperties.Script->GetUsageId() == GetScriptUsageId(); });

	if (EventScriptProperties != nullptr)
	{
		SetDisplayName(FText::Format(LOCTEXT("FormatEventScriptDisplayName", "Event Handler - Source: {0}"), FText::FromName(EventScriptProperties->SourceEventName)));
	}
	else
	{
		SetDisplayName(LOCTEXT("UnassignedEventDisplayName", "Unassigned Event"));
	}

	if (EventHandlerProperties == nullptr)
	{
		EventHandlerProperties = NewObject<UNiagaraStackEventHandlerPropertiesItem>(this);
		EventHandlerProperties->Initialize(CreateDefaultChildRequiredData(), GetScriptUsageId());
	}
	NewChildren.Add(EventHandlerProperties);

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

bool UNiagaraStackEventScriptItemGroup::TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const
{
	if (HasBaseEventHandler())
	{
		OutCanDeleteMessage = LOCTEXT("CantDeleteInherited", "Can not delete this event handler because it's inherited.");
		return false;
	}
	else
	{
		OutCanDeleteMessage = LOCTEXT("CanDelete", "Delete this event handler.");
		return true;
	}
}

void UNiagaraStackEventScriptItemGroup::Delete()
{
	TSharedPtr<FNiagaraScriptViewModel> ScriptViewModelPinned = ScriptViewModel.Pin();
	checkf(ScriptViewModelPinned.IsValid(), TEXT("Can not delete when the script view model has been deleted."));

	FVersionedNiagaraEmitter VersionedEmitter = GetEmitterViewModel()->GetEmitter();
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(EmitterData->GraphSource);

	if (!Source || !Source->NodeGraph)
	{
		return;
	}

	//Need to tear down existing systems now.
	FNiagaraSystemUpdateContext UpdateCtx;
	UpdateCtx.SetDestroyOnAdd(true);
	UpdateCtx.Add(VersionedEmitter, true);

	FScopedTransaction Transaction(FText::Format(LOCTEXT("DeleteEventHandler", "Deleted {0}"), GetDisplayName()));
	VersionedEmitter.Emitter->Modify();
	Source->NodeGraph->Modify();
	TArray<UNiagaraNode*> EventIndexNodes;
	Source->NodeGraph->BuildTraversal(EventIndexNodes, GetScriptUsage(), GetScriptUsageId());
	for (UNiagaraNode* Node : EventIndexNodes)
	{
		Node->Modify();
	}
	
	// First, remove the event handler script properties object.
	VersionedEmitter.Emitter->RemoveEventHandlerByUsageId(GetScriptUsageId(), VersionedEmitter.Version);
	
	// Now remove all graph nodes associated with the event script index.
	for (UNiagaraNode* Node : EventIndexNodes)
	{
		Node->DestroyNode();
	}

	// Set the emitter here to that the internal state of the view model is updated.
	// TODO: Move the logic for managing event handlers into the emitter view model or script view model.
	ScriptViewModelPinned->SetScripts(VersionedEmitter);
	
	OnModifiedEventHandlersDelegate.ExecuteIfBound();
}

bool UNiagaraStackEventScriptItemGroup::GetIsInherited() const
{
	return HasBaseEventHandler();
}

FText UNiagaraStackEventScriptItemGroup::GetInheritanceMessage() const
{
	return LOCTEXT("EventGroupInheritanceMessage", "This event handler is inherited from a parent emitter.  Inherited\nevent handlers can only be deleted while editing the parent emitter.");
}

FHierarchyElementIdentity UNiagaraStackEventScriptItemGroup::DetermineSummaryIdentity() const
{
	FHierarchyElementIdentity Identity;
	Identity.Guids.Add(GetScriptUsageId());
	return Identity;
}

bool UNiagaraStackEventScriptItemGroup::HasBaseEventHandler() const
{
	if (bHasBaseEventHandlerCache.IsSet() == false)
	{
		if (GetEmitterViewModel().IsValid())
		{
			FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetParentEmitter();
			bHasBaseEventHandlerCache = BaseEmitter.Emitter != nullptr && FNiagaraScriptMergeManager::Get()->HasBaseEventHandler(BaseEmitter, GetScriptUsageId());
		}
		else
		{
			bHasBaseEventHandlerCache = false;
		}
	}
	return bHasBaseEventHandlerCache.GetValue();
}

void UNiagaraStackEventScriptItemGroup::SetOnModifiedEventHandlers(FOnModifiedEventHandlers OnModifiedEventHandlers)
{
	OnModifiedEventHandlersDelegate = OnModifiedEventHandlers;
}

#undef LOCTEXT_NAMESPACE

