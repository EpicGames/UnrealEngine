// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "Utilities/NiagaraDynamicMaterialParameterUtils.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/StaticBitArray.h"
#include "EdGraphSchema_Niagara.h"
#include "Editor.h"
#include "INiagaraEditorTypeUtilities.h"
#include "IPythonScriptPlugin.h"
#include "NiagaraAnalytics.h"
#include "NiagaraClipboard.h"
#include "NiagaraConstants.h"
#include "NiagaraConvertInPlaceUtilityBase.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraGraph.h"
#include "NiagaraMessageManager.h"
#include "NiagaraMessages.h"
#include "NiagaraMessageUtilities.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraScript.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "NiagaraScriptMergeManager.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSettings.h"
#include "NiagaraTrace.h"
#include "ViewModels/NiagaraSystemScriptViewModel.h"
#include "ScopedTransaction.h"
#include "Algo/RemoveIf.h"
#include "EdGraph/EdGraphPin.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionDynamicParameter.h"
#include "UObject/StructOnScope.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraPlaceholderDataInterfaceManager.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"
#include "ViewModels/Stack/NiagaraStackObject.h"

#include "Widgets/Notifications/SNotificationList.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackFunctionInput)

#define LOCTEXT_NAMESPACE "NiagaraStackViewModel"

static FText TooManyConversionScripts = LOCTEXT("TooManyConversionScripts", "There is more than one dynamic input script available auto-convert the dragged parameter. Please fix this by disabling conversion for all but one of them:\n{0}");

UNiagaraStackFunctionInput::FNiagaraAvailableParameterInfo::FNiagaraAvailableParameterInfo(FNiagaraVariableBase InVariableBase)
{
	Variable = InVariableBase;
}

UNiagaraStackFunctionInput::FNiagaraAvailableParameterInfo::FNiagaraAvailableParameterInfo(FNiagaraVariableBase InVariableBase,	UNiagaraScript* InConversionScript)
{
	Variable = InVariableBase;
	ConversionScript = InConversionScript;
}

UNiagaraStackFunctionInput::UNiagaraStackFunctionInput()
	: OwningModuleNode(nullptr)
	, OwningFunctionCallNode(nullptr)
	, bUpdatingGraphDirectly(false)
	, bUpdatingLocalValueDirectly(false)
	, bShowEditConditionInline(false)
	, bIsInlineEditConditionToggle(false)
	, bIsDynamicInputScriptReassignmentPending(false)
{
}

// Traverses the path between the owning module node and the function call node this input belongs too collecting up the input handles between them.
void GenerateInputParameterHandlePath(UNiagaraNodeFunctionCall& ModuleNode, UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraParameterHandle>& OutHandlePath)
{
	UNiagaraNodeFunctionCall* CurrentFunctionCallNode = &FunctionCallNode;
	FPinCollectorArray FunctionOutputPins;
	while (CurrentFunctionCallNode != &ModuleNode)
	{
		FunctionOutputPins.Reset();
		CurrentFunctionCallNode->GetOutputPins(FunctionOutputPins);
		if (FunctionOutputPins.Num() == 1 && FunctionOutputPins[0]->LinkedTo.Num() == 1 && FunctionOutputPins[0]->LinkedTo[0]->GetOwningNode()->IsA<UNiagaraNodeParameterMapSet>())
		{
			FNiagaraParameterHandle AliasedHandle(FunctionOutputPins[0]->LinkedTo[0]->PinName);
			OutHandlePath.Add(FNiagaraParameterHandle::CreateModuleParameterHandle(AliasedHandle.GetName()));
			UNiagaraNodeParameterMapSet* NextOverrideNode = CastChecked<UNiagaraNodeParameterMapSet>(FunctionOutputPins[0]->LinkedTo[0]->GetOwningNode());
			UEdGraphPin* NextOverrideNodeOutputPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*NextOverrideNode);
			
			CurrentFunctionCallNode = nullptr;
			for (UEdGraphPin* NextOverrideNodeOutputPinLinkedPin : NextOverrideNodeOutputPin->LinkedTo)
			{
				UNiagaraNodeFunctionCall* NextFunctionCallNode = Cast<UNiagaraNodeFunctionCall>(NextOverrideNodeOutputPinLinkedPin->GetOwningNode());
				if (NextFunctionCallNode != nullptr && NextFunctionCallNode->GetFunctionName() == AliasedHandle.GetNamespace().ToString())
				{
					CurrentFunctionCallNode = NextFunctionCallNode;
					break;
				}
			}

			if (ensureMsgf(CurrentFunctionCallNode != nullptr, TEXT("Invalid Stack Graph - Function call node for override pin %s could not be found."), *FunctionOutputPins[0]->PinName.ToString()) == false)
			{
				OutHandlePath.Empty();
				return;
			}
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Log, "Invalid Stack Graph - Dynamic Input Function call didn't have a valid connected output.");

			OutHandlePath.Empty();
			return;
		}
	}
}

void UNiagaraStackFunctionInput::Initialize(
	FRequiredEntryData InRequiredEntryData,
	UNiagaraNodeFunctionCall& InModuleNode,
	UNiagaraNodeFunctionCall& InInputFunctionCallNode,
	FName InInputParameterHandle,
	FNiagaraTypeDefinition InInputType,
	EStackParameterBehavior InParameterBehavior,
	FString InOwnerStackItemEditorDataKey)
{
	checkf(OwningModuleNode.IsValid() == false && OwningFunctionCallNode.IsValid() == false, TEXT("Can only initialize once."));
	ParameterBehavior = InParameterBehavior;
	FString InputStackEditorDataKey = FString::Printf(TEXT("%s-Input-%s"), *InInputFunctionCallNode.NodeGuid.ToString(EGuidFormats::DigitsWithHyphens), *InInputParameterHandle.ToString());
	Super::Initialize(InRequiredEntryData, InOwnerStackItemEditorDataKey, InputStackEditorDataKey);
	OwningModuleNode = &InModuleNode;
	OwningFunctionCallNode = &InInputFunctionCallNode;
	OwningFunctionCallInitialScript = OwningFunctionCallNode->FunctionScript;
	OwningAssignmentNode = Cast<UNiagaraNodeAssignment>(OwningFunctionCallNode.Get());
	
	TOptional<FGuid> VariableGuid = OwningFunctionCallNode->GetFunctionScriptSource()->NodeGraph->GetScriptVariableGuid(FNiagaraVariable(InInputType, InInputParameterHandle));
	if(VariableGuid.IsSet())
	{
		UHierarchyRoot* HierarchyScriptParameterRoot = OwningFunctionCallNode->GetFunctionScriptSource()->NodeGraph->GetScriptParameterHierarchyRoot();
		HierarchyScriptParameter = Cast<UNiagaraHierarchyScriptParameter>(HierarchyScriptParameterRoot->FindChildWithIdentity(FHierarchyElementIdentity({VariableGuid.GetValue()}, {}), true));
	}

	UNiagaraSystem& ParentSystem = GetSystemViewModel()->GetSystem();
	FVersionedNiagaraEmitter ParentEmitter = GetEmitterViewModel().IsValid() ? GetEmitterViewModel()->GetEmitter() : FVersionedNiagaraEmitter();

	FNiagaraStackGraphUtilities::FindAffectedScripts(&ParentSystem, ParentEmitter, *OwningModuleNode.Get(), AffectedScripts);

	UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*OwningModuleNode.Get());
	for (TWeakObjectPtr<UNiagaraScript> AffectedScript : AffectedScripts)
	{
		if (AffectedScript.IsValid() && AffectedScript->IsEquivalentUsage(OutputNode->GetUsage()) && AffectedScript->GetUsageId() == OutputNode->GetUsageId())
		{
			SourceScript = AffectedScript;
			RapidIterationParametersChangedHandle = SourceScript->RapidIterationParameters.AddOnChangedHandler(
				FNiagaraParameterStore::FOnChanged::FDelegate::CreateUObject(this, &UNiagaraStackFunctionInput::OnRapidIterationParametersChanged));
			SourceScript->GetLatestSource()->OnChanged().AddUObject(this, &UNiagaraStackFunctionInput::OnScriptSourceChanged);
			break;
		}
	}

	if (!SourceScript.IsValid())
		UE_LOGF(LogNiagaraEditor, Warning, "Coudn't find source script in affected scripts.");

	GraphChangedHandle = OwningFunctionCallNode->GetGraph()->AddOnGraphChangedHandler(
		FOnGraphChanged::FDelegate::CreateUObject(this, &UNiagaraStackFunctionInput::OnGraphChanged));
	OnRecompileHandle = OwningFunctionCallNode->GetNiagaraGraph()->AddOnGraphNeedsRecompileHandler(
		FOnGraphChanged::FDelegate::CreateUObject(this, &UNiagaraStackFunctionInput::OnGraphChanged));

	InputParameterHandle = FNiagaraParameterHandle(InInputParameterHandle);
	GenerateInputParameterHandlePath(*OwningModuleNode, *OwningFunctionCallNode, InputParameterHandlePath);
	InputParameterHandlePath.Add(InputParameterHandle);

	DisplayName = FText::FromName(InputParameterHandle.GetName());

	InputType = InInputType;
	StackEditorDataKey = FNiagaraStackGraphUtilities::StackKeys::GenerateStackFunctionInputEditorDataKey(*OwningFunctionCallNode.Get(), InputParameterHandle);

	TArray<UNiagaraScript*> AffectedScriptsNotWeak;
	for (TWeakObjectPtr<UNiagaraScript> AffectedScript : AffectedScripts)
	{
		AffectedScriptsNotWeak.Add(AffectedScript.Get());
	}

	FCompileConstantResolver ConstantResolver = GetEmitterViewModel().IsValid()
		? FCompileConstantResolver(GetEmitterViewModel()->GetEmitter(), SourceScript->GetUsage())
		: FCompileConstantResolver(&GetSystemViewModel()->GetSystem(), SourceScript->GetUsage());
	FString UniqueEmitterName = GetEmitterViewModel().IsValid() ? GetEmitterViewModel()->GetEmitter().Emitter->GetUniqueEmitterName() : FString();
	EditCondition.Initialize(SourceScript.Get(), AffectedScriptsNotWeak, ConstantResolver, UniqueEmitterName, OwningFunctionCallNode.Get());
	VisibleCondition.Initialize(SourceScript.Get(), AffectedScriptsNotWeak, ConstantResolver, UniqueEmitterName, OwningFunctionCallNode.Get());

	MessageLogGuid = GetSystemViewModel()->GetMessageLogGuid();

	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackFunctionInput::FilterInlineChildren));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackFunctionInput::FilterForVisibleCondition));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackFunctionInput::FilterOnlyModified));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackFunctionInput::FilterForIsInlineEditConditionToggle));
}

void UNiagaraStackFunctionInput::FinalizeInternal()
{
	if (OwningFunctionCallNode.IsValid())
	{
		OwningFunctionCallNode->GetGraph()->RemoveOnGraphChangedHandler(GraphChangedHandle);
		OwningFunctionCallNode->GetNiagaraGraph()->RemoveOnGraphNeedsRecompileHandler(OnRecompileHandle);
	}

	if (SourceScript.IsValid())
	{
		SourceScript->RapidIterationParameters.RemoveOnChangedHandler(RapidIterationParametersChangedHandle);
		SourceScript->GetLatestSource()->OnChanged().RemoveAll(this);
	}

	if (MessageManagerRegistrationKey.IsValid())
	{
		FNiagaraMessageManager::Get()->Unsubscribe(
			  DisplayName
			, MessageLogGuid
			, MessageManagerRegistrationKey);
	}

	if (PlaceholderDataInterfaceHandle.IsValid())
	{
		PlaceholderDataInterfaceHandle.Reset();
	}

	Super::FinalizeInternal();
}

const UNiagaraNodeFunctionCall& UNiagaraStackFunctionInput::GetInputFunctionCallNode() const
{
	return *OwningFunctionCallNode.Get();
}

UNiagaraNodeFunctionCall* UNiagaraStackFunctionInput::GetInputFunctionCallNodePtr() const
{
	return OwningFunctionCallNode.Get();
}

UNiagaraScript* UNiagaraStackFunctionInput::GetInputFunctionCallInitialScript() const
{
	return OwningFunctionCallInitialScript.Get();
}

UNiagaraStackFunctionInput::EValueMode UNiagaraStackFunctionInput::GetValueMode() const
{
	return InputValues.Mode;
}

const FNiagaraTypeDefinition& UNiagaraStackFunctionInput::GetInputType() const
{
	return InputType;
}

EUnit UNiagaraStackFunctionInput::GetInputDisplayUnit() const
{
	return InputMetaData.IsSet() ? InputMetaData->DisplayUnit : EUnit::Unspecified;
}

FNiagaraInputParameterCustomization UNiagaraStackFunctionInput::GetInputWidgetCustomization() const
{
	return InputMetaData.IsSet() ? InputMetaData->WidgetCustomization : FNiagaraInputParameterCustomization();
}

bool UNiagaraStackFunctionInput::GetShouldShowInStack() const
{
	// Using the FilterOnlyModified child filter, we let inputs pass that are at default value but have modified children inputs
	// If we removed them via filter, the modified children inputs also wouldn't be displayed.
	// Instead, we remove these inputs from the display
	if (IsFinalized() == false)
	{
		if (GetStackEditorData().GetShowOnlyModified())
		{
			if (CanReset() == false)
			{
				return false;
			}
		}
		if (bIsSharedLinkedValueInput)
		{
			return false;
		}
	}

	return true;
}

FText UNiagaraStackFunctionInput::GetTooltipText() const
{
	FText Description = SummaryViewTooltipOverride.IsSet() && !SummaryViewTooltipOverride->Get().IsEmptyOrWhitespace() ? SummaryViewTooltipOverride->Get() : InputMetaData.IsSet() ? InputMetaData->Description : FText::GetEmpty();
	FText BaseTooltip = FNiagaraEditorUtilities::FormatVariableDescription(Description, GetDisplayName(), InputType.GetNameText());

	// Special handling to resolve dynamic parameter default value
	TOptional<float> MaterialDefault = GetMaterialDynamicParamDefaultValue();
	if (MaterialDefault.IsSet())
	{
		return FText::Format(LOCTEXT("MaterialDefaultFormat", "{0}\nMaterial Default: {1}"), BaseTooltip, FText::AsNumber(MaterialDefault.GetValue()));
	}

	return BaseTooltip;
}

bool UNiagaraStackFunctionInput::GetIsEnabled() const
{
	return OwningFunctionCallNode->GetDesiredEnabledState() == ENodeEnabledState::Enabled;
}

UObject* UNiagaraStackFunctionInput::GetExternalAsset() const
{
	if(GetValueMode() == EValueMode::Dynamic)
	{
		if(InputValues.DynamicNode.IsValid() && InputValues.DynamicNode->FunctionScript != nullptr && InputValues.DynamicNode->FunctionScript->IsAsset())
		{
			return InputValues.DynamicNode->FunctionScript;
		}
	}
	else if (OwningFunctionCallNode.IsValid() && OwningFunctionCallNode->FunctionScript != nullptr && OwningFunctionCallNode->FunctionScript->IsAsset())
	{
		return OwningFunctionCallNode->FunctionScript;
	}
	return nullptr;
}

bool UNiagaraStackFunctionInput::TestCanCutWithMessage(FText& OutMessage) const
{
	if (InputValues.HasEditableData() == false)
	{
		OutMessage = LOCTEXT("CantCutInvalidMessage", "The current input state doesn't support cutting.");
		return false;
	}
	if (GetIsEnabledAndOwnerIsEnabled() == false)
	{
		OutMessage = LOCTEXT("CantCutDisabled", "Can not cut and input when it's owner is disabled.");
		return false;
	}
	OutMessage = LOCTEXT("CanCutMessage", "Cut will copy the value of this input including\nany data objects and dynamic inputs, and will reset it to default.");
	return true;
}

FText UNiagaraStackFunctionInput::GetCutTransactionText() const
{
	return LOCTEXT("CutInputTransaction", "Cut niagara input");
}

void UNiagaraStackFunctionInput::CopyForCut(UNiagaraClipboardContent* ClipboardContent) const
{
	Copy(ClipboardContent);
}

void UNiagaraStackFunctionInput::RemoveForCut()
{
	Reset();
}

bool UNiagaraStackFunctionInput::TestCanCopyWithMessage(FText& OutMessage) const
{
	if (InputValues.HasEditableData() == false)
	{
		OutMessage = LOCTEXT("CantCopyInvalidMessage", "The current input state doesn't support copying.");
		return false;
	}
	OutMessage = LOCTEXT("CanCopyMessage", "Copy the value of this input including\nany data objects and dynamic inputs.");
	return true;
}

void UNiagaraStackFunctionInput::Copy(UNiagaraClipboardContent* ClipboardContent) const
{
	const UNiagaraClipboardFunctionInput* ClipboardInput = ToClipboardFunctionInput(ClipboardContent);
	if (ClipboardInput != nullptr)
	{
		ClipboardContent->FunctionInputs.Add(ClipboardInput);
		FNiagaraClipboardPortableValue InputPortableValue;
		if (ClipboardInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::Local)
		{
			FNiagaraVariant LocalValue;
			LocalValue.SetBytes(ClipboardInput->Local.GetData(), ClipboardInput->Local.Num());
			InputPortableValue = FNiagaraClipboardPortableValue::CreateFromTypedValue(InputType, LocalValue);
		}
		else if (ClipboardInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::Data)
		{
			FNiagaraVariant DataValue;
			DataValue.SetDataInterface(ClipboardInput->Data);
			InputPortableValue = FNiagaraClipboardPortableValue::CreateFromTypedValue(InputType, DataValue);
		}
		if (InputPortableValue.IsValid())
		{
			ClipboardContent->PortableValues.Add(InputPortableValue);
		}
	}
}

bool UNiagaraStackFunctionInput::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	if ((ClipboardContent->FunctionInputs.Num() == 0 && ClipboardContent->PortableValues.Num() == 0) || GetIsEnabledAndOwnerIsEnabled() == false)
	{
		// Empty clipboard, or disabled don't allow paste, but be silent.
		return false;
	}
	else if (ClipboardContent->FunctionInputs.Num() == 1)
	{
		const UNiagaraClipboardFunctionInput* ClipboardFunctionInput = ClipboardContent->FunctionInputs[0];
		if (ClipboardFunctionInput != nullptr)
		{
			if (ClipboardFunctionInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::Dynamic)
			{
				UNiagaraScript* ClipboardFunctionScript = ClipboardFunctionInput->Dynamic->Script.LoadSynchronous();
				if (ClipboardFunctionScript == nullptr)
				{
					OutMessage = LOCTEXT("CantPasteInvalidDynamicInputScript", "Can not paste the dynamic input because its script is no longer valid.");
					return false;
				}
			}
			if (ClipboardFunctionInput->InputType == InputType)
			{
				OutMessage = LOCTEXT("PasteMessage", "Paste the input from the clipboard here.");
				return true;
			}
			if (GetPossibleConversionScripts(ClipboardFunctionInput->InputType).Num() > 0)
			{
				OutMessage = LOCTEXT("PasteWithConversionMessage", "Paste the input from the clipboard here and auto-convert it with a dynamic input.");
				return true;
			}
			
			OutMessage = LOCTEXT("CantPasteIncorrectType", "Can not paste inputs with mismatched types.");
			return false;
		}
		return false;
	}
	else if (ClipboardContent->FunctionInputs.Num() > 1)
	{
		OutMessage = LOCTEXT("CantPasteMultipleInputs", "Can't paste multiple inputs onto a single input.");
		return false;
	}
	else if (ClipboardContent->PortableValues.Num() == 1)
	{
		if (ClipboardContent->PortableValues[0].IsValid())
		{
			if (ClipboardContent->PortableValues[0].CanUpdateTypedValue(InputType))
			{
				OutMessage = LOCTEXT("PasteValueMessage", "Paste the value from the clipboard here.");
				return true;
			}
			else
			{
				OutMessage = LOCTEXT("UnsupportedPortableValueMessage", "Pasting the copied value to this input is unsupported.");
				return false;
			}
		}
	}
	return false;
}

FText UNiagaraStackFunctionInput::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	return LOCTEXT("PasteInputTransactionText", "Paste Niagara inputs");
}

void UNiagaraStackFunctionInput::PasteFunctionInput(const UNiagaraClipboardFunctionInput* ClipboardInput)
{
	// For the main input (could be a parent input), the user is intentional about copy pasting, so we only require assignable types
	if (FNiagaraEditorUtilities::AreTypesAssignable(ClipboardInput->InputType, InputType))
	{
		SetValueFromClipboardFunctionInput(*ClipboardInput);
	}
	// or alternatively check for a conversion script
	else
	{
		SetClipboardContentViaConversionScript(*ClipboardInput);
	}

	RequestPulse();
	
	// For children inputs we are more strict and require name & type match as the user is less intentional about it when pasting
	if(ClipboardInput->ChildrenInputs.Num() > 0)
	{
		TArray<UNiagaraStackFunctionInput*> StackChildrenInputs;
		GetUnfilteredChildrenOfType(StackChildrenInputs, false);
		for(const UNiagaraClipboardFunctionInput* ClipboardChildInput : ClipboardInput->ChildrenInputs)
		{
			if(ClipboardChildInput == nullptr)
			{
				continue;
			}
			
			for(UNiagaraStackFunctionInput* StackChildInput : StackChildrenInputs)
			{
				if(StackChildInput->GetInputType() == ClipboardChildInput->GetTypeDef() &&
					StackChildInput->GetInputParameterHandle().GetName() == ClipboardChildInput->InputName)
				{
					StackChildInput->PasteFunctionInput(ClipboardChildInput);
				}
			}
		}
	}
}

void UNiagaraStackFunctionInput::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	if (ensureMsgf(ClipboardContent != nullptr && (ClipboardContent->FunctionInputs.Num() == 1 || ClipboardContent->PortableValues.Num() == 1),
		TEXT("Clipboard must not be null, and must contain a single input or portable value.  Call TestCanPasteWithMessage to validate")))
	{
		
		if (ClipboardContent->FunctionInputs.Num() == 1)
		{
			if (const UNiagaraClipboardFunctionInput* ClipboardInput = ClipboardContent->FunctionInputs[0])
			{
				PasteFunctionInput(ClipboardInput);
			}
		}
		else if (ClipboardContent->PortableValues.Num() == 1)
		{
			FNiagaraVariant InputValue;
			if (ClipboardContent->PortableValues[0].TryUpdateTypedValue(InputType, InputValue))
			{
				const UNiagaraClipboardFunctionInput* ClipboardInput = nullptr;
				if (InputValue.GetMode() == ENiagaraVariantMode::Bytes)
				{
					TArray<uint8> ValueBytes;
					ValueBytes.Append(InputValue.GetBytes(), InputValue.GetNumBytes());
					ClipboardInput = UNiagaraClipboardFunctionInput::CreateLocalValue(GetTransientPackage(), NAME_None, InputType, TOptional<bool>(), ValueBytes);
				}
				else if (InputValue.GetMode() == ENiagaraVariantMode::DataInterface)
				{
					ClipboardInput = UNiagaraClipboardFunctionInput::CreateDataValue(GetTransientPackage(), NAME_None, InputType, TOptional<bool>(), InputValue.GetDataInterface());
				}
				if (ClipboardInput != nullptr)
				{
					SetValueFromClipboardFunctionInput(*ClipboardInput);
				}
			}
		}
	}
}

bool UNiagaraStackFunctionInput::HasOverridenContent() const
{
	return CanReset();
}

bool UNiagaraStackFunctionInput::SupportsSummaryView() const
{
	if(GetInputMetaData().IsSet())
	{
		if(GetInputMetaData()->bInlineEditConditionToggle)
		{
			return false;
		}
	}

	return true;
}

FHierarchyElementIdentity UNiagaraStackFunctionInput::DetermineSummaryIdentity() const
{
	FHierarchyElementIdentity Identity;
	Identity.Guids.Add(GetInputFunctionCallNode().NodeGuid);
	if(Cast<UNiagaraNodeAssignment>(&GetInputFunctionCallNode()) != nullptr)
	{
		Identity.Names.Add(GetInputParameterHandle().GetParameterHandleString());
	}
	else
	{
		FNiagaraVariable ThisInput(GetInputType(), GetInputParameterHandle().GetParameterHandleString());
		Identity.Guids.Add(GetInputFunctionCallNode().GetCalledGraph()->GetScriptVariableGuid(ThisInput).GetValue());
	}

	return Identity;
}

bool UNiagaraStackFunctionInput::GetCanExpand() const
{
	// If we are in local mode, we want to always be expanded (by setting GetCanExpand == false), to ensure potential children inputs always show up 
	return GetValueMode() != EValueMode::Local;
}

TArray<UNiagaraStackFunctionInput*> UNiagaraStackFunctionInput::GetChildInputs() const
{
	TArray<UNiagaraStackFunctionInput*> ChildInputs;
	GetUnfilteredChildrenOfType(ChildInputs, true);	
	return ChildInputs;
}

TOptional<FNiagaraVariableMetaData> UNiagaraStackFunctionInput::GetInputMetaData() const
{
	return InputMetaData;
}

void UNiagaraStackFunctionInput::GetFilteredChildInputs(TArray<UNiagaraStackFunctionInput*>& OutFilteredChildInputs) const
{
	GetFilteredChildrenOfType(OutFilteredChildInputs, true);
}

UNiagaraStackObject* UNiagaraStackFunctionInput::GetChildDataObject() const
{
	TArray<UNiagaraStackObject*> ChildDataObjects;
	GetUnfilteredChildrenOfType(ChildDataObjects);
	if (ensureMsgf(ChildDataObjects.Num() <= 1, TEXT("Function input should have at most one child object.  Function Path: %s Function Name: %s Input Name: %s"),
		OwningFunctionCallNode.IsValid() ? *OwningFunctionCallNode->GetPathName() : TEXT("[Unknown]"), 
		OwningFunctionCallNode.IsValid() ? *OwningFunctionCallNode->GetFunctionName() : TEXT("[Unknown]"),
		*InputParameterHandle.GetName().ToString()))
	{
		return ChildDataObjects.Num() == 1 ? ChildDataObjects[0] : nullptr;
	}
	return nullptr;
}

void UNiagaraStackFunctionInput::GetCurrentChangeIds(FGuid& OutOwningGraphChangeId, FGuid& OutFunctionGraphChangeId) const
{
	OutOwningGraphChangeId = OwningFunctionCallNode->GetNiagaraGraph()->GetChangeID();
	OutFunctionGraphChangeId = OwningFunctionCallNode->GetCalledGraph() != nullptr ? OwningFunctionCallNode->GetCalledGraph()->GetChangeID() : FGuid();
}

bool UNiagaraStackFunctionInput::FilterInlineChildren(const UNiagaraStackEntry& Child) const
{
	return GbEnableExperimentalInlineDynamicInputs == 0 || GetInlineDisplayMode() == ENiagaraStackEntryInlineDisplayMode::None;
}

bool UNiagaraStackFunctionInput::FilterForVisibleCondition(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackFunctionInput* StackFunctionInputChild = Cast<UNiagaraStackFunctionInput>(&Child);
	return StackFunctionInputChild == nullptr || StackFunctionInputChild->GetShouldPassFilterForVisibleCondition();
}

bool UNiagaraStackFunctionInput::FilterOnlyModified(const UNiagaraStackEntry& Child) const
{
	if(GetStackEditorData().GetShowOnlyModified() == false)
	{
		return true;
	}
	
	const UNiagaraStackFunctionInput* FunctionInput = Cast<UNiagaraStackFunctionInput>(&Child);
	if (FunctionInput == nullptr || FunctionInput->CanReset() || FunctionInput->HasAnyResettableChildrenInputs())
	{
		return true;
	}

	return false;
}

bool UNiagaraStackFunctionInput::FilterForIsInlineEditConditionToggle(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackFunctionInput* StackFunctionInputChild = Cast<UNiagaraStackFunctionInput>(&Child);
	return StackFunctionInputChild == nullptr || StackFunctionInputChild->GetIsInlineEditConditionToggle() == false;
}

void UNiagaraStackFunctionInput::ReportScriptVersionChange() const
{
	// analytics
	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Type"), TEXT("DynamicInput")));
	if (NiagaraAnalytics::IsPluginAsset(GetDynamicInputNode()->FunctionScript))
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AssetName"), GetDynamicInputNode()->FunctionScript->GetPackage()->GetName()));
	}
	NiagaraAnalytics::RecordEvent(TEXT("Versioning.ScriptVersionChanged"), Attributes);
}

FText UNiagaraStackFunctionInput::GetCollapsedStateText() const
{
	if (IsFinalized())
	{
		return FText();
	}
	
	if (CollapsedTextCache.IsSet() == false)
	{
		switch (InputValues.Mode)
		{
		case EValueMode::Local:
			{
				FNiagaraEditorModule& EditorModule = FNiagaraEditorModule::Get();
				auto TypeUtilityValue = EditorModule.GetTypeUtilities(InputType);
				FNiagaraVariable Var(InputType, NAME_None);
				Var.SetData(InputValues.LocalStruct->GetStructMemory());
				CollapsedTextCache = TypeUtilityValue->GetStackDisplayText(Var);
			}
			break;
		case EValueMode::Data:
			CollapsedTextCache = FText::Format(FText::FromString("[{0}]"), InputValues.DataObject.IsValid() ? InputValues.DataObject->GetClass()->GetDisplayNameText() : FText::FromString("?"));
			break;
		case EValueMode::ObjectAsset:
			{
				UNiagaraNodeInput* InputNode = InputValues.ObjectAssetInputNode.Get();
				CollapsedTextCache = FText::Format(FText::FromString("[{0}]"), InputNode ? InputNode->Input.GetType().GetClass()->GetDisplayNameText() : FText::FromString("?"));
			}
			break;
		case EValueMode::Dynamic:
			if (InputValues.DynamicNode->FunctionScript != nullptr)
			{
				FFormatOrderedArguments Arguments;
				for (const auto& Child : GetChildInputs())
				{
					FText ChildText;
					if (Child)
					{
						ChildText = Child->GetCollapsedStateText();
					}
					if (ChildText.IsEmptyOrWhitespace())
					{
						ChildText = FText::FromString("[?]");
					}
					Arguments.Add(ChildText);
				}
				CollapsedTextCache = FText::Format(InputValues.DynamicNode->GetScriptData()->CollapsedViewFormat, Arguments);
			}
			else
			{
				CollapsedTextCache = FText();
			}
			break;
		case EValueMode::Linked:
		case EValueMode::LinkedShared:
			CollapsedTextCache = FText::FromString(InputValues.LinkedParameter.GetName().ToString());
			break;
		case EValueMode::Expression:
			CollapsedTextCache = FText::Format(FText::FromString("({0})"), FText::FromString(InputValues.ExpressionNode->GetCustomHlsl()));
			break;
		default:
			CollapsedTextCache = FText();
			break;
		}
	}
	return CollapsedTextCache.GetValue();
}

void UNiagaraStackFunctionInput::SetSummaryViewDisplayName(TAttribute<FText> InDisplayName)
{
	SummaryViewDisplayNameOverride = InDisplayName;
}

void UNiagaraStackFunctionInput::SetSummaryViewTooltip(TAttribute<FText> InTooltipOverride)
{
	SummaryViewTooltipOverride = InTooltipOverride;
}

FText UNiagaraStackFunctionInput::GetValueToolTip() const
{
	if (IsFinalized())
	{
		return FText();
	}

	if (ValueToolTipCache.IsSet() == false)
	{
		ValueToolTipCache = FText();
		switch (InputValues.Mode)
		{
		case EValueMode::Data:
		{
			if (InputValues.DataObject.IsValid())
			{
				FString DataInterfaceDescription = InputValues.DataObject->GetClass()->GetDescription();
				if (DataInterfaceDescription.Len() > 0)
				{
					ValueToolTipCache = FText::FromString(DataInterfaceDescription);
				}
			}
			break;
		}
		case EValueMode::ObjectAsset:
		{
			if ( UNiagaraNodeInput* InputNode = InputValues.ObjectAssetInputNode.Get() )
			{
				const FString ObjectDescription = InputNode->Input.GetType().GetClass()->GetDescription();
				if (ObjectDescription.Len() > 0)
				{
					ValueToolTipCache = FText::FromString(ObjectDescription);
				}
			}
			break;
		}
		case EValueMode::DefaultFunction:
			if (InputValues.DefaultFunctionNode->GetScriptData() != nullptr)
			{
				ValueToolTipCache = InputValues.DefaultFunctionNode->GetScriptData()->Description;
			}
			break;
		case EValueMode::Dynamic:
			if (FVersionedNiagaraScriptData* ScriptData = InputValues.DynamicNode->GetScriptData())
			{
				FText FunctionName = FText::FromString(FName::NameToDisplayString(InputValues.DynamicNode->GetFunctionName(), false));
				if (ScriptData->Description.IsEmptyOrWhitespace())
				{
					ValueToolTipCache = FText::Format(FText::FromString("Compiled Name: {0}"), FunctionName);
				}
				else
				{
					ValueToolTipCache = FText::Format(FText::FromString("{0}\n\nCompiled Name: {1}"), ScriptData->Description, FunctionName);
				}
			}
			break;
		case EValueMode::InvalidOverride:
			ValueToolTipCache = LOCTEXT("InvalidOverrideToolTip", "The script is in an invalid and unrecoverable state for this\ninput.  Resetting to default may fix this issue.");
			break;
		case EValueMode::Linked:
		case EValueMode::LinkedShared:
		{
			FNiagaraVariable InputVariable = InputValues.LinkedParameter;
			if (FNiagaraConstants::IsNiagaraConstant(InputVariable))
			{
				const FNiagaraVariableMetaData* FoundMetaData = FNiagaraConstants::GetConstantMetaData(InputVariable);
				if (FoundMetaData != nullptr)
				{
					ValueToolTipCache = FoundMetaData->Description;
				}
			}
			break;
		}
		case EValueMode::UnsupportedDefault:
			ValueToolTipCache = LOCTEXT("UnsupportedDefault", "The default value defined in the script graph\nis custom and can not be shown in the selection stack.");
			break;
		}
	}
	return ValueToolTipCache.GetValue();
}

TArray<FNiagaraStackFunctionInputActions> UNiagaraStackFunctionInput::GetCustomActions()
{
	TArray<FNiagaraStackFunctionInputActions> Actions;

	TOptional<float> MaterialDefault = GetMaterialDynamicParamDefaultValue();
	if (MaterialDefault.IsSet())
	{
		const float DefaultValue = MaterialDefault.GetValue();
		Actions.Emplace(
			FText::FormatOrdered(LOCTEXT("SetToMaterialDefault", "Set to Material Default ({0})"), FText::AsNumber(DefaultValue)),
			LOCTEXT("SetToMaterialDefaultToolTip", "Sets this input to a constant matching the default value defined in the material's Dynamic Parameter node."),
			FExecuteAction::CreateLambda(
				[this, DefaultValue]()
				{
					TSharedRef<FStructOnScope> ValueStruct = MakeShared<FStructOnScope>(GetInputType().GetStruct());
					float* FloatData = reinterpret_cast<float*>(ValueStruct->GetStructMemory());
					*FloatData = DefaultValue;
					SetLocalValue(ValueStruct);
				}
			)
		);
	}

	return Actions;
}

UNiagaraStackEntry::FStackIssueFixDelegate UNiagaraStackFunctionInput::GetUpgradeDynamicInputVersionFix()
{
	if (!InputValues.DynamicNode.IsValid())
	{
		return FStackIssueFixDelegate();
	}
	return FStackIssueFixDelegate::CreateLambda([this]()
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("UpgradeVersionFix", "Change dynamic input version"));
		FNiagaraScriptVersionUpgradeContext UpgradeContext;
		UpgradeContext.CreateClipboardCallback = [this](UNiagaraClipboardContent* ClipboardContent)
		{
			TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
			if (CachedSysViewModel->GetSystemStackViewModel())
				CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();

			RefreshChildren();
			Copy(ClipboardContent);
			if (ClipboardContent->Functions.Num() > 0)
			{
				ClipboardContent->FunctionInputs = ClipboardContent->Functions[0]->Inputs;
				ClipboardContent->Functions.Empty();
			}
		};
		UNiagaraNodeFunctionCall* FunctionCallNode = GetDynamicInputNode();
		UpgradeContext.ApplyClipboardCallback = [this](UNiagaraClipboardContent* ClipboardContent, FText& OutWarning) { Paste(ClipboardContent, OutWarning); };
		UpgradeContext.ConstantResolver = GetEmitterViewModel().IsValid() ?
			FCompileConstantResolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCallNode)) :
			FCompileConstantResolver(&GetSystemViewModel()->GetSystem(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCallNode));
		FunctionCallNode->ChangeScriptVersion(FunctionCallNode->FunctionScript->GetExposedVersion().VersionGuid, UpgradeContext, true);
		if (FunctionCallNode->RefreshFromExternalChanges())
		{
			FunctionCallNode->GetNiagaraGraph()->NotifyGraphNeedsRecompile();
			GetSystemViewModel()->ResetSystem();
		}

		ReportScriptVersionChange();
	});
}

void UNiagaraStackFunctionInput::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	AliasedInputParameterHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(InputParameterHandle, OwningFunctionCallNode.Get());
	RapidIterationParameter = CreateRapidIterationVariable(AliasedInputParameterHandle.GetParameterHandleString());

	RefreshFromMetaData(NewIssues);
	RefreshValues();

	// we check for local ints acting as enums whether the local value is allowed with the current metadata
	// it can happen if a module version changes or a module in use is modified to lose options that were in use
	if(InputMetaData.IsSet() && InputMetaData->WidgetCustomization.WidgetType == ENiagaraInputWidgetType::EnumStyle && GetInputType() == FNiagaraTypeDefinition::GetIntDef() && GetValueMode() == EValueMode::Local)
	{
		int32 LocalValue = *((int32*) GetLocalValueStruct()->GetStructMemory());
		if(LocalValue < 0 || LocalValue >= InputMetaData->WidgetCustomization.EnumStyleDropdownValues.Num())
		{
			NewIssues.Add(FStackIssue(
			EStackIssueSeverity::Error,
			LOCTEXT("IntegerAsEnumOutsideOfRangeShort", "Invalid value"),
			LOCTEXT("IntegerAsEnumOutsideOfRangeLong", "The chosen value is not valid. Please choose a new valid entry."),
			GetStackEditorDataKey(),
			false));
		}
	}
	
	if (InputValues.DynamicNode.IsValid())
	{
		FNiagaraStackGraphUtilities::CheckForDeprecatedScriptVersion(GetDynamicInputNode(), GetStackEditorDataKey(), GetUpgradeDynamicInputVersionFix(), NewIssues);
		if (MessageManagerRegistrationKey.IsValid() == false)
		{
			FNiagaraMessageManager::Get()->SubscribeToAssetMessagesByObject(
				  DisplayName
				, MessageLogGuid
				, FObjectKey(InputValues.DynamicNode.Get())
				, MessageManagerRegistrationKey
			).BindUObject(this, &UNiagaraStackFunctionInput::OnMessageManagerRefresh);
		}
	}
	else
	{
		if (MessageManagerRegistrationKey.IsValid())
		{
			FNiagaraMessageManager::Get()->Unsubscribe(DisplayName, MessageLogGuid, MessageManagerRegistrationKey);
		}
	}

	if (GetShouldPassFilterForVisibleCondition() && InputValues.Mode == EValueMode::InvalidOverride && InputType.IsDataInterface())
	{
		NewIssues.Add(FStackIssue(
            EStackIssueSeverity::Warning,
            LOCTEXT("InvalidDataInterfaceOverrideShort", "Invalid data interface override"),
            LOCTEXT("InvalidDataInterfaceOverrideLong", "There is no valid value assigned for the input, because data interface inputs are created without a binding. Please link a valid reference from the stack or hit 'Fix issue' to populate the binding with a default value."),
            GetStackEditorDataKey(),
            false,
            { FStackIssueFix(
                LOCTEXT("ResetDataInterfaceInputFix", "Reset this input to its default value"),
                FStackIssueFixDelegate::CreateLambda([this]() { this->Reset(); }))
            }
        ));
	}
	
	if (InputValues.Mode == EValueMode::Expression && SupportsCustomExpressions() == false)
	{
		NewIssues.Add(FStackIssue(
			EStackIssueSeverity::Error,
			LOCTEXT("UnsupportedExpressionShort", "Expression Input Unsupported"),
			LOCTEXT("UnsupportedExpressionLong", "Use of expressions for function inputs is not currently supported in the current editor context."),
			GetStackEditorDataKey(),
			false));
	}

	if (InputValues.Mode == EValueMode::Dynamic && InputValues.DynamicNode.IsValid())
	{
		FVersionedNiagaraScriptData* ScriptData = InputValues.DynamicNode->GetScriptData();
		if (ScriptData != nullptr)
		{
			UNiagaraStackScriptHierarchyRoot* DynamicInputEntry = FindCurrentChildOfTypeByPredicate<UNiagaraStackScriptHierarchyRoot>(CurrentChildren,
				[this](UNiagaraStackScriptHierarchyRoot* CurrentFunctionInputEntry)
			{
				return &CurrentFunctionInputEntry->GetOwningFunctionCallNode() == InputValues.DynamicNode.Get() &&
					&CurrentFunctionInputEntry->GetOwningModuleNode() == OwningModuleNode.Get();
			});

			if (DynamicInputEntry == nullptr)
			{
				DynamicInputEntry = NewObject<UNiagaraStackScriptHierarchyRoot>(this);
				DynamicInputEntry->Initialize(CreateDefaultChildRequiredData(), *OwningModuleNode, *InputValues.DynamicNode.Get(), GetOwnerStackItemEditorDataKey());
				DynamicInputEntry->SetShouldDisplayLabel(false);
			}

			if (ScriptData != nullptr)
			{
				if (ScriptData->bDeprecated)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("ScriptName"), FText::FromString(InputValues.DynamicNode->GetFunctionName()));

					if (ScriptData->DeprecationRecommendation != nullptr)
					{
						Args.Add(TEXT("Recommendation"), FText::FromString(ScriptData->DeprecationRecommendation->GetPathName()));
					}

					if (ScriptData->DeprecationMessage.IsEmptyOrWhitespace() == false)
					{
						Args.Add(TEXT("Message"), ScriptData->DeprecationMessage);
					}

					FText FormatString = LOCTEXT("DynamicInputScriptDeprecationUnknownLong", "The script asset for the assigned dynamic input {ScriptName} has been deprecated.");

					if (ScriptData->DeprecationRecommendation != nullptr &&
						ScriptData->DeprecationMessage.IsEmptyOrWhitespace() == false)
					{
						FormatString = LOCTEXT("DynamicInputScriptDeprecationMessageAndRecommendationLong", "The script asset for the assigned dynamic input {ScriptName} has been deprecated. Reason:\n{Message}.\nSuggested replacement: {Recommendation}");
					}
					else if (ScriptData->DeprecationRecommendation != nullptr)
					{
						FormatString = LOCTEXT("DynamicInputScriptDeprecationLong", "The script asset for the assigned dynamic input {ScriptName} has been deprecated. Suggested replacement: {Recommendation}");
					}
					else if (ScriptData->DeprecationMessage.IsEmptyOrWhitespace() == false)
					{
						FormatString = LOCTEXT("DynamicInputScriptDeprecationMessageLong", "The script asset for the assigned dynamic input {ScriptName} has been deprecated. Reason:\n{Message}");
					}

					FText LongMessage = FText::Format(FormatString, Args);

					int32 AddIdx = NewIssues.Add(FStackIssue(
						EStackIssueSeverity::Warning,
						LOCTEXT("DynamicInputScriptDeprecationShort", "Deprecated dynamic input"),
						LongMessage,
						GetStackEditorDataKey(),
						false,
						{
							FStackIssueFix(
								LOCTEXT("SelectNewDynamicInputScriptFix", "Select a new dynamic input script"),
								FStackIssueFixDelegate::CreateLambda([this]() { this->bIsDynamicInputScriptReassignmentPending = true; })),
							FStackIssueFix(
								LOCTEXT("ResetDynamicInputFix", "Reset this input to its default value"),
								FStackIssueFixDelegate::CreateLambda([this]() { this->Reset(); }))
						}));

					if (ScriptData->DeprecationRecommendation != nullptr)
					{
						NewIssues[AddIdx].InsertFix(0,
							FStackIssueFix(
							LOCTEXT("SelectNewDynamicInputScriptFixUseRecommended", "Use recommended replacement"),
							FStackIssueFixDelegate::CreateLambda([this, ScriptData]() 
								{ 
									if (ScriptData->DeprecationRecommendation->GetUsage() != ENiagaraScriptUsage::DynamicInput)
									{
										FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("FailedDynamicInputDeprecationReplacement", "Failed to replace dynamic input as recommended replacement script is not a dynamic input!"));
										return;
									}
									ReassignDynamicInputScript(ScriptData->DeprecationRecommendation); 
								})));
					}
				}

				if (ScriptData->bExperimental)
				{
					FText ErrorMessage;
					if (ScriptData->ExperimentalMessage.IsEmptyOrWhitespace())
					{
						ErrorMessage = FText::Format(LOCTEXT("DynamicInputScriptExperimental", "The script asset for the dynamic input {0} is experimental, use with care!"), FText::FromString(InputValues.DynamicNode->GetFunctionName()));
					}
					else
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("Function"), FText::FromString(InputValues.DynamicNode->GetFunctionName()));
						Args.Add(TEXT("Message"), ScriptData->ExperimentalMessage);
						ErrorMessage = FText::Format(LOCTEXT("DynamicInputScriptExperimentalReason", "The script asset for the dynamic input {Function} is experimental, reason: {Message}"), Args);
					}

					NewIssues.Add(FStackIssue(
						EStackIssueSeverity::Info,
						LOCTEXT("DynamicInputScriptExperimentalShort", "Experimental dynamic input"),
						ErrorMessage,
						GetStackEditorDataKey(),
						true));
				}

				const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
				if (InputValues.DynamicNode->FunctionScript != nullptr &&
					NiagaraEditorSettings->IsAllowedAssetObjectByClassUsage(*InputValues.DynamicNode->FunctionScript) == false)
				{
					NewIssues.Add(FStackIssue(
						EStackIssueSeverity::Error,
						LOCTEXT("UnsupportedDynamicInputScriptShort", "Unsupported Dynamic Input Script"),
						LOCTEXT("UnsupportedDynamicInputScriptLong", "This dynamic input uses a script which uses types which are unsupported in this editor context.  This dynamic input must either be deleted, or the referenced script must be fixed."),
						GetStackEditorDataKey(),
						false));
				}

				if (!ScriptData->NoteMessage.IsEmptyOrWhitespace())
				{
					FStackIssue NoteIssue = FStackIssue(
						EStackIssueSeverity::Info,
						LOCTEXT("DynamicInputScriptNoteShort", "Input Usage Note"),
						ScriptData->NoteMessage,
						GetStackEditorDataKey(),
						true);
					NoteIssue.SetIsExpandedByDefault(false);
					NewIssues.Add(NoteIssue);
				}
			}
			NewChildren.Add(DynamicInputEntry);
		}
		else
		{
			NewIssues.Add(FStackIssue(
				EStackIssueSeverity::Error,
				LOCTEXT("DynamicInputScriptMissingShort", "Missing dynamic input script"),
				FText::Format(LOCTEXT("DynamicInputScriptMissingLong", "The script asset for the assigned dynamic input {0} is missing."), FText::FromString(InputValues.DynamicNode->GetFunctionName())),
				GetStackEditorDataKey(),
				false,
				{
					FStackIssueFix(
						LOCTEXT("SelectNewDynamicInputScriptFix", "Select a new dynamic input script"),
						FStackIssueFixDelegate::CreateLambda([this]() { this->bIsDynamicInputScriptReassignmentPending = true; })),
					FStackIssueFix(
						LOCTEXT("ResetFix", "Reset this input to its default value"),
						FStackIssueFixDelegate::CreateLambda([this]() { this->Reset(); }))
				}));
		}
	}

	if (InputValues.Mode == EValueMode::Data && InputValues.DataObject.IsValid())
	{
		UNiagaraStackObject* ValueObjectEntry = FindCurrentChildOfTypeByPredicate<UNiagaraStackObject>(CurrentChildren,
			[this](UNiagaraStackObject* CurrentObjectEntry) { return CurrentObjectEntry->GetObject() == InputValues.DataObject.Get(); });

		if(ValueObjectEntry == nullptr)
		{
			ValueObjectEntry = NewObject<UNiagaraStackObject>(this);
			bool bIsTopLevelObject = false;
			bool bHideTopLevelCategories = false;
			ValueObjectEntry->Initialize(CreateDefaultChildRequiredData(), InputValues.DataObject.Get(), bIsTopLevelObject, bHideTopLevelCategories, GetOwnerStackItemEditorDataKey(), OwningFunctionCallNode.Get());
		}
		NewChildren.Add(ValueObjectEntry);
	}

	if (InputValues.Mode == EValueMode::LinkedShared && InputValues.SharedSourceAssignmentNode.IsValid())
	{
		NewChildren.Add(GetSharedLinkedValueInput());
	}
	else
	{
		SharedLinkedInput = nullptr;
	}

	DisplayNameOverride.Reset();

	if (InputMetaData)
	{
		const FString* FoundDisplayName = InputMetaData->PropertyMetaData.Find(TEXT("DisplayName"));
		const FString* FoundDisplayNameArg0 = InputMetaData->PropertyMetaData.Find(TEXT("DisplayNameArg0"));
		if (FoundDisplayName != nullptr)
		{
			FString DisplayNameStr = *FoundDisplayName;
			if (FoundDisplayNameArg0 != nullptr)
			{
				TArray<FStringFormatArg> Args;
				Args.Add(FStringFormatArg(ResolveDisplayNameArgument(*FoundDisplayNameArg0)));
				DisplayNameStr = FString::Format(*DisplayNameStr, Args);
			}
			DisplayNameOverride = FText::FromString(DisplayNameStr);
		}	
	}

	if(HierarchyScriptParameter.IsValid())
	{
		TArray<UNiagaraHierarchyScriptParameter*> ChildScriptParameters;
		HierarchyScriptParameter->GetChildrenOfType(ChildScriptParameters);
		
		for(UNiagaraHierarchyScriptParameter* ChildScriptParameter : ChildScriptParameters)
		{
			TOptional<FNiagaraVariable> InputVariableCandidate = ChildScriptParameter->GetVariable();

			if(InputVariableCandidate.IsSet() == false)
			{
				continue;
			}

			FNiagaraVariable InputVariable = InputVariableCandidate.GetValue();
			
			if(ScriptInstanceData.UsedInputs.Contains(InputVariable) == false)
			{
				continue;
			}
			
			UNiagaraStackFunctionInput* InputChild = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
			{ 
				return CurrentInput->GetInputParameterHandle() == FNiagaraParameterHandle(InputVariable.GetName()) && CurrentInput->GetInputType() == InputVariable.GetType() && &CurrentInput->GetInputFunctionCallNode() == OwningFunctionCallNode;
			});

			if (InputChild == nullptr)
			{
				EStackParameterBehavior Behavior = ChildScriptParameter->GetScriptVariable()->GetIsStaticSwitch() ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic;
				InputChild = NewObject<UNiagaraStackFunctionInput>(this);
				InputChild->Initialize(CreateDefaultChildRequiredData(), *OwningModuleNode, *OwningFunctionCallNode,
					InputVariable.GetName(), InputVariable.GetType(), Behavior, GetOwnerStackItemEditorDataKey());
			}
			
			InputChild->SetScriptInstanceData(ScriptInstanceData);
			FGuid VariableGuid = ChildScriptParameter->GetScriptVariable()->Metadata.GetVariableGuid();
			// Would be nice if eventually inputs could easily access the cache so that they don't rely on the parent pushing data in to determine visibility
			// If we let inputs use GetStackFunctionInputs & GetStackFunctionStaticSwitchPins there is a noticeable perf decrease, due to the static switch as they are less cached
			InputChild->SetIsHidden(ScriptInstanceData.PerInputInstanceData[VariableGuid].bIsHidden);
				
			NewChildren.Add(InputChild);	
		}
	}
	
	if (GetIsHidden())
	{
		// Hidden inputs should not generate issues because they are impossible for the user to see.
		NewIssues.Empty();
	}
	else
	{
		NewIssues.Append(MessageManagerIssues);
	}

	FGuid CurrentOwningGraphChangeId;
	FGuid CurrentFunctionGraphChangeId;
	GetCurrentChangeIds(CurrentOwningGraphChangeId, CurrentFunctionGraphChangeId);
	LastOwningGraphChangeId = CurrentOwningGraphChangeId;
	LastFunctionGraphChangeId = CurrentFunctionGraphChangeId;
}

FString UNiagaraStackFunctionInput::ResolveDisplayNameArgument(const FString& InArg) const
{
	//~ Begin helper functions
	auto GetMaterialsFromEmitter = [](const FVersionedNiagaraEmitter& InEmitter, const FNiagaraEmitterInstance* InEmitterInstance)->TArray<UMaterial*> {
		TArray<UMaterial*> ResultMaterials;
		if (InEmitter.Emitter != nullptr)
		{
			for (UNiagaraRendererProperties* RenderProperties : InEmitter.GetEmitterData()->GetRenderers())
			{
				TArray<UMaterialInterface*> UsedMaterialInteraces;
				RenderProperties->GetUsedMaterials(InEmitterInstance, UsedMaterialInteraces);
				for (UMaterialInterface* UsedMaterialInterface : UsedMaterialInteraces)
				{
					if (UsedMaterialInterface != nullptr)
					{
						UMaterial* UsedMaterial = UsedMaterialInterface->GetBaseMaterial();
						if (UsedMaterial != nullptr)
						{
							ResultMaterials.AddUnique(UsedMaterial);
							break;
						}
					}
				}
			}
		}
		return ResultMaterials;
	};

	auto GetChannelUsedBitMask =
		[](FExpressionInput* Input, TStaticBitArray<4>& ChannelUsedMask)
		{
			if (Input->Expression)
			{
				TArray<FExpressionOutput>& Outputs = Input->Expression->GetOutputs();

				if (Outputs.Num() > 0)
				{
					const bool bOutputIndexIsValid = Outputs.IsValidIndex(Input->OutputIndex)
						// Attempt to handle legacy connections before OutputIndex was used that had a mask
						&& (Input->OutputIndex != 0 || Input->Mask == 0);

					for (int32 OutputIndex=0; OutputIndex < Outputs.Num(); ++OutputIndex)
					{
						const FExpressionOutput& Output = Outputs[OutputIndex];

						if ((bOutputIndexIsValid && OutputIndex == Input->OutputIndex)
							|| (!bOutputIndexIsValid
								&& Output.Mask == Input->Mask
								&& Output.MaskR == Input->MaskR
								&& Output.MaskG == Input->MaskG
								&& Output.MaskB == Input->MaskB
								&& Output.MaskA == Input->MaskA))
						{
							ChannelUsedMask[0] = ChannelUsedMask[0] || (Input->MaskR != 0);
							ChannelUsedMask[1] = ChannelUsedMask[1] || (Input->MaskG != 0);
							ChannelUsedMask[2] = ChannelUsedMask[2] || (Input->MaskB != 0);
							ChannelUsedMask[3] = ChannelUsedMask[3] || (Input->MaskA != 0);
							return;
						}
					}
				}
			}
		};
	//~ End helper functions

	// If the DisplayNameArgument to resolve is not a MaterialDynamicParam, early out.
	if (InArg.StartsWith(TEXT("MaterialDynamicParam")) == false)
	{
		return FString();
	}

	// Get the target indices of the MaterialDynamicParam. Early out if they are invalid.
	FString Suffix = InArg.Right(3);
	int32 ParamIdx;
	LexFromString(ParamIdx, *Suffix.Left(1));
	int32 ParamSlotIdx;
	LexFromString(ParamSlotIdx, *Suffix.Right(1));

	if (ParamIdx < 0 || ParamIdx > 3 || ParamSlotIdx < 0 || ParamSlotIdx > 3)
	{
		return InArg.Replace(TEXT("MaterialDynamic"), TEXT("")) + TEXT(" (error parsing parameter name)");
	}

	TArray<UMaterial*> Materials;
	TSharedPtr<FNiagaraEmitterViewModel> ThisEmitterViewModel = GetEmitterViewModel();
	if (ThisEmitterViewModel.IsValid())
	{
		FNiagaraEmitterInstance* ThisEmitterInstance = ThisEmitterViewModel->GetSimulation().Pin().Get();
		Materials = GetMaterialsFromEmitter(ThisEmitterViewModel->GetEmitter(), ThisEmitterInstance);
	} 

	// Determine the MaterialDynamicParam name to display inline based on the friendly names for each UMaterialExpressionDynamicParameter in the graph, and whether each UMaterialExpressionDynamicParameter pin is linked.
	// For each graph; traverse every expression, and for each expression check ExpressionInputs; if an input of ExpressionInputs is recording a link to a UMaterialExpressionDynamicParameter, record this in DynamicParameterExpressionToOutputMaskMap.
	// After recording all UMaterialExpressionDynamicParameter output pins that are linked at least once, iterate DynamicParameterExpressionToOutputMaskMap to determine the final name.
	// NOTE: This check does not constitute a true "reachability analysis" as we are only recording if each pin of the UMaterialExpressionDynamicParameter is linked, and not whether that pin is connected to a route of expressions that would eventually output.
	TMap<UMaterialExpressionDynamicParameter*, TStaticBitArray<4>> DynamicParameterExpressionToOutputMaskMap;
	TArray<FExpressionInput*> ExpressionInputsToProcess;

	// Visit each material and gather all expression inputs for each expression.
	for (UMaterial* Material : Materials)
	{
		if (Material == nullptr)
		{
			continue;
		}

		for (int32 MaterialPropertyIndex = 0; MaterialPropertyIndex < MP_MAX; ++MaterialPropertyIndex)
		{
			FExpressionInput* ExpressionInput = Material->GetExpressionInputForProperty(EMaterialProperty(MaterialPropertyIndex));
			if (ExpressionInput != nullptr)
			{
				ExpressionInputsToProcess.Add(ExpressionInput);
			}
		}

		TArray<UMaterialExpression*> Expressions;
		Material->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpression>(Expressions);
		for (UMaterialExpression* Expression : Expressions)
		{
			for (FExpressionInputIterator It{ Expression}; It; ++It)
			{
				ExpressionInputsToProcess.Push(It.Input);
			}
		}
	}

	// Visit each expression input and record which inputs are associated with UMaterialExpressionDynamicParameter outputs.
	bool bAnyDynamicParametersFound = false;
	for (FExpressionInput* ExpressionInput : ExpressionInputsToProcess)
	{
		if (ExpressionInput->Expression == nullptr)
		{
			continue;
		}

		UMaterialExpressionDynamicParameter* DynamicParameterExpression = Cast<UMaterialExpressionDynamicParameter>(ExpressionInput->Expression);
		if (DynamicParameterExpression == nullptr)
		{
			continue;
		}

		bAnyDynamicParametersFound = true;
		GetChannelUsedBitMask(ExpressionInput, DynamicParameterExpressionToOutputMaskMap.FindOrAdd(DynamicParameterExpression));
	}

	// Construct the final dynamic param UI name. Visit each UMaterialExpressionDynamicParameter and for those which have an output which is used, consider them for the param name.
	FName ParamName = NAME_None;
	bool bMultipleAliasesFound = false;
	const FString DefaultDynamicParameterNameString = "Param" + FString::Printf(TEXT("%d"), ParamIdx + 1);
	for (auto It = DynamicParameterExpressionToOutputMaskMap.CreateConstIterator(); It; ++It)
	{
		UMaterialExpressionDynamicParameter* ExpressionDynamicParameter = It.Key();
		const TStaticBitArray<4>& ExpressionOutputMask = It.Value();
		if (ExpressionDynamicParameter->ParameterIndex != ParamSlotIdx || ExpressionOutputMask[ParamIdx] == false)
		{
			continue;
		}

		const FExpressionOutput& Output = ExpressionDynamicParameter->GetOutputs()[ParamIdx];
		if (ParamName == NAME_None)
		{
			ParamName = Output.OutputName;
		}
		else if (ParamName != Output.OutputName)
		{
			bMultipleAliasesFound = true;
		}
	}

	// Return the final dynamic param UI name.
	if (bAnyDynamicParametersFound == false)
	{
		return InArg.Replace(TEXT("MaterialDynamic"), TEXT("")) + TEXT(" (No material found using dynamic params)");
	}
	else if (ParamName != NAME_None)
	{
		if (bMultipleAliasesFound == false)
		{
			return ParamName.ToString();
		}
		else
		{
			return ParamName.ToString() + TEXT(" (Multiple Aliases Found)");
		}
	}

	return InArg.Replace(TEXT("MaterialDynamic"), TEXT("")) + TEXT(" (Parameter not used in materials.)");
}

TOptional<float> UNiagaraStackFunctionInput::GetMaterialDynamicParamDefaultValue() const
{
	if (!InputMetaData.IsSet())
	{
		return TOptional<float>();
	}

	const FString* FoundDisplayNameArg0 = InputMetaData->PropertyMetaData.Find(TEXT("DisplayNameArg0"));
	if (FoundDisplayNameArg0 == nullptr || !FoundDisplayNameArg0->StartsWith(TEXT("MaterialDynamicParam")))
	{
		return TOptional<float>();
	}

	// Parse ParamIdx (channel: 0-3) and ParamSlotIdx (parameter index: 0-3) from the suffix.
	// The format is MaterialDynamicParam{ParamIdx}{X}{ParamSlotIdx} where X is an ignored middle char.
	const FString& InArg = *FoundDisplayNameArg0;
	FString Suffix = InArg.Right(3);
	int32 ParamIdx;
	LexFromString(ParamIdx, *Suffix.Left(1));
	int32 ParamSlotIdx;
	LexFromString(ParamSlotIdx, *Suffix.Right(1));

	if (ParamIdx < 0 || ParamIdx > 3 || ParamSlotIdx < 0 || ParamSlotIdx > 3)
	{
		return TOptional<float>();
	}

	TArray<UMaterial*> Materials;
	TSharedPtr<FNiagaraEmitterViewModel> ThisEmitterViewModel = GetEmitterViewModel();
	if (ThisEmitterViewModel.IsValid())
	{
		FNiagaraEmitterInstance* ThisEmitterInstance = ThisEmitterViewModel->GetSimulation().Pin().Get();
		if (ThisEmitterViewModel->GetEmitter().Emitter != nullptr)
		{
			Materials = NiagaraDynamicMaterialParameterUtils::GetMaterialsFromRenderers(
				ThisEmitterViewModel->GetEmitter().GetEmitterData()->GetRenderers(), ThisEmitterInstance);
		}
	}

	return NiagaraDynamicMaterialParameterUtils::GetDynamicParameterDefaultValue(Materials, ParamSlotIdx, ParamIdx);
}

void UNiagaraStackFunctionInput::RefreshValues()
{
	if (ensureMsgf(IsStaticParameter() || InputParameterHandle.IsModuleHandle(), TEXT("Function inputs can only be generated for module parameters.")) == false)
	{
		return;
	}

	FGuid CurrentOwningGraphChangeId;
	FGuid CurrentFunctionGraphChangeId;
	GetCurrentChangeIds(CurrentOwningGraphChangeId, CurrentFunctionGraphChangeId);
	if (LastOwningGraphChangeId.IsSet() == false || (CurrentOwningGraphChangeId != LastOwningGraphChangeId.GetValue()) ||
		LastFunctionGraphChangeId.IsSet() == false || (CurrentFunctionGraphChangeId != LastFunctionGraphChangeId.GetValue()))
	{
		// First collect the default values which are used to figure out if an input can be reset, and are used to
		// determine the current displayed value.
		DefaultInputValues = FInputValues();
		UpdateValuesFromScriptDefaults(DefaultInputValues);
	}

	FInputValues OldValues = InputValues;
	InputValues = FInputValues();

	// If there is an override pin available it's value will take precedence so check that first.
	UEdGraphPin* OverridePin = GetOverridePin();
	if (OverridePin != nullptr)
	{
		UpdateValuesFromOverridePin(OldValues, InputValues, *OverridePin);
		if (InputValues.Mode == EValueMode::Data)
		{
			FGuid EmitterHandleId = GetEmitterViewModel().IsValid()
				? FNiagaraEditorUtilities::GetEmitterHandleForEmitter(GetSystemViewModel()->GetSystem(), GetEmitterViewModel()->GetEmitter())->GetId()
				: FGuid();
			PlaceholderDataInterfaceHandle = GetSystemViewModel()->GetPlaceholderDataInterfaceManager()->GetPlaceholderDataInterface(EmitterHandleId, *OwningFunctionCallNode.Get(), InputParameterHandle);
			if (PlaceholderDataInterfaceHandle.IsValid())
			{
				// If there is an active placeholder data interface, display and edit it to keep other views consistent.  Changes to it will be copied to the target data interface
				// by the placeholder manager.
				InputValues.DataObject = PlaceholderDataInterfaceHandle->GetDataInterface();
			}
		}
		else if (InputValues.Mode == EValueMode::ObjectAsset)
		{
			InputValues.ObjectAssetInputNode = DefaultInputValues.ObjectAssetInputNode;
		}
	}
	else
	{
		if (InputType.IsDataInterface() && DefaultInputValues.Mode == EValueMode::Data)
		{
			// If the input it a data interface but hasn't been edited yet, we need to provide a placeholder data interface to edit.
			FGuid EmitterHandleId = GetEmitterViewModel().IsValid()
				? FNiagaraEditorUtilities::GetEmitterHandleForEmitter(GetSystemViewModel()->GetSystem(), GetEmitterViewModel()->GetEmitter())->GetId()
				: FGuid();
			PlaceholderDataInterfaceHandle = GetSystemViewModel()->GetPlaceholderDataInterfaceManager()->GetOrCreatePlaceholderDataInterface(EmitterHandleId, *OwningFunctionCallNode.Get(), InputParameterHandle, InputType.GetClass());
			InputValues.Mode = EValueMode::Data;
			InputValues.DataObject = PlaceholderDataInterfaceHandle->GetDataInterface();
			if (DefaultInputValues.DataObject.IsValid() && InputValues.DataObject->Equals(DefaultInputValues.DataObject.Get()) == false)
			{
				DefaultInputValues.DataObject->CopyTo(InputValues.DataObject.Get());
			}
		}
		else if (InputType.IsUObject() && DefaultInputValues.Mode == EValueMode::ObjectAsset)
		{
			InputValues.Mode = EValueMode::ObjectAsset;
			InputValues.ObjectAssetInputNode = DefaultInputValues.ObjectAssetInputNode;
		}
		else if (IsRapidIterationCandidate())
		{
			// If the value is a rapid iteration parameter it's a local value so copy it's value from the rapid iteration parameter store if it's in there,
			// otherwise copy the value from the default.
			InputValues.Mode = EValueMode::Local;
			InputValues.LocalStruct = MakeShared<FStructOnScope>(InputType.GetStruct());

			uint8* DestinationData = InputValues.LocalStruct->GetStructMemory();
			if (!SourceScript->RapidIterationParameters.CopyParameterData(RapidIterationParameter, DestinationData))
			{
				if (InputType.GetSize() > 0 && DestinationData && DefaultInputValues.LocalStruct.IsValid())
				{
					FMemory::Memcpy(DestinationData, DefaultInputValues.LocalStruct->GetStructMemory(), InputType.GetSize());
				}
				else
				{
					UE_LOGF(LogNiagaraEditor, Warning, "Type %ls has no data! Cannot refresh values.", *InputType.GetName())
				}
			}
			
			// we check if variable guid is already available in the parameter store and update it if that's not the case
			if (InputMetaData.IsSet())
			{
				SourceScript->RapidIterationParameters.ParameterGuidMapping.Add(RapidIterationParameter, InputMetaData->GetVariableGuid());
			}
		}
		else
		{
			// Otherwise if there isn't an override pin and it's not a rapid iteration parameter use the default value.
			InputValues = DefaultInputValues;
		}
	}

	bCanResetCache.Reset();
	bCanResetToBaseCache.Reset();
	ValueToolTipCache.Reset();
	bIsScratchDynamicInputCache.Reset();
	CollapsedTextCache.Reset();
	ValueChangedDelegate.Broadcast();
}

void UNiagaraStackFunctionInput::RefreshFromMetaData(TArray<FStackIssue>& NewIssues)
{
	FGuid CurrentOwningGraphChangeId;
	FGuid CurrentFunctionGraphChangeId;
	GetCurrentChangeIds(CurrentOwningGraphChangeId, CurrentFunctionGraphChangeId);
	if (LastOwningGraphChangeId.IsSet() && CurrentOwningGraphChangeId == LastOwningGraphChangeId &&
		LastFunctionGraphChangeId.IsSet() && CurrentFunctionGraphChangeId == LastFunctionGraphChangeId)
	{
		// If the called function graph hasn't changed, then the metadata will also be the same and we can skip updating these values.
		NewIssues.Append(InputMetaDataIssues);
		return;
	}

	InputMetaData.Reset();
	InputMetaDataIssues.Empty();
	if (OwningFunctionCallNode->IsA<UNiagaraNodeAssignment>())
	{
		// Set variables nodes have no metadata, but if they're setting a defined constant see if there's metadata for that.
		FNiagaraVariable InputVariable(InputType, InputParameterHandle.GetName());
		if (FNiagaraConstants::IsNiagaraConstant(InputVariable))
		{
			const FNiagaraVariableMetaData* FoundMetaData = FNiagaraConstants::GetConstantMetaData(InputVariable);
			if (FoundMetaData)
			{
				InputMetaData = *FoundMetaData;
			}
		}
		else if (UNiagaraScriptVariable* ScriptVariable = GetSystemViewModel()->FindSubscribedParameterDefinitionsScriptVarByName(InputParameterHandle.GetName()))
		{
			InputMetaData = ScriptVariable->Metadata;
		}
	}
	else if (OwningFunctionCallNode->FunctionScript != nullptr)
	{
		// Otherwise just get it from the defining graph.
		UNiagaraGraph* FunctionGraph = CastChecked<UNiagaraScriptSource>(OwningFunctionCallNode->GetFunctionScriptSource())->NodeGraph;
		FNiagaraVariable InputVariable(InputType, InputParameterHandle.GetParameterHandleString());
		InputMetaData = FunctionGraph->GetMetaData(InputVariable);
	}

	if (InputMetaData.IsSet())
	{
		SetIsAdvanced(InputMetaData->bAdvancedDisplay);

		FText EditConditionError;
		EditCondition.Refresh(InputMetaData->EditCondition, EditConditionError);
		if (EditCondition.IsValid() && EditCondition.GetConditionInputType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()))
		{
			TOptional<FNiagaraVariableMetaData> EditConditionInputMetadata = EditCondition.GetConditionInputMetaData();
			if (EditConditionInputMetadata.IsSet())
			{
				bShowEditConditionInline = EditConditionInputMetadata->bInlineEditConditionToggle;
			}
		}
		else
		{
			bShowEditConditionInline = false;
		}

		if (EditConditionError.IsEmpty() == false)
		{
			InputMetaDataIssues.Add(FStackIssue(
				EStackIssueSeverity::Info,
				LOCTEXT("EditConditionErrorShort", "Edit condition error"),
				FText::Format(LOCTEXT("EditConditionErrorLongFormat", "Edit condition failed to bind.  Function: {0} Input: {1} Message: {2}"), 
					OwningFunctionCallNode->GetNodeTitle(ENodeTitleType::ListView), 
					FText::FromName(InputParameterHandle.GetName()),
					EditConditionError),
				GetStackEditorDataKey(),
				true));
		}

		FText VisibleConditionError;
		VisibleCondition.Refresh(InputMetaData->VisibleCondition, VisibleConditionError);

		if (VisibleConditionError.IsEmpty() == false)
		{
			InputMetaDataIssues.Add(FStackIssue(
				EStackIssueSeverity::Info,
				LOCTEXT("VisibleConditionErrorShort", "Visible condition error"),
				FText::Format(LOCTEXT("VisibleConditionErrorLongFormat", "Visible condition failed to bind.  Function: {0} Input: {1} Message: {2}"),
					OwningFunctionCallNode->GetNodeTitle(ENodeTitleType::ListView),
					FText::FromName(InputParameterHandle.GetName()),
					VisibleConditionError),
				GetStackEditorDataKey(),
				true));
		}

		bIsInlineEditConditionToggle = InputType.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()) && 
			InputMetaData->bInlineEditConditionToggle;
	}

	NewIssues.Append(InputMetaDataIssues);
}

FText UNiagaraStackFunctionInput::GetDisplayName() const
{
	return SummaryViewDisplayNameOverride.IsSet() && !SummaryViewDisplayNameOverride->Get().IsEmptyOrWhitespace() ? SummaryViewDisplayNameOverride->Get() : DisplayNameOverride.IsSet() ? DisplayNameOverride.GetValue() : DisplayName;
}

const TArray<FNiagaraParameterHandle>& UNiagaraStackFunctionInput::GetInputParameterHandlePath() const
{
	return InputParameterHandlePath;
}

const FNiagaraParameterHandle& UNiagaraStackFunctionInput::GetInputParameterHandle() const
{
	return InputParameterHandle;
}

const FNiagaraVariableBase& UNiagaraStackFunctionInput::GetLinkedParameterValue() const
{
	return InputValues.LinkedParameter;
}

void UNiagaraStackFunctionInput::SetLinkedParameterValue(const FNiagaraVariableBase& InParameter)
{
	if (InParameter == InputValues.LinkedParameter)
	{
		return;
	}

	TGuardValue<bool> UpdateGuard(bUpdatingLocalValueDirectly, true);
	FScopedTransaction ScopedTransaction(LOCTEXT("UpdateLinkedInputValue", "Update linked input value"));
	RemoveOverridePin();

	if (IsRapidIterationCandidate())
	{
		RemoveRapidIterationParametersForAffectedScripts();
	}

	if (InParameter != DefaultInputValues.LinkedParameter)
	{
		FNiagaraParameterHandle ParameterHandle = FNiagaraParameterHandle(InParameter.GetName());
		if (ParameterHandle.IsUserHandle())
		{
			// If the handle is a user parameter, make sure the system has it exposed.  If it's not exposed add it directly here
			// rather than waiting on the compile results so that it's immediately available.
			FNiagaraParameterStore& UserParameters = GetSystemViewModel()->GetSystem().GetExposedParameters();

			// we only know the input type, and the parameter handle. We don't know the assigned parameter's real type (a vector parameter can be linked to a position input).
			// therefore, assuming we can FindOrAdd a parameter based on just input type & handle is not possible.
			// We special case the position<->vector assignment and skip the creation of the parameter if one of the same name but the other type already exists.
			bool bSkipCreation = false;
			if(InputType == FNiagaraTypeDefinition::GetPositionDef())
			{
				FNiagaraVariable ExistingVectorParameter(FNiagaraTypeDefinition::GetVec3Def(), InParameter.GetName());
				if(UserParameters.IndexOf(ExistingVectorParameter) != INDEX_NONE)
				{
					bSkipCreation = true;
				}
			}
			else if(InputType == FNiagaraTypeDefinition::GetVec3Def())
			{
				FNiagaraVariable ExistingPositionParameter(FNiagaraTypeDefinition::GetPositionDef(), InParameter.GetName());
				if(UserParameters.IndexOf(ExistingPositionParameter) != INDEX_NONE)
				{
					bSkipCreation = true;
				}
			}
			if (UserParameters.IndexOf(InParameter) == INDEX_NONE && bSkipCreation == false)
			{
				if (InputType.IsDataInterface())
				{
					int32 DataInterfaceOffset;
					bool bInitialize = true;
					bool bTriggerRebind = true;
					UserParameters.AddParameter(InParameter, bInitialize, bTriggerRebind, &DataInterfaceOffset);
					if (InputValues.Mode == EValueMode::Data && InputValues.DataObject.IsValid())
					{
						InputValues.DataObject->CopyTo(UserParameters.GetDataInterface(DataInterfaceOffset));
						UserParameters.OnInterfaceChange();
					}
				}
				else if (InputType.IsUObject())
				{
					const bool bInitialize = true;
					const bool bTriggerRebind = true;
					int32 ParameterOffset = INDEX_NONE;
					UserParameters.AddParameter(InParameter, bInitialize, bTriggerRebind, &ParameterOffset);
					if (InputValues.Mode == EValueMode::ObjectAsset)
					{
						UNiagaraNodeInput* InputNode = InputValues.ObjectAssetInputNode.Get();
						UserParameters.SetUObject(InputNode ? InputNode->GetObjectAsset() : nullptr, ParameterOffset);
						UserParameters.OnUObjectChange();
					}
				}
				else
				{
					FNiagaraVariable ParameterWithData = InParameter;
					if (InputValues.Mode == EValueMode::Local && InputValues.LocalStruct.IsValid())
					{
						// If the current value is local, and valid transfer that value to the user parameter.
						ParameterWithData.SetData(InputValues.LocalStruct->GetStructMemory());
					}
					else
					{
						FNiagaraEditorUtilities::ResetVariableToDefaultValue(ParameterWithData);
					}
					UserParameters.SetParameterData(ParameterWithData.GetData(), ParameterWithData, true);
				}
			}
		}

		// Only set the linked value if it's actually different from the default.
		UEdGraphPin& OverridePin = GetOrCreateOverridePin();
		TSet<FNiagaraVariableBase> KnownParameters;
		FNiagaraStackGraphUtilities::GetParametersForContext(OverridePin.GetOwningNode()->GetGraph(), GetSystemViewModel()->GetSystem(), KnownParameters);
		FNiagaraStackGraphUtilities::SetLinkedParameterValueForFunctionInput(OverridePin, InParameter, KnownParameters);
		FGuid LinkedOutputId = FNiagaraStackGraphUtilities::GetScriptVariableIdForLinkedModuleParameter(InParameter, *OwningFunctionCallNode->GetNiagaraGraph());
		if (LinkedOutputId.IsValid())
		{
			OwningFunctionCallNode->UpdateInputNameBinding(LinkedOutputId, InParameter.GetName());
		}
	}

	FNiagaraStackGraphUtilities::RelayoutGraph(*OwningFunctionCallNode->GetGraph());
	RefreshValues();
}

void UNiagaraStackFunctionInput::SetLinkedParameterValueViaConversionScript(const FNiagaraVariableBase& InParameter, UNiagaraScript& ConversionScript)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("SetConversionInput", "Make auto-convert dynamic input"));
	SetDynamicInput(&ConversionScript);
	for (UNiagaraStackFunctionInput* ChildInput : GetChildInputs())
	{
		if (InParameter.GetType() == ChildInput->GetInputType())
		{
			ChildInput->SetLinkedParameterValue(InParameter);
			break;
		}
	}
}

bool UsageRunsBefore(ENiagaraScriptUsage UsageA, ENiagaraScriptUsage UsageB, bool bCheckInterpSpawn, FVersionedNiagaraEmitter InEmitter, FGuid UsageAID, FGuid UsageBID)
{
	static TArray<ENiagaraScriptUsage> UsagesOrderedByExecution
	{
		ENiagaraScriptUsage::SystemSpawnScript,
		ENiagaraScriptUsage::SystemUpdateScript,
		ENiagaraScriptUsage::EmitterSpawnScript,
		ENiagaraScriptUsage::EmitterUpdateScript,
		ENiagaraScriptUsage::ParticleSpawnScript,
		ENiagaraScriptUsage::ParticleEventScript,	// When not using interpolated spawn
		ENiagaraScriptUsage::ParticleUpdateScript,
		ENiagaraScriptUsage::ParticleEventScript,	// When using interpolated spawn and is spawn
		ENiagaraScriptUsage::ParticleSimulationStageScript
	};

	int32 IndexA;
	int32 IndexB;
	if (bCheckInterpSpawn)
	{
		UsagesOrderedByExecution.FindLast(UsageA, IndexA);
		UsagesOrderedByExecution.FindLast(UsageB, IndexB);
	}
	else
	{
		UsagesOrderedByExecution.Find(UsageA, IndexA);
		UsagesOrderedByExecution.Find(UsageB, IndexB);
	}

	if (IndexA == IndexB && UsageA == ENiagaraScriptUsage::ParticleSimulationStageScript && InEmitter.Emitter)
	{
		FVersionedNiagaraEmitterData* EmitterData = InEmitter.GetEmitterData();
		const TArray<UNiagaraSimulationStageBase*> SimStages = EmitterData->GetSimulationStages();
		UNiagaraSimulationStageBase* StageA = EmitterData->GetSimulationStageById(UsageAID);
		UNiagaraSimulationStageBase* StageB = EmitterData->GetSimulationStageById(UsageBID);

		int32 SimIndexA = SimStages.IndexOfByKey(StageA);
		int32 SimIndexB = SimStages.IndexOfByKey(StageB);

		if (SimIndexA < SimIndexB)
			return true;
		else if (SimIndexA == INDEX_NONE && SimIndexB != INDEX_NONE)
			return true;
		else
			return false;
	}
	return IndexA < IndexB;
}

bool IsSpawnUsage(ENiagaraScriptUsage Usage)
{
	return
		Usage == ENiagaraScriptUsage::SystemSpawnScript ||
		Usage == ENiagaraScriptUsage::EmitterSpawnScript ||
		Usage == ENiagaraScriptUsage::ParticleSpawnScript;
}

FName GetNamespaceForUsage(ENiagaraScriptUsage Usage)
{
	switch (Usage)
	{
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
		return FNiagaraConstants::ParticleAttributeNamespace;
	case ENiagaraScriptUsage::EmitterSpawnScript:
	case ENiagaraScriptUsage::EmitterUpdateScript:
		return FNiagaraConstants::EmitterNamespace;
	case ENiagaraScriptUsage::SystemSpawnScript:
	case ENiagaraScriptUsage::SystemUpdateScript:
		return FNiagaraConstants::SystemNamespace;
	default:
		return NAME_None;
	}
}

bool IsSourceNamespaceAllowedInTargetNamespace(FName SourceNamespace, FName TargetNamespace)
{
	if (TargetNamespace == FNiagaraConstants::SystemNamespace && (SourceNamespace == FNiagaraConstants::EmitterNamespace || SourceNamespace == FNiagaraConstants::ParticleAttributeNamespace))
	{
		return false;
	}
	if (TargetNamespace == FNiagaraConstants::EmitterNamespace && SourceNamespace == FNiagaraConstants::ParticleAttributeNamespace)
	{
		return false;
	}
	return true;
}

UNiagaraScript* UNiagaraStackFunctionInput::FindConversionScript(const FNiagaraTypeDefinition& FromType, TMap<FNiagaraTypeDefinition, UNiagaraScript*>& ConversionScriptCache, bool bIncludeConversionScripts) const
{
	if (bIncludeConversionScripts == false)
	{
		return nullptr;
	}
	if (UNiagaraScript** CacheEntry = ConversionScriptCache.Find(FromType))
	{
		return *CacheEntry;
	}
	TArray<UNiagaraScript*> Scripts = GetPossibleConversionScripts(FromType);
	ConversionScriptCache.Add(FromType, Scripts.Num() == 0? nullptr : Scripts[0]);
	return ConversionScriptCache[FromType];
}

void UNiagaraStackFunctionInput::GetAvailableParameters(TSet<FNiagaraAvailableParameterInfo>& OutAvailableParameters, FGetAvailableParameterArgs Arguments) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(UNiagaraStackFunctionInput::GetAvailableParameters, NiagaraChannel);
	
	TMap<FNiagaraTypeDefinition, UNiagaraScript*> ConversionScriptCache;
	
	// Engine Handles.
	for (const FNiagaraVariable& SystemVariable : FNiagaraConstants::GetEngineConstants())
	{
		if (FNiagaraEditorUtilities::AreTypesAssignable(SystemVariable.GetType(), InputType))
		{
			OutAvailableParameters.Add(SystemVariable);
		}
		else if (UNiagaraScript* ConversionScript = FindConversionScript(SystemVariable.GetType(), ConversionScriptCache, Arguments.bIncludeConversionScripts))
		{
			OutAvailableParameters.Add({SystemVariable, ConversionScript});
		}
	}

	// user parameters
	TArray<FNiagaraVariable> ExposedVars;
	GetSystemViewModel()->GetSystem().GetExposedParameters().GetParameters(ExposedVars);
	for (const FNiagaraVariable& ExposedVar : ExposedVars)
	{
		if (FNiagaraEditorUtilities::AreTypesAssignable(ExposedVar.GetType(), InputType))
		{
			OutAvailableParameters.Add(ExposedVar);
		}
		else if (UNiagaraScript* ConversionScript = FindConversionScript(ExposedVar.GetType(), ConversionScriptCache, Arguments.bIncludeConversionScripts))
		{
			OutAvailableParameters.Add({ExposedVar, ConversionScript});
		}
	}

	// gather variables from the parameter map history
	FVersionedNiagaraEmitter Emitter;
	TArray<UNiagaraNodeOutput*> AllOutputNodes;
	if (GetEmitterViewModel().IsValid())
	{
		GetEmitterViewModel()->GetSharedScriptViewModel()->GetGraphViewModel()->GetGraph()->GetNodesOfClass<UNiagaraNodeOutput>(AllOutputNodes);
		Emitter = GetEmitterViewModel()->GetEmitter();
	}
	if (GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
	{
		GetSystemViewModel()->GetSystemScriptViewModel()->GetGraphViewModel()->GetGraph()->GetNodesOfClass<UNiagaraNodeOutput>(AllOutputNodes);
	}
	UNiagaraNodeOutput* CurrentOutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*OwningModuleNode);
	ENiagaraScriptUsage CurrentUsage = CurrentOutputNode ? CurrentOutputNode->GetUsage() : ENiagaraScriptUsage::Module;
	FName CurrentNamespace = GetNamespaceForUsage(CurrentUsage);
	TArray<FName> StackContextRoots = FNiagaraStackGraphUtilities::StackContextResolution(Emitter, CurrentOutputNode);
	for (UNiagaraNodeOutput* OutputNode : AllOutputNodes)
	{
		if (OutputNode == nullptr)
		{
			continue;
		}

		// Check if this is in a spawn event handler and the emitter is not using interpolated spawn so we
		// we can hide particle update parameters
		bool bSpawnScript = false;
		if (CurrentOutputNode != nullptr && CurrentOutputNode->GetUsage() == ENiagaraScriptUsage::ParticleEventScript)
		{
			for (const FNiagaraEventScriptProperties &EventHandlerProps : GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetEventHandlers())
			{
				if (EventHandlerProps.Script->GetUsageId() == CurrentOutputNode->ScriptTypeId)
				{
					bSpawnScript = EventHandlerProps.ExecutionMode == EScriptExecutionMode::SpawnedParticles;
					break;
				}
			}
		}
		bool bInterpolatedSpawn = GetEmitterViewModel().IsValid() && GetEmitterViewModel()->GetEmitter().GetEmitterData()->UsesInterpolatedSpawning();
		bool bCheckInterpSpawn = bInterpolatedSpawn || !bSpawnScript;
		if (OutputNode == CurrentOutputNode || (CurrentOutputNode != nullptr && UsageRunsBefore(OutputNode->GetUsage(), CurrentOutputNode->GetUsage(), bCheckInterpSpawn, Emitter, OutputNode->GetUsageId(), CurrentOutputNode->GetUsageId())) || (CurrentOutputNode != nullptr && IsSpawnUsage(CurrentOutputNode->GetUsage())))
		{
			FName OutputNamespace = GetNamespaceForUsage(OutputNode->GetUsage());
			TArray<TPair<FNiagaraVariableBase, FNiagaraParameterHandle>> AvailableParametersForThisOutput;

			TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackGroups;
			FNiagaraStackGraphUtilities::GetStackNodeGroups(*OutputNode, StackGroups);

			int32 CurrentModuleIndex = OutputNode == CurrentOutputNode
				? StackGroups.IndexOfByPredicate([this](const FNiagaraStackGraphUtilities::FStackNodeGroup Group) { return Group.EndNode == OwningModuleNode; })
				: INDEX_NONE;

			int32 MaxGroupIndex = CurrentModuleIndex != INDEX_NONE ? CurrentModuleIndex : StackGroups.Num() - 1;
			for (int32 i = 1; i < MaxGroupIndex; i++)
			{
				UNiagaraNodeFunctionCall* ModuleToCheck = Cast<UNiagaraNodeFunctionCall>(StackGroups[i].EndNode);
				TArray<FNiagaraParameterMapHistory> Histories;
				FNiagaraStackGraphUtilities::BuildParameterMapHistoryWithStackContextResolution(Emitter, OutputNode, ModuleToCheck, Histories, false);

				if (Histories.Num() == 1)
				{
					for (int32 j = 0; j < Histories[0].Variables.Num(); j++)
					{
						FNiagaraVariable& HistoryVariable = Histories[0].Variables[j];
						FNiagaraParameterHandle AvailableHandle = FNiagaraParameterHandle(HistoryVariable.GetName());

						if (IsSourceNamespaceAllowedInTargetNamespace(AvailableHandle.GetNamespace(), CurrentNamespace) == false)
						{
							continue;
						}

						// check if the variable was written to
						bool bWritten = false;
						for (const FNiagaraParameterMapHistory::FModuleScopedPin& WritePin : Histories[0].PerVariableWriteHistory[j])
						{
							if (Cast<UNiagaraNodeParameterMapSet>(WritePin.Pin->GetOwningNode()) != nullptr)
							{
								bWritten = true;
								break;
							}
						}

						if (bWritten == false)
						{
							continue;
						}

						// as transient variables & output variables are only valid within their current stage, we may want to skip the current parameter handle
						if(OutputNode != CurrentOutputNode && (AvailableHandle.IsOutputHandle() || AvailableHandle.IsTransientHandle()))
						{
							continue;
						}

						if (FNiagaraEditorUtilities::AreTypesAssignable(HistoryVariable.GetType(), InputType))
						{
							OutAvailableParameters.Add(HistoryVariable);
							if (AvailableHandle.GetNamespace() == OutputNamespace)
							{
								AvailableParametersForThisOutput.Add(TPair<FNiagaraVariableBase, FNiagaraParameterHandle>(HistoryVariable, AvailableHandle));
							}

							// Check to see if any variables can be converted to StackContext. This may be more portable for people to setup.
							for (int32 StackRootIdx = 0; StackRootIdx < StackContextRoots.Num(); StackRootIdx++)
							{
								if (HistoryVariable.IsInNameSpace(StackContextRoots[StackRootIdx]))
								{
									// We do a replace here so that we can leave modifiers and other parts intact that might also be aliased.
									FString NewName = HistoryVariable.GetName().ToString().Replace(*StackContextRoots[StackRootIdx].ToString(), *FNiagaraConstants::StackContextNamespace.ToString());

									FNiagaraVariableBase AvailableAliasedParameter = FNiagaraVariableBase(HistoryVariable.GetType(), *NewName);
									OutAvailableParameters.Add(AvailableAliasedParameter);
								}
							}
						}
						else if (UNiagaraScript* ConversionScript = FindConversionScript(HistoryVariable.GetType(), ConversionScriptCache, Arguments.bIncludeConversionScripts))
						{
							OutAvailableParameters.Add({HistoryVariable, ConversionScript});
						}
					}
				}
			}

			if (OutputNode != CurrentOutputNode && IsSpawnUsage(OutputNode->GetUsage()) && OutputNamespace != NAME_None)
			{
				for (const TPair<FNiagaraVariableBase, FNiagaraParameterHandle>& AvailableParameterForThisOutput : AvailableParametersForThisOutput)
				{
					const FNiagaraVariableBase& AvailableParameter = AvailableParameterForThisOutput.Key;
					const FNiagaraParameterHandle& AvailableHandle = AvailableParameterForThisOutput.Value;
					if (AvailableParameter.GetType().IsStatic() == false)
					{
						FNiagaraParameterHandle InitialParameterHandle = FNiagaraParameterHandle::CreateInitialParameterHandle(AvailableHandle);
						OutAvailableParameters.Add(FNiagaraVariableBase(AvailableParameter.GetType(), InitialParameterHandle.GetParameterHandleString()));
					}
				}
			}
		}
	}

	//Parameter Collections
	TArray<UNiagaraParameterCollection*> AvailableParameterCollections;
	FNiagaraEditorUtilities::GetAvailableParameterCollections(AvailableParameterCollections);
	for (UNiagaraParameterCollection* Collection : AvailableParameterCollections)
	{
		for (const FNiagaraVariable& CollectionParam : Collection->GetParameters())
		{
			if (FNiagaraEditorUtilities::AreTypesAssignable(CollectionParam.GetType(), InputType))
			{
				OutAvailableParameters.Add(CollectionParam);
			}
			else if (UNiagaraScript* ConversionScript = FindConversionScript(CollectionParam.GetType(), ConversionScriptCache, Arguments.bIncludeConversionScripts))
			{
				OutAvailableParameters.Add({CollectionParam, ConversionScript});
			}
		}
	}

	if(Arguments.bIncludeParameterDefinitions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(UNiagaraStackFunctionInput::GetAvailableParameters::IncludeParameterDefinitions, NiagaraChannel);
		// At last, we patch up all the found parameters with available parameter definitions
		TArray<UNiagaraParameterDefinitions*> AvailableParameterDefinitions;
		FNiagaraEditorUtilities::GetAvailableParameterDefinitions(AvailableParameterDefinitions);
		TMap<FNiagaraVariableBase, UNiagaraScriptVariable*> ParameterDefinitionsCache;

		for(UNiagaraParameterDefinitions* DefinitionsAsset : AvailableParameterDefinitions)
		{
			for(UNiagaraScriptVariable* ScriptVariable : DefinitionsAsset->GetParametersConst())
			{
				ParameterDefinitionsCache.Add(ScriptVariable->Variable, ScriptVariable);
			}
		}
		
		for(FNiagaraAvailableParameterInfo& ParameterInfo : OutAvailableParameters)
		{
			if(ParameterDefinitionsCache.Contains(ParameterInfo.Variable))
			{
				ParameterInfo.MatchingParameterDefinition = ParameterDefinitionsCache[ParameterInfo.Variable];
			}		
		}
	}
}

UNiagaraNodeFunctionCall* UNiagaraStackFunctionInput::GetDefaultFunctionNode() const
{
	return InputValues.DefaultFunctionNode.Get();
}

UNiagaraNodeFunctionCall* UNiagaraStackFunctionInput::GetDynamicInputNode() const
{
	return InputValues.DynamicNode.Get();
}

void UNiagaraStackFunctionInput::GetAvailableDynamicInputs(TArray<UNiagaraScript*>& AvailableDynamicInputs, bool bIncludeNonLibraryInputs)
{
	TArray<FAssetData> DynamicInputAssets;
	FNiagaraEditorUtilities::FGetFilteredScriptAssetsOptions DynamicInputScriptFilterOptions;
	DynamicInputScriptFilterOptions.ScriptUsageToInclude = ENiagaraScriptUsage::DynamicInput;
	DynamicInputScriptFilterOptions.bIncludeNonLibraryScripts = bIncludeNonLibraryInputs;
	FNiagaraEditorUtilities::GetFilteredScriptAssets(DynamicInputScriptFilterOptions, DynamicInputAssets);

	FPinCollectorArray InputPins;
	TArray<UNiagaraNodeOutput*> OutputNodes;
	auto MatchesInputType = [this, &InputPins, &OutputNodes](UNiagaraScript* Script)
	{
		UNiagaraScriptSource* DynamicInputScriptSource = Cast<UNiagaraScriptSource>(Script->GetLatestSource());
		OutputNodes.Reset();
		DynamicInputScriptSource->NodeGraph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);
		if (OutputNodes.Num() == 1)
		{
			InputPins.Reset();
			OutputNodes[0]->GetInputPins(InputPins);
			if (InputPins.Num() == 1)
			{
				const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
				FNiagaraTypeDefinition PinType = NiagaraSchema->PinToTypeDefinition(InputPins[0]);
				return FNiagaraEditorUtilities::AreTypesAssignable(PinType, InputType);
			}
		}
		return false;
	};

	for (const FAssetData& DynamicInputAsset : DynamicInputAssets)
	{
		UNiagaraScript* DynamicInputScript = Cast<UNiagaraScript>(DynamicInputAsset.GetAsset());
		if (DynamicInputScript != nullptr)
		{
			if(MatchesInputType(DynamicInputScript))
			{
				AvailableDynamicInputs.Add(DynamicInputScript);
			}
		}
	}

	for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel : GetSystemViewModel()->GetScriptScratchPadViewModel()->GetScriptViewModels())
	{
		if (MatchesInputType(ScratchPadScriptViewModel->GetOriginalScript()))
		{
			AvailableDynamicInputs.Add(ScratchPadScriptViewModel->GetOriginalScript());
		}
	}
}

void UNiagaraStackFunctionInput::SetDynamicInput(UNiagaraScript* DynamicInput, FString SuggestedName, const FGuid& InScriptVersion)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("SetDynamicInput", "Make dynamic input"));

	UEdGraphPin& OverridePin = GetOrCreateOverridePin();
	RemoveNodesForOverridePin(OverridePin);
	if (IsRapidIterationCandidate())
	{
		RemoveRapidIterationParametersForAffectedScripts();
	}

	UNiagaraNodeFunctionCall* FunctionCallNode;
	FNiagaraStackGraphUtilities::SetDynamicInputForFunctionInput(OverridePin, DynamicInput, FunctionCallNode, FGuid(), SuggestedName, InScriptVersion);
	FNiagaraStackGraphUtilities::InitializeStackFunctionInputs(GetSystemViewModel(), GetEmitterViewModel(), GetStackEditorData(), *OwningModuleNode, *FunctionCallNode);
	FNiagaraStackGraphUtilities::RelayoutGraph(*OwningFunctionCallNode->GetGraph());

	TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
	if (CachedSysViewModel->GetSystemStackViewModel())
		CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();

	RefreshChildren();
}

FText UNiagaraStackFunctionInput::GetCustomExpressionText() const
{
	return InputValues.ExpressionNode != nullptr ? FText::FromString(InputValues.ExpressionNode->GetCustomHlsl()) : FText();
}

void UNiagaraStackFunctionInput::SetCustomExpression(const FString& InCustomExpression)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("SetCustomExpressionInput", "Make custom expression input"));

	UEdGraphPin& OverridePin = GetOrCreateOverridePin();
	RemoveNodesForOverridePin(OverridePin);
	if (IsRapidIterationCandidate())
	{
		RemoveRapidIterationParametersForAffectedScripts();
	}

	UNiagaraNodeCustomHlsl* FunctionCallNode;
	FNiagaraStackGraphUtilities::SetCustomExpressionForFunctionInput(OverridePin, InCustomExpression, FunctionCallNode);
	FNiagaraStackGraphUtilities::InitializeStackFunctionInputs(GetSystemViewModel(), GetEmitterViewModel(), GetStackEditorData(), *OwningModuleNode, *FunctionCallNode);
	FNiagaraStackGraphUtilities::RelayoutGraph(*OwningFunctionCallNode->GetGraph());
	TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
	if (CachedSysViewModel->GetSystemStackViewModel())
		CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();
	RefreshChildren();
}

void UNiagaraStackFunctionInput::SetScratch()
{
	FScopedTransaction ScopedTransaction(LOCTEXT("SetScratch", "Make new scratch dynamic input"));
	TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchScriptViewModel = GetSystemViewModel()->GetScriptScratchPadViewModel()->CreateNewScript(ENiagaraScriptUsage::DynamicInput, SourceScript->GetUsage(), InputType);
	if (ScratchScriptViewModel.IsValid())
	{
		SetDynamicInput(ScratchScriptViewModel->GetOriginalScript());
		GetSystemViewModel()->GetScriptScratchPadViewModel()->FocusScratchPadScriptViewModel(ScratchScriptViewModel.ToSharedRef());
		ScratchScriptViewModel->SetIsPendingRename(true);
	}
}

TSharedPtr<const FStructOnScope> UNiagaraStackFunctionInput::GetLocalValueStruct()
{
	return InputValues.LocalStruct;
}

UNiagaraDataInterface* UNiagaraStackFunctionInput::GetDataValueObject()
{
	return InputValues.DataObject.Get();
}

UObject* UNiagaraStackFunctionInput::GetObjectAssetValue()
{
	UNiagaraNodeInput* InputNode = InputValues.ObjectAssetInputNode.Get();
	return InputNode ? InputNode->GetObjectAsset() : nullptr;
}

void UNiagaraStackFunctionInput::SetObjectAssetValue(UObject* NewValue)
{
	if (UNiagaraNodeInput* InputNode = InputValues.ObjectAssetInputNode.Get())
	{
		const FScopedTransaction Transaction(NSLOCTEXT("Niagara", "SetObject", "Set Object"));
		InputNode->Modify();
		InputNode->SetObjectAsset(NewValue);

		GetSystemViewModel()->NotifyObjectAssetChanged(*InputNode, InputNode->Input.GetName(), InputNode->GetObjectAsset());
	}
}

void UNiagaraStackFunctionInput::NotifyBeginLocalValueChange()
{
	GEditor->BeginTransaction(LOCTEXT("BeginEditModuleInputLocalValue", "Edit input local value."));
}

void UNiagaraStackFunctionInput::NotifyEndLocalValueChange()
{
	if (GEditor->IsTransactionActive())
	{
		GEditor->EndTransaction();
	}
}

bool UNiagaraStackFunctionInput::IsRapidIterationCandidate() const
{
	// Rapid iteration parameters will only be used if the input is not static and the input value default is a
	// local value, if it's linked in graph or through metadata or a default dynamic input the compiler generates
	// code for that instead.
	return !IsStaticParameter() && FNiagaraStackGraphUtilities::IsRapidIterationType(InputType) && DefaultInputValues.Mode == EValueMode::Local;
}

void UNiagaraStackFunctionInput::SetLocalValue(TSharedRef<FStructOnScope> InLocalValue)
{
	checkf(InLocalValue->GetStruct() == InputType.GetStruct(), TEXT("Can not set an input to an unrelated type."));

	if (InputValues.Mode == EValueMode::Local && FNiagaraEditorUtilities::DataMatches(*InputValues.LocalStruct.Get(), InLocalValue.Get()))
	{
		// The value matches the current value so noop.
		return;
	}

	TGuardValue<bool> UpdateGuard(bUpdatingLocalValueDirectly, true);
	FScopedTransaction ScopedTransaction(LOCTEXT("UpdateInputLocalValue", "Update input local value"));
	bool bGraphWillNeedRelayout = false;
	UEdGraphPin* OverridePin = GetOverridePin();

	if (OverridePin != nullptr && OverridePin->LinkedTo.Num() > 0)
	{
		// If there is an override pin and it's linked we'll need to remove all of the linked nodes to set a local value.
		RemoveNodesForOverridePin(*OverridePin);
		bGraphWillNeedRelayout = true;
	}

	if (IsRapidIterationCandidate())
	{
		// If there is currently an override pin, it must be removed to allow the rapid iteration parameter to be used.
		if (OverridePin != nullptr)
		{
			UNiagaraNode* OverrideNode = CastChecked<UNiagaraNode>(OverridePin->GetOwningNode());
			OverrideNode->Modify();
			OverrideNode->RemovePin(OverridePin);
			bGraphWillNeedRelayout = true;
		}

		// Update the value on all affected scripts.
		for (TWeakObjectPtr<UNiagaraScript> Script : AffectedScripts)
		{
			bool bAddParameterIfMissing = true;
			Script->Modify();
			Script->RapidIterationParameters.SetParameterData(InLocalValue->GetStructMemory(), RapidIterationParameter, bAddParameterIfMissing);
			if (InputMetaData.IsSet())
			{
				Script->RapidIterationParameters.ParameterGuidMapping.Add(RapidIterationParameter, InputMetaData->GetVariableGuid());
			}
			if (InputType.IsStatic()) // Need to potentially trigger a recompile.
			{
				UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(Script->GetLatestSource());
				if (Source && Source->NodeGraph)
				{
					Source->NodeGraph->NotifyGraphNeedsRecompile();
				}
			}
		}

		UNiagaraSystem& NiagaraSystem = GetSystemViewModel()->GetSystem();
		if (NiagaraSystem.ShouldUseRapidIterationParameters() == false)
		{
			NiagaraSystem.RequestCompile(false);
		}
	}
	else
	{
		// If rapid iteration parameters can't be used the string representation of the value needs to be set on the override pin
		// for this input.  For static switch inputs the override pin in on the owning function call node and for standard parameter
		// pins the override pin is on the override parameter map set node.
		FNiagaraVariable LocalValueVariable(InputType, NAME_None);
		LocalValueVariable.SetData(InLocalValue->GetStructMemory());
		FString PinDefaultValue;
		const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
		if (ensureMsgf(NiagaraSchema->TryGetPinDefaultValueFromNiagaraVariable(LocalValueVariable, PinDefaultValue),
			TEXT("Could not generate default value string for non-rapid iteration parameter.")))
		{
			if (OverridePin == nullptr)
			{
				OverridePin = &GetOrCreateOverridePin();
				bGraphWillNeedRelayout = true;
			}

			OverridePin->Modify();
			OverridePin->DefaultValue = PinDefaultValue;
			Cast<UNiagaraNode>(OverridePin->GetOwningNode())->MarkNodeRequiresSynchronization(TEXT("OverridePin Default Value Changed"), true);
		}
	}
	
	if (bGraphWillNeedRelayout)
	{
		FNiagaraStackGraphUtilities::RelayoutGraph(*OwningFunctionCallNode->GetNiagaraGraph());
	}
	TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
	if (CachedSysViewModel->GetSystemStackViewModel())
		CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();
	EValueMode PreviousValueMode = InputValues.Mode;
	RefreshValues();

	if (InputValues.Mode != PreviousValueMode)
	{
		RefreshChildren();
	}
}

void UNiagaraStackFunctionInput::SetDataInterfaceValueExternal(TSubclassOf<UNiagaraDataInterface> InDataInterfaceType, TFunction<void(UNiagaraDataInterface*)> ExternalSet)
{
	if (!ensureMsgf(InDataInterfaceType && InDataInterfaceType->IsChildOf(InputType.GetClass()), TEXT("Can not set an input to an unrelated type.")))
	{
		return;
	}

	TGuardValue<bool> UpdateGuard(bUpdatingLocalValueDirectly, true);
	FScopedTransaction ScopedTransaction(LOCTEXT("UpdateInputLocalDataValue", "Update input local data interface value"));

	// Edit the placeholder, not the override; the manager's OnChanged handler syncs to the override.
	// PostEditChangeProperty does not broadcast OnChanged, so we do it explicitly. Modify() captures
	// the placeholder under the transaction so undo/redo stays in step with the override.
	UNiagaraNodeFunctionCall& FunctionCallNode = *OwningFunctionCallNode.Get();
	const TSharedRef<FNiagaraSystemViewModel> OwningSystemVM = GetSystemViewModel();

	FGuid EmitterHandleId;
	if (GetEmitterViewModel().IsValid())
	{
		UNiagaraSystem& OwningSystem = OwningSystemVM->GetSystem();
		if (const FNiagaraEmitterHandle* Handle = FNiagaraEditorUtilities::GetEmitterHandleForEmitter(OwningSystem, GetEmitterViewModel()->GetEmitter()))
		{
			EmitterHandleId = Handle->GetId();
		}
	}

	TSharedRef<FNiagaraPlaceholderDataInterfaceHandle> PlaceholderHandle =
		OwningSystemVM->GetPlaceholderDataInterfaceManager()->GetOrCreatePlaceholderDataInterface(
			EmitterHandleId, FunctionCallNode, InputParameterHandle, InDataInterfaceType);
	UNiagaraDataInterface* PlaceholderDataInterface = PlaceholderHandle->GetDataInterface();

	if (PlaceholderDataInterface && ExternalSet)
	{
		PlaceholderDataInterface->Modify();
		ExternalSet(PlaceholderDataInterface);
		PlaceholderDataInterface->OnChanged().Broadcast();
		RefreshValues();
	}
}

void UNiagaraStackFunctionInput::SetDataInterfaceValue(TSubclassOf<UNiagaraDataInterface> InDataInterfaceType)
{
	checkf(InDataInterfaceType->IsChildOf(InputType.GetClass()), TEXT("Can not set an input to an unrelated type."));

	if (InputValues.Mode == EValueMode::Data && InputValues.DataObject->Equals(GetDefault<UNiagaraDataInterface>(InDataInterfaceType)))
	{
		// The value matches the current value so noop.
		return;
	}

	TGuardValue<bool> UpdateGuard(bUpdatingLocalValueDirectly, true);
	FScopedTransaction ScopedTransaction(LOCTEXT("UpdateInputLocalDataValue", "Update input local data interface value"));
	bool bGraphWillNeedRelayout = false;
	UEdGraphPin* OverridePin = GetOverridePin();

	if (OverridePin != nullptr && OverridePin->LinkedTo.Num() > 0)
	{
		// If there is an override pin and it's linked we'll need to remove all of the linked nodes to set a local value.
		RemoveNodesForOverridePin(*OverridePin);
		bGraphWillNeedRelayout = true;
	}

	if (OverridePin == nullptr)
	{
		OverridePin = &GetOrCreateOverridePin();
	}

	UNiagaraDataInterface* InputDataInterface = nullptr;
	FNiagaraStackGraphUtilities::SetDataInterfaceValueForFunctionInput(*OverridePin, InDataInterfaceType, AliasedInputParameterHandle.GetParameterHandleString().ToString(), InputDataInterface);
	RefreshValues();
}

bool UNiagaraStackFunctionInput::CanReset() const
{
	if (bCanResetCache.IsSet() == false)
	{
		bool bNewCanReset = false;
		if (DefaultInputValues.Mode == EValueMode::None)
		{
			// Can't reset if no valid default was set.
			bNewCanReset = false;
		}
		else if (InputValues.Mode != DefaultInputValues.Mode)
		{
			// If the current value mode is different from the default value mode, it can always be reset.
			bNewCanReset = true;
		}
		else
		{
			switch (InputValues.Mode)
			{
			case EValueMode::Data:
				// the DI could have been garbage collected at this point
				if(InputValues.DataObject.IsValid())
				{
					bNewCanReset = InputValues.DataObject->Equals(DefaultInputValues.DataObject.Get()) == false;
				}
				break;
			case EValueMode::ObjectAsset:
				bNewCanReset = InputValues.ObjectAssetInputNode == DefaultInputValues.ObjectAssetInputNode;
				break;
			case EValueMode::Dynamic:
				// For now assume that default dynamic inputs can always be reset to default since they're not currently supported properly.
				// TODO: Fix when default dynamic inputs are properly supported.
				bNewCanReset = false;
				break;
			case EValueMode::Linked:
				bNewCanReset = InputValues.LinkedParameter != DefaultInputValues.LinkedParameter;
				break;
			case EValueMode::Local:
				bNewCanReset =
					DefaultInputValues.LocalStruct.IsValid() &&
					InputValues.LocalStruct->GetStruct() == DefaultInputValues.LocalStruct->GetStruct() &&
					FMemory::Memcmp(InputValues.LocalStruct->GetStructMemory(), DefaultInputValues.LocalStruct->GetStructMemory(), InputType.GetSize());
				break;
			}
		}
		bCanResetCache = bNewCanReset;
	}
	return bCanResetCache.GetValue();
}

bool UNiagaraStackFunctionInput::UpdateRapidIterationParametersForAffectedScripts(const uint8* Data)
{
	for (TWeakObjectPtr<UNiagaraScript> Script : AffectedScripts)
	{
		Script->Modify();
	}

	for (TWeakObjectPtr<UNiagaraScript> Script : AffectedScripts)
	{
		bool bAddParameterIfMissing = true;
		Script->RapidIterationParameters.SetParameterData(Data, RapidIterationParameter, bAddParameterIfMissing);
	}
	GetSystemViewModel()->ResetSystem();
	return true;
}

bool UNiagaraStackFunctionInput::RemoveRapidIterationParametersForAffectedScripts(bool bUpdateGraphGuidsForAffected)
{
	for (TWeakObjectPtr<UNiagaraScript> Script : AffectedScripts)
	{
		Script->Modify();
	}

	for (TWeakObjectPtr<UNiagaraScript> Script : AffectedScripts)
	{
		if (Script->RapidIterationParameters.RemoveParameter(RapidIterationParameter))
		{
			if (bUpdateGraphGuidsForAffected)
			{
				// Because these scripts are not versioned usually, we pass in the 0-0-0-0 id.
				Script->MarkScriptAndSourceDesynchronized(TEXT("Invalidated GUIDS at request of RemoveRapidIterationParametersForAffectedScripts"), FGuid());
			}

			UE_LOGF(LogNiagaraEditor, Log, "Removed Var '%ls' from Script %ls", *RapidIterationParameter.GetName().ToString(), *Script->GetFullName());
		}
	}
	return true;
}

void UNiagaraStackFunctionInput::Reset()
{
	if (CanReset())
	{
		bool bBroadcastDataObjectChanged = false;
		bool bBroadcastUObjectChanged = false;
		if (DefaultInputValues.Mode == EValueMode::Data)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("ResetInputObjectTransaction", "Reset the inputs data interface object to default."));
			RemoveOverridePin();
			PlaceholderDataInterfaceHandle.Reset();
			bBroadcastDataObjectChanged = true;
		}
		else if (DefaultInputValues.Mode == EValueMode::ObjectAsset)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("ResetInputObjectAssetTransaction", "Reset the inputs object to default."));
			RemoveOverridePin();
			bBroadcastUObjectChanged = true;
		}
		else if (DefaultInputValues.Mode == EValueMode::Linked)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("ResetInputLinkedValueTransaction", "Reset the input to its default linked value."));
			SetLinkedParameterValue(DefaultInputValues.LinkedParameter);
		}
		else if (DefaultInputValues.Mode == EValueMode::Local)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("ResetInputLocalValueTransaction", "Reset the input to its default local value."));
			SetLocalValue(DefaultInputValues.LocalStruct.ToSharedRef());
		}
		else if (DefaultInputValues.Mode == EValueMode::DefaultFunction ||
			DefaultInputValues.Mode == EValueMode::UnsupportedDefault)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("ResetInputValueTransaction", "Reset the input to its default value."));
			RemoveOverridePin();
		}
		else
		{
			ensureMsgf(false, TEXT("Attempted to reset a function input to default without a valid default."));
		}
		
		TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
		if (CachedSysViewModel->GetSystemStackViewModel())
			CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();

		RefreshChildren();
		if (bBroadcastDataObjectChanged && InputValues.DataObject.IsValid())
		{
			TArray<UObject*> ChangedObjects;
			ChangedObjects.Add(InputValues.DataObject.Get());
			OnDataObjectModified().Broadcast(ChangedObjects, ENiagaraDataObjectChange::Unknown);
		}
		if (bBroadcastUObjectChanged)
		{
			if (UNiagaraNodeInput* InputNode = InputValues.ObjectAssetInputNode.Get())
			{
				CachedSysViewModel->NotifyObjectAssetChanged(*InputNode, InputNode->Input.GetName(), InputNode->GetObjectAsset());
			}
		}
	}
}

bool UNiagaraStackFunctionInput::HasAnyResettableChildrenInputs() const
{
	TArray<UNiagaraStackFunctionInput*> FilteredChildInputs;
	GetFilteredChildInputs(FilteredChildInputs);
	
	for(UNiagaraStackFunctionInput* Input : FilteredChildInputs)
	{
		if(Input->CanReset())
		{
			return true;
		}
	}

	return false;
}

bool UNiagaraStackFunctionInput::IsStaticParameter() const
{
	return ParameterBehavior == EStackParameterBehavior::Static;
}

bool UNiagaraStackFunctionInput::CanResetToBase() const
{
	if (HasBaseEmitter())
	{
		if (bCanResetToBaseCache.IsSet() == false)
		{
			bool bIsModuleInput = OwningFunctionCallNode == OwningModuleNode;
			if (bIsModuleInput)
			{
				TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();

				UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*OwningFunctionCallNode.Get());
				if(MergeManager->IsMergeableScriptUsage(OutputNode->GetUsage()))
				{
					FNiagaraVariableBase Variable(GetInputType(), GetInputParameterHandle().GetName());
					TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModelPtr = GetEmitterViewModel();
					ensureMsgf(EmitterViewModelPtr.IsValid(), TEXT("ViewMode is nullptr and it never should be"));
					FVersionedNiagaraEmitter BaseEmitter = EmitterViewModelPtr.IsValid() ? EmitterViewModelPtr->GetParentEmitter() : FVersionedNiagaraEmitter();
					bCanResetToBaseCache = BaseEmitter.Emitter != nullptr && MergeManager->IsModuleInputDifferentFromBase(
						EmitterViewModelPtr->GetEmitter(),
						BaseEmitter,
						OutputNode->GetUsage(),
						OutputNode->GetUsageId(),
						OwningModuleNode->NodeGuid,
						Variable);
				}
				else
				{
					bCanResetToBaseCache = false;
				}
			}
			else
			{
				bCanResetToBaseCache = false;
			}
		}
		return bCanResetToBaseCache.GetValue();
	}
	return false;
}

void UNiagaraStackFunctionInput::ResetToBase()
{
	if (CanResetToBase())
	{
		TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();

		UNiagaraScript* OwningScript = nullptr;
		FVersionedNiagaraEmitterData* EmitterData = GetEmitterViewModel()->GetEmitter().GetEmitterData();
		UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*OwningFunctionCallNode.Get());
		if (ensure(OutputNode))
		{
			OwningScript = EmitterData->GetScript(OutputNode->GetUsage(), OutputNode->GetUsageId());
		}

		if (!OwningScript)
		{
			return;
		}

		FVersionedNiagaraEmitter BaseEmitter = EmitterData->GetParent();
		TArray<FNiagaraVariable> OriginalRapidIterationParameters;
		OwningScript->RapidIterationParameters.GetParameters(OriginalRapidIterationParameters);

		FScopedTransaction ScopedTransaction(LOCTEXT("ResetInputToBaseTransaction", "Reset this input to match the parent emitter."));
		FNiagaraScriptMergeManager::FApplyDiffResults Results = MergeManager->ResetModuleInputToBase(
			GetEmitterViewModel()->GetEmitter(),
			BaseEmitter,
			OutputNode->GetUsage(),
			OutputNode->GetUsageId(),
			OwningModuleNode->NodeGuid,
			InputParameterHandle.GetName().ToString());

		if (Results.bSucceeded)
		{
			// If resetting to the base succeeded, an unknown number of rapid iteration parameters may have been added.  To fix
			// this copy all of the owning scripts rapid iteration parameters to all other affected scripts.
			// TODO: Either the merge should take care of this directly, or at least provide more information about what changed.
			TArray<FNiagaraVariable> OwningScriptRapidIterationParameters;
			OwningScript->RapidIterationParameters.GetParameters(OwningScriptRapidIterationParameters);

			// we also need to check if we've removed any RI parameters, and if so pass that forward to the affected scripts as well
			TArray<FNiagaraVariable> RemovedVariables = OriginalRapidIterationParameters;
			RemovedVariables.SetNum(Algo::RemoveIf(RemovedVariables, [&OwningScriptRapidIterationParameters](const FNiagaraVariable& Var) -> bool
			{
				return OwningScriptRapidIterationParameters.Contains(Var);
			}));
			
			if (!OwningScriptRapidIterationParameters.IsEmpty() || !RemovedVariables.IsEmpty())
			{
				auto ContainsStaticVariable = [](const FNiagaraVariable& Variable) -> bool
				{
					return Variable.GetType().IsStatic();
				};

				const bool bChangeRequiresRecompile = OwningScriptRapidIterationParameters.ContainsByPredicate(ContainsStaticVariable)
					|| RemovedVariables.ContainsByPredicate(ContainsStaticVariable);

				for (TWeakObjectPtr<UNiagaraScript> AffectedScript : AffectedScripts)
				{
					if (AffectedScript.Get() != OwningScript)
					{
						AffectedScript->Modify();
						for (FNiagaraVariable& OwningScriptRapidIterationParameter : OwningScriptRapidIterationParameters)
						{
							bool bAddParameterIfMissing = true;
							AffectedScript->RapidIterationParameters.SetParameterData(
								OwningScript->RapidIterationParameters.GetParameterData(OwningScriptRapidIterationParameter), OwningScriptRapidIterationParameter, bAddParameterIfMissing);
						}

						for (FNiagaraVariable& RemovedVariable : RemovedVariables)
						{
							AffectedScript->RapidIterationParameters.RemoveParameter(RemovedVariable);
						}
					}

					// we mark the graphs associated with the scripts as dirty to also ensure that they 
					if (bChangeRequiresRecompile)
					{
						UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(AffectedScript->GetLatestSource());
						if (Source && Source->NodeGraph)
						{
							Source->NodeGraph->NotifyGraphNeedsRecompile();
						}
					}
				}
			}
		}

		TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
		if (CachedSysViewModel->GetSystemStackViewModel())
			CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();

		RefreshChildren();
	}
}

FNiagaraVariable UNiagaraStackFunctionInput::CreateRapidIterationVariable(const FName& InName)
{
	UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*OwningModuleNode.Get());
	FString UniqueEmitterName = GetEmitterViewModel().IsValid() ? GetEmitterViewModel()->GetEmitter().Emitter->GetUniqueEmitterName() : FString();
	return FNiagaraStackGraphUtilities::CreateRapidIterationParameter(UniqueEmitterName, OutputNode->GetUsage(), InName, InputType);
}

void UNiagaraStackFunctionInput::OnMessageManagerRefresh(const TArray<TSharedRef<const INiagaraMessage>>& NewMessages)
{
	if(MessageManagerIssues.Num() != 0 || NewMessages.Num() != 0)
	{
		MessageManagerIssues.Reset();
		if (InputValues.DynamicNode.IsValid())
		{
			for (TSharedRef<const INiagaraMessage> Message : NewMessages)
			{
				if(Message->ShouldOnlyLog())
				{
					continue;
				}
				
				FStackIssue Issue = FNiagaraMessageUtilities::MessageToStackIssue(Message, GetStackEditorDataKey());
				if (MessageManagerIssues.ContainsByPredicate([&Issue](const FStackIssue& NewIssue)
					{ return NewIssue.GetUniqueIdentifier() == Issue.GetUniqueIdentifier(); }) == false)
				{
					MessageManagerIssues.Add(Issue);	
				}
			}
		}
		RefreshChildren();
	}
}

bool UNiagaraStackFunctionInput::SupportsRename() const
{
	// Only module level assignment node inputs can be renamed.
	return OwningAssignmentNode.IsValid() && InputParameterHandlePath.Num() == 1 &&
		OwningAssignmentNode->FindAssignmentTarget(InputParameterHandle.GetName()) != INDEX_NONE;
}

void UNiagaraStackFunctionInput::OnRenamed(FText NewNameText)
{
	FName NewName(*NewNameText.ToString());
	FNiagaraVariable OldVar = FNiagaraVariable(InputType, InputParameterHandle.GetName());
	FNiagaraVariable NewVar = FNiagaraVariable(InputType, NewName);
	if (InputParameterHandle.GetName() != NewName && OwningAssignmentNode.IsValid() && SourceScript.IsValid())
	{
		TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
		TSharedPtr<FNiagaraEmitterViewModel> CachedEmitterViewModel = GetEmitterViewModel();
		UNiagaraSystem& System = GetSystemViewModel()->GetSystem();
		FVersionedNiagaraEmitter Emitter = GetEmitterViewModel().IsValid() ? GetEmitterViewModel()->GetEmitter() : FVersionedNiagaraEmitter();

		FScopedTransaction ScopedTransaction(LOCTEXT("RenameInput", "Rename this function's input."));
		FNiagaraStackGraphUtilities::RenameAssignmentTarget(
			System,
			Emitter,
			*SourceScript.Get(),
			*OwningAssignmentNode.Get(),
			OldVar,
			NewName);
		ensureMsgf(IsFinalized(), TEXT("Input not finalized when renamed."));


		if (CachedSysViewModel->GetSystemStackViewModel())
			CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();

		CachedSysViewModel->NotifyParameterRenamedExternally(OldVar, NewVar, Emitter.Emitter);

	}
}

bool UNiagaraStackFunctionInput::CanDeleteInput() const
{
	return GetInputFunctionCallNode().IsA(UNiagaraNodeAssignment::StaticClass());
}

void UNiagaraStackFunctionInput::DeleteInput()
{
	if (UNiagaraNodeAssignment* NodeAssignment = Cast<UNiagaraNodeAssignment>(OwningFunctionCallNode.Get()))
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("RemoveInputTransaction", "Remove Input"));
		TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
		FVersionedNiagaraEmitter VersionedEmitter = GetEmitterViewModel().IsValid() ? GetEmitterViewModel()->GetEmitter() : FVersionedNiagaraEmitter();

		
		{
			// Rapid iteration parameters might be affected by this removal, so add them. Variables might also be removed in other bindings, but that is handled elsewhere.
			UNiagaraSystem& System = GetSystemViewModel()->GetSystem();

			FNiagaraStackGraphUtilities::FindAffectedScripts(&System, VersionedEmitter, *OwningModuleNode.Get(), AffectedScripts);

			for (TWeakObjectPtr<UNiagaraScript> AffectedScript : AffectedScripts)
			{
				if (AffectedScript.IsValid())
					AffectedScript->Modify();
			}
		}

		// If there is an override pin and connected nodes, remove them before removing the input since removing
		// the input will prevent us from finding the override pin.
		RemoveOverridePin();
		FNiagaraVariable Var = FNiagaraVariable(GetInputType(), GetInputParameterHandle().GetName());
		NodeAssignment->Modify();
		NodeAssignment->RemoveParameter(Var);
		if (CachedSysViewModel->GetSystemStackViewModel())
			CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();
		CachedSysViewModel->NotifyParameterRemovedExternally(Var, VersionedEmitter.Emitter);
	}
}

void UNiagaraStackFunctionInput::GetNamespacesForNewReadParameters(TArray<FName>& OutNamespacesForNewParameters) const
{
	UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*OwningFunctionCallNode);
	bool bIsEditingSystem = GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset;

	TArray<FName> Namespaces;
	FNiagaraStackGraphUtilities::GetNamespacesForNewReadParameters(
		bIsEditingSystem ? FNiagaraStackGraphUtilities::EStackEditContext::System : FNiagaraStackGraphUtilities::EStackEditContext::Emitter,
		OutputNode->GetUsage(), Namespaces);

	for (FName Namespace : Namespaces)
	{
		// Check the registry to make sure a new parameter of the type expected can be created in this namespace
		if (Namespace == FNiagaraConstants::UserNamespace)
		{
			if (!FNiagaraTypeRegistry::GetRegisteredUserVariableTypes().Contains(InputType))
			{
				continue;
			}
		}
		else if (Namespace == FNiagaraConstants::SystemNamespace)
		{
			if (!FNiagaraTypeRegistry::GetRegisteredSystemVariableTypes().Contains(InputType))
			{
				continue;
			}
		}
		else if (Namespace == FNiagaraConstants::EmitterNamespace)
		{
			if (!FNiagaraTypeRegistry::GetRegisteredEmitterVariableTypes().Contains(InputType))
			{
				continue;
			}
		}
		else if (Namespace == FNiagaraConstants::ParticleAttributeNamespace)
		{
			if (!FNiagaraTypeRegistry::GetRegisteredParticleVariableTypes().Contains(InputType))
			{
				continue;
			}
		}

		OutNamespacesForNewParameters.Add(Namespace);
	}
}

void UNiagaraStackFunctionInput::GetNamespacesForNewWriteParameters(TArray<FName>& OutNamespacesForNewParameters) const
{
	UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*OwningFunctionCallNode);
	bool bIsEditingSystem = GetSystemViewModel()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset;
	TOptional<FName> StackContextNamespace = OutputNode->GetStackContextOverride();
	FNiagaraStackGraphUtilities::GetNamespacesForNewWriteParameters(
		bIsEditingSystem ? FNiagaraStackGraphUtilities::EStackEditContext::System : FNiagaraStackGraphUtilities::EStackEditContext::Emitter,
		OutputNode->GetUsage(), StackContextNamespace, OutNamespacesForNewParameters);
}

UNiagaraStackFunctionInput::FOnValueChanged& UNiagaraStackFunctionInput::OnValueChanged()
{
	return ValueChangedDelegate;
}

TOptional<FNiagaraVariable> UNiagaraStackFunctionInput::GetEditConditionVariable() const
{
	if(GetHasEditCondition())
	{
		return FNiagaraVariable(EditCondition.GetConditionInputType(), EditCondition.GetConditionInputName());
	}

	return TOptional<FNiagaraVariable>();
}

bool UNiagaraStackFunctionInput::GetHasEditCondition() const
{
	return EditCondition.IsValid();
}

bool UNiagaraStackFunctionInput::GetShowEditConditionInline() const
{
	return bShowEditConditionInline;
}

bool UNiagaraStackFunctionInput::GetEditConditionEnabled() const
{
	return EditCondition.IsValid() && EditCondition.GetConditionIsEnabled();
}

void UNiagaraStackFunctionInput::SetEditConditionEnabled(bool bIsEnabled)
{
	if(EditCondition.CanSetConditionIsEnabled())
	{ 
		EditCondition.SetConditionIsEnabled(bIsEnabled);
	}
}

TOptional<FNiagaraVariable> UNiagaraStackFunctionInput::GetVisibleConditionVariable() const
{
	if(GetHasVisibleCondition())
	{
		return FNiagaraVariable(VisibleCondition.GetConditionInputType(), VisibleCondition.GetConditionInputName());
	}

	return TOptional<FNiagaraVariable>();
}

bool UNiagaraStackFunctionInput::GetHasVisibleCondition() const
{
	return VisibleCondition.IsValid();
}

bool UNiagaraStackFunctionInput::GetVisibleConditionEnabled() const
{
	return VisibleCondition.IsValid() && VisibleCondition.GetConditionIsEnabled();
}

bool UNiagaraStackFunctionInput::GetIsInlineEditConditionToggle() const
{
	return bIsInlineEditConditionToggle;
}

bool UNiagaraStackFunctionInput::GetIsDynamicInputScriptReassignmentPending() const
{
	return bIsDynamicInputScriptReassignmentPending;
}

void UNiagaraStackFunctionInput::SetIsDynamicInputScriptReassignmentPending(bool bIsPending)
{
	bIsDynamicInputScriptReassignmentPending = bIsPending;
}

void UNiagaraStackFunctionInput::ReassignDynamicInputScript(UNiagaraScript* DynamicInputScript)
{
	if (DynamicInputScript == nullptr)
	{
		return;
	}
	if (ensureMsgf(InputValues.Mode == EValueMode::Dynamic && InputValues.DynamicNode != nullptr && InputValues.DynamicNode->GetClass() == UNiagaraNodeFunctionCall::StaticClass(),
		TEXT("Can not reassign the dynamic input script when tne input doesn't have a valid dynamic input.")))
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("ReassignDynamicInputTransaction", "Reassign dynamic input script"));

		const FString OldName = InputValues.DynamicNode->GetFunctionName();

		InputValues.DynamicNode->Modify();

		UNiagaraClipboardContent* OldClipboardContent = nullptr;
		UNiagaraScript* OldScript = InputValues.DynamicNode->FunctionScript;
		FVersionedNiagaraScriptData* OldScriptData = InputValues.DynamicNode->GetScriptData();
		FVersionedNiagaraScriptData* NewScriptData = DynamicInputScript->GetLatestScriptData();
		if (NewScriptData->ConversionUtility || (OldScriptData && OldScriptData->bUsePythonScriptConversion))
		{
			OldClipboardContent = UNiagaraClipboardContent::Create();
			Copy(OldClipboardContent);
		}

		InputValues.DynamicNode->FunctionScript = DynamicInputScript;
		InputValues.DynamicNode->SelectedScriptVersion = DynamicInputScript && DynamicInputScript->IsVersioningEnabled() ? DynamicInputScript->GetExposedVersion().VersionGuid : FGuid();

		// intermediate refresh to purge any rapid iteration parameters that have been removed in the new script
		RefreshChildren();

		InputValues.DynamicNode->SuggestName(FString());

		const FString NewName = InputValues.DynamicNode->GetFunctionName();
		UNiagaraSystem& System = GetSystemViewModel()->GetSystem();
		FVersionedNiagaraEmitter Emitter = GetEmitterViewModel().IsValid() ? GetEmitterViewModel()->GetEmitter() : FVersionedNiagaraEmitter();
		FNiagaraStackGraphUtilities::RenameReferencingParameters(&System, Emitter, *InputValues.DynamicNode.Get(), OldName, NewName);

		InputValues.DynamicNode->RefreshFromExternalChanges();
		TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
		if (CachedSysViewModel->GetSystemStackViewModel())
			CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();

		InputValues.DynamicNode->MarkNodeRequiresSynchronization(TEXT("Dynamic input script reassigned."), true);
		RefreshChildren();

		bool bIsPythonAvailable = IPythonScriptPlugin::Get()->IsPythonAvailable();
		if (OldScriptData && OldScriptData->bUsePythonScriptConversion && bIsPythonAvailable && OldClipboardContent != nullptr)
		{
			UNiagaraClipboardContent* NewClipboardContent = UNiagaraClipboardContent::Create();
			UNiagaraClipboardContent* NewPythonContent = nullptr;
			Copy(NewClipboardContent);
			if (OldClipboardContent->FunctionInputs.Num() == 1 && OldClipboardContent->FunctionInputs[0]->Dynamic)
			{
				OldClipboardContent->FunctionInputs = OldClipboardContent->FunctionInputs[0]->Dynamic->Inputs;
			}
			if (NewClipboardContent->FunctionInputs.Num() == 1 && NewClipboardContent->FunctionInputs[0]->Dynamic)
			{
				NewPythonContent = UNiagaraClipboardContent::Create();
				NewPythonContent->FunctionInputs = NewClipboardContent->FunctionInputs[0]->Dynamic->Inputs;
			}
			InputValues.DynamicNode->PythonUpgradeScriptWarnings.Empty();
			FText Warnings;
			if (UNiagaraClipboardContent* NewInputs = FNiagaraEditorUtilities::RunPythonConversionScript(*NewScriptData, NewPythonContent, *OldScriptData, OldClipboardContent, Warnings))
			{
				NewClipboardContent->FunctionInputs[0]->Dynamic->Inputs = NewInputs->FunctionInputs;
				Paste(NewClipboardContent, Warnings);
				if (!Warnings.IsEmpty()) {
					InputValues.DynamicNode->PythonUpgradeScriptWarnings = Warnings.ToString();
				}
			}			
		}
		else if (NewScriptData->ConversionUtility != nullptr && OldClipboardContent != nullptr)
		{
			UNiagaraConvertInPlaceUtilityBase* ConversionUtility = NewScriptData->ConversionUtility.GetDefaultObject();

			UNiagaraClipboardContent* NewClipboardContent = UNiagaraClipboardContent::Create();
			Copy(NewClipboardContent);
			TArray<UNiagaraStackScriptHierarchyRoot*> DynamicInputRoots;
			GetUnfilteredChildrenOfType(DynamicInputRoots);

			if (ConversionUtility && DynamicInputRoots.Num() == 1)
			{
				FText ConvertMessage;
				if (ConversionUtility->UsesLegacyConversion())
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					ConversionUtility->Convert(OldScript, OldClipboardContent, DynamicInputScript, DynamicInputRoots[0], NewClipboardContent, InputValues.DynamicNode.Get(), ConvertMessage);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				else if (ConversionUtility->ConvertInputs(OldScript, OldClipboardContent, DynamicInputScript, NewClipboardContent, ConvertMessage))
				{
					Paste(NewClipboardContent, ConvertMessage);
				}
				
				if (!ConvertMessage.IsEmptyOrWhitespace())
				{
					// Notify the end-user about the convert message, but continue the process as they could always undo.
					FNotificationInfo Msg(FText::Format(LOCTEXT("FixConvertInPlace", "Conversion Note: {0}"), ConvertMessage));
					Msg.ExpireDuration = 5.0f;
					Msg.bFireAndForget = true;
					Msg.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Note"));
					FSlateNotificationManager::Get().AddNotification(Msg);
				}
			}
		}
	}
}

bool UNiagaraStackFunctionInput::GetShouldPassFilterForVisibleCondition() const
{
	return GetHasVisibleCondition() == false || GetVisibleConditionEnabled();
}

TArray<UNiagaraScript*> UNiagaraStackFunctionInput::GetPossibleConversionScripts(const FNiagaraTypeDefinition& FromType) const
{
	return GetPossibleConversionScripts(FromType, InputType);
}

TArray<UNiagaraScript*> UNiagaraStackFunctionInput::GetPossibleConversionScripts(const FNiagaraTypeDefinition& FromType, const FNiagaraTypeDefinition& ToType)
{
    FPinCollectorArray InputPins;
    TArray<UNiagaraNodeOutput*> OutputNodes;
    auto MatchesTypeConversion = [&InputPins, &OutputNodes, &FromType, &ToType](UNiagaraScript* Script)
    {
	    OutputNodes.Reset();
	    UNiagaraGraph* NodeGraph = Cast<UNiagaraScriptSource>(Script->GetLatestSource())->NodeGraph;
	    NodeGraph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);
    	if (OutputNodes.Num() == 1)
    	{
    		// checking via metadata is not really correct, but it's super fast and good enough for the prefiltered list of scripts
    		TArray<FNiagaraVariable> AvailableVars;
		NodeGraph->GetAllVariables(AvailableVars);
    		int MatchingVars = 0;
    		for (const FNiagaraVariable& Var : AvailableVars)
    		{
    			if (Var.IsInNameSpace(FNiagaraConstants::ModuleNamespaceString) && Var.GetType() == FromType)
    			{
    				MatchingVars++;
    			}
    		}

    		// check that the output matches as well
    		InputPins.Reset();
    		OutputNodes[0]->GetInputPins(InputPins);
    		if (InputPins.Num() == 1 && MatchingVars == 1)
    		{
    			const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
    			FNiagaraTypeDefinition PinTypeIn = NiagaraSchema->PinToTypeDefinition(InputPins[0]);
    			if (PinTypeIn == ToType)
    			{
    				return true;
    			}
    		}
    	}
    	return false;
    };

	const TArray<UNiagaraScript*>& AllConversionScripts = FNiagaraEditorModule::Get().GetCachedTypeConversionScripts();
	TArray<UNiagaraScript*> MatchingDynamicInputs;
    for (UNiagaraScript* DynamicInputScript : AllConversionScripts)
    {
    	if (MatchesTypeConversion(DynamicInputScript))
    	{
    		MatchingDynamicInputs.Add(DynamicInputScript);
    	}
    }
	return MatchingDynamicInputs;
}

void UNiagaraStackFunctionInput::SetClipboardContentViaConversionScript(const UNiagaraClipboardFunctionInput& ClipboardFunctionInput)
{
	TArray<UNiagaraScript*> NiagaraScripts = GetPossibleConversionScripts(ClipboardFunctionInput.InputType);
	int32 ScriptCount = NiagaraScripts.Num();
	if (ScriptCount == 0)
	{
		return;
	}
	if (ScriptCount > 1)
	{
		FString ScriptNames;
		for (UNiagaraScript* NiagaraScript : NiagaraScripts)
		{
			ScriptNames += NiagaraScript->GetPathName() + "\n";
		}
		FNiagaraEditorUtilities::WarnWithToastAndLog(FText::Format(TooManyConversionScripts, FText::FromString(ScriptNames)));
	}
	FScopedTransaction ScopedTransaction(LOCTEXT("SetConversionInput", "Make auto-convert dynamic input"));
	SetDynamicInput(NiagaraScripts[0]);
	for (UNiagaraStackFunctionInput* ChildInput : GetChildInputs())
	{
		if (ClipboardFunctionInput.InputType == ChildInput->GetInputType())
		{
			ChildInput->SetValueFromClipboardFunctionInput(ClipboardFunctionInput);
			break;
		}
	}
}

void UNiagaraStackFunctionInput::ChangeScriptVersion(FGuid NewScriptVersion)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("NiagaraChangeVersion_Transaction", "Changing dynamic input version"));
	FNiagaraScriptVersionUpgradeContext UpgradeContext;
	UpgradeContext.CreateClipboardCallback = [this](UNiagaraClipboardContent* ClipboardContent) { Copy(ClipboardContent); };
	UpgradeContext.ApplyClipboardCallback = [this](UNiagaraClipboardContent* ClipboardContent, FText& OutWarning) { Paste(ClipboardContent, OutWarning); };
	UpgradeContext.ConstantResolver = GetEmitterViewModel().IsValid() ?
      FCompileConstantResolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*GetDynamicInputNode())) :
      FCompileConstantResolver(&GetSystemViewModel()->GetSystem(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*GetDynamicInputNode()));
	GetDynamicInputNode()->ChangeScriptVersion(NewScriptVersion, UpgradeContext, true);
	GetDynamicInputNode()->RefreshFromExternalChanges();
	TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
	if (CachedSysViewModel->GetSystemStackViewModel())
		CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();
	RefreshChildren();

	ReportScriptVersionChange();
}

const UNiagaraClipboardFunctionInput* UNiagaraStackFunctionInput::ToClipboardFunctionInput(UObject* InOuter) const
{
	const UNiagaraClipboardFunctionInput* ClipboardInput = nullptr;
	FName InputName = InputParameterHandle.GetName();
	TOptional<bool> bEditConditionValue = GetHasEditCondition() ? GetEditConditionEnabled() : TOptional<bool>();
	switch (InputValues.Mode)
	{
	case EValueMode::Local:
	{
		TArray<uint8> LocalValueData;
		LocalValueData.AddUninitialized(InputType.GetSize());
		FMemory::Memcpy(LocalValueData.GetData(), InputValues.LocalStruct->GetStructMemory(), InputType.GetSize());
		ClipboardInput = UNiagaraClipboardFunctionInput::CreateLocalValue(InOuter, InputName, InputType, bEditConditionValue, LocalValueData);
		break;
	}
	case EValueMode::Linked:
	case EValueMode::LinkedShared:
		ClipboardInput = UNiagaraClipboardFunctionInput::CreateLinkedValue(InOuter, InputName, InputType, bEditConditionValue, InputValues.LinkedParameter);
		break;
	case EValueMode::Data:
		ClipboardInput = UNiagaraClipboardFunctionInput::CreateDataValue(InOuter, InputName, InputType, bEditConditionValue, InputValues.DataObject.Get());
		break;
	case EValueMode::ObjectAsset:
	{
		UNiagaraNodeInput* InputNode = InputValues.ObjectAssetInputNode.Get();
		UObject* ObjectAsset = InputNode ? InputNode->GetObjectAsset().Get() : nullptr;
		ClipboardInput = UNiagaraClipboardFunctionInput::CreateObjectAssetValue(InOuter, InputName, InputType, bEditConditionValue, ObjectAsset);
		break;
	}
	case EValueMode::Expression:
		ClipboardInput = UNiagaraClipboardFunctionInput::CreateExpressionValue(InOuter, InputName, InputType, bEditConditionValue, InputValues.ExpressionNode->GetHlslText().ToString());
		break;
	case EValueMode::Dynamic:
	{
		ClipboardInput = UNiagaraClipboardFunctionInput::CreateDynamicValue(InOuter, InputName, InputType, bEditConditionValue, InputValues.DynamicNode->GetFunctionName(), InputValues.DynamicNode->FunctionScript, InputValues.DynamicNode->SelectedScriptVersion);

		TArray<UNiagaraStackScriptHierarchyRoot*> DynamicInputRoots;
		GetUnfilteredChildrenOfType(DynamicInputRoots);
		for (UNiagaraStackScriptHierarchyRoot* DynamicInputRoot : DynamicInputRoots)
		{
			DynamicInputRoot->ToClipboardFunctionInputs(ClipboardInput->Dynamic, MutableView(ClipboardInput->Dynamic->Inputs));
		}

		break;
	}
	case EValueMode::InvalidOverride:
	case EValueMode::UnsupportedDefault:
	case EValueMode::DefaultFunction:
	case EValueMode::None:
		// Do nothing.
		break;
	default:
		ensureMsgf(false, TEXT("A new value mode was added without adding support for copy paste."));
		break;
	}

	RequestPulse();

	if(ClipboardInput)
	{
		TArray<UNiagaraStackFunctionInput*> ChildInputs;
		GetFilteredChildrenOfType(ChildInputs, false);
		
		for(UNiagaraStackFunctionInput* ChildInput : ChildInputs)
		{
			const_cast<UNiagaraClipboardFunctionInput*>(ClipboardInput)->ChildrenInputs.Add(ChildInput->ToClipboardFunctionInput(const_cast<UNiagaraClipboardFunctionInput*>(ClipboardInput)));
		}
	}

	return ClipboardInput;
}

void UNiagaraStackFunctionInput::SetValueFromClipboardFunctionInput(const UNiagaraClipboardFunctionInput& ClipboardFunctionInput)
{
	if (ensureMsgf(FNiagaraEditorUtilities::AreTypesAssignable(ClipboardFunctionInput.InputType, InputType), TEXT("Can not set input value from clipboard, input types don't match.")))
	{
		switch (ClipboardFunctionInput.ValueMode)
		{
		case ENiagaraClipboardFunctionInputValueMode::Local:
		{
			TSharedRef<FStructOnScope> ValueStruct = MakeShared<FStructOnScope>(InputType.GetStruct());
			FMemory::Memcpy(ValueStruct->GetStructMemory(), ClipboardFunctionInput.Local.GetData(), InputType.GetSize());
			SetLocalValue(ValueStruct);
			break;
		}
		case ENiagaraClipboardFunctionInputValueMode::Linked:
			SetLinkedParameterValue(ClipboardFunctionInput.Linked);
			break;
		case ENiagaraClipboardFunctionInputValueMode::Data:
		{
			if (GetDataValueObject() == nullptr)
			{
				Reset();
			}
			UNiagaraDataInterface* InputDataInterface = GetDataValueObject();
			if (ensureMsgf(InputDataInterface != nullptr, TEXT("Data interface paste failed.  Current data value object null even after reset.")))
			{
				InputDataInterface->Modify();
				ClipboardFunctionInput.Data->CopyTo(InputDataInterface);
			}
			break;
		}
		case ENiagaraClipboardFunctionInputValueMode::ObjectAsset:
		{
			if (InputValues.Mode != EValueMode::ObjectAsset)
			{
				Reset();
			}

			if (UNiagaraNodeInput* InputNode = InputValues.ObjectAssetInputNode.Get())
			{
				InputNode->Modify();
				InputNode->SetObjectAsset(ClipboardFunctionInput.ObjectAsset);
				RefreshChildren();
				RefreshValues();

				TSharedRef<FNiagaraSystemViewModel> SystemViewModelPtr = GetSystemViewModel();
				SystemViewModelPtr->NotifyObjectAssetChanged(*InputNode, InputNode->Input.GetName(), InputNode->GetObjectAsset());
			}
			break;
		}

		case ENiagaraClipboardFunctionInputValueMode::Expression:
			SetCustomExpression(ClipboardFunctionInput.Expression);
			break;
		case ENiagaraClipboardFunctionInputValueMode::Dynamic:
			if (ensureMsgf(ClipboardFunctionInput.Dynamic->ScriptMode == ENiagaraClipboardFunctionScriptMode::ScriptAsset,
				TEXT("Dynamic input values can only be set from script asset clipboard functions.")))
			{
				UNiagaraScript* ClipboardFunctionScript = ClipboardFunctionInput.Dynamic->Script.LoadSynchronous();
				if (ClipboardFunctionScript != nullptr)
				{
					UNiagaraScript* NewDynamicInputScript;
					if (ClipboardFunctionScript->IsAsset() ||
						GetSystemViewModel()->GetScriptScratchPadViewModel()->GetViewModelForScript(ClipboardFunctionScript).IsValid())
					{
						// If the clipboard script is an asset, or it's in the scratch pad of the current asset, it can be used directly.
						NewDynamicInputScript = ClipboardFunctionScript;
					}
					else
					{
						// Otherwise it's a scratch pad script from another asset so we need to add a duplicate scratch pad script to this asset.
						NewDynamicInputScript = GetSystemViewModel()->GetScriptScratchPadViewModel()->CreateNewScriptAsDuplicate(ClipboardFunctionScript)->GetOriginalScript();
					}
					SetDynamicInput(NewDynamicInputScript, ClipboardFunctionInput.Dynamic->FunctionName, ClipboardFunctionInput.Dynamic->ScriptVersion);

					TArray<UNiagaraStackScriptHierarchyRoot*> DynamicInputRoots;
					GetUnfilteredChildrenOfType(DynamicInputRoots);
					for (UNiagaraStackScriptHierarchyRoot* DynamicInputRoot : DynamicInputRoots)
					{
						DynamicInputRoot->SetValuesFromClipboardFunctionInputs(ClipboardFunctionInput.Dynamic->Inputs);
					}
				}
			}
			break;
		default:
			ensureMsgf(false, TEXT("A new value mode was added without adding support for copy paste."));
			break;
		}
	}

	if (GetHasEditCondition() && ClipboardFunctionInput.bHasEditCondition)
	{
		SetEditConditionEnabled(ClipboardFunctionInput.bEditConditionValue);
	}

	TSharedRef<FNiagaraSystemViewModel> CachedSysViewModel = GetSystemViewModel();
	if (CachedSysViewModel->GetSystemStackViewModel())
		CachedSysViewModel->GetSystemStackViewModel()->InvalidateCachedParameterUsage();

	// If we pasted into a static parameter, make sure to refresh children as this can cause new inputs to pop up as we might be in the middle of pasting multiple inputs
	if(IsStaticParameter())
	{
		RefreshChildren();
	}
}

bool UNiagaraStackFunctionInput::IsScratchDynamicInput() const
{
	if (bIsScratchDynamicInputCache.IsSet() == false)
	{
		bIsScratchDynamicInputCache = 
			InputValues.Mode == EValueMode::Dynamic &&
			InputValues.DynamicNode.IsValid() &&
			GetSystemViewModel()->GetScriptScratchPadViewModel()->GetViewModelForScript(InputValues.DynamicNode->FunctionScript).IsValid();
	}
	return bIsScratchDynamicInputCache.GetValue();
}

bool UNiagaraStackFunctionInput::ShouldDisplayInline() const
{
	// we only allow values to show up that are supposed to be visible
	// we don't allow advanced values to be inlined as they can introduce problems with filtering
	if(InputMetaData.IsSet() && InputMetaData->bDisplayInOverviewStack && (InputValues.Mode == EValueMode::Local || InputValues.Mode == EValueMode::Dynamic) && GetShouldPassFilterForVisibleCondition())
	{
		return true;				
	}

	return false;
}

bool UNiagaraStackFunctionInput::IsSemanticChild() const
{
	return bIsSemanticChild;
}

void UNiagaraStackFunctionInput::SetSemanticChild(bool IsSemanticChild)
{
	//GetUnfilteredChildren()
	bIsSemanticChild = IsSemanticChild;
	for (UNiagaraStackFunctionInput* Child : GetChildInputs())
	{
		Child->SetSemanticChild(bIsSemanticChild);
	}
}

void UNiagaraStackFunctionInput::GetSearchItems(TArray<FStackSearchItem>& SearchItems) const
{
	if (GetShouldPassFilterForVisibleCondition() && GetIsInlineEditConditionToggle() == false)
	{
		SearchItems.Add({ FName("DisplayName"), GetDisplayName() });

		if (InputValues.Mode == EValueMode::Local && InputType.IsValid() && InputValues.LocalStruct->IsValid())
		{
			FNiagaraVariable LocalValue = FNiagaraVariable(InputType, "");
			LocalValue.SetData(InputValues.LocalStruct->GetStructMemory());
			TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> ParameterTypeUtilities = FNiagaraEditorModule::Get().GetTypeUtilities(RapidIterationParameter.GetType());
			if (ParameterTypeUtilities.IsValid() && ParameterTypeUtilities->CanHandlePinDefaults())
			{
				FText SearchText = ParameterTypeUtilities->GetSearchTextFromValue(LocalValue);
				if (SearchText.IsEmpty() == false)
				{
					SearchItems.Add({ FName("LocalValueText"), SearchText });
				}
			}
		}
		else if (InputValues.Mode == EValueMode::Linked)
		{
			SearchItems.Add({ FName("LinkedParamName"), FText::FromName(InputValues.LinkedParameter.GetName()) });
		}
		else if (InputValues.Mode == EValueMode::Dynamic && InputValues.DynamicNode.Get() != nullptr)
		{
			SearchItems.Add({ FName("LinkedDynamicInputName"), InputValues.DynamicNode->GetNodeTitle(ENodeTitleType::MenuTitle)});
		}
		else if (InputValues.Mode == EValueMode::Data && InputValues.DataObject.IsValid())
		{
			SearchItems.Add({ FName("LinkedDataInterfaceName"), FText::FromString(InputValues.DataObject->GetName()) });
		}
		else if (InputValues.Mode == EValueMode::ObjectAsset && InputValues.ObjectAssetInputNode.IsValid())
		{
			const UNiagaraNodeInput* InputNode = InputValues.ObjectAssetInputNode.Get();
			const FName ObjectName = InputNode ? InputNode->Input.GetName() : NAME_None;
			SearchItems.Add({ FName("LinkedObjectName"), FText::FromString(ObjectName.ToString()) });
		}
		else if (InputValues.Mode == EValueMode::Expression && InputValues.ExpressionNode.Get() != nullptr)
		{
			SearchItems.Add({ FName("LinkedExpressionText"), InputValues.ExpressionNode->GetHlslText() });
		}
	}
}

const UNiagaraStackFunctionInput::FCollectedUsageData& UNiagaraStackFunctionInput::GetCollectedUsageData() const
{
	if (CachedCollectedUsageData.IsSet() == false)
	{
		TSharedRef<FNiagaraSystemViewModel> SystemVM = GetSystemViewModel();
		INiagaraParameterPanelViewModel* ParamVM = SystemVM->GetParameterPanelViewModel();
		if (ParamVM)
		{
			bool bFoundOverride = false;
			if (InputValues.Mode == EValueMode::Linked)
			{
				bFoundOverride = ParamVM->IsVariableSelected(InputValues.LinkedParameter);
			}
			else if (InputValues.Mode == EValueMode::Dynamic)
			{
				CachedCollectedUsageData = UNiagaraStackEntry::GetCollectedUsageData();
				return CachedCollectedUsageData.GetValue();
			}
			CachedCollectedUsageData = FCollectedUsageData();
			FNiagaraVariableBase Var(InputType, AliasedInputParameterHandle.GetParameterHandleString());
			CachedCollectedUsageData.GetValue().bHasReferencedParameterRead = ParamVM->IsVariableSelected(Var) || bFoundOverride;

			TArray<UNiagaraStackFunctionInput*> ChildrenInputs;
			GetUnfilteredChildrenOfType<UNiagaraStackFunctionInput>(ChildrenInputs);
			for(UNiagaraStackFunctionInput* ChildInput : ChildrenInputs)
			{
				if(ChildInput->GetCollectedUsageData().bHasReferencedParameterRead)
				{
					CachedCollectedUsageData.GetValue().bHasReferencedParameterRead = true;
				}

				if(ChildInput->GetCollectedUsageData().bHasReferencedParameterWrite)
				{
					CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite = true;
				}
			}
		}
		else
		{
			CachedCollectedUsageData = FCollectedUsageData();
		}

	}

	return CachedCollectedUsageData.GetValue();
}

ENiagaraStackEntryInlineDisplayMode UNiagaraStackFunctionInput::GetInlineDisplayMode() const
{
	return GetStackEditorData().GetStackEntryInlineDisplayMode(GetStackEditorDataKey());
}

void UNiagaraStackFunctionInput::SetInlineDisplayMode(ENiagaraStackEntryInlineDisplayMode InlineDisplayMode)
{
	GetStackEditorData().SetStackEntryInlineDisplayMode(GetStackEditorDataKey(), InlineDisplayMode);
	RefreshFilteredChildren();
}

void UNiagaraStackFunctionInput::OnGraphChanged(const struct FEdGraphEditAction& InAction)
{
	if (bUpdatingGraphDirectly == false)
	{
		OverrideNodeCache.Reset();
		OverridePinCache.Reset();
	}
}

void UNiagaraStackFunctionInput::OnRapidIterationParametersChanged()
{
	if (IsFinalized())
	{
		// This delegate isn't always unbound correctly, especially when dealing with undo, so guard against responding in a finalized entry. 
		return;
	}

	bCanResetCache.Reset();
	bCanResetToBaseCache.Reset();
	if (ensureMsgf(OwningModuleNode.IsValid() && OwningFunctionCallNode.IsValid(), TEXT("Stack entry with invalid module or function call not cleaned up.")))
	{
		if (bUpdatingLocalValueDirectly == false && IsRapidIterationCandidate() && (OverridePinCache.IsSet() == false || OverridePinCache.GetValue() == nullptr))
		{
			RefreshValues();
		}
	}
}

void UNiagaraStackFunctionInput::OnScriptSourceChanged()
{
	bCanResetCache.Reset();
	bCanResetToBaseCache.Reset();
}

UNiagaraNodeParameterMapSet* UNiagaraStackFunctionInput::GetOverrideNode() const
{
	if (OverrideNodeCache.IsSet() == false)
	{
		UNiagaraNodeParameterMapSet* OverrideNode = nullptr;
		if (OwningFunctionCallNode.IsValid())
		{
			OverrideNode = FNiagaraStackGraphUtilities::GetStackFunctionOverrideNode(*OwningFunctionCallNode);
		}
		OverrideNodeCache = OverrideNode;
	}
	return OverrideNodeCache.GetValue();
}

UNiagaraNodeParameterMapSet& UNiagaraStackFunctionInput::GetOrCreateOverrideNode()
{
	UNiagaraNodeParameterMapSet* OverrideNode = GetOverrideNode();
	if (OverrideNode == nullptr)
	{
		TGuardValue<bool> Guard(bUpdatingGraphDirectly, true);
		OverrideNode = &FNiagaraStackGraphUtilities::GetOrCreateStackFunctionOverrideNode(*OwningFunctionCallNode);
		OverrideNodeCache = OverrideNode;
	}
	return *OverrideNode;
}

UEdGraphPin* UNiagaraStackFunctionInput::GetOverridePin() const
{
	if (OverridePinCache.IsSet() == false)
	{
		OverridePinCache = FNiagaraStackGraphUtilities::GetStackFunctionInputOverridePin(*OwningFunctionCallNode, AliasedInputParameterHandle);
	}
	return OverridePinCache.GetValue();
}

UEdGraphPin& UNiagaraStackFunctionInput::GetOrCreateOverridePin()
{
	UEdGraphPin* OverridePin = GetOverridePin();
	if (OverridePin == nullptr)
	{
		TGuardValue<bool> Guard(bUpdatingGraphDirectly, true);
		FGuid InputScriptVariableId = InputMetaData.IsSet() ? InputMetaData->GetVariableGuid() : FGuid();
		OverridePin = &FNiagaraStackGraphUtilities::GetOrCreateStackFunctionInputOverridePin(*OwningFunctionCallNode, AliasedInputParameterHandle, InputType, InputScriptVariableId, FGuid());
		OverridePinCache = OverridePin;
	}
	return *OverridePin;
}

void UNiagaraStackFunctionInput::GetDefaultDataInterfaceValueFromDefaultPin(UEdGraphPin* DefaultPin, UNiagaraStackFunctionInput::FInputValues& InInputValues) const
{
	if (DefaultPin->LinkedTo.Num() == 1 && DefaultPin->LinkedTo[0]->GetOwningNode() != nullptr && DefaultPin->LinkedTo[0]->GetOwningNode()->IsA<UNiagaraNodeInput>())
	{
		// If a valid input node was linked, use the data interface from there.
		InInputValues.Mode = EValueMode::Data;
		UNiagaraNodeInput* DataInputNode = CastChecked<UNiagaraNodeInput>(DefaultPin->LinkedTo[0]->GetOwningNode());
		InInputValues.DataObject = DataInputNode->GetDataInterface();
	}
	else
	{
		// If there was no input node, try to get a linked data interface default.
		GetDefaultLinkedParameterOrLinkedFunctionFromDefaultPin(DefaultPin, InInputValues);
		if (InInputValues.Mode == EValueMode::None)
		{
			// If there is no specified default input and no linked value, use the CDO as the default data value.
			InInputValues.Mode = EValueMode::Data;
			InInputValues.DataObject = Cast<UNiagaraDataInterface>(InputType.GetClass()->GetDefaultObject());
		}
	}
}

void UNiagaraStackFunctionInput::GetDefaultObjectAssetValueFromDefaultPin(UEdGraphPin* DefaultPin, UNiagaraStackFunctionInput::FInputValues& InInputValues) const
{
	InInputValues.Mode = EValueMode::ObjectAsset;

	FNiagaraVariable ParameterVariable(FNiagaraTypeDefinition(InputType.GetClass()), InputParameterHandle.GetParameterHandleString());
	FGuid ParameterVariableGuid = FGuid();
	UEdGraphPin& OverridePin = FNiagaraStackGraphUtilities::GetOrCreateStackFunctionInputOverridePin(*OwningFunctionCallNode.Get(), AliasedInputParameterHandle, ParameterVariable.GetType(), ParameterVariableGuid, FGuid());

	if (OverridePin.LinkedTo.Num() != 1)
	{
		FNiagaraStackGraphUtilities::RemoveNodesForStackFunctionInputOverridePin(OverridePin);
		FNiagaraStackGraphUtilities::SetObjectAssetValueForFunctionInput(OverridePin, InputType.GetClass(), AliasedInputParameterHandle.GetParameterHandleString().ToString(), nullptr);
		FNiagaraStackGraphUtilities::RelayoutGraph(*OwningFunctionCallNode.Get()->GetNiagaraGraph());

		UNiagaraNodeInput* DefaultInput = DefaultPin->LinkedTo.Num() == 1 ? Cast<UNiagaraNodeInput>(DefaultPin->LinkedTo[0]->GetOwningNode()) : nullptr;
		UNiagaraNodeInput* NewInput = OverridePin.LinkedTo.Num() == 1 ? Cast<UNiagaraNodeInput>(OverridePin.LinkedTo[0]->GetOwningNode()) : nullptr;
		if ( NewInput && DefaultInput )
		{
			NewInput->SetObjectAsset(DefaultInput->GetObjectAsset());
		}
	}

	InInputValues.ObjectAssetInputNode = Cast<UNiagaraNodeInput>(OverridePin.LinkedTo[0]->GetOwningNode());
}

void UNiagaraStackFunctionInput::GetDefaultLocalValueFromDefaultPin(UEdGraphPin* DefaultPin, UNiagaraStackFunctionInput::FInputValues& InInputValues) const
{
	// Local default values are stored in the pin's default value string.
	InInputValues.Mode = EValueMode::Local;
	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraVariable LocalValueVariable = NiagaraSchema->PinToNiagaraVariable(DefaultPin);
	InInputValues.LocalStruct = MakeShared<FStructOnScope>(InputType.GetStruct());
	if (LocalValueVariable.IsDataAllocated() == false)
	{
		FNiagaraEditorUtilities::ResetVariableToDefaultValue(LocalValueVariable);
	}
	if (ensureMsgf(LocalValueVariable.IsDataAllocated(), TEXT("Neither PinToNiagaraVariable or ResetVariableToDefaultValue generated a value.  Allocating with 0s.")) == false)
	{
		LocalValueVariable.AllocateData();
	}
	FMemory::Memcpy(InInputValues.LocalStruct->GetStructMemory(), LocalValueVariable.GetData(), InputType.GetSize());
}

struct FLinkedParameterOrFunctionNode
{
	TOptional<FNiagaraVariableBase> LinkedParameter;
	TWeakObjectPtr<UNiagaraNodeFunctionCall> LinkedFunctionCallNode;
};

void UNiagaraStackFunctionInput::GetDefaultLinkedParameterOrLinkedFunctionFromDefaultPin(UEdGraphPin* DefaultPin, UNiagaraStackFunctionInput::FInputValues& InInputValues) const
{
	// A default pin linked to a parameter map set is a default linked value.  Linked inputs can be setup in a chain and the first
	// available one will be used so that case must be handled too.  So first collect up the potential linked values and validate 
	// the linked node structure.
	TArray<FLinkedParameterOrFunctionNode> LinkedValues;
	UEdGraphPin* CurrentDefaultPin = DefaultPin;
	while (CurrentDefaultPin != nullptr && CurrentDefaultPin->LinkedTo.Num() == 1)
	{
		UEdGraphPin* LinkedPin = CurrentDefaultPin->LinkedTo[0];
	
		if (LinkedPin->GetOwningNode()->IsA<UNiagaraNodeParameterMapGet>())
		{
			UNiagaraNodeParameterMapGet* GetNode = CastChecked<UNiagaraNodeParameterMapGet>(LinkedPin->GetOwningNode());
			FLinkedParameterOrFunctionNode LinkedValue;
			LinkedValue.LinkedParameter = UEdGraphSchema_Niagara::PinToNiagaraVariable(LinkedPin);
			LinkedValues.Add(LinkedValue);
			CurrentDefaultPin = GetNode->GetDefaultPin(LinkedPin);
		}
		else if (LinkedPin->GetOwningNode()->IsA<UNiagaraNodeFunctionCall>())
		{
			UNiagaraNodeFunctionCall* FunctionCallNode = CastChecked<UNiagaraNodeFunctionCall>(LinkedPin->GetOwningNode());
			FLinkedParameterOrFunctionNode LinkedValue;
			LinkedValue.LinkedFunctionCallNode = FunctionCallNode;
			LinkedValues.Add(LinkedValue);
			CurrentDefaultPin = nullptr;
		}
		else
		{
			// Only parameter map get nodes and function calls are valid for a default linked input chain so clear the linked
			// handles and stop searching.
			CurrentDefaultPin = nullptr;
			LinkedValues.Empty();
		}
	}

	FLinkedParameterOrFunctionNode* ValueToUse = nullptr;
	if (LinkedValues.Num() == 1)
	{
		ValueToUse = &LinkedValues[0];
	}
	else if(LinkedValues.Num() > 1)
	{
		// If there are a chain of linked values use the first one that's available, otherwise just use the last one.
		TSet<FNiagaraAvailableParameterInfo> AvailableParameterInfos;
		FGetAvailableParameterArgs Args;
		Args.bIncludeConversionScripts = false;
		GetAvailableParameters(AvailableParameterInfos, Args);
		for (FLinkedParameterOrFunctionNode& LinkedValue : LinkedValues)
		{
			if (LinkedValue.LinkedFunctionCallNode.IsValid() ||
				AvailableParameterInfos.Contains(LinkedValue.LinkedParameter.GetValue()))
			{
				ValueToUse = &LinkedValue;
				break;
			}
		}
	}

	if (ValueToUse != nullptr)
	{
		if (ValueToUse->LinkedParameter.IsSet())
		{
			InInputValues.Mode = EValueMode::Linked;
			InInputValues.LinkedParameter = ValueToUse->LinkedParameter.GetValue();
		}
		else
		{
			InInputValues.Mode = EValueMode::DefaultFunction;
			InInputValues.DefaultFunctionNode = ValueToUse->LinkedFunctionCallNode;
		}
	}
}

void UNiagaraStackFunctionInput::UpdateValuesFromScriptDefaults(FInputValues& InInputValues) const
{
	// Get the script variable first since it's used to determine static switch and bound input values.
	TOptional<ENiagaraDefaultMode> DefaultMode;
	TOptional<int32> StaticSwitchDefaultValue;
	FNiagaraScriptVariableBinding DefaultBinding;

	FNiagaraVariable InputVariable(InputType, InputParameterHandle.GetParameterHandleString());

	if (OwningFunctionCallNode->FunctionScript != nullptr)
	{
		if (UNiagaraGraph* FunctionGraph = CastChecked<UNiagaraScriptSource>(OwningFunctionCallNode->GetFunctionScriptSource())->NodeGraph)
		{
			DefaultMode = FunctionGraph->GetDefaultMode(InputVariable, &DefaultBinding);
			StaticSwitchDefaultValue = FunctionGraph->GetStaticSwitchDefaultValue(InputVariable);
		}
	}

	if (IsStaticParameter())
	{
		// Static switch parameters are always locally set values.
		if (StaticSwitchDefaultValue.IsSet())
		{
			TSharedPtr<FStructOnScope> StaticSwitchLocalStruct = FNiagaraEditorUtilities::StaticSwitchDefaultIntToStructOnScope(*StaticSwitchDefaultValue, InputType);
			if(ensureMsgf(StaticSwitchLocalStruct.IsValid(), TEXT("Unsupported static struct default value.")))
			{ 
				InInputValues.Mode = EValueMode::Local;
				InInputValues.LocalStruct = StaticSwitchLocalStruct;
			}
		}
	}
	else
	{
		if (DefaultMode.IsSet() && *DefaultMode == ENiagaraDefaultMode::Binding && DefaultBinding.IsValid())
		{
			// The next highest precedence value is a linked value from a variable binding so check that.
			InInputValues.Mode = EValueMode::Linked;
			InInputValues.LinkedParameter = FNiagaraVariable(InputType, DefaultBinding.GetName());
		}
		else if (SourceScript.IsValid())
		{
			// Otherwise we need to check the pin that defined the variable in the graph to determine the default.

			FCompileConstantResolver ConstantResolver;
			if (GetEmitterViewModel())
			{
				ConstantResolver = FCompileConstantResolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*OwningFunctionCallNode));
			}
			else
			{
				// if we don't have an emitter model, we must be in a system context
				ConstantResolver = FCompileConstantResolver(&GetSystemViewModel()->GetSystem(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*OwningFunctionCallNode));
			}
			UEdGraphPin* DefaultPin = OwningFunctionCallNode->FindParameterMapDefaultValuePin(InputParameterHandle.GetParameterHandleString(), SourceScript->GetUsage(), ConstantResolver);
			if (DefaultPin != nullptr)
			{
				if (InputType.IsDataInterface())
				{
					// Data interfaces are handled differently than other values types so collect them here.
					GetDefaultDataInterfaceValueFromDefaultPin(DefaultPin, InInputValues);
				}
				else if (InputType.IsUObject())
				{
					GetDefaultObjectAssetValueFromDefaultPin(DefaultPin, InInputValues);
				}
				else
				{
					// Otherwise check for local and linked values.
					if (DefaultPin->LinkedTo.Num() == 0)
					{
						// If the default pin isn't wired to anything then it's a local value.
						GetDefaultLocalValueFromDefaultPin(DefaultPin, InInputValues);
					}
					else if(DefaultPin->LinkedTo.Num() == 1 && DefaultPin->LinkedTo[0]->GetOwningNode() != nullptr)
					{
						// If a default pin is linked to a parameter map it can be a linked value.
						GetDefaultLinkedParameterOrLinkedFunctionFromDefaultPin(DefaultPin, InInputValues);
					}

					if(InInputValues.Mode == EValueMode::None)
					{
						// If an input mode wasn't found than the graph is configured in a way that can't be displayed in the stack.
						InInputValues.Mode = EValueMode::UnsupportedDefault;
					}
				}
			}
		}
	}
}

namespace FNiagaraStackFunctionInputLinkedSharedUtilities
{
	bool IsSharedParameterName(FName InParameterName)
	{
		FNiagaraParameterHandle ParameterHandle = FNiagaraParameterHandle(InParameterName);
		return ParameterHandle.GetHandleParts().Num() > 2 && ParameterHandle.GetHandleParts()[1] == FNiagaraConstants::SharedNamespace;
	}

	FName MakeSharedParameterName(TOptional<ENiagaraScriptUsage> TargetScriptUsage, const FString& FunctionName, FName InputName)
	{
		TArray<FString> NameParts;
		if (TargetScriptUsage == ENiagaraScriptUsage::SystemSpawnScript || TargetScriptUsage == ENiagaraScriptUsage::SystemUpdateScript)
		{
			NameParts.Add(FNiagaraConstants::SystemNamespaceString);
		}
		else if (TargetScriptUsage == ENiagaraScriptUsage::EmitterSpawnScript || TargetScriptUsage == ENiagaraScriptUsage::EmitterUpdateScript)
		{
			NameParts.Add(FNiagaraConstants::EmitterNamespaceString);
		}
		else if (TargetScriptUsage == ENiagaraScriptUsage::ParticleSpawnScript || TargetScriptUsage == ENiagaraScriptUsage::ParticleUpdateScript)
		{
			NameParts.Add(FNiagaraConstants::ParticleAttributeNamespaceString);
		}
		else
		{
			NameParts.Add(FNiagaraConstants::TransientNamespace.ToString());
		}

		NameParts.Add(FNiagaraConstants::SharedNamespace.ToString());
		NameParts.Add(FString::Printf(TEXT("%s_%s"), *FunctionName, *InputName.ToString()));

		return *FString::Join(NameParts, TEXT("."));
	}

	void GatherSharedParametersAndSharedParameterAssignmentNodes(UNiagaraScript& TargetScript, TSet<FNiagaraVariableBase>& OutSharedParameters, TArray<UNiagaraNodeAssignment*>& OutSharedParameterAssignmentNodes)
	{
		UNiagaraScriptSource* TargetScriptSource = CastChecked<UNiagaraScriptSource>(TargetScript.GetLatestSource());
		UNiagaraGraph* TargetGraph = TargetScriptSource->NodeGraph;
		UNiagaraNodeOutput* TargetOutputNode = TargetGraph->FindEquivalentOutputNode(TargetScript.GetUsage());
		if (TargetOutputNode == nullptr)
		{
			return;
		}

		TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackGroups;
		FNiagaraStackGraphUtilities::GetStackNodeGroups(*TargetOutputNode, StackGroups);
		for (const FNiagaraStackGraphUtilities::FStackNodeGroup& StackGroup : StackGroups)
		{
			UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(StackGroup.EndNode);
			if (AssignmentNode == nullptr)
			{
				continue;
			}

			bool bSharedParameterFound = false;
			for (const FNiagaraVariable& AssignmentTarget : AssignmentNode->GetAssignmentTargets())
			{
				if (FNiagaraStackFunctionInputLinkedSharedUtilities::IsSharedParameterName(AssignmentTarget.GetName()))
				{
					OutSharedParameters.Add(AssignmentTarget);
					bSharedParameterFound = true;
				}
			}

			if (bSharedParameterFound)
			{
				OutSharedParameterAssignmentNodes.Add(AssignmentNode);
			}
		}
	}
}

void UNiagaraStackFunctionInput::UpdateValuesFromOverridePin(const FInputValues& OldInputValues, FInputValues& NewInputValues, UEdGraphPin& InOverridePin) const
{
	NewInputValues.Mode = EValueMode::InvalidOverride;
	if (InOverridePin.LinkedTo.Num() == 0)
	{
		// If an override pin exists but it's not connected, the only valid state is a local struct value stored in the pins default value string.
		if (InputType.IsUObject() == false)
		{
			// If there was an old local struct, reuse it if it's of the correct type.
			TSharedPtr<FStructOnScope> LocalStruct;
			if (OldInputValues.Mode == EValueMode::Local && OldInputValues.LocalStruct.IsValid() && OldInputValues.LocalStruct->GetStruct() == InputType.GetStruct())
			{
				LocalStruct = OldInputValues.LocalStruct;
			}
			else
			{
				LocalStruct = MakeShared<FStructOnScope>(InputType.GetStruct());
			}
			const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
			FNiagaraVariable ValueVariable = NiagaraSchema->PinToNiagaraVariable(&InOverridePin, false);
			if (ValueVariable.IsDataAllocated())
			{
				ValueVariable.CopyTo(LocalStruct->GetStructMemory());
				NewInputValues.Mode = EValueMode::Local;
				NewInputValues.LocalStruct = LocalStruct;
			}
		}
	}
	else if (InOverridePin.LinkedTo.Num() == 1 && InOverridePin.LinkedTo[0] != nullptr && InOverridePin.LinkedTo[0]->GetOwningNode() != nullptr)
	{
		UEdGraphNode* LinkedNode = InOverridePin.LinkedTo[0]->GetOwningNode();
		if (LinkedNode->IsA<UNiagaraNodeInput>())
		{
			// Input nodes handle data interface values.
			UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(LinkedNode);
			if (InputNode->GetDataInterface() != nullptr)
			{
				NewInputValues.Mode = EValueMode::Data;
				NewInputValues.DataObject = InputNode->GetDataInterface();
			}
			else
			{
				NewInputValues.Mode = EValueMode::ObjectAsset;
				NewInputValues.ObjectAssetInputNode = InputNode;
			}
		}
		else if (LinkedNode->IsA<UNiagaraNodeParameterMapGet>())
		{
			// Parameter map get nodes handle linked values.
			NewInputValues.LinkedParameter = UEdGraphSchema_Niagara::PinToNiagaraVariable(InOverridePin.LinkedTo[0]);
			bool bCanBeLinkedShared = FNiagaraStackFunctionInputLinkedSharedUtilities::IsSharedParameterName(NewInputValues.LinkedParameter.GetName());
			if (bCanBeLinkedShared)
			{
				// Parameters with the shared namespace modifier can be displayed inline if they're setup correctly, so check that.
				const TArray<FNiagaraStackModuleData>& StackModuleData = GetSystemViewModel()->GetStackModuleDataByModuleEntry(this);
				int32 OwningModuleIndex = StackModuleData.IndexOfByPredicate([this](const FNiagaraStackModuleData& StackModuleDataItem)
					{ return StackModuleDataItem.WeakModuleNode.IsValid() && StackModuleDataItem.WeakModuleNode.Get() == OwningModuleNode.Get(); });
				UNiagaraNodeAssignment* SharedSourceAssignmentNode = nullptr;
				for (int32 StackModuleDataIndex = OwningModuleIndex - 1; StackModuleDataIndex >= 0 && SharedSourceAssignmentNode == nullptr; StackModuleDataIndex--)
				{
					UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(StackModuleData[StackModuleDataIndex].WeakModuleNode.Get());
					if (AssignmentNode != nullptr && AssignmentNode->GetAssignmentTargets().Contains(NewInputValues.LinkedParameter))
					{
						SharedSourceAssignmentNode = AssignmentNode;
					}
				}
				if (SharedSourceAssignmentNode != nullptr)
				{
					NewInputValues.Mode = EValueMode::LinkedShared;
					NewInputValues.SharedSourceAssignmentNode = SharedSourceAssignmentNode;
				}
			}

			if (NewInputValues.Mode != EValueMode::LinkedShared)
			{
				NewInputValues.Mode = EValueMode::Linked;
			}
		}
		else if (LinkedNode->IsA<UNiagaraNodeCustomHlsl>())
		{
			// Custom hlsl nodes handle expression values.
			UNiagaraNodeCustomHlsl* ExpressionNode = CastChecked<UNiagaraNodeCustomHlsl>(LinkedNode);
			NewInputValues.Mode = EValueMode::Expression;
			NewInputValues.ExpressionNode = ExpressionNode;
		}
		else if (LinkedNode->IsA<UNiagaraNodeFunctionCall>())
		{
			// Function call nodes handle dynamic inputs.
			UNiagaraNodeFunctionCall* DynamicNode = CastChecked<UNiagaraNodeFunctionCall>(LinkedNode);
			NewInputValues.Mode = EValueMode::Dynamic;
			NewInputValues.DynamicNode = DynamicNode;
		}
	}
}

void UNiagaraStackFunctionInput::RemoveNodesForOverridePin(UEdGraphPin& OverridePin)
{
	TArray<TWeakObjectPtr<UNiagaraDataInterface>> RemovedDataObjects;
	FNiagaraStackGraphUtilities::RemoveNodesForStackFunctionInputOverridePin(OverridePin, RemovedDataObjects);
	TArray<UObject*> RemovedObjects;
	for (TWeakObjectPtr<UNiagaraDataInterface> RemovedDataObject : RemovedDataObjects)
	{
		if (RemovedDataObject.IsValid())
		{
			RemovedObjects.Add(RemovedDataObject.Get());
		}
	}
	OnDataObjectModified().Broadcast(RemovedObjects, ENiagaraDataObjectChange::Removed);
}

void UNiagaraStackFunctionInput::RemoveOverridePin()
{
	UEdGraphPin* OverridePin = GetOverridePin();
	if (OverridePin != nullptr)
	{
		RemoveNodesForOverridePin(*OverridePin);
		UNiagaraNodeParameterMapSet* OverrideNode = CastChecked<UNiagaraNodeParameterMapSet>(OverridePin->GetOwningNode());
		OverrideNode->Modify();
		OverrideNode->RemovePin(OverridePin);
	}
}

bool UNiagaraStackFunctionInput::OpenSourceAsset() const
{
	// Helper to open scratch script or function script in the right sub-editor.
	UNiagaraNodeFunctionCall* DynamicInputNode = GetDynamicInputNode();
	if (DynamicInputNode->FunctionScript != nullptr)
	{
		if (DynamicInputNode->FunctionScript->IsAsset())
		{
			DynamicInputNode->FunctionScript->VersionToOpenInEditor = DynamicInputNode->SelectedScriptVersion;
			return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(DynamicInputNode->FunctionScript);
		}
		else
		{
			TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel = GetSystemViewModel()->GetScriptScratchPadViewModel()->GetViewModelForScript(DynamicInputNode->FunctionScript);
			if (ScratchPadScriptViewModel.IsValid())
			{
				GetSystemViewModel()->GetScriptScratchPadViewModel()->FocusScratchPadScriptViewModel(ScratchPadScriptViewModel.ToSharedRef());
				return true;
			}
		}
	}

	return false;
}

bool UNiagaraStackFunctionInput::SupportsCustomExpressions() const
{
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	return NiagaraEditorSettings->IsVisibleClass(UNiagaraNodeCustomHlsl::StaticClass());
}

void UNiagaraStackFunctionInput::GetAvailableScriptUsagesForSharedParameters(TArray<ENiagaraScriptUsage>& OutUsages) const
{
	if (bIsSharedLinkedValueInput)
	{
		// Inputs which are already shared linked values can not be linked to additional shared parameters.
		return;
	}

	static const TArray<ENiagaraScriptUsage> OrderedUsages =
	{
		ENiagaraScriptUsage::SystemSpawnScript,
		ENiagaraScriptUsage::SystemUpdateScript,
		ENiagaraScriptUsage::EmitterSpawnScript,
		ENiagaraScriptUsage::EmitterUpdateScript,
		ENiagaraScriptUsage::ParticleSpawnScript,
		ENiagaraScriptUsage::ParticleUpdateScript,
	};

	ENiagaraScriptUsage SourceUsage = SourceScript->GetUsage();
	int32 SourceUsageIndex = OrderedUsages.IndexOfByPredicate([SourceUsage](const ENiagaraScriptUsage& Usage)
	{
		if (UNiagaraScript::IsEquivalentUsage(Usage, SourceUsage))
		{
			return true;
		}
		return Usage == ENiagaraScriptUsage::ParticleUpdateScript &&
			(SourceUsage == ENiagaraScriptUsage::ParticleEventScript || SourceUsage == ENiagaraScriptUsage::ParticleSimulationStageScript);
	});

	for (int32 UsageIndex = 0; UsageIndex <= SourceUsageIndex; UsageIndex++)
	{
		OutUsages.Add(OrderedUsages[UsageIndex]);
	}
}

UNiagaraStackFunctionInput* UNiagaraStackFunctionInput::GetSharedLinkedValueInput()
{
	if (InputValues.Mode == EValueMode::LinkedShared)
	{
		FNiagaraParameterHandle AssignmentInputHandle = FNiagaraParameterHandle(FNiagaraConstants::ModuleNamespace, InputValues.LinkedParameter.GetName());
		if (SharedLinkedInput == nullptr ||
			&SharedLinkedInput->GetInputFunctionCallNode() != InputValues.SharedSourceAssignmentNode.Get() ||
			SharedLinkedInput->GetInputParameterHandle() != AssignmentInputHandle)
		{
			SharedLinkedInput = NewObject<UNiagaraStackFunctionInput>(this);
			SharedLinkedInput->Initialize(CreateDefaultChildRequiredData(),
				*InputValues.SharedSourceAssignmentNode.Get(),
				*InputValues.SharedSourceAssignmentNode.Get(),
				AssignmentInputHandle.GetParameterHandleString(),
				InputValues.LinkedParameter.GetType(),
				EStackParameterBehavior::Dynamic,
				GetOwnerStackItemEditorDataKey());
			SharedLinkedInput->SetIsSharedLinkedValueInput(true);
		}
		return SharedLinkedInput;
	}
	return nullptr;
}

void UNiagaraStackFunctionInput::MoveValueToLinkedSharedParameter(TOptional<ENiagaraScriptUsage> LinkedSharedParameterTargetUsage)
{
	// Gather the existing shared parameters and shared parameter assignment nodes.
	UNiagaraScript* TargetScript = nullptr;
	UNiagaraScript* SameNamespaceScript = nullptr;

	if (LinkedSharedParameterTargetUsage.IsSet())
	{
		switch (LinkedSharedParameterTargetUsage.GetValue())
		{
		case ENiagaraScriptUsage::SystemSpawnScript:
			TargetScript = GetSystemViewModel()->GetSystem().GetSystemSpawnScript();
			SameNamespaceScript = GetSystemViewModel()->GetSystem().GetSystemUpdateScript();
			break;
		case ENiagaraScriptUsage::SystemUpdateScript:
			TargetScript = GetSystemViewModel()->GetSystem().GetSystemUpdateScript();
			SameNamespaceScript = GetSystemViewModel()->GetSystem().GetSystemSpawnScript();
			break;
		case ENiagaraScriptUsage::EmitterSpawnScript:
			TargetScript = GetEmitterViewModel()->GetEmitter().GetEmitterData()->EmitterSpawnScriptProps.Script;
			SameNamespaceScript = GetEmitterViewModel()->GetEmitter().GetEmitterData()->EmitterUpdateScriptProps.Script;
			break;
		case ENiagaraScriptUsage::EmitterUpdateScript:
			TargetScript = GetEmitterViewModel()->GetEmitter().GetEmitterData()->EmitterUpdateScriptProps.Script;
			SameNamespaceScript = GetEmitterViewModel()->GetEmitter().GetEmitterData()->EmitterSpawnScriptProps.Script;
			break;
		case ENiagaraScriptUsage::ParticleSpawnScript:
		case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
			TargetScript = GetEmitterViewModel()->GetEmitter().GetEmitterData()->SpawnScriptProps.Script;
			SameNamespaceScript = GetEmitterViewModel()->GetEmitter().GetEmitterData()->UpdateScriptProps.Script;
			break;
		case ENiagaraScriptUsage::ParticleUpdateScript:
			TargetScript = GetEmitterViewModel()->GetEmitter().GetEmitterData()->UpdateScriptProps.Script;
			SameNamespaceScript = GetEmitterViewModel()->GetEmitter().GetEmitterData()->SpawnScriptProps.Script;
			break;
		}
	}
	else
	{
		TargetScript = SourceScript.Get();
	}

	if (TargetScript == nullptr)
	{
		return;
	}

	TSet<FNiagaraVariableBase> SharedParameters;
	TArray<UNiagaraNodeAssignment*> TargetAssignmentNodes;
	FNiagaraStackFunctionInputLinkedSharedUtilities::GatherSharedParametersAndSharedParameterAssignmentNodes(*TargetScript, SharedParameters, TargetAssignmentNodes);
	if (SameNamespaceScript != nullptr)
	{
		TArray<UNiagaraNodeAssignment*> SameNamespaceAssignmentNodes;
		FNiagaraStackFunctionInputLinkedSharedUtilities::GatherSharedParametersAndSharedParameterAssignmentNodes(*SameNamespaceScript, SharedParameters, SameNamespaceAssignmentNodes);
	}

	FName ParameterName = FNiagaraStackFunctionInputLinkedSharedUtilities::MakeSharedParameterName(LinkedSharedParameterTargetUsage, OwningFunctionCallNode->GetFunctionName(), InputParameterHandle.GetName());
	TSet<FName> SharedParameterNames;
	for (const FNiagaraVariableBase& SharedParameter : SharedParameters)
	{
		SharedParameterNames.Add(SharedParameter.GetName());
	}
	FName UniqueParameterName = FNiagaraUtilities::GetUniqueName(ParameterName, SharedParameterNames);
	FNiagaraVariableBase SharedParameter = FNiagaraVariableBase(InputType, UniqueParameterName);

	// Copy the current value of the input before making any edits so it can be pasted to the shared input.
	UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
	Copy(ClipboardContent);

	// Add the linked shared input to an existing or new assignment node.
	FScopedTransaction ScopedTransaction(LOCTEXT("MoveToSharedTransaction", "Move input value to shared parameter."));
	UNiagaraNodeAssignment* TargetAssignmentNode;
	if (TargetAssignmentNodes.Num() > 0)
	{
		TargetAssignmentNode = TargetAssignmentNodes[0];
		TargetAssignmentNode->AddAssignmentTarget(SharedParameter);
		TargetAssignmentNode->RefreshFromExternalChanges();
	}
	else
	{
		UNiagaraScriptSource* TargetScriptSource = CastChecked<UNiagaraScriptSource>(TargetScript->GetLatestSource());
		UNiagaraGraph* TargetGraph = TargetScriptSource->NodeGraph;
		UNiagaraNodeOutput* TargetOutputNode = TargetGraph->FindEquivalentOutputNode(TargetScript->GetUsage(), TargetScript->GetUsageId());
		if (TargetOutputNode == nullptr)
		{
			return;
		}
		TargetAssignmentNode = FNiagaraStackGraphUtilities::AddParameterModuleToStack({ FNiagaraVariable(InputType, UniqueParameterName) }, *TargetOutputNode, 0, { FString() });
	}
	SetLinkedParameterValue(SharedParameter);

	// Refresh children to make the shared linked input available and then paste the copied value.
	RefreshChildren();
	UNiagaraStackFunctionInput* SharedLinkedValueInput = GetSharedLinkedValueInput();
	if (SharedLinkedValueInput != nullptr)
	{
		FText UnusedMessage;
		SharedLinkedValueInput->Paste(ClipboardContent, UnusedMessage);
	}
}

void UNiagaraStackFunctionInput::UnlinkFromSharedParameter()
{
	UNiagaraStackFunctionInput* SharedLinkedValueInput = GetSharedLinkedValueInput();
	if (SharedLinkedValueInput != nullptr)
	{
		UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
		SharedLinkedValueInput->Copy(ClipboardContent);

		FScopedTransaction ScopedTransaction(LOCTEXT("UnlinkFromSharedParameter", "Unlink from shared parameter."));
		Reset();
		FText UnusedMessage;
		Paste(ClipboardContent, UnusedMessage);
	}
}

#undef LOCTEXT_NAMESPACE
