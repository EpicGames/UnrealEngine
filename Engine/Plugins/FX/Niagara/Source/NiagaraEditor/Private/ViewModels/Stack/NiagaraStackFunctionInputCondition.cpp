// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackFunctionInputCondition.h"
#include "NiagaraEditorModule.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraTypes.h"
#include "INiagaraEditorTypeUtilities.h"

#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "NiagaraStackFunctionInputCondition"

void FNiagaraStackFunctionInputCondition::Initialize(UNiagaraScript* InScript,
	TArray<UNiagaraScript*> InDependentScripts,
	FCompileConstantResolver InConstantResolver,
	FString InOwningEmitterUniqueName,
	UNiagaraNodeFunctionCall* InFunctionCallNode)
{
	Script = InScript;
	DependentScripts = InDependentScripts;
	ConstantResolver = InConstantResolver;
	OwningEmitterUniqueName = InOwningEmitterUniqueName;
	FunctionCallNode = InFunctionCallNode;
}

void FNiagaraStackFunctionInputCondition::Refresh(const FNiagaraInputConditionMetadata& InputCondition, FText& OutErrorMessage)
{
	TargetValuesData.Empty();
	InputBinder.Reset();

	if (InputCondition.InputName == "")
	{
		return;
	}

	TArray<FString> TargetValues(InputCondition.TargetValues);
	if (TargetValues.Num() == 0)
	{
		// If no target values were specified it's assumed that the input is a bool and that the target value is "true".
		TargetValues.Add("true");
	}

	if (InputBinder.TryBind(Script, DependentScripts, ConstantResolver, OwningEmitterUniqueName, FunctionCallNode, InputCondition.InputName, TOptional<FNiagaraTypeDefinition>(), true, OutErrorMessage))
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(InputBinder.GetInputType());
		if (TypeEditorUtilities.IsValid())
		{
			FNiagaraVariable TempVariable(InputBinder.GetInputType(), "Temp");
			for (FString& TargetValue : TargetValues)
			{
				bool bValueParsed = true;
				if (TypeEditorUtilities->CanHandlePinDefaults())
				{
					bValueParsed = TypeEditorUtilities->SetValueFromPinDefaultString(TargetValue, TempVariable);
				}
				if (bValueParsed == false && TypeEditorUtilities->CanSetValueFromDisplayName())
				{
					bValueParsed = TypeEditorUtilities->SetValueFromDisplayName(FText::FromString(TargetValue), TempVariable);
				}

				if (bValueParsed)
				{
					TArray<uint8>& TargetValueData = TargetValuesData.AddDefaulted_GetRef();
					TargetValueData.AddUninitialized(InputBinder.GetInputType().GetSize());
					TempVariable.CopyTo(TargetValueData.GetData());
				}
				else
				{
					OutErrorMessage = FText::Format(LOCTEXT("ParseValueError", "Target value {0} is not a valid for type {1}"),
						FText::FromString(TargetValue), InputBinder.GetInputType().GetNameText());
					InputBinder.Reset();
					break;
				}
			}
		}
		else
		{
			OutErrorMessage = FText::Format(LOCTEXT("TypeEditorUtilitiesNotFoundError", "No type editor utilites registered for type {0}.  Can not parse target input condition {1}."),
				InputBinder.GetInputType().GetNameText(), FText::FromName(InputCondition.InputName));
			InputBinder.Reset();
		}
	}
}

bool FNiagaraStackFunctionInputCondition::IsValid() const
{
	return InputBinder.IsValid() && TargetValuesData.Num() > 0;
}

bool FNiagaraStackFunctionInputCondition::GetConditionIsEnabled() const
{
	if (IsValid())
	{
		TArray<uint8> InputValue = InputBinder.GetData();
		return nullptr != TargetValuesData.FindByPredicate([&InputValue](const TArray<uint8>& TargetValueData) 
			{ return FMemory::Memcmp(TargetValueData.GetData(), InputValue.GetData(), TargetValueData.Num()) == 0; }
		);
	}
	return false;
}

bool FNiagaraStackFunctionInputCondition::CanSetConditionIsEnabled() const
{
	return IsValid() && InputBinder.GetInputType().IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef());
}

void FNiagaraStackFunctionInputCondition::SetConditionIsEnabled(bool bInIsEnabled)
{
	checkf(CanSetConditionIsEnabled(), TEXT("Can not set this condition"));
	InputBinder.SetValue(bInIsEnabled);
}

FName FNiagaraStackFunctionInputCondition::GetConditionInputName() const
{
	checkf(IsValid(), TEXT("Can not get the input name for an invalid input condition"));
	return InputBinder.GetInputName();
}

FNiagaraTypeDefinition FNiagaraStackFunctionInputCondition::GetConditionInputType() const
{
	checkf(IsValid(), TEXT("Can not get the input type for an invalid input condition"));
	return InputBinder.GetInputType();
}

TOptional<FNiagaraVariableMetaData> FNiagaraStackFunctionInputCondition::GetConditionInputMetaData() const
{
	checkf(IsValid(), TEXT("Can not get the input metadata for an invalid input condition"));
	FNiagaraVariable InputVariable(InputBinder.GetInputType(), InputBinder.GetInputName());
	UNiagaraGraph* FunctionGraph = CastChecked<UNiagaraScriptSource>(FunctionCallNode->GetFunctionScriptSource())->NodeGraph;
	return FunctionGraph->GetMetaData(InputVariable);
}

#undef LOCTEXT_NAMESPACE
