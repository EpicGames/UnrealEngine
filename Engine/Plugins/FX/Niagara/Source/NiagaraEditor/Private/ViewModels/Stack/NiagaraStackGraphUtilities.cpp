// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "NiagaraParameterMapHistory.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "NiagaraGraph.h"
#include "NiagaraNode.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraNodeEmitter.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "EdGraphSchema_Niagara.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraConstants.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "EdGraphUtilities.h"
#include "ObjectTools.h"
#include "NiagaraParameterMapHistory.h"
#include "NiagaraMessageManager.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraEmitter.h"
#include "NiagaraScriptVariable.h"
#include "INiagaraEditorTypeUtilities.h"
#include "NiagaraEditorData.h"
#include "NiagaraGraphDataCache.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "NiagaraSettings.h"
#include "NiagaraSimulationStageBase.h"
#include "NiagaraEmitter.h"
#include "DataHierarchyViewModelBase.h"
#include "NiagaraTrace.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"
#include "TraversalCache/StaticContextBuilder.h"
#include "TraversalCache/TraversalCache.h"

DECLARE_CYCLE_STAT(TEXT("Niagara - StackGraphUtilities - RelayoutGraph"), STAT_NiagaraEditor_StackGraphUtilities_RelayoutGraph, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - StackGraphUtilities - GatherInputRelationsForStack"), STAT_NiagaraEditor_StackGraphUtilities_GatherInputRelationsForStack, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - StackGraphUtilities - FindInputData"), STAT_NiagaraEditor_StackGraphUtilities_FindInputData, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - StackGraphUtilities - FindModuleInputData"), STAT_NiagaraEditor_StackGraphUtilities_FindModuleInputData, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - StackGraphUtilities - FindAssignmentInputData"), STAT_NiagaraEditor_StackGraphUtilities_FindAssignmentInputData, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - StackGraphUtilities - GetStackFunctionInputs"), STAT_NiagaraEditor_StackGraphUtilities_GetStackFunctionInputs, STATGROUP_NiagaraEditor);
DECLARE_CYCLE_STAT(TEXT("Niagara - StackGraphUtilities - GetStackFunctionStaticSwitchPins"), STAT_NiagaraEditor_StackGraphUtilities_GetStackFunctionStaticSwitchPins, STATGROUP_NiagaraEditor);

#define LOCTEXT_NAMESPACE "NiagaraStackGraphUtilities"

namespace FNiagaraStackGraphUtilitiesImpl
{

static FVersionedNiagaraEmitter GetOuterEmitter(const UNiagaraNode* Node)
{
	FVersionedNiagaraEmitter Result;
	if (Node)
	{
		if (const UNiagaraGraph* NiagaraGraph = Node->GetNiagaraGraph())
		{
			Result = NiagaraGraph->GetOwningEmitter();
		}

		if (!Result.Emitter)
		{
			Result.Emitter = Node->GetTypedOuter<UNiagaraEmitter>();
		}
	}

	return Result;
}

static void GetFunctionStaticVariables(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& StaticVars)
{
	FVersionedNiagaraEmitter Emitter = GetOuterEmitter(&FunctionCallNode);
	UNiagaraSystem* System = FunctionCallNode.GetTypedOuter<UNiagaraSystem>();

	TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalData;
	if (Emitter.GetEmitterData())
	{
		CachedTraversalData = Emitter.Emitter->GetCachedTraversalData(Emitter.Version);
	}
	else if (System)
	{
		CachedTraversalData = System->GetCachedTraversalData();
	}
	if (CachedTraversalData.IsValid())
	{
		CachedTraversalData.Get()->GetStaticVariables(StaticVars);
	}
}

} // FNiagaraStackGraphUtilitiesImpl::

void FNiagaraStackGraphUtilities::MakeLinkTo(UEdGraphPin* PinA, UEdGraphPin* PinB)
{
	PinA->MakeLinkTo(PinB);
	PinA->GetOwningNode()->PinConnectionListChanged(PinA);
	PinB->GetOwningNode()->PinConnectionListChanged(PinB);
}

void FNiagaraStackGraphUtilities::BreakAllPinLinks(UEdGraphPin* PinA)
{
	PinA->BreakAllPinLinks(true);
}

void FNiagaraStackGraphUtilities::RelayoutGraph(UEdGraph& Graph)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_StackGraphUtilities_RelayoutGraph);
	TArray<TArray<TArray<UEdGraphNode*>>> OutputNodeTraversalStacks;
	TArray<UNiagaraNodeOutput*> OutputNodes;
	Graph.GetNodesOfClass(OutputNodes);
	TMap<UEdGraphNode*, int32> AllTraversedNodes; // track indentation level per node
	for (UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		TMap<UEdGraphNode*, int32> TraversedNodes; // map to track node level
		TArray<TArray<UEdGraphNode*>> TraversalStack;
		TArray<UEdGraphNode*> CurrentNodesToTraverse;
		CurrentNodesToTraverse.Add(OutputNode);
		int32 NodeLevel = 0;
		while (CurrentNodesToTraverse.Num() > 0)
		{
			TArray<UEdGraphNode*> TraversedNodesThisLevel;
			TArray<UEdGraphNode*> NextNodesToTraverse;
			for (UEdGraphNode* CurrentNodeToTraverse : CurrentNodesToTraverse)
			{
				if (TraversedNodes.Contains(CurrentNodeToTraverse))
				{
					if (NodeLevel > TraversedNodes[CurrentNodeToTraverse])
					{
						// if the node is referenced in a longer chain we pull it back
						TraversedNodes[CurrentNodeToTraverse] = NodeLevel;
					}
					continue;
				}
				
				for (UEdGraphPin* Pin : CurrentNodeToTraverse->GetAllPins())
				{
					if (Pin->Direction == EGPD_Input)
					{
						for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
						{
							if (LinkedPin->GetOwningNode() != nullptr)
							{
								NextNodesToTraverse.Add(LinkedPin->GetOwningNode());
							}
						}
					}
				}
				TraversedNodes.Add(CurrentNodeToTraverse, NodeLevel);
				TraversedNodesThisLevel.Add(CurrentNodeToTraverse);
			}
			TraversalStack.Add(TraversedNodesThisLevel);
			NodeLevel++;
			CurrentNodesToTraverse.Empty();
			CurrentNodesToTraverse.Append(NextNodesToTraverse);
		}
		OutputNodeTraversalStacks.Add(TraversalStack);

		for (const auto& Pair : TraversedNodes)
		{
			if (AllTraversedNodes.Contains(Pair.Key))
			{
				// if the node was already traversed, update the indentation level to get the longest chain
				AllTraversedNodes[Pair.Key] = FMath::Max(AllTraversedNodes[Pair.Key], Pair.Value);
			}
			else
			{
				AllTraversedNodes.Add(Pair);
			}
		}
	}

	// Find all nodes which were not traversed and put them in a separate traversal stack.
	TArray<UEdGraphNode*> UntraversedNodes;
	for (UEdGraphNode* Node : Graph.Nodes)
	{
		if (AllTraversedNodes.Contains(Node) == false)
		{
			UntraversedNodes.Add(Node);
		}
	}
	TArray<TArray<UEdGraphNode*>> UntraversedNodeStack;
	for (UEdGraphNode* UntraversedNode : UntraversedNodes)
	{
		TArray<UEdGraphNode*> UntraversedStackItem;
		UntraversedStackItem.Add(UntraversedNode);
		UntraversedNodeStack.Add(UntraversedStackItem);
	}
	OutputNodeTraversalStacks.Add(UntraversedNodeStack);
	
	// Layout the traversed node stacks.
	float YOffset = 0;
	float XDistance = -400;
	float YDistance = 50;
	float YPinDistance = 50;
	for (const TArray<TArray<UEdGraphNode*>>& TraversalStack : OutputNodeTraversalStacks)
	{
		
		float MaxYOffset = YOffset;
		for (const TArray<UEdGraphNode*>& TraversalLevel : TraversalStack)
		{
			float CurrentYOffset = YOffset;
			for (UEdGraphNode* Node : TraversalLevel)
			{
				int32 Level = AllTraversedNodes.FindOrAdd(Node);
				float CurrentXOffset = Level * XDistance;
				Node->Modify();
				Node->NodePosX = FMath::RoundToInt(CurrentXOffset);
				Node->NodePosY = FMath::RoundToInt(CurrentYOffset);
				int NumInputPins = 0;
				int NumOutputPins = 0;
				for (UEdGraphPin* Pin : Node->GetAllPins())
				{
					if (Pin->Direction == EGPD_Input)
					{
						NumInputPins++;
					}
					else
					{
						NumOutputPins++;
					}
				}
				int MaxPins = FMath::Max(NumInputPins, NumOutputPins);
				CurrentYOffset += YDistance + (MaxPins * YPinDistance);
			}
			MaxYOffset = FMath::Max(MaxYOffset, CurrentYOffset);
		}
		YOffset = MaxYOffset + YDistance;
	}

	Graph.NotifyGraphChanged();
}

void FNiagaraStackGraphUtilities::ConnectPinToInputNode(UEdGraphPin& Pin, UNiagaraNodeInput& InputNode)
{
	FPinCollectorArray InputPins;
	InputNode.GetOutputPins(InputPins);
	if (InputPins.Num() == 1)
	{
		MakeLinkTo(&Pin, InputPins[0]);
	}
}

UEdGraphPin* GetParameterMapPin(TArrayView<UEdGraphPin* const> Pins)
{
	auto IsParameterMapPin = [](const UEdGraphPin* Pin)
	{
		const UEdGraphSchema_Niagara* NiagaraSchema = CastChecked<UEdGraphSchema_Niagara>(Pin->GetSchema());
		FNiagaraTypeDefinition PinDefinition = NiagaraSchema->PinToTypeDefinition(Pin);
		return PinDefinition == FNiagaraTypeDefinition::GetParameterMapDef();
	};

	UEdGraphPin*const* ParameterMapPinPtr = Pins.FindByPredicate(IsParameterMapPin);

	return ParameterMapPinPtr != nullptr ? *ParameterMapPinPtr : nullptr;
}

UEdGraphPin* FNiagaraStackGraphUtilities::GetParameterMapInputPin(UNiagaraNode& Node)
{
	FPinCollectorArray InputPins;
	Node.GetInputPins(InputPins);
	return GetParameterMapPin(InputPins);
}

const UEdGraphPin* FNiagaraStackGraphUtilities::GetParameterMapInputPin(const UNiagaraNode& Node)
{
	FPinCollectorArray InputPins;
	Node.GetInputPins(InputPins);
	return GetParameterMapPin(InputPins);
}

UEdGraphPin* FNiagaraStackGraphUtilities::GetParameterMapOutputPin(UNiagaraNode& Node)
{
	FPinCollectorArray OutputPins;
	Node.GetOutputPins(OutputPins);
	return GetParameterMapPin(OutputPins);
}

void FNiagaraStackGraphUtilities::GetOrderedModuleNodes(UNiagaraNodeOutput& OutputNode, TArray<UNiagaraNodeFunctionCall*>& ModuleNodes)
{
	UNiagaraNode* PreviousNode = &OutputNode;
	while (PreviousNode != nullptr)
	{
		UEdGraphPin* PreviousNodeInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*PreviousNode);
		if (PreviousNodeInputPin != nullptr && PreviousNodeInputPin->LinkedTo.Num() == 1)
		{
			UNiagaraNode* CurrentNode = Cast<UNiagaraNode>(PreviousNodeInputPin->LinkedTo[0]->GetOwningNode());
			UNiagaraNodeFunctionCall* ModuleNode = Cast<UNiagaraNodeFunctionCall>(CurrentNode);
			if (ModuleNode != nullptr)
			{
				ModuleNodes.Insert(ModuleNode, 0);
			}
			PreviousNode = CurrentNode;
		}
		else
		{
			PreviousNode = nullptr;
		}
	}
}

TArray<UNiagaraNodeFunctionCall*> FNiagaraStackGraphUtilities::GetAllModuleNodes(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
	TArray<UNiagaraNodeOutput*> OutputNodes = GetAllOutputNodes(EmitterViewModel);

	for(UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		TArray<UNiagaraNodeFunctionCall*> FunctionCalls;
		FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*OutputNode, FunctionCalls);
		
		for(UNiagaraNodeFunctionCall* FunctionCall : FunctionCalls)
		{
			ModuleNodes.Add(FunctionCall);
		}
	}

	return ModuleNodes;
}

TArray<UNiagaraNodeFunctionCall*> FNiagaraStackGraphUtilities::GetAllModuleNodes(UNiagaraGraph* Graph)
{
	TArray<UNiagaraNodeFunctionCall*> ModuleNodes;

	TArray<UNiagaraNodeOutput*> OutputNodes;
	Graph->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);

	for(UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		TArray<UNiagaraNodeFunctionCall*> FunctionCalls;
		FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*OutputNode, FunctionCalls);
		
		for(UNiagaraNodeFunctionCall* FunctionCall : FunctionCalls)
		{
			ModuleNodes.Add(FunctionCall);
		}
	}

	return ModuleNodes;
}

TArray<UNiagaraNodeFunctionCall*> FNiagaraStackGraphUtilities::GetAllSimStagesModuleNodes(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
	TArray<UNiagaraNodeOutput*> OutputNodes = GetAllOutputNodes(EmitterViewModel);

	for(UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		if(OutputNode->GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			TArray<UNiagaraNodeFunctionCall*> FunctionCalls;
			FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*OutputNode, FunctionCalls);
			
			for(UNiagaraNodeFunctionCall* FunctionCall : FunctionCalls)
			{
				ModuleNodes.Add(FunctionCall);
			}
		}
	}

	return ModuleNodes;
}

TArray<UNiagaraNodeFunctionCall*> FNiagaraStackGraphUtilities::GetAllEventHandlerModuleNodes(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
	TArray<UNiagaraNodeOutput*> OutputNodes = GetAllOutputNodes(EmitterViewModel);

	for(UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		if(OutputNode->GetUsage() == ENiagaraScriptUsage::ParticleEventScript)
		{
			TArray<UNiagaraNodeFunctionCall*> FunctionCalls;
			FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*OutputNode, FunctionCalls);
			
			for(UNiagaraNodeFunctionCall* FunctionCall : FunctionCalls)
			{
				ModuleNodes.Add(FunctionCall);
			}
		}
	}

	return ModuleNodes;
}

UNiagaraNodeFunctionCall* FNiagaraStackGraphUtilities::GetPreviousModuleNode(UNiagaraNodeFunctionCall& CurrentNode)
{
	UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(CurrentNode);
	if (OutputNode != nullptr)
	{
		TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
		GetOrderedModuleNodes(*OutputNode, ModuleNodes);

		int32 ModuleIndex;
		ModuleNodes.Find(&CurrentNode, ModuleIndex);
		return ModuleIndex > 0 ? ModuleNodes[ModuleIndex - 1] : nullptr;
	}
	return nullptr;
}

UNiagaraNodeFunctionCall* FNiagaraStackGraphUtilities::GetNextModuleNode(UNiagaraNodeFunctionCall& CurrentNode)
{
	UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(CurrentNode);
	if (OutputNode != nullptr)
	{
		TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
		GetOrderedModuleNodes(*OutputNode, ModuleNodes);

		int32 ModuleIndex;
		ModuleNodes.Find(&CurrentNode, ModuleIndex);
		return ModuleIndex < ModuleNodes.Num() - 2 ? ModuleNodes[ModuleIndex + 1] : nullptr;
	}
	return nullptr;
}

template<typename OutputNodeType, typename InputNodeType>
OutputNodeType* GetEmitterOutputNodeForStackNodeInternal(InputNodeType& StackNode)
{
	TArray<InputNodeType*> NodesToCheck;
	TSet<InputNodeType*> NodesSeen;
	FPinCollectorArray OutputPins;
	NodesSeen.Add(&StackNode);
	NodesToCheck.Add(&StackNode);
	while (NodesToCheck.Num() > 0)
	{
		InputNodeType* NodeToCheck = NodesToCheck[0];
		NodesToCheck.RemoveAt(0);

		if (NodeToCheck->GetClass() == UNiagaraNodeOutput::StaticClass())
		{
			return CastChecked<UNiagaraNodeOutput>(NodeToCheck);
		}

		OutputPins.Reset();
		NodeToCheck->GetOutputPins(OutputPins);
		for (const UEdGraphPin* OutputPin : OutputPins)
		{
			for (UEdGraphPin* LinkedPin : OutputPin->LinkedTo)
			{
				InputNodeType* LinkedNiagaraNode = LinkedPin ? Cast<UNiagaraNode>(LinkedPin->GetOwningNodeUnchecked()) : nullptr;
				if (LinkedNiagaraNode != nullptr && NodesSeen.Contains(LinkedNiagaraNode) == false)
				{
					NodesSeen.Add(LinkedNiagaraNode);
					NodesToCheck.Add(LinkedNiagaraNode);
				}
			}
		}
	}
	return nullptr;
}

UNiagaraNodeOutput* FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(UNiagaraNode& StackNode)
{
	return GetEmitterOutputNodeForStackNodeInternal<UNiagaraNodeOutput, UNiagaraNode>(StackNode);
}

ENiagaraScriptUsage FNiagaraStackGraphUtilities::GetOutputNodeUsage(const UNiagaraNode& StackNode)
{
	const UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(StackNode);
	return OutputNode ? OutputNode->GetUsage() : ENiagaraScriptUsage::Function;
}

const UNiagaraNodeOutput* FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(const UNiagaraNode& StackNode)
{
	return GetEmitterOutputNodeForStackNodeInternal<const UNiagaraNodeOutput, const UNiagaraNode>(StackNode);
}

TArray<FName> FNiagaraStackGraphUtilities::StackContextResolution(FVersionedNiagaraEmitter OwningEmitter, UNiagaraNodeOutput* OutputNodeInChain)
{
	TArray<FName> PossibleRootNames;
	ENiagaraScriptUsage Usage = OutputNodeInChain->GetUsage();
	FName StageName;
	FName AlternateStageName;
	switch (Usage)
	{
		case ENiagaraScriptUsage::Function:
		case ENiagaraScriptUsage::Module:
		case ENiagaraScriptUsage::DynamicInput:
			break;
		case ENiagaraScriptUsage::ParticleSpawnScript:
		case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
		case ENiagaraScriptUsage::ParticleUpdateScript:
		case ENiagaraScriptUsage::ParticleEventScript:
			StageName = TEXT("Particles");
			break;
		case ENiagaraScriptUsage::ParticleSimulationStageScript:
		{
			if (FVersionedNiagaraEmitterData* EmitterData = OwningEmitter.GetEmitterData())
			{
				if (UNiagaraSimulationStageBase* Base = EmitterData->GetSimulationStageById(OutputNodeInChain->GetUsageId()))
				{
					StageName = Base->GetStackContextReplacementName();
				}
			}
			
			if (StageName == NAME_None)
				StageName = TEXT("Particles");
		}
		break;
		case ENiagaraScriptUsage::ParticleGPUComputeScript:
			StageName = TEXT("Particles");
			break;
		case ENiagaraScriptUsage::EmitterSpawnScript:
		case ENiagaraScriptUsage::EmitterUpdateScript:
			StageName = TEXT("Emitter");
			{
				if (OwningEmitter.Emitter)
				{
					FString EmitterAliasStr = OwningEmitter.Emitter->GetUniqueEmitterName();
					if (EmitterAliasStr.Len())
					{
						StageName = *EmitterAliasStr;
						AlternateStageName = TEXT("Emitter");
					}
				}
			}
			break;
		case ENiagaraScriptUsage::SystemSpawnScript:
		case ENiagaraScriptUsage::SystemUpdateScript:
			StageName = TEXT("System");
			break;
	}

	if (StageName != NAME_None)	
		PossibleRootNames.Add(StageName);
	if (AlternateStageName != NAME_None)
		PossibleRootNames.Add(AlternateStageName);

	return PossibleRootNames;
}

void FNiagaraStackGraphUtilities::BuildParameterMapHistoryWithStackContextResolution(FVersionedNiagaraEmitter OwningEmitter, UNiagaraNodeOutput* OutputNodeInChain, UNiagaraNode* NodeToVisit, TArray<FNiagaraParameterMapHistory>& OutHistories, bool bRecursive /*= true*/, bool bFilterForCompilation /*= true*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(FNiagaraStackGraphUtilities::BuildParameterMapHistoryWithStackContextResolution, NiagaraChannel);
	
	FNiagaraParameterMapHistoryBuilder Builder;

	bool bSetUsage = false;
	FVersionedNiagaraEmitterData* EmitterData = OwningEmitter.GetEmitterData();
	if (EmitterData && OutputNodeInChain)
	{
		ENiagaraScriptUsage Usage = OutputNodeInChain->GetUsage();
		FName StageName;
		if (Usage == ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			if (UNiagaraSimulationStageBase* Base = EmitterData->GetSimulationStageById(OutputNodeInChain->GetUsageId()))
			{
				StageName = Base->GetStackContextReplacementName();
			}
		}
		Builder.BeginUsage(Usage, StageName);
		bSetUsage = true;
	}

	TArray<FNiagaraVariable> StaticVars;
	TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalData;
	if (OwningEmitter.GetEmitterData())
	{
		CachedTraversalData = OwningEmitter.Emitter->GetCachedTraversalData(OwningEmitter.Version);
	}
	
	if (CachedTraversalData.IsValid())
	{
		CachedTraversalData.Get()->GetStaticVariables(StaticVars);
	}

	Builder.RegisterExternalStaticVariables(StaticVars);

	NodeToVisit->BuildParameterMapHistory(Builder, bRecursive, bFilterForCompilation);

	if (bSetUsage)
	{
		Builder.EndUsage();
	}

	OutHistories = MoveTemp(Builder.Histories);
}


UNiagaraNodeInput* FNiagaraStackGraphUtilities::GetEmitterInputNodeForStackNode(UNiagaraNode& StackNode)
{
	// Since the stack graph can have arbitrary branches when traversing inputs, the only safe way to get the initial input
	// is to start at the output node and then trace only parameter map inputs.
	UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(StackNode);

	UNiagaraNode* PreviousNode = OutputNode;
	while (PreviousNode != nullptr)
	{
		UEdGraphPin* PreviousNodeInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*PreviousNode);
		if (PreviousNodeInputPin != nullptr && PreviousNodeInputPin->LinkedTo.Num() == 1)
		{
			UNiagaraNode* CurrentNode = Cast<UNiagaraNode>(PreviousNodeInputPin->LinkedTo[0]->GetOwningNode());
			UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(CurrentNode);
			if (InputNode != nullptr)
			{
				return InputNode;
			}
			PreviousNode = CurrentNode;
		}
		else
		{
			PreviousNode = nullptr;
		}
	}
	return nullptr;
}

FText GetChangelistText(FNiagaraVersionedObject* VersionedObject, const FNiagaraAssetVersion& FromVersion, const FNiagaraAssetVersion& ToVersion)
{
	FText Result;
	for (FNiagaraAssetVersion Version : VersionedObject->GetAllAvailableVersions())
	{
		if (Version <= FromVersion)
		{
			continue;
		}
		if (ToVersion < Version)
		{
			break;
		}
		FText ChangeDescription = VersionedObject->GetVersionDataAccessor(Version.VersionGuid)->GetVersionChangeDescription();
		if (!ChangeDescription.IsEmpty())
		{
			Result = FText::Format(FText::FromString("{0}{1}.{2}: {3}\n"), Result, Version.MajorVersion, Version.MinorVersion, ChangeDescription);
		}
	}
	return Result;
}

void FNiagaraStackGraphUtilities::CheckForDeprecatedScriptVersion(UNiagaraNodeFunctionCall* InputFunctionCallNode, const FString& StackEditorDataKey, UNiagaraStackEntry::FStackIssueFixDelegate VersionUpgradeFix, TArray<UNiagaraStackEntry::FStackIssue>& OutIssues)
{
	// Generate an issue if the script version is out of date
	if (InputFunctionCallNode->FunctionScript && InputFunctionCallNode->FunctionScript->IsVersioningEnabled() && VersionUpgradeFix.IsBound())
	{
		FNiagaraAssetVersion ExposedVersion = InputFunctionCallNode->FunctionScript->GetExposedVersion();
		FVersionedNiagaraScriptData* ScriptData = InputFunctionCallNode->FunctionScript->GetScriptData(InputFunctionCallNode->SelectedScriptVersion);
		FNiagaraAssetVersion ReferencedVersion = ScriptData->Version;
		if (ReferencedVersion.MajorVersion < ExposedVersion.MajorVersion)
		{
			TArray<UNiagaraStackEntry::FStackIssueFix> Fixes;
			Fixes.Add(UNiagaraStackEntry::FStackIssueFix(LOCTEXT("UpgradeVersionFix", "Upgrade to newest version."), VersionUpgradeFix));
			//TODO MV: add "fix all" and "copy and fix" actions
			
			FText ChangelistDescriptions = GetChangelistText(InputFunctionCallNode->FunctionScript, ReferencedVersion, ExposedVersion);
			FText DeprecationDescription = ScriptData->bDeprecated && !ScriptData->DeprecationMessage.IsEmpty() ? FText::Format(LOCTEXT("DeprecatedScriptVersionMessage", "\n\nDeprecation message:\n{0}"), ScriptData->DeprecationMessage) : FText();
			FText LongDescription = FText::Format(LOCTEXT("DeprecatedVersionFormat", "This script has a newer version available.\nYou can upgrade now, but major version upgrades can sometimes come with breaking changes! So check that everything is still working as expected afterwards.{0}{1}"),
				DeprecationDescription,
				ChangelistDescriptions.IsEmpty() ? FText() : FText::Format(LOCTEXT("DeprecatedVersionFormatChanges", "\n\nVersion change description:\n{0}"), ChangelistDescriptions));
			UNiagaraStackEntry::FStackIssue UpgradeVersion(
                ScriptData->bDeprecated ? EStackIssueSeverity::Warning : EStackIssueSeverity::Info,
                FText::Format(LOCTEXT("DeprecatedScriptVersionSummaryFormat", "Upgrade script version: {0}.{1} -> {2}.{3}"),
                	FText::AsNumber(ReferencedVersion.MajorVersion), FText::AsNumber(ReferencedVersion.MinorVersion), FText::AsNumber(ExposedVersion.MajorVersion), FText::AsNumber(ExposedVersion.MinorVersion)),
                LongDescription,
                StackEditorDataKey,
                true,
                Fixes);
			OutIssues.Add(UpgradeVersion);
		}
	}

	// Generate a note of the changelist when the script version was manually changed
	if (InputFunctionCallNode->FunctionScript && InputFunctionCallNode->FunctionScript->IsVersioningEnabled())
	{
		FGuid SelectedVersionGuid = InputFunctionCallNode->SelectedScriptVersion;
		FGuid PreviousVersionGuid = InputFunctionCallNode->PreviousScriptVersion;
		FVersionedNiagaraScriptData* SelectedVersion = InputFunctionCallNode->FunctionScript->GetScriptData(SelectedVersionGuid);
		FVersionedNiagaraScriptData* PreviousVersion = InputFunctionCallNode->FunctionScript->GetScriptData(PreviousVersionGuid);
		if (PreviousVersionGuid.IsValid() && PreviousVersionGuid != SelectedVersionGuid && SelectedVersion && PreviousVersion)
		{
			FText ChangelistDescriptions = GetChangelistText(InputFunctionCallNode->FunctionScript, PreviousVersion->Version, SelectedVersion->Version);
			if (!ChangelistDescriptions.IsEmpty())
			{
				UNiagaraStackEntry::FStackIssue UpgradeInfo(
	                EStackIssueSeverity::Info,
	                LOCTEXT("VersionUpgradeInfoSummary", "Version upgrade note"),
	                FText::Format(LOCTEXT("VersionUpgradeFormatChanges", "The version of this script was recently upgraded; here is a list of changes:\n{0}"), ChangelistDescriptions),
	                StackEditorDataKey,
	                true);
				OutIssues.Add(UpgradeInfo);
			}
		}
	}

	// Generate a note if the last version upgrade generated warnings
	if (InputFunctionCallNode->FunctionScript && InputFunctionCallNode->FunctionScript->IsVersioningEnabled() && InputFunctionCallNode->PythonUpgradeScriptWarnings.IsEmpty() == false)
	{
		UNiagaraStackEntry::FStackIssue PythonLog(
                    EStackIssueSeverity::Warning,
                    LOCTEXT("PythonLogSummary", "Python upgrade script log"),
                    FText::Format(LOCTEXT("VersionUpgradePythonLog", "Upgrading the version generated some warnings from the python script:\n{0}"), FText::FromString(InputFunctionCallNode->PythonUpgradeScriptWarnings)),
                    StackEditorDataKey,
                    true);
		OutIssues.Add(PythonLog);
	}
}

void FNiagaraStackGraphUtilities::CheckForDeprecatedEmitterVersion(TSharedPtr<FNiagaraEmitterViewModel> ViewModel, const FString& StackEditorDataKey, UNiagaraStackEntry::FStackIssueFixDelegate VersionUpgradeFix, TArray<UNiagaraStackEntry::FStackIssue>& OutIssues)
{
	if (!ViewModel)
	{
		return;
	}
	
	FVersionedNiagaraEmitter VersionedEmitter = ViewModel->GetParentEmitter();
	UNiagaraEmitter* Emitter = VersionedEmitter.Emitter;
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();

	if (!Emitter || !Emitter->IsVersioningEnabled())
	{
		return;
	}

	if (!EmitterData)
	{
		if (VersionUpgradeFix.IsBound())
		{
			TArray<UNiagaraStackEntry::FStackIssueFix> Fixes;
			Fixes.Add(UNiagaraStackEntry::FStackIssueFix(LOCTEXT("InvalidParentVersionFix", "Switch to exposed parent version."), VersionUpgradeFix));
			
			UNiagaraStackEntry::FStackIssue UpgradeVersion(
				EStackIssueSeverity::Error,
				LOCTEXT("MissingParentVersionFormat", "Invalid parent emitter"),
				LOCTEXT("MissingParentVersionLongFormat", "The parent version selected for this emitter does not exist.\nPlease select a valid version or changes cannot be merged down from the parent emitter."),
				StackEditorDataKey,
				false,
				Fixes);
			OutIssues.Add(UpgradeVersion);
		}
		return;
	}
	
	// Generate an issue if the emitter version is out of date
	if (VersionUpgradeFix.IsBound())
	{
		FNiagaraAssetVersion ExposedVersion = Emitter->GetExposedVersion();
		if (FNiagaraAssetVersion ReferencedVersion = EmitterData->Version; ReferencedVersion.MajorVersion < ExposedVersion.MajorVersion)
		{
			TArray<UNiagaraStackEntry::FStackIssueFix> Fixes;
			Fixes.Add(UNiagaraStackEntry::FStackIssueFix(LOCTEXT("UpgradeNewestParentFix", "Upgrade to newest parent version."), VersionUpgradeFix));
			
			FText ChangelistDescriptions = GetChangelistText(Emitter, ReferencedVersion, ExposedVersion);
			FText DeprecationDescription = EmitterData->bDeprecated && !EmitterData->DeprecationMessage.IsEmpty() ? FText::Format(LOCTEXT("DeprecatedEmitterVersionMessage", "\n\nDeprecation message:\n{0}"), EmitterData->DeprecationMessage) : FText();
			FText LongDescription = FText::Format(LOCTEXT("EmitterDeprecatedVersionFormat", "This emitter has a newer parent version available.\nYou can upgrade now, but major version upgrades can sometimes come with breaking changes! So check that everything is still working as expected afterwards.{0}{1}"),
				DeprecationDescription,
				ChangelistDescriptions.IsEmpty() ? FText() : FText::Format(LOCTEXT("DeprecatedVersionFormatChanges", "\n\nVersion change description:\n{0}"), ChangelistDescriptions));
			UNiagaraStackEntry::FStackIssue UpgradeVersion(
                EmitterData->bDeprecated ? EStackIssueSeverity::Warning : EStackIssueSeverity::Info,
                FText::Format(LOCTEXT("DeprecatedVersionSummaryFormat", "Upgrade emitter version: {0}.{1} -> {2}.{3}"),
                	FText::AsNumber(ReferencedVersion.MajorVersion), FText::AsNumber(ReferencedVersion.MinorVersion), FText::AsNumber(ExposedVersion.MajorVersion), FText::AsNumber(ExposedVersion.MinorVersion)),
                LongDescription,
                StackEditorDataKey,
                true,
                Fixes);
			OutIssues.Add(UpgradeVersion);
		}
	}

	// Generate a note of the changelist when the emitter version was manually changed
	{
		FGuid SelectedVersionGuid = VersionedEmitter.Version;
		FGuid PreviousVersionGuid = ViewModel->PreviousEmitterVersion;
		FVersionedNiagaraEmitterData* PreviousVersion = Emitter->GetEmitterData(PreviousVersionGuid);
		if (PreviousVersionGuid.IsValid() && PreviousVersionGuid != SelectedVersionGuid && PreviousVersion)
		{
			FText ChangelistDescriptions = GetChangelistText(Emitter, PreviousVersion->Version, EmitterData->Version);
			if (!ChangelistDescriptions.IsEmpty())
			{
				UNiagaraStackEntry::FStackIssue UpgradeInfo(
	                EStackIssueSeverity::Info,
	                LOCTEXT("VersionUpgradeInfoSummary", "Version upgrade note"),
	                FText::Format(LOCTEXT("EmitterVersionUpgradeFormatChanges", "The version of this emitter was recently upgraded; here is a list of changes:\n{0}"), ChangelistDescriptions),
	                StackEditorDataKey,
	                true);
				OutIssues.Add(UpgradeInfo);
			}
		}
	}

	// Generate a note if the last version upgrade generated warnings
	if (ViewModel->PythonUpgradeScriptWarnings.IsEmpty() == false)
	{
		UNiagaraStackEntry::FStackIssue PythonLog(
            EStackIssueSeverity::Warning,
            LOCTEXT("PythonLogSummary", "Python upgrade script log"),
            FText::Format(LOCTEXT("VersionUpgradePythonLog", "Upgrading the version generated some warnings from the python script:\n{0}"), FText::FromString(ViewModel->PythonUpgradeScriptWarnings)),
            StackEditorDataKey,
            true);
		OutIssues.Add(PythonLog);
	}
}

void GetGroupNodesRecursive(const TArray<UNiagaraNode*>& CurrentStartNodes, UNiagaraNode* EndNode, TArray<UNiagaraNode*>& OutAllNodes)
{
	FPinCollectorArray InputPins;
	FPinCollectorArray OutputPins;
	for (UNiagaraNode* CurrentStartNode : CurrentStartNodes)
	{
		if (OutAllNodes.Contains(CurrentStartNode) == false)
		{
			OutAllNodes.Add(CurrentStartNode);

			// Check input pins for this node to handle any UNiagaraNodeInput nodes which are wired directly to one of the group nodes.
			UEdGraphPin* ParameterMapInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*CurrentStartNode);
			if (ParameterMapInputPin != nullptr)
			{
				InputPins.Reset();
				CurrentStartNode->GetInputPins(InputPins);
				for (UEdGraphPin* InputPin : InputPins)
				{
					if (InputPin != ParameterMapInputPin)
					{
						for (UEdGraphPin* InputLinkedPin : InputPin->LinkedTo)
						{
							UNiagaraNode* LinkedNode = Cast<UNiagaraNode>(InputLinkedPin->GetOwningNodeUnchecked());
							if (LinkedNode != nullptr)
							{
								OutAllNodes.AddUnique(LinkedNode);
							}
						}
					}
				}
			}

			// Handle nodes connected to the output.
			if (CurrentStartNode != EndNode)
			{
				TArray<UNiagaraNode*> LinkedNodes;
				OutputPins.Reset();
				CurrentStartNode->GetOutputPins(OutputPins);
				for (UEdGraphPin* OutputPin : OutputPins)
				{
					for (UEdGraphPin* LinkedPin : OutputPin->LinkedTo)
					{
						UNiagaraNode* LinkedNode = Cast<UNiagaraNode>(LinkedPin->GetOwningNodeUnchecked());
						if (LinkedNode != nullptr)
						{
							LinkedNodes.Add(LinkedNode);
						}
					}
				}
				GetGroupNodesRecursive(LinkedNodes, EndNode, OutAllNodes);
			}
		}
	}
}

void FNiagaraStackGraphUtilities::FStackNodeGroup::GetAllNodesInGroup(TArray<UNiagaraNode*>& OutAllNodes) const
{
	GetGroupNodesRecursive(StartNodes, EndNode, OutAllNodes);
}

void FNiagaraStackGraphUtilities::GetStackNodeGroups(UNiagaraNode& StackNode, TArray<FNiagaraStackGraphUtilities::FStackNodeGroup>& OutStackNodeGroups)
{
	UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(StackNode);
	if (OutputNode != nullptr)
	{
		UNiagaraNodeInput* InputNode = GetEmitterInputNodeForStackNode(*OutputNode);
		if (InputNode != nullptr)
		{
			FStackNodeGroup InputGroup;
			InputGroup.StartNodes.Add(InputNode);
			InputGroup.EndNode = InputNode;
			OutStackNodeGroups.Add(InputGroup);

			TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
			GetOrderedModuleNodes(*OutputNode, ModuleNodes);
			for (UNiagaraNodeFunctionCall* ModuleNode : ModuleNodes)
			{
				UNiagaraNode& LastEndNode = *OutStackNodeGroups.Last().EndNode;
				UEdGraphPin* PreviousOutputPin = GetParameterMapOutputPin(LastEndNode);
				if (!ensureMsgf(PreviousOutputPin, TEXT("Could not find a valid parameter map output pin for %s"), *LastEndNode.GetPathName()))
				{
					OutStackNodeGroups.Empty();
					return; // unrecoverable, sorry.
				}
				FStackNodeGroup ModuleGroup;
				for (UEdGraphPin* PreviousOutputPinLinkedPin : PreviousOutputPin->LinkedTo)
				{
					if (PreviousOutputPinLinkedPin)
					{
						ModuleGroup.StartNodes.Add(CastChecked<UNiagaraNode>(PreviousOutputPinLinkedPin->GetOwningNode()));
					}
					else
					{
						UE_LOGF(LogNiagaraEditor, Log, "Invalid PreviousOutputPinLinkedPin in %ls", *ModuleNode->GetPathName());
					}
				}
				// previous tests could make this malformed.
				if (!ensureMsgf(!ModuleGroup.StartNodes.IsEmpty(), TEXT("ModuleGroup for %s has no StartNodes"), *ModuleNode->GetPathName()))
				{
					OutStackNodeGroups.Empty();
					return;
				}
				ModuleGroup.EndNode = ModuleNode;
				OutStackNodeGroups.Add(ModuleGroup);
			}

			UNiagaraNode& LastEndNode = *OutStackNodeGroups.Last().EndNode;
			UEdGraphPin* PreviousOutputPin = GetParameterMapOutputPin(LastEndNode);
			if (!ensureMsgf(PreviousOutputPin, TEXT("Could not find a valid parameter map output pin for %s"), *LastEndNode.GetPathName()))
			{
				OutStackNodeGroups.Empty();
				return; // unrecoverable, sorry.
			}
			FStackNodeGroup OutputGroup;
			for (UEdGraphPin* PreviousOutputPinLinkedPin : PreviousOutputPin->LinkedTo)
			{
				if (PreviousOutputPinLinkedPin)
				{
					OutputGroup.StartNodes.Add(CastChecked<UNiagaraNode>(PreviousOutputPinLinkedPin->GetOwningNode()));
				}
				else
				{
					UE_LOGF(LogNiagaraEditor, Log, "Invalid PreviousOutputPinLinkedPin in %ls", *StackNode.GetPathName());
				}
			}
			// previous tests could make this malformed.
			if (!ensureMsgf(!OutputGroup.StartNodes.IsEmpty(), TEXT("OutputGroup for %s has no StartNodes"), *StackNode.GetPathName()))
			{
				OutStackNodeGroups.Empty();
				return;
			}
			OutputGroup.EndNode = OutputNode;
			OutStackNodeGroups.Add(OutputGroup);
		}
	}
}

void FNiagaraStackGraphUtilities::DisconnectStackNodeGroup(const FStackNodeGroup& DisconnectGroup, const FStackNodeGroup& PreviousGroup, const FStackNodeGroup& NextGroup)
{
	UEdGraphPin* PreviousOutputPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*PreviousGroup.EndNode);
	BreakAllPinLinks(PreviousOutputPin);

	UEdGraphPin* DisconnectOutputPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*DisconnectGroup.EndNode);
	BreakAllPinLinks(DisconnectOutputPin);

	for (UNiagaraNode* NextStartNode : NextGroup.StartNodes)
	{
		UEdGraphPin* NextStartInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*NextStartNode);
		MakeLinkTo(PreviousOutputPin, NextStartInputPin);
	}
}

void FNiagaraStackGraphUtilities::ConnectStackNodeGroup(const FStackNodeGroup& ConnectGroup, const FStackNodeGroup& NewPreviousGroup, const FStackNodeGroup& NewNextGroup)
{
	UEdGraphPin* NewPreviousOutputPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*NewPreviousGroup.EndNode);
	BreakAllPinLinks(NewPreviousOutputPin);

	for (UNiagaraNode* ConnectStartNode : ConnectGroup.StartNodes)
	{
		UEdGraphPin* ConnectInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*ConnectStartNode);
		MakeLinkTo(NewPreviousOutputPin, ConnectInputPin);

	}

	UEdGraphPin* ConnectOutputPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*ConnectGroup.EndNode);

	for (UNiagaraNode* NewNextStartNode : NewNextGroup.StartNodes)
	{
		UEdGraphPin* NewNextStartInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*NewNextStartNode);
		MakeLinkTo(ConnectOutputPin, NewNextStartInputPin);
	}
}

DECLARE_DELEGATE_RetVal_OneParam(bool, FInputSelector, UNiagaraStackFunctionInput*);

void InitializeStackFunctionInputsInternal(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel, UNiagaraStackEditorData& StackEditorData, UNiagaraNodeFunctionCall& ModuleNode, UNiagaraNodeFunctionCall& InputFunctionCallNode, FInputSelector InputSelector)
{
	UNiagaraStackScriptHierarchyRoot* FunctionHierarchyRoot = NewObject<UNiagaraStackScriptHierarchyRoot>(GetTransientPackage()); 
	UNiagaraStackEntry::FRequiredEntryData RequiredEntryData(SystemViewModel, EmitterViewModel, NAME_None, NAME_None, StackEditorData);
	FunctionHierarchyRoot->Initialize(RequiredEntryData, ModuleNode, InputFunctionCallNode, FString());
	FunctionHierarchyRoot->RefreshChildren();

	// Reset all direct inputs on this function to initialize data interfaces and default dynamic inputs.
	TArray<UNiagaraStackFunctionInput*> StackFunctionInputs;
	FunctionHierarchyRoot->GetUnfilteredChildrenOfType(StackFunctionInputs);
	for (UNiagaraStackFunctionInput* StackFunctionInput : StackFunctionInputs)
	{
		if (StackFunctionInput != nullptr && (InputSelector.IsBound() == false || InputSelector.Execute(StackFunctionInput)) && StackFunctionInput->CanReset())
		{
			StackFunctionInput->Reset();
		}
	}

	FunctionHierarchyRoot->Finalize();
	SystemViewModel->NotifyDataObjectChanged(TArray<UObject*>(), ENiagaraDataObjectChange::Unknown);
}

void FNiagaraStackGraphUtilities::InitializeStackFunctionInputs(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel, UNiagaraStackEditorData& StackEditorData, UNiagaraNodeFunctionCall& ModuleNode, UNiagaraNodeFunctionCall& InputFunctionCallNode)
{
	InitializeStackFunctionInputsInternal(SystemViewModel, EmitterViewModel, StackEditorData, ModuleNode, InputFunctionCallNode, FInputSelector());
}

void FNiagaraStackGraphUtilities::InitializeStackFunctionInput(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel, UNiagaraStackEditorData& StackEditorData, UNiagaraNodeFunctionCall& ModuleNode, UNiagaraNodeFunctionCall& InputFunctionCallNode, FName InputName)
{
	FInputSelector InputSelector;
	InputSelector.BindLambda([&InputName](UNiagaraStackFunctionInput* Input)
	{
		return Input->GetInputParameterHandle().GetName() == InputName;
	});
	InitializeStackFunctionInputsInternal(SystemViewModel, EmitterViewModel, StackEditorData, ModuleNode, InputFunctionCallNode, InputSelector);
}

FString FNiagaraStackGraphUtilities::StackKeys::GenerateStackFunctionInputEditorDataKey(const UNiagaraNodeFunctionCall& FunctionCallNode, FNiagaraParameterHandle InputParameterHandle)
{
	return FunctionCallNode.GetFunctionName() + InputParameterHandle.GetParameterHandleString().ToString();
}

FString FNiagaraStackGraphUtilities::StackKeys::GenerateStackModuleEditorDataKey(const UNiagaraNodeFunctionCall& ModuleNode)
{
	return ModuleNode.NodeGuid.ToString(EGuidFormats::DigitsWithHyphens);
}

FString FNiagaraStackGraphUtilities::StackKeys::GenerateStackRendererEditorDataKey(const UNiagaraRendererProperties& Renderer)
{
	return FString::Printf(TEXT("Renderer-%s"), *Renderer.GetName());
}

void FNiagaraStackGraphUtilities::ExtractInputPinsFromHistory(FNiagaraParameterMapHistory& History, UEdGraph* FunctionGraph, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options, TArray<const UEdGraphPin*>& OutPins)
{
	const int32 ModuleNamespaceLength = FNiagaraConstants::ModuleNamespaceString.Len();

	for (int32 i = 0; i < History.Variables.Num(); i++)
	{
		FNiagaraVariable& Variable = History.Variables[i];
		const TArray<FNiagaraParameterMapHistory::FReadHistory>& ReadHistory = History.PerVariableReadHistory[i];

		// A read is only really exposed if it's the first read and it has no corresponding write.
		if (ReadHistory.Num() > 0 && ReadHistory[0].PreviousWritePin.Pin == nullptr)
		{
			const UEdGraphPin* InputPin = ReadHistory[0].ReadPin.Pin;

			// Make sure that the module input is from the called graph, and not a nested graph.
			UEdGraph* NodeGraph = InputPin->GetOwningNode()->GetGraph();
			if (NodeGraph == FunctionGraph)
			{
				if (Options == FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::AllInputs)
				{
					OutPins.Add(InputPin);
				}
				else
				{
					FNameBuilder InputPinName(InputPin->PinName);
					FStringView InputPinView(InputPinName);
					if (InputPinView.StartsWith(FNiagaraConstants::ModuleNamespaceString)
						&& InputPinView.Len() > ModuleNamespaceLength
						&& InputPinView[ModuleNamespaceLength] == TCHAR('.'))
					{
						OutPins.Add(InputPin);
					}
				}
			}
		}
	}
}

void FNiagaraStackGraphUtilities::GetStackFunctionInputsLegacy(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, TSet<FNiagaraVariable>& OutHiddenVariables, FCompileConstantResolver ConstantResolver, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled)
{
	TArray<FNiagaraVariable> StaticVars;
	FNiagaraStackGraphUtilitiesImpl::GetFunctionStaticVariables(FunctionCallNode, StaticVars);

	FNiagaraEditorModule::Get().GetGraphDataCache().GetStackFunctionInputs(FunctionCallNode, StaticVars, OutInputVariables, OutHiddenVariables, ConstantResolver, Options, bIgnoreDisabled);
}

void FNiagaraStackGraphUtilities::GetStackFunctionInputsLegacy(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, FCompileConstantResolver ConstantResolver, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled)
{
	TArray<FNiagaraVariable> StaticVars;
	FNiagaraStackGraphUtilitiesImpl::GetFunctionStaticVariables(FunctionCallNode, StaticVars);

	FNiagaraEditorModule::Get().GetGraphDataCache().GetStackFunctionInputs(FunctionCallNode, StaticVars, OutInputVariables, ConstantResolver, Options, bIgnoreDisabled);
}

void FNiagaraStackGraphUtilities::GetStackFunctionInputsLegacy(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, ENiagaraGetStackFunctionInputPinsOptions Options /*= ENiagaraGetStackFunctionInputPinsOptions::AllInputs*/, bool bIgnoreDisabled /*= false*/)
{
	FCompileConstantResolver EmptyResolver;
	GetStackFunctionInputsLegacy(FunctionCallNode, OutInputVariables, EmptyResolver, Options, bIgnoreDisabled);
}

namespace UE::Niagara::TraversalCache::StackGraphUtilities
{
	void ConstantResolverToTopLevelScriptContext(const FCompileConstantResolver& ConstantResolver, const UNiagaraNodeFunctionCall& FunctionCallNode, FTopLevelScriptStaticContext& OutTopLevelScriptContext)
	{
		const UNiagaraSystem* System = nullptr;
		const FVersionedNiagaraEmitterData* EmitterData = nullptr;
		const UNiagaraScript* Script = nullptr;

		// Get the system.
		if (ConstantResolver.GetSystem() != nullptr)
		{
			System = ConstantResolver.GetSystem();
		}
		else if (ConstantResolver.GetEmitter() != nullptr)
		{
			System = ConstantResolver.GetEmitter()->GetTypedOuter<UNiagaraSystem>();
		}
		else
		{
			System = FunctionCallNode.GetTypedOuter<UNiagaraSystem>();
		}

		if (System == nullptr)
		{
			return;
		}

		// Get the emitter if available
		bool bIsEmitterOrParticleScript = UNiagaraScript::IsEmitterScript(ConstantResolver.GetUsage()) || UNiagaraScript::IsParticleScript(ConstantResolver.GetUsage());
		if (bIsEmitterOrParticleScript)
		{
			if (ConstantResolver.GetEmitter() != nullptr)
			{
				EmitterData = ConstantResolver.GetEmitter()->GetLatestEmitterData();
			}
			else
			{
				const UNiagaraEmitter* Emitter = FunctionCallNode.GetTypedOuter<UNiagaraEmitter>();
				EmitterData = Emitter->GetLatestEmitterData();
			}
		}

		if (bIsEmitterOrParticleScript && EmitterData == nullptr)
		{
			return;
		}

		// Get the script.
		if (UNiagaraScript::IsSystemScript(ConstantResolver.GetUsage()))
		{
			Script = ConstantResolver.GetUsage() == ENiagaraScriptUsage::SystemSpawnScript
				? System->GetSystemSpawnScript()
				: System->GetSystemUpdateScript();
		}
		else if(bIsEmitterOrParticleScript)
		{
			const UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(FunctionCallNode);
			Script = OutputNode != nullptr ? EmitterData->GetScript(OutputNode->GetUsage(), OutputNode->GetUsageId()) : nullptr;
		}

		if (Script == nullptr)
		{
			return;
		}

		FStaticContextBuilder::CreateTopLevelScriptContext(*System, EmitterData, *Script, OutTopLevelScriptContext);
	}

	EStackFunctionReadFilterFlags InputOptionsToReadFilterFlags(FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled)
	{
		bool bInputsOnly = Options == FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly;
		return 
			(bInputsOnly ? EStackFunctionReadFilterFlags::InputsOnly : EStackFunctionReadFilterFlags::None) |
			(bIgnoreDisabled ? EStackFunctionReadFilterFlags::EnabledOnly : EStackFunctionReadFilterFlags::None);
	}
}

void FNiagaraStackGraphUtilities::GetStackFunctionInputs(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, TSet<FNiagaraVariable>& OutHiddenVariables, FCompileConstantResolver ConstantResolver, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_StackGraphUtilities_GetStackFunctionInputs);
	if (GbNiagaraEnableTraversalCache == 0)
	{
		GetStackFunctionInputsLegacy(FunctionCallNode, OutInputVariables, OutHiddenVariables, ConstantResolver, Options, bIgnoreDisabled);
	}
	else
	{
		using namespace UE::Niagara::TraversalCache;
		FTopLevelScriptStaticContext ScriptContext;
		StackGraphUtilities::ConstantResolverToTopLevelScriptContext(ConstantResolver, FunctionCallNode, ScriptContext);
		EStackFunctionReadFilterFlags FilterFlags = StackGraphUtilities::InputOptionsToReadFilterFlags(Options, bIgnoreDisabled);
		FTraversalCache::GetStackFunctionReads(ScriptContext, FunctionCallNode, OutInputVariables, OutHiddenVariables, FilterFlags);
	}
}

void FNiagaraStackGraphUtilities::GetStackFunctionInputs(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, FCompileConstantResolver ConstantResolver, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_StackGraphUtilities_GetStackFunctionInputs);
	if (GbNiagaraEnableTraversalCache == 0)
	{
		GetStackFunctionInputsLegacy(FunctionCallNode, OutInputVariables, ConstantResolver, Options, bIgnoreDisabled);
	}
	else
	{
		using namespace UE::Niagara::TraversalCache;
		TSet<FNiagaraVariable> HiddenVariables;
		FTopLevelScriptStaticContext ScriptContext;
		StackGraphUtilities::ConstantResolverToTopLevelScriptContext(ConstantResolver, FunctionCallNode, ScriptContext);
		EStackFunctionReadFilterFlags FilterFlags = StackGraphUtilities::InputOptionsToReadFilterFlags(Options, bIgnoreDisabled);
		FTraversalCache::GetStackFunctionReads(ScriptContext, FunctionCallNode, OutInputVariables, HiddenVariables, FilterFlags);
	}
}

void FNiagaraStackGraphUtilities::GetStackFunctionInputs(
	const UNiagaraNodeFunctionCall& FunctionCallNode,
	TArray<FNiagaraVariable>& OutInputVariables,
	ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_StackGraphUtilities_GetStackFunctionInputs);
	if (GbNiagaraEnableTraversalCache == 0)
	{
		GetStackFunctionInputsLegacy(FunctionCallNode, OutInputVariables, Options, bIgnoreDisabled);
	}
	else
	{
		using namespace UE::Niagara::TraversalCache;
		TSet<FNiagaraVariable> HiddenVariables;
		FTopLevelScriptStaticContext ScriptContext;
		EStackFunctionReadFilterFlags FilterFlags = StackGraphUtilities::InputOptionsToReadFilterFlags(Options, bIgnoreDisabled);
		FTraversalCache::GetStackFunctionReads(ScriptContext, FunctionCallNode, OutInputVariables, HiddenVariables, FilterFlags);
	}
}

void FNiagaraStackGraphUtilities::GatherInputRelationsForStack(FInputDataCollection& State, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_StackGraphUtilities_GatherInputRelationsForStack);

	FText UncategorizedName = UNiagaraStackValueCollection::UncategorizedName;
	TSet<const UEdGraphPin*> HiddenPins;
    TArray<const UEdGraphPin*> InputPins;
	
	UHierarchyRoot* SummaryRoot = EmitterViewModel->GetEditorData().GetSummaryRoot();

	TArray<UNiagaraNodeOutput*> OutputNodes = GetAllOutputNodes(EmitterViewModel);

	TSharedRef<FNiagaraScriptGraphViewModel> GraphViewModel = EmitterViewModel->GetSharedScriptViewModel()->GetGraphViewModel();

	TArray<UNiagaraNodeFunctionCall*> FunctionCallNodes;
	GraphViewModel->GetGraph()->GetNodesOfClass<UNiagaraNodeFunctionCall>(FunctionCallNodes);
	
	// first we create a map so it's easy to access all relevant module & dynamic input nodes
	TMap<FGuid, UNiagaraNodeFunctionCall*> NodeGuidToModuleNodeMap;
	for(UNiagaraNodeFunctionCall* FunctionCallNode : FunctionCallNodes)
	{
		NodeGuidToModuleNodeMap.Add(FunctionCallNode->NodeGuid, FunctionCallNode);
	}

	// the inputs found in the root are the ones we want to display; these might contain outdated entries so it is important to sync the root view model to data before calling this
	TArray<UNiagaraHierarchyModuleInput*> ModuleInputs;
	SummaryRoot->GetChildrenOfType(ModuleInputs, true);
	
	// now we create a map from summary input to the function node it (modules, dynamic inputs, assignment nodes etc.) maps to
	TMap<UNiagaraHierarchyModuleInput*, UNiagaraNodeFunctionCall*> HierarchyInputToFunctionCallNodeMap;
	TMap<UNiagaraHierarchyModuleInput*, bool> HierarchyInputToStaticSwitchMap;
	for(UNiagaraHierarchyModuleInput* Input : ModuleInputs)
	{
		HierarchyInputToFunctionCallNodeMap.Add(Input, NodeGuidToModuleNodeMap[Input->GetPersistentIdentity().Guids[0]]); 
	}

	// then we create a map from summary input to its actual input script variable. This does not include children inputs yet as the root only contains parent level inputs.
	TMap<UNiagaraHierarchyModuleInput*, TObjectPtr<UNiagaraScriptVariable>> HierarchyInputToScriptVariableMap;
	TMap<UNiagaraHierarchyModuleInput*, FNiagaraVariable> HierarchyInputToAssignmentMap;
	for(UNiagaraHierarchyModuleInput* Input : ModuleInputs)
	{
		UNiagaraGraph* GraphWithInput = HierarchyInputToFunctionCallNodeMap[Input]->GetCalledGraph();
		if(ensure(GraphWithInput != nullptr))
		{
			bool bIsModuleInput = Input->GetPersistentIdentity().Guids.Num() == 2;

			// the input guid is only valid for module inputs, not assignment targets
			if(bIsModuleInput)
			{
				TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>> MetaDataMap = GraphWithInput->GetAllMetaData();
				TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptVariables;
				MetaDataMap.GenerateValueArray(ScriptVariables);

				FGuid InputGuid = Input->GetPersistentIdentity().Guids[1];
				TObjectPtr<UNiagaraScriptVariable>* MatchingScriptVariable = ScriptVariables.FindByPredicate([InputGuid](TObjectPtr<UNiagaraScriptVariable> ScriptVariableCandidate)
				{
					return ScriptVariableCandidate->Metadata.GetVariableGuid() == InputGuid;
				});

				if(ensure(MatchingScriptVariable != nullptr))
				{
					TOptional<bool> bIsStaticSwitch = GraphWithInput->IsStaticSwitch((*MatchingScriptVariable)->Variable);
					HierarchyInputToScriptVariableMap.Add(Input, *MatchingScriptVariable);
					HierarchyInputToStaticSwitchMap.Add(Input, bIsStaticSwitch.GetValue());
				}
			}
			// otherwise we have to use the name
			else
			{
				TArray<FNiagaraVariable> Variables;
				GraphWithInput->GetAllVariables(Variables);

				FName InputName = Input->GetPersistentIdentity().Names[0];
				FNiagaraVariableBase* MatchingVariable = Variables.FindByPredicate([InputName](FNiagaraVariable& Candidate)
				{
					return Candidate.GetName() == InputName;
				});

				if(ensure(MatchingVariable != nullptr))
				{
					HierarchyInputToAssignmentMap.Add(Input, *MatchingVariable);
				}
			}
		}
	}

	TMap<UNiagaraHierarchyModuleInput*, TArray<FGuid>> HierarchyInputToChildrenGuidMap;
	TMap<FGuid, TObjectPtr<UNiagaraScriptVariable>> ChildrenGuidToScriptVariablesMap;
	// then we go through the child inputs and add them to the maps as well
	for(UNiagaraHierarchyModuleInput* Input : ModuleInputs)
	{
		UNiagaraGraph* AssetGraph = HierarchyInputToFunctionCallNodeMap[Input]->GetCalledGraph();
		if(ensure(AssetGraph != nullptr))
		{
			TArray<FNiagaraVariable> Variables;
			AssetGraph->GetAllVariables(Variables);

			FNiagaraVariable ThisInput;
			if(Input->GetPersistentIdentity().Guids.Num() == 2)
			{
				ThisInput = HierarchyInputToScriptVariableMap[Input]->Variable;
			}
			else
			{
				FNiagaraVariable* FoundTargetVariable = Variables.FindByPredicate([InputName = Input->GetPersistentIdentity().Names[0]](FNiagaraVariable& CandidateVariable)
				{
					return CandidateVariable.GetName() == InputName;	
				});

				if(FoundTargetVariable)
				{
					ThisInput = *FoundTargetVariable;
				}
			}

			ensure(ThisInput.IsValid());
			
			TArray<FNiagaraVariable> ChildrenInputs;
			FNiagaraParameterHandle ThisInputHandle(ThisInput.GetName());

			TMap<FGuid, int32> ChildrenGuidSortOrderMap;
			TArray<FGuid> ChildrenInputGuids;
			for(const FNiagaraVariable& Variable : Variables)
			{
				bool bIsModuleInput = Variable.IsInNameSpace(FNiagaraConstants::ModuleNamespaceString);
				TOptional<bool> bIsStaticSwitchInputOptional = AssetGraph->IsStaticSwitch(Variable);
				if(!bIsModuleInput && !bIsStaticSwitchInputOptional.Get(false))
				{
					continue;
				}

				TOptional<FNiagaraVariableMetaData> VariableMetaData = AssetGraph->GetMetaData(Variable);
				if(VariableMetaData.IsSet())
				{
					// @TODO Handle Deprecated values
					if(VariableMetaData.GetValue().GetParentAttribute_DEPRECATED() == ThisInputHandle.GetName())
					{
						HierarchyInputToChildrenGuidMap.FindOrAdd(Input).Add(VariableMetaData.GetValue().GetVariableGuid());
						ChildrenGuidToScriptVariablesMap.Add(VariableMetaData->GetVariableGuid(), AssetGraph->GetScriptVariable(Variable));
						ChildrenGuidSortOrderMap.Add(VariableMetaData->GetVariableGuid(), VariableMetaData->GetEditorSortPriority_DEPRECATED());
					}
				}
			}
			
			// we sort children by sort order specified in the metadata
			if(HierarchyInputToChildrenGuidMap.Contains(Input))
			{
				HierarchyInputToChildrenGuidMap[Input].Sort([&ChildrenGuidSortOrderMap](const FGuid& GuidA, const FGuid& GuidB)
				{
					return ChildrenGuidSortOrderMap[GuidA] < ChildrenGuidSortOrderMap[GuidB];
				});
			}
		}
	}

	State.NodeGuidToModuleNodeMap = NodeGuidToModuleNodeMap;
	State.HierarchyInputToAssignmentMap = HierarchyInputToAssignmentMap;
	State.HierarchyInputToScriptVariableMap = HierarchyInputToScriptVariableMap;
	State.HierarchyInputToChildrenGuidMap = HierarchyInputToChildrenGuidMap;
	State.ChildrenGuidToScriptVariablesMap = ChildrenGuidToScriptVariablesMap;
}

void FNiagaraStackGraphUtilities::GetStackFunctionInputPinsWithoutCache(
	const UNiagaraNodeFunctionCall& FunctionCallNode,
	TConstArrayView<FNiagaraVariable> StaticVars,
	TArray<const UEdGraphPin*>& OutInputPins,
	const FCompileConstantResolver& ConstantResolver,
	ENiagaraGetStackFunctionInputPinsOptions Options,
	bool bIgnoreDisabled,
	bool bFilterForCompilation)
{
	FNiagaraParameterMapHistoryBuilder Builder;
	Builder.SetIgnoreDisabled(bIgnoreDisabled);
	*Builder.ConstantResolver = ConstantResolver;
	Builder.RegisterExternalStaticVariables(StaticVars);
	
	/* The line below represents a compromise. It basically says don't look up the static variable from the 
	static variables list set in RegisterExternalStaticVariables as a rapid iteration parameter. Why? Rapid Iteration Parameters
	may be stale and linger around from prior changes that have been abandoned. We keep them around so that when you toggle back
	in the UI, you haven't also lost your last set values for that version. So therefore, we can't officially remove them.
	
	The issue comes in when a static variable is set via some other means, linking to another variable for example. When generating
	the UI, it *should* show anything that would match up to the linked variable, but because this is a parameter map history
	traversal that is in isolation on the node and doesn't include upstream set nodes, it will fail and just look up the rapid iteration
	version if left to its' own devices. 
	
	Trial 1 at a fix was to cache all the values at the root of the graph and look them up when we build the static variables list
	we use above. That isn't good because that version skips all disabled modules and we want the UI to be consistent regardless 
	of disabled module status.
	
	Trial 2 was to back up and include the precursor gets/sets. This fails for the same reason, ultimately something will
	be bound to a variable that will assume that it needs to look up a bogus rapid iteration value and we end up back here.
	
	Trial 3 just says, we know the static variables list above includes basically all relevant intermediate static values. So if 
	we just circumvent the logic in the parameter map history traversal that looks up by rapid iteration parameter, it will
	find the right "cached" value. So we just do that below.
	 */
	Builder.SetIgnoreStaticRapidIterationParameters(true);

	// if we are only dealing with the module input pins then we don't need to delve deep into the graph
	if (Options == ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly)
	{
		Builder.MaxGraphDepthTraversal = 1;
	}

	FunctionCallNode.BuildParameterMapHistory(Builder, false, bFilterForCompilation);

	OutInputPins.Empty();

	if (Builder.Histories.Num() == 1)
	{
		ExtractInputPinsFromHistory(Builder.Histories[0], FunctionCallNode.GetCalledGraph(), Options, OutInputPins);
	}
}

TArray<UNiagaraNodeOutput*> FNiagaraStackGraphUtilities::GetAllOutputNodes(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TSharedRef<FNiagaraScriptGraphViewModel> GraphViewModel = EmitterViewModel->GetSharedScriptViewModel()->GetGraphViewModel();

	TArray<UNiagaraNodeOutput*> OutputNodes;
	GraphViewModel->GetGraph()->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);
	
	return OutputNodes;
}

UNiagaraNodeFunctionCall* FNiagaraStackGraphUtilities::FindModuleNode(FGuid ModuleNodeGuid, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TArray<UNiagaraNodeOutput*> OutputNodes = GetAllOutputNodes(EmitterViewModel);
	
	for(UNiagaraNodeOutput* OutputNode : OutputNodes)
	{
		TArray<UNiagaraNodeFunctionCall*> FunctionCalls;
		FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*OutputNode, FunctionCalls);
		
		for(UNiagaraNodeFunctionCall* FunctionCall : FunctionCalls)
		{
			if(FunctionCall->NodeGuid == ModuleNodeGuid)
			{
				return FunctionCall;
			}
		}
	}

	return nullptr;
}

UNiagaraNodeFunctionCall* FNiagaraStackGraphUtilities::FindDynamicInputNodeForInput(UNiagaraNodeFunctionCall& OwningFunctionNode, FName UnaliasedParameterName)
{
	FNiagaraParameterHandle InputHandle(UnaliasedParameterName);
	FNiagaraParameterHandle AliasedHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(InputHandle, &OwningFunctionNode);

	if(UEdGraphPin* OverridePin = FNiagaraStackGraphUtilities::GetStackFunctionInputOverridePin(OwningFunctionNode, AliasedHandle))
	{
		if(OverridePin->LinkedTo.Num() == 1 && OverridePin->LinkedTo[0] != nullptr && OverridePin->LinkedTo[0]->GetOwningNode() != nullptr)
		{
			UEdGraphNode* LinkedNode = OverridePin->LinkedTo[0]->GetOwningNode();
			return Cast<UNiagaraNodeFunctionCall>(LinkedNode);
		}
	}

	return nullptr;
}

UNiagaraNodeFunctionCall* FNiagaraStackGraphUtilities::FindFunctionCallNode(FGuid FunctionCallGuid,	TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TSharedRef<FNiagaraScriptGraphViewModel> GraphViewModel = EmitterViewModel->GetSharedScriptViewModel()->GetGraphViewModel();

	if (UNiagaraGraph* Graph = GraphViewModel->GetGraph())
	{
		TArray<UNiagaraNodeFunctionCall*> FunctionCalls;
		Graph->GetNodesOfClass<UNiagaraNodeFunctionCall>(FunctionCalls);

		for(UNiagaraNodeFunctionCall* FunctionCall : FunctionCalls)
		{
			if(FunctionCall->NodeGuid == FunctionCallGuid)
			{
				return FunctionCall;
			}
		}
	}

	return nullptr;
}

TArray<UNiagaraNodeFunctionCall*> FNiagaraStackGraphUtilities::FindModuleNodesForSimulationStage(UNiagaraSimulationStageBase& SimStage, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TArray<UNiagaraNodeOutput*> OutputNodes = GetAllOutputNodes(EmitterViewModel);
	for(UNiagaraNodeOutput* Output : OutputNodes)
	{
		if(Output->GetUsage() == ENiagaraScriptUsage::ParticleSimulationStageScript && Output->GetUsageId() == SimStage.Script->GetUsageId())
		{
			TArray<UNiagaraNodeFunctionCall*> SimStageModules;
			GetOrderedModuleNodes(*Output, SimStageModules);
			return SimStageModules;
		}
	}

	return {};
}

TArray<UNiagaraNodeFunctionCall*> FNiagaraStackGraphUtilities::FindModuleNodesForEventHandler(FNiagaraEventScriptProperties& EventScriptProperties, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TArray<UNiagaraNodeOutput*> OutputNodes = GetAllOutputNodes(EmitterViewModel);
	for(UNiagaraNodeOutput* Output : OutputNodes)
	{
		if(Output->GetUsage() == ENiagaraScriptUsage::ParticleEventScript && Output->GetUsageId() == EventScriptProperties.Script->GetUsageId())
		{
			TArray<UNiagaraNodeFunctionCall*> EventHandlerModuleNodes;
			GetOrderedModuleNodes(*Output, EventHandlerModuleNodes);
			return EventHandlerModuleNodes;
		}
	}

	return {};
}

UNiagaraNodeAssignment* FNiagaraStackGraphUtilities::FindAssignmentNode(FGuid AssignmentNodeGuid,	TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	UNiagaraNodeFunctionCall* AssignmentNode = FindModuleNode(AssignmentNodeGuid, EmitterViewModel);

	if(AssignmentNode)
	{
		return CastChecked<UNiagaraNodeAssignment>(AssignmentNode);
	}

	return nullptr;
}

TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData> FNiagaraStackGraphUtilities::FindAssignmentInputData(const UNiagaraNodeAssignment& AssignmentNode, FName VariableName, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_StackGraphUtilities_FindAssignmentInputData);
	
	FMatchingFunctionInputData Result;
	UNiagaraNodeAssignment* AssignmentNodeNonConst = const_cast<UNiagaraNodeAssignment*>(&AssignmentNode);
	Result.FunctionCallNode = AssignmentNodeNonConst;
	
	FCompileConstantResolver ConstantResolver(EmitterViewModel->GetEmitter(), GetOutputNodeUsage(AssignmentNode));

	// our goal is to find the correct variable's metadata from the graph
	TOptional<FNiagaraVariableMetaData> MetaData;
				
	// first, we check the normal module inputs
	TArray<FNiagaraVariable> InputVariables;
	TSet<FNiagaraVariable> HiddenVariables;

	FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options = FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::AllInputs;
	FNiagaraStackGraphUtilities::GetStackFunctionInputs(AssignmentNode, InputVariables, HiddenVariables, ConstantResolver, Options, false);
	
	for(const FNiagaraVariable& InputVariable : InputVariables)
	{
		if (InputVariable.GetType().IsValid() == false)
		{
			continue;
		}

		// the input variables we are iterating on have the Module prefix so we add it if necessary
		FNiagaraParameterHandle SearchedParameterHandle(VariableName);
		if(!SearchedParameterHandle.IsModuleHandle())
		{
			FString VariableNameWithModulePrefix = FNiagaraConstants::ModuleNamespaceString + ".";
			VariableNameWithModulePrefix.Append(VariableName.ToString());
			SearchedParameterHandle = FNiagaraParameterHandle(FName(VariableNameWithModulePrefix));
		}
		
		if(InputVariable.GetName().IsEqual(SearchedParameterHandle.GetParameterHandleString()))
		{
			// we use the name without the prefix as that's what's stored in the hierarchy data
			if(SearchedParameterHandle.IsModuleHandle())
			{
				FString VariableNameWithoutPrefix = SearchedParameterHandle.GetParameterHandleString().ToString().RightChop(FNiagaraConstants::ModuleNamespaceString.Len() + 1);
				SearchedParameterHandle = FNiagaraParameterHandle(FName(VariableNameWithoutPrefix));
			}
			
			Result.InputName = SearchedParameterHandle.GetParameterHandleString();
			Result.Type = InputVariable.GetType();
			Result.MetaData = FNiagaraVariableMetaData();
			Result.bIsStatic = false;
			Result.bIsHidden = HiddenVariables.Contains(InputVariable);
			
			return Result;
		}
	}

	return TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData>();
}

TArray<UEdGraphPin*> FNiagaraStackGraphUtilities::GetUnusedFunctionInputPins(const UNiagaraNodeFunctionCall& FunctionCallNode, FCompileConstantResolver ConstantResolver)
{
	UNiagaraGraph* FunctionGraph = FunctionCallNode.GetCalledGraph();
	if (!FunctionGraph || FunctionCallNode.FunctionScript->Usage != ENiagaraScriptUsage::Module)
	{
		return TArray<UEdGraphPin*>();
	}
	
	// Set the static switch values so we traverse the correct node paths
	FPinCollectorArray InputPins;
	FunctionCallNode.GetInputPins(InputPins);

	FNiagaraEditorUtilities::SetStaticSwitchConstants(FunctionGraph, InputPins, ConstantResolver);

	// Find the start node for the traversal
	UNiagaraNodeOutput* OutputNode = FunctionGraph->FindOutputNode(ENiagaraScriptUsage::Module);
	if (OutputNode == nullptr)
	{
		return TArray<UEdGraphPin*>();
	}

	// Get the used function parameters from the parameter map set node linked to the function's input pin.
	// Note that this is only valid for module scripts, not function scripts.
	TArray<UEdGraphPin*> ResultPins;
	FString FunctionScriptName = FunctionCallNode.GetFunctionName();
	if (InputPins.Num() > 0 && InputPins[0]->LinkedTo.Num() > 0)
	{
		UNiagaraNodeParameterMapSet* ParamMapNode = Cast<UNiagaraNodeParameterMapSet>(InputPins[0]->LinkedTo[0]->GetOwningNode());
		if (ParamMapNode)
		{
			InputPins.Reset();
			ParamMapNode->GetInputPins(InputPins);
			for (UEdGraphPin* Pin : InputPins)
			{
				FString PinName = Pin->PinName.ToString();
				if (PinName.StartsWith(FunctionScriptName + "."))
				{
					ResultPins.Add(Pin);
				}
			}
		}
	}
	if (ResultPins.Num() == 0)
	{
		return ResultPins;
	}

	// Find reachable nodes
	TArray<UNiagaraNode*> ReachedNodes;
	FunctionGraph->BuildTraversal(ReachedNodes, OutputNode, true);

	FPinCollectorArray OutPins;
	// We only care about reachable parameter map get nodes with module inputs
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	for (UNiagaraNode* Node : ReachedNodes)
	{
		UNiagaraNodeParameterMapGet* ParamMapNode = Cast<UNiagaraNodeParameterMapGet>(Node);
		if (ParamMapNode)
		{
			OutPins.Reset();
			ParamMapNode->GetOutputPins(OutPins);
			for (UEdGraphPin* OutPin : OutPins)
			{
				FString OutPinName = OutPin->PinName.ToString();
				if (!OutPinName.RemoveFromStart(TEXT("Module.")) || OutPin->LinkedTo.Num() == 0)
				{
					continue;
				}
				for (UEdGraphPin* Pin : ResultPins)
				{
					if (Pin->GetName() == FunctionScriptName + "." + OutPinName && Pin->PinType == OutPin->PinType)
					{
						ResultPins.RemoveSwap(Pin);
						break;
					}
				}
			}
		}
	}
	return ResultPins;
}

void FNiagaraStackGraphUtilities::GetStackFunctionStaticSwitchPinsLegacy(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<UEdGraphPin*>& OutInputPins, TSet<UEdGraphPin*>& OutHiddenPins,
	FCompileConstantResolver& ConstantResolver)
{
	const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(FunctionCallNode.GetSchema());
	UNiagaraGraph* FunctionCallGraph = FunctionCallNode.GetCalledGraph();
	if (FunctionCallGraph == nullptr)
	{
		return;
	}

	FPinCollectorArray InputPins;
	FunctionCallNode.GetInputPins(InputPins);
	FNiagaraEditorUtilities::SetStaticSwitchConstants(FunctionCallGraph, InputPins, ConstantResolver);

	FVersionedNiagaraEmitter OuterEmitter = FNiagaraStackGraphUtilitiesImpl::GetOuterEmitter(&FunctionCallNode);
	UNiagaraSystem* System = FunctionCallNode.GetTypedOuter<UNiagaraSystem>();

	TArray<FNiagaraVariable> StaticVars;
	TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalData;
	FString EmitterName;
	FNiagaraAliasContext::ERapidIterationParameterMode Mode = FNiagaraAliasContext::ERapidIterationParameterMode::StaticVariables;
	if (OuterEmitter.GetEmitterData())
	{
		CachedTraversalData = OuterEmitter.Emitter->GetCachedTraversalData(OuterEmitter.Version);
		EmitterName = OuterEmitter.Emitter->GetUniqueEmitterName();
	}
	else if (System)
	{
		CachedTraversalData = System->GetCachedTraversalData();
	}
	if (CachedTraversalData.IsValid())
	{
		TArray<FNiagaraVariable> CachedStatics;
		CachedTraversalData.Get()->GetStaticVariables(CachedStatics);

		// Because we've lost all context here about the specific function call we're in, we need to convert static vars back to 
		// normal mode.
		FString ModuleName = FunctionCallNode.GetFunctionName();
		FNiagaraAliasContext AliasContext(Mode);
		AliasContext.ChangeModuleNameToModule(ModuleName);
		if (EmitterName.Len() != 0)	
			AliasContext.ChangeEmitterNameToEmitter(EmitterName);

		for (int32 i = 0; i < CachedStatics.Num(); i++)
		{
			FNiagaraVariable Var = FNiagaraUtilities::ResolveAliases(CachedStatics[i], AliasContext);
			StaticVars.Add(Var);
		}
	}

	TArray<FNiagaraVariable> SwitchInputs = FunctionCallGraph->FindStaticSwitchInputs();
	TArray<FNiagaraVariable> ReachableInputs = FunctionCallGraph->FindStaticSwitchInputs(true, StaticVars);
	for (FNiagaraVariable SwitchInput : SwitchInputs)
	{
		FEdGraphPinType PinType = Schema->TypeDefinitionToPinType(SwitchInput.GetType());
		for (UEdGraphPin* Pin : FunctionCallNode.Pins)
		{
			if (Pin->Direction != EEdGraphPinDirection::EGPD_Input)
			{
				continue;
			}
			if (Pin->PinName.IsEqual(SwitchInput.GetName()) && Pin->PinType == PinType)
			{
				OutInputPins.Add(Pin);
				if (!ReachableInputs.Contains(SwitchInput))
				{
					OutHiddenPins.Add(Pin);
				}
				break;
			}
		}
	}
}

void FNiagaraStackGraphUtilities::GetStackFunctionStaticSwitchPins(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<UEdGraphPin*>& OutInputPins, TSet<UEdGraphPin*>& OutHiddenPins, FCompileConstantResolver& ConstantResolver)
{
	SCOPE_CYCLE_COUNTER(STAT_NiagaraEditor_StackGraphUtilities_GetStackFunctionStaticSwitchPins);
	if (GbNiagaraEnableTraversalCache == 0)
	{
		GetStackFunctionStaticSwitchPinsLegacy(FunctionCallNode, OutInputPins, OutHiddenPins, ConstantResolver);
	}
	else
	{
		using namespace UE::Niagara::TraversalCache;
		FTopLevelScriptStaticContext ScriptContext;
		StackGraphUtilities::ConstantResolverToTopLevelScriptContext(ConstantResolver, FunctionCallNode, ScriptContext);
		FTraversalCache::GetStackFunctionStaticInputPins(ScriptContext, FunctionCallNode, OutInputPins, OutHiddenPins);
	}
}

void FNiagaraStackGraphUtilities::GetStackFunctionOutputVariables(UNiagaraNodeFunctionCall& FunctionCallNode, FCompileConstantResolver ConstantResolver, TArray<FNiagaraVariable>& OutOutputVariables, TArray<FNiagaraVariable>& OutOutputVariablesWithOriginalAliasesIntact)
{
	FNiagaraParameterMapHistoryBuilder Builder;
	Builder.SetIgnoreDisabled(false);
	*Builder.ConstantResolver = ConstantResolver;

	FVersionedNiagaraEmitter OuterEmitter = FNiagaraStackGraphUtilitiesImpl::GetOuterEmitter(&FunctionCallNode);
	UNiagaraSystem* System = FunctionCallNode.GetTypedOuter<UNiagaraSystem>();

	TArray<FNiagaraVariable> StaticVars;
	TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalData;
	if (OuterEmitter.GetEmitterData())
	{
		CachedTraversalData = OuterEmitter.Emitter->GetCachedTraversalData(OuterEmitter.Version);
	}
	else if (System)
	{
		CachedTraversalData = System->GetCachedTraversalData();
	}
	if (CachedTraversalData.IsValid())
	{
		CachedTraversalData.Get()->GetStaticVariables(StaticVars);
	}

	Builder.RegisterExternalStaticVariables(StaticVars);

	FunctionCallNode.BuildParameterMapHistory(Builder, false);

	if (Builder.Histories.Num() == 1)
	{
		for (int32 i = 0; i < Builder.Histories[0].Variables.Num(); i++)
		{
			bool bHasParameterMapSetWrite = false;
			for (const FNiagaraParameterMapHistory::FModuleScopedPin& WritePin : Builder.Histories[0].PerVariableWriteHistory[i])
			{
				if (WritePin.Pin != nullptr && WritePin.Pin->GetOwningNode() != nullptr && WritePin.Pin->GetOwningNode()->IsA<UNiagaraNodeParameterMapSet>())
				{
					bHasParameterMapSetWrite = true;
					break;
				}
			}

			if (bHasParameterMapSetWrite)
			{
				FNiagaraVariable& Variable = Builder.Histories[0].Variables[i];
				FNiagaraVariable& VariableWithOriginalAliasIntact = Builder.Histories[0].VariablesWithOriginalAliasesIntact[i];
				OutOutputVariables.Add(Variable);
				OutOutputVariablesWithOriginalAliasesIntact.Add(VariableWithOriginalAliasIntact);
			}
		}
	}
	else
	{
		UE_LOGF(LogNiagaraEditor, Log, "Invalid Stack Graph - Function call node has invalid history count!");
	}
}

bool FNiagaraStackGraphUtilities::GetStackFunctionInputAndOutputVariables(UNiagaraNodeFunctionCall& FunctionCallNode, FCompileConstantResolver ConstantResolver, TArray<FNiagaraVariable>& OutVariables, TArray<FNiagaraVariable>& OutVariablesWithOriginalAliasesIntact)
{
	FNiagaraParameterMapHistoryBuilder Builder;
	Builder.SetIgnoreDisabled(false);
	*Builder.ConstantResolver = ConstantResolver;

	FVersionedNiagaraEmitter OuterEmitter = FNiagaraStackGraphUtilitiesImpl::GetOuterEmitter(&FunctionCallNode);
	UNiagaraSystem* System = FunctionCallNode.GetTypedOuter<UNiagaraSystem>();

	TArray<FNiagaraVariable> StaticVars;
	TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CachedTraversalData;
	if (OuterEmitter.GetEmitterData())
	{
		CachedTraversalData = OuterEmitter.Emitter->GetCachedTraversalData(OuterEmitter.Version);
	}
	else if (System)
	{
		CachedTraversalData = System->GetCachedTraversalData();
	}
	if (CachedTraversalData.IsValid())
	{
		CachedTraversalData.Get()->GetStaticVariables(StaticVars);
	}

	Builder.RegisterExternalStaticVariables(StaticVars);

	FunctionCallNode.BuildParameterMapHistory(Builder, false);

	if (Builder.Histories.Num() == 0)
	{
		// No builder histories; it is possible the script does not have a complete path from input to output node.
		return false;
	}

	for (int32 i = 0; i < Builder.Histories[0].Variables.Num(); ++i)
	{
		bool bHasParameterMapSetWrite = false;
		for (const auto& WritePin : Builder.Histories[0].PerVariableWriteHistory[i])
		{
			if (WritePin.Pin != nullptr && WritePin.Pin->GetOwningNode() != nullptr &&
				WritePin.Pin->GetOwningNode()->IsA<UNiagaraNodeParameterMapSet>())
			{
				bHasParameterMapSetWrite = true;
				break;
			}
		}

		if (bHasParameterMapSetWrite)
		{
			FNiagaraVariable& Variable = Builder.Histories[0].Variables[i];
			FNiagaraVariable& VariableWithOriginalAliasIntact = Builder.Histories[0].VariablesWithOriginalAliasesIntact[i];
			OutVariables.Add(Variable);
			OutVariablesWithOriginalAliasesIntact.Add(VariableWithOriginalAliasIntact);
		}
	}

	TArray<const UEdGraphPin*> InputPins;
	ExtractInputPinsFromHistory(Builder.Histories[0], FunctionCallNode.GetCalledGraph(), FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly, InputPins);

	for (const UEdGraphPin* Pin : InputPins)
	{
		for (int32 i = 0; i < Builder.Histories[0].Variables.Num(); ++i)
		{
			FNiagaraVariable& VariableWithOriginalAliasIntact = Builder.Histories[0].VariablesWithOriginalAliasesIntact[i];
			if (VariableWithOriginalAliasIntact.GetName() == Pin->PinName)
			{
				FNiagaraVariable& Variable = Builder.Histories[0].Variables[i];
				OutVariables.AddUnique(Variable);
				OutVariablesWithOriginalAliasesIntact.AddUnique(VariableWithOriginalAliasIntact);
			}
		}
	}
	return true;
}

UNiagaraNodeParameterMapSet* FNiagaraStackGraphUtilities::GetStackFunctionOverrideNode(UNiagaraNodeFunctionCall& FunctionCallNode)
{
	UEdGraphPin* ParameterMapInput = FNiagaraStackGraphUtilities::GetParameterMapInputPin(FunctionCallNode);
	if (ParameterMapInput != nullptr && ParameterMapInput->LinkedTo.Num() == 1)
	{
		return Cast<UNiagaraNodeParameterMapSet>(ParameterMapInput->LinkedTo[0]->GetOwningNode());
	}
	return nullptr;
}

UNiagaraNodeParameterMapSet& FNiagaraStackGraphUtilities::GetOrCreateStackFunctionOverrideNode(UNiagaraNodeFunctionCall& StackFunctionCall, const FGuid& PreferredOverrideNodeGuid)
{
	UNiagaraNodeParameterMapSet* OverrideNode = GetStackFunctionOverrideNode(StackFunctionCall);
	if (OverrideNode == nullptr)
	{
		UEdGraph* Graph = StackFunctionCall.GetGraph();
		Graph->Modify();
		FGraphNodeCreator<UNiagaraNodeParameterMapSet> ParameterMapSetNodeCreator(*Graph);
		OverrideNode = ParameterMapSetNodeCreator.CreateNode();
		ParameterMapSetNodeCreator.Finalize();
		if (PreferredOverrideNodeGuid.IsValid())
		{
			OverrideNode->NodeGuid = PreferredOverrideNodeGuid;
		}
		OverrideNode->SetEnabledState(StackFunctionCall.GetDesiredEnabledState(), StackFunctionCall.HasUserSetTheEnabledState());

		UEdGraphPin* OverrideNodeInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*OverrideNode);
		UEdGraphPin* OverrideNodeOutputPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*OverrideNode);

		UEdGraphPin* OwningFunctionCallInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(StackFunctionCall);
		UEdGraphPin* PreviousStackNodeOutputPin = OwningFunctionCallInputPin->LinkedTo[0];

		BreakAllPinLinks(OwningFunctionCallInputPin);
		MakeLinkTo(OwningFunctionCallInputPin, OverrideNodeOutputPin);
		for (UEdGraphPin* PreviousStackNodeOutputLinkedPin : PreviousStackNodeOutputPin->LinkedTo)
		{
			MakeLinkTo(PreviousStackNodeOutputLinkedPin, OverrideNodeOutputPin);
		}
		BreakAllPinLinks(PreviousStackNodeOutputPin);
		MakeLinkTo(PreviousStackNodeOutputPin, OverrideNodeInputPin);
	}
	return *OverrideNode;
}

UEdGraphPin* FNiagaraStackGraphUtilities::GetStackFunctionInputOverridePin(UNiagaraNodeFunctionCall& StackFunctionCall, FNiagaraParameterHandle AliasedInputParameterHandle)
{
	if (UEdGraphPin* SwitchPin = StackFunctionCall.FindStaticSwitchInputPin(AliasedInputParameterHandle.GetName()))
	{
		return SwitchPin;
	}

	UNiagaraNodeParameterMapSet* OverrideNode = GetStackFunctionOverrideNode(StackFunctionCall);
	if (OverrideNode != nullptr)
	{
		FPinCollectorArray InputPins;
		OverrideNode->GetInputPins(InputPins);
		UEdGraphPin** OverridePinPtr = InputPins.FindByPredicate([&](const UEdGraphPin* Pin) { return Pin->PinName == AliasedInputParameterHandle.GetParameterHandleString(); });
		if (OverridePinPtr != nullptr)
		{
			return *OverridePinPtr;
		}
	}
	return nullptr;
}

UEdGraphPin& FNiagaraStackGraphUtilities::GetOrCreateStackFunctionInputOverridePin(UNiagaraNodeFunctionCall& StackFunctionCall, FNiagaraParameterHandle AliasedInputParameterHandle, FNiagaraTypeDefinition InputType, const FGuid& InputScriptVariableId, const FGuid& PreferredOverrideNodeGuid)
{
	UEdGraphPin* OverridePin = GetStackFunctionInputOverridePin(StackFunctionCall, AliasedInputParameterHandle);
	if (OverridePin == nullptr)
	{
		UNiagaraNodeParameterMapSet& OverrideNode = GetOrCreateStackFunctionOverrideNode(StackFunctionCall, PreferredOverrideNodeGuid);
		OverrideNode.Modify();

		FPinCollectorArray OverrideInputPins;
		OverrideNode.GetInputPins(OverrideInputPins);

		const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
		FEdGraphPinType PinType = NiagaraSchema->TypeDefinitionToPinType(InputType);
		OverridePin = OverrideNode.CreatePin(EEdGraphPinDirection::EGPD_Input, PinType, AliasedInputParameterHandle.GetParameterHandleString(), OverrideInputPins.Num() - 1);

		if(InputScriptVariableId.IsValid())
		{
			StackFunctionCall.UpdateInputNameBinding(InputScriptVariableId, AliasedInputParameterHandle.GetParameterHandleString());
		}
	}
	return *OverridePin;
}

bool FNiagaraStackGraphUtilities::IsOverridePinForFunction(UEdGraphPin& OverridePin, UNiagaraNodeFunctionCall& FunctionCallNode)
{
	if (OverridePin.PinType.PinCategory == UEdGraphSchema_Niagara::PinCategoryMisc ||
		OverridePin.PinType.PinSubCategoryObject == FNiagaraTypeDefinition::GetParameterMapStruct())
	{
		return false;
	}

	FNiagaraParameterHandle InputHandle(OverridePin.PinName);
	return InputHandle.GetNamespace().ToString() == FunctionCallNode.GetFunctionName();
}

TArray<UEdGraphPin*> FNiagaraStackGraphUtilities::GetOverridePinsForFunction(UNiagaraNodeParameterMapSet& OverrideNode, UNiagaraNodeFunctionCall& FunctionCallNode)
{
	TArray<UEdGraphPin*> OverridePins;
	TArray<UEdGraphPin*> OverrideNodeInputPins;
	OverrideNode.GetInputPins(OverrideNodeInputPins);
	for (UEdGraphPin* OverrideNodeInputPin : OverrideNodeInputPins)
	{
		if(IsOverridePinForFunction(*OverrideNodeInputPin, FunctionCallNode))
		{
			OverridePins.Add(OverrideNodeInputPin);
		}
	}
	return OverridePins;
}

void FNiagaraStackGraphUtilities::RemoveNodesForStackFunctionInputOverridePin(UEdGraphPin& StackFunctionInputOverridePin)
{
	TArray<TWeakObjectPtr<UNiagaraDataInterface>> RemovedDataObjects;
	RemoveNodesForStackFunctionInputOverridePin(StackFunctionInputOverridePin, RemovedDataObjects);
}

void FNiagaraStackGraphUtilities::RemoveNodesForStackFunctionInputOverridePin(UEdGraphPin& StackFunctionInputOverridePin, TArray<TWeakObjectPtr<UNiagaraDataInterface>>& OutRemovedDataObjects)
{
	if (StackFunctionInputOverridePin.LinkedTo.Num() == 1)
	{
		UEdGraphNode* OverrideValueNode = StackFunctionInputOverridePin.LinkedTo[0]->GetOwningNode();
		UEdGraph* Graph = OverrideValueNode->GetGraph();
		if (OverrideValueNode->IsA<UNiagaraNodeInput>() || OverrideValueNode->IsA<UNiagaraNodeParameterMapGet>())
		{
			UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(OverrideValueNode);
			if (InputNode != nullptr && InputNode->GetDataInterface() != nullptr)
			{
				OutRemovedDataObjects.Add(InputNode->GetDataInterface());
			}
			Graph->RemoveNode(OverrideValueNode);
		}
		else if (OverrideValueNode->IsA<UNiagaraNodeFunctionCall>())
		{
			UNiagaraNodeFunctionCall* DynamicInputNode = CastChecked<UNiagaraNodeFunctionCall>(OverrideValueNode);
			UEdGraphPin* DynamicInputNodeInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*DynamicInputNode);
			if (DynamicInputNodeInputPin && DynamicInputNodeInputPin->LinkedTo.Num() > 0 && DynamicInputNodeInputPin->LinkedTo[0] != nullptr)
			{
				UNiagaraNodeParameterMapSet* DynamicInputNodeOverrideNode = Cast<UNiagaraNodeParameterMapSet>(DynamicInputNodeInputPin->LinkedTo[0]->GetOwningNode());
				if (DynamicInputNodeOverrideNode != nullptr)
				{
					TArray<UEdGraphPin*> DynamicInputOverridePins = GetOverridePinsForFunction(*DynamicInputNodeOverrideNode, *DynamicInputNode);
					for(UEdGraphPin* DynamicInputOverridePin : DynamicInputOverridePins)
					{
						RemoveNodesForStackFunctionInputOverridePin(*DynamicInputOverridePin, OutRemovedDataObjects);
						DynamicInputNodeOverrideNode->RemovePin(DynamicInputOverridePin);
					}

					FPinCollectorArray NewInputPins;
					DynamicInputNodeOverrideNode->GetInputPins(NewInputPins);
					if (NewInputPins.Num() == 2)
					{
						// If there are only 2 input pins left, they are the parameter map input and the add pin, so the dynamic input's override node 
						// can be removed.  This not always be the case when removing dynamic input nodes because they share the same override node.
						UEdGraphPin* InputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*DynamicInputNodeOverrideNode);
						UEdGraphPin* OutputPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*DynamicInputNodeOverrideNode);

						if (ensureMsgf(InputPin != nullptr && InputPin->LinkedTo.Num() == 1 && OutputPin != nullptr &&
							OutputPin->LinkedTo.Num() >= 2, TEXT("Invalid Stack - Dynamic input node override node not connected correctly.")))
						{
							// The DynamicInputOverrideNode will have a single input which is the previous module or override map set, and
							// two or more output links, one to the dynamic input node, one to the next override map set, and 0 or more links
							// to other dynamic inputs on sibling inputs.  Collect these linked pins to reconnect after removing the override node.
							UEdGraphPin* LinkedInputPin = InputPin->LinkedTo[0];
							TArray<UEdGraphPin*> LinkedOutputPins;
							for (UEdGraphPin* LinkedOutputPin : OutputPin->LinkedTo)
							{
								if (LinkedOutputPin->GetOwningNode() != DynamicInputNode)
								{
									LinkedOutputPins.Add(LinkedOutputPin);
								}
							}

							// Disconnect the override node and remove it.
							BreakAllPinLinks(InputPin);
							BreakAllPinLinks(OutputPin);
							Graph->RemoveNode(DynamicInputNodeOverrideNode);

							// Reconnect the pins which were connected to the removed override node.
							for (UEdGraphPin* LinkedOutputPin : LinkedOutputPins)
							{
								MakeLinkTo(LinkedInputPin, LinkedOutputPin);
							}
						}
					}
				}
			}

			Graph->RemoveNode(DynamicInputNode);
		}
	}
}

UEdGraphPin* FNiagaraStackGraphUtilities::GetLinkedValueHandleForFunctionInput(const UEdGraphPin& OverridePin)
{
	UNiagaraNodeParameterMapSet* OverrideNode = Cast<UNiagaraNodeParameterMapSet>(OverridePin.GetOwningNode());
	if (!OverrideNode || OverridePin.LinkedTo.Num() != 1)
	{
		return nullptr;
	}
	UEdGraphPin* GetOutputPin = OverridePin.LinkedTo[0];
	FNiagaraParameterHandle PinHandle(GetOutputPin->PinName);
	if (!GetOutputPin->GetOwningNode()->IsA<UNiagaraNodeParameterMapGet>() || !PinHandle.IsValid() || PinHandle.GetNamespace().IsNone())
	{
		return nullptr;
	}
	return GetOutputPin;
}

void FNiagaraStackGraphUtilities::GetParametersForContext(UEdGraph* InGraph, UNiagaraSystem& System, TSet<FNiagaraVariableBase>& OutParameters)
{
	if (UNiagaraGraph* NiagaraGraph = Cast<UNiagaraGraph>(InGraph))
	{
		for (const TPair<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& ReferencePair : NiagaraGraph->GetParameterReferenceMap())
		{
			OutParameters.Add(ReferencePair.Key);
		}
	}

	TArray<FNiagaraVariable> UserParams;
	System.GetExposedParameters().GetUserParameters(UserParams);
	for (FNiagaraVariable& Var : UserParams)
	{
		FNiagaraUserRedirectionParameterStore::MakeUserVariable(Var);
		OutParameters.Add(Var);
	}
}

void FNiagaraStackGraphUtilities::SetLinkedParameterValueForFunctionInput(UEdGraphPin& OverridePin, const FNiagaraVariableBase& LinkedParameter, const TSet<FNiagaraVariableBase>& KnownParameters, ENiagaraDefaultMode DesiredDefaultMode, const FGuid& NewNodePersistentId)
{
	checkf(OverridePin.LinkedTo.Num() == 0, TEXT("Can't set a linked value handle when the override pin already has a value."));
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();

	UNiagaraNodeParameterMapSet* OverrideNode = CastChecked<UNiagaraNodeParameterMapSet>(OverridePin.GetOwningNode());
	UNiagaraGraph* Graph = OverrideNode->GetNiagaraGraph();
	Graph->Modify();
	FGraphNodeCreator<UNiagaraNodeParameterMapGet> GetNodeCreator(*Graph);
	UNiagaraNodeParameterMapGet* GetNode = GetNodeCreator.CreateNode();
	GetNodeCreator.Finalize();
	GetNode->SetEnabledState(OverrideNode->GetDesiredEnabledState(), OverrideNode->HasUserSetTheEnabledState());

	UEdGraphPin* GetInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*GetNode);
	checkf(GetInputPin != nullptr, TEXT("Parameter map get node was missing it's parameter map input pin."));

	UEdGraphPin* OverrideNodeInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*OverrideNode);
	UEdGraphPin* PreviousStackNodeOutputPin = OverrideNodeInputPin->LinkedTo[0];
	checkf(PreviousStackNodeOutputPin != nullptr, TEXT("Invalid Stack Graph - No previous stack node."));

	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraTypeDefinition InputType = NiagaraSchema->PinToTypeDefinition(&OverridePin);

	FNiagaraVariableBase ParameterToRead = LinkedParameter;
	if (Settings->bEnforceStrictStackTypes == false && (InputType == FNiagaraTypeDefinition::GetVec3Def() || InputType == FNiagaraTypeDefinition::GetPositionDef()))
	{
		// see if the requested input exists or if we can use one of the equivalent loose types
		FNiagaraTypeDefinition AlternateType = InputType == FNiagaraTypeDefinition::GetVec3Def() ? FNiagaraTypeDefinition::GetPositionDef() : FNiagaraTypeDefinition::GetVec3Def();
		FNiagaraVariable ParameterAlternative(AlternateType, LinkedParameter.GetName());
		if (KnownParameters.Contains(ParameterToRead) == false && KnownParameters.Contains(ParameterAlternative))
		{
			ParameterToRead = ParameterAlternative;
		}
	}
	else
	{
		// check if a static var of the requested input exists and use that instead
		FNiagaraTypeDefinition StaticInputType = InputType.ToStaticDef();
		FNiagaraVariable StaticParameter(StaticInputType, LinkedParameter.GetName());
		if (KnownParameters.Contains(StaticParameter))
		{
			ParameterToRead = StaticParameter;
		}
	}
	
	UEdGraphPin* GetOutputPin = GetNode->RequestNewTypedPin(EGPD_Output, ParameterToRead.GetType(), ParameterToRead.GetName());
	MakeLinkTo(GetInputPin, PreviousStackNodeOutputPin);
	MakeLinkTo(GetOutputPin, &OverridePin);

	UNiagaraScriptVariable* ScriptVar = Graph->AddParameter(ParameterToRead, false);
	if (ScriptVar)
	{
		ScriptVar->DefaultMode = DesiredDefaultMode; 
	}

	if (NewNodePersistentId.IsValid())
	{
		GetNode->NodeGuid = NewNodePersistentId;
	}
}

void FNiagaraStackGraphUtilities::SetDataInterfaceValueForFunctionInput(UEdGraphPin& OverridePin, UClass* DataObjectType, FString InputNodeInputName, UNiagaraDataInterface*& OutDataObject, const FGuid& NewNodePersistentId)
{
	checkf(OverridePin.LinkedTo.Num() == 0, TEXT("Can't set a data value when the override pin already has a value."));
	checkf(DataObjectType->IsChildOf<UNiagaraDataInterface>(), TEXT("Can only set a function input to a data interface value object"));

	UNiagaraNodeParameterMapSet* OverrideNode = CastChecked<UNiagaraNodeParameterMapSet>(OverridePin.GetOwningNode());
	UEdGraph* Graph = OverrideNode->GetGraph();
	Graph->Modify();
	FGraphNodeCreator<UNiagaraNodeInput> InputNodeCreator(*Graph);
	UNiagaraNodeInput* InputNode = InputNodeCreator.CreateNode();
	FNiagaraEditorUtilities::InitializeParameterInputNode(*InputNode, FNiagaraTypeDefinition(DataObjectType), CastChecked<UNiagaraGraph>(Graph), *InputNodeInputName);

	const EObjectFlags DataInterfaceObjectFlags = UNiagaraDataInterface::BuildObjectFlagsForOwner(InputNode, RF_Transactional);
	OutDataObject = NewObject<UNiagaraDataInterface>(InputNode, DataObjectType, *ObjectTools::SanitizeObjectName(InputNodeInputName), DataInterfaceObjectFlags);
	InputNode->SetDataInterface(OutDataObject);

	InputNodeCreator.Finalize();
	FNiagaraStackGraphUtilities::ConnectPinToInputNode(OverridePin, *InputNode);

	if (NewNodePersistentId.IsValid())
	{
		InputNode->NodeGuid = NewNodePersistentId;
	}
}

void FNiagaraStackGraphUtilities::SetObjectAssetValueForFunctionInput(UEdGraphPin& OverridePin, UClass* DataObjectType, FString InputNodeInputName, UObject* ObjectAsset, const FGuid& NewNodePersistentId)
{
	checkf(OverridePin.LinkedTo.Num() == 0, TEXT("Can't set a data value when the override pin already has a value."));

	UNiagaraNodeParameterMapSet* OverrideNode = CastChecked<UNiagaraNodeParameterMapSet>(OverridePin.GetOwningNode());
	UEdGraph* Graph = OverrideNode->GetGraph();
	Graph->Modify();
	FGraphNodeCreator<UNiagaraNodeInput> InputNodeCreator(*Graph);
	UNiagaraNodeInput* InputNode = InputNodeCreator.CreateNode();
	FNiagaraEditorUtilities::InitializeParameterInputNode(*InputNode, FNiagaraTypeDefinition(DataObjectType), CastChecked<UNiagaraGraph>(Graph), *InputNodeInputName);
	InputNode->SetObjectAsset(ObjectAsset);

	InputNodeCreator.Finalize();
	FNiagaraStackGraphUtilities::ConnectPinToInputNode(OverridePin, *InputNode);

	if (NewNodePersistentId.IsValid())
	{
		InputNode->NodeGuid = NewNodePersistentId;
	}
}

void FNiagaraStackGraphUtilities::SetDynamicInputForFunctionInput(UEdGraphPin& OverridePin, UNiagaraScript* DynamicInput, UNiagaraNodeFunctionCall*& OutDynamicInputFunctionCall, const FGuid& NewNodePersistentId, FString SuggestedName, const FGuid& InScriptVersion)
{
	checkf(OverridePin.LinkedTo.Num() == 0, TEXT("Can't set a data value when the override pin already has a value."));

	UNiagaraNodeParameterMapSet* OverrideNode = CastChecked<UNiagaraNodeParameterMapSet>(OverridePin.GetOwningNode());
	UEdGraph* Graph = OverrideNode->GetGraph();
	Graph->Modify();
	FGraphNodeCreator<UNiagaraNodeFunctionCall> FunctionCallNodeCreator(*Graph);
	UNiagaraNodeFunctionCall* FunctionCallNode = FunctionCallNodeCreator.CreateNode();
	FunctionCallNode->FunctionScript = DynamicInput;
	if (DynamicInput && DynamicInput->IsVersioningEnabled())
	{
		FunctionCallNode->SelectedScriptVersion = InScriptVersion.IsValid() ? InScriptVersion : DynamicInput->GetExposedVersion().VersionGuid;
	}
	FunctionCallNodeCreator.Finalize();

	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	FNiagaraTypeDefinition InputType = NiagaraSchema->PinToTypeDefinition(&OverridePin);

	if (DynamicInput == nullptr)
	{
		// If there is no dynamic input script we need to add default pins so that the function call node can be connected properly.
		FunctionCallNode->CreatePin(EGPD_Input, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("InputMap"));
		FunctionCallNode->CreatePin(EGPD_Output, NiagaraSchema->TypeDefinitionToPinType(InputType), TEXT("Output"));
	}

	FunctionCallNode->SetEnabledState(OverrideNode->GetDesiredEnabledState(), OverrideNode->HasUserSetTheEnabledState());

	UEdGraphPin* FunctionCallInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*FunctionCallNode);
	FPinCollectorArray FunctionCallOutputPins;
	FunctionCallNode->GetOutputPins(FunctionCallOutputPins);

	UEdGraphPin* OverrideNodeInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*OverrideNode);
	UEdGraphPin* PreviousStackNodeOutputPin = nullptr;
	if (OverrideNodeInputPin != nullptr)
	{
		PreviousStackNodeOutputPin = OverrideNodeInputPin->LinkedTo[0];
	}
	
	if (FunctionCallInputPin != nullptr && PreviousStackNodeOutputPin != nullptr)
	{
		MakeLinkTo(FunctionCallInputPin, PreviousStackNodeOutputPin);
	}
	
	if (FunctionCallOutputPins.Num() >= 1 && FunctionCallOutputPins[0] != nullptr)
	{
		MakeLinkTo(FunctionCallOutputPins[0], &OverridePin);
	}

	OutDynamicInputFunctionCall = FunctionCallNode;

	if (NewNodePersistentId.IsValid())
	{
		FunctionCallNode->NodeGuid = NewNodePersistentId;
	}

	if (SuggestedName.IsEmpty() == false)
	{
		FunctionCallNode->SuggestName(SuggestedName);
	}
}

void FNiagaraStackGraphUtilities::SetCustomExpressionForFunctionInput(UEdGraphPin& OverridePin, const FString& CustomExpression, UNiagaraNodeCustomHlsl*& OutDynamicInputFunctionCall, const FGuid& NewNodePersistentId)
{
	checkf(OverridePin.LinkedTo.Num() == 0, TEXT("Can't set a data value when the override pin already has a value."));

	if(UNiagaraNodeParameterMapSet* OverrideNode = Cast<UNiagaraNodeParameterMapSet>(OverridePin.GetOwningNode()))
	{
		UEdGraph* Graph = OverrideNode->GetGraph();
		const UEdGraphSchema_Niagara* Schema = CastChecked<UEdGraphSchema_Niagara>(OverrideNode->GetSchema());

		Graph->Modify();
		FGraphNodeCreator<UNiagaraNodeCustomHlsl> FunctionCallNodeCreator(*Graph);
		UNiagaraNodeCustomHlsl* FunctionCallNode = FunctionCallNodeCreator.CreateNode();
		FunctionCallNode->InitAsCustomHlslDynamicInput(Schema->PinToTypeDefinition(&OverridePin));
		FunctionCallNodeCreator.Finalize();
		FunctionCallNode->SetEnabledState(OverrideNode->GetDesiredEnabledState(), OverrideNode->HasUserSetTheEnabledState());

		UEdGraphPin* FunctionCallInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*FunctionCallNode);
		FPinCollectorArray FunctionCallOutputPins;
		FunctionCallNode->GetOutputPins(FunctionCallOutputPins);

		const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();

		FNiagaraTypeDefinition InputType = NiagaraSchema->PinToTypeDefinition(&OverridePin);
		checkf(FunctionCallInputPin != nullptr, TEXT("Dynamic Input function call did not have a parameter map input pin."));
		checkf(FunctionCallOutputPins.Num() == 2 && NiagaraSchema->PinToTypeDefinition(FunctionCallOutputPins[0]) == InputType, TEXT("Invalid Stack Graph - Dynamic Input function did not have the correct typed output pin"));

		UEdGraphPin* OverrideNodeInputPin = FNiagaraStackGraphUtilities::GetParameterMapInputPin(*OverrideNode);
		UEdGraphPin* PreviousStackNodeOutputPin = OverrideNodeInputPin->LinkedTo[0];
		checkf(PreviousStackNodeOutputPin != nullptr, TEXT("Invalid Stack Graph - No previous stack node."));

		MakeLinkTo(FunctionCallInputPin, PreviousStackNodeOutputPin);
		MakeLinkTo(FunctionCallOutputPins[0], &OverridePin);

		OutDynamicInputFunctionCall = FunctionCallNode;

		if (NewNodePersistentId.IsValid())
		{
			FunctionCallNode->NodeGuid = NewNodePersistentId;
		}

		FunctionCallNode->SetCustomHlsl(CustomExpression);
	}
	else
	{
		OutDynamicInputFunctionCall = nullptr;
	}
}

bool FNiagaraStackGraphUtilities::RemoveModuleFromStack(UNiagaraSystem& OwningSystem, FGuid OwningEmitterId, UNiagaraNodeFunctionCall& ModuleNode)
{
	TArray<TWeakObjectPtr<UNiagaraNodeInput>> RemovedNodes;
	return RemoveModuleFromStack(OwningSystem, OwningEmitterId, ModuleNode, RemovedNodes);
}

bool FNiagaraStackGraphUtilities::RemoveModuleFromStack(UNiagaraSystem& OwningSystem, FGuid OwningEmitterId, UNiagaraNodeFunctionCall& ModuleNode, TArray<TWeakObjectPtr<UNiagaraNodeInput>>& OutRemovedInputNodes)
{
	// Find the owning script so it can be modified as part of the transaction so that rapid iteration parameters values are retained upon undo.
	UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(ModuleNode);
	checkf(OutputNode != nullptr, TEXT("Invalid Stack - Output node could not be found for module"));

	UNiagaraScript* OwningScript = FNiagaraEditorUtilities::GetScriptFromSystem(
		OwningSystem, OwningEmitterId, OutputNode->GetUsage(), OutputNode->GetUsageId());
	checkf(OwningScript != nullptr, TEXT("Invalid Stack - Owning script could not be found for module"));

	return RemoveModuleFromStack(*OwningScript, ModuleNode, OutRemovedInputNodes);
}

bool FNiagaraStackGraphUtilities::RemoveModuleFromStack(UNiagaraScript& OwningScript, UNiagaraNodeFunctionCall& ModuleNode)
{
	TArray<TWeakObjectPtr<UNiagaraNodeInput>> RemovedNodes;
	return RemoveModuleFromStack(OwningScript, ModuleNode, RemovedNodes);
}

bool FNiagaraStackGraphUtilities::RemoveModuleFromStack(UNiagaraScript& OwningScript, UNiagaraNodeFunctionCall& ModuleNode, TArray<TWeakObjectPtr<UNiagaraNodeInput>>& OutRemovedInputNodes)
{
	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackNodeGroups;
	FNiagaraStackGraphUtilities::GetStackNodeGroups(ModuleNode, StackNodeGroups);

	int32 ModuleStackIndex = StackNodeGroups.IndexOfByPredicate([&](const FNiagaraStackGraphUtilities::FStackNodeGroup& StackNodeGroup) { return StackNodeGroup.EndNode == &ModuleNode; });
	if (ModuleStackIndex == INDEX_NONE)
	{
		return false;
	}

	OwningScript.Modify();

	// Disconnect the group from the stack first to make collecting the nodes to remove easier.
	FNiagaraStackGraphUtilities::DisconnectStackNodeGroup(StackNodeGroups[ModuleStackIndex], StackNodeGroups[ModuleStackIndex - 1], StackNodeGroups[ModuleStackIndex + 1]);

	// Traverse all of the nodes in the group to find the nodes to remove.
	FNiagaraStackGraphUtilities::FStackNodeGroup ModuleGroup = StackNodeGroups[ModuleStackIndex];
	
	// Remove the nodes in the group from the graph.
	if (UNiagaraGraph* Graph = ModuleNode.GetNiagaraGraph())
	{
		TArray<UNiagaraNode*> NodesToRemove;
		Graph->BuildTraversal(NodesToRemove, ModuleGroup.EndNode);

		for (UNiagaraNode* NodeToRemove : NodesToRemove)
		{
			NodeToRemove->Modify();
			Graph->RemoveNode(NodeToRemove);
			UNiagaraNodeInput* InputNode = Cast<UNiagaraNodeInput>(NodeToRemove);
			if (InputNode != nullptr)
			{
				OutRemovedInputNodes.Add(InputNode);
			}
		}
	}

	return true;
}

void ConnectModuleNode(UNiagaraNodeFunctionCall& ModuleNode, UNiagaraNodeOutput& TargetOutputNode, int32 TargetIndex)
{
	FNiagaraStackGraphUtilities::FStackNodeGroup ModuleGroup;
	ModuleGroup.StartNodes.Add(&ModuleNode);
	ModuleGroup.EndNode = &ModuleNode;

	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackNodeGroups;
	FNiagaraStackGraphUtilities::GetStackNodeGroups(TargetOutputNode, StackNodeGroups);
	checkf(StackNodeGroups.Num() >= 2, TEXT("Stack graph is invalid, can not connect module"));

	int32 InsertIndex;
	if (TargetIndex != INDEX_NONE)
	{
		// The first stack node group is always the input node so we add one to the target module index to get the insertion index.
		InsertIndex = FMath::Clamp(TargetIndex + 1, 1, StackNodeGroups.Num() - 1);
	}
	else
	{
		// If no insert index was specified, add the module at the end.
		InsertIndex = StackNodeGroups.Num() - 1;
	}

	FNiagaraStackGraphUtilities::FStackNodeGroup& TargetInsertGroup = StackNodeGroups[InsertIndex];
	FNiagaraStackGraphUtilities::FStackNodeGroup& TargetInsertPreviousGroup = StackNodeGroups[InsertIndex - 1];
	FNiagaraStackGraphUtilities::ConnectStackNodeGroup(ModuleGroup, TargetInsertPreviousGroup, TargetInsertGroup);
}

bool FNiagaraStackGraphUtilities::FindScriptModulesInStack(FAssetData ModuleScriptAsset, UNiagaraNodeOutput& TargetOutputNode, TArray<UNiagaraNodeFunctionCall*>& OutFunctionCalls)
{
	UNiagaraGraph* Graph = TargetOutputNode.GetNiagaraGraph();
	TArray<UNiagaraNode*> Nodes;
	Graph->BuildTraversal(Nodes, &TargetOutputNode);

	OutFunctionCalls.Empty();
	FString ModuleObjectName = ModuleScriptAsset.GetObjectPathString();
	FName ModuleScriptAssetSoftObjectPathName = *ModuleScriptAsset.GetSoftObjectPath().ToString();
	for (UEdGraphNode* Node : Nodes)
	{
		if (UNiagaraNodeFunctionCall* FunctionCallNode = Cast<UNiagaraNodeFunctionCall>(Node))
		{
			if (FunctionCallNode->FunctionScriptAssetObjectPath == ModuleScriptAssetSoftObjectPathName ||
				(FunctionCallNode->FunctionScript != nullptr && FunctionCallNode->FunctionScript->GetPathName() == ModuleObjectName))
			{
				OutFunctionCalls.Add(FunctionCallNode);
			}
		}
	}

	return OutFunctionCalls.Num() > 0;
}

UNiagaraNodeFunctionCall* FNiagaraStackGraphUtilities::AddScriptModuleToStack(const FAddScriptModuleToStackArgs& Args)
{
	// Unpack args struct
	const FAssetData& ModuleScriptAsset = Args.ModuleScriptAsset;
	UNiagaraScript* const ModuleScript = Args.ModuleScript;
	UNiagaraNodeOutput* const TargetOutputNode = Args.TargetOutputNode;
	const int32 TargetIndex = Args.TargetIndex;
	FString SuggestedName = Args.SuggestedName;
	const bool bFixupTargetIndex = Args.bFixupTargetIndex;
	const FGuid& VersionGuid = Args.VersionGuid;

	// Get the stack graph via the TargetOutputNode.
	UNiagaraGraph* Graph = TargetOutputNode->GetNiagaraGraph();
	Graph->Modify();

	// Create the UNiagaraNodeFunctionCall to represent the script in the stack.
	FGraphNodeCreator<UNiagaraNodeFunctionCall> ModuleNodeCreator(*Graph);
	UNiagaraNodeFunctionCall* NewModuleNode = ModuleNodeCreator.CreateNode();

	// Set the script on the UNiagaraNodeFunctionCall, and set the script version if specified.
	if (ModuleScript != nullptr)
	{
		NewModuleNode->FunctionScript = ModuleScript;
	}
	else if(ModuleScriptAsset.IsValid())
	{
		NewModuleNode->FunctionScript = CastChecked<UNiagaraScript>(ModuleScriptAsset.GetAsset());
	}
	else
	{
		ensureMsgf(false, TEXT("Encountered invalid FAddScriptModuleToStackArgs! ModuleScript or ModuleScriptAsset must be valid!"));
		return nullptr;
	}

	if (NewModuleNode->FunctionScript->IsVersioningEnabled())
	{
		NewModuleNode->SelectedScriptVersion = VersionGuid.IsValid() ? VersionGuid : NewModuleNode->FunctionScript->GetExposedVersion().VersionGuid;
	}
	else
	{
		NewModuleNode->SelectedScriptVersion = FGuid();
	}
	ModuleNodeCreator.Finalize();

	// Ensure there are input and output pins on the UNiagaraNodeFunctionCall to wire into the stack graph.
	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	if (NewModuleNode->FunctionScript == nullptr)
	{
		// If the module script is null, add parameter map inputs and outputs so that the node can be wired into the graph correctly.
		NewModuleNode->CreatePin(EGPD_Input, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("InputMap"));
		NewModuleNode->CreatePin(EGPD_Output, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("OutputMap"));
		if (SuggestedName.IsEmpty())
		{
			SuggestedName = TEXT("InvalidScript");
		}
	}
	else
	{
		// Make sure that the input and output pins are available to prevent failures in the connect module node.  Once the node is
		// connected these missing pins will generate compile errors which the user can find and fix.
		if (FNiagaraStackGraphUtilities::GetParameterMapInputPin(*NewModuleNode) == nullptr)
		{
			NewModuleNode->CreatePin(EGPD_Input, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("InputMap"));
		}
		if (FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*NewModuleNode) == nullptr)
		{
			NewModuleNode->CreatePin(EGPD_Output, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("OutputMap"));
		}
	}

	// If specified, suggest the name for the script.
	if (SuggestedName.IsEmpty() == false)
	{
		NewModuleNode->SuggestName(SuggestedName);
	}

	// If specified, find the nearest index to TargetIndex that satisfies the new module script's order dependencies.
	int32 BestFoundIndex = INDEX_NONE;
	if (bFixupTargetIndex)
	{
		BestFoundIndex = DependencyUtilities::FindBestIndexForModuleInStack(*NewModuleNode, *TargetOutputNode);
	}
	int32 FinalTargetIndex = BestFoundIndex != INDEX_NONE ? BestFoundIndex : TargetIndex;

	ConnectModuleNode(*NewModuleNode, *TargetOutputNode, FinalTargetIndex);
	return NewModuleNode;
}

UNiagaraNodeFunctionCall* FNiagaraStackGraphUtilities::AddScriptModuleToStack(FAssetData ModuleScriptAsset, UNiagaraNodeOutput& TargetOutputNode, int32 TargetIndex, FString SuggestedName)
{
	UEdGraph* Graph = TargetOutputNode.GetGraph();
	Graph->Modify();

	FGraphNodeCreator<UNiagaraNodeFunctionCall> ModuleNodeCreator(*Graph);
	UNiagaraNodeFunctionCall* NewModuleNode = ModuleNodeCreator.CreateNode();
	NewModuleNode->FunctionScriptAssetObjectPath = FName(*ModuleScriptAsset.GetSoftObjectPath().ToString());
	ModuleNodeCreator.Finalize();

	if (NewModuleNode->HasValidScriptAndGraph() == false)
	{
		// If the module script or graph are invalid, add parameter map inputs and outputs so that the node can be wired into the owning graph correctly.
		const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
		NewModuleNode->CreatePin(EGPD_Input, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("InputMap"));
		NewModuleNode->CreatePin(EGPD_Output, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("OutputMap"));
		if (SuggestedName.IsEmpty())
		{
			SuggestedName = TEXT("InvalidScript");
		}
	}

	if (SuggestedName.IsEmpty() == false)
	{
		NewModuleNode->SuggestName(SuggestedName);
	}

	ConnectModuleNode(*NewModuleNode, TargetOutputNode, TargetIndex);
	return NewModuleNode;
}

UNiagaraNodeFunctionCall* FNiagaraStackGraphUtilities::AddScriptModuleToStack(UNiagaraScript* ModuleScript, UNiagaraNodeOutput& TargetOutputNode, int32 TargetIndex, FString SuggestedName, const FGuid& VersionGuid)
{
	UEdGraph* Graph = TargetOutputNode.GetGraph();
	Graph->Modify();

	FGraphNodeCreator<UNiagaraNodeFunctionCall> ModuleNodeCreator(*Graph);
	UNiagaraNodeFunctionCall* NewModuleNode = ModuleNodeCreator.CreateNode();
	NewModuleNode->FunctionScript = ModuleScript;
	if (ModuleScript && ModuleScript->IsVersioningEnabled())
	{
		NewModuleNode->SelectedScriptVersion = VersionGuid.IsValid() ? VersionGuid : ModuleScript->GetExposedVersion().VersionGuid;
	}
	else
	{
		NewModuleNode->SelectedScriptVersion = FGuid();
	}
	ModuleNodeCreator.Finalize();

	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	if (NewModuleNode->FunctionScript == nullptr)
	{
		// If the module script is null, add parameter map inputs and outputs so that the node can be wired into the graph correctly.
		NewModuleNode->CreatePin(EGPD_Input, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("InputMap"));
		NewModuleNode->CreatePin(EGPD_Output, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("OutputMap"));
		if (SuggestedName.IsEmpty())
		{
			SuggestedName = TEXT("InvalidScript");
		}
	}
	else
	{
		// Make sure that the input and output pins are available to prevent failures in the connect module node.  Once the node is
		// connected these missing pins will generate compile errors which the user can find and fix.
		if (FNiagaraStackGraphUtilities::GetParameterMapInputPin(*NewModuleNode) == nullptr)
		{
			NewModuleNode->CreatePin(EGPD_Input, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("InputMap"));
		}
		if (FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*NewModuleNode) == nullptr)
		{
			NewModuleNode->CreatePin(EGPD_Output, NiagaraSchema->TypeDefinitionToPinType(FNiagaraTypeDefinition::GetParameterMapDef()), TEXT("OutputMap"));
		}
	}

	if (SuggestedName.IsEmpty() == false)
	{
		NewModuleNode->SuggestName(SuggestedName);
	}

	ConnectModuleNode(*NewModuleNode, TargetOutputNode, TargetIndex);
	return NewModuleNode;
}

UNiagaraNodeAssignment* FNiagaraStackGraphUtilities::AddParameterModuleToStack(const TArray<FNiagaraVariable>& ParameterVariables, UNiagaraNodeOutput& TargetOutputNode, int32 TargetIndex, const TArray<FString>& InDefaultValues)
{
	UEdGraph* Graph = TargetOutputNode.GetGraph();
	Graph->Modify();

	FGraphNodeCreator<UNiagaraNodeAssignment> AssignmentNodeCreator(*Graph);
	UNiagaraNodeAssignment* NewAssignmentNode = AssignmentNodeCreator.CreateNode();

	check(ParameterVariables.Num() == InDefaultValues.Num());
	for (int32 i = 0; i < ParameterVariables.Num(); i++)
	{
		NewAssignmentNode->AddAssignmentTarget(ParameterVariables[i], &InDefaultValues[i]);
	}
	AssignmentNodeCreator.Finalize();

	ConnectModuleNode(*NewAssignmentNode, TargetOutputNode, TargetIndex);
	NewAssignmentNode->UpdateUsageBitmaskFromOwningScript();

	return NewAssignmentNode;
}

void GetAllNodesForModule(UNiagaraNodeFunctionCall& ModuleFunctionCall, TArray<UNiagaraNode*>& ModuleNodes)
{
	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackNodeGroups;
	FNiagaraStackGraphUtilities::GetStackNodeGroups(ModuleFunctionCall, StackNodeGroups);

	int32 ThisGroupIndex = StackNodeGroups.IndexOfByPredicate([&](const FNiagaraStackGraphUtilities::FStackNodeGroup& Group) { return Group.EndNode == &ModuleFunctionCall; });
	checkf(ThisGroupIndex > 0 && ThisGroupIndex < StackNodeGroups.Num() - 1, TEXT("Stack graph is invalid"));

	TArray<UNiagaraNode*> AllGroupNodes;
	StackNodeGroups[ThisGroupIndex].GetAllNodesInGroup(ModuleNodes);
}

TOptional<bool> FNiagaraStackGraphUtilities::GetModuleIsEnabled(UNiagaraNodeFunctionCall& FunctionCallNode)
{
	TArray<UNiagaraNode*> AllModuleNodes;
	GetAllNodesForModule(FunctionCallNode, AllModuleNodes);
	bool bIsEnabled = AllModuleNodes[0]->IsNodeEnabled();
	for (int32 i = 1; i < AllModuleNodes.Num(); i++)
	{
		if (AllModuleNodes[i]->IsNodeEnabled() != bIsEnabled)
		{
			return TOptional<bool>();
		}
	}
	return TOptional<bool>(bIsEnabled);
}

void FNiagaraStackGraphUtilities::SetModuleIsEnabled(UNiagaraNodeFunctionCall& FunctionCallNode, bool bIsEnabled)
{
	FunctionCallNode.Modify();
	TArray<UNiagaraNode*> ModuleNodes;
	GetAllNodesForModule(FunctionCallNode, ModuleNodes);
	for (UNiagaraNode* ModuleNode : ModuleNodes)
	{
		ModuleNode->Modify();
		ModuleNode->SetEnabledState(bIsEnabled ? ENodeEnabledState::Enabled : ENodeEnabledState::Disabled, true);
		ModuleNode->MarkNodeRequiresSynchronization(__FUNCTION__, false);
	}
	FunctionCallNode.GetNiagaraGraph()->NotifyGraphNeedsRecompile();
}

bool FNiagaraStackGraphUtilities::ValidateGraphForOutput(UNiagaraGraph& NiagaraGraph, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FText& ErrorMessage)
{
	UNiagaraNodeOutput* OutputNode = NiagaraGraph.FindEquivalentOutputNode(ScriptUsage, ScriptUsageId);
	if (OutputNode == nullptr)
	{
		ErrorMessage = LOCTEXT("ValidateNoOutputMessage", "Output node doesn't exist for script.");
		return false;
	}

	TArray<FStackNodeGroup> NodeGroups;
	GetStackNodeGroups(*OutputNode, NodeGroups);
	
	if (NodeGroups.Num() < 2 || NodeGroups[0].EndNode->IsA<UNiagaraNodeInput>() == false || NodeGroups.Last().EndNode->IsA<UNiagaraNodeOutput>() == false)
	{
		ErrorMessage = LOCTEXT("ValidateInvalidStackMessage", "Stack graph does not include an input node connected to an output node.");
		return false;
	}

	return true;
}

UNiagaraNodeOutput* FNiagaraStackGraphUtilities::ResetGraphForOutput(UNiagaraGraph& NiagaraGraph, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, const FGuid& PreferredOutputNodeGuid, const FGuid& PreferredInputNodeGuid)
{
	NiagaraGraph.Modify();
	UNiagaraNodeOutput* OutputNode = NiagaraGraph.FindOutputNode(ScriptUsage, ScriptUsageId);
	UEdGraphPin* OutputNodeInputPin = OutputNode != nullptr ? GetParameterMapInputPin(*OutputNode) : nullptr;
	if (OutputNode != nullptr && OutputNodeInputPin == nullptr)
	{
		NiagaraGraph.RemoveNode(OutputNode);
		OutputNode = nullptr;
	}

	if (OutputNode == nullptr)
	{
		FGraphNodeCreator<UNiagaraNodeOutput> OutputNodeCreator(NiagaraGraph);
		OutputNode = OutputNodeCreator.CreateNode();
		OutputNode->SetUsage(ScriptUsage);
		OutputNode->SetUsageId(ScriptUsageId);
		OutputNode->Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("Out")));
		OutputNodeCreator.Finalize();

		if (PreferredOutputNodeGuid.IsValid())
		{
			OutputNode->NodeGuid = PreferredOutputNodeGuid;
		}

		OutputNodeInputPin = GetParameterMapInputPin(*OutputNode);
	}
	else
	{
		OutputNode->Modify();
	}

	FNiagaraVariable InputVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("InputMap"));
	FGraphNodeCreator<UNiagaraNodeInput> InputNodeCreator(NiagaraGraph);
	UNiagaraNodeInput* InputNode = InputNodeCreator.CreateNode();
	InputNode->Input = FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("InputMap"));
	InputNode->Usage = ENiagaraInputNodeUsage::Parameter;
	InputNodeCreator.Finalize();

	if (PreferredInputNodeGuid.IsValid())
	{
		InputNode->NodeGuid = PreferredInputNodeGuid;
	}

	UEdGraphPin* InputNodeOutputPin = GetParameterMapOutputPin(*InputNode);
	BreakAllPinLinks(OutputNodeInputPin);
	MakeLinkTo(OutputNodeInputPin, InputNodeOutputPin);

	if (ScriptUsage == ENiagaraScriptUsage::SystemSpawnScript || ScriptUsage == ENiagaraScriptUsage::SystemUpdateScript)
	{
		// TODO: Move the emitter node wrangling to a utility function instead of getting the typed outer here and creating a view model.
		UNiagaraSystem* System = NiagaraGraph.GetTypedOuter<UNiagaraSystem>();
		if (System != nullptr && System->GetEmitterHandles().Num() != 0)
		{
			RebuildEmitterNodes(*System);
		}
	}

	return OutputNode;
}

void GetFunctionNamesRecursive(UNiagaraNode* CurrentNode, TArray<UNiagaraNode*>& VisitedNodes, TArray<FString>& FunctionNames)
{
	if (VisitedNodes.Contains(CurrentNode) == false)
	{
		VisitedNodes.Add(CurrentNode);
		UNiagaraNodeFunctionCall* FunctionCall = Cast<UNiagaraNodeFunctionCall>(CurrentNode);
		if (FunctionCall != nullptr)
		{
			FunctionNames.Add(FunctionCall->GetFunctionName());
		}
		TArray<UEdGraphPin*> InputPins;
		CurrentNode->GetInputPins(InputPins);
		for (UEdGraphPin* InputPin : InputPins)
		{
			for (UEdGraphPin* LinkedPin : InputPin->LinkedTo)
			{
				UNiagaraNode* LinkedNode = Cast<UNiagaraNode>(LinkedPin->GetOwningNode());
				if (LinkedNode != nullptr)
				{
					GetFunctionNamesRecursive(LinkedNode, VisitedNodes, FunctionNames);
				}
			}
		}
	}
}

void GetFunctionNamesForOutputNode(UNiagaraNodeOutput& OutputNode, TArray<FString>& FunctionNames)
{
	TArray<UNiagaraNode*> VisitedNodes;
	GetFunctionNamesRecursive(&OutputNode, VisitedNodes, FunctionNames);
}

bool FNiagaraStackGraphUtilities::IsRapidIterationType(const FNiagaraTypeDefinition& InputType)
{
	checkf(InputType.IsValid(), TEXT("Type is invalid."));
	if (InputType.IsStatic())
		return true;
	return InputType != FNiagaraTypeDefinition::GetBoolDef() && (!InputType.IsEnum()) &&
		InputType != FNiagaraTypeDefinition::GetParameterMapDef() && !InputType.IsUObject();
}

FNiagaraVariable FNiagaraStackGraphUtilities::CreateRapidIterationParameter(const FString& UniqueEmitterName, ENiagaraScriptUsage ScriptUsage, const FName& AliasedInputName, const FNiagaraTypeDefinition& InputType)
{
	FNiagaraVariable InputVariable(InputType, AliasedInputName);
	FNiagaraVariable RapidIterationVariable;
	if (ScriptUsage == ENiagaraScriptUsage::SystemSpawnScript || ScriptUsage == ENiagaraScriptUsage::SystemUpdateScript)
	{
		RapidIterationVariable = FNiagaraParameterUtilities::ConvertVariableToRapidIterationConstantName(InputVariable, nullptr, ScriptUsage); // These names *should* have the emitter baked in...
	}
	else
	{
		RapidIterationVariable = FNiagaraParameterUtilities::ConvertVariableToRapidIterationConstantName(InputVariable, *UniqueEmitterName, ScriptUsage);
	}

	return RapidIterationVariable;
}

void FNiagaraStackGraphUtilities::CleanUpStaleRapidIterationParameters(UNiagaraScript& Script, FVersionedNiagaraEmitter OwningEmitter)
{
	UNiagaraScriptSource* Source = CastChecked<UNiagaraScriptSource>(Script.GetLatestSource());
	UNiagaraNodeOutput* OutputNode = Source->NodeGraph->FindOutputNode(Script.GetUsage(), Script.GetUsageId());
	if (OutputNode != nullptr)
	{
		TArray<FString> ValidFunctionCallNames;
		GetFunctionNamesForOutputNode(*OutputNode, ValidFunctionCallNames);
		TArray<FNiagaraVariable> RapidIterationParameters;
		Script.RapidIterationParameters.GetParameters(RapidIterationParameters);
		for (const FNiagaraVariable& RapidIterationParameter : RapidIterationParameters)
		{
			FString EmitterName;
			FString FunctionCallName;
			FString InputName;
			if (FNiagaraParameterUtilities::SplitRapidIterationParameterName(RapidIterationParameter, Script.GetUsage(), EmitterName, FunctionCallName, InputName))
			{
				if (EmitterName != OwningEmitter.Emitter->GetUniqueEmitterName() || ValidFunctionCallNames.Contains(FunctionCallName) == false)
				{
					Script.RapidIterationParameters.RemoveParameter(RapidIterationParameter);
				}
			}
		}
	}
}

void FNiagaraStackGraphUtilities::CleanUpStaleRapidIterationParameters(FVersionedNiagaraEmitter Emitter)
{
	TArray<UNiagaraScript*> Scripts;
	Emitter.GetEmitterData()->GetScripts(Scripts, false);
	for (UNiagaraScript* Script : Scripts)
	{
		CleanUpStaleRapidIterationParameters(*Script, Emitter);
	}
}

void FNiagaraStackGraphUtilities::GetNewParameterAvailableTypes(TArray<FNiagaraTypeDefinition>& OutAvailableTypes, FName Namespace)
{
	TArray<FNiagaraTypeDefinition> ParamTypes;
	if (Namespace == FNiagaraConstants::UserNamespace)
	{
		FNiagaraEditorUtilities::GetAllowedUserVariableTypes(ParamTypes);
	}
	else if (Namespace == FNiagaraConstants::SystemNamespace)
	{
		FNiagaraEditorUtilities::GetAllowedSystemVariableTypes(ParamTypes);
	}
	else if (Namespace == FNiagaraConstants::EmitterNamespace)
	{
		FNiagaraEditorUtilities::GetAllowedEmitterVariableTypes(ParamTypes);
	}
	else if (Namespace == FNiagaraConstants::ParticleAttributeNamespace)
	{
		FNiagaraEditorUtilities::GetAllowedParticleVariableTypes(ParamTypes);
	}
	else
	{
		FNiagaraEditorUtilities::GetAllowedParameterTypes(ParamTypes);
	}

	for (const FNiagaraTypeDefinition& RegisteredParameterType : ParamTypes)
	{
		//Object types only allowed in user namespace at the moment.
		if (RegisteredParameterType.IsUObject() && RegisteredParameterType.IsDataInterface() == false && Namespace != FNiagaraConstants::UserNamespace)
		{
			continue;
		}

		if (RegisteredParameterType.IsInternalType())
		{
			continue;
		}

		if (RegisteredParameterType != FNiagaraTypeDefinition::GetGenericNumericDef() && RegisteredParameterType != FNiagaraTypeDefinition::GetParameterMapDef())
		{
			OutAvailableTypes.Add(RegisteredParameterType);
		}
	}
}

TOptional<FName> FNiagaraStackGraphUtilities::GetNamespaceForOutputNode(const UNiagaraNodeOutput* OutputNode)
{
	if (OutputNode)
	{
		TOptional<FName> StackContextAlias = OutputNode->GetStackContextOverride();
		if (StackContextAlias.IsSet() && StackContextAlias.GetValue() != NAME_None)
		{
			return StackContextAlias.GetValue();
		}
		return GetNamespaceForScriptUsage(OutputNode->GetUsage());
	}
	return TOptional<FName>();
}

TOptional<FName> FNiagaraStackGraphUtilities::GetNamespaceForScriptUsage(ENiagaraScriptUsage ScriptUsage)
{
	switch (ScriptUsage)
	{
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
		return FNiagaraConstants::ParticleAttributeNamespace;
	case ENiagaraScriptUsage::EmitterSpawnScript:
	case ENiagaraScriptUsage::EmitterUpdateScript:
		return FNiagaraConstants::EmitterNamespace;
	case ENiagaraScriptUsage::SystemSpawnScript:
	case ENiagaraScriptUsage::SystemUpdateScript:
		return FNiagaraConstants::SystemNamespace;
	default:
		return TOptional<FName>();
	}
}

struct FRapidIterationParameterContext
{
	FRapidIterationParameterContext()
		: UniqueEmitterName()
		, OwningFunctionCall(nullptr)
	{
	}

	FRapidIterationParameterContext(FString InUniqueEmitterName, UNiagaraNodeFunctionCall& InOwningFunctionCall)
		: UniqueEmitterName(InUniqueEmitterName)
		, OwningFunctionCall(&InOwningFunctionCall)
	{
	}

	bool IsValid()
	{
		return UniqueEmitterName.IsEmpty() == false && OwningFunctionCall != nullptr;
	}

	FNiagaraVariable GetValue(UNiagaraScript& OwningScript, FName InputName, FNiagaraTypeDefinition Type)
	{
		FNiagaraParameterHandle ModuleHandle = FNiagaraParameterHandle::CreateModuleParameterHandle(InputName);
		FNiagaraParameterHandle AliasedFunctionHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(ModuleHandle, OwningFunctionCall);
		FNiagaraVariable RapidIterationParameter = FNiagaraStackGraphUtilities::CreateRapidIterationParameter(UniqueEmitterName, OwningScript.GetUsage(), AliasedFunctionHandle.GetParameterHandleString(), Type);
		const uint8* ValueData = OwningScript.RapidIterationParameters.GetParameterData(RapidIterationParameter);
		if (ValueData != nullptr)
		{
			RapidIterationParameter.SetData(ValueData);
			return RapidIterationParameter;
		}
		return FNiagaraVariable();
	}

	const FString UniqueEmitterName;
	UNiagaraNodeFunctionCall* const OwningFunctionCall;
};

bool FNiagaraStackGraphUtilities::CanWriteParameterFromUsageViaOutput(FNiagaraVariable Parameter, const UNiagaraNodeOutput* OutputNode)
{
	bool bCanWrite = CanWriteParameterFromUsage(Parameter, OutputNode->GetUsage(), OutputNode->GetStackContextOverride(), OutputNode->GetAllStackContextOverrides());	
	return bCanWrite;
}

bool FNiagaraStackGraphUtilities::CanWriteParameterFromUsage(FNiagaraVariable Parameter, ENiagaraScriptUsage Usage, const TOptional<FName>& StackContextOverride, const TArray<FName>& StackContextAllOverrides)
{
	const FNiagaraParameterHandle ParameterHandle(Parameter.GetName());

	if (ParameterHandle.IsReadOnlyHandle())
	{
		return false;
	}

	if (ParameterHandle.IsTransientHandle())
	{
		return true;
	}

	if (ParameterHandle.IsStackContextHandle())
	{
		return true;
	}

	// Are we in the specified namespace for this stack context override? If so, we can definitely be written
	if (StackContextOverride.IsSet() && Parameter.IsInNameSpace(StackContextOverride.GetValue()))
	{
		return true;
	}

	// Do we belong to any of the namespaces that are stack context overrides? If so, we aren't the one that is currently set as that would pass above, so definitely can't write here.
	for (const FName& OverrideNamespace : StackContextAllOverrides)
	{
		if (Parameter.IsInNameSpace(OverrideNamespace))
		{
			return false;
		}
	}

	switch (Usage)
	{
	case ENiagaraScriptUsage::SystemSpawnScript:
	case ENiagaraScriptUsage::SystemUpdateScript:
		return ParameterHandle.IsSystemHandle();
	case ENiagaraScriptUsage::EmitterSpawnScript:
	case ENiagaraScriptUsage::EmitterUpdateScript:
		return ParameterHandle.IsEmitterHandle();
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
		return ParameterHandle.IsParticleAttributeHandle();
	default:
		return false;
	}
}

bool FNiagaraStackGraphUtilities::GetStackIssuesRecursively(const UNiagaraStackEntry* const Entry, TArray<UNiagaraStackErrorItem*>& OutIssues)
{
	TArray<UNiagaraStackEntry*> Entries;
	Entry->GetUnfilteredChildren(Entries);
	while (Entries.Num() > 0)
	{
		UNiagaraStackEntry* EntryToProcess = Entries[0];
		UNiagaraStackErrorItem* ErrorItem = Cast<UNiagaraStackErrorItem>(EntryToProcess);
		if (ErrorItem != nullptr)
		{
			OutIssues.Add(ErrorItem);
		}
		else // don't process error items, errors don't have errors
		{
			EntryToProcess->GetUnfilteredChildren(Entries);
		}
		Entries.RemoveAtSwap(0); 
	}
	return OutIssues.Num() > 0;
}

void FNiagaraStackGraphUtilities::MoveModule(UNiagaraScript& SourceScript, UNiagaraNodeFunctionCall& ModuleToMove, UNiagaraSystem& TargetSystem, FGuid TargetEmitterHandleId, ENiagaraScriptUsage TargetUsage, FGuid TargetUsageId, int32 TargetModuleIndex, bool bForceCopy, UNiagaraNodeFunctionCall*& OutMovedModule)
{
	UE_LOGF(LogNiagaraEditor, Log, "Move module %ls", *ModuleToMove.GetPathName());

	UNiagaraScript* TargetScript = FNiagaraEditorUtilities::GetScriptFromSystem(TargetSystem, TargetEmitterHandleId, TargetUsage, TargetUsageId);
	checkf(TargetScript != nullptr, TEXT("Target script not found"));

	UNiagaraNodeOutput* TargetOutputNode = FNiagaraEditorUtilities::GetScriptOutputNode(*TargetScript);
	checkf(TargetOutputNode != nullptr, TEXT("Target stack is invalid"));

	TArray<FStackNodeGroup> SourceGroups;
	GetStackNodeGroups(ModuleToMove, SourceGroups);
	int32 SourceGroupIndex = SourceGroups.IndexOfByPredicate([&ModuleToMove](const FStackNodeGroup& SourceGroup) { return SourceGroup.EndNode == &ModuleToMove; });
	TArray<UNiagaraNode*> SourceGroupNodes;
	SourceGroups[SourceGroupIndex].GetAllNodesInGroup(SourceGroupNodes);

	UNiagaraGraph* SourceGraph = ModuleToMove.GetNiagaraGraph();
	UNiagaraGraph* TargetGraph = TargetOutputNode->GetNiagaraGraph();

	if (SourceGraph != TargetGraph && bForceCopy == false)
	{
		SourceGraph->Modify();
	}
	TargetGraph->Modify();

	// If the source and target scripts don't match, or we're forcing a copy we need to collect the rapid iteration parameter values for each function in the source group
	// so we can restore them after moving.
	TMap<FGuid, TArray<FNiagaraVariable>> SourceFunctionIdToRapidIterationParametersMap;
	if (&SourceScript != TargetScript || bForceCopy)
	{
		TMap<FString, FGuid> FunctionCallNameToNodeIdMap;
		for (UNiagaraNode* SourceGroupNode : SourceGroupNodes)
		{
			UNiagaraNodeFunctionCall* FunctionNode = Cast<UNiagaraNodeFunctionCall>(SourceGroupNode);
			if (FunctionNode != nullptr)
			{
				FunctionCallNameToNodeIdMap.Add(FunctionNode->GetFunctionName(), FunctionNode->NodeGuid);
			}
		}

		TArray<FNiagaraVariable> ScriptRapidIterationParameters;
		SourceScript.RapidIterationParameters.GetParameters(ScriptRapidIterationParameters);
		for (FNiagaraVariable& ScriptRapidIterationParameter : ScriptRapidIterationParameters)
		{
			FString EmitterName;
			FString FunctionCallName;
			FString InputName;
			if (FNiagaraParameterUtilities::SplitRapidIterationParameterName(
				ScriptRapidIterationParameter, SourceScript.GetUsage(), EmitterName, FunctionCallName, InputName))
			{
				FGuid* NodeIdPtr = FunctionCallNameToNodeIdMap.Find(FunctionCallName);
				if (NodeIdPtr != nullptr)
				{
					TArray<FNiagaraVariable>& RapidIterationParameters = SourceFunctionIdToRapidIterationParametersMap.FindOrAdd(*NodeIdPtr);
					RapidIterationParameters.Add(ScriptRapidIterationParameter);
					RapidIterationParameters.Last().SetData(SourceScript.RapidIterationParameters.GetParameterData(ScriptRapidIterationParameter));
				}
			}
		}
	}

	FStackNodeGroup TargetGroup;
	TArray<UNiagaraNode*> TargetGroupNodes;
	TMap<FGuid, FGuid> OldNodeIdToNewIdMap;
	if (SourceGraph == TargetGraph && bForceCopy == false)
	{
		TargetGroup = SourceGroups[SourceGroupIndex];
		TargetGroupNodes = SourceGroupNodes;
	}
	else 
	{
		// If the module is being inserted into a different graph, or it's being copied, all of the nodes need to be copied into the target graph.
		FStackNodeGroup SourceGroup = SourceGroups[SourceGroupIndex];

		// HACK! The following code and the code after the import/export is necessary since sub-objects with a "." in them will not be correctly imported from text!
		TMap<FGuid, FString> NodeIdToOriginalName;
		for (UNiagaraNode* SourceGroupNode : SourceGroupNodes)
		{
			UNiagaraNodeInput* InputSourceGroupNode = Cast<UNiagaraNodeInput>(SourceGroupNode);
			if (InputSourceGroupNode != nullptr && InputSourceGroupNode->GetDataInterface() != nullptr)
			{
				NodeIdToOriginalName.Add(InputSourceGroupNode->NodeGuid, InputSourceGroupNode->GetDataInterface()->GetName());
				FString NewSanitizedName = InputSourceGroupNode->GetDataInterface()->GetName().Replace(TEXT("."), TEXT("_"));
				InputSourceGroupNode->GetDataInterface()->Rename(*NewSanitizedName);
			}
		}
		// HACK end

		TSet<UObject*> NodesToCopy;
		for (UNiagaraNode* SourceGroupNode : SourceGroupNodes)
		{
			SourceGroupNode->PrepareForCopying();
			NodesToCopy.Add(SourceGroupNode);
		}

		FString ExportedText;
		FEdGraphUtilities::ExportNodesToText(NodesToCopy, ExportedText);

		TSet<UEdGraphNode*> CopiedNodesSet;
		FEdGraphUtilities::ImportNodesFromText(TargetGraph, ExportedText, CopiedNodesSet);
		TArray<UEdGraphNode*> CopiedNodes = CopiedNodesSet.Array();

		// HACK continued.
		if (NodeIdToOriginalName.Num() > 0)
		{
			for (UEdGraphNode* CopiedNode : CopiedNodes)
			{
				FString* OriginalName = NodeIdToOriginalName.Find(CopiedNode->NodeGuid);
				if (OriginalName != nullptr)
				{
					CastChecked<UNiagaraNodeInput>(CopiedNode)->GetDataInterface()->Rename(**OriginalName);
				}
			}
		}
		// HACK end

		// Collect the start and end nodes for the group by ID before assigning the copied nodes new ids.
		UEdGraphNode** CopiedEndNode = CopiedNodes.FindByPredicate([SourceGroup](UEdGraphNode* CopiedNode)
			{ return CopiedNode->NodeGuid == SourceGroup.EndNode->NodeGuid; });
		checkf(CopiedEndNode != nullptr, TEXT("Group copy failed"));
		TargetGroup.EndNode = CastChecked<UNiagaraNode>(*CopiedEndNode);

		for (UNiagaraNode* StartNode : SourceGroup.StartNodes)
		{
			UEdGraphNode** CopiedStartNode = CopiedNodes.FindByPredicate([StartNode](UEdGraphNode* CopiedNode)
				{ return CopiedNode->NodeGuid == StartNode->NodeGuid; });
			checkf(CopiedStartNode != nullptr, TEXT("Group copy failed"));
			TargetGroup.StartNodes.Add(CastChecked<UNiagaraNode>(*CopiedStartNode));
		}

		TargetGroup.GetAllNodesInGroup(TargetGroupNodes);

		// Assign all of the new nodes new ids and mark them as requiring synchronization.
		for (UEdGraphNode* CopiedNode : CopiedNodes)
		{
			FGuid OldId = CopiedNode->NodeGuid;
			CopiedNode->CreateNewGuid();
			OldNodeIdToNewIdMap.Add(OldId, CopiedNode->NodeGuid);
			UNiagaraNode* CopiedNiagaraNode = Cast<UNiagaraNode>(CopiedNode);
			if (CopiedNiagaraNode != nullptr)
			{
				CopiedNiagaraNode->MarkNodeRequiresSynchronization(__FUNCTION__, false);
			}
		}

		FNiagaraEditorUtilities::FixUpPastedNodes(TargetGraph, CopiedNodesSet);
	}

	TArray<FStackNodeGroup> TargetGroups;
	GetStackNodeGroups(*TargetOutputNode, TargetGroups);

	// The first group is the output node, so to get the group index from module index we need to add 1, but 
	// if a valid index wasn't supplied, than we insert at the end.
	int32 TargetGroupIndex = TargetModuleIndex != INDEX_NONE ? TargetModuleIndex + 1 : TargetGroups.Num() - 1;

	// If we're not forcing a copy of the moved module, remove the source module group from it's stack.
	if (bForceCopy == false)
	{
		DisconnectStackNodeGroup(SourceGroups[SourceGroupIndex], SourceGroups[SourceGroupIndex - 1], SourceGroups[SourceGroupIndex + 1]);
		if (SourceGraph != TargetGraph)
		{
			// If the graphs were different also remove the nodes from the source graph.
			for (UNiagaraNode* SourceGroupNode : SourceGroupNodes)
			{
				SourceGraph->RemoveNode(SourceGroupNode);
			}
		}
	}

	// Insert the source or copied nodes into the target stack.
	ConnectStackNodeGroup(TargetGroup, TargetGroups[TargetGroupIndex - 1], TargetGroups[TargetGroupIndex]);

	// Copy any rapid iteration parameters cached earlier into the target script.
	if (SourceFunctionIdToRapidIterationParametersMap.Num() != 0)
	{
		SourceScript.Modify();
		TargetScript->Modify();
		if (SourceGraph == TargetGraph && bForceCopy == false)
		{
			// If we're not copying and if the module was dropped in the same graph than neither the emitter or function call name could have changed
			// so we can just add them directly to the target script.
			for (auto It = SourceFunctionIdToRapidIterationParametersMap.CreateIterator(); It; ++It)
			{
				TArray<FNiagaraVariable>& RapidIterationParameters = It.Value();
				for (FNiagaraVariable& RapidIterationParameter : RapidIterationParameters)
				{
					TargetScript->RapidIterationParameters.SetParameterData(RapidIterationParameter.GetData(), RapidIterationParameter, true);
				}
			}
		}
		else
		{
			// If we're copying or the module was moved to a different graph it's possible that the emitter name or function call name could have
			// changed so we need to construct new rapid iteration parameters.
			FString EmitterName;
			if (TargetEmitterHandleId.IsValid())
			{
				const FNiagaraEmitterHandle* TargetEmitterHandle = TargetSystem.GetEmitterHandles().FindByPredicate(
					[TargetEmitterHandleId](const FNiagaraEmitterHandle& EmitterHandle) { return EmitterHandle.GetId() == TargetEmitterHandleId; });
				EmitterName = TargetEmitterHandle->GetUniqueInstanceName();
			}

			for (auto It = SourceFunctionIdToRapidIterationParametersMap.CreateIterator(); It; ++It)
			{
				FGuid& FunctionId = It.Key();
				TArray<FNiagaraVariable>& RapidIterationParameters = It.Value();

				FGuid TargetNodeId = OldNodeIdToNewIdMap[FunctionId];
				UNiagaraNode** TargetFunctionNodePtr = TargetGroupNodes.FindByPredicate(
					[TargetNodeId](UNiagaraNode* TargetGroupNode) { return TargetGroupNode->NodeGuid == TargetNodeId; });
				checkf(TargetFunctionNodePtr != nullptr, TEXT("Target nodes not copied correctly"));
				UNiagaraNodeFunctionCall* TargetFunctionNode = CastChecked<UNiagaraNodeFunctionCall>(*TargetFunctionNodePtr);
				for (FNiagaraVariable& RapidIterationParameter : RapidIterationParameters)
				{
					FString OldEmitterName;
					FString OldFunctionCallName;
					FString InputName;
					FNiagaraParameterUtilities::SplitRapidIterationParameterName(RapidIterationParameter, SourceScript.GetUsage(), OldEmitterName, OldFunctionCallName, InputName);
					FNiagaraParameterHandle ModuleHandle = FNiagaraParameterHandle::CreateModuleParameterHandle(*InputName);
					FNiagaraParameterHandle AliasedModuleHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(ModuleHandle, TargetFunctionNode);
					FNiagaraVariable TargetRapidIterationParameter = CreateRapidIterationParameter(EmitterName, TargetUsage, AliasedModuleHandle.GetParameterHandleString(), RapidIterationParameter.GetType());
					TargetScript->RapidIterationParameters.SetParameterData(RapidIterationParameter.GetData(), TargetRapidIterationParameter, true);
				}
			}
		}
	}

	OutMovedModule = Cast<UNiagaraNodeFunctionCall>(TargetGroup.EndNode);


	UE_LOGF(LogNiagaraEditor, Log, "Finished moving!");
}

bool FNiagaraStackGraphUtilities::ParameterAllowedInExecutionCategory(const FName InParameterName, const FName ExecutionCategory)
{
	FNiagaraParameterHandle Handle = FNiagaraParameterHandle(InParameterName);
	if (Handle.IsSystemHandle())
	{
		return ExecutionCategory == UNiagaraStackEntry::FExecutionCategoryNames::System
			|| ExecutionCategory == UNiagaraStackEntry::FExecutionCategoryNames::Emitter
			|| ExecutionCategory == UNiagaraStackEntry::FExecutionCategoryNames::Particle;
	}
	else if (Handle.IsEmitterHandle())
	{
		return ExecutionCategory == UNiagaraStackEntry::FExecutionCategoryNames::Emitter
			|| ExecutionCategory == UNiagaraStackEntry::FExecutionCategoryNames::Particle;
	}
	else if (Handle.IsParticleAttributeHandle())
	{
		return ExecutionCategory == UNiagaraStackEntry::FExecutionCategoryNames::Particle;
	}

	return true;
}

void FNiagaraStackGraphUtilities::RebuildEmitterNodes(UNiagaraSystem& System)
{
	UNiagaraScriptSource* SystemScriptSource = Cast<UNiagaraScriptSource>(System.GetSystemSpawnScript()->GetLatestSource());
	UNiagaraGraph* SystemGraph = SystemScriptSource->NodeGraph;
	if (SystemGraph == nullptr)
	{
		return;
	}
	SystemGraph->Modify();

	TArray<UNiagaraNodeEmitter*> CurrentEmitterNodes;
	SystemGraph->GetNodesOfClass<UNiagaraNodeEmitter>(CurrentEmitterNodes);

	const UEdGraphSchema_Niagara* Schema = Cast<UEdGraphSchema_Niagara>(SystemGraph->GetSchema());

	// Remove the old emitter nodes since they will be rebuilt below.
	for (UNiagaraNodeEmitter* CurrentEmitterNode : CurrentEmitterNodes)
	{
		CurrentEmitterNode->Modify();
		UEdGraphPin* InPin = CurrentEmitterNode->GetInputPin(0);
		UEdGraphPin* OutPin = CurrentEmitterNode->GetOutputPin(0);
		UEdGraphPin* InPinLinkedPin = InPin != nullptr && InPin->LinkedTo.Num() == 1 ? InPin->LinkedTo[0] : nullptr;
		UEdGraphPin* OutPinLinkedPin = OutPin != nullptr && OutPin->LinkedTo.Num() == 1 ? OutPin->LinkedTo[0] : nullptr;
		CurrentEmitterNode->DestroyNode();

		if (InPinLinkedPin != nullptr && OutPinLinkedPin != nullptr)
		{
			MakeLinkTo(InPinLinkedPin, OutPinLinkedPin);
		}
	}

	// Add output nodes if they don't exist.
	TArray<UNiagaraNodeInput*> TempInputNodes;
	TArray<UNiagaraNodeInput*> InputNodes;
	TArray<UNiagaraNodeOutput*> OutputNodes;
	OutputNodes.SetNum(2);
	OutputNodes[0] = SystemGraph->FindOutputNode(ENiagaraScriptUsage::SystemSpawnScript);
	OutputNodes[1] = SystemGraph->FindOutputNode(ENiagaraScriptUsage::SystemUpdateScript);

	// Add input nodes if they don't exist
	UNiagaraGraph::FFindInputNodeOptions Options;
	Options.bFilterDuplicates = false;
	Options.bIncludeParameters = true;
	SystemGraph->FindInputNodes(TempInputNodes);
	for (int32 i = 0; i < TempInputNodes.Num(); i++)
	{
		if (Schema->PinToTypeDefinition(TempInputNodes[i]->GetOutputPin(0)) == FNiagaraTypeDefinition::GetParameterMapDef())
		{
			InputNodes.Add(TempInputNodes[i]);
		}
	}

	// Create a default id variable for the input nodes.
	FNiagaraVariable SharedInputVar(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("InputMap"));
	InputNodes.SetNum(2);

	// Now create the nodes if they are needed, synchronize if already created.
	for (int32 i = 0; i < 2; i++)
	{
		if (OutputNodes[i] == nullptr)
		{
			FGraphNodeCreator<UNiagaraNodeOutput> OutputNodeCreator(*SystemGraph);
			OutputNodes[i] = OutputNodeCreator.CreateNode();
			OutputNodes[i]->SetUsage((ENiagaraScriptUsage)(i + (int32)ENiagaraScriptUsage::SystemSpawnScript));

			OutputNodes[i]->Outputs.Add(FNiagaraVariable(FNiagaraTypeDefinition::GetParameterMapDef(), TEXT("Out")));
			OutputNodes[i]->NodePosX = 0;
			OutputNodes[i]->NodePosY = 0 + i * 25;

			OutputNodeCreator.Finalize();
		}
		if (InputNodes[i] == nullptr)
		{
			FGraphNodeCreator<UNiagaraNodeInput> InputNodeCreator(*SystemGraph);
			InputNodes[i] = InputNodeCreator.CreateNode();
			InputNodes[i]->Input = SharedInputVar;
			InputNodes[i]->Usage = ENiagaraInputNodeUsage::Parameter;
			InputNodes[i]->NodePosX = -50;
			InputNodes[i]->NodePosY = 0 + i * 25;

			InputNodeCreator.Finalize();

			MakeLinkTo(InputNodes[i]->GetOutputPin(0), OutputNodes[i]->GetInputPin(0));
		}
	}

	// Add new nodes.
	UNiagaraNode* TargetNodes[2];
	TargetNodes[0] = OutputNodes[0];
	TargetNodes[1] = OutputNodes[1];

	for (const FNiagaraEmitterHandle& EmitterHandle : System.GetEmitterHandles())
	{
		for (int32 i = 0; i < 2; i++)
		{
			FGraphNodeCreator<UNiagaraNodeEmitter> EmitterNodeCreator(*SystemGraph);
			UNiagaraNodeEmitter* EmitterNode = EmitterNodeCreator.CreateNode();
			EmitterNode->SetOwnerSystem(&System);
			EmitterNode->SetEmitterHandleId(EmitterHandle.GetId());
			EmitterNode->SetUsage((ENiagaraScriptUsage)(i + (int32)ENiagaraScriptUsage::EmitterSpawnScript));
			EmitterNode->AllocateDefaultPins();
			EmitterNodeCreator.Finalize();

			TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackNodeGroups;
			FNiagaraStackGraphUtilities::GetStackNodeGroups(*OutputNodes[i], StackNodeGroups);

			if (StackNodeGroups.Num() >= 2)
			{
				FNiagaraStackGraphUtilities::FStackNodeGroup EmitterGroup;
				EmitterGroup.StartNodes.Add(EmitterNode);
				EmitterGroup.EndNode = EmitterNode;

				FNiagaraStackGraphUtilities::FStackNodeGroup& OutputGroup = StackNodeGroups[StackNodeGroups.Num() - 1];
				FNiagaraStackGraphUtilities::FStackNodeGroup& OutputGroupPrevious = StackNodeGroups[StackNodeGroups.Num() - 2];
				FNiagaraStackGraphUtilities::ConnectStackNodeGroup(EmitterGroup, OutputGroupPrevious, OutputGroup);
			}
		}
	}

	RelayoutGraph(*SystemGraph);
}

void FNiagaraStackGraphUtilities::FindAffectedScripts(UNiagaraSystem* System, FVersionedNiagaraEmitter Emitter, UNiagaraNodeFunctionCall& ModuleNode, TArray<TWeakObjectPtr<UNiagaraScript>>& OutAffectedScripts)
{
	UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(ModuleNode);

	if (OutputNode)
	{
		TArray<UNiagaraScript*> Scripts;
		if (FVersionedNiagaraEmitterData* EmitterData = Emitter.GetEmitterData())
		{
			EmitterData->GetScripts(Scripts, false);
		}

		if (System != nullptr)
		{
			OutAffectedScripts.Add(System->GetSystemSpawnScript());
			OutAffectedScripts.Add(System->GetSystemUpdateScript());
		}

		for (UNiagaraScript* Script : Scripts)
		{
			if (OutputNode->GetUsage() == ENiagaraScriptUsage::ParticleEventScript)
			{
				if (Script->GetUsage() == ENiagaraScriptUsage::ParticleEventScript && Script->GetUsageId() == OutputNode->GetUsageId())
				{
					OutAffectedScripts.Add(Script);
					break;
				}
			}
			else if (Script->ContainsUsage(OutputNode->GetUsage()))
			{
				OutAffectedScripts.Add(Script);
			}
		}
	}
}

void FNiagaraStackGraphUtilities::GatherRenamedStackFunctionOutputVariableNames(FVersionedNiagaraEmitter Emitter, UNiagaraNodeFunctionCall& FunctionCallNode, const FString& OldFunctionName, const FString& NewFunctionName, TMap<FName, FName>& OutOldToNewNameMap)
{
	if (OldFunctionName == NewFunctionName)
	{
		return;
	}

	const UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(FunctionCallNode);

	TArray<FNiagaraVariable> OutputVariables;
	TArray<FNiagaraVariable> OutputVariablesWithOriginalAliasesIntact;
	FCompileConstantResolver ConstantResolver = OutputNode != nullptr 
		? FCompileConstantResolver(Emitter, OutputNode->GetUsage(), FunctionCallNode.DebugState)
		: FCompileConstantResolver();
	FNiagaraStackGraphUtilities::GetStackFunctionOutputVariables(FunctionCallNode, ConstantResolver, OutputVariables, OutputVariablesWithOriginalAliasesIntact);

	for (FNiagaraVariable& OutputVariableWithOriginalAliasesIntact : OutputVariablesWithOriginalAliasesIntact)
	{
		TArray<FString> SplitAliasedVariableName;
		OutputVariableWithOriginalAliasesIntact.GetName().ToString().ParseIntoArray(SplitAliasedVariableName, TEXT("."));
		if (SplitAliasedVariableName.Contains(TEXT("Module")))
		{
			TArray<FString> SplitOldVariableName = SplitAliasedVariableName;
			TArray<FString> SplitNewVariableName = SplitAliasedVariableName;
			for (int32 i = 0; i < SplitAliasedVariableName.Num(); i++)
			{
				if (SplitAliasedVariableName[i] == TEXT("Module"))
				{
					SplitOldVariableName[i] = OldFunctionName;
					SplitNewVariableName[i] = NewFunctionName;
				}
			}

			OutOldToNewNameMap.Add(*FString::Join(SplitOldVariableName, TEXT(".")), *FString::Join(SplitNewVariableName, TEXT(".")));
		}
	}
}

void FNiagaraStackGraphUtilities::GatherRenamedStackFunctionInputAndOutputVariableNames(FVersionedNiagaraEmitter Emitter, UNiagaraNodeFunctionCall& FunctionCallNode, const FString& OldFunctionName, const FString& NewFunctionName, TMap<FName, FName>& OutOldToNewNameMap)
{
	if (OldFunctionName == NewFunctionName)
	{
		return;
	}

	const UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(FunctionCallNode);

	TArray<FNiagaraVariable> Variables;
	TArray<FNiagaraVariable> VariablesWithOriginalAliasesIntact;
	FCompileConstantResolver ConstantResolver = OutputNode != nullptr
		? FCompileConstantResolver(Emitter, OutputNode->GetUsage(), FunctionCallNode.DebugState)
		: FCompileConstantResolver();
	FNiagaraStackGraphUtilities::GetStackFunctionInputAndOutputVariables(FunctionCallNode, ConstantResolver, Variables, VariablesWithOriginalAliasesIntact);

	for (FNiagaraVariable& Variable : VariablesWithOriginalAliasesIntact)
	{
		TArray<FString> SplitAliasedVariableName;
		Variable.GetName().ToString().ParseIntoArray(SplitAliasedVariableName, TEXT("."));
		if (SplitAliasedVariableName.Contains(TEXT("Module")))
		{
			TArray<FString> SplitOldVariableName = SplitAliasedVariableName;
			TArray<FString> SplitNewVariableName = SplitAliasedVariableName;
			for (int32 i = 0; i < SplitAliasedVariableName.Num(); i++)
			{
				if (SplitAliasedVariableName[i] == TEXT("Module"))
				{
					SplitOldVariableName[i] = OldFunctionName;
					SplitNewVariableName[i] = NewFunctionName;
				}
			}

			OutOldToNewNameMap.Add(*FString::Join(SplitOldVariableName, TEXT(".")), *FString::Join(SplitNewVariableName, TEXT(".")));
		}
	}
}

void FNiagaraStackGraphUtilities::RenameReferencingParameters(UNiagaraSystem* System, FVersionedNiagaraEmitter Emitter, UNiagaraNodeFunctionCall& FunctionCallNode, const FString& OldModuleName, const FString& NewModuleName)
{
	TMap<FName, FName> OldNameToNewNameMap;
	FNiagaraStackGraphUtilities::GatherRenamedStackFunctionInputAndOutputVariableNames(Emitter, FunctionCallNode, OldModuleName, NewModuleName, OldNameToNewNameMap);

	// local function to rename pins referencing the given module
	auto RenamePinsReferencingModule = [&OldNameToNewNameMap](UNiagaraNodeParameterMapBase* Node)
	{
		for (UEdGraphPin* Pin : Node->Pins)
		{
			FName* NewName = OldNameToNewNameMap.Find(Pin->PinName);
			if (NewName != nullptr)
			{
				Node->SetPinName(Pin, *NewName);
			}
		}
	};

	UNiagaraNodeParameterMapSet* ParameterMapSet = FNiagaraStackGraphUtilities::GetStackFunctionOverrideNode(FunctionCallNode);
	if (ParameterMapSet != nullptr)
	{
		RenamePinsReferencingModule(ParameterMapSet);
	}

	TArray<TWeakObjectPtr<UNiagaraScript>> Scripts;
	FindAffectedScripts(System, Emitter, FunctionCallNode, Scripts);

	FString OwningEmitterName = Emitter.Emitter != nullptr ? Emitter.Emitter->GetUniqueEmitterName() : FString();

	for (TWeakObjectPtr<UNiagaraScript> Script : Scripts)
	{
		if (!Script.IsValid(false))
		{
			continue;
		}

		Script->Modify();
		TArray<FNiagaraVariable> RapidIterationVariables;
		Script->RapidIterationParameters.GetParameters(RapidIterationVariables);

		for (FNiagaraVariable& Variable : RapidIterationVariables)
		{
			FString EmitterName, FunctionCallName, InputName;
			if (FNiagaraParameterUtilities::SplitRapidIterationParameterName(Variable, Script->GetUsage(), EmitterName, FunctionCallName, InputName))
			{
				if (EmitterName == OwningEmitterName && FunctionCallName == OldModuleName)
				{
					FName NewParameterName(*(NewModuleName + TEXT(".") + InputName));
					FNiagaraVariable NewParameter = FNiagaraStackGraphUtilities::CreateRapidIterationParameter(EmitterName, Script->GetUsage(), NewParameterName, Variable.GetType());
					Script->RapidIterationParameters.RenameParameter(Variable, NewParameter.GetName());
				}
			}
		}

		if (UNiagaraScriptSource* ScriptSource = Cast<UNiagaraScriptSource>(Script->GetLatestSource()))
		{
			// rename all parameter map get nodes that use the parameter name
			TArray<UNiagaraNodeParameterMapGet*> ParameterGetNodes;
			ScriptSource->NodeGraph->GetNodesOfClass<UNiagaraNodeParameterMapGet>(ParameterGetNodes);

			for (UNiagaraNodeParameterMapGet* Node : ParameterGetNodes)
			{
				RenamePinsReferencingModule(Node);
			}
		}
	}
}

void FNiagaraStackGraphUtilities::GetNamespacesForNewReadParameters(EStackEditContext EditContext, ENiagaraScriptUsage Usage, TArray<FName>& OutNamespacesForNewParameters)
{
	switch (Usage)
	{
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
	{
		OutNamespacesForNewParameters.Add(FNiagaraConstants::ParticleAttributeNamespace);
		OutNamespacesForNewParameters.Add(FNiagaraConstants::EmitterNamespace);
		break;
	}
	case ENiagaraScriptUsage::EmitterSpawnScript:
	case ENiagaraScriptUsage::EmitterUpdateScript:
	{
		OutNamespacesForNewParameters.Add(FNiagaraConstants::EmitterNamespace);
		break;
	}
	}

	if (EditContext == EStackEditContext::System)
	{
		OutNamespacesForNewParameters.Add(FNiagaraConstants::UserNamespace);
		OutNamespacesForNewParameters.Add(FNiagaraConstants::SystemNamespace);
	}
	OutNamespacesForNewParameters.Add(FNiagaraConstants::TransientNamespace);
}

void FNiagaraStackGraphUtilities::GetNamespacesForNewWriteParameters(EStackEditContext EditContext, ENiagaraScriptUsage Usage, const TOptional<FName>& StackContextAlias, TArray<FName>& OutNamespacesForNewParameters)
{
	switch (Usage)
	{
	case ENiagaraScriptUsage::ParticleSpawnScript:
	case ENiagaraScriptUsage::ParticleSpawnScriptInterpolated:
	case ENiagaraScriptUsage::ParticleUpdateScript:
	case ENiagaraScriptUsage::ParticleEventScript:
	case ENiagaraScriptUsage::ParticleSimulationStageScript:
	{
		OutNamespacesForNewParameters.Add(FNiagaraConstants::ParticleAttributeNamespace);
		break;
	}
	case ENiagaraScriptUsage::EmitterSpawnScript:
	case ENiagaraScriptUsage::EmitterUpdateScript:
	{
		OutNamespacesForNewParameters.Add(FNiagaraConstants::EmitterNamespace);
		break;
	}
	case ENiagaraScriptUsage::SystemSpawnScript:
	case ENiagaraScriptUsage::SystemUpdateScript:
		if (EditContext == EStackEditContext::System)
		{
			OutNamespacesForNewParameters.Add(FNiagaraConstants::SystemNamespace);
		}
		break;
	}

	OutNamespacesForNewParameters.Add(FNiagaraConstants::TransientNamespace);
	OutNamespacesForNewParameters.Add(FNiagaraConstants::StackContextNamespace);

	if (StackContextAlias.IsSet())
		OutNamespacesForNewParameters.Add(StackContextAlias.GetValue());
}

bool FNiagaraStackGraphUtilities::TryRenameAssignmentTarget(UNiagaraNodeAssignment& OwningAssignmentNode, FNiagaraVariable CurrentAssignmentTarget, FName NewAssignmentTargetName)
{
	UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(OwningAssignmentNode);
	if (OutputNode != nullptr)
	{
		UNiagaraSystem* OwningSystem = OwningAssignmentNode.GetTypedOuter<UNiagaraSystem>();
		FVersionedNiagaraEmitter OwningEmitter = OwningAssignmentNode.GetNiagaraGraph()->GetOwningEmitter();
		UNiagaraScript* OwningScript = nullptr;
		if (OwningEmitter.Emitter)
		{
			OwningScript = OwningEmitter.GetEmitterData()->GetScript(OutputNode->GetUsage(), OutputNode->GetUsageId());
		}
		else if(OwningSystem != nullptr)
		{
			if (OutputNode->GetUsage() == ENiagaraScriptUsage::SystemSpawnScript)
			{
				OwningScript = OwningSystem->GetSystemSpawnScript();
			}
			else if (OutputNode->GetUsage() == ENiagaraScriptUsage::SystemUpdateScript)
			{
				OwningScript = OwningSystem->GetSystemUpdateScript();
			}
		}

		if (OwningSystem != nullptr && OwningScript != nullptr)
		{
			RenameAssignmentTarget(*OwningSystem, OwningEmitter, *OwningScript, OwningAssignmentNode, CurrentAssignmentTarget, NewAssignmentTargetName);
			return true;
		}
	}
	return false;
}

void FNiagaraStackGraphUtilities::RenameAssignmentTarget(
	UNiagaraSystem& OwningSystem,
	FVersionedNiagaraEmitter OwningEmitter,
	UNiagaraScript& OwningScript,
	UNiagaraNodeAssignment& OwningAssignmentNode,
	FNiagaraVariable CurrentAssignmentTarget,
	FName NewAssignmentTargetName)
{
	UNiagaraStackEditorData* StackEditorData;
	FVersionedNiagaraEmitterData* EmitterData = OwningEmitter.GetEmitterData();
	if (EmitterData != nullptr)
	{
		UNiagaraEmitterEditorData* EmitterEditorData = Cast<UNiagaraEmitterEditorData>(EmitterData->GetEditorData());
		StackEditorData = EmitterEditorData != nullptr ? &EmitterEditorData->GetStackEditorData() : nullptr;
	}
	else
	{
		UNiagaraSystemEditorData* SystemEditorData = Cast<UNiagaraSystemEditorData>(OwningSystem.GetEditorData());
		StackEditorData = SystemEditorData != nullptr ? &SystemEditorData->GetStackEditorData() : nullptr;
	}

	bool bIsCurrentlyExpanded = StackEditorData != nullptr 
		? StackEditorData->GetStackEntryIsExpanded(StackKeys::GenerateStackModuleEditorDataKey(OwningAssignmentNode), false) 
		: false;

	if (ensureMsgf(OwningAssignmentNode.RenameAssignmentTarget(CurrentAssignmentTarget.GetName(), NewAssignmentTargetName), TEXT("Failed to rename assignment node input.")))
	{
		// Fixing up the stack graph and rapid iteration parameters must happen first so that when the stack is refreshed the UI is correct.
		FNiagaraParameterHandle CurrentInputParameterHandle = FNiagaraParameterHandle::CreateModuleParameterHandle(CurrentAssignmentTarget.GetName());
		FNiagaraParameterHandle CurrentAliasedInputParameterHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(CurrentInputParameterHandle, &OwningAssignmentNode);
		FNiagaraParameterHandle NewInputParameterHandle = FNiagaraParameterHandle(CurrentInputParameterHandle.GetNamespace(), NewAssignmentTargetName);
		FNiagaraParameterHandle NewAliasedInputParameterHandle = FNiagaraParameterHandle::CreateAliasedModuleParameterHandle(NewInputParameterHandle, &OwningAssignmentNode);
		UEdGraphPin* OverridePin = GetStackFunctionInputOverridePin(OwningAssignmentNode, CurrentAliasedInputParameterHandle);
		if (OverridePin != nullptr)
		{
			// If there is an override pin then the only thing that needs to happen is that it's name needs to be updated so that the value it
			// holds or is linked to stays intact.
			OverridePin->Modify();
			OverridePin->PinName = NewAliasedInputParameterHandle.GetParameterHandleString();
		}
		else if (IsRapidIterationType(CurrentAssignmentTarget.GetType()))
		{
			// Otherwise if this is a rapid iteration type check to see if there is an existing rapid iteration value, and if so, rename it.
			FString UniqueEmitterName = OwningEmitter.Emitter != nullptr ? OwningEmitter.Emitter->GetUniqueEmitterName() : FString();
			FNiagaraVariable CurrentRapidIterationParameter = CreateRapidIterationParameter(UniqueEmitterName, OwningScript.GetUsage(), CurrentAliasedInputParameterHandle.GetParameterHandleString(), CurrentAssignmentTarget.GetType());
			if (OwningScript.RapidIterationParameters.IndexOf(CurrentRapidIterationParameter) != INDEX_NONE)
			{
				FNiagaraVariable NewRapidIterationParameter = CreateRapidIterationParameter(UniqueEmitterName, OwningScript.GetUsage(), NewAliasedInputParameterHandle.GetParameterHandleString(), CurrentAssignmentTarget.GetType());
				OwningScript.Modify();
				OwningScript.RapidIterationParameters.RenameParameter(CurrentRapidIterationParameter, NewRapidIterationParameter.GetName());
			}
		}

		if (StackEditorData != nullptr)
		{
			// Restore the expanded state with the new editor data key.
			FString NewStackEditorDataKey = StackKeys::GenerateStackFunctionInputEditorDataKey(OwningAssignmentNode, NewInputParameterHandle);
			StackEditorData->SetStackEntryIsExpanded(NewStackEditorDataKey, bIsCurrentlyExpanded);
		}

		// This refresh call must come last because it will finalize this input entry which would cause earlier fixup to fail.
		OwningAssignmentNode.RefreshFromExternalChanges();
	}
}

void FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(UNiagaraNodeParameterMapBase* MapBaseNode, bool bCreateInputPin, const FNiagaraVariable& NewVariable)
{
	const EEdGraphPinDirection NewPinDirection = bCreateInputPin ? EGPD_Input : EGPD_Output;

	UNiagaraGraph* Graph = MapBaseNode->GetNiagaraGraph();
	if (!Graph)
	{
		return;
	}

	// First check that the new variable exists on the UNiagaraGraph. If not, add the new variable as a parameter.
	if (!Graph->HasVariable(NewVariable))
	{
		Graph->Modify();
		Graph->AddParameter(NewVariable);
	}

	// Then add the pin.
	MapBaseNode->Modify();
	UEdGraphPin* Pin = MapBaseNode->RequestNewTypedPin(NewPinDirection, NewVariable.GetType(), NewVariable.GetName());
	MapBaseNode->CancelEditablePinName(FText::GetEmpty(), Pin);
}

void FNiagaraStackGraphUtilities::AddNewVariableToParameterMapNode(UNiagaraNodeParameterMapBase* MapBaseNode, bool bCreateInputPin, const UNiagaraScriptVariable* NewScriptVar)
{
	const FNiagaraVariable& NewVariable = NewScriptVar->Variable;
	const EEdGraphPinDirection NewPinDirection = bCreateInputPin ? EGPD_Input : EGPD_Output;

	UNiagaraGraph* Graph = MapBaseNode->GetNiagaraGraph();
	if (!Graph)
	{
		return;
	}

	// First check that the new variable exists on the UNiagaraGraph. If not, add the new variable as a parameter.
	if (!Graph->HasVariable(NewVariable))
	{
		Graph->Modify();
		Graph->AddParameter(NewScriptVar);
	}

	// Then add the pin.
	MapBaseNode->Modify();
	UEdGraphPin* Pin = MapBaseNode->RequestNewTypedPin(NewPinDirection, NewVariable.GetType(), NewVariable.GetName());
	MapBaseNode->CancelEditablePinName(FText::GetEmpty(), Pin);
}

void FNiagaraStackGraphUtilities::SynchronizeVariableToLibraryAndApplyToGraph(UNiagaraScriptVariable* ScriptVarToSync)
{
	// Get all requisite utilties/objects first.
	const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
	UNiagaraGraph* Graph = Cast<UNiagaraGraph>(ScriptVarToSync->GetOuter());
	FNiagaraEditorModule& EditorModule = FNiagaraEditorModule::Get();
	TSharedPtr<class INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeUtilityValue = EditorModule.GetTypeUtilities(ScriptVarToSync->Variable.GetType());
	if (Graph == nullptr || TypeUtilityValue.IsValid() == false)
	{
		ensureMsgf(false, TEXT("Could not force synchronize definition value to graph: failed to get graph and/or create parameter type utility!"));
		return;
	}

	Graph->Modify();
	ScriptVarToSync->Modify();

	// Synchronize with parameter definitions.
	TSharedPtr<INiagaraParameterDefinitionsSubscriberViewModel> ParameterDefinitionsSubscriberViewModel = FNiagaraEditorUtilities::GetOwningLibrarySubscriberViewModelForGraph(Graph);
	if (ParameterDefinitionsSubscriberViewModel.IsValid())
	{
		const bool bForceSync = true;
		ParameterDefinitionsSubscriberViewModel->SynchronizeScriptVarWithParameterDefinitions(ScriptVarToSync, bForceSync);
	}

	// Set the value on the FNiagaraVariable from the UNiagaraScriptVariable.
	if (ScriptVarToSync->GetDefaultValueData() != nullptr)
	{
		ScriptVarToSync->Variable.SetData(ScriptVarToSync->GetDefaultValueData());
	}
	const FString NewDefaultValue = TypeUtilityValue->GetPinDefaultStringFromValue(ScriptVarToSync->Variable);

	// Set the value on the graph pins.
	for (UEdGraphPin* Pin : Graph->FindParameterMapDefaultValuePins(ScriptVarToSync->Variable.GetName()))
	{
		Pin->Modify();
		Schema->TrySetDefaultValue(*Pin, NewDefaultValue, true);
	}

	Graph->ScriptVariableChanged(ScriptVarToSync->Variable);
#if WITH_EDITOR
	Graph->NotifyGraphNeedsRecompile();
#endif
}

TSharedRef<TMap<FName, FGuid>> GetVariableNameToVariableIdMap(UNiagaraNodeFunctionCall& InFunctionCallNode)
{
	TSharedRef<TMap<FName, FGuid>> VariableNameToVariableIdMap = MakeShared<TMap<FName, FGuid>>();
	UNiagaraGraph* CalledGraph = InFunctionCallNode.GetCalledGraph();
	if (CalledGraph != nullptr)
	{
		for (const TPair<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>& VariableMetadataPair : CalledGraph->GetAllMetaData())
		{
			FGuid VariableGuid = VariableMetadataPair.Value->Metadata.GetVariableGuid();
			if (VariableGuid.IsValid())
			{
				VariableNameToVariableIdMap->FindOrAdd(VariableMetadataPair.Key.GetName()) = VariableGuid;
                // Add in alternate names as well so that we can map them cleanly in the second pass.
				for (const FName& AltName : VariableMetadataPair.Value->Metadata.AlternateAliases)
				{
					VariableNameToVariableIdMap->FindOrAdd(AltName) = VariableGuid;
				}
			}
		}
	}
	return VariableNameToVariableIdMap;
}

TSharedRef<TMap<FGuid, FNiagaraVariable>> GetVariableIdToVariableMap(UNiagaraNodeFunctionCall& InFunctionCallNode)
{
	TSharedRef<TMap<FGuid, FNiagaraVariable>> VariableIdToVariableMap = MakeShared<TMap<FGuid, FNiagaraVariable>>();
	UNiagaraGraph* CalledGraph = InFunctionCallNode.GetCalledGraph();
	if (CalledGraph != nullptr)
	{
		for (const TPair<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>& VariableMetadataPair : CalledGraph->GetAllMetaData())
		{
			FGuid VariableGuid = VariableMetadataPair.Value->Metadata.GetVariableGuid();
			if (VariableGuid.IsValid())
			{
				VariableIdToVariableMap->FindOrAdd(VariableGuid) = VariableMetadataPair.Key;
			}
		}
	}
	return VariableIdToVariableMap;
}

// Searches the graph which contains InFunctionCallNode and finds parameter map get pins which reference outputs from the function call node.
TArray<UEdGraphPin*> GetStackLinkedOutputPinsForFunction(UNiagaraNodeFunctionCall& InFunctionCallNode)
{
	TArray<UEdGraphPin*> LinkedOutputPins;
	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackNodeGroups;
	FName FunctionCallName = *InFunctionCallNode.GetFunctionName();
	GetStackNodeGroups(InFunctionCallNode, StackNodeGroups);
	for (const FNiagaraStackGraphUtilities::FStackNodeGroup& StackNodeGroup : StackNodeGroups)
	{
		TArray<UNiagaraNode*> NodesInGroup;
		StackNodeGroup.GetAllNodesInGroup(NodesInGroup);
		for (UNiagaraNode* NodeInGroup : NodesInGroup)
		{
			UNiagaraNodeParameterMapGet* MapGet = Cast<UNiagaraNodeParameterMapGet>(NodeInGroup);
			if (MapGet != nullptr)
			{
				TArray<UEdGraphPin*> MapGetOutputPins;
				MapGet->GetOutputPins(MapGetOutputPins);
				for (UEdGraphPin* MapGetOutputPin : MapGetOutputPins)
				{
					FNiagaraParameterHandle MapGetHandle(MapGetOutputPin->PinName);
					if (MapGetHandle.IsOutputHandle())
					{
						TArray<FName> HandleParts = MapGetHandle.GetHandleParts();
						if (HandleParts.Num() >= 3 && HandleParts[1] == FunctionCallName)
						{
							LinkedOutputPins.Add(MapGetOutputPin);
						}
					}
				}
			}
		}
	}
	return LinkedOutputPins;
}

// Searches the graph that contains InFunctionCallNode and any dependent graphs and finds parameter map get pins which reference data set attributes from the function call node.
TArray<UEdGraphPin*> GetStackLinkedAttributePinsForFunction(UNiagaraNodeFunctionCall& InFunctionCallNode, const TArray<UNiagaraGraph*>& DependentGraphs)
{
	TArray<UEdGraphPin*> LinkedAttributePins;
	FName FunctionCallName = *InFunctionCallNode.GetFunctionName();

	// Validate context.
	UNiagaraGraph* CalledGraph = InFunctionCallNode.GetCalledGraph();
	if (CalledGraph == nullptr)
	{
		return LinkedAttributePins;
	}
	UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(InFunctionCallNode);
	if (OutputNode == nullptr)
	{
		return LinkedAttributePins;
	}
	TOptional<FName> AttributeNamespace = FNiagaraStackGraphUtilities::GetNamespaceForOutputNode(OutputNode);
	if (AttributeNamespace.IsSet() == false)
	{
		return LinkedAttributePins;
	}

	// Check parameter map write nodes in the called graph to see if they write to any attributes in the current namespace.
	TArray<UNiagaraNodeParameterMapSet*> CalledGraphSetNodes;
	CalledGraph->GetNodesOfClass(CalledGraphSetNodes);
	TSet<FName> WritePinNames;
	for (UNiagaraNodeParameterMapSet* CalledGraphSetNode : CalledGraphSetNodes)
	{
		TArray<UEdGraphPin*> InputPins;
		CalledGraphSetNode->GetInputPins(InputPins);
		for (UEdGraphPin* InputPin : InputPins)
		{
			WritePinNames.Add(InputPin->PinName);
		}
	}

	bool bHasAttributeWriteInNamespace = false;
	for (FName WritePinName : WritePinNames)
	{
		FNiagaraParameterHandle WriteHandle(WritePinName);
		TArray<FName> WriteHandleParts = WriteHandle.GetHandleParts();
		if (WriteHandleParts.Num() >= 3 && 
			(WriteHandleParts[0] == AttributeNamespace.GetValue() || WriteHandleParts[0] == FNiagaraConstants::StackContextNamespace) &&
			WriteHandleParts[1] == FNiagaraConstants::ModuleNamespace)
		{
			bHasAttributeWriteInNamespace = true;
			break;
		}
	}

	// If the function call writes module attributes for the current namespace, check the current graph and any dependent graphs for reads from those
	// module attributes.
	if (bHasAttributeWriteInNamespace)
	{
		TArray<UNiagaraGraph*> GraphsToCheck = { InFunctionCallNode.GetNiagaraGraph() };
		GraphsToCheck.Append(DependentGraphs);
		for (UNiagaraGraph* GraphToCheck : GraphsToCheck)
		{
			TArray<UNiagaraNodeParameterMapGet*> MapGetNodes;
			GraphToCheck->GetNodesOfClass(MapGetNodes);
			for (UNiagaraNodeParameterMapGet* MapGetNode : MapGetNodes)
			{
				TArray<UEdGraphPin*> MapGetOutputPins;
				MapGetNode->GetOutputPins(MapGetOutputPins);
				for (UEdGraphPin* MapGetOutputPin : MapGetOutputPins)
				{
					FNiagaraParameterHandle MapGetHandle(MapGetOutputPin->PinName);
					TArray<FName> MapGetHandleParts = MapGetHandle.GetHandleParts();
					if (MapGetHandleParts.Num() >= 3 && 
						(MapGetHandleParts[0] == AttributeNamespace.GetValue() || MapGetHandleParts[0] == FNiagaraConstants::StackContextNamespace) &&
						MapGetHandleParts[1] == FunctionCallName)
					{
						LinkedAttributePins.Add(MapGetOutputPin);
					}
				}
			}
		}
	}
	return LinkedAttributePins;
}

// Takes a parameter map output pin which represents a module output or module attribute and finds the target function call which is linking that output or attribute.
UNiagaraNodeFunctionCall* GetTargetFunctionCallForStackLinkedModuleParameterPin(UEdGraphPin& StackLinkedModuleParameterPin)
{
	for (UEdGraphPin* LinkedOverridePin : StackLinkedModuleParameterPin.LinkedTo)
	{
		UNiagaraNodeParameterMapSet* LinkedOverrideNode = Cast<UNiagaraNodeParameterMapSet>(LinkedOverridePin->GetOwningNode());
		if (LinkedOverrideNode != nullptr)
		{
			FNiagaraParameterHandle LinkedOverrideHandle(LinkedOverridePin->PinName);
			UEdGraphPin* ParameterMapOutputPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*LinkedOverrideNode);
			if (ParameterMapOutputPin != nullptr)
			{
				for (UEdGraphPin* LinkedInputPin : ParameterMapOutputPin->LinkedTo)
				{
					UNiagaraNodeFunctionCall* LinkedFunctionCallNode = Cast<UNiagaraNodeFunctionCall>(LinkedInputPin->GetOwningNode());
					if (LinkedFunctionCallNode != nullptr && *LinkedFunctionCallNode->GetFunctionName() == LinkedOverrideHandle.GetHandleParts()[0])
					{
						return LinkedFunctionCallNode;
					}
				}
			}
		}
	}
	return nullptr;
}

void GetDependentGraphsForOuterGraph(UNiagaraNodeFunctionCall& InFunctionCallNode, TArray<UNiagaraGraph*>& OutDependentGraphs)
{
	UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(InFunctionCallNode);
	if (OutputNode != nullptr &&
		(OutputNode->GetUsage() == ENiagaraScriptUsage::SystemSpawnScript || OutputNode->GetUsage() == ENiagaraScriptUsage::SystemUpdateScript))
	{
		UNiagaraSystem* OuterSystem = InFunctionCallNode.GetTypedOuter<UNiagaraSystem>();
		if (OuterSystem != nullptr)
		{
			for (const FNiagaraEmitterHandle& EmitterHandle : OuterSystem->GetEmitterHandles())
			{
				if (EmitterHandle.GetEmitterData() != nullptr && EmitterHandle.GetEmitterData()->GraphSource != nullptr)
				{
					UNiagaraScriptSource* EmitterSource = Cast<UNiagaraScriptSource>(EmitterHandle.GetEmitterData()->GraphSource);
					if (EmitterSource != nullptr && EmitterSource->NodeGraph != nullptr)
					{
						OutDependentGraphs.Add(EmitterSource->NodeGraph);
					}
				}
			}
		}
	}
}

void FNiagaraStackGraphUtilities::PopulateFunctionCallNameBindings(UNiagaraNodeFunctionCall& InFunctionCallNode)
{
	UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(InFunctionCallNode);
	if (OutputNode == nullptr ||
		OutputNode->GetUsage() == ENiagaraScriptUsage::Module ||
		OutputNode->GetUsage() == ENiagaraScriptUsage::DynamicInput ||
		OutputNode->GetUsage() == ENiagaraScriptUsage::Function)
	{
		// Early out if an output node can't be found, or if the output node's usage is not a stack usage.
		return;
	}

	TSharedPtr<TMap<FName, FGuid>> VariableNameToVariableIdMap;
	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();

	// Populate input ids from the inputs on this function which are overridden.
	TArray<UEdGraphPin*> OverridePins;
	UNiagaraNodeParameterMapSet* OverrideNode = GetStackFunctionOverrideNode(InFunctionCallNode);
	if (OverrideNode != nullptr)
	{
		OverridePins = GetOverridePinsForFunction(*OverrideNode, InFunctionCallNode);
	}
	if (OverridePins.Num() > 0)
	{
		VariableNameToVariableIdMap = GetVariableNameToVariableIdMap(InFunctionCallNode);
		for (UEdGraphPin* OverridePin : OverridePins)
		{
			FNiagaraVariable OverrideVariable = NiagaraSchema->PinToNiagaraVariable(OverridePin);
			FNiagaraParameterHandle OverrideHandle(OverrideVariable.GetName());
			FNiagaraVariable InputVariable = FNiagaraUtilities::ResolveAliases(OverrideVariable, FNiagaraAliasContext().ChangeModuleNameToModule(InFunctionCallNode.GetFunctionName()));
			FGuid* IdByInputVariableName = VariableNameToVariableIdMap->Find(InputVariable.GetName());
			if (IdByInputVariableName != nullptr)
			{
				InFunctionCallNode.UpdateInputNameBinding(*IdByInputVariableName, OverrideHandle.GetParameterHandleString());
			}
		}
	}

	// Populate ids on target functions which use outputs or module attributes from this function.
	auto PopulateForStackLinkedModuleParameterPins = [&NiagaraSchema, &InFunctionCallNode, &VariableNameToVariableIdMap](const TArray<UEdGraphPin*>& StackLinkedModuleParameterPins)
	{
		if (VariableNameToVariableIdMap.IsValid() == false)
		{
			VariableNameToVariableIdMap = GetVariableNameToVariableIdMap(InFunctionCallNode);
		}
		for (UEdGraphPin* StackLinkedModuleParameterPin : StackLinkedModuleParameterPins)
		{
			UNiagaraNodeFunctionCall* TargetFunctionCallNode = GetTargetFunctionCallForStackLinkedModuleParameterPin(*StackLinkedModuleParameterPin);
			if (TargetFunctionCallNode != nullptr)
			{
				FNiagaraVariable LinkedModuleParameterVariable = NiagaraSchema->PinToNiagaraVariable(StackLinkedModuleParameterPin);
				FNiagaraParameterHandle LinkedModuleParameterHandle(LinkedModuleParameterVariable.GetName());
				FNiagaraVariable ModuleParameterVariable = FNiagaraUtilities::ResolveAliases(LinkedModuleParameterVariable, FNiagaraAliasContext().ChangeModuleNameToModule(InFunctionCallNode.GetFunctionName()));
				FGuid* IdByVariableName = VariableNameToVariableIdMap->Find(ModuleParameterVariable.GetName());
				if (IdByVariableName != nullptr)
				{
					TargetFunctionCallNode->UpdateInputNameBinding(*IdByVariableName, LinkedModuleParameterHandle.GetParameterHandleString());
				}
			}
		}
	};

	TArray<UEdGraphPin*> StackLinkedOutputPins = GetStackLinkedOutputPinsForFunction(InFunctionCallNode);
	if (StackLinkedOutputPins.Num() > 0)
	{
		PopulateForStackLinkedModuleParameterPins(StackLinkedOutputPins);
	}

	TArray<UNiagaraGraph*> DependentGraphs;
	GetDependentGraphsForOuterGraph(InFunctionCallNode, DependentGraphs);
	TArray<UEdGraphPin*> StackLinkedAttributePins = GetStackLinkedAttributePinsForFunction(InFunctionCallNode, DependentGraphs);
	if (StackLinkedAttributePins.Num() > 0)
	{
		PopulateForStackLinkedModuleParameterPins(StackLinkedAttributePins);
	}
}

void FNiagaraStackGraphUtilities::SynchronizeReferencingMapPinsWithFunctionCall(UNiagaraNodeFunctionCall& InFunctionCallNode)
{
	UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(InFunctionCallNode);
	if (OutputNode == nullptr ||
		OutputNode->GetUsage() == ENiagaraScriptUsage::Module ||
		OutputNode->GetUsage() == ENiagaraScriptUsage::DynamicInput ||
		OutputNode->GetUsage() == ENiagaraScriptUsage::Function)
	{
		// Early out if an output node can't be found, or if the output node's usage is not a stack usage.
		return;
	}

	TSharedPtr<TMap<FGuid, FNiagaraVariable>> AssetVariableIdToVariableMap;
	const UEdGraphSchema_Niagara* NiagaraSchema = GetDefault<UEdGraphSchema_Niagara>();
	TSet<UNiagaraGraph*> ModifiedGraphs;
	
	// Update the override parameter map set nodes which are used to set the function inputs.
	TArray<UEdGraphPin*> OverridePins;
	UNiagaraNodeParameterMapSet* OverrideNode = GetStackFunctionOverrideNode(InFunctionCallNode);
	if (OverrideNode != nullptr)
	{	
		OverridePins = GetOverridePinsForFunction(*OverrideNode, InFunctionCallNode);
	}
	if (OverridePins.Num() > 0)
	{
		// this contains the current up to date metadata of the asset graph this function call points to
		AssetVariableIdToVariableMap = GetVariableIdToVariableMap(InFunctionCallNode);
		bool bModifiedGraph = false;
		for (UEdGraphPin* OverridePin : OverridePins)
		{
			FNiagaraVariable CurrentPinOverrideVariable = NiagaraSchema->PinToNiagaraVariable(OverridePin);
			FNiagaraParameterHandle AliasedOverridePinHandle(CurrentPinOverrideVariable.GetName());
			// we look up the guids associated with the override pin whose name may be out of date
			TArray<FGuid> BoundGuids = InFunctionCallNode.GetBoundPinGuidsByName(AliasedOverridePinHandle.GetParameterHandleString());
			for (FGuid BoundGuid : BoundGuids)
			{
				// since the guid stayed the same, we can now look up the current parameter that might have a different name
				FNiagaraVariable* CurrentAssetParameter = AssetVariableIdToVariableMap->Find(BoundGuid);
				if (CurrentAssetParameter != nullptr)
				{
					FNiagaraParameterHandle CurrentAssetParameterHandle(CurrentAssetParameter->GetName());
					if (CurrentAssetParameterHandle.IsModuleHandle())
					{
						FNiagaraVariable UnaliasedCurrentPinOverrideVariable = FNiagaraUtilities::ResolveAliases(CurrentPinOverrideVariable, FNiagaraAliasContext().ChangeModuleNameToModule(AliasedOverridePinHandle.GetName().ToString()));
						FNiagaraParameterHandle UnaliasedCurrentPinOverrideVariableHandle = FNiagaraParameterHandle(UnaliasedCurrentPinOverrideVariable.GetName());

						// update the override pin if the parameter has changed
						if (UnaliasedCurrentPinOverrideVariableHandle.GetName() != CurrentAssetParameterHandle.GetName())
						{
							FNiagaraVariable UpdatedAliasedOverride = FNiagaraUtilities::ResolveAliases(*CurrentAssetParameter, FNiagaraAliasContext().ChangeModuleToModuleName(InFunctionCallNode.GetFunctionName()));
							FNiagaraParameterHandle UpdatedOverrideHandle = FNiagaraParameterHandle(UpdatedAliasedOverride.GetName());
							OverridePin->Modify();
							OverridePin->PinName = UpdatedOverrideHandle.GetParameterHandleString();
							InFunctionCallNode.Modify();
							InFunctionCallNode.UpdateInputNameBinding(BoundGuid, UpdatedOverrideHandle.GetParameterHandleString());
							bModifiedGraph = true;
						}
						if (CurrentPinOverrideVariable.GetType() != CurrentAssetParameter->GetType())
						{
							FEdGraphPinType NewPinType = NiagaraSchema->TypeDefinitionToPinType(CurrentAssetParameter->GetType());
							OverridePin->Modify();
							OverridePin->PinType = NewPinType;
							if (OverridePin->LinkedTo.Num() == 1 &&
								OverridePin->LinkedTo[0]->GetOwningNode() != nullptr &&
								OverridePin->LinkedTo[0]->GetOwningNode()->IsA<UNiagaraNodeCustomHlsl>())
							{
								// Custom hlsl nodes in the stack are expression dynamic inputs and their output pin type must match their connected input
								// so update that here.
								OverridePin->LinkedTo[0]->Modify();
								OverridePin->LinkedTo[0]->PinType = NewPinType;
							}
							bModifiedGraph = true;
						}
						break;
					}
				}
			}
		}
		if (bModifiedGraph)
		{
			ModifiedGraphs.Add(InFunctionCallNode.GetNiagaraGraph());
		}
	}

	// Update linked module outputs or module attributes referenced by downstream parameter map gets.
	auto SynchronizeForStackLinkedModuleParameterPins = [&NiagaraSchema, &InFunctionCallNode, &AssetVariableIdToVariableMap, &ModifiedGraphs](const TArray<UEdGraphPin*>& StackLinkedModuleParameterPins)
	{
		if (AssetVariableIdToVariableMap.IsValid() == false)
		{
			AssetVariableIdToVariableMap = GetVariableIdToVariableMap(InFunctionCallNode);
		}
		for (UEdGraphPin* StackLinkedModuleParameterPin : StackLinkedModuleParameterPins)
		{
			FNiagaraVariable LinkedModuleParameterVariable = NiagaraSchema->PinToNiagaraVariable(StackLinkedModuleParameterPin);
			FNiagaraParameterHandle LinkedModuleParameterHandle(LinkedModuleParameterVariable.GetName());
			UNiagaraNodeFunctionCall* TargetFunctionCallNode = GetTargetFunctionCallForStackLinkedModuleParameterPin(*StackLinkedModuleParameterPin);
			if (TargetFunctionCallNode != nullptr)
			{
				bool bModifiedGraph = false;
				TArray<FGuid> BoundGuids = TargetFunctionCallNode->GetBoundPinGuidsByName(LinkedModuleParameterHandle.GetParameterHandleString());
				for (const FGuid& BoundGuid : BoundGuids)
				{
					FNiagaraVariable* BoundVariable = AssetVariableIdToVariableMap->Find(BoundGuid);
					if (BoundVariable != nullptr)
					{
						FNiagaraVariable BoundVariableResolvedName = FNiagaraUtilities::ResolveAliases(*BoundVariable,FNiagaraAliasContext().ChangeModuleToModuleName(InFunctionCallNode.GetFunctionName()));
						FNiagaraParameterHandle BoundVariableHandle(BoundVariableResolvedName.GetName());
						if (LinkedModuleParameterHandle.GetNamespace() == BoundVariableHandle.GetNamespace())
						{
							if (LinkedModuleParameterHandle.GetName() != BoundVariableHandle.GetName())
							{
								StackLinkedModuleParameterPin->Modify();
								StackLinkedModuleParameterPin->PinName = BoundVariableResolvedName.GetName();
								TargetFunctionCallNode->Modify();
								TargetFunctionCallNode->UpdateInputNameBinding(BoundGuid, BoundVariableHandle.GetParameterHandleString());
								bModifiedGraph = true;
							}
							if (LinkedModuleParameterVariable.GetType() != BoundVariable->GetType())
							{
								StackLinkedModuleParameterPin->Modify();
								StackLinkedModuleParameterPin->PinType = NiagaraSchema->TypeDefinitionToPinType(BoundVariable->GetType());
								bModifiedGraph = true;
							}
							break;
						}
					}
				}
				if (bModifiedGraph)
				{
					ModifiedGraphs.Add(TargetFunctionCallNode->GetNiagaraGraph());
				}
			}
		}
	};

	TArray<UEdGraphPin*> StackLinkedOutputPins = GetStackLinkedOutputPinsForFunction(InFunctionCallNode);
	if (StackLinkedOutputPins.Num() > 0)
	{
		SynchronizeForStackLinkedModuleParameterPins(StackLinkedOutputPins);
	}

	TArray<UNiagaraGraph*> DependentGraphs;
	GetDependentGraphsForOuterGraph(InFunctionCallNode, DependentGraphs);
	TArray<UEdGraphPin*> StackLinkedAttributePins = GetStackLinkedAttributePinsForFunction(InFunctionCallNode, DependentGraphs);
	if (StackLinkedAttributePins.Num() > 0)
	{
		SynchronizeForStackLinkedModuleParameterPins(StackLinkedAttributePins);
	}

	for (UNiagaraGraph* ModifiedGraph : ModifiedGraphs)
	{
		ModifiedGraph->MarkGraphRequiresSynchronization(__FUNCTION__);
	}
}

FGuid FNiagaraStackGraphUtilities::GetScriptVariableIdForLinkedModuleParameter(const FNiagaraVariableBase& LinkedParameter, UNiagaraGraph& TargetGraph)
{
	FNiagaraParameterHandle LinkedHandle = FNiagaraParameterHandle(LinkedParameter.GetName());
	if (LinkedHandle.GetHandleParts().Num() >= 3 && (
			LinkedHandle.IsOutputHandle() ||
			LinkedHandle.IsStackContextHandle() ||
			LinkedHandle.IsSystemHandle() ||
			LinkedHandle.IsEmitterHandle() ||
			LinkedHandle.IsParticleAttributeHandle()))
	{
		// First find the source function call node.
		FString LinkedFunctionName = LinkedHandle.GetHandleParts()[1].ToString();
		TArray<UNiagaraNodeFunctionCall*> FunctionCallNodes;
		TargetGraph.GetNodesOfClass(FunctionCallNodes);
		UNiagaraNodeFunctionCall** SourceFunctionCallNodePtr = FunctionCallNodes.FindByPredicate([LinkedFunctionName](const UNiagaraNodeFunctionCall* FunctionCallNode)
			{ return FunctionCallNode->GetFunctionName() == LinkedFunctionName; });
		if (SourceFunctionCallNodePtr != nullptr)
		{
			UNiagaraGraph* CalledGraph = (*SourceFunctionCallNodePtr)->GetCalledGraph();
			if (CalledGraph != nullptr)
			{
				FNiagaraVariable LinkedModuleVariable = FNiagaraUtilities::ResolveAliases(LinkedParameter, FNiagaraAliasContext()
					.ChangeModuleNameToModule(LinkedFunctionName));
				for (const TPair<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>& VariableMetadataPair : CalledGraph->GetAllMetaData())
				{
					if (VariableMetadataPair.Key == LinkedModuleVariable)
					{
						return VariableMetadataPair.Value->Metadata.GetVariableGuid();
					}
				}
			}
		}
	}
	return FGuid();
}

bool FNiagaraStackGraphUtilities::DependencyUtilities::DoesStackModuleProvideDependency(const FNiagaraStackModuleData& StackModuleData, const FNiagaraModuleDependency& SourceModuleRequiredDependency, const UNiagaraNodeOutput& SourceOutputNode)
{
	UNiagaraNodeFunctionCall* ModuleNode = StackModuleData.WeakModuleNode.Get();
	if (ModuleNode != nullptr && ModuleNode->FunctionScript != nullptr)
	{
		FVersionedNiagaraScriptData* ScriptData = ModuleNode->FunctionScript->GetScriptData(ModuleNode->SelectedScriptVersion);
		if (ScriptData && ScriptData->ProvidedDependencies.Contains(SourceModuleRequiredDependency.Id))
		{
			if (SourceModuleRequiredDependency.ScriptConstraint == ENiagaraModuleDependencyScriptConstraint::AllScripts)
			{
				return true;
			}

			if (SourceModuleRequiredDependency.ScriptConstraint == ENiagaraModuleDependencyScriptConstraint::SameScript)
			{
				UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*ModuleNode);
				return OutputNode != nullptr && UNiagaraScript::IsEquivalentUsage(OutputNode->GetUsage(), SourceOutputNode.GetUsage()) && OutputNode->GetUsageId() == SourceOutputNode.GetUsageId();
			}
		}
	}
	return false;
}

void FNiagaraStackGraphUtilities::DependencyUtilities::GetModuleScriptAssetsByDependencyProvided(FName DependencyName, TOptional<ENiagaraScriptUsage> RequiredUsage, TArray<FAssetData>& OutAssets)
{
	FNiagaraEditorUtilities::FGetFilteredScriptAssetsOptions ScriptFilterOptions;
	ScriptFilterOptions.bIncludeDeprecatedScripts = false;
	ScriptFilterOptions.bIncludeNonLibraryScripts = true;
	ScriptFilterOptions.ScriptUsageToInclude = ENiagaraScriptUsage::Module;
	ScriptFilterOptions.TargetUsageToMatch = RequiredUsage;
	TArray<FAssetData> ModuleAssets;
	FNiagaraEditorUtilities::GetFilteredScriptAssets(ScriptFilterOptions, ModuleAssets);

	for (const FAssetData& ModuleAsset : ModuleAssets)
	{
		FString ProvidedDependenciesString;
		if (ModuleAsset.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, ProvidedDependencies), ProvidedDependenciesString) && ProvidedDependenciesString.IsEmpty() == false)
		{
			TArray<FString> DependencyStrings;
			ProvidedDependenciesString.ParseIntoArray(DependencyStrings, TEXT(","));
			for (FString DependencyString : DependencyStrings)
			{
				if (FName(*DependencyString) == DependencyName)
				{
					OutAssets.Add(ModuleAsset);
					break;
				}
			}
		}
	}
}

void FNiagaraStackGraphUtilities::FixDynamicInputNodeOutputPinsFromExternalChanges(UNiagaraNodeFunctionCall& InFunctionCallNode)
{
	if (InFunctionCallNode.FunctionScript == nullptr || InFunctionCallNode.FunctionScript->GetUsage() != ENiagaraScriptUsage::DynamicInput)
	{
		return;
	}

	FPinCollectorArray OutputPins;
	InFunctionCallNode.GetOutputPins(OutputPins);
	if (OutputPins.Num() > 1)
	{
		// Try to find an orphaned pin which is still linked, and any non-orphaned unlinked pins.
		UEdGraphPin* LinkedOrphanedPin = nullptr;
		TArray<UEdGraphPin*> UnlinkedValidPins;
		for (UEdGraphPin* OutputPin : OutputPins)
		{
			if (OutputPin->bOrphanedPin)
			{
				if (OutputPin->LinkedTo.Num() == 1 && ensureMsgf(LinkedOrphanedPin == nullptr, TEXT("Dynamic inputs should only ever have one linked pin.")))
				{
					LinkedOrphanedPin = OutputPin;
				}
			}
			else
			{
				if (OutputPin->LinkedTo.Num() == 0)
				{
					UnlinkedValidPins.Add(OutputPin);
				}
			}
		}

		UEdGraphPin* LinkedValidPin = nullptr;
		if (LinkedOrphanedPin != nullptr && UnlinkedValidPins.Num() > 0)
		{
			// Try to match the first available valid pin to the orphaned pin by type and then fix up the pin links.
			for (UEdGraphPin* UnlinkedValidPin : UnlinkedValidPins)
			{
				if (UnlinkedValidPin->PinType == LinkedOrphanedPin->PinType)
				{
					// A valid unlinked pin was found, so move the link from the orphaned pin to the valid pin.
					UnlinkedValidPin->MakeLinkTo(LinkedOrphanedPin->LinkedTo[0]);
					LinkedOrphanedPin->BreakAllPinLinks(false);
					LinkedValidPin = UnlinkedValidPin;
					break;
				}
			}
		}

		if (LinkedValidPin != nullptr)
		{
			// If a valid pin was linked, remove all of the other output pins since they're not valid for a dynamic input node.
			for (UEdGraphPin* OutputPin : OutputPins)
			{
				if (OutputPin != LinkedValidPin)
				{
					InFunctionCallNode.RemovePin(OutputPin);
				}
			}
		}
	}
}

void FNiagaraStackGraphUtilities::GetEmitterHandleAndCompiledScriptsForStackNode(const UNiagaraSystem& OwningSystem, UNiagaraNode& StackNode, const FNiagaraEmitterHandle*& OutEmitterHandle, TArray<const UNiagaraScript*>& OutCompiledScripts)
{
	UNiagaraNodeOutput* OutputNode = GetEmitterOutputNodeForStackNode(StackNode);
	if (OutputNode != nullptr)
	{
		if (OutputNode->GetUsage() == ENiagaraScriptUsage::SystemSpawnScript)
		{
			OutCompiledScripts.Add(OwningSystem.GetSystemSpawnScript());
			OutEmitterHandle = nullptr;
		}
		else if (OutputNode->GetUsage() == ENiagaraScriptUsage::SystemUpdateScript)
		{
			OutCompiledScripts.Add(OwningSystem.GetSystemUpdateScript());
			OutEmitterHandle = nullptr;
		}
		else
		{
			OutEmitterHandle = OwningSystem.GetEmitterHandles().FindByPredicate(
				[&StackNode](const FNiagaraEmitterHandle& EmitterHandle)
				{
					FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
					UNiagaraScriptSource* ScriptSource = EmitterData && EmitterData->GraphSource ? CastChecked<UNiagaraScriptSource>(EmitterData->GraphSource) : nullptr;
					return ScriptSource ? ScriptSource->NodeGraph == StackNode.GetNiagaraGraph() : false;
				}
			);

			if (OutEmitterHandle != nullptr)
			{
				switch (OutputNode->GetUsage())
				{
					case ENiagaraScriptUsage::EmitterSpawnScript:
						OutCompiledScripts.Add(OwningSystem.GetSystemSpawnScript());
						break;
					case ENiagaraScriptUsage::EmitterUpdateScript:
						OutCompiledScripts.Add(OwningSystem.GetSystemUpdateScript());
						break;
					case ENiagaraScriptUsage::ParticleSpawnScript:
					case ENiagaraScriptUsage::ParticleUpdateScript:
					case ENiagaraScriptUsage::ParticleEventScript:
					case ENiagaraScriptUsage::ParticleSimulationStageScript:
					{
						if ( FVersionedNiagaraEmitterData* EmitterData = OutEmitterHandle->GetEmitterData() )
						{
							OutCompiledScripts.Add(EmitterData->GetScript(OutputNode->GetUsage(), OutputNode->GetUsageId()));
							if (EmitterData->SimTarget == ENiagaraSimTarget::GPUComputeSim)
							{
								OutCompiledScripts.Add(EmitterData->GetScript(ENiagaraScriptUsage::ParticleGPUComputeScript, FGuid()));
							}
						}
						break;
					}
				}
			}
		}
	}
}

int32 FNiagaraStackGraphUtilities::DependencyUtilities::FindBestIndexForModuleInStack(UNiagaraNodeFunctionCall& ModuleNode, const UNiagaraNodeOutput& TargetOutputNode)
{
	// Check if the new module node has any dependencies to begin with. If not, early exit.
	FVersionedNiagaraScriptData* ScriptData = ModuleNode.GetScriptData();
	if (ScriptData == nullptr || (ScriptData->RequiredDependencies.Num() == 0 && ScriptData->ProvidedDependencies.Num() == 0))
	{
		return INDEX_NONE;
	}

	// Get and validate the owning graph.
	UNiagaraGraph* Graph = ModuleNode.GetNiagaraGraph();
	if (Graph == nullptr || Graph->Nodes.Contains(&ModuleNode) == false)
	{
		return INDEX_NONE;
	}

	// Get the Emitter and System the emitter script script graph is outered to.
	UNiagaraSystem* System = ModuleNode.GetTypedOuter<UNiagaraSystem>();
	FVersionedNiagaraEmitter OuterEmitter = Graph->GetOwningEmitter();
	if (System == nullptr && OuterEmitter.Emitter == nullptr)
	{
		return INDEX_NONE;
	}

	// Get the stack module data for the stack to find dependencies.
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(System);
	if (!ensureMsgf(SystemViewModel.IsValid(), TEXT("Failed to get systemviewmodel for valid system when getting best index in stack for module!")))
	{
		return INDEX_NONE;
	}

	FGuid EmitterHandleId;
	if (OuterEmitter.Emitter != nullptr)
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = SystemViewModel->GetEmitterHandleViewModelForEmitter(OuterEmitter);
		if (!ensureMsgf(EmitterHandleViewModel.IsValid(), TEXT("Failed to get emitterhandleviewmodel for valid emitter when getting best index in stack for module!")))
		{
			return INDEX_NONE;
		}
		else
		{
			EmitterHandleId = EmitterHandleViewModel->GetId();
		}
	}
	
	// Calling this with an invalid handle will retrieve the system stack module data.
	const TArray<FNiagaraStackModuleData>& StackModuleData = SystemViewModel->GetStackModuleDataByEmitterHandleId(EmitterHandleId);


	// Find the greatest and least indices for the stack first.
	int32 LeastIndex = INT_MAX;
	int32 GreatestIndex = INDEX_NONE;
	for (const FNiagaraStackModuleData& CurrentStackModuleData : StackModuleData)
	{
		// Stack module data indices are only valid for a specific script which is identified by the usage and usage id.
		if (CurrentStackModuleData.Usage != TargetOutputNode.GetUsage() || CurrentStackModuleData.UsageId != TargetOutputNode.GetUsageId())
		{
			continue;
		}
		int32 Index = CurrentStackModuleData.Index;
		LeastIndex = LeastIndex > Index ? Index : LeastIndex;
		GreatestIndex = GreatestIndex < Index ? Index : GreatestIndex;
	}

	// Find the greatest and least indices to satisfy the pre and post dependencies of the new module script being added, respectively.
	int32 GreatestRequiredDependencyLowerBoundIdx = LeastIndex;
	int32 LeastRequiredDependencyUpperBoundIdx = GreatestIndex;
	bool bRequiredDependencyLowerBoundFound = false;
	bool bRequiredDependencyUpperBoundFound = false;

	const TArray<FNiagaraModuleDependency>& NewModuleScriptRequiredDependencies = ScriptData->RequiredDependencies;

	TMap<ENiagaraScriptUsage, UNiagaraNodeOutput*> ScriptUsageToOutputNode;
	auto GetOutputNodeForStackModuleData = [&ScriptUsageToOutputNode](const FNiagaraStackModuleData& StackModuleData)->UNiagaraNodeOutput* {
		if (UNiagaraNodeOutput** OutputNodePtr = ScriptUsageToOutputNode.Find(StackModuleData.Usage))
		{
			return *OutputNodePtr;
		}
		UNiagaraNodeFunctionCall* ModuleNode = StackModuleData.WeakModuleNode.Get();
		check(ModuleNode != nullptr);
		UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*ModuleNode);
		ScriptUsageToOutputNode.Add(StackModuleData.Usage, OutputNode);
		return OutputNode;
	};

	for (const FNiagaraStackModuleData& CurrentStackModuleData : StackModuleData)
	{
		// Stack module data indices are only valid for a specific script which is identified by the usage and usage id.
		if (CurrentStackModuleData.Usage != TargetOutputNode.GetUsage() || CurrentStackModuleData.UsageId != TargetOutputNode.GetUsageId())
		{
			continue;
		}
		for (const FNiagaraModuleDependency& RequiredDependency : NewModuleScriptRequiredDependencies)
		{
			if (DoesStackModuleProvideDependency(CurrentStackModuleData, RequiredDependency, *GetOutputNodeForStackModuleData(CurrentStackModuleData)))
			{
				if (RequiredDependency.Type == ENiagaraModuleDependencyType::PreDependency)
				{
					int32 SuggestedIndex = CurrentStackModuleData.Index + 1;
					GreatestRequiredDependencyLowerBoundIdx = GreatestRequiredDependencyLowerBoundIdx < SuggestedIndex ? SuggestedIndex : GreatestRequiredDependencyLowerBoundIdx;
					bRequiredDependencyLowerBoundFound = true;
				}
				else if (RequiredDependency.Type == ENiagaraModuleDependencyType::PostDependency)
				{
					int32 SuggestedIndex = CurrentStackModuleData.Index;
					LeastRequiredDependencyUpperBoundIdx = LeastRequiredDependencyUpperBoundIdx > SuggestedIndex ? SuggestedIndex : LeastRequiredDependencyUpperBoundIdx;
					bRequiredDependencyUpperBoundFound = true;
				}
			}
		}
	}

	// Check that there is valid index which satisfies all dependencies for the new module script.
	int32 TargetIndex = INDEX_NONE;
	if (bRequiredDependencyLowerBoundFound)
	{
		if (bRequiredDependencyUpperBoundFound && LeastRequiredDependencyUpperBoundIdx < GreatestRequiredDependencyLowerBoundIdx)
		{
			// It is impossible to satisfy both target indices as the upper bound is less than the lower bound: do nothing.
			return INDEX_NONE;
		}
		TargetIndex = GreatestRequiredDependencyLowerBoundIdx;
	}
	else if (bRequiredDependencyUpperBoundFound)
	{
		TargetIndex = LeastRequiredDependencyUpperBoundIdx;
	}

	// Do another pass to find a target index that satisfies dependencies for other modules in the stack by providing as many dependencies as possible.
	int32 GreatestProvidedDependencyLowerBoundIdx = GreatestIndex;
	int32 LeastProvidedDependencyUpperBoundIdx = LeastIndex;
	bool bProvidedDependencyLowerBoundFound = false;
	bool bProvidedDependencyUpperBoundFound = false;

	const TArray<FName>& NewModuleScriptProvidedDependencies = ScriptData->ProvidedDependencies;
	auto GetStackModuleDataRequiredDependenciesBeingProvided =
		[&NewModuleScriptProvidedDependencies](const FNiagaraStackModuleData& StackModuleData)->const TArray<FNiagaraModuleDependency> /*OutDependencies*/
		{
			TArray<FNiagaraModuleDependency> OutDependencies;
			if (UNiagaraNodeFunctionCall* ModuleNode = StackModuleData.WeakModuleNode.Get())
			{
				for (FNiagaraModuleDependency& RequiredDependency : ModuleNode->GetScriptData()->RequiredDependencies)
				{
					if (NewModuleScriptProvidedDependencies.Contains(RequiredDependency.Id))
					{
						OutDependencies.Add(RequiredDependency);
					}
				}
			}
			return OutDependencies;
		};

	for (const FNiagaraStackModuleData& CurrentStackModuleData : StackModuleData)
	{
		// Stack module data indices are only valid for a specific script which is identified by the usage and usage id.
		if (CurrentStackModuleData.Usage != TargetOutputNode.GetUsage() || CurrentStackModuleData.UsageId != TargetOutputNode.GetUsageId())
		{
			continue;
		}
		for (const FNiagaraModuleDependency& ProvidedDependency : GetStackModuleDataRequiredDependenciesBeingProvided(CurrentStackModuleData))
		{
			if (ProvidedDependency.Type == ENiagaraModuleDependencyType::PostDependency)
			{
				int32 SuggestedIndex = CurrentStackModuleData.Index + 1;
				GreatestProvidedDependencyLowerBoundIdx = GreatestProvidedDependencyLowerBoundIdx < SuggestedIndex ? SuggestedIndex : GreatestProvidedDependencyLowerBoundIdx;
				bProvidedDependencyLowerBoundFound = true;
			}
			else if (ProvidedDependency.Type == ENiagaraModuleDependencyType::PreDependency)
			{
				int32 SuggestedIndex = CurrentStackModuleData.Index;
				LeastProvidedDependencyUpperBoundIdx = LeastProvidedDependencyUpperBoundIdx > SuggestedIndex ? SuggestedIndex : LeastProvidedDependencyUpperBoundIdx;
				bProvidedDependencyUpperBoundFound = true;
			}
		}
	}

	// Alias bounds without least and greatest prefixes for legibility.
	int32& RequiredDependencyLowerBoundIdx = GreatestRequiredDependencyLowerBoundIdx;
	int32& RequiredDependencyUpperBoundIdx = LeastRequiredDependencyUpperBoundIdx;
	int32& ProvidedDependencyLowerBoundIdx = GreatestProvidedDependencyLowerBoundIdx;
	int32& ProvidedDependencyUpperBoundIdx = LeastProvidedDependencyUpperBoundIdx;

	// Provided dependency lower and upper bound must be inside required dependency lower and upper bound respectively.
	ProvidedDependencyLowerBoundIdx = (bRequiredDependencyLowerBoundFound && ProvidedDependencyLowerBoundIdx < RequiredDependencyLowerBoundIdx) ? RequiredDependencyLowerBoundIdx : ProvidedDependencyLowerBoundIdx;
	ProvidedDependencyUpperBoundIdx = (bRequiredDependencyUpperBoundFound && ProvidedDependencyUpperBoundIdx > RequiredDependencyUpperBoundIdx) ? RequiredDependencyUpperBoundIdx : ProvidedDependencyUpperBoundIdx;

	// Do another validity test for provided dependency lower and upper bound indices.
	if (bProvidedDependencyLowerBoundFound)
	{
		if (bProvidedDependencyUpperBoundFound && ProvidedDependencyUpperBoundIdx < ProvidedDependencyLowerBoundIdx)
		{
			// It is impossible to satisfy both target indices as the upper bound is less than the lower bound: do nothing.
			return TargetIndex;
		}
		TargetIndex = ProvidedDependencyLowerBoundIdx;
	}
	else if (bProvidedDependencyUpperBoundFound)
	{
		TargetIndex = ProvidedDependencyUpperBoundIdx;
	}

	return TargetIndex;
}

#undef LOCTEXT_NAMESPACE
