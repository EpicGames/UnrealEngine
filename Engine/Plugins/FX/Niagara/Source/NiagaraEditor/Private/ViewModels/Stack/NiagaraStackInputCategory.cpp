// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackInputCategory.h"

#include "IDetailTreeNode.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraClipboard.h"
#include "NiagaraConstants.h"
#include "NiagaraSimulationStageBase.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"
#include "ViewModels/Stack/NiagaraStackEmitterSettingsGroup.h"
#include "ViewModels/Stack/NiagaraStackEventScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackPropertyRow.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackRenderersOwner.h"
#include "ViewModels/Stack/NiagaraStackSimulationStageGroup.h"
#include "Algo/Accumulate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackInputCategory)

using namespace FNiagaraStackGraphUtilities;

#define LOCTEXT_NAMESPACE "NiagaraStack"

void UNiagaraStackCategory::Initialize(FRequiredEntryData InRequiredEntryData, FString InOwningStackItemEditorDataKey, FString InStackEditorDataKey)
{
	Super::Initialize(InRequiredEntryData, InOwningStackItemEditorDataKey, InStackEditorDataKey);
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackCategory::FilterForVisibleCondition));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackCategory::FilterOnlyModified));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackCategory::FilterForIsInlineEditConditionToggle));
	bShouldShowInStack = true;
	CategorySpacer = nullptr;
}

FText UNiagaraStackCategory::GetDisplayName() const
{
	return FText::GetEmpty();
}

bool UNiagaraStackCategory::GetShouldShowInStack() const
{
	// Categories may be empty if their children have all been hidden due to visible filters or advanced display.
	// in the case where all children have been hidden, don't show the category in the stack.
	TArray<UNiagaraStackEntry*> CurrentFilteredChildren;
	GetFilteredChildren(CurrentFilteredChildren);
	int32 EmptyCount = CategorySpacer == nullptr ? 0 : 1;
	return bShouldShowInStack && CurrentFilteredChildren.Num() > EmptyCount;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackCategory::GetStackRowStyle() const
{
	return IsTopLevelCategory() ? EStackRowStyle::ItemCategory : EStackRowStyle::ItemSubCategory;
}

void UNiagaraStackCategory::GetSearchItems(TArray<FStackSearchItem>& SearchItems) const
{
	// Don't return search results if we're not being shown in the stack, otherwise we'll generate search results which can't be navigated to.
	if (GetShouldShowInStack())
	{
		Super::GetSearchItems(SearchItems);
	}
}

int32 UNiagaraStackCategory::GetChildIndentLevel() const
{
	// We want to keep inputs under a top level category at the same indent level as the category.
	return IsTopLevelCategory() ? GetIndentLevel() : Super::GetChildIndentLevel();
}

void UNiagaraStackCategory::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren,	TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (IsTopLevelCategory())
	{
		if (CategorySpacer == nullptr)
		{
			CategorySpacer = NewObject<UNiagaraStackSpacer>(this);
			TAttribute<bool> ShouldShowSpacerInStack;
			ShouldShowSpacerInStack.BindUObject(this, &UNiagaraStackCategory::GetShouldShowInStack);
			CategorySpacer->Initialize(CreateDefaultChildRequiredData(), 6, ShouldShowSpacerInStack, GetStackEditorDataKey());
		}
		NewChildren.Add(CategorySpacer);
	}
}

bool UNiagaraStackCategory::FilterForVisibleCondition(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackFunctionInput* StackFunctionInputChild = Cast<UNiagaraStackFunctionInput>(&Child);
	return StackFunctionInputChild == nullptr || StackFunctionInputChild->GetShouldPassFilterForVisibleCondition();
}

bool UNiagaraStackCategory::FilterOnlyModified(const UNiagaraStackEntry& Child) const
{
	if(GetStackEditorData().GetShowOnlyModified() == false)
	{
		return true;
	}
	
	const UNiagaraStackFunctionInput* FunctionInput = Cast<UNiagaraStackFunctionInput>(&Child);
	if (FunctionInput == nullptr || FunctionInput->CanReset() || FunctionInput->HasAnyResettableChildrenInputs())
	{
		return true;
	}

	return false;
}

bool UNiagaraStackCategory::FilterForIsInlineEditConditionToggle(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackFunctionInput* StackFunctionInputChild = Cast<UNiagaraStackFunctionInput>(&Child);
	return StackFunctionInputChild == nullptr || StackFunctionInputChild->GetIsInlineEditConditionToggle() == false;
}

void UNiagaraStackScriptHierarchyCategory::Initialize(FRequiredEntryData InRequiredEntryData, const UHierarchyCategory& InHierarchyCategory, FString InOwningStackItemEditorDataKey, FString InStackEditorDataKey)
{
	Super::Initialize(InRequiredEntryData, InOwningStackItemEditorDataKey, InStackEditorDataKey);
	HierarchyCategory = &InHierarchyCategory;
}

void UNiagaraStackScriptHierarchyCategory::Copy(UNiagaraClipboardContent* ClipboardContent) const
{
	ClipboardContent->FunctionInputs.Append(ToClipboardFunctionInputs(ClipboardContent));
}

void UNiagaraStackScriptHierarchyCategory::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	PasteFromClipboard(ClipboardContent);
}

bool UNiagaraStackScriptHierarchyCategory::TestCanCopyWithMessage(FText& OutMessage) const
{
	TArray<UNiagaraStackFunctionInput*> StackFunctionInputs;
	GetFilteredChildrenOfType(StackFunctionInputs, true);

	if(StackFunctionInputs.Num() == 0)
	{
		OutMessage = LOCTEXT("CantCopyCategory_NoInputs", "No inputs available for copying.");
		return false;
	}

	OutMessage = LOCTEXT("TestCanCopyCategory_Success", "Copy all inputs of this category.");
	return true;
}

bool UNiagaraStackScriptHierarchyCategory::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	if(ClipboardContent->FunctionInputs.Num() == 0)
	{
		OutMessage = LOCTEXT("TestCanPasteCategory_NoCopiedInputsFound", "No inputs were copied.");
		return false;
	}
	
	TArray<UNiagaraStackFunctionInput*> AllContainedInputs;
	GetFilteredChildrenOfType(AllContainedInputs, true);

	int32 MatchingInputsNum = Algo::Accumulate(AllContainedInputs, 0, [ClipboardContent](int32 Value, UNiagaraStackFunctionInput* Input)
	{
		Value += ClipboardContent->FunctionInputs.ContainsByPredicate([Input](const UNiagaraClipboardFunctionInput* ClipboardFunctionInputCandidate)
		{
			return ClipboardFunctionInputCandidate->InputName == Input->GetInputParameterHandle().GetName() && ClipboardFunctionInputCandidate->InputType == Input->GetInputType();
		}) ? 1 : 0;

		return Value;
	});
	
	if(MatchingInputsNum == 0)
	{
		OutMessage = LOCTEXT("TestCanPasteCategory_NoMatchingInputsFound", "No matching copied inputs were found for pasting into this category.");
		return false;
	}

	OutMessage = LOCTEXT("TestCanPasteCategory_Success", "Paste matching inputs into this category.");
	return true;
}

void UNiagaraStackScriptHierarchyCategory::PasteFromClipboard(const UNiagaraClipboardContent* ClipboardContent)
{
	TArray<UNiagaraStackFunctionInput*> AllContainedInputs;
	GetUnfilteredChildrenOfType(AllContainedInputs, true);
	
	for(const UNiagaraClipboardFunctionInput* ClipboardInput : ClipboardContent->FunctionInputs)
	{
		if(ClipboardInput == nullptr)
		{
			continue;
		}

		// Since we pasting into a category, we require name & type match to ensure we aren't writing into unintended inputs
		for(UNiagaraStackFunctionInput* StackInput : AllContainedInputs)
		{
			if(StackInput->GetInputParameterHandle().GetName() == ClipboardInput->InputName && StackInput->GetInputType() == ClipboardInput->InputType)
			{
				StackInput->PasteFunctionInput(ClipboardInput);
			}
		}
	}
}

TArray<const UNiagaraClipboardFunctionInput*> UNiagaraStackScriptHierarchyCategory::ToClipboardFunctionInputs(UObject* InOuter) const
{
	TArray<const UNiagaraClipboardFunctionInput*> Result;
	
	TArray<UNiagaraStackScriptHierarchyCategory*> SubCategories;
	GetFilteredChildrenOfType(SubCategories, false);

	for(UNiagaraStackScriptHierarchyCategory* SubCategory : SubCategories)
	{
		Result.Append(SubCategory->ToClipboardFunctionInputs(InOuter));
	}

	TArray<UNiagaraStackFunctionInput*> ContainedInputs;
	GetFilteredChildrenOfType(ContainedInputs, false);

	for(UNiagaraStackFunctionInput* ContainedInput : ContainedInputs)
	{
		Result.Add(ContainedInput->ToClipboardFunctionInput(InOuter));
	}

	return Result;
}

FText UNiagaraStackScriptHierarchyCategory::GetDisplayName() const
{
	return HierarchyCategory->GetCategoryAsText();
}

FText UNiagaraStackScriptHierarchyCategory::GetTooltipText() const
{
	return HierarchyCategory->GetTooltip();
}

void UNiagaraStackScriptHierarchyCategory::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	for(const UHierarchyElement* ChildHierarchyItem : HierarchyCategory->GetChildren())
	{
		if(const UNiagaraHierarchyScriptParameter* HierarchyParameter = Cast<UNiagaraHierarchyScriptParameter>(ChildHierarchyItem))
		{
			TOptional<FNiagaraVariable> InputVariableCandidate = HierarchyParameter->GetVariable();

			if(InputVariableCandidate.IsSet() == false)
			{
				continue;
			}

			FNiagaraVariable InputVariable = InputVariableCandidate.GetValue();
			
			if(ScriptInstanceData.UsedInputs.Contains(InputVariable) == false)
			{
				continue;
			}
			
			UNiagaraStackFunctionInput* InputChild = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
			{ 
				return CurrentInput->GetInputParameterHandle() == FNiagaraParameterHandle(InputVariable.GetName()) && CurrentInput->GetInputType() == InputVariable.GetType() && &CurrentInput->GetInputFunctionCallNode() == OwningFunctionCallNode.Get();
			});

			if (InputChild == nullptr)
			{
				EStackParameterBehavior Behavior = HierarchyParameter->GetScriptVariable()->GetIsStaticSwitch() ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic;
				InputChild = NewObject<UNiagaraStackFunctionInput>(this);
				InputChild->Initialize(CreateDefaultChildRequiredData(), *OwningModuleNode.Get(), *OwningFunctionCallNode.Get(),
					InputVariable.GetName(), InputVariable.GetType(), Behavior, GetOwnerStackItemEditorDataKey());
			}

			InputChild->SetScriptInstanceData(ScriptInstanceData);
			FGuid VariableGuid = HierarchyParameter->GetScriptVariable()->Metadata.GetVariableGuid();
			InputChild->SetIsHidden(ScriptInstanceData.PerInputInstanceData[VariableGuid].bIsHidden);
			
			NewChildren.Add(InputChild);
		}
		if(const UHierarchyCategory* ChildHierarchyCategory = Cast<UHierarchyCategory>(ChildHierarchyItem))
		{
			// Try to find an already existing category to reuse
			UNiagaraStackScriptHierarchyCategory* ChildCategory = FindCurrentChildOfTypeByPredicate<UNiagaraStackScriptHierarchyCategory>(CurrentChildren,
				[&](UNiagaraStackScriptHierarchyCategory* CurrentCategory) { return CurrentCategory->GetHierarchyCategory() == ChildHierarchyItem; });

			if (ChildCategory == nullptr)
			{
				// If we don't have a current child for this category make a new one.
				ChildCategory = NewObject<UNiagaraStackScriptHierarchyCategory>(this);
				ChildCategory->SetOwningModuleNode(*OwningModuleNode.Get());
				ChildCategory->SetOwningFunctionCallNode(*OwningFunctionCallNode.Get());
				FString InputCategoryStackEditorDataKey = FString::Printf(TEXT("%s-InputCategory-%s"), *OwningFunctionCallNode->NodeGuid.ToString(EGuidFormats::DigitsWithHyphens), *ChildHierarchyCategory->ToString());
				ChildCategory->Initialize(CreateDefaultChildRequiredData(), *ChildHierarchyCategory, GetOwnerStackItemEditorDataKey(), InputCategoryStackEditorDataKey);
			}

			ChildCategory->SetScriptInstanceData(ScriptInstanceData);
			NewChildren.Add(ChildCategory);
		}		
	}
	
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

void UNiagaraStackSummaryCategory::Initialize(FRequiredEntryData InRequiredEntryData, TSharedPtr<FHierarchyCategoryViewModel> InCategoryViewModel, FString InOwnerStackItemEditorDataKey)
{
	CategoryViewModelWeakPtr = InCategoryViewModel;
	
	FString EditorDataKey = FString::Printf(TEXT("%s-%s"), *InOwnerStackItemEditorDataKey, *InCategoryViewModel->GetCategoryName().ToString());
	Super::Initialize(InRequiredEntryData, InOwnerStackItemEditorDataKey, EditorDataKey);
}

FText UNiagaraStackSummaryCategory::GetDisplayName() const
{
	TSharedPtr<FHierarchyCategoryViewModel> CategoryViewModel = CategoryViewModelWeakPtr.Pin();
	if (ensure(CategoryViewModel.IsValid()))
	{
		return CategoryViewModel->GetCategoryName();
	}
	return FText();
}

void UNiagaraStackSummaryCategory::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	FInputDataCollection State;
	GatherInputRelationsForStack(State, GetEmitterViewModel().ToSharedRef());

	TArray<TSharedPtr<FHierarchyElementViewModel>> ChildrenViewModels;
	CategoryViewModelWeakPtr.Pin()->GetChildrenViewModelsForType<UHierarchyElement, FHierarchyElementViewModel>(ChildrenViewModels, false);

	TArray<TSharedPtr<FHierarchyElementViewModel>> AllChildrenViewModels;
	CategoryViewModelWeakPtr.Pin()->GetChildrenViewModelsForType<UHierarchyElement, FHierarchyElementViewModel>(AllChildrenViewModels, true);
	
	// first we gather all function call nodes so we can create cache for them instead of looking it up for each input individually
	TSet<UNiagaraNodeFunctionCall*> UsedFunctionCallNodes;
	for(TSharedPtr<FHierarchyElementViewModel> HierarchyViewModel : AllChildrenViewModels)
	{
		UHierarchyElement* Data = HierarchyViewModel->GetDataMutable();
		TOptional<FGuid> FunctionCallGuid;
		if(UNiagaraHierarchyModuleInput* ModuleInput = Cast<UNiagaraHierarchyModuleInput>(Data))
		{
			FunctionCallGuid = ModuleInput->GetPersistentIdentity().Guids[0];
		}
		else if(UNiagaraHierarchyAssignmentInput* AssignmentInput = Cast<UNiagaraHierarchyAssignmentInput>(Data))
		{
			FunctionCallGuid = AssignmentInput->GetPersistentIdentity().Guids[0];
		}
		else if(UNiagaraHierarchyModule* Module = Cast<UNiagaraHierarchyModule>(Data))
		{
			FunctionCallGuid = Module->GetPersistentIdentity().Guids[0];
		}

		if(FunctionCallGuid.IsSet() && State.NodeGuidToModuleNodeMap.Contains(FunctionCallGuid.GetValue()))
		{
			UsedFunctionCallNodes.Add(State.NodeGuidToModuleNodeMap[FunctionCallGuid.GetValue()]);
		}
	}
	
	TMap<FGuid, TSet<FNiagaraVariable>> FunctionCallToHiddenVariablesMap;
	for(UNiagaraNodeFunctionCall* FunctionCall : UsedFunctionCallNodes)
	{
		TArray<FNiagaraVariable> InputVariables;
		TSet<FNiagaraVariable> HiddenVariables;
		FCompileConstantResolver Resolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCall));
		FNiagaraStackGraphUtilities::GetStackFunctionInputs(*FunctionCall, InputVariables, HiddenVariables, Resolver, ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly, true);
		FunctionCallToHiddenVariablesMap.Add(FunctionCall->NodeGuid, HiddenVariables);
		
		TArray<UEdGraphPin*> OutInputPins;
		TSet<UEdGraphPin*> OutHiddenPins;
		FNiagaraStackGraphUtilities::GetStackFunctionStaticSwitchPins(*FunctionCall, OutInputPins, OutHiddenPins, Resolver);
		for(UEdGraphPin* HiddenStaticSwitchPin : OutHiddenPins)
		{
			FunctionCallToHiddenVariablesMap[FunctionCall->NodeGuid].Add(UEdGraphSchema_Niagara::PinToNiagaraVariable(HiddenStaticSwitchPin));
		}
	}
	
	for(TSharedPtr<FHierarchyElementViewModel> HierarchyViewModel : ChildrenViewModels)
	{
		UHierarchyElement* Data = HierarchyViewModel->GetDataMutable();

		if(UNiagaraHierarchyModuleInput* ModuleInput = Cast<UNiagaraHierarchyModuleInput>(Data))
		{
			if(GetEmitterViewModel().IsValid())
			{
				UNiagaraNodeFunctionCall* OwningFunctionCallNode = State.NodeGuidToModuleNodeMap[ModuleInput->GetPersistentIdentity().Guids[0]];
				
				TSharedPtr<FNiagaraModuleInputViewModel> ModuleInputViewModel = StaticCastSharedPtr<FNiagaraModuleInputViewModel>(HierarchyViewModel);
				TOptional<FInputData> ModuleBaseInputData = ModuleInputViewModel->GetInputData();
				
				if(ModuleBaseInputData.IsSet())
				{
					UNiagaraStackFunctionInput* TopLevelInput = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput)
					{
						UNiagaraNodeFunctionCall* FunctionCallNode = CurrentInput->GetInputFunctionCallNodePtr();
						return FunctionCallNode != nullptr
							&& CurrentInput->GetInputParameterHandle() == ModuleBaseInputData->InputName && CurrentInput->GetInputType() == ModuleBaseInputData->Type
							&& CurrentInput->GetInputFunctionCallInitialScript() == OwningFunctionCallNode->FunctionScript
							// we are checking for node guid as we could have 2x the same input from 2 identical modules
							&& FunctionCallNode->NodeGuid == OwningFunctionCallNode->NodeGuid;
					});
	
					if(TopLevelInput == nullptr)
					{
						TopLevelInput = NewObject<UNiagaraStackFunctionInput>(this);
						TopLevelInput->Initialize(CreateDefaultChildRequiredData(), *OwningFunctionCallNode, *OwningFunctionCallNode, ModuleBaseInputData->InputName, ModuleBaseInputData->Type, ModuleBaseInputData->bIsStatic ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic, GetOwnerStackItemEditorDataKey());
						TAttribute<FText> SummaryViewDisplayNameOverride;
						SummaryViewDisplayNameOverride.BindUObject(ModuleInput, &UNiagaraHierarchyModuleInput::GetDisplayNameOverride);
						TopLevelInput->SetSummaryViewDisplayName(SummaryViewDisplayNameOverride);
						TAttribute<FText> SummaryViewTooltipOverride;
						SummaryViewTooltipOverride.BindUObject(ModuleInput, &UNiagaraHierarchyModuleInput::GetTooltipOverride);
						TopLevelInput->SetSummaryViewTooltip(SummaryViewTooltipOverride);
					}

					FNiagaraVariable DisplayedVariable(TopLevelInput->GetInputType(), TopLevelInput->GetInputParameterHandle().GetParameterHandleString());
					bool bIsTopLevelHidden = FunctionCallToHiddenVariablesMap[ModuleInput->GetPersistentIdentity().Guids[0]].Contains(DisplayedVariable);
					TopLevelInput->SetIsHidden(bIsTopLevelHidden);
					
					NewChildren.Add(TopLevelInput);

					TArray<UNiagaraHierarchyModuleInput*> ChildInputs;
					ModuleInput->GetChildrenOfType<UNiagaraHierarchyModuleInput>(ChildInputs);
					
					for(UNiagaraHierarchyModuleInput* ChildInput : ChildInputs)
					{
						TSharedPtr<FNiagaraModuleInputViewModel> ChildInputViewModel = StaticCastSharedPtr<FNiagaraModuleInputViewModel>(ModuleInputViewModel->FindViewModelForChild(ChildInput, false));
						TOptional<FInputData> ChildInputData = ChildInputViewModel->GetInputData();

						if(ChildInputData.IsSet())
						{
							UNiagaraNodeFunctionCall* ChildFunctionNode = ChildInputData->FunctionCallNode;
							UNiagaraStackFunctionInput* StackChildInput = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput)
							{
								UNiagaraNodeFunctionCall* FunctionCallNode = CurrentInput->GetInputFunctionCallNodePtr();
								return FunctionCallNode != nullptr
									&& CurrentInput->GetInputParameterHandle() == ChildInputData->InputName && CurrentInput->GetInputType() == ChildInputData->Type
									&& CurrentInput->GetInputFunctionCallInitialScript() == ChildFunctionNode->FunctionScript
									// we are checking for node guid as we could have 2x the same input from 2 identical modules
									&& FunctionCallNode->NodeGuid == ChildFunctionNode->NodeGuid;
							});
							
							if(StackChildInput == nullptr)
							{
								StackChildInput = NewObject<UNiagaraStackFunctionInput>(this);
								StackChildInput->Initialize(CreateDefaultChildRequiredData(), *ChildInputData->FunctionCallNode, *ChildInputData->FunctionCallNode, ChildInputData->InputName, ChildInputData->Type, ChildInputData->bIsStatic ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic, GetOwnerStackItemEditorDataKey());
								TAttribute<FText> SummaryViewDisplayNameOverride;
								SummaryViewDisplayNameOverride.BindUObject(ChildInput, &UNiagaraHierarchyModuleInput::GetDisplayNameOverride);
								StackChildInput->SetSummaryViewDisplayName(SummaryViewDisplayNameOverride);
								TAttribute<FText> SummaryViewTooltipOverride;
								SummaryViewTooltipOverride.BindUObject(ChildInput, &UNiagaraHierarchyModuleInput::GetTooltipOverride);
								StackChildInput->SetSummaryViewTooltip(SummaryViewTooltipOverride);
								
								StackChildInput->SetSemanticChild(true);
							}

							FNiagaraVariable DisplayedChildVariable(StackChildInput->GetInputType(), StackChildInput->GetInputParameterHandle().GetParameterHandleString());
							bool bIsChildInputHidden = FunctionCallToHiddenVariablesMap[ChildInput->GetPersistentIdentity().Guids[0]].Contains(DisplayedChildVariable);
							StackChildInput->SetIsHidden(bIsChildInputHidden);
							
							NewChildren.Add(StackChildInput);
						}
					}
					
					/** Automatically add children inputs. Disabled as user is managing this. */
					// if(State.HierarchyInputToChildrenGuidMap.Contains(ModuleInput))
					// {
					// 	// children guids are already sorted by sort order
					// 	TArray<FGuid> ChildrenInputs = State.HierarchyInputToChildrenGuidMap[ModuleInput];
					//
					// 	for(FGuid ChildrenInputGuid : ChildrenInputs)
					// 	{
					// 		if(UNiagaraScriptVariable* ChildScriptVariable = State.ChildrenGuidToScriptVariablesMap[ChildrenInputGuid])
					// 		{
					// 			FNiagaraVariable ChildVariable = ChildScriptVariable->Variable;
					//
					// 			// we generally don't show inline edit toggles as the managed items will display an inline checkbox instead
					// 			if(ChildScriptVariable->Metadata.bInlineEditConditionToggle)
					// 			{
					// 				continue;
					// 			}
					// 			
					// 			UNiagaraStackFunctionInput* InputChild = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
					// 			{
					// 				return CurrentInput->GetInputParameterHandle() == ChildScriptVariable->Variable.GetName() && CurrentInput->GetInputType() == ChildScriptVariable->Variable.GetType() && CurrentInput->GetInputFunctionCallNode().NodeGuid == ModuleBaseInputData->FunctionCallNode->NodeGuid;
					// 			});
					//
					// 			TOptional<bool> bIsStatic = OwningFunctionCallNode->GetCalledGraph()->IsStaticSwitch(ChildVariable);
					// 			if(InputChild == nullptr)
					// 			{
					// 				InputChild = NewObject<UNiagaraStackFunctionInput>(this);
					// 				InputChild->Initialize(CreateDefaultChildRequiredData(), *ModuleBaseInputData->FunctionCallNode, *ModuleBaseInputData->FunctionCallNode, ChildScriptVariable->Variable.GetName(), ChildScriptVariable->Variable.GetType(), bIsStatic.GetValue() ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic, GetOwnerStackItemEditorDataKey());						
					// 				InputChild->SetSemanticChild(true);
					// 			}
					//
					// 			// we update the hidden flag every time
					// 			bool bIsChildHidden = FunctionCallToHiddenVariablesMap[ModuleInput->GetPersistentIdentity().Guids[0]].Contains(ChildVariable);
					// 			InputChild->SetIsHidden(bIsTopLevelHidden || bIsChildHidden);
					// 			NewChildren.Add(InputChild);								
					// 		}
					// 	}
					// }					
				}
			}
		}
		else if(UNiagaraHierarchyAssignmentInput* AssignmentInput = Cast<UNiagaraHierarchyAssignmentInput>(Data))
		{
			TSharedPtr<FNiagaraAssignmentInputViewModel> AssignmentInputViewModel = StaticCastSharedPtr<FNiagaraAssignmentInputViewModel>(HierarchyViewModel);
			TOptional<FMatchingFunctionInputData> InputData = AssignmentInputViewModel->GetInputData();
			if(InputData.IsSet())
			{
				FString VariableNameWithModulePrefix = FNiagaraConstants::ModuleNamespaceString + ".";
				VariableNameWithModulePrefix.Append(InputData->InputName.ToString());
				FName VariableNameToTestAgainst(VariableNameWithModulePrefix);
				
				UNiagaraStackFunctionInput* TopLevelInput = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
				{
					return CurrentInput->GetInputParameterHandle() == VariableNameToTestAgainst && CurrentInput->GetInputType() == InputData->Type && &CurrentInput->GetInputFunctionCallNode() == InputData->FunctionCallNode;
				});
	
				if(TopLevelInput == nullptr)
				{
					TopLevelInput = NewObject<UNiagaraStackFunctionInput>(this);
					TopLevelInput->Initialize(CreateDefaultChildRequiredData(), *InputData->FunctionCallNode, *InputData->FunctionCallNode, FName(VariableNameWithModulePrefix), InputData->Type, InputData->bIsStatic ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic, GetOwnerStackItemEditorDataKey());
					TAttribute<FText> SummaryViewTooltipOverride;
					SummaryViewTooltipOverride.BindUObject(AssignmentInput, &UNiagaraHierarchyAssignmentInput::GetTooltipOverride);
					TopLevelInput->SetSummaryViewTooltip(SummaryViewTooltipOverride);
				}
				
				TopLevelInput->SetIsHidden(InputData->bIsHidden);
				
				NewChildren.Add(TopLevelInput);
			}
		}
		else if(UNiagaraHierarchyModule* SummaryModule = Cast<UNiagaraHierarchyModule>(Data))
		{
			if(UNiagaraNodeFunctionCall* MatchingFunctionCall = FindFunctionCallNode(SummaryModule->GetPersistentIdentity().Guids[0], GetEmitterViewModel().ToSharedRef()))
			{
				UNiagaraStackModuleItem* Module = FindCurrentChildOfTypeByPredicate<UNiagaraStackModuleItem>(CurrentChildren,
				[&](UNiagaraStackModuleItem* CurrentModule) { return &CurrentModule->GetModuleNode() == MatchingFunctionCall; });
		
				if (Module == nullptr)
				{
					Module = NewObject<UNiagaraStackModuleItem>(this);
					Module->Initialize(CreateDefaultChildRequiredData(), nullptr, *MatchingFunctionCall);
				}
			
				NewChildren.Add(Module);	
			}		
		}
		else if(UNiagaraHierarchyEventHandler* EventHandler = Cast<UNiagaraHierarchyEventHandler>(Data))
		{
			const TArray<FNiagaraEventScriptProperties>& EventScriptProperties = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetEventHandlers();
			const FNiagaraEventScriptProperties* EventScriptPropertiesItem = EventScriptProperties.FindByPredicate([EventHandlerIdentity = EventHandler->GetPersistentIdentity()](const FNiagaraEventScriptProperties& Candidate)
			{
				return Candidate.Script->GetUsageId() == EventHandlerIdentity.Guids[0];
			});

			if(EventScriptPropertiesItem != nullptr)
			{
				UNiagaraStackEventScriptItemGroup* StackEventGroup = FindCurrentChildOfTypeByPredicate<UNiagaraStackEventScriptItemGroup>(CurrentChildren,
				[&](UNiagaraStackEventScriptItemGroup* CurrentEventProperties) { return CurrentEventProperties->GetScriptUsageId() == (*EventScriptPropertiesItem).Script->GetUsageId() && CurrentEventProperties->GetEventSourceEmitterId() == (*EventScriptPropertiesItem).SourceEmitterID; });
		
				if (StackEventGroup == nullptr)
				{
					StackEventGroup = NewObject<UNiagaraStackEventScriptItemGroup>(this);
					StackEventGroup->Initialize(CreateDefaultChildRequiredData(), GetEmitterViewModel()->GetSharedScriptViewModel(), ENiagaraScriptUsage::ParticleEventScript, (*EventScriptPropertiesItem).Script->GetUsageId(), (*EventScriptPropertiesItem).SourceEmitterID);
					StackEventGroup->SetOnModifiedEventHandlers(UNiagaraStackEventScriptItemGroup::FOnModifiedEventHandlers::CreateUObject(this, &UNiagaraStackEntry::RefreshChildren));

				}
			
				NewChildren.Add(StackEventGroup);	
			}
		}
		else if(UNiagaraHierarchyEventHandlerProperties* EventHandlerProperties = Cast<UNiagaraHierarchyEventHandlerProperties>(Data))
		{
			const TArray<FNiagaraEventScriptProperties>& EventScriptProperties = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetEventHandlers();
			const FNiagaraEventScriptProperties* EventScriptPropertiesItem = EventScriptProperties.FindByPredicate([EventHandlerIdentity = EventHandlerProperties->GetPersistentIdentity()](const FNiagaraEventScriptProperties& Candidate)
			{
				FHierarchyElementIdentity CandidateIdentity = UNiagaraHierarchyEventHandlerProperties::MakeIdentity(Candidate);
				return CandidateIdentity == EventHandlerIdentity;
			});

			if(EventScriptPropertiesItem != nullptr)
			{
				UNiagaraStackEventHandlerPropertiesItem* StackEventProperties = FindCurrentChildOfTypeByPredicate<UNiagaraStackEventHandlerPropertiesItem>(CurrentChildren,
				[&](UNiagaraStackEventHandlerPropertiesItem* CurrentEventProperties) { return CurrentEventProperties->GetEventScriptUsageId() == EventScriptPropertiesItem->Script->GetUsageId(); });
		
				if (StackEventProperties == nullptr)
				{
					StackEventProperties = NewObject<UNiagaraStackEventHandlerPropertiesItem>(this);
					StackEventProperties->Initialize(CreateDefaultChildRequiredData(), (*EventScriptPropertiesItem).Script->GetUsageId());
				}
			
				NewChildren.Add(StackEventProperties);	
			}
		}
		else if(UNiagaraHierarchyRenderer* SummaryRenderer = Cast<UNiagaraHierarchyRenderer>(Data))
		{
			TArray<UNiagaraRendererProperties*> RendererProperties = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetRenderers();
			UNiagaraRendererProperties** MatchingRendererProperties = RendererProperties.FindByPredicate([RendererIdentity = SummaryRenderer->GetPersistentIdentity().Guids[0]](UNiagaraRendererProperties* Candidate)
			{
				return Candidate->GetMergeId() == RendererIdentity;
			});

			if(MatchingRendererProperties != nullptr)
			{
				UNiagaraStackRendererItem* StackRenderer = FindCurrentChildOfTypeByPredicate<UNiagaraStackRendererItem>(CurrentChildren,
				[&](UNiagaraStackRendererItem* CurrentRenderer) { return CurrentRenderer->GetRendererProperties()->GetMergeId() == (*MatchingRendererProperties)->GetMergeId(); });
		
				if (StackRenderer == nullptr)
				{
					StackRenderer = NewObject<UNiagaraStackRendererItem>(this);
					StackRenderer->Initialize(CreateDefaultChildRequiredData(), FNiagaraStackRenderersOwnerStandard::CreateShared(GetEmitterViewModel().ToSharedRef()), *MatchingRendererProperties);
				}
			
				NewChildren.Add(StackRenderer);	
			}
		}
		else if(UNiagaraHierarchyEmitterProperties* EmitterProperties = Cast<UNiagaraHierarchyEmitterProperties>(Data))
		{
			UNiagaraStackEmitterPropertiesItem* StackEmitterPropertiesItem = FindCurrentChildOfTypeByPredicate<UNiagaraStackEmitterPropertiesItem>(CurrentChildren,
			[&](UNiagaraStackEmitterPropertiesItem* CurrentEmitterProperties) { return CurrentEmitterProperties->GetEmitterViewModel()->GetEmitter().Emitter->GetUniqueEmitterName() == EmitterProperties->GetPersistentIdentity().Names[0]; });
	
			if (StackEmitterPropertiesItem == nullptr)
			{
				StackEmitterPropertiesItem = NewObject<UNiagaraStackEmitterPropertiesItem>(this);
				StackEmitterPropertiesItem->Initialize(CreateDefaultChildRequiredData());
			}
		
			NewChildren.Add(StackEmitterPropertiesItem);	
		}
		else if(UNiagaraHierarchySimStage* SummarySimStage = Cast<UNiagaraHierarchySimStage>(Data))
		{
			TArray<UNiagaraSimulationStageBase*> SimStages = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetSimulationStages();
			UNiagaraSimulationStageBase** MatchingSimStage = SimStages.FindByPredicate([RendererIdentity = SummarySimStage->GetPersistentIdentity().Guids[0]](UNiagaraSimulationStageBase* Candidate)
			{
				return Candidate->GetMergeId() == RendererIdentity;
			});

			if(MatchingSimStage != nullptr)
			{
				UNiagaraStackSimulationStageGroup* SimStageGroup = FindCurrentChildOfTypeByPredicate<UNiagaraStackSimulationStageGroup>(CurrentChildren,
				[&](UNiagaraStackSimulationStageGroup* CurrentSimStageItem)
				{
					if(CurrentSimStageItem->GetSimulationStage())
					{
						return CurrentSimStageItem->GetSimulationStage()->GetMergeId() == (*MatchingSimStage)->GetMergeId();
					}

					return false;
				});
		
				if (SimStageGroup == nullptr)
				{
					SimStageGroup = NewObject<UNiagaraStackSimulationStageGroup>(this);
					FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
						FExecutionCategoryNames::Particle, FExecutionSubcategoryNames::SimulationStage,
						GetEmitterViewModel()->GetEditorData().GetStackEditorData());
					SimStageGroup->Initialize(RequiredEntryData, GetEmitterViewModel()->GetSharedScriptViewModel(), *MatchingSimStage);
					SimStageGroup->SetOnModifiedSimulationStages(UNiagaraStackSimulationStageGroup::FOnModifiedSimulationStages::CreateUObject(this, &UNiagaraStackEntry::RefreshChildren));
				}
			
				NewChildren.Add(SimStageGroup);	
			}
		}
		else if(UNiagaraHierarchySimStageProperties* SummarySimStageProperties = Cast<UNiagaraHierarchySimStageProperties>(Data))
		{
			TArray<UNiagaraSimulationStageBase*> SimStages = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetSimulationStages();
			UNiagaraSimulationStageBase** MatchingSimStage = SimStages.FindByPredicate([RendererIdentity = SummarySimStageProperties->GetPersistentIdentity().Guids[0]](UNiagaraSimulationStageBase* Candidate)
			{
				return Candidate->GetMergeId() == RendererIdentity;
			});

			if(MatchingSimStage != nullptr)
			{
				UNiagaraStackSimulationStagePropertiesItem* SimStageProperties = FindCurrentChildOfTypeByPredicate<UNiagaraStackSimulationStagePropertiesItem>(CurrentChildren,
				[&](UNiagaraStackSimulationStagePropertiesItem* CurrentSimStageItem)
				{
					if(CurrentSimStageItem->GetSimulationStage().IsValid())
					{
						return CurrentSimStageItem->GetSimulationStage()->GetMergeId() == (*MatchingSimStage)->GetMergeId();
					}

					return false;
				});
		
				if (SimStageProperties == nullptr)
				{
					SimStageProperties = NewObject<UNiagaraStackSimulationStagePropertiesItem>(this);
					SimStageProperties->Initialize(CreateDefaultChildRequiredData(), *MatchingSimStage);
				}
			
				NewChildren.Add(SimStageProperties);	
			}
		}
		else if(UHierarchyCategory* HierarchyCategory = Cast<UHierarchyCategory>(Data))
		{			
			TSharedPtr<FHierarchyCategoryViewModel> CategoryViewModel = StaticCastSharedPtr<FHierarchyCategoryViewModel>(HierarchyViewModel);
			
			UNiagaraStackSummaryCategory* StackCategory = FindCurrentChildOfTypeByPredicate<UNiagaraStackSummaryCategory>(CurrentChildren,
			[&](UNiagaraStackSummaryCategory* StackCategoryCandidate)
			{
				if(StackCategoryCandidate->GetHierarchyCategory().IsValid())
				{
					return StackCategoryCandidate->GetHierarchyCategory().Pin()->GetData() == CategoryViewModel->GetData();
				}

				return false;
			});
	
			if (StackCategory == nullptr)
			{
				StackCategory = NewObject<UNiagaraStackSummaryCategory>(this);
				StackCategory->Initialize(CreateDefaultChildRequiredData(), CategoryViewModel, GetOwnerStackItemEditorDataKey());
			}
		
			NewChildren.Add(StackCategory);	
		}
		else if(const UNiagaraHierarchyObjectProperty* ObjectProperty = Cast<UNiagaraHierarchyObjectProperty>(Data))
		{
			TMap<FGuid, UObject*> ObjectsForProperties = GetEmitterViewModel()->GetSummaryHierarchyViewModel()->GetObjectsForProperties();
			FGuid ObjectGuid = ObjectProperty->GetPersistentIdentity().Guids.Num() > 0 ? ObjectProperty->GetPersistentIdentity().Guids[0] : FGuid();
			
			if(ObjectsForProperties.Contains(ObjectGuid))
			{
				UObject* Object = ObjectsForProperties[ObjectProperty->GetPersistentIdentity().Guids[0]];

				UNiagaraStackObject* StackObjectWithProperty = FindCurrentChildOfTypeByPredicate<UNiagaraStackObject>(CurrentChildren,
				[&](UNiagaraStackObject* StackObjectCandidate)
				{
					return StackObjectCandidate->GetObject() == Object && StackObjectCandidate->GetCustomName() == ObjectProperty->GetPersistentIdentity().Names[0];
				});

				if(StackObjectWithProperty == nullptr)
				{
					StackObjectWithProperty = NewObject<UNiagaraStackObject>(this);
					bool bIsInTopLevelObject = false;
					bool bHideTopLevelCategories = false;
					StackObjectWithProperty->Initialize(CreateDefaultChildRequiredData(), Object, bIsInTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey(), nullptr);
					StackObjectWithProperty->SetCustomName(ObjectProperty->GetPersistentIdentity().Names[0]);
					StackObjectWithProperty->SetOnFilterDetailNodes(FNiagaraStackObjectShared::FOnFilterDetailNodes::CreateLambda([PropertyName = ObjectProperty->GetPersistentIdentity().Names[0]]( const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes)
					{
						for(const TSharedRef<IDetailTreeNode>& SourceNode : InSourceNodes)
						{
							TArray<TSharedRef<IDetailTreeNode>> ChildrenNodes;
							SourceNode->GetChildren(ChildrenNodes);
							
							for(TSharedRef<IDetailTreeNode> ChildNode : ChildrenNodes)
							{
								if(ChildNode->GetNodeName() == PropertyName)
								{
									OutFilteredNodes.Add(ChildNode);
								}
							}
						}
					}));
				}
				
				NewChildren.Add(StackObjectWithProperty);
			}
		}
	}

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

bool UNiagaraStackSummaryCategory::IsTopLevelCategory() const
{
	TSharedPtr<FHierarchyCategoryViewModel> CategoryViewModel = CategoryViewModelWeakPtr.Pin();
	if (ensure(CategoryViewModel.IsValid()))
	{
		return CategoryViewModel->GetData()->GetOuter()->IsA<UHierarchyRoot>();
	}
	return false;
}

FText UNiagaraStackSummaryCategory::GetTooltipText() const
{
	TSharedPtr<FHierarchyCategoryViewModel> CategoryViewModel = CategoryViewModelWeakPtr.Pin();
	if (ensure(CategoryViewModel.IsValid()))
	{
		if (const UHierarchyCategory* Category = Cast<const UHierarchyCategory>(CategoryViewModel->GetData()))
		{
			return Category->GetTooltip();
		}
	}
	return FText();
}

int32 UNiagaraStackSummaryCategory::GetChildIndentLevel() const
{
	return UNiagaraStackEntry::GetChildIndentLevel();
}

#undef LOCTEXT_NAMESPACE
