// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackItemFooter.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "NiagaraStackEditorData.h"

#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackItem)

void UNiagaraStackItem::Initialize(FRequiredEntryData InRequiredEntryData, FString InStackEditorDataKey)
{
	Super::Initialize(InRequiredEntryData, InStackEditorDataKey);
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackItem::FilterAdvancedChildren));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackItem::FilterHiddenChildren));
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackItem::GetStackRowStyle() const
{
	return UNiagaraStackEntry::EStackRowStyle::ItemHeader;
}

UNiagaraStackItem::FOnModifiedGroupItems& UNiagaraStackItem::OnModifiedGroupItems()
{
	return ModifiedGroupItemsDelegate;
}

void UNiagaraStackItem::SetOnRequestCanPaste(FOnRequestCanPaste InOnRequestCanPaste)
{
	RequestCanPasteDelegete = InOnRequestCanPaste;
}

void UNiagaraStackItem::SetOnRequestPaste(FOnRequestPaste InOnRequestCanPaste)
{
	RequestPasteDelegate = InOnRequestCanPaste;
}

void UNiagaraStackItem::SetIsEnabled(bool bInIsEnabled)
{
	if (ItemFooter != nullptr)
	{
		ItemFooter->SetIsEnabled(bInIsEnabled);
	}
	SetIsEnabledInternal(bInIsEnabled);
}

void UNiagaraStackItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (ItemFooter == nullptr)
	{
		ItemFooter = NewObject<UNiagaraStackItemFooter>(this);
		ItemFooter->Initialize(CreateDefaultChildRequiredData(), GetStackEditorDataKey());
		ItemFooter->SetOnToggleShowAdvanced(UNiagaraStackItemFooter::FOnToggleShowAdvanced::CreateUObject(this, &UNiagaraStackItem::ToggleShowAdvanced));
	}
	ItemFooter->SetIsEnabled(GetIsEnabled());

	NewChildren.Add(ItemFooter);
}

void GetContentChildren(UNiagaraStackEntry& CurrentEntry, TArray<UNiagaraStackItemContent*>& ContentChildren)
{
	TArray<UNiagaraStackEntry*> Children;
	CurrentEntry.GetUnfilteredChildren(Children);
	for (UNiagaraStackEntry* Child : Children)
	{
		UNiagaraStackItemContent* ContentChild = Cast<UNiagaraStackItemContent>(Child);
		if (ContentChild != nullptr)
		{
			ContentChildren.Add(ContentChild);
		}
		GetContentChildren(*Child, ContentChildren);
	}
}

void UNiagaraStackItem::PostRefreshChildrenInternal()
{
	Super::PostRefreshChildrenInternal();
	bool bHasAdvancedContent = false;
	bool bHasChangedContent = false;
	TArray<UNiagaraStackItemContent*> ContentChildren;
	GetContentChildren(*this, ContentChildren);
	bool bHasAdvancedIssues = false;
	for (UNiagaraStackItemContent* ContentChild : ContentChildren)
	{
		if (ContentChild->GetIsAdvanced())
		{
			bHasAdvancedContent = true;
			bHasChangedContent |= ContentChild->HasOverridenContent();
			bHasAdvancedIssues |= ContentChild->HasIssuesOrAnyChildHasIssues();
		}
	}
	ItemFooter->SetHasAdvancedContent(bHasAdvancedContent, bHasChangedContent);

	if (bHasAdvancedIssues)
	{
		GetStackEditorData().SetStackItemShowAdvanced(GetStackEditorDataKey(), true);
	}
}

int32 UNiagaraStackItem::GetChildIndentLevel() const
{
	return GetIndentLevel();
}

bool UNiagaraStackItem::FilterAdvancedChildren(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackItemContent* ItemContent = Cast<UNiagaraStackItemContent>(&Child);
	if (ItemContent == nullptr || ItemContent->GetIsAdvanced() == false)
	{
		return true;
	}
	else
	{
		return GetStackEditorData().GetShowAllAdvanced() || GetStackEditorData().GetStackItemShowAdvanced(GetStackEditorDataKey(), false);
	}
}

bool UNiagaraStackItem::FilterHiddenChildren(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackItemContent* ItemContent = Cast<UNiagaraStackItemContent>(&Child);
	return ItemContent == nullptr || ItemContent->GetIsHidden() == false;
}

void UNiagaraStackItem::ToggleShowAdvanced()
{
	bool bCurrentShowAdvanced = GetStackEditorData().GetStackItemShowAdvanced(GetStackEditorDataKey(), false);
	GetStackEditorData().SetStackItemShowAdvanced(GetStackEditorDataKey(), !bCurrentShowAdvanced);
	ToggleShowAdvancedInternal();
}

void UNiagaraStackItem::ToggleShowAdvancedInternal()
{
	RefreshFilteredChildren();
}

void UNiagaraStackItemContent::Initialize(FRequiredEntryData InRequiredEntryData, FString InOwningStackItemEditorDataKey, FString InStackEditorDataKey)
{
	Super::Initialize(InRequiredEntryData, InStackEditorDataKey);
	OwningStackItemEditorDataKey = InOwningStackItemEditorDataKey;
	bIsAdvanced = false;
	bIsHidden = false;
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackItemContent::FilterAdvancedChildren));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackItemContent::FilterHiddenChildren));
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackItemContent::GetStackRowStyle() const
{
	return bIsAdvanced ? EStackRowStyle::ItemContentAdvanced : EStackRowStyle::ItemContent;
}

bool UNiagaraStackItemContent::GetIsAdvanced() const
{
	return bIsAdvanced;
}

bool UNiagaraStackItemContent::GetIsHidden() const
{
	return bIsHidden;
}


void UNiagaraStackItemContent::SetIsHidden(bool bInIsHidden)
{
	if (bIsHidden != bInIsHidden)
	{
		// When changing is hidden, invalidate the structure so that the filters run again.
		bIsHidden = bInIsHidden;
		OnStructureChanged().Broadcast(ENiagaraStructureChangedFlags::FilteringChanged);
	}
}

bool UNiagaraStackItemContent::HasOverridenContent() const
{
	return false;
}

FString UNiagaraStackItemContent::GetOwnerStackItemEditorDataKey() const
{
	return OwningStackItemEditorDataKey;
}

void UNiagaraStackItemContent::SetIsAdvanced(bool bInIsAdvanced)
{
	if (bIsAdvanced != bInIsAdvanced)
	{
		// When changing advanced, invalidate the structure so that the filters run again.
		bIsAdvanced = bInIsAdvanced;
		OnStructureChanged().Broadcast(ENiagaraStructureChangedFlags::FilteringChanged);
	}
}

bool UNiagaraStackItemContent::FilterAdvancedChildren(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackItemContent* ItemContent = Cast<UNiagaraStackItemContent>(&Child);
	if (ItemContent == nullptr || ItemContent->GetIsAdvanced() == false || ItemContent->GetIsSearchResult())
	{
		return true;
	}
	else
	{
		return GetStackEditorData().GetShowAllAdvanced() || GetStackEditorData().GetStackItemShowAdvanced(OwningStackItemEditorDataKey, false);
	}
}

bool UNiagaraStackItemContent::FilterHiddenChildren(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackItemContent* ItemContent = Cast<UNiagaraStackItemContent>(&Child);
	return ItemContent == nullptr || ItemContent->GetIsHidden() == false;
}

void UNiagaraStackItemTextContent::Initialize(FRequiredEntryData InRequiredEntryData, FText InDisplayText, FString InOwningStackItemEditorDataKey)
{
	Super::Initialize(InRequiredEntryData, InOwningStackItemEditorDataKey, InOwningStackItemEditorDataKey + InDisplayText.ToString());
	DisplayText = InDisplayText;
}

FText UNiagaraStackItemTextContent::GetDisplayName() const
{
	return DisplayText;
}


