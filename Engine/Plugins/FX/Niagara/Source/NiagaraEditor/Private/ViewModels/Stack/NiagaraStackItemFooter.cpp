// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackItemFooter.h"
#include "NiagaraStackEditorData.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackItemFooter)

void UNiagaraStackItemFooter::Initialize(
	FRequiredEntryData InRequiredEntryData,
	FString InOwnerStackItemEditorDataKey)
{
	Super::Initialize(InRequiredEntryData, FString());
	OwnerStackItemEditorDataKey = InOwnerStackItemEditorDataKey;
	bHasAdvancedContent = false;
	bIsEnabled = true;
}

bool UNiagaraStackItemFooter::GetCanExpand() const
{
	return false;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackItemFooter::GetStackRowStyle() const
{
	return UNiagaraStackEntry::EStackRowStyle::ItemFooter;
}

bool UNiagaraStackItemFooter::GetIsEnabled() const
{
	return bIsEnabled;
}

void UNiagaraStackItemFooter::SetIsEnabled(bool bInIsEnabled)
{
	bIsEnabled = bInIsEnabled;
}


bool UNiagaraStackItemFooter::GetHasAdvancedContent() const
{
	return bHasAdvancedContent;
}

void UNiagaraStackItemFooter::SetHasAdvancedContent(bool bInHasAdvancedRows, bool bInHasChangedContent)
{
	bHasAdvancedContent = bInHasAdvancedRows;
	bHasChangedAdvancedContent = bInHasChangedContent;
}

void UNiagaraStackItemFooter::SetOnToggleShowAdvanced(FOnToggleShowAdvanced OnExpandedChanged)
{
	ToggleShowAdvancedDelegate = OnExpandedChanged;
}

bool UNiagaraStackItemFooter::GetShowAdvanced() const
{
	if (IsFinalized())
	{
		return false;
	}
	return GetStackEditorData().GetStackItemShowAdvanced(OwnerStackItemEditorDataKey, false);
}

void UNiagaraStackItemFooter::ToggleShowAdvanced()
{
	ToggleShowAdvancedDelegate.ExecuteIfBound();
}

