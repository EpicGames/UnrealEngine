// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "NiagaraStackEditorData.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackItemGroup)

void UNiagaraStackItemGroup::Initialize(FRequiredEntryData InRequiredEntryData, FText InDisplayName, FText InToolTip, INiagaraStackItemGroupAddUtilities* InAddUtilities)
{
	Super::Initialize(InRequiredEntryData, InDisplayName.ToString());
	GroupDisplayName = InDisplayName;
	GroupToolTip = InToolTip;
	AddUtilities = InAddUtilities;
	GroupFooter = nullptr;

	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackItemGroup::FilterChildrenWithIssues));
}

FText UNiagaraStackItemGroup::GetDisplayName() const
{
	return GroupDisplayName;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackItemGroup::GetStackRowStyle() const
{
	return EStackRowStyle::GroupHeader;
}

FText UNiagaraStackItemGroup::GetTooltipText() const 
{
	return GroupToolTip;
}

bool UNiagaraStackItemGroup::GetIsEnabled() const
{
	return OwningEmitterHandleViewModelWeak.IsValid() == false || OwningEmitterHandleViewModelWeak.Pin()->GetIsEnabled();
}

INiagaraStackItemGroupAddUtilities* UNiagaraStackItemGroup::GetAddUtilities() const
{
	return AddUtilities;
}

uint32 UNiagaraStackItemGroup::GetRecursiveStackIssuesCount() const
{
	if (RecursiveStackIssuesCount.IsSet() == false)
	{
		TArray<UNiagaraStackErrorItem*> RecursiveIssues;
		FNiagaraStackGraphUtilities::GetStackIssuesRecursively(this, RecursiveIssues);
		RecursiveStackIssuesCount = RecursiveIssues.Num();
		EStackIssueSeverity MinSeverity = EStackIssueSeverity::None;
		for (auto Issue : RecursiveIssues)
		{
			if (Issue->GetStackIssue().GetSeverity() < MinSeverity)
			{
				MinSeverity = Issue->GetStackIssue().GetSeverity();
			}
		}
		HighestIssueSeverity = MinSeverity;
	}
	return RecursiveStackIssuesCount.GetValue();
}

EStackIssueSeverity UNiagaraStackItemGroup::GetHighestStackIssueSeverity() const
{
	if (HighestIssueSeverity.IsSet() == false)
	{
		GetRecursiveStackIssuesCount();
	}
	return HighestIssueSeverity.GetValue();
}

void UNiagaraStackItemGroup::SetDisplayName(FText InDisplayName)
{
	GroupDisplayName = InDisplayName;
}

void UNiagaraStackItemGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (GroupFooter == nullptr)
	{
		GroupFooter = NewObject<UNiagaraStackItemGroupFooter>(this);
		GroupFooter->Initialize(FRequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(), NAME_None, NAME_None, GetStackEditorData()));
	}

	NewChildren.Add(GroupFooter);

	RecursiveStackIssuesCount.Reset();
	HighestIssueSeverity.Reset();

	TSharedPtr<FNiagaraEmitterViewModel> OwningEmitterViewModel = GetEmitterViewModel();
	if (OwningEmitterViewModel.IsValid())
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> OwningEmitterHandleViewModel = OwningEmitterHandleViewModelWeak.Pin();
		if (OwningEmitterHandleViewModel.IsValid() == false || OwningEmitterHandleViewModel->GetEmitterViewModel() != OwningEmitterViewModel)
		{
			OwningEmitterHandleViewModelWeak = TWeakPtr<FNiagaraEmitterHandleViewModel>(
				GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(OwningEmitterViewModel->GetEmitter()));
		}
	}
	else
	{
		OwningEmitterHandleViewModelWeak.Reset();
	}
}

int32 UNiagaraStackItemGroup::GetChildIndentLevel() const
{
	return GetIndentLevel();
}

void UNiagaraStackItemGroup::ChildStructureChangedInternal()
{
	Super::ChildStructureChangedInternal();
	RecursiveStackIssuesCount.Reset();
	HighestIssueSeverity.Reset();
}

bool UNiagaraStackItemGroup::FilterChildrenWithIssues(const UNiagaraStackEntry& Child) const
{
	if (GetStackEditorData().GetShowOnlyIssues() == false)
	{
		return true;
	}

	const UNiagaraStackModuleItem* Module = Cast<UNiagaraStackModuleItem>(&Child);
	if (Module != nullptr)
	{
		return Module->HasIssuesOrAnyChildHasIssues();
	}

	return false;
}

void UNiagaraStackItemGroupFooter::Initialize(FRequiredEntryData InRequiredEntryData)
{
	Super::Initialize(InRequiredEntryData, FString());
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackItemGroupFooter::GetStackRowStyle() const
{
	return UNiagaraStackEntry::EStackRowStyle::GroupFooter;
}

bool UNiagaraStackItemGroupFooter::GetCanExpand() const
{
	return false;
}

