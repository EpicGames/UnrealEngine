// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackItemPropertyHeaderValueShared.h"

#include "ScopedTransaction.h"
#include "UObject/EnumProperty.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "NiagaraEmitterStatelessSpawnGroup"

FNiagaraStackItemPropertyHeaderValue::FNiagaraStackItemPropertyHeaderValue(UObject& InOwnerObject, uint8* InOwnerStructMemory, FEnumProperty& InEnumProperty, FSimpleDelegate InOnChanged)
	: OwnerObjectWeak(&InOwnerObject)
	, OwnerStructMemory(InOwnerStructMemory)
	, EnumProperty(&InEnumProperty)
	, BoolProperty(nullptr)
	, OnChangedDelegate(InOnChanged)
	, ValueMode(INiagaraStackItemHeaderValueHandler::EValueMode::EnumDropDown)
	, Enum(InEnumProperty.GetEnum())
{
	UpdateFromProperty(EnumProperty);
}

FNiagaraStackItemPropertyHeaderValue::FNiagaraStackItemPropertyHeaderValue(UObject& InOwnerObject, uint8* InOwnerStructMemory, FBoolProperty& InBoolProperty, FSimpleDelegate InOnChanged)
	: OwnerObjectWeak(&InOwnerObject)
	, OwnerStructMemory(InOwnerStructMemory)
	, EnumProperty(nullptr)
	, BoolProperty(&InBoolProperty)
	, OnChangedDelegate(InOnChanged)
	, ValueMode(INiagaraStackItemHeaderValueHandler::EValueMode::BoolToggle)
	, Enum(nullptr)
{
	UpdateFromProperty(BoolProperty);
}

int32 FNiagaraStackItemPropertyHeaderValue::GetEnumValue() const
{
	if (EnumProperty == nullptr)
	{
		return false;
	}

	if (EnumValueCache.IsSet() == false)
	{
		UObject* OwnerObject = OwnerObjectWeak.Get();
		if (OwnerObject != nullptr)
		{
			uint8* ValuePtr;
			if (OwnerStructMemory != nullptr)
			{
				ValuePtr = OwnerStructMemory + EnumProperty->GetOffset_ForInternal();
			}
			else
			{
				ValuePtr = reinterpret_cast<uint8*>(OwnerObject) + EnumProperty->GetOffset_ForInternal();
			}
			EnumValueCache = static_cast<uint32>(EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(ValuePtr));
		}
		else
		{
			EnumValueCache = INDEX_NONE;
		}
	}
	return EnumValueCache.GetValue();
}

void FNiagaraStackItemPropertyHeaderValue::NotifyEnumValueChanged(int32 InValue)
{
	if (InValue == GetEnumValue() || EnumProperty == nullptr)
	{
		return;
	}

	UObject* OwnerObject = OwnerObjectWeak.Get();
	if (OwnerObject == nullptr)
	{
		return;
	}

	uint8* ValuePtr;
	if (OwnerStructMemory != nullptr)
	{
		ValuePtr = OwnerStructMemory + EnumProperty->GetOffset_ForInternal();
	}
	else
	{
		ValuePtr = (uint8*)OwnerObject + EnumProperty->GetOffset_ForInternal();
	}

	EnumValueCache = InValue;
	FScopedTransaction ScopedTransaction(FText::Format(LOCTEXT("PropertyHeaderValueEnumChangeTransaction", "Change {0}"), LabelText));
	OwnerObject->Modify();
	EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(ValuePtr, (int64)InValue);
	FPropertyChangedEvent PropertyChangedEvent(EnumProperty->GetUnderlyingProperty());
	OwnerObject->PostEditChangeProperty(PropertyChangedEvent);
	OnChangedDelegate.ExecuteIfBound();
}

bool FNiagaraStackItemPropertyHeaderValue::GetBoolValue() const
{
	if (BoolProperty == nullptr)
	{
		return false;
	}

	if (BoolValueCache.IsSet() == false)
	{
		UObject* OwnerObject = OwnerObjectWeak.Get();
		if (OwnerObject != nullptr)
		{
			uint8* ValuePtr;
			if (OwnerStructMemory != nullptr)
			{
				ValuePtr = OwnerStructMemory + BoolProperty->GetOffset_ForInternal();
			}
			else
			{
				ValuePtr = (uint8*)OwnerObject + BoolProperty->GetOffset_ForInternal();
			}
			BoolValueCache = BoolProperty->GetPropertyValue(ValuePtr);
		}
		else
		{
			BoolValueCache = false;
		}
	}
	return BoolValueCache.GetValue();
}

void FNiagaraStackItemPropertyHeaderValue::NotifyBoolValueChanged(bool bInValue)
{
	if (bInValue == GetBoolValue() || BoolProperty == nullptr)
	{
		return;
	}

	UObject* OwnerObject = OwnerObjectWeak.Get();
	if (OwnerObject == nullptr)
	{
		return;
	}

	uint8* ValuePtr;
	if (OwnerStructMemory != nullptr)
	{
		ValuePtr = OwnerStructMemory + BoolProperty->GetOffset_ForInternal();
	}
	else
	{
		ValuePtr = (uint8*)OwnerObject + BoolProperty->GetOffset_ForInternal();
	}

	BoolValueCache = bInValue;
	FScopedTransaction ScopedTransaction(FText::Format(LOCTEXT("PropertyHeaderValueBoolChangeTransaction", "Change {0}"), LabelText));
	OwnerObject->Modify();
	BoolProperty->SetPropertyValue(ValuePtr, bInValue);
	FPropertyChangedEvent PropertyChangedEvent(BoolProperty);
	OwnerObject->PostEditChangeProperty(PropertyChangedEvent);
	OnChangedDelegate.ExecuteIfBound();
}

void FNiagaraStackItemPropertyHeaderValue::Refresh()
{
	EnumValueCache.Reset();
	BoolValueCache.Reset();
}

void FNiagaraStackItemPropertyHeaderValue::UpdateFromProperty(FProperty* Property)
{
	LabelText = Property->GetDisplayNameText();
	IconBrush = Property->HasMetaData("StackItemHeaderIcon")
		? FAppStyle::GetBrush(*Property->GetMetaData("StackItemHeaderIcon"))
		: nullptr;
	HAlign = Property->GetMetaData("StackItemHeaderAlignment") == TEXT("Left")
		? EHorizontalAlignment::HAlign_Left
		: EHorizontalAlignment::HAlign_Right;
}

void FNiagaraStackItemPropertyHeaderValueShared::GenerateHeaderValueHandlers(UObject& InOwnerObject, uint8* InOwnerStructMemory, UStruct& InTargetStruct, FSimpleDelegate InOnChanged, TArray<TSharedRef<FNiagaraStackItemPropertyHeaderValue>>& OutHeaderValueHandlers)
{
	for (TFieldIterator<FProperty> PropertyIt(&InTargetStruct, EFieldIteratorFlags::SuperClassFlags::IncludeSuper, EFieldIteratorFlags::DeprecatedPropertyFlags::ExcludeDeprecated); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if (Property->HasMetaData("ShowInStackItemHeader"))
		{
			FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property);
			if (EnumProperty != nullptr)
			{
				OutHeaderValueHandlers.Add(MakeShared<FNiagaraStackItemPropertyHeaderValue>(InOwnerObject, InOwnerStructMemory, *EnumProperty, InOnChanged));
			}
			else
			{
				FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property);
				if (BoolProperty != nullptr)
				{
					OutHeaderValueHandlers.Add(MakeShared<FNiagaraStackItemPropertyHeaderValue>(InOwnerObject, InOwnerStructMemory, *BoolProperty, InOnChanged));
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
