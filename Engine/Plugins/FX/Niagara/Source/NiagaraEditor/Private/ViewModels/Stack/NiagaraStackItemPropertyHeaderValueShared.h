// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItem.h"

class FNiagaraStackItemPropertyHeaderValue : public INiagaraStackItemHeaderValueHandler
{
public:
	FNiagaraStackItemPropertyHeaderValue(UObject& InOwnerObject, uint8* InOwnerStructMemory, FEnumProperty& InEnumProperty, FSimpleDelegate InOnChanged);
	FNiagaraStackItemPropertyHeaderValue(UObject& InOwnerObject, uint8* InOwnerStructMemory, FBoolProperty& InBoolProperty, FSimpleDelegate InOnChanged);

	virtual EValueMode GetMode() const override { return ValueMode; }
	virtual const UEnum* GetEnum() const override { return Enum; }
	virtual const FText& GetLabelText() const override { return LabelText; }
	virtual const FSlateBrush* GetIconBrush() const override { return IconBrush; }
	virtual const EHorizontalAlignment GetHAlign() const override { return HAlign; }
	virtual int32 GetEnumValue() const override;
	virtual void NotifyEnumValueChanged(int32 InValue) override;
	virtual bool GetBoolValue() const override;
	virtual void NotifyBoolValueChanged(bool bInValue) override;

	void Refresh();

private:
	void UpdateFromProperty(FProperty* Property);

private:
	TWeakObjectPtr<UObject> OwnerObjectWeak;
	uint8* OwnerStructMemory;
	FEnumProperty* EnumProperty;
	FBoolProperty* BoolProperty;
	FSimpleDelegate OnChangedDelegate;

	EValueMode ValueMode;
	UEnum* Enum;
	FText LabelText;
	const FSlateBrush* IconBrush;
	EHorizontalAlignment HAlign;

	mutable TOptional<int32> EnumValueCache;
	mutable TOptional<bool> BoolValueCache;
};

namespace FNiagaraStackItemPropertyHeaderValueShared
{
	void GenerateHeaderValueHandlers(UObject& InOwnerObject, uint8* InOwnerStructMemory, UStruct& InTargetStruct, FSimpleDelegate InOnChanged, TArray<TSharedRef<FNiagaraStackItemPropertyHeaderValue>>& OutHeaderValueHandlers);
}