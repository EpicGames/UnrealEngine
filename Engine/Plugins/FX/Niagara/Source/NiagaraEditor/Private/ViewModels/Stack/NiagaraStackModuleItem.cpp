// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackModuleItem.h"

#include "NiagaraActions.h"
#include "NiagaraAnalytics.h"
#include "NiagaraClipboard.h"
#include "NiagaraCommon.h"
#include "NiagaraConstants.h"
#include "NiagaraConvertInPlaceUtilityBase.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraGraph.h"
#include "NiagaraMessageManager.h"
#include "NiagaraMessages.h"
#include "NiagaraMessageUtilities.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraScript.h"
#include "NiagaraScriptMergeManager.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraSystem.h"
#include "ScopedTransaction.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackModuleItemLinkedInputCollection.h"
#include "ViewModels/Stack/NiagaraStackModuleItemOutputCollection.h"
#include "NiagaraEditorModule.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "Widgets/SToolTip.h"

// TODO: Remove these
#include "IPythonScriptPlugin.h"

#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackModuleItem)

#define LOCTEXT_NAMESPACE "NiagaraStackModuleItem"

TArray<ENiagaraScriptUsage> UsagePriority = { // Ordered such as the highest priority has the largest index
	ENiagaraScriptUsage::ParticleUpdateScript,
	ENiagaraScriptUsage::ParticleSpawnScript,
	ENiagaraScriptUsage::EmitterUpdateScript,
	ENiagaraScriptUsage::EmitterSpawnScript,
	ENiagaraScriptUsage::SystemUpdateScript,
	ENiagaraScriptUsage::SystemSpawnScript };

UNiagaraStackModuleItem::UNiagaraStackModuleItem()
	: FunctionCallNode(nullptr)
	, bCanRefresh(false)
	, bIsModuleScriptReassignmentPending(false)
{
}

UNiagaraNodeFunctionCall& UNiagaraStackModuleItem::GetModuleNode() const
{
	return *FunctionCallNode;
}

void UNiagaraStackModuleItem::Initialize(FRequiredEntryData InRequiredEntryData, INiagaraStackItemGroupAddUtilities* InGroupAddUtilities, UNiagaraNodeFunctionCall& InFunctionCallNode)
{
	checkf(FunctionCallNode == nullptr, TEXT("Can not set the node more than once."));
	Super::Initialize(InRequiredEntryData, FNiagaraStackGraphUtilities::StackKeys::GenerateStackModuleEditorDataKey(InFunctionCallNode));
	GroupAddUtilities = InGroupAddUtilities;
	FunctionCallNode = &InFunctionCallNode;
	OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*FunctionCallNode);

	// We do not need to include child filters for NiagaraNodeAssignments as they do not display their output or linked input collections
	if (!FunctionCallNode->IsA<UNiagaraNodeAssignment>())
	{
		AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackModuleItem::FilterOutputCollection));
		AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackModuleItem::FilterLinkedInputCollection));
	}

	MessageLogGuid = GetSystemViewModel()->GetMessageLogGuid();

	FNiagaraMessageManager::Get()->SubscribeToAssetMessagesByObject(
		  FText::FromString("StackModuleItem")
		, MessageLogGuid
		, FObjectKey(FunctionCallNode)
		, MessageManagerRegistrationKey
	).BindUObject(this, &UNiagaraStackModuleItem::OnMessageManagerRefresh);
}

FText UNiagaraStackModuleItem::GetDisplayName() const
{
	if (DisplayNameCache.IsSet() == false)
	{
		DisplayNameCache = FunctionCallNode->GetNodeTitle(ENodeTitleType::ListView);
	}
	return DisplayNameCache.GetValue();
}

UObject* UNiagaraStackModuleItem::GetDisplayedObject() const
{
	return FunctionCallNode;
}

FText UNiagaraStackModuleItem::GetTooltipText() const
{
	if (FunctionCallNode != nullptr)
	{
		return FunctionCallNode->GetTooltipText();
	}
	else
	{
		return FText();
	}
}

bool UNiagaraStackModuleItem::UsesCustomTooltipWidget() const
{
	return GetDefault<UNiagaraEditorSettings>()->bDisplayPreviewMoviesInTooltips && FunctionCallNode != nullptr && FunctionCallNode->FunctionScript != nullptr;
}

TSharedPtr<IToolTip> UNiagaraStackModuleItem::GetCustomTooltipWidget() const
{
	using namespace FNiagaraEditorUtilities::Tooltips;
	if(FunctionCallNode != nullptr && FunctionCallNode->FunctionScript != nullptr)
	{
		if (TSharedPtr<FNiagaraSystemViewModel> SystemViewModelPtr = UNiagaraStackEntry::GetSystemViewModelPtr())
		{
			FAssetData AssetData(FunctionCallNode->FunctionScript);
			if (GetDefault<UNiagaraEditorSettings>()->bDisplayPreviewMoviesInTooltips && SystemViewModelPtr->GetPreviewMovieViewModel()->IsPlayingPreviewMovieForAsset(AssetData) == false)
			{
				SystemViewModelPtr->GetPreviewMovieViewModel()->PlayMovieForAsset(AssetData);
			}

			if (TooltipCache.IsValid() == false)
			{
				FCreateTooltipArguments CreateTooltipArguments;
				CreateTooltipArguments.bTryCreatePreviewMovie = GetDefault<UNiagaraEditorSettings>()->bDisplayPreviewMoviesInTooltips;
				CreateTooltipArguments.PreviewMovieOwner = GetSystemViewModel()->GetPreviewMovieViewModel();
				TooltipCache = CreateTooltipForScript(*FunctionCallNode->FunctionScript, FunctionCallNode->SelectedScriptVersion, CreateTooltipArguments);
			}

			return TooltipCache.ToSharedRef();
		}
	}
	
	return nullptr; 
}

INiagaraStackItemGroupAddUtilities* UNiagaraStackModuleItem::GetGroupAddUtilities()
{
	return GroupAddUtilities;
}

void UNiagaraStackModuleItem::FinalizeInternal()
{
	if (MessageManagerRegistrationKey.IsValid())
	{
		FNiagaraMessageManager::Get()->Unsubscribe(FText::FromString("StackModuleItem"), MessageLogGuid, MessageManagerRegistrationKey);
	}
	
	Super::FinalizeInternal();
}

void UNiagaraStackModuleItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	bCanRefresh = false;
	bCanMoveAndDeleteCache.Reset();
	bIsScratchModuleCache.Reset();
	DisplayNameCache.Reset();
	TooltipCache.Reset();

	if (FunctionCallNode != nullptr &&
		(FunctionCallNode->HasValidScriptAndGraph() || FunctionCallNode->Signature.IsValid()) &&
		FunctionCallNode->GetGraph() != nullptr && 
		FunctionCallNode->GetGraph()->Nodes.Contains(FunctionCallNode))
	{
		bool bDiscardInputRoot = LastRefreshVersionCache != FunctionCallNode->SelectedScriptVersion;
		LastRefreshVersionCache = FunctionCallNode->SelectedScriptVersion; 

		// Determine if meta-data requires that we add our own refresh button here.
		if (FunctionCallNode->HasValidScriptAndGraph())
		{
			bCanRefresh = true;
		}

		if(bDiscardInputRoot)
		{
			InputRoot = nullptr;
		}
		
		if(InputRoot == nullptr)
		{
			InputRoot = NewObject<UNiagaraStackScriptHierarchyRoot>(this);
			InputRoot->Initialize(CreateDefaultChildRequiredData(), *FunctionCallNode, *FunctionCallNode, GetStackEditorDataKey());
		}

		// NiagaraNodeAssignments should not display OutputCollection and LinkedInputCollection as they effectively handle this through their InputCollection 
		if (!FunctionCallNode->IsA<UNiagaraNodeAssignment>())
		{
			if (LinkedInputCollection == nullptr)
			{
				LinkedInputCollection = NewObject<UNiagaraStackModuleItemLinkedInputCollection>(this);
				LinkedInputCollection->Initialize(CreateDefaultChildRequiredData(), *FunctionCallNode);
				LinkedInputCollection->AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackModuleItem::FilterLinkedInputCollectionChild));
			}

			if (OutputCollection == nullptr)
			{
				OutputCollection = NewObject<UNiagaraStackModuleItemOutputCollection>(this);
				OutputCollection->Initialize(CreateDefaultChildRequiredData(), *FunctionCallNode);
				OutputCollection->AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackModuleItem::FilterOutputCollectionChild));
			}
			
			InputRoot->SetShouldDisplayLabel(GetStackEditorData().GetShowOutputs() || GetStackEditorData().GetShowLinkedInputs());
			NewChildren.Add(InputRoot);
			
			NewChildren.Add(LinkedInputCollection);
			NewChildren.Add(OutputCollection);
		
		}
		else
		{
			NewChildren.Add(InputRoot);

			UNiagaraNodeAssignment* AssignmentNode = CastChecked<UNiagaraNodeAssignment>(FunctionCallNode);
			if (AssignmentNode->GetAssignmentTargets().Num() == 0)
			{
				FText EmptyAssignmentNodeMessageText = LOCTEXT("EmptyAssignmentNodeMessage", "No Parameters\n\nTo add a parameter use the add button in the header, or drag a parameter from the parameters tab to the header.");
				UNiagaraStackItemTextContent* EmptyAssignmentNodeMessage = FindCurrentChildOfTypeByPredicate<UNiagaraStackItemTextContent>(NewChildren,
					[&](UNiagaraStackItemTextContent* CurrentStackItemTextContent) { return CurrentStackItemTextContent->GetDisplayName().IdenticalTo(EmptyAssignmentNodeMessageText); });

				if (EmptyAssignmentNodeMessage == nullptr)
				{
					EmptyAssignmentNodeMessage = NewObject<UNiagaraStackItemTextContent>(this);
					EmptyAssignmentNodeMessage->Initialize(CreateDefaultChildRequiredData(), EmptyAssignmentNodeMessageText, GetStackEditorDataKey());
				}
				NewChildren.Add(EmptyAssignmentNodeMessage);
			}
		}
		RefreshIsEnabled();
		RefreshIssues(NewIssues);
	}

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}


const UNiagaraStackModuleItem::FCollectedUsageData& UNiagaraStackModuleItem::GetCollectedUsageData() const
{
	if (CachedCollectedUsageData.IsSet() == false)
	{
		CachedCollectedUsageData = FCollectedUsageData();
		if (FunctionCallNode && FunctionCallNode->IsA<UNiagaraNodeAssignment>() && IsSystemViewModelValid())
		{
			UNiagaraNodeAssignment* AssignmentNode = CastChecked<UNiagaraNodeAssignment>(FunctionCallNode);
			TSharedRef<FNiagaraSystemViewModel> SystemVM = GetSystemViewModel();
			INiagaraParameterPanelViewModel* ParamVM = SystemVM->GetParameterPanelViewModel();

			CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite = false;
			CachedCollectedUsageData.GetValue().bHasReferencedParameterRead = false;

			if (ParamVM)
			{
				const TArray<FNiagaraVariable>& TargetVars = AssignmentNode->GetAssignmentTargets();
				for (const FNiagaraVariable& Var : TargetVars)
				{
					FNiagaraVariableBase TestVar = Var;
					bool bHandled = ParamVM->IsVariableSelected(TestVar);
					if (bHandled)
					{
						CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite = true;
					}
				}
			}
		}

		if (LinkedInputCollection)
		{
			if (LinkedInputCollection->GetCollectedUsageData().bHasReferencedParameterRead)
				CachedCollectedUsageData.GetValue().bHasReferencedParameterRead = true;
			if (LinkedInputCollection->GetCollectedUsageData().bHasReferencedParameterWrite)
				CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite = true;
		}

		if(InputRoot)
		{
			if(InputRoot->GetCollectedUsageData().bHasReferencedParameterRead)
			{
				CachedCollectedUsageData.GetValue().bHasReferencedParameterRead = true;
			}
			if (InputRoot->GetCollectedUsageData().bHasReferencedParameterWrite)
			{
				CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite = true;
			}
		}

		if (OutputCollection)
		{
			if (OutputCollection->GetCollectedUsageData().bHasReferencedParameterRead)
				CachedCollectedUsageData.GetValue().bHasReferencedParameterRead = true;
			if (OutputCollection->GetCollectedUsageData().bHasReferencedParameterWrite)
				CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite = true;
		}
		
	}

	return CachedCollectedUsageData.GetValue();
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackModuleItem::CanDropInternal(const FDropRequest& DropRequest)
{
	if ((DropRequest.DropOptions != UNiagaraStackEntry::EDropOptions::Overview || DropRequest.DropZone == EItemDropZone::OntoItem) &&
		DropRequest.DragDropOperation->IsOfType<FNiagaraParameterDragOperation>() &&
		FunctionCallNode->IsA<UNiagaraNodeAssignment>())
	{
		TSharedRef<const FNiagaraParameterDragOperation> ParameterDragDropOp = StaticCastSharedRef<const FNiagaraParameterDragOperation>(DropRequest.DragDropOperation);
		UNiagaraNodeAssignment* AssignmentNode = CastChecked<UNiagaraNodeAssignment>(FunctionCallNode);
		TSharedPtr<const FNiagaraParameterAction> ParameterAction = StaticCastSharedPtr<const FNiagaraParameterAction>(ParameterDragDropOp->GetSourceAction());
		if (ParameterAction.IsValid())
		{
			if (AssignmentNode->GetAssignmentTargets().Contains(ParameterAction->GetParameter()))
			{
				return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantDropDuplicateParameter", "Can not drop this parameter here because it's already set by this module."));
			}
			else if (FNiagaraStackGraphUtilities::CanWriteParameterFromUsageViaOutput(ParameterAction->GetParameter(), OutputNode) == false)
			{
				return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantDropParameterByUsage", "Can not drop this parameter here because it can't be written in this usage context."));
			}
			else
			{
				return FDropRequestResponse(EItemDropZone::OntoItem, LOCTEXT("DropParameterToAdd", "Add this parameter to this 'Set Parameters' node."));
			}
		}
	}
	return TOptional<FDropRequestResponse>();
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackModuleItem::DropInternal(const FDropRequest& DropRequest)
{
	if ((DropRequest.DropOptions != UNiagaraStackEntry::EDropOptions::Overview || DropRequest.DropZone == EItemDropZone::OntoItem) &&
		DropRequest.DragDropOperation->IsOfType<FNiagaraParameterDragOperation>() &&
		FunctionCallNode->IsA<UNiagaraNodeAssignment>())
	{
		TSharedRef<const FNiagaraParameterDragOperation> ParameterDragDropOp = StaticCastSharedRef<const FNiagaraParameterDragOperation>(DropRequest.DragDropOperation);
		UNiagaraNodeAssignment* AssignmentNode = CastChecked<UNiagaraNodeAssignment>(FunctionCallNode);
		TSharedPtr<const FNiagaraParameterAction> ParameterAction = StaticCastSharedPtr<const FNiagaraParameterAction>(ParameterDragDropOp->GetSourceAction());
		if (ParameterAction.IsValid() && 
			AssignmentNode->GetAssignmentTargets().Contains(ParameterAction->GetParameter()) == false &&
			FNiagaraStackGraphUtilities::CanWriteParameterFromUsage(ParameterAction->GetParameter(), OutputNode->GetUsage()))
		{
			AddInput(ParameterAction->GetParameter());
			return FDropRequestResponse(DropRequest.DropZone);
		}
	}
	return TOptional<FDropRequestResponse>();
}

namespace NiagaraStackModuleItemIssues {
	int32 GetIndexOfLastDependentModuleData(
		const TArray<FNiagaraStackModuleData>& StackModuleData,
		int32 StartIndex,
		int32 EndIndex,
		int32 NextIndexOffset,
		ENiagaraScriptUsage DependentScriptUsage,
		FGuid DependentScriptUsageId,
		FNiagaraModuleDependency& RequiredDependency)
	{
		int LastModuleDataIndexRequiringDependency = INDEX_NONE;
		for (int32 StackModuleDataIndex = StartIndex; StackModuleDataIndex != EndIndex; StackModuleDataIndex += NextIndexOffset)
		{
			const FNiagaraStackModuleData& CurrentStackModuleData = StackModuleData[StackModuleDataIndex];
			if (RequiredDependency.ScriptConstraint == ENiagaraModuleDependencyScriptConstraint::SameScript &&
				(UNiagaraScript::IsEquivalentUsage(CurrentStackModuleData.Usage, DependentScriptUsage) == false || CurrentStackModuleData.UsageId != DependentScriptUsageId))
			{
				break;
			}

			UNiagaraNodeFunctionCall* ModuleNode = CurrentStackModuleData.WeakModuleNode.Get();
			if (ModuleNode && ModuleNode->GetScriptData() != nullptr &&
				ModuleNode->GetScriptData()->RequiredDependencies.ContainsByPredicate(
					[&RequiredDependency](const FNiagaraModuleDependency& CurrentRequiredDependency) { return CurrentRequiredDependency.Id == RequiredDependency.Id; }))
			{
				LastModuleDataIndexRequiringDependency = StackModuleDataIndex;
			}
		}
		return LastModuleDataIndexRequiringDependency;
	}

	UNiagaraNodeOutput* GetCompatibleTargetOutputNodeFromOrderedScripts(const TArray<ENiagaraScriptUsage>& CompatibleUsages, const TArray<UNiagaraScript*>& OrderedScripts, int32 DependentScriptIndex, int32 LastScriptIndex, int32 NextScriptOffset)
	{
		for (int32 CurrentScriptIndex = DependentScriptIndex + NextScriptOffset; CurrentScriptIndex != LastScriptIndex; CurrentScriptIndex += NextScriptOffset)
		{
			UNiagaraScript* CurrentScript = OrderedScripts[CurrentScriptIndex];
			if (UNiagaraScript::ContainsEquivilentUsage(CompatibleUsages, CurrentScript->GetUsage()))
			{
				UNiagaraScriptSource* CurrentScriptSource = CastChecked<UNiagaraScriptSource>(CurrentScript->GetLatestSource());
				UNiagaraNodeOutput* CurrentOutputNode = CurrentScriptSource->NodeGraph->FindEquivalentOutputNode(CurrentScript->GetUsage(), CurrentScript->GetUsageId());
				if (CurrentOutputNode != nullptr)
				{
					return CurrentOutputNode;
				}
			}
		}
		return nullptr;
	}

	void GetCompatibleOutputNodeAndIndex(TSharedRef<FNiagaraSystemViewModel> DependentSystemViewModel, FGuid EmitterHandleId, UNiagaraScript& DependencyProviderScript, const FNiagaraStackModuleData& LastDependentModuleData, FNiagaraModuleDependency RequiredDependency, UNiagaraNodeOutput*& OutTargetOutputNode, TOptional<int32>& OutTargetIndex)
	{
		TArray<ENiagaraScriptUsage> CompatibleUsages = UNiagaraScript::GetSupportedUsageContextsForBitmask(DependencyProviderScript.GetLatestScriptData()->ModuleUsageBitmask);
		if (UNiagaraScript::ContainsEquivilentUsage(CompatibleUsages, LastDependentModuleData.Usage))
		{
			// If the dependency provider is compatible with the last dependent usage it can be added directly before or after the last dependent.
			if ( UNiagaraNodeFunctionCall* ModuleNode = LastDependentModuleData.WeakModuleNode.Get() )
			{
				OutTargetOutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*ModuleNode);
				OutTargetIndex = LastDependentModuleData.Index + (RequiredDependency.Type == ENiagaraModuleDependencyType::PostDependency ? 1 : 0);
			}
		}
		else
		{
			// Otherwise we need to search for a compatible script to insert the module into.
			// We always use INDEX_NONE as target index (add to end), because PreDependency and PostDependency across stages doesn't mean much
			// If a module in ParticleSpawn requires a module in ParticleUpdate, only other ParticleUpdate scripts are relevant for its placement
			TArray<UNiagaraScript*> OrderedScripts;
			DependentSystemViewModel->GetOrderedScriptsForEmitterHandleId(EmitterHandleId, OrderedScripts);
			int32 DependentScriptIndex = OrderedScripts.IndexOfByPredicate([&LastDependentModuleData](UNiagaraScript* Script)
				{ return UNiagaraScript::IsEquivalentUsage(Script->GetUsage(), LastDependentModuleData.Usage) && Script->GetUsageId() == LastDependentModuleData.UsageId; });

			if (DependentScriptIndex != INDEX_NONE)
			{
				if (RequiredDependency.Type == ENiagaraModuleDependencyType::PreDependency)
				{
					UNiagaraNodeOutput* CompatibleOutputNode = GetCompatibleTargetOutputNodeFromOrderedScripts(CompatibleUsages, OrderedScripts, DependentScriptIndex, -1, -1);
					if (CompatibleOutputNode != nullptr)
					{
						OutTargetOutputNode = CompatibleOutputNode;
						OutTargetIndex = INDEX_NONE;
					}
				}
				else if (RequiredDependency.Type == ENiagaraModuleDependencyType::PostDependency)
				{
					UNiagaraNodeOutput* CompatibleOutputNode = GetCompatibleTargetOutputNodeFromOrderedScripts(CompatibleUsages, OrderedScripts, DependentScriptIndex, OrderedScripts.Num(), 1);
					if (CompatibleOutputNode != nullptr)
					{
						OutTargetOutputNode = CompatibleOutputNode;
						OutTargetIndex = INDEX_NONE;
					}
				}
			}
		}
	}

	void AddModuleToFixDependencyIssue(
		TWeakPtr<FNiagaraSystemViewModel> DependentSystemViewModelWeak,
		FGuid DependentEmitterHandleId,
		TWeakObjectPtr<UNiagaraNodeFunctionCall> DependentModuleNodeWeak,
		FNiagaraModuleDependency RequiredDependency,
		FAssetData DependencyProviderModuleAsset)
	{
		TSharedPtr<FNiagaraSystemViewModel> DependentSystemViewModel = DependentSystemViewModelWeak.Pin();
		UNiagaraNodeFunctionCall* DependentModuleNode = DependentModuleNodeWeak.Get();
		if (DependentSystemViewModel.IsValid() == false || DependentModuleNode == nullptr)
		{
			FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("AddDependencyModuleFailedInvalidSourceData", "Failed to add a dependency module because the fix source data was no longer valid."));
			return;
		}

		UNiagaraNodeOutput* SourceOutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*DependentModuleNode);
		if (SourceOutputNode == nullptr)
		{
			FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("AddDependencyModuleFailedInvalidSourceOutputData", "Failed to add a dependency module because the fix source data was no longer valid."));
			return;
		}

		UNiagaraScript* DependencyProviderScript = Cast<UNiagaraScript>(DependencyProviderModuleAsset.GetAsset());
		if (DependencyProviderScript == nullptr)
		{
			FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("AddDependencyModuleFailedInvalidScriptData", "Failed to add a dependency module because the script asset was not valid."));
			return;
		}

		const TArray<FNiagaraStackModuleData>& StackModuleData = DependentSystemViewModel->GetStackModuleDataByEmitterHandleId(DependentEmitterHandleId);
		int32 DependentModuleIndex = StackModuleData.IndexOfByPredicate([DependentModuleNode](const FNiagaraStackModuleData& StackModuleDataItem) { return StackModuleDataItem.WeakModuleNode.Get() == DependentModuleNode; });
		
		UNiagaraNodeOutput* TargetOutputNode = nullptr;
		TOptional<int32> TargetIndex;

		int32 LastDependentModuleDataIndex = INDEX_NONE;
		if (RequiredDependency.Type == ENiagaraModuleDependencyType::PreDependency)
		{
			LastDependentModuleDataIndex = GetIndexOfLastDependentModuleData(StackModuleData, DependentModuleIndex, -1, -1,
				SourceOutputNode->GetUsage(), SourceOutputNode->GetUsageId(), RequiredDependency);
		}
		else if (RequiredDependency.Type == ENiagaraModuleDependencyType::PostDependency)
		{
			LastDependentModuleDataIndex = GetIndexOfLastDependentModuleData(StackModuleData, DependentModuleIndex, StackModuleData.Num(), 1,
				SourceOutputNode->GetUsage(), SourceOutputNode->GetUsageId(), RequiredDependency);
		}

		if (LastDependentModuleDataIndex != INDEX_NONE)
		{
			GetCompatibleOutputNodeAndIndex(DependentSystemViewModel.ToSharedRef(), DependentEmitterHandleId, *DependencyProviderScript,
				StackModuleData[LastDependentModuleDataIndex], RequiredDependency, TargetOutputNode, TargetIndex);
		}

		if (TargetOutputNode == nullptr || TargetIndex.IsSet() == false)
		{
			FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("AddDependencyModuleFailedNoValidLocation", "Failed to add a dependency module because an acceptable location could not be found."));
			return;
		}

		FScopedTransaction ScopedTransaction(LOCTEXT("AddDependencyFixTransaction", "Add a module to fix a dependency"));
		FNiagaraStackGraphUtilities::FAddScriptModuleToStackArgs AddScriptModuleToStackArgs(DependencyProviderScript, *TargetOutputNode);
		AddScriptModuleToStackArgs.TargetIndex = TargetIndex.GetValue();
		AddScriptModuleToStackArgs.bFixupTargetIndex = true;
		FNiagaraStackGraphUtilities::AddScriptModuleToStack(AddScriptModuleToStackArgs);
	}

	void GenerateFixesForAddingDependencyProviders(
		TSharedRef<FNiagaraSystemViewModel> DependentSystemViewModel,
		FGuid DependentEmitterHandleId,
		UNiagaraNodeFunctionCall& DependentModuleNode,
		ENiagaraScriptUsage DependentUsage,
		const FNiagaraModuleDependency& RequiredDependency,
		TArray<UNiagaraStackEntry::FStackIssueFix>& OutFixes)
	{
		TOptional<ENiagaraScriptUsage> RequiredUsage = RequiredDependency.ScriptConstraint == ENiagaraModuleDependencyScriptConstraint::SameScript
			? DependentUsage
			: TOptional<ENiagaraScriptUsage>();
		TArray<FAssetData> ModuleAssetsForDependency;
		FNiagaraStackGraphUtilities::DependencyUtilities::GetModuleScriptAssetsByDependencyProvided(RequiredDependency.Id, RequiredUsage, ModuleAssetsForDependency);

		// Gather duplicate module names so their fixes can be disambiguated.
		TSet<FName> ModuleNames;
		TSet<FName> DuplicateModuleNames;
		for (FAssetData ModuleAsset : ModuleAssetsForDependency)
		{
			bool bIsDuplicate;
			ModuleNames.Add(ModuleAsset.AssetName, &bIsDuplicate);
			if (bIsDuplicate)
			{
				DuplicateModuleNames.Add(ModuleAsset.AssetName);
			}
		}

		for (const FAssetData& ModuleAsset : ModuleAssetsForDependency)
		{
			FText DependencyAssetDisplayName = DuplicateModuleNames.Contains(ModuleAsset.AssetName) ? FText::FromName(ModuleAsset.PackageName) : FText::FromName(ModuleAsset.AssetName);
			FText FixDescription = FText::Format(LOCTEXT("AddDependencyFixDescription", "Add new dependency module {0}"), DependencyAssetDisplayName);
			OutFixes.Add(UNiagaraStackEntry::FStackIssueFix(FixDescription, UNiagaraStackEntry::FStackIssueFixDelegate::CreateStatic(&AddModuleToFixDependencyIssue, 
				TWeakPtr<FNiagaraSystemViewModel>(DependentSystemViewModel),
				DependentEmitterHandleId,
				TWeakObjectPtr<UNiagaraNodeFunctionCall>(&DependentModuleNode),
				RequiredDependency,
				ModuleAsset)));
		}
	}

	void MoveModuleToFixDependencyIssue(
		TWeakObjectPtr<UNiagaraSystem> DependentSystemWeak,
		FGuid DependentEmitterHandleId,
		TWeakObjectPtr<UNiagaraScript> DependentScriptWeak,
		TWeakObjectPtr<UNiagaraNodeFunctionCall> DependentModuleWeak,
		ENiagaraScriptUsage TargetUsage,
		FGuid TargetUsageId,
		int32 TargetMoveIndex)
	{
		UNiagaraSystem* TargetSystem = DependentSystemWeak.Get();
		if (TargetSystem == nullptr)
		{
			FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("MoveDependentModuleFailedInvalidSystem", "Failed to move a dependent module because the owning system was no longer valid."));
			return;
		}

		UNiagaraScript* SourceScript = DependentScriptWeak.Get();
		if (SourceScript == nullptr)
		{
			FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("MoveDependentModuleFailedInvalidScript", "Failed to move a dependent module because the owning script was no longer valid."));
			return;
		}

		UNiagaraNodeFunctionCall* ModuleToMove = DependentModuleWeak.Get();
		if (ModuleToMove == nullptr)
		{
			FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("MoveDependentModuleFailedInvalidModule", "Failed to move a dependent module because the module to move was no longer valid."));
			return;
		}

		FScopedTransaction ScopedTransaction(LOCTEXT("MoveDependentFixTransaction", "Move a dependent module to fix a dependency"));
		UNiagaraNodeFunctionCall* MovedNode;
		FNiagaraStackGraphUtilities::MoveModule(*SourceScript, *ModuleToMove, *TargetSystem, DependentEmitterHandleId, TargetUsage, TargetUsageId, TargetMoveIndex, false, MovedNode);
	}
	
	ENiagaraModuleDependencyUsage ConvertScriptUsageToDependencyUsage(ENiagaraScriptUsage ScriptUsage)
	{
		if (ScriptUsage == ENiagaraScriptUsage::ParticleEventScript)
		{
			return ENiagaraModuleDependencyUsage::Event;
		}
		if (ScriptUsage == ENiagaraScriptUsage::ParticleSimulationStageScript)
		{
			return ENiagaraModuleDependencyUsage::SimulationStage;
		}
		if (ScriptUsage == ENiagaraScriptUsage::EmitterSpawnScript || ScriptUsage == ENiagaraScriptUsage::SystemSpawnScript || ScriptUsage == ENiagaraScriptUsage::ParticleSpawnScriptInterpolated || ScriptUsage == ENiagaraScriptUsage::ParticleSpawnScript)
		{
			return ENiagaraModuleDependencyUsage::Spawn;
		}
		if (ScriptUsage == ENiagaraScriptUsage::EmitterUpdateScript || ScriptUsage == ENiagaraScriptUsage::SystemUpdateScript || ScriptUsage == ENiagaraScriptUsage::ParticleUpdateScript)
		{
			return ENiagaraModuleDependencyUsage::Update;
		}
		return ENiagaraModuleDependencyUsage::None;
	}

	bool IsUsageAllowed(ENiagaraScriptUsage ModuleUsage, int32 AllowedUsageBitmask)
	{
		ENiagaraModuleDependencyUsage Usage = ConvertScriptUsageToDependencyUsage(ModuleUsage);
		return AllowedUsageBitmask & (1 << static_cast<int32>(Usage));
	}

	void GenerateFixesForReorderingModules(
		TSharedRef<FNiagaraSystemViewModel> DependentSystemViewModel,
		FGuid DependentEmitterHandleId,
		UNiagaraScript& DependentScript,
		UNiagaraNodeFunctionCall& DependentModuleNode,
		FNiagaraModuleDependency RequiredDependency,
		const TArray<FNiagaraStackModuleData>& StackModuleData,
		const TArray<int32>& WrongOrderDependencyProviderIndices,
		TArray<UNiagaraStackEntry::FStackIssueFix>& OutFixes)
	{
		for (int32 WrongOrderDependencyProviderIndex : WrongOrderDependencyProviderIndices)
		{
			const FNiagaraStackModuleData& CurrentStackModuleData = StackModuleData[WrongOrderDependencyProviderIndex];

			FText LocationText;
			int32 TargetIndex = INDEX_NONE;
			if (RequiredDependency.Type == ENiagaraModuleDependencyType::PreDependency)
			{
				LocationText = LOCTEXT("MoveLocationAfter", "After");
				TargetIndex = CurrentStackModuleData.Index + 1;
			}
			else if (RequiredDependency.Type == ENiagaraModuleDependencyType::PostDependency)
			{
				LocationText = LOCTEXT("MoveLocationBefore", "Before");
				TargetIndex = CurrentStackModuleData.Index;
			}

			if (TargetIndex != INDEX_NONE)
			{
				UNiagaraNodeFunctionCall* ModuleNode = CurrentStackModuleData.WeakModuleNode.Get();
				if (ensure(ModuleNode))
				{
					FText FixDescription = FText::Format(LOCTEXT("MoveDependentFixDescriptionFormat", "Move module {0} {1} {2}"), FText::FromString(DependentModuleNode.GetFunctionName()), LocationText, FText::FromString(ModuleNode->GetFunctionName()));
					OutFixes.Add(UNiagaraStackEntry::FStackIssueFix(FixDescription, UNiagaraStackEntry::FStackIssueFixDelegate::CreateStatic(&MoveModuleToFixDependencyIssue,
						TWeakObjectPtr<UNiagaraSystem>(&DependentSystemViewModel->GetSystem()), DependentEmitterHandleId, TWeakObjectPtr<UNiagaraScript>(&DependentScript), TWeakObjectPtr<UNiagaraNodeFunctionCall>(&DependentModuleNode),
						CurrentStackModuleData.Usage, CurrentStackModuleData.UsageId, TargetIndex)));
				}
			}
		}
	}

	UNiagaraStackModuleItem* FindStackModuleItem(TSharedRef<FNiagaraSystemViewModel> SourceSystemViewModel, const FGuid& SourceEmitterHandleId, const UNiagaraNodeFunctionCall* ModuleNode)
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = SourceSystemViewModel->GetEmitterHandleViewModelById(SourceEmitterHandleId);
		if (ModuleNode == nullptr || !EmitterHandleViewModel.IsValid())
		{
			return nullptr;
		}
		UNiagaraStackViewModel* StackViewModel = EmitterHandleViewModel->GetEmitterStackViewModel();
		
		TArray<UNiagaraStackEntry*> EntriesToCheck;
		if (UNiagaraStackEntry* RootEntry = StackViewModel->GetRootEntry())
		{
			RootEntry->GetUnfilteredChildren(EntriesToCheck);
			while (EntriesToCheck.Num() > 0)
			{
				UNiagaraStackEntry* Entry = EntriesToCheck.Pop();
				if (UNiagaraStackModuleItem* ItemToCheck = Cast<UNiagaraStackModuleItem>(Entry))
				{
					if (&ItemToCheck->GetModuleNode() == ModuleNode)
					{
						return ItemToCheck;
					}
				}
				Entry->GetUnfilteredChildren(EntriesToCheck);
			}
		}
		return nullptr;
	}

	UNiagaraStackEntry::FStackIssueFix GetSwitchVersionFix(const FNiagaraAssetVersion& FromVersion, const FNiagaraAssetVersion& ToVersion, UNiagaraNodeFunctionCall* ModuleNode, TSharedRef<FNiagaraSystemViewModel> SourceSystemViewModel, const FGuid& SourceEmitterHandleId)
	{
		FText FixDescription = FText::Format(LOCTEXT("SwitchVersionFixDescriptionFormat", "Switch {0} module version from {1}.{2} to {3}.{4}"), FText::FromString(ModuleNode->GetFunctionName()), FText::AsNumber(FromVersion.MajorVersion), FText::AsNumber(FromVersion.MinorVersion), FText::AsNumber(ToVersion.MajorVersion), FText::AsNumber(ToVersion.MinorVersion));
		TWeakPtr<FNiagaraSystemViewModel> SystemViewModelPtr = SourceSystemViewModel->AsWeak();
		TWeakObjectPtr<UNiagaraNodeFunctionCall> ModuleNodePtr = ModuleNode;
		return UNiagaraStackEntry::FStackIssueFix(FixDescription, UNiagaraStackEntry::FStackIssueFixDelegate::CreateLambda([SystemViewModelPtr, SourceEmitterHandleId, ModuleNodePtr, ToVersion]()
		{
			if (!SystemViewModelPtr.IsValid() || !ModuleNodePtr.IsValid())
			{
				return;
			}
			if(UNiagaraStackModuleItem* StackModule = FindStackModuleItem(SystemViewModelPtr.Pin().ToSharedRef(), SourceEmitterHandleId, ModuleNodePtr.Get()))
			{
				StackModule->ChangeScriptVersion(ToVersion.VersionGuid);
			}		
		}));
	}

	void GenerateFixesForSwitchingModuleVersions(TSharedRef<FNiagaraSystemViewModel> SourceSystemViewModel, const FGuid& SourceEmitterHandleId, const TArray<FNiagaraStackModuleData>& StackModuleData, const FNiagaraModuleDependency& SourceRequiredDependency, const TArray<int32>& WrongVersionDependencyProviderIndices, TArray<UNiagaraStackEntry::FStackIssueFix>& NewIssues)
	{
		for (int32 WrongVersionDependencyProviderIndex : WrongVersionDependencyProviderIndices)
		{
			const FNiagaraStackModuleData& CurrentStackModuleData = StackModuleData[WrongVersionDependencyProviderIndex];
			UNiagaraNodeFunctionCall* ModuleNode = CurrentStackModuleData.WeakModuleNode.Get();
			
			if (FindStackModuleItem(SourceSystemViewModel, SourceEmitterHandleId, ModuleNode) == nullptr)
			{
				continue;
			}

			if (UNiagaraScript* NiagaraScript = ModuleNode->FunctionScript)
			{
				FNiagaraAssetVersion CurrentVersion = ModuleNode->GetScriptData()->Version;
			
				// check the exposed version first
				FNiagaraAssetVersion ExposedVersion = NiagaraScript->GetExposedVersion();
				if (SourceRequiredDependency.IsVersionAllowed(ExposedVersion))
				{
					NewIssues.Add(GetSwitchVersionFix(CurrentVersion, ExposedVersion, ModuleNode, SourceSystemViewModel, SourceEmitterHandleId));
					continue;
				}

				// no match, walk from highest to lowest version
				TArray<FNiagaraAssetVersion> AvailableVersions = NiagaraScript->GetAllAvailableVersions();
				for (int32 i = AvailableVersions.Num() - 1; i >= 0; i--)
				{
					FNiagaraAssetVersion Version = AvailableVersions[i];
					if (SourceRequiredDependency.IsVersionAllowed(Version))
					{
						NewIssues.Add(GetSwitchVersionFix(CurrentVersion, Version, ModuleNode, SourceSystemViewModel, SourceEmitterHandleId));
						break;
					}
				}
			}
		}
	}

	void EnableModuleToFixDependencyIssue(TWeakObjectPtr<UNiagaraNodeFunctionCall> DependencyProviderModuleNodeWeak)
	{
		UNiagaraNodeFunctionCall* ModuleToEnable = DependencyProviderModuleNodeWeak.Get();
		if (ModuleToEnable == nullptr)
		{
			FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("EnableDependencyProviderModuleFailedInvalidNode", "Failed to a dependency providing module because it was no longer valid."));
			return;
		}

		FScopedTransaction ScopedTransaction(LOCTEXT("EnableModuleFixTransaction", "Enable a dependency providing module to fix a dependency"));
		FNiagaraStackGraphUtilities::SetModuleIsEnabled(*ModuleToEnable, true);
	}

	void GenerateFixesForEnablingModules(
		const TArray<FNiagaraStackModuleData>& StackModuleData,
		const TArray<int32>& DisabledDependencyProviderIndices,
		TArray<UNiagaraStackEntry::FStackIssueFix>& OutFixes)
	{
		for (int32 DisabledDependencyProviderIndex : DisabledDependencyProviderIndices)
		{
			const FNiagaraStackModuleData& CurrentStackModuleData = StackModuleData[DisabledDependencyProviderIndex];
			UNiagaraNodeFunctionCall* ModuleNode = CurrentStackModuleData.WeakModuleNode.Get();
			if (ensure(ModuleNode))
			{
				FText FixDescription = FText::Format(LOCTEXT("EnableDependencyProviderFixDescriptionFormat", "Enable module {0} which provides the dependency."),
					FText::FromString(ModuleNode->GetFunctionName()));
				OutFixes.Add(UNiagaraStackEntry::FStackIssueFix(FixDescription, UNiagaraStackEntry::FStackIssueFixDelegate::CreateStatic(&EnableModuleToFixDependencyIssue,
					TWeakObjectPtr<UNiagaraNodeFunctionCall>(ModuleNode))));
			}
		}
	}

	bool IsCorrectDependencyVersion(const FNiagaraStackModuleData& StackModuleData, const FNiagaraModuleDependency& SourceModuleRequiredDependency)
	{
		UNiagaraNodeFunctionCall* ModuleNode = StackModuleData.WeakModuleNode.Get();
		if (ensure(ModuleNode))
		{
			FVersionedNiagaraScriptData* ScriptData = ModuleNode->FunctionScript->GetScriptData(ModuleNode->SelectedScriptVersion);
			return SourceModuleRequiredDependency.IsVersionAllowed(ScriptData->Version);
		}
		return false;
	}

	void GenerateDependencyIssues(
		TSharedRef<FNiagaraSystemViewModel> SourceSystemViewModel,
		FGuid SourceEmitterHandleId,
		UNiagaraScript& SourceScript,
		UNiagaraNodeFunctionCall& SourceModuleNode,
		FString SourceStackEditorDataKey,
		const UNiagaraNodeOutput& SourceOutputNode,
		const TArray<FNiagaraStackModuleData>& SourceStackModuleData,
		TArray<UNiagaraStackEntry::FStackIssue>& NewIssues)
	{
		if (SourceModuleNode.GetScriptData() == nullptr || SourceModuleNode.GetScriptData()->RequiredDependencies.Num() == 0)
		{
			return;
		}

		FVersionedNiagaraScriptData* ScriptData = SourceModuleNode.FunctionScript->GetScriptData(SourceModuleNode.SelectedScriptVersion);
		int32 ModuleIndex = SourceStackModuleData.IndexOfByPredicate([&SourceModuleNode](const FNiagaraStackModuleData& ModuleData) { return ModuleData.WeakModuleNode.Get() == &SourceModuleNode; });
		if (ensureMsgf(ModuleIndex != INDEX_NONE, TEXT("In system %s, module %s (%s) did not exist in the stack module data."),
			*SourceSystemViewModel->GetSystem().GetPathName(), *SourceModuleNode.GetFunctionName(), *SourceModuleNode.GetName()))
		{
			for (const FNiagaraModuleDependency& SourceRequiredDependency : SourceModuleNode.GetScriptData()->RequiredDependencies)
			{
				if (!IsUsageAllowed(SourceOutputNode.GetUsage(), SourceRequiredDependency.OnlyEvaluateInScriptUsage))
				{
					continue;
				}
				TArray<int32> DependencyProviderIndices;
				for (int32 StackModuleDataIndex = 0; StackModuleDataIndex < SourceStackModuleData.Num(); StackModuleDataIndex++)
				{
					if (FNiagaraStackGraphUtilities::DependencyUtilities::DoesStackModuleProvideDependency(SourceStackModuleData[StackModuleDataIndex], SourceRequiredDependency, SourceOutputNode))
					{
						DependencyProviderIndices.Add(StackModuleDataIndex);
					}
				}

				// Validate that dependency providers are enabled and in the correct direction.
				bool bDependencyProviderFound = false;
				TArray<int32> WrongOrderDependencyProviderIndices;
				TArray<int32> WrongVersionDependencyProviderIndices;
				TArray<int32> DisabledDependencyProviderIndices;
				TArray<ENiagaraScriptUsage> SupportedUsages = UNiagaraScript::GetSupportedUsageContextsForBitmask(ScriptData->ModuleUsageBitmask);
				for (int32 DependencyProviderIndex : DependencyProviderIndices)
				{
					const FNiagaraStackModuleData& DependencyProviderData = SourceStackModuleData[DependencyProviderIndex];
					UNiagaraNodeFunctionCall* ModuleNode = DependencyProviderData.WeakModuleNode.Get();
					bool bCorrectOrder =
						(SourceRequiredDependency.Type == ENiagaraModuleDependencyType::PreDependency && DependencyProviderIndex < ModuleIndex) ||
						(SourceRequiredDependency.Type == ENiagaraModuleDependencyType::PostDependency && DependencyProviderIndex > ModuleIndex);
					bool bCorrectVersion = IsCorrectDependencyVersion(DependencyProviderData, SourceRequiredDependency);
					bool bEnabled = ModuleNode && ModuleNode->GetDesiredEnabledState() == ENodeEnabledState::Enabled;
					bool bUsageIsSupported = UNiagaraScript::ContainsEquivilentUsage(SupportedUsages, DependencyProviderData.Usage);

					if (bEnabled && bCorrectOrder && bCorrectVersion)
					{
						bDependencyProviderFound = true;
						break;
					}
					
					if (bCorrectOrder == false)
					{
						// We can only reorder a module if it supports being moved to the usage of the target module.
						if (bUsageIsSupported)
						{
							WrongOrderDependencyProviderIndices.Add(DependencyProviderIndex);
						}
					}
					else if (bEnabled == false)
					{
						DisabledDependencyProviderIndices.Add(DependencyProviderIndex);
					}
					else if (bCorrectVersion == false)
					{
						WrongVersionDependencyProviderIndices.Add(DependencyProviderIndex);
					}
				}

				if (bDependencyProviderFound == false)
				{
					TArray<UNiagaraStackEntry::FStackIssueFix> Fixes;
					if (WrongOrderDependencyProviderIndices.Num() == 0 && DisabledDependencyProviderIndices.Num() == 0 && WrongVersionDependencyProviderIndices.Num() == 0)
					{
						// No valid dependency providers found so add fixes for new providers to add.
						GenerateFixesForAddingDependencyProviders(SourceSystemViewModel, SourceEmitterHandleId, SourceModuleNode, SourceOutputNode.GetUsage(), SourceRequiredDependency, Fixes);
					}
					else
					{
						if (WrongOrderDependencyProviderIndices.Num() > 0)
						{
							GenerateFixesForReorderingModules(SourceSystemViewModel, SourceEmitterHandleId, SourceScript, SourceModuleNode, SourceRequiredDependency, SourceStackModuleData, WrongOrderDependencyProviderIndices, Fixes);
						}
						if (DisabledDependencyProviderIndices.Num() > 0)
						{
							GenerateFixesForEnablingModules(SourceStackModuleData, DisabledDependencyProviderIndices, Fixes);
						}
						if (WrongVersionDependencyProviderIndices.Num() > 0)
						{
							GenerateFixesForSwitchingModuleVersions(SourceSystemViewModel, SourceEmitterHandleId, SourceStackModuleData, SourceRequiredDependency, WrongVersionDependencyProviderIndices, Fixes);
						}
					}

					FText DependencyTypeString = SourceRequiredDependency.Type == ENiagaraModuleDependencyType::PreDependency ? LOCTEXT("PreDependency", "pre-dependency") : LOCTEXT("PostDependency", "post-dependency");
					NewIssues.Add(UNiagaraStackEntry::FStackIssue(
						EStackIssueSeverity::Error,
						LOCTEXT("DependencyWarning", "The module has unmet dependencies."),
						FText::Format(LOCTEXT("DependencyWarningLong", "The following {0} is not met: {1}; {2}"), DependencyTypeString, FText::FromName(SourceRequiredDependency.Id), SourceRequiredDependency.Description),
						FString::Printf(TEXT("%s-dependency-%s"), *SourceStackEditorDataKey, *SourceRequiredDependency.Id.ToString()),
						true,
						Fixes));
				}
			}
		}
	}
}

UNiagaraStackEntry::FStackIssueFixDelegate UNiagaraStackModuleItem::GetUpgradeVersionFix()
{
	if (!CanMoveAndDelete())
	{
		return FStackIssueFixDelegate();
	}
	return FStackIssueFixDelegate::CreateLambda([WeakModule=MakeWeakObjectPtr(this)]()
    {
	    UNiagaraStackModuleItem* Item = WeakModule.Get();
		if (!Item)
		{
			return;
		}
		
	    FScopedTransaction ScopedTransaction(LOCTEXT("UpgradeVersionFix", "Change module version"));
		TStrongObjectPtr ModuleItem(Item); // gc lock
        FNiagaraScriptVersionUpgradeContext UpgradeContext;
		UpgradeContext.CreateClipboardCallback = [ModuleItem](UNiagaraClipboardContent* ClipboardContent)
	    {
	        ModuleItem->RefreshChildren();
	        ModuleItem->Copy(ClipboardContent);
	        if (ClipboardContent->Functions.Num() > 0)
	        {
	            ClipboardContent->FunctionInputs = ClipboardContent->Functions[0]->Inputs;
	            ClipboardContent->Functions.Empty();
	        }
	    };
        UpgradeContext.ApplyClipboardCallback = [ModuleItem](UNiagaraClipboardContent* ClipboardContent, FText& OutWarning) { ModuleItem->Paste(ClipboardContent, OutWarning); };
		UpgradeContext.ConstantResolver = ModuleItem->GetEmitterViewModel().IsValid() ?
	        FCompileConstantResolver(ModuleItem->GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*ModuleItem->FunctionCallNode)) :
	        FCompileConstantResolver(&ModuleItem->GetSystemViewModel()->GetSystem(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*ModuleItem->FunctionCallNode));
        FGuid NewScriptVersion = ModuleItem->FunctionCallNode->FunctionScript->GetExposedVersion().VersionGuid;
        ModuleItem->FunctionCallNode->ChangeScriptVersion(NewScriptVersion, UpgradeContext, true);
        ModuleItem->Refresh();

		ModuleItem->ReportScriptVersionChange();
    });
}

void UNiagaraStackModuleItem::ReportScriptVersionChange() const
{
	// analytics
	TArray<FAnalyticsEventAttribute> Attributes;
	Attributes.Add(FAnalyticsEventAttribute(TEXT("Type"), TEXT("Module")));
	if (NiagaraAnalytics::IsPluginAsset(GetModuleNode().FunctionScript))
	{
		Attributes.Add(FAnalyticsEventAttribute(TEXT("AssetName"), GetModuleNode().FunctionScript->GetPackage()->GetName()));
	}
	NiagaraAnalytics::RecordEvent(TEXT("Versioning.ScriptVersionChanged"), Attributes);
}

void UNiagaraStackModuleItem::RefreshIssues(TArray<FStackIssue>& NewIssues)
{
	if (GetSystemViewModel()->GetIsForDataProcessingOnly())
	{
		NewIssues.Empty();
		return;
	}

	if (FunctionCallNode == nullptr)
	{
		return;
	}

	if (FunctionCallNode->FunctionScript != nullptr)
	{
		const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();
		if (EditorSettings->IsAllowedAssetObjectByClassUsage(*FunctionCallNode->FunctionScript) == false)
		{
			NewIssues.Add(FStackIssue(
				EStackIssueSeverity::Error,
				LOCTEXT("UnsupportedModuleScriptShort", "Unsupported Module Script"),
				LOCTEXT("UnsupportedModuleScriptLong", "This module uses a script which uses types which are unsupported in this editor context.  This module must either be deleted, or the referenced script must be fixed."),
				GetStackEditorDataKey(),
				false));
		}
	}

	if (!GetIsEnabled())
	{
		return;
	}

	FNiagaraStackGraphUtilities::CheckForDeprecatedScriptVersion(FunctionCallNode, GetStackEditorDataKey(), GetUpgradeVersionFix(), NewIssues);

	FVersionedNiagaraScriptData* ScriptData = FunctionCallNode->GetScriptData();
	if (ScriptData != nullptr)
	{
		if (ScriptData->bDeprecated)
		{
			FText ModuleScriptDeprecationShort = LOCTEXT("ModuleScriptDeprecationShort", "Deprecated module");
			if (CanMoveAndDelete())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ScriptName"), FText::FromString(FunctionCallNode->GetFunctionName()));

				if (ScriptData->DeprecationRecommendation != nullptr)
				{
					Args.Add(TEXT("Recommendation"), FText::FromString(ScriptData->DeprecationRecommendation->GetPathName()));
				}

				if (ScriptData->DeprecationMessage.IsEmptyOrWhitespace() == false)
				{
					Args.Add(TEXT("Message"), ScriptData->DeprecationMessage);
				}

				FText FormatString = LOCTEXT("ModuleScriptDeprecationUnknownLong", "The script asset for the assigned module {ScriptName} has been deprecated.");

				if (ScriptData->DeprecationRecommendation != nullptr &&
					ScriptData->DeprecationMessage.IsEmptyOrWhitespace() == false)
				{
					FormatString = LOCTEXT("ModuleScriptDeprecationMessageAndRecommendationLong", "The script asset for the assigned module {ScriptName} has been deprecated. Reason:\n{Message}.\nSuggested replacement: {Recommendation}");
				}
				else if (ScriptData->DeprecationRecommendation != nullptr)
				{
					FormatString = LOCTEXT("ModuleScriptDeprecationLong", "The script asset for the assigned module {ScriptName} has been deprecated. Suggested replacement: {Recommendation}");
				}
				else if (ScriptData->DeprecationMessage.IsEmptyOrWhitespace() == false)
				{
					FormatString = LOCTEXT("ModuleScriptDeprecationMessageLong", "The script asset for the assigned module {ScriptName} has been deprecated. Reason:\n{Message}");
				}

				FText LongMessage = FText::Format(FormatString, Args);

				int32 AddIdx = NewIssues.Add(FStackIssue(
					EStackIssueSeverity::Warning,
					ModuleScriptDeprecationShort,
					LongMessage,
					GetStackEditorDataKey(),
					false,
					{
						FStackIssueFix(
							LOCTEXT("SelectNewModuleScriptFix", "Select a new module script"),
							FStackIssueFixDelegate::CreateLambda([this]() { this->bIsModuleScriptReassignmentPending = true; })),
						FStackIssueFix(
							LOCTEXT("DeleteFix", "Delete this module"),
							FStackIssueFixDelegate::CreateLambda([this]() { this->Delete(); }))
					}));

				if (ScriptData->DeprecationRecommendation != nullptr)
				{
					NewIssues[AddIdx].InsertFix(0,
						FStackIssueFix(
						LOCTEXT("SelectNewModuleScriptFixUseRecommended", "Use recommended replacement and keep a disabled backup"),
						FStackIssueFixDelegate::CreateLambda([this]() 
						{
								if (DeprecationDelegate.IsBound())
								{
									DeprecationDelegate.Execute(this);
								}
						})));
				}
			}
			else
			{
				NewIssues.Add(FStackIssue(
					EStackIssueSeverity::Warning,
					ModuleScriptDeprecationShort,
					FText::Format(LOCTEXT("ModuleScriptDeprecationFixParentLong", "The script asset for the assigned module {0} has been deprecated.\nThis module is inherited and this issue must be fixed in the parent emitter.\nYou will need to touch up this instance once that is done."),
					FText::FromString(FunctionCallNode->GetFunctionName())),
					GetStackEditorDataKey(),
					false));
			}
		}

		if (ScriptData->bExperimental)
		{
			FText ErrorMessage;
			if (ScriptData->ExperimentalMessage.IsEmptyOrWhitespace())
			{
				ErrorMessage = FText::Format(LOCTEXT("ModuleScriptExperimental", "The script asset for this module is experimental, use with care!"), FText::FromString(FunctionCallNode->GetFunctionName()));
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Module"), FText::FromString(FunctionCallNode->GetFunctionName()));
				Args.Add(TEXT("Message"), ScriptData->ExperimentalMessage);
				ErrorMessage = FText::Format(LOCTEXT("ModuleScriptExperimentalReason", "The script asset for this module is marked as experimental, reason:\n{Message}."), Args);
			}

			NewIssues.Add(FStackIssue(
				EStackIssueSeverity::Info,
				LOCTEXT("ModuleScriptExperimentalShort", "Experimental module"),
				ErrorMessage,
				GetStackEditorDataKey(),
				true));
		}

		if (!ScriptData->NoteMessage.IsEmptyOrWhitespace())
		{
			FStackIssue NoteIssue = FStackIssue(
				EStackIssueSeverity::Info,
				LOCTEXT("ModuleScriptNoteShort", "Module Usage Note"),
				ScriptData->NoteMessage,
				GetStackEditorDataKey(),
				true);
			NoteIssue.SetIsExpandedByDefault(GetDefault<UNiagaraEditorSettings>()->bExpandModuleNotesByDefault);
			NewIssues.Add(NoteIssue);
		}
	}

	// Generate dependency errors with their fixes
	FGuid EmitterHandleId = FGuid();
	if (GetEmitterViewModel().IsValid())
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(GetEmitterViewModel()->GetEmitter());
		if (EmitterHandleViewModel.IsValid())
		{
			EmitterHandleId = EmitterHandleViewModel->GetId();
		}
	}
	
	const TArray<FNiagaraStackModuleData>& StackModuleData = GetSystemViewModel()->GetStackModuleDataByModuleEntry(this);
	UNiagaraScript* OwningScript = FNiagaraEditorUtilities::GetScriptFromSystem(GetSystemViewModel()->GetSystem(), EmitterHandleId, OutputNode->GetUsage(), OutputNode->GetUsageId());
	if (OwningScript != nullptr)
	{
		NiagaraStackModuleItemIssues::GenerateDependencyIssues(
			GetSystemViewModel(),
			EmitterHandleId,
			*OwningScript,
			*FunctionCallNode,
			GetStackEditorDataKey(),
			*OutputNode,
			StackModuleData,
			NewIssues);
	}

	NewIssues.Append(MessageManagerIssues);

	if (FunctionCallNode->FunctionScript == nullptr && FunctionCallNode->GetClass() == UNiagaraNodeFunctionCall::StaticClass())
	{
		FText ModuleScriptMissingShort = LOCTEXT("ModuleScriptMissingShort", "Missing module script");
		if (CanMoveAndDelete())
		{
			NewIssues.Add(FStackIssue(
				EStackIssueSeverity::Error,
				ModuleScriptMissingShort,
				FText::Format(LOCTEXT("ModuleScriptMissingLong", "The script asset for the assigned module {0} is missing."), FText::FromString(FunctionCallNode->GetFunctionName())),
				GetStackEditorDataKey(),
				false,
				{
					FStackIssueFix(
						LOCTEXT("SelectNewModuleScriptFix", "Select a new module script"),
						FStackIssueFixDelegate::CreateLambda([this]() { this->bIsModuleScriptReassignmentPending = true; })),
					FStackIssueFix(
						LOCTEXT("DeleteFix", "Delete this module"),
						FStackIssueFixDelegate::CreateLambda([this]() { this->Delete(); }))
				}));
		}
		else
		{
			// If the module can't be moved or deleted it's inherited and it's not valid to reassign scripts in child emitters because it breaks merging.
			NewIssues.Add(FStackIssue(
				EStackIssueSeverity::Error,
				ModuleScriptMissingShort,
				FText::Format(LOCTEXT("ModuleScriptMissingFixParentLong", "The script asset for the assigned module {0} is missing.  This module is inherited and this issue must be fixed in the parent emitter."), 
					FText::FromString(FunctionCallNode->GetFunctionName())),
				GetStackEditorDataKey(),
				false));
		}
	}
	else if (FunctionCallNode->HasValidScriptAndGraph() == false && FunctionCallNode->Signature.IsValid() == false)
	{
		FStackIssue InvalidScriptError(
			EStackIssueSeverity::Error,
			LOCTEXT("InvalidModuleScriptErrorSummary", "Invalid module script."),
			LOCTEXT("InvalidModuleScriptError", "The script this module is supposed to execute is missing or invalid for other reasons."),
			GetStackEditorDataKey(),
			false);

		NewIssues.Add(InvalidScriptError);
	}

	TOptional<bool> IsEnabled = FNiagaraStackGraphUtilities::GetModuleIsEnabled(*FunctionCallNode);
	if (!IsEnabled.IsSet())
	{
		bIsEnabled = false;
		FText FixDescription = LOCTEXT("EnableModule", "Enable module");
		FStackIssueFix EnableFix(
			FixDescription,
			FStackIssueFixDelegate::CreateLambda([this]()
		{
			SetIsEnabled(true);;
		}));
		FStackIssue InconsistentEnabledError(
			EStackIssueSeverity::Error,
			LOCTEXT("InconsistentEnabledErrorSummary", "The enabled state for this module is inconsistent."),
			LOCTEXT("InconsistentEnabledError", "This module is using multiple functions and their enabled states are inconsistent.\nClick \"Fix issue\" to make all of the functions for this module enabled."),
			GetStackEditorDataKey(),
			false,
			EnableFix);

		NewIssues.Add(InconsistentEnabledError);
	}

	UNiagaraNodeAssignment* AssignmentFunctionCall = Cast<UNiagaraNodeAssignment>(FunctionCallNode);
	if (AssignmentFunctionCall != nullptr)
	{
		TSet<FNiagaraVariable> FoundAssignmentTargets;
		for (const FNiagaraVariable& AssignmentTarget : AssignmentFunctionCall->GetAssignmentTargets())
		{
			if (FoundAssignmentTargets.Contains(AssignmentTarget))
			{
				FText FixDescription = LOCTEXT("RemoveDuplicate", "Remove Duplicate");
				FStackIssueFix RemoveDuplicateFix(FixDescription, FStackIssueFixDelegate::CreateLambda([AssignmentFunctionCall, AssignmentTarget]()
				{
					AssignmentFunctionCall->RemoveParameter(AssignmentTarget);
				}));
				FStackIssue DuplicateAssignmentTargetError(
					EStackIssueSeverity::Error,
					LOCTEXT("DuplicateAssignmentTargetErrorSummary", "Duplicate variables detected."),
					LOCTEXT("DuplicateAssignmentTargetError", "This 'Set Parameters' module is attempting to set the same variable more than once, which is unsupported."),
					GetStackEditorDataKey(),
					false,
					RemoveDuplicateFix);

				NewIssues.Add(DuplicateAssignmentTargetError);
			}
			FoundAssignmentTargets.Add(AssignmentTarget);
		}
	}
}

bool UNiagaraStackModuleItem::FilterOutputCollection(const UNiagaraStackEntry& Child) const
{
	if (Child.IsA<UNiagaraStackModuleItemOutputCollection>())
	{
		TArray<UNiagaraStackEntry*> ChildFilteredChildren;
		Child.GetFilteredChildren(ChildFilteredChildren);
		if (ChildFilteredChildren.Num() != 0)
		{
			return true;
		}
		else if (GetStackEditorData().GetShowOutputs() == false)
		{
			return false;
		}
	}
	return true;
}

bool UNiagaraStackModuleItem::FilterOutputCollectionChild(const UNiagaraStackEntry& Child) const
{
	// Filter to only show search result matches inside collapsed collection
	if (GetStackEditorData().GetShowOutputs() == false)
	{
		return Child.GetIsSearchResult();
	}
	return true;
}

bool UNiagaraStackModuleItem::FilterLinkedInputCollection(const UNiagaraStackEntry& Child) const
{
	if (Child.IsA<UNiagaraStackModuleItemLinkedInputCollection>())
	{
		TArray<UNiagaraStackEntry*> ChildFilteredChildren;
		Child.GetFilteredChildren(ChildFilteredChildren);
		if (ChildFilteredChildren.Num() != 0)
		{
 			return true;
		}
		else if (GetStackEditorData().GetShowLinkedInputs() == false && Child.GetShouldShowInStack())
		{
			return false;
		}
	}
	return true;
}

bool UNiagaraStackModuleItem::FilterLinkedInputCollectionChild(const UNiagaraStackEntry& Child) const
{
	// Filter to only show search result matches inside collapsed collection
	if (GetStackEditorData().GetShowLinkedInputs() == false)
	{
		return Child.GetIsSearchResult();
	}
	return true;
}

void UNiagaraStackModuleItem::RefreshIsEnabled()
{
	TOptional<bool> IsEnabled = FNiagaraStackGraphUtilities::GetModuleIsEnabled(*FunctionCallNode);
	if (IsEnabled.IsSet())
	{
		bIsEnabled = IsEnabled.GetValue();
	}
}

void UNiagaraStackModuleItem::OnMessageManagerRefresh(const TArray<TSharedRef<const INiagaraMessage>>& NewMessages)
{
	if (MessageManagerIssues.Num() != 0 || NewMessages.Num() != 0)
	{
		MessageManagerIssues.Reset();
		for (TSharedRef<const INiagaraMessage> Message : NewMessages)
		{
			// we skip issues that do not have relevance for the stack, such as notes that should only appear in the log despite being related to a module
			if(Message->ShouldOnlyLog())
			{
				continue;
			}
			
			// Sometimes compile errors with the same info are generated, so guard against duplicates here.
			FStackIssue Issue = FNiagaraMessageUtilities::MessageToStackIssue(Message, GetStackEditorDataKey());
			if (MessageManagerIssues.ContainsByPredicate([&Issue](const FStackIssue& NewIssue)
				{ return NewIssue.GetUniqueIdentifier() == Issue.GetUniqueIdentifier(); }) == false)
			{
				MessageManagerIssues.Add(Issue);
			}
		}

		RefreshChildren();
	}
}

bool UNiagaraStackModuleItem::CanMoveAndDelete() const
{
	if (bCanMoveAndDeleteCache.IsSet() == false)
	{
		if (HasBaseEmitter() == false)
		{
			// If there is no base emitter all modules can be moved and deleted.
			bCanMoveAndDeleteCache = true;
		}
		else
		{
			// When editing systems only non-base modules can be moved and deleted.
			TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();

			FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetParentEmitter();

			bool bIsMergeable = MergeManager->IsMergeableScriptUsage(OutputNode->GetUsage());
			bool bHasBaseModule = bIsMergeable && BaseEmitter.Emitter != nullptr && MergeManager->HasBaseModule(BaseEmitter, OutputNode->GetUsage(), OutputNode->GetUsageId(), FunctionCallNode->NodeGuid);
			bCanMoveAndDeleteCache = bHasBaseModule == false;
		}
	}
	return bCanMoveAndDeleteCache.GetValue();
}

bool UNiagaraStackModuleItem::CanRefresh() const
{
	return bCanRefresh;
}

void UNiagaraStackModuleItem::Refresh()
{
	if (CanRefresh())
	{
		if (FunctionCallNode->RefreshFromExternalChanges())
		{
			FunctionCallNode->GetNiagaraGraph()->NotifyGraphNeedsRecompile();
			GetSystemViewModel()->ResetSystem();
		}
		RefreshChildren();
	}
}

bool UNiagaraStackModuleItem::GetIsEnabled() const
{
	return bIsEnabled;
}

void UNiagaraStackModuleItem::SetIsEnabledInternal(bool bInIsEnabled)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("EnableDisableModule", "Enable/Disable Module"));
	FNiagaraStackGraphUtilities::SetModuleIsEnabled(*FunctionCallNode, bInIsEnabled);
	bIsEnabled = bInIsEnabled;
	OnRequestFullRefreshDeferred().Broadcast();
}

bool UNiagaraStackModuleItem::SupportsDebugDraw() const
{
	return GetModuleNode().ContainsDebugSwitch();
}

TOptional<FText> UNiagaraStackModuleItem::GetDebugDrawTooltip() const
{
	if (FVersionedNiagaraScriptData* ScriptData = GetModuleNode().GetScriptData())
	{
		if (ScriptData->DebugDrawMessage.IsEmpty() == false)
		{
			return ScriptData->DebugDrawMessage;
		}
	}
	return {};
}

bool UNiagaraStackModuleItem::IsDebugDrawEnabled() const
{
	return FunctionCallNode->DebugState != ENiagaraFunctionDebugState::NoDebug;
}

void UNiagaraStackModuleItem::SetDebugDrawEnabled(bool bInEnabled)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("EnableDisableDebugModule", "Enable/Disable Debug for Module"));
	FunctionCallNode->DebugState = bInEnabled ? ENiagaraFunctionDebugState::Basic : ENiagaraFunctionDebugState::NoDebug;
	FunctionCallNode->MarkNodeRequiresSynchronization(__FUNCTION__, true);
	OnRequestFullRefreshDeferred().Broadcast();
}

int32 UNiagaraStackModuleItem::GetModuleIndex() const
{
	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackGroups;
	FNiagaraStackGraphUtilities::GetStackNodeGroups(*FunctionCallNode, StackGroups);
	int32 ModuleIndex = 0;
	for (FNiagaraStackGraphUtilities::FStackNodeGroup& StackGroup : StackGroups)
	{
		if (StackGroup.EndNode == FunctionCallNode)
		{
			return ModuleIndex;
		}
		if (StackGroup.EndNode->IsA<UNiagaraNodeFunctionCall>())
		{
			ModuleIndex++;
		}
	}
	return INDEX_NONE;
}

UNiagaraNodeOutput* UNiagaraStackModuleItem::GetOutputNode() const
{
	return OutputNode;
}

bool UNiagaraStackModuleItem::CanAddInput(FNiagaraVariable InputParameter) const
{
	UNiagaraNodeAssignment* AssignmentModule = Cast<UNiagaraNodeAssignment>(FunctionCallNode);
	return AssignmentModule != nullptr &&
		AssignmentModule->GetAssignmentTargets().Contains(InputParameter) == false &&
		FNiagaraStackGraphUtilities::CanWriteParameterFromUsage(InputParameter, OutputNode->GetUsage());
}

void UNiagaraStackModuleItem::AddInput(FNiagaraVariable InputParameter)
{
	if(ensureMsgf(CanAddInput(InputParameter), TEXT("This module doesn't support adding this input.")))
	{
		UNiagaraNodeAssignment* AssignmentNode = CastChecked<UNiagaraNodeAssignment>(FunctionCallNode);
		AssignmentNode->AddParameter(InputParameter, FNiagaraConstants::GetAttributeDefaultValue(InputParameter));
		FNiagaraStackGraphUtilities::InitializeStackFunctionInput(GetSystemViewModel(), GetEmitterViewModel(), GetStackEditorData(), *FunctionCallNode, *FunctionCallNode, InputParameter.GetName());
	}
}

bool UNiagaraStackModuleItem::GetIsModuleScriptReassignmentPending() const
{
	return bIsModuleScriptReassignmentPending;
}

void UNiagaraStackModuleItem::SetIsModuleScriptReassignmentPending(bool bIsPending)
{
	bIsModuleScriptReassignmentPending = bIsPending;
}

void UNiagaraStackModuleItem::ReassignModuleScript(UNiagaraScript* ModuleScript)
{
	if (ModuleScript == nullptr)
	{
		return;
	}
	if (ensureMsgf(FunctionCallNode != nullptr && FunctionCallNode->GetClass() == UNiagaraNodeFunctionCall::StaticClass(),
		TEXT("Can not reassign the module script when the module isn't a valid function call module.")))
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("ReassignModuleTransaction", "Reassign module script"));

		const FString OldName = FunctionCallNode->GetFunctionName();
		UNiagaraScript* OldScript = FunctionCallNode->FunctionScript;
		FVersionedNiagaraScriptData* OldScriptData = FunctionCallNode->GetScriptData();

		FunctionCallNode->Modify();
		UNiagaraClipboardContent* OldClipboardContent = nullptr;
		FVersionedNiagaraScriptData* NewScriptData = ModuleScript->GetLatestScriptData();
		if (NewScriptData->ConversionUtility || OldScriptData->bUsePythonScriptConversion)
		{
			OldClipboardContent = UNiagaraClipboardContent::Create();
			Copy(OldClipboardContent);
		}
		FunctionCallNode->FunctionScript = ModuleScript;
		FunctionCallNode->SelectedScriptVersion = ModuleScript && ModuleScript->IsVersioningEnabled() ? ModuleScript->GetExposedVersion().VersionGuid : FGuid();
		
		// intermediate refresh to purge any rapid iteration parameters that have been removed in the new script
		RefreshChildren();

		FunctionCallNode->SuggestName(FString());
		const FString NewName = FunctionCallNode->GetFunctionName();
		if (NewName != OldName)
		{
			UNiagaraSystem& System = GetSystemViewModel()->GetSystem();
			FVersionedNiagaraEmitter Emitter = GetEmitterViewModel().IsValid() ? GetEmitterViewModel()->GetEmitter() : FVersionedNiagaraEmitter();
			FNiagaraStackGraphUtilities::RenameReferencingParameters(&System, Emitter, *FunctionCallNode, OldName, NewName);
		}
		// we need to refresh the node here to update static switch inputs, as they are set on the pins
		FunctionCallNode->RefreshFromExternalChanges();
		FunctionCallNode->MarkNodeRequiresSynchronization(TEXT("Module script reassigned."), true);
		RefreshChildren();

		bool bIsPythonAvailable = IPythonScriptPlugin::Get()->IsPythonAvailable();
		if (OldScriptData->bUsePythonScriptConversion && bIsPythonAvailable && OldClipboardContent != nullptr)
		{
			UNiagaraClipboardContent* NewClipboardContent = UNiagaraClipboardContent::Create();
			Copy(NewClipboardContent);
			if (OldClipboardContent->Functions.Num() > 0)
			{
				OldClipboardContent->FunctionInputs = OldClipboardContent->Functions[0]->Inputs;
				OldClipboardContent->Functions.Empty();
			}
			if (NewClipboardContent->Functions.Num() > 0)
			{
				NewClipboardContent->FunctionInputs = NewClipboardContent->Functions[0]->Inputs;
				NewClipboardContent->Functions.Empty();
			}

			// There can be hidden inputs due to static switches, so we need to add all of them to the clipboard for the python script to see them.
			TArray<FNiagaraVariable> StaticSwitchInputs = FunctionCallNode->GetCalledGraph()->FindStaticSwitchInputs();
			TArray<FNiagaraVariable> InputVariables;
			TSet<FNiagaraVariable> HiddenInputVariables;
			FCompileConstantResolver Resolver = GetEmitterViewModel().IsValid() ?
			FCompileConstantResolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCallNode)) :
			FCompileConstantResolver(&GetSystemViewModel()->GetSystem(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCallNode));
			GetStackFunctionInputs(*FunctionCallNode, InputVariables, HiddenInputVariables, Resolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly);

			TSet<FNiagaraVariableBase> SeenVars;
			for (const UNiagaraClipboardFunctionInput* Input : NewClipboardContent->FunctionInputs)
			{
				SeenVars.Add(FNiagaraVariable(Input->GetTypeDef(), Input->InputName));
			}
			for (const FNiagaraVariableBase& StaticVar : StaticSwitchInputs)
			{
				if (!SeenVars.Contains(StaticVar))
				{
					NewClipboardContent->FunctionInputs.Add(UNiagaraClipboardFunctionInput::CreateDefaultInputValue(NewClipboardContent, StaticVar.GetName(), StaticVar.GetType()));
					SeenVars.Add(StaticVar);
				}
			}
			for (const FNiagaraVariableBase& InputVar : InputVariables)
			{
				FString InputName = InputVar.GetName().ToString();
				InputName.RemoveFromStart(PARAM_MAP_MODULE_STR);
				FNiagaraVariableBase NewVar(InputVar.GetType(), FName(InputName));
				if (!SeenVars.Contains(NewVar))
				{
					NewClipboardContent->FunctionInputs.Add(UNiagaraClipboardFunctionInput::CreateDefaultInputValue(NewClipboardContent, NewVar.GetName(), NewVar.GetType()));
					SeenVars.Add(NewVar);
				}
			}
			
			FunctionCallNode->PythonUpgradeScriptWarnings.Empty();
			FText Warnings;
			if (UNiagaraClipboardContent* NewInputs = FNiagaraEditorUtilities::RunPythonConversionScript(*NewScriptData, NewClipboardContent, *OldScriptData, OldClipboardContent, Warnings))
			{
				Paste(NewInputs, Warnings);
				if (!Warnings.IsEmpty()) {
					FunctionCallNode->PythonUpgradeScriptWarnings = Warnings.ToString();
				}
			}			
		}
		else if (NewScriptData->ConversionUtility != nullptr && OldClipboardContent != nullptr)
		{
			UNiagaraConvertInPlaceUtilityBase* ConversionUtility = NewScriptData->ConversionUtility.GetDefaultObject();

			UNiagaraClipboardContent* NewClipboardContent = UNiagaraClipboardContent::Create();
			Copy(NewClipboardContent);

			if (ConversionUtility)
			{
				FText ConvertMessage;
				if (ConversionUtility->UsesLegacyConversion())
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					ConversionUtility->Convert(OldScript, OldClipboardContent, ModuleScript, InputRoot, NewClipboardContent, FunctionCallNode, ConvertMessage);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				else if (ConversionUtility->ConvertInputs(OldScript, OldClipboardContent, ModuleScript, NewClipboardContent, ConvertMessage))
				{
					Paste(NewClipboardContent, ConvertMessage);
				}
				if (!ConvertMessage.IsEmptyOrWhitespace())
				{
					// Notify the end-user about the convert message, but continue the process as they could always undo.
					FNotificationInfo Msg(FText::Format(LOCTEXT("FixConvertInPlace", "Conversion Note: {0}"), ConvertMessage));
					Msg.ExpireDuration = 5.0f;
					Msg.bFireAndForget = true;
					Msg.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Note"));
					FSlateNotificationManager::Get().AddNotification(Msg);
				}
			}
		}
	}
}

void UNiagaraStackModuleItem::ChangeScriptVersion(FGuid NewScriptVersion)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("NiagaraChangeVersion_Transaction", "Changing module version"));
	FNiagaraScriptVersionUpgradeContext UpgradeContext;
	UpgradeContext.CreateClipboardCallback = [this](UNiagaraClipboardContent* ClipboardContent)
	{
		RefreshChildren();
		Copy(ClipboardContent);
		if (ClipboardContent->Functions.Num() > 0)
		{
			ClipboardContent->FunctionInputs = ClipboardContent->Functions[0]->Inputs;
			ClipboardContent->Functions.Empty();
		}
	};
	UpgradeContext.ApplyClipboardCallback = [this](UNiagaraClipboardContent* ClipboardContent, FText& OutWarning) { Paste(ClipboardContent, OutWarning); };
	UpgradeContext.ConstantResolver = GetEmitterViewModel().IsValid() ?
	    FCompileConstantResolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCallNode)) :
	    FCompileConstantResolver(&GetSystemViewModel()->GetSystem(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCallNode));
	GetModuleNode().ChangeScriptVersion(NewScriptVersion, UpgradeContext, true);
	Refresh();

	ReportScriptVersionChange();
}

void UNiagaraStackModuleItem::SetInputValuesFromClipboardFunctionInputs(const TArray<const UNiagaraClipboardFunctionInput*>& ClipboardFunctionInputs)
{
	InputRoot->SetValuesFromClipboardFunctionInputs(ClipboardFunctionInputs);
}

void UNiagaraStackModuleItem::GetParameterInputs(TArray<UNiagaraStackFunctionInput*>& OutResult) const
{
	InputRoot->GetChildInputs(OutResult);
}

TArray<UNiagaraStackFunctionInput*> UNiagaraStackModuleItem::GetInlineParameterInputs() const
{
	return InputRoot->GetInlineParameters();
}

bool UNiagaraStackModuleItem::TestCanCutWithMessage(FText& OutMessage) const
{
	FText CanCopyMessage;
	if (TestCanCopyWithMessage(CanCopyMessage) == false)
	{
		OutMessage = FText::Format(LOCTEXT("CantCutBecauseCantCopyFormat", "This module can not be cut because it can't be copied.  {0}"), CanCopyMessage);
		return false;
	}

	FText CanDeleteMessage;
	if (TestCanDeleteWithMessage(CanDeleteMessage) == false)
	{
		OutMessage = FText::Format(LOCTEXT("CantCutBecauseCantDeleteFormat", "This module can't be cut because it can't be deleted.  {0}"), CanDeleteMessage);
		return false;
	}

	OutMessage = LOCTEXT("CanCut", "Cut this module.");
	return true;
}

FText UNiagaraStackModuleItem::GetCutTransactionText() const
{
	return LOCTEXT("CutModuleTransactionText", "Cut modules");
}

void UNiagaraStackModuleItem::CopyForCut(UNiagaraClipboardContent* ClipboardContent) const
{
	Copy(ClipboardContent);
}

void UNiagaraStackModuleItem::RemoveForCut()
{
	Delete();
}

bool UNiagaraStackModuleItem::TestCanCopyWithMessage(FText& OutMessage) const
{
	if (FunctionCallNode->GetClass() == UNiagaraNodeFunctionCall::StaticClass())
	{
		if (FunctionCallNode->FunctionScript == nullptr)
		{
			OutMessage = LOCTEXT("CantCopyInvalidModule", "This module can't be copied because it's referenced script is not valid.");
			return false;
		}
	}
	OutMessage = LOCTEXT("CopyModule", "Copy this module.");
	return true;
}

void UNiagaraStackModuleItem::Copy(UNiagaraClipboardContent* ClipboardContent) const
{
	UNiagaraClipboardFunction* ClipboardFunction;
	UNiagaraNodeAssignment* AssignmentNode = Cast<UNiagaraNodeAssignment>(FunctionCallNode);
	if (AssignmentNode != nullptr)
	{
		ClipboardFunction = UNiagaraClipboardFunction::CreateAssignmentFunction(ClipboardContent, AssignmentNode->GetFunctionName(), AssignmentNode->GetAssignmentTargets(), AssignmentNode->GetAssignmentDefaults(), GetStackNoteData());
	}
	else
	{
		checkf(FunctionCallNode->FunctionScript != nullptr, TEXT("Can't copy this module because it's script is invalid.  Call TestCanCopyWithMessage to check this."));
		ClipboardFunction = UNiagaraClipboardFunction::CreateScriptFunction(ClipboardContent, FunctionCallNode->GetFunctionName(), FunctionCallNode->FunctionScript, FunctionCallNode->SelectedScriptVersion, GetStackNoteData());
	}

	ClipboardFunction->DisplayName = GetAlternateDisplayName().Get(FText::GetEmpty());

	InputRoot->ToClipboardFunctionInputs(ClipboardFunction, MutableView(ClipboardFunction->Inputs));
	ClipboardContent->Functions.Add(ClipboardFunction);

	RequestPulse();
}

bool UNiagaraStackModuleItem::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	if (ClipboardContent->FunctionInputs.Num() > 0)
	{
		OutMessage = LOCTEXT("PasteInputs", "Paste inputs from the clipboard which match inputs on this module by name and type.");
		return true;
	}

	if (RequestCanPasteDelegete.IsBound())
	{
		return RequestCanPasteDelegete.Execute(ClipboardContent, OutMessage);
	}

	OutMessage = FText();
	return false;
}

FText UNiagaraStackModuleItem::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	if (ClipboardContent->FunctionInputs.Num() > 0)
	{
		return LOCTEXT("PasteInputsTransactionText", "Paste inputs to module.");
	}
	else
	{
		return LOCTEXT("PasteModuleTransactionText", "Paste niagara modules");
	}
}

void UNiagaraStackModuleItem::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	if (ClipboardContent->FunctionInputs.Num() > 0)
	{
		SetInputValuesFromClipboardFunctionInputs(ClipboardContent->FunctionInputs);
	}
	else if (RequestCanPasteDelegete.IsBound())
	{
		// Pasted modules should go after this module, so add 1 to the index.
		int32 PasteIndex = GetModuleIndex() + 1;
		RequestPasteDelegate.Execute(ClipboardContent, PasteIndex, OutPasteWarning);
	}
}

bool UNiagaraStackModuleItem::TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const
{
	if (GetOwnerIsEnabled() == false)
	{
		OutCanDeleteMessage = LOCTEXT("CantDeleteOwnerDisabledToolTip", "This module can not be deleted because its owner is disabled.");
		return false;
	}
	else if (CanMoveAndDelete())
	{
		OutCanDeleteMessage = LOCTEXT("DeleteToolTip", "Delete this module.");
		return true;
	}
	else
	{
		OutCanDeleteMessage = LOCTEXT("CantDeleteToolTip", "This module can not be deleted becaue it is inherited.");
		return false;
	}
}

FText UNiagaraStackModuleItem::GetDeleteTransactionText() const
{
	return LOCTEXT("DeleteModuleTransaction", "Delete modules");
}

void UNiagaraStackModuleItem::Delete()
{
	checkf(CanMoveAndDelete(), TEXT("This module can't be deleted"));
	const FNiagaraEmitterHandle* EmitterHandle = GetEmitterViewModel().IsValid()
		? FNiagaraEditorUtilities::GetEmitterHandleForEmitter(GetSystemViewModel()->GetSystem(), GetEmitterViewModel()->GetEmitter())
		: nullptr;
	FGuid EmitterHandleId = EmitterHandle != nullptr ? EmitterHandle->GetId() : FGuid();

	TArray<TWeakObjectPtr<UNiagaraNodeInput>> RemovedNodes;
	if (IsSystemViewModelValid() && FNiagaraStackGraphUtilities::RemoveModuleFromStack(GetSystemViewModel()->GetSystem(), EmitterHandleId, *FunctionCallNode, RemovedNodes))
	{
		UNiagaraGraph* Graph = FunctionCallNode->GetNiagaraGraph();
		Graph->NotifyGraphNeedsRecompile();
		FNiagaraStackGraphUtilities::RelayoutGraph(*FunctionCallNode->GetGraph());
		TArray<UObject*> RemovedDataInterfaces;
		for (auto InputNode : RemovedNodes)
		{
			if (InputNode != nullptr && InputNode->Usage == ENiagaraInputNodeUsage::Parameter && InputNode->GetDataInterface() != nullptr)
			{
				RemovedDataInterfaces.Add(InputNode->GetDataInterface());
			}
		}
		GetSystemViewModel()->NotifyDataObjectChanged(RemovedDataInterfaces, ENiagaraDataObjectChange::Removed);
		ModifiedGroupItemsDelegate.Broadcast();
	}
}

bool UNiagaraStackModuleItem::GetIsInherited() const
{
	return CanMoveAndDelete() == false;
}

FText UNiagaraStackModuleItem::GetInheritanceMessage() const
{
	return LOCTEXT("ModuleItemInheritanceMessage", "This module is inherited from a parent emitter.  Inherited modules\ncan only be moved, deleted, and versioned while editing the parent emitter.");
}

FHierarchyElementIdentity UNiagaraStackModuleItem::DetermineSummaryIdentity() const
{
	FHierarchyElementIdentity Identity;
	Identity.Guids.Add(GetModuleNode().NodeGuid);
	return Identity;
}

bool UNiagaraStackModuleItem::IsScratchModule() const
{
	if (bIsScratchModuleCache.IsSet() == false)
	{
		bIsScratchModuleCache = GetSystemViewModel()->GetScriptScratchPadViewModel()->GetViewModelForScript(FunctionCallNode->FunctionScript).IsValid();
	}
	return bIsScratchModuleCache.GetValue();
}

UObject* UNiagaraStackModuleItem::GetExternalAsset() const
{
	if (GetModuleNode().FunctionScript != nullptr && GetModuleNode().FunctionScript->IsAsset())
	{
		return GetModuleNode().FunctionScript;
	}
	return nullptr;
}

bool UNiagaraStackModuleItem::CanDrag() const
{
	return true;
}

bool UNiagaraStackModuleItem::OpenSourceAsset() const
{
	// Helper to open scratch script or function script in the appropriate sub editor based on type.
	const UNiagaraNodeFunctionCall& ModuleFunctionCall = GetModuleNode();
	if (ModuleFunctionCall.FunctionScript != nullptr)
	{
		if (ModuleFunctionCall.FunctionScript->IsAsset() || GbShowNiagaraDeveloperWindows > 0)
		{
			ModuleFunctionCall.FunctionScript->VersionToOpenInEditor = ModuleFunctionCall.SelectedScriptVersion;

			UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

			if(AssetEditorSubsystem->CanOpenEditorForAsset(ToRawPtr(ModuleFunctionCall.FunctionScript), EAssetTypeActivationOpenedMethod::Edit, nullptr))
			{
				return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets({ToRawPtr(ModuleFunctionCall.FunctionScript)}, EAssetTypeActivationOpenedMethod::Edit);
			}
			else if(AssetEditorSubsystem->CanOpenEditorForAsset(ToRawPtr(ModuleFunctionCall.FunctionScript), EAssetTypeActivationOpenedMethod::View, nullptr))
			{
				return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets({ToRawPtr(ModuleFunctionCall.FunctionScript)}, EAssetTypeActivationOpenedMethod::View);
			}			
		}
		else if (IsScratchModule())
		{
			TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel = GetSystemViewModel()->GetScriptScratchPadViewModel()->GetViewModelForScript(ModuleFunctionCall.FunctionScript);
			if (ScratchPadScriptViewModel.IsValid())
			{
				GetSystemViewModel()->GetScriptScratchPadViewModel()->FocusScratchPadScriptViewModel(ScratchPadScriptViewModel.ToSharedRef());
				return true;
			}
		}
		else if (GetIsInherited() && !ModuleFunctionCall.FunctionScript->IsAsset() && GetOutputNode() && GetEmitterViewModel())
		{
			TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();

			FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetParentEmitter();

			UNiagaraScript* FoundScript = nullptr;
			FGuid FoundScriptVersion;
			FVersionedNiagaraEmitter FoundBaseEmitter;
			if (MergeManager->FindBaseModule(BaseEmitter, GetOutputNode()->GetUsage(), OutputNode->GetUsageId(), ModuleFunctionCall.NodeGuid, FoundScript, FoundScriptVersion, FoundBaseEmitter) && FoundScript && FoundBaseEmitter.Emitter)
			{
				if (GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(FoundBaseEmitter.Emitter))
				{
					if (FNiagaraSystemToolkit* EditorInstance = static_cast<FNiagaraSystemToolkit*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(FoundBaseEmitter.Emitter, true)))
					{
						EditorInstance->SwitchToVersion(FoundBaseEmitter.Version);
						TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel = EditorInstance->GetSystemViewModel()->GetScriptScratchPadViewModel()->GetViewModelForScript(FoundScript->GetFName());
						if (ScratchPadScriptViewModel.IsValid())
						{
							EditorInstance->GetSystemViewModel()->GetScriptScratchPadViewModel()->FocusScratchPadScriptViewModel(ScratchPadScriptViewModel.ToSharedRef());
							return true;
						}
					}
					return true;
				}
			}
			else
			{
				FNiagaraEditorUtilities::InfoWithToastAndLog(FText::Format(LOCTEXT("CannotAutoOpenBaseModule", "Cannot auto-open base module {0} as this was created before merging tracked this data. Newer assets should work. You should be able to open it manually however."), FText::FromString(ModuleFunctionCall.FunctionScript->GetName())));
			}
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
