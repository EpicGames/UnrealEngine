// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackModuleItemLinkedInputCollection.h"
#include "ViewModels/Stack/NiagaraStackModuleItemOutput.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraEmitterEditorData.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

#include "EdGraph/EdGraphPin.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraNodeOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackModuleItemLinkedInputCollection)

UNiagaraStackModuleItemLinkedInputCollection::UNiagaraStackModuleItemLinkedInputCollection()
	: FunctionCallNode(nullptr)
{
}

void UNiagaraStackModuleItemLinkedInputCollection::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraNodeFunctionCall& InFunctionCallNode)
{
	checkf(FunctionCallNode == nullptr, TEXT("Can not set the node more than once."));
	FString LinkedInputCollectionStackEditorDataKey = FString::Printf(TEXT("%s-LinkedInputs"), *InFunctionCallNode.NodeGuid.ToString(EGuidFormats::DigitsWithHyphens));
	Super::Initialize(InRequiredEntryData, LinkedInputCollectionStackEditorDataKey);
	FunctionCallNode = &InFunctionCallNode;
}
 
FText UNiagaraStackModuleItemLinkedInputCollection::GetDisplayName() const
{
	return NSLOCTEXT("StackModuleItemLinkedInputCollection", "ParameterReadsLabel", "Parameter Reads");
}

bool UNiagaraStackModuleItemLinkedInputCollection::IsExpandedByDefault() const
{
	return false;
}

bool UNiagaraStackModuleItemLinkedInputCollection::GetIsEnabled() const
{
	return FunctionCallNode->GetDesiredEnabledState() == ENodeEnabledState::Enabled;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackModuleItemLinkedInputCollection::GetStackRowStyle() const
{
	return EStackRowStyle::ItemContent;
}

bool UNiagaraStackModuleItemLinkedInputCollection::GetShouldShowInStack() const
{
	TArray<UNiagaraStackEntry*> UnfilteredChildren;
	GetUnfilteredChildren(UnfilteredChildren);
	if (UnfilteredChildren.Num() != 0)
	{
		return true;
	}
	return false;
}

void UNiagaraStackModuleItemLinkedInputCollection::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	FGuid CurrentOwningGraphChangeId;
	FGuid CurrentFunctionGraphChangeId;
	GetCurrentChangeIds(CurrentOwningGraphChangeId, CurrentFunctionGraphChangeId);
	if (LastOwningGraphChangeId.IsSet() && CurrentOwningGraphChangeId == LastOwningGraphChangeId &&
		LastFunctionGraphChangeId.IsSet() && CurrentFunctionGraphChangeId == LastFunctionGraphChangeId)
	{
		// If the owning and called graph haven't changed, then the child inputs haven't changed either.
		NewChildren.Append(CurrentChildren);
		return;
	}

	UEdGraphPin* OutputParameterMapPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*FunctionCallNode);
	if (ensureMsgf(OutputParameterMapPin != nullptr, TEXT("Invalid Stack Graph - Function call node has no output pin.")))
	{
		FNiagaraParameterMapHistoryBuilder Builder;
		Builder.SetIgnoreDisabled(false);
		*Builder.ConstantResolver = GetEmitterViewModel().IsValid()
			? FCompileConstantResolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCallNode))
			: FCompileConstantResolver();
		FunctionCallNode->BuildParameterMapHistory(Builder, false);

		if (Builder.Histories.Num() == 1)
		{
			for (int32 i = 0; i < Builder.Histories[0].Variables.Num(); i++)
			{
				FNiagaraVariable& Variable = Builder.Histories[0].Variables[i];
				const auto& ReadHistory = Builder.Histories[0].PerVariableReadHistory[i];
				if (ReadHistory.Num() > 0)
				{
					const FNiagaraNamespaceMetadata NamespaceMetadata = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(Variable.GetName());
					if ( (NamespaceMetadata.IsValid()) && (NamespaceMetadata.Namespaces.Contains(FNiagaraConstants::LocalNamespace) == false) && (NamespaceMetadata.Namespaces.Contains(FNiagaraConstants::OutputNamespace) == false) )
					{
						for (const FNiagaraParameterMapHistory::FReadHistory& ReadPair : ReadHistory)
						{
							if (Cast<UNiagaraNodeParameterMapGet>(ReadPair.ReadPin.Pin->GetOwningNode()) != nullptr)
							{
								UNiagaraStackModuleItemOutput* Output = FindCurrentChildOfTypeByPredicate<UNiagaraStackModuleItemOutput>(CurrentChildren,
									[&](UNiagaraStackModuleItemOutput* CurrentOutput) { return CurrentOutput->GetOutputParameterHandle().GetParameterHandleString() == Variable.GetName(); });

								if (Output == nullptr)
								{
									Output = NewObject<UNiagaraStackModuleItemOutput>(this);
									Output->Initialize(CreateDefaultChildRequiredData(), *FunctionCallNode, Variable.GetName(), Variable.GetType());
								}

								NewChildren.Add(Output);
								break;
							}
						}
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Log, "Invalid Stack Graph - Function call node has invalid history count!");
		}

		auto SortNewChildrenPred = [](const UNiagaraStackEntry& EntryA, const UNiagaraStackEntry& EntryB) {
			const UNiagaraStackModuleItemOutput* ModuleItemA = static_cast<const UNiagaraStackModuleItemOutput*>(&EntryA);
			const UNiagaraStackModuleItemOutput* ModuleItemB = static_cast<const UNiagaraStackModuleItemOutput*>(&EntryB);
			return FNiagaraEditorUtilities::GetVariableSortPriority(ModuleItemA->GetOutputParameterHandle().GetParameterHandleString(), ModuleItemB->GetOutputParameterHandle().GetParameterHandleString());
		};

		NewChildren.Sort(SortNewChildrenPred);
	}

	LastOwningGraphChangeId = CurrentOwningGraphChangeId;
	LastFunctionGraphChangeId = CurrentFunctionGraphChangeId;
}

void UNiagaraStackModuleItemLinkedInputCollection::GetCurrentChangeIds(FGuid& OutOwningGraphChangeId, FGuid& OutFunctionGraphChangeId) const
{
	OutOwningGraphChangeId = FunctionCallNode->GetNiagaraGraph()->GetChangeID();
	OutFunctionGraphChangeId = FunctionCallNode->GetCalledGraph() != nullptr ? FunctionCallNode->GetCalledGraph()->GetChangeID() : FGuid();
}

