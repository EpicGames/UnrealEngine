// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackModuleItemOutput.h"

#include "NiagaraConstants.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraScriptSource.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackModuleItemOutputCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackModuleItemOutput)


#define LOCTEXT_NAMESPACE "NiagaraStackViewModel"

UNiagaraStackModuleItemOutput::UNiagaraStackModuleItemOutput()
	: FunctionCallNode(nullptr)
{
}

void UNiagaraStackModuleItemOutput::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraNodeFunctionCall& InFunctionCallNode, FName InOutputParameterHandle,
	FNiagaraTypeDefinition InOutputType)
{
	checkf(FunctionCallNode.Get() == nullptr, TEXT("Can only set the Output once."));
	FString OutputStackEditorDataKey = FString::Printf(TEXT("%s-Output-%s"), *InFunctionCallNode.NodeGuid.ToString(EGuidFormats::DigitsWithHyphens), *InOutputParameterHandle.ToString());
	Super::Initialize(InRequiredEntryData, OutputStackEditorDataKey);
	FunctionCallNode = &InFunctionCallNode;
	OutputType = InOutputType;
	OutputParameterHandle = FNiagaraParameterHandle(InOutputParameterHandle);
	DisplayName = FText::FromName(OutputParameterHandle.GetName());
}

FText UNiagaraStackModuleItemOutput::GetDisplayName() const
{
	return DisplayName;
}

const UNiagaraStackModuleItemOutput::FCollectedUsageData& UNiagaraStackModuleItemOutput::GetCollectedUsageData() const
{
	if (CachedCollectedUsageData.IsSet() == false)
	{
		CachedCollectedUsageData = FCollectedUsageData();
		TSharedRef<FNiagaraSystemViewModel> SystemVM = GetSystemViewModel();
		INiagaraParameterPanelViewModel* ParamVM = SystemVM->GetParameterPanelViewModel();
		if (ParamVM)
		{
			FNiagaraVariableBase Var(OutputType, OutputParameterHandle.GetParameterHandleString());
			bool bHandled = ParamVM->IsVariableSelected(Var);
			if (GetOuter() && GetOuter()->IsA<UNiagaraStackModuleItemOutputCollection>())
				CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite = bHandled;
			else
				CachedCollectedUsageData.GetValue().bHasReferencedParameterRead = bHandled;
		}
	}

	return CachedCollectedUsageData.GetValue();
}

FText UNiagaraStackModuleItemOutput::GetTooltipText() const
{
	FNiagaraVariable ValueVariable(OutputType, OutputParameterHandle.GetParameterHandleString());

	FText VariableText = FText::Format(LOCTEXT("OutputVariableTooltip", "{0} ({1})"), FText::FromName(ValueVariable.GetName()), ValueVariable.GetType().GetNameText());

	if (FunctionCallNode.IsValid() && FunctionCallNode->FunctionScript != nullptr)
	{
		UNiagaraScriptSource* Source = FunctionCallNode->GetFunctionScriptSource();
		TOptional<FNiagaraVariableMetaData> MetaData;
		if (FNiagaraConstants::IsNiagaraConstant(ValueVariable))
		{
			MetaData = *FNiagaraConstants::GetConstantMetaData(ValueVariable);
		}
		else if (Source->NodeGraph != nullptr)
		{
			MetaData = Source->NodeGraph->GetMetaData(ValueVariable);
		}
		if (MetaData.IsSet() && !MetaData->Description.IsEmpty())
		{
			return FText::Format(LOCTEXT("OutputVariableWithDescTooltip", "{0}\n{1}"), VariableText, MetaData->Description);
		}
	}
	return VariableText;
}

bool UNiagaraStackModuleItemOutput::GetIsEnabled() const
{
	return FunctionCallNode->GetDesiredEnabledState() == ENodeEnabledState::Enabled;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackModuleItemOutput::GetStackRowStyle() const
{
	return EStackRowStyle::ItemContent;
}

void UNiagaraStackModuleItemOutput::GetSearchItems(TArray<FStackSearchItem>& SearchItems) const
{
	SearchItems.Add({ FName("OutputParamHandleText"), GetOutputParameterHandleText() });
}

const FNiagaraParameterHandle& UNiagaraStackModuleItemOutput::GetOutputParameterHandle() const
{
	return OutputParameterHandle;
}

FText UNiagaraStackModuleItemOutput::GetOutputParameterHandleText() const
{
	return FText::FromName(OutputParameterHandle.GetParameterHandleString());
}

#undef LOCTEXT_NAMESPACE

