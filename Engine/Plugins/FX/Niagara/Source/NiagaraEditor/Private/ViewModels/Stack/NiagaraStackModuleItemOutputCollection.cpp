// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackModuleItemOutputCollection.h"
#include "ViewModels/Stack/NiagaraStackModuleItemOutput.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeParameterMapSet.h"
#include "NiagaraEmitterEditorData.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

#include "EdGraph/EdGraphPin.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraNodeOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackModuleItemOutputCollection)

UNiagaraStackModuleItemOutputCollection::UNiagaraStackModuleItemOutputCollection()
	: FunctionCallNode(nullptr)
{
}

void UNiagaraStackModuleItemOutputCollection::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraNodeFunctionCall& InFunctionCallNode)
{
	checkf(FunctionCallNode == nullptr, TEXT("Can not set the node more than once."));
	FString OutputCollectionStackEditorDataKey = FString::Printf(TEXT("%s-Outputs"), *InFunctionCallNode.NodeGuid.ToString(EGuidFormats::DigitsWithHyphens));
	Super::Initialize(InRequiredEntryData, OutputCollectionStackEditorDataKey);
	FunctionCallNode = &InFunctionCallNode;
}

FText UNiagaraStackModuleItemOutputCollection::GetDisplayName() const
{
	return NSLOCTEXT("StackModuleItemOutputCollection", "ParameterWritesLabel", "Parameter Writes");
}

bool UNiagaraStackModuleItemOutputCollection::IsExpandedByDefault() const
{
	return false;
}

bool UNiagaraStackModuleItemOutputCollection::GetIsEnabled() const
{
	return FunctionCallNode->GetDesiredEnabledState() == ENodeEnabledState::Enabled;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackModuleItemOutputCollection::GetStackRowStyle() const
{
	return EStackRowStyle::ItemContent;
}

void UNiagaraStackModuleItemOutputCollection::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	FGuid CurrentOwningGraphChangeId;
	FGuid CurrentFunctionGraphChangeId;
	GetCurrentChangeIds(CurrentOwningGraphChangeId, CurrentFunctionGraphChangeId);
	if (LastOwningGraphChangeId.IsSet() && CurrentOwningGraphChangeId == LastOwningGraphChangeId &&
		LastFunctionGraphChangeId.IsSet() && CurrentFunctionGraphChangeId == LastFunctionGraphChangeId)
	{
		// If the owning and called graph haven't changed, then the child outputs haven't changed either.
		NewChildren.Append(CurrentChildren);
		return;
	}

	UEdGraphPin* OutputParameterMapPin = FNiagaraStackGraphUtilities::GetParameterMapOutputPin(*FunctionCallNode);
	if (ensureMsgf(OutputParameterMapPin != nullptr, TEXT("Invalid Stack Graph - Function call node has no output pin.")))
	{
		TArray<FNiagaraVariable> OutputVariables;
		TArray<FNiagaraVariable> Unused;
		FCompileConstantResolver ConstantResolver = GetEmitterViewModel().IsValid()
			? FCompileConstantResolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCallNode))
			: FCompileConstantResolver();
		FNiagaraStackGraphUtilities::GetStackFunctionOutputVariables(*FunctionCallNode, ConstantResolver, OutputVariables, Unused);
		
		for(int32 OutputIndex = 0; OutputIndex < OutputVariables.Num(); OutputIndex++)
		{
			FNiagaraVariable& Variable = OutputVariables[OutputIndex];
			const FNiagaraNamespaceMetadata NamespaceMetadata = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(Variable.GetName());
			if ( (NamespaceMetadata.IsValid()) && (NamespaceMetadata.Namespaces.Contains(FNiagaraConstants::LocalNamespace) == false) && (NamespaceMetadata.Namespaces.Contains(FNiagaraConstants::ModuleNamespace) == false) )
			{
				UNiagaraStackModuleItemOutput* Output = FindCurrentChildOfTypeByPredicate<UNiagaraStackModuleItemOutput>(CurrentChildren,
					[&](UNiagaraStackModuleItemOutput* CurrentOutput) { return CurrentOutput->GetOutputParameterHandle().GetParameterHandleString() == Variable.GetName(); });

				if (Output == nullptr)
				{
					Output = NewObject<UNiagaraStackModuleItemOutput>(this);
					Output->Initialize(CreateDefaultChildRequiredData(), *FunctionCallNode, Variable.GetName(), Variable.GetType());
				}

				NewChildren.Add(Output);
			}
		}
		auto SortNewChildrenPred = [](const UNiagaraStackEntry& EntryA, const UNiagaraStackEntry& EntryB) {
			const UNiagaraStackModuleItemOutput* ModuleItemA = static_cast<const UNiagaraStackModuleItemOutput*>(&EntryA);
			const UNiagaraStackModuleItemOutput* ModuleItemB = static_cast<const UNiagaraStackModuleItemOutput*>(&EntryB);
			return FNiagaraEditorUtilities::GetVariableSortPriority(ModuleItemA->GetOutputParameterHandle().GetParameterHandleString(), ModuleItemB->GetOutputParameterHandle().GetParameterHandleString());
		};

		NewChildren.Sort(SortNewChildrenPred);
	}

	LastOwningGraphChangeId = CurrentOwningGraphChangeId;
	LastFunctionGraphChangeId = CurrentFunctionGraphChangeId;
}

void UNiagaraStackModuleItemOutputCollection::GetCurrentChangeIds(FGuid& OutOwningGraphChangeId, FGuid& OutFunctionGraphChangeId) const
{
	OutOwningGraphChangeId = FunctionCallNode->GetNiagaraGraph()->GetChangeID();
	OutFunctionGraphChangeId = FunctionCallNode->GetCalledGraph() != nullptr ? FunctionCallNode->GetCalledGraph()->GetChangeID() : FGuid();
}

