// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackNote.h"

#include "NiagaraEditorSettings.h"
#include "NiagaraMessages.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackNote)

#define LOCTEXT_NAMESPACE "UNiagaraStackNote"

void UNiagaraStackNote::Initialize(FRequiredEntryData InRequiredEntryData, FString InTargetStackEntryKey)
{
	TargetStackEntryKey = InTargetStackEntryKey;
	FString NoteStackEntryKey = GetTargetStackEntryKey();
	NoteStackEntryKey.Append("-Note");
	Super::Initialize(InRequiredEntryData, NoteStackEntryKey);
}

FString UNiagaraStackNote::GetTargetStackEntryKey() const
{
	return TargetStackEntryKey;
}

TOptional<FNiagaraStackNoteData> UNiagaraStackNote::GetTargetStackNoteData() const
{
	if (!IsFinalized())
	{
		return GetStackEditorData().GetStackNote(GetTargetStackEntryKey());
	}
	return {};
}

void UNiagaraStackNote::ToggleInlineDisplay()
{
	FScopedTransaction Transaction(LOCTEXT("ToggleNoteDisplayTransaction", "Toggled Note Display"));
	GetStackEditorData().Modify();

	FNiagaraStackNoteData UpdatedStackNoteData = GetTargetStackNoteData().GetValue();
	UpdatedStackNoteData.bInlineNote = !UpdatedStackNoteData.bInlineNote;

	OnNoteChangedDelegate.ExecuteIfBound(UpdatedStackNoteData);
}

void UNiagaraStackNote::DeleteTargetStackNote()
{
	FScopedTransaction Transaction(LOCTEXT("DeleteNoteTransaction", "Deleted Note"));
	GetStackEditorData().Modify();

	GetStackEditorData().DeleteStackNote(GetTargetStackEntryKey());
	
	OnNoteChangedDelegate.ExecuteIfBound(FNiagaraStackNoteData());
}

void UNiagaraStackNote::UpdateNoteColor(FLinearColor LinearColor)
{
	FScopedTransaction Transaction(LOCTEXT("ChangeNoteColorTransaction", "Changed Note Color"));
	GetStackEditorData().Modify();

	FNiagaraStackNoteData UpdatedStackNoteData = GetTargetStackNoteData().GetValue();

	if(UpdatedStackNoteData.Color != LinearColor)
	{
		UpdatedStackNoteData.Color = LinearColor;
		
		OnNoteChangedDelegate.ExecuteIfBound(UpdatedStackNoteData);
	}
	else
	{
		Transaction.Cancel();
	}
}

bool UNiagaraStackNote::GetShouldShowInStack() const
{
	if (GetTargetStackNoteData().IsSet())
	{
		//  notes are not displayed in the stack in the sense that the stack note entries themselves won't be displayed.
		return GetTargetStackNoteData().GetValue().bInlineNote == false;
	}

	return false;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackNote::GetStackRowStyle() const
{
	return EStackRowStyle::ItemContentNote;
}

void UNiagaraStackNote::Copy(UNiagaraClipboardContent* ClipboardContent) const
{
	ClipboardContent->StackNote = GetTargetStackNoteData().GetValue();
}

void UNiagaraStackNote::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	if (ClipboardContent->StackNote.IsValid())
	{
		GetStackEditorData().Modify();

		FNiagaraStackNoteData NoteData = ClipboardContent->StackNote;
		GetStackEditorData().AddOrReplaceStackNote(GetTargetStackEntryKey(), NoteData);
	}
}

bool UNiagaraStackNote::TestCanCopyWithMessage(FText& OutMessage) const
{
	if(GetTargetStackNoteData()->IsValid())
	{
		OutMessage = LOCTEXT("CanCopyNoteTest", "Copy the contents of this note");
		return true;
	}

	OutMessage = LOCTEXT("CantCopyNoteTest", "Can not copy this note due to invalid content.");
	return false;
}

bool UNiagaraStackNote::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	if(ClipboardContent->StackNote.IsValid())
	{
		OutMessage = LOCTEXT("CanPasteNoteTest", "Paste the contents of a previously copied note");
		return true;
	}

	OutMessage = LOCTEXT("CantPasteNoteTest", "Can not paste into this note. No valid note in clipboard.");
	return false;
}

FLinearColor UNiagaraStackNote::GetColor() const
{
	if(GetTargetStackNoteData().IsSet())
	{
		return GetTargetStackNoteData()->Color.IsAlmostBlack() == false ? GetTargetStackNoteData()->Color : GetDefault<UNiagaraEditorSettings>()->GetDefaultNoteColor();
	}

	return GetDefault<UNiagaraEditorSettings>()->GetDefaultNoteColor(); 
}

#undef LOCTEXT_NAMESPACE
