// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackPropertyRow.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "NiagaraNode.h"
#include "NiagaraEditorModule.h"
#include "NiagaraMessageManager.h"
#include "NiagaraMessageUtilities.h"
#include "NiagaraEditorSettings.h"

#include "Modules/ModuleManager.h"
#include "IPropertyRowGenerator.h"
#include "PropertyEditorModule.h"
#include "IDetailTreeNode.h"
#include "PropertyHandle.h"

#include "NiagaraPlatformSet.h"
#include "DataHierarchyViewModelBase.h"
#include "NiagaraStackEditorData.h"
#include "ViewModels/Stack/NiagaraStackObjectIssueGenerator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackObject)

UNiagaraStackObject::UNiagaraStackObject()
{
}

void UNiagaraStackObject::Initialize(FRequiredEntryData InRequiredEntryData, UObject* InObject, bool bInIsTopLevelObject, bool bInHideTopLevelCategories, FString InOwnerStackItemEditorDataKey, UNiagaraNode* InOwningNiagaraNode)
{
	checkf(WeakObject.IsValid() == false || DisplayedStruct.IsValid() == false, TEXT("Can only initialize once."));
	FString ObjectStackEditorDataKey = FString::Printf(TEXT("%s-%s"), *InOwnerStackItemEditorDataKey, *InObject->GetName());
	Super::Initialize(InRequiredEntryData, InOwnerStackItemEditorDataKey, ObjectStackEditorDataKey);
	WeakObject = InObject;
	bIsTopLevel = bInIsTopLevelObject;
	bHideTopLevelCategories = bInHideTopLevelCategories;
	OwningNiagaraNode = InOwningNiagaraNode;
	FilterMode = EDetailNodeFilterMode::FilterRootNodesOnly;

	MessageLogGuid = GetSystemViewModel()->GetMessageLogGuid();

	FNiagaraMessageManager::Get()->SubscribeToAssetMessagesByObject(
		FText::FromString("StackObject")
		, MessageLogGuid
		, FObjectKey(InObject)
		, MessageManagerRegistrationKey
	).BindUObject(this, &UNiagaraStackObject::OnMessageManagerRefresh);

	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackObject::FilterOnlyModified));
}

void UNiagaraStackObject::Initialize(
		FRequiredEntryData InRequiredEntryData,
		UObject* InOwningObject,
		TSharedRef<FStructOnScope> InDisplayedStruct,
		const FString& InStructName,
		bool bInIsTopLevelStruct,
		bool bInHideTopLevelCategories,
		FString InOwnerStackItemEditorDataKey,
		UNiagaraNode* InOwningNiagaraNode)
{
	checkf(WeakObject.IsValid() == false && DisplayedStruct.IsValid() == false, TEXT("Can only initialize once."));
	FString ObjectStackEditorDataKey = FString::Printf(TEXT("%s-%s"), *InOwnerStackItemEditorDataKey, *InStructName);
	Super::Initialize(InRequiredEntryData, InOwnerStackItemEditorDataKey, ObjectStackEditorDataKey);
	WeakObject = InOwningObject;
	DisplayedStruct = InDisplayedStruct;
	bIsTopLevel = bInIsTopLevelStruct;
	bHideTopLevelCategories = bInHideTopLevelCategories;
	OwningNiagaraNode = InOwningNiagaraNode;

	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackObject::FilterOnlyModified));
}

void UNiagaraStackObject::SetOnFilterDetailNodes(FNiagaraStackObjectShared::FOnFilterDetailNodes InOnFilterDetailNodes, EDetailNodeFilterMode InFilterMode)
{
	OnFilterDetailNodesDelegate = InOnFilterDetailNodes;
	FilterMode = InFilterMode;
}

void UNiagaraStackObject::RegisterInstancedCustomPropertyLayout(UStruct* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate)
{
	checkf(PropertyRowGenerator.IsValid() == false, TEXT("Can not add additional customizations after children have been refreshed."));
	RegisteredClassCustomizations.Add({ Class, DetailLayoutDelegate });
}

void UNiagaraStackObject::RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier)
{
	checkf(PropertyRowGenerator.IsValid() == false, TEXT("Can not add additional customizations after children have been refreshed."));
	RegisteredPropertyCustomizations.Add({ PropertyTypeName, PropertyTypeLayoutDelegate, Identifier });
}

void UNiagaraStackObject::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	UObject* Object = GetObject();
	if (Object != nullptr && DisplayedStruct.IsValid() && DisplayedStruct->OwnsStructMemory() == false)
	{
		Object->Modify();
	}
}

void UNiagaraStackObject::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (GetObject() != nullptr)
	{
		UNiagaraDataInterface* DataInterfaceObject = Cast<UNiagaraDataInterface>(WeakObject.Get());
		if (DataInterfaceObject != nullptr)
		{
			// First we tell the Details Panel to refresh its error UI
			{
				DataInterfaceObject->RefreshErrors();
			}
		}
		
		TArray<UObject*> ChangedObjects;
		ChangedObjects.Add(GetObject());
		OnDataObjectModified().Broadcast(ChangedObjects, ENiagaraDataObjectChange::Changed);
	}
}

bool UNiagaraStackObject::GetIsEnabled() const
{
	return OwningNiagaraNode == nullptr || OwningNiagaraNode->GetDesiredEnabledState() == ENodeEnabledState::Enabled;
}

bool UNiagaraStackObject::GetShouldShowInStack() const
{
	return false;
}

UObject* UNiagaraStackObject::GetDisplayedObject() const
{
	return WeakObject.Get();
}

FHierarchyElementIdentity UNiagaraStackObject::DetermineSummaryIdentity() const
{
	FHierarchyElementIdentity Identity;
	Identity.Guids.Add(ObjectGuid.GetValue());
	return Identity;
}

void UNiagaraStackObject::FinalizeInternal()
{
	if (PropertyRowGenerator.IsValid())
	{
		PropertyRowGenerator->OnRowsRefreshed().RemoveAll(this);

		if (DisplayedStruct.IsValid())
		{
			PropertyRowGenerator->SetStructure(TSharedPtr<FStructOnScope>());
		}
		else
		{
			PropertyRowGenerator->SetObjects(TArray<UObject*>());
		}

		// Enqueue the row generator for destruction because stack entries might be finalized during the system view model tick
		// and you can't destruct tickables while other tickables are being ticked.
		FNiagaraEditorModule::Get().EnqueueObjectForDeferredDestruction(PropertyRowGenerator.ToSharedRef());
		PropertyRowGenerator.Reset();
	}

	if (MessageManagerRegistrationKey.IsValid())
	{
		FNiagaraMessageManager::Get()->Unsubscribe(FText::FromString("StackObject"), MessageLogGuid, MessageManagerRegistrationKey);
	}

	Super::FinalizeInternal();
}

void UNiagaraStackObject::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	FNiagaraEditorModule* NiagaraEditorModule = &FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	
	TFunction<void(uint8*, UStruct*, bool)> GatherIssueFromProperties;
	
	//Recurse into all our child properties and gather any issues they may generate via the INiagaraStackObjectIssueGenerator helper objects.
	GatherIssueFromProperties = [&](uint8* BasePtr, UStruct* InStruct, bool bRecurseChildren)
	{
		//TODO: Walk up the base class hierarchy. 
		//This class may not have an issue generator but it's base might.

		//Generate any issue for this property.
		if (INiagaraStackObjectIssueGenerator* IssueGenerator = NiagaraEditorModule->FindStackObjectIssueGenerator(InStruct->GetFName()))
		{
			IssueGenerator->GenerateIssues(BasePtr, this, NewIssues);
		}

		//Recurse into child properties to generate any issue there.
		for (TFieldIterator<FProperty> PropertyIt(InStruct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			uint8* PropPtr = BasePtr + PropertyIt->GetOffset_ForInternal();
			
			if (bRecurseChildren)
			{
				if (const FStructProperty* StructProp = CastField<const FStructProperty>(Property))
				{
					GatherIssueFromProperties(PropPtr, StructProp->Struct, bRecurseChildren);
				}
				else if (const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(Property))
				{
					if (ArrayProp->Inner->IsA<FStructProperty>())
					{
						const FStructProperty* StructInner = CastFieldChecked<const FStructProperty>(ArrayProp->Inner);

						FScriptArrayHelper ArrayHelper(ArrayProp, PropPtr);
						for (int32 ArrayEntryIndex = 0; ArrayEntryIndex < ArrayHelper.Num(); ++ArrayEntryIndex)
						{
							uint8* ArrayEntryData = ArrayHelper.GetRawPtr(ArrayEntryIndex);
							GatherIssueFromProperties(ArrayEntryData, StructInner->Struct, bRecurseChildren);
						}
					}
				}
				//Recursing to object refs seems to have some circular links causing explosions.
				//For now lets just recurse down structs. 
				//UObjects are mostly their own stack objects anyway.
// 				else if (const FObjectPropertyBase* ObjProperty = CastField<const FObjectPropertyBase>(Property))
// 				{
// 					GatherIssueFromProperties(PropPtr, ObjProperty->PropertyClass, bRecurseChildren);
// 				}
			}
		}
	};

	UObject* Object = WeakObject.Get();
	if (Object == nullptr &&  DisplayedStruct.IsValid() == false)
	{
		return;
	}

	if (Object != nullptr && DisplayedStruct.IsValid() == false)
	{
		GatherIssueFromProperties((uint8*)Object, Object->GetClass(), true);
	}

	if (GetSystemViewModel()->GetIsForDataProcessingOnly() == false && PropertyRowGenerator.IsValid() == false)
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FPropertyRowGeneratorArgs Args;
		Args.NotifyHook = this;
		PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(Args);

		for (FRegisteredClassCustomization& RegisteredClassCustomization : RegisteredClassCustomizations)
		{
			PropertyRowGenerator->RegisterInstancedCustomPropertyLayout(RegisteredClassCustomization.Class, RegisteredClassCustomization.DetailLayoutDelegate);
		}

		for (FRegisteredPropertyCustomization& RegisteredPropertyCustomization : RegisteredPropertyCustomizations)
		{
			PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(RegisteredPropertyCustomization.PropertyTypeName,
				RegisteredPropertyCustomization.PropertyTypeLayoutDelegate, RegisteredPropertyCustomization.Identifier);
		}

		if (DisplayedStruct.IsValid())
		{
			PropertyRowGenerator->SetStructure(DisplayedStruct);
		}
		else if (Object != nullptr)
		{
			TArray<UObject*> Objects;
			Objects.Add(Object);
			PropertyRowGenerator->SetObjects(Objects);
		}

		// Add the refresh delegate after setting the objects to prevent refreshing children immediately.
		PropertyRowGenerator->OnRowsRefreshed().AddUObject(this, &UNiagaraStackObject::PropertyRowsRefreshed);
	}


	if (!Object->HasAllFlags(EObjectFlags::RF_Transactional))
	{
		NewIssues.Add(FStackIssue(
			EStackIssueSeverity::Warning,
			NSLOCTEXT("StackObject", "ObjectNotTransactionalShort", "Object is not transctional, undo won't work for it!"),
			NSLOCTEXT("StackObject", "ObjectNotTransactionalLong", "Object is not transctional, undo won't work for it! Please report this to the Niagara dev team."),
			GetStackEditorDataKey(),
			false,
			{
				FStackIssueFix(
					NSLOCTEXT("StackObject","TransactionalFix", "Fix transactional status."),
					FStackIssueFixDelegate::CreateLambda(
					[WeakObject=this->WeakObject]()
					{ 
						if ( UObject* Object = WeakObject.Get() )
						{
							Object->SetFlags(RF_Transactional);
						}
					}
				)),
			}));
	}

	TArray<FString> DontCollapseCategoriesOverride;

	// TODO: Handle this in a more generic way.  Maybe add error apis to UNiagaraMergable, or use a UObject interface, or create a
	// TODO: Possibly move to use INiagaraStackObjectIssueGenerator interface.
	// data interface specific implementation of UNiagaraStackObject.
	UNiagaraDataInterface* DataInterfaceObject = Cast<UNiagaraDataInterface>(Object);
	if (DataInterfaceObject != nullptr)
	{
		// Generate the summary stack issue for any errors which are generated.
		TArray<FNiagaraDataInterfaceError> Errors;
		TArray<FNiagaraDataInterfaceFeedback> Warnings, Info;
		FNiagaraEditorModule::Get().GetDataInterfaceFeedbackSafe(DataInterfaceObject, Errors, Warnings, Info);

		if (Errors.Num() > 0)
		{
			NewIssues.Add(FStackIssue(
				EStackIssueSeverity::Error,
				NSLOCTEXT("StackObject", "ObjectErrorsShort", "Object has errors"),
				NSLOCTEXT("StackObject", "ObjectErrorsLong", "The displayed object has errors.  Check the object properties or the message log for details."),
				GetStackEditorDataKey(),
				false));
		}
		if (Warnings.Num() > 0)
		{
			NewIssues.Add(FStackIssue(
				EStackIssueSeverity::Warning,
				NSLOCTEXT("StackObject", "ObjectWarningsShort", "Object has warnings"),
				NSLOCTEXT("StackObject", "ObjectWarningsLong", "The displayed object has warnings.  Check the object properties or the message log for details."),
				GetStackEditorDataKey(),
				false));
		}

		DontCollapseCategoriesOverride.Add("Errors");
	}

	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	if (Object != nullptr && NiagaraEditorSettings->IsReferenceableClass(Object->GetClass()) == false)
	{
		NewIssues.Add(FStackIssue(
			EStackIssueSeverity::Error,
			NSLOCTEXT("StackObject", "InvalidClassShort", "Unsupported Object Type"),
			FText::Format(NSLOCTEXT("StackObject", "InvalidClassLongFormat", "Use of an object of type {0} is unsupported in the current editor context."), Object->GetClass()->GetDisplayNameText()),
			GetStackEditorDataKey(),
			false));
	}

	if(PropertyRowGenerator.IsValid())
	{
		PropertyRowGenerator->InvalidateCachedState();
		TArray<TSharedRef<IDetailTreeNode>> DefaultRootTreeNodes = PropertyRowGenerator->GetRootTreeNodes();
		TArray<TSharedRef<IDetailTreeNode>> RootTreeNodes;
		if (OnFilterDetailNodesDelegate.IsBound())
		{
			OnFilterDetailNodesDelegate.Execute(DefaultRootTreeNodes, RootTreeNodes);
		}
		else
		{
			RootTreeNodes = DefaultRootTreeNodes;
		}

		TArray<TSharedRef<IDetailTreeNode>> CollapsedRootTreeNodes;
		if(Object->GetClass()->HasAnyClassFlags(CLASS_CollapseCategories))
		{
			if(Object->GetClass()->HasMetaData("DontCollapseCategoriesOverride"))
			{
				FString MetaDataValue = Object->GetClass()->GetMetaData(TEXT("DontCollapseCategoriesOverride"));
				TArray<FString> MetaDataDontCollapseOverride;
				MetaDataValue.ParseIntoArray(MetaDataDontCollapseOverride, TEXT(" "), true);
				DontCollapseCategoriesOverride.Append(MetaDataDontCollapseOverride);
			}
			for (TSharedRef<IDetailTreeNode> RootTreeNode : RootTreeNodes)
			{
				if (RootTreeNode->GetNodeType() == EDetailNodeType::Advanced)
				{
					continue;
				}
				
				if(RootTreeNode->GetNodeType() == EDetailNodeType::Category && !DontCollapseCategoriesOverride.Contains(RootTreeNode->GetNodeName().ToString()))
				{
					TArray<TSharedRef<IDetailTreeNode>> RootChildren;
					RootTreeNode->GetChildren(RootChildren);
					CollapsedRootTreeNodes.Append(RootChildren);
				}
				else
				{
					CollapsedRootTreeNodes.Add(RootTreeNode);
				}
			}
		}

		if(!CollapsedRootTreeNodes.IsEmpty())
		{
			RootTreeNodes = CollapsedRootTreeNodes;
		}

		TSharedPtr<const FNiagaraStackObjectPropertyCustomization> Customization = NiagaraEditorModule->GetStackObjectCustomizationRegistry().GetCustomizationForStackObject(*this);
		for (TSharedRef<IDetailTreeNode> RootTreeNode : RootTreeNodes)
		{
			if (RootTreeNode->GetNodeType() == EDetailNodeType::Advanced)
			{
				continue;
			}

			UNiagaraStackPropertyRow* ChildRow = FindCurrentChildOfTypeByPredicate<UNiagaraStackPropertyRow>(CurrentChildren,
				[=](UNiagaraStackPropertyRow* CurrentChild) { return CurrentChild->GetDetailTreeNode() == RootTreeNode; });

			if (ChildRow == nullptr)
			{
				ChildRow = NewObject<UNiagaraStackPropertyRow>(this);
				ChildRow->Initialize(CreateDefaultChildRequiredData(), RootTreeNode, bIsTopLevel, bHideTopLevelCategories, GetOwnerStackItemEditorDataKey(), GetOwnerStackItemEditorDataKey(), OwningNiagaraNode);
				ChildRow->SetOwnerGuid(ObjectGuid);
				if (OnFilterDetailNodesDelegate.IsBound() && FilterMode == EDetailNodeFilterMode::FilterAllNodes)
				{
					ChildRow->SetOnFilterDetailNodes(OnFilterDetailNodesDelegate);
				}
				if(Customization)
				{
					ChildRow->SetPropertyCustomization(Customization);
				}
			}

			NewChildren.Add(ChildRow);
		}
	}

	NewIssues.Append(MessageManagerIssues);
}

void UNiagaraStackObject::PostRefreshChildrenInternal()
{
	Super::PostRefreshChildrenInternal();
}

bool UNiagaraStackObject::FilterOnlyModified(const UNiagaraStackEntry& Child) const
{
	if(GetStackEditorData().GetShowOnlyModified() == false)
	{
		return true;
	}
	
	const UNiagaraStackPropertyRow* PropertyRow = Cast<UNiagaraStackPropertyRow>(&Child);
	if (PropertyRow == nullptr)
	{
		return true;
	}

	// We filter out categories that themselves have no property rows to show.
	// We don't need to account for sub-categories (that are of type Item) as we only have top level categories here
	if(PropertyRow->GetDetailTreeNode()->GetNodeType() == EDetailNodeType::Category || PropertyRow->GetDetailTreeNode()->GetNodeType() == EDetailNodeType::Advanced)
	{
		TArray<UNiagaraStackPropertyRow*> FilteredChildrenOfChild;
		PropertyRow->GetFilteredChildrenOfType(FilteredChildrenOfChild);
		return FilteredChildrenOfChild.Num() > 0;
	}
	
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow->GetDetailTreeNode()->CreatePropertyHandle();
	if(PropertyHandle == nullptr)
	{
		return true;
	}
	
	if(PropertyHandle->CanResetToDefault())
	{
		return true;
	}

	return false;
}

void UNiagaraStackObject::PropertyRowsRefreshed()
{
	RefreshChildren();
}

void UNiagaraStackObject::OnMessageManagerRefresh(const TArray<TSharedRef<const INiagaraMessage>>& NewMessages)
{
	if (MessageManagerIssues.Num() != 0 || NewMessages.Num() != 0)
	{
		MessageManagerIssues.Reset();
		for (TSharedRef<const INiagaraMessage> Message : NewMessages)
		{
			FStackIssue Issue = FNiagaraMessageUtilities::MessageToStackIssue(Message, GetStackEditorDataKey());
			if (MessageManagerIssues.ContainsByPredicate([&Issue](const FStackIssue& NewIssue)
				{ return NewIssue.GetUniqueIdentifier() == Issue.GetUniqueIdentifier(); }) == false)
			{
				MessageManagerIssues.Add(Issue);
			}
		}

		RefreshChildren();
	}
}
