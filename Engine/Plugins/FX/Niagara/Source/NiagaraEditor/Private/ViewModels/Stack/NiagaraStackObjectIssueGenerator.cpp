// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackObjectIssueGenerator.h"
#include "NiagaraEmitter.h"
#include "NiagaraPlatformSet.h"
#include "NiagaraRendererProperties.h"

void FNiagaraPlatformSetIssueGenerator::GenerateIssues(void* Object, UNiagaraStackObject* StackObject, TArray<UNiagaraStackEntry::FStackIssue>& NewIssues)
{
	FNiagaraPlatformSet* PlatformSet = (FNiagaraPlatformSet*)Object;

	TArray<FName> InvalidCVars;
	for (FNiagaraPlatformSetCVarCondition& CVarCondition : PlatformSet->CVarConditions)
	{
		IConsoleVariable* CVar = CVarCondition.GetCVar();
		if (CVar == nullptr)
		{
			InvalidCVars.AddUnique(CVarCondition.CVarName);
		}
	}

	if (InvalidCVars.Num() > 0)
	{
		FText ShortMessageFormat;
		FText LongMessageFormat;
		if (UNiagaraEmitter* OwnerEmitter = Cast<UNiagaraEmitter>(StackObject->GetObject()))
		{
			ShortMessageFormat = NSLOCTEXT("StackObject", "EmitterInvalidCVarConditionShort", "Emitter Scalability has an invalid cvar '{0}'!");
			LongMessageFormat = NSLOCTEXT("StackObject", "EmitterInvalidCVarConditionLong", "Emitter Scalability has an invalid cvar '{0}'!");
		}
		else if (UNiagaraRendererProperties* OwnerRenderer = Cast<UNiagaraRendererProperties>(StackObject->GetObject()))
		{
			ShortMessageFormat = NSLOCTEXT("StackObject", "RendererInvalidCVarConditionShort", "Renderer Scalability has an invalid cvar '{0}'!");
			LongMessageFormat = NSLOCTEXT("StackObject", "RendererInvalidCVarConditionLong", "Renderer Scalability has an invalid cvar '{0}'!");
		}
		else
		{
			ShortMessageFormat = NSLOCTEXT("StackObject", "InvalidCVarConditionShort", "PlatformSet has an invalid cvar '{0}'!");
			LongMessageFormat = NSLOCTEXT("StackObject", "InvalidCVarConditionLong", "PlatformSet has an invalid cvar '{0}'!");
		}

		for (const FName& CVarName : InvalidCVars)
		{
			const FText CVarNameText = FText::FromName(CVarName);
			NewIssues.Emplace(
				EStackIssueSeverity::Error,
				FText::Format(ShortMessageFormat, CVarNameText),
				FText::Format(LongMessageFormat, CVarNameText),
				StackObject->GetStackEditorDataKey(),
				false,
				UNiagaraStackEntry::FStackIssueFix(
					FText::Format(NSLOCTEXT("StackObject", "RemoveInvalidCVar", "Remove invalid cvar '{0}'"), CVarNameText),
					UNiagaraStackEntry::FStackIssueFixDelegate::CreateStatic(FNiagaraPlatformSetIssueGenerator::FixIssue, PlatformSet, TWeakObjectPtr<UObject>(StackObject->GetObject()), CVarName)
				)
			);
		}
	}
}

void FNiagaraPlatformSetIssueGenerator::FixIssue(FNiagaraPlatformSet* PlatformSet, TWeakObjectPtr<UObject> WeakObject, FName CVarName)
{
	UObject* OwnerObject = WeakObject.Get();
	if (OwnerObject == nullptr)
	{
		return;
	}

	OwnerObject->Modify();
	PlatformSet->CVarConditions.RemoveAll(
		[&](const FNiagaraPlatformSetCVarCondition& CVarCondition)
		{
			return CVarCondition.CVarName == CVarName;
		}
	);
}
