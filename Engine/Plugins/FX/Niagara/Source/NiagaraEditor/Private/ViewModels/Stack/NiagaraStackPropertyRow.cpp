// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackPropertyRow.h"
#include "NiagaraClipboard.h"
#include "NiagaraNode.h"

#include "IDetailTreeNode.h"
#include "PropertyHandle.h"
#include "IDetailPropertyRow.h"
#include "ScopedTransaction.h"
#include "DataHierarchyViewModelBase.h"
#include "NiagaraStackEditorData.h"
#include "Customizations/NiagaraStackObjectPropertyCustomization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackPropertyRow)

#define LOCTEXT_NAMESPACE "NiagaraStackPropertyRow"

void UNiagaraStackPropertyRow::Initialize(
	FRequiredEntryData InRequiredEntryData,
	TSharedRef<IDetailTreeNode> InDetailTreeNode,
	bool bInIsTopLevelProperty,
	bool bInHideTopLevelCategories,
	FString InOwnerStackItemEditorDataKey,
	FString InOwnerStackEditorDataKey,
	UNiagaraNode* InOwningNiagaraNode)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = InDetailTreeNode->CreatePropertyHandle();
	FString RowStackEditorDataKey = FString::Printf(TEXT("%s-%s"), *InOwnerStackEditorDataKey, *InDetailTreeNode->GetNodeName().ToString());
	Super::Initialize(InRequiredEntryData, InOwnerStackItemEditorDataKey, RowStackEditorDataKey);
	bool bRowIsAdvanced = PropertyHandle.IsValid() && PropertyHandle->GetProperty() && PropertyHandle->GetProperty()->HasAnyPropertyFlags(CPF_AdvancedDisplay);
	SetIsAdvanced(bRowIsAdvanced);
	DetailTreeNode = InDetailTreeNode;
	bIsTopLevelProperty = bInIsTopLevelProperty;
	bHideTopLevelCategories = bInHideTopLevelCategories;
	OwningNiagaraNode = InOwningNiagaraNode;
	CategorySpacer = nullptr;
	if (DetailTreeNode->GetNodeType() == EDetailNodeType::Category)
	{
		RowStyle = bInIsTopLevelProperty ? EStackRowStyle::ItemCategory : EStackRowStyle::ItemSubCategory;
	}
	else
	{
		RowStyle = EStackRowStyle::ItemContent;
	}
	bCannotEditInThisContext = false;
	if (PropertyHandle.IsValid() && PropertyHandle.Get() && PropertyHandle->GetProperty())
	{
		FProperty* Prop = PropertyHandle->GetProperty();
		FObjectPropertyBase* ObjProp = CastField<FObjectPropertyBase>(Prop);
		if (ObjProp && ObjProp->PropertyClass && (ObjProp->PropertyClass->IsChildOf(AActor::StaticClass()) || ObjProp->PropertyClass->IsChildOf(UActorComponent::StaticClass())))
		{
			bCannotEditInThisContext = true;
		}
	}
	bIsHiddenCategory = false;
	
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackPropertyRow::FilterOnlyModified));
}

TSharedRef<IDetailTreeNode> UNiagaraStackPropertyRow::GetDetailTreeNode() const
{
	return DetailTreeNode.ToSharedRef();
}

bool UNiagaraStackPropertyRow::GetIsEnabled() const
{
	if (bCannotEditInThisContext) 
		return false;
	return OwningNiagaraNode == nullptr || OwningNiagaraNode->GetDesiredEnabledState() == ENodeEnabledState::Enabled;
}

void UNiagaraStackPropertyRow::SetPropertyCustomization(TSharedPtr<const FNiagaraStackObjectPropertyCustomization> Customization)
{
	PropertyCustomization = Customization;
}

UNiagaraStackEntry::EStackRowStyle UNiagaraStackPropertyRow::GetStackRowStyle() const
{
	return RowStyle;
}

bool UNiagaraStackPropertyRow::GetShouldShowInStack() const
{
	if (DetailTreeNode == nullptr)
	{
		return bIsHiddenCategory == false;
	}

	if(DetailTreeNode->GetNodeType() == EDetailNodeType::Category || DetailTreeNode->GetNodeType() == EDetailNodeType::Advanced)
	{
		if(bIsHiddenCategory)
		{
			return false;
		}
		
		TArray<UNiagaraStackEntry*> CurrentFilteredChildren;
		GetFilteredChildren(CurrentFilteredChildren);
		int32 EmptyCount = CategorySpacer == nullptr ? 0 : 1;

		return CurrentFilteredChildren.Num() > EmptyCount;
	}
	
	return true;
}

bool UNiagaraStackPropertyRow::HasOverridenContent() const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeNode->CreatePropertyHandle();
	if (PropertyHandle.IsValid() && PropertyHandle.Get())
	{
		return PropertyHandle->DiffersFromDefault();
	}
	return false;
}

bool UNiagaraStackPropertyRow::IsExpandedByDefault() const
{
	return DetailTreeNode->GetInitiallyCollapsed() == false;
}

bool UNiagaraStackPropertyRow::CanDrag() const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeNode->CreatePropertyHandle();
	return PropertyHandle.IsValid() && PropertyHandle->GetParentHandle().IsValid() && PropertyHandle->GetParentHandle()->AsArray().IsValid();
}

bool UNiagaraStackPropertyRow::SupportsCopy() const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeNode->CreatePropertyHandle();
	return PropertyHandle.IsValid() && PropertyHandle->IsValidHandle();
}

bool UNiagaraStackPropertyRow::TestCanCopyWithMessage(FText& OutMessage) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeNode->CreatePropertyHandle();
	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle() && FNiagaraClipboardPortableValue::CreateFromPropertyHandle(*PropertyHandle.Get()).IsValid())
	{
		OutMessage = LOCTEXT("CopyMessage", "Copy the value of this property.");
		return true;
	}
	else
	{
		OutMessage = LOCTEXT("CantCopyMessage", "This row does not support copying.");
		return false;
	}
}

void UNiagaraStackPropertyRow::Copy(UNiagaraClipboardContent* ClipboardContent) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeNode->CreatePropertyHandle();
	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		FNiagaraClipboardPortableValue PortableValue = FNiagaraClipboardPortableValue::CreateFromPropertyHandle(*PropertyHandle.Get());
		if (PortableValue.IsValid())
		{
			ClipboardContent->PortableValues.Add(PortableValue);
		}
	}
}

bool UNiagaraStackPropertyRow::SupportsPaste() const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeNode->CreatePropertyHandle();
	return PropertyHandle.IsValid() && PropertyHandle->IsValidHandle();
}

bool UNiagaraStackPropertyRow::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	if (ClipboardContent->PortableValues.Num() != 1 || ClipboardContent->PortableValues[0].IsValid() == false)
	{
		return false;
	}

	TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeNode->CreatePropertyHandle();
	if (PropertyHandle.IsValid() == false || PropertyHandle->IsValidHandle() == false)
	{
		OutMessage = LOCTEXT("CantPasteMessage", "Can not paste the clipboard value to this row.");
		return false;
	}
	else
	{
		OutMessage = LOCTEXT("PasteMessage", "Paste the value from the clipboard to this property.");
		return true;
	}
}

FText UNiagaraStackPropertyRow::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	return LOCTEXT("PasteValueToPropertyTransaction", "Paste value to property.");
}

void UNiagaraStackPropertyRow::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeNode->CreatePropertyHandle();
	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle() &&
		ClipboardContent->PortableValues.Num() == 1 && ClipboardContent->PortableValues[0].IsValid())
	{
		if (ClipboardContent->PortableValues[0].TryUpdatePropertyHandle(*PropertyHandle.Get()) == false)
		{
			OutPasteWarning = LOCTEXT("PasteFailWarning", "Failed to paste the value from the clipboard");
		}
	}
}

void UNiagaraStackPropertyRow::FinalizeInternal()
{
	Super::FinalizeInternal();
	DetailTreeNode.Reset();
}

void UNiagaraStackPropertyRow::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	RefreshCustomization();
	
	TArray<TSharedRef<IDetailTreeNode>> AllNodeChildren;
	DetailTreeNode->GetChildren(AllNodeChildren);

	TArray<TSharedRef<IDetailTreeNode>> NodeChildren;
	if (OnFilterDetailNodes.IsBound())
	{
		OnFilterDetailNodes.Execute(AllNodeChildren, NodeChildren);
	}
	else
	{
		NodeChildren = AllNodeChildren;
	}

	bIsHiddenCategory = DetailTreeNode->GetNodeType() == EDetailNodeType::Category && (NodeChildren.Num() == 0 || (bIsTopLevelProperty && bHideTopLevelCategories));
	for (TSharedRef<IDetailTreeNode> NodeChild : NodeChildren)
	{
		if (NodeChild->GetNodeType() == EDetailNodeType::Advanced)
		{
			continue;
		}

		UNiagaraStackPropertyRow* ChildRow = FindCurrentChildOfTypeByPredicate<UNiagaraStackPropertyRow>(CurrentChildren,
			[=](UNiagaraStackPropertyRow* CurrentChild) { return CurrentChild->GetDetailTreeNode() == NodeChild; });

		if (ChildRow == nullptr)
		{
			bool bChildIsTopLevelProperty = false;
			ChildRow = NewObject<UNiagaraStackPropertyRow>(this);
			ChildRow->Initialize(CreateDefaultChildRequiredData(), NodeChild, bChildIsTopLevelProperty, bHideTopLevelCategories, GetOwnerStackItemEditorDataKey(), GetStackEditorDataKey(), OwningNiagaraNode);
			ChildRow->SetOwnerGuid(OwnerGuid);
			if(PropertyCustomization.IsValid())
			{
				ChildRow->SetPropertyCustomization(PropertyCustomization);
			}
		}

		NewChildren.Add(ChildRow);
	}

	if (bIsTopLevelProperty && DetailTreeNode->GetNodeType() == EDetailNodeType::Category)
	{
		if (CategorySpacer == nullptr)
		{
			CategorySpacer = NewObject<UNiagaraStackSpacer>(this);
			TAttribute<bool> ShouldShowSpacerInStack;
			ShouldShowSpacerInStack.BindUObject(this, &UNiagaraStackPropertyRow::GetShouldShowInStack);
			CategorySpacer->Initialize(CreateDefaultChildRequiredData(), 6, ShouldShowSpacerInStack, GetStackEditorDataKey());
		}
		NewChildren.Add(CategorySpacer);
	}
}

void UNiagaraStackPropertyRow::RefreshCustomization()
{
	if(PropertyCustomization.IsValid())
	{
		//PropertyCustomization->GenerateNameWidget(DetailTreeNode.ToSharedRef());
	}
}

int32 UNiagaraStackPropertyRow::GetChildIndentLevel() const
{
	// We want to keep inputs under a top level category at the same indent level as the category.
	return bIsTopLevelProperty && DetailTreeNode->GetNodeType() == EDetailNodeType::Category ? GetIndentLevel() : Super::GetChildIndentLevel();
}

void UNiagaraStackPropertyRow::GetSearchItems(TArray<FStackSearchItem>& SearchItems) const
{
	SearchItems.Add({ FName("DisplayName"), GetDisplayName() });

	TArray<FString> NodeFilterStrings;
	DetailTreeNode->GetFilterStrings(NodeFilterStrings);
	for (FString& FilterString : NodeFilterStrings)
	{
		SearchItems.Add({ "PropertyRowFilterString", FText::FromString(FilterString) });
	}

	TSharedPtr<IDetailPropertyRow> DetailPropertyRow = DetailTreeNode->GetRow();
	if (DetailPropertyRow.IsValid())
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailPropertyRow->GetPropertyHandle();
		if (PropertyHandle)
		{
			FText PropertyRowHandleText;
			PropertyHandle->GetValueAsDisplayText(PropertyRowHandleText);
			SearchItems.Add({ "PropertyRowHandleText", PropertyRowHandleText });
		}
	}	
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackPropertyRow::CanDropInternal(const FDropRequest& DropRequest)
{
	// Validate stack, drop zone, and drag type.
	if (DropRequest.DropOptions == UNiagaraStackEntry::EDropOptions::Overview ||
		DropRequest.DropZone == EItemDropZone::OntoItem ||
		DropRequest.DragDropOperation->IsOfType<FNiagaraStackEntryDragDropOp>() == false)
	{
		return TOptional<FDropRequestResponse>();
	}

	// Validate stack entry count and type.
	TSharedRef<const FNiagaraStackEntryDragDropOp> StackEntryDragDropOp = StaticCastSharedRef<const FNiagaraStackEntryDragDropOp>(DropRequest.DragDropOperation);
	if (StackEntryDragDropOp->GetDraggedEntries().Num() != 1 || StackEntryDragDropOp->GetDraggedEntries()[0]->IsA<UNiagaraStackPropertyRow>() == false)
	{
		return TOptional<FDropRequestResponse>();
	}

	auto HaveSameParent = [](TSharedPtr<IPropertyHandle> HandleA, TSharedPtr<IPropertyHandle> HandleB)
	{
		TSharedPtr<IPropertyHandle> ParentA = HandleA->GetParentHandle();
		TSharedPtr<IPropertyHandle> ParentB = HandleB->GetParentHandle();
		if (ParentA.IsValid() && ParentB.IsValid() && ParentA->GetProperty() == ParentB->GetProperty())
		{
			TArray<UObject*> OuterObjectsA;
			ParentA->GetOuterObjects(OuterObjectsA);
			TArray<UObject*> OuterObjectsB;
			ParentB->GetOuterObjects(OuterObjectsB);
			return OuterObjectsA == OuterObjectsB;
		}
		return false;
	};

	// Validate property handle.
	UNiagaraStackPropertyRow* DraggedPropertyRow = CastChecked<UNiagaraStackPropertyRow>(StackEntryDragDropOp->GetDraggedEntries()[0]);
	TSharedPtr<IPropertyHandle> DraggedPropertyHandle = DraggedPropertyRow->GetDetailTreeNode()->CreatePropertyHandle();
	TSharedPtr<IPropertyHandle> TargetPropertyHandle = DetailTreeNode->CreatePropertyHandle();
	if (DraggedPropertyHandle.IsValid() == false ||
		TargetPropertyHandle.IsValid() == false ||
		DraggedPropertyHandle == TargetPropertyHandle ||
		DraggedPropertyHandle->GetParentHandle().IsValid() == false ||
		HaveSameParent(DraggedPropertyHandle, TargetPropertyHandle) == false ||
		DraggedPropertyHandle->GetParentHandle()->AsArray().IsValid() == false)
	{
		return TOptional<FDropRequestResponse>();
	}
	return FDropRequestResponse(DropRequest.DropZone, NSLOCTEXT("NiagaraStackPropertyRow", "DropArrayItemMessage", "Move this array entry here."));
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackPropertyRow::DropInternal(const FDropRequest& DropRequest)
{
	TOptional<FDropRequestResponse> CanDropResponse = CanDropInternal(DropRequest);
	if (CanDropResponse.IsSet())
	{
		TSharedRef<const FNiagaraStackEntryDragDropOp> StackEntryDragDropOp = StaticCastSharedRef<const FNiagaraStackEntryDragDropOp>(DropRequest.DragDropOperation);
		UNiagaraStackPropertyRow* DraggedPropertyRow = CastChecked<UNiagaraStackPropertyRow>(StackEntryDragDropOp->GetDraggedEntries()[0]);
		TSharedPtr<IPropertyHandle> DraggedPropertyHandle = DraggedPropertyRow->GetDetailTreeNode()->CreatePropertyHandle();
		TSharedPtr<IPropertyHandle> TargetPropertyHandle = DetailTreeNode->CreatePropertyHandle();
		TSharedPtr<IPropertyHandle> ParentHandle = TargetPropertyHandle->GetParentHandle();
		int32 IndexOffset = DropRequest.DropZone == EItemDropZone::AboveItem ? 0 : 1;
		uint32 NumElements = 0;
		ParentHandle->AsArray()->GetNumElements(NumElements);

		// we clamp the offset
		if((uint32) (TargetPropertyHandle->GetIndexInArray() + IndexOffset) >= (NumElements - 1))
		{
			IndexOffset = 0;
		}

		FScopedTransaction Transaction(NSLOCTEXT("NiagaraStackPropertyRow", "DropArrayItem", "Move Array Item"));
		ParentHandle->NotifyPreChange();
		ParentHandle->AsArray()->MoveElementTo(DraggedPropertyHandle->GetIndexInArray(), TargetPropertyHandle->GetIndexInArray() + IndexOffset);
		ParentHandle->NotifyPostChange(EPropertyChangeType::ArrayMove);
	}
	return CanDropResponse;
}

bool UNiagaraStackPropertyRow::SupportsSummaryView() const
{
	return OwnerGuid.IsSet() && OwnerGuid->IsValid() && DetailTreeNode->GetNodeType() == EDetailNodeType::Item;
}

FHierarchyElementIdentity UNiagaraStackPropertyRow::DetermineSummaryIdentity() const
{
	FHierarchyElementIdentity Identity;
	Identity.Guids.Add(OwnerGuid.GetValue());
	Identity.Names.Add(DetailTreeNode->GetNodeName());
	return Identity;
}

bool UNiagaraStackPropertyRow::FilterOnlyModified(const UNiagaraStackEntry& Child) const
{
	if(GetStackEditorData().GetShowOnlyModified() == false)
	{
		return true;
	}
	
	const UNiagaraStackPropertyRow* PropertyRow = Cast<UNiagaraStackPropertyRow>(&Child);
	if (PropertyRow == nullptr)
	{
		return true;
	}

	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow->GetDetailTreeNode()->CreatePropertyHandle();
	EDetailNodeType DetailNodeType = PropertyRow->GetDetailTreeNode()->GetNodeType();
	
	// We filter out categories that themselves have no property rows to show
	// If this is an item but the property handle is nullptr, this is a sub-category (thanks Property Editor..)
	if((DetailNodeType == EDetailNodeType::Item && PropertyHandle == nullptr) || DetailNodeType == EDetailNodeType::Category || DetailNodeType == EDetailNodeType::Advanced)
	{
		TArray<UNiagaraStackPropertyRow*> FilteredChildrenOfChild;
		PropertyRow->GetFilteredChildrenOfType(FilteredChildrenOfChild);
		return FilteredChildrenOfChild.Num() > 0;
	}
	
	if(PropertyHandle == nullptr)
	{
		return true;
	}
	
	if(PropertyHandle->CanResetToDefault())
	{
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
