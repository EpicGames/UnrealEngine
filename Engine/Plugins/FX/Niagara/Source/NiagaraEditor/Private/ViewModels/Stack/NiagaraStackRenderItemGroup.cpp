// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackRenderItemGroup.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackRenderersOwner.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraEmitter.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "NiagaraEmitterEditorData.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraClipboard.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraStackEditorData.h"

#include "ScopedTransaction.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Styling/CoreStyle.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackRenderItemGroup)

#define LOCTEXT_NAMESPACE "UNiagaraStackRenderItemGroup"

class FRenderItemGroupAddAction : public INiagaraStackItemGroupAddAction
{
public:
	FRenderItemGroupAddAction(FNiagaraRendererCreationInfo InRendererCreationInfo)
		: RendererCreationInfo(InRendererCreationInfo)
	{
	}

	virtual TArray<FString> GetCategories() const override
	{
		return {};
	}

	virtual FText GetDisplayName() const override
	{
		return RendererCreationInfo.DisplayName;
	}

	virtual FText GetDescription() const override
	{
		return RendererCreationInfo.Description;
	}

	virtual FText GetKeywords() const override
	{
		return FText();
	}

	virtual TOptional<FNiagaraFavoritesActionData> GetFavoritesData() const override
	{
		return TOptional<FNiagaraFavoritesActionData>();
	}
	
	virtual bool IsInLibrary() const override
	{
		return true;
	}
	
	virtual FNiagaraActionSourceData GetSourceData() const override
	{
		return FNiagaraActionSourceData();
	}

	virtual TOptional<FSoftObjectPath> GetPreviewMoviePath() const override
	{
		return {};
	}

	const FNiagaraRendererCreationInfo& GetRendererCreationInfo() { return RendererCreationInfo; }

private:
	FNiagaraRendererCreationInfo RendererCreationInfo;
};

class FRenderItemGroupAddUtilities : public TNiagaraStackItemGroupAddUtilities<UNiagaraRendererProperties*>
{
public:
	FRenderItemGroupAddUtilities(TSharedPtr<INiagaraStackRenderersOwner> InRenderersOwner, FRenderItemGroupAddUtilities::FOnItemAdded OnItemAdded = FRenderItemGroupAddUtilities::FOnItemAdded())
		: TNiagaraStackItemGroupAddUtilities(LOCTEXT("RenderGroupAddItemName", "Renderer"), EAddMode::AddFromAction, true, false, OnItemAdded)
		, RenderersOwner(InRenderersOwner)
	{
	}

	virtual void AddItemDirectly() override { unimplemented(); }

	virtual void GenerateAddActions(TArray<TSharedRef<INiagaraStackItemGroupAddAction>>& OutAddActions, const FNiagaraStackItemGroupAddOptions& AddProperties) const override
	{
		const TArray<FNiagaraRendererCreationInfo>& RendererCreationInfos = FNiagaraEditorModule::Get().GetRendererCreationInfos();

		const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
		for (const FNiagaraRendererCreationInfo& RendererCreationInfo : RendererCreationInfos)
		{
			FSoftClassPath SoftClassPath(RendererCreationInfo.RendererClassPath.ToString());
			UClass* RendererClass = SoftClassPath.ResolveClass();
			if (NiagaraEditorSettings->IsVisibleClass(RendererClass) && RenderersOwner->SupportsRendererClass(RendererClass))
			{
				OutAddActions.Add(MakeShared<FRenderItemGroupAddAction>(RendererCreationInfo));
			}
		}
	}

	virtual void ExecuteAddAction(TSharedRef<INiagaraStackItemGroupAddAction> AddAction, int32 TargetIndex) override
	{
		if (RenderersOwner.IsValid() == false || RenderersOwner->IsValid() == false)
		{
			return;
		}

		TSharedRef<FRenderItemGroupAddAction> RenderAddAction = StaticCastSharedRef<FRenderItemGroupAddAction>(AddAction);

		FScopedTransaction ScopedTransaction(LOCTEXT("AddNewRendererTransaction", "Add new renderer"));

		UNiagaraRendererProperties* RendererProperties = RenderAddAction->GetRendererCreationInfo().RendererFactory.Execute(RenderersOwner->GetOwnerObject());
		check(RendererProperties != nullptr);
		
		RenderersOwner->AddRenderer(RendererProperties);
		OnItemAdded.ExecuteIfBound(RendererProperties);
	}

private:
	TSharedPtr<INiagaraStackRenderersOwner> RenderersOwner;
};

void UNiagaraStackRenderItemGroup::Initialize(FRequiredEntryData InRequiredEntryData, TSharedPtr<INiagaraStackRenderersOwner> InRenderersOwner)
{
	FText DisplayName = LOCTEXT("RenderGroupName", "Render");
	FText ToolTip = LOCTEXT("RendererGroupTooltip", "Describes how we should display/present each particle. Note that this doesn't have to be visual. Multiple renderers are supported. Order in this stack is not necessarily relevant to draw order.");
	RenderersOwner = InRenderersOwner;
	RenderersOwner->OnRenderersChanged().BindUObject(this, &UNiagaraStackRenderItemGroup::OwnerRenderersChanged);
	AddUtilities = MakeShared<FRenderItemGroupAddUtilities>(RenderersOwner, FRenderItemGroupAddUtilities::FOnItemAdded::CreateUObject(this, &UNiagaraStackRenderItemGroup::OnRendererAdded));
	Super::Initialize(InRequiredEntryData, DisplayName, ToolTip, AddUtilities.Get());
}

UNiagaraStackEntry::EIconMode UNiagaraStackRenderItemGroup::GetSupportedIconMode() const
{
	return RenderersOwner.IsValid() ? RenderersOwner->GetSupportedIconMode() : EIconMode::None;
}

const FSlateBrush* UNiagaraStackRenderItemGroup::GetIconBrush() const
{
	return RenderersOwner.IsValid() ? RenderersOwner->GetIconBrush() : nullptr;
}

FText UNiagaraStackRenderItemGroup::GetIconText() const
{
	return RenderersOwner.IsValid() ? RenderersOwner->GetIconText() : FText();
}

bool UNiagaraStackRenderItemGroup::GetCanExpandInOverview() const
{
	return RenderersOwner.IsValid() && RenderersOwner->ShouldShowRendererItemsInOverview();
}

bool UNiagaraStackRenderItemGroup::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	if (ClipboardContent->Renderers.Num() > 0)
	{
		OutMessage = LOCTEXT("PasteRenderers", "Paste renderers from the clipboard.");
		return true;
	}
	OutMessage = FText();
	return false;
}

FText UNiagaraStackRenderItemGroup::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	return LOCTEXT("PasteRenderersTransactionText", "Paste renderers");
}

void UNiagaraStackRenderItemGroup::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	if (RenderersOwner.IsValid() && RenderersOwner->IsValid())
	{
		for (const UNiagaraClipboardRenderer* ClipboardRenderer : ClipboardContent->Renderers)
		{
			if (!ClipboardRenderer || !ClipboardRenderer->RendererProperties)
			{
				continue;
			}

			if (!RenderersOwner->SupportsRendererClass(ClipboardRenderer->RendererProperties->GetClass()))
			{
				continue;
			}

			UNiagaraRendererProperties* NewRenderer = ClipboardRenderer->RendererProperties->StaticDuplicateWithNewMergeId(RenderersOwner->GetOwnerObject());
			RenderersOwner->AddRenderer(NewRenderer);
			if(ClipboardRenderer->StackNoteData.IsSet())
			{
				GetStackEditorData().AddOrReplaceStackNote(FNiagaraStackGraphUtilities::StackKeys::GenerateStackRendererEditorDataKey(*NewRenderer), ClipboardRenderer->StackNoteData.GetValue());
			}
			if (ClipboardRenderer->AlternateDisplayName.IsSet())
			{
				GetStackEditorData().SetStackEntryDisplayName(FNiagaraStackGraphUtilities::StackKeys::GenerateStackRendererEditorDataKey(*NewRenderer), ClipboardRenderer->AlternateDisplayName.GetValue());
			}
		}
	}
}

void UNiagaraStackRenderItemGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	int32 RendererIndex = 0;
	if (RenderersOwner.IsValid())
	{
		TArray<UNiagaraRendererProperties*> Renderers;
		RenderersOwner->GetRenderers(Renderers);
		for (UNiagaraRendererProperties* RendererProperties : Renderers)
		{
			UNiagaraStackRendererItem* RendererItem = FindCurrentChildOfTypeByPredicate<UNiagaraStackRendererItem>(CurrentChildren,
				[=](UNiagaraStackRendererItem* CurrentRendererItem) { return CurrentRendererItem->GetRendererProperties() == RendererProperties; });

			if (RendererItem == nullptr)
			{
				RendererItem = NewObject<UNiagaraStackRendererItem>(this);
				RendererItem->Initialize(CreateDefaultChildRequiredData(), RenderersOwner, RendererProperties);
				RendererItem->SetOnRequestCanPaste(UNiagaraStackRendererItem::FOnRequestCanPaste::CreateUObject(this, &UNiagaraStackRenderItemGroup::ChildRequestCanPaste));
				RendererItem->SetOnRequestPaste(UNiagaraStackRendererItem::FOnRequestPaste::CreateUObject(this, &UNiagaraStackRenderItemGroup::ChildRequestPaste));
			}

			NewChildren.Add(RendererItem);

			RendererIndex++;
		}
	}

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

void UNiagaraStackRenderItemGroup::OwnerRenderersChanged()
{
	if (IsFinalized() == false)
	{
		// With undo/redo sometimes it's not possible to unbind this delegate, so we have to check to insure safety in those cases.
		OnDataObjectModified().Broadcast(TArray<UObject*>(), ENiagaraDataObjectChange::Unknown);
		RefreshChildren();
	}
}

bool UNiagaraStackRenderItemGroup::ChildRequestCanPaste(const UNiagaraClipboardContent* ClipboardContent, FText& OutCanPasteMessage)
{
	return TestCanPasteWithMessage(ClipboardContent, OutCanPasteMessage);
}

void UNiagaraStackRenderItemGroup::ChildRequestPaste(const UNiagaraClipboardContent* ClipboardContent, int32 PasteIndex, FText& OutPasteWarning)
{
	Paste(ClipboardContent, OutPasteWarning);
}

void UNiagaraStackRenderItemGroup::OnRendererAdded(UNiagaraRendererProperties* RendererProperties) const
{
	GetSystemViewModel()->GetSelectionViewModel()->EmptySelection();
	GetSystemViewModel()->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectKeyDeferred(FObjectKey(RendererProperties));
	GetSystemViewModel()->GetSystemStackViewModel()->RequestValidationUpdate();
}

void UNiagaraStackRenderItemGroup::FinalizeInternal()
{
	if (RenderersOwner.IsValid())
	{
		RenderersOwner->OnRenderersChanged().Unbind();
		RenderersOwner.Reset();
	}
	Super::FinalizeInternal();
}

#undef LOCTEXT_NAMESPACE


