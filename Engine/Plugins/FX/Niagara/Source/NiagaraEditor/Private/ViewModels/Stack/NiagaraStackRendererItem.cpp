// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackRenderersOwner.h"
#include "NiagaraEmitter.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraScript.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "Internationalization/Internationalization.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraConstants.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "NiagaraGraph.h"
#include "ScopedTransaction.h"
#include "NiagaraScriptSource.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "NiagaraScriptMergeManager.h"
#include "NiagaraClipboard.h"
#include "NiagaraEmitterEditorData.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "NiagaraEditorStyle.h"

#include "Materials/MaterialInterface.h"
#include "MaterialEditorValidation.h"
#include "SourceCodeNavigation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackRendererItem)

#define LOCTEXT_NAMESPACE "UNiagaraStackRendererItem"

UNiagaraStackRendererItem::UNiagaraStackRendererItem()
	: RendererObject(nullptr)
{
}

void UNiagaraStackRendererItem::Initialize(FRequiredEntryData InRequiredEntryData, TSharedPtr<INiagaraStackRenderersOwner> InRenderersOwner, UNiagaraRendererProperties* InRendererProperties)
{
	checkf(RendererProperties.IsValid() == false, TEXT("Can not initialize more than once."));
	Super::Initialize(InRequiredEntryData, FNiagaraStackGraphUtilities::StackKeys::GenerateStackRendererEditorDataKey(*InRendererProperties));
	RenderersOwner = InRenderersOwner;
	RendererProperties = InRendererProperties;
	RendererProperties->OnChanged().AddUObject(this, &UNiagaraStackRendererItem::RendererChanged);
}

void UNiagaraStackRendererItem::FinalizeInternal()
{
	if (RendererProperties.IsValid())
	{
		RendererProperties->OnChanged().RemoveAll(this);
	}
	Super::FinalizeInternal();
}

const UNiagaraStackRendererItem::FCollectedUsageData& UNiagaraStackRendererItem::GetCollectedUsageData() const
{

	if (CachedCollectedUsageData.IsSet() == false)
	{
		CachedCollectedUsageData = FCollectedUsageData();

		if (RendererProperties.IsValid())
		{
			TArray<FNiagaraVariable> BoundAttribs = RendererProperties->GetBoundAttributes();
			TSharedRef<FNiagaraSystemViewModel> SystemVM = GetSystemViewModel();
			INiagaraParameterPanelViewModel* ParamVM = SystemVM->GetParameterPanelViewModel();

			FNiagaraAliasContext ResolveAliasesContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript);
			
			if (GetEmitterViewModel().IsValid())
			{
				TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleVM = SystemVM->GetEmitterHandleViewModelForEmitter(GetEmitterViewModel().Get()->GetEmitter());
				if (EmitterHandleVM.IsValid() && EmitterHandleVM->GetEmitterHandle())
				{
					ResolveAliasesContext.ChangeEmitterNameToEmitter(EmitterHandleVM->GetEmitterHandle()->GetUniqueInstanceName());
				}
			}

			if (ParamVM)
			{
				bool bFoundMatch = false;
				for (FNiagaraVariable Var : BoundAttribs)
				{			
					Var = FNiagaraUtilities::ResolveAliases(Var, ResolveAliasesContext);
					bFoundMatch = ParamVM->IsVariableSelected(Var);
					if (bFoundMatch)
					{
						break;
					}
				}

				CachedCollectedUsageData.GetValue().bHasReferencedParameterRead = bFoundMatch;
			}
		}
	}

	return CachedCollectedUsageData.GetValue();
}

bool UNiagaraStackRendererItem::CanMoveRendererUp() const
{
	if (HasBaseRenderer() || RendererProperties.IsValid() == false)
	{
		return false;
	}
	if (RenderersOwner.IsValid() && RenderersOwner->IsValid())
	{
		TArray<UNiagaraRendererProperties*> Renderers;
		RenderersOwner->GetRenderers(Renderers);
		return Renderers.IndexOfByKey(RendererProperties.Get()) > 0;
	}
	return false;
}

void UNiagaraStackRendererItem::MoveRendererUp() const
{
	if (!CanMoveRendererUp())
	{
		return;
	}

	if (RenderersOwner.IsValid() && RenderersOwner->IsValid())
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("MoveRendererUpTransaction", "Move renderer up"));
		TArray<UNiagaraRendererProperties*> Renderers;
		RenderersOwner->GetRenderers(Renderers);
		int32 CurrentIndex = Renderers.IndexOfByKey(RendererProperties.Get());
		RenderersOwner->MoveRenderer(RendererProperties.Get(), CurrentIndex - 1);
	}
}

bool UNiagaraStackRendererItem::CanMoveRendererDown() const
{
	if (HasBaseRenderer() || RendererProperties.IsValid() == false)
	{
		return false;
	}
	if (RenderersOwner.IsValid() && RenderersOwner->IsValid())
	{
		TArray<UNiagaraRendererProperties*> Renderers;
		RenderersOwner->GetRenderers(Renderers);
		return Renderers.IndexOfByKey(RendererProperties.Get()) < Renderers.Num() - 1;
	}
	return false;
}

void UNiagaraStackRendererItem::MoveRendererDown() const
{
	if (!CanMoveRendererDown())
	{
		return;
	}

	if (RenderersOwner.IsValid() && RenderersOwner->IsValid())
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("MoveRendererDownTransaction", "Move renderer down"));
		TArray<UNiagaraRendererProperties*> Renderers;
		RenderersOwner->GetRenderers(Renderers);
		int32 CurrentIndex = Renderers.IndexOfByKey(RendererProperties.Get());
		RenderersOwner->MoveRenderer(RendererProperties.Get(), CurrentIndex + 1);
	}
}

TArray<FNiagaraVariable> UNiagaraStackRendererItem::GetMissingVariables(UNiagaraRendererProperties* RendererProperties, const FVersionedNiagaraEmitterData* EmitterData)
{
	TArray<FNiagaraVariable> MissingAttributes;
	const TArray<FNiagaraVariable>& RequiredAttrs = RendererProperties->GetRequiredAttributes();
	const UNiagaraScript* Script = EmitterData->SpawnScriptProps.Script;
	if (Script != nullptr && Script->IsReadyToRun(ENiagaraSimTarget::CPUSim))
	{
		MissingAttributes.Empty();
		for (FNiagaraVariable Attr : RequiredAttrs)
		{
			FNiagaraVariable OriginalAttr = Attr;
			// TODO .. should we always be namespaced?
			FString AttrName = Attr.GetName().ToString();
			if (AttrName.RemoveFromStart(TEXT("Particles.")))
			{
				Attr.SetName(*AttrName);
			}

			bool ContainsVar = Script->GetVMExecutableData().Attributes.ContainsByPredicate([&Attr](const FNiagaraVariable& Var) { return Var.GetName() == Attr.GetName(); });
			if (!ContainsVar)
			{
				MissingAttributes.Add(OriginalAttr);
			}
		}
	}
	return MissingAttributes;
}

bool UNiagaraStackRendererItem::AddMissingVariable(const FVersionedNiagaraEmitterData* EmitterData, const FNiagaraVariable& Variable)
{
	UNiagaraScript* Script = EmitterData->SpawnScriptProps.Script;
	if (!Script)
	{
		return false;
	}
	UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(Script->GetLatestSource());
	if (!Source)
	{
		return false;
	}

	UNiagaraGraph* Graph = Source->NodeGraph;
	if (!Graph)
	{
		return false;
	}

	UNiagaraNodeOutput* OutputNode = Graph->FindOutputNode(ENiagaraScriptUsage::ParticleSpawnScript);
	if (!OutputNode)
	{
		return false;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("FixRendererError", "Fixing rendering module error: Add Attribute"));
	Graph->Modify();

	FGraphNodeCreator<UNiagaraNodeAssignment> NodeBuilder(*Graph);
	UNiagaraNodeAssignment* NewAssignmentNode = NodeBuilder.CreateNode();
	FString VarDefaultValue = FNiagaraConstants::GetAttributeDefaultValue(Variable);
	NewAssignmentNode->AddAssignmentTarget(Variable, &VarDefaultValue);
	NodeBuilder.Finalize();

	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> StackNodeGroups;
	FNiagaraStackGraphUtilities::GetStackNodeGroups(*OutputNode, StackNodeGroups);

	FNiagaraStackGraphUtilities::FStackNodeGroup AssignmentGroup;
	AssignmentGroup.StartNodes.Add(NewAssignmentNode);
	AssignmentGroup.EndNode = NewAssignmentNode;

	check(StackNodeGroups.Num() >= 2);

	FNiagaraStackGraphUtilities::FStackNodeGroup& OutputGroup = StackNodeGroups[StackNodeGroups.Num() - 1];
	FNiagaraStackGraphUtilities::FStackNodeGroup& OutputGroupPrevious = StackNodeGroups[StackNodeGroups.Num() - 2];
	FNiagaraStackGraphUtilities::ConnectStackNodeGroup(AssignmentGroup, OutputGroupPrevious, OutputGroup);

	FNiagaraStackGraphUtilities::RelayoutGraph(*Graph);
	return true;
}

bool UNiagaraStackRendererItem::IsExcludedFromScalability() const
{
	if(RendererProperties.IsValid()) 
	{
		return !RendererProperties.Get()->Platforms.IsActive();
	}

	return false;
}

bool UNiagaraStackRendererItem::IsOwningEmitterExcludedFromScalability() const
{
	// TODO - Stateless - RenderersOwner refactor
	return GetEmitterViewModel().IsValid() ? !GetEmitterViewModel()->GetEmitter().GetEmitterData()->IsAllowedByScalability() : false;
}

UNiagaraRendererProperties* UNiagaraStackRendererItem::GetRendererProperties()
{
	return RendererProperties.Get();
}

const UNiagaraRendererProperties* UNiagaraStackRendererItem::GetRendererProperties() const
{
	return RendererProperties.Get();
}

void UNiagaraStackRendererItem::OpenSourceCodeLocation() const
{
	if (UNiagaraRendererProperties* RendererProps = RendererProperties.Get())
	{
		FSourceCodeNavigation::NavigateToClass(RendererProps->GetClass());
	}
}

FText UNiagaraStackRendererItem::GetDisplayName() const
{
	if(DisplayNameCache.IsSet() == false)
	{
		if (RendererProperties != nullptr)
		{
			DisplayNameCache = RendererProperties->GetWidgetDisplayName();
		}
		else
		{
			DisplayNameCache = FText::FromName(NAME_None);
		}
	}
	return DisplayNameCache.GetValue();
}

bool UNiagaraStackRendererItem::TestCanCutWithMessage(FText& OutMessage) const
{
	FText CanCopyMessage;
	if (TestCanCopyWithMessage(CanCopyMessage) == false)
	{
		OutMessage = FText::Format(LOCTEXT("CantCutBecauseCantCopyFormat", "This renderer can not be cut because it can't be copied.  {0}"), CanCopyMessage);
		return false;
	}

	FText CanDeleteMessage;
	if (TestCanDeleteWithMessage(CanDeleteMessage) == false)
	{
		OutMessage = FText::Format(LOCTEXT("CantCutBecauseCantDeleteFormat", "This renderer can't be cut because it can't be deleted.  {0}"), CanDeleteMessage);
		return false;
	}

	OutMessage = LOCTEXT("CanCut", "Cut this renderer.");
	return true;
}

FText UNiagaraStackRendererItem::GetCutTransactionText() const
{
	return LOCTEXT("CutRendererTransactionText", "Cut renderers");
}

void UNiagaraStackRendererItem::CopyForCut(UNiagaraClipboardContent* ClipboardContent) const
{
	Copy(ClipboardContent);
}

void UNiagaraStackRendererItem::RemoveForCut()
{
	Delete();
}

bool UNiagaraStackRendererItem::TestCanCopyWithMessage(FText& OutMessage) const
{
	OutMessage = LOCTEXT("CopyRenderer", "Copy this renderer.");
	return true;
}

void UNiagaraStackRendererItem::Copy(UNiagaraClipboardContent* ClipboardContent) const
{
	ClipboardContent->Renderers.Add(UNiagaraClipboardRenderer::CreateRenderer(ClipboardContent, RendererProperties.Get(), GetStackNoteData(), GetAlternateDisplayName()));
}

bool UNiagaraStackRendererItem::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	if (RequestCanPasteDelegete.IsBound())
	{
		return RequestCanPasteDelegete.Execute(ClipboardContent, OutMessage);
	}
	OutMessage = FText();
	return false;
}

FText UNiagaraStackRendererItem::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	return LOCTEXT("PasteRenderersTransactionText", "Paste renderers");
}

void UNiagaraStackRendererItem::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	RequestPasteDelegate.ExecuteIfBound(ClipboardContent, INDEX_NONE, OutPasteWarning);
}

bool UNiagaraStackRendererItem::TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const
{
	if (GetOwnerIsEnabled() == false)
	{
		OutCanDeleteMessage = LOCTEXT("CantDeleteOwnerDisabledToolTip", "This renderer can not be deleted because its owner is disabled.");
		return false;
	}
	else if (HasBaseRenderer() == false)
	{
		OutCanDeleteMessage = LOCTEXT("DeleteToolTip", "Delete this renderer.");
		return true;
	}
	else
	{
		OutCanDeleteMessage = LOCTEXT("CantDeleteToolTip", "This renderer can not be deleted becaue it is inherited.");
		return false;
	}
}

FText UNiagaraStackRendererItem::GetDeleteTransactionText() const
{
	return LOCTEXT("DeleteRenderer", "Delete Renderer");
}

void UNiagaraStackRendererItem::Delete()
{
	if (RenderersOwner.IsValid() && RenderersOwner->IsValid())
	{
		GetStackEditorData().Modify();
		GetStackEditorData().SetStackEntryDisplayName(GetStackEditorDataKey(), FText());

		UNiagaraRendererProperties* Renderer = RendererProperties.Get();
		RenderersOwner->RemoveRenderer(Renderer);

		TArray<UObject*> ChangedObjects;
		ChangedObjects.Add(Renderer);
		OnDataObjectModified().Broadcast(ChangedObjects, ENiagaraDataObjectChange::Removed);
	}
}

bool UNiagaraStackRendererItem::GetIsInherited() const
{
	return HasBaseRenderer() == true;
}

FText UNiagaraStackRendererItem::GetInheritanceMessage() const
{
	return LOCTEXT("RendererItemInheritanceMessage", "This renderer is inherited from a parent emitter.  Inherited\nrenderers can only be deleted while editing the parent emitter.");
}

FHierarchyElementIdentity UNiagaraStackRendererItem::DetermineSummaryIdentity() const
{
	FHierarchyElementIdentity Identity;
	Identity.Guids.Add(GetRendererProperties()->GetMergeId());
	return Identity;
}

bool UNiagaraStackRendererItem::HasBaseRenderer() const
{
	if (HasBaseEmitter())
	{
		if (bHasBaseRendererCache.IsSet() == false)
		{
			bHasBaseRendererCache = RenderersOwner.IsValid() && RenderersOwner->IsValid() && RenderersOwner->HasBaseRenderer(RendererProperties.Get());
		}
		return bHasBaseRendererCache.GetValue();
	}
	return false;
}

bool UNiagaraStackRendererItem::TestCanResetToBaseWithMessage(FText& OutCanResetToBaseMessage) const
{
	if (bCanResetToBaseCache.IsSet() == false)
	{
		if (HasBaseRenderer() && RenderersOwner.IsValid() && RenderersOwner->IsValid())
		{
			bCanResetToBaseCache = RenderersOwner->IsRendererDifferentFromBase(RendererProperties.Get());
		}
		else
		{
			bCanResetToBaseCache = false;
		}
	}
	if (bCanResetToBaseCache.GetValue())
	{
		OutCanResetToBaseMessage = LOCTEXT("CanResetToBase", "Reset this renderer to the state defined by the parent emitter.");
		return true;
	}
	else
	{
		OutCanResetToBaseMessage = LOCTEXT("CanNotResetToBase", "No parent to reset to, or not different from parent.");
		return false;
	}
}

void UNiagaraStackRendererItem::ResetToBase()
{
	FText Unused;
	if (TestCanResetToBaseWithMessage(Unused))
	{
		if (RenderersOwner.IsValid() && RenderersOwner->IsValid())
		{
			RenderersOwner->ResetRendererToBase(RendererProperties.Get());
			ModifiedGroupItemsDelegate.Broadcast();
		}
	}
}

bool UNiagaraStackRendererItem::GetShouldShowInOverview() const
{
	return RenderersOwner.IsValid() && RenderersOwner->ShouldShowRendererItemsInOverview();
}

bool UNiagaraStackRendererItem::GetIsEnabled() const
{
	if (IsFinalized() == false && RendererProperties.IsValid())
	{
		return RendererProperties->GetIsEnabled();
	}
	else
	{
		return false;
	}
}

void UNiagaraStackRendererItem::SetIsEnabledInternal(bool bInIsEnabled)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("SetRendererEnabledState", "Set renderer enabled/disabled state."));
	RendererProperties->Modify();
	RendererProperties->SetIsEnabled(bInIsEnabled);
	TArray<UObject*> ChangedObjects;
	ChangedObjects.Add(RendererProperties.Get());
	OnDataObjectModified().Broadcast(ChangedObjects, ENiagaraDataObjectChange::Changed);
	RefreshChildren();

	if (GetEmitterViewModel().IsValid())
	{
		GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(GetEmitterViewModel()->GetEmitter()).Get()->GetEmitterStackViewModel()->RequestValidationUpdate();
	}
}

const FSlateBrush* UNiagaraStackRendererItem::GetIconBrush() const
{
	if (IsFinalized() == false && RendererProperties.IsValid())
	{
		return RendererProperties->GetStackIcon();
	}
	else
	{
		return nullptr;
	}
}

void UNiagaraStackRendererItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (RendererObject == nullptr)
	{
		RendererObject = NewObject<UNiagaraStackObject>(this);
		bool bIsTopLevelObject = true;
		bool bHideTopLevelCategories = false;
		RendererObject->Initialize(CreateDefaultChildRequiredData(), RendererProperties.Get(), bIsTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey());
		RendererObject->SetObjectGuid(GetRendererProperties()->GetMergeId());
	}

	NewChildren.Add(RendererObject);
	MissingAttributes = GetMissingVariables(RendererProperties.Get(), GetEmitterViewModel()->GetEmitter().GetEmitterData());
	bHasBaseRendererCache.Reset();
	bCanResetToBaseCache.Reset();
	DisplayNameCache.Reset();
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
	
	RefreshIssues(NewIssues);
}

void UNiagaraStackRendererItem::ProcessRendererIssues(const TArray<FNiagaraRendererFeedback>& InIssues, EStackIssueSeverity Severity, TArray<FStackIssue>& OutIssues)
{
	for (const FNiagaraRendererFeedback& Item : InIssues)
	{
		TArray<FStackIssueFix> Fixes;
		if (Item.IsFixable())
		{
			Fixes.Emplace(
				Item.GetFixDescriptionText(),
				FStackIssueFixDelegate::CreateLambda(
					[WeakStackItem=TWeakObjectPtr<UNiagaraStackRendererItem>(this), Item]()
					{
						if ( Item.IsFixable() )
						{
							const FScopedTransaction ScopedTransaction(LOCTEXT("RendererItemFixTransaction", "Apply renderer fix"));

							UNiagaraStackRendererItem* StackItem = WeakStackItem.Get();
							UNiagaraRendererProperties* RendererPropertiesLocal = StackItem ? StackItem->GetRendererProperties() : nullptr;
							if (RendererPropertiesLocal)
							{
								RendererPropertiesLocal->Modify();
							}

							Item.TryFix();

							if (RendererPropertiesLocal)
							{
								RendererPropertiesLocal->PostEditChange();

								TArray<UObject*> ChangedObjects;
								ChangedObjects.Add(RendererPropertiesLocal);
								StackItem->OnDataObjectModified().Broadcast(ChangedObjects, ENiagaraDataObjectChange::Changed);
							}
						}
					}
				)
			);
		}
		FStackIssue TargetSupportError(Severity, Item.GetSummaryText(), Item.GetDescriptionText(), GetStackEditorDataKey(), Item.IsDismissable(), Fixes);
		OutIssues.Add(TargetSupportError);
	}
}

void UNiagaraStackRendererItem::RefreshIssues(TArray<FStackIssue>& NewIssues)
{
	if (!GetIsEnabled())
	{
		NewIssues.Empty();
		return;
	}
	FVersionedNiagaraEmitterData* EmitterData = GetEmitterViewModel()->GetEmitter().GetEmitterData();
	for (FNiagaraVariable Attribute : MissingAttributes)
	{
		FText FixDescription = LOCTEXT("AddMissingVariable", "Add missing variable");
		FStackIssueFix AddAttributeFix(
			FixDescription,
			FStackIssueFixDelegate::CreateLambda([=]()
		{
			FScopedTransaction ScopedTransaction(FixDescription);
			if (AddMissingVariable(EmitterData, Attribute))
			{
				FNotificationInfo Info(FText::Format(LOCTEXT("AddedVariableForFix", "Added {0} to the Spawn script to support the renderer."), FText::FromName(Attribute.GetName())));
				Info.ExpireDuration = 5.0f;
				Info.bFireAndForget = true;
				Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Info"));
				FSlateNotificationManager::Get().AddNotification(Info);
			}
		}));

		FStackIssue MissingAttributeError(
			EStackIssueSeverity::Error,
			LOCTEXT("FailedRendererBindShort", "An attribute is missing."),
			FText::Format(LOCTEXT("FailedRendererBind", "Missing attribute \"{0}\" of Type \"{1}\"."), FText::FromName(Attribute.GetName()), Attribute.GetType().GetNameText()),
			GetStackEditorDataKey(),
			false,
			AddAttributeFix);

		NewIssues.Add(MissingAttributeError);
	}

	if (RendererProperties->GetIsEnabled() && !RendererProperties->IsSimTargetSupported(EmitterData->SimTarget))
	{
		const ENiagaraSimTarget SimTargets[] = {ENiagaraSimTarget::CPUSim, ENiagaraSimTarget::GPUComputeSim};

		TArray<FStackIssueFix> Fixes;
		for (ENiagaraSimTarget SimTarget : SimTargets)
		{
			if ( !RendererProperties->IsSimTargetSupported(SimTarget) )
			{
				continue;
			}
			Fixes.Emplace(
				FText::Format(LOCTEXT("RendererChangeSimTargetFix", "Change Sim Target to \"{0}\""), UEnum::GetDisplayValueAsText(SimTarget)),
				FStackIssueFixDelegate::CreateLambda(
					[WeakEmitterPtr=GetEmitterViewModel()->GetEmitter().ToWeakPtr(), SimTarget]()
					{
						FVersionedNiagaraEmitter VersionedEmitter = WeakEmitterPtr.ResolveWeakPtr();
						if (FVersionedNiagaraEmitterData* VersionedEmitterData = VersionedEmitter.GetEmitterData())
						{
							const FScopedTransaction Transaction(LOCTEXT("ChangeSimTarget", "Change Sim Target"));
							VersionedEmitter.Emitter->Modify();
							VersionedEmitterData->SimTarget = SimTarget;

							FProperty* SimTargetProperty = FindFProperty<FProperty>(FVersionedNiagaraEmitterData::StaticStruct(), GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, SimTarget));
							FPropertyChangedEvent PropertyChangedEvent(SimTargetProperty);
							VersionedEmitter.Emitter->PostEditChangeVersionedProperty(PropertyChangedEvent, VersionedEmitter.Version);
						}
					}
				)
			);
		}

		NewIssues.Emplace(
			EStackIssueSeverity::Error,
			LOCTEXT("FailedRendererDueToSimTarget", "Renderer incompatible with chosen Sim Target in Emitter Properties."),
			FText::Format(LOCTEXT("FailedRendererDueToSimTargetLong", "Renderer incompatible with Sim Target \"{0}\"."), UEnum::GetDisplayValueAsText(EmitterData->SimTarget)),
			GetStackEditorDataKey(),
			false,
			Fixes
		);
	}

	if (RendererProperties->GetIsEnabled())
	{
		TArray<FNiagaraRendererFeedback> Errors;
		TArray<FNiagaraRendererFeedback> Warnings;
		TArray<FNiagaraRendererFeedback> Infos;

		// Query feedback from the renderer
		RendererProperties->GetRendererFeedback(GetEmitterViewModel()->GetEmitter(), Errors, Warnings, Infos);

		// Query feedback from the material validation system
		UMaterialEditorValidationSubsystem* MaterialValidationSubsystem = GEditor ? GEditor->GetEditorSubsystem<UMaterialEditorValidationSubsystem>() : nullptr;
		if (MaterialValidationSubsystem)
		{
			TArray<UMaterialInterface*> RendererMaterials;
			RendererProperties->GetUsedMaterials(nullptr, RendererMaterials);

			for (UMaterialInterface* Material : RendererMaterials)
			{
				if (Material == nullptr)
				{
					continue;
				}

				FMaterialEditorValidatorContext MaterialValidatorContext;
				MaterialValidationSubsystem->Validate(Material, MaterialValidatorContext);
				if (MaterialValidatorContext.Message.IsEmpty() == false)
				{
					Warnings.Emplace(
						MaterialValidatorContext.Message,
						FText::Format(LOCTEXT("MaterialValidationMessage", "Material '{0}' has validation issues."), FText::FromName(Material->GetFName()))
					);
				}
			}
		}

		// Convert feedback into stack issues
		ProcessRendererIssues(Errors, EStackIssueSeverity::Error, NewIssues);		
		ProcessRendererIssues(Warnings, EStackIssueSeverity::Warning, NewIssues);		
		ProcessRendererIssues(Infos, EStackIssueSeverity::Info, NewIssues);	
	}
}

void UNiagaraStackRendererItem::RendererChanged()
{
	if (IsFinalized() == false)
	{
		// Undo/redo can cause objects to disappear and reappear which can prevent safe removal of delegates
		// so guard against receiving an event when finalized here.
		bCanResetToBaseCache.Reset();
		RefreshChildren();

		if (GetSystemViewModel()->GetSystemStackViewModel())
		{
			GetSystemViewModel()->GetSystemStackViewModel()->InvalidateCachedParameterUsage();
		}
		if (GetSystemViewModel()->GetParameterPanelViewModel())
		{
			GetSystemViewModel()->GetParameterPanelViewModel()->RefreshNextTick();
		}

		if (GetEmitterViewModel().IsValid())
		{
			GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(GetEmitterViewModel()->GetEmitter()).Get()->GetEmitterStackViewModel()->RequestValidationUpdate();
		}
	}
}

bool UNiagaraStackRendererItem::SupportsDebugDraw() const
{
	UNiagaraRendererProperties* Renderer = RendererProperties.Get();
	return Renderer ? Renderer->SupportsDebugDraw() : false;
}

TOptional<FText> UNiagaraStackRendererItem::GetDebugDrawTooltip() const
{
	UNiagaraRendererProperties* Renderer = RendererProperties.Get();
	return Renderer ? Renderer->GetDebugDrawTooltip() : TOptional<FText>();
}

bool UNiagaraStackRendererItem::IsDebugDrawEnabled() const
{
	UNiagaraRendererProperties* Renderer = RendererProperties.Get();
	return Renderer ? Renderer->IsDebugDrawEnabled() : false;
}

void UNiagaraStackRendererItem::SetDebugDrawEnabled(bool bInEnabled)
{
	if (UNiagaraRendererProperties* Renderer = RendererProperties.Get())
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("SetRendererDebugDrawEnabledState", "Set renderer debug draw enabled state."));
		Renderer->Modify();
		Renderer->SetDebugDrawEnabled(bInEnabled);

		TArray<UObject*> ChangedObjects;
		ChangedObjects.Add(Renderer);
		OnDataObjectModified().Broadcast(ChangedObjects, ENiagaraDataObjectChange::Changed);
		RefreshChildren();
	}
}

TArray<FNiagaraStackItemCustomIcon> UNiagaraStackRendererItem::GetCustomIcons() const
{
	TArray<FNiagaraStackItemCustomIcon> CustomIcons;

	UNiagaraRendererProperties* Renderer = RendererProperties.Get();
	if (Renderer == nullptr)
	{
		return CustomIcons;
	}

	// Gather all our material parameter bindings
	TArray<FNiagaraRendererMaterialParameters*> MaterialParameters;
	for (TFieldIterator<FStructProperty> PropIt(Renderer->GetClass()); PropIt; ++PropIt)
	{
		FStructProperty* StructProp = *PropIt;
		if (!StructProp || !StructProp->Struct || !StructProp->Struct->IsChildOf(FNiagaraRendererMaterialParameters::StaticStruct()))
		{
			continue;
		}

		MaterialParameters.Add(StructProp->ContainerPtrToValuePtr<FNiagaraRendererMaterialParameters>(Renderer));
	}

	if (MaterialParameters.Num() > 0)
	{
		FNiagaraStackItemCustomIcon& Icon = CustomIcons.AddDefaulted_GetRef();

		Icon.IconBrush			= FNiagaraEditorStyle::Get().GetBrush(TEXT("NiagaraEditor.Overview.HasMaterialParameterBindings"));
		Icon.ToolTipText		= LOCTEXT("BoundMaterialParametersTooltip", "Renderer has bound material parameters.  See the bindings section on the renderer for more details.");
		Icon.VisibleFunction	= TAttribute<EVisibility>::CreateLambda(
			[MaterialParameters, WeakRenderer=RendererProperties]()
			{
				if (WeakRenderer.Get())
				{
					for (FNiagaraRendererMaterialParameters* Parameters : MaterialParameters)
					{
						if (Parameters->HasAnyBindings())
						{
							return EVisibility::Visible;
						}
					}
				}
				return EVisibility::Hidden;
			}
		);
	}

	return CustomIcons;
}

#undef LOCTEXT_NAMESPACE
