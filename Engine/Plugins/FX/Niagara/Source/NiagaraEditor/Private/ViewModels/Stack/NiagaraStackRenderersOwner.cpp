// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackRenderersOwner.h"

#include "Framework/Notifications/NotificationManager.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraScriptMergeManager.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Styling/CoreStyle.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "NiagaraRenderersOwner"

TSharedRef<INiagaraStackRenderersOwner> FNiagaraStackRenderersOwnerStandard::CreateShared(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TSharedRef<FNiagaraStackRenderersOwnerStandard> SharedInstance = MakeShared<FNiagaraStackRenderersOwnerStandard>();
	SharedInstance->Initialize(EmitterViewModel);
	return SharedInstance;
}

void FNiagaraStackRenderersOwnerStandard::Initialize(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel)
{
	EmitterWeak = EmitterViewModel->GetEmitter().ToWeakPtr();
	if (UNiagaraEmitter* Emitter = EmitterWeak.Emitter.Get())
	{
		Emitter->OnRenderersChanged().AddSP(this->AsShared(), &FNiagaraStackRenderersOwnerStandard::EmitterRenderersChanged);
	}
}

FNiagaraStackRenderersOwnerStandard::~FNiagaraStackRenderersOwnerStandard()
{
	if (EmitterWeak.IsValid())
	{
		EmitterWeak.Emitter->OnRenderersChanged().RemoveAll(this);
	}
}

bool FNiagaraStackRenderersOwnerStandard::IsValid() const
{
	return EmitterWeak.IsValid();
}

UObject* FNiagaraStackRenderersOwnerStandard::GetOwnerObject() const
{
	return EmitterWeak.IsValid() ? EmitterWeak.Emitter.Get() : nullptr;
}

void FNiagaraStackRenderersOwnerStandard::GetRenderers(TArray<UNiagaraRendererProperties*>& OutRenderers) const
{
	if (EmitterWeak.IsValid())
	{
		OutRenderers.Append(EmitterWeak.GetEmitterData()->GetRenderers());
	}
}

void FNiagaraStackRenderersOwnerStandard::AddRenderer(UNiagaraRendererProperties* RendererToAdd)
{
	if (EmitterWeak.IsValid())
	{
		EmitterWeak.Emitter->AddRenderer(RendererToAdd, EmitterWeak.Version);

		FVersionedNiagaraEmitterData* EmitterData = EmitterWeak.GetEmitterData();
		bool bVarsAdded = false;
		TArray<FNiagaraVariable> MissingAttributes = UNiagaraStackRendererItem::GetMissingVariables(RendererToAdd, EmitterData);
		for (int32 i = 0; i < MissingAttributes.Num(); i++)
		{
			if (UNiagaraStackRendererItem::AddMissingVariable(EmitterData, MissingAttributes[i]))
			{
				bVarsAdded = true;
			}
		}

		FNiagaraSystemUpdateContext SystemUpdate(EmitterWeak.ResolveWeakPtr(), true);

		if (bVarsAdded)
		{
			FNotificationInfo Info(LOCTEXT("AddedVariables", "One or more variables have been added to the Spawn script to support the added renderer."));
			Info.ExpireDuration = 5.0f;
			Info.bFireAndForget = true;
			Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Info"));
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
}

void FNiagaraStackRenderersOwnerStandard::RemoveRenderer(UNiagaraRendererProperties* Renderer)
{
	if (EmitterWeak.IsValid())
	{
		EmitterWeak.Emitter->RemoveRenderer(Renderer, EmitterWeak.Version);
	}
}

void FNiagaraStackRenderersOwnerStandard::MoveRenderer(UNiagaraRendererProperties* Renderer, int32 NewIndex)
{
	if (EmitterWeak.IsValid())
	{
		EmitterWeak.Emitter->MoveRenderer(Renderer, NewIndex, EmitterWeak.Version);
	}
}

bool FNiagaraStackRenderersOwnerStandard::HasBaseRenderer(UNiagaraRendererProperties* Renderer) const
{
	bool bHasBaseRenderer = false;
	if (EmitterWeak.IsValid() && Renderer != nullptr)
	{
		TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
		FVersionedNiagaraEmitter BaseEmitter = EmitterWeak.GetEmitterData()->GetParent();
		bHasBaseRenderer = BaseEmitter.Emitter != nullptr && MergeManager->HasBaseRenderer(BaseEmitter, Renderer->GetMergeId());
	}
	return bHasBaseRenderer;
}

bool FNiagaraStackRenderersOwnerStandard::IsRendererDifferentFromBase(UNiagaraRendererProperties* Renderer) const
{
	bool bRendererIsDifferentFromBase = false;
	if (EmitterWeak.IsValid() && Renderer != nullptr)
	{
		TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
		FVersionedNiagaraEmitter BaseEmitter = EmitterWeak.GetEmitterData()->GetParent();
		bRendererIsDifferentFromBase = BaseEmitter.Emitter != nullptr && MergeManager->IsRendererDifferentFromBase(EmitterWeak.ResolveWeakPtr(), BaseEmitter, Renderer->GetMergeId());
	}
	return bRendererIsDifferentFromBase;
}

void FNiagaraStackRenderersOwnerStandard::ResetRendererToBase(UNiagaraRendererProperties* Renderer) const
{
	if (EmitterWeak.IsValid() && Renderer != nullptr)
	{
		TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
		FVersionedNiagaraEmitter BaseEmitter = EmitterWeak.GetEmitterData()->GetParent();
		MergeManager->ResetRendererToBase(EmitterWeak.ResolveWeakPtr(), BaseEmitter, Renderer->GetMergeId());
	}
}

TSharedRef<INiagaraStackRenderersOwner> FNiagaraStackRenderersOwnerStateless::CreateShared(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	TSharedRef<FNiagaraStackRenderersOwnerStateless> SharedInstance = MakeShared<FNiagaraStackRenderersOwnerStateless>();
	SharedInstance->Initialize(EmitterHandleViewModel);
	return SharedInstance;
}

void FNiagaraStackRenderersOwnerStateless::Initialize(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	StatelessEmitterWeak = EmitterHandleViewModel->GetEmitterHandle()->GetStatelessEmitter();
	if (StatelessEmitterWeak.IsValid())
	{
		StatelessEmitterWeak->OnRenderersChanged().AddSP(this->AsShared(), &FNiagaraStackRenderersOwnerStateless::EmitterRenderersChanged);
	}
}

FNiagaraStackRenderersOwnerStateless::~FNiagaraStackRenderersOwnerStateless()
{
	if (StatelessEmitterWeak.IsValid())
	{
		StatelessEmitterWeak->OnRenderersChanged().RemoveAll(this);
	}
}

bool FNiagaraStackRenderersOwnerStateless::IsValid() const
{
	return StatelessEmitterWeak.IsValid();
}

UObject* FNiagaraStackRenderersOwnerStateless::GetOwnerObject() const
{
	return StatelessEmitterWeak.Get();
}

void FNiagaraStackRenderersOwnerStateless::GetRenderers(TArray<UNiagaraRendererProperties*>& OutRenderers) const
{
	if (StatelessEmitterWeak.IsValid())
	{
		OutRenderers.Append(StatelessEmitterWeak->GetRenderers());
	}
}

bool FNiagaraStackRenderersOwnerStateless::SupportsRendererClass(const UClass* RendererClass) const
{
	static const FName NAME_SupportsStateless("SupportsStateless");
	return RendererClass->HasMetaData(NAME_SupportsStateless);
}

void FNiagaraStackRenderersOwnerStateless::AddRenderer(UNiagaraRendererProperties* RendererToAdd)
{
	if (StatelessEmitterWeak.IsValid())
	{
		StatelessEmitterWeak->AddRenderer(RendererToAdd, FGuid());
	}
}

void FNiagaraStackRenderersOwnerStateless::RemoveRenderer(UNiagaraRendererProperties* Renderer)
{
	if (StatelessEmitterWeak.IsValid())
	{
		StatelessEmitterWeak->RemoveRenderer(Renderer, FGuid());
	}
}

void FNiagaraStackRenderersOwnerStateless::MoveRenderer(UNiagaraRendererProperties* Renderer, int32 NewIndex)
{
	if (StatelessEmitterWeak.IsValid())
	{
		StatelessEmitterWeak->MoveRenderer(Renderer, NewIndex, FGuid());
	}
}

const FSlateBrush* FNiagaraStackRenderersOwnerStateless::GetIconBrush() const
{
	return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stateless.RenderIcon");
}

#undef LOCTEXT_NAMESPACE