// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"

class FNiagaraEmitterHandleViewModel;
class FNiagaraEmitterViewModel;
struct FNiagaraRendererCreationInfo;
class UNiagaraEmitter;
class UNiagaraRendererProperties;
class UNiagaraStatelessEmitter;

class INiagaraStackRenderersOwner
{
public:
	virtual ~INiagaraStackRenderersOwner() = default;

	virtual bool IsValid() const = 0;
	virtual UObject* GetOwnerObject() const = 0;
	virtual void GetRenderers(TArray<UNiagaraRendererProperties*>& OutRenderers) const = 0;
	virtual bool SupportsRendererClass(const UClass* RendererClass) const = 0;

	virtual void AddRenderer(UNiagaraRendererProperties* RendererToAdd) = 0;
	virtual void RemoveRenderer(UNiagaraRendererProperties* Renderer) = 0;
	virtual void MoveRenderer(UNiagaraRendererProperties* Renderer, int32 NewIndex) = 0;

	virtual bool HasBaseRenderer(UNiagaraRendererProperties* Renderer) const = 0;
	virtual bool IsRendererDifferentFromBase(UNiagaraRendererProperties* Renderer) const = 0;
	virtual void ResetRendererToBase(UNiagaraRendererProperties* Renderer) const = 0;

	virtual bool ShouldShowRendererItemsInOverview() const = 0;
	virtual UNiagaraStackEntry::EIconMode GetSupportedIconMode() const = 0;
	virtual const FSlateBrush* GetIconBrush() const = 0;
	virtual FText GetIconText() const = 0;

	virtual FSimpleDelegate& OnRenderersChanged() = 0;
};

class FNiagaraStackRenderersOwner : public INiagaraStackRenderersOwner
{
public:
	virtual bool SupportsRendererClass(const UClass* RendererClass) const override { return true; }

	virtual bool HasBaseRenderer(UNiagaraRendererProperties* Renderer) const override { return false; }
	virtual bool IsRendererDifferentFromBase(UNiagaraRendererProperties* Renderer) const override { return false; }
	virtual void ResetRendererToBase(UNiagaraRendererProperties* Renderer) const override { }

	virtual bool ShouldShowRendererItemsInOverview() const override { return true; }
	virtual UNiagaraStackEntry::EIconMode GetSupportedIconMode() const override { return UNiagaraStackEntry::EIconMode::None; }
	virtual const FSlateBrush* GetIconBrush() const override { return nullptr; }
	virtual FText GetIconText() const override { return FText(); }

	virtual FSimpleDelegate& OnRenderersChanged() override { return OnRenderersChangedDelegate; }

private:
	FSimpleDelegate OnRenderersChangedDelegate;
};

class FNiagaraStackRenderersOwnerStandard : public FNiagaraStackRenderersOwner, public TSharedFromThis<FNiagaraStackRenderersOwnerStandard>
{
public:
	static TSharedRef<INiagaraStackRenderersOwner> CreateShared(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	void Initialize(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);

	~FNiagaraStackRenderersOwnerStandard();

	virtual bool IsValid() const override;
	virtual UObject* GetOwnerObject() const override;
	virtual void GetRenderers(TArray<UNiagaraRendererProperties*>& OutRenderers) const override;
	virtual void AddRenderer(UNiagaraRendererProperties* RendererToAdd);
	virtual void RemoveRenderer(UNiagaraRendererProperties* Renderer) override;
	virtual void MoveRenderer(UNiagaraRendererProperties* Renderer, int32 NewIndex) override;
	virtual bool HasBaseRenderer(UNiagaraRendererProperties* Renderer) const override;
	virtual bool IsRendererDifferentFromBase(UNiagaraRendererProperties* Renderer) const override;
	virtual void ResetRendererToBase(UNiagaraRendererProperties* Renderer) const override;

private:
	void EmitterRenderersChanged() { OnRenderersChanged().ExecuteIfBound(); }

private:
	FVersionedNiagaraEmitterWeakPtr EmitterWeak;
};

class FNiagaraStackRenderersOwnerStateless : public FNiagaraStackRenderersOwner, public TSharedFromThis<FNiagaraStackRenderersOwnerStateless>
{
public:
	static TSharedRef<INiagaraStackRenderersOwner> CreateShared(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);
	void Initialize(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);

	~FNiagaraStackRenderersOwnerStateless();

	virtual bool IsValid() const override;
	virtual UObject* GetOwnerObject() const override;
	virtual void GetRenderers(TArray<UNiagaraRendererProperties*>& OutRenderers) const override;
	virtual bool SupportsRendererClass(const UClass* RendererClass) const override;
	virtual void AddRenderer(UNiagaraRendererProperties* RendererToAdd) override;
	virtual void RemoveRenderer(UNiagaraRendererProperties* Renderer) override;
	virtual void MoveRenderer(UNiagaraRendererProperties* Renderer, int32 NewIndex) override;
	virtual bool ShouldShowRendererItemsInOverview() const override { return true; }
	virtual UNiagaraStackEntry::EIconMode GetSupportedIconMode() const override { return UNiagaraStackEntry::EIconMode::Brush; }
	virtual const FSlateBrush* GetIconBrush() const override;

private:
	void EmitterRenderersChanged() { OnRenderersChanged().ExecuteIfBound(); }

private:
	TWeakObjectPtr<UNiagaraStatelessEmitter> StatelessEmitterWeak;
};
