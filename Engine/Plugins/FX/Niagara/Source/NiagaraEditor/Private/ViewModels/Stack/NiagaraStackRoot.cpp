// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackRoot.h"
#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackEmitterPropertiesGroup.h"
#include "ViewModels/Stack/NiagaraStackEmitterSettingsGroup.h"
#include "ViewModels/Stack/NiagaraStackStatelessEmitterGroup.h"
#include "ViewModels/Stack/NiagaraStackStatelessEmitterSimulateGroup.h"
#include "ViewModels/Stack/NiagaraStackStatelessEmitterSpawnGroup.h"
#include "ViewModels/Stack/NiagaraStackRenderersOwner.h"
#include "ViewModels/Stack/NiagaraStackRenderItemGroup.h"
#include "ViewModels/Stack/NiagaraStackEventScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackSimulationStageGroup.h"
#include "ViewModels/Stack/NiagaraStackSystemSettingsGroup.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraSystemScriptViewModel.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraSimulationStageBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackRoot)

#define LOCTEXT_NAMESPACE "NiagaraStackViewModel"

UNiagaraStackRoot::UNiagaraStackRoot()
{
}

void UNiagaraStackRoot::Initialize(FRequiredEntryData InRequiredEntryData, bool bInIncludeSystemInformation, bool bInIncludeEmitterInformation)
{
	Super::Initialize(InRequiredEntryData, TEXT("Root"));
	bIncludeSystemInformation = bInIncludeSystemInformation;
	bIncludeEmitterInformation = bInIncludeEmitterInformation;
	if (bInIncludeEmitterInformation && GetEmitterViewModel())
	{
		GetEmitterViewModel()->GetEditorData().OnSummaryViewStateChanged().AddUObject(this, &UNiagaraStackRoot::OnSummaryViewStateChanged);
	}
}

void UNiagaraStackRoot::FinalizeInternal()
{
	if (bIncludeEmitterInformation && GetEmitterViewModel() && GetEmitterViewModel()->GetEmitter().GetEmitterData())
	{
		GetEmitterViewModel()->GetEditorData().OnSummaryViewStateChanged().RemoveAll(this);
	}	
	Super::FinalizeInternal();
}

bool UNiagaraStackRoot::GetCanExpand() const
{
	return false;
}

bool UNiagaraStackRoot::GetShouldShowInStack() const
{
	return false;
}

UNiagaraStackRenderItemGroup* UNiagaraStackRoot::GetRenderGroup() const
{
	TArray<UNiagaraStackRenderItemGroup*> RenderItemGroups;
	GetUnfilteredChildrenOfType(RenderItemGroups);
	return RenderItemGroups.Num() == 1 ? RenderItemGroups[0] : nullptr;
}

UNiagaraStackCommentCollection* UNiagaraStackRoot::GetCommentCollection() const
{
	TArray<UNiagaraStackCommentCollection*> CommentCollections;
	GetUnfilteredChildrenOfType(CommentCollections);
	return CommentCollections.Num() == 1 ? CommentCollections[0] : nullptr;
}

void UNiagaraStackRoot::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (bIncludeSystemInformation)
	{
		RefreshSystemChildren(CurrentChildren, NewChildren);
	}

	if (bIncludeEmitterInformation && GetEmitterViewModel().IsValid())
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(GetEmitterViewModel()->GetEmitter());
		bool bShouldShowEmitterStatelessView = EmitterHandleViewModel.IsValid() && EmitterHandleViewModel->GetEmitterHandle()->GetEmitterMode() == ENiagaraEmitterMode::Stateless;
		
		if (bShouldShowEmitterStatelessView)
		{
			RefreshEmitterStatelessChildren(CurrentChildren, NewChildren, EmitterHandleViewModel);
		}
		else
		{
			bool bShouldShowSummaryView = GetEmitterViewModel() ? GetEmitterViewModel()->GetEditorData().ShouldShowSummaryView() : false;
			if (bShouldShowSummaryView)
			{
				RefreshEmitterSummaryChildren(CurrentChildren, NewChildren);
			}
			else
			{
				RefreshEmitterFullChildren(CurrentChildren, NewChildren);
			}
		}
	}
}

void UNiagaraStackRoot::RefreshSystemChildren(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren)
{
	UNiagaraStackEditorData& SystemStackEditorData = GetSystemViewModel()->GetEditorData().GetStackEditorData();
	TSharedRef<FNiagaraScriptViewModel> SystemScriptViewModel = GetSystemViewModel()->GetSystemScriptViewModel().ToSharedRef();
	FName ExecutionCategory = FExecutionCategoryNames::System;

	NewChildren.Add(GetOrCreateSystemPropertiesGroup(CurrentChildren));
	NewChildren.Add(GetOrCreateSystemUserParametersGroup(CurrentChildren));

	UNiagaraStackEntry* SystemSpawnEntry = GetCurrentScriptGroup(CurrentChildren, ENiagaraScriptUsage::SystemSpawnScript, FGuid());
	if (SystemSpawnEntry == nullptr)
	{
		ENiagaraScriptUsage ScriptUsage = ENiagaraScriptUsage::SystemSpawnScript;
		FGuid ScriptUsageId = FGuid();
		FName ExecutionSubCategory = FExecutionSubcategoryNames::Spawn;
		FText DisplayName = LOCTEXT("SystemSpawnGroupName", "System Spawn");
		FText ToolTip =	LOCTEXT("SystemSpawnGroupToolTip", "Occurs once at System creation on the CPU. Modules in this stage should initialize defaults and/or do initial setup.\r\nModules are executed in order from top to bottom of the stack.");
		SystemSpawnEntry = CreateScriptGroup(SystemScriptViewModel, ScriptUsage, ScriptUsageId, SystemStackEditorData, ExecutionCategory, ExecutionSubCategory, DisplayName, ToolTip);
	}
	NewChildren.Add(SystemSpawnEntry);

	UNiagaraStackEntry* SystemUpdateEntry = GetCurrentScriptGroup(CurrentChildren, ENiagaraScriptUsage::SystemUpdateScript, FGuid());
	if (SystemUpdateEntry == nullptr)
	{
		ENiagaraScriptUsage ScriptUsage = ENiagaraScriptUsage::SystemUpdateScript;
		FGuid ScriptUsageId = FGuid();
		FName ExecutionSubCategory = FExecutionSubcategoryNames::Update;
		FText DisplayName = LOCTEXT("SystemUpdateGroupName", "System Update");
		FText ToolTip = LOCTEXT("SystemUpdateGroupToolTip", "Occurs every Emitter tick on the CPU.Modules in this stage should compute values for parameters for emitter or particle update or spawning this frame.\r\nModules are executed in order from top to bottom of the stack.");
		SystemUpdateEntry = CreateScriptGroup(SystemScriptViewModel, ScriptUsage, ScriptUsageId, SystemStackEditorData, ExecutionCategory, ExecutionSubCategory, DisplayName, ToolTip);
	}
	NewChildren.Add(SystemUpdateEntry);

	NewChildren.Add(GetOrCreateSystemCommentCollection(CurrentChildren));
}

void UNiagaraStackRoot::RefreshEmitterFullChildren(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren)
{
	UNiagaraStackEditorData& EmitterStackEditorData = GetEmitterViewModel()->GetEditorData().GetStackEditorData();
	TSharedRef<FNiagaraScriptViewModel> EmitterScriptViewModel = GetEmitterViewModel()->GetSharedScriptViewModel();

	NewChildren.Add(GetOrCreateEmitterPropertiesGroup(CurrentChildren));

	NewChildren.Add(GetOrCreateEmitterSummaryGroup(CurrentChildren));

	UNiagaraStackEntry* EmitterSpawnEntry = GetCurrentScriptGroup(CurrentChildren, ENiagaraScriptUsage::EmitterSpawnScript, FGuid());
	if (EmitterSpawnEntry == nullptr)
	{
		ENiagaraScriptUsage ScriptUsage = ENiagaraScriptUsage::EmitterSpawnScript;
		FGuid ScriptUsageId = FGuid();
		FName ExecutionCategory = FExecutionCategoryNames::Emitter;
		FName ExecutionSubCategory = FExecutionSubcategoryNames::Spawn;
		FText DisplayName = LOCTEXT("EmitterSpawnGroupName", "Emitter Spawn");
		FText ToolTip = LOCTEXT("EmitterSpawnGroupTooltip", "Occurs once at Emitter creation on the CPU. Modules in this stage should initialize defaults and/or do initial setup.\r\nModules are executed in order from top to bottom of the stack.");
		EmitterSpawnEntry = CreateScriptGroup(EmitterScriptViewModel, ScriptUsage, ScriptUsageId, EmitterStackEditorData, ExecutionCategory, ExecutionSubCategory, DisplayName, ToolTip);
	}
	NewChildren.Add(EmitterSpawnEntry);

	UNiagaraStackEntry* EmitterUpdateEntry = GetCurrentScriptGroup(CurrentChildren, ENiagaraScriptUsage::EmitterUpdateScript, FGuid());
	if (EmitterUpdateEntry == nullptr)
	{
		ENiagaraScriptUsage ScriptUsage = ENiagaraScriptUsage::EmitterUpdateScript;
		FGuid ScriptUsageId = FGuid();
		FName ExecutionCategory = FExecutionCategoryNames::Emitter;
		FName ExecutionSubCategory = FExecutionSubcategoryNames::Update;
		FText DisplayName = LOCTEXT("EmitterUpdateGroupName", "Emitter Update");
		FText ToolTip = LOCTEXT("EmitterUpdateGroupTooltip", "Occurs every Emitter tick on the CPU. Modules in this stage should compute values for parameters for Particle Update or Spawning this frame.\r\nModules are executed in order from top to bottom of the stack.");
		EmitterUpdateEntry = CreateScriptGroup(EmitterScriptViewModel, ScriptUsage, ScriptUsageId, EmitterStackEditorData, ExecutionCategory, ExecutionSubCategory, DisplayName, ToolTip);
	}
	NewChildren.Add(EmitterUpdateEntry);

	UNiagaraStackEntry* ParticleSpawnEntry = GetCurrentScriptGroup(CurrentChildren, ENiagaraScriptUsage::ParticleSpawnScript, FGuid());
	if (ParticleSpawnEntry == nullptr)
	{
		ENiagaraScriptUsage ScriptUsage = ENiagaraScriptUsage::ParticleSpawnScript;
		FGuid ScriptUsageId = FGuid();
		FName ExecutionCategory = FExecutionCategoryNames::Particle;
		FName ExecutionSubCategory = FExecutionSubcategoryNames::Spawn;
		FText DisplayName = LOCTEXT("ParticleSpawnGroupName", "Particle Spawn");
		FText ToolTip = LOCTEXT("ParticleSpawnGroupTooltip", "Called once per created particle. Modules in this stage should set up initial values for each particle.\r\nIf \"Use Interpolated Spawning\" is set, we will also run the Particle Update stage after the Particle Spawn stage.\r\nModules are executed in order from top to bottom of the stack.");
		ParticleSpawnEntry = CreateScriptGroup(EmitterScriptViewModel, ScriptUsage, ScriptUsageId, EmitterStackEditorData, ExecutionCategory, ExecutionSubCategory, DisplayName, ToolTip);
	}
	NewChildren.Add(ParticleSpawnEntry);

	UNiagaraStackEntry* ParticleUpdateEntry = GetCurrentScriptGroup(CurrentChildren, ENiagaraScriptUsage::ParticleUpdateScript, FGuid());
	if (ParticleUpdateEntry == nullptr)
	{
		ENiagaraScriptUsage ScriptUsage = ENiagaraScriptUsage::ParticleUpdateScript;
		FGuid ScriptUsageId = FGuid();
		FName ExecutionCategory = FExecutionCategoryNames::Particle;
		FName ExecutionSubCategory = FExecutionSubcategoryNames::Update;
		FText DisplayName = LOCTEXT("ParticleUpdateGroupName", "Particle Update");
		FText ToolTip = LOCTEXT("ParticleUpdateGroupTooltip", "Called every frame per particle. Modules in this stage should update new values for this frame.\r\nModules are executed in order from top to bottom of the stack.");
		ParticleUpdateEntry = CreateScriptGroup(EmitterScriptViewModel, ScriptUsage, ScriptUsageId, EmitterStackEditorData, ExecutionCategory, ExecutionSubCategory, DisplayName, ToolTip);
	}
	NewChildren.Add(ParticleUpdateEntry);

	FVersionedNiagaraEmitterData* EmitterData = GetEmitterViewModel()->GetEmitter().GetEmitterData();
	if (EmitterData != nullptr)
	{
		for (const FNiagaraEventScriptProperties& EventScriptProperties : EmitterData->GetEventHandlers())
		{
			UNiagaraStackEntry* EventEntry = GetCurrentScriptGroup(CurrentChildren, ENiagaraScriptUsage::ParticleEventScript, EventScriptProperties.Script->GetUsageId());
			if (EventEntry == nullptr)
			{
				EventEntry = CreateEventScriptGroup(EventScriptProperties.SourceEmitterID, EmitterScriptViewModel, EventScriptProperties.Script->GetUsageId(), EmitterStackEditorData);
			}
			NewChildren.Add(EventEntry);
		}

		for (UNiagaraSimulationStageBase* SimulationStage : EmitterData->GetSimulationStages())
		{
			UNiagaraStackEntry* SimulationStageEntry = GetCurrentScriptGroup(CurrentChildren, ENiagaraScriptUsage::ParticleSimulationStageScript, SimulationStage->Script->GetUsageId());
			if (SimulationStageEntry == nullptr)
			{
				SimulationStageEntry = CreateSimulationStageScriptGroup(SimulationStage, EmitterScriptViewModel, EmitterStackEditorData);
			}
			NewChildren.Add(SimulationStageEntry);
		}
	}

	NewChildren.Add(GetOrCreateEmitterRendererGroup(CurrentChildren, FNiagaraStackRenderersOwnerStandard::CreateShared(GetEmitterViewModel().ToSharedRef())));
}

void UNiagaraStackRoot::RefreshEmitterSummaryChildren(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren)
{
	NewChildren.Add(GetOrCreateEmitterSummaryGroup(CurrentChildren));
}

void UNiagaraStackRoot::RefreshEmitterStatelessChildren(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	NewChildren.Add(GetOrCreateStatelessEmitterGroup(CurrentChildren, EmitterHandleViewModel.ToSharedRef()));
	NewChildren.Add(GetOrCreateStatelessEmitterSpawnGroup(CurrentChildren, EmitterHandleViewModel.ToSharedRef()));
	NewChildren.Add(GetOrCreateStatelessEmitterSimulateGroup(CurrentChildren, EmitterHandleViewModel.ToSharedRef()));
	NewChildren.Add(GetOrCreateEmitterRendererGroup(CurrentChildren, FNiagaraStackRenderersOwnerStateless::CreateShared(EmitterHandleViewModel.ToSharedRef())));
}

UNiagaraStackEntry* UNiagaraStackRoot::GetOrCreateSystemPropertiesGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren)
{
	UNiagaraStackSystemPropertiesGroup* SystemPropertiesGroup = FindCurrentChildOfType<UNiagaraStackSystemPropertiesGroup>(CurrentChildren);
	if (SystemPropertiesGroup == nullptr)
	{
		SystemPropertiesGroup = NewObject<UNiagaraStackSystemPropertiesGroup>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
			FExecutionCategoryNames::System, FExecutionSubcategoryNames::Settings,
			GetSystemViewModel()->GetEditorData().GetStackEditorData());
		SystemPropertiesGroup->Initialize(RequiredEntryData);
	}
	return SystemPropertiesGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::GetOrCreateSystemUserParametersGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren)
{
	UNiagaraStackSystemUserParametersGroup* SystemUserParametersGroup = FindCurrentChildOfType<UNiagaraStackSystemUserParametersGroup>(CurrentChildren);
	if (SystemUserParametersGroup == nullptr)
	{
		SystemUserParametersGroup = NewObject<UNiagaraStackSystemUserParametersGroup>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
			FExecutionCategoryNames::System, FExecutionSubcategoryNames::Settings,
			GetSystemViewModel()->GetEditorData().GetStackEditorData());
		SystemUserParametersGroup->Initialize(RequiredEntryData, &GetSystemViewModel()->GetSystem(), &GetSystemViewModel()->GetSystem().GetExposedParameters());
	}
	return SystemUserParametersGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::GetOrCreateSystemCommentCollection(const TArray<UNiagaraStackEntry*>& CurrentChildren)
{
	UNiagaraStackCommentCollection* CommentCollection = FindCurrentChildOfType<UNiagaraStackCommentCollection>(CurrentChildren);
	if (CommentCollection == nullptr)
	{
		CommentCollection = NewObject<UNiagaraStackCommentCollection>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
			NAME_None, NAME_None,
			GetSystemViewModel()->GetEditorData().GetStackEditorData());
		CommentCollection->Initialize(RequiredEntryData);
	}
	return CommentCollection;
}

UNiagaraStackEntry* UNiagaraStackRoot::GetOrCreateEmitterPropertiesGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren)
{
	UNiagaraStackEmitterPropertiesGroup* EmitterPropertiesGroup = FindCurrentChildOfType<UNiagaraStackEmitterPropertiesGroup>(CurrentChildren);
	if (EmitterPropertiesGroup == nullptr)
	{
		EmitterPropertiesGroup = NewObject<UNiagaraStackEmitterPropertiesGroup>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
			FExecutionCategoryNames::Emitter, FExecutionSubcategoryNames::Settings,
			GetEmitterViewModel()->GetEditorData().GetStackEditorData());
		EmitterPropertiesGroup->Initialize(RequiredEntryData);
	}
	return EmitterPropertiesGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::GetOrCreateEmitterSummaryGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren)
{
	UNiagaraStackEmitterSummaryGroup* EmitterSummaryGroup = FindCurrentChildOfType<UNiagaraStackEmitterSummaryGroup>(CurrentChildren);
	if (EmitterSummaryGroup == nullptr)
	{
		EmitterSummaryGroup = NewObject<UNiagaraStackEmitterSummaryGroup>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
			FExecutionCategoryNames::Emitter, FExecutionSubcategoryNames::Settings,
			GetEmitterViewModel()->GetEditorData().GetStackEditorData());
		FText DisplayName = LOCTEXT("EmitterSummaryGroupName", "Emitter Summary");
		FText Tooltip = LOCTEXT("EmitterSummaryTooltip", "Summary of parameters for this Emitter.");
		EmitterSummaryGroup->Initialize(RequiredEntryData, DisplayName, Tooltip, nullptr);
	}
	return EmitterSummaryGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::GetOrCreateEmitterRendererGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, TSharedPtr<INiagaraStackRenderersOwner> RenderersOwner)
{
	UNiagaraStackRenderItemGroup* RenderGroup = FindCurrentChildOfTypeByPredicate<UNiagaraStackRenderItemGroup>(CurrentChildren, 
		[RenderersOwner](const UNiagaraStackRenderItemGroup* CurrentChild) { return CurrentChild->GetRenderersOwner()->GetOwnerObject() == RenderersOwner->GetOwnerObject(); });
	if (RenderGroup == nullptr)
	{
		RenderGroup = NewObject<UNiagaraStackRenderItemGroup>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
			FExecutionCategoryNames::Render, FExecutionSubcategoryNames::Render,
			GetEmitterViewModel()->GetEditorData().GetStackEditorData());
		RenderGroup->Initialize(RequiredEntryData, RenderersOwner);
	}
	return RenderGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::GetOrCreateStatelessEmitterGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandleViewModel->GetEmitterHandle()->GetStatelessEmitter();
	UNiagaraStackStatelessEmitterGroup* StatelessEmitterObjectGroup = FindCurrentChildOfTypeByPredicate<UNiagaraStackStatelessEmitterGroup>(CurrentChildren,
		[StatelessEmitter](const UNiagaraStackStatelessEmitterGroup* Group) { return Group->GetStatelessEmitter() == StatelessEmitter; });
	if (StatelessEmitterObjectGroup == nullptr)
	{
		StatelessEmitterObjectGroup = NewObject<UNiagaraStackStatelessEmitterGroup>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
			FExecutionCategoryNames::Emitter, FExecutionSubcategoryNames::Settings,
			GetEmitterViewModel()->GetEditorData().GetStackEditorData());
		StatelessEmitterObjectGroup->Initialize(RequiredEntryData, StatelessEmitter);
	}
	return StatelessEmitterObjectGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::GetOrCreateStatelessEmitterSpawnGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandleViewModel->GetEmitterHandle()->GetStatelessEmitter();
	UNiagaraStackStatelessEmitterSpawnGroup* StatelessEmitterSpawnGroup = FindCurrentChildOfTypeByPredicate<UNiagaraStackStatelessEmitterSpawnGroup>(CurrentChildren,
		[StatelessEmitter](const UNiagaraStackStatelessEmitterSpawnGroup* Group) { return Group->GetStatelessEmitter() == StatelessEmitter; });
	if (StatelessEmitterSpawnGroup == nullptr)
	{
		StatelessEmitterSpawnGroup = NewObject<UNiagaraStackStatelessEmitterSpawnGroup>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
			FExecutionCategoryNames::Emitter, FExecutionSubcategoryNames::Spawn,
			GetEmitterViewModel()->GetEditorData().GetStackEditorData());
		StatelessEmitterSpawnGroup->Initialize(RequiredEntryData, EmitterHandleViewModel->GetEmitterHandle()->GetStatelessEmitter());
	}
	return StatelessEmitterSpawnGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::GetOrCreateStatelessEmitterSimulateGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel)
{
	UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandleViewModel->GetEmitterHandle()->GetStatelessEmitter();
	UNiagaraStackStatelessEmitterSimulateGroup* StatelessEmitterSimulateGroup = FindCurrentChildOfTypeByPredicate<UNiagaraStackStatelessEmitterSimulateGroup>(CurrentChildren,
		[StatelessEmitter](const UNiagaraStackStatelessEmitterSimulateGroup* Group) { return Group->GetStatelessEmitter() == StatelessEmitter; });
	if (StatelessEmitterSimulateGroup == nullptr)
	{
		StatelessEmitterSimulateGroup = NewObject<UNiagaraStackStatelessEmitterSimulateGroup>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
			FExecutionCategoryNames::Particle, FExecutionSubcategoryNames::Update,
			GetEmitterViewModel()->GetEditorData().GetStackEditorData());
		StatelessEmitterSimulateGroup->Initialize(RequiredEntryData, EmitterHandleViewModel->GetEmitterHandle()->GetStatelessEmitter());
	}
	return StatelessEmitterSimulateGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::GetCurrentScriptGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, ENiagaraScriptUsage InScriptUsage, FGuid InScriptUsageId) const
{
	return FindCurrentChildOfTypeByPredicate<UNiagaraStackScriptItemGroup>(
		CurrentChildren, [InScriptUsage, InScriptUsageId](UNiagaraStackScriptItemGroup* ScriptItemGroup) {
			return ScriptItemGroup->GetScriptUsage() == InScriptUsage && ScriptItemGroup->GetScriptUsageId() == InScriptUsageId; });
}

UNiagaraStackEntry* UNiagaraStackRoot::CreateScriptGroup(
	TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
	ENiagaraScriptUsage InScriptUsage,
	FGuid InScriptUsageId,
	UNiagaraStackEditorData& InStackEditorData,
	FName InExecutionCategoryName,
	FName InExecutionSubcategoryName,
	FText InDisplayName,
	FText InToolTip)
{
	UNiagaraStackScriptItemGroup* ScriptItemGroup = NewObject<UNiagaraStackScriptItemGroup>(this);
	FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
		InExecutionCategoryName, InExecutionSubcategoryName, InStackEditorData);
	ScriptItemGroup->Initialize(RequiredEntryData, InDisplayName, InToolTip, InScriptViewModel, InScriptUsage, InScriptUsageId);
	return ScriptItemGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::CreateEventScriptGroup(
	FGuid SourceEmitterId,
	TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
	FGuid InScriptUsageId,
	UNiagaraStackEditorData& InStackEditorData)
{
	UNiagaraStackEventScriptItemGroup* EventHandlerGroup = NewObject<UNiagaraStackEventScriptItemGroup>(this);
	FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
		FExecutionCategoryNames::Particle, FExecutionSubcategoryNames::Event,
		InStackEditorData);
	EventHandlerGroup->Initialize(RequiredEntryData, GetEmitterViewModel()->GetSharedScriptViewModel(), ENiagaraScriptUsage::ParticleEventScript, InScriptUsageId, SourceEmitterId);
	EventHandlerGroup->SetOnModifiedEventHandlers(UNiagaraStackEventScriptItemGroup::FOnModifiedEventHandlers::CreateUObject(this, &UNiagaraStackRoot::EmitterArraysChanged));
	return EventHandlerGroup;
}

UNiagaraStackEntry* UNiagaraStackRoot::CreateSimulationStageScriptGroup(
	UNiagaraSimulationStageBase* InSimulationStage,
	TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
	UNiagaraStackEditorData& InStackEditorData)
{
	UNiagaraStackSimulationStageGroup* SimulationStageGroup = NewObject<UNiagaraStackSimulationStageGroup>(this);
	FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
		FExecutionCategoryNames::Particle, FExecutionSubcategoryNames::SimulationStage,
		InStackEditorData);
	SimulationStageGroup->Initialize(RequiredEntryData, InScriptViewModel, InSimulationStage);
	SimulationStageGroup->SetOnModifiedSimulationStages(UNiagaraStackSimulationStageGroup::FOnModifiedSimulationStages::CreateUObject(this, &UNiagaraStackRoot::EmitterArraysChanged));
	return SimulationStageGroup;
}

void UNiagaraStackRoot::EmitterArraysChanged()
{
	RefreshChildren();
}

void UNiagaraStackRoot::OnSummaryViewStateChanged()
{
	if (!IsFinalized())
	{
		RefreshChildren();
	}
}

#undef LOCTEXT_NAMESPACE


