// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackScriptHierarchyRoot.h"

#include "EdGraphSchema_Niagara.h"
#include "NiagaraClipboard.h"
#include "NiagaraScriptSource.h"
#include "Logging/StructuredLog.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackScriptHierarchyRoot)

#define LOCTEXT_NAMESPACE "NiagaraStack"

const FText UNiagaraStackScriptHierarchyRoot::AllSectionName = LOCTEXT("AllSectionName", "All");

void FNiagaraScriptInstanceData::Reset()
{
	UsedInputs.Empty();
	PerInputInstanceData.Empty();
}

void UNiagaraStackScriptHierarchyRoot::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraNodeFunctionCall& InModuleNode, UNiagaraNodeFunctionCall& InInputFunctionCallNode, FString InOwnerStackItemEditorDataKey)
{
	checkf(ModuleNode == nullptr && OwningFunctionCallNode == nullptr, TEXT("Can not set the node more than once."));
	FString InputCollectionStackEditorDataKey = FString::Printf(TEXT("%s-Inputs"), *InInputFunctionCallNode.NodeGuid.ToString(EGuidFormats::DigitsWithHyphens));
	Super::Initialize(InRequiredEntryData, InOwnerStackItemEditorDataKey, InputCollectionStackEditorDataKey);
	ModuleNode = &InModuleNode;
	OwningFunctionCallNode = &InInputFunctionCallNode;
	OwningFunctionCallNode->OnInputsChanged().AddUObject(this, &UNiagaraStackScriptHierarchyRoot::RefreshChildren);

	FNiagaraEditorModule::Get().OnScriptApplied().AddUObject(this, &UNiagaraStackScriptHierarchyRoot::OnScriptApplied);
	
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackScriptHierarchyRoot::FilterForVisibleCondition));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackScriptHierarchyRoot::FilterForIsInlineEditConditionToggle));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackScriptHierarchyRoot::FilterByActiveSection));
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackScriptHierarchyRoot::FilterOnlyModified));
	
	FText LastActiveSection = GetStackEditorData().GetStackEntryActiveSection(GetStackEditorDataKey(), AllSectionName);
	ActiveSection = FindSectionByName(FName(LastActiveSection.ToString()));
}

void UNiagaraStackScriptHierarchyRoot::FinalizeInternal()
{
	if(OwningFunctionCallNode.IsValid())
	{
		OwningFunctionCallNode->OnInputsChanged().RemoveAll(this);
	}
	FNiagaraEditorModule::Get().OnScriptApplied().RemoveAll(this);

	Super::FinalizeInternal();
}

TConstArrayView<const TObjectPtr<UHierarchySection>> UNiagaraStackScriptHierarchyRoot::GerHierarchySectionData() const
{
	if(OwningFunctionCallNode->GetCalledUsage() == ENiagaraScriptUsage::Module)
	{
		return GetScriptParameterHierarchyRoot()->GetSectionData();
	}

	static TArray<const TObjectPtr<UHierarchySection>> Dummy;
	return Dummy;
}

const UHierarchySection* UNiagaraStackScriptHierarchyRoot::GetActiveHierarchySection() const
{
	return ActiveSection.IsValid() ? ActiveSection.Get() : nullptr;
}

void UNiagaraStackScriptHierarchyRoot::SetActiveHierarchySection(const UHierarchySection* InActiveSection)
{
	ActiveSection = InActiveSection;
	
	if(ActiveSection.IsValid())
	{
		GetStackEditorData().SetStackEntryActiveSection(GetStackEditorDataKey(), ActiveSection->GetSectionNameAsText());
	}
	else
	{
		GetStackEditorData().ClearStackEntryActiveSection(GetStackEditorDataKey());
	}
	
	RefreshFilteredChildren();
}

UHierarchyRoot* UNiagaraStackScriptHierarchyRoot::GetScriptParameterHierarchyRoot() const
{
	return OwningFunctionCallNode->GetFunctionScriptSource()->NodeGraph->GetScriptParameterHierarchyRoot();
}

void UNiagaraStackScriptHierarchyRoot::SetShouldDisplayLabel(bool bInShouldDisplayLabel)
{
	bShouldDisplayLabel = bInShouldDisplayLabel;
}

void UNiagaraStackScriptHierarchyRoot::RefreshInstanceData()
{
	ScriptInstanceData.Reset();

	TSet<FNiagaraVariable>& UsedInputs = ScriptInstanceData.UsedInputs;
	
	TSet<FNiagaraVariable> HiddenVariables;
	TArray<FNiagaraVariable> InputVariables;
	FCompileConstantResolver ConstantResolver;
	if (GetEmitterViewModel().IsValid())
	{
		ConstantResolver = FCompileConstantResolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*OwningFunctionCallNode));
	}
	else
	{
		// if we don't have an emitter model, we must be in a system context
		ConstantResolver = FCompileConstantResolver(&GetSystemViewModel()->GetSystem(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*OwningFunctionCallNode));
	}
	GetStackFunctionInputs(*OwningFunctionCallNode, InputVariables, HiddenVariables, ConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly);

	UsedInputs.Append(InputVariables);
	// Gather static switch parameters
	TSet<UEdGraphPin*> HiddenSwitchPins;
	TArray<UEdGraphPin*> SwitchPins;
	FNiagaraStackGraphUtilities::GetStackFunctionStaticSwitchPins(*OwningFunctionCallNode, SwitchPins, HiddenSwitchPins, ConstantResolver);

	TArray<FNiagaraVariable> StaticSwitchVariables;

	Algo::Transform(SwitchPins, UsedInputs, [](UEdGraphPin* StaticSwitchPin)
	{
		return UEdGraphSchema_Niagara::PinToNiagaraVariable(StaticSwitchPin);
	});
	
	Algo::Transform(HiddenSwitchPins, HiddenVariables, [](UEdGraphPin* Pin)
	{
		return UEdGraphSchema_Niagara::PinToNiagaraVariable(Pin);
	});

	UNiagaraGraph* NiagaraGraph = OwningFunctionCallNode->GetFunctionScriptSource()->NodeGraph;
	const TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>& ScriptVariableMap = NiagaraGraph->GetAllMetaData();

	auto IsInputValid = [&ScriptVariableMap](const FNiagaraVariable& InputVariable) -> bool
	{
		if (InputVariable.GetType().IsValid() == false)
		{
			return false;
		}

		if (const UNiagaraScriptVariable* MatchingScriptVariable = ScriptVariableMap.FindRef(InputVariable))
		{
			return MatchingScriptVariable->Metadata.bInlineEditConditionToggle == false;
		}

		return false;
	};

	for(auto It(UsedInputs.CreateIterator()); It; ++It)
	{
		if (!IsInputValid(*It))
		{
			It.RemoveCurrent();
		}
	}
	
	for(const FNiagaraVariable& InputVariable : UsedInputs)
	{
		UNiagaraScriptVariable* ScriptVariable = ScriptVariableMap[InputVariable];		
		FNiagaraFunctionInputInstanceData& InputInstanceData = ScriptInstanceData.PerInputInstanceData.Add(ScriptVariable->Metadata.GetVariableGuid());
		
		if(HiddenVariables.Contains(InputVariable))
		{
			InputInstanceData.bIsHidden = true;	
		}
	}
}

void UNiagaraStackScriptHierarchyRoot::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if(OwningFunctionCallNode->FunctionScript == nullptr)
	{
		return;
	}
	
	RefreshInstanceData();
	
	// First we determine the inputs that the hierarchy _does not_ take care of. We add them at the end.
	TArray<UNiagaraHierarchyScriptParameter*> AllScriptParametersInHierarchy;
	TSet<FNiagaraVariable> AllNiagaraVariablesInHierarchy;
	TSet<FNiagaraVariable> LeftoverInputs;
	GetScriptParameterHierarchyRoot()->GetChildrenOfType(AllScriptParametersInHierarchy, true);
	
	AllNiagaraVariablesInHierarchy.Reserve(AllScriptParametersInHierarchy.Num());
	Algo::Transform(AllScriptParametersInHierarchy, AllNiagaraVariablesInHierarchy, [this](UNiagaraHierarchyScriptParameter* ScriptParameter)
	{
		if(UNiagaraScriptVariable* ScriptVariable = ScriptParameter->GetScriptVariable())
		{
			return ScriptVariable->Variable;
		}

		UE_LOGFMT(LogNiagaraEditor, Verbose, "Invalid hierarchy script parameter encountered. A refresh to the hierarchy script data of script {0} should fix this. Skipping for now.", OwningFunctionCallNode->FunctionScript->GetPathName());
		return FNiagaraVariable();		
	});

	// This should generally not happen but we remove the invalid variable just in case
	AllNiagaraVariablesInHierarchy.Remove(FNiagaraVariable());
	
	for(const FNiagaraVariable& UsedInput : ScriptInstanceData.UsedInputs)
	{		
		if(AllNiagaraVariablesInHierarchy.Contains(UsedInput) == false)
		{
			LeftoverInputs.Add(UsedInput);
		}
	}
	
	for(const UHierarchyElement* ChildElement : GetScriptParameterHierarchyRoot()->GetChildren())
	{
		if(const UNiagaraHierarchyScriptParameter* HierarchyParameter = Cast<UNiagaraHierarchyScriptParameter>(ChildElement))
		{
			TOptional<FNiagaraVariable> InputVariableCandidate = HierarchyParameter->GetVariable();

			if(InputVariableCandidate.IsSet() == false)
			{
				continue;
			}

			FNiagaraVariable InputVariable = InputVariableCandidate.GetValue();

			// If an input isn't used at all (such as when a parameter exists on a loose map get node, or on no node), we skip it
			if(ScriptInstanceData.UsedInputs.Contains(InputVariable) == false)
			{
				continue;
			}
			
			UNiagaraStackFunctionInput* InputChild = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
			{ 
				return CurrentInput->GetInputParameterHandle() == FNiagaraParameterHandle(InputVariable.GetName()) && CurrentInput->GetInputType() == InputVariable.GetType() && &CurrentInput->GetInputFunctionCallNode() == OwningFunctionCallNode;
			});

			if (InputChild == nullptr)
			{
				EStackParameterBehavior Behavior = HierarchyParameter->GetScriptVariable()->GetIsStaticSwitch() ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic;
				InputChild = NewObject<UNiagaraStackFunctionInput>(this);
				InputChild->Initialize(CreateDefaultChildRequiredData(), *ModuleNode, *OwningFunctionCallNode.Get(),
					InputVariable.GetName(), InputVariable.GetType(), Behavior, GetOwnerStackItemEditorDataKey());
			}

			InputChild->SetScriptInstanceData(ScriptInstanceData);
			FGuid VariableGuid = HierarchyParameter->GetScriptVariable()->Metadata.GetVariableGuid();
			InputChild->SetIsHidden(ScriptInstanceData.PerInputInstanceData[VariableGuid].bIsHidden);
			
			NewChildren.AddUnique(InputChild);
		}
		
		if(const UHierarchyCategory* HierarchyCategory = Cast<UHierarchyCategory>(ChildElement))
		{
			// Try to find an already existing category to reuse
			UNiagaraStackScriptHierarchyCategory* StackCategory = FindCurrentChildOfTypeByPredicate<UNiagaraStackScriptHierarchyCategory>(CurrentChildren,
				[&](UNiagaraStackScriptHierarchyCategory* CurrentCategory) { return CurrentCategory->GetHierarchyCategory() == ChildElement; });

			if (StackCategory == nullptr)
			{
				// If we don't have a current child for this category make a new one.
				StackCategory = NewObject<UNiagaraStackScriptHierarchyCategory>(this);
				StackCategory->SetOwningModuleNode(GetOwningModuleNode());
				StackCategory->SetOwningFunctionCallNode(GetOwningFunctionCallNode());
				StackCategory->SetScriptInstanceData(ScriptInstanceData);
				FString InputCategoryStackEditorDataKey = FString::Printf(TEXT("%s-InputCategory-%s"), *OwningFunctionCallNode->NodeGuid.ToString(EGuidFormats::DigitsWithHyphens), *HierarchyCategory->ToString());
				StackCategory->Initialize(CreateDefaultChildRequiredData(), *HierarchyCategory, GetOwnerStackItemEditorDataKey(), InputCategoryStackEditorDataKey);
			}
			
			StackCategory->SetScriptInstanceData(ScriptInstanceData);

			NewChildren.AddUnique(StackCategory);
		}		
	}

	for(FNiagaraVariable LeftoverInput : LeftoverInputs)
	{
		if(ScriptInstanceData.UsedInputs.Contains(LeftoverInput) == false)
		{
			continue;
		}
		
		UNiagaraScriptVariable* ScriptVariable = GetOwningFunctionCallNode().GetCalledGraph()->GetScriptVariable(LeftoverInput);
		UNiagaraStackFunctionInput* InputChild = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
		{ 
			return CurrentInput->GetInputParameterHandle() == FNiagaraParameterHandle(LeftoverInput.GetName()) && CurrentInput->GetInputType() == LeftoverInput.GetType() && &CurrentInput->GetInputFunctionCallNode() == OwningFunctionCallNode;
		});

		if (InputChild == nullptr)
		{
			EStackParameterBehavior Behavior = ScriptVariable->GetIsStaticSwitch() ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic;
			InputChild = NewObject<UNiagaraStackFunctionInput>(this);
			InputChild->SetScriptInstanceData(ScriptInstanceData);
			InputChild->Initialize(CreateDefaultChildRequiredData(), *ModuleNode, *OwningFunctionCallNode.Get(),
				LeftoverInput.GetName(), LeftoverInput.GetType(), Behavior, GetOwnerStackItemEditorDataKey());
		}

		FGuid VariableGuid = ScriptVariable->Metadata.GetVariableGuid();
		if(ScriptInstanceData.PerInputInstanceData.Contains(VariableGuid))
		{
			InputChild->SetIsHidden(ScriptInstanceData.PerInputInstanceData[VariableGuid].bIsHidden);
		}
		NewChildren.AddUnique(InputChild);
	}
}

const UHierarchySection* UNiagaraStackScriptHierarchyRoot::FindSectionByName(FName SectionName)
{
	const TObjectPtr<UHierarchySection>* FoundSection = GerHierarchySectionData().FindByPredicate([SectionName](const UHierarchySection* SectionCandidate)
	{
		return SectionCandidate->GetSectionName() == SectionName;
	});

	if(FoundSection == nullptr)
	{
		return nullptr;
	}
	
	return *FoundSection; 
}

bool UNiagaraStackScriptHierarchyRoot::FilterByActiveSection(const UNiagaraStackEntry& Child) const
{
	if(ActiveSection.IsValid() == false)
	{
		return true;
	}

	if(const UNiagaraStackScriptHierarchyCategory* StackHierarchyCategory = Cast<UNiagaraStackScriptHierarchyCategory>(&Child))
	{
		const UHierarchyCategory* HierarchyCategory = StackHierarchyCategory->GetHierarchyCategory();
		const FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchyCategory->FindMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
		if(SectionAssociation && SectionAssociation->Section == ActiveSection)
		{
			return true;
		}
	}

	return false;
}

bool UNiagaraStackScriptHierarchyRoot::FilterForVisibleCondition(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackFunctionInput* StackFunctionInputChild = Cast<UNiagaraStackFunctionInput>(&Child);
	return StackFunctionInputChild == nullptr || StackFunctionInputChild->GetShouldPassFilterForVisibleCondition();
}

bool UNiagaraStackScriptHierarchyRoot::FilterOnlyModified(const UNiagaraStackEntry& Child) const
{
	if(GetStackEditorData().GetShowOnlyModified() == false)
	{
		return true;
	}
	
	const UNiagaraStackFunctionInput* FunctionInput = Cast<UNiagaraStackFunctionInput>(&Child);
	if (FunctionInput == nullptr || FunctionInput->CanReset() || FunctionInput->HasAnyResettableChildrenInputs())
	{
		return true;
	}

	return false;
}

bool UNiagaraStackScriptHierarchyRoot::FilterForIsInlineEditConditionToggle(const UNiagaraStackEntry& Child) const
{
	const UNiagaraStackFunctionInput* StackFunctionInputChild = Cast<UNiagaraStackFunctionInput>(&Child);
	return StackFunctionInputChild == nullptr || StackFunctionInputChild->GetIsInlineEditConditionToggle() == false;
}

bool UNiagaraStackScriptHierarchyRoot::GetCanExpand() const
{
	return bShouldDisplayLabel;
}

bool UNiagaraStackScriptHierarchyRoot::GetShouldShowInStack() const
{
	return GerHierarchySectionData().Num() > 0 || bShouldDisplayLabel;
}

int32 UNiagaraStackScriptHierarchyRoot::GetChildIndentLevel() const
{
	return GetIndentLevel();
}

void UNiagaraStackScriptHierarchyRoot::OnScriptApplied(UNiagaraScript* NiagaraScript, FGuid Guid)
{
	if(OwningFunctionCallNode->FunctionScript == NiagaraScript)
	{
		RefreshChildren();
	}
}

bool UNiagaraStackScriptHierarchyRoot::GetShouldDisplayLabel() const
{
	return bShouldDisplayLabel;
}

void UNiagaraStackScriptHierarchyRoot::ToClipboardFunctionInputs(UObject* InOuter, TArray<const UNiagaraClipboardFunctionInput*>& OutClipboardFunctionInputs) const
{
	TArray<UNiagaraStackScriptHierarchyCategory*> Categories;
	GetUnfilteredChildrenOfType(Categories, true);

	for (UNiagaraStackScriptHierarchyCategory* Category : Categories)
	{
		TArray<UNiagaraStackFunctionInput*> Inputs;
		Category->GetUnfilteredChildrenOfType(Inputs, false);

		for (UNiagaraStackFunctionInput* Input : Inputs)
		{
			const UNiagaraClipboardFunctionInput* ClipboardFunctionInput = Input->ToClipboardFunctionInput(InOuter);

			if(ClipboardFunctionInput)
			{
				OutClipboardFunctionInputs.Add(ClipboardFunctionInput);
			}
		}
	}
	
	TArray<UNiagaraStackFunctionInput*> Inputs;
	GetUnfilteredChildrenOfType(Inputs, false);
	for (UNiagaraStackFunctionInput* Input : Inputs)
	{
		const UNiagaraClipboardFunctionInput* ClipboardFunctionInput = Input->ToClipboardFunctionInput(InOuter);

		if(ClipboardFunctionInput)
		{
			OutClipboardFunctionInputs.Add(ClipboardFunctionInput);
		}
	}
}

void UNiagaraStackScriptHierarchyRoot::SetValuesFromClipboardFunctionInputs(const TArray<const UNiagaraClipboardFunctionInput*>& ClipboardFunctionInputs)
{
	// Copy/paste of function inputs must be restricted to inputs owned by a single function but we need to collect inputs recursively since
	// the hierarchy view allows inputs within a single function to be "child inputs" of each other.  When collecting recursively by type we
	// can end up with inputs from dynamic input functions which aren't owned by this root, so we need to filter those out to avoid setting 
	// another functions input values and to prevent crashes which can happen when setting an input on this function removes the dynamic input
	// which will finalize the dynamic input's child inputs.
	auto GetChildInputsWithMatchingOwner = [this](TArray<UNiagaraStackFunctionInput*>& OutMatchingInputs)
	{
		TArray<UNiagaraStackFunctionInput*> AllInputs;
		GetUnfilteredChildrenOfType(AllInputs, true);
		for (UNiagaraStackFunctionInput* Input : AllInputs)
		{
			if (&Input->GetInputFunctionCallNode() == OwningFunctionCallNode.Get())
			{
				OutMatchingInputs.Add(Input);
			}
		}
	};
	TArray<UNiagaraStackFunctionInput*> StackFunctionInputs;
	GetChildInputsWithMatchingOwner(StackFunctionInputs);

	//First try to set each input as a static switch, and if a switch is set refresh the categories
	// before applying additional inputs.  This is necessary because static switches can change the set
	// of exposed inputs.
	// NOTE: It's still possible that inputs could end up missing in cases where the switch dependencies are 
	// especially complex since we're not doing a full switch dependency check, but this should handle the
	// vast majority of cases.
	for (const UNiagaraClipboardFunctionInput* ClipboardFunctionInput : ClipboardFunctionInputs)
	{
		bool bInputSetAsSwitch = false;
		for (UNiagaraStackFunctionInput* StackFunctionInput : StackFunctionInputs)
		{
			if(StackFunctionInput->IsStaticParameter() &&
				StackFunctionInput->GetInputParameterHandle().GetName() == ClipboardFunctionInput->InputName &&
				StackFunctionInput->GetInputType() == ClipboardFunctionInput->InputType)
			{
				if(ClipboardFunctionInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::ResetToDefault)
				{
					StackFunctionInput->Reset();
				}
				else
				{
					StackFunctionInput->PasteFunctionInput(ClipboardFunctionInput);
				}

				bInputSetAsSwitch = true;
				break;
			}
		}
		
		if (bInputSetAsSwitch)
		{
			RefreshChildren();
			StackFunctionInputs.Empty();
			GetChildInputsWithMatchingOwner(StackFunctionInputs);
		}
	}

	for (const UNiagaraClipboardFunctionInput* ClipboardFunctionInput : ClipboardFunctionInputs)
	{
		// After all static switches have been set the remaining standard inputs can be set without additional refreshes.
		for (UNiagaraStackFunctionInput* StackFunctionInput : StackFunctionInputs)
		{
			if (StackFunctionInput->IsStaticParameter() == false && 
				StackFunctionInput->GetInputParameterHandle().GetName() == ClipboardFunctionInput->InputName &&
				StackFunctionInput->GetInputType() == ClipboardFunctionInput->InputType)
			{
				if (ClipboardFunctionInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::ResetToDefault)
				{
					StackFunctionInput->Reset();
				}
				else
				{
					StackFunctionInput->PasteFunctionInput(ClipboardFunctionInput);
				}
			}
		}
	}
}

void UNiagaraStackScriptHierarchyRoot::GetChildInputs(TArray<UNiagaraStackFunctionInput*>& OutInputs) const
{
	TArray<UNiagaraStackScriptHierarchyCategory*> Categories;
	GetUnfilteredChildrenOfType(Categories, true);

	for (UNiagaraStackScriptHierarchyCategory* Category : Categories)
	{
		TArray<UNiagaraStackFunctionInput*> Inputs;
		Category->GetUnfilteredChildrenOfType(Inputs, false);

		for (UNiagaraStackFunctionInput* Input : Inputs)
		{
			OutInputs.Add(Input);
		}
	}
	
	TArray<UNiagaraStackFunctionInput*> Inputs;
	GetUnfilteredChildrenOfType(Inputs, false);
	for (UNiagaraStackFunctionInput* Input : Inputs)
	{
		OutInputs.Add(Input);
	}
}

TArray<UNiagaraStackFunctionInput*> UNiagaraStackScriptHierarchyRoot::GetInlineParameters() const
{
	TArray<UNiagaraStackFunctionInput*> Result;
	
	TArray<UNiagaraStackScriptHierarchyCategory*> Categories;
	GetFilteredChildrenOfType(Categories, true);

	for (UNiagaraStackScriptHierarchyCategory* Category : Categories)
	{
		TArray<UNiagaraStackFunctionInput*> Inputs;
		Category->GetFilteredChildrenOfType(Inputs, false);

		for (UNiagaraStackFunctionInput* Input : Inputs)
		{
			Result.Add(Input);
		}
	}
	
	TArray<UNiagaraStackFunctionInput*> Inputs;
	GetFilteredChildrenOfType(Inputs, false);
	for (UNiagaraStackFunctionInput* Input : Inputs)
	{
		Result.Add(Input);
	}

	Result.RemoveAll([](UNiagaraStackFunctionInput* RemovalCandidate)
	{
		return RemovalCandidate->ShouldDisplayInline() == false;
	});

	return Result;
}

#undef LOCTEXT_NAMESPACE
