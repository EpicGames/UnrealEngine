// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraClipboard.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "NiagaraStackEditorData.h"
#include "ScopedTransaction.h"
#include "Config/NiagaraFavoriteActionsConfig.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "Widgets/DataChannel/NiagaraDataChannelWizard.h"
#include "Widgets/Wizard/SNiagaraModuleWizard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackScriptItemGroup)

#define LOCTEXT_NAMESPACE "UNiagaraStackScriptItemGroup"

using namespace UE::Niagara::Wizard;

const FName INiagaraStackItemGroupAddAction::FavoriteActionsProfile = FName("StackGroupAdd");

class FScriptGroupAddAction : public INiagaraStackItemGroupAddAction
{
public:
	enum class ESpecificActionType
	{
		NewScratchModule,
		NewSetSpecificParameters,
		ModuleWizard,
		Undefined
	};
	
	struct FCreationArguments
	{
		FText DisplayName;
		TArray<FString> Categories;
		FText Description;
		FText Keywords;
		TOptional<FNiagaraFavoritesActionData> FavoritesActionData;
		TOptional<FSoftObjectPath> PreviewMoviePath;
		bool bIsInLibrary = false;
		FNiagaraVariable ModuleParameterVariable;
		FAssetData ModuleAssetData;
		UNiagaraScript* ModuleScript = nullptr;
		TSharedPtr<FModuleWizardModel> ModuleWizardModel;
		TOptional<ESpecificActionType> SpecificActionType;
		FNiagaraActionSourceData SourceData = FNiagaraActionSourceData(EScriptSource::Niagara, FText::FromString("Niagara"), true);
		bool bDisplaySource = true;
		
		bool bRenameParameterOnAdd = false;
		
	};
	static TSharedRef<FScriptGroupAddAction> CreateAssetModuleAction(FAssetData AssetData)
	{		
		FText Category;
		AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Category), Category);
		if (Category.IsEmptyOrWhitespace())
		{
			Category = LOCTEXT("ModuleNotCategorized", "Uncategorized Modules");
		}
		
		FText AssetDescription;
		AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Description), AssetDescription);

		FText Keywords;
		AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, Keywords), Keywords);

		bool bIsInLibrary = FNiagaraEditorUtilities::IsScriptAssetInLibrary(AssetData);
		TTuple<EScriptSource, FText> Source = FNiagaraEditorUtilities::GetScriptSource(AssetData);

		bool bSuggested = AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, bSuggested));
		FNiagaraFavoritesActionData FavoritesActionData;
		FavoritesActionData.ActionIdentifier.Names.Add(FName(AssetData.GetObjectPathString()));
		FavoritesActionData.bFavoriteByDefault = bSuggested;

		FCreationArguments CreationArguments;
		CreationArguments.ModuleAssetData = AssetData;
		CreationArguments.DisplayName = FNiagaraEditorUtilities::FormatScriptName(AssetData.AssetName, bIsInLibrary);
		CreationArguments.Categories = { Category.ToString() };
		CreationArguments.Description = FNiagaraEditorUtilities::FormatScriptDescription(AssetDescription, AssetData.GetSoftObjectPath(), bIsInLibrary);
		CreationArguments.Keywords = Keywords;
		CreationArguments.SourceData = FNiagaraActionSourceData(Source.Key, Source.Value, true);
		CreationArguments.FavoritesActionData = FavoritesActionData;
		CreationArguments.bIsInLibrary = bIsInLibrary;
		CreationArguments.PreviewMoviePath = FNiagaraEditorUtilities::Preview::GetPreviewMovieObjectPath(AssetData);
		return MakeShareable(new FScriptGroupAddAction(CreationArguments));
	}

	static TSharedRef<FScriptGroupAddAction> CreateModuleActionFromScratchPadScript(UNiagaraScript* ModuleScript)
	{
		FVersionedNiagaraScriptData* ScriptData = ModuleScript->GetLatestScriptData();
		FText Category = ScriptData->Category;
		if (Category.IsEmptyOrWhitespace())
		{
			Category = LOCTEXT("ModuleNotCategorized", "Uncategorized Modules");
		}
		
		// we assume scratch pads should always be displayed, so we act like it's a library action so it's not accidentally filtered out
		bool bIsInLibrary = true;
		FText DisplayName = FNiagaraEditorUtilities::FormatScriptName(ModuleScript->GetFName(), bIsInLibrary);
		FText Description = FNiagaraEditorUtilities::FormatScriptDescription(ScriptData->Description, FSoftObjectPath(ModuleScript), bIsInLibrary);
		FText Keywords = ScriptData->Keywords;

		FNiagaraFavoritesActionData FavoritesActionData;
		FavoritesActionData.ActionIdentifier.Names.Add(FName(GetPathNameSafe(ModuleScript)));
		FavoritesActionData.bFavoriteByDefault = ScriptData->bSuggested;

		TOptional<FSoftObjectPath> PreviewMoviePath = FNiagaraEditorUtilities::Preview::GetPreviewMovieObjectPath(FAssetData(ModuleScript));
		
		FNiagaraActionSourceData SourceData(EScriptSource::Niagara, FText::FromString("Scratch Pad"), true);

		FCreationArguments CreationArguments;
		CreationArguments.ModuleScript = ModuleScript;
		CreationArguments.DisplayName = DisplayName;
		CreationArguments.Categories = { Category.ToString() };
		CreationArguments.FavoritesActionData = FavoritesActionData;
		CreationArguments.PreviewMoviePath = PreviewMoviePath;
		CreationArguments.SourceData = SourceData;
		CreationArguments.bIsInLibrary = bIsInLibrary;
		CreationArguments.Description = Description;
		CreationArguments.Keywords = Keywords;
		return MakeShareable(new FScriptGroupAddAction(CreationArguments));
	}

	static TSharedRef<FScriptGroupAddAction> CreateSetExistingParameterModuleAction(FNiagaraVariable ParameterVariable)
	{
		FText Category = LOCTEXT("ExistingParameterModuleCategory", "Set Specific Parameters");

		FText DisplayName = FText::FromName(ParameterVariable.GetName());
		FText AttributeDescription = FNiagaraConstants::GetAttributeDescription(ParameterVariable);
		FText Description = FText::Format(LOCTEXT("ExistingParameterModuleDescriptoinFormat", "Description: Set the parameter {0}. {1}"), FText::FromName(ParameterVariable.GetName()), AttributeDescription);

		FCreationArguments CreationArguments;
		CreationArguments.DisplayName = DisplayName;
		CreationArguments.Description = Description;
		CreationArguments.Categories = { Category.ToString() };
		CreationArguments.bIsInLibrary = true;
		CreationArguments.ModuleParameterVariable = ParameterVariable;
		CreationArguments.bRenameParameterOnAdd = false;
		CreationArguments.SpecificActionType = ESpecificActionType::NewSetSpecificParameters;
		return MakeShareable(new FScriptGroupAddAction(CreationArguments));
	}

	static TSharedRef<FScriptGroupAddAction> CreateNewParameterModuleAction(FName NewParameterNamespace, FNiagaraTypeDefinition NewParameterType)
	{
		FText Category = LOCTEXT("NewParameterModuleCategory", "Create New Parameter");
		FText DisplayName = NewParameterType.GetNameText();
		FText Description = FText::Format(LOCTEXT("NewParameterModuleDescriptionFormat", "Description: Create a new {0} parameter."), DisplayName);

		FNiagaraParameterHandle NewParameterHandle(NewParameterNamespace, *(TEXT("New") + NewParameterType.GetName()));
		FNiagaraVariable NewParameter(NewParameterType, NewParameterHandle.GetParameterHandleString());

		FCreationArguments CreationArguments;
		CreationArguments.DisplayName = DisplayName;
		CreationArguments.Description = Description;
		CreationArguments.Categories = { Category.ToString() };
		CreationArguments.bIsInLibrary = true;
		CreationArguments.bRenameParameterOnAdd = true;
		CreationArguments.ModuleParameterVariable = NewParameter;
		return MakeShareable(new FScriptGroupAddAction(CreationArguments));
	}

	static TSharedRef<FScriptGroupAddAction> CreateNewScratchModuleAction()
	{
		FText DisplayName = LOCTEXT("NewScratchModuleName", "New Scratch Pad Module");
		FText Description = LOCTEXT("NewScratchModuleDescription", "Description: Create a new scratch pad module.");
		
		FNiagaraFavoritesActionData FavoritesActionData;
		FavoritesActionData.ActionIdentifier.Names.Add(FName(DisplayName.ToString()));
		FavoritesActionData.bFavoriteByDefault = true;

		FCreationArguments CreationArguments;
		CreationArguments.DisplayName = DisplayName;
		CreationArguments.Description = Description;
		CreationArguments.bIsInLibrary = true;
		CreationArguments.SpecificActionType = ESpecificActionType::NewScratchModule;
		CreationArguments.FavoritesActionData = FavoritesActionData;
		return MakeShareable(new FScriptGroupAddAction(CreationArguments));
	}

	static TSharedRef<FScriptGroupAddAction> CreateNewSetSpecificModuleAction()
	{
		FText DisplayName = LOCTEXT("NewSetSpecificModuleName", "Set new or existing parameter directly");
		FText Description = LOCTEXT("NewSetSpecificModuleDescription", "Description: Create a new module that can set new or existing parameters directly.");
		
		FNiagaraFavoritesActionData FavoritesActionData;
		FavoritesActionData.ActionIdentifier.Names.Add(FName(DisplayName.ToString()));
		FavoritesActionData.bFavoriteByDefault = true;

		FCreationArguments CreationArguments;
		CreationArguments.DisplayName = DisplayName;
		CreationArguments.Description = Description;
		CreationArguments.bIsInLibrary = true;
		CreationArguments.SpecificActionType = ESpecificActionType::NewSetSpecificParameters;
		CreationArguments.FavoritesActionData = FavoritesActionData;
		return MakeShareable(new FScriptGroupAddAction(CreationArguments));
	}

	static TSharedRef<FScriptGroupAddAction> CreateWizardAction(const FModuleWizardGenerator::FAction& Action)
	{
		FNiagaraFavoritesActionData FavoritesActionData;
		FavoritesActionData.ActionIdentifier.Names.Add(Action.WizardModel->GetIdentifier());
		FavoritesActionData.bFavoriteByDefault = Action.bSuggestedAction;

		FCreationArguments CreationArguments;
		CreationArguments.DisplayName = Action.DisplayName;
		CreationArguments.Description = Action.Description;
		CreationArguments.Keywords = Action.Keywords;
		CreationArguments.bIsInLibrary = true;
		CreationArguments.ModuleWizardModel = Action.WizardModel;
		CreationArguments.SpecificActionType = ESpecificActionType::ModuleWizard;
		CreationArguments.FavoritesActionData = FavoritesActionData;
		
		TSharedRef<FScriptGroupAddAction> AddAction = MakeShareable(new FScriptGroupAddAction( CreationArguments));
		return AddAction;
	}

	virtual TArray<FString> GetCategories() const override
	{
		return Categories;
	}

	virtual FText GetDisplayName() const override
	{
		return DisplayName;
	}

	virtual FText GetDescription() const override
	{
		return Description;
	}

	virtual FText GetKeywords() const override
	{
		return Keywords;
	}

	virtual TOptional<FNiagaraFavoritesActionData> GetFavoritesData() const override
	{
		return FavoritesData;
	}

	virtual TOptional<FSoftObjectPath> GetPreviewMoviePath() const override
	{
		return PreviewMoviePath;
	}

	virtual bool IsInLibrary() const override
	{
		return bIsInLibrary;
	}

	virtual FNiagaraActionSourceData GetSourceData() const override
	{
		return SourceData;
	}

	const FNiagaraVariable& GetModuleParameterVariable() const
	{
		return ModuleParameterVariable;
	}

	bool GetRenameParameterOnAdd() const
	{
		return bRenameParameterOnAdd;
	}

	const FAssetData& GetModuleAssetData() const
	{
		return ModuleAssetData;
	}

	UNiagaraScript* GetModuleScript() const
	{
		return ModuleScript;
	}

	bool IsNewScratchModuleAction() const
	{
		return SpecificActionType.IsSet() && SpecificActionType.GetValue() == ESpecificActionType::NewScratchModule;
	}

	bool IsNewSetSpecificParametersModuleAction() const
	{
		return SpecificActionType.IsSet() && SpecificActionType.GetValue() == ESpecificActionType::NewSetSpecificParameters;
	}

	bool HasWizardWidget() const
	{
		return SpecificActionType.IsSet() && SpecificActionType.GetValue() == ESpecificActionType::ModuleWizard && ModuleWizardModel.IsValid();
	}

	TSharedRef<SNiagaraModuleWizard> CreateWizard() const
	{
		check(HasWizardWidget());
		return SNew(SNiagaraModuleWizard, ModuleWizardModel.ToSharedRef());
	}

private:
	FScriptGroupAddAction(const FCreationArguments& CreationArguments)
		: DisplayName(CreationArguments.DisplayName)
		, Categories(CreationArguments.Categories)
		, Description(CreationArguments.Description)
		, Keywords(CreationArguments.Keywords)
		, FavoritesData(CreationArguments.FavoritesActionData)
		, PreviewMoviePath(CreationArguments.PreviewMoviePath)
		, bIsInLibrary(CreationArguments.bIsInLibrary)
		, ModuleParameterVariable(CreationArguments.ModuleParameterVariable)
		, ModuleAssetData(CreationArguments.ModuleAssetData)
		, ModuleScript(CreationArguments.ModuleScript)
		, ModuleWizardModel(CreationArguments.ModuleWizardModel)
		, SpecificActionType(CreationArguments.SpecificActionType)
		, SourceData(CreationArguments.SourceData)
		, bRenameParameterOnAdd(CreationArguments.bRenameParameterOnAdd)
		{
			
		}

	FText DisplayName;
	TArray<FString> Categories;
	FText Description;
	FText Keywords;
	TOptional<FNiagaraFavoritesActionData> FavoritesData;
	TOptional<FSoftObjectPath> PreviewMoviePath;
	bool bIsInLibrary;
	FNiagaraVariable ModuleParameterVariable;
	FAssetData ModuleAssetData;
	UNiagaraScript* ModuleScript = nullptr;
	TSharedPtr<FModuleWizardModel> ModuleWizardModel;
	TOptional<ESpecificActionType> SpecificActionType;
	FNiagaraActionSourceData SourceData;
	bool bRenameParameterOnAdd;
};

class FScriptItemGroupAddUtilities : public TNiagaraStackItemGroupAddUtilities<UNiagaraNodeFunctionCall*>
{
public:
	FScriptItemGroupAddUtilities(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel, TSharedPtr<FNiagaraEmitterViewModel> InEmitterViewModel, UNiagaraStackEditorData& InStackEditorData, FOnItemAdded InOnItemAdded)
		: TNiagaraStackItemGroupAddUtilities(LOCTEXT("ScriptGroupAddItemName", "Module"), EAddMode::AddFromAction, false, false, InOnItemAdded)
		, OutputNode(nullptr)
		, SystemViewModel(InSystemViewModel)
		, EmitterViewModel(InEmitterViewModel)
		, StackEditorData(InStackEditorData)
	{
	}

	void SetOutputNode(UNiagaraNodeOutput* InOutputNode)
	{
		OutputNode = InOutputNode;
	}

	virtual void AddItemDirectly() override { unimplemented(); }

	virtual void GenerateAddActions(TArray<TSharedRef<INiagaraStackItemGroupAddAction>>& OutAddActions, const FNiagaraStackItemGroupAddOptions& AddProperties) const override
	{
		if (SystemViewModel.IsValid() == false || OutputNode == nullptr)
		{
			return;
		}

		// Generate actions for adding script asset modules.
		TArray<FAssetData> ModuleAssets;
		ENiagaraScriptUsage TargetUsage = OutputNode->GetUsage();
		FNiagaraEditorUtilities::FGetFilteredScriptAssetsOptions ModuleScriptFilterOptions;
		ModuleScriptFilterOptions.ScriptUsageToInclude = ENiagaraScriptUsage::Module;
		ModuleScriptFilterOptions.TargetUsageToMatch = TargetUsage;
		ModuleScriptFilterOptions.bIncludeDeprecatedScripts = AddProperties.bIncludeDeprecated;
		ModuleScriptFilterOptions.bIncludeNonLibraryScripts = AddProperties.bIncludeNonLibrary;
		FNiagaraEditorUtilities::GetFilteredScriptAssets(ModuleScriptFilterOptions, ModuleAssets);
		for (const FAssetData& ModuleAsset : ModuleAssets)
		{
			OutAddActions.Add(FScriptGroupAddAction::CreateAssetModuleAction(ModuleAsset));
		}

		// Generate actions for scratch pad modules
		for (TSharedRef<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel : SystemViewModel.Pin()->GetScriptScratchPadViewModel()->GetScriptViewModels())
		{
			UNiagaraScript* ScratchPadScript = ScratchPadScriptViewModel->GetOriginalScript();
			if (ScratchPadScript->Usage == ENiagaraScriptUsage::Module)
			{
				TArray<ENiagaraScriptUsage> SupportedUsages = ScratchPadScript->GetLatestScriptData()->GetSupportedUsageContexts();
				if (SupportedUsages.Contains(TargetUsage))
				{
					OutAddActions.Add(FScriptGroupAddAction::CreateModuleActionFromScratchPadScript(ScratchPadScript));
				}
			}
		}

		OutAddActions.Add(FScriptGroupAddAction::CreateNewScratchModuleAction());
		OutAddActions.Add(FScriptGroupAddAction::CreateNewSetSpecificModuleAction());

		// Generate wizard actions
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		for (TSharedRef<FModuleWizardGenerator> WizardGenerator : NiagaraEditorModule.GetModuleWizards())
		{
			for (const FModuleWizardGenerator::FAction& Action : WizardGenerator->CreateWizardActions(TargetUsage))
			{
				OutAddActions.Add(FScriptGroupAddAction::CreateWizardAction(Action));
			}
		}
	}

	virtual void ExecuteAddAction(TSharedRef<INiagaraStackItemGroupAddAction> AddAction, int32 TargetIndex) override
	{
		TSharedRef<FScriptGroupAddAction> ScriptGroupAddAction = StaticCastSharedRef<FScriptGroupAddAction>(AddAction);
		FScopedTransaction ScopedTransaction(LOCTEXT("InsertNewModule", "Insert new module"));
		UNiagaraNodeFunctionCall* NewModuleNode = nullptr;
		if (ScriptGroupAddAction->GetModuleAssetData().IsValid())
		{
			FNiagaraStackGraphUtilities::FAddScriptModuleToStackArgs AddScriptModuleToStackArgs(ScriptGroupAddAction->GetModuleAssetData(), *OutputNode);
			AddScriptModuleToStackArgs.TargetIndex = TargetIndex;

			// If the user has specified a target index, do not try to fixup the target index.
			if (TargetIndex == INDEX_NONE)
			{
				AddScriptModuleToStackArgs.bFixupTargetIndex = true;
			}

			NewModuleNode = FNiagaraStackGraphUtilities::AddScriptModuleToStack(AddScriptModuleToStackArgs);
		}
		else if (ScriptGroupAddAction->GetModuleScript() != nullptr)
		{
			NewModuleNode = FNiagaraStackGraphUtilities::AddScriptModuleToStack(ScriptGroupAddAction->GetModuleScript(), *OutputNode, TargetIndex);
		}
		else if (ScriptGroupAddAction->HasWizardWidget())
		{
			// create the wizard widget
			TSharedRef<SNiagaraModuleWizard> ModuleWizard = ScriptGroupAddAction->CreateWizard();
			ModuleWizard->Model->TargetUsage = FNiagaraStackGraphUtilities::GetOutputNodeUsage(*OutputNode);
			ModuleWizard->OnCreateModule = FSimpleDelegate::CreateLambda([TargetIndex, ModuleWizard = ModuleWizard.ToSharedPtr(), SystemViewModel = SystemViewModel, EmitterViewModel = EmitterViewModel, OutputNode = TWeakObjectPtr<UNiagaraNodeOutput>(OutputNode), StackEditorData = TWeakObjectPtr<UNiagaraStackEditorData>(&StackEditorData)]()
			{
				if (SystemViewModel.IsValid() && OutputNode.IsValid() && StackEditorData.IsValid())
				{
					// create the module and add it to the stack
					FScopedTransaction AddModuleTransaction(LOCTEXT("AddWizardModuleTransaction", "Add module via wizard"));
					TSharedPtr<FNiagaraSystemViewModel> SystemModel = SystemViewModel.Pin();
					SystemModel->GetSelectionViewModel()->EmptySelection();
					TArray<const UNiagaraNodeFunctionCall*> AddedModules = ModuleWizard->AddModulesToStack(SystemModel, OutputNode.Get(), TargetIndex, EmitterViewModel.Pin(), StackEditorData.Get());
					for (const UNiagaraNodeFunctionCall* ModuleNode : AddedModules)
					{
						SystemModel->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectDeferred(ModuleNode);
					}

					if(EmitterViewModel.IsValid())
					{
						if(TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel = SystemModel->GetEmitterHandleViewModelForEmitter(EmitterViewModel.Pin()->GetEmitter()))
						{
							EmitterHandleViewModel->GetEmitterStackViewModel()->RequestRefreshDeferred();
						}
					}
					else
					{
						SystemModel->GetSystemStackViewModel()->RequestRefreshDeferred();
					}
				}
			});

			// create a new window with the wizard
			const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
			const TSharedRef<SWindow> AddModuleWindow = SNew(SWindow)
				.Title(LOCTEXT("Wizard_Title", "Add new module"))
				.CreateTitleBar(true)
				.ScreenPosition(CursorPos)
				.FocusWhenFirstShown(true)
				.ClientSize(FVector2D(700, 500))
				[
					ModuleWizard
				];

			const IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
			const TSharedPtr<SWindow> ParentWindow = MainFrameModule.GetParentWindow();
			FSlateApplication::Get().AddModalWindow(AddModuleWindow, ParentWindow);
			return;
		}
		else if (ScriptGroupAddAction->IsNewScratchModuleAction())
		{
			TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel = SystemViewModel.Pin()->GetScriptScratchPadViewModel()->CreateNewScript(ENiagaraScriptUsage::Module, OutputNode->GetUsage(), FNiagaraTypeDefinition());
			if (ScratchPadScriptViewModel.IsValid())
			{
				NewModuleNode = FNiagaraStackGraphUtilities::AddScriptModuleToStack(ScratchPadScriptViewModel->GetOriginalScript(), *OutputNode, TargetIndex);
				SystemViewModel.Pin()->GetScriptScratchPadViewModel()->FocusScratchPadScriptViewModel(ScratchPadScriptViewModel.ToSharedRef());
				SystemViewModel.Pin()->FocusTab(FNiagaraSystemToolkitModeBase::ScratchPadScriptsTabID);
				ScratchPadScriptViewModel->SetIsPendingRename(true);
			}
		}
		else if (ScriptGroupAddAction->IsNewSetSpecificParametersModuleAction())
		{
			TOptional<const FNiagaraVariable> InitializeVariable;

			if(ScriptGroupAddAction->GetModuleParameterVariable().IsValid())
			{
				InitializeVariable = ScriptGroupAddAction->GetModuleParameterVariable();
			}
			
			NewModuleNode = AddParameterModule(InitializeVariable, ScriptGroupAddAction->GetRenameParameterOnAdd(), TargetIndex);
		}

		if (NewModuleNode != nullptr)
		{
			FNiagaraStackGraphUtilities::InitializeStackFunctionInputs(SystemViewModel.Pin().ToSharedRef(), EmitterViewModel.Pin(), StackEditorData, *NewModuleNode, *NewModuleNode);
			FNiagaraStackGraphUtilities::RelayoutGraph(*OutputNode->GetGraph());
			OnItemAdded.ExecuteIfBound(NewModuleNode);
		}
	}
private:
	UNiagaraNodeFunctionCall* AddScriptAssetModule(const FAssetData& AssetData, int32 TargetIndex)
	{
		FNiagaraStackGraphUtilities::FAddScriptModuleToStackArgs AddScriptModuleToStackArgs(AssetData, *OutputNode);
		AddScriptModuleToStackArgs.TargetIndex = TargetIndex;
		AddScriptModuleToStackArgs.bFixupTargetIndex = true;
		return FNiagaraStackGraphUtilities::AddScriptModuleToStack(AddScriptModuleToStackArgs);
	}

	UNiagaraNodeFunctionCall* AddParameterModule(TOptional<const FNiagaraVariable>& ParameterVariable, bool bRenameParameterOnAdd, int32 TargetIndex)
	{
		TArray<FNiagaraVariable> Vars;
		TArray<FString> DefaultVals;

		if (ParameterVariable.IsSet() && ParameterVariable.GetValue().IsValid())
		{
			Vars.Add(ParameterVariable.GetValue());
			DefaultVals.Add(FNiagaraConstants::GetAttributeDefaultValue(ParameterVariable.GetValue()));
		}

		UNiagaraNodeAssignment* NewAssignmentModule = FNiagaraStackGraphUtilities::AddParameterModuleToStack(Vars, *OutputNode, TargetIndex,DefaultVals );
		
		TArray<FNiagaraVariable> InputVariables;
		FNiagaraStackGraphUtilities::GetStackFunctionInputs(*NewAssignmentModule, InputVariables, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions::AllInputs);
		if (InputVariables.Num() == 1)
		{
			FString FunctionInputEditorDataKey = FNiagaraStackGraphUtilities::StackKeys::GenerateStackFunctionInputEditorDataKey(*NewAssignmentModule, InputVariables[0].GetName());
			if (bRenameParameterOnAdd)
			{
				StackEditorData.SetStackEntryIsRenamePending(FunctionInputEditorDataKey, true);
			}
		}

		return NewAssignmentModule;
	}

	virtual bool SupportsLibraryFilter() const override { return true; }
	virtual bool SupportsSourceFilter() const override { return true; }

private:
	UNiagaraNodeOutput* OutputNode;
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;
	TWeakPtr<FNiagaraEmitterViewModel> EmitterViewModel;
	UNiagaraStackEditorData& StackEditorData;
};

void UNiagaraStackScriptItemGroup::Initialize(
	FRequiredEntryData InRequiredEntryData,
	FText InDisplayName,
	FText InToolTip,
	TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
	ENiagaraScriptUsage InScriptUsage,
	FGuid InScriptUsageId)
{
	checkf(ScriptViewModel.IsValid() == false, TEXT("Can not set the script view model more than once."));
	AddUtilities = MakeShared<FScriptItemGroupAddUtilities>(InRequiredEntryData.SystemViewModel, InRequiredEntryData.EmitterViewModel,
		*InRequiredEntryData.StackEditorData, TNiagaraStackItemGroupAddUtilities<UNiagaraNodeFunctionCall*>::FOnItemAdded::CreateUObject(this, &UNiagaraStackScriptItemGroup::ItemAdded));
	Super::Initialize(InRequiredEntryData, InDisplayName, InToolTip, AddUtilities.Get());
	ScriptViewModel = InScriptViewModel;
	ScriptUsage = InScriptUsage;
	ScriptUsageId = InScriptUsageId;
	ScriptGraph = InScriptViewModel->GetGraphViewModel()->GetGraph();
	//-TODO:Stateless: Do we need stateless support here?
	if (ScriptGraph.IsValid())
	{
		ScriptGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateUObject(this, &UNiagaraStackScriptItemGroup::OnScriptGraphChanged));
	}

	if (ScriptUsage == ENiagaraScriptUsage::SystemSpawnScript || ScriptUsage == ENiagaraScriptUsage::EmitterSpawnScript)
	{
		OwningSystemWeak = &GetSystemViewModel()->GetSystem();
		OwningSystemWeak->GetSystemSpawnScript()->OnVMScriptCompiled().AddUObject(this, &UNiagaraStackScriptItemGroup::OnSystemScriptCompiled);
	}
	else if (ScriptUsage == ENiagaraScriptUsage::SystemUpdateScript || ScriptUsage == ENiagaraScriptUsage::EmitterUpdateScript)
	{
		OwningSystemWeak = &GetSystemViewModel()->GetSystem();
		OwningSystemWeak->GetSystemUpdateScript()->OnVMScriptCompiled().AddUObject(this, &UNiagaraStackScriptItemGroup::OnSystemScriptCompiled);
	}
	else if (GetEmitterViewModel().IsValid())
	{
		if (FVersionedNiagaraEmitterData* EmitterData = GetEmitterViewModel()->GetEmitter().GetEmitterData())
		{
			OwningParticleScriptWeak = EmitterData->GetScript(ScriptUsage, ScriptUsageId);
			if (OwningParticleScriptWeak.IsValid())
			{
				OwningParticleScriptWeak->OnVMScriptCompiled().AddUObject(this, &UNiagaraStackScriptItemGroup::OnParticleScriptCompiled);
				OwningParticleScriptWeak->OnGPUScriptCompiled().AddUObject(this, &UNiagaraStackScriptItemGroup::OnParticleScriptCompiled);
			}
		}
	}
}

UNiagaraNodeOutput* UNiagaraStackScriptItemGroup::GetScriptOutputNode() const
{
	TSharedPtr<FNiagaraScriptViewModel> ScriptViewModelPinned = ScriptViewModel.Pin();
	checkf(ScriptViewModelPinned.IsValid(), TEXT("Can not get script output node when the script view model has been deleted."));

	UNiagaraGraph* Graph = ScriptViewModelPinned->GetGraphViewModel()->GetGraph();
	return Graph->FindEquivalentOutputNode(ScriptUsage, ScriptUsageId);
}

UObject* UNiagaraStackScriptItemGroup::GetDisplayedObject() const
{
	return ScriptViewModel.IsValid() ? ScriptViewModel.Pin()->GetScript(ScriptUsage, ScriptUsageId) : nullptr;
}

void UNiagaraStackScriptItemGroup::FinalizeInternal()
{
	if (ScriptGraph.IsValid())
	{
		ScriptGraph->RemoveOnGraphChangedHandler(OnGraphChangedHandle);
	}
	if (AddUtilities.IsValid())
	{
		AddUtilities.Reset();
	}
	if (ScriptViewModel.IsValid())
	{
		ScriptViewModel.Reset();
	}

	if ((ScriptUsage == ENiagaraScriptUsage::SystemSpawnScript || ScriptUsage == ENiagaraScriptUsage::EmitterSpawnScript) && OwningSystemWeak.IsValid())
	{
		OwningSystemWeak->GetSystemSpawnScript()->OnVMScriptCompiled().RemoveAll(this);
	}
	else if ((ScriptUsage == ENiagaraScriptUsage::SystemUpdateScript || ScriptUsage == ENiagaraScriptUsage::EmitterUpdateScript) && OwningSystemWeak.IsValid())
	{
		OwningSystemWeak->GetSystemUpdateScript()->OnVMScriptCompiled().RemoveAll(this);
	}
	else if (OwningParticleScriptWeak.IsValid())
	{
		OwningParticleScriptWeak->OnVMScriptCompiled().RemoveAll(this);
		OwningParticleScriptWeak->OnGPUScriptCompiled().RemoveAll(this);
	}

	Super::FinalizeInternal();
}

bool UNiagaraStackScriptItemGroup::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	if (ClipboardContent->Functions.Num() > 0)
	{
		bool bValidFunction = false;
		bool bValidUsage = false;
		UNiagaraNodeOutput* OutputNode = GetScriptOutputNode();
		for (const UNiagaraClipboardFunction* Function : ClipboardContent->Functions)
		{
			if (Function == nullptr)
			{
				continue;
			}
			else if (Function->ScriptMode == ENiagaraClipboardFunctionScriptMode::ScriptAsset)
			{
				UNiagaraScript* ClipboardFunctionScript = Function->Script.LoadSynchronous();
				if (ClipboardFunctionScript != nullptr)
				{
					bValidFunction = true;
					if (ClipboardFunctionScript->GetLatestScriptData()->GetSupportedUsageContexts().Contains(OutputNode->GetUsage()))
					{
						bValidUsage = true;
					}
				}
			}
			else if (Function->ScriptMode == ENiagaraClipboardFunctionScriptMode::Assignment)
			{
				bValidFunction = true;
				for (const FNiagaraVariable& AssignmentTarget : Function->AssignmentTargets)
				{
					if (FNiagaraStackGraphUtilities::CanWriteParameterFromUsage(AssignmentTarget, OutputNode->GetUsage()))
					{
						bValidUsage = true;
						continue;
					}
				}
			}
		}

		if (bValidFunction && bValidUsage)
		{
			OutMessage = LOCTEXT("PasteModules", "Paste modules from the clipboard which have valid function scripts and valid usage.");
			return true;
		}
		else
		{
			if (bValidFunction == false)
			{
				OutMessage = LOCTEXT("CantPasteModulesInvalidFunction", "Can't paste the copied modules because they don't references scripts which are still valid.");
			}
			else
			{
				OutMessage = LOCTEXT("CantPasteModulesForUsage", "Can't paste the copied modules because they don't support the correct usage.");
			}
			return false;
		}
	}
	OutMessage = FText();
	return false;
}

FText UNiagaraStackScriptItemGroup::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	return LOCTEXT("PasteModuleTransactionText", "Paste niagara modules");
}

void UNiagaraStackScriptItemGroup::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	if (ClipboardContent->Functions.Num() > 0)
	{
		TArray<UNiagaraStackModuleItem*> ModuleItems;
		GetUnfilteredChildrenOfType(ModuleItems);
		int32 PasteIndex = ModuleItems.Num() > 0 ? ModuleItems.Last()->GetModuleIndex() + 1 : 0;
		PasteModules(ClipboardContent, PasteIndex, OutPasteWarning);

		RequestPulse();
	}
}

void UNiagaraStackScriptItemGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	TSharedPtr<FNiagaraScriptViewModel> ScriptViewModelPinned = ScriptViewModel.Pin();
	checkf(ScriptViewModelPinned.IsValid(), TEXT("Can not refresh children when the script view model has been deleted."));

	UNiagaraGraph* Graph = ScriptViewModelPinned->GetGraphViewModel()->GetGraph();
	//-TODO:Stateless: Do we need stateless support here?
	if (Graph == nullptr)
	{
		return;
	}

	FText ErrorMessage;
	bIsValidForOutput = false;
	if (FNiagaraStackGraphUtilities::ValidateGraphForOutput(*Graph, ScriptUsage, ScriptUsageId, ErrorMessage) == true)
	{
		bIsValidForOutput = true;

		UNiagaraNodeOutput* MatchingOutputNode = Graph->FindEquivalentOutputNode(ScriptUsage, ScriptUsageId);
		AddUtilities->SetOutputNode(MatchingOutputNode);

		TArray<UNiagaraNodeFunctionCall*> ModuleNodes;
		FNiagaraStackGraphUtilities::GetOrderedModuleNodes(*MatchingOutputNode, ModuleNodes);
		for (UNiagaraNodeFunctionCall* ModuleNode : ModuleNodes)
		{
			UNiagaraStackModuleItem* ModuleItem = FindCurrentChildOfTypeByPredicate<UNiagaraStackModuleItem>(CurrentChildren, 
				[=](UNiagaraStackModuleItem* CurrentModuleItem) { return &CurrentModuleItem->GetModuleNode() == ModuleNode; });

			if (ModuleItem == nullptr)
			{
				ModuleItem = NewObject<UNiagaraStackModuleItem>(this);
				ModuleItem->Initialize(CreateDefaultChildRequiredData(), GetAddUtilities(), *ModuleNode);
				ModuleItem->OnModifiedGroupItems().AddUObject(this, &UNiagaraStackScriptItemGroup::ChildModifiedGroupItems);
				ModuleItem->SetOnRequestCanPaste(UNiagaraStackModuleItem::FOnRequestCanPaste::CreateUObject(this, &UNiagaraStackScriptItemGroup::ChildRequestCanPaste));
				ModuleItem->SetOnRequestPaste(UNiagaraStackModuleItem::FOnRequestPaste::CreateUObject(this, &UNiagaraStackScriptItemGroup::ChildRequestPaste));
				ModuleItem->SetOnRequestDeprecationRecommended(UNiagaraStackModuleItem::FOnRequestDeprecationRecommended::CreateUObject(this, &UNiagaraStackScriptItemGroup::ChildRequestDeprecatedRecommendation));
			}

			NewChildren.Add(ModuleItem);
		}
	}
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
	RefreshIssues(NewIssues);
}

void UNiagaraStackScriptItemGroup::RefreshIssues(TArray<FStackIssue>& NewIssues)
{
	TSharedPtr<FNiagaraScriptViewModel> ScriptViewModelPinned = ScriptViewModel.Pin();
	checkf(ScriptViewModelPinned.IsValid(), TEXT("Can not refresh issues when the script view model has been deleted."));
	UNiagaraGraph* Graph = ScriptViewModelPinned->GetGraphViewModel()->GetGraph();
	FText ErrorMessage;
	
	if (FNiagaraStackGraphUtilities::ValidateGraphForOutput(*Graph, ScriptUsage, ScriptUsageId, ErrorMessage) == false)
	{

		FText FixDescription = LOCTEXT("FixStackGraph", "Fix invalid stack graph");
		FStackIssueFix ResetStackFix(
			FixDescription,
			FStackIssueFixDelegate::CreateLambda([this, FixDescription, Graph]()
		{
			FScopedTransaction ScopedTransaction(FixDescription);
			FNiagaraStackGraphUtilities::ResetGraphForOutput(*Graph, ScriptUsage, ScriptUsageId);
			FNiagaraStackGraphUtilities::RelayoutGraph(*Graph);
		}));

		FStackIssue InvalidStackError(
			EStackIssueSeverity::Error,
			LOCTEXT("InvalidErrorSummaryText", "The stack data is invalid"),
			LOCTEXT("InvalidErrorText", "The data used to generate the stack has been corrupted and can not be used.\nUsing the fix option will reset this part of the stack to its default empty state."),
			GetStackEditorDataKey(),
			false,
			ResetStackFix);

		NewIssues.Add(InvalidStackError); 
	}
	else
	{
		bool bForcedError = false;
		if (ScriptUsage == ENiagaraScriptUsage::SystemUpdateScript)
		{
			// We need to make sure that System Update Scripts have the SystemLifecycle script for now.
			// The factor ensures this, but older assets may not have it or it may have been removed accidentally.
			// For now, treat this as an error and allow them to resolve.
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			const FAssetData ModuleScriptAsset = AssetRegistryModule.Get().GetAssetByObjectPath(GetDefault<UNiagaraEditorSettings>()->RequiredSystemUpdateScript);

			TArray<UNiagaraNodeFunctionCall*> FoundCalls;
			UNiagaraNodeOutput* MatchingOutputNode = Graph->FindOutputNode(ScriptUsage, ScriptUsageId);
			if (ModuleScriptAsset.IsValid() && !FNiagaraStackGraphUtilities::FindScriptModulesInStack(ModuleScriptAsset, *MatchingOutputNode, FoundCalls))
			{
				bForcedError = true;

				FText FixDescription = FText::Format(LOCTEXT("AddingRequiredSystemUpdateModuleFormat", "Add {0} module."), FText::FromName(ModuleScriptAsset.AssetName));
				FStackIssueFix AddRequiredSystemUpdateModule(
					FixDescription,
					FStackIssueFixDelegate::CreateLambda([=]()
				{
					FScopedTransaction ScopedTransaction(FixDescription);
					UNiagaraNodeFunctionCall* AddedModuleNode = FNiagaraStackGraphUtilities::AddScriptModuleToStack(ModuleScriptAsset, *MatchingOutputNode);
					if (AddedModuleNode == nullptr)
					{
						FNotificationInfo Info(LOCTEXT("FailedToAddSystemLifecycle", "Failed to add required module.\nCheck the log for errors."));
						Info.ExpireDuration = 5.0f;
						Info.bFireAndForget = true;
						Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
						FSlateNotificationManager::Get().AddNotification(Info);
					}
				}));

				FStackIssue MissingRequiredSystemUpdateModuleError(
					EStackIssueSeverity::Error,
					FText::Format(LOCTEXT("RequiredSystemUpdateModuleWarningFormat", "The stack needs a {0} module."), FText::FromName(ModuleScriptAsset.AssetName)),
					LOCTEXT("MissingRequiredMode", "Missing required module."),
					GetStackEditorDataKey(),
					false,
					AddRequiredSystemUpdateModule);

				NewIssues.Add(MissingRequiredSystemUpdateModuleError);
			}
		}

		ENiagaraScriptCompileStatus Status = ScriptViewModelPinned->GetScriptCompileStatus(GetScriptUsage(), GetScriptUsageId());
		if (!bForcedError)
		{

			if (Status == ENiagaraScriptCompileStatus::NCS_Error)
			{
				FStackIssue CompileError(
					EStackIssueSeverity::Error,
					LOCTEXT("ConpileErrorSummary", "This stage has compile errors."),
					LOCTEXT("CompileErrorLong", "Errors were generated while compiling this stage. Check for modules showing errors or check the niagara log for details."),
					GetStackEditorDataKey(),
					false,
					FStackIssueFix(
						LOCTEXT("ShowLogForErrorsFix", "Show the errors in the Niagara Log"), 
						FStackIssueFixDelegate::CreateLambda([this]()
						{
							GetSystemViewModel()->FocusTab(FNiagaraSystemToolkitModeBase::MessageLogTabID);
						}),
						EStackIssueFixStyle::Link));

				NewIssues.Add(CompileError);
			}
			else if (Status == ENiagaraScriptCompileStatus::NCS_UpToDateWithWarnings || Status == ENiagaraScriptCompileStatus::NCS_ComputeUpToDateWithWarnings)
			{
				FStackIssue CompileWarning(
					EStackIssueSeverity::Warning,
					LOCTEXT("CompileWarningSummary", "This stage has compile warnings."),
					LOCTEXT("CompileWarningLong", "Warnings were generated while compiling this stage. Check for modules showing warnings or check the niagara log for details."),
					GetStackEditorDataKey(),
					false,
					FStackIssueFix(
						LOCTEXT("ShowLogForWarningsFix", "Show the warnings in the Niagara Log"),
						FStackIssueFixDelegate::CreateLambda([this]()
							{
								GetSystemViewModel()->FocusTab(FNiagaraSystemToolkitModeBase::MessageLogTabID);
							}),
						EStackIssueFixStyle::Link));
				NewIssues.Add(CompileWarning);
			}
		}
	}
}

void GenerateDragDropData(
	UNiagaraNodeFunctionCall& SourceModule,
	UNiagaraNodeFunctionCall* TargetModule,
	UNiagaraNodeOutput* TargetOutputNode,
	EItemDropZone TargetZone,
	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup>& OutSourceStackGroups, int32& OutSourceGroupIndex,
	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup>& OutTargetStackGroups, int32& OutTargetGroupIndex)
{
	// Find the output node for the source
	UNiagaraNodeOutput* SourceOutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(SourceModule);

	// Collect the stack node groups for the source and target.
	FNiagaraStackGraphUtilities::GetStackNodeGroups(*SourceOutputNode, OutSourceStackGroups);
	if (SourceOutputNode == TargetOutputNode)
	{
		OutTargetStackGroups.Append(OutSourceStackGroups);
	}
	else
	{
		FNiagaraStackGraphUtilities::GetStackNodeGroups(*TargetOutputNode, OutTargetStackGroups);
	}

	// Calculate the source and target groups indexes for the drag/drop
	OutSourceGroupIndex = INDEX_NONE;
	for (int32 GroupIndex = 0; GroupIndex < OutSourceStackGroups.Num(); GroupIndex++)
	{
		if (OutSourceStackGroups[GroupIndex].EndNode == &SourceModule)
		{
			OutSourceGroupIndex = GroupIndex;
			break;
		}
	}

	if (TargetModule == nullptr)
	{
		// If no target module was supplied then the module should be inserted at the beginning.
		OutTargetGroupIndex = 1;
	}
	else
	{
		if (TargetModule == &SourceModule)
		{
			OutTargetGroupIndex = OutSourceGroupIndex;
		}
		else
		{
			for (int32 GroupIndex = 0; GroupIndex < OutTargetStackGroups.Num(); GroupIndex++)
			{
				if (OutTargetStackGroups[GroupIndex].EndNode == TargetModule)
				{
					OutTargetGroupIndex = GroupIndex;
					break;
				}
			}
		}

		if (TargetZone == EItemDropZone::BelowItem)
		{
			OutTargetGroupIndex++;
		}
	}
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::CanDropInternal(const FDropRequest& DropRequest)
{
	return CanDropOnTarget(*this, DropRequest);
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::DropInternal(const FDropRequest& DropRequest)
{
	return DropOnTarget(*this, DropRequest);
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::ChildRequestCanDropInternal(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest)
{
	return CanDropOnTarget(TargetChild, DropRequest);
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::ChildRequestDropInternal(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest)
{
	return DropOnTarget(TargetChild, DropRequest);
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::CanDropOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	if (DropRequest.DragDropOperation->IsOfType<FNiagaraStackEntryDragDropOp>())
	{
		return CanDropEntriesOnTarget(TargetEntry, DropRequest);
	}
	else if (DropRequest.DragDropOperation->IsOfType<FAssetDragDropOp>())
	{
		return CanDropAssetsOnTarget(TargetEntry, DropRequest);
	}
	else if (DropRequest.DragDropOperation->IsOfType<FNiagaraParameterDragOperation>())
	{
		return CanDropParameterOnTarget(TargetEntry, DropRequest);
	}
	else if (DropRequest.DragDropOperation->IsOfType<FNiagaraScriptDragOperation>())
	{
		return CanDropScriptsOnTarget(TargetEntry, DropRequest);
	}
	return TOptional<FDropRequestResponse>();
}

TOptional<EItemDropZone> GetTargetDropZoneForTargetEntry(UNiagaraStackScriptItemGroup* ThisEntry, const UNiagaraStackEntry& TargetEntry, EItemDropZone RequestedDropZone)
{
	TOptional<EItemDropZone> TargetDropZone;
	if (&TargetEntry == ThisEntry)
	{
		// Items dragged onto the group entry are always inserted below it.
		TargetDropZone = EItemDropZone::BelowItem;
	}
	else
	{
		// Otherwise only allow drops above or below items.
		if (RequestedDropZone == EItemDropZone::AboveItem || RequestedDropZone == EItemDropZone::BelowItem)
		{
			TargetDropZone = RequestedDropZone;
		}
	}
	return TargetDropZone;
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::CanDropEntriesOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	TSharedRef<const FNiagaraStackEntryDragDropOp> StackEntryDragDropOp = StaticCastSharedRef<const FNiagaraStackEntryDragDropOp>(DropRequest.DragDropOperation);
	bool ModuleEntriesDragged = false;
	for (UNiagaraStackEntry* DraggedEntry : StackEntryDragDropOp->GetDraggedEntries())
	{
		if (DraggedEntry->IsA<UNiagaraStackModuleItem>())
		{
			ModuleEntriesDragged = true;
			break;
		}
	}

	if (ModuleEntriesDragged == false)
	{
		// Only handle dragged module items.
		return TOptional<FDropRequestResponse>();
	}
	if (&TargetEntry != this && TargetEntry.IsA<UNiagaraStackModuleItem>() == false)
	{
		// Only handle drops onto this script group, or child drop requests from module items.
		return TOptional<FDropRequestResponse>();
	}
	if (DropRequest.DropOptions != UNiagaraStackEntry::EDropOptions::Overview)
	{
		// Only allow dropping in the overview stacks.
		return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantDropModuleOnStack", "Modules can only be dropped into the overview."));
	}
	if (StackEntryDragDropOp->GetDraggedEntries().Num() != 1)
	{
		// Only handle a single module.
		return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantDropMultipleModules", "Only single modules can be dragged and dropped."));
	}

	UNiagaraStackModuleItem* SourceModuleItem = CastChecked<UNiagaraStackModuleItem>(StackEntryDragDropOp->GetDraggedEntries()[0]);
	if (DropRequest.DragOptions != EDragOptions::Copy && SourceModuleItem->CanMoveAndDelete() == false)
	{
		return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantMoveModuleError", "This inherited module can't be moved."));
	}

	if ( FVersionedNiagaraScriptData* ScriptData = SourceModuleItem->GetModuleNode().GetScriptData() )
	{
		TArray<ENiagaraScriptUsage> SourceUsages = ScriptData->GetSupportedUsageContexts();
		if (SourceUsages.ContainsByPredicate([this](ENiagaraScriptUsage SourceUsage) { return UNiagaraScript::IsEquivalentUsage(ScriptUsage, SourceUsage); }) == false)
		{
			return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantMoveModuleByUsage", "This module can't be moved to this section of the\nstack because it's not valid for this usage context."));
		}
	}

	TOptional<EItemDropZone> TargetDropZone = GetTargetDropZoneForTargetEntry(this, TargetEntry, DropRequest.DropZone);
	if(TargetDropZone.IsSet() == false)
	{
		return TOptional<FDropRequestResponse>();
	}

	const UNiagaraStackModuleItem* TargetModuleItem = Cast<UNiagaraStackModuleItem>(&TargetEntry);

	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> SourceStackGroups;
	TArray<FNiagaraStackGraphUtilities::FStackNodeGroup> TargetStackGroups;
	int32 SourceGroupIndex;
	int32 TargetGroupIndex;
	UNiagaraNodeFunctionCall* TargetModuleNode = TargetModuleItem != nullptr ? &TargetModuleItem->GetModuleNode() : nullptr;
	UNiagaraNodeOutput* TargetOutputNode = GetScriptOutputNode();
	GenerateDragDropData(
		SourceModuleItem->GetModuleNode(), TargetModuleNode, 
		TargetOutputNode, DropRequest.DropZone,
		SourceStackGroups, SourceGroupIndex,
		TargetStackGroups, TargetGroupIndex);

	// Make sure the source and target indices are within safe ranges, and make sure that the insert target isn't the source target or the spot directly
	// after the source target since that won't actually move the module.
	bool bSourceWithinRange = SourceGroupIndex > 0 && SourceGroupIndex < SourceStackGroups.Num() - 1;
	bool bTargetWithinRange = TargetGroupIndex > 0 && TargetGroupIndex < TargetStackGroups.Num();
	bool bWillMove = DropRequest.DragOptions == EDragOptions::Copy ||
		(SourceStackGroups[SourceGroupIndex].EndNode != TargetStackGroups[TargetGroupIndex].EndNode &&
			SourceStackGroups[SourceGroupIndex].EndNode != TargetStackGroups[TargetGroupIndex - 1].EndNode);
	if (bSourceWithinRange && bTargetWithinRange && bWillMove)
	{
		FText DropMessage;
		if (DropRequest.DragOptions == UNiagaraStackEntry::EDragOptions::Copy)
		{
			DropMessage = LOCTEXT("CopyModuleResult", "Copy this module here.");
		}
		else
		{
			DropMessage = LOCTEXT("MoveModuleResult", "Move this module here.");
		}
		return FDropRequestResponse(TargetDropZone, DropMessage);
	}

	return TOptional<FDropRequestResponse>();
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::CanDropAssetsOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	TSharedRef<const FAssetDragDropOp> AssetDragDropOp = StaticCastSharedRef<const FAssetDragDropOp>(DropRequest.DragDropOperation);
	const UEnum* NiagaraScriptUsageEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Niagara.ENiagaraScriptUsage"), EFindObjectFlags::ExactClass);

	if (&TargetEntry != this && TargetEntry.IsA<UNiagaraStackModuleItem>() == false)
	{
		// Only handle drops onto this script group, or child drop requests from module items.
		return TOptional<FDropRequestResponse>();
	}
	if (DropRequest.DropOptions != UNiagaraStackEntry::EDropOptions::Overview)
	{
		// Only allow dropping in the overview stacks.
		return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("AssetCantDropOnStack", "Assets can only be dropped into the overview."));
	}

	for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
	{
		if (AssetData.GetClass() != UNiagaraScript::StaticClass())
		{
			FString AssetName;
			AssetData.GetFullName(AssetName);
			return FDropRequestResponse(TOptional<EItemDropZone>(), FText::Format(LOCTEXT("CantDropNonScriptAssetFormat", "Can not drop asset {0} here because it is not a niagara script."), FText::FromString(AssetName)));
		}

		FString AssetScriptUsageString = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UNiagaraScript, Usage));
		int64 AssetScriptUsageValue = NiagaraScriptUsageEnum->GetValueByNameString(AssetScriptUsageString);
		if (AssetScriptUsageValue == INDEX_NONE)
		{
			FString AssetName;
			AssetData.GetFullName(AssetName);
			return FDropRequestResponse(TOptional<EItemDropZone>(), FText::Format(LOCTEXT("CantDropAssetWithInvalidUsageFormat", "Can not drop asset {0} here because it doesn't have a valid usage."), FText::FromString(AssetName)));
		}

		ENiagaraScriptUsage AssetScriptUsage = static_cast<ENiagaraScriptUsage>(AssetScriptUsageValue);
		if (AssetScriptUsage != ENiagaraScriptUsage::Module)
		{
			FString AssetName;
			AssetData.GetFullName(AssetName);
			return FDropRequestResponse(TOptional<EItemDropZone>(), FText::Format(LOCTEXT("CantDropNonModuleAssetFormat", "Can not drop asset {0} here because it is not a module script."), FText::FromString(AssetName)));
		}

		FString BitfieldTagValue = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(FVersionedNiagaraScriptData, ModuleUsageBitmask));
		int32 BitfieldValue = FCString::Atoi(*BitfieldTagValue);
		TArray<ENiagaraScriptUsage> SupportedUsages = UNiagaraScript::GetSupportedUsageContextsForBitmask(BitfieldValue);
		if (SupportedUsages.Contains(GetScriptUsage()) == false)
		{
			FString AssetName;
			AssetData.GetFullName(AssetName);
			return FDropRequestResponse(TOptional<EItemDropZone>(), FText::Format(LOCTEXT("CantDropAssetByUsageFormat", "Can not drop asset {0} in this part of the stack\nbecause it's not valid for this usage context."), FText::FromString(AssetName)));
		}
	}

	TOptional<EItemDropZone> TargetDropZone = GetTargetDropZoneForTargetEntry(this, TargetEntry, DropRequest.DropZone);
	if (TargetDropZone.IsSet() == false)
	{
		return TOptional<FDropRequestResponse>();
	}

	FText DropMessage;
	if (AssetDragDropOp->GetAssets().Num() > 1)
	{
		DropMessage = LOCTEXT("DropAssets", "Insert modules for these assets here.");
	}
	else
	{
		DropMessage = LOCTEXT("DropAsset", "Insert a module for this asset here.");
	}
	return FDropRequestResponse(TargetDropZone, DropMessage);
}


TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::CanDropScriptsOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	TSharedRef<const FNiagaraScriptDragOperation> DragDropOp = StaticCastSharedRef<const FNiagaraScriptDragOperation>(DropRequest.DragDropOperation);
	if (&TargetEntry != this && TargetEntry.IsA<UNiagaraStackModuleItem>() == false)
	{
		// Only handle drops onto this script group, or child drop requests from module items.
		return TOptional<FDropRequestResponse>();
	}
	if (DropRequest.DropOptions != UNiagaraStackEntry::EDropOptions::Overview)
	{
		// Only allow dropping in the overview stacks.
		return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("ScriptCantDropOnStack", "Scripts can only be dropped into the overview."));
	}
	UNiagaraScript* Script = DragDropOp->Script.Get();
	if (Script)
	{
		ENiagaraScriptUsage ScriptscriptUsage = Script->Usage;
		if (ScriptscriptUsage != ENiagaraScriptUsage::Module)
		{
			return FDropRequestResponse(TOptional<EItemDropZone>(), FText::Format(LOCTEXT("CantDropNonModuleAssetFormat", "Can not drop asset {0} here because it is not a module script."), DragDropOp->FriendlyName));
		}

		int32 BitfieldValue = Script->GetScriptData(DragDropOp->Version)->ModuleUsageBitmask;;
		TArray<ENiagaraScriptUsage> SupportedUsages = UNiagaraScript::GetSupportedUsageContextsForBitmask(BitfieldValue);
		if (SupportedUsages.Contains(GetScriptUsage()) == false)
		{
			return FDropRequestResponse(TOptional<EItemDropZone>(), FText::Format(LOCTEXT("CantDropAssetByUsageFormat", "Can not drop asset {0} in this part of the stack\nbecause it's not valid for this usage context."), DragDropOp->FriendlyName));
		}
	}

	TOptional<EItemDropZone> TargetDropZone = GetTargetDropZoneForTargetEntry(this, TargetEntry, DropRequest.DropZone);
	if (TargetDropZone.IsSet() == false)
	{
		return TOptional<FDropRequestResponse>();
	}

	FText DropMessage;
	if (Script)
	{
		DropMessage = LOCTEXT("DropAsset", "Insert a module for this asset here.");
	}
	return FDropRequestResponse(TargetDropZone, DropMessage);
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::CanDropParameterOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	TSharedRef<const FNiagaraParameterDragOperation> ParameterDragDropOp = StaticCastSharedRef<const FNiagaraParameterDragOperation>(DropRequest.DragDropOperation);
	TSharedPtr<const FNiagaraParameterAction> ParameterAction = StaticCastSharedPtr<const FNiagaraParameterAction>(ParameterDragDropOp->GetSourceAction());

	if (&TargetEntry != this && TargetEntry.IsA<UNiagaraStackModuleItem>() == false)
	{
		// Only handle drops onto this script group, or child drop requests from module items.
		return TOptional<FDropRequestResponse>();
	}
	if (DropRequest.DropOptions != UNiagaraStackEntry::EDropOptions::Overview)
	{
		// Only allow dropping in the overview stacks.
		return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantDropParameterOnStack", "Parameters can only be dropped onto 'Set Parameters' modules, or correctly\ntyped inputs in the selection view.  If you want to add a new 'Set Parameters' module for\n this parameter, you can drop it into one of the nodes in the System Overview graph."));
	}
	if (ParameterAction.IsValid())
	{
		if (FNiagaraStackGraphUtilities::CanWriteParameterFromUsageViaOutput(ParameterAction->GetParameter(), GetScriptOutputNode()) == false)
		{
			return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantDropParameterByUsage", "Can not drop this parameter here because\nit can't be written in this usage context."));
		}

		{
			TOptional<EItemDropZone> TargetDropZone = GetTargetDropZoneForTargetEntry(this, TargetEntry, DropRequest.DropZone);
			if (TargetDropZone.IsSet() == false)
			{
				return TOptional<FDropRequestResponse>();
			}
			else
			{
				return FDropRequestResponse(TargetDropZone, LOCTEXT("DropParameterFormat", "Insert a module for this parameter here."));
			}
		}

	}
	return TOptional<FDropRequestResponse>();
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::DropOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	if (DropRequest.DragDropOperation->IsOfType<FNiagaraStackEntryDragDropOp>())
	{
		return DropEntriesOnTarget(TargetEntry, DropRequest);
	}
	else if (DropRequest.DragDropOperation->IsOfType<FAssetDragDropOp>())
	{
		return DropAssetsOnTarget(TargetEntry, DropRequest);
	}
	else if (DropRequest.DragDropOperation->IsOfType<FNiagaraParameterDragOperation>())
	{
		return DropParameterOnTarget(TargetEntry, DropRequest);
	}
	else if (DropRequest.DragDropOperation->IsOfType<FNiagaraScriptDragOperation>())
	{
		return DropScriptsOnTarget(TargetEntry, DropRequest);
	}
	return TOptional<FDropRequestResponse>();
}

int32 GetTargetIndexForTargetEntry(UNiagaraStackScriptItemGroup* ThisEntry, const UNiagaraStackEntry& TargetEntry, EItemDropZone DropZone)
{
	int32 TargetIndex;
	if (&TargetEntry == ThisEntry)
	{
		// Items dragged onto this group entry are always inserted at index 0.
		TargetIndex = 0;
	}
	else
	{
		// Otherwise get the index from the module and the drop zone.
		const UNiagaraStackModuleItem* TargetModuleItem = CastChecked<UNiagaraStackModuleItem>(&TargetEntry);
		TargetIndex = TargetModuleItem->GetModuleIndex();
		if (DropZone == EItemDropZone::BelowItem)
		{
			TargetIndex++;
		}
	}
	return TargetIndex;
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::DropEntriesOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	TSharedRef<const FNiagaraStackEntryDragDropOp> StackEntryDragDropOp = StaticCastSharedRef<const FNiagaraStackEntryDragDropOp>(DropRequest.DragDropOperation);

	UNiagaraStackModuleItem* SourceModuleItem = CastChecked<UNiagaraStackModuleItem>(StackEntryDragDropOp->GetDraggedEntries()[0]);
	const FNiagaraEmitterHandle* SourceEmitterHandle = SourceModuleItem->GetEmitterViewModel().IsValid()
		? FNiagaraEditorUtilities::GetEmitterHandleForEmitter(SourceModuleItem->GetSystemViewModel()->GetSystem(), SourceModuleItem->GetEmitterViewModel()->GetEmitter())
		: nullptr;
	FGuid SourceEmitterHandleId = SourceEmitterHandle != nullptr
		? SourceEmitterHandle->GetId()
		: FGuid();
	UNiagaraNodeOutput* SourceModuleOutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(SourceModuleItem->GetModuleNode());
	UNiagaraScript* SourceModuleScript = FNiagaraEditorUtilities::GetScriptFromSystem(SourceModuleItem->GetSystemViewModel()->GetSystem(), SourceEmitterHandleId,
		SourceModuleOutputNode->GetUsage(), SourceModuleOutputNode->GetUsageId());

	const FNiagaraEmitterHandle* TargetEmitterHandle = GetEmitterViewModel().IsValid()
		? FNiagaraEditorUtilities::GetEmitterHandleForEmitter(GetSystemViewModel()->GetSystem(), GetEmitterViewModel()->GetEmitter())
		: nullptr;
	FGuid TargetEmitterHandleId = TargetEmitterHandle != nullptr
		? TargetEmitterHandle->GetId()
		: FGuid();
	int32 TargetIndex = GetTargetIndexForTargetEntry(this, TargetEntry, DropRequest.DropZone);

	FScopedTransaction ScopedTransaction(LOCTEXT("DragAndDropModule", "Drag and drop module"));
	{
		GetSystemViewModel()->GetSelectionViewModel()->RemoveEntryFromSelectionByDisplayedObject(SourceModuleItem->GetDisplayedObject());
		UNiagaraNodeFunctionCall* MovedModule = nullptr;
		FNiagaraStackGraphUtilities::MoveModule(*SourceModuleScript, SourceModuleItem->GetModuleNode(), GetSystemViewModel()->GetSystem(), TargetEmitterHandleId,
			ScriptUsage, ScriptUsageId, TargetIndex, DropRequest.DragOptions == EDragOptions::Copy, MovedModule);

		UNiagaraGraph* TargetGraph = ScriptViewModel.Pin()->GetGraphViewModel()->GetGraph();
		FNiagaraStackGraphUtilities::RelayoutGraph(*TargetGraph);
		TargetGraph->NotifyGraphNeedsRecompile();
		GetSystemViewModel()->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectDeferred(MovedModule);
	}

	SourceModuleItem->OnRequestFullRefreshDeferred().Broadcast();
	TargetEntry.OnRequestFullRefreshDeferred().Broadcast();
	return FDropRequestResponse(DropRequest.DropZone);
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::DropAssetsOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	TSharedRef<const FAssetDragDropOp> AssetDragDropOp = StaticCastSharedRef<const FAssetDragDropOp>(DropRequest.DragDropOperation);

	int32 TargetIndex = GetTargetIndexForTargetEntry(this, TargetEntry, DropRequest.DropZone);

	FScopedTransaction ScopedTransaction(LOCTEXT("DragAndDropAsset", "Insert modules for assets"));
	for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
	{
		TSharedRef<FScriptGroupAddAction> AddAction = FScriptGroupAddAction::CreateAssetModuleAction(AssetData);
		AddUtilities->ExecuteAddAction(AddAction, TargetIndex);
		TargetIndex++;
	}
	return FDropRequestResponse(DropRequest.DropZone);
}


TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::DropScriptsOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	TSharedRef<const FNiagaraScriptDragOperation> DragDropOp = StaticCastSharedRef<const FNiagaraScriptDragOperation>(DropRequest.DragDropOperation);

	int32 TargetIndex = GetTargetIndexForTargetEntry(this, TargetEntry, DropRequest.DropZone);

	FScopedTransaction ScopedTransaction(LOCTEXT("DragAndDropScript", "Insert modules for scripts"));
	if (UNiagaraScript* Script = DragDropOp->Script.Get())
	{
		TSharedRef<FScriptGroupAddAction> AddAction = FScriptGroupAddAction::CreateModuleActionFromScratchPadScript(Script);
		AddUtilities->ExecuteAddAction(AddAction, TargetIndex);
	}
	return FDropRequestResponse(DropRequest.DropZone);
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackScriptItemGroup::DropParameterOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest)
{
	TSharedRef<const FNiagaraParameterDragOperation> ParameterDragDropOp = StaticCastSharedRef<const FNiagaraParameterDragOperation>(DropRequest.DragDropOperation);
	TSharedPtr<FNiagaraParameterAction> ParameterAction = StaticCastSharedPtr<FNiagaraParameterAction>(ParameterDragDropOp->GetSourceAction());

	int32 TargetIndex = GetTargetIndexForTargetEntry(this, TargetEntry, DropRequest.DropZone);
	TSharedRef<FScriptGroupAddAction> AddAction = FScriptGroupAddAction::CreateSetExistingParameterModuleAction(ParameterAction->GetParameter());
	AddUtilities->ExecuteAddAction(AddAction, TargetIndex);

	return FDropRequestResponse(DropRequest.DropZone);
}

void UNiagaraStackScriptItemGroup::ItemAdded(UNiagaraNodeFunctionCall* AddedModule)
{
	GetSystemViewModel()->GetSelectionViewModel()->EmptySelection();
	GetSystemViewModel()->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectDeferred(AddedModule);
	RefreshChildren();
}

void UNiagaraStackScriptItemGroup::ChildModifiedGroupItems()
{
	RefreshChildren();
}

bool UNiagaraStackScriptItemGroup::ChildRequestCanPaste(const UNiagaraClipboardContent* ClipboardContent, FText& OutCanPasteMessage)
{
	return TestCanPasteWithMessage(ClipboardContent, OutCanPasteMessage);
}

void UNiagaraStackScriptItemGroup::ChildRequestPaste(const UNiagaraClipboardContent* ClipboardContent, int32 PasteIndex, FText& OutPasteWarning)
{
	PasteModules(ClipboardContent, PasteIndex, OutPasteWarning);
}

void UNiagaraStackScriptItemGroup::ChildRequestDeprecatedRecommendation(UNiagaraStackModuleItem* TargetChild)
{

	FScopedTransaction ScopedTransaction(LOCTEXT("UpdateDeprecatedScript", "Update module script to new version"));

	UNiagaraClipboardContent* ClipboardContent = UNiagaraClipboardContent::Create();
	TargetChild->GetModuleNode().GetNiagaraGraph()->Modify();
	TargetChild->GetModuleNode().Modify();
	Modify();
	TargetChild->Modify();

	// Step 1: Make a copy of the existing node so that we can perform surgery on it to update.
	UNiagaraScript* TargetScript = TargetChild->GetModuleNode().GetScriptData()->DeprecationRecommendation;
	if (TargetScript->GetUsage() != ENiagaraScriptUsage::Module)
	{
		// Early out if the deprecation recommendation asset is not module script usage.
		FNiagaraEditorUtilities::WarnWithToastAndLog(LOCTEXT("DeprecationAssetReplacementFail","Failed to insert recommended deprecation asset as it is not a module script!"));
		return;
	}
	TargetChild->Copy(ClipboardContent);

	// Step 2: Disable the old owner so that end users can use it as reference and inheritance isn't wiped out.
	//TargetChild->Rename();
	//TargetChild->GetModuleNode().SuggestName(TEXT("Deprecated Original " + TargetChild->GetModuleNode().GetFunctionName()), true);
	TargetChild->OnRenamed(FText::Format(LOCTEXT("NewDeprecationName", "Deprecated Original {0}"), TargetChild->GetDisplayName()));
	TargetChild->SetEnabled(false);

	// Step 3: Paste in the copy
	FText PasteWarning;
	int32 NewChildIdx = TargetChild->GetModuleIndex() + 1;
	PasteModules(ClipboardContent, TargetChild->GetModuleIndex() + 1, PasteWarning);
	RefreshChildren();

	// Step 4: convert new copy in place to suggested script.
	TArray<UNiagaraStackModuleItem*> ModuleItems;
	GetUnfilteredChildrenOfType(ModuleItems);

	if (ModuleItems.Num() > NewChildIdx)
	{
		ModuleItems[NewChildIdx]->ReassignModuleScript(TargetScript);
	}
	//
}

void UNiagaraStackScriptItemGroup::OnScriptGraphChanged(const struct FEdGraphEditAction& InAction)
{
	if (InAction.Action == GRAPHACTION_RemoveNode)
	{
		OnRequestFullRefreshDeferred().Broadcast();
	}
}

void UNiagaraStackScriptItemGroup::OnSystemScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion)
{
	if (IsFinalized() == false)
	{
		RefreshChildren();
	}
}

void UNiagaraStackScriptItemGroup::OnParticleScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion)
{
	if (IsFinalized() == false)
	{
		RefreshChildren();
	}
}

void GatherRenamedModuleOutputs(
	UNiagaraStackScriptItemGroup* InOwnerEntry,
	TArray<TPair<const UNiagaraClipboardFunction*, UNiagaraNodeFunctionCall*>> InClipboardFunctionAndNodeFunctionPairs,
	TMap<FName, FName>& OutOldModuleOutputNameToNewModuleOutputNameMap)
{
	FVersionedNiagaraEmitter Emitter = InOwnerEntry->GetEmitterViewModel().IsValid() ? InOwnerEntry->GetEmitterViewModel()->GetEmitter() : FVersionedNiagaraEmitter();

	for (TPair<const UNiagaraClipboardFunction*, UNiagaraNodeFunctionCall*>& ClipboardFunctionAndNodeFunctionPair : InClipboardFunctionAndNodeFunctionPairs)
	{
		const UNiagaraClipboardFunction* ClipboardFunction = ClipboardFunctionAndNodeFunctionPair.Key;
		UNiagaraNodeFunctionCall* NewFunctionCallNode = ClipboardFunctionAndNodeFunctionPair.Value;
		if (NewFunctionCallNode->GetFunctionName() != ClipboardFunction->FunctionName)
		{
			FNiagaraStackGraphUtilities::GatherRenamedStackFunctionOutputVariableNames(Emitter, *NewFunctionCallNode, ClipboardFunction->FunctionName, NewFunctionCallNode->GetFunctionName(), OutOldModuleOutputNameToNewModuleOutputNameMap);
		}
	}
}

void RenameInputsFromClipboard(TMap<FName, FName> OldModuleOutputNameToNewModuleOutputNameMap, UObject* InOuter, const TArray<const UNiagaraClipboardFunctionInput*>& InSourceInputs, TArray<const UNiagaraClipboardFunctionInput*>& OutRenamedInputs)
{
	for (const UNiagaraClipboardFunctionInput* SourceInput : InSourceInputs)
	{
		TOptional<bool> bEditConditionValue = SourceInput->bHasEditCondition 
			? TOptional<bool>(SourceInput->bEditConditionValue)
			: TOptional<bool>();
		if (SourceInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::Linked && OldModuleOutputNameToNewModuleOutputNameMap.Contains(SourceInput->Linked.GetName()))
		{
			FName RenamedLinkedValue = OldModuleOutputNameToNewModuleOutputNameMap[SourceInput->Linked.GetName()];
			FNiagaraVariableBase RenamedLinkedParameter = FNiagaraVariableBase(SourceInput->Linked.GetType(), RenamedLinkedValue);
			OutRenamedInputs.Add(UNiagaraClipboardFunctionInput::CreateLinkedValue(InOuter, SourceInput->InputName, SourceInput->InputType, bEditConditionValue, RenamedLinkedParameter));
		}
		else if (SourceInput->ValueMode == ENiagaraClipboardFunctionInputValueMode::Dynamic)
		{
			const UNiagaraClipboardFunctionInput* RenamedDynamicInput = UNiagaraClipboardFunctionInput::CreateDynamicValue(InOuter, SourceInput->InputName, SourceInput->InputType, bEditConditionValue, SourceInput->Dynamic->FunctionName, SourceInput->Dynamic->Script.Get(), SourceInput->Dynamic->ScriptVersion);
			RenameInputsFromClipboard(OldModuleOutputNameToNewModuleOutputNameMap, RenamedDynamicInput->Dynamic, SourceInput->Dynamic->Inputs, MutableView(RenamedDynamicInput->Dynamic->Inputs));
			OutRenamedInputs.Add(RenamedDynamicInput);
		}
		else
		{
			OutRenamedInputs.Add(CastChecked<UNiagaraClipboardFunctionInput>(StaticDuplicateObject(SourceInput, InOuter)));
		}
	}
}

void UNiagaraStackScriptItemGroup::PasteModules(const UNiagaraClipboardContent* ClipboardContent, int32 PasteIndex, FText& OutPasteWarning)
{
	UNiagaraNodeOutput* OutputNode = GetScriptOutputNode();
	
	// Add the new function call nodes from the clipboard functions and collected the pairs.
	TArray<TPair<const UNiagaraClipboardFunction*, UNiagaraNodeFunctionCall*>> ClipboardFunctionAndNodeFunctionPairs;
	int32 CurrentPasteIndex = PasteIndex;
	int32 ModulesSkippedForIncorrectUsage = 0;
	int32 AssignmentTargetsSkippedForIncorrectUsage = 0;
	for (const UNiagaraClipboardFunction* ClipboardFunction : ClipboardContent->Functions)
	{
		if (ClipboardFunction != nullptr)
		{
			UNiagaraNodeFunctionCall* NewFunctionCallNode = nullptr;
			switch (ClipboardFunction->ScriptMode)
			{
			case ENiagaraClipboardFunctionScriptMode::Assignment:
			{
				TArray<FNiagaraVariable> CompatibleAssignmentTargets;
				TArray<FString> CompatibleAssignmentDefaults;
				for (int32 AssignmentTargetIndex = 0; AssignmentTargetIndex < ClipboardFunction->AssignmentTargets.Num(); ++AssignmentTargetIndex)
				{
					if (FNiagaraStackGraphUtilities::CanWriteParameterFromUsage(ClipboardFunction->AssignmentTargets[AssignmentTargetIndex], OutputNode->GetUsage()))
					{
						CompatibleAssignmentTargets.Add(ClipboardFunction->AssignmentTargets[AssignmentTargetIndex]);
						CompatibleAssignmentDefaults.Add(ClipboardFunction->AssignmentDefaults[AssignmentTargetIndex]);
					}
					else
					{
						AssignmentTargetsSkippedForIncorrectUsage++;
					}
				}
				if (CompatibleAssignmentTargets.Num() > 0)
				{
					NewFunctionCallNode = FNiagaraStackGraphUtilities::AddParameterModuleToStack(CompatibleAssignmentTargets, *OutputNode, CurrentPasteIndex, CompatibleAssignmentDefaults);
					NewFunctionCallNode->SuggestName(ClipboardFunction->FunctionName);
				}
				break;
			}
			case ENiagaraClipboardFunctionScriptMode::ScriptAsset:
			{
				UNiagaraScript* ClipboardFunctionScript = ClipboardFunction->Script.LoadSynchronous();
				if (ClipboardFunctionScript != nullptr && ClipboardFunctionScript->GetLatestScriptData()->GetSupportedUsageContexts().Contains(OutputNode->GetUsage()))
				{
					UNiagaraScript* NewFunctionScript;
					if (ClipboardFunctionScript->IsAsset() ||
						GetSystemViewModel()->GetScriptScratchPadViewModel()->GetViewModelForScript(ClipboardFunctionScript).IsValid())
					{
						// If the clipboard script is an asset, or it's in the scratch pad of the current asset, it can be used directly.
						NewFunctionScript = ClipboardFunctionScript;
					}
					else
					{
						// Otherwise it's a scratch pad script from another asset so we need to add a duplicate scratch pad script to this asset.
						NewFunctionScript = GetSystemViewModel()->GetScriptScratchPadViewModel()->CreateNewScriptAsDuplicate(ClipboardFunctionScript)->GetOriginalScript();
					}

					FNiagaraStackGraphUtilities::FAddScriptModuleToStackArgs AddScriptModuleToStackArgs(NewFunctionScript, *OutputNode);
					AddScriptModuleToStackArgs.TargetIndex = CurrentPasteIndex;
					AddScriptModuleToStackArgs.bFixupTargetIndex = ClipboardContent->bFixupPasteIndexForScriptDependenciesInStack;
					AddScriptModuleToStackArgs.SuggestedName = ClipboardFunction->FunctionName;
					AddScriptModuleToStackArgs.VersionGuid = ClipboardFunction->ScriptVersion;
					NewFunctionCallNode = FNiagaraStackGraphUtilities::AddScriptModuleToStack(AddScriptModuleToStackArgs);
				}
				else
				{
					ModulesSkippedForIncorrectUsage++;
				}
				break;
			}
			default:
				checkf(false, TEXT("Unsupported clipboard function mode."));
			}

			if (NewFunctionCallNode != nullptr)
			{
				if(ClipboardFunction->StackNoteData.IsValid())
				{
					FString NewModuleStackEditorDataKey = FNiagaraStackGraphUtilities::StackKeys::GenerateStackModuleEditorDataKey(*NewFunctionCallNode);
					GetStackEditorData().AddOrReplaceStackNote(NewModuleStackEditorDataKey, ClipboardFunction->StackNoteData);	
				}
				
				ClipboardFunction->OnPastedFunctionCallNodeDelegate.ExecuteIfBound(NewFunctionCallNode);
				ClipboardFunctionAndNodeFunctionPairs.Add(TPair<const UNiagaraClipboardFunction*, UNiagaraNodeFunctionCall*>(ClipboardFunction, NewFunctionCallNode));
				CurrentPasteIndex++;
			}
		}
	}
	RefreshChildren();

	// Check to see if any of the functions were renamed, and if they were, check if they had aliased outputs that need to be fixed up.
	TMap<FName, FName> OldModuleOutputNameToNewModuleOutputNameMap;
	GatherRenamedModuleOutputs(this, ClipboardFunctionAndNodeFunctionPairs, OldModuleOutputNameToNewModuleOutputNameMap);

	// Apply the function inputs now that we have the map of old module output names to new module output names.
	for(TPair<const UNiagaraClipboardFunction*, UNiagaraNodeFunctionCall*>& ClipboardFunctionAndNodeFunctionPair : ClipboardFunctionAndNodeFunctionPairs)
	{
		const UNiagaraClipboardFunction* ClipboardFunction = ClipboardFunctionAndNodeFunctionPair.Key;
		UNiagaraNodeFunctionCall* NewFunctionCallNode = ClipboardFunctionAndNodeFunctionPair.Value;

		TArray<UNiagaraStackModuleItem*> ModuleItems;
		GetUnfilteredChildrenOfType(ModuleItems);

		UNiagaraStackModuleItem** NewModuleItemPtr = ModuleItems.FindByPredicate([NewFunctionCallNode](UNiagaraStackModuleItem* ModuleItem) { return &ModuleItem->GetModuleNode() == NewFunctionCallNode; });
		if (NewModuleItemPtr != nullptr)
		{
			TArray<const UNiagaraClipboardFunctionInput*> FunctionInputs;
			if (OldModuleOutputNameToNewModuleOutputNameMap.Num() > 0)
			{
				RenameInputsFromClipboard(OldModuleOutputNameToNewModuleOutputNameMap, GetTransientPackage(), ClipboardFunction->Inputs, FunctionInputs);
			}
			else
			{
				FunctionInputs = ClipboardFunction->Inputs;
			}
			(*NewModuleItemPtr)->SetInputValuesFromClipboardFunctionInputs(FunctionInputs);

			if (!ClipboardFunction->DisplayName.IsEmptyOrWhitespace())
			{
				(*NewModuleItemPtr)->OnRenamed(ClipboardFunction->DisplayName);
			}
		}
	}

	if (ModulesSkippedForIncorrectUsage > 0 || AssignmentTargetsSkippedForIncorrectUsage > 0)
	{
		OutPasteWarning = FText::Format(LOCTEXT("NonPastedModulesWarningFormat", "{0} modules not pasted due to incorrect usage.\n{1} inputs not pasted in Set Variables nodes due to incorrect usage."),
			FText::AsNumber(ModulesSkippedForIncorrectUsage), FText::AsNumber(AssignmentTargetsSkippedForIncorrectUsage));
	}
}

#undef LOCTEXT_NAMESPACE
