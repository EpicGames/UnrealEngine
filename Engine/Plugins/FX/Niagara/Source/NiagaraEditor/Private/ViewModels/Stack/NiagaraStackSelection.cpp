// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackSelection.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "NiagaraSystemEditorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackSelection)

void UNiagaraStackSelection::Initialize(FRequiredEntryData InRequiredEntryData)
{
	Super::Initialize(InRequiredEntryData, TEXT("Selection"));
}

bool UNiagaraStackSelection::GetCanExpand() const
{
	return false;
}

bool UNiagaraStackSelection::GetShouldShowInStack() const
{
	return false;
}

void UNiagaraStackSelection::SetSelectedEntries(const TArray<UNiagaraStackEntry*>& InSelectedEntries)
{
	SelectedEntries.Empty();
	for (UNiagaraStackEntry* SelectedEntry : InSelectedEntries)
	{
		SelectedEntries.Add(SelectedEntry);
	}
	RefreshChildren();
}

void UNiagaraStackSelection::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	for (TWeakObjectPtr<UNiagaraStackEntry> SelectedEntry : SelectedEntries)
	{
		if (SelectedEntry.IsValid() && SelectedEntry->IsFinalized() == false)
		{
			SelectedEntry->SetIsExpanded(true);
			NewChildren.Add(SelectedEntry.Get());
		}
	}
}
