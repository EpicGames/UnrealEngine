// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackSimulationStageGroup.h"
#include "NiagaraEmitter.h"
#include "NiagaraSimulationStageBase.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeFunctionCall.h"
#include "EdGraphSchema_Niagara.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptMergeManager.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackEmitterPropertiesGroup.h"

#include "Internationalization/Internationalization.h"
#include "ScopedTransaction.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackSimulationStageGroup)

#define LOCTEXT_NAMESPACE "UNiagaraStackSimulationStageGroup"

void UNiagaraStackSimulationStagePropertiesItem::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraSimulationStageBase* InSimulationStage)
{
	checkf(SimulationStage.IsValid() == false, TEXT("Can not initialize more than once."));
	SimulationStage = InSimulationStage;
	FString SimulationStageStackEditorDataKey = FString::Printf(TEXT("SimulationStage-%s-Properties"), *SimulationStage->Script->GetUsageId().ToString(EGuidFormats::DigitsWithHyphens));
	Super::Initialize(InRequiredEntryData, SimulationStageStackEditorDataKey);

	SimulationStage->OnChanged().AddUObject(this, &UNiagaraStackSimulationStagePropertiesItem::SimulationStagePropertiesChanged);
}

void UNiagaraStackSimulationStagePropertiesItem::FinalizeInternal()
{
	if (SimulationStage.IsValid())
	{
		SimulationStage->OnChanged().RemoveAll(this);
	}
	Super::FinalizeInternal();
}

FText UNiagaraStackSimulationStagePropertiesItem::GetDisplayName() const
{
	return FText::Format(LOCTEXT("SimulationStagePropertiesDisplayNameFormat", "{0} Settings"), SimulationStage->GetClass()->GetDisplayNameText());
}

bool UNiagaraStackSimulationStagePropertiesItem::TestCanResetToBaseWithMessage(FText& OutCanResetToBaseMessage) const
{
	if (bCanResetToBaseCache.IsSet() == false)
	{
		if (HasBaseSimulationStage())
		{
			FVersionedNiagaraEmitter VersionedEmitter = GetEmitterViewModel()->GetEmitter();
			FVersionedNiagaraEmitter BaseEmitter = VersionedEmitter.GetEmitterData()->GetParent();
			if (BaseEmitter.Emitter != nullptr && VersionedEmitter.Emitter != BaseEmitter.Emitter)
			{
				TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
				bCanResetToBaseCache = MergeManager->IsSimulationStagePropertySetDifferentFromBase(VersionedEmitter, BaseEmitter, SimulationStage->Script->GetUsageId());
			}
			else
			{
				bCanResetToBaseCache = false;
			}
		}
		else
		{
			bCanResetToBaseCache = false;
		}
	}
	if (bCanResetToBaseCache.GetValue())
	{
		OutCanResetToBaseMessage = LOCTEXT("CanResetToBase", "Reset this simulation stage to the one defined in the parent emitter.");
		return true;
	}
	else
	{
		OutCanResetToBaseMessage = LOCTEXT("CanNotResetToBase", "No parent to reset to, or not different from parent.");
		return false;
	}
}

void UNiagaraStackSimulationStagePropertiesItem::ResetToBase()
{
	FText Unused;
	if (TestCanResetToBaseWithMessage(Unused))
	{
		FVersionedNiagaraEmitter VersionedNiagaraEmitter = GetEmitterViewModel()->GetEmitter();
		FVersionedNiagaraEmitter BaseEmitter = VersionedNiagaraEmitter.GetEmitterData()->GetParent();
		TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
		MergeManager->ResetSimulationStagePropertySetToBase(VersionedNiagaraEmitter, BaseEmitter, SimulationStage->Script->GetUsageId());
		RefreshChildren();
	}
}

void UNiagaraStackSimulationStagePropertiesItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (SimulationStageObject == nullptr)
	{
		SimulationStageObject = NewObject<UNiagaraStackObject>(this);
		bool bIsTopLevelObject = true;
		bool bHideTopLevelCategories = false;
		SimulationStageObject->Initialize(CreateDefaultChildRequiredData(), SimulationStage.Get(), bIsTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey());
		SimulationStageObject->SetObjectGuid(SimulationStage->GetMergeId());
	}

	if ( SimulationStage.IsValid() )
	{
		FVersionedNiagaraEmitterData* EmitterData = GetEmitterViewModel()->GetEmitter().GetEmitterData();
		if (EmitterData && (EmitterData->SimTarget != ENiagaraSimTarget::GPUComputeSim) && SimulationStage->bEnabled )
		{
			TArray<FStackIssueFix> IssueFixes;
			IssueFixes.Emplace(
				LOCTEXT("DisableSimulationStageFix", "Disable Simulation Stage"),
				FStackIssueFixDelegate::CreateUObject(this, &UNiagaraStackSimulationStagePropertiesItem::SetSimulationStageEnabled, false)
			);
			IssueFixes.Emplace(
				LOCTEXT("SetGpuSimulationFix", "Set GPU simulation"),
				FStackIssueFixDelegate::CreateLambda(
					[WeakEmitter=GetEmitterViewModel()->GetEmitter().ToWeakPtr()]()
					{
						const FVersionedNiagaraEmitter VersionedEmitter = WeakEmitter.ResolveWeakPtr();
						if (VersionedEmitter.Emitter && VersionedEmitter.GetEmitterData() )
						{
							FScopedTransaction Transaction(LOCTEXT("SetGpuSimulation", "Set Gpu Simulation"));
							VersionedEmitter.Emitter->Modify();
							VersionedEmitter.GetEmitterData()->SimTarget = ENiagaraSimTarget::GPUComputeSim;

							FProperty* SimTargetProperty = FindFProperty<FProperty>(FVersionedNiagaraEmitterData::StaticStruct(), GET_MEMBER_NAME_CHECKED(FVersionedNiagaraEmitterData, SimTarget));
							FPropertyChangedEvent PropertyChangedEvent(SimTargetProperty);
							VersionedEmitter.Emitter->PostEditChangeVersionedProperty(PropertyChangedEvent, VersionedEmitter.Version);
						}
					}
				)
			);

			NewIssues.Emplace(
				EStackIssueSeverity::Error,
				LOCTEXT("SimulationStagesNotSupportedOnCPU", "Simulation stages are not supported on CPU"),
				LOCTEXT("SimulationStagesNotSupportedOnCPULong", "Simulations stages are currently not supported on CPU, please disable or remove."),
				GetStackEditorDataKey(),
				false,
				IssueFixes
			);
		}
	}

	NewChildren.Add(SimulationStageObject);

	bCanResetToBaseCache.Reset();
	bHasBaseSimulationStageCache.Reset();

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

FHierarchyElementIdentity UNiagaraStackSimulationStagePropertiesItem::DetermineSummaryIdentity() const
{
	return UNiagaraHierarchySimStageProperties::MakeIdentity(*GetSimulationStage().Get());
}

void UNiagaraStackSimulationStagePropertiesItem::SimulationStagePropertiesChanged()
{
	if (SimulationStage.IsValid() == false || GetEmitterViewModel().IsValid() == false)
	{
		return;
	}
	FVersionedNiagaraEmitter VersionedEmitter = GetEmitterViewModel()->GetEmitter();
	GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(VersionedEmitter).Get()->GetEmitterStackViewModel()->RequestValidationUpdate();

	bCanResetToBaseCache.Reset();
}

bool UNiagaraStackSimulationStagePropertiesItem::HasBaseSimulationStage() const
{
	if (GetSystemViewModel()->GetIsForDataProcessingOnly())
	{
		// If the model is just for data processing we don't want to go through the whole merge procedure and treat the stage entry as non-inherited.
		return false;
	}
	if (bHasBaseSimulationStageCache.IsSet() == false)
	{
		FVersionedNiagaraEmitter VersionedEmitter = GetEmitterViewModel()->GetEmitter();
		FVersionedNiagaraEmitter BaseEmitter = VersionedEmitter.GetEmitterData()->GetParent();
		if (BaseEmitter.Emitter != nullptr && VersionedEmitter.Emitter != BaseEmitter.Emitter)
		{
			TSharedRef<FNiagaraScriptMergeManager> MergeManager = FNiagaraScriptMergeManager::Get();
			bHasBaseSimulationStageCache = MergeManager->HasBaseSimulationStage(BaseEmitter, SimulationStage->Script->GetUsageId());
		}
		else
		{
			bHasBaseSimulationStageCache = false;
		}
	}
	return bHasBaseSimulationStageCache.GetValue();
}

void UNiagaraStackSimulationStagePropertiesItem::SetSimulationStageEnabled(bool bIsEnabled)
{
	if (UNiagaraSimulationStageBase* SimStage = SimulationStage.Get())
	{
		static FText TEXT_Enabled(LOCTEXT("Enabled", "Enabled"));
		static FText TEXT_Disabled(LOCTEXT("Disabled", "Disabled"));
		FScopedTransaction Transaction(FText::Format(LOCTEXT("SetSimulationStageEnable", "Set Simulation Stage {1} {0}"), bIsEnabled ? TEXT_Enabled : TEXT_Disabled, GetDisplayName()));
		SimStage->Modify();
		SimStage->SetEnabled(bIsEnabled);
	}
}

void UNiagaraStackSimulationStageGroup::Initialize(
	FRequiredEntryData InRequiredEntryData,
	TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
	UNiagaraSimulationStageBase* InSimulationStage)
{
	SimulationStage = InSimulationStage;
	SimulationStage->OnChanged().AddUObject(this, &UNiagaraStackSimulationStageGroup::SimulationStagePropertiesChanged);

	FText ToolTip = LOCTEXT("SimulationStageGroupTooltip", "Defines properties and script modules for a simulation stage.");
	FText DisplayName = FText::FromName(SimulationStage->SimulationStageName);
	Super::Initialize(InRequiredEntryData, DisplayName, ToolTip, InScriptViewModel, ENiagaraScriptUsage::ParticleSimulationStageScript, SimulationStage->Script->GetUsageId());
}

UNiagaraSimulationStageBase* UNiagaraStackSimulationStageGroup::GetSimulationStage() const
{
	return SimulationStage.Get();
}

bool UNiagaraStackSimulationStageGroup::GetIsEnabled() const
{
	bool bEnabled = true;
	if (UNiagaraSimulationStageBase* SimStage = SimulationStage.Get())
	{
		bEnabled &= SimStage->bEnabled;
	}
	bEnabled &= Super::GetIsEnabled();
	return bEnabled;
}

void UNiagaraStackSimulationStageGroup::SetIsEnabled(bool bEnabled)
{
	SimulationStageProperties->SetSimulationStageEnabled(bEnabled);
}

INiagaraStackItemGroupAddUtilities* UNiagaraStackSimulationStageGroup::GetEmitterStageAddUtilities() const
{
	if (EmitterStageAddUtilities.IsValid() == false)
	{
		EmitterStageAddUtilities = MakeShared<FNiagaraStackEmitterStageAddUtilities>(GetEmitterViewModel(), FNiagaraStackEmitterStageAddUtilities::FOnItemAdded::CreateUObject(this, &UNiagaraStackSimulationStageGroup::StageAdded), false, true);
	}
	return EmitterStageAddUtilities.Get();
}

int32 UNiagaraStackSimulationStageGroup::GetStageIndex() const
{
	UNiagaraSimulationStageBase* SimulationStagePinned = SimulationStage.Get();
	return SimulationStagePinned != nullptr 
		? GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetSimulationStages().IndexOfByKey(SimulationStagePinned)
		: INDEX_NONE;
}

void UNiagaraStackSimulationStageGroup::FinalizeInternal()
{
	if (SimulationStage.IsValid())
	{
		SimulationStage->OnChanged().RemoveAll(this);
	}
	Super::FinalizeInternal();
}

void UNiagaraStackSimulationStageGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	bHasBaseSimulationStageCache.Reset();

	SetDisplayName(FText::FromName(SimulationStage->SimulationStageName));

	if (SimulationStageProperties == nullptr)
	{
		SimulationStageProperties = NewObject<UNiagaraStackSimulationStagePropertiesItem>(this);
		SimulationStageProperties->Initialize(CreateDefaultChildRequiredData(), SimulationStage.Get());
	}
	NewChildren.Add(SimulationStageProperties);
	
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackSimulationStageGroup::CanDropInternal(const FDropRequest& DropRequest)
{
	if (DropRequest.DragDropOperation->IsOfType<FNiagaraStackEntryDragDropOp>() && 
		(DropRequest.DropZone == EItemDropZone::AboveItem || DropRequest.DropZone == EItemDropZone::BelowItem))
	{
		TSharedRef<const FNiagaraStackEntryDragDropOp> StackEntryDragDropOp = StaticCastSharedRef<const FNiagaraStackEntryDragDropOp>(DropRequest.DragDropOperation);
		bool SimulationStageEntriesDragged = false;
		for (UNiagaraStackEntry* DraggedEntry : StackEntryDragDropOp->GetDraggedEntries())
		{
			if (DraggedEntry->IsA<UNiagaraStackSimulationStageGroup>())
			{
				SimulationStageEntriesDragged = true;
				break;
			}
		}

		if (SimulationStageEntriesDragged == false)
		{
			// Only handle dragged SimulationStage items.
			return Super::CanDropInternal(DropRequest);
		}

		if (DropRequest.DropOptions != UNiagaraStackEntry::EDropOptions::Overview)
		{
			// Only allow dropping in the overview stacks.
			return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantDropSimulationStageOnStack", "Simulation stages can only be dropped into the overview."));
		}

		if (StackEntryDragDropOp->GetDraggedEntries().Num() != 1)
		{
			// Only handle a single items.
			return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantDropMultipleSimulationStages", "Only single simulation stages can be dragged and dropped."));
		}

		UNiagaraStackSimulationStageGroup* SourceSimulationStageGroup = CastChecked<UNiagaraStackSimulationStageGroup>(StackEntryDragDropOp->GetDraggedEntries()[0]);
		if (DropRequest.DragOptions != EDragOptions::Copy && SourceSimulationStageGroup->HasBaseSimulationStage())
		{
			return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantMoveSimulationStageError", "This inherited simulation stage can't be moved."));
		}

		if (SourceSimulationStageGroup == this)
		{
			// Don't allow dropping on yourself.
			return Super::CanDropInternal(DropRequest);
		}

		FVersionedNiagaraEmitterData* OwningEmitter = GetEmitterViewModel()->GetEmitter().GetEmitterData();
		int32 SourceIndex = OwningEmitter->GetSimulationStages().IndexOfByKey(SourceSimulationStageGroup->GetSimulationStage());
		if (SourceIndex == INDEX_NONE)
		{
			return FDropRequestResponse(TOptional<EItemDropZone>(), LOCTEXT("CantDropSimulationStageFromOtherEmitterError", "This simulation stage can't be moved here\nbecause it's owned by a different emitter."));
		}

		int32 TargetIndexOffset = DropRequest.DropZone == EItemDropZone::BelowItem ? 1 : 0;
		int32 TargetIndex = OwningEmitter->GetSimulationStages().IndexOfByKey(SimulationStage.Get()) + TargetIndexOffset;
		if (SourceIndex < TargetIndex)
		{
			// If the source index is less than the target index, the target index will decrease by 1 after the source is removed.
			TargetIndex--;
		}

		if (SourceIndex == TargetIndex)
		{
			// Only handle the drag if the item would actually move.
			return Super::CanDropInternal(DropRequest);
		}

		return FDropRequestResponse(DropRequest.DropZone, LOCTEXT("MoveSimulationStageDragMessage", "Move this simulation stage here."));
	}
	return Super::CanDropInternal(DropRequest);
}

TOptional<UNiagaraStackEntry::FDropRequestResponse> UNiagaraStackSimulationStageGroup::DropInternal(const FDropRequest& DropRequest)
{
	if (DropRequest.DragDropOperation->IsOfType<FNiagaraStackEntryDragDropOp>() &&
		(DropRequest.DropZone == EItemDropZone::AboveItem || DropRequest.DropZone == EItemDropZone::BelowItem))
	{
		TSharedRef<const FNiagaraStackEntryDragDropOp> StackEntryDragDropOp = StaticCastSharedRef<const FNiagaraStackEntryDragDropOp>(DropRequest.DragDropOperation);
		UNiagaraStackSimulationStageGroup* SourceSimulationStageGroup = Cast<UNiagaraStackSimulationStageGroup>(StackEntryDragDropOp->GetDraggedEntries()[0]);
		if (SourceSimulationStageGroup)
		{
			FVersionedNiagaraEmitter OwningEmitter = GetEmitterViewModel()->GetEmitter();
			FVersionedNiagaraEmitterData* EmitterData = OwningEmitter.GetEmitterData();
			int32 SourceIndex = EmitterData->GetSimulationStages().IndexOfByKey(SourceSimulationStageGroup->GetSimulationStage());
			if (SourceIndex != INDEX_NONE)
			{
				int32 TargetOffset = DropRequest.DropZone == EItemDropZone::BelowItem ? 1 : 0;
				int32 TargetIndex = EmitterData->GetSimulationStages().IndexOfByKey(SimulationStage.Get()) + TargetOffset;

				FScopedTransaction Transaction(FText::Format(LOCTEXT("MoveSimulationStage", "Move Shader Stage {0}"), GetDisplayName()));
				OwningEmitter.Emitter->MoveSimulationStageToIndex(SourceSimulationStageGroup->GetSimulationStage(), TargetIndex, OwningEmitter.Version);

				OnRequestFullRefreshDeferred().Broadcast();
				return FDropRequestResponse(DropRequest.DropZone, FText());
			}
		}
	}
	return Super::DropInternal(DropRequest);
}

bool UNiagaraStackSimulationStageGroup::TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const
{
	if (HasBaseSimulationStage())
	{
		OutCanDeleteMessage = LOCTEXT("CantDeleteInherited", "Can not delete this simulation stage because it's inherited.");
		return false;
	}
	else
	{
		OutCanDeleteMessage = LOCTEXT("CanDelete", "Delete this simulation stage.");
		return true;
	}
}

void UNiagaraStackSimulationStageGroup::Delete()
{
	TSharedPtr<FNiagaraScriptViewModel> ScriptViewModelPinned = ScriptViewModel.Pin();
	checkf(ScriptViewModelPinned.IsValid(), TEXT("Can not delete when the script view model has been deleted."));

	FVersionedNiagaraEmitter VersionedEmitter = GetEmitterViewModel()->GetEmitter();
	FVersionedNiagaraEmitterData* EmitterData = VersionedEmitter.GetEmitterData();
	UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(EmitterData->GraphSource);

	if (!Source || !Source->NodeGraph)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("DeleteSimulationStage", "Delete {0}"), GetDisplayName()));

	VersionedEmitter.Emitter->Modify();
	Source->NodeGraph->Modify();
	TArray<UNiagaraNode*> SimulationStageNodes;
	Source->NodeGraph->BuildTraversal(SimulationStageNodes, GetScriptUsage(), GetScriptUsageId());
	for (UNiagaraNode* Node : SimulationStageNodes)
	{
		Node->Modify();
	}
	
	// First, remove the simulation stage object.
	VersionedEmitter.Emitter->RemoveSimulationStage(SimulationStage.Get(), VersionedEmitter.Version);
	
	// Now remove all graph nodes associated with the simulation stage.
	for (UNiagaraNode* Node : SimulationStageNodes)
	{
		Node->DestroyNode();
	}

	// Set the emitter here to that the internal state of the view model is updated.
	// TODO: Move the logic for managing additional scripts into the emitter view model or script view model.
	ScriptViewModelPinned->SetScripts(VersionedEmitter);
	
	OnModifiedSimulationStagesDelegate.ExecuteIfBound();
}

bool UNiagaraStackSimulationStageGroup::GetIsInherited() const
{
	return HasBaseSimulationStage();
}

FText UNiagaraStackSimulationStageGroup::GetInheritanceMessage() const
{
	return LOCTEXT("SimulationStageGroupInheritanceMessage", "This simulation stage is inherited from a parent emitter.  Inherited\nsimulation stages can only be deleted while editing the parent emitter.");
}

FHierarchyElementIdentity UNiagaraStackSimulationStageGroup::DetermineSummaryIdentity() const
{
	FHierarchyElementIdentity Identity;
	Identity.Guids.Add(GetSimulationStage()->GetMergeId());
	return Identity;
}

void UNiagaraStackSimulationStageGroup::SimulationStagePropertiesChanged()
{
	SetDisplayName(FText::FromName(SimulationStage->SimulationStageName));
}

bool UNiagaraStackSimulationStageGroup::HasBaseSimulationStage() const
{
	if (bHasBaseSimulationStageCache.IsSet() == false)
	{
		// todo (me) emitter view model validity check should not be required but fixes a crash in UI when the view model is invalid for some reason
		if(!GetEmitterViewModel().IsValid())
		{
			bHasBaseSimulationStageCache = false;
		}
		else
		{
			FVersionedNiagaraEmitter BaseEmitter = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetParent();
			bHasBaseSimulationStageCache = BaseEmitter.Emitter != nullptr && FNiagaraScriptMergeManager::Get()->HasBaseSimulationStage(BaseEmitter, GetScriptUsageId());
		}
	}
	return bHasBaseSimulationStageCache.GetValue();
}

void UNiagaraStackSimulationStageGroup::StageAdded(FGuid AddedEventHandlerId, UNiagaraSimulationStageBase* AddedSimulationStage) const
{
	OnRequestFullRefreshDeferred().Broadcast();
}

void UNiagaraStackSimulationStageGroup::SetOnModifiedSimulationStages(FOnModifiedSimulationStages OnModifiedSimulationStages)
{
	OnModifiedSimulationStagesDelegate = OnModifiedSimulationStages;
}

#undef LOCTEXT_NAMESPACE

