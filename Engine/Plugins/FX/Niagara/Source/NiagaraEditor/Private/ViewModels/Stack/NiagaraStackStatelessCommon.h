// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"

namespace FNiagaraStackStatelessCommon
{
	bool ShouldHideTopLevelCategories(UStruct* Struct);
}
