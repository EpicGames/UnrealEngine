// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackStatelessEmitterGroup.h"

#include "Stateless/NiagaraStatelessEmitterDetails.h"
#include "Stateless/NiagaraEmitterStatePropertyCustomization.h"
#include "Stateless/NiagaraDistributionPropertyCustomization.h"
#include "Stateless/NiagaraDistributionIntPropertyCustomization.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackStatelessCommon.h"
#include "NiagaraEditorStyle.h"

#include "IDetailTreeNode.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackStatelessEmitterGroup)

#define LOCTEXT_NAMESPACE "NiagaraEmitterStatelessGroup"

namespace NiagaraStackStatelessEmitterGroupPrivate
{
	bool bShowRawObject = false;
	FAutoConsoleVariableRef CVarShowRawObject(
		TEXT("fx.NiagaraStateless.UI.ShowRawObject"),
		bShowRawObject,
		TEXT("When enabled we will show the raw object in the stateless emitter UI."),
		ECVF_Default
	);
}

void UNiagaraStackStatelessEmitterGroup::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter)
{
	Super::Initialize(
		InRequiredEntryData,
		LOCTEXT("EmitterStatelessGroupDisplayName", "Properties"), 
		LOCTEXT("EmitterStatelessGroupToolTip", "Properties for this lightweight emitter"),
		nullptr);
	StatelessEmitterWeak = InStatelessEmitter;
}

const FSlateBrush* UNiagaraStackStatelessEmitterGroup::GetIconBrush() const
{
	return FAppStyle::Get().GetBrush("Icons.Details");
}

bool UNiagaraStackStatelessEmitterGroup::SupportsSecondaryIcon() const
{
	return true;
}

const FSlateBrush* UNiagaraStackStatelessEmitterGroup::GetSecondaryIconBrush() const
{
	return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.LWEIcon");
}

void UNiagaraStackStatelessEmitterGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);

	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	if (StatelessEmitter != nullptr)
	{
		if ( NiagaraStackStatelessEmitterGroupPrivate::bShowRawObject )
		{
			UNiagaraStackStatelessEmitterObjectItem* RawObjectItem = RawObjectItemWeak.Get();
			if (RawObjectItem == nullptr || RawObjectItem->GetStatelessEmitter() != StatelessEmitter)
			{
				bool bExpandedByDefault = false;
				RawObjectItem = NewObject<UNiagaraStackStatelessEmitterObjectItem>(this);
				RawObjectItem->Initialize(
					CreateDefaultChildRequiredData(),
					StatelessEmitter,
					LOCTEXT("EmitterObjectDisplayName", "Raw Object"),
					bExpandedByDefault,
					FNiagaraStackObjectShared::FOnFilterDetailNodes());
				RawObjectItemWeak = RawObjectItem;
			}
			NewChildren.Add(RawObjectItem);
		}

		UNiagaraStackStatelessEmitterObjectItem* FilteredObjectItem = FilteredObjectItemWeak.Get();
		if (FilteredObjectItem == nullptr || FilteredObjectItem->GetStatelessEmitter() != StatelessEmitter)
		{
			bool bExpandedByDefault = true;
			FilteredObjectItem = NewObject<UNiagaraStackStatelessEmitterObjectItem>(this);
			FilteredObjectItem->Initialize(
				CreateDefaultChildRequiredData(),
				StatelessEmitter,
				LOCTEXT("EmitterPropertiesDisplayName", "Emitter Properties"),
				bExpandedByDefault,
				FNiagaraStackObjectShared::FOnFilterDetailNodes::CreateStatic(&UNiagaraStackStatelessEmitterGroup::FilterDetailNodes));
			FilteredObjectItemWeak = FilteredObjectItem;
		}
		NewChildren.Add(FilteredObjectItem);
	}
	else
	{
		RawObjectItemWeak.Reset();
		FilteredObjectItemWeak.Reset();
	}
}

void UNiagaraStackStatelessEmitterGroup::FilterDetailNodes(const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes)
{
	for (const TSharedRef<IDetailTreeNode>& SourceNode : InSourceNodes)
	{
		bool bIncludeNode = true;
		if (SourceNode->GetNodeType() == EDetailNodeType::Item)
		{
			TSharedPtr<IPropertyHandle> NodePropertyHandle = SourceNode->CreatePropertyHandle();
			if (NodePropertyHandle.IsValid() && NodePropertyHandle->HasMetaData("HideInStack"))
			{
				bIncludeNode = false;
			}
		}
		if (bIncludeNode)
		{
			OutFilteredNodes.Add(SourceNode);
		}
	}
}

void UNiagaraStackStatelessEmitterObjectItem::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter, FText InDisplayName, bool bInIsExpandedByDefault, FNiagaraStackObjectShared::FOnFilterDetailNodes InOnFilterDetailNodes)
{
	Super::Initialize(InRequiredEntryData, InStatelessEmitter->GetPathName() + TEXT("StatelessEmitterStackObjectItem"));
	StatelessEmitterWeak = InStatelessEmitter;
	DisplayName = InDisplayName;
	bIsExpandedByDefault = bInIsExpandedByDefault;
	OnFilterDetailNodes = InOnFilterDetailNodes;
}

void UNiagaraStackStatelessEmitterObjectItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	if (StatelessEmitter != nullptr)
	{
		UNiagaraStackObject* StatelessEmitterStackObject = StatelessEmitterStackObjectWeak.Get();
		if (StatelessEmitterStackObject == nullptr)
		{
			const bool bIsTopLevelObject = true;
			const bool bHideTopLevelCategories = FNiagaraStackStatelessCommon::ShouldHideTopLevelCategories(StatelessEmitter->GetClass());
			StatelessEmitterStackObject = NewObject<UNiagaraStackObject>(this);
			StatelessEmitterStackObject->Initialize(CreateDefaultChildRequiredData(), StatelessEmitter, bIsTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey());
			StatelessEmitterStackObject->SetOnFilterDetailNodes(OnFilterDetailNodes, UNiagaraStackObject::EDetailNodeFilterMode::FilterAllNodes);
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyLayout(UNiagaraStatelessEmitter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraStatelessEmitterDetails::MakeInstance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraEmitterStateData::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraEmitterStatePropertyCustomization::MakeInstance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionFloat::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeFloatInstance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionVector2::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector2Instance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionVector3::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector3Instance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionPosition::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakePositionInstance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionColor::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeColorInstance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeFloat::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeFloatInstance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeVector2::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector2Instance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeVector3::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector3Instance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeColor::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeColorInstance));
			StatelessEmitterStackObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeInt::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionIntPropertyCustomization::MakeIntInstance));
			StatelessEmitterStackObjectWeak = StatelessEmitterStackObject;
		}
		NewChildren.Add(StatelessEmitterStackObject);
	}
	else
	{
		StatelessEmitterStackObjectWeak.Reset();
	}

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

#undef LOCTEXT_NAMESPACE
