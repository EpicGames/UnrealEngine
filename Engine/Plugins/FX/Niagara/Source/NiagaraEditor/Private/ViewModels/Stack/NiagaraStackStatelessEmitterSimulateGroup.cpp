// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackStatelessEmitterSimulateGroup.h"

#include "IDetailTreeNode.h"
#include "NiagaraClipboard.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraStackEditorData.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Stateless/NiagaraDistributionPropertyCustomization.h"
#include "Stateless/NiagaraDistributionIntPropertyCustomization.h"
#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessModule.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackItemPropertyHeaderValueShared.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackStatelessCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackStatelessEmitterSimulateGroup)

#define LOCTEXT_NAMESPACE "NiagaraEmitterStatelessSimulateGroup"

namespace NiagaraStackStatelessEmitterSimulateGroupPrivate
{
	bool TestCanPasteModules(UNiagaraStatelessEmitter* StatelessEmitter, const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage)
	{
		if (StatelessEmitter)
		{
			for (const UObject* ClipboardModuleObject : ClipboardContent->StatelessModules)
			{
				const UNiagaraStatelessModule* ClipboardModule = Cast<const UNiagaraStatelessModule>(ClipboardModuleObject);
				UNiagaraStatelessModule* StatelessModule = ClipboardModule ? StatelessEmitter->GetModule(ClipboardModule->GetClass()) : nullptr;
				if (!StatelessModule)
				{
					continue;
				}

				OutMessage = LOCTEXT("CanPasteStatelessModules", "Paste module data, either adding or replacing existing module.");
				return true;
			}
		}

		OutMessage = LOCTEXT("CanPasteStatelessModuleUnsupported", "Incompatible or no data to paste.");
		return false;
	}

	FText GetPasteModulesTransactionText(const UNiagaraClipboardContent* ClipboardContent)
	{
		return LOCTEXT("PasteStatelessModulesTransaction", "Paste modules(s).");
	}

	TArray<UObject*> PasteModules(UNiagaraStatelessEmitter* StatelessEmitter, const UNiagaraClipboardContent* ClipboardContent)
	{
		TArray<UObject*> ModifiedObjects;
		if (StatelessEmitter)
		{
			for (const UObject* ClipboardModuleObject : ClipboardContent->StatelessModules)
			{
				const UNiagaraStatelessModule* ClipboardModule = Cast<const UNiagaraStatelessModule>(ClipboardModuleObject);
				UNiagaraStatelessModule* StatelessModule = ClipboardModule ? StatelessEmitter->GetModule(ClipboardModule->GetClass()) : nullptr;
				if (!StatelessModule)
				{
					continue;
				}

				StatelessModule->Modify();

				UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
				CopyParams.bDoDelta = false;
				UEngine::CopyPropertiesForUnrelatedObjects(const_cast<UNiagaraStatelessModule*>(ClipboardModule), StatelessModule, CopyParams);

				if (StatelessModule->CanDisableModule())
				{
					StatelessModule->SetIsModuleEnabled(ClipboardModule->IsModuleEnabled());
				}
				StatelessModule->PostEditChange();

				ModifiedObjects.Add(StatelessModule);
			}
		}

		return ModifiedObjects;
	}
}

class FNiagaraStatelessEmitterAddModuleAction : public INiagaraStackItemGroupAddAction
{

public:
	FNiagaraStatelessEmitterAddModuleAction(UNiagaraStatelessModule* StatelessModule)
	{
		StatelessModuleWeak = StatelessModule;
		DisplayName = StatelessModule->GetClass()->GetDisplayNameText();
	}

	UNiagaraStatelessModule* GetModule() const { return StatelessModuleWeak.Get(); }

	virtual TArray<FString> GetCategories() const override { return Categories; }
	virtual FText GetDisplayName() const override { return DisplayName; }
	virtual FText GetDescription() const override { return FText(); }
	virtual FText GetKeywords() const override { return FText(); }
	virtual TOptional<FNiagaraFavoritesActionData> GetFavoritesData() const override { return TOptional<FNiagaraFavoritesActionData>(); }
	virtual bool IsInLibrary() const override { return true; }
	virtual FNiagaraActionSourceData GetSourceData() const override { return FNiagaraActionSourceData(); }
	virtual TOptional<FSoftObjectPath> GetPreviewMoviePath() const override { return {}; }

private:
	TWeakObjectPtr<UNiagaraStatelessModule> StatelessModuleWeak;
	TArray<FString> Categories;
	FText DisplayName;
};

class FNiagaraStatelessEmitterSimulateGroupAddUtilities : public TNiagaraStackItemGroupAddUtilities<UNiagaraStatelessModule*>
{
public:
	FNiagaraStatelessEmitterSimulateGroupAddUtilities(UNiagaraStatelessEmitter& StatelessEmitter, UNiagaraStackEditorData& StackEditorData, FOnItemAdded InOnItemAdded)
		: TNiagaraStackItemGroupAddUtilities<UNiagaraStatelessModule*>(LOCTEXT("ModuleName", "Module"), EAddMode::AddFromAction, true, false, InOnItemAdded)
	{
		StatelessEmitterWeak = &StatelessEmitter;
		StackEditorDataWeak = &StackEditorData;
	}

	virtual void AddItemDirectly() { unimplemented(); };

	virtual void GenerateAddActions(TArray<TSharedRef<INiagaraStackItemGroupAddAction>>& OutAddActions, const FNiagaraStackItemGroupAddOptions& AddProperties = FNiagaraStackItemGroupAddOptions()) const override
	{
		UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
		UNiagaraStackEditorData* StackEditorData = StackEditorDataWeak.Get();
		if (StatelessEmitter != nullptr && StackEditorData != nullptr)
		{
			for (UNiagaraStatelessModule* StatelessModule : StatelessEmitter->GetModules())
			{
				if (StatelessModule->IsModuleEnabled() == false && 
					StackEditorData->GetStatelessModuleShowWhenDisabled(UNiagaraStackStatelessModuleItem::GenerateStackEditorDataKey(StatelessModule)) == false)
				{
					if (FNiagaraEditorUtilities::IsClassVisibileInGlobalFilter(StatelessModule->GetClass()))
					{
						OutAddActions.Add(MakeShared<FNiagaraStatelessEmitterAddModuleAction>(StatelessModule));
					}
				}
			}
		}
	}

	virtual void ExecuteAddAction(TSharedRef<INiagaraStackItemGroupAddAction> AddAction, int32 TargetIndex) override
	{
		TSharedRef<FNiagaraStatelessEmitterAddModuleAction> AddModuleAction = StaticCastSharedRef<FNiagaraStatelessEmitterAddModuleAction>(AddAction);
		UNiagaraStatelessModule* StatelessModule = AddModuleAction->GetModule();
		UNiagaraStackEditorData* StackEditorData = StackEditorDataWeak.Get();
		if (StatelessModule != nullptr && StackEditorData != nullptr)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("AddStatelessModuleTransaction", "Add module."));
			StatelessModule->Modify();
			StatelessModule->SetIsModuleEnabled(true);
			StatelessModule->PostEditChange();
			StackEditorData->Modify();
			StackEditorData->SetStatelessModuleShowWhenDisabled(UNiagaraStackStatelessModuleItem::GenerateStackEditorDataKey(StatelessModule), true);
			OnItemAdded.ExecuteIfBound(StatelessModule);
		}
	}

protected:
	TWeakObjectPtr<UNiagaraStatelessEmitter> StatelessEmitterWeak;
	TWeakObjectPtr<UNiagaraStackEditorData> StackEditorDataWeak;
};

void UNiagaraStackStatelessEmitterSimulateGroup::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter)
{
	AddUtilities = MakeShared<FNiagaraStatelessEmitterSimulateGroupAddUtilities>(*InStatelessEmitter, *InRequiredEntryData.StackEditorData,
		FNiagaraStatelessEmitterSimulateGroupAddUtilities::FOnItemAdded::CreateUObject(this, &UNiagaraStackStatelessEmitterSimulateGroup::ModuleAdded));
	Super::Initialize(
		InRequiredEntryData, 
		LOCTEXT("EmitterStatelessSimulateGroupDisplayName", "Simulate"),
		LOCTEXT("EmitterStatelessSimulateGroupToolTip", "Data related to the simulation of the particles"),
		AddUtilities.Get());
	StatelessEmitterWeak = InStatelessEmitter;

	InStatelessEmitter->GetOnTemplateChanged().AddUObject(this, &UNiagaraStackStatelessEmitterSimulateGroup::OnTemplateChanged);
}

const FSlateBrush* UNiagaraStackStatelessEmitterSimulateGroup::GetIconBrush() const
{
	return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stateless.UpdateIcon");
}

bool UNiagaraStackStatelessEmitterSimulateGroup::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	return NiagaraStackStatelessEmitterSimulateGroupPrivate::TestCanPasteModules(GetStatelessEmitter(), ClipboardContent, OutMessage);
}

FText UNiagaraStackStatelessEmitterSimulateGroup::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	return NiagaraStackStatelessEmitterSimulateGroupPrivate::GetPasteModulesTransactionText(ClipboardContent);
}

void UNiagaraStackStatelessEmitterSimulateGroup::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	TArray<UObject*> ModifiedObjects = NiagaraStackStatelessEmitterSimulateGroupPrivate::PasteModules(GetStatelessEmitter(), ClipboardContent);
	if (ModifiedObjects.Num() > 0)
	{
		OnDataObjectModified().Broadcast(ModifiedObjects, ENiagaraDataObjectChange::Changed);
		RefreshChildren();
	}
}

void UNiagaraStackStatelessEmitterSimulateGroup::OnTemplateChanged()
{
	if (IsFinalized() == false)
	{
		RefreshChildren();
	}
}

void UNiagaraStackStatelessEmitterSimulateGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);

	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	if (StatelessEmitter != nullptr)
	{
		for (UNiagaraStatelessModule* StatelessModule : StatelessEmitter->GetModules())
		{
			FString ModuleStackEditorDataKey = UNiagaraStackStatelessModuleItem::GenerateStackEditorDataKey(StatelessModule);
			if (StatelessModule->CanDisableModule() && 
				StatelessModule->IsModuleEnabled() == false &&
				GetStackEditorData().GetStatelessModuleShowWhenDisabled(ModuleStackEditorDataKey) == false)
			{
				// If a module is disabled and doesn't have the show when disabled flag set, filter it from the UI.
				continue;
			}

			UNiagaraStackStatelessModuleItem* ModuleItem = FindCurrentChildOfTypeByPredicate<UNiagaraStackStatelessModuleItem>(CurrentChildren,
				[StatelessModule](const UNiagaraStackStatelessModuleItem* CurrentChild) { return CurrentChild->GetStatelessModule() == StatelessModule; });
			if (ModuleItem == nullptr)
			{
				ModuleItem = NewObject<UNiagaraStackStatelessModuleItem>(this);
				ModuleItem->Initialize(CreateDefaultChildRequiredData(), StatelessModule);
				ModuleItem->OnModifiedGroupItems().AddUObject(this, &UNiagaraStackStatelessEmitterSimulateGroup::ModuleModifiedGroupItems);
			}
			NewChildren.Add(ModuleItem);
		}
	}
}

void UNiagaraStackStatelessEmitterSimulateGroup::ModuleAdded(UNiagaraStatelessModule* StatelessModule)
{
	GetSystemViewModel()->GetSelectionViewModel()->EmptySelection();
	GetSystemViewModel()->GetSelectionViewModel()->AddEntryToSelectionByDisplayedObjectDeferred(StatelessModule);
	OnDataObjectModified().Broadcast({ StatelessModule }, ENiagaraDataObjectChange::Changed);
	RefreshChildren();
}

void UNiagaraStackStatelessEmitterSimulateGroup::ModuleModifiedGroupItems()
{
	RefreshChildren();
}

FString UNiagaraStackStatelessModuleItem::GenerateStackEditorDataKey(const UNiagaraStatelessModule* InStatelessModule)
{
	return FString::Printf(TEXT("StatelessModuleItem-%s"), *InStatelessModule->GetName());
}

void UNiagaraStackStatelessModuleItem::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessModule* InStatelessModule)
{
	Super::Initialize(InRequiredEntryData, GenerateStackEditorDataKey(InStatelessModule));
	StatelessModuleWeak = InStatelessModule;
	DisplayName = InStatelessModule->GetClass()->GetDisplayNameText();
}

FText UNiagaraStackStatelessModuleItem::GetTooltipText() const
{
	if (UNiagaraStatelessModule* Module = StatelessModuleWeak.Get())
	{
		if (UClass* ModuleClass = Module->GetClass())
		{
			return ModuleClass->GetToolTipText();
		}
	}
	return Super::GetTooltipText();
}

bool UNiagaraStackStatelessModuleItem::TestCanCopyWithMessage(FText& OutMessage) const
{
	if (UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get())
	{
		OutMessage = LOCTEXT("CanCopyStatelessModule", "Copy module to the clipboard.");
		return true;
	}
	OutMessage = LOCTEXT("CanCopyStatelessModuleUnsupported", "This module does not support copy.");
	return false;
}

void UNiagaraStackStatelessModuleItem::Copy(UNiagaraClipboardContent* ClipboardContent) const
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	if (StatelessModule == nullptr)
	{
		return;
	}

	ClipboardContent->StatelessModules.Add(StaticDuplicateObject(StatelessModule, ClipboardContent));
}

bool UNiagaraStackStatelessModuleItem::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	return NiagaraStackStatelessEmitterSimulateGroupPrivate::TestCanPasteModules(GetStatelessEmitter(), ClipboardContent, OutMessage);
}

FText UNiagaraStackStatelessModuleItem::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	return NiagaraStackStatelessEmitterSimulateGroupPrivate::GetPasteModulesTransactionText(ClipboardContent);
}

void UNiagaraStackStatelessModuleItem::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	TArray<UObject*> ModifiedObjects = NiagaraStackStatelessEmitterSimulateGroupPrivate::PasteModules(GetStatelessEmitter(), ClipboardContent);
	if (ModifiedObjects.Num() > 0)
	{
		OnDataObjectModified().Broadcast(ModifiedObjects, ENiagaraDataObjectChange::Changed);
		GetStackEditorData().Modify();
		OnModifiedGroupItems().Broadcast();
	}
}

bool UNiagaraStackStatelessModuleItem::TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	if (StatelessModule != nullptr)
	{
		if (StatelessModule->CanDisableModule())
		{
			OutCanDeleteMessage = LOCTEXT("DeleteStatelessModule", "Delete this module.");
			return true;
		}
	}
	OutCanDeleteMessage = LOCTEXT("DeleteStatelessModuleUnsupported", "This module does not support being deleted.");
	return false;
}

FText UNiagaraStackStatelessModuleItem::GetDeleteTransactionText() const
{
	return LOCTEXT("DeleteStatelessModuleTransaction", "Delete module from lightweight emitter.");
}

void UNiagaraStackStatelessModuleItem::Delete()
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	if (StatelessModule != nullptr && StatelessModule->CanDisableModule())
	{
		StatelessModule->Modify();

		UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
		CopyParams.bDoDelta = false;
		UEngine::CopyPropertiesForUnrelatedObjects(StatelessModule->GetClass()->GetDefaultObject(), StatelessModule, CopyParams);

		// we want to reset the module to a valid initial state which may require any changes done in PostInitProperties()
		StatelessModule->PostInitProperties();

		StatelessModule->SetIsModuleEnabled(false);
		StatelessModule->PostEditChange();
		OnDataObjectModified().Broadcast({ StatelessModule }, ENiagaraDataObjectChange::Changed);
		GetStackEditorData().Modify();
		GetStackEditorData().SetStatelessModuleShowWhenDisabled(GetStackEditorDataKey(), false);
		OnModifiedGroupItems().Broadcast();
	}
}

bool UNiagaraStackStatelessModuleItem::SupportsChangeEnabled() const
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	return StatelessModule != nullptr && StatelessModule->CanDisableModule();
}

bool UNiagaraStackStatelessModuleItem::GetIsEnabled() const
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	return StatelessModule != nullptr && StatelessModule->IsModuleEnabled();
}

UNiagaraStatelessEmitter* UNiagaraStackStatelessModuleItem::GetStatelessEmitter() const
{
	UNiagaraStatelessModule* StatelessModule = GetStatelessModule();
	return StatelessModule ? StatelessModule->GetTypedOuter<UNiagaraStatelessEmitter>() : nullptr;
}

void UNiagaraStackStatelessModuleItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);

	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	if (StatelessModule != nullptr)
	{
		UNiagaraStackObject* ModuleObject = ModuleObjectWeak.Get();
		if (ModuleObject == nullptr || ModuleObject->GetObject() != StatelessModule)
		{
			const bool bIsInTopLevelObject = true;
			const bool bHideTopLevelCategories = FNiagaraStackStatelessCommon::ShouldHideTopLevelCategories(StatelessModule->GetClass());
			ModuleObject = NewObject<UNiagaraStackObject>(this);
			ModuleObject->Initialize(CreateDefaultChildRequiredData(), StatelessModule, bIsInTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey());
			ModuleObject->SetOnFilterDetailNodes(FNiagaraStackObjectShared::FOnFilterDetailNodes::CreateStatic(&UNiagaraStackStatelessModuleItem::FilterDetailNodes), UNiagaraStackObject::EDetailNodeFilterMode::FilterAllNodes);
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionFloat::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeFloatInstance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionVector2::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector2Instance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionVector3::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector3Instance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionPosition::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakePositionInstance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionColor::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeColorInstance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeFloat::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeFloatInstance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeVector2::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector2Instance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeVector3::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector3Instance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeColor::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeColorInstance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeRotator::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeRotatorInstance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeInt::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionIntPropertyCustomization::MakeIntInstance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionCurveFloat::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeFloatInstance));
			ModuleObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionCurveVector3::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector3Instance));
			ModuleObjectWeak = ModuleObject;
		}
		NewChildren.Add(ModuleObject);

		if (bGeneratedHeaderValueHandlers == false)
		{
			bGeneratedHeaderValueHandlers = true;
			FNiagaraStackItemPropertyHeaderValueShared::GenerateHeaderValueHandlers(*StatelessModule, nullptr, *StatelessModule->GetClass(), FSimpleDelegate::CreateUObject(this, &UNiagaraStackStatelessModuleItem::OnHeaderValueChanged), HeaderValueHandlers);
			if (StatelessModule->CanDebugDraw())
			{
				// Debug draw is handled separately here since it's visibility needs to be determined by the results of the CanDebugDraw function, rather than by property metadata.
				if ( FBoolProperty* DebugDrawProperty = GetDebugDrawProperty() )
				{
					HeaderValueHandlers.Add(MakeShared<FNiagaraStackItemPropertyHeaderValue>(
						*StatelessModule, nullptr, *DebugDrawProperty, 
						FSimpleDelegate::CreateUObject(this, &UNiagaraStackStatelessModuleItem::OnHeaderValueChanged)));
				}
			}
		}
		else
		{
			for (TSharedRef<FNiagaraStackItemPropertyHeaderValue> HeaderValueHandler : HeaderValueHandlers)
			{
				HeaderValueHandler->Refresh();
			}
		}
	}
	else
	{
		ModuleObjectWeak.Reset();
		HeaderValueHandlers.Empty();
	}
}

void UNiagaraStackStatelessModuleItem::SetIsEnabledInternal(bool bInIsEnabled)
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	if (StatelessModule != nullptr && StatelessModule->CanDisableModule() && StatelessModule->IsModuleEnabled() != bInIsEnabled)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("ChangeStatelessModuleEnabledTransaction", "Change module enabled"));
		StatelessModule->Modify();
		StatelessModule->SetIsModuleEnabled(bInIsEnabled);
		StatelessModule->PostEditChange();
		GetStackEditorData().Modify();
		GetStackEditorData().SetStatelessModuleShowWhenDisabled(GetStackEditorDataKey(), true);
		OnDataObjectModified().Broadcast({ StatelessModule }, ENiagaraDataObjectChange::Changed);
		RefreshChildren();
	}
}

void UNiagaraStackStatelessModuleItem::GetHeaderValueHandlers(TArray<TSharedRef<INiagaraStackItemHeaderValueHandler>>& OutHeaderValueHandlers) const
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	if (StatelessModule != nullptr)
	{
		OutHeaderValueHandlers.Append(HeaderValueHandlers);
	}
}

bool UNiagaraStackStatelessModuleItem::SupportsDebugDraw() const
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	return StatelessModule ? StatelessModule->CanDebugDraw() : false;
}

TOptional<FText> UNiagaraStackStatelessModuleItem::GetDebugDrawTooltip() const
{
	TOptional<FText> Tooltip;
	if (FBoolProperty* DebugDrawProperty = GetDebugDrawProperty())
	{
		Tooltip = DebugDrawProperty->GetToolTipText();
	}
	return Tooltip;
}

bool UNiagaraStackStatelessModuleItem::IsDebugDrawEnabled() const
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	return StatelessModule ? StatelessModule->IsDebugDrawEnabled() : false;
}

void UNiagaraStackStatelessModuleItem::SetDebugDrawEnabled(bool bInEnabled)
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	FBoolProperty* DebugDrawProperty = GetDebugDrawProperty();
	if (StatelessModule && DebugDrawProperty)
	{
		FScopedTransaction ScopedTransaction(FText::Format(LOCTEXT("SetDebugDrawEnabled", "Change {0}"), DebugDrawProperty->GetDisplayNameText()));
		StatelessModule->Modify();

		DebugDrawProperty->SetPropertyValue(DebugDrawProperty->ContainerPtrToValuePtr<void>(StatelessModule), bInEnabled);
		
		FPropertyChangedEvent PropertyChangedEvent(DebugDrawProperty);
		StatelessModule->PostEditChangeProperty(PropertyChangedEvent);

		OnHeaderValueChanged();
	}
}

const UNiagaraStackEntry::FCollectedUsageData& UNiagaraStackStatelessModuleItem::GetCollectedUsageData() const
{
	if (CachedCollectedUsageData.IsSet() == false)
	{
		CachedCollectedUsageData = FCollectedUsageData();

		INiagaraParameterPanelViewModel* ParamVM = GetSystemViewModel()->GetParameterPanelViewModel();
		const UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
		const UNiagaraStatelessEmitter* StatelessEmitter = StatelessModule ? StatelessModule->GetTypedOuter<const UNiagaraStatelessEmitter>() : nullptr;
		if ( ParamVM && StatelessModule && StatelessEmitter )
		{
			StatelessEmitter->GatherSimulationAttributeReferences(
				StatelessModule,
				[this, &ParamVM](FName WhereForm, const FNiagaraVariableBase& Variable, bool bRead, bool bWrite)
				{
					if (ParamVM->IsVariableSelected(Variable))
					{
						CachedCollectedUsageData.GetValue().bHasReferencedParameterRead |= bRead;
						CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite |= bWrite;
					}
				}
			);
		}
	}

	return CachedCollectedUsageData.GetValue();
}

void UNiagaraStackStatelessModuleItem::FilterDetailNodes(const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes)
{
	for (const TSharedRef<IDetailTreeNode>& SourceNode : InSourceNodes)
	{
		bool bIncludeNode = true;
		if (SourceNode->GetNodeType() == EDetailNodeType::Item)
		{
			TSharedPtr<IPropertyHandle> NodePropertyHandle = SourceNode->CreatePropertyHandle();
			if (NodePropertyHandle.IsValid() && (NodePropertyHandle->HasMetaData("HideInStack") || NodePropertyHandle->HasMetaData("ShowInStackItemHeader")))
			{
				bIncludeNode = false;
			}
		}
		if (bIncludeNode)
		{
			OutFilteredNodes.Add(SourceNode);
		}
	}
}

void UNiagaraStackStatelessModuleItem::OnHeaderValueChanged()
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	if (StatelessModule != nullptr)
	{
		TArray<UObject*> ChangedObjects;
		ChangedObjects.Add(StatelessModule);
		OnDataObjectModified().Broadcast(ChangedObjects, ENiagaraDataObjectChange::Changed);
	}
}

FBoolProperty* UNiagaraStackStatelessModuleItem::GetDebugDrawProperty() const
{
	UNiagaraStatelessModule* StatelessModule = StatelessModuleWeak.Get();
	if (StatelessModule == nullptr)
	{
		return nullptr;
	}

	UClass* ModuleClass = StatelessModule->GetClass();
	return CastField<FBoolProperty>(ModuleClass->FindPropertyByName(UNiagaraStatelessModule::PrivateMemberNames::bDebugDrawEnabled));
}

#undef LOCTEXT_NAMESPACE
