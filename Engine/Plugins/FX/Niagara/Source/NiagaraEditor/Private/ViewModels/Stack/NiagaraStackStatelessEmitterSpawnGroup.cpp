// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackStatelessEmitterSpawnGroup.h"

#include "IDetailTreeNode.h"
#include "NiagaraEditorStyle.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Stateless/NiagaraDistributionPropertyCustomization.h"
#include "Stateless/NiagaraDistributionIntPropertyCustomization.h"
#include "Stateless/NiagaraSpawnInfoPropertyCustomization.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "Stateless/NiagaraStatelessModule.h"
#include "Styling/AppStyle.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackItemPropertyHeaderValueShared.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackStatelessCommon.h"
#include "NiagaraClipboard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackStatelessEmitterSpawnGroup)

#define LOCTEXT_NAMESPACE "NiagaraEmitterStatelessSpawnGroup"

namespace NiagaraStackStatelessEmitterSpawnGroupPrivate
{
	TOptional<FNiagaraStatelessSpawnInfo> ConvertPortableValue(const FNiagaraClipboardPortableValue& PortableValue)
	{
		TOptional<FNiagaraStatelessSpawnInfo> ReturnValue;

		FNiagaraStatelessSpawnInfo TempSpawnInfo;
		if (PortableValue.TryUpdateStructValue(*FNiagaraStatelessSpawnInfo::StaticStruct(), reinterpret_cast<uint8*>(&TempSpawnInfo)))
		{
			ReturnValue = TempSpawnInfo;
		}
		return ReturnValue;
	}

	bool TestCanPaste(UNiagaraStatelessEmitter* StatelessEmitter, const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage)
	{
		if (StatelessEmitter)
		{
			for (const FNiagaraClipboardPortableValue& PortableValue : ClipboardContent->PortableValues)
			{
				if (ConvertPortableValue(PortableValue).IsSet())
				{
					OutMessage = LOCTEXT("CanPasteSpawnInfo", "Paste spawn info(s).");
					return true;
				}
			}
		}

		OutMessage = LOCTEXT("CanPasteSpawnInfoUnsupported", "Incompatible or no data to paste.");
		return false;
	}

	FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent)
	{
		return LOCTEXT("PasteSpawnInfoTransaction", "Paste spawn info(s).");
	}

	bool Paste(UNiagaraStatelessEmitter* StatelessEmitter, const UNiagaraClipboardContent* ClipboardContent)
	{
		bool bHasPastedValues = false;
		if (StatelessEmitter)
		{
			for (const FNiagaraClipboardPortableValue& PortableValue : ClipboardContent->PortableValues)
			{
				TOptional<FNiagaraStatelessSpawnInfo> NewSpawnInfo = ConvertPortableValue(PortableValue);
				if (!NewSpawnInfo.IsSet())
				{
					continue;
				}

				if (!bHasPastedValues)
				{
					bHasPastedValues = true;
					StatelessEmitter->Modify();
				}

				FNiagaraStatelessSpawnInfo& SpawnInfo = StatelessEmitter->AddSpawnInfo();
				SpawnInfo = NewSpawnInfo.GetValue();
				SpawnInfo.SourceId = FGuid::NewGuid();
			}

			if (bHasPastedValues)
			{
				StatelessEmitter->PostEditChange();
			}
		}

		return bHasPastedValues;
	}
}

class FNiagaraStatelessEmitterAddSpawnInfoAction : public INiagaraStackItemGroupAddAction
{
public:
	explicit FNiagaraStatelessEmitterAddSpawnInfoAction(ENiagaraStatelessSpawnInfoType InSpawnInfoType)
		: SpawnInfoType(InSpawnInfoType)
		, DisplayName(UNiagaraStackStatelessEmitterSpawnItem::GetDisplayName(InSpawnInfoType))
	{
	}

	ENiagaraStatelessSpawnInfoType GetSpawnInfoType() const { return SpawnInfoType; }

	virtual TArray<FString> GetCategories() const override { return Categories; }
	virtual FText GetDisplayName() const override { return DisplayName; }
	virtual FText GetDescription() const override { return FText(); }
	virtual FText GetKeywords() const override { return FText(); }
	virtual TOptional<FNiagaraFavoritesActionData> GetFavoritesData() const override { return TOptional<FNiagaraFavoritesActionData>(); }
	virtual bool IsInLibrary() const override { return true; }
	virtual FNiagaraActionSourceData GetSourceData() const override { return FNiagaraActionSourceData(); }
	virtual TOptional<FSoftObjectPath> GetPreviewMoviePath() const override { return {}; }

private:
	ENiagaraStatelessSpawnInfoType SpawnInfoType;
	TWeakObjectPtr<UNiagaraStatelessModule> StatelessModuleWeak;
	TArray<FString> Categories;
	FText DisplayName;
};

class FNiagaraStackStatelessEmitterSpawnGroupAddUtilities : public TNiagaraStackItemGroupAddUtilities<FGuid>
{
public:
	FNiagaraStackStatelessEmitterSpawnGroupAddUtilities(UNiagaraStatelessEmitter* StatelessEmitter, FOnItemAdded InOnItemAdded)
		: TNiagaraStackItemGroupAddUtilities(LOCTEXT("AddUtilitiesName", "Spawn Data"), EAddMode::AddFromAction, true, false, InOnItemAdded)
	{
		StatelessEmitterWeak = StatelessEmitter;
	}

	virtual void AddItemDirectly() override 
	{ 
		unimplemented();
	}

	virtual void GenerateAddActions(TArray<TSharedRef<INiagaraStackItemGroupAddAction>>& OutAddActions, const FNiagaraStackItemGroupAddOptions& AddProperties) const override
	{
		OutAddActions.Emplace(MakeShared<FNiagaraStatelessEmitterAddSpawnInfoAction>(ENiagaraStatelessSpawnInfoType::Burst));
		OutAddActions.Emplace(MakeShared<FNiagaraStatelessEmitterAddSpawnInfoAction>(ENiagaraStatelessSpawnInfoType::Rate));
	}

	virtual void ExecuteAddAction(TSharedRef<INiagaraStackItemGroupAddAction> AddAction, int32 TargetIndex) override
	{
		UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
		if (StatelessEmitter != nullptr)
		{
			TSharedRef<FNiagaraStatelessEmitterAddSpawnInfoAction> AddSpawnInfoAction = StaticCastSharedRef<FNiagaraStatelessEmitterAddSpawnInfoAction>(AddAction);

			FScopedTransaction ScopedTransaction(LOCTEXT("AddNewSpawnInfoTransaction", "Add new spawn data"));
			StatelessEmitter->Modify();

			FNiagaraStatelessSpawnInfo& SpawnInfo = StatelessEmitter->AddSpawnInfo();
			SpawnInfo.Type = AddSpawnInfoAction->GetSpawnInfoType();

			SpawnInfo.SourceId = FGuid::NewGuid();
			OnItemAdded.ExecuteIfBound(SpawnInfo.SourceId);
		}
	}

private:
	TWeakObjectPtr<UNiagaraStatelessEmitter> StatelessEmitterWeak;
};

void UNiagaraStackStatelessEmitterSpawnGroup::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter)
{
	AddUtilities = MakeShared<FNiagaraStackStatelessEmitterSpawnGroupAddUtilities>(InStatelessEmitter,
		FNiagaraStackStatelessEmitterSpawnGroupAddUtilities::FOnItemAdded::CreateUObject(this, &UNiagaraStackStatelessEmitterSpawnGroup::OnSpawnInfoAdded));
	Super::Initialize(
		InRequiredEntryData,
		LOCTEXT("EmitterStatelessSpawningGroupDisplayName", "Spawn"),
		LOCTEXT("EmitterStatelessSpawningGroupToolTip", "Data related to spawning particles"),
		AddUtilities.Get());
	StatelessEmitterWeak = InStatelessEmitter;
}

const FSlateBrush* UNiagaraStackStatelessEmitterSpawnGroup::GetIconBrush() const
{
	return FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stateless.SpawnIcon");
}

void UNiagaraStackStatelessEmitterSpawnGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);

	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	if (StatelessEmitter != nullptr)
	{
		for (int32 SpawnInfoIndex = 0; SpawnInfoIndex < StatelessEmitter->GetNumSpawnInfos(); SpawnInfoIndex++)
		{
			UNiagaraStackStatelessEmitterSpawnItem* SpawningItem = FindCurrentChildOfTypeByPredicate<UNiagaraStackStatelessEmitterSpawnItem>(CurrentChildren,
				[StatelessEmitter, SpawnInfoIndex](const UNiagaraStackStatelessEmitterSpawnItem* CurrentChild)
			{ 
				return
					CurrentChild->GetStatelessEmitter() == StatelessEmitter &&
					CurrentChild->GetIndex() == SpawnInfoIndex &&
					CurrentChild->GetSourceId() == StatelessEmitter->GetSpawnInfoByIndex(SpawnInfoIndex)->SourceId;
			});
			if (SpawningItem == nullptr)
			{
				SpawningItem = NewObject<UNiagaraStackStatelessEmitterSpawnItem>(this);
				SpawningItem->Initialize(CreateDefaultChildRequiredData(), StatelessEmitter, SpawnInfoIndex);
				SpawningItem->OnRequestDelete().BindUObject(this, &UNiagaraStackStatelessEmitterSpawnGroup::OnChildRequestDelete);
			}
			NewChildren.Add(SpawningItem);
		}
	}
}

void UNiagaraStackStatelessEmitterSpawnGroup::OnSpawnInfoAdded(FGuid AddedItemId)
{
	GetSystemViewModel()->GetSelectionViewModel()->EmptySelection();
	GetSystemViewModel()->GetSelectionViewModel()->AddEntryToSelectionBySelectionIdDeferred(AddedItemId);
	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	if (StatelessEmitter != nullptr)
	{
		OnDataObjectModified().Broadcast({ StatelessEmitter }, ENiagaraDataObjectChange::Changed);
	}
	RefreshChildren();
}

void UNiagaraStackStatelessEmitterSpawnGroup::OnChildRequestDelete(FGuid DeleteItemId)
{
	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	if (StatelessEmitter != nullptr)
	{
		int32 IndexToDelete = StatelessEmitter->IndexOfSpawnInfoBySourceId(DeleteItemId);
		if (IndexToDelete != INDEX_NONE)
		{
			StatelessEmitter->Modify();
			StatelessEmitter->RemoveSpawnInfoBySourceId(DeleteItemId);
			RefreshChildren();
		}
	}
}

bool UNiagaraStackStatelessEmitterSpawnGroup::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	return NiagaraStackStatelessEmitterSpawnGroupPrivate::TestCanPaste(GetStatelessEmitter(), ClipboardContent, OutMessage);
}

FText UNiagaraStackStatelessEmitterSpawnGroup::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	return NiagaraStackStatelessEmitterSpawnGroupPrivate::GetPasteTransactionText(ClipboardContent);
}

void UNiagaraStackStatelessEmitterSpawnGroup::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	UNiagaraStatelessEmitter* StatelessEmitter = GetStatelessEmitter();
	if (NiagaraStackStatelessEmitterSpawnGroupPrivate::Paste(StatelessEmitter, ClipboardContent))
	{
		OnDataObjectModified().Broadcast({ StatelessEmitter }, ENiagaraDataObjectChange::Changed);
		RefreshChildren();
	}
}

void UNiagaraStackStatelessEmitterSpawnItem::Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter, int32 InIndex)
{
	Super::Initialize(InRequiredEntryData, FString::Printf(TEXT("StatelessEmitterSpawnItem-%i"), InIndex));
	StatelessEmitterWeak = InStatelessEmitter;
	Index = InIndex;
	SourceId = InStatelessEmitter->GetSpawnInfoByIndex(InIndex)->SourceId;
	OnDataObjectModified().AddUObject(this, &UNiagaraStackStatelessEmitterSpawnItem::OnSpawnInfoModified);
}

FText UNiagaraStackStatelessEmitterSpawnItem::GetDisplayName(ENiagaraStatelessSpawnInfoType SpawnInfoType)
{
	switch (SpawnInfoType)
	{
		case ENiagaraStatelessSpawnInfoType::Burst:	return LOCTEXT("EmitterSpawnBurstDisplayName", "Spawn Burst Instantaneous");
		case ENiagaraStatelessSpawnInfoType::Rate:	return LOCTEXT("EmitterSpawnRateDisplayName", "Spawn Rate");
		default:									checkNoEntry();	return LOCTEXT("EmitterSpawnUnknownDisplayName", "Unknown");
	}
}

FText UNiagaraStackStatelessEmitterSpawnItem::GetTooltipText(ENiagaraStatelessSpawnInfoType SpawnInfoType)
{
	switch (SpawnInfoType)
	{
		case ENiagaraStatelessSpawnInfoType::Burst:	return LOCTEXT("EmitterSpawnBurstTooltpText", "Spawns a burst of particles instantaneously.");
		case ENiagaraStatelessSpawnInfoType::Rate:	return LOCTEXT("EmitterSpawnRateTooltpText", "Spawns particles continuously at a particular rate.");
		default:									checkNoEntry();	return LOCTEXT("EmitterSpawnUnknownTooltpText", "Unknown");
	}
}


FText UNiagaraStackStatelessEmitterSpawnItem::GetDisplayName() const
{
	const FNiagaraStatelessSpawnInfo* SpawnInfo = GetSpawnInfo();
	const ENiagaraStatelessSpawnInfoType SpawnInfoType = SpawnInfo ? SpawnInfo->Type : ENiagaraStatelessSpawnInfoType::Burst;
	return GetDisplayName(SpawnInfoType);
}

FText UNiagaraStackStatelessEmitterSpawnItem::GetTooltipText() const
{
	const FNiagaraStatelessSpawnInfo* SpawnInfo = GetSpawnInfo();
	const ENiagaraStatelessSpawnInfoType SpawnInfoType = SpawnInfo ? SpawnInfo->Type : ENiagaraStatelessSpawnInfoType::Burst;
	return GetTooltipText(SpawnInfoType);
}

FGuid UNiagaraStackStatelessEmitterSpawnItem::GetSelectionId() const
{
	return SourceId;
}

bool UNiagaraStackStatelessEmitterSpawnItem::TestCanCopyWithMessage(FText& OutMessage) const
{
	if (FNiagaraStatelessSpawnInfo* SpawnInfo = GetSpawnInfo())
	{
		OutMessage = LOCTEXT("CanCopyStatelessSpawnInfo", "Copy spawn info to the clipboard.");
		return true;
	}
	OutMessage = LOCTEXT("CanCopyStatelessSpawnInfoUnsupported", "This spawn info does not support copy.");
	return false;
}

void UNiagaraStackStatelessEmitterSpawnItem::Copy(UNiagaraClipboardContent* ClipboardContent) const
{
	if (FNiagaraStatelessSpawnInfo* SpawnInfo = GetSpawnInfo())
	{
		ClipboardContent->PortableValues.Emplace(
			FNiagaraClipboardPortableValue::CreateFromStructValue(*FNiagaraStatelessSpawnInfo::StaticStruct(), reinterpret_cast<uint8*>(SpawnInfo))
		);
	}
}

bool UNiagaraStackStatelessEmitterSpawnItem::TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const
{
	return NiagaraStackStatelessEmitterSpawnGroupPrivate::TestCanPaste(GetStatelessEmitter(), ClipboardContent, OutMessage);
}

FText UNiagaraStackStatelessEmitterSpawnItem::GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const
{
	return NiagaraStackStatelessEmitterSpawnGroupPrivate::GetPasteTransactionText(ClipboardContent);
}

void UNiagaraStackStatelessEmitterSpawnItem::Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning)
{
	UNiagaraStatelessEmitter* StatelessEmitter = GetStatelessEmitter();
	if (NiagaraStackStatelessEmitterSpawnGroupPrivate::Paste(StatelessEmitter, ClipboardContent))
	{
		if (UNiagaraStackStatelessEmitterSpawnGroup* SpawnGroup = GetTypedOuter<UNiagaraStackStatelessEmitterSpawnGroup>())
		{
			SpawnGroup->OnDataObjectModified().Broadcast({ StatelessEmitter }, ENiagaraDataObjectChange::Changed);
			SpawnGroup->RefreshChildren();
		}
	}
}

bool UNiagaraStackStatelessEmitterSpawnItem::TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const
{
	OutCanDeleteMessage = LOCTEXT("DeleteSpawnDataMessage", "Delete this spawn data.");
	return true;
}

FText UNiagaraStackStatelessEmitterSpawnItem::GetDeleteTransactionText() const
{
	return LOCTEXT("DeleteSpawnTransaction", "Delete spawn data");
}

void UNiagaraStackStatelessEmitterSpawnItem::Delete()
{
	OnRequestDeleteDelegate.ExecuteIfBound(SourceId);
}

bool UNiagaraStackStatelessEmitterSpawnItem::SupportsChangeEnabled() const
{
	return true;
}

bool UNiagaraStackStatelessEmitterSpawnItem::GetIsEnabled() const
{
	const FNiagaraStatelessSpawnInfo* SpawnInfo = GetSpawnInfo();
	return SpawnInfo ? SpawnInfo->bEnabled : false;
}

void UNiagaraStackStatelessEmitterSpawnItem::GetHeaderValueHandlers(TArray<TSharedRef<INiagaraStackItemHeaderValueHandler>>& OutHeaderValueHandlers) const
{
	if (GetSpawnInfo() != nullptr)
	{
		OutHeaderValueHandlers.Append(HeaderValueHandlers);
	}
}

const UNiagaraStackEntry::FCollectedUsageData& UNiagaraStackStatelessEmitterSpawnItem::GetCollectedUsageData() const
{
	if (CachedCollectedUsageData.IsSet() == false)
	{
		CachedCollectedUsageData = FCollectedUsageData();

		INiagaraParameterPanelViewModel* ParamVM = GetSystemViewModel()->GetParameterPanelViewModel();
		FNiagaraStatelessSpawnInfo* SpawnInfo = GetSpawnInfo();
		UNiagaraStatelessEmitter* StatelessEmitter = GetStatelessEmitter();
		if (StatelessEmitter && SpawnInfo && ParamVM)
		{
			StatelessEmitter->GatherSimulationAttributeReferences(
				*SpawnInfo,
				[this, &ParamVM](FName WhereForm, const FNiagaraVariableBase& Variable, bool bRead, bool bWrite)
				{
					if (ParamVM->IsVariableSelected(Variable))
					{
						CachedCollectedUsageData.GetValue().bHasReferencedParameterRead |= bRead;
						CachedCollectedUsageData.GetValue().bHasReferencedParameterWrite |= bWrite;
					}
				}
			);
		}
	}

	return CachedCollectedUsageData.GetValue();
}
void UNiagaraStackStatelessEmitterSpawnItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);

	FNiagaraStatelessSpawnInfo* SpawnInfo = GetSpawnInfo();
	if (SpawnInfo != nullptr)
	{
		uint8* SpawnInfoPtr = reinterpret_cast<uint8*>(SpawnInfo);
		if (SpawnInfoStructOnScope.IsValid() == false || SpawnInfoStructOnScope->GetStructMemory() != SpawnInfoPtr)
		{
			SpawnInfoStructOnScope = MakeShared<FStructOnScope>(FNiagaraStatelessSpawnInfo::StaticStruct(), SpawnInfoPtr);
		}

		UNiagaraStackObject* SpawnInfoObject = SpawnInfoObjectWeak.Get();
		UObject* StatelessEmitterObject = StatelessEmitterWeak.Get();
		if (SpawnInfoObject == nullptr || SpawnInfoObject->GetObject() != StatelessEmitterObject ||
			SpawnInfoObject->GetDisplayedStruct().IsValid() == false || SpawnInfoObject->GetDisplayedStruct()->GetStructMemory() != SpawnInfoPtr)
		{
			const bool bIsInTopLevelStruct = true;
			const bool bHideTopLevelCategories = FNiagaraStackStatelessCommon::ShouldHideTopLevelCategories(FNiagaraStatelessSpawnInfo::StaticStruct());

			SpawnInfoObject = NewObject<UNiagaraStackObject>(this);
			SpawnInfoObject->Initialize(CreateDefaultChildRequiredData(), StatelessEmitterObject, SpawnInfoStructOnScope.ToSharedRef(), TEXT("SpawnInfo"), bIsInTopLevelStruct, bHideTopLevelCategories, GetStackEditorDataKey());
			SpawnInfoObject->SetOnFilterDetailNodes(
				FNiagaraStackObjectShared::FOnFilterDetailNodes::CreateStatic(&UNiagaraStackStatelessEmitterSpawnItem::FilterDetailNodes),
				UNiagaraStackObject::EDetailNodeFilterMode::FilterAllNodes);
			SpawnInfoObject->RegisterInstancedCustomPropertyLayout(FNiagaraStatelessSpawnInfo::StaticStruct(), FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraSpawnInfoDetailCustomization::MakeInstance));

			//SpawnInfoObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionFloat::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeFloatInstance, StatelessEmitterObject));
			//SpawnInfoObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionVector2::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector2Instance));
			//SpawnInfoObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionVector3::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector3Instance));
			//SpawnInfoObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionColor::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeColorInstance));
			SpawnInfoObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeFloat::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeFloatInstance, StatelessEmitterObject));
			//SpawnInfoObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeVector2::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector2Instance));
			//SpawnInfoObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeVector3::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeVector3Instance));
			//SpawnInfoObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeColor::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionPropertyCustomization::MakeColorInstance));
			SpawnInfoObject->RegisterInstancedCustomPropertyTypeLayout(FNiagaraDistributionRangeInt::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraDistributionIntPropertyCustomization::MakeIntInstance, StatelessEmitterObject));

			SpawnInfoObjectWeak = SpawnInfoObject;
		}
		NewChildren.Add(SpawnInfoObject);

		if (bGeneratedHeaderValueHandlers == false)
		{
			bGeneratedHeaderValueHandlers = true;
			FNiagaraStackItemPropertyHeaderValueShared::GenerateHeaderValueHandlers(*StatelessEmitterObject, (uint8*)SpawnInfo, *SpawnInfo->StaticStruct(), FSimpleDelegate::CreateUObject(this, &UNiagaraStackStatelessEmitterSpawnItem::OnHeaderValueChanged), HeaderValueHandlers);
		}
		else
		{
			for (TSharedRef<FNiagaraStackItemPropertyHeaderValue> HeaderValueHandler : HeaderValueHandlers)
			{
				HeaderValueHandler->Refresh();
			}
		}
	}
	else
	{
		SpawnInfoStructOnScope.Reset();
		SpawnInfoObjectWeak.Reset();
		HeaderValueHandlers.Empty();
	}
}

void UNiagaraStackStatelessEmitterSpawnItem::SetIsEnabledInternal(bool bInIsEnabled)
{
	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	FNiagaraStatelessSpawnInfo* SpawnInfo = GetSpawnInfo();
	if (StatelessEmitter && SpawnInfo && SpawnInfo->bEnabled != bInIsEnabled)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("ChangeStatelessSpawnInfoEnabledTransaction", "Change spawn info enabled"));
		StatelessEmitter->Modify();
		SpawnInfo->bEnabled = bInIsEnabled;
		StatelessEmitter->PostEditChange();

		TArray<UObject*> ChangedObjects;
		ChangedObjects.Add(StatelessEmitter);
		OnDataObjectModified().Broadcast(ChangedObjects, ENiagaraDataObjectChange::Changed);

		RefreshChildren();
	}
}

FNiagaraStatelessSpawnInfo* UNiagaraStackStatelessEmitterSpawnItem::GetSpawnInfo() const
{
	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	if (StatelessEmitter != nullptr && Index >= 0 && Index < StatelessEmitter->GetNumSpawnInfos())
	{
		return StatelessEmitter->GetSpawnInfoByIndex(Index);
	}
	return nullptr;
}

void UNiagaraStackStatelessEmitterSpawnItem::FilterDetailNodes(const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes)
{
	for (const TSharedRef<IDetailTreeNode>& SourceNode : InSourceNodes)
	{
		bool bIncludeNode = true;
		if (SourceNode->GetNodeType() == EDetailNodeType::Item)
		{
			TSharedPtr<IPropertyHandle> NodePropertyHandle = SourceNode->CreatePropertyHandle();
			if (NodePropertyHandle.IsValid() && (NodePropertyHandle->HasMetaData("HideInStack") || NodePropertyHandle->HasMetaData("ShowInStackItemHeader")))
			{
				bIncludeNode = false;
			}
		}
		if (bIncludeNode)
		{
			OutFilteredNodes.Add(SourceNode);
		}
	}
}

void UNiagaraStackStatelessEmitterSpawnItem::OnHeaderValueChanged()
{
	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	if (StatelessEmitter != nullptr)
	{
		TArray<UObject*> ChangedObjects;
		ChangedObjects.Add(StatelessEmitter);
		OnDataObjectModified().Broadcast(ChangedObjects, ENiagaraDataObjectChange::Changed);
	}
}

//-TODO:Stateless: There should be cleaner way of doing this
void UNiagaraStackStatelessEmitterSpawnItem::OnSpawnInfoModified(TArray<UObject*> Objects, ENiagaraDataObjectChange ChangeType)
{
	UNiagaraStatelessEmitter* StatelessEmitter = StatelessEmitterWeak.Get();
	if (Objects.Num() == 1 && Objects[0] == StatelessEmitter && StatelessEmitter)
	{
		FNiagaraStatelessSpawnInfo* SpawnInfo = GetSpawnInfo();
		SpawnInfo->Rate.UpdateValuesFromDistribution();
		SpawnInfo->SpawnProbability.UpdateValuesFromDistribution();

		FPropertyChangedEvent EmptyPropertyUpdateStruct(nullptr);
		StatelessEmitter->PostEditChangeProperty(EmptyPropertyUpdateStruct);
	}
}

#undef LOCTEXT_NAMESPACE
