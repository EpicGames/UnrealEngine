// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackSummaryViewInputCollection.h"

#include "IDetailTreeNode.h"
#include "NiagaraClipboard.h"
#include "NiagaraConstants.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraRendererProperties.h"
#include "NiagaraSimulationStageBase.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"
#include "ViewModels/Stack/NiagaraStackEmitterPropertiesGroup.h"
#include "ViewModels/Stack/NiagaraStackEmitterSettingsGroup.h"
#include "ViewModels/Stack/NiagaraStackEventScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackValueCollection.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackPropertyRow.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackRenderersOwner.h"
#include "ViewModels/Stack/NiagaraStackSimulationStageGroup.h"
#include "Editor.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackSummaryViewInputCollection)

#define LOCTEXT_NAMESPACE "UNiagaraStackSummaryViewObject"

using namespace FNiagaraStackGraphUtilities;

void UNiagaraStackSummaryViewCollection::Initialize(FRequiredEntryData InRequiredEntryData,FVersionedNiagaraEmitterWeakPtr InEmitter, FString InOwningStackItemEditorDataKey)
{
	checkf(Emitter.Emitter == nullptr, TEXT("Can only initialize once."));
	FString ObjectStackEditorDataKey = FString::Printf(TEXT("%s-FilteredView"), *InOwningStackItemEditorDataKey);
	Super::Initialize(InRequiredEntryData, InOwningStackItemEditorDataKey, ObjectStackEditorDataKey);

	Emitter = InEmitter;
	GetEmitterViewModel()->GetEditorData().OnSummaryViewStateChanged().AddUObject(this, &UNiagaraStackSummaryViewCollection::OnViewStateChanged);
	GetEmitterViewModel()->GetSummaryHierarchyViewModel()->GetOnHierarchyChangedDelegate().AddUObject(this, &UNiagaraStackSummaryViewCollection::OnHierarchyChanged);

	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackSummaryViewCollection::FilterOnlyModified));
}

void UNiagaraStackSummaryViewCollection::FinalizeInternal()
{
	TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModelPtr = GetEmitterViewModel();
	if (EmitterViewModelPtr.IsValid() && GetEmitterViewModel()->GetEmitter().GetEmitterData())
	{
		GetEmitterViewModel()->GetEditorData().OnSummaryViewStateChanged().RemoveAll(this);
	}

	if(EmitterViewModelPtr.IsValid() && EmitterViewModelPtr->GetSummaryHierarchyViewModel() != nullptr)
	{
		EmitterViewModelPtr->GetSummaryHierarchyViewModel()->GetOnHierarchyChangedDelegate().RemoveAll(this);
	}

	Super::FinalizeInternal();
}

FText UNiagaraStackSummaryViewCollection::GetDisplayName() const
{
	return LOCTEXT("FilteredInputCollectionDisplayName", "Filtered Inputs");
}

bool UNiagaraStackSummaryViewCollection::GetIsEnabled() const
{
	return true;
}

void UNiagaraStackSummaryViewCollection::RefreshForAdvancedToggle()
{
	if (!IsFinalized())
	{
		CacheLastActiveSection();
		// category visibility will be determined by filtered children, so we need to refresh them before updating cached section data
		RefreshFilteredChildren();
		UpdateCachedSectionData();
		// as section data might have changed, we now have to refresh filtered children again
		RefreshFilteredChildren();
	}
}

bool UNiagaraStackSummaryViewCollection::GetShouldShowInStack() const
{
	return true;
}

void UNiagaraStackSummaryViewCollection::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{	
	TSharedPtr<FNiagaraEmitterViewModel> ViewModel = GetEmitterViewModel();
	TSharedPtr<FNiagaraScriptViewModel> ScriptViewModelPinned = ViewModel->GetSharedScriptViewModel();
	checkf(ScriptViewModelPinned.IsValid(), TEXT("Can not refresh children when the script view model has been deleted."));

	const UHierarchyRoot* Root = ViewModel->GetEditorData().GetSummaryRoot();
	TSharedPtr<FHierarchyRootViewModel> RootViewModel = ViewModel->GetSummaryHierarchyViewModel()->GetHierarchyRootViewModel();
	//-TODO:Stateless: Do we need stateless support here?
	if (RootViewModel == nullptr)
	{
		return;
	}

	// we make sure to sync view models to data before refreshing in order to get rid of possibly removed entries
	RootViewModel->SyncViewModelsToData();

	FNiagaraStackGraphUtilities::FInputDataCollection State;
	FNiagaraStackGraphUtilities::GatherInputRelationsForStack(State, GetEmitterViewModel().ToSharedRef());
	
	TArray<TSharedPtr<FHierarchyElementViewModel>> ChildrenViewModels;
	RootViewModel->GetChildrenViewModelsForType<UHierarchyElement, FHierarchyElementViewModel>(ChildrenViewModels, false);

	TArray<TSharedPtr<FHierarchyElementViewModel>> AllChildrenViewModels;
	RootViewModel->GetChildrenViewModelsForType<UHierarchyElement, FHierarchyElementViewModel>(AllChildrenViewModels, true);

	// first we gather all function call nodes so we can create cache for them instead of looking it up for each input individually
	TSet<UNiagaraNodeFunctionCall*> UsedFunctionCallNodes;
	for(TSharedPtr<FHierarchyElementViewModel> HierarchyViewModel : AllChildrenViewModels)
	{
		UHierarchyElement* Data = HierarchyViewModel->GetDataMutable();
		TOptional<FGuid> FunctionCallGuid;
		if(UNiagaraHierarchyModuleInput* ModuleInput = Cast<UNiagaraHierarchyModuleInput>(Data))
		{
			FunctionCallGuid = ModuleInput->GetPersistentIdentity().Guids[0];
		}
		else if(UNiagaraHierarchyAssignmentInput* AssignmentInput = Cast<UNiagaraHierarchyAssignmentInput>(Data))
		{
			FunctionCallGuid = AssignmentInput->GetPersistentIdentity().Guids[0];
		}
		else if(UNiagaraHierarchyModule* Module = Cast<UNiagaraHierarchyModule>(Data))
		{
			FunctionCallGuid = Module->GetPersistentIdentity().Guids[0];
		}

		if(FunctionCallGuid.IsSet() && State.NodeGuidToModuleNodeMap.Contains(FunctionCallGuid.GetValue()))
		{
			UsedFunctionCallNodes.Add(State.NodeGuidToModuleNodeMap[FunctionCallGuid.GetValue()]);
		}
	}
	
	TMap<FGuid, TSet<FNiagaraVariable>> FunctionCallToHiddenVariablesMap;
	for(UNiagaraNodeFunctionCall* FunctionCall : UsedFunctionCallNodes)
	{
		TArray<FNiagaraVariable> InputVariables;
		TSet<FNiagaraVariable> HiddenVariables;
		FCompileConstantResolver Resolver(GetEmitterViewModel()->GetEmitter(), FNiagaraStackGraphUtilities::GetOutputNodeUsage(*FunctionCall));
		FNiagaraStackGraphUtilities::GetStackFunctionInputs(*FunctionCall, InputVariables, HiddenVariables, Resolver, ENiagaraGetStackFunctionInputPinsOptions::ModuleInputsOnly, true);
		FunctionCallToHiddenVariablesMap.Add(FunctionCall->NodeGuid, HiddenVariables);

		TArray<UEdGraphPin*> OutInputPins;
		TSet<UEdGraphPin*> OutHiddenPins;
		FNiagaraStackGraphUtilities::GetStackFunctionStaticSwitchPins(*FunctionCall, OutInputPins, OutHiddenPins, Resolver);
		for(UEdGraphPin* HiddenStaticSwitchPin : OutHiddenPins)
		{
			FunctionCallToHiddenVariablesMap[FunctionCall->NodeGuid].Add(UEdGraphSchema_Niagara::PinToNiagaraVariable(HiddenStaticSwitchPin));
		}
	}
	
	for(TSharedPtr<FHierarchyElementViewModel> HierarchyViewModel : ChildrenViewModels)
	{
		UHierarchyElement* Data = HierarchyViewModel->GetDataMutable();

		if(UNiagaraHierarchyModuleInput* ModuleInput = Cast<UNiagaraHierarchyModuleInput>(Data))
		{
			if(GetEmitterViewModel().IsValid())
			{
				UNiagaraNodeFunctionCall* OwningFunctionCallNode = State.NodeGuidToModuleNodeMap[ModuleInput->GetPersistentIdentity().Guids[0]];
				
				TSharedPtr<FNiagaraModuleInputViewModel> ModuleInputViewModel = StaticCastSharedPtr<FNiagaraModuleInputViewModel>(HierarchyViewModel);
				TOptional<FInputData> ModuleBaseInputData = ModuleInputViewModel->GetInputData();
				
				if(ModuleBaseInputData.IsSet())
				{
					UNiagaraStackFunctionInput* TopLevelInput = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
					{
						return CurrentInput->GetInputParameterHandle() == ModuleBaseInputData->InputName && CurrentInput->GetInputType() == ModuleBaseInputData->Type
							&& CurrentInput->GetInputFunctionCallInitialScript() == OwningFunctionCallNode->FunctionScript
							// we are checking for node guid as we could have 2x the same input from 2 identical modules
							&& CurrentInput->GetInputFunctionCallNode().NodeGuid == OwningFunctionCallNode->NodeGuid;
					});
	
					if(TopLevelInput == nullptr)
					{
						TopLevelInput = NewObject<UNiagaraStackFunctionInput>(this);
						TopLevelInput->Initialize(CreateDefaultChildRequiredData(), *OwningFunctionCallNode, *OwningFunctionCallNode, ModuleBaseInputData->InputName, ModuleBaseInputData->Type, ModuleBaseInputData->bIsStatic ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic, GetOwnerStackItemEditorDataKey());
						TAttribute<FText> SummaryViewDisplayNameOverride;
						SummaryViewDisplayNameOverride.BindUObject(ModuleInput, &UNiagaraHierarchyModuleInput::GetDisplayNameOverride);
						TopLevelInput->SetSummaryViewDisplayName(SummaryViewDisplayNameOverride);
						TAttribute<FText> SummaryViewTooltipOverride;
						SummaryViewTooltipOverride.BindUObject(ModuleInput, &UNiagaraHierarchyModuleInput::GetTooltipOverride);
						TopLevelInput->SetSummaryViewTooltip(SummaryViewTooltipOverride);
					}

					FNiagaraVariable DisplayedVariable(TopLevelInput->GetInputType(), TopLevelInput->GetInputParameterHandle().GetParameterHandleString());
					bool bIsTopLevelHidden = FunctionCallToHiddenVariablesMap[ModuleInput->GetPersistentIdentity().Guids[0]].Contains(DisplayedVariable);
					TopLevelInput->SetIsHidden(bIsTopLevelHidden);
					
					NewChildren.Add(TopLevelInput);

					TArray<UNiagaraHierarchyModuleInput*> ChildInputs;
					ModuleInput->GetChildrenOfType<UNiagaraHierarchyModuleInput>(ChildInputs);

					for(UNiagaraHierarchyModuleInput* ChildInput : ChildInputs)
					{
						TSharedPtr<FNiagaraModuleInputViewModel> ChildInputViewModel = StaticCastSharedPtr<FNiagaraModuleInputViewModel>(ModuleInputViewModel->FindViewModelForChild(ChildInput, false));
						TOptional<FInputData> ChildInputData = ChildInputViewModel->GetInputData();

						if(ChildInputData.IsSet())
						{
							UNiagaraNodeFunctionCall* ChildFunctionNode = ChildInputData->FunctionCallNode;
							UNiagaraStackFunctionInput* StackChildInput = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
							{
								return CurrentInput->GetInputParameterHandle() == ChildInputData->InputName && CurrentInput->GetInputType() == ChildInputData->Type
									&& CurrentInput->GetInputFunctionCallInitialScript() == ChildFunctionNode->FunctionScript
									// we are checking for node guid as we could have 2x the same input from 2 identical modules
									&& CurrentInput->GetInputFunctionCallNode().NodeGuid == ChildFunctionNode->NodeGuid;
							});
							
							if(StackChildInput == nullptr)
							{
								StackChildInput = NewObject<UNiagaraStackFunctionInput>(this);
								StackChildInput->Initialize(CreateDefaultChildRequiredData(), *ChildInputData->FunctionCallNode, *ChildInputData->FunctionCallNode, ChildInputData->InputName, ChildInputData->Type, ChildInputData->bIsStatic ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic, GetOwnerStackItemEditorDataKey());
								TAttribute<FText> SummaryViewDisplayNameOverride;
								SummaryViewDisplayNameOverride.BindUObject(ChildInput, &UNiagaraHierarchyModuleInput::GetDisplayNameOverride);
								StackChildInput->SetSummaryViewDisplayName(SummaryViewDisplayNameOverride);
								TAttribute<FText> SummaryViewTooltipOverride;
								SummaryViewTooltipOverride.BindUObject(ChildInput, &UNiagaraHierarchyModuleInput::GetTooltipOverride);
								StackChildInput->SetSummaryViewTooltip(SummaryViewTooltipOverride);
								
								StackChildInput->SetSemanticChild(true);
							}

							FNiagaraVariable DisplayedChildVariable(StackChildInput->GetInputType(), StackChildInput->GetInputParameterHandle().GetParameterHandleString());
							bool bIsChildInputHidden = FunctionCallToHiddenVariablesMap[ChildInput->GetPersistentIdentity().Guids[0]].Contains(DisplayedChildVariable);
							StackChildInput->SetIsHidden(bIsChildInputHidden);
							
							NewChildren.Add(StackChildInput);
						}
					}
					
					/** Add children inputs automatically. Disabled as we want children inputs to be managed by the user for summary view. */
					// if(State.HierarchyInputToChildrenGuidMap.Contains(ModuleInput))
					// {
					// 	// children guids are already sorted by sort order
					// 	TArray<FGuid> ChildrenInputs = State.HierarchyInputToChildrenGuidMap[ModuleInput];
					//
					// 	for(FGuid ChildrenInputGuid : ChildrenInputs)
					// 	{
					// 		if(UNiagaraScriptVariable* ChildScriptVariable = State.ChildrenGuidToScriptVariablesMap[ChildrenInputGuid])
					// 		{
					// 			FNiagaraVariable ChildVariable = ChildScriptVariable->Variable;
					// 			
					// 			// we generally don't show inline edit toggles as the managed items will display an inline checkbox instead
					// 			if(ChildScriptVariable->Metadata.bInlineEditConditionToggle)
					// 			{
					// 				continue;
					// 			}
					// 			
					// 			UNiagaraStackFunctionInput* InputChild = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
					// 			{
					// 				return CurrentInput->GetInputParameterHandle() == ChildScriptVariable->Variable.GetName() && CurrentInput->GetInputType() == ChildScriptVariable->Variable.GetType() && CurrentInput->GetInputFunctionCallNode().NodeGuid == ModuleBaseInputData->FunctionCallNode->NodeGuid;
					// 			});
					//
					// 			TOptional<bool> bIsStatic = OwningFunctionCallNode->GetCalledGraph()->IsStaticSwitch(ChildVariable);
					// 			if(InputChild == nullptr)
					// 			{
					// 				InputChild = NewObject<UNiagaraStackFunctionInput>(this);
					// 				InputChild->Initialize(CreateDefaultChildRequiredData(), *ModuleBaseInputData->FunctionCallNode, *ModuleBaseInputData->FunctionCallNode, ChildScriptVariable->Variable.GetName(), ChildScriptVariable->Variable.GetType(), bIsStatic.GetValue() ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic, GetOwnerStackItemEditorDataKey());						
					// 				InputChild->SetSemanticChild(true);
					// 			}
					//
					// 			// we update the hidden flag every time
					// 			bool bIsChildHidden = FunctionCallToHiddenVariablesMap[ModuleInput->GetPersistentIdentity().Guids[0]].Contains(ChildVariable);
					// 			InputChild->SetIsHidden(bIsTopLevelHidden || bIsChildHidden);
					// 			NewChildren.Add(InputChild);								
					// 		}
					// 	}
					// }					
				}
			}
		}
		else if(UNiagaraHierarchyAssignmentInput* AssignmentInput = Cast<UNiagaraHierarchyAssignmentInput>(Data))
		{
			TSharedPtr<FNiagaraAssignmentInputViewModel> AssignmentInputViewModel = StaticCastSharedPtr<FNiagaraAssignmentInputViewModel>(HierarchyViewModel);
			TOptional<FMatchingFunctionInputData> InputData = AssignmentInputViewModel->GetInputData();
			if(InputData.IsSet())
			{
				FString VariableNameWithModulePrefix = FNiagaraConstants::ModuleNamespaceString + ".";
				VariableNameWithModulePrefix.Append(InputData->InputName.ToString());
				FName VariableNameToTestAgainst(VariableNameWithModulePrefix);

				UNiagaraStackFunctionInput* TopLevelInput = FindCurrentChildOfTypeByPredicate<UNiagaraStackFunctionInput>(CurrentChildren, [&](UNiagaraStackFunctionInput* CurrentInput) 
				{
					return CurrentInput->GetInputParameterHandle() == VariableNameToTestAgainst && CurrentInput->GetInputType() == InputData->Type && &CurrentInput->GetInputFunctionCallNode() == InputData->FunctionCallNode;
				});
	
				if(TopLevelInput == nullptr)
				{
					TopLevelInput = NewObject<UNiagaraStackFunctionInput>(this);
					TopLevelInput->Initialize(CreateDefaultChildRequiredData(), *InputData->FunctionCallNode, *InputData->FunctionCallNode, FName(VariableNameWithModulePrefix), InputData->Type, InputData->bIsStatic ? EStackParameterBehavior::Static : EStackParameterBehavior::Dynamic, GetOwnerStackItemEditorDataKey());
					TAttribute<FText> SummaryViewTooltipOverride;
					SummaryViewTooltipOverride.BindUObject(AssignmentInput, &UNiagaraHierarchyAssignmentInput::GetTooltipOverride);
					TopLevelInput->SetSummaryViewTooltip(SummaryViewTooltipOverride);
				}
				
				TopLevelInput->SetIsHidden(InputData->bIsHidden);
				
				NewChildren.Add(TopLevelInput);
			}
		}
		else if(UNiagaraHierarchyModule* SummaryModule = Cast<UNiagaraHierarchyModule>(Data))
		{
			if(UNiagaraNodeFunctionCall* MatchingFunctionCall = FNiagaraStackGraphUtilities::FindFunctionCallNode(SummaryModule->GetPersistentIdentity().Guids[0], GetEmitterViewModel().ToSharedRef()))
			{
				UNiagaraStackModuleItem* Module = FindCurrentChildOfTypeByPredicate<UNiagaraStackModuleItem>(CurrentChildren,
				[&](UNiagaraStackModuleItem* CurrentModule) { return &CurrentModule->GetModuleNode() == MatchingFunctionCall; });
		
				if (Module == nullptr)
				{
					Module = NewObject<UNiagaraStackModuleItem>(this);
					Module->Initialize(CreateDefaultChildRequiredData(), nullptr, *MatchingFunctionCall);
				}
				
				NewChildren.Add(Module);	
			}		
		}
		else if(UNiagaraHierarchyRenderer* SummaryRenderer = Cast<UNiagaraHierarchyRenderer>(Data))
		{
			TArray<UNiagaraRendererProperties*> RendererProperties = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetRenderers();
			UNiagaraRendererProperties** MatchingRendererProperties = RendererProperties.FindByPredicate([RendererIdentity = SummaryRenderer->GetPersistentIdentity().Guids[0]](UNiagaraRendererProperties* Candidate)
			{
				return Candidate->GetMergeId() == RendererIdentity;
			});

			if(MatchingRendererProperties != nullptr)
			{
				UNiagaraStackRendererItem* StackRenderer = FindCurrentChildOfTypeByPredicate<UNiagaraStackRendererItem>(CurrentChildren,
				[&](UNiagaraStackRendererItem* CurrentRenderer) { return CurrentRenderer->GetRendererProperties()->GetMergeId() == (*MatchingRendererProperties)->GetMergeId(); });
		
				if (StackRenderer == nullptr)
				{
					StackRenderer = NewObject<UNiagaraStackRendererItem>(this);
					StackRenderer->Initialize(CreateDefaultChildRequiredData(), FNiagaraStackRenderersOwnerStandard::CreateShared(GetEmitterViewModel().ToSharedRef()), *MatchingRendererProperties);
				}
			
				NewChildren.Add(StackRenderer);	
			}
		}
		else if(UNiagaraHierarchyEmitterProperties* EmitterProperties = Cast<UNiagaraHierarchyEmitterProperties>(Data))
		{
			UNiagaraStackEmitterPropertiesItem* StackEmitterPropertiesItem = FindCurrentChildOfTypeByPredicate<UNiagaraStackEmitterPropertiesItem>(CurrentChildren,
			[&](UNiagaraStackEmitterPropertiesItem* CurrentEmitterProperties) { return CurrentEmitterProperties->GetEmitterViewModel()->GetEmitter().Emitter->GetUniqueEmitterName() == EmitterProperties->GetPersistentIdentity().Names[0]; });
	
			if (StackEmitterPropertiesItem == nullptr)
			{
				StackEmitterPropertiesItem = NewObject<UNiagaraStackEmitterPropertiesItem>(this);
				StackEmitterPropertiesItem->Initialize(CreateDefaultChildRequiredData());
			}
		
			NewChildren.Add(StackEmitterPropertiesItem);	
		}
		else if(UNiagaraHierarchyEventHandler* EventHandler = Cast<UNiagaraHierarchyEventHandler>(Data))
		{
			const TArray<FNiagaraEventScriptProperties>& EventScriptProperties = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetEventHandlers();
			const FNiagaraEventScriptProperties* EventScriptPropertiesItem = EventScriptProperties.FindByPredicate([EventHandlerIdentity = EventHandler->GetPersistentIdentity()](const FNiagaraEventScriptProperties& Candidate)
			{
				return Candidate.Script->GetUsageId() == EventHandlerIdentity.Guids[0];
			});

			if(EventScriptPropertiesItem != nullptr)
			{
				UNiagaraStackEventScriptItemGroup* StackEventGroup = FindCurrentChildOfTypeByPredicate<UNiagaraStackEventScriptItemGroup>(CurrentChildren,
				[&](UNiagaraStackEventScriptItemGroup* CurrentEventProperties) { return CurrentEventProperties->GetScriptUsageId() == (*EventScriptPropertiesItem).Script->GetUsageId() && CurrentEventProperties->GetEventSourceEmitterId() == (*EventScriptPropertiesItem).SourceEmitterID; });
		
				if (StackEventGroup == nullptr)
				{
					StackEventGroup = NewObject<UNiagaraStackEventScriptItemGroup>(this);
					StackEventGroup->Initialize(CreateDefaultChildRequiredData(), GetEmitterViewModel()->GetSharedScriptViewModel(), ENiagaraScriptUsage::ParticleEventScript, (*EventScriptPropertiesItem).Script->GetUsageId(), (*EventScriptPropertiesItem).SourceEmitterID);
					StackEventGroup->SetOnModifiedEventHandlers(UNiagaraStackEventScriptItemGroup::FOnModifiedEventHandlers::CreateUObject(this, &UNiagaraStackEntry::RefreshChildren));
				}
			
				NewChildren.Add(StackEventGroup);	
			}
		}
		else if(UNiagaraHierarchyEventHandlerProperties* EventHandlerProperties = Cast<UNiagaraHierarchyEventHandlerProperties>(Data))
		{
			const TArray<FNiagaraEventScriptProperties>& EventScriptProperties = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetEventHandlers();
			const FNiagaraEventScriptProperties* EventScriptPropertiesItem = EventScriptProperties.FindByPredicate([EventHandlerIdentity = EventHandlerProperties->GetPersistentIdentity()](const FNiagaraEventScriptProperties& Candidate)
			{
				FHierarchyElementIdentity CandidateIdentity = UNiagaraHierarchyEventHandlerProperties::MakeIdentity(Candidate);
				return CandidateIdentity == EventHandlerIdentity;
			});

			if(EventScriptPropertiesItem != nullptr)
			{
				UNiagaraStackEventHandlerPropertiesItem* StackEventProperties = FindCurrentChildOfTypeByPredicate<UNiagaraStackEventHandlerPropertiesItem>(CurrentChildren,
				[&](UNiagaraStackEventHandlerPropertiesItem* CurrentEventProperties) { return CurrentEventProperties->GetEventScriptUsageId() == EventScriptPropertiesItem->Script->GetUsageId(); });
		
				if (StackEventProperties == nullptr)
				{
					StackEventProperties = NewObject<UNiagaraStackEventHandlerPropertiesItem>(this);
					StackEventProperties->Initialize(CreateDefaultChildRequiredData(), (*EventScriptPropertiesItem).Script->GetUsageId());
				}
			
				NewChildren.Add(StackEventProperties);	
			}
		}	
		else if(UNiagaraHierarchySimStage* SummarySimStage = Cast<UNiagaraHierarchySimStage>(Data))
		{
			TArray<UNiagaraSimulationStageBase*> SimStages = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetSimulationStages();
			UNiagaraSimulationStageBase** MatchingSimStage = SimStages.FindByPredicate([RendererIdentity = SummarySimStage->GetPersistentIdentity().Guids[0]](UNiagaraSimulationStageBase* Candidate)
			{
				return Candidate->GetMergeId() == RendererIdentity;
			});

			if(MatchingSimStage != nullptr)
			{
				UNiagaraStackSimulationStageGroup* SimStageGroup = FindCurrentChildOfTypeByPredicate<UNiagaraStackSimulationStageGroup>(CurrentChildren,
				[&](UNiagaraStackSimulationStageGroup* CurrentSimStageItem)
				{
					if(CurrentSimStageItem->GetSimulationStage())
					{
						return CurrentSimStageItem->GetSimulationStage()->GetMergeId() == (*MatchingSimStage)->GetMergeId();
					}

					return false;
				});
		
				if (SimStageGroup == nullptr)
				{
					SimStageGroup = NewObject<UNiagaraStackSimulationStageGroup>(this);
					FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(),
						FExecutionCategoryNames::Particle, FExecutionSubcategoryNames::SimulationStage,
						GetEmitterViewModel()->GetEditorData().GetStackEditorData());
					SimStageGroup->Initialize(RequiredEntryData, GetEmitterViewModel()->GetSharedScriptViewModel(), *MatchingSimStage);
					SimStageGroup->SetOnModifiedSimulationStages(UNiagaraStackSimulationStageGroup::FOnModifiedSimulationStages::CreateUObject(this, &UNiagaraStackEntry::RefreshChildren));
				}
			
				NewChildren.Add(SimStageGroup);	
			}
		}
		else if(UNiagaraHierarchySimStageProperties* SummarySimStageProperties = Cast<UNiagaraHierarchySimStageProperties>(Data))
		{
			TArray<UNiagaraSimulationStageBase*> SimStages = GetEmitterViewModel()->GetEmitter().GetEmitterData()->GetSimulationStages();
			UNiagaraSimulationStageBase** MatchingSimStage = SimStages.FindByPredicate([RendererIdentity = SummarySimStageProperties->GetPersistentIdentity().Guids[0]](UNiagaraSimulationStageBase* Candidate)
			{
				return Candidate->GetMergeId() == RendererIdentity;
			});

			if(MatchingSimStage != nullptr)
			{
				UNiagaraStackSimulationStagePropertiesItem* SimStageProperties = FindCurrentChildOfTypeByPredicate<UNiagaraStackSimulationStagePropertiesItem>(CurrentChildren,
				[&](UNiagaraStackSimulationStagePropertiesItem* CurrentSimStageItem)
				{
					if(CurrentSimStageItem->GetSimulationStage().IsValid())
					{
						return CurrentSimStageItem->GetSimulationStage()->GetMergeId() == (*MatchingSimStage)->GetMergeId();
					}

					return false;
				});
		
				if (SimStageProperties == nullptr)
				{
					SimStageProperties = NewObject<UNiagaraStackSimulationStagePropertiesItem>(this);
					SimStageProperties->Initialize(CreateDefaultChildRequiredData(), *MatchingSimStage);
				}
			
				NewChildren.Add(SimStageProperties);	
			}
		}
		else if(UHierarchyCategory* HierarchyCategory = Cast<UHierarchyCategory>(Data))
		{			
			TSharedPtr<FHierarchyCategoryViewModel> CategoryViewModel = StaticCastSharedPtr<FHierarchyCategoryViewModel>(HierarchyViewModel);
			
			UNiagaraStackSummaryCategory* StackCategory = FindCurrentChildOfTypeByPredicate<UNiagaraStackSummaryCategory>(CurrentChildren,
			[&](UNiagaraStackSummaryCategory* StackCategoryCandidate)
			{
				if(StackCategoryCandidate->GetHierarchyCategory().IsValid())
				{
					return StackCategoryCandidate->GetHierarchyCategory().Pin()->GetData() == CategoryViewModel->GetData();
				}

				return false;
			});
	
			if (StackCategory == nullptr)
			{
				StackCategory = NewObject<UNiagaraStackSummaryCategory>(this);
				StackCategory->Initialize(CreateDefaultChildRequiredData(), CategoryViewModel, GetOwnerStackItemEditorDataKey());
			}
		
			NewChildren.Add(StackCategory);	
		}
		else if(const UNiagaraHierarchyObjectProperty* ObjectProperty = Cast<UNiagaraHierarchyObjectProperty>(Data))
		{
			TMap<FGuid, UObject*> ObjectsForProperties = GetEmitterViewModel()->GetSummaryHierarchyViewModel()->GetObjectsForProperties();
			FGuid ObjectGuid = ObjectProperty->GetPersistentIdentity().Guids.Num() > 0 ? ObjectProperty->GetPersistentIdentity().Guids[0] : FGuid();
			
			if(ObjectsForProperties.Contains(ObjectGuid))
			{
				UObject* Object = ObjectsForProperties[ObjectProperty->GetPersistentIdentity().Guids[0]];

				UNiagaraStackObject* StackObjectWithProperty = FindCurrentChildOfTypeByPredicate<UNiagaraStackObject>(CurrentChildren,
				[&](UNiagaraStackObject* StackObjectCandidate)
				{
					return StackObjectCandidate->GetObject() == Object && StackObjectCandidate->GetCustomName() == ObjectProperty->GetPersistentIdentity().Names[0];
				});

				if(StackObjectWithProperty == nullptr)
				{
					StackObjectWithProperty = NewObject<UNiagaraStackObject>(this);
					bool bIsInTopLevelObject = false;
					bool bHideTopLevelCategories = false;
					StackObjectWithProperty->Initialize(CreateDefaultChildRequiredData(), Object, bIsInTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey(), nullptr);
					StackObjectWithProperty->SetCustomName(ObjectProperty->GetPersistentIdentity().Names[0]);
					StackObjectWithProperty->SetOnFilterDetailNodes(FNiagaraStackObjectShared::FOnFilterDetailNodes::CreateLambda([PropertyName = ObjectProperty->GetPersistentIdentity().Names[0]](const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes)
					{
						for(const TSharedRef<IDetailTreeNode>& SourceNode : InSourceNodes)
						{
							TArray<TSharedRef<IDetailTreeNode>> ChildrenNodes;
							SourceNode->GetChildren(ChildrenNodes);
							
							for(TSharedRef<IDetailTreeNode> ChildNode : ChildrenNodes)
							{
								if(ChildNode->GetNodeName() == PropertyName)
								{
									OutFilteredNodes.Add(ChildNode);
								}
							}
						}
					}));
				}
				
				NewChildren.Add(StackObjectWithProperty);
			}
		}
	}
	
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);

	if (NewChildren.Num() == 0)
	{
		FText EmptyAssignmentNodeMessageText = LOCTEXT("EmptySummaryNodeMessage", "No Parameters in Emitter Summary.\n\nYou can click on the 'Edit' button in the top right to summon the Summary View Editor. It lets you add categories, modules or individual module inputs as well as renderer properties via drag & drop.");
		UNiagaraStackItemTextContent* EmtpySummaryMessage = FindCurrentChildOfTypeByPredicate<UNiagaraStackItemTextContent>(CurrentChildren,
			[&](UNiagaraStackItemTextContent* CurrentStackItemTextContent) { return CurrentStackItemTextContent->GetDisplayName().IdenticalTo(EmptyAssignmentNodeMessageText); });
		
		if (EmtpySummaryMessage == nullptr)
		{
			EmtpySummaryMessage = NewObject<UNiagaraStackItemTextContent>(this);
			EmtpySummaryMessage->Initialize(CreateDefaultChildRequiredData(), EmptyAssignmentNodeMessageText, GetStackEditorDataKey());
		}
		
		EmtpySummaryMessage->SetIsHidden(!GetShouldShowInStack());
		
		NewChildren.Add(EmtpySummaryMessage);	
	}
}

void UNiagaraStackSummaryViewCollection::GetSectionsInternal(TArray<FNiagaraStackSection>& OutStackSections) const
{
	TConstArrayView<const TObjectPtr<UHierarchySection>> HierarchySections = GetHierarchySections();

	TMap<const UHierarchySection*, TArray<UHierarchyCategory*>> SectionToCategoryMap;

	TSharedPtr<FNiagaraEmitterViewModel> ViewModel = GetEmitterViewModel();
	const UHierarchyRoot* Root = ViewModel->GetEditorData().GetSummaryRoot();
	
	TArray<UHierarchyCategory*> HierarchyCategories;
	Root->GetChildrenOfType<UHierarchyCategory>(HierarchyCategories, false);
	
	for(UHierarchyCategory* TopLevelHierarchyCategories : HierarchyCategories)
	{
		if(const FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = TopLevelHierarchyCategories->FindMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>())
		{
			if(SectionAssociation->Section.IsValid())
			{
				SectionToCategoryMap.FindOrAdd(SectionAssociation->Section.Get()).Add(TopLevelHierarchyCategories);	
			}
		}
	}
	
	for(const UHierarchySection* HierarchySection : HierarchySections)
	{
		FNiagaraStackSection Section;
		Section.SectionDisplayName = HierarchySection->GetSectionNameAsText();
		Section.Tooltip = HierarchySection->GetTooltip();
		
		if(SectionToCategoryMap.Contains(HierarchySection))
		{
			for(UHierarchyCategory* Category : SectionToCategoryMap[HierarchySection])
			{
				Section.Categories.Add(FText::FromString(Category->ToString()));
			}
		}
		
		OutStackSections.Add(Section);
	}
}

TConstArrayView<const TObjectPtr<UHierarchySection>> UNiagaraStackSummaryViewCollection::GetHierarchySections() const
{
	if (GetEmitterViewModel().IsValid())
	{
		return GetEmitterViewModel()->GetEditorData().GetSummarySections();
	}

	static TArray<const TObjectPtr<UHierarchySection>> Dummy;
	return Dummy;
}

bool UNiagaraStackSummaryViewCollection::FilterByActiveSection(const UNiagaraStackEntry& Child) const
{
	bool bIsAllowed = Super::FilterByActiveSection(Child);

	// we additionally filter out stack function inputs directly under the root if we are not in the 'All' section
	FText ActiveSection = GetActiveSection();
	if (ActiveSection.IdenticalTo(AllSectionName) == false && Child.IsA<UNiagaraStackFunctionInput>())
	{
		bIsAllowed = false;
	}
	
	return bIsAllowed;
}

bool UNiagaraStackSummaryViewCollection::FilterOnlyModified(const UNiagaraStackEntry& Child) const
{
	if(GetStackEditorData().GetShowOnlyModified() == false)
	{
		return true;
	}
	
	const UNiagaraStackFunctionInput* FunctionInput = Cast<UNiagaraStackFunctionInput>(&Child);
	if (FunctionInput == nullptr || FunctionInput->CanReset() || FunctionInput->HasAnyResettableChildrenInputs())
	{
		return true;
	}

	return false;
}

void UNiagaraStackSummaryViewCollection::OnViewStateChanged()
{
	if (!IsFinalized())
	{
		CacheLastActiveSection();
		UpdateCachedSectionData();
		RefreshChildren();
	}
}

void UNiagaraStackSummaryViewCollection::OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload)
{
	// Deferred to the next tick because OnHierarchyChangedDelegate is broadcast synchronously from inside the hierarchy
	// view model's mutation paths (e.g. DeleteChild). Calling RefreshChildren() inline would re-enter SyncViewModelsToData
	// on the root while the outer walk is still iterating, corrupting Children. Same-scope defer keeps the work local to
	// this collection rather than escalating to a full stack refresh.
	if (IsFinalized() || GEditor == nullptr)
	{
		return;
	}

	TWeakObjectPtr<UNiagaraStackSummaryViewCollection> WeakThis(this);
	GEditor->GetTimerManager()->SetTimerForNextTick([WeakThis]()
	{
		if (UNiagaraStackSummaryViewCollection* StrongThis = WeakThis.Get())
		{
			if (!StrongThis->IsFinalized())
			{
				StrongThis->OnViewStateChanged();
			}
		}
	});
}

#undef LOCTEXT_NAMESPACE
