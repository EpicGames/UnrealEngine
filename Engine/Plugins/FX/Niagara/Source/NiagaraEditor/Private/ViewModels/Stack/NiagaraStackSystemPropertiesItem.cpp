// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackSystemPropertiesItem.h"
#include "NiagaraEmitter.h"
#include "NiagaraSettings.h"
#include "ViewModels/Stack/NiagaraStackObject.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemDetailsCustomization.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ScopedTransaction.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackSystemPropertiesItem)

#define LOCTEXT_NAMESPACE "UNiagaraStackSystemItemGroup"

void UNiagaraStackSystemPropertiesItem::Initialize(FRequiredEntryData InRequiredEntryData)
{
	Super::Initialize(InRequiredEntryData, TEXT("SystemProperties"));
	System = &GetSystemViewModel()->GetSystem();
}

FText UNiagaraStackSystemPropertiesItem::GetDisplayName() const
{
	return LOCTEXT("SystemPropertiesName", "System Properties");
}

FText UNiagaraStackSystemPropertiesItem::GetTooltipText() const
{
	return LOCTEXT("SystemPropertiesTooltip", "Properties of the System. These cannot change at runtime.");
}

void UNiagaraStackSystemPropertiesItem::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	if (SystemObject == nullptr)
	{
		SystemObject = NewObject<UNiagaraStackObject>(this);
		FRequiredEntryData RequiredEntryData(GetSystemViewModel(), GetEmitterViewModel(), FExecutionCategoryNames::System, NAME_None, GetStackEditorData());
		bool bIsTopLevelObject = true;
		bool bHideTopLevelCategories = false;
		SystemObject->Initialize(RequiredEntryData, System.Get(), bIsTopLevelObject, bHideTopLevelCategories, GetStackEditorDataKey());
		SystemObject->RegisterInstancedCustomPropertyLayout(UNiagaraSystem::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraSystemDetails::MakeInstance));
	}
	NewChildren.Add(SystemObject);
	bCanResetToBase.Reset();

	//Check if we're trying to override scalability settings without an EffectType. Ideally we can allow this but it's somewhat awkward so for now we just post a warning and ignore this.
	UNiagaraSystem* SystemPtr = System.Get();
	TWeakPtr<FNiagaraSystemViewModel> WeakSysViewModel = GetSystemViewModel();
	if (SystemPtr && SystemPtr->GetOverrideScalabilitySettings() && SystemPtr->GetEffectType() == nullptr)
	{
		FText FixDescription = LOCTEXT("FixOverridesWithNoEffectType", "Disable Overrides");
		FStackIssueFix FixIssue(
			FixDescription,
			FStackIssueFixDelegate::CreateLambda([=]()
				{
					if (auto Pinned = WeakSysViewModel.Pin())
					{
						FScopedTransaction ScopedTransaction(FixDescription);
						Pinned->GetSystem().Modify();
						Pinned->GetSystem().SetOverrideScalabilitySettings(false);
						Pinned->RefreshAll();
					}
				}));

		FStackIssue OverridesWithNoEffectTypeWarning(
			EStackIssueSeverity::Warning,
			LOCTEXT("FixOverridesWithNoEffectTypeSummaryText", "Scalability overrides with no Effect Type."),
			LOCTEXT("FixOverridesWithNoEffectTypeErrorText", "Scalability settings cannot be overriden if the System has no Effect Type."),
			GetStackEditorDataKey(),
			false,
			FixIssue);

		NewIssues.Add(OverridesWithNoEffectTypeWarning);
	}

	//Check emitters don't have a max delta time that exceeds the warmup delta time.
	if (System->NeedsWarmup())
	{
		float WarmupDelta = System->GetWarmupTickDelta();
		const UNiagaraSettings* NiagaraSettings = GetDefault<UNiagaraSettings>();
		for (FNiagaraEmitterHandle& EmitterHandle : System->GetEmitterHandles())
		{
			if (NiagaraSettings->bLimitDeltaTime && NiagaraSettings->MaxDeltaTimePerTick < WarmupDelta)
			{
				TArray<FStackIssueFix> Fixes;

				float MaxEmitterDt = NiagaraSettings->MaxDeltaTimePerTick;
				//This emitter does not allow ticks with a delta time so large.
				FText FixDescriptionReduceWarmupDt = LOCTEXT("FixWarmupDeltaTime", "Reduce System Warmup Delta Time");
				Fixes.Emplace(
					FixDescriptionReduceWarmupDt,
					FStackIssueFixDelegate::CreateLambda([=]()
						{
							if (auto Pinned = WeakSysViewModel.Pin())
							{
								FScopedTransaction ScopedTransaction(FixDescriptionReduceWarmupDt);
								Pinned->GetSystem().Modify();
								Pinned->GetSystem().SetWarmupTickDelta(MaxEmitterDt);
								Pinned->RefreshAll();
							}
						}));

				FStackIssue WarmupDeltaTimeExceedsEmitterDeltaTimeWarning(
					EStackIssueSeverity::Warning,
					LOCTEXT("WarmupDeltaTimeExceedsEmitterDeltaTimeWarningSummary", "System Warmup Delta Time Exceeds Emitter Max Delta Time."),
					FText::Format(LOCTEXT("WarmupDeltaTimeExceedsEmitterDeltaTimeWarningText", "Emitter {0} has a Max Tick Delta Time smaller than the System's Warmup Delta Time. This could cause unintended results during warmup for this emitter.\nThe max tick delta time can be changed in the Niagara setting."), FText::FromString(EmitterHandle.GetUniqueInstanceName())),
					GetStackEditorDataKey(),
					false,
					Fixes);

				NewIssues.Add(WarmupDeltaTimeExceedsEmitterDeltaTimeWarning);
			}
		}
	}

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

void UNiagaraStackSystemPropertiesItem::SystemPropertiesChanged()
{
	bCanResetToBase.Reset();

	GetSystemViewModel()->GetSystemStackViewModel()->RequestValidationUpdate();
	for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel : GetSystemViewModel()->GetEmitterHandleViewModels())
	{
		EmitterHandleViewModel->GetEmitterStackViewModel()->RequestValidationUpdate();
	}
}

#undef LOCTEXT_NAMESPACE

