// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackSystemSettingsGroup.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "NiagaraNodeFunctionCall.h"
#include "ViewModels/Stack/NiagaraStackErrorItem.h"
#include "Internationalization/Internationalization.h"
#include "NiagaraStackEditorData.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackSystemPropertiesItem.h"
#include "NiagaraConstants.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "NiagaraEditorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackSystemSettingsGroup)


#define LOCTEXT_NAMESPACE "UNiagaraStackParameterStoreGroup"

class FParameterStoreGroupAddAction : public INiagaraStackItemGroupAddAction
{
public:
	FParameterStoreGroupAddAction(FNiagaraVariable InNewParameterVariable)
		: NewParameterVariable(InNewParameterVariable)
	{
	}

	FNiagaraVariable GetNewParameterVariable() const
	{
		return NewParameterVariable;
	}

	virtual TArray<FString> GetCategories() const override
	{
		return {FNiagaraEditorUtilities::GetVariableTypeCategory(GetNewParameterVariable()).ToString()};
	}

	virtual FText GetDisplayName() const override
	{
		return NewParameterVariable.GetType().GetNameText();
	}

	virtual FText GetDescription() const override
	{
		return FText::Format(LOCTEXT("AddParameterActionDescriptionFormat", "Create a new {0} parameter."), GetDisplayName());
	}

	virtual FText GetKeywords() const override
	{
		return FText();
	}

	virtual TOptional<FNiagaraFavoritesActionData> GetFavoritesData() const override
	{
		return TOptional<FNiagaraFavoritesActionData>();
	}
	
	virtual bool IsInLibrary() const override
	{
		return true;
	}
	
	virtual FNiagaraActionSourceData GetSourceData() const override
	{
		return FNiagaraActionSourceData();
	}

	virtual TOptional<FSoftObjectPath> GetPreviewMoviePath() const override
	{
		return {};
	}

private:
	FNiagaraVariable NewParameterVariable;
};

class FParameterStoreGroupAddUtiliites : public TNiagaraStackItemGroupAddUtilities<FNiagaraVariable>
{
public:
	FParameterStoreGroupAddUtiliites(UObject& InParameterStoreOwner, FNiagaraParameterStore& InParameterStore, UNiagaraStackEditorData& InStackEditorData, FOnItemAdded InOnItemAdded)
		: TNiagaraStackItemGroupAddUtilities(LOCTEXT("ScriptGroupAddItemName", "Parameter"), EAddMode::AddFromAction, true, false, InOnItemAdded)
		, ParameterStoreOwner(InParameterStoreOwner)
		, ParameterStore(InParameterStore)
		, StackEditorData(InStackEditorData)
	{
	}

	virtual void AddItemDirectly() override { unimplemented(); }

	virtual void GenerateAddActions(TArray<TSharedRef<INiagaraStackItemGroupAddAction>>& OutAddActions, const FNiagaraStackItemGroupAddOptions& AddProperties) const override
	{
		TArray<FNiagaraTypeDefinition> AvailableTypes;
		FNiagaraStackGraphUtilities::GetNewParameterAvailableTypes(AvailableTypes, FNiagaraConstants::UserNamespace);
		for (const FNiagaraTypeDefinition& AvailableType : AvailableTypes)
		{
			FNiagaraParameterHandle NewParameterHandle(FNiagaraConstants::UserNamespace, *(TEXT("New") + AvailableType.GetName()));
			FNiagaraVariable NewParameterVariable(AvailableType, NewParameterHandle.GetParameterHandleString());
			OutAddActions.Add(MakeShared<FParameterStoreGroupAddAction>(NewParameterVariable));
		}
	}

	virtual void ExecuteAddAction(TSharedRef<INiagaraStackItemGroupAddAction> AddAction, int32 TargetIndex) override
	{
		TSharedRef<FParameterStoreGroupAddAction> ParameterAddAction = StaticCastSharedRef<FParameterStoreGroupAddAction>(AddAction);
		FNiagaraVariable NewParameterVariable = ParameterAddAction->GetNewParameterVariable();
		bool bSuccess = FNiagaraEditorUtilities::AddParameter(NewParameterVariable, ParameterStore, ParameterStoreOwner, &StackEditorData);
		if (bSuccess)
		{
			OnItemAdded.ExecuteIfBound(NewParameterVariable);
		}
	}

private:
	UObject& ParameterStoreOwner;
	FNiagaraParameterStore& ParameterStore;
	UNiagaraStackEditorData& StackEditorData;
};

void UNiagaraStackSystemPropertiesGroup::Initialize(FRequiredEntryData InRequiredEntryData)
{
	FText DisplayName = LOCTEXT("SystemPropertiesGroupName", "Properties");
	FText Tooltip = LOCTEXT("SystemSettingsTooltip", "Properties set for the entire system.");
	Super::Initialize(InRequiredEntryData, DisplayName, Tooltip, nullptr);
}

const FSlateBrush* UNiagaraStackSystemPropertiesGroup::GetIconBrush() const
{
	return FAppStyle::Get().GetBrush("Icons.Details");
}

void UNiagaraStackSystemPropertiesGroup::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	UNiagaraStackSystemPropertiesItem* SystemPropertiesItem = FindCurrentChildOfTypeByPredicate<UNiagaraStackSystemPropertiesItem>(CurrentChildren,
		[=](UNiagaraStackSystemPropertiesItem* CurrentItem) { return true; });

	if (SystemPropertiesItem == nullptr)
	{
		SystemPropertiesItem = NewObject<UNiagaraStackSystemPropertiesItem>(this);
		SystemPropertiesItem->Initialize(CreateDefaultChildRequiredData());
	}
	NewChildren.Add(SystemPropertiesItem);

	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
}

void UNiagaraStackSystemUserParametersGroup::Initialize(
	FRequiredEntryData InRequiredEntryData,
	UObject* InOwner,
	FNiagaraParameterStore* InParameterStore)
{
	FText DisplayName = LOCTEXT("SystemUserParametersGroupName", "User Parameters");
	FText Tooltip = LOCTEXT("SystemUserParametersTooltip", "Brings the User Parameters tab to the front. This button will go away with the next release.\nThe User Parameters tab allows you to set default values and hierarchies for user parameters.\nUser Parameters are exposed to systems outside Niagara (like Blueprints) and can be dynamically changed at runtime.\n");
	Super::Initialize(InRequiredEntryData, DisplayName, Tooltip, nullptr);

	Owner = InOwner;
}

FText UNiagaraStackSystemUserParametersGroup::GetIconText() const
{
	return FText::FromString(FString(TEXT("\xf007")/* fa-user */));
}

#undef LOCTEXT_NAMESPACE

