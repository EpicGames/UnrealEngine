// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackValueCollection.h"

#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeFunctionCall.h"
#include "ScopedTransaction.h"
#include "EdGraph/EdGraphPin.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"
#include "NiagaraEmitterEditorData.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "NiagaraStackEditorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackValueCollection)

#define LOCTEXT_NAMESPACE "UNiagaraStackFunctionInputCollection"

FText UNiagaraStackValueCollection::UncategorizedName = LOCTEXT("Uncategorized", "Uncategorized");

FText UNiagaraStackValueCollection::AllSectionName = LOCTEXT("All", "All");

using namespace FNiagaraStackGraphUtilities;

static FText GetUserFriendlyFunctionName(UNiagaraNodeFunctionCall* Node)
{
	if (Node->IsA<UNiagaraNodeAssignment>())
	{
		// The function name of assignment nodes contains a guid, which is just confusing for the user to see 
		return LOCTEXT("AssignmentNodeName", "SetVariables");
	}
	return FText::FromString(Node->GetFunctionName());
}

void UNiagaraStackValueCollection::Initialize(FRequiredEntryData InRequiredEntryData, FString InOwningStackItemEditorDataKey, FString InStackEditorDataKey)
{
	Super::Initialize(InRequiredEntryData, InOwningStackItemEditorDataKey, InStackEditorDataKey);
	AddChildFilter(FOnFilterChild::CreateUObject(this, &UNiagaraStackValueCollection::FilterByActiveSection));
	ActiveSectionCache = GetStackEditorData().GetStackEntryActiveSection(GetStackEditorDataKey(), AllSectionName);
}

void UNiagaraStackValueCollection::SetShouldDisplayLabel(bool bInShouldDisplayLabel)
{
	bShouldDisplayLabel = bInShouldDisplayLabel;
}

const TArray<FText>& UNiagaraStackValueCollection::GetSections() const
{
	if (SectionsCache.IsSet() == false)
	{
		UpdateCachedSectionData();
	}
	return SectionsCache.GetValue();
}

FText UNiagaraStackValueCollection::GetActiveSection() const
{
	if (ActiveSectionCache.IsSet() == false)
	{
		UpdateCachedSectionData();
	}
	return ActiveSectionCache.GetValue();
}

void UNiagaraStackValueCollection::SetActiveSection(FText InActiveSection)
{
	ActiveSectionCache = InActiveSection;
	GetStackEditorData().SetStackEntryActiveSection(GetStackEditorDataKey(), InActiveSection);
	RefreshFilteredChildren();
}

FText UNiagaraStackValueCollection::GetTooltipForSection(FString Section) const
{
	if(SectionToTooltipMapCache.IsSet() && SectionToTooltipMapCache->Contains(Section))
	{
		return SectionToTooltipMapCache.GetValue()[Section];
	}

	return FText::GetEmpty();
}

void UNiagaraStackValueCollection::CacheLastActiveSection()
{
	if(ActiveSectionCache.IsSet())
	{
		LastActiveSection = ActiveSectionCache.GetValue();
	}
}

bool UNiagaraStackValueCollection::GetCanExpand() const
{
	return bShouldDisplayLabel;
}

bool UNiagaraStackValueCollection::GetShouldShowInStack() const
{
	return GetSections().Num() > 0 || bShouldDisplayLabel;
}

void UNiagaraStackValueCollection::RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues)
{
	Super::RefreshChildrenInternal(CurrentChildren, NewChildren, NewIssues);
	LastActiveSection = ActiveSectionCache.IsSet() ? ActiveSectionCache.GetValue() : AllSectionName;
	SectionsCache.Reset();
	SectionToCategoryMapCache.Reset();
	ActiveSectionCache.Reset();
}

int32 UNiagaraStackValueCollection::GetChildIndentLevel() const
{
	// We don't want the child categories to be indented.
	return GetIndentLevel();
}

bool UNiagaraStackValueCollection::FilterByActiveSection(const UNiagaraStackEntry& Child) const
{
	const TArray<FText>& Sections = GetSections();
	FText ActiveSection = GetActiveSection();
	if (Sections.Num() == 0 || ActiveSection.IdenticalTo(AllSectionName) || SectionToCategoryMapCache.IsSet() == false)
	{
		return true;
	}

	TArray<FText>* ActiveCategoryNames = SectionToCategoryMapCache->Find(ActiveSection.ToString());
	const UNiagaraStackCategory* ChildCategory = Cast<UNiagaraStackCategory>(&Child);
	return ChildCategory == nullptr || ActiveCategoryNames == nullptr || ActiveCategoryNames->ContainsByPredicate(
		[ChildCategory](const FText& ActiveCategoryName) { return ActiveCategoryName.EqualTo(ChildCategory->GetDisplayName()); });
}

void UNiagaraStackValueCollection::UpdateCachedSectionData() const
{
	TArray<FText> Sections;
	TMap<FString, TArray<FText>> SectionToCategoryMap;
	TMap<FString, FText> SectionToTooltipMap;
	TArray<FNiagaraStackSection> StackSections;
	GetSectionsInternal(StackSections);

	if (StackSections.Num() > 0)
	{
		// Get the current list of categories.
		TArray<FText> CategoryNames;
		TArray<UNiagaraStackCategory*> ChildCategories;
		GetUnfilteredChildrenOfType(ChildCategories);
		
		for (UNiagaraStackCategory* ChildCategory : ChildCategories)
		{
			if (ChildCategory->GetShouldShowInStack())
			{
				CategoryNames.Add(ChildCategory->GetDisplayName());
			}
		}

		// Match sections to valid categories.
		for (const FNiagaraStackSection& StackSection : StackSections)
		{
			TArray<FText> ContainedCategories;
			for (FText SectionCategory : StackSection.Categories)
			{
				if (CategoryNames.ContainsByPredicate([SectionCategory](FText CategoryName) { return CategoryName.EqualTo(SectionCategory); }))
				{
					ContainedCategories.Add(SectionCategory);
				}
			}
			if (ContainedCategories.Num() > 0)
			{
				Sections.Add(StackSection.SectionDisplayName);
				SectionToCategoryMap.Add(StackSection.SectionDisplayName.ToString(), ContainedCategories);
			}

			SectionToTooltipMap.Add(StackSection.SectionDisplayName.ToString(), StackSection.Tooltip);
		}

		Sections.Add(AllSectionName);
		SectionToCategoryMap.Add(AllSectionName.ToString(), CategoryNames);
		
		if (Sections.Num() == 1)
		{
			// If there is only one section, it's the "All" section which is not useful.
			Sections.Empty();
			SectionToCategoryMap.Empty();
			SectionToTooltipMap.Empty();
		}
	}

	SectionsCache = Sections;
	SectionToCategoryMapCache = SectionToCategoryMap;
	SectionToTooltipMapCache = SectionToTooltipMap;
	FText LastActiveSectionLocal = LastActiveSection;
	if (Sections.ContainsByPredicate([LastActiveSectionLocal](FText Section) { return Section.EqualTo(LastActiveSectionLocal); }))
	{
		ActiveSectionCache = LastActiveSection;
	}
	else
	{
		ActiveSectionCache = AllSectionName;
	}
}

#undef LOCTEXT_NAMESPACE
