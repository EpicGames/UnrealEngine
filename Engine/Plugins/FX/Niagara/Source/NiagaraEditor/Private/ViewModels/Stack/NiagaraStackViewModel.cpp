// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/Stack/NiagaraStackViewModel.h"

#include "NiagaraEmitterEditorData.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraValidationRules.h"
#include "ScopedTransaction.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraStackViewModel)

#define LOCTEXT_NAMESPACE "NiagaraStackViewModel"
const double UNiagaraStackViewModel::MaxSearchTime = .02f; // search at 50 fps

UNiagaraStackViewModel::FTopLevelViewModel::FTopLevelViewModel(TSharedPtr<FNiagaraSystemViewModel> InSystemViewModel)
	: SystemViewModel(InSystemViewModel)
	, RootEntry(InSystemViewModel->GetSystemStackViewModel()->GetRootEntry())
{
}

UNiagaraStackViewModel::FTopLevelViewModel::FTopLevelViewModel(TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel)
	: EmitterHandleViewModel(InEmitterHandleViewModel)
	, RootEntry(InEmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry())
{
}

bool UNiagaraStackViewModel::FTopLevelViewModel::IsValid() const
{
	return (SystemViewModel.IsValid() && EmitterHandleViewModel.IsValid() == false) ||
		(SystemViewModel.IsValid() == false && EmitterHandleViewModel.IsValid());
}

UNiagaraStackEditorData* UNiagaraStackViewModel::FTopLevelViewModel::GetStackEditorData() const
{
	if (SystemViewModel.IsValid())
	{
		return &SystemViewModel->GetEditorData().GetStackEditorData();
	}
	else if (EmitterHandleViewModel.IsValid())
	{
		return &EmitterHandleViewModel->GetEmitterViewModel()->GetEditorData().GetStackEditorData();
	}
	else
	{
		return nullptr;
	}
}

void UNiagaraStackViewModel::FTopLevelViewModel::GetMessageStores(TArray<FNiagaraMessageSourceAndStore>& OutMessageStores)
{
	if (SystemViewModel.IsValid())
	{
		SystemViewModel->GetSystemMessageStores(OutMessageStores);
	}
	else if (EmitterHandleViewModel.IsValid())
	{
		EmitterHandleViewModel->GetEmitterViewModel()->GetEmitterMessageStores(OutMessageStores);
	}
}

FText UNiagaraStackViewModel::FTopLevelViewModel::GetDisplayName() const
{
	if (SystemViewModel.IsValid())
	{
		return SystemViewModel->GetDisplayName();
	}
	else if (EmitterHandleViewModel.IsValid())
	{
		return EmitterHandleViewModel->GetNameText();
	}
	else
	{
		return FText();
	}
}

bool UNiagaraStackViewModel::FTopLevelViewModel::operator==(const FTopLevelViewModel& Other) const
{
	return Other.SystemViewModel == SystemViewModel && Other.EmitterHandleViewModel == EmitterHandleViewModel && Other.RootEntry == RootEntry;
}

void UNiagaraStackViewModel::UpdateStackWithValidationResults()
{
	bValidatorUpdatePending = false;

	// clear out old stack entries
	TArray<UNiagaraStackEntry*> EntriesToClear;
	RootEntry->GetUnfilteredChildren(EntriesToClear);
	while (EntriesToClear.Num() > 0)
	{
		UNiagaraStackEntry* Entry = EntriesToClear.Pop();
		Entry->ClearExternalIssues();
		Entry->GetUnfilteredChildren(EntriesToClear);
	}
	
	TSharedPtr<FNiagaraSystemViewModel> SysViewModel = SystemViewModel.Pin();
	NiagaraValidation::ValidateAllRulesInSystem(SysViewModel, [](const FNiagaraValidationResult& Result)
	{
		EStackIssueSeverity Severity = (Result.Severity == ENiagaraValidationSeverity::Error) ? EStackIssueSeverity::Error : (Result.Severity == ENiagaraValidationSeverity::Warning ? EStackIssueSeverity::Warning : EStackIssueSeverity::Info);
		if (UNiagaraStackEntry* SourceEntry = Cast<UNiagaraStackEntry>(Result.SourceObject.Get()))
		{
			SourceEntry->AddValidationIssue(Severity, Result.SummaryText, Result.Description, Result.Severity == ENiagaraValidationSeverity::Info, Result.Fixes, Result.Links);
		}
	});
}

void UNiagaraStackViewModel::InitializeWithViewModels(TSharedPtr<FNiagaraSystemViewModel> InSystemViewModel, TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel, FNiagaraStackViewModelOptions InOptions)
{
	Reset();
	Options = InOptions;
	if (InSystemViewModel.IsValid())
	{
		TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel;
		if (InEmitterHandleViewModel.IsValid())
		{
			EmitterViewModel = InEmitterHandleViewModel->GetEmitterViewModel();
			if (EmitterViewModel.IsValid())
			{
				EmitterViewModel->OnParentRemoved().AddUObject(this, &UNiagaraStackViewModel::EmitterParentRemoved);

				if (UNiagaraEmitter* Emitter = EmitterViewModel->GetEmitter().Emitter)
				{
					EmitterViewModel->GetEditorData().OnSummaryViewStateChanged().AddUObject(this, &UNiagaraStackViewModel::RequestRefreshDeferred);
				}
			}
			InEmitterHandleViewModel->GetEmitterHandle()->OnEmitterModeChanged().AddUObject(this, &UNiagaraStackViewModel::RequestRefreshDeferred);
		}
		else
		{
			EmitterViewModel = TSharedPtr<FNiagaraEmitterViewModel>();
		}
		
		UNiagaraStackRoot* StackRoot = NewObject<UNiagaraStackRoot>(this);
		UNiagaraStackEntry::FRequiredEntryData RequiredEntryData(InSystemViewModel.ToSharedRef(), EmitterViewModel,
			UNiagaraStackEntry::FExecutionCategoryNames::System, UNiagaraStackEntry::FExecutionSubcategoryNames::Settings,
			InSystemViewModel->GetEditorData().GetStackEditorData());
		StackRoot->Initialize(RequiredEntryData, Options.GetIncludeSystemInformation(), Options.GetIncludeEmitterInformation());
		StackRoot->RefreshChildren();
		StackRoot->OnStructureChanged().AddUObject(this, &UNiagaraStackViewModel::EntryStructureChanged);
		StackRoot->OnExpansionChanged().AddUObject(this, &UNiagaraStackViewModel::EntryExpansionChanged);
		StackRoot->OnExpansionInOverviewChanged().AddUObject(this, &UNiagaraStackViewModel::EntryExpansionInOverviewChanged);
		StackRoot->OnDataObjectModified().AddUObject(this, &UNiagaraStackViewModel::EntryDataObjectModified);
		StackRoot->OnRequestFullRefresh().AddUObject(this, &UNiagaraStackViewModel::EntryRequestFullRefresh);
		StackRoot->OnRequestFullRefreshDeferred().AddUObject(this, &UNiagaraStackViewModel::RequestRefreshDeferred);
		RootEntry = StackRoot;
		RootEntries.Add(RootEntry);

		bExternalRootEntry = false;

		UpdateStackWithValidationResults();
	}

	SystemViewModel = InSystemViewModel;
	EmitterHandleViewModel = InEmitterHandleViewModel;

	StructureChangedDelegate.Broadcast(ENiagaraStructureChangedFlags::StructureChanged);
}

void UNiagaraStackViewModel::InitializeWithRootEntry(UNiagaraStackEntry* InRootEntry)
{
	Reset();
	bUsesTopLevelViewModels = true;

	RootEntry = InRootEntry;
	RootEntry->OnStructureChanged().AddUObject(this, &UNiagaraStackViewModel::EntryStructureChanged);
	RootEntry->OnExpansionChanged().AddUObject(this, &UNiagaraStackViewModel::EntryExpansionChanged);
	RootEntry->OnExpansionInOverviewChanged().AddUObject(this, &UNiagaraStackViewModel::EntryExpansionInOverviewChanged);
	RootEntry->OnRequestFullRefresh().AddUObject(this, &UNiagaraStackViewModel::EntryRequestFullRefresh);
	RootEntry->OnRequestFullRefreshDeferred().AddUObject(this, &UNiagaraStackViewModel::RequestRefreshDeferred);
	RootEntries.Add(RootEntry);

	bExternalRootEntry = true;

	StructureChangedDelegate.Broadcast(ENiagaraStructureChangedFlags::StructureChanged);

	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelPinned = EmitterHandleViewModel.Pin();
	if (EmitterHandleViewModelPinned.IsValid())
	{
		TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = EmitterHandleViewModelPinned->GetEmitterViewModel();
		EmitterViewModel->GetEditorData().OnSummaryViewStateChanged().RemoveAll(this);
	}
}

void UNiagaraStackViewModel::Reset()
{
	if (RootEntry != nullptr)
	{
		RootEntry->OnExpansionChanged().RemoveAll(this);
		RootEntry->OnExpansionInOverviewChanged().RemoveAll(this);
		RootEntry->OnStructureChanged().RemoveAll(this);
		RootEntry->OnDataObjectModified().RemoveAll(this);
		RootEntry->OnRequestFullRefresh().RemoveAll(this);
		RootEntry->OnRequestFullRefreshDeferred().RemoveAll(this);
		if (bExternalRootEntry == false)
		{
			RootEntry->Finalize();
		}
		RootEntry = nullptr;
	}
	RootEntries.Empty();

	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModelPinned = EmitterHandleViewModel.Pin();
	if (EmitterHandleViewModelPinned.IsValid())
	{
		TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel = EmitterHandleViewModelPinned->GetEmitterViewModel();
		if (EmitterViewModel.IsValid())
		{
			EmitterViewModel->OnParentRemoved().RemoveAll(this);
			if (EmitterViewModel->GetEmitter().GetEmitterData())
			{
				EmitterViewModel->GetEditorData().OnSummaryViewStateChanged().RemoveAll(this);
			}
		}

		// Note we can not use EmitterHandleViewModelPinned->GetEmitterHandle() as it will be invalid on undo
		if (SystemViewModel.IsValid())
		{
			UNiagaraSystem& NiagaraSystem = SystemViewModel.Pin()->GetSystem();
			for (FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem.GetEmitterHandles())
			{
				EmitterHandle.OnEmitterModeChanged().RemoveAll(this);
			}
		}
	}

	EmitterHandleViewModel.Reset();
	SystemViewModel.Reset();

	TopLevelViewModels.Empty();

	CurrentIssueCycleIndex = -1;
	FocusedSearchResultCache.Reset();
	bRestartSearch = false;
	bRefreshPending = false;
	bValidatorUpdatePending = false;
	bUsesTopLevelViewModels = false;
}

bool UNiagaraStackViewModel::HasIssues() const
{
	return bHasIssues;
}

void UNiagaraStackViewModel::InvalidateCachedParameterUsage()
{
	if (RootEntry)
		RootEntry->InvalidateCollectedUsage();

}

void UNiagaraStackViewModel::Finalize()
{
	Reset();
}

void UNiagaraStackViewModel::BeginDestroy()
{
	checkf(HasAnyFlags(RF_ClassDefaultObject) || (SystemViewModel.IsValid() == false && EmitterHandleViewModel.IsValid() == false), TEXT("Stack view model not finalized."));
	Super::BeginDestroy();
}

void UNiagaraStackViewModel::Tick()
{
	if (RootEntry)
	{
		if (bRefreshPending)
		{
			RootEntry->RefreshChildren();
			bRefreshPending = false;
			FNiagaraStackSearchConfig SearchConfig;
			SearchConfig.bScrollToResult = false;
			RestartSearch(SearchConfig);
			bValidatorUpdatePending = true;
		}

		if (bValidatorUpdatePending)
		{
			UpdateStackWithValidationResults();
		}

		SearchTick();
	}
}

void UNiagaraStackViewModel::ResetSearchText()
{
	FocusedSearchResultCache.Reset();
	CurrentSearchText = FText::GetEmpty();
	// restarting the search with empty text will reset search results
	bRestartSearch = true;	
}

void UNiagaraStackViewModel::SetSearchTextExternal(const FText& NewSearchText)
{
	if(!CurrentSearchText.EqualTo(NewSearchText))
	{
		// this will forward a change in search text in a search box in a widget that registered itself, which in turn will call OnSearchTextChanged
		OnChangeSearchTextExternalDelegate.ExecuteIfBound(NewSearchText);
	}
}

void UNiagaraStackViewModel::OnSearchTextChanged(const FText& SearchText)
{	
	if (RootEntry && !CurrentSearchText.EqualTo(SearchText))
	{
		CurrentSearchText = SearchText;
		// postpone searching until next tick; protects against crashes from the GC
		// also this can be triggered by multiple events, so better wait
		bRestartSearch = true;
	}

	if(SearchText.IsEmpty())
	{
		FocusedSearchResultCache.Reset();
	}
}

bool UNiagaraStackViewModel::IsSearching()
{
	return ItemsToSearch.Num() > 0;
}

const TArray<UNiagaraStackViewModel::FSearchResult>& UNiagaraStackViewModel::GetCurrentSearchResults()
{
	return CurrentSearchResults;
}

UNiagaraStackEntry* UNiagaraStackViewModel::GetCurrentFocusedEntry() const
{	
	if (FocusedSearchResultCache.IsSet() && FocusedSearchResultCache->GetEntry() != nullptr && !FocusedSearchResultCache->GetEntry()->IsFinalized())
	{
		return FocusedSearchResultCache->GetEntry();
	}
	
	return nullptr;
}

int32 UNiagaraStackViewModel::GetCurrentFocusedEntryIndex() const
{
	if (FocusedSearchResultCache.IsSet() && FocusedSearchResultCache->GetEntry() != nullptr && !FocusedSearchResultCache->GetEntry()->IsFinalized())
	{
		for(int32 SearchResultIndex = 0; SearchResultIndex < CurrentSearchResults.Num(); SearchResultIndex++)
		{
			const FSearchResult& SearchResult = CurrentSearchResults[SearchResultIndex];
			if(SearchResult.GetEntry() == FocusedSearchResultCache->GetEntry())
			{
				return SearchResultIndex;
			}
		}
	}

	return INDEX_NONE;
}

void UNiagaraStackViewModel::AddSearchScrollOffset(int NumberOfSteps)
{
	int32 CurrentFocusedEntryIndex = GetCurrentFocusedEntryIndex();
	int32 NewFocusedEntryIndex = CurrentFocusedEntryIndex + NumberOfSteps;
	
	if (NewFocusedEntryIndex >= CurrentSearchResults.Num())
	{
		NewFocusedEntryIndex = 0;
	}
	if (NewFocusedEntryIndex < 0)
	{
		NewFocusedEntryIndex = CurrentSearchResults.Num() - 1;
	}

	if(CurrentSearchResults.IsValidIndex(NewFocusedEntryIndex))
	{
		FocusedSearchResultCache = CurrentSearchResults[NewFocusedEntryIndex];
	}
	else
	{
		FocusedSearchResultCache.Reset();
	}
}

void UNiagaraStackViewModel::CollapseToHeaders()
{
	CollapseToHeadersRecursive(GetRootEntryAsArray());
}

void UNiagaraStackViewModel::UndismissAllIssues()
{
	FScopedTransaction ScopedTransaction(LOCTEXT("UnDismissIssues", "Undismiss issues"));

	TArray<UNiagaraStackEditorData*> StackEditorDatas;
	TArray<FNiagaraMessageSourceAndStore> MessageStores;
	for (TSharedRef<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid())
		{
			StackEditorDatas.AddUnique(TopLevelViewModel->GetStackEditorData());
			TopLevelViewModel->GetMessageStores(MessageStores);
		}
	}

	for (UNiagaraStackEditorData* StackEditorData : StackEditorDatas)
	{
		StackEditorData->Modify();
		StackEditorData->UndismissAllIssues();
	}

	for (const FNiagaraMessageSourceAndStore& MessageStore : MessageStores)
	{
		MessageStore.GetSource()->Modify();
		MessageStore.GetStore()->ClearDismissedMessages();
	}

	RootEntry->GetSystemViewModel()->RefreshAssetMessagesDeferred();
	RootEntry->RefreshChildren();
}

bool UNiagaraStackViewModel::HasDismissedStackIssues()
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid())
		{
			if (TopLevelViewModel->GetStackEditorData()->GetDismissedStackIssueIds().Num() > 0)
			{
				return true;
			}
			else
			{
				TArray<FNiagaraMessageSourceAndStore> MessageStores;
				TopLevelViewModel->GetMessageStores(MessageStores);
				for (const FNiagaraMessageSourceAndStore& MessageStore : MessageStores)
				{
					if (MessageStore.GetStore()->HasDismissedMessages())
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

const TArray<TSharedRef<UNiagaraStackViewModel::FTopLevelViewModel>>& UNiagaraStackViewModel::GetTopLevelViewModels() const
{
	return TopLevelViewModels;
}

TSharedPtr<UNiagaraStackViewModel::FTopLevelViewModel> UNiagaraStackViewModel::GetTopLevelViewModelForEntry(UNiagaraStackEntry& InEntry) const
{
	if (InEntry.GetEmitterViewModel().IsValid())
	{
		const TSharedRef<FTopLevelViewModel>* MatchingTopLevelViewModel = TopLevelViewModels.FindByPredicate([&InEntry](const TSharedRef<FTopLevelViewModel>& TopLevelViewModel) 
		{ 
			return TopLevelViewModel->EmitterHandleViewModel.IsValid() && TopLevelViewModel->EmitterHandleViewModel->GetEmitterViewModel() == InEntry.GetEmitterViewModel(); 
		});

		if (MatchingTopLevelViewModel != nullptr)
		{
			return *MatchingTopLevelViewModel;
		}
	}
	else
	{
		const TSharedRef<FTopLevelViewModel>* MatchingTopLevelViewModel = TopLevelViewModels.FindByPredicate([&InEntry](const TSharedRef<FTopLevelViewModel>& TopLevelViewModel) 
		{ 
			return TopLevelViewModel->SystemViewModel == InEntry.GetSystemViewModel(); 
		});

		if (MatchingTopLevelViewModel != nullptr)
		{
			return *MatchingTopLevelViewModel;
		}
	}
	return TSharedPtr<FTopLevelViewModel>();
}

bool UNiagaraStackViewModel::ShouldHideDisabledModules() const
{
	if (SystemViewModel.IsValid())
	{
		UNiagaraSystemEditorData& EditorData = SystemViewModel.Pin()->GetEditorData();
		return EditorData.GetStackEditorData().bHideDisabledModules;
	}
	return false;
}

void UNiagaraStackViewModel::CollapseToHeadersRecursive(TArray<UNiagaraStackEntry*> Entries)
{
	for (UNiagaraStackEntry* Entry : Entries)
	{
		if (Entry->GetCanExpand())
		{
			if (Entry->IsA<UNiagaraStackItemGroup>())
			{
				Entry->SetIsExpanded(true);
			}
			else if (Entry->IsA<UNiagaraStackItem>())
			{
				Entry->SetIsExpanded(false);
			}
		}

		TArray<UNiagaraStackEntry*> Children;
		Entry->GetUnfilteredChildren(Children);
		CollapseToHeadersRecursive(Children);
	}
}

void UNiagaraStackViewModel::GetPathForEntry(UNiagaraStackEntry* Entry, TArray<UNiagaraStackEntry*>& EntryPath) const
{
	GeneratePathForEntry(RootEntry, Entry, TArray<UNiagaraStackEntry*>(), EntryPath);
}

void UNiagaraStackViewModel::EmitterParentRemoved()
{
	RootEntry->RefreshChildren();
}

void UNiagaraStackViewModel::SearchTick()
{
	// perform partial searches here, by processing a fixed number of entries (maybe more than one?)
	if (bRestartSearch)
	{
		// clear the search results
		for (auto SearchResult : GetCurrentSearchResults())
		{
			SearchResult.GetEntry()->SetIsSearchResult(false);
		}
		CurrentSearchResults.Empty();
		// generates ItemsToSearch, these will be processed on tick, in batches
		if (CurrentSearchText.IsEmpty() == false)
		{
			GenerateTraversalEntries(RootEntry, TArray<UNiagaraStackEntry*>(), ItemsToSearch);
		}
		// we need to call the SearchCompletedDelegate to go through SynchronizeTreeExpansion in SNiagaraStack so that when exiting search we return the stack expansion to the state it was before searching
		else
		{
			SearchCompletedDelegate.Broadcast(CurrentSearchConfig.Get(FNiagaraStackSearchConfig()));
		}
		bRestartSearch = false;
	}

	if (IsSearching())
	{
		double SearchStartTime = FPlatformTime::Seconds();
		double CurrentSearchLoopTime = SearchStartTime;
		// process at least one item, but don't go over MaxSearchTime for the rest
		while (ItemsToSearch.Num() > 0 && CurrentSearchLoopTime - SearchStartTime < MaxSearchTime)
		{
			UNiagaraStackEntry* EntryToProcess = ItemsToSearch[0].GetEntry();
			ensure(EntryToProcess != nullptr); // should never happen so something went wrong if this is hit
			if (EntryToProcess != nullptr)
			{
				TArray<UNiagaraStackEntry::FStackSearchItem> SearchItems;
				EntryToProcess->GetSearchItems(SearchItems);
				
				for (UNiagaraStackEntry::FStackSearchItem SearchItem : SearchItems)
				{
					if (!EntryToProcess->GetStackEditorDataKey().IsEmpty())
					{
						EntryToProcess->GetStackEditorData().SetStackEntryWasExpandedPreSearch(EntryToProcess->GetStackEditorDataKey(), EntryToProcess->GetIsExpanded());
					}

					if (ItemMatchesSearchCriteria(SearchItem)) 
					{
						EntryToProcess->SetIsSearchResult(true);
						CurrentSearchResults.Add({ ItemsToSearch[0].EntryPath, SearchItem });
						break;
					}
				}
			}
			ItemsToSearch.RemoveAt(0); // can't use RemoveAtSwap because we need to preserve the order
			CurrentSearchLoopTime = FPlatformTime::Seconds();
		}
		if (ItemsToSearch.Num() == 0)
		{
			// The search can change the child filtering so refresh it when the search finishes.
			RootEntry->RefreshFilteredChildren();
			
			// we attempt to restore the focused search result we had from before
			int32 CurrentSearchIndex = GetCurrentFocusedEntryIndex();
			if(CurrentSearchIndex != INDEX_NONE)
			{
				FocusedSearchResultCache = CurrentSearchResults[CurrentSearchIndex];
			}
			else
			{
				// if we don't have a valid search result from before, we select the first result by default, if available
				if(CurrentSearchResults.Num() > 0)
				{
					FocusedSearchResultCache = CurrentSearchResults[0];
				}
				else
				{
					FocusedSearchResultCache.Reset();
				}
			}
			
			SearchCompletedDelegate.Broadcast(CurrentSearchConfig.Get(FNiagaraStackSearchConfig()));
			CurrentSearchConfig.Reset();
		}
	}
}

void UNiagaraStackViewModel::GenerateTraversalEntries(UNiagaraStackEntry* Root, TArray<UNiagaraStackEntry*> ParentChain,
	TArray<FSearchWorkItem>& TraversedArray)
{
	TArray<UNiagaraStackEntry*> Children;
	Root->GetUnfilteredChildren(Children);
	ParentChain.Add(Root);
	TraversedArray.Add(FSearchWorkItem{ParentChain});
	for (auto Child : Children)
	{
		GenerateTraversalEntries(Child, ParentChain, TraversedArray);
	}
}

bool UNiagaraStackViewModel::ItemMatchesSearchCriteria(UNiagaraStackEntry::FStackSearchItem SearchItem)
{
	// this is a simple text compare, we need to replace this with a complex search on future passes
	return SearchItem.Value.ToString().Contains(CurrentSearchText.ToString());
}

void UNiagaraStackViewModel::GeneratePathForEntry(UNiagaraStackEntry* Root, UNiagaraStackEntry* Entry, TArray<UNiagaraStackEntry*> CurrentPath, TArray<UNiagaraStackEntry*>& EntryPath) const
{
	if (EntryPath.Num() > 0)
	{
		return;
	}
	TArray<UNiagaraStackEntry*> Children;
	Root->GetUnfilteredChildren(Children);
	CurrentPath.Add(Root);
	for (auto Child : Children)
	{
		if (Child == Entry)
		{
			EntryPath.Append(CurrentPath);
			return;
		}
		GeneratePathForEntry(Child, Entry, CurrentPath, EntryPath);
	}
}

void UNiagaraStackViewModel::RestartSearch(const FNiagaraStackSearchConfig& InConfig)
{
	if(bRestartSearch == false)
	{
		CurrentSearchConfig = InConfig;
		
		bRestartSearch = true;
	}
}

UNiagaraStackEntry* UNiagaraStackViewModel::GetRootEntry()
{
	return RootEntry;
}

TArray<UNiagaraStackEntry*>& UNiagaraStackViewModel::GetRootEntryAsArray()
{
	return RootEntries;
}

UNiagaraStackViewModel::FOnChangeSearchTextExternal& UNiagaraStackViewModel::OnChangeSearchTextExternal()
{
	return OnChangeSearchTextExternalDelegate;
}

UNiagaraStackViewModel::FOnExpansionChanged& UNiagaraStackViewModel::OnExpansionChanged()
{
	return ExpansionChangedDelegate;
}

UNiagaraStackViewModel::FOnExpansionChanged& UNiagaraStackViewModel::OnExpansionInOverviewChanged()
{
	return ExpansionInOverviewChangedDelegate;
}

UNiagaraStackViewModel::FOnStructureChanged& UNiagaraStackViewModel::OnStructureChanged()
{
	return StructureChangedDelegate;
}

UNiagaraStackViewModel::FOnSearchCompleted& UNiagaraStackViewModel::OnSearchCompleted()
{
	return SearchCompletedDelegate;
}

UNiagaraStackViewModel::FOnDataObjectChanged& UNiagaraStackViewModel::OnDataObjectChanged()
{
	return DataObjectChangedDelegate;
}

UNiagaraStackViewModel::FOnStackEntrySelectionRequested& UNiagaraStackViewModel::OnStackEntrySelectionRequested()
{
	return OnStackEntrySelectionRequestedDelegate;
}

bool UNiagaraStackViewModel::GetShowOnlyModified() const
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid() && TopLevelViewModel->GetStackEditorData()->GetShowOnlyModified())
		{
			return true;
		}
	}
	return false;
}

void UNiagaraStackViewModel::SetShowOnlyModified(bool bInShowOnlyModified)
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid()) 
		{
			TopLevelViewModel->GetStackEditorData()->SetShowOnlyModified(bInShowOnlyModified);
		}
	}

	FNiagaraStackSearchConfig SearchConfig;
	SearchConfig.bScrollToResult = false;
	RestartSearch(SearchConfig);
	RootEntry->RefreshChildren();
}

bool UNiagaraStackViewModel::GetShowAllAdvanced() const
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid() && TopLevelViewModel->GetStackEditorData()->GetShowAllAdvanced())
		{
			return true;
		}
	}
	return false;
}

void UNiagaraStackViewModel::SetShowAllAdvanced(bool bInShowAllAdvanced)
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid()) 
		{
			TopLevelViewModel->GetStackEditorData()->SetShowAllAdvanced(bInShowAllAdvanced);
		}
	}

	FNiagaraStackSearchConfig SearchConfig;
	SearchConfig.bScrollToResult = false;
	RestartSearch(SearchConfig);
	RootEntry->RefreshFilteredChildren();
}

bool UNiagaraStackViewModel::GetShowOutputs() const
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid() && TopLevelViewModel->GetStackEditorData()->GetShowOutputs())
		{
			return true;
		}
	}
	return false;
}

void UNiagaraStackViewModel::SetShowOutputs(bool bInShowOutputs)
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid())
		{
			TopLevelViewModel->GetStackEditorData()->SetShowOutputs(bInShowOutputs);
		}
	}
	
	// Showing outputs changes indenting so a full refresh is needed.
	FNiagaraStackSearchConfig SearchConfig;
	SearchConfig.bScrollToResult = false;
	RestartSearch(SearchConfig);
	RootEntry->RefreshChildren();
}

bool UNiagaraStackViewModel::GetShowLinkedInputs() const
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid() && TopLevelViewModel->GetStackEditorData()->GetShowLinkedInputs())
		{
			return true;
		}
	}
	return false;
}

void UNiagaraStackViewModel::SetShowLinkedInputs(bool bInShowLinkedInputs)
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid())
		{
			TopLevelViewModel->GetStackEditorData()->SetShowLinkedInputs(bInShowLinkedInputs);
		}
	}

	// Showing linked inputs changes indenting so a full refresh is needed.
	FNiagaraStackSearchConfig SearchConfig;
	SearchConfig.bScrollToResult = false;
	RestartSearch(SearchConfig);
	RootEntry->RefreshChildren();
}

bool UNiagaraStackViewModel::GetShowOnlyIssues() const
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid() && TopLevelViewModel->GetStackEditorData()->GetShowOnlyIssues())
		{
			return true;
		}
	}
	return false;
}

void UNiagaraStackViewModel::SetShowOnlyIssues(bool bInShowOnlyIssues)
{
	for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
	{
		if (TopLevelViewModel->IsValid())
		{
			TopLevelViewModel->GetStackEditorData()->SetShowOnlyIssues(bInShowOnlyIssues);
		}
	}

	FNiagaraStackSearchConfig SearchConfig;
	SearchConfig.bScrollToResult = false;
	RestartSearch(SearchConfig);
	RootEntry->RefreshChildren();
}

double UNiagaraStackViewModel::GetLastScrollPosition() const
{
	// TODO: Fix this with the new overview paradigm.
	if (EmitterHandleViewModel.IsValid())
	{
		return EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEditorData().GetStackEditorData().GetLastScrollPosition();
	}
	return 0;
}

void UNiagaraStackViewModel::SetLastScrollPosition(double InLastScrollPosition)
{
	// TODO: Fix this with the new overview paradigm.
	if (EmitterHandleViewModel.IsValid())
	{
		EmitterHandleViewModel.Pin()->GetEmitterViewModel()->GetEditorData().GetStackEditorData().SetLastScrollPosition(InLastScrollPosition);
	}
}

void UNiagaraStackViewModel::SelectStackEntry(UNiagaraStackEntry* StackEntryToSelect, FNiagaraStackEntrySelectionConfig Config) const
{
	OnStackEntrySelectionRequestedDelegate.ExecuteIfBound(StackEntryToSelect, Config);
}

void UNiagaraStackViewModel::EntryExpansionChanged()
{
	ExpansionChangedDelegate.Broadcast();
}

void UNiagaraStackViewModel::EntryExpansionInOverviewChanged()
{
	ExpansionInOverviewChangedDelegate.Broadcast();
}

void UNiagaraStackViewModel::EntryStructureChanged(ENiagaraStructureChangedFlags Flags)
{
	if (bUsesTopLevelViewModels)
	{
		RefreshTopLevelViewModels();
	}
	RefreshHasIssues();
	StructureChangedDelegate.Broadcast(Flags);

	// if the structure changed, additionally invalidate search results
	if((Flags & StructureChanged) != 0)
	{
		FNiagaraStackSearchConfig SearchConfig;
		SearchConfig.bScrollToResult = false;
		RestartSearch(SearchConfig);
	}
}

void UNiagaraStackViewModel::EntryDataObjectModified(TArray<UObject*> ChangedObjects, ENiagaraDataObjectChange ChangeType)
{
	if (SystemViewModel.IsValid())
	{
		SystemViewModel.Pin()->NotifyDataObjectChanged(ChangedObjects, ChangeType);
	}
	
	DataObjectChangedDelegate.Broadcast(ChangedObjects, ChangeType);
}

void UNiagaraStackViewModel::EntryRequestFullRefresh()
{
	checkf(RootEntry != nullptr, TEXT("Can not process full refresh when the root entry doesn't exist"));
	RootEntry->RefreshChildren();
}

void UNiagaraStackViewModel::RequestRefreshDeferred()
{
	bRefreshPending = true;
}

void UNiagaraStackViewModel::RefreshTopLevelViewModels()
{
	TArray<TSharedRef<FTopLevelViewModel>> CurrentTopLevelViewModels = TopLevelViewModels;
	TopLevelViewModels.Empty();

	TArray<UNiagaraStackEntry*> RootChildren;
	RootEntry->GetUnfilteredChildren(RootChildren);
	for (UNiagaraStackEntry* RootChild : RootChildren)
	{
		if (RootChild->IsFinalized())
		{
			// It's possible for this to run when a system or emitter stack view model has updated it's children, but
			// before the selection view model with the top level view models has refreshed and removed the finalized
			// children in the selection so we need to guard against that here.
			continue;
		}
		TSharedPtr<FTopLevelViewModel> TopLevelViewModel;
		if (RootChild->GetEmitterViewModel().IsValid())
		{
			TSharedPtr<FNiagaraEmitterHandleViewModel> RootChildEmitterHandleViewModel = RootChild->GetSystemViewModel()->GetEmitterHandleViewModelForEmitter(RootChild->GetEmitterViewModel()->GetEmitter());
			//-TODO:Stateless: Do we need stateless support here?
			if (RootChildEmitterHandleViewModel == nullptr)
			{
				return;
			}
			TSharedRef<FTopLevelViewModel>* CurrentTopLevelViewModelPtr = CurrentTopLevelViewModels.FindByPredicate([&RootChildEmitterHandleViewModel](const TSharedRef<FTopLevelViewModel>& TopLevelViewModel)
			{ 
				return TopLevelViewModel->EmitterHandleViewModel == RootChildEmitterHandleViewModel &&
					TopLevelViewModel->RootEntry == RootChildEmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry(); 
			});
			if (CurrentTopLevelViewModelPtr != nullptr)
			{
				TopLevelViewModel = *CurrentTopLevelViewModelPtr;
			}
			else
			{
				TopLevelViewModel = MakeShared<FTopLevelViewModel>(RootChildEmitterHandleViewModel);
			}
		}
		else
		{
			TSharedRef<FTopLevelViewModel>* CurrentTopLevelViewModelPtr = CurrentTopLevelViewModels.FindByPredicate([RootChild](const TSharedRef<FTopLevelViewModel>& TopLevelViewModel)
			{ 
				return TopLevelViewModel->SystemViewModel == RootChild->GetSystemViewModel() &&
					TopLevelViewModel->RootEntry == RootChild->GetSystemViewModel()->GetSystemStackViewModel()->GetRootEntry();
			});
			if (CurrentTopLevelViewModelPtr != nullptr)
			{
				TopLevelViewModel = *CurrentTopLevelViewModelPtr;
			}
			else
			{
				TopLevelViewModel = MakeShared<FTopLevelViewModel>(RootChild->GetSystemViewModel());
			}
		}
		if (TopLevelViewModels.ContainsByPredicate([&TopLevelViewModel](const TSharedRef<FTopLevelViewModel>& ExistingTopLevelViewModel) { return *TopLevelViewModel == *ExistingTopLevelViewModel; }) == false)
		{
			TopLevelViewModels.Add(TopLevelViewModel.ToSharedRef());
		}
	}
}

void UNiagaraStackViewModel::RefreshHasIssues()
{
	bHasIssues = false;
	if (bUsesTopLevelViewModels)
	{
		for (TSharedRef<FTopLevelViewModel> TopLevelViewModel : TopLevelViewModels)
		{
			if (TopLevelViewModel->SystemViewModel.IsValid())
			{
				if (TopLevelViewModel->SystemViewModel->GetSystemStackViewModel()->GetRootEntry()->HasIssuesOrAnyChildHasIssues())
				{
					bHasIssues = true;
					return;
				}
			}
			else if (TopLevelViewModel->EmitterHandleViewModel.IsValid())
			{
				if (TopLevelViewModel->EmitterHandleViewModel->GetEmitterStackViewModel()->GetRootEntry()->HasIssuesOrAnyChildHasIssues())
				{
					bHasIssues = true;
					return;
				}
			}
		}
	}
	else
	{
		bHasIssues = RootEntry->HasIssuesOrAnyChildHasIssues();
	}
}

UNiagaraStackEntry* UNiagaraStackViewModel::GetCurrentFocusedIssue() const
{
	if (CurrentIssueCycleIndex >= 0)
	{
		UNiagaraStackEntry* CyclingRootEntry = CyclingIssuesForTopLevel.Pin()->RootEntry.Get();
		if (CyclingRootEntry != nullptr)
		{
			const TArray<UNiagaraStackEntry*>& Issues = CyclingRootEntry->GetAllChildrenWithIssues();
			return Issues[CurrentIssueCycleIndex];
		}
	}
	
	return nullptr;
}

void UNiagaraStackViewModel::OnCycleThroughIssues(TSharedPtr<UNiagaraStackViewModel::FTopLevelViewModel> TopLevelToCycle)
{
	if (RootEntries.Num() == 0)
	{
		CurrentIssueCycleIndex = -1;
		CyclingIssuesForTopLevel.Reset();
		return;
	}

	if (CyclingIssuesForTopLevel.IsValid() && CyclingIssuesForTopLevel != TopLevelToCycle)
	{
		CurrentIssueCycleIndex = -1;
	}

	CyclingIssuesForTopLevel = TopLevelToCycle;

	UNiagaraStackEntry* CyclingRootEntry = nullptr;
	if (CyclingIssuesForTopLevel.IsValid())
	{
		CyclingRootEntry = CyclingIssuesForTopLevel.Pin()->RootEntry.Get();
	}
	
	if (CyclingRootEntry == nullptr)
	{
		CurrentIssueCycleIndex = -1;
		CyclingIssuesForTopLevel.Reset();
		return;
	}

	const TArray<UNiagaraStackEntry*>& Issues = CyclingRootEntry->GetAllChildrenWithIssues();
	if (Issues.Num() > 0)
	{
		++CurrentIssueCycleIndex;
		if (CurrentIssueCycleIndex >= Issues.Num())
		{
			CurrentIssueCycleIndex = 0;
		}
	}
}

#undef LOCTEXT_NAMESPACE

