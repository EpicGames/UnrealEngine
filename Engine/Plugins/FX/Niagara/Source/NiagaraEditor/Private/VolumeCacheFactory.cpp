// Copyright Epic Games, Inc. All Rights Reserved.

#include "VolumeCacheFactory.h"
#include "Styling/SlateBrush.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "VolumeCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumeCacheFactory)

#define LOCTEXT_NAMESPACE "VolumeCacheFactory"

#if WITH_EDITOR
UVolumeCacheFactory::UVolumeCacheFactory( const FObjectInitializer& ObjectInitializer )
 : Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UVolumeCache::StaticClass();
}

FText UVolumeCacheFactory::GetDisplayName() const
{
	return LOCTEXT("VolumeCacheFactoryDescription", "Volume Texture");
}

bool UVolumeCacheFactory::ConfigureProperties()
{
	return true;
}

UObject* UVolumeCacheFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	UVolumeCache* NewVolumeCache = NewObject<UVolumeCache>(InParent, Name, Flags);	

	return NewVolumeCache;
}
#endif

#undef LOCTEXT_NAMESPACE

