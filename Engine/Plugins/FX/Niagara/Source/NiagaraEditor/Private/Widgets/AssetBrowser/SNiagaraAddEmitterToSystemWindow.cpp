// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetBrowser/SNiagaraAddEmitterToSystemWindow.h"

#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraRecentAndFavoritesManager.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "NiagaraEditor"

FText FFrontendFilter_NiagaraEmitterInheritance::GetDisplayName() const
{
	return FText::FormatOrdered(LOCTEXT("NiagaraEmitterInheritanceFilterDisplayName", "Inheritance: {0}"), bRequiredInheritanceState ?
	    LOCTEXT("NiagaraEmitterInheritanceFilterDisplayName_Yes", "Yes") : LOCTEXT("NiagaraEmitterInheritanceFilterDisplayName_No", "No"));
}

FText FFrontendFilter_NiagaraEmitterInheritance::GetToolTipText() const
{
	if(bRequiredInheritanceState)
	{
		return LOCTEXT("NiagaraEmitterInheritanceFilterTooltip_Yes", "Only display emitters with Inheritance");
	}

	return LOCTEXT("NiagaraEmitterInheritanceFilterTooltip_No", "Only display emitters without Inheritance");
}

bool FFrontendFilter_NiagaraEmitterInheritance::PassesFilter(const FContentBrowserItem& InItem) const
{
	FAssetData AssetData;
	InItem.Legacy_TryGetAssetData(AssetData);

	// If we are also displaying assets of different classes, the emitter inheritance filter should have no effect on them
	if(AssetData.GetClass() != UNiagaraEmitter::StaticClass())
	{
		return true;
	}
	
	bool bUseInheritance = false;
	if(FNiagaraEditorUtilities::GetIsInheritableFromAssetRegistryTags(AssetData, bUseInheritance))
	{
		return bRequiredInheritanceState == bUseInheritance;
	}

	return false;
}

FText FFrontendFilter_NiagaraSystemEffectType::GetDisplayName() const
{
	return FText::FormatOrdered(LOCTEXT("NiagaraSystemEffectTypeFilterDisplayName", "Effect Type: {0}"), bRequiresEffectType ?
	    LOCTEXT("NiagaraSystemEffectTypeFilterDisplayName_Yes", "Yes") : LOCTEXT("NiagaraSystemEffectTypeFilterDisplayName_No", "No"));
}

FText FFrontendFilter_NiagaraSystemEffectType::GetToolTipText() const
{
	if(bRequiresEffectType)
	{
		return LOCTEXT("NiagaraSystemEffectTypeFilterTooltip_Yes", "Only display Niagara Systems with an effect type");
	}

	return LOCTEXT("NiagaraSystemEffectTypeFilterTooltip_No", "Only display Niagara Systems without Inheritance");
}

bool FFrontendFilter_NiagaraSystemEffectType::PassesFilter(const FContentBrowserItem& InItem) const
{
	FAssetData AssetData;
	InItem.Legacy_TryGetAssetData(AssetData);

	// If we are also displaying assets of different classes, the system effect type filter should have no effect on them
	if(AssetData.GetClass() != UNiagaraSystem::StaticClass())
	{
		return true;
	}
	
	if(InItem.GetItemAttribute("EffectType").IsValid())
	{
		FName EffectType = InItem.GetItemAttribute("EffectType").GetValue<FName>();
		if(EffectType == NAME_None && bRequiresEffectType == false)
		{
			return true;
		}

		if(EffectType != NAME_None && bRequiresEffectType == true)
		{
			return true;
		}
	}
	else
	{
		if(bRequiresEffectType == false)
		{
			return true;
		}
	}

	return false;
}

bool SNiagaraAddEmitterToSystemWindow::CheckValidArguments(const FArguments& InArgs) const
{
	if(!ensureMsgf(!InArgs._AssetBrowserWindowArgs._AssetBrowserArgs._OnAssetsActivated.IsBound(), TEXT("OnAssetsActivated should not be bound.")))
	{
		return false;
	}

	return true;
}

void SNiagaraAddEmitterToSystemWindow::Construct(const FArguments& InArgs, const UTaggedAssetBrowserConfiguration& Configuration, TSharedRef<FNiagaraSystemViewModel> SystemViewModel)
{
	WeakSystemViewModel = SystemViewModel;

	if(!CheckValidArguments(InArgs))
	{
		SWindow::FArguments WindowArguments;
		SWindow::Construct(WindowArguments);
		return;
	}
	
	FDefaultDetailsTabConfiguration DefaultDetailsTabConfiguration;
	DefaultDetailsTabConfiguration.bUseDefaultDetailsTab = true;
	DefaultDetailsTabConfiguration.EmptySelectionMessage = LOCTEXT("EmptyEmitterSelectionUserText", "Select an emitter to add it to your system, or add a new minimal emitter");
	
	STaggedAssetBrowser::FArguments AssetBrowserArgs;
	AssetBrowserArgs._AvailableClasses = { UNiagaraEmitter::StaticClass() };
	AssetBrowserArgs._AssetSelectionMode = ESelectionMode::Single;
	AssetBrowserArgs._DefaultDetailsTabConfiguration = DefaultDetailsTabConfiguration;
	AssetBrowserArgs._OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &SNiagaraAddEmitterToSystemWindow::OnAssetsActivated);
	AssetBrowserArgs._RecentAndFavoritesList = FNiagaraEditorModule::Get().GetRecentsManager()->GetRecentEmitterAndSystemsList();
	AssetBrowserArgs._AdditionalReferencingAssets = { FAssetData(&SystemViewModel->GetSystem()) };

	AssetBrowserArgs._AdditionalBottomWidget = SNew(SBox)
	   .Padding(16.f, 16.f)
	   [
		   SNew(SHorizontalBox)
		   + SHorizontalBox::Slot()
		   .AutoWidth()
		   .Padding(5.f, 1.f)
		   .HAlign(HAlign_Left)
		   [
			   SNew(SButton)
			   .OnClicked(this, &SNiagaraAddEmitterToSystemWindow::AddMinimalEmitter)
			   .ToolTipText(FNiagaraEditorUtilities::Tooltips::GetMinimalEmitterCreationTooltip())
			   [
				   SNew(SHorizontalBox)
				   + SHorizontalBox::Slot()
				   .AutoWidth()
				   .Padding(2.f)
				   [
					   SNew(SImage)
					   .Image(FSlateIconFinder::FindIconForClass(UNiagaraEmitter::StaticClass()).GetIcon())
				   ]
				   + SHorizontalBox::Slot()
				   .AutoWidth()
				   .Padding(2.f)
				   [
					   SNew(STextBlock).Text(FText::FormatOrdered(LOCTEXT("AddMinimalEmitterButtonLabel", "Add Minimal {0}"), UNiagaraEmitter::StaticClass()->GetDisplayNameText()))
				   ]
			   ]
		   ]
		   + SHorizontalBox::Slot()
		   [
			   SNew(SSpacer)
		   ]
		   + SHorizontalBox::Slot()
		   .AutoWidth()
		   .Padding(5.f, 1.f)
		   [
			   SNew(SButton)
			   .ContentPadding(FMargin(50.f, 4.f, 50.f, 1.f))
			   .ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
			   .TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PrimaryButtonText"))
			   .HAlign(HAlign_Center)
			   .Text(LOCTEXT("AddEmitterPrimaryButtonLabel", "Add"))
			   .OnClicked(this, &SNiagaraAddEmitterToSystemWindow::AddSelectedEmitters)
			   .IsEnabled(this, &SNiagaraAddEmitterToSystemWindow::HasSelectedAssets)
			   .ToolTipText(this, &SNiagaraAddEmitterToSystemWindow::GetAddButtonTooltip)
		   ]
		   + SHorizontalBox::Slot()
		   .AutoWidth()
		   .Padding(5.f, 1.f)
		   [
			   SNew(SButton)
			   .ContentPadding(FMargin(50.f, 4.f, 50.f, 1.f))
			   .TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PrimaryButtonText"))
			   .HAlign(HAlign_Center)
			   .Text(LOCTEXT("CancelButtonLabel", "Cancel"))
			   .OnClicked(this, &SNiagaraAddEmitterToSystemWindow::Cancel)
		   ]
	   ];

	SWindow::FArguments WindowArgs;
	WindowArgs.Title(LOCTEXT("AddEmitterToSystemWindowTitle", "Add Emitter to your System"));
	WindowArgs.SupportsMaximize(false);
	WindowArgs.SupportsMinimize(false);
	WindowArgs.ClientSize(FVector2D(1400, 750));
	WindowArgs.SizingRule(ESizingRule::UserSized);
	
	STaggedAssetBrowserWindow::FArguments TaggedAssetBrowserWindowArgs;
	TaggedAssetBrowserWindowArgs.AssetBrowserArgs(AssetBrowserArgs);
	TaggedAssetBrowserWindowArgs.WindowArgs(WindowArgs);
	
	STaggedAssetBrowserWindow::Construct(TaggedAssetBrowserWindowArgs, Configuration);
}

void SNiagaraAddEmitterToSystemWindow::OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type)
{
	if(WeakSystemViewModel.IsValid())
	{
		if(AssetData.Num() == 0)
		{
			WeakSystemViewModel.Pin()->AddMinimalEmitter();
		}
		else
		{
			for(const FAssetData& Asset : AssetData)
			{
				WeakSystemViewModel.Pin()->AddEmitterFromAssetData(Asset);
			}
		}
	}

	RequestDestroyWindow();
}

FReply SNiagaraAddEmitterToSystemWindow::AddMinimalEmitter()
{
	OnAssetsActivated({}, EAssetTypeActivationMethod::Opened);
	return FReply::Handled();
}

FReply SNiagaraAddEmitterToSystemWindow::AddSelectedEmitters()
{
	OnAssetsActivated(AssetBrowser->GetSelectedAssets(), EAssetTypeActivationMethod::Opened);
	return FReply::Handled();
}

FReply SNiagaraAddEmitterToSystemWindow::Cancel()
{
	RequestDestroyWindow();
	return FReply::Handled();
}

FText SNiagaraAddEmitterToSystemWindow::GetAddButtonTooltip() const
{
	return GetSelectedAssets().Num() > 0
	? LOCTEXT("AddEmitterButtonTooltip_ValidSelection", "Add the selected emitter to your System")
	: LOCTEXT("AddEmitterButtonTooltip_InvalidSelection", "Select an emitter to add to your System");
}

#undef LOCTEXT_NAMESPACE
