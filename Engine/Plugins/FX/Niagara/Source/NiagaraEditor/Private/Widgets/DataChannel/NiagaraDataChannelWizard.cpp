// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelWizard.h"

#include "EdGraphSchema_Niagara.h"
#include "NiagaraClipboard.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannelPublic.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "DataInterface/NiagaraDataInterfaceDataChannelRead.h"
#include "DataInterface/NiagaraDataInterfaceDataChannelWrite.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "IDetailsView.h"
#include "NiagaraNodeOp.h"
#include "NiagaraScriptSource.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NiagaraDataChannelWizard)


using namespace UE::Niagara::Wizard;

#define LOCTEXT_NAMESPACE "NiagaraDataChannelWizard"

namespace UE::Niagara::Wizard::DataChannel
{
	struct FSelectAssetPageBase : FModuleWizardPage
	{
		FSelectAssetPageBase()
		{
			Name = LOCTEXT("AssetPageName", "Select asset");
		}
		virtual ~FSelectAssetPageBase() override = default;

		virtual bool CanGoToNextPage() const override { return GetAsset() != nullptr; };
		virtual bool CanCompleteWizard() const override { return CanGoToNextPage(); };

		virtual UNiagaraDataChannelAsset* GetAsset() const = 0;
		
		UNiagaraDataChannel* GetDataChannel() const
		{
			if (UNiagaraDataChannelAsset* ChannelAsset = GetAsset())
			{
				return ChannelAsset->Get();
			}			
			return nullptr;
		}
		
		TSharedRef<SWidget> GetDetailsViewContent(UObject* DetailsViewObject)
		{
			TSharedRef<IDetailsView> DetailsView = Utilities::CreateDetailsView();
			DetailsView->SetObject(DetailsViewObject, true);
			
			return SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(15)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AssetPageLabel", "Please select which data channel you want to use"))
				]
				+SVerticalBox::Slot()
				[
					DetailsView
				];
		}
	};

	struct FSelectVariablesPageBase : FModuleWizardPage
	{
		explicit FSelectVariablesPageBase(FSelectAssetPageBase* InPreviousPage) : PreviousPage(InPreviousPage)
		{
			Name = LOCTEXT("VariablesPageName", "Select variables");
			SupportedNamespaces.Add(MakeShared<FString>("StackContext.Module"));
			SupportedNamespaces.Add(MakeShared<FString>("Output.Module"));
			SupportedNamespaces.Add(MakeShared<FString>("StackContext"));
			SupportedNamespaces.Add(MakeShared<FString>("Transient"));
			TargetNamespace = SupportedNamespaces[0];
		}

		virtual ~FSelectVariablesPageBase() override = default;

		virtual bool CanGoToNextPage() const override { return VariablesToProcess.Num() > 0; };
		virtual bool CanCompleteWizard() const override { return AllVariables.IsEmpty() || CanGoToNextPage(); };

		virtual FText GetHeaderLabel() = 0;

		virtual void RefreshContent() override
		{
			FObjectKey NewDataChannelRef;
			TArray<FNiagaraDataChannelVariable> DataChannelVariables;
			if (UNiagaraDataChannelAsset* ChannelAsset = PreviousPage->GetAsset())
			{
				NewDataChannelRef = ChannelAsset;
				if (UNiagaraDataChannel* DataChannel = ChannelAsset->Get())
				{
					DataChannelVariables = DataChannel->GetVariables();
				}
			}
			if (NewDataChannelRef != LastDataChannelRef)
			{
				ModuleName = CreateNewModuleName(); 
			} 

			bool bCheckAll = AllVariables.IsEmpty() || NewDataChannelRef != LastDataChannelRef;
			AllVariables.Empty(DataChannelVariables.Num());
			for (const FNiagaraDataChannelVariable& Var : DataChannelVariables)
			{
				*AllVariables.Add_GetRef(MakeShared<FNiagaraDataChannelVariable>()).Get() = Var;
				if (bCheckAll)
				{
					VariablesToProcess.Add(Var.Version);
				}
			}

			if (VarListView.IsValid())
			{
				VarListView->RebuildList();
			}
			LastDataChannelRef = NewDataChannelRef;
		}

		virtual TSharedRef<SWidget> GetContent() override
		{
			return SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(2)
				.AutoHeight()
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]
				+SVerticalBox::Slot()
				.Padding(15)
				.AutoHeight()
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(GetHeaderLabel())
				]
				+SVerticalBox::Slot()
				[
					SAssignNew(VarListView, SListView<TSharedPtr<FNiagaraDataChannelVariable>>)
						.ListItemsSource(&AllVariables)
						.OnGenerateRow(this, &FSelectVariablesPageBase::GenerateRow)
						.SelectionMode(ESelectionMode::Single)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TargetNamespaceNameText", "Target Namespace: "))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0)
					[
						SNew(SComboBox<TSharedPtr<FString>>)
						.OptionsSource(&SupportedNamespaces)
						.ContentPadding(2.0f)
						.InitiallySelectedItem(TargetNamespace)
						.ToolTipText(LOCTEXT("TargetNamespaceTooltip", "Select the namespace where the variables should be written to. The StackContext namespace changes depending on the script context it is used in (system, emitter, particle)."))
						.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
						{
							return SNew(SNiagaraParameterName)
							.ParameterName(FName(*Item + ". "))
							.IsReadOnly(true)
							.SingleNameDisplayMode(SNiagaraParameterName::ESingleNameDisplayMode::Namespace);
						})
						.OnSelectionChanged(this, &FSelectVariablesPageBase::HandleNamespaceSelectionChanged)
						[
							SNew(SNiagaraParameterName)
							.ParameterName_Lambda([this]()
							{
								return FName(GetTargetNamespace() + ". ");
							})
							.IsReadOnly(true)
							.SingleNameDisplayMode(SNiagaraParameterName::ESingleNameDisplayMode::Namespace)
						]
					]
				]
				+SVerticalBox::Slot()
				.Padding(0, 10)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("WriteModuleNameText", "Module Name: "))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0)
					[
						SNew( SEditableTextBox )
						.MinDesiredWidth(200)
						.Padding(2.0f)
						.Text(this, &FSelectVariablesPageBase::GetModuleNameText)
						.SelectAllTextWhenFocused(true)
						.ClearKeyboardFocusOnCommit(false)
						.OnTextCommitted(this, &FSelectVariablesPageBase::SetModuleName)
					]
				];
		}

		void HandleNamespaceSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type)
		{
			TargetNamespace = InItem;
		}

		FText GetModuleNameText() const
		{
			return ModuleName;
		}

		void SetModuleName(const FText& NewName, ETextCommit::Type)
		{
			ModuleName = NewName;
		}

		FText CreateNewModuleName() const
		{
			FText AssetName;
			if (UNiagaraDataChannelAsset* DataChannelAsset = PreviousPage->GetAsset())
			{
				AssetName = FText::FromString(DataChannelAsset->GetName());
			}
			return GetFormattedModuleName(AssetName);
		}

		virtual FText GetFormattedModuleName(const FText& AssetName) const = 0;

		TSharedRef<ITableRow> GenerateRow(const TSharedPtr<FNiagaraDataChannelVariable> Var, const TSharedRef<STableViewBase>& OwnerTable)
		{
			FLinearColor TypeColor = UEdGraphSchema_Niagara::GetTypeColor(Var->GetType());
			return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
				.Padding(FMargin(5, 0))
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &FSelectVariablesPageBase::OnCheckStateChanged, *Var.Get())
					.IsChecked(this, &FSelectVariablesPageBase::OnGetCheckState, *Var.Get())
					.ToolTipText(FText::Format(LOCTEXT("VariablesSelectionTooltipFmt", "Name: {0}\nType: {1}"), FText::FromName(Var->GetName()), Var->GetType().GetNameText()))
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.AutoWidth()
						[
							SNew(SImage)
							.ColorAndOpacity(TypeColor)
							.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"))
						]
						+ SHorizontalBox::Slot()
						.Padding(4, 2, 2, 2)
						[
							SNew(STextBlock)
							.MinDesiredWidth(150)
							.Text(FText::FromName(Var->GetName()))
						]
					]
				];
		}

		void OnCheckStateChanged(const ECheckBoxState NewState, FNiagaraDataChannelVariable Var)
		{
			if (NewState == ECheckBoxState::Checked)
			{
				VariablesToProcess.Add(Var.Version);
			}
			else if (NewState == ECheckBoxState::Unchecked)
			{
				VariablesToProcess.Remove(Var.Version);
			}
		}

		ECheckBoxState OnGetCheckState(FNiagaraDataChannelVariable Var) const
		{
			return VariablesToProcess.Contains(Var.Version) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		
		FString GetTargetNamespace() const
		{
			return *TargetNamespace.Get();
		}

		TArray<TSharedPtr<FNiagaraDataChannelVariable>> AllVariables;
		TSet<FGuid> VariablesToProcess;
		FSelectAssetPageBase* PreviousPage;
		FObjectKey LastDataChannelRef;
		FText ModuleName;
		TSharedPtr<FString> TargetNamespace;

		TSharedPtr<SListView<TSharedPtr<FNiagaraDataChannelVariable>>> VarListView;
		TArray<TSharedPtr<FString>> SupportedNamespaces;
	};

	struct FSelectSpawnAssetPage : FSelectAssetPageBase
	{
		virtual UNiagaraDataChannelAsset* GetAsset() const override
		{
			if (UNiagaraDataChannelSpawnModuleData* ModuleData = Data.Get())
			{
				return ModuleData->DataChannel;
			}
			return nullptr;
		}
		
		virtual TSharedRef<SWidget> GetContent() override
		{
			Data.Reset(NewObject<UNiagaraDataChannelSpawnModuleData>());
			return GetDetailsViewContent(Data.Get());
		}
		
		TStrongObjectPtr<UNiagaraDataChannelSpawnModuleData> Data;
	};
	
	struct FSpawnConditionPage : FModuleWizardPage
	{
		explicit FSpawnConditionPage(FSelectSpawnAssetPage* InPreviousPage) : PreviousPage(InPreviousPage)
		{
			Name = LOCTEXT("SpawnConditionPageName", "Spawn conditions");
		}

		virtual ~FSpawnConditionPage() override = default;

		virtual bool CanGoToNextPage() const override
		{
			if (PreviousPage->Data->SpawnMode == ENiagaraDataChanneSpawnModuleMode::DirectSpawn)
			{
				return ConditionVariables.Num() > 0;
			}
			return true;
		};
		virtual bool CanCompleteWizard() const override { return CanGoToNextPage(); };

		virtual FText GetHeaderLabel() const
		{
			if (PreviousPage->Data->SpawnMode == ENiagaraDataChanneSpawnModuleMode::ConditionalSpawn)
			{
				return LOCTEXT("SpawnConditionalPageHeader", "OPTIONAL:\nSelect which data channel variables should be used as conditions to spawn particles.\nModule inputs will be created for all selected variables.\nParticles will only be spawned if the data channel variables match the module input values.");
			}
			return LOCTEXT("SpawnDirectPageHeader", "Please select which data channel variable should be used as particle spawn count. This needs to be an integer parameter in the data channel.");
		}

		virtual void RefreshContent() override
		{
			FObjectKey NewDataChannelRef;
			TArray<FNiagaraDataChannelVariable> DataChannelVariables;
			if (UNiagaraDataChannelAsset* ChannelAsset = PreviousPage->GetAsset())
			{
				NewDataChannelRef = ChannelAsset;
				if (UNiagaraDataChannel* DataChannel = ChannelAsset->Get())
				{
					DataChannelVariables = DataChannel->GetVariables();
				}
			}
			if (NewDataChannelRef != LastDataChannelRef)
			{
				ConditionVariables.Empty();
			} 

			AllVariables.Empty(DataChannelVariables.Num());
			for (const FNiagaraDataChannelVariable& Var : DataChannelVariables)
			{
				*AllVariables.Add_GetRef(MakeShared<FNiagaraDataChannelVariable>()).Get() = Var;
			}

			if (VarListView.IsValid())
			{
				VarListView->RebuildList();
			}
			LastDataChannelRef = NewDataChannelRef;
		}

		virtual TSharedRef<SWidget> GetContent() override
		{
			return SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(2)
				.AutoHeight()
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]
				+SVerticalBox::Slot()
				.Padding(15)
				.AutoHeight()
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(this, &FSpawnConditionPage::GetHeaderLabel)
				]
				+SVerticalBox::Slot()
				[
					SAssignNew(VarListView, SListView<TSharedPtr<FNiagaraDataChannelVariable>>)
						.ListItemsSource(&AllVariables)
						.OnGenerateRow(this, &FSpawnConditionPage::GenerateRow)
						.SelectionMode(ESelectionMode::Single)
				];
		}

		TSharedRef<ITableRow> GenerateRow(const TSharedPtr<FNiagaraDataChannelVariable> Var, const TSharedRef<STableViewBase>& OwnerTable)
		{
			FLinearColor TypeColor = UEdGraphSchema_Niagara::GetTypeColor(Var->GetType());
			return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
				.IsEnabled(this, &FSpawnConditionPage::IsRowEnabled, *Var.Get())
				.Padding(FMargin(5, 0))
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &FSpawnConditionPage::OnCheckStateChanged, *Var.Get())
					.IsChecked(this, &FSpawnConditionPage::OnGetCheckState, *Var.Get())
					.ToolTipText(FText::Format(LOCTEXT("ConditionSelectionTooltipFmt", "Name: {0}\nType: {1}"), FText::FromName(Var->GetName()), Var->GetType().GetNameText()))
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.AutoWidth()
						[
							SNew(SImage)
							.ColorAndOpacity(TypeColor)
							.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"))
						]
						+ SHorizontalBox::Slot()
						.Padding(4, 2, 2, 2)
						[
							SNew(STextBlock)
							.MinDesiredWidth(150)
							.Text(FText::FromName(Var->GetName()))
						]
					]
				];
		}
		
		bool IsRowEnabled(FNiagaraDataChannelVariable Var) const
		{
			if (PreviousPage->Data->SpawnMode == ENiagaraDataChanneSpawnModuleMode::DirectSpawn)
			{
				return Var.GetType() == FNiagaraTypeDefinition::GetIntDef();
			}
			return true;
		}

		void OnCheckStateChanged(const ECheckBoxState NewState, FNiagaraDataChannelVariable Var)
		{
			if (PreviousPage->Data->SpawnMode == ENiagaraDataChanneSpawnModuleMode::DirectSpawn)
			{
				ConditionVariables.Empty();
			}
			if (NewState == ECheckBoxState::Checked)
			{
				ConditionVariables.Add(Var.Version);
			}
			else if (NewState == ECheckBoxState::Unchecked)
			{
				ConditionVariables.Remove(Var.Version);
			}
		}

		ECheckBoxState OnGetCheckState(FNiagaraDataChannelVariable Var) const
		{
			return ConditionVariables.Contains(Var.Version) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		TArray<TSharedPtr<FNiagaraDataChannelVariable>> AllVariables;
		TSet<FGuid> ConditionVariables;
		FSelectSpawnAssetPage* PreviousPage;
		FObjectKey LastDataChannelRef;

		TSharedPtr<SListView<TSharedPtr<FNiagaraDataChannelVariable>>> VarListView;
	};
}

TSharedRef<FModuleWizardModel> DataChannel::CreateReadNDCModuleWizardModel()
{
	struct FSelectAssetPage : FSelectAssetPageBase
	{
		virtual UNiagaraDataChannelAsset* GetAsset() const override
		{
			if (UNiagaraDataChannelReadModuleData* ModuleData = Data.Get())
			{
				return ModuleData->DataChannel;
			}
			return nullptr;
		}
		
		virtual TSharedRef<SWidget> GetContent() override
		{
			Data.Reset(NewObject<UNiagaraDataChannelReadModuleData>());
			return GetDetailsViewContent(Data.Get());
		}

		
		TStrongObjectPtr<UNiagaraDataChannelReadModuleData> Data;
	};

	struct FSelectVariablesPage : FSelectVariablesPageBase
	{
		explicit FSelectVariablesPage(FSelectAssetPage* InPreviousPage) : FSelectVariablesPageBase(InPreviousPage)
		{}
		virtual ~FSelectVariablesPage() override = default;

		virtual FText GetHeaderLabel() override
		{
			return LOCTEXT("VariablesPageLabel", "Please select the variables to read from the data channel");
		}
		
		virtual FText GetFormattedModuleName(const FText& AssetName) const override
		{
			return FText::Format(LOCTEXT("ReadModuleNameFmt", "Read {0}"), AssetName);
		}
	};
	
	struct FReadNDCModel : FModuleWizardModel
	{
		FReadNDCModel()
		{
			AssetPage = MakeShared<FSelectAssetPage>();
			VariablesPage = MakeShared<FSelectVariablesPage>(AssetPage.Get());
			Pages.Add(AssetPage.ToSharedRef());
			Pages.Add(VariablesPage.ToSharedRef());
		}
		virtual ~FReadNDCModel() override = default;

		virtual FName GetIdentifier() const override
		{
			return "ReadNDCWizardModel";
		}
		
		virtual void GenerateNewModuleContent(TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel, const TArray<const UNiagaraNodeFunctionCall*>& PreviousModules) override
		{
			FText ScriptName = VariablesPage->ModuleName;
			ScratchPadScriptViewModel->SetScriptName(ScriptName.IsEmptyOrWhitespace() ? VariablesPage->CreateNewModuleName() : ScriptName);
			ScratchPadScriptViewModel->GetEditScript().GetScriptData()->ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::Particle;
			
			UNiagaraDataChannel* Channel = AssetPage->GetDataChannel();
			UNiagaraGraph* Graph = ScratchPadScriptViewModel->GetGraphViewModel()->GetGraph();
			if (Channel && Graph)
			{
				const UEdGraphSchema_Niagara* GraphSchema = Graph->GetNiagaraSchema();
				UNiagaraNodeParameterMapGet* MapGetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapGet>(Graph);
				UNiagaraNodeParameterMapSet* MapSetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapSet>(Graph);

				// Add inputs
				UEdGraphPin* DIPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), FName("Data Channel"), MapGetNode);
				UEdGraphPin* IndexPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition::GetIntDef(), FName("Read Index"), MapGetNode);

				// Call read function
				if (UNiagaraNodeFunctionCall* ReadFunction = Utilities::CreateDataInterfaceFunctionNode(UNiagaraDataInterfaceDataChannelRead::StaticClass(), FName("Read"), Graph))
				{
					// connect index input pins
					ReadFunction->AutowireNewNode(DIPin);
					UEdGraphPin* IndexInput = ReadFunction->GetInputPin(1);
					if (IndexInput && IndexInput->GetName() == TEXT("Index"))
					{
						GraphSchema->TryCreateConnection(IndexPin, IndexInput);
					}

					// create and connect read success output pin
					UEdGraphPin* SuccessVarPin = Utilities::AddWriteParameterPin(FNiagaraTypeDefinition::GetBoolDef(), FName(VariablesPage->GetTargetNamespace() + ".ReadSuccess"), MapSetNode);
					UEdGraphPin* SuccessOutPin = ReadFunction->GetOutputPin(0);
					if (SuccessOutPin && SuccessOutPin->GetName() == TEXT("Success"))
					{
						GraphSchema->TryCreateConnection(SuccessOutPin, SuccessVarPin);
					}

					// add channel variable pins to read node
					for (const FNiagaraDataChannelVariable& Var : Channel->GetVariables())
					{
						if (!VariablesPage->VariablesToProcess.Contains(Var.Version))
						{
							continue;
						}
						
						FNiagaraTypeDefinition SwcType = Var.GetType();
						if (SwcType.IsEnum() == false)
						{
							SwcType = FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(Var.GetType().GetScriptStruct()));
						}
						FNiagaraVariable SWCVar(SwcType, Var.GetName());
						UEdGraphPin* ReadParamPin = ReadFunction->AddParameterPin(SWCVar, EGPD_Output);

						// add matching node on map set and connect them
						UEdGraphPin* SetVarPin = Utilities::AddWriteParameterPin(SwcType, FName(VariablesPage->GetTargetNamespace() + TEXT(".") + Var.GetName().ToString()), MapSetNode);
						if (ReadParamPin && SetVarPin)
						{
							if (SwcType == FNiagaraTypeDefinition::GetPositionDef() && AssetPage->Data->bAutoTransformPositionData)
							{
								// transform position if necessary
								if (UNiagaraNodeFunctionCall* TransformNode = Utilities::CreateFunctionCallNode(LoadObject<UNiagaraScript>(nullptr, TEXT("/Niagara/Functions/Localspace/TransformPosition.TransformPosition")), Graph))
								{
									GraphSchema->TryCreateConnection(ReadParamPin, TransformNode->FindPin(FName("Position"), EGPD_Input));
									GraphSchema->TryCreateConnection(SetVarPin, TransformNode->FindPin(FName("Position"), EGPD_Output));
								}								
							}
							else
							{
								GraphSchema->TryCreateConnection(ReadParamPin, SetVarPin);
							}
						}
					}
				}
				
				FNiagaraStackGraphUtilities::RelayoutGraph(*Graph);
				ScratchPadScriptViewModel->ApplyChanges();
			}
		}
		
		virtual bool UpdateModuleInputs(UNiagaraClipboardContent* NewModule, const TArray<const UNiagaraNodeFunctionCall*>& PreviousModules) override
		{
			if (UNiagaraDataChannelAsset* Channel = AssetPage->GetAsset())
			{
				TArray<TObjectPtr<const UNiagaraClipboardFunctionInput>> FunctionInputs = NewModule->FunctionInputs;
				for (const UNiagaraClipboardFunctionInput* FunctionInput : FunctionInputs)
				{
					if (FunctionInput->InputType == FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()))
					{
						// set data interface module input
						if (UNiagaraDataInterfaceDataChannelRead* DataInterface = Cast<UNiagaraDataInterfaceDataChannelRead>(FunctionInput->Data))
						{
							DataInterface->Channel = Channel;
							DataInterface->InitAccessContext();
							DataInterface->bReadCurrentFrame = AssetPage->Data->bReadCurrentFrame;
							DataInterface->bUpdateSourceDataEveryTick = AssetPage->Data->bUpdateSourceDataEveryTick;
						}
					}
				}
				return true;
			}
			return false;
		}

		TSharedPtr<FSelectAssetPage> AssetPage;
		TSharedPtr<FSelectVariablesPage> VariablesPage;
	};
	
	return MakeShared<FReadNDCModel>();
}

TSharedRef<FModuleWizardModel> DataChannel::CreateWriteNDCModuleWizardModel()
{
	struct FSelectAssetPage : FSelectAssetPageBase
	{
		virtual UNiagaraDataChannelAsset* GetAsset() const override
		{
			if (UNiagaraDataChannelWriteModuleData* ModuleData = Data.Get())
			{
				return ModuleData->DataChannel;
			}
			return nullptr;
		}
		
		virtual TSharedRef<SWidget> GetContent() override
		{
			Data.Reset(NewObject<UNiagaraDataChannelWriteModuleData>());
			return GetDetailsViewContent(Data.Get());
		}

		TStrongObjectPtr<UNiagaraDataChannelWriteModuleData> Data;
	};

	struct FSelectVariablesPage : FSelectVariablesPageBase
	{
		explicit FSelectVariablesPage(FSelectAssetPage* InPreviousPage) : FSelectVariablesPageBase(InPreviousPage)
		{}
		virtual ~FSelectVariablesPage() override = default;

		virtual FText GetHeaderLabel() override
		{
			return LOCTEXT("VariablesWritePageLabel", "Please select which data channel variables to write to");
		}

		virtual FText GetFormattedModuleName(const FText& AssetName) const override
		{
			return FText::Format(LOCTEXT("WriteModuleNameFmt", "Write {0}"), AssetName);
		}
	};
	
	struct FWriteNDCModel : FModuleWizardModel
	{
		FWriteNDCModel()
		{
			AssetPage = MakeShared<FSelectAssetPage>();
			VariablesPage = MakeShared<FSelectVariablesPage>(AssetPage.Get());
			Pages.Add(AssetPage.ToSharedRef());
			Pages.Add(VariablesPage.ToSharedRef());
		}
		virtual ~FWriteNDCModel() override = default;

		virtual FName GetIdentifier() const override
		{
			return "WriteNDCWizardModel";
		}
		
		virtual void GenerateNewModuleContent(TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel, const TArray<const UNiagaraNodeFunctionCall*>& PreviousModules) override
		{
			FText ScriptName = VariablesPage->ModuleName;
			ScratchPadScriptViewModel->SetScriptName(ScriptName.IsEmptyOrWhitespace() ? VariablesPage->CreateNewModuleName() : ScriptName);
			ScratchPadScriptViewModel->GetEditScript().GetScriptData()->ModuleUsageBitmask = ENiagaraScriptUsageMask::System | ENiagaraScriptUsageMask::Emitter | ENiagaraScriptUsageMask::Particle;
			
			UNiagaraDataChannel* Channel = AssetPage->GetDataChannel();
			UNiagaraGraph* Graph = ScratchPadScriptViewModel->GetGraphViewModel()->GetGraph();
			if (Channel == nullptr || Graph == nullptr)
			{
				return;
			}
			const UEdGraphSchema_Niagara* GraphSchema = Graph->GetNiagaraSchema();
			UNiagaraNodeParameterMapGet* MapGetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapGet>(Graph);
			UNiagaraNodeParameterMapSet* MapSetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapSet>(Graph);
			UNiagaraNodeInput* InputNode = Utilities::FindSingleNodeChecked<UNiagaraNodeInput>(Graph);
			ENiagaraDataChanneWriteModuleMode WriteMode = AssetPage->Data->WriteMode;
			
			// Add inputs
			UEdGraphPin* DIPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelWrite::StaticClass()), FName("Data Channel"), MapGetNode);
			UEdGraphPin* ExecWritePin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition::GetBoolDef(), FName("Execute Write"), MapGetNode);
			Utilities::SetDefaultValue(Graph, ExecWritePin->PinName, FNiagaraTypeDefinition::GetBoolDef(), true);
			UEdGraphPin* IndexPin = nullptr;
			if (WriteMode == ENiagaraDataChanneWriteModuleMode::WriteToExistingElement)
			{
				IndexPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition::GetIntDef(), FName("Write Index"), MapGetNode);
			}

			// Call write function
			FName FunctionName = WriteMode == ENiagaraDataChanneWriteModuleMode::AppendNewElement ? FName("Append") : FName("Write");
			if (UNiagaraNodeFunctionCall* WriteFunction = Utilities::CreateDataInterfaceFunctionNode(UNiagaraDataInterfaceDataChannelWrite::StaticClass(), FunctionName, Graph))
			{
				// connect default function pins
				WriteFunction->AutowireNewNode(DIPin);
				GraphSchema->TryCreateConnection(InputNode->GetOutputPin(0), WriteFunction->GetInputPin(0));
				GraphSchema->TryCreateConnection(WriteFunction->GetOutputPin(0), MapSetNode->GetInputPin(0));
				
				UEdGraphPin* ExecInput = WriteFunction->GetInputPin(2);
				if (ExecInput && ExecInput->GetName() == TEXT("Emit"))
				{
					GraphSchema->TryCreateConnection(ExecWritePin, ExecInput);
				}
				
				UEdGraphPin* IndexInput = WriteFunction->GetInputPin(3);
				if (IndexInput && IndexInput->GetName() == TEXT("Index"))
				{
					GraphSchema->TryCreateConnection(IndexPin, IndexInput);
				}

				// create and connect write success output pin
				UEdGraphPin* SuccessVarPin = Utilities::AddWriteParameterPin(FNiagaraTypeDefinition::GetBoolDef(), FName(VariablesPage->GetTargetNamespace() + ".WriteSuccess"), MapSetNode);
				UEdGraphPin* SuccessOutPin = WriteFunction->GetOutputPin(1);
				if (SuccessOutPin && SuccessOutPin->GetName() == TEXT("Success"))
				{
					GraphSchema->TryCreateConnection(SuccessOutPin, SuccessVarPin);
				}

				// add channel variable pins to write node
				for (const FNiagaraDataChannelVariable& Var : Channel->GetVariables())
				{
					if (!VariablesPage->VariablesToProcess.Contains(Var.Version))
					{
						continue;
					}
					
					FNiagaraTypeDefinition SwcType = Var.GetType();
					if (SwcType.IsEnum() == false)
					{
						SwcType = FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(Var.GetType().GetScriptStruct()));
					}
					FNiagaraVariable SWCVar(SwcType, Var.GetName());
					UEdGraphPin* WriteParamPin = WriteFunction->AddParameterPin(SWCVar, EGPD_Input);

					// add matching node on map get and connect them
					UEdGraphPin* SetVarPin = Utilities::AddReadParameterPin(SwcType, FName(TEXT("Module.") + Var.GetName().ToString()), MapGetNode);
					if (WriteParamPin && SetVarPin)
					{
						if (SwcType == FNiagaraTypeDefinition::GetPositionDef() && AssetPage->Data->bAutoTransformPositionData)
						{
							// transform position if necessary
							if (UNiagaraNodeFunctionCall* TransformNode = Utilities::CreateFunctionCallNode(LoadObject<UNiagaraScript>(nullptr, TEXT("/Niagara/Functions/Localspace/TransformPosition.TransformPosition")), Graph))
							{
								GraphSchema->TryCreateConnection(WriteParamPin, TransformNode->FindPin(FName("Position"), EGPD_Output));
								GraphSchema->TryCreateConnection(SetVarPin, TransformNode->FindPin(FName("Position"), EGPD_Input));
								TransformNode->FindPin(FName("Source Space"), EGPD_Input)->DefaultValue = TEXT("Simulation");
								TransformNode->FindPin(FName("Destination Space"), EGPD_Input)->DefaultValue = TEXT("World");
							}								
						}
						else
						{
							GraphSchema->TryCreateConnection(WriteParamPin, SetVarPin);
						}
					}
				}
				
				FNiagaraStackGraphUtilities::RelayoutGraph(*Graph);
				ScratchPadScriptViewModel->ApplyChanges();
			}
		}
		
		virtual bool UpdateModuleInputs(UNiagaraClipboardContent* NewModule, const TArray<const UNiagaraNodeFunctionCall*>& PreviousModules) override
		{
			if (UNiagaraDataChannelAsset* Channel = AssetPage->GetAsset())
			{
				TArray<TObjectPtr<const UNiagaraClipboardFunctionInput>> FunctionInputs = NewModule->FunctionInputs;
				for (const UNiagaraClipboardFunctionInput* FunctionInput : FunctionInputs)
				{
					if (FunctionInput->InputType == FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelWrite::StaticClass()))
					{
						// set data interface module input
						if (UNiagaraDataInterfaceDataChannelWrite* DataInterface = Cast<UNiagaraDataInterfaceDataChannelWrite>(FunctionInput->Data))
						{
							DataInterface->Channel = Channel;
							DataInterface->InitAccessContext();
							DataInterface->bPublishToGame = AssetPage->Data->bPublishToGame;
							DataInterface->bPublishToCPU = AssetPage->Data->bPublishToCPU;
							DataInterface->bPublishToGPU = AssetPage->Data->bPublishToGPU;
							DataInterface->AllocationCount = AssetPage->Data->AllocationCount;
							DataInterface->AllocationMode = AssetPage->Data->AllocationMode;
							DataInterface->bUpdateDestinationDataEveryTick = AssetPage->Data->bUpdateDestinationDataEveryTick;
						}
					}
				}
				return true;
			}
			return false;
		}

		TSharedPtr<FSelectAssetPage> AssetPage;
		TSharedPtr<FSelectVariablesPage> VariablesPage;
	};
		
	return MakeShared<FWriteNDCModel>();
}

TSharedRef<FModuleWizardModel> DataChannel::CreateSpawnNDCModuleWizardModel()
{
	struct FSelectVariablesPage : FSelectVariablesPageBase
	{
		explicit FSelectVariablesPage(FSelectSpawnAssetPage* InPreviousPage) : FSelectVariablesPageBase(InPreviousPage)
		{
			SupportedNamespaces.Empty();
			SupportedNamespaces.Add(MakeShared<FString>("Particles"));
			SupportedNamespaces.Add(MakeShared<FString>("Particles.Module"));
			SupportedNamespaces.Add(MakeShared<FString>("Output.Module"));
			SupportedNamespaces.Add(MakeShared<FString>("Transient"));
			TargetNamespace = SupportedNamespaces[0];
		}
		
		virtual ~FSelectVariablesPage() override = default;

		virtual FText GetHeaderLabel() override
		{
			return LOCTEXT("VariablesPageSpawnLabel", "Please select which variables should be read into particle attributes when spawning.");
		}
		
		virtual FText GetFormattedModuleName(const FText& AssetName) const override
		{
			return FText::Format(LOCTEXT("SpawnModuleNameFmt", "Spawn From {0}"), AssetName);
		}
	};
	
	struct FSpawnNDCModel : FModuleWizardModel
	{
		FSpawnNDCModel()
		{
			AssetPage = MakeShared<FSelectSpawnAssetPage>();
			ConditionPage = MakeShared<FSpawnConditionPage>(AssetPage.Get());
			VariablesPage = MakeShared<FSelectVariablesPage>(AssetPage.Get());
			Pages.Add(AssetPage.ToSharedRef());
			Pages.Add(ConditionPage.ToSharedRef());
			Pages.Add(VariablesPage.ToSharedRef());
		}
		virtual ~FSpawnNDCModel() override = default;

		virtual FName GetIdentifier() const override
		{
			return "SpawnNDCWizardModel";
		}
		
		virtual TArray<FModuleCreationEntry> GetModulesToCreate(UNiagaraNodeOutput* ProvidedOutputNode, int32 ProvidedTargetIndex, TSharedPtr<FNiagaraSystemViewModel> SystemModel, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel) override
		{
			TArray<FModuleCreationEntry> Result;
			if (UNiagaraScriptSource* EmitterGraphSource = Cast<UNiagaraScriptSource>(EmitterViewModel->GetEmitter().GetEmitterData()->GraphSource))
			{
				UNiagaraNodeOutput* SpawnScriptNode = EmitterGraphSource->NodeGraph->FindEquivalentOutputNode(ENiagaraScriptUsage::EmitterSpawnScript, FGuid());
				Result.Add({SpawnScriptNode, INDEX_NONE}); // this is the module in emitter spawn to set up the common data channel parameter
			}
			Result.Add({ProvidedOutputNode, ProvidedTargetIndex}); // this is the spawn module in emitter update
			if (UNiagaraScriptSource* EmitterGraphSource = Cast<UNiagaraScriptSource>(EmitterViewModel->GetEmitter().GetEmitterData()->GraphSource))
			{
				UNiagaraNodeOutput* SpawnScriptNode = EmitterGraphSource->NodeGraph->FindEquivalentOutputNode(ENiagaraScriptUsage::ParticleSpawnScript, FGuid());
				Result.Add({SpawnScriptNode, 1}); // this is the module in particle spawn to write the particle data from the ndc
			}
			return Result;
		}
		
		virtual void GenerateNewModuleContent(TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel, const TArray<const UNiagaraNodeFunctionCall*>& PreviousModules) override
		{
			if (PreviousModules.Num() == 0)
			{
				// the emitter spawn module sets up the ndc parameter used by the other two modules
				GenerateEmitterSpawnModule(ScratchPadScriptViewModel);
			}
			if (PreviousModules.Num() == 1)
			{
				// the emitter update module spawns calls the spawn functions
				GenerateEmitterUpdateModule(ScratchPadScriptViewModel);
			}
			else if (PreviousModules.Num() == 2)
			{
				// the particle spawn module reads the data from the data channel row that spawned each particle
				GenerateParticleSpawnModule(ScratchPadScriptViewModel);
			}
		}

		void GenerateEmitterSpawnModule(TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel)
		{
			ScratchPadScriptViewModel->SetScriptName(FText::FromString("Init data channel"));
			
			UNiagaraDataChannel* Channel = AssetPage->GetDataChannel();
			UNiagaraGraph* Graph = ScratchPadScriptViewModel->GetGraphViewModel()->GetGraph();
			if (Channel && Graph)
			{
				const UEdGraphSchema_Niagara* GraphSchema = Graph->GetNiagaraSchema();
				UNiagaraNodeParameterMapGet* MapGetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapGet>(Graph);
				UNiagaraNodeParameterMapSet* MapSetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapSet>(Graph);
				UEdGraphPin* DIInputPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), FName("Data Channel"), MapGetNode);
				UEdGraphPin* DIVarPin = Utilities::AddWriteParameterPin(UNiagaraDataInterfaceDataChannelRead::StaticClass(), FName("Emitter.SpawnDataChannel"), MapSetNode);
				GraphSchema->TryCreateConnection(DIInputPin, DIVarPin);
				
				FNiagaraStackGraphUtilities::RelayoutGraph(*Graph);
                ScratchPadScriptViewModel->ApplyChanges();
			}
		}

		void GenerateEmitterUpdateModule(TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel)
		{
			FText ScriptName = VariablesPage->ModuleName;
			ScratchPadScriptViewModel->SetScriptName(ScriptName.IsEmptyOrWhitespace() ? VariablesPage->CreateNewModuleName() : ScriptName);
			
			UNiagaraDataChannel* Channel = AssetPage->GetDataChannel();
			UNiagaraGraph* Graph = ScratchPadScriptViewModel->GetGraphViewModel()->GetGraph();
			if (Channel && Graph)
			{
				const UEdGraphSchema_Niagara* GraphSchema = Graph->GetNiagaraSchema();
				UNiagaraNodeParameterMapGet* MapGetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapGet>(Graph);
				UNiagaraNodeParameterMapSet* MapSetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapSet>(Graph);
				Graph->RemoveNode(MapSetNode);
				UNiagaraNodeInput* InputNode = Utilities::FindSingleNodeChecked<UNiagaraNodeInput>(Graph);
				UNiagaraNodeOutput* OutputNode = Utilities::FindSingleNodeChecked<UNiagaraNodeOutput>(Graph);

				// Call spawn function
				ENiagaraDataChanneSpawnModuleMode SpawnMode = AssetPage->Data->SpawnMode;
				if (UNiagaraNodeFunctionCall* SpawnFunction = Utilities::CreateDataInterfaceFunctionNode(UNiagaraDataInterfaceDataChannelRead::StaticClass(), FName(SpawnMode == ENiagaraDataChanneSpawnModuleMode::ConditionalSpawn ? "SpawnConditional" : "SpawnDirect"), Graph))
				{
					// connect base pins of the function call
					UEdGraphPin* DIPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), FName("Data Channel"), MapGetNode);
					SpawnFunction->AutowireNewNode(DIPin);
					GraphSchema->TryCreateConnection(InputNode->GetOutputPin(0), SpawnFunction->GetInputPin(0));
					GraphSchema->TryCreateConnection(SpawnFunction->GetOutputPin(0), OutputNode->GetInputPin(0));
					Utilities::SetDefaultBinding(Graph, DIPin->PinName, FName("Emitter.SpawnDataChannel"));

					// Add module inputs
					if (UEdGraphPin* EnableInput = SpawnFunction->FindPin(FName("Enable"), EGPD_Input))
					{
						UEdGraphPin* EnablePin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition::GetBoolDef(), FName("Spawn Enabled"), MapGetNode);
						Utilities::SetDefaultValue(Graph, EnablePin->PinName, FNiagaraTypeDefinition::GetBoolDef(), true);
						Utilities::SetTooltip(Graph, EnablePin->PinName, SpawnFunction->Signature.InputDescriptions[FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), EnableInput->PinName)]);
						GraphSchema->TryCreateConnection(EnablePin, EnableInput);
					}
					if (UEdGraphPin* EmitterIDInput = SpawnFunction->FindPin(FName("Emitter ID"), EGPD_Input))
					{
						UEdGraphPin* EmitterIDPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition(FNiagaraEmitterID::StaticStruct()), FName("Emitter ID"), MapGetNode);
						GraphSchema->TryCreateConnection(EmitterIDInput, EmitterIDPin);
						Utilities::SetDefaultBinding(Graph, EmitterIDPin->PinName, SYS_PARAM_ENGINE_EMITTER_ID.GetName());
						Utilities::SetTooltip(Graph, EmitterIDPin->PinName, SpawnFunction->Signature.InputDescriptions[FNiagaraVariable(FNiagaraTypeDefinition(FNiagaraEmitterID::StaticStruct()), EmitterIDInput->PinName)]);
					}
					if (UEdGraphPin* ModeInput = SpawnFunction->FindPin(FName("Mode"), EGPD_Input))
					{
						UEdGraphPin* SpawnModePin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition(StaticEnum<ENDIDataChannelSpawnMode>()), FName("Spawn Mode"), MapGetNode);
						GraphSchema->TryCreateConnection(ModeInput, SpawnModePin);
						Utilities::SetTooltip(Graph, SpawnModePin->PinName, SpawnFunction->Signature.InputDescriptions[FNiagaraVariable(FNiagaraTypeDefinition(StaticEnum<ENDIDataChannelSpawnMode>()), ModeInput->PinName)]);
					}
					if (UEdGraphPin* OperatorInput = SpawnFunction->FindPin(FName("Operator"), EGPD_Input))
					{
						UEdGraphPin* OperatorPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition(StaticEnum<ENiagaraConditionalOperator>()), FName("Comparison Operator"), MapGetNode);
						GraphSchema->TryCreateConnection(OperatorInput, OperatorPin);
						Utilities::SetTooltip(Graph, OperatorPin->PinName, SpawnFunction->Signature.InputDescriptions[FNiagaraVariable(FNiagaraTypeDefinition(StaticEnum<ENiagaraConditionalOperator>()), OperatorInput->PinName)]);
					}
					if (UEdGraphPin* MinInput = SpawnFunction->FindPin(FName(SpawnMode == ENiagaraDataChanneSpawnModuleMode::ConditionalSpawn ? "Min Spawn Count" : "ClampMin"), EGPD_Input))
					{
						UEdGraphPin* SpawnMinPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition::GetIntDef(), FName("Min Count"), MapGetNode);
						Utilities::SetTooltip(Graph, SpawnMinPin->PinName, SpawnFunction->Signature.InputDescriptions[FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), MinInput->PinName)]);
						int32 Default = SpawnMode == ENiagaraDataChanneSpawnModuleMode::ConditionalSpawn ? 1 : -1;//Default to -1 for spawn direct as this is a clamp. aka, no clamp by default.
						Utilities::SetDefaultValue(Graph, SpawnMinPin->PinName, FNiagaraTypeDefinition::GetIntDef(), Default);
						if (AssetPage->Data->bModifySpawnCountByScalability)
						{
							// Multiply by emitter scalability
							if (UNiagaraNodeFunctionCall* ScaleSpawnNode = Utilities::CreateFunctionCallNode(LoadObject<UNiagaraScript>(nullptr, TEXT("/Niagara/Functions/Spawn/ScaleBurstSpawnCount.ScaleBurstSpawnCount")), Graph))
							{
								GraphSchema->TryCreateConnection(InputNode->GetOutputPin(0), ScaleSpawnNode->FindPin(FName("ParamMap"), EGPD_Input));
								GraphSchema->TryCreateConnection(SpawnMinPin, ScaleSpawnNode->FindPin(FName("SpawnCount"), EGPD_Input));
								GraphSchema->TryCreateConnection(MinInput, ScaleSpawnNode->GetOutputPin(0));
							}
						}
						else
						{
							GraphSchema->TryCreateConnection(MinInput, SpawnMinPin);
						}
					}
					if (UEdGraphPin* MaxInput = SpawnFunction->FindPin(FName(SpawnMode == ENiagaraDataChanneSpawnModuleMode::ConditionalSpawn ? "Max Spawn Count" : "ClampMax"), EGPD_Input))
					{
						UEdGraphPin* SpawnMaxPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition::GetIntDef(), FName("Max Count"), MapGetNode);
						Utilities::SetTooltip(Graph, SpawnMaxPin->PinName, SpawnFunction->Signature.InputDescriptions[FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), MaxInput->PinName)]);
						int32 Default = SpawnMode == ENiagaraDataChanneSpawnModuleMode::ConditionalSpawn ? 1 : -1;//Default to -1 for spawn direct as this is a clamp. aka, no clamp by default.
						Utilities::SetDefaultValue(Graph, SpawnMaxPin->PinName, FNiagaraTypeDefinition::GetIntDef(), Default);
						if (AssetPage->Data->bModifySpawnCountByScalability)
						{
							// Multiply by emitter scalability
							if (UNiagaraNodeFunctionCall* ScaleSpawnNode = Utilities::CreateFunctionCallNode(LoadObject<UNiagaraScript>(nullptr, TEXT("/Niagara/Functions/Spawn/ScaleBurstSpawnCount.ScaleBurstSpawnCount")), Graph))
							{
								GraphSchema->TryCreateConnection(InputNode->GetOutputPin(0), ScaleSpawnNode->FindPin(FName("ParamMap"), EGPD_Input));
								GraphSchema->TryCreateConnection(SpawnMaxPin, ScaleSpawnNode->FindPin(FName("SpawnCount"), EGPD_Input));
								GraphSchema->TryCreateConnection(MaxInput, ScaleSpawnNode->GetOutputPin(0));
							}
						}
						else
						{
							GraphSchema->TryCreateConnection(MaxInput, SpawnMaxPin);
						}
					}
					if (SpawnMode == ENiagaraDataChanneSpawnModuleMode::DirectSpawn)
					{
						if (UEdGraphPin* ScaleMinInput = SpawnFunction->FindPin(FName("RandomScaleMin"), EGPD_Input))
						{
							UEdGraphPin* ScaleMinPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition::GetFloatDef(), FName("Random Scale Min"), MapGetNode);
							Utilities::SetDefaultValue(Graph, ScaleMinPin->PinName, FNiagaraTypeDefinition::GetFloatDef(), 1.0f);
							Utilities::SetTooltip(Graph, ScaleMinPin->PinName, SpawnFunction->Signature.InputDescriptions[FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), ScaleMinInput->PinName)]);
							GraphSchema->TryCreateConnection(ScaleMinInput, ScaleMinPin);
						}
						if (UEdGraphPin* ScaleMaxInput = SpawnFunction->FindPin(FName("RandomScaleMax"), EGPD_Input))
						{
							UEdGraphPin* ScaleMaxPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition::GetFloatDef(), FName("Random Scale Max"), MapGetNode);
							Utilities::SetDefaultValue(Graph, ScaleMaxPin->PinName, FNiagaraTypeDefinition::GetFloatDef(), 1.0f);
							Utilities::SetTooltip(Graph, ScaleMaxPin->PinName, SpawnFunction->Signature.InputDescriptions[FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), ScaleMaxInput->PinName)]);
							GraphSchema->TryCreateConnection(ScaleMaxInput, ScaleMaxPin);
						}

						// Encode spawn direct condition variable
						if (ConditionPage->ConditionVariables.Num() == 1)
						{
							for (const FNiagaraDataChannelVariable& Var : Channel->GetVariables())
							{
								if (ConditionPage->ConditionVariables.Contains(Var.Version))
								{
									const static FName VarNameSpecifierKey(TEXT("VarName"));
									const static FName VarTypeSpecifierKey(TEXT("VarType"));
									FString TypeStr;
									UScriptStruct* TypeStruct = FNiagaraTypeDefinition::StaticStruct();
									FNiagaraTypeDefinition TypeDef = FNiagaraTypeDefinition::GetIntDef();
									TypeStruct->ExportText(TypeStr, &TypeDef, nullptr, nullptr, PPF_None, nullptr);
									SpawnFunction->SetFunctionSpecifier(VarNameSpecifierKey, Var.GetName());
									SpawnFunction->SetFunctionSpecifier(VarTypeSpecifierKey, *TypeStr);
									break;
								}
							}
						}
					}
					
					if (SpawnMode == ENiagaraDataChanneSpawnModuleMode::ConditionalSpawn)
					{
						for (const FNiagaraDataChannelVariable& Var : Channel->GetVariables())
						{
							if (!ConditionPage->ConditionVariables.Contains(Var.Version))
							{
								continue;
							}
							FNiagaraTypeDefinition SwcType = Var.GetType();
							if (SwcType.IsEnum() == false)
							{
								SwcType = FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(Var.GetType().GetScriptStruct()));
							}
							FNiagaraVariable SWCVar(SwcType, Var.GetName());
							UEdGraphPin* ConditionParamPin = SpawnFunction->AddParameterPin(SWCVar, EGPD_Input);

							// add matching node on map get and connect them
							UEdGraphPin* SetVarPin = Utilities::AddReadParameterPin(SwcType, FName(TEXT("Module.") + Var.GetName().ToString() + " Condition"), MapGetNode);
							if (ConditionParamPin && SetVarPin)
							{
								if (SwcType == FNiagaraTypeDefinition::GetPositionDef() && AssetPage->Data->bAutoTransformPositionData)
								{
									// transform position if necessary
									if (UNiagaraNodeFunctionCall* TransformNode = Utilities::CreateFunctionCallNode(LoadObject<UNiagaraScript>(nullptr, TEXT("/Niagara/Functions/Localspace/TransformPosition.TransformPosition")), Graph))
									{
										GraphSchema->TryCreateConnection(ConditionParamPin, TransformNode->FindPin(FName("Position"), EGPD_Output));
										GraphSchema->TryCreateConnection(SetVarPin, TransformNode->FindPin(FName("Position"), EGPD_Input));
										TransformNode->FindPin(FName("Source Space"), EGPD_Input)->DefaultValue = TEXT("Simulation");
										TransformNode->FindPin(FName("Destination Space"), EGPD_Input)->DefaultValue = TEXT("World");
									}								
								}
								else
								{
									GraphSchema->TryCreateConnection(ConditionParamPin, SetVarPin);
								}
							}
						}
					}
				}
				
				FNiagaraStackGraphUtilities::RelayoutGraph(*Graph);
				ScratchPadScriptViewModel->ApplyChanges();
			}
		}

		void GenerateParticleSpawnModule(TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel)
		{
			ScratchPadScriptViewModel->SetScriptName(LOCTEXT("SpawnParticleModuleName", "Init Particle From NDC"));
			
			UNiagaraDataChannel* Channel = AssetPage->GetDataChannel();
			UNiagaraGraph* Graph = ScratchPadScriptViewModel->GetGraphViewModel()->GetGraph();
			if (Channel && Graph)
			{
				const UEdGraphSchema_Niagara* GraphSchema = Graph->GetNiagaraSchema();
				UNiagaraNodeParameterMapGet* MapGetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapGet>(Graph);
				UNiagaraNodeParameterMapSet* MapSetNode = Utilities::FindSingleNodeChecked<UNiagaraNodeParameterMapSet>(Graph);

				// Call read functions
				UNiagaraNodeFunctionCall* SpawnDataFunction = Utilities::CreateDataInterfaceFunctionNode(UNiagaraDataInterfaceDataChannelRead::StaticClass(), FName("GetNDCSpawnData"), Graph);
				UNiagaraNodeFunctionCall* ReadFunction = Utilities::CreateDataInterfaceFunctionNode(UNiagaraDataInterfaceDataChannelRead::StaticClass(), FName("Read"), Graph);
				if (SpawnDataFunction && ReadFunction)
				{
					// Add module inputs
					UEdGraphPin* DIPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()), FName("Data Channel"), MapGetNode);
					SpawnDataFunction->AutowireNewNode(DIPin);
					ReadFunction->AutowireNewNode(DIPin);
					Utilities::SetDefaultBinding(Graph, DIPin->PinName, FName("Emitter.SpawnDataChannel"));
					
					if (UEdGraphPin* EmitterIDInput = SpawnDataFunction->FindPin(FName("Emitter ID"), EGPD_Input))
					{
						UEdGraphPin* EmitterIDPin = Utilities::AddReadParameterPin(FNiagaraTypeDefinition(FNiagaraEmitterID::StaticStruct()), FName("Emitter ID"), MapGetNode);
						GraphSchema->TryCreateConnection(EmitterIDInput, EmitterIDPin);
						Utilities::SetDefaultBinding(Graph, EmitterIDPin->PinName, SYS_PARAM_ENGINE_EMITTER_ID.GetName());
					}
					
					// create and connect exec index node
					UNiagaraNodeOp* ExecIndexNode = Utilities::CreateOpNode(FName("Util::ExecIndex"), Graph);
					if (UEdGraphPin* EmitterIDInput = SpawnDataFunction->FindPin(FName("Spawned Particle Exec Index"), EGPD_Input))
					{
						GraphSchema->TryCreateConnection(EmitterIDInput, ExecIndexNode->Pins[0]);
					}

					// connect index pins
					GraphSchema->TryCreateConnection(SpawnDataFunction->GetOutputPin(0), ReadFunction->GetInputPin(1));

					// create and connect read success output pin
					UEdGraphPin* SuccessVarPin = Utilities::AddWriteParameterPin(FNiagaraTypeDefinition::GetBoolDef(), FName(VariablesPage->GetTargetNamespace() + ".ReadSuccess"), MapSetNode);
					UEdGraphPin* SuccessOutPin = ReadFunction->GetOutputPin(0);
					if (SuccessOutPin && SuccessOutPin->GetName() == TEXT("Success"))
					{
						GraphSchema->TryCreateConnection(SuccessOutPin, SuccessVarPin);
					}

					// add channel variable pins to read node
					for (const FNiagaraDataChannelVariable& Var : Channel->GetVariables())
					{
						if (!VariablesPage->VariablesToProcess.Contains(Var.Version))
						{
							continue;
						}
						
						FNiagaraTypeDefinition SwcType = Var.GetType();
						if (SwcType.IsEnum() == false)
						{
							SwcType = FNiagaraTypeDefinition(FNiagaraTypeHelper::GetSWCStruct(Var.GetType().GetScriptStruct()));
						}
						FNiagaraVariable SWCVar(SwcType, Var.GetName());
						UEdGraphPin* ReadParamPin = ReadFunction->AddParameterPin(SWCVar, EGPD_Output);

						// add matching node on map set and connect them
						UEdGraphPin* SetVarPin = Utilities::AddWriteParameterPin(SwcType, FName(VariablesPage->GetTargetNamespace() + TEXT(".") + Var.GetName().ToString()), MapSetNode);
						if (ReadParamPin && SetVarPin)
						{
							if (SwcType == FNiagaraTypeDefinition::GetPositionDef() && AssetPage->Data->bAutoTransformPositionData)
							{
								// transform position if necessary
								if (UNiagaraNodeFunctionCall* TransformNode = Utilities::CreateFunctionCallNode(LoadObject<UNiagaraScript>(nullptr, TEXT("/Niagara/Functions/Localspace/TransformPosition.TransformPosition")), Graph))
								{
									GraphSchema->TryCreateConnection(ReadParamPin, TransformNode->FindPin(FName("Position"), EGPD_Input));
									GraphSchema->TryCreateConnection(SetVarPin, TransformNode->FindPin(FName("Position"), EGPD_Output));
								}								
							}
							else
							{
								GraphSchema->TryCreateConnection(ReadParamPin, SetVarPin);
							}
						}
					}
				}
				
				FNiagaraStackGraphUtilities::RelayoutGraph(*Graph);
				ScratchPadScriptViewModel->ApplyChanges();
			}
		}

		virtual bool UpdateModuleInputs(UNiagaraClipboardContent* NewModule, const TArray<const UNiagaraNodeFunctionCall*>& PreviousModules) override
		{
			if (UNiagaraDataChannelAsset* Channel = AssetPage->GetAsset())
			{
				TArray<TObjectPtr<const UNiagaraClipboardFunctionInput>> FunctionInputs = NewModule->FunctionInputs;
				for (const UNiagaraClipboardFunctionInput* FunctionInput : FunctionInputs)
				{
					if (PreviousModules.Num() == 0 && FunctionInput->InputType == FNiagaraTypeDefinition(UNiagaraDataInterfaceDataChannelRead::StaticClass()))
					{
						// set data interface module input
						if (UNiagaraDataInterfaceDataChannelRead* DataInterface = Cast<UNiagaraDataInterfaceDataChannelRead>(FunctionInput->Data))
						{
							DataInterface->Channel = Channel;
							DataInterface->InitAccessContext();
							DataInterface->bReadCurrentFrame = AssetPage->Data->bReadCurrentFrame;
							DataInterface->bUpdateSourceDataEveryTick = AssetPage->Data->bUpdateSourceDataEveryTick;
						}
					}
				}
				return true;
			}
			return false;
		}

		TSharedPtr<FSelectSpawnAssetPage> AssetPage;
		TSharedPtr<FSpawnConditionPage> ConditionPage;
		TSharedPtr<FSelectVariablesPage> VariablesPage;
	};
	
	return MakeShared<FSpawnNDCModel>();
}

TSharedRef<FModuleWizardGenerator> DataChannel::CreateNDCWizardGenerator()
{
	class NDCWizardGenerator : public FModuleWizardGenerator
	{
	public:
		virtual TArray<FAction> CreateWizardActions(ENiagaraScriptUsage Usage) override
		{
			TArray<FAction> WizardActions;

			FAction& ReadAction = WizardActions.AddDefaulted_GetRef();
			ReadAction.DisplayName = LOCTEXT("NewReadNDCModuleName", "Read From Data Channel...");
			ReadAction.Description = LOCTEXT("NewReadNDCModuleDescription", "Description: Create a new scratch pad module to read attributes from a data channel");
			ReadAction.Keywords = LOCTEXT("NewReadNDCModuleKeywords", "ndc reader datachannel get external");
			ReadAction.WizardModel = CreateReadNDCModuleWizardModel();

			FAction& WriteAction = WizardActions.AddDefaulted_GetRef();
			WriteAction.DisplayName = LOCTEXT("NewWriteNDCModuleName", "Write To Data Channel...");
			WriteAction.Description = LOCTEXT("NewWriteNDCModuleDescription", "Description: Create a new scratch pad module to write attributes to a data channel");
			WriteAction.Keywords = LOCTEXT("NewWriteNDCModuleKeywords", "ndc writer datachannel save append external");
			WriteAction.WizardModel = CreateWriteNDCModuleWizardModel();

			if (Usage == ENiagaraScriptUsage::EmitterUpdateScript)
			{
				FAction& SpawnAction = WizardActions.AddDefaulted_GetRef();
				SpawnAction.DisplayName = LOCTEXT("NewSpawnNDCModuleName", "Spawn From Data Channel...");
				SpawnAction.Description = LOCTEXT("NewSpawnNDCModuleDescription", "Description: Create a new scratch pad module to spawn particles from data channel entries. Every time an entry is added to the data channel, it will burst spawn new particles.");
				SpawnAction.Keywords = LOCTEXT("NewSpawnNDCModuleKeywords", "ndc spawner datachannel particles burst external");
				SpawnAction.WizardModel = CreateSpawnNDCModuleWizardModel();
			}
			
			return WizardActions;
		}

		virtual ~NDCWizardGenerator() override = default;
	};
	return MakeShared<NDCWizardGenerator>();
}

#undef LOCTEXT_NAMESPACE
