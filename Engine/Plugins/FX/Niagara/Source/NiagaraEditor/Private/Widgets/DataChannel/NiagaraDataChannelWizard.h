// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataChannelPublic.h"
#include "DataInterface/NiagaraDataInterfaceDataChannelCommon.h"
#include "Widgets/Wizard/SNiagaraModuleWizard.h"
#include "Templates/SharedPointer.h"

#include "NiagaraDataChannelWizard.generated.h"

class UNiagaraNodeParameterMapGet;
class UNiagaraNodeParameterMapSet;
class UNiagaraNodeFunctionCall;
class UNiagaraGraph;
class IDetailsView;

/**
 * Helper class to display properties in the read data channel module wizard
 */
UCLASS()
class UNiagaraDataChannelReadModuleData : public UObject
{
	GENERATED_BODY()

public:
	/** The source asset to read from */
	UPROPERTY(EditAnywhere, Category = "Source")
	TObjectPtr<UNiagaraDataChannelAsset> DataChannel;

	/** True if this reader will read the current frame's data. If false, we read the previous frame.
	* Reading the previous frame's data introduces a frame of latency but ensures we never miss any data as we have access to the whole frame.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	bool bReadCurrentFrame = false;

	/**
	Whether this DI should request updated source data from the Data Channel each tick.
	Some Data Channels have multiple separate source data elements for things such as spatial subdivision. 
	Each DI will request the correct one for it's owning system instance from the data channel. 
	Depending on the data channel this could be an expensive search so we should avoid doing this every tick if possible.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	bool bUpdateSourceDataEveryTick = true;

	/**
	If true then position inputs are automatically transformed from world space to simulation space, so the read works correctly for localspace emitters.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	bool bAutoTransformPositionData = true;
};

UENUM()
enum class ENiagaraDataChanneSpawnModuleMode : uint8
{
	/**
	 Spawn particles for each entry in the data channel. Optionally checks if certain conditions are met, for example if
	 the entry has the correct material attribute or if a bool attribute is set to true.
	 */
	ConditionalSpawn,
	
	/**
	 The number of particles to spawn is read directly from an attribute in the data channel. 
	 */
	DirectSpawn
};

/**
 * Helper class to display properties in the read data channel module wizard
 */
UCLASS()
class UNiagaraDataChannelSpawnModuleData : public UObject
{
	GENERATED_BODY()

public:
	/** The source asset to spawn from */
	UPROPERTY(EditAnywhere, Category = "Source")
	TObjectPtr<UNiagaraDataChannelAsset> DataChannel;

	/** Determines how new particles should be spawned.
	 *
	 * ConditionalSpawn - Always spawn particles in a when a data channel entry fulfills the (optional) conditions. 
	 *
	 * DirectSpawn - The number of particles to spawn is read directly from an attribute in the data channel.
	 */
	UPROPERTY(EditAnywhere, Category = "Source")
	ENiagaraDataChanneSpawnModuleMode SpawnMode = ENiagaraDataChanneSpawnModuleMode::ConditionalSpawn;


	/** True if this reader will read the current frame's data. If false, we read the previous frame.
	* Reading the previous frame's data introduces a frame of latency but ensures we never miss any data as we have access to the whole frame.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	bool bReadCurrentFrame = false;

	/**
	Whether this DI should request updated source data from the Data Channel each tick.
	Some Data Channels have multiple separate source data elements for things such as spatial subdivision. 
	Each DI will request the correct one for it's owning system instance from the data channel. 
	Depending on the data channel this could be an expensive search so we should avoid doing this every tick if possible.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	bool bUpdateSourceDataEveryTick = true;

	/**
	If true then position inputs are automatically transformed from world space to simulation space, so the spawning works correctly for localspace emitters.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	bool bAutoTransformPositionData = true;

	/**If true then min and max spawn counts are multiplied by emitter spawn count scale, similar to existing spawn modules like spawn rate or spawn burst.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	bool bModifySpawnCountByScalability = true;
};

UENUM()
enum class ENiagaraDataChanneWriteModuleMode : uint8
{
	AppendNewElement,
	WriteToExistingElement
};

/**
 * Helper class to display properties in the write data channel module wizard
 */
UCLASS()
class UNiagaraDataChannelWriteModuleData : public UObject
{
	GENERATED_BODY()

public:
	/** The source asset to read from */
	UPROPERTY(EditAnywhere, Category = "Source")
	TObjectPtr<UNiagaraDataChannelAsset> DataChannel;

	/** Should the generated module append to the data channel or write to an existing element? */
	UPROPERTY(EditAnywhere, Category = "Source")
	ENiagaraDataChanneWriteModuleMode WriteMode = ENiagaraDataChanneWriteModuleMode::AppendNewElement;

	/** Whether the data generated by the niagara data interface should be published to the world game data channel. This is required to allow game BP and C++ to read this data. Setting this to true will have a minor performance impact. */
	UPROPERTY(EditAnywhere, Category = "Source", meta = (DisplayName = "Visible to Blueprint"))
	bool bPublishToGame = true;

	/** Whether the data generated by the niagara data interface should be published to CPU emitters in other Niagara systems. Setting this to true will have a minor performance impact. */
	UPROPERTY(EditAnywhere, Category = "Source", meta = (DisplayName = "Visible to CPU systems"))
	bool bPublishToCPU = true;

	/** Whether the data generated by the niagara data interface should be published to GPU emitters in other Niagara systems. Setting this to true will have a minor performance impact.  */
	UPROPERTY(EditAnywhere, Category = "Source", meta = (DisplayName = "Visible to GPU systems"))
	bool bPublishToGPU = true;

	/** How should we allocate the buffer into which we write data. */
	UPROPERTY(EditAnywhere, Category = "Advanced")
	ENiagaraDataChannelAllocationMode AllocationMode = ENiagaraDataChannelAllocationMode::Static;

	/** How many elements to allocate for writing per frame? Usage depends on AllocationMode. */
	UPROPERTY(EditAnywhere, Category = "Advanced")
	uint32 AllocationCount = 1;

	/**
	Whether this DI should request updated destination data from the Data Channel each tick.
	Depending on the data channel this could be an expensive search so we should avoid doing this every tick if possible.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	bool bUpdateDestinationDataEveryTick = true;

	/**
	If true then position inputs are automatically transformed from simulation space to world space, so the write works correctly for localspace emitters.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	bool bAutoTransformPositionData = true;
};

namespace UE::Niagara::Wizard::DataChannel
{
	TSharedRef<FModuleWizardModel> CreateReadNDCModuleWizardModel();
	TSharedRef<FModuleWizardModel> CreateWriteNDCModuleWizardModel();
	TSharedRef<FModuleWizardModel> CreateSpawnNDCModuleWizardModel();
	TSharedRef<FModuleWizardGenerator> CreateNDCWizardGenerator();
};
