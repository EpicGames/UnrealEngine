// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "Stateless/NiagaraStatelessDistribution.h"
#include "NiagaraCommon.h"
#include "NiagaraVariableMetaData.h"

struct FRichCurve;

/** Defines different edit modes for a niagara distribution */
enum class ENiagaraDistributionEditorMode
{
	/* A multi-channel value which is mapped to an array of values. */
	Array,
	/* Bound to an script attribute. */
	Binding,
	/** Bound to an expression chain */
	Expression,
	/** Constant value */
	Constant,
	/** A multi-channel value edited with a single constant value. */
	UniformConstant,
	/** A multi-channel value edited with a constant value per channel. */
	NonUniformConstant,
	/** A multi-channel value edited with a single constant color. */
	ColorConstant,
	/** A single value which is distributed between a single range of values. */
	Range,
	/** A multi-channel value which is distributed between a single range of values. */
	UniformRange,
	/** A multi-channel value which is distributed between ranges of values, one range per channel. */
	/** A multi-channel value which is distributed between ranges of values, one range per channel. */
	NonUniformRange,
	/** A multi-channel value which is distributed between a single range of colors. */
	ColorRange,
	/** A single value which has been mapped to a range of values along a curve. */
	Curve,
	/** A multi-channel value which has been mapped to a single range of values along a curve. */
	UniformCurve,
	/** A multi-channel value which has been mapped to a range of values along a curve, one curve per channel. */
	NonUniformCurve,
	/** A multi-channel value which has been mapped to a single range of colors along a gradient. */
	ColorGradient
};

/** Defines the interface for editing a distribution of values with different numbers of channels and different representations. */
class INiagaraDistributionAdapter
{
public:
	virtual ~INiagaraDistributionAdapter() = default;

	/** Returns whether or not this adapter valid for use */
	virtual bool IsValid() const = 0;

	/** Gets the number of channels currently supported by this adapter. This can change with different distribution modes. */
	virtual int32 GetNumChannels() const = 0;

	/** Gets the display name for a specific channel by index. */
	virtual FText GetChannelDisplayName(int32 ChannelIndex) const = 0;

	/** Gets any tooltip associated with this channel by index. */
	virtual FText GetChannelToolTip(int32 ChannelIndex) const = 0;

	/** Gets the color used to display a channel by index. */
	virtual FSlateColor GetChannelColor(int32 ChannelIndex) const = 0;

	/** Gets the editor modes supported by this distribution. */
	virtual void GetSupportedDistributionModes(TArray<ENiagaraDistributionEditorMode>& OutSupportedModes) const = 0;

	/** Gets the current mode for this distribution. */
	virtual ENiagaraDistributionEditorMode GetDistributionMode() const = 0;

	/** Sets the current mode for this distribution. */
	virtual void SetDistributionMode(ENiagaraDistributionEditorMode InMode) = 0;

	/** A delegate which is called whenever the distribution mode has changed. */
	virtual FSimpleMulticastDelegate& OnDistributionEditorModeChanged() = 0;

	/** Returns the widget customization options defined by the property metadata. */
	virtual FNiagaraInputParameterCustomization GetWidgetCustomization() = 0;

	/** If specified, the unit to display for the property */
	virtual EUnit GetDisplayUnit() = 0;

	/** 
		Gets a constant or range value by channel index and value index. 
		@param ChannelIndex the index of the channel to get.
		@param ValueIndex the index of the value to get, 0 for constants, 0 for minimum range values, and 1 for maximum range values.
	*/
	virtual float GetConstantOrRangeValue(int32 ChannelIndex, int32 ValueIndex) const = 0;

	/** 
		Sets a constant or range value by channel index and value index. 
		@param ChannelIndex the index of the channel to set.
		@param ValueIndex the index of the value to set, 0 for constants, 0 for minimum range values, and 1 for maximum range values.
		@param InValue the value to set.
	*/
	virtual void SetConstantOrRangeValue(int32 ChannelIndex, int32 ValueIndex, float InValue) = 0;

	/** 
		Sets all channel values of a constant or range by value index. 
		@param ValueIndex the index of the values to set, 0 for constants, 0 for minimum range values, and 1 for maximum range values.
		@param InValues the values to set, one per channel.
	*/
	virtual void SetConstantOrRangeValues(int32 ValueIndex, const TArray<float>& InValues) = 0;

	/** Returns a view into the flat float array data. */
	virtual TConstArrayView<float> GetArrayValues() const = 0;
	/** Sets the array values. */
	virtual void SetArrayValues(TConstArrayView<float> EntryData) = 0;

	/** Get the type used when generating an expression to run */
	virtual FNiagaraTypeDefinition GetExpressionTypeDef() const = 0;
	/** Get the root expression struct. */
	virtual struct FInstancedStruct& GetExpressionRoot() const = 0;
	/** Run the provided function inside a transaction, calling pre / post modify on the owning UObject. */
	virtual void ExecuteTransaction(FText TransactionText, TFunction<void()> TransactionFunc) = 0;

	/** Get the name of the variable we are currently bound to. */
	virtual FNiagaraVariableBase GetBindingValue() const = 0;
	/** Set the name of the variable we want to bind to. */
	virtual void SetBindingValue(FNiagaraVariableBase Binding) = 0;
	/** Get a list of available variables we can bind to. */
	virtual TArray<FNiagaraVariableBase> GetAvailableBindings() const = 0;

	/** Gets a curve value for the specified channel */
	virtual const FRichCurve* GetCurveValue(int32 ChannelIndex) const = 0;

	/** Gets a curve value for the specified channel */
	virtual FRichCurve* GetCurveValue(int32 ChannelIndex) = 0;

	/** Notifies this distribution adapter that a curve from GetCurveValue has been modified externally. */
	virtual void NotifyCurveValueChangedExternally(int32 ChannelIndex) = 0;

	/** Set a curve value for the specified channel */
	virtual void SetCurveValue(int32 ChannelIndex, const FRichCurve& InValue, bool bShouldTransact = true) = 0;

	/** Are we allowed to read / write the interpolation mode for the current distribution mode. */
	virtual bool AllowInterpolationMode() const = 0;
	/** Get the interpolation mode from the distribution */
	virtual ENiagaraDistributionInterpolationMode GetInterpolationMode() const = 0;
	/** Set the interpolation mode on the distribution */
	virtual void SetInterpolationMode(ENiagaraDistributionInterpolationMode Mode) = 0;

	/** Are we allowed to read / write the address mode for the current distribution mode. */
	virtual bool AllowAddressMode() const = 0;
	/** Get the address mode from the distribution */
	virtual ENiagaraDistributionAddressMode GetAddressMode() const = 0;
	/** Set the address mode on the distribution */
	virtual void SetAddressMode(ENiagaraDistributionAddressMode Mode) = 0;

	/** Are we allowed to read / write the value lookup mode for the current distribution mode. */
	virtual bool AllowLookupValueMode() const = 0;
	/** Get the lookup value mode from the distribution */
	virtual uint8 GetLookupValueMode() const = 0;
	/** Set the lookup value mode on the distribution */
	virtual void SetLookupValueMode(uint8 Mode) = 0;
	/** Gets the UEnum to be used in lookup value mode. */
	virtual const UEnum* GetLookupValueModeEnum() const = 0;

	/** Called to notify the distribution that a continuous change has begun. */
	virtual void BeginContinuousChange() = 0;

	/** Called to notify the distribution that a continuous change has ended. */
	virtual void EndContinuousChange() = 0;

	/** Called to notify the distribution that a continuous change has been cancelled. */
	virtual void CancelContinuousChange() = 0;

	/** A delegate that is called whenever the value of the distribution is changed. */
	virtual FSimpleMulticastDelegate& OnDistributionValueChanged() = 0;

	/** Gets the owning objects for this distribution. */
	virtual void GetOwners(TArray<UObject*>& OutOwners) const = 0;

	/** Called to modify the distribution that an external editor has started a 
		transaction and that owning objects should be modified to take part in the transaction. */
	virtual void ModifyOwners() = 0;
};
