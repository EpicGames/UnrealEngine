// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/NiagaraDistributionEditorUtilities.h"

#include "NiagaraEditorStyle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/INiagaraDistributionAdapter.h"

#define LOCTEXT_NAMESPACE "NiagaraDistributionEditorUtilities"

FText FNiagaraDistributionEditorUtilities::DistributionModeToDisplayName(ENiagaraDistributionEditorMode InMode)
{
	switch (InMode)
	{
	case ENiagaraDistributionEditorMode::Array:
		return LOCTEXT("ArrayDisplayName", "Array");
	case ENiagaraDistributionEditorMode::Binding:
		return LOCTEXT("BindingDisplayName", "Binding");
	case ENiagaraDistributionEditorMode::Expression:
		return LOCTEXT("ExpressionDisplayName", "Expression");
	case ENiagaraDistributionEditorMode::Constant:
		return LOCTEXT("ConstantDisplayName", "Constant");
	case ENiagaraDistributionEditorMode::UniformConstant:
		return LOCTEXT("UniformContstantDisplayName", "Uniform Constant");
	case ENiagaraDistributionEditorMode::NonUniformConstant:
		return LOCTEXT("NonUniformConstantDisplayName", "Non-uniform Constant");
	case ENiagaraDistributionEditorMode::ColorConstant:
		return LOCTEXT("ColorConstantDisplayName", "Color Constant");

	case ENiagaraDistributionEditorMode::Range:
		return LOCTEXT("RangeDisplayName", "Range");
	case ENiagaraDistributionEditorMode::UniformRange:
		return LOCTEXT("UniformRangeDisplayName", "Uniform Range");
	case ENiagaraDistributionEditorMode::NonUniformRange:
		return LOCTEXT("NonUniformRangeDisplayName", "Non-uniform Range");
	case ENiagaraDistributionEditorMode::ColorRange:
		return LOCTEXT("ColorRangeDisplayName", "Color Range");

	case ENiagaraDistributionEditorMode::Curve:
		return LOCTEXT("CurveDisplayName", "Curve");
	case ENiagaraDistributionEditorMode::UniformCurve:
		return LOCTEXT("UniformCurveDisplayName", "Uniform Curve");
	case ENiagaraDistributionEditorMode::NonUniformCurve:
		return LOCTEXT("NonUniformCurveDisplayName", "Non-uniform Curve");
	case ENiagaraDistributionEditorMode::ColorGradient:
		return LOCTEXT("ColorGradientDisplayName", "Color Gradient");

	default:
		return LOCTEXT("UnknownDisplayName", "Unknown");
	}
}

FText FNiagaraDistributionEditorUtilities::DistributionModeToToolTipText(ENiagaraDistributionEditorMode InMode)
{
	switch (InMode)
	{
	case ENiagaraDistributionEditorMode::Array:
		return LOCTEXT("ArrayToolTip", "Value driven by an array of values.");
	case ENiagaraDistributionEditorMode::Binding:
		return LOCTEXT("BindingToolTip", "Value bound to an attribute.");
	case ENiagaraDistributionEditorMode::Expression:
		return LOCTEXT("ExpressionToolTip", "Value bound to a expression chain.");
	case ENiagaraDistributionEditorMode::Constant:
		return LOCTEXT("ConstantToolTip", "A constant single value.");
	case ENiagaraDistributionEditorMode::UniformConstant:
		return LOCTEXT("UniformContstantToolTip", "A constant value applied to all value components.");
	case ENiagaraDistributionEditorMode::NonUniformConstant:
		return LOCTEXT("NonUniformConstantToolTip", "Constant values which can be different for each value component.");
	case ENiagaraDistributionEditorMode::ColorConstant:
		return LOCTEXT("ColorConstantToolTip", "A constant color value.");

	case ENiagaraDistributionEditorMode::Range:
		return LOCTEXT("RangeToolTip", "A single min/max range.");
	case ENiagaraDistributionEditorMode::UniformRange:
		return LOCTEXT("UniformRangeToolTip", "A min/max range applied to all value components.");
	case ENiagaraDistributionEditorMode::NonUniformRange:
		return LOCTEXT("NonUniformRangeToolTip", "Min/max ranges which can be different for each value component.");
	case ENiagaraDistributionEditorMode::ColorRange:
		return LOCTEXT("ColorRangeToolTip", "A color min/max range.");

	case ENiagaraDistributionEditorMode::Curve:
		return LOCTEXT("CurveToolTip", "This value is driven by a curve.");
	case ENiagaraDistributionEditorMode::UniformCurve:
		return LOCTEXT("UniformCurveToolTip", "All components of this value are driven by the same curve.");
	case ENiagaraDistributionEditorMode::NonUniformCurve:
		return LOCTEXT("NonUniformCurveToolTip", "Each component of this value is driven by its own curve.");
	case ENiagaraDistributionEditorMode::ColorGradient:
		return LOCTEXT("ColorGradientToolTip", "This color value is driven by a gradient.");

	default:
		return LOCTEXT("UnknownToolTip", "Unknown");
	}
}

FName FNiagaraDistributionEditorUtilities::DistributionModeToIconBrushName(ENiagaraDistributionEditorMode InMode)
{
	switch (InMode)
	{
	case ENiagaraDistributionEditorMode::Array:
		return "NiagaraEditor.DistributionEditor.Array";
	case ENiagaraDistributionEditorMode::Binding:
		return "NiagaraEditor.DistributionEditor.Binding";
	case ENiagaraDistributionEditorMode::Expression:
		return "NiagaraEditor.DistributionEditor.Binding";
	case ENiagaraDistributionEditorMode::Constant:
	case ENiagaraDistributionEditorMode::UniformConstant:
		return "NiagaraEditor.DistributionEditor.UniformConstant";
	case ENiagaraDistributionEditorMode::NonUniformConstant:
		return "NiagaraEditor.DistributionEditor.NonUniformConstant";
	case ENiagaraDistributionEditorMode::ColorConstant:
		return "NiagaraEditor.DistributionEditor.ColorConstant";

	case ENiagaraDistributionEditorMode::Range:
	case ENiagaraDistributionEditorMode::UniformRange:
		return "NiagaraEditor.DistributionEditor.UniformRange";
	case ENiagaraDistributionEditorMode::NonUniformRange:
		return "NiagaraEditor.DistributionEditor.NonUniformRange";
	case ENiagaraDistributionEditorMode::ColorRange:
		return "NiagaraEditor.DistributionEditor.ColorRange";

	case ENiagaraDistributionEditorMode::Curve:
	case ENiagaraDistributionEditorMode::UniformCurve:
		return "NiagaraEditor.DistributionEditor.UniformCurve";
	case ENiagaraDistributionEditorMode::NonUniformCurve:
		return "NiagaraEditor.DistributionEditor.NonUniformCurve";
	case ENiagaraDistributionEditorMode::ColorGradient:
		return "NiagaraEditor.DistributionEditor.ColorGradient";

	default:
		return NAME_None;
	}
}

const FSlateBrush* FNiagaraDistributionEditorUtilities::DistributionModeToIconBrush(ENiagaraDistributionEditorMode InMode)
{
	return FNiagaraEditorStyle::Get().GetBrush(DistributionModeToIconBrushName(InMode));
}

FSlateIcon FNiagaraDistributionEditorUtilities::DistributionModeToIcon(ENiagaraDistributionEditorMode InMode)
{
	return FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), DistributionModeToIconBrushName(InMode));
}

bool FNiagaraDistributionEditorUtilities::IsArray(ENiagaraDistributionEditorMode InMode)
{
	return
		InMode == ENiagaraDistributionEditorMode::Array;
}

bool FNiagaraDistributionEditorUtilities::IsBinding(ENiagaraDistributionEditorMode InMode)
{
	return
		InMode == ENiagaraDistributionEditorMode::Binding;
}

bool FNiagaraDistributionEditorUtilities::IsExpression(ENiagaraDistributionEditorMode InMode)
{
	return
		InMode == ENiagaraDistributionEditorMode::Expression;
}

bool FNiagaraDistributionEditorUtilities::IsUniform(ENiagaraDistributionEditorMode InMode)
{
	return
		InMode == ENiagaraDistributionEditorMode::Constant ||
		InMode == ENiagaraDistributionEditorMode::UniformConstant ||
		InMode == ENiagaraDistributionEditorMode::Range ||
		InMode == ENiagaraDistributionEditorMode::UniformRange ||
		InMode == ENiagaraDistributionEditorMode::Curve ||
		InMode == ENiagaraDistributionEditorMode::UniformCurve;
}

bool FNiagaraDistributionEditorUtilities::IsColor(ENiagaraDistributionEditorMode InMode)
{
	return
		InMode == ENiagaraDistributionEditorMode::ColorConstant ||
		InMode == ENiagaraDistributionEditorMode::ColorRange ||
		InMode == ENiagaraDistributionEditorMode::ColorGradient;
}

bool FNiagaraDistributionEditorUtilities::IsConstant(ENiagaraDistributionEditorMode InMode)
{
	return
		InMode == ENiagaraDistributionEditorMode::Constant ||
		InMode == ENiagaraDistributionEditorMode::UniformConstant ||
		InMode == ENiagaraDistributionEditorMode::NonUniformConstant ||
		InMode == ENiagaraDistributionEditorMode::ColorConstant;
}

bool FNiagaraDistributionEditorUtilities::IsRange(ENiagaraDistributionEditorMode InMode)
{
	return
		InMode == ENiagaraDistributionEditorMode::Range ||
		InMode == ENiagaraDistributionEditorMode::UniformRange ||
		InMode == ENiagaraDistributionEditorMode::NonUniformRange ||
		InMode == ENiagaraDistributionEditorMode::ColorRange;
}

bool FNiagaraDistributionEditorUtilities::IsCurve(ENiagaraDistributionEditorMode InMode)
{
	return
		InMode == ENiagaraDistributionEditorMode::Curve ||
		InMode == ENiagaraDistributionEditorMode::UniformCurve ||
		InMode == ENiagaraDistributionEditorMode::NonUniformCurve;
}

bool FNiagaraDistributionEditorUtilities::IsGradient(ENiagaraDistributionEditorMode InMode)
{
	return InMode == ENiagaraDistributionEditorMode::ColorGradient;
}

#undef LOCTEXT_NAMESPACE