// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/INiagaraDistributionAdapter.h"

#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

struct FSlateBrush;
struct FSlateIcon;

namespace FNiagaraDistributionEditorUtilities
{
	FText DistributionModeToDisplayName(ENiagaraDistributionEditorMode InMode);
	FText DistributionModeToToolTipText(ENiagaraDistributionEditorMode InMode);

	FName DistributionModeToIconBrushName(ENiagaraDistributionEditorMode InMode);
	const FSlateBrush* DistributionModeToIconBrush(ENiagaraDistributionEditorMode InMode);
	FSlateIcon DistributionModeToIcon(ENiagaraDistributionEditorMode InMode);

	bool IsArray(ENiagaraDistributionEditorMode InMode);
	bool IsBinding(ENiagaraDistributionEditorMode InMode);
	bool IsExpression(ENiagaraDistributionEditorMode InMode);
	bool IsUniform(ENiagaraDistributionEditorMode InMode);
	bool IsColor(ENiagaraDistributionEditorMode InMode);

	bool IsConstant(ENiagaraDistributionEditorMode InMode);
	bool IsRange(ENiagaraDistributionEditorMode InMode);
	bool IsCurve(ENiagaraDistributionEditorMode InMode);
	bool IsGradient(ENiagaraDistributionEditorMode InMode);
}