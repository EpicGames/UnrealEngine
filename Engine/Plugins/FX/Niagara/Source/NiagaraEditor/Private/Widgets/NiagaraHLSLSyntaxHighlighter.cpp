// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/NiagaraHLSLSyntaxHighlighter.h"
#include "NiagaraEditorStyle.h"
#include "Framework/Text/SlateTextRun.h"

TSharedRef<FNiagaraHLSLSyntaxHighlighter> FNiagaraHLSLSyntaxHighlighter::Create(const FSyntaxTextStyle& InSyntaxTextStyle)
{
	return MakeShareable(new FNiagaraHLSLSyntaxHighlighter(CreateTokenizer(), InSyntaxTextStyle));
}

FNiagaraHLSLSyntaxHighlighter::FSyntaxTextStyle FNiagaraHLSLSyntaxHighlighter::GetSyntaxTextStyle()
{
	return FSyntaxTextStyle( FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Normal"),
			FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Operator"),
			FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Keyword"),
			FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.String"),
			FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Number"),
			FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Comment"),
			FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.PreProcessorKeyword"),
			FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Error"));
}


FNiagaraHLSLSyntaxHighlighter::FNiagaraHLSLSyntaxHighlighter(TSharedPtr<ISyntaxTokenizer> InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle) :
	FHLSLSyntaxHighlighterMarshaller(MoveTemp(InTokenizer), InSyntaxTextStyle)
{
}
