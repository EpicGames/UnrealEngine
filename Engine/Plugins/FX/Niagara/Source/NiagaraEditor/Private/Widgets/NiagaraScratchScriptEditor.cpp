// Copyright Epic Games, Inc. All Rights Reserved.

//-TODO: Some platforms require an offline compile to get instruction counts, this needs to be handled

#include "NiagaraScratchScriptEditor.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraSystemEditorDocumentsViewModel.h"
#include "NiagaraEditorStyle.h"
#include "Toolkits/NiagaraSystemToolkit.h"
#include "Widgets/Input/SButton.h"


#define LOCTEXT_NAMESPACE "NiagaraScratchPadScriptEditor"

void SNiagaraScratchPadScriptEditor::Construct(const FArguments& InArgs, TSharedRef<FNiagaraScratchPadScriptViewModel> InScriptViewModel)
{

	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 3.0f, 5.0f, 3.0f)
				[
					SNew(SButton)
					.OnClicked(this, &SNiagaraScratchPadScriptEditor::OnApplyButtonClicked)
					.ToolTipText(LOCTEXT("ApplyButtonToolTip", "Apply the current changes to this script.  This will update the selection stack UI and compile neccessary scripts."))
					.IsEnabled(this, &SNiagaraScratchPadScriptEditor::GetApplyButtonIsEnabled)
					.ContentPadding(FMargin(3.0f, 0.0f))
					.Content()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f, 2.0f, 2.0f, 2.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("AssetEditor.Apply"))
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.0f, 2.0f, 2.0f, 3.0f)
						[
							SNew(STextBlock)
							.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ScratchPad.EditorHeaderText")
							.Text(LOCTEXT("ApplyButtonLabel", "Apply"))
						]
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 3.0f, 5.0f, 3.0f)
				[
					SNew(SButton)
					.OnClicked(this, &SNiagaraScratchPadScriptEditor::OnApplyAndSaveButtonClicked)
					.ToolTipText(LOCTEXT("ApplyAndSaveButtonToolTip", "Apply the current changes to this script and save.  This will update the selection stack UI and compile neccessary scripts."))
					.IsEnabled(this, &SNiagaraScratchPadScriptEditor::GetApplyButtonIsEnabled)
					.ContentPadding(FMargin(3.0f, 0.0f))
					.Content()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f, 2.0f, 2.0f, 2.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("AssetEditor.Apply"))
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.0f, 2.0f, 2.0f, 3.0f)
						[
							SNew(STextBlock)
							.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ScratchPad.EditorHeaderText")
							.Text(LOCTEXT("ApplyAndSaveButtonLabel", "Apply & Save"))
						]
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(5, 0, 2, 0)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ScratchPad.EditorHeaderText")
					.Text(this, &SNiagaraScratchPadScriptEditor::GetNameText)
					.ToolTipText(this, &SNiagaraScratchPadScriptEditor::GetNameToolTipText)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0, 0, 2, 0)
				[
					SNew(STextBlock)
					.Visibility(this, &SNiagaraScratchPadScriptEditor::GetUnappliedChangesVisibility)
					.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ScratchPad.EditorHeaderText")
					.Text(FText::FromString(TEXT("*")))
					.ToolTipText(this, &SNiagaraScratchPadScriptEditor::GetNameToolTipText)
				]
			]
			+ SVerticalBox::Slot()
			[
				SAssignNew(Graph, SNiagaraScriptGraph, InScriptViewModel->GetGraphViewModel())
				.ZoomToFitOnLoad(true)
				.ShowHeader(false)
			]
		];

	SetViewModel(InScriptViewModel);
}

void SNiagaraScratchPadScriptEditor::SetViewModel(TSharedPtr<FNiagaraScratchPadScriptViewModel> InScriptViewModel)
{
	ClearHandles();

	ScriptViewModel = InScriptViewModel;

	if (ScriptViewModel.IsValid())
	{
		if (Graph.IsValid() && Graph->GetViewModel() != InScriptViewModel->GetGraphViewModel())
		{
			Graph->UpdateViewModel(InScriptViewModel->GetGraphViewModel());
		}
		NodeIDHandle = ScriptViewModel.Pin()->OnNodeIDFocusRequested().AddLambda(
			[this](FNiagaraScriptIDAndGraphFocusInfo* FocusInfo)
			{
				if (Graph.IsValid() && FocusInfo != nullptr)
				{
					Graph->FocusGraphElement(FocusInfo->GetScriptGraphFocusInfo().Get());
				}
			}
		);

		PinIDHandle = ScriptViewModel.Pin()->OnPinIDFocusRequested().AddLambda(
			[this](FNiagaraScriptIDAndGraphFocusInfo* FocusInfo)
			{
				if (Graph.IsValid() && FocusInfo != nullptr)
				{
					Graph->FocusGraphElement(FocusInfo->GetScriptGraphFocusInfo().Get());
				}
			}
		);
	}
}


void FNiagaraGraphTabHistory::EvokeHistory(TSharedPtr<FTabInfo> InTabInfo, bool bPrevTabMatches)
{
	FWorkflowTabSpawnInfo SpawnInfo;
	SpawnInfo.Payload = Payload;
	SpawnInfo.TabInfo = InTabInfo;

	if (bPrevTabMatches)
	{
		TSharedPtr<SDockTab> DockTab = InTabInfo->GetTab().Pin();
		GraphEditor = StaticCastSharedRef<SNiagaraScratchPadScriptEditor>(DockTab->GetContent());
	}
	else
	{
		TSharedRef< SNiagaraScratchPadScriptEditor > GraphEditorRef = StaticCastSharedRef< SNiagaraScratchPadScriptEditor >(FactoryPtr.Pin()->CreateTabBody(SpawnInfo));
		GraphEditor = GraphEditorRef;
		TSharedPtr<SDockTab> TabPtr = InTabInfo->GetTab().Pin();
		FactoryPtr.Pin()->UpdateTab(TabPtr, SpawnInfo, GraphEditorRef);
	}
}

void FNiagaraGraphTabHistory::SaveHistory()
{
	if (IsHistoryValid())
	{
		check(GraphEditor.IsValid());
		TSharedPtr<SNiagaraScratchPadScriptEditor> Editor = GraphEditor.Pin();
		if (Editor->GetGraphEditor())
		{
			FVector2f ViewLocation;
			Editor->GetGraphEditor()->GetViewLocation(ViewLocation, SavedZoomAmount);
			SavedLocation = ViewLocation;
			Editor->GetGraphEditor()->GetViewBookmark(SavedBookmarkId);
		}
	}
}

void FNiagaraGraphTabHistory::RestoreHistory()
{
	if (IsHistoryValid())
	{
		check(GraphEditor.IsValid());
		TSharedPtr<SNiagaraScratchPadScriptEditor> Editor = GraphEditor.Pin();
		if (Editor->GetGraphEditor())
			Editor->GetGraphEditor()->SetViewLocation(SavedLocation, SavedZoomAmount, SavedBookmarkId);
	}
}
void FDocumentTabFactory_NiagaraScratchPad::OnTabActivated(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SNiagaraScratchPadScriptEditor> GraphEditor = StaticCastSharedRef<SNiagaraScratchPadScriptEditor>(Tab->GetContent());
	EditorPtr.Pin()->GetSystemViewModel()->GetDocumentViewModel()->SetActiveDocumentTab(Tab);
}

void FDocumentTabFactory_NiagaraScratchPad::OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SNiagaraScratchPadScriptEditor> GraphEditor = StaticCastSharedRef<SNiagaraScratchPadScriptEditor>(Tab->GetContent());
	//EditorPtr.Pin()->OnGraphEditorBackgrounded(GraphEditor);
}

void FDocumentTabFactory_NiagaraScratchPad::OnTabRefreshed(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SNiagaraScratchPadScriptEditor> GraphEditor = StaticCastSharedRef<SNiagaraScratchPadScriptEditor>(Tab->GetContent());
	GraphEditor->GetGraphEditor()->NotifyGraphChanged();
}

void FDocumentTabFactory_NiagaraScratchPad::SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const
{
	// Do nothing here for now
	/*
	TSharedRef<SNiagaraScratchPadScriptEditor> GraphEditor = StaticCastSharedRef<SNiagaraScratchPadScriptEditor>(Tab->GetContent());

	FVector2D ViewLocation;
	float ZoomAmount;
	GraphEditor->GetGraphEditor()->GetViewLocation(ViewLocation, ZoomAmount);

	UEdGraph* Graph = Payload->IsValid() ? FTabPayload_UObject::CastChecked<UEdGraph>(Payload) : nullptr;
	*/
}

FDocumentTabFactory_NiagaraScratchPad::FDocumentTabFactory_NiagaraScratchPad(TSharedPtr<class FNiagaraSystemToolkit> InToolkit, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback)
	: FDocumentTabFactoryForObjects<UEdGraph>(TEXT("NiagaraSystemEditor_ScratchPad"), InToolkit->AsShared())
, EditorPtr(InToolkit)
, OnCreateGraphEditorWidget(CreateGraphEditorWidgetCallback)
{
	
}

TSharedRef<SWidget> FDocumentTabFactory_NiagaraScratchPad::CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	check(Info.TabInfo.IsValid());

	// We need to register the tab being closed as we need to clear out the active selection logic when that happens.
	// There may be better places to put this, but this works well in practice.
	SDockTab::FOnTabClosedCallback TabClosedCallback = SDockTab::FOnTabClosedCallback::CreateLambda([this](TSharedRef<SDockTab> DockTab)
		{
			TSharedPtr<FNiagaraSystemToolkit> Toolkit = EditorPtr.Pin();
			if (Toolkit.IsValid())
				Toolkit->GetSystemViewModel()->GetDocumentViewModel()->SetActiveDocumentTab(nullptr);
		});
	TSharedPtr<SDockTab> TabPtr = Info.TabInfo->GetTab().Pin();
	if (TabPtr.IsValid())
		TabPtr->SetOnTabClosed(TabClosedCallback);

	// Create the widget!
	return OnCreateGraphEditorWidget.Execute(Info.TabInfo.ToSharedRef(), DocumentID);
}

const FSlateBrush* FDocumentTabFactory_NiagaraScratchPad::GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	return FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "Tab.ScratchPad").GetSmallIcon();
}

TSharedRef<FGenericTabHistory> FDocumentTabFactory_NiagaraScratchPad::CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload)
{
	return MakeShareable(new FNiagaraGraphTabHistory(SharedThis(this), Payload));
}

bool FDocumentTabFactory_NiagaraScratchPad::IsPayloadValid(TSharedRef<FTabPayload> Payload) const
{
	bool bBaseResult = FDocumentTabFactoryForObjects<UEdGraph>::IsPayloadValid(Payload);

	if(bBaseResult == false)
	{
		return false;
	}

	UEdGraph* ObjectPayload = FTabPayload_UObject::CastChecked<UEdGraph>(Payload);

	// if our script view models don't contain the tab's graph, we know the tab is outdated and should be closed
	// this can happen if we undo/redo the creation of a scratch pad, where the data is deleted but the tab is still displaying the graph
	for(TSharedRef<FNiagaraScratchPadScriptViewModel> ScriptViewModel : EditorPtr.Pin()->GetSystemViewModel()->GetScriptScratchPadViewModel()->GetScriptViewModels())
	{
		if(ScriptViewModel->GetEditableGraphs().Contains(ObjectPayload))
		{
			return true;
		}
	}
	
	return false;
}

TAttribute<FText> FDocumentTabFactory_NiagaraScratchPad::ConstructTabNameForObject(UEdGraph* DocumentID) const 
{
	UNiagaraScript* Script = DocumentID->GetTypedOuter<UNiagaraScript>();
	TSharedPtr<FNiagaraSystemViewModel> SysViewModel = EditorPtr.Pin()->GetSystemViewModel();

	if (Script)
	{
		if (SysViewModel.IsValid())
		{
			TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchScriptViewModel = UNiagaraSystemEditorDocumentsViewModel::GetScratchPadViewModelFromGraph(SysViewModel.Get(), DocumentID);
			if (ScratchScriptViewModel)
			{
				return TAttribute<FText>::CreateSP(ScratchScriptViewModel.Get(), &FNiagaraScratchPadScriptViewModel::GetDisplayName);
			}
		}
	}
	return FText();
}

TSharedRef<SDockTab> FDocumentTabFactory_NiagaraScratchPad::OnSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const
{
	TSharedRef<SDockTab> TabRef = FDocumentTabFactoryForObjects<UEdGraph>::OnSpawnTab(SpawnArgs, WeakTabManager);
	return TabRef;
}


#undef LOCTEXT_NAMESPACE
