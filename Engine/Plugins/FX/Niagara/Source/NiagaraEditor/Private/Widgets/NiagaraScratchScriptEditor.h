// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraGraph.h"
#include "Widgets/SCompoundWidget.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "NiagaraScriptSource.h"
#include "Widgets/SNiagaraScriptGraph.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"

class SNiagaraScratchPadScriptEditor : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraScratchPadScriptEditor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FNiagaraScratchPadScriptViewModel> InScriptViewModel);

	~SNiagaraScratchPadScriptEditor()
	{
		ClearHandles();
	}

	TSharedPtr<SGraphEditor> GetGraphEditor()
	{
		if (Graph)
			return Graph->GetGraphEditor();
		else
			return nullptr;
	}

	TSharedPtr<FNiagaraScratchPadScriptViewModel> GetViewModel()
	{
		return ScriptViewModel.Pin();
	}

	void SetViewModel(TSharedPtr<FNiagaraScratchPadScriptViewModel> InViewModel);


private:

	void ClearHandles()
	{
		if (ScriptViewModel.IsValid())
		{
			ScriptViewModel.Pin()->OnNodeIDFocusRequested().Remove(NodeIDHandle);
			ScriptViewModel.Pin()->OnPinIDFocusRequested().Remove(PinIDHandle);
		}
	}

	FText GetNameText() const
	{
		return ScriptViewModel.Pin()->GetDisplayName();
	}

	FText GetNameToolTipText() const
	{
		return ScriptViewModel.Pin()->GetToolTip();
	}

	EVisibility GetUnappliedChangesVisibility() const
	{
		return ScriptViewModel.Pin()->HasUnappliedChanges() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FReply OnApplyButtonClicked()
	{
		ScriptViewModel.Pin()->ApplyChanges();
		return FReply::Handled();
	}

	FReply OnApplyAndSaveButtonClicked()
	{
		ScriptViewModel.Pin()->ApplyChangesAndSave();
		return FReply::Handled();
	}

	bool GetApplyButtonIsEnabled() const
	{
		return ScriptViewModel.Pin()->HasUnappliedChanges();
	}

private:
	TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModel;
	TSharedPtr<SNiagaraScriptGraph> Graph;

	FDelegateHandle NodeIDHandle;
	FDelegateHandle PinIDHandle;
};


/////////////////////////////////////////////////////
// FNiagaraGraphTabHistory

struct FNiagaraGraphTabHistory : public FGenericTabHistory
{
public:
	/**
	 * @param InFactory		The factory used to regenerate the content
	 * @param InPayload		The payload object used to regenerate the content
	 */
	FNiagaraGraphTabHistory(TSharedPtr<FDocumentTabFactory> InFactory, TSharedPtr<FTabPayload> InPayload)
		: FGenericTabHistory(InFactory, InPayload)
		, SavedLocation(FVector2f::ZeroVector)
		, SavedZoomAmount(INDEX_NONE)
	{

	}

	virtual void EvokeHistory(TSharedPtr<FTabInfo> InTabInfo, bool bPrevTabMatches) override;

	virtual void SaveHistory() override;

	virtual void RestoreHistory() override;

private:
	/** The graph editor represented by this history node. While this node is inactive, the graph editor is invalid */
	TWeakPtr< class SNiagaraScratchPadScriptEditor > GraphEditor;
	/** Saved location the graph editor was at when this history node was last visited */
	FDeprecateSlateVector2D SavedLocation;
	/** Saved zoom the graph editor was at when this history node was last visited */
	float SavedZoomAmount;
	/** Saved bookmark ID the graph editor was at when this history node was last visited */
	FGuid SavedBookmarkId;
};


struct FDocumentTabFactory_NiagaraScratchPad : public FDocumentTabFactoryForObjects<UEdGraph>
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SNiagaraScratchPadScriptEditor>, FOnCreateGraphEditorWidget, TSharedRef<FTabInfo>, UEdGraph*);
public:
	FDocumentTabFactory_NiagaraScratchPad(TSharedPtr<class FNiagaraSystemToolkit> InToolkit, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback);

	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override;

	virtual void OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const override;

	virtual void OnTabRefreshed(TSharedPtr<SDockTab> Tab) const override;
	
	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override;

	virtual TSharedRef<SDockTab> OnSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const override;

protected:
	virtual TAttribute<FText> ConstructTabNameForObject(UEdGraph* DocumentID) const override;

	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;

	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;

	virtual TSharedRef<FGenericTabHistory> CreateTabHistoryNode(TSharedPtr<FTabPayload> Payload) override;

	/** If the payload (a scratch pad script's graph) is invalid, it will be closed whenever the document tracker refreshes or cleans tabs. */
	virtual bool IsPayloadValid(TSharedRef<FTabPayload> Payload) const override;
protected:
	TWeakPtr<FNiagaraSystemToolkit> EditorPtr;
	FOnCreateGraphEditorWidget OnCreateGraphEditorWidget;
};
