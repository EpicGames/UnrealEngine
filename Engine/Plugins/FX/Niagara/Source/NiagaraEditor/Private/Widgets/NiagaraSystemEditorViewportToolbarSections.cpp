// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSystemEditorViewportToolbarSections.h"

#include "EditorViewportCommands.h"
#include "NiagaraEditorCommands.h"
#include "NiagaraEditorStyle.h"
#include "SNiagaraSystemViewport.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "NiagaraSystemEditorViewportToolbarSections"

namespace UE::NiagaraSystemEditor
{
	void FillShowSubmenu(UToolMenu* InMenu, bool bInShowViewportStatsToggle)
	{
		const FNiagaraEditorCommands& Commands = FNiagaraEditorCommands::Get();

		if (bInShowViewportStatsToggle)
		{
			FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);
			UnnamedSection.AddSubMenu(
				"ViewportStats",
				LOCTEXT("ViewportStatsSubMenu", "Viewport Stats"),
				LOCTEXT("CameraSubmenuTooltip", "Camera options"),
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* Submenu) -> void
					{
						FToolMenuSection& CommonStatsSection =
							Submenu->FindOrAddSection("CommonStats", LOCTEXT("CommonStatsLabel", "Common Stats"));

						CommonStatsSection.AddMenuEntry(
							FEditorViewportCommands::Get().ToggleStats, LOCTEXT("ViewportStatsLabel", "Show Stats")
						);

						CommonStatsSection.AddSeparator(NAME_None);

						CommonStatsSection.AddMenuEntry(
							FEditorViewportCommands::Get().ToggleFPS, LOCTEXT("ViewportFPSLabel", "Show FPS")
						);
					}
				)
			);
		}

		FToolMenuSection& CommonShowFlagsSection =
			InMenu->FindOrAddSection("CommonShowFlags", LOCTEXT("CommonShowFlagsLabel", "Common Show Flags"));

		CommonShowFlagsSection.AddMenuEntry(Commands.ToggleEmitterExecutionOrder);
		CommonShowFlagsSection.AddMenuEntry(Commands.ToggleGpuTickInformation);
		CommonShowFlagsSection.AddMenuEntry(Commands.ToggleInstructionCounts);
		CommonShowFlagsSection.AddMenuEntry(Commands.ToggleMemoryInfo);
		CommonShowFlagsSection.AddMenuEntry(Commands.ToggleParticleCounts);
		CommonShowFlagsSection.AddMenuEntry(Commands.ToggleStatelessInfo);
	}

	TSharedPtr<SNiagaraSystemViewport> ResolveSystemViewport(UToolMenu* InMenu)
	{
		if ( UUnrealEdViewportToolbarContext* ViewportToolbarContext = InMenu->FindContext<UUnrealEdViewportToolbarContext>() )
		{
			return StaticCastSharedPtr<SNiagaraSystemViewport>(ViewportToolbarContext->Viewport.Pin());
		}
		return nullptr;
	}

	void AddMotionSettingsToSection(FToolMenuSection& InSection)
	{
		InSection.AddSubMenu(
			"MotionOptions",
			LOCTEXT("MotionOptionsSubMenu", "Motion Options"),
			LOCTEXT("MotionOptionsSubMenu_ToolTip", "Set Motion Options for the Niagara Component"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InMenu)
				{
					FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);

					UnnamedSection.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleMotion);

					TSharedPtr<SNiagaraSystemViewport> NiagaraSystemViewport = ResolveSystemViewport(InMenu);
					if (!NiagaraSystemViewport)
					{
						return;
					}

					FToolMenuEntry MotionRateEntry = FToolMenuEntry::InitWidget(
						"MotionRate",
						SNew(SSpinBox<float>)
						.IsEnabled(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::IsMotionEnabled)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.MinSliderValue(0.0f)
						.MaxSliderValue(360.0f)
						.Value(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::GetMotionRate)
						.OnValueChanged(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::SetMotionRate),
						LOCTEXT("MotionSpeed", "Motion Speed")
					);

					FToolMenuEntry MotionRadiusEntry = FToolMenuEntry::InitWidget(
						"MotionRadius",
						SNew(SSpinBox<float>)
						.IsEnabled(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::IsMotionEnabled)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.MinSliderValue(0.0f)
						.MaxSliderValue(1000.0f)
						.Value(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::GetMotionRadius)
						.OnValueChanged(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::SetMotionRadius),
						LOCTEXT("MotionRadius", "Motion Radius")
					);

					UnnamedSection.AddEntry(MotionRateEntry);
					UnnamedSection.AddEntry(MotionRadiusEntry);
				}
			),
			false,
			FSlateIcon()
		);
	}

	void AddFloorSettingsToSection(FToolMenuSection& InSection)
	{
		InSection.AddSubMenu(
			"FloorOptions",
			LOCTEXT("FloorOptionsSubMenu", "Floor Options"),
			LOCTEXT("FloorOptionsSubMenu_ToolTip", "Set Floor Options for the preview window"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InMenu)
				{
					TSharedPtr<SNiagaraSystemViewport> NiagaraSystemViewport = ResolveSystemViewport(InMenu);
					if (!NiagaraSystemViewport)
					{
						return;
					}

					FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);

					UnnamedSection.AddMenuEntry(
						NAME_None,
						LOCTEXT("FloorEnable", "Floor Enabled"),
						LOCTEXT("FloorEnableToolTip", "Controls visibility of the floor"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::ToggleFloorEnabled),
							FCanExecuteAction(),
							FIsActionChecked::CreateSP(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::IsFloorEnabled)
						),
						EUserInterfaceActionType::ToggleButton
					);

					UnnamedSection.AddEntry(
						FToolMenuEntry::InitWidget(
							NAME_None,
							SNew(SSpinBox<float>)
							.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
							.IsEnabled(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::IsFloorEnabled)
							.Value(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::GetFloorOffset)
							.OnValueChanged(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::SetFloorOffset),
							LOCTEXT("FloorHeight", "Floor Height")
						)
					);
				}
			),
			false,
			FSlateIcon()
		);
	}

	void AddFocusSettingsToSection(FToolMenuSection& InSection)
	{
		InSection.AddSubMenu(
			"FocusOptions",
			LOCTEXT("FocusOptionsSubMenu", "Focus Options"),
			LOCTEXT("FocusOptionsSubMenu_ToolTip", "Adjusts how we focus on the Niagara component"),
			FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InMenu)
				{
					TSharedPtr<SNiagaraSystemViewport> NiagaraSystemViewport = ResolveSystemViewport(InMenu);
					if (!NiagaraSystemViewport)
					{
						return;
					}

					FToolMenuSection& UnnamedSection = InMenu->FindOrAddSection(NAME_None);

					UnnamedSection.AddMenuEntry(
						NAME_None,
						LOCTEXT("FocusOverrideEnable", "Focus Override Enabled"),
						LOCTEXT("FocusOverrideEnableToolTip", "When enable we will override the default focus method, which is to use the bounding box."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::ToggleManualFocusDistance),
							FCanExecuteAction(),
							FIsActionChecked::CreateSP(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::IsManualFocusDistanceEnabled)
						),
						EUserInterfaceActionType::ToggleButton
					);

					UnnamedSection.AddEntry(
						FToolMenuEntry::InitWidget(
							NAME_None,
							SNew(SSpinBox<float>)
							.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
							.IsEnabled(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::IsManualFocusDistanceEnabled)
							.Value(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::GetManualFocusDistance)
							.OnValueChanged(NiagaraSystemViewport.Get(), &SNiagaraSystemViewport::SetManualFocusDistance),
							LOCTEXT("FocusDistance", "Focus Distance")
						)
					);
				}
			),
			false,
			FSlateIcon()
		);
	}
}

FToolMenuEntry UE::NiagaraSystemEditor::CreateShowSubmenu()
{
	return UE::UnrealEd::CreateShowSubmenu(FNewToolMenuDelegate::CreateStatic(&UE::NiagaraSystemEditor::FillShowSubmenu, true));
}

void UE::NiagaraSystemEditor::ExtendPreviewSceneSettingsSubmenu(FName InSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InSubmenuName);
	if (!Submenu)
	{
		return;
	}

	FToolMenuInsert AudioInsertPosition("AssetViewerProfileSelectionSection", EToolMenuInsertType::Before);
	FToolMenuSection& PreviewControlsSection = Submenu->FindOrAddSection(
		"AssetViewerPreviewControlsSection", LOCTEXT("AssetViewerPreviewControlsSectionLabel", "Preview Controls"), AudioInsertPosition
	);

	UE::NiagaraSystemEditor::AddMotionSettingsToSection(PreviewControlsSection);
	UE::NiagaraSystemEditor::AddFloorSettingsToSection(PreviewControlsSection);
	UE::NiagaraSystemEditor::AddFocusSettingsToSection(PreviewControlsSection);

	//Adds toggle origin axis to menu
	FToolMenuSection& ProfileOptionsSection = Submenu->FindOrAddSection(
		"PreviewSceneSettings", LOCTEXT("AssetViewerProfileOptionsSectionLabel", "Preview Scene Options")
	);

	ProfileOptionsSection.AddMenuEntry(
		FNiagaraEditorCommands::Get().ToggleOriginAxis, 
		TAttribute<FText>(), 
		TAttribute<FText>(),
		FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.ToggleOriginAxis")
	);
}

#undef LOCTEXT_NAMESPACE
