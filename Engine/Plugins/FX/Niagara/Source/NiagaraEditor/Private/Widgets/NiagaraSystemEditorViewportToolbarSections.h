// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "ToolMenuEntry.h"

class SNiagaraSystemViewport;

namespace UE::NiagaraSystemEditor
{
	extern FToolMenuEntry CreateShowSubmenu();
	extern void ExtendPreviewSceneSettingsSubmenu(FName InSubmenuName);
}
