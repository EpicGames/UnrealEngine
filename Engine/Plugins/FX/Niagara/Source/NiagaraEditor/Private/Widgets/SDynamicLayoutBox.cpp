// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDynamicLayoutBox.h"

SDynamicLayoutBox::FNamedWidgetProvider::FNamedWidgetProvider()
{
}

SDynamicLayoutBox::FNamedWidgetProvider::FNamedWidgetProvider(SDynamicLayoutBox::FOnGenerateNamedWidget InGenerateNamedWidget)
{
	GenerateNamedWidget = InGenerateNamedWidget;
}

TSharedRef<SWidget> SDynamicLayoutBox::FNamedWidgetProvider::GetNamedWidget(FName InWidgetName) const
{
	if (GenerateNamedWidget.IsBound())
	{
		TSharedRef<SWidget>* CachedNamedWidget = NameToGeneratedWidgetCache.Find(InWidgetName);
		if (CachedNamedWidget != nullptr)
		{
			return *CachedNamedWidget;
		}
		else
		{
			TSharedRef<SWidget> NamedWidget = GenerateNamedWidget.Execute(InWidgetName);
			NameToGeneratedWidgetCache.Add(InWidgetName, NamedWidget);
			return NamedWidget;
		}
	}
	return SNullWidget::NullWidget;
}

void SDynamicLayoutBox::Construct(const FArguments& InArgs)
{
	if (InArgs._GenerateNamedWidget.IsBound())
	{
		NamedWidgetProvider = FNamedWidgetProvider(InArgs._GenerateNamedWidget);
	}
	GenerateNamedLayout = InArgs._GenerateNamedLayout;
	ChooseLayout = InArgs._ChooseLayout;
}

void SDynamicLayoutBox::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (GetCachedGeometry() != AllottedGeometry)
	{
		FName NewLayoutName;
		if (ChooseLayout.IsBound())
		{
			NewLayoutName = ChooseLayout.Execute();
		}
		if (GenerateNamedLayout.IsBound() && (CurrentLayout.IsSet() == false || CurrentLayout.GetValue() != NewLayoutName))
		{
			ChildSlot[GenerateNamedLayout.Execute(NewLayoutName, NamedWidgetProvider)];
			CurrentLayout = NewLayoutName;
		}
	}
}