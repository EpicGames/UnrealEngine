// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraBakerTimelineWidget.h"
//#include "SNiagaraBakerViewport.h"
//#include "NiagaraEditorCommon.h"
//#include "NiagaraComponent.h"
//#include "NiagaraSystem.h"
#include "ViewModels/NiagaraBakerViewModel.h"

#include "Framework/Application/SlateApplication.h"
#include "Fonts/FontMeasure.h"
//#include "Modules/ModuleManager.h"
//#include "Widgets/SBoxPanel.h"
//#include "Widgets/SOverlay.h"
//#include "Widgets/SViewport.h"
//#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "NiagaraBakerTimelineWidget"

void SNiagaraBakerTimelineWidget::Construct(const FArguments& InArgs)
{
	WeakViewModel = InArgs._WeakViewModel;
}

int32 SNiagaraBakerTimelineWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	if ( !ViewModel )
	{
		return LayerId;
	}

	// Used to track the layer ID we will return.
	int32 RetLayerId = LayerId;
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const FLinearColor ColorAndOpacitySRGB = InWidgetStyle.GetColorAndOpacityTint();
	static const FLinearColor SelectedBarColor(FLinearColor::White);

	// Gather information to display
	const int NumFrames = ViewModel->GetCurrentOutputNumFrames();
	const FNiagaraBakerOutputFrameIndices OutputFrameIndices = ViewModel->GetCurrentOutputFrameIndices(RelativeTime);

	//////////////////////////////////////////////////////////////////////////
	// Draw boxes to show frames
	{
		const FSlateBrush* BoxBrush = FAppStyle::GetBrush("Sequencer.Section.BackgroundTint");
		const FLinearColor BoxTints[2] = { FLinearColor(0.5f, 0.5f, 0.5f, 1.0f), FLinearColor(0.3f, 0.3f, 0.3f, 1.0f) };
		const FLinearColor CurrentTint(0.5f, 0.5f, 1.0f, 1.0f);
		if (NumFrames > 0)
		{
			const float UStep = AllottedGeometry.Size.X / float(NumFrames);

			for (int32 i = 0; i < NumFrames; ++i)
			{
				const FVector2D BoxLocation(UStep * float(i), 0.0f);
				const FVector2D BoxSize(UStep, AllottedGeometry.Size.Y);
				const FLinearColor& Tint = (i == OutputFrameIndices.FrameIndexA) ? CurrentTint : BoxTints[i & 1];
				FSlateDrawElement::MakeBox(OutDrawElements, RetLayerId++, AllottedGeometry.ToPaintGeometry(BoxSize, FSlateLayoutTransform(BoxLocation)), BoxBrush, ESlateDrawEffect::None, Tint);
			}
		}
		else
		{
			FSlateDrawElement::MakeBox(OutDrawElements, RetLayerId++, AllottedGeometry.ToPaintGeometry(), BoxBrush, ESlateDrawEffect::None, BoxTints[0]);
		}
	}

	//////////////////////////////////////////////////////////////////////////
	// Draw a line to show the current time
	{
		TArray<FVector2D> LinePoints;
		LinePoints.AddUninitialized(2);

		const float LineXPos = AllottedGeometry.Size.X * OutputFrameIndices.NormalizedTime;
		LinePoints[0] = FVector2D(LineXPos, 0.0f);
		LinePoints[1] = FVector2D(LineXPos, AllottedGeometry.Size.Y);

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			RetLayerId++,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			FLinearColor::White
		);
	}
	return RetLayerId;
}

FReply SNiagaraBakerTimelineWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		const FVector2D LocalLocation = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		const float NormalizeTime = LocalLocation.X / MyGeometry.Size.X;

		ViewModel->SetDisplayTimeFromNormalized(NormalizeTime);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

//FReply SNiagaraBakerTimelineWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
//{
//	return FReply::Unhandled();
//}
//
//FReply SNiagaraBakerTimelineWidget::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
//{
//	return FReply::Unhandled();
//}

FVector2D SNiagaraBakerTimelineWidget::ComputeDesiredSize(float) const
{
	return FVector2D(8.0f, 8.0f);
}

#undef LOCTEXT_NAMESPACE
