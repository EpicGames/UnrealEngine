// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraBakerViewport.h"
#include "Application/ThrottleManager.h"
#include "ViewModels/NiagaraBakerViewModel.h"
#include "NiagaraBakerRenderer.h"
#include "NiagaraBatchedElements.h"
#include "NiagaraComponent.h"

#include "Engine/Canvas.h"
#include "Engine/Font.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "EditorViewportCommands.h"
#include "EngineModule.h"
#include "LegacyScreenPercentageDriver.h"
#include "ImageUtils.h"
#include "SEditorViewportToolBarMenu.h"
#include "TextureResource.h"

#define LOCTEXT_NAMESPACE "NiagaraBakerViewport"

//////////////////////////////////////////////////////////////////////////

class FNiagaraBakerViewportClient final : public FEditorViewportClient
{
public:
	FNiagaraBakerViewportClient(const TSharedRef<SNiagaraBakerViewport>& InOwnerViewport)
		: FEditorViewportClient(nullptr, nullptr, StaticCastSharedRef<SEditorViewport>(InOwnerViewport))
	{
		SetViewportType(ELevelViewportType::LVT_OrthoXZ);
	}

	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override
	{
		// Only allow movement when in preview mode
		auto ViewModel = WeakViewModel.Pin();
		UNiagaraBakerSettings* BakerSettings = ViewModel ? ViewModel->GetBakerSettings() : nullptr;
		if ( ViewModel->ShowRealtimePreview() && BakerSettings )
		{
			bool bForwardKeyState = false;
			bool bBackwardKeyState = false;
			bool bRightKeyState = false;
			bool bLeftKeyState = false;

			bool bLocalUpKeyState = false;
			bool bLocalDownKeyState = false;
			bool bZoomOutKeyState = false;
			bool bZoomInKeyState = false;

			bool bFocus = false;

			// Iterate through all key mappings to generate key state flags
			for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
			{
				EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex> (i);
				bForwardKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().Forward->GetActiveChord(ChordIndex)->Key);
				bBackwardKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().Backward->GetActiveChord(ChordIndex)->Key);
				bRightKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().Right->GetActiveChord(ChordIndex)->Key);
				bLeftKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().Left->GetActiveChord(ChordIndex)->Key);

				bLocalUpKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().LocalUp->GetActiveChord(ChordIndex)->Key);
				bLocalDownKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().LocalDown->GetActiveChord(ChordIndex)->Key);
				bZoomOutKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().FovZoomOut->GetActiveChord(ChordIndex)->Key);
				bZoomInKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().FovZoomIn->GetActiveChord(ChordIndex)->Key);

				bFocus |= Viewport->KeyState(FEditorViewportCommands::Get().FocusViewportToSelection->GetActiveChord(ChordIndex)->Key);
			}

			if ( BakerSettings->GetCurrentCamera().IsOrthographic() )
			{
				LocalMovement.X += bLeftKeyState ? KeyboardMoveSpeed : 0.0f;
				LocalMovement.X -= bRightKeyState ? KeyboardMoveSpeed : 0.0f;
				LocalMovement.Y += bBackwardKeyState ? KeyboardMoveSpeed : 0.0f;
				LocalMovement.Y -= bForwardKeyState ? KeyboardMoveSpeed : 0.0f;
			}
			else
			{
				//LocalMovement.X += bLeftKeyState ? KeyboardMoveSpeed : 0.0f;
				//LocalMovement.X -= bRightKeyState ? KeyboardMoveSpeed : 0.0f;
				//LocalMovement.Y += bLocalUpKeyState ? KeyboardMoveSpeed : 0.0f;
				//LocalMovement.Y -= bLocalDownKeyState ? KeyboardMoveSpeed : 0.0f;
				//LocalMovement.Z += bBackwardKeyState ? KeyboardMoveSpeed : 0.0f;
				//LocalMovement.Z -= bForwardKeyState ? KeyboardMoveSpeed : 0.0f;
			}

			LocalZoom += bZoomOutKeyState ? KeyboardMoveSpeed : 0.0f;
			LocalZoom -= bZoomInKeyState ? KeyboardMoveSpeed : 0.0f;

			// Focus
			if (bFocus)
			{
				FocusCamera();
			}
		}

		return true;
	}

	virtual bool InputAxis(const FInputKeyEventArgs& Args) override
	{
		const FKey& Key = Args.Key;
		FViewport* InViewport = Args.Viewport;
		const float Delta = Args.AmountDepressed;
		
		// Viewport movement only enabled when preview is enabled otherwise there is no feedback
		auto ViewModel = WeakViewModel.Pin();
		UNiagaraBakerSettings* BakerSettings = ViewModel ? ViewModel->GetBakerSettings() : nullptr;
		if ( ViewModel->ShowRealtimePreview() && BakerSettings && (Key == EKeys::MouseX || Key == EKeys::MouseY) )
		{
			if (BakerSettings->GetCurrentCamera().IsOrthographic())
			{
				if (InViewport->KeyState(EKeys::RightMouseButton))
				{
					// Zoom
					if (InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl))
					{
						LocalZoom += (Key == EKeys::MouseY) ? Delta : 0.0f;
					}
					// Aspect
					else if (InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt))
					{
						LocalAspect += (Key == EKeys::MouseY) ? Delta : 0.0f;
					}
					// Move
					else
					{
						LocalMovement.X += (Key == EKeys::MouseX) ? Delta : 0.0f;
						LocalMovement.Y += (Key == EKeys::MouseY) ? Delta : 0.0f;
					}
				}
			}
			else
			{
				// Zoom
				if (InViewport->KeyState(EKeys::RightMouseButton))
				{
					if (InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt))
					{
						LocalAspect += (Key == EKeys::MouseY) ? Delta : 0.0f;
					}
					else
					{
						LocalMovement.Z += (Key == EKeys::MouseY) ? Delta : 0.0f;
					}
				}
				// Middle button translate orbit location
				else if (InViewport->KeyState(EKeys::MiddleMouseButton))
				{
					LocalMovement.X += (Key == EKeys::MouseX) ? Delta : 0.0f;
					LocalMovement.Y += (Key == EKeys::MouseY) ? Delta : 0.0f;
				}
				// Rotation
				else if (InViewport->KeyState(EKeys::LeftMouseButton))
				{
					LocalRotation.X += (Key == EKeys::MouseX) ? Delta : 0.0f;
					LocalRotation.Y += (Key == EKeys::MouseY) ? Delta : 0.0f;
				}
			}
		}

		return true;
	}

	virtual void Tick(float DeltaSeconds) override
	{
		//FEditorViewportClient::Tick(DeltaSeconds);

		// Apply local movement
		auto ViewModel = WeakViewModel.Pin();
		UNiagaraBakerSettings* BakerSettings = ViewModel ? ViewModel->GetBakerSettings() : nullptr;
		if ( BakerSettings )
		{
			const FMatrix ViewMatrix = BakerSettings->GetViewMatrix().Inverse();
			const FVector XAxis = ViewMatrix.GetUnitAxis(EAxis::X);
			const FVector YAxis = ViewMatrix.GetUnitAxis(EAxis::Y);
			const FVector ZAxis = ViewMatrix.GetUnitAxis(EAxis::Z);

			FNiagaraBakerCameraSettings& CurrentCamera = BakerSettings->GetCurrentCamera();

			FVector WorldMovement = FVector::ZeroVector;
			if (CurrentCamera.IsOrthographic())
			{
				const FVector2D MoveSpeed = GetPreviewOrthoUnits();

				WorldMovement -= LocalMovement.X * MoveSpeed.X * XAxis;
				WorldMovement -= LocalMovement.Y * MoveSpeed.Y * YAxis;
				//WorldMovement += LocalMovement.Z * MoveSpeed * ZAxis;

				CurrentCamera.ViewportLocation += WorldMovement;
			}
			else
			{
				FRotator& WorldRotation = CurrentCamera.ViewportRotation;
				WorldRotation.Yaw = FRotator::ClampAxis(WorldRotation.Yaw + LocalRotation.X);
				WorldRotation.Roll = FMath::Clamp(WorldRotation.Roll + LocalRotation.Y, 0.0f, 180.0f);

				const float MoveSpeed = PerspectiveMoveSpeed;
				WorldMovement -= LocalMovement.X * MoveSpeed * XAxis;
				WorldMovement -= LocalMovement.Y * MoveSpeed * YAxis;
				//WorldMovement -= LocalMovement.Z * MoveSpeed * ZAxis;
				CurrentCamera.ViewportLocation += WorldMovement;

				CurrentCamera.OrbitDistance = FMath::Max(CurrentCamera.OrbitDistance + LocalMovement.Z, 0.01f);
			}

			if (!FMath::IsNearlyZero(LocalZoom))
			{
				if (CurrentCamera.IsPerspective())
				{
					CurrentCamera.FOV = FMath::Clamp(CurrentCamera.FOV + LocalZoom, 0.001f, 179.0f);
				}
				else
				{
					CurrentCamera.OrthoWidth = FMath::Max(CurrentCamera.OrthoWidth + LocalZoom, 1.0f);
				}
			}

			if (!FMath::IsNearlyZero(LocalAspect))
			{
				if (CurrentCamera.bUseAspectRatio)
				{
					CurrentCamera.AspectRatio = FMath::Max(CurrentCamera.AspectRatio + (LocalAspect / 50.0f), 0.01f);
				}
			}
		}

		// Clear data
		LocalMovement = FVector::ZeroVector;
		LocalZoom = 0.0f;
		LocalAspect = 0.0f;
		LocalRotation = FVector::ZeroVector;
	}

	/** FViewportClient interface */
	virtual void Draw(FViewport* InViewport, FCanvas* Canvas) override
	{
		Canvas->Clear(FLinearColor::Transparent);

		FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
		if (ViewModel == nullptr)
		{
			return;
		}

		UFont* DisplayFont = GetFont();
		const float FontHeight = DisplayFont->GetMaxCharHeight() + 1.0f;
		//const FVector2D TextStartOffset(5.0f, 30.0f);

		const UNiagaraBakerSettings* BakerSettings = ViewModel->GetBakerSettings();
		const UNiagaraBakerSettings* BakerGeneratedSettings = ViewModel->GetBakerGeneratedSettings();
		const float WorldTime = RelativeTime + BakerSettings->StartSeconds;

		// Update Baker Renderer to the correct time
		FNiagaraBakerRenderer& BakerRenderer = ViewModel->GetBakerRenderer();
		BakerRenderer.SetAbsoluteTime(WorldTime, ViewModel->ShowRealtimePreview());

		TUniquePtr<FNiagaraBakerOutputRenderer> BakerOutputRenderer;
		UNiagaraBakerOutput* BakerPreviewOutput = ViewModel->GetCurrentOutput();
		if (BakerPreviewOutput)
		{
			BakerOutputRenderer.Reset(FNiagaraBakerRenderer::GetOutputRenderer(BakerPreviewOutput->GetClass()));
		}

		UNiagaraBakerOutput* BakerGeneratedOutput = nullptr;
		if ( BakerOutputRenderer.IsValid() && BakerGeneratedSettings && BakerGeneratedSettings->Outputs.IsValidIndex(ViewModel->GetCurrentOutputIndex()) )
		{
			BakerGeneratedOutput = BakerGeneratedSettings->Outputs[ViewModel->GetCurrentOutputIndex()];
			if ((BakerGeneratedOutput->GetClass() != BakerPreviewOutput->GetClass()) ||
				!FMath::IsNearlyEqual(BakerSettings->DurationSeconds, BakerGeneratedSettings->DurationSeconds) )
			{
				BakerGeneratedOutput = nullptr;
			}
		}

		const float DPIScaleFactor = ShouldDPIScaleSceneCanvas() ? GetDPIScale() : 1.0f;
		const FIntRect ViewRect(
			FMath::Floor(Canvas->GetViewRect().Min.X / DPIScaleFactor) + 2,
			FMath::Floor(Canvas->GetViewRect().Min.Y / DPIScaleFactor) + 2,
			FMath::Floor(Canvas->GetViewRect().Max.X / DPIScaleFactor) - 2,
			FMath::Floor(Canvas->GetViewRect().Max.Y / DPIScaleFactor) - 2
		);
		if ( ViewRect.Width() <= 0 || ViewRect.Height() <= 0 )
		{
			return;
		}

		// Calculate view rects
		PreviewViewRect = FIntRect();
		GeneratedViewRect = FIntRect();
		bool bPreviewValid = false;
		bool bGeneratedValid = false;
		{
			const int32 ViewWidth = ViewModel->ShowRealtimePreview() && ViewModel->ShowBakedView() ? (ViewRect.Width() >> 1) - 1 : ViewRect.Width();

			if (ViewModel->ShowRealtimePreview())
			{
				PreviewViewRect = FIntRect(ViewRect.Min.X, ViewRect.Min.Y, ViewRect.Min.X + ViewWidth, ViewRect.Max.Y);
				if (BakerOutputRenderer.IsValid())
				{
					const FIntPoint PreviewSize = BakerOutputRenderer->GetPreviewSize(BakerPreviewOutput, PreviewViewRect.Size());
					if (PreviewSize.SizeSquared() > 0 )
					{
						bPreviewValid = true;
						PreviewViewRect = ConstrainRect(PreviewViewRect, PreviewSize);

						// Resize our render target for the output
						if (!PreviewRenderTarget || PreviewRenderTarget->SizeX != PreviewSize.X || PreviewRenderTarget->SizeY != PreviewSize.Y)
						{
							if (PreviewRenderTarget == nullptr)
							{
								PreviewRenderTarget = NewObject<UTextureRenderTarget2D>();
							}
							PreviewRenderTarget->ClearColor = FLinearColor::Transparent;
							PreviewRenderTarget->TargetGamma = 1.0f;
							PreviewRenderTarget->InitCustomFormat(PreviewSize.X, PreviewSize.Y, PF_FloatRGBA, false);
						}
					}
				}
			}

			if (ViewModel->ShowBakedView())
			{
				GeneratedViewRect = FIntRect(ViewRect.Max.X - ViewWidth, ViewRect.Min.Y, ViewRect.Max.X, ViewRect.Max.Y);
				if ( BakerGeneratedOutput )
				{
					const FIntPoint GeneratedSize = BakerOutputRenderer->GetGeneratedSize(BakerGeneratedOutput, GeneratedViewRect.Size());
					if (GeneratedSize.SizeSquared() > 0)
					{
						bGeneratedValid = true;
						GeneratedViewRect = ConstrainRect(GeneratedViewRect, GeneratedSize);

						// Resize our render target for the output
						if (!GeneratedRenderTarget || GeneratedRenderTarget->SizeX != GeneratedSize.X || GeneratedRenderTarget->SizeY != GeneratedSize.Y)
						{
							if (GeneratedRenderTarget == nullptr)
							{
								GeneratedRenderTarget = NewObject<UTextureRenderTarget2D>();
							}
							GeneratedRenderTarget->ClearColor = FLinearColor::Transparent;
							GeneratedRenderTarget->TargetGamma = 1.0f;
							GeneratedRenderTarget->InitCustomFormat(GeneratedSize.X, GeneratedSize.Y, PF_FloatRGBA, false);
						}
					}
				}
			}
		}

		// Determine which color channels to show
		const bool bRGBEnabled = ViewModel->IsChannelEnabled(ENiagaraBakerColorChannel::Red) || ViewModel->IsChannelEnabled(ENiagaraBakerColorChannel::Green) || ViewModel->IsChannelEnabled(ENiagaraBakerColorChannel::Blue);
		const bool bAEnabled = ViewModel->IsChannelEnabled(ENiagaraBakerColorChannel::Alpha);
		const bool bAlphaBlend = bAEnabled && bRGBEnabled;
		FMatrix ColorTransform;
		{
			FPlane RPlane = ViewModel->IsChannelEnabled(ENiagaraBakerColorChannel::Red)   ? FPlane(1.0f, 0.0f, 0.0f, 0.0f) : FPlane(0.0f, 0.0f, 0.0f, 0.0f);
			FPlane GPlane = ViewModel->IsChannelEnabled(ENiagaraBakerColorChannel::Green) ? FPlane(0.0f, 1.0f, 0.0f, 0.0f) : FPlane(0.0f, 0.0f, 0.0f, 0.0f);
			FPlane BPlane = ViewModel->IsChannelEnabled(ENiagaraBakerColorChannel::Blue)  ? FPlane(0.0f, 0.0f, 1.0f, 0.0f) : FPlane(0.0f, 0.0f, 0.0f, 0.0f);
			if (bAEnabled && !bRGBEnabled)
			{
				RPlane = FPlane(0.0f, 0.0f, 0.0f, 1.0f);
				GPlane = FPlane(0.0f, 0.0f, 0.0f, 1.0f);
				BPlane = FPlane(0.0f, 0.0f, 0.0f, 1.0f);
			}
			ColorTransform = FMatrix(RPlane, GPlane, BPlane, FPlane(0.0f, 0.0f, 0.0f, 1.0f));
		}

		// Render the realtime preview
		if (ViewModel->ShowRealtimePreview())
		{
			ClearViewArea(Canvas, PreviewViewRect);

			TOptional<FString> ErrorString;
			if ( BakerOutputRenderer.IsValid() && bPreviewValid )
			{
				BakerOutputRenderer->RenderPreview(BakerPreviewOutput, BakerRenderer, PreviewRenderTarget, ErrorString);
				if (ErrorString.IsSet() == false)
				{
					const FVector2D HalfPixel(0.5f / float(PreviewRenderTarget->GetSurfaceWidth()), 0.5f / float(PreviewRenderTarget->GetSurfaceHeight()));
					FCanvasTileItem TileItem(
						FVector2D(PreviewViewRect.Min.X, PreviewViewRect.Min.Y),
						PreviewRenderTarget->GetResource(),
						FVector2D(PreviewViewRect.Width(), PreviewViewRect.Height()),
						FVector2D(HalfPixel.X, HalfPixel.Y),
						FVector2D(1.0f - HalfPixel.X, 1.0f - HalfPixel.Y),
						FLinearColor::White
					);
					TileItem.BatchedElementParameters = new FBatchedElementNiagaraSimple(ColorTransform, bAlphaBlend);
					Canvas->DrawItem(TileItem);
				}
			}
			else
			{
				ErrorString = TEXT("No output selected or the preview is invalid.");
			}

			DrawViewportText(Canvas, PreviewViewRect, TEXT("Live Sim"), ErrorString.Get(FString()));

			DrawRectBorder(Canvas, PreviewViewRect);
		}

		// Render Generated View
		if (ViewModel->ShowBakedView())
		{
			ClearViewArea(Canvas, GeneratedViewRect);

			TOptional<FString> ErrorString;
			if ( BakerGeneratedOutput && bGeneratedValid )
			{
				BakerOutputRenderer->RenderGenerated(BakerGeneratedOutput, BakerRenderer, GeneratedRenderTarget, ErrorString);
				if (ErrorString.IsSet() == false)
				{
					const FVector2D HalfPixel(0.5f / float(GeneratedRenderTarget->GetSurfaceWidth()), 0.5f / float(GeneratedRenderTarget->GetSurfaceHeight()));
					FCanvasTileItem TileItem(
						FVector2D(GeneratedViewRect.Min.X, GeneratedViewRect.Min.Y),
						GeneratedRenderTarget->GetResource(),
						FVector2D(GeneratedViewRect.Width(), GeneratedViewRect.Height()),
						FVector2D(HalfPixel.X, HalfPixel.Y),
						FVector2D(1.0f - HalfPixel.X, 1.0f - HalfPixel.Y),
						FLinearColor::White
					);
					TileItem.BatchedElementParameters = new FBatchedElementNiagaraSimple(ColorTransform, bAlphaBlend);
					Canvas->DrawItem(TileItem);
				}
			}
			else
			{
				ErrorString = TEXT("No output or output not generated.\nPlease bake to generate the output");
			}

			DrawViewportText(Canvas, GeneratedViewRect, TEXT("Baked Sim"), ErrorString.Get(FString()));

			DrawRectBorder(Canvas, GeneratedViewRect);
		}
	}

	FIntRect ConstrainRect(const FIntRect& InRect, FIntPoint InSize) const
	{
		const float Scale = FMath::Min(float(InRect.Width()) /  float(InSize.X), float(InRect.Height()) / float(InSize.Y));
		const FIntPoint ScaledSize(FMath::FloorToInt(float(InSize.X) * Scale), FMath::FloorToInt(float(InSize.Y) * Scale));
		FIntPoint RectMin(
			InRect.Min.X + ((InRect.Width()  - ScaledSize.X) >> 1),
			InRect.Min.Y + ((InRect.Height() - ScaledSize.Y) >> 1)
		);

		return FIntRect(RectMin, RectMin + ScaledSize);
	}

	void ClearViewArea(FCanvas* Canvas, const FIntRect& InRect)
	{
		UTexture2D* Texture = nullptr;
		FLinearColor Color = ClearColor;
		FVector2D EndUV(1.0f, 1.0f);

		if (WeakViewModel.Pin()->IsCheckerboardEnabled())
		{
			Texture = GetCheckerboardTexture();
			Color = FLinearColor::White;
			EndUV.X = float(InRect.Width()) / float(FMath::Max(Texture->GetSizeX(), 1));
			EndUV.Y = float(InRect.Height()) / float(FMath::Max(Texture->GetSizeY(), 1));
		}
		Canvas->DrawTile(InRect.Min.X, InRect.Min.Y, InRect.Width(), InRect.Height(), 0.0f, 0.0f, EndUV.X, EndUV.Y, Color, Texture ? Texture->GetResource() : nullptr, false);
	}

	FVector2f GetStringSize(UFont* Font, const TCHAR* Text)
	{
		FVector2f MaxSize = FVector2f::ZeroVector;
		FVector2f CurrSize = FVector2f::ZeroVector;

		const float fAdvanceHeight = Font->GetMaxCharHeight();
		const TCHAR* PrevChar = nullptr;
		while (*Text)
		{
			if (*Text == '\n')
			{
				CurrSize.X = 0.0f;
				CurrSize.Y = CurrSize.Y + fAdvanceHeight;
				PrevChar = nullptr;
				++Text;
				continue;
			}

			float TmpWidth, TmpHeight;
			Font->GetCharSize(*Text, TmpWidth, TmpHeight);

			int8 CharKerning = 0;
			if (PrevChar)
			{
				CharKerning = Font->GetCharKerning(*PrevChar, *Text);
			}

			CurrSize.X += TmpWidth + CharKerning;
			MaxSize.X = FMath::Max(MaxSize.X, CurrSize.X);
			MaxSize.Y = FMath::Max(MaxSize.Y, CurrSize.Y + TmpHeight);

			PrevChar = Text++;
		}

		return MaxSize;
	}

	void DrawViewportText(FCanvas* Canvas, FIntRect ViewportRect, FStringView InfoString, FStringView ErrorString)
	{
		// Anything to render?
		const bool bShowInfoString = WeakViewModel.Pin()->ShowInfoText() && !InfoString.IsEmpty();
		const bool bShowErrorString = !ErrorString.IsEmpty();
		if ( !bShowInfoString && !bShowErrorString )
		{
			return;
		}

		// We need to flush to ensure previous data is gone from scene capture rendering
		Canvas->Flush_GameThread();

		UFont* Font = GetFont();

		if (bShowInfoString)
		{
			const FIntPoint TextPosition(ViewportRect.Min.X + 3, ViewportRect.Min.Y + 3);
			Canvas->DrawShadowedString(TextPosition.X, TextPosition.Y, InfoString.GetData(), Font, FLinearColor::White);
		}

		if (bShowErrorString)
		{
			const FVector2f StringSize = GetStringSize(Font, ErrorString.GetData());
			const FIntPoint TextCenter(ViewportRect.Min.X + (ViewportRect.Width() >> 1), ViewportRect.Min.Y + (ViewportRect.Height() >> 1));
			Canvas->DrawShadowedString(TextCenter.X - int32(StringSize.X * 0.5f), TextCenter.Y - int32(StringSize.Y * 0.5f), ErrorString.GetData(), Font, FLinearColor::White);
		}

		Canvas->Flush_GameThread();
		Canvas->SetRenderTargetScissorRect(FIntRect(0, 0, 0, 0));
	};

	void DrawRectBorder(FCanvas* Canvas, const FIntRect& Rect)
	{
		Canvas->DrawTile(Rect.Min.X - 1, Rect.Min.Y - 1, 1, Rect.Size().Y + 2, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::White);
		Canvas->DrawTile(Rect.Max.X + 1, Rect.Min.Y - 1, 1, Rect.Size().Y + 2, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::White);

		Canvas->DrawTile(Rect.Min.X - 1, Rect.Min.Y - 1, Rect.Size().X + 2, 1, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::White);
		Canvas->DrawTile(Rect.Min.X - 1, Rect.Max.Y + 1, Rect.Size().X + 2, 1, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::White);
	}

	UTexture2D* GetCheckerboardTexture()
	{
		if (CheckerboardTexture == nullptr)
		{
			CheckerboardTexture = FImageUtils::CreateCheckerboardTexture(CheckerboardColorOne, CheckerboardColorTwo, CheckerSize);
		}
		return CheckerboardTexture;
	}

	void DestroyCheckerboardTexture()
	{
		if (CheckerboardTexture)
		{
			if (CheckerboardTexture->GetResource())
			{
				CheckerboardTexture->ReleaseResource();
			}
			CheckerboardTexture->MarkAsGarbage();
			CheckerboardTexture = nullptr;
		}
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		FEditorViewportClient::AddReferencedObjects(Collector);

		Collector.AddReferencedObject(CheckerboardTexture);
		Collector.AddReferencedObject(PreviewRenderTarget);
		Collector.AddReferencedObject(GeneratedRenderTarget);
	}

	void FocusCamera()
	{
		auto ViewModel = WeakViewModel.Pin();
		UNiagaraBakerSettings* BakerSettings = ViewModel ? ViewModel->GetBakerSettings() : nullptr;
		UNiagaraComponent* NiagaraComponent = ViewModel->GetBakerRenderer().GetPreviewComponent();
		if (ViewModel->ShowRealtimePreview() && BakerSettings && NiagaraComponent )
		{
			//-TODO: Should take aspect ratio into account here
			const FBoxSphereBounds ComponentBounds = NiagaraComponent->CalcBounds(NiagaraComponent->GetComponentTransform());
			FNiagaraBakerCameraSettings& CurrentCamera = BakerSettings->GetCurrentCamera();
			if (CurrentCamera.IsOrthographic() )
			{
				CurrentCamera.ViewportLocation = ComponentBounds.Origin;
				CurrentCamera.OrthoWidth = ComponentBounds.SphereRadius * 2.0f;
			}
			else
			{
				const float HalfFOVRadians = FMath::DegreesToRadians(CurrentCamera.FOV) * 0.5f;
				const float CameraDistance = ComponentBounds.SphereRadius / FMath::Tan(HalfFOVRadians);
				//const FVector CameraOffset = BakerSettings->GetViewMatrix().Inverse().GetUnitAxis(EAxis::Z) * CameraDistance;
				//BakerSettings->CameraViewportLocation[(int)ENiagaraBakerViewMode::Perspective] = ComponentBounds.Origin - CameraOffset;
				CurrentCamera.ViewportLocation = ComponentBounds.Origin;
				CurrentCamera.OrbitDistance = CameraDistance;
			}
		}
	}

	FVector2D GetPreviewOrthoUnits() const
	{
		FVector2D OrthoUnits = FVector2D::ZeroVector;

		auto ViewModel = WeakViewModel.Pin();
		const UNiagaraBakerSettings* BakerSettings = ViewModel ? ViewModel->GetBakerSettings() : nullptr;
		if (BakerSettings && (PreviewViewRect.Area() > 0))
		{
			const FNiagaraBakerCameraSettings& CurrentCamera = BakerSettings->GetCurrentCamera();
			const float AspectRatioY = CurrentCamera.bUseAspectRatio ? CurrentCamera.AspectRatio : 1.0f;
			OrthoUnits.X = CurrentCamera.OrthoWidth / float(PreviewViewRect.Width());
			OrthoUnits.Y = CurrentCamera.OrthoWidth * AspectRatioY / float(PreviewViewRect.Height());
		}
		return OrthoUnits;
	}


	virtual UWorld* GetWorld() const override
	{
		auto ViewModel = WeakViewModel.Pin();
		return ViewModel ? ViewModel->GetBakerRenderer().GetPreviewComponent()->GetWorld() : nullptr;
	}

	UFont* GetFont() const { return GetStatsFont(); }

public:
	TObjectPtr<UTextureRenderTarget2D>						PreviewRenderTarget = nullptr;
	TObjectPtr<UTextureRenderTarget2D>						GeneratedRenderTarget = nullptr;

	FVector										LocalMovement = FVector::ZeroVector;
	float										LocalZoom = 0.0f;
	float										LocalAspect = 0.0f;
	FVector										LocalRotation = FVector::ZeroVector;
	float										KeyboardMoveSpeed = 5.0f;
	float										PerspectiveMoveSpeed = 2.0f;

	FIntRect									PreviewViewRect;
	FIntRect									GeneratedViewRect;

	TObjectPtr<UTexture2D>									CheckerboardTexture = nullptr;
	FColor										CheckerboardColorOne = FColor(128, 128, 128);
	FColor										CheckerboardColorTwo = FColor(64, 64, 64);
	int32										CheckerSize = 32;

	FLinearColor								ClearColor = FColor::Black;

	TWeakPtr<FNiagaraBakerViewModel>			WeakViewModel;
	float										RelativeTime = 0.0f;
	float										DeltaTime = 0.0f;

	//FNiagaraBakerRenderer						BakerRenderer;
};

//////////////////////////////////////////////////////////////////////////

void SNiagaraBakerViewport::Construct(const FArguments& InArgs)
{
	WeakViewModel = InArgs._WeakViewModel;

	SEditorViewport::FArguments ParentArgs;
	SEditorViewport::Construct(ParentArgs);
}

TSharedRef<FEditorViewportClient> SNiagaraBakerViewport::MakeEditorViewportClient()
{
	ViewportClient = MakeShareable(new FNiagaraBakerViewportClient(SharedThis(this)));
	ViewportClient->WeakViewModel = WeakViewModel;

	return ViewportClient.ToSharedRef();
}

void SNiagaraBakerViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	ViewportClient->bNeedsRedraw = true;

	if (FSlateThrottleManager::Get().IsAllowingExpensiveTasks())
	{
		ViewportClient->Tick(InDeltaTime);
		GEditor->UpdateSingleViewportClient(ViewportClient.Get(), /*bInAllowNonRealtimeViewportToDraw=*/ true, /*bLinkedOrthoMovement=*/ false);
	}
}

TSharedRef<SEditorViewport> SNiagaraBakerViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SNiagaraBakerViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SNiagaraBakerViewport::OnFloatingButtonClicked()
{
}

void SNiagaraBakerViewport::RefreshView(const float RelativeTime, const float DeltaTime)
{
	ViewportClient->RelativeTime = RelativeTime;
	ViewportClient->DeltaTime = DeltaTime;
}

#undef LOCTEXT_NAMESPACE
