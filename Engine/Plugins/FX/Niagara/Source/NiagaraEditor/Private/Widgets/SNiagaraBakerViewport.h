// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Slate/SceneViewport.h"
#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"

#include "NiagaraBakerSettings.h"

class FNiagaraBakerViewModel;
class FNiagaraBakerViewportClient;

class SNiagaraBakerViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SNiagaraBakerViewport) {}
		SLATE_ARGUMENT(TWeakPtr<FNiagaraBakerViewModel>, WeakViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// SEditorViewport interface
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	//TSharedPtr<SWidget> MakeViewportToolbar() override;
	// SEditorViewport interface

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// ICommonEditorViewportToolbarInfoProvider interface

	void RefreshView(const float RelativeTime, const float DeltaTime);

private:
	TWeakPtr<FNiagaraBakerViewModel>		WeakViewModel;
	TSharedPtr<FNiagaraBakerViewportClient>	ViewportClient;
};
