// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraBakerWidget.h"
#include "NiagaraBakerOutputRegistry.h"
#include "SNiagaraBakerTimelineWidget.h"
#include "SNiagaraBakerViewport.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraComponent.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "ViewModels/NiagaraBakerViewModel.h"

#include "Dialog/SMessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SViewport.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorWidgetsModule.h"
#include "IDetailCustomization.h"
#include "IDocumentation.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "NiagaraBakerWidget"

//////////////////////////////////////////////////////////////////////////

namespace NiagaraBakerWidgetLocal
{
	template<typename TType>
	TSharedRef<SWidget> MakeSpinBox(TOptional<TType> MinValue, TOptional<TType> MaxValue, TOptional<TType> MinSliderValue, TOptional<TType> MaxSliderValue, TAttribute<TType> GetValue, typename SSpinBox<TType>::FOnValueChanged SetValue)
	{
		return SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SSpinBox<TType>)
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.MinValue(MinValue)
				.MaxValue(MaxValue)
				.MinSliderValue(MinSliderValue)
				.MaxSliderValue(MaxSliderValue)
				.Value(GetValue)
				.OnValueChanged(SetValue)
			]
		];
	}

	template<typename TType, int NumElements>
	struct FMakeVectorBoxHelper
	{
		DECLARE_DELEGATE_RetVal(TType, FGetter);
		DECLARE_DELEGATE_OneParam(FSetter, TType);

		static TSharedRef<SWidget> Construct(FGetter GetValue, FSetter SetValue, TOptional<TType> MinValue = TOptional<TType>(), TOptional<TType> MaxValue = TOptional<TType>(), TOptional<TType> MinSliderValue = TOptional<TType>(), TOptional<TType> MaxSliderValue = TOptional<TType>())
		{
			TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

			for ( int i=0; i < NumElements; ++i )
			{
				HorizontalBox->AddSlot()
				.FillWidth(1.0f / float(NumElements))
				.MaxWidth(60.0f)
				[
					SNew(SSpinBox<float>)
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.MinValue(MinValue.IsSet() ? MinValue.GetValue()[i] : TOptional<float>())
					.MaxValue(MaxValue.IsSet() ? MaxValue.GetValue()[i] : TOptional<float>())
					.MinSliderValue(MinSliderValue.IsSet() ? MinSliderValue.GetValue()[i] : TOptional<float>())
					.MaxSliderValue(MaxSliderValue.IsSet() ? MaxSliderValue.GetValue()[i] : TOptional<float>())
					.Value_Lambda([=]() { return GetValue.Execute()[i]; })
					.OnValueChanged_Lambda([=](float InValue) { FVector VectorValue = GetValue.Execute(); VectorValue[i] = InValue; SetValue.Execute(VectorValue); })
				];
			}

			return
				SNew(SBox)
				.WidthOverride(180.0f)
				.HAlign(HAlign_Right)
				[
					HorizontalBox
				];
		}
	};

	using FMakeVectorBox = FMakeVectorBoxHelper<FVector, 3>;

	struct FMakeRotatorBox
	{
		DECLARE_DELEGATE_RetVal(FRotator, FGetter);
		DECLARE_DELEGATE_OneParam(FSetter, FRotator);

		static TSharedRef<SWidget> Construct(FGetter GetValue, FSetter SetValue)
		{
			return
				SNew(SBox)
				.WidthOverride(180.0f)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f / float(3.0f))
					.MaxWidth(60.0f)
					[
						SNew(SSpinBox<float>)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.Value_Lambda([=]() { return GetValue.Execute().Pitch; })
						.OnValueChanged_Lambda([=](float InValue) { FRotator VectorValue = GetValue.Execute(); VectorValue.Pitch = InValue; SetValue.Execute(VectorValue); })
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f / float(3.0f))
					.MaxWidth(60.0f)
					[
						SNew(SSpinBox<float>)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.Value_Lambda([=]() { return GetValue.Execute().Yaw; })
						.OnValueChanged_Lambda([=](float InValue) { FRotator VectorValue = GetValue.Execute(); VectorValue.Yaw = InValue; SetValue.Execute(VectorValue); })
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f / float(3.0f))
					.MaxWidth(60.0f)
					[
						SNew(SSpinBox<float>)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.Value_Lambda([=]() { return GetValue.Execute().Roll; })
						.OnValueChanged_Lambda([=](float InValue) { FRotator VectorValue = GetValue.Execute(); VectorValue.Roll = InValue; SetValue.Execute(VectorValue); })
					]
				];
		}
	};

	TSharedRef<SWidget> MakeChannelWidget(TWeakPtr<FNiagaraBakerViewModel> WeakViewModel)
	{
		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

		static const FText ChannelLetter[int(ENiagaraBakerColorChannel::Num)] = { FText::FromString("R"), FText::FromString("G"), FText::FromString("B"), FText::FromString("A") };

		for ( int32 i=0; i < int(ENiagaraBakerColorChannel::Num); ++i )
		{
			const ENiagaraBakerColorChannel Channel = ENiagaraBakerColorChannel(i);

			HorizontalBox->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "TextureEditor.ChannelButtonStyle")
				//.BorderBackgroundColor_Lambda(
				//	[WeakViewModel, Channel]() -> FLinearColor
				//	{
				//		FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
				//		return ViewModel && ViewModel->IsChannelEnabled(Channel) ? FLinearColor::Gray : FLinearColor::Gray;
				//	}
				//)
				.ForegroundColor_Lambda(
					[WeakViewModel, Channel]() -> FLinearColor
					{
						FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
						if ( ViewModel && ViewModel->IsChannelEnabled(Channel) )
						{
							switch (Channel)
							{
								case ENiagaraBakerColorChannel::Red:	return FLinearColor::Red;
								case ENiagaraBakerColorChannel::Green:	return FLinearColor::Green;
								case ENiagaraBakerColorChannel::Blue:	return FLinearColor::Blue;
								case ENiagaraBakerColorChannel::Alpha:	return FLinearColor::White;
							}
						}
						return FLinearColor::Black;
					}
				)
				.OnCheckStateChanged_Lambda(
					[WeakViewModel, Channel](ECheckBoxState CheckState)
					{
						if ( FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get() )
						{
							ViewModel->SetChannelEnabled(Channel, CheckState == ECheckBoxState::Checked);
						}
					}
				)
				.IsChecked_Lambda(
					[WeakViewModel, Channel]() -> ECheckBoxState
					{
						FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
						const bool bEnabled = ViewModel ? ViewModel->IsChannelEnabled(Channel) : false;
						return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
				)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("TextureEditor.ChannelButtonFont"))
					.Text(ChannelLetter[i])
				]
			];
		}

		return HorizontalBox;
	}
}

//////////////////////////////////////////////////////////////////////////

void SNiagaraBakerWidget::Construct(const FArguments& InArgs)
{
	WeakViewModel = InArgs._WeakViewModel;
	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel);

	OnCurrentOutputIndexChangedHandle = ViewModel->OnCurrentOutputChanged.AddSP(this, &SNiagaraBakerWidget::RefreshWidget);

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::Get().LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");

	// Baker Toolbar
	FSlimHorizontalToolBarBuilder BakerToolbarBuilder(MakeShareable(new FUICommandList), FMultiBoxCustomization::None);
	BakerToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda([this]() { OnCapture(); })),
		NAME_None,
		FText::GetEmpty(),	//LOCTEXT("Bake", "Bake"),
		LOCTEXT("BakeTooltip", "Run the bake process"),
		FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.BakerIcon")
	);

	BakerToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &SNiagaraBakerWidget::MakeSettingsMenu),
		FText::GetEmpty(),
		LOCTEXT("SettingsToolTip", "Modify baker settings"),
		FSlateIcon(FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.BakerSettings")
	);

	BakerToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &SNiagaraBakerWidget::MakeCameraModeMenu),
		TAttribute<FText>::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetCurrentCameraModeText),
		LOCTEXT("CameraModeToolTip", "Change the camera used to render from"),
		TAttribute<FSlateIcon>::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetCurrentCameraModeIcon)
	);

	BakerToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &SNiagaraBakerWidget::MakeViewOptionsMenu),
		FText::GetEmpty(),
		LOCTEXT("ViewOptionsToolTip", "Modify view options"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visibility")
	);

	// Outputs
	BakerToolbarBuilder.BeginSection(NAME_None);
	{
		BakerToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SNiagaraBakerWidget::MakeOutputSelectMenu),
			TAttribute<FText>::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetCurrentOutputText),
			LOCTEXT("CurrentOutputTooltip", "Select which output is currently visible in the preview area"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit")
		);
		BakerToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SNiagaraBakerWidget::MakeAddOutputMenu),
			FText::GetEmpty(),
			LOCTEXT("AddOutputTooltip", "Add a new output to bake"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus")
		);
		BakerToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::RemoveCurrentOutput),
				FCanExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::CanRemoveCurrentOutput)
			),
			NAME_None,
			FText::GetEmpty(),
			LOCTEXT("RemoveOutputTooltip", "Delete the currently selected output"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete")
		);
	}
	BakerToolbarBuilder.EndSection();

	// Color Channels
	BakerToolbarBuilder.BeginSection(NAME_None);
	{
		BakerToolbarBuilder.AddWidget(NiagaraBakerWidgetLocal::MakeChannelWidget(WeakViewModel));
	}
	BakerToolbarBuilder.EndSection();

	// Warnings
	BakerToolbarBuilder.BeginSection(NAME_None);
	{
		BakerToolbarBuilder.AddComboButton(
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(this, &SNiagaraBakerWidget::HasWarnings)
			),
			FOnGetContent::CreateSP(this, &SNiagaraBakerWidget::MakeWarningsMenu),
			LOCTEXT("Warnings", "Warnings"),
			LOCTEXT("WarningsToolTip", "Contains any warnings that may need attention"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.WarningWithColor")
		);
	}
	BakerToolbarBuilder.EndSection();


	// Baker Settings
	FDetailsViewArgs DetailsArgs;
	DetailsArgs.bHideSelectionTip = true;
	DetailsArgs.bAllowSearch = false;

	BakerSettingsDetails = PropertyModule.CreateDetailView(DetailsArgs);
	FNiagaraBakerOutputRegistry::Get().RegisterCustomizations(BakerSettingsDetails.Get());

	// Transport control args
	{
		FTransportControlArgs TransportControlArgs;
		TransportControlArgs.OnGetPlaybackMode.BindLambda([&]() -> EPlaybackMode::Type { return PlaybackMode; });
		TransportControlArgs.OnBackwardEnd.BindSP(this, &SNiagaraBakerWidget::OnTransportBackwardEnd);
		TransportControlArgs.OnBackwardStep.BindSP(this, &SNiagaraBakerWidget::OnTransportBackwardStep);
		TransportControlArgs.OnBackwardPlay.BindSP(this, &SNiagaraBakerWidget::OnTransportBackwardPlay);
		TransportControlArgs.OnForwardPlay.BindSP(this, &SNiagaraBakerWidget::OnTransportForwardPlay);
		TransportControlArgs.OnForwardStep.BindSP(this, &SNiagaraBakerWidget::OnTransportForwardStep);
		TransportControlArgs.OnForwardEnd.BindSP(this, &SNiagaraBakerWidget::OnTransportForwardEnd);
		TransportControlArgs.OnToggleLooping.BindSP(this, &SNiagaraBakerWidget::OnTransportToggleLooping);
		TransportControlArgs.OnGetLooping.BindSP(ViewModel, &FNiagaraBakerViewModel::IsPlaybackLooping);

		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::BackwardEnd));
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::BackwardStep));
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::BackwardPlay));
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::ForwardPlay));
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::ForwardStep));
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::ForwardEnd));
		TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::Loop));

		TransportControlArgs.bAreButtonsFocusable = false;

		TransportControls = EditorWidgetsModule.CreateTransportControl(TransportControlArgs);
	}

	//////////////////////////////////////////////////////////////////////////
	// Widgets
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			BakerToolbarBuilder.MakeWidget()
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Vertical)
			.PhysicalSplitterHandleSize(1.0f)
			.HitDetectionSplitterHandleSize(5.0f)
			.ResizeMode(ESplitterResizeMode::Fill)
			+ SSplitter::Slot()
			.Value(0.70f)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SAssignNew(ViewportWidget, SNiagaraBakerViewport)
					.WeakViewModel(WeakViewModel)
				]
			]
			+SSplitter::Slot()
			.Value(0.30f)
			[
				BakerSettingsDetails.ToSharedRef()
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			MakeSettingsWidget()
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				TransportControls.ToSharedRef()
			]
			+SHorizontalBox::Slot()
			[
				SAssignNew(TimelineWidget, SNiagaraBakerTimelineWidget)
				.WeakViewModel(WeakViewModel)
			]
		]
	];

	RefreshWidget();
}

SNiagaraBakerWidget::~SNiagaraBakerWidget()
{
	if ( FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		ViewModel->OnCurrentOutputChanged.Remove(OnCurrentOutputIndexChangedHandle);
	}
}

TSharedRef<SWidget> SNiagaraBakerWidget::MakeSettingsWidget()
{
	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel != nullptr);

	const FString DocLink = TEXT("Shared/Editors/NiagaraBaker");

	return
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("TimeRange", "Time Range"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpinBox<float>)
			.MinDesiredWidth(50.0f)
			.MinValue(0.0f)
			.MinSliderValue(0.0f)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value(ViewModel, &FNiagaraBakerViewModel::GetTimelineStart)
			.OnValueChanged(ViewModel, &FNiagaraBakerViewModel::SetTimelineStart)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpinBox<float>)
			.MinDesiredWidth(50.0f)
			.MinValue(0.0f)
			.MinSliderValue(0.0f)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value(ViewModel, &FNiagaraBakerViewModel::GetTimelineEnd)
			.OnValueChanged(ViewModel, &FNiagaraBakerViewModel::SetTimelineEnd)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(10.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FramesPerDimension", "Frames Per Dimension"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpinBox<int32>)
			.MinDesiredWidth(50.0f)
			.MinValue(1)
			.MinSliderValue(1)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value(ViewModel, &FNiagaraBakerViewModel::GetFramesOnX)
			.OnValueChanged(ViewModel, &FNiagaraBakerViewModel::SetFramesOnX)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpinBox<int32>)
			.MinDesiredWidth(50.0f)
			.MinValue(1)
			.MinSliderValue(1)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value(ViewModel, &FNiagaraBakerViewModel::GetFramesOnY)
			.OnValueChanged(ViewModel, &FNiagaraBakerViewModel::SetFramesOnY)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(10.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PlaybackRate", "Playback Rate"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpinBox<float>)
			.MinDesiredWidth(50.0f)
			.MinValue(0.0f)
			.MinSliderValue(0.0f)
			.MaxValue(10.0f)
			.MaxSliderValue(10.0f)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value(ViewModel, &FNiagaraBakerViewModel::GetPlaybackRate)
			.OnValueChanged(ViewModel, &FNiagaraBakerViewModel::SetPlaybackRate)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(10.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(this, &SNiagaraBakerWidget::GetCurrentTimeText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("CurrentTimeSecondsTooltip", "Current time in seconds"), nullptr, DocLink, "CurrentTimeText"))
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(10.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(this, &SNiagaraBakerWidget::GetCurrentFrameText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("CurrentFrameTooltip", "Current frame we are rendering, this is the baked frame index not the preview frame"), nullptr, DocLink, "CurrentFrameText"))
		]
	;
}

TSharedRef<SWidget> SNiagaraBakerWidget::MakeCameraModeMenu()
{
	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel != nullptr);

	FMenuBuilder MenuBuilder(true, nullptr);

	if ( UNiagaraBakerSettings* BakerSettings = ViewModel->GetBakerSettings() )
	{
		for (int i = 0; i < BakerSettings->CameraSettings.Num(); ++i)
		{
			if ( i == int32(ENiagaraBakerViewMode::Num) )
			{
				MenuBuilder.AddSeparator();
			}

			MenuBuilder.AddMenuEntry(
				ViewModel->GetCameraSettingsText(i),
				FText::GetEmpty(),
				ViewModel->GetCameraSettingsIcon(i),
				FUIAction(
					FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetCameraSettingsIndex, i),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::IsCameraSettingIndex, i)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AddBookmark", "Add Bookmark"),
			FText::GetEmpty(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Bookmark.Outline"),
			FUIAction(
				FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::AddCameraBookmark)
			)
		);

		if ( BakerSettings->CurrentCameraIndex >= int32(ENiagaraBakerViewMode::Num) )
		{
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("RemoveBookmarkFormat", "Remove Bookmark '{0}'"), ViewModel->GetCameraSettingsText(BakerSettings->CurrentCameraIndex)),
				FText::GetEmpty(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Bookmark"),
				FUIAction(
					FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::RemoveCameraBookmark, BakerSettings->CurrentCameraIndex)
				)
			);
		}
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SNiagaraBakerWidget::MakeSettingsMenu()
{
	using namespace NiagaraBakerWidgetLocal;

	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel != nullptr);

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowRealtimePreview", "Show Realtime Preview"),
		LOCTEXT("ShowRealtimePreviewTooltip", "When enabled shows a live preview of what will be rendered, this may not be accurate with all visualization modes."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::ToggleRealtimePreview),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::ShowRealtimePreview)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowBakedView", "Show Baked View"),
		LOCTEXT("ShowBakedViewTooltip", "When enabled shows the baked texture."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::ToggleBakedView),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::ShowBakedView)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowCheckerboard", "Show Checkerboard"),
		LOCTEXT("ShowCheckerboardTooltip", "Show a checkerboard rather than a solid color to easily visualize alpha blending."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::ToggleCheckerboardEnabled),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::IsCheckerboardEnabled)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowInfoText", "Show Info Text"),
		LOCTEXT("ShowInfoTextTooltip", "Shows information about the preview and baked outputs."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::ToggleInfoText),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::ShowInfoText)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowNiagaraOnly", "Render Niagara Only"),
		LOCTEXT("ShowNiagaraOnlyTooltip", "Renders only the Niagara System when enabled."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::ToggleRenderComponentOnly),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::ShowRenderComponentOnly)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowLoopedOutput", "Preview Looped Output"),
		LOCTEXT("ShowLoopedOutputTooltip", "Renders only the Niagara System when enabled."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::TogglePreviewLoopedOutput),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::PreviewLoopedOutput)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddSubMenu(
		LOCTEXT("BakeQualityLevel", "Bake Quality Level"),
		LOCTEXT("BakeQualityLevelToolTip", "Allows you to optionally override the quality level when baking, i.e. use cinematic quality while baking"),
		FNewMenuDelegate::CreateSP(this, &SNiagaraBakerWidget::MakeBakeQualityLevelMenu)
	);

	MenuBuilder.AddSubMenu(
		LOCTEXT("SimulationTickRate", "Simulation Tick Rate"),
		LOCTEXT("SimulationTickRateToolTip", "The rate at which the simulation will tick, i.e. 120fps, 60fps, 30fps, etc."),
		FNewMenuDelegate::CreateSP(this, &SNiagaraBakerWidget::MakeSimTickRateMenu)
	);

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SNiagaraBakerWidget::MakeViewOptionsMenu()
{
	using namespace NiagaraBakerWidgetLocal;

	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel != nullptr);

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddWidget(
		FMakeVectorBox::Construct(
			FMakeVectorBox::FGetter::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetCurrentCameraLocation),
			FMakeVectorBox::FSetter::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetCurrentCameraLocation)
		),
		LOCTEXT("CameraLocation", "Camera Location")
	);

	MenuBuilder.AddWidget(
		FMakeRotatorBox::Construct(
			FMakeRotatorBox::FGetter::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetCurrentCameraRotation),
			FMakeRotatorBox::FSetter::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetCurrentCameraRotation)
		),
		LOCTEXT("CameraRotation", "Camera Rotation")
	);

	if ( ViewModel->IsCurrentCameraPerspective() )
	{
		MenuBuilder.AddWidget(
			MakeSpinBox<float>(1.0f, 170.0f, 1.0f, 170.0f, TAttribute<float>::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetCameraFOV), SSpinBox<float>::FOnValueChanged::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetCameraFOV)),
			LOCTEXT("FOVAngle", "Field of View (H)")
		);

		MenuBuilder.AddWidget(
			MakeSpinBox<float>(0.0f, TOptional<float>(), 0.0f, 1000.0f, TAttribute<float>::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetCameraOrbitDistance), SSpinBox<float>::FOnValueChanged::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetCameraOrbitDistance)),
			LOCTEXT("OrbitDistance", "Orbit Distance")
		);
	}
	else
	{
		MenuBuilder.AddWidget(
			MakeSpinBox<float>(0.1f, TOptional<float>(), 0.1f, 1000.0f, TAttribute<float>::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetCameraOrthoWidth), SSpinBox<float>::FOnValueChanged::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetCameraOrthoWidth)),
			LOCTEXT("CameraOrthoWidth", "Camera Orthographic Width")
		);
	}

	//-TODO: Clean this up
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CustomAspectRatio", "Custom Aspect Ratio"),
		LOCTEXT("CustomAspectRatioTooltip", "Use a custom aspect ratio to render with."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::ToggleCameraAspectRatioEnabled),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::IsCameraAspectRatioEnabled)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	if ( ViewModel->IsCameraAspectRatioEnabled() )
	{
		MenuBuilder.AddWidget(
			MakeSpinBox<float>(0.1f, 5.0f, 0.1f, 5.0f, TAttribute<float>::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetCameraAspectRatio), SSpinBox<float>::FOnValueChanged::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetCameraAspectRatio)),
			LOCTEXT("CameraAspectRatio", "Camera Aspect Ratio")
		);
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ResetCamera", "Reset Camera"),
		LOCTEXT("ResetCameraTooltip", "Resets the current camera back to the default settings."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::ResetCurrentCamera)
		)
	);

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SNiagaraBakerWidget::MakeOutputSelectMenu()
{
	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel != nullptr);

	FMenuBuilder MenuBuilder(true, nullptr);

	if ( UNiagaraBakerSettings* BakerSettings = ViewModel->GetBakerSettings() )
	{
		for (int i = 0; i < BakerSettings->Outputs.Num(); ++i)
		{
			MenuBuilder.AddMenuEntry(
				ViewModel->GetOutputText(i),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetCurrentOutputIndex, i),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::IsCurrentOutputIndex, i)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SNiagaraBakerWidget::MakeAddOutputMenu()
{
	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel != nullptr);

	FMenuBuilder MenuBuilder(true, nullptr);

	for ( UClass* OutputClass : FNiagaraBakerOutputRegistry::Get().GetOutputClasses() )
	{
		MenuBuilder.AddMenuEntry(
			OutputClass->GetDisplayNameText(),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::AddOutput, OutputClass))
		);
	}

	return MenuBuilder.MakeWidget();
}

void SNiagaraBakerWidget::MakeBakeQualityLevelMenu(FMenuBuilder& MenuBuilder) const
{
	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel != nullptr);

	// Special entry for 'leave as default'
	MenuBuilder.AddMenuEntry(
		LOCTEXT("CurrentQualityLevel" ,"Current Quality Level"),
		LOCTEXT("CurrentQualityLevelTooltip", "Uses the current quality level that is set in the editor when baking."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetBakeQualityLevel, FName()),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::IsBakeQualityLevel, FName())
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	// Engine driven quality levels
	MenuBuilder.BeginSection("QualityLevel", LOCTEXT("QualityLevel", "QualityLevel"));
	{
		for (const FText& QualityLevelText : GetDefault<UNiagaraSettings>()->QualityLevels)
		{
			const FName QualityLevel(*QualityLevelText.ToString());

			MenuBuilder.AddMenuEntry(
				QualityLevelText,
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetBakeQualityLevel, QualityLevel),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::IsBakeQualityLevel, QualityLevel)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
}

void SNiagaraBakerWidget::MakeSimTickRateMenu(FMenuBuilder& MenuBuilder) const
{
	using namespace NiagaraBakerWidgetLocal;

	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel != nullptr);

	MenuBuilder.AddWidget(
		MakeSpinBox<int>(1, 480, 1, 480, TAttribute<int>::CreateSP(ViewModel, &FNiagaraBakerViewModel::GetSimTickRate), SSpinBox<int>::FOnValueChanged::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetSimTickRate)),
		LOCTEXT("SimTickRate", "Simulation Tick Rate")
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("LockToSimulationFrameRate", "Lock To Simulation Frame Rate"),
		LOCTEXT("LockToSimulationFrameRateTooltip", "When enabled we will update at the simulation frame rate.  I.e. only ever tick once per frame even if the application frame rate is lower, this will result in the sim slowing down."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::ToggleLockToSimulationFrameRate),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::LockToSimulationFrameRate)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.BeginSection("FPS", LOCTEXT("FPS", "FPS"));
	{
		static const int32 DefaultFPS[] = { 240, 120, 60, 50, 30, 20, 15 };

		for (int32 FPS : DefaultFPS)
		{
			FText FPSText = FText::Format(LOCTEXT("FPSFormat", "{0} fps"), FPS);
			MenuBuilder.AddMenuEntry(
				FPSText,
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(ViewModel, &FNiagaraBakerViewModel::SetSimTickRate, FPS),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(ViewModel, &FNiagaraBakerViewModel::IsSimTickRate, FPS)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
	MenuBuilder.EndSection();
}

void SNiagaraBakerWidget::FindWarnings()
{
	FoundWarnings.Reset();

	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	if ( ViewModel == nullptr )
	{
		return;
	}

	// Check determinism
	if ( UNiagaraComponent* PreviewComponent = ViewModel->GetBakerRenderer().GetPreviewComponent() )
	{
		if ( UNiagaraSystem* NiagaraSystem = PreviewComponent->GetAsset() )
		{
			if ( NiagaraSystem->NeedsDeterminism() == false )
			{
				FoundWarnings.Emplace(LOCTEXT("SystemDeterminism", "System is not set to deterministic, results will vary each bake"));
			}
			for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles() )
			{
				FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetInstance().GetEmitterData();
				if (EmitterData == nullptr || EmitterData->bDeterminism)
				{
					continue;
				}

				FoundWarnings.Emplace(FText::Format(LOCTEXT("EmitterDeterminismFormat", "Emitter '{0}' is not set to deterministic, results will vary each bake"), FText::FromString(EmitterHandle.GetInstance().Emitter->GetUniqueEmitterName())));
			}
		}
	}

	// Check outputs for any warnings
	if ( UNiagaraBakerSettings* BakerSettings = ViewModel->GetBakerSettings() )
	{
		for ( UNiagaraBakerOutput* BakerOutput : BakerSettings->Outputs )
		{
			BakerOutput->FindWarnings(FoundWarnings);
		}
	}
}

TSharedRef<SWidget> SNiagaraBakerWidget::MakeWarningsMenu()
{
	FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get();
	check(ViewModel != nullptr);

	FMenuBuilder MenuBuilder(true, nullptr);

	for (const FText& Warning : FoundWarnings)
	{
		MenuBuilder.AddMenuEntry(
			Warning,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction()
		);
	}

	return MenuBuilder.MakeWidget();
}

void SNiagaraBakerWidget::RefreshWidget()
{
	UObject* OutputObject = nullptr;
	if ( FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		OutputObject = ViewModel->GetCurrentOutput();
	}
	BakerSettingsDetails->SetObject(OutputObject, true);
}

void SNiagaraBakerWidget::Tick(const FGeometry& AllottedGeometry, const double _UnusedCurrentTime, float DeltaTime)
{
	UNiagaraBakerSettings* BakerSettings = GetBakerSettings();
	if (BakerSettings == nullptr)
	{
		return;
	}

	const float DurationSeconds = BakerSettings->DurationSeconds;
	if ( DurationSeconds > 0.0f )
	{
		if (TSharedPtr<FNiagaraBakerViewModel> ViewModel = WeakViewModel.Pin())
		{
			DeltaTime *= ViewModel->GetPlaybackRate();
		}

		if (BakerSettings->bLockToSimulationFrameRate)
		{
			const float MaxTickRate = 1.0f / BakerSettings->FramesPerSecond;
			DeltaTime = FMath::Min(MaxTickRate, DeltaTime);
		}

		if (PlaybackMode != EPlaybackMode::Stopped)
		{
			PreviewRelativeTime += PlaybackMode == EPlaybackMode::PlayingForward ? DeltaTime : -DeltaTime;
			if ( BakerSettings->bPreviewLooping )
			{
				PreviewRelativeTime = FMath::Fmod(PreviewRelativeTime, DurationSeconds);
				PreviewRelativeTime += PreviewRelativeTime < 0.0f ? DurationSeconds : 0.0f;
			}
			else if (PlaybackMode == EPlaybackMode::PlayingReverse && PreviewRelativeTime <= 0.0f)
			{
				PlaybackMode = EPlaybackMode::Stopped;
			}
			else if (PlaybackMode == EPlaybackMode::PlayingForward && PreviewRelativeTime >= DurationSeconds)
			{
				PlaybackMode = EPlaybackMode::Stopped;
			}
		}
		PreviewRelativeTime = FMath::Clamp(PreviewRelativeTime, 0.0f, DurationSeconds);

		ViewportWidget->RefreshView(PreviewRelativeTime, DeltaTime);
		TimelineWidget->SetRelativeTime(PreviewRelativeTime);
	}

	FindWarnings();
}

void SNiagaraBakerWidget::SetPreviewRelativeTime(float RelativeTime)
{
	PlaybackMode = EPlaybackMode::Stopped;
	PreviewRelativeTime = RelativeTime;
}

FReply SNiagaraBakerWidget::OnCapture()
{
	if (auto ViewModel = WeakViewModel.Pin())
	{
		const FNiagaraBakerFeedbackContext FeedbackContext = ViewModel->RenderBaker();
		if (FeedbackContext.HasIssues())
		{
			FString IssuesString;
			for (const FString& Error : FeedbackContext.Errors)
			{
				IssuesString.Appendf(TEXT("Error: %s\n\n"), *Error);
			}
			for (const FString& Warning : FeedbackContext.Warnings)
			{
				IssuesString.Appendf(TEXT("Warning: %s\n\n"), *Warning);
			}

			TSharedRef<SMessageDialog> IssuesDialog = SNew(SMessageDialog)
				.Title(FText(LOCTEXT("BakerIssuesDialogTitle", "Baker Issues")))
				.Message(FText::FromString(IssuesString))
				.Buttons({SCustomDialog::FButton(LOCTEXT("Ok", "Ok"))});

			IssuesDialog->Show();
		}
	}
	return FReply::Handled();
}

UNiagaraBakerSettings* SNiagaraBakerWidget::GetBakerSettings() const
{
	if (auto ViewModel = WeakViewModel.Pin())
	{
		return ViewModel->GetBakerSettings();
	}
	return nullptr;
}

const UNiagaraBakerSettings* SNiagaraBakerWidget::GetBakerGeneratedSettings() const
{
	if (auto ViewModel = WeakViewModel.Pin())
	{
		return ViewModel->GetBakerGeneratedSettings();
	}
	return nullptr;
}

FReply SNiagaraBakerWidget::OnTransportBackwardEnd()
{
	PlaybackMode = EPlaybackMode::Stopped;
	PreviewRelativeTime = 0.0f;

	return FReply::Handled();
}

FReply SNiagaraBakerWidget::OnTransportBackwardStep()
{
	PlaybackMode = EPlaybackMode::Stopped;
	if (FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get())
	{
		const FNiagaraBakerOutputFrameIndices OutputFrameIndices = ViewModel->GetCurrentOutputFrameIndices(PreviewRelativeTime);
		const int NewFrame = OutputFrameIndices.Interp > 0.25f ? OutputFrameIndices.FrameIndexA : FMath::Max(OutputFrameIndices.FrameIndexA - 1, 0);
		PreviewRelativeTime = (ViewModel->GetDurationSeconds() / float(OutputFrameIndices.NumFrames)) * float(NewFrame);
	}

	return FReply::Handled();
}

FReply SNiagaraBakerWidget::OnTransportBackwardPlay()
{
	PlaybackMode = PlaybackMode == EPlaybackMode::PlayingReverse ? EPlaybackMode::Stopped : EPlaybackMode::PlayingReverse;
	return FReply::Handled();
}

FReply SNiagaraBakerWidget::OnTransportForwardPlay()
{
	PlaybackMode = PlaybackMode == EPlaybackMode::PlayingForward ? EPlaybackMode::Stopped : EPlaybackMode::PlayingForward;
	return FReply::Handled();
}

FReply SNiagaraBakerWidget::OnTransportForwardStep()
{
	PlaybackMode = EPlaybackMode::Stopped;
	if (FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get())
	{
		const FNiagaraBakerOutputFrameIndices OutputFrameIndices = ViewModel->GetCurrentOutputFrameIndices(PreviewRelativeTime);
		const int NewFrame = OutputFrameIndices.Interp < 0.75f ? OutputFrameIndices.FrameIndexB : FMath::Min(OutputFrameIndices.FrameIndexB + 1, OutputFrameIndices.NumFrames - 1);
		PreviewRelativeTime = (ViewModel->GetDurationSeconds() / float(OutputFrameIndices.NumFrames)) * float(NewFrame);
	}

	return FReply::Handled();
}

FReply SNiagaraBakerWidget::OnTransportForwardEnd()
{
	PlaybackMode = EPlaybackMode::Stopped;
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		PreviewRelativeTime = BakerSettings->DurationSeconds - KINDA_SMALL_NUMBER;
	}

	return FReply::Handled();
}

FReply SNiagaraBakerWidget::OnTransportToggleLooping() const
{
	if (auto ViewModel = WeakViewModel.Pin())
	{
		ViewModel->TogglePlaybackLooping();
	}
	return FReply::Handled();
}

FText SNiagaraBakerWidget::GetCurrentTimeText() const
{
	if (UNiagaraBakerSettings* BakerSettings = GetBakerSettings())
	{
		const float TimeSeconds = BakerSettings->StartSeconds + PreviewRelativeTime;
		const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
			.SetMaximumFractionalDigits(2)
			.SetMinimumFractionalDigits(2);
		return FText::Format(LOCTEXT("CurrentTimeTextFormat", "{0} s"), FText::AsNumber(TimeSeconds, &FormatOptions));
	}
	return FText::GetEmpty();
}

FText SNiagaraBakerWidget::GetCurrentFrameText() const
{
	if ( FNiagaraBakerViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		FNiagaraBakerOutputFrameIndices FrameIndices = ViewModel->GetCurrentOutputFrameIndices(PreviewRelativeTime);
		return FText::Format(LOCTEXT("CurrentFrameTextFormat", "{0} / {1} f"), FText::AsNumber(FrameIndices.FrameIndexA), FText::AsNumber(FrameIndices.NumFrames));
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
