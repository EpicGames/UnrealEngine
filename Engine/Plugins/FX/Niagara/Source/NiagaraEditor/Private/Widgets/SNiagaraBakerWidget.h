// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "ITransportControl.h"

class FNiagaraBakerViewModel;
class SNiagaraBakerViewport;
class SNiagaraBakerTimelineWidget;
class IDetailsView;
class FMenuBuilder;

class SNiagaraBakerWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraBakerWidget) {}
		SLATE_ARGUMENT(TWeakPtr<FNiagaraBakerViewModel>, WeakViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SNiagaraBakerWidget();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	FReply OnCapture();

	TSharedRef<SWidget> MakeSettingsWidget();

	TSharedRef<SWidget> MakeCameraModeMenu();
	TSharedRef<SWidget> MakeSettingsMenu();
	TSharedRef<SWidget> MakeViewOptionsMenu();
	TSharedRef<SWidget> MakeOutputSelectMenu();
	TSharedRef<SWidget> MakeAddOutputMenu();

	void MakeBakeQualityLevelMenu(FMenuBuilder& MenuBuilder) const;
	void MakeSimTickRateMenu(FMenuBuilder& MenuBuilder) const;

	void FindWarnings();
	bool HasWarnings() const { return FoundWarnings.Num() > 0; }
	TSharedRef<SWidget> MakeWarningsMenu();

	void RefreshWidget();

	class UNiagaraBakerSettings* GetBakerSettings() const;
	const class UNiagaraBakerSettings* GetBakerGeneratedSettings() const;

	void SetPreviewRelativeTime(float RelativeTime);

	FReply OnTransportBackwardEnd();
	FReply OnTransportBackwardStep();
	FReply OnTransportBackwardPlay();
	FReply OnTransportForwardPlay();
	FReply OnTransportForwardStep();
	FReply OnTransportForwardEnd();
	FReply OnTransportToggleLooping() const;

	FText GetCurrentTimeText() const;
	FText GetCurrentFrameText() const;

private:
	TWeakPtr<FNiagaraBakerViewModel>			WeakViewModel;
	TSharedPtr<SNiagaraBakerViewport>			ViewportWidget;
	TSharedPtr<SNiagaraBakerTimelineWidget>		TimelineWidget;
	TSharedPtr<IDetailsView>					BakerSettingsDetails;
	TSharedPtr<SWidget>							TransportControls;

	EPlaybackMode::Type							PlaybackMode = EPlaybackMode::PlayingForward;
	float										PreviewRelativeTime = 0.0f;

	FDelegateHandle								OnCurrentOutputIndexChangedHandle;

	TArray<FText>								FoundWarnings;
};
