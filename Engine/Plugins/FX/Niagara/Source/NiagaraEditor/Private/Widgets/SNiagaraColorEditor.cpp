// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraColorEditor.h"

#include "Engine/Engine.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SNiagaraExpandedToggle.h"

#define LOCTEXT_NAMESPACE "NiagaraColorEditor"

void SNiagaraColorEditor::Construct(const FArguments& InArgs)
{
	bShowAlpha = InArgs._ShowAlpha;
	bShowExpander = InArgs._ShowExpander;
	Color = InArgs._Color;
	ExpandComponents = InArgs._ExpandComponents;
	LabelText = InArgs._LabelText;
	MinDesiredColorBlockWidth = InArgs._MinDesiredColorBlockWidth;
	OnColorChangedDelegate = InArgs._OnColorChanged;
	OnBeginEditingDelegate = InArgs._OnBeginEditing;
	OnEndEditingDelegate = InArgs._OnEndEditing;
	OnCancelEditingDelegate = InArgs._OnCancelEditing;
	OnExpandComponentsChangedDelegate = InArgs._OnExpandComponentsChanged;
	OnColorPickerOpenedDelegate = InArgs._OnColorPickerOpened;
	OnColorPickerClosedDelegate = InArgs._OnColorPickerClosed;

	ColorBlock = SNew(SColorBlock)
		.AlphaBackgroundBrush(FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground"))
		.Color(Color)
		.ShowBackgroundForAlpha(true)
		.AlphaDisplayMode(bShowAlpha ? EColorBlockAlphaDisplayMode::Separate : EColorBlockAlphaDisplayMode::Ignore)
		.OnMouseButtonDown(this, &SNiagaraColorEditor::OnColorBlockMouseButtonDown)
		.Size(FVector2D(75.0f, 18.0f))
		.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f));

	RootBox = SNew(SGridPanel)
		.FillColumn(2, 1.0f);

	if (bShowExpander)
	{
		RootBox->AddSlot(0, 0)
		.VAlign(VAlign_Center)
		.Padding(0, 0, 3, 0)
		[
			SNew(SNiagaraExpandedToggle)
			.Expanded(ExpandComponents)
			.OnExpandedChanged(this, &SNiagaraColorEditor::ExpanderExpandedChanged)
		];
	}

	if (LabelText.IsBound() || LabelText.Get().IsEmptyOrWhitespace() == false)
	{
		RootBox->AddSlot(1, 0)
		.VAlign(VAlign_Center)
		.Padding(0, 0, 5, 0)
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			.Text(LabelText)
		];
	}

	TSharedPtr<SWidget> ColorBlockWidgetForSlot;
	if (MinDesiredColorBlockWidth.IsSet())
	{
		ColorBlockWidgetForSlot = SNew(SBox)
			.MinDesiredWidth(MinDesiredColorBlockWidth)
			[
				ColorBlock.ToSharedRef()
			];
	}
	else
	{
		ColorBlockWidgetForSlot = ColorBlock.ToSharedRef();
	}

	RootBox->AddSlot(2, 0)
	.Padding(0, 1, 1, 1)
	[
		ColorBlockWidgetForSlot.ToSharedRef()
	];

	ChildSlot
	[
		RootBox.ToSharedRef()
	];

	UpdateComponentWidgets();
}

void SNiagaraColorEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bComponentsConstructed == false && ExpandComponents.Get() == true)
	{
		UpdateComponentWidgets();
	}
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SNiagaraColorEditor::UpdateComponentWidgets()
{
	if (bComponentsConstructed == false && ExpandComponents.Get())
	{
		TSharedPtr<SGridPanel> ComponentGrid;
		RootBox->AddSlot(2, 1)
			.HAlign(HAlign_Left)
			.Padding(0, 3, 0, 0)
			[
				SAssignNew(ComponentGrid, SGridPanel)
				.Visibility(this, &SNiagaraColorEditor::GetComponentsVisibility)
			];
		ConstructComponentWidgets(ComponentGrid.ToSharedRef(), LOCTEXT("RedLabel", "Red"), 0);
		ConstructComponentWidgets(ComponentGrid.ToSharedRef(), LOCTEXT("GreenLabel", "Green"), 1);
		ConstructComponentWidgets(ComponentGrid.ToSharedRef(), LOCTEXT("BlueLabel", "Blue"), 2);
		if (bShowAlpha)
		{
			ConstructComponentWidgets(ComponentGrid.ToSharedRef(), LOCTEXT("AlphaLabel", "Alpha"), 3);
		}
		bComponentsConstructed = true;
	}
}

FReply SNiagaraColorEditor::OnColorBlockMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	OnBeginEditingDelegate.ExecuteIfBound();

	FColorPickerArgs PickerArgs;
	{
		PickerArgs.bUseAlpha = bShowAlpha;
		PickerArgs.bOnlyRefreshOnMouseUp = false;
		PickerArgs.bOnlyRefreshOnOk = false;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SNiagaraColorEditor::ColorPickerColorCommitted);
		PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SNiagaraColorEditor::OnColorPickerCancelled);
		PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &SNiagaraColorEditor::OnColorPickerClosed);
		PickerArgs.InitialColor = Color.Get(FLinearColor::White);
		PickerArgs.ParentWidget = ColorBlock;
		FWidgetPath ParentWidgetPath;
		if (FSlateApplication::Get().FindPathToWidget(ColorBlock.ToSharedRef(), ParentWidgetPath))
		{
			PickerArgs.bOpenAsMenu = FSlateApplication::Get().FindMenuInWidgetPath(ParentWidgetPath).IsValid();
		}
	}

	OpenColorPicker(PickerArgs);
	OnColorPickerOpenedDelegate.ExecuteIfBound();
	return FReply::Handled();
}

void SNiagaraColorEditor::ExpanderExpandedChanged(bool bIsExpanded)
{
	if (ExpandComponents.Get() != bIsExpanded)
	{
		if (ExpandComponents.IsBound() == false)
		{
			ExpandComponents = bIsExpanded;
		}
		OnExpandComponentsChangedDelegate.ExecuteIfBound(bIsExpanded);
	}
}

void SNiagaraColorEditor::ConstructComponentWidgets(TSharedRef<SGridPanel> ComponentGrid, FText ComponentLabelText, int32 ComponentIndex)
{
	ComponentGrid->AddSlot(0, ComponentIndex)
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	.Padding(0, 0, 5, 0)
	[
		SNew(STextBlock)
		.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
		.Text(ComponentLabelText)
	];
	ComponentGrid->AddSlot(1, ComponentIndex)
	.Padding(0, 2, 0, 2)
	[
		SNew(SNumericEntryBox<float>)
		.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
		.Value(this, &SNiagaraColorEditor::GetComponentValue, ComponentIndex)
		.OnValueChanged(this, &SNiagaraColorEditor::ComponentValueChanged, ComponentIndex)
		.OnValueCommitted(this, &SNiagaraColorEditor::ComponentValueCommitted, ComponentIndex)
		.OnBeginSliderMovement(this, &SNiagaraColorEditor::BeginComponentValueChange)
		.OnEndSliderMovement(this, &SNiagaraColorEditor::EndComponentValueChange)
		.AllowSpin(true)
		.MinValue(TOptional<float>())
		.MaxValue(TOptional<float>())
		.MinSliderValue(TOptional<float>())
		.MaxSliderValue(TOptional<float>())
		.BroadcastValueChangesPerKey(false)
		.MinDesiredValueWidth(50)
	];
}

TOptional<float> SNiagaraColorEditor::GetComponentValue(int32 ComponentIndex) const
{
	FLinearColor CurrentColor = Color.Get(FLinearColor::White);
	switch(ComponentIndex)
	{
	case 0:
		return CurrentColor.R;
	case 1:
		return CurrentColor.G;
	case 2:
		return CurrentColor.B;
	case 3:
		return CurrentColor.A;
	default:
		return 0.0f;
	}
}

void SNiagaraColorEditor::ComponentValueChanged(float Value, int32 ComponentIndex)
{
	FLinearColor CurrentColor = Color.Get(FLinearColor::White);
	switch (ComponentIndex)
	{
	case 0:
		CurrentColor.R = Value;
		break;
	case 1:
		CurrentColor.G = Value;
		break;
	case 2:
		CurrentColor.B = Value;
		break;
	case 3:
		CurrentColor.A = Value;
		break;
	}
	OnColorChangedDelegate.ExecuteIfBound(CurrentColor);
}

void SNiagaraColorEditor::ComponentValueCommitted(float Value, ETextCommit::Type CommitInfo, int32 ComponentIndex)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		ComponentValueChanged(Value, ComponentIndex);
	}
}

void SNiagaraColorEditor::BeginComponentValueChange()
{
	OnBeginEditingDelegate.ExecuteIfBound();
}

void SNiagaraColorEditor::EndComponentValueChange(float Value)
{
	OnEndEditingDelegate.ExecuteIfBound();
}

EVisibility SNiagaraColorEditor::GetComponentsVisibility() const
{
	return ExpandComponents.Get() 
		? EVisibility::Visible 
		: EVisibility::Collapsed;
}

void SNiagaraColorEditor::ColorPickerColorCommitted(FLinearColor InColor)
{
	OnColorChangedDelegate.ExecuteIfBound(InColor);
}

void SNiagaraColorEditor::OnColorPickerClosed(const TSharedRef<SWindow>& InWindow)
{
	OnEndEditingDelegate.ExecuteIfBound();
	OnColorPickerClosedDelegate.ExecuteIfBound();
}

void SNiagaraColorEditor::OnColorPickerCancelled(FLinearColor InColor)
{
	OnCancelEditingDelegate.ExecuteIfBound(InColor);
}

#undef LOCTEXT_NAMESPACE
