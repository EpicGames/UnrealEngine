// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/SlateStructs.h"
#include "Widgets/SCompoundWidget.h"

class SColorBlock;
class SGridPanel;

class SNiagaraColorEditor : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnValueChanged, FLinearColor /* Value */);
	DECLARE_DELEGATE_OneParam(FOnExpandComponentsChanged, bool /* bIsExpanded */);
	DECLARE_DELEGATE_OneParam(FOnCancelEditing, FLinearColor /* OriginalValue */);

public:
	SLATE_BEGIN_ARGS(SNiagaraColorEditor)
		: _ShowAlpha(true)
		, _ShowExpander(true)
		, _ExpandComponents(false)
		{ }
		SLATE_ARGUMENT(bool, ShowAlpha)
		SLATE_ARGUMENT(bool, ShowExpander)
		SLATE_ATTRIBUTE(FLinearColor, Color)
		SLATE_ATTRIBUTE(bool, ExpandComponents)
		SLATE_ATTRIBUTE(FText, LabelText)
		SLATE_ATTRIBUTE(FOptionalSize, MinDesiredColorBlockWidth)
		SLATE_EVENT(FOnValueChanged, OnColorChanged)
		SLATE_EVENT(FSimpleDelegate, OnBeginEditing)
		SLATE_EVENT(FSimpleDelegate, OnEndEditing)
		SLATE_EVENT(FOnCancelEditing, OnCancelEditing)
		SLATE_EVENT(FOnExpandComponentsChanged, OnExpandComponentsChanged)
		SLATE_EVENT(FSimpleDelegate, OnColorPickerOpened)
		SLATE_EVENT(FSimpleDelegate, OnColorPickerClosed)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	void UpdateComponentWidgets();

	FReply OnColorBlockMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	void ExpanderExpandedChanged(bool bIsExpanded);

	void ConstructComponentWidgets(TSharedRef<SGridPanel> ComponentGrid, FText ComponentLabelText, int32 ComponentIndex);

	TOptional<float> GetComponentValue(int32 ComponentIndex) const;
	void ComponentValueChanged(float Value, int32 ComponentIndex);
	void ComponentValueCommitted(float Value, ETextCommit::Type CommitInfo, int32 ComponentIndex);

	void BeginComponentValueChange();
	void EndComponentValueChange(float Value);

	EVisibility GetComponentsVisibility() const;

	void ColorPickerColorCommitted(FLinearColor InColor);
	void OnColorPickerClosed(const TSharedRef<SWindow>& InWindow);
	void OnColorPickerCancelled(FLinearColor InColor);

private:
	bool bShowAlpha;
	bool bShowExpander;

	TAttribute<bool> ExpandComponents;
	TAttribute<FLinearColor> Color;
	TAttribute<FText> LabelText;
	TAttribute<FOptionalSize> MinDesiredColorBlockWidth;

	FOnValueChanged OnColorChangedDelegate;
	FSimpleDelegate OnBeginEditingDelegate;
	FSimpleDelegate OnEndEditingDelegate;
	FOnCancelEditing OnCancelEditingDelegate;
	FOnExpandComponentsChanged OnExpandComponentsChangedDelegate;
	FSimpleDelegate OnColorPickerOpenedDelegate;
	FSimpleDelegate OnColorPickerClosedDelegate;

	bool bComponentsConstructed = false;
	TSharedPtr<SGridPanel> RootBox;
	TSharedPtr<SColorBlock> ColorBlock;
};