// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraConvertPinSocket.h"
#include "ViewModels/NiagaraConvertNodeViewModel.h"
#include "ViewModels/NiagaraConvertPinViewModel.h"
#include "ViewModels/NiagaraConvertPinSocketViewModel.h"
#include "ConnectionDrawingPolicy.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/SButton.h"

#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Layout/WidgetPath.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "NiagaraConvertPinSocket"

FNiagaraConvertDragDropOp::FNiagaraConvertDragDropOp(TSharedRef<FNiagaraConvertPinSocketViewModel> InSocketViewModel)
	: SocketViewModel(InSocketViewModel)
{
	InSocketViewModel->SetIsBeingDragged(true);
}

void FNiagaraConvertDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	FDecoratedDragDropOp::OnDrop(bDropWasHandled, MouseEvent);
	SocketViewModel->SetIsBeingDragged(false);
}

void FNiagaraConvertDragDropOp::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	FDecoratedDragDropOp::OnDragged(DragDropEvent);
	SocketViewModel->SetAbsoluteDragPosition(DragDropEvent.GetScreenSpacePosition());
}


void SNiagaraConvertPinSocket::Construct(const FArguments& InArgs, TSharedRef<FNiagaraConvertPinSocketViewModel> InSocketViewModel)
{
	SocketViewModel = InSocketViewModel;

	BackgroundBrush = FAppStyle::GetBrush("Graph.Pin.Background");
	BackgroundHoveredBrush = FAppStyle::GetBrush("Graph.Pin.BackgroundHovered");

	ConnectedBrush = FAppStyle::GetBrush("Graph.Pin.Connected");
	DisconnectedBrush = FAppStyle::GetBrush("Graph.Pin.Disconnected");

	bIsDraggedOver = false;
	
	if (SocketViewModel->GetDirection() == EGPD_Input)
	{
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(this, &SNiagaraConvertPinSocket::GetBackgroundBrush)
			.OnMouseDoubleClick(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::OnMouseDoubleClick)
			.Padding(0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FCoreStyle::Get(), "NoBorder")
					.OnClicked(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::ExpandButtonClicked)
					.ForegroundColor(FSlateColor::UseForeground())
					[
						SNew(SImage)
						.Image(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::GetExpansionBrush)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Visibility(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::GetExpansionBrushVisibility)
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(TAttribute<FMargin>(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::GetSocketPadding))
				[
					SNew(STextBlock)
					.Text(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::GetDisplayNameText)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(3, 0, 0, 1)
				[
					SNew(SImage)
					.Image(this, &SNiagaraConvertPinSocket::GetSocketBrush)
				]
			]
		];
	}
	else
	{
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(this, &SNiagaraConvertPinSocket::GetBackgroundBrush)
			.OnMouseDoubleClick(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::OnMouseDoubleClick)
			.Padding(0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 3, 1)
				[
					SNew(SImage)
					.Image(this, &SNiagaraConvertPinSocket::GetSocketBrush)
					.Visibility(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::GetSocketIconVisibility)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FCoreStyle::Get(), "NoBorder")
					.OnClicked(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::ExpandButtonClicked)
					.ForegroundColor(FSlateColor::UseForeground())
					[
						SNew(SImage)
						.Image(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::GetExpansionBrush)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Visibility(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::GetExpansionBrushVisibility)
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(0)
				[
					SNew(STextBlock)
					.Text(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::GetDisplayPathText)
					.Visibility(SocketViewModel.ToSharedRef(), &FNiagaraConvertPinSocketViewModel::GetSocketTextVisibility)
				]
			]
		];
	}
}

void SNiagaraConvertPinSocket::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	SCompoundWidget::OnArrangeChildren(AllottedGeometry, ArrangedChildren);

	FVector2D ConnectionPosition = SocketViewModel->GetDirection() == EGPD_Input ? FGeometryHelper::VerticalMiddleRightOf(AllottedGeometry): FGeometryHelper::VerticalMiddleLeftOf(AllottedGeometry);

	SocketViewModel->SetAbsoluteConnectionPosition(ConnectionPosition);
}

FReply SNiagaraConvertPinSocket::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (MouseEvent.IsAltDown())
		{
			// Alt-Left clicking will break all existing connections to a pin
			TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>> ConnectedSockets;
			SocketViewModel->GetConnectedSockets(ConnectedSockets);
			if (ConnectedSockets.Num() > 0)
			{
				SocketViewModel->DisconnectAll();
			}
			return FReply::Handled();
		}

		if (SocketViewModel->CanBeConnected())
		{
			return FReply::Handled()
				.DetectDrag(SharedThis(this), MouseEvent.GetEffectingButton())
				.CaptureMouse(SharedThis(this));
		}
	}
	else if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		// Mark handled and capture the mouse to avoid having mouse up eaten by the graph panel.
		return FReply::Handled()
			.CaptureMouse(SharedThis(this));
	}
	return FReply::Unhandled();
}

FReply SNiagaraConvertPinSocket::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		TSharedRef<SWidget> MenuContent = OnSummonContextMenu();
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent, MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SNiagaraConvertPinSocket::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedRef<FNiagaraConvertDragDropOp> DragDropOp = MakeShareable(new FNiagaraConvertDragDropOp(SocketViewModel.ToSharedRef()));
	DragDropOp->CurrentHoverText = SocketViewModel->GetDisplayPathText();
	DragDropOp->SetupDefaults();
	DragDropOp->Construct();
	return FReply::Handled().BeginDragDrop(DragDropOp);
}

void SNiagaraConvertPinSocket::OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent)
{
	TSharedPtr<FNiagaraConvertDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FNiagaraConvertDragDropOp>();
	if (DragDropOp.IsValid())
	{
		DragDropOp->ResetToDefaultToolTip();
		bIsDraggedOver = true;
	}
}

void SNiagaraConvertPinSocket::OnDragLeave(FDragDropEvent const& DragDropEvent)
{
	TSharedPtr<FNiagaraConvertDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FNiagaraConvertDragDropOp>();
	if (DragDropOp.IsValid())
	{
		DragDropOp->ResetToDefaultToolTip();
		bIsDraggedOver = false;
	}
}

FReply SNiagaraConvertPinSocket::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FNiagaraConvertDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FNiagaraConvertDragDropOp>();
	if (DragDropOp.IsValid() && SocketViewModel->CanBeConnected() && SocketViewModel != DragDropOp->SocketViewModel)
	{
		DragDropOp->ResetToDefaultToolTip();
		bIsDraggedOver = true;

		FText ConnectionMessage;
		bool bWarning = false;
		bool bCanConnectSockets = SocketViewModel->CanConnect(DragDropOp->SocketViewModel, ConnectionMessage, bWarning);
		DragDropOp->CurrentHoverText = ConnectionMessage;
		if (bCanConnectSockets == false)
		{
			DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
		}
		else if (bWarning)
		{
			DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OKWarn"));
		}
		else
		{
			DragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
		}

		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SNiagaraConvertPinSocket::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FNiagaraConvertDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FNiagaraConvertDragDropOp>();
	if (DragDropOp.IsValid() && SocketViewModel->CanBeConnected() && SocketViewModel != DragDropOp->SocketViewModel)
	{
		FText ConnectionMessage;
		bool bWarning = false;
		bool bCanConnectSockets = SocketViewModel->CanConnect(DragDropOp->SocketViewModel, ConnectionMessage, bWarning);
		if (bCanConnectSockets)
		{
			FScopedTransaction ConnectTransaction(NSLOCTEXT("NiagaraConvertPinSocket", "ConvertNodeConnectTransaction", "Connect inner convert pins"));
			SocketViewModel->Connect(DragDropOp->SocketViewModel);
		}
		bIsDraggedOver = false;
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

const FSlateBrush* SNiagaraConvertPinSocket::GetBackgroundBrush() const
{
	return IsHovered() || (bIsDraggedOver && SocketViewModel->CanBeConnected())
		? BackgroundHoveredBrush
		: BackgroundBrush;
}

const FSlateBrush* SNiagaraConvertPinSocket::GetSocketBrush() const
{
	return SocketViewModel->GetIsConnected()
		? ConnectedBrush
		: DisconnectedBrush;
}

TSharedRef<SWidget> SNiagaraConvertPinSocket::OnSummonContextMenu()
{
	FMenuBuilder SocketMenuBuilder(true, nullptr);

	SocketMenuBuilder.BeginSection("NiagaraConvertPinSocketPinActions", LOCTEXT("PinActionsMenuHeader", "Pin Actions"));
	SocketMenuBuilder.AddMenuEntry(
		LOCTEXT("BreakAllConnections", "Break Link(s)"),
		LOCTEXT("RemoveConvertPinToolTip", "Break all links for this internal pin."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SNiagaraConvertPinSocket::OnBreakConnections)));

	TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>> ConnectedSockets;
	SocketViewModel->GetConnectedSockets(ConnectedSockets);
	if (ConnectedSockets.Num() > 1)
	{
		SocketMenuBuilder.AddSubMenu(
			LOCTEXT("BreakSpecificConnection", "Break Link To..."),
			LOCTEXT("BreakSpecificConnectionToolTip", "Break a specific link to an internal pin."),
			FNewMenuDelegate::CreateSP(this, &SNiagaraConvertPinSocket::GenerateBreakSpecificSubMenu, ConnectedSockets));
	}

	return SocketMenuBuilder.MakeWidget();
}

void SNiagaraConvertPinSocket::GenerateBreakSpecificSubMenu(FMenuBuilder& MenuBuilder, TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>> ConnectedSockets)
{
	if (ConnectedSockets.Num() > 0)
	{
		for (TSharedRef<FNiagaraConvertPinSocketViewModel> ConnectedSocket : ConnectedSockets)
		{
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("BreakSpecificConnectionFormat", "Break link to {0}"), ConnectedSocket->GetDisplayPathText()),
				FText(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SNiagaraConvertPinSocket::OnBreakConnection, ConnectedSocket)));
		}
	}
}

void SNiagaraConvertPinSocket::OnBreakConnections()
{
	SocketViewModel->DisconnectAll();
}

void SNiagaraConvertPinSocket::OnBreakConnection(TSharedRef<FNiagaraConvertPinSocketViewModel> SocketToDisconnect)
{
	SocketViewModel->DisconnectSpecific(SocketToDisconnect);
}

#undef LOCTEXT_NAMESPACE
