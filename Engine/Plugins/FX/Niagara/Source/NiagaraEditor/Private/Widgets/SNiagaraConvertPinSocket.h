// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

class FNiagaraConvertPinSocketViewModel;
class FMenuBuilder;

/** Contains data for a socket drag and drop operation in the convert node. */
class FNiagaraConvertDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FNiagaraConvertDragDropOp, FDecoratedDragDropOp)

public:
	FNiagaraConvertDragDropOp(TSharedRef<FNiagaraConvertPinSocketViewModel> InSocketViewModel);

	//~ FDragDropOperation interface.
	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
	virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;

	/** The socket view model being dragged and dropped. */
	const TSharedRef<FNiagaraConvertPinSocketViewModel> SocketViewModel;
};

/** A widget for displaying and interacting with a socket in a convert node. */
class SNiagaraConvertPinSocket : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraConvertPinSocket) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<FNiagaraConvertPinSocketViewModel> InSocketViewModel);

	//~ SWidget interface
	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent) override;
	virtual void OnDragLeave(FDragDropEvent const& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

private:
	const FSlateBrush* GetBackgroundBrush() const;

	const FSlateBrush* GetSocketBrush() const;

	TSharedRef<SWidget> OnSummonContextMenu();
	void GenerateBreakSpecificSubMenu(FMenuBuilder& MenuBuilder, TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>> ConnectedSockets);

	void OnBreakConnections();

	void OnBreakConnection(TSharedRef<FNiagaraConvertPinSocketViewModel> SocketToDisconnect);

private:
	const FSlateBrush* BackgroundBrush;
	const FSlateBrush* BackgroundHoveredBrush;

	const FSlateBrush* ConnectedBrush;
	const FSlateBrush* DisconnectedBrush;

	mutable TSharedPtr<FNiagaraConvertPinSocketViewModel> SocketViewModel;

	bool bIsDraggedOver;
};
