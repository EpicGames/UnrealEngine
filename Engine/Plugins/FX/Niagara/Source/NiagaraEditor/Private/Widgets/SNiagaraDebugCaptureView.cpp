// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraDebugCaptureView.h"

#include "NiagaraComponent.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraSettings.h"
#include "PropertyEditorModule.h"
#include "ToolBuilderUtil.h"
#include "UObject/UObjectIterator.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "NiagaraDebugCaptureView"

void SNiagaraDebugCaptureView::OnNumFramesChanged(int32 InNumFrames)
{
	NumFrames = FMath::Max(1, InNumFrames);
}

void SNiagaraDebugCaptureView::CreateComponentSelectionMenuContent(FMenuBuilder& MenuBuilder) 
{
	for(TObjectIterator<UNiagaraComponent> NiagaraComponentIt; NiagaraComponentIt; ++NiagaraComponentIt)
	{
		UNiagaraComponent* NiagaraComponent = *NiagaraComponentIt;

		// Ignore dying or CDO versions of data..
		// No need to check the unreachable flag here as TObjectIterator already does that
		if(!IsValid(NiagaraComponent) || NiagaraComponent->HasAnyFlags(RF_ClassDefaultObject))
		{
			continue;
		}

		// Ignore any component not referencing this system.
		if (NiagaraComponent->GetAsset() != &SystemViewModel->GetSystem())
		{
			continue;
		}

		UWorld* World = NiagaraComponent->GetWorld();
		if(!World)
		{
			continue;
		}

		// Only allow the component from our preview world to exist in the component list
		// Without this test things like sim cache previews, or the baker's component will show up in the capture list which is confusing
		if (World->IsPreviewWorld() && NiagaraComponent != SystemViewModel->GetPreviewComponent())
		{
			continue;
		}

		FText ComponentName;
		FText ComponentTooltip;
		GetComponentNameAndTooltip(NiagaraComponent, ComponentName, ComponentTooltip);

		MenuBuilder.AddMenuEntry(
			ComponentName,
			ComponentTooltip,
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([&, WeakNiagaraComponent=MakeWeakObjectPtr(NiagaraComponent)]()
			{
				WeakTargetComponent = WeakNiagaraComponent;
			})));
	}
}

void SNiagaraDebugCaptureView::GetComponentNameAndTooltip(const UNiagaraComponent* InComponent, FText& OutName, FText& OutTooltip) const
{
	const UNiagaraComponent* PreviewComponent = SystemViewModel->GetPreviewComponent();

    if (InComponent == nullptr)
    {
    	OutName = LOCTEXT("NullComponentLabel", "Unknown");
    	OutTooltip = LOCTEXT("NullComponentTooltip", "Unknown");
    }
    else if (PreviewComponent == InComponent)
    {
    	OutName = LOCTEXT("PreviewComponentLabel", "Editor Viewport");
    	OutTooltip = LOCTEXT("PreviewComponentTooltip", "The instance of the Niagara Component in the Niagara editor viewport.");
    }
    else
    {
	    const UWorld* World = InComponent->GetWorld();
		const EWorldType::Type WorldType = World ? EWorldType::Type(World->WorldType) : EWorldType::None;
    	const AActor* Actor = InComponent->GetOwner();
    	OutName = FText::Format(LOCTEXT("SourceComponentLabel","World: \"{0} - {1}\" Actor: \"{2}\""), World ? FText::FromString(World->GetName()) : FText::GetEmpty(), FText::FromString(LexToString(WorldType)), Actor ? FText::FromString(Actor->GetActorNameOrLabel()) : FText::GetEmpty());
    	OutTooltip = OutName;
    }
}

TSharedRef<SWidget> SNiagaraDebugCaptureView::GenerateCaptureMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("CaptureFrameMode", LOCTEXT("CaptureModeMenu", "Capture Mode"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SingleFrameCaptureMode", "Single Frame"),
			LOCTEXT("SingleFrameCaptureModeTooltip", "Capture a single frame and display it in the current window."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){FrameMode = ENiagaraDebugCaptureFrameMode::SingleFrame;}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([&]()
				{
					return FrameMode == ENiagaraDebugCaptureFrameMode::SingleFrame;
				})
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
		MenuBuilder.AddMenuEntry(
			LOCTEXT("MultiFrameCaptureMode", "Multi Frame (Standalone Editor)"),
			LOCTEXT("MultiFrameCaptureModeTooltip", "Capture the specified number of frames and open the results in the standalone Sim Cache editor."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this](){FrameMode = ENiagaraDebugCaptureFrameMode::MultiFrame;}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([&]()
				{
					return FrameMode == ENiagaraDebugCaptureFrameMode::MultiFrame;
				})
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
		
	}
	MenuBuilder.EndSection();
	MenuBuilder.BeginSection("TargetSelection", LOCTEXT("TargetSelectionMenu", "Component Selection"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("TargetSelection", "Target Component"),
			LOCTEXT("TargetSelectionTooltip", "Select the Niagara Component of this system for capturing."),
			FNewMenuDelegate::CreateRaw(this, &SNiagaraDebugCaptureView::CreateComponentSelectionMenuContent)
		);
	}
	MenuBuilder.EndSection();

	
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SNiagaraDebugCaptureView::GenerateFilterMenuContent()
{
	TSharedRef<SWidget> Content = SNew(SBox)
	.MinDesiredHeight(400.0f)
	.MinDesiredWidth(200.0f)
	[
		OverviewFilterWidget.ToSharedRef()
	];
	
	return Content;
}

FText SNiagaraDebugCaptureView::GetCaptureLabel()
{
	switch(FrameMode)
	{
	case ENiagaraDebugCaptureFrameMode::SingleFrame:
		return LOCTEXT("CaptureSingle", "Capture (Single)");
	case ENiagaraDebugCaptureFrameMode::MultiFrame:
		return LOCTEXT("CaptureMulti", "Capture (Multi)");
	default:
		return FText();
	}
}

FText SNiagaraDebugCaptureView::GetCaptureTooltip()
{
	switch(FrameMode)
	{
	case ENiagaraDebugCaptureFrameMode::SingleFrame:
		return LOCTEXT("CapturSingleToolTip", "Captures a single frame, and displays it in the current window.");
	case ENiagaraDebugCaptureFrameMode::MultiFrame:
		return LOCTEXT("CaptureMultiToolTip", "Captures the specified number of frames, and opens the cache in a standalone editor.");
	default:
		return FText();
	}
}

FSlateIcon SNiagaraDebugCaptureView::GetCaptureIcon()
{
	switch(FrameMode)
	{
	case ENiagaraDebugCaptureFrameMode::SingleFrame:
		return FSlateIcon (FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.SimCache.CaptureSingleIcon");
	case ENiagaraDebugCaptureFrameMode::MultiFrame:
		return FSlateIcon (FNiagaraEditorStyle::Get().GetStyleSetName(), "NiagaraEditor.SimCache.CaptureMultiIcon");
	default:
		return FSlateIcon();
	}
}



FName GetTempCacheName(const FString& SystemName)
{
	return FNiagaraEditorUtilities::GetUniqueObjectName<UNiagaraSimCache>(GetTransientPackage(), TEXT("TempCache_") + SystemName);
}

void SNiagaraDebugCaptureView::Construct(const FArguments& InArgs, const TSharedRef<FNiagaraSystemViewModel> InSystemViewModel, const TSharedRef<FNiagaraSimCacheViewModel> InSimCacheViewModel)
{
	NumFrames = FMath::Max(1, GetDefault<UNiagaraSettings>()->QuickSimCacheCaptureFrameCount);

	WeakTargetComponent = InSystemViewModel->GetPreviewComponent();
	SimCacheViewModel = InSimCacheViewModel;
	SystemViewModel = InSystemViewModel;

	CapturedCache = NewObject<UNiagaraSimCache>(GetTransientPackage(), GetTempCacheName(InSystemViewModel->GetSystem().GetName()));
	SimCacheViewModel.Get()->Initialize(CapturedCache);
	CapturedCache->SetFlags(RF_Transient);

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;

	SAssignNew(OverviewFilterWidget, SNiagaraSimCacheOverview)
			.SimCacheViewModel(SimCacheViewModel);
	
	FSlimHorizontalToolBarBuilder DebugCaptureToolbarBuilder (MakeShareable(new FUICommandList), FMultiBoxCustomization::None);

	DebugCaptureToolbarBuilder.BeginSection("Filter");
	{
		DebugCaptureToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateRaw(this, &SNiagaraDebugCaptureView::GenerateFilterMenuContent),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(SimCacheViewModel.ToSharedRef(), &FNiagaraSimCacheViewModel::GetSelectedText)),
			LOCTEXT("FilterCombo_Tooltip", "Change filter and selection"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(),"Icons.Filter")
		);
	}
	DebugCaptureToolbarBuilder.EndSection();

	DebugCaptureToolbarBuilder.BeginSection("Capture");
	{
		DebugCaptureToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraDebugCaptureView::OnCaptureSelected)),
			NAME_None,
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &SNiagaraDebugCaptureView::GetCaptureLabel)),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &SNiagaraDebugCaptureView::GetCaptureTooltip)),
			TAttribute< FSlateIcon >::Create(TAttribute< FSlateIcon >::FGetter::CreateRaw(this, &SNiagaraDebugCaptureView::GetCaptureIcon))
		);
		DebugCaptureToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateRaw(this, &SNiagaraDebugCaptureView::GenerateCaptureMenuContent),
			LOCTEXT("CaptureCombo_Label", "Capture Options"),
			LOCTEXT("CaptureCombo_Tooltip", "Capture Options Menu"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(),"LevelEditor.Build"),
			true
		);
	}
	
	DebugCaptureToolbarBuilder.EndSection();

	DebugCaptureToolbarBuilder.BeginSection("Frames");
	{
		DebugCaptureToolbarBuilder.AddWidget(
			SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NumberOfFrames", "Frames"))
					.IsEnabled_Lambda([this]{return FrameMode == ENiagaraDebugCaptureFrameMode::MultiFrame;})
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpinBox<int32>)
					.Value(NumFrames)
					.MinValue(1)
					.OnValueChanged(this, &SNiagaraDebugCaptureView::OnNumFramesChanged)
					.IsEnabled_Lambda([this]{return FrameMode == ENiagaraDebugCaptureFrameMode::MultiFrame;})
				]
		);
	}
	DebugCaptureToolbarBuilder.EndSection();

	DebugCaptureToolbarBuilder.BeginSection("Export");
	{
		TSharedRef<FNiagaraSimCacheViewModel> SimCacheViewModelRef = SimCacheViewModel.ToSharedRef();
		DebugCaptureToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::CopyActiveToClipboard),
				FCanExecuteAction::CreateSP(SimCacheViewModelRef, &FNiagaraSimCacheViewModel::CanCopyActiveToClipboard)
			),
			NAME_None,
			LOCTEXT("CopyToCSV", "Copy to CSV"),
			LOCTEXT("CopyToCSVTooltip", "Copies the currently selected view to the clipboard in CSV format."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy")
		);
	}
	DebugCaptureToolbarBuilder.EndSection();

	ChildSlot
	[
		DebugCaptureToolbarBuilder.MakeWidget()
	];
	
}

SNiagaraDebugCaptureView::SNiagaraDebugCaptureView()
{
	CapturedCache = nullptr;
}

SNiagaraDebugCaptureView::~SNiagaraDebugCaptureView()
{
	if(CapturedCache.IsValid())
	{
		CapturedCache->MarkAsGarbage();
		CapturedCache.Reset();
	}
}

void SNiagaraDebugCaptureView::OnCaptureSelected()
{
	switch(FrameMode)
	{
	case ENiagaraDebugCaptureFrameMode::SingleFrame:
		OnSingleFrameSelected();
		break;
	case ENiagaraDebugCaptureFrameMode::MultiFrame:
		OnMultiFrameSelected();
		break;
	default:
		break;
	}
}

void SNiagaraDebugCaptureView::OnSingleFrameSelected()
{
	UNiagaraComponent* TargetComponent = WeakTargetComponent.Get();
	if(!bIsCaptureActive && TargetComponent && CapturedCache.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = SystemViewModel.Get()->GetSequencer();
		Sequencer->OnPlay(false);

		FQualifiedFrameTime StartTime = Sequencer->GetGlobalTime();
		float CurrentAge = TargetComponent->GetDesiredAge();

		UNiagaraSimCache* OutCache = CapturedCache.Get();

		SimCacheCapture.CaptureCurrentFrameImmediate(CapturedCache.Get(), FNiagaraSimCacheCreateParameters::CreateForDebugging(), TargetComponent, OutCache, true, 0.01666f);

		if(OutCache)
		{
			// 60fps
			FFrameRate SystemFrameRate (60, 1);
			FFrameTime NewTime = StartTime.ConvertTo(SystemFrameRate) + FQualifiedFrameTime(1, SystemFrameRate).Time;
			
			TargetComponent->SetDesiredAge(CurrentAge + 0.01666f);
			
			Sequencer->SetGlobalTime(ConvertFrameTime(NewTime, SystemFrameRate, StartTime.Rate));
			
			OnCaptureComplete(CapturedCache.Get());
		}
	}
}

void SNiagaraDebugCaptureView::OnMultiFrameSelected()
{
	UNiagaraComponent* TargetComponent = WeakTargetComponent.Get();
	if(!bIsCaptureActive && TargetComponent)
	{
		UNiagaraSimCache* MultiFrameCache = NewObject<UNiagaraSimCache>(GetTransientPackage(), GetTempCacheName(TargetComponent->GetFXSystemAsset()->GetName()));
		MultiFrameCache->SetFlags(RF_Transient);
		
		FNiagaraSimCacheCaptureParameters CaptureParameters;
		CaptureParameters.NumFrames = NumFrames;
		CaptureParameters.bCaptureAllFramesImmediatly = true;
		CaptureParameters.ImmediateCaptureDeltaTime = 0.01666f;
		
		bIsCaptureActive = true;

		SimCacheCapture.OnCaptureComplete().AddSP(this, &SNiagaraDebugCaptureView::OnCaptureComplete);
		
		SimCacheCapture.CaptureNiagaraSimCache(MultiFrameCache, FNiagaraSimCacheCreateParameters::CreateForDebugging(), TargetComponent, CaptureParameters);
	}
}

void SNiagaraDebugCaptureView::OnCaptureComplete(UNiagaraSimCache* CapturedSimCache)
{
	bIsCaptureActive = false;
	SystemViewModel.Get()->GetSequencer().Get()->Pause();
				
	SimCacheViewModel->RefreshSelection();

	switch(FrameMode)
	{
	case ENiagaraDebugCaptureFrameMode::SingleFrame:
		RequestSpreadsheetTab.ExecuteIfBound();
		break;
	case ENiagaraDebugCaptureFrameMode::MultiFrame:
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CapturedSimCache, EToolkitMode::Standalone);
		break;
	default:
		break;
	}
}

#undef LOCTEXT_NAMESPACE
