// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NiagaraSimCacheCapture.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"

class FNiagaraSimCacheCapture;
class UNiagaraSimCache;

enum class ENiagaraDebugCaptureFrameMode : uint8
{
	// Capture a single frame and display the results in the Niagara editor.
	SingleFrame,
	// Capture a user defined number of frames, and open the results in a standalone editor.
	MultiFrame
};

class SNiagaraDebugCaptureView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDebugCaptureView)
	{}

	SLATE_END_ARGS();

	DECLARE_DELEGATE(FOnRequestSpreadsheetTab)

	void OnNumFramesChanged(int32 InNumFrames);
	void CreateComponentSelectionMenuContent(FMenuBuilder& MenuBuilder);
	void GetComponentNameAndTooltip(const UNiagaraComponent* InComponent, FText& OutName, FText& OutTooltip) const;
	TSharedRef<SWidget> GenerateCaptureMenuContent();
	TSharedRef<SWidget> GenerateFilterMenuContent();

	FText GetCaptureLabel();
	FText GetCaptureTooltip();
	FSlateIcon GetCaptureIcon();
	void Construct(const FArguments& InArgs, const TSharedRef<FNiagaraSystemViewModel> SystemViewModel, const TSharedRef<FNiagaraSimCacheViewModel> SimCacheViewModel);
	SNiagaraDebugCaptureView();
	virtual ~SNiagaraDebugCaptureView() override;
	FOnRequestSpreadsheetTab& OnRequestSpreadsheetTab() {return RequestSpreadsheetTab; }

protected:
	void OnCaptureSelected();
	void OnSingleFrameSelected();
	void OnMultiFrameSelected();
	void OnCaptureComplete(UNiagaraSimCache* CapturedSimCache);

	ENiagaraDebugCaptureFrameMode FrameMode = ENiagaraDebugCaptureFrameMode::SingleFrame;
	
	FNiagaraSimCacheCapture SimCacheCapture;
	TWeakObjectPtr<UNiagaraSimCache> CapturedCache = nullptr;

	TWeakObjectPtr<UNiagaraComponent> WeakTargetComponent;
	int32 NumFrames = 1;
	bool bIsCaptureActive = false;
	TSharedPtr<FNiagaraSimCacheViewModel> SimCacheViewModel;
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
	FOnRequestSpreadsheetTab RequestSpreadsheetTab;
	TSharedPtr<SWidget> OverviewFilterWidget;
};
