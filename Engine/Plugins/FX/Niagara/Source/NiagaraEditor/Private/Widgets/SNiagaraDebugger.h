// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/TargetDeviceId.h"
#include "Interfaces/ITargetDevice.h"
#include "Misc/Optional.h"
#include "Widgets/SCompoundWidget.h"
#include "IPropertyChangeListener.h"
#include "NiagaraDebugger.h"

class FMenuBuilder;

class SNiagaraDebugger : public SCompoundWidget
{
public:
	static const FName DebugWindowName;

public:
	SLATE_BEGIN_ARGS(SNiagaraDebugger) {}
		SLATE_ARGUMENT(TSharedPtr<class FTabManager>, TabManager)
		SLATE_ARGUMENT(TSharedPtr<class FNiagaraDebugger>, Debugger)
	SLATE_END_ARGS();

	SNiagaraDebugger();
	virtual ~SNiagaraDebugger();

	void Construct(const FArguments& InArgs);
	void FillWindowMenu(FMenuBuilder& MenuBuilder);

	static void RegisterTabSpawner();
	static void UnregisterTabSpawner();
	static TSharedRef<class SDockTab> SpawnNiagaraDebugger(const class FSpawnTabArgs& Args);

	static void InvokeDebugger(UNiagaraComponent* InComponent);
	static void InvokeDebugger(UNiagaraSystem* InSystem);
	static void InvokeDebugger(FNiagaraEmitterHandle& InEmitterHandle);

	static void InvokeDebugger(UNiagaraSystem* InSystem, TArray<FNiagaraEmitterHandle>& InSelectedHandles, TArray<FNiagaraVariableBase>& InAttributes);

	void FocusDebugTab();
	void FocusOutlineTab();

private:
	TSharedRef<SWidget> MakeToolbar();
	TSharedRef<SWidget> MakePlaybackOptionsMenu();
	TSharedRef<SWidget> MakeModeOptionsMenu();
	TSharedRef<SWidget> MakeBoundsOptionsMenu();

protected:
	TSharedPtr<FTabManager>		TabManager;
	TSharedPtr<FNiagaraDebugger>	Debugger;
};
