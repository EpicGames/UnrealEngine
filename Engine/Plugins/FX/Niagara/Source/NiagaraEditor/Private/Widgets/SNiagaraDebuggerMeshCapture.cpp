// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraDebuggerMeshCapture.h"
#include "NiagaraBakerRendererOutputStaticMesh.h"
#include "NiagaraEditorStyle.h"

#include "Engine/Selection.h"
#include "Engine/StaticMesh.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Editor.h"
#include "IAssetTools.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SNiagaraDebuggerMeshCapture)

#if WITH_NIAGARA_DEBUGGER
#define LOCTEXT_NAMESPACE "SNiagaraDebuggerMeshCapture"

namespace NiagaraDebuggerMeshCapturePrivate
{
	void ProcessMeshCapture(const FNiagaraRendererReadbackResult& ReadbackResult, TWeakObjectPtr<UStaticMesh> WeakMeshOutput)
	{
		// If a mesh was not specified pop open a dialog
		UStaticMesh* MeshOutput = WeakMeshOutput.Get();
		if (MeshOutput == nullptr)
		{
			UNiagaraBakerStaticMeshFactoryNew* Factory = NewObject<UNiagaraBakerStaticMeshFactoryNew>();
			MeshOutput = Cast<UStaticMesh>(IAssetTools::Get().CreateAssetWithDialog(UStaticMesh::StaticClass(), Factory));
			if (MeshOutput == nullptr)
			{
				return;
			}
		}

		// Failed or no data
		if (!ReadbackResult.NumVertices)
		{
			FMessageDialog::Open(
				EAppMsgType::Ok,
				LOCTEXT("CaptureFailedMessage", "Capture process failed, this could be due to nothing being renderered or an internal error."),
				LOCTEXT("CaptureFailedTitle", "Failed to capture any data!")
			);
			return;
		}

		FNiagaraBakerRendererOutputStaticMesh::ConvertReadbackResultsToStaticMesh(ReadbackResult, MeshOutput);
	}
}

void SNiagaraDebuggerMeshCapture::Construct(const FArguments& InArgs)
{
	// Setup defaults
	CaptureSettings.ExportParameters.CoordinateSpace = ENiagaraRendererReadbackCoordinateSpace::Local;

	// Create details view for the export parameters
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bHideSelectionTip = true;

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	TSharedRef<IStructureDetailsView> StructureDetailsView = PropertyModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);

	TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FNiagaraDebuggerMeshCaptureSettings::StaticStruct(), reinterpret_cast<uint8*>(&CaptureSettings));
	StructureDetailsView->SetStructureData(StructOnScope);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MakeToolbar()
		]
 		+ SVerticalBox::Slot()
		.Padding(2.0)
		[
			StructureDetailsView->GetWidget().ToSharedRef()
		]
	];
}

TSharedRef<SWidget> SNiagaraDebuggerMeshCapture::MakeToolbar()
{
	FToolBarBuilder ToolbarBuilder(MakeShareable(new FUICommandList), FMultiBoxCustomization::None);
	ToolbarBuilder.BeginSection("DebuggerSpawn");

	ToolbarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SNiagaraDebuggerMeshCapture::OnCaptureSelected),
			FCanExecuteAction::CreateSP(this, &SNiagaraDebuggerMeshCapture::CanCaptureSelected)
		),
		NAME_None,
		LOCTEXT("CaptureSelected", "Capture Selected"),
		LOCTEXT("CaptureSelectedTooltip", "Captures the visible Niagara components that are children of the outliner selection.")
	);

	ToolbarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SNiagaraDebuggerMeshCapture::OnCaptureAll),
			FCanExecuteAction::CreateSP(this, &SNiagaraDebuggerMeshCapture::CanCaptureAll)
		),
		NAME_None,
		LOCTEXT("CaptureAll", "Capture All"),
		LOCTEXT("CaptureAllTooltip", "Captures all the visible Niagara components that are next rendered to the scene.")
	);

	ToolbarBuilder.EndSection();

	return ToolbarBuilder.MakeWidget();
}

TArray<UNiagaraComponent*> SNiagaraDebuggerMeshCapture::GetSelectComponents(bool bFindFirstOnly) const
{
	TArray<UNiagaraComponent*> SelectedComponents;
	if (GEditor)
	{
		if (USelection* ActorSelection = GEditor->GetSelectedActors())
		{
			for (FSelectionIterator It(*ActorSelection); It; ++It)
			{
				if (AActor* Actor = Cast<AActor>(*It))
				{
					TArray<UNiagaraComponent*> FoundComponents;
					Actor->GetComponents(FoundComponents);
					SelectedComponents.Append(FoundComponents);
					if (SelectedComponents.Num() > 0 && bFindFirstOnly)
					{
						return SelectedComponents;
					}
				}
			}
		}
		if (USelection* ComponentSelection = GEditor->GetSelectedComponents())
		{
			for (FSelectionIterator It(*ComponentSelection); It; ++It)
			{
				if (UNiagaraComponent* Component = Cast<UNiagaraComponent>(*It))
				{
					SelectedComponents.Add(Component);
					if (bFindFirstOnly)
					{
						return SelectedComponents;
					}
				}
			}
		}
	}
	return SelectedComponents;
}

bool SNiagaraDebuggerMeshCapture::IsOutputLocationValid() const
{
	//return CaptureSettings.MeshOutput != nullptr;
	return true;
}

bool SNiagaraDebuggerMeshCapture::CanCaptureSelected() const
{
	return IsOutputLocationValid() && GetSelectComponents(true).Num() > 0;
}

void SNiagaraDebuggerMeshCapture::OnCaptureSelected()
{
	TArray<UNiagaraComponent*> SelectedComponents = GetSelectComponents();
	if (SelectedComponents.Num() == 0)
	{
		return;
	}

#if WITH_NIAGARA_RENDERER_READBACK
	NiagaraRendererReadback::EnqueueReadback(
		SelectedComponents,
		[WeakMeshOutput=MakeWeakObjectPtr(CaptureSettings.MeshOutput)](const FNiagaraRendererReadbackResult& ReadbackResult)
		{
			NiagaraDebuggerMeshCapturePrivate::ProcessMeshCapture(ReadbackResult, WeakMeshOutput);
		},
		CaptureSettings.ExportParameters
	);
#endif //WITH_NIAGARA_RENDERER_READBACK
}

bool SNiagaraDebuggerMeshCapture::CanCaptureAll() const
{
#if WITH_NIAGARA_RENDERER_READBACK
	UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
	return World != nullptr;
#else
	return false;
#endif //WITH_NIAGARA_RENDERER_READBACK
}

void SNiagaraDebuggerMeshCapture::OnCaptureAll()
{
#if WITH_NIAGARA_RENDERER_READBACK
	UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;

	NiagaraRendererReadback::EnqueueReadback(
		World,
		[WeakMeshOutput=MakeWeakObjectPtr(CaptureSettings.MeshOutput)](const FNiagaraRendererReadbackResult& ReadbackResult)
		{
			NiagaraDebuggerMeshCapturePrivate::ProcessMeshCapture(ReadbackResult, WeakMeshOutput);
		},
		CaptureSettings.ExportParameters
	);
#endif //WITH_NIAGARA_RENDERER_READBACK
}

#undef LOCTEXT_NAMESPACE
#endif //WITH_NIAGARA_DEBUGGER

