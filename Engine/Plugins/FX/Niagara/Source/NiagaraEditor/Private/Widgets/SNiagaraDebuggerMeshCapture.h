// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "NiagaraDebugger.h"
#include "NiagaraRendererReadback.h"

//#include "TickableEditorObject.h"

#include "SNiagaraDebuggerMeshCapture.generated.h"

USTRUCT()
struct FNiagaraDebuggerMeshCaptureSettings
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Settings")
	TObjectPtr<UStaticMesh> MeshOutput;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ShowOnlyInnerProperties))
	FNiagaraRendererReadbackParameters ExportParameters;
};

#if WITH_NIAGARA_DEBUGGER
class SNiagaraDebuggerMeshCapture : public SCompoundWidget
{
public:
	static const FName TabName;

	SLATE_BEGIN_ARGS(SNiagaraDebuggerMeshCapture) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

private:
	TSharedRef<SWidget> MakeToolbar();

	TArray<UNiagaraComponent*> GetSelectComponents(bool bFindFirstOnly = false) const;
	
	bool IsOutputLocationValid() const;

	bool CanCaptureSelected() const;
	void OnCaptureSelected();

	bool CanCaptureAll() const;
	void OnCaptureAll();

private:
	FNiagaraDebuggerMeshCaptureSettings	CaptureSettings;
};
#endif //WITH_NIAGARA_DEBUGGER
