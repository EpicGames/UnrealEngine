// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraDebuggerSpawn.h"
#include "NiagaraEditorStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/FileHelper.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SNiagaraDebuggerSpawn)

#if WITH_NIAGARA_DEBUGGER
#define LOCTEXT_NAMESPACE "SNiagaraDebuggerSpawn"

void SNiagaraDebuggerSpawn::Construct(const FArguments& InArgs)
{
	Debugger = InArgs._Debugger;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bHideSelectionTip = true;
	//DetailsViewArgs.NotifyHook = DebugHudSettings;

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	TSharedRef<IStructureDetailsView> StructureDetailsView = PropertyModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);
	//StructureDetailsView->GetDetailsView()->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDebugHUDSettingsDetailsCustomization::MakeInstance, DebugHudSettings));

	TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FNiagaraDebuggerSpawnData::StaticStruct(), reinterpret_cast<uint8*>(&SpawnData));
	StructureDetailsView->SetStructureData(StructOnScope);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MakeToolbar()
		]
 		+ SVerticalBox::Slot()
		.Padding(2.0)
		[
			StructureDetailsView->GetWidget().ToSharedRef()
		]
	];
}

TSharedRef<SWidget> SNiagaraDebuggerSpawn::MakeToolbar()
{
	FToolBarBuilder ToolbarBuilder(MakeShareable(new FUICommandList), FMultiBoxCustomization::None);
	ToolbarBuilder.BeginSection("DebuggerSpawn");

	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SNiagaraDebuggerSpawn::ExecuteSpawn), FCanExecuteAction::CreateSP(this, &SNiagaraDebuggerSpawn::CanExecuteSpawn)),
		NAME_None,
		LOCTEXT("SpawnSystems", "Spawn Systems"),
		LOCTEXT("SpawnSystemsTooltip", "Spawn the current systems.")
	);

	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SNiagaraDebuggerSpawn::SpawnFromTextFile), FCanExecuteAction::CreateSP(this, &SNiagaraDebuggerSpawn::CanSpawnFromTextFile)),
		NAME_None,
		LOCTEXT("SpawnFromTextFile", "Spawn From Text File"),
		LOCTEXT("SpawnFromTextFileTooltip", "Select file should contain a list of systems that you wish to spawn.")
	);

	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SNiagaraDebuggerSpawn::KillExisting)),
		NAME_None,
		LOCTEXT("KillExisting", "Kill Existing"),
		LOCTEXT("KillExistingTooltip", "Kill all existing systems we have debug spawned.")
	);

	ToolbarBuilder.EndSection();

	return ToolbarBuilder.MakeWidget();
}

bool SNiagaraDebuggerSpawn::CanExecuteSpawn() const
{
	return Debugger.IsValid() && SpawnData.SystemsToSpawn.Num() && SystemsToSpwan.Num() == 0;
}

void SNiagaraDebuggerSpawn::ExecuteSpawn()
{
	if (!CanExecuteSpawn())
	{
		return;
	}

	TArray<FString> SystemNames;
	SystemNames.Reserve(SpawnData.SystemsToSpawn.Num());
	for ( const TSoftObjectPtr<UNiagaraSystem>& SoftObject : SpawnData.SystemsToSpawn )
	{
		SystemNames.Emplace(SoftObject.ToString());
	}

	SpawnSystems(SystemNames);
}

void SNiagaraDebuggerSpawn::KillExisting()
{
	if ( !Debugger.IsValid() )
	{
		return;
	}
	Debugger->ExecConsoleCommand(TEXT("fx.Niagara.Debug.KillSpawned"), true);
	SystemsToSpwan.Empty();
}

bool SNiagaraDebuggerSpawn::CanSpawnFromTextFile() const
{
	return Debugger.IsValid() && SystemsToSpwan.Num() == 0;
}

void SNiagaraDebuggerSpawn::SpawnFromTextFile()
{
	if ( !CanSpawnFromTextFile() )
	{
		return;
	}

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform == nullptr)
	{
		return;
	}

	void* ParentWindowWindowHandle = nullptr;
	{
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}
	}

	FString DefaultFolder = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::PROJECT);
	FString DefaultFilename;
	FString FileTypes = TEXT("All Files (*.*)|*.*");
	TArray<FString> OpenFilenames;

	if (!DesktopPlatform->OpenFileDialog(ParentWindowWindowHandle, TEXT("System List"), DefaultFolder, DefaultFilename, FileTypes, EFileDialogFlags::None, OpenFilenames))
	{
		return;
	}

	if ( !OpenFilenames.Num() )
	{
		return;
	}

	TArray<FString> SystemNames;
	if ( FFileHelper::LoadFileToStringArray(SystemNames, *OpenFilenames[0]) )
	{
		SpawnSystems(SystemNames);
	}
}

void SNiagaraDebuggerSpawn::SpawnSystems(TConstArrayView<FString> SystemNames)
{
	TStringBuilder<512> ArgsBuilder;
	ArgsBuilder.Appendf(TEXT("AttachToPlayer=%d "), SpawnData.bAttachToPlayer);
	ArgsBuilder.Appendf(TEXT("AutoDestroy=%d "), SpawnData.bAutoDestroy);
	ArgsBuilder.Appendf(TEXT("AutoActivate=%d "), SpawnData.bAutoActivate);
	ArgsBuilder.Appendf(TEXT("PreCullCheck=%d "), SpawnData.bDoPreCullCheck);
	if (SpawnData.bWorldLocation)
	{
		ArgsBuilder.Appendf(TEXT("Location=%f,%f,%f "), SpawnData.Location.X, SpawnData.Location.Y, SpawnData.Location.Z);
	}
	else
	{
		ArgsBuilder.Appendf(TEXT("LocationFromPlayer=%f,%f,%f "), SpawnData.Location.X, SpawnData.Location.Y, SpawnData.Location.Z);
	}

	bSpawnAllAtOnce = SpawnData.bSpawnAllAtOnce;
	bKillBeforeSpawn = SpawnData.bKillBeforeSpawn;
	DelayBetweenSpawn = SpawnData.TimeBetweenSpawns;
	CurrentTimeBetweenSpawn = DelayBetweenSpawn;
	SpawnCommandArgs = ArgsBuilder.ToString();
	SystemsToSpwan = SystemNames;
}

void SNiagaraDebuggerSpawn::Tick(float DeltaTime)
{
	if (SystemsToSpwan.Num() == 0 || !Debugger.IsValid())
	{
		return;
	}

	if ( bSpawnAllAtOnce )
	{
		if (bKillBeforeSpawn)
		{
			Debugger->ExecConsoleCommand(TEXT("fx.Niagara.Debug.KillSpawned"), true);
		}
		for ( const FString& SystemName : SystemsToSpwan )
		{
			FString Cmd = FString::Printf(TEXT("fx.Niagara.Debug.SpawnComponent %s %s"), *SystemsToSpwan[0], *SpawnCommandArgs);
			Debugger->ExecConsoleCommand(*Cmd, true);
		}
		SystemsToSpwan.Empty();
	}
	else
	{
		CurrentTimeBetweenSpawn += DeltaTime;
		while ((CurrentTimeBetweenSpawn >= DelayBetweenSpawn) && SystemsToSpwan.Num())
		{
			CurrentTimeBetweenSpawn -= DelayBetweenSpawn;

			if (bKillBeforeSpawn)
			{
				Debugger->ExecConsoleCommand(TEXT("fx.Niagara.Debug.KillSpawned"), true);
			}

			FString Cmd = FString::Printf(TEXT("fx.Niagara.Debug.SpawnComponent %s %s"), *SystemsToSpwan[0], *SpawnCommandArgs);
			Debugger->ExecConsoleCommand(*Cmd, true);

			SystemsToSpwan.RemoveAt(0);
		}
	}
}

bool SNiagaraDebuggerSpawn::IsTickable() const
{
	return true;
}

TStatId SNiagaraDebuggerSpawn::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(SNiagaraDebuggerSpawn, STATGROUP_Tickables);
}

#undef LOCTEXT_NAMESPACE
#endif //WITH_NIAGARA_DEBUGGER

