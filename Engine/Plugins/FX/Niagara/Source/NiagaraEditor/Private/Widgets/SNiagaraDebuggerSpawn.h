// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "NiagaraDebugger.h"

#include "TickableEditorObject.h"

#include "SNiagaraDebuggerSpawn.generated.h"

USTRUCT()
struct FNiagaraDebuggerSpawnData
{
	GENERATED_BODY()

	/** List of all the systems we want to spawn. */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	TArray<TSoftObjectPtr<UNiagaraSystem>> SystemsToSpawn;

	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (InlineEditConditionToggle))
	bool bSpawnAllAtOnce = true;

	/** The time delay we should use between spawning if we have a list to spawn. */
	UPROPERTY(EditAnywhere, Category = "Spawn", meta = (EditCondition = "!bSpawnAllAtOnce"))
	float TimeBetweenSpawns = 1.0f;

	/** Should we kill systems we spawn before we spawn another. */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	bool bKillBeforeSpawn = true;

	/** When true the location is a world location, when false it's relative to the player and is in camera space. */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	bool bWorldLocation = false;

	/** The location we should use to spawn the system at, either world or local to the player depending on WorldLocation flag. */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	FVector Location = FVector(500.0f, 0.0f, 0.0f);

	/** Should we attach to the player controlled by the camera or not */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	bool bAttachToPlayer = false;

	/** Should we auto activate or not */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	bool bAutoActivate = true;

	/** Should we auto destroy when complete or not */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	bool bAutoDestroy = true;

	/** Should we perform the pre cull check or not */
	UPROPERTY(EditAnywhere, Category = "Spawn")
	bool bDoPreCullCheck = true;
};

#if WITH_NIAGARA_DEBUGGER
class SNiagaraDebuggerSpawn : public SCompoundWidget, FTickableEditorObject
{
public:
	static const FName TabName;

	SLATE_BEGIN_ARGS(SNiagaraDebuggerSpawn) {}
		SLATE_ARGUMENT(TSharedPtr<class FNiagaraDebugger>, Debugger)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

	TSharedRef<SWidget> MakeToolbar();

	bool CanExecuteSpawn() const;
	void ExecuteSpawn();

	void KillExisting();

	bool CanSpawnFromTextFile() const;
	void SpawnFromTextFile();

	void SpawnSystems(TConstArrayView<FString> SystemNames);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override { SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime); }

	// FTickableEditorObject Impl
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;
	// FTickableEditorObject Impl

private:
	TSharedPtr<class FNiagaraDebugger>	Debugger;
	FNiagaraDebuggerSpawnData			SpawnData;

	bool			bSpawnAllAtOnce = false;
	bool			bKillBeforeSpawn = true;
	float			DelayBetweenSpawn = 0.0f;
	float			CurrentTimeBetweenSpawn = 0.0f;
	FString			SpawnCommandArgs;
	TArray<FString>	SystemsToSpwan;
};
#endif //WITH_NIAGARA_DEBUGGER
