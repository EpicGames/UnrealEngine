// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraDistributionArrayEditor.h"

#include "Widgets/INiagaraDistributionAdapter.h"
#include "Widgets/SNiagaraDistributionLookupValueWidget.h"
#include "Widgets/SNiagaraParameterName.h"
#include "SNiagaraParameterEditor.h"
#include "NiagaraVariableMetaData.h"
#include "NiagaraEditorModule.h"
#include "INiagaraEditorTypeUtilities.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "UObject/StructOnScope.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "IDetailCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IStructureDetailsView.h"

#define LOCTEXT_NAMESPACE "NiagaraDistributionArrayEditor"

//-TODO: Copy & Paste handling :grimacing:

namespace SNiagaraDistributionArrayEditorPrivate
{
	template<typename TEnumType>
	FText GetEnumToolTipTextByValue(TEnumType EnumValue)
	{
		const UEnum* Enum = StaticEnum<TEnumType>();
		const int64 EnumIndex = Enum->GetIndexByValue(int64(EnumValue));
		return Enum->GetToolTipTextByIndex(EnumIndex);
	}

	TSharedRef<SWidget> GetEnumMenuOptions(const UEnum* Enum, TFunction<FExecuteAction(int64)> CreateAction)
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		MenuBuilder.BeginSection("Actions");

		for (int64 iValue=0; iValue < Enum->GetMaxEnumValue(); ++iValue)
		{
			if (Enum->IsValidEnumValue(iValue) == false)
			{
				continue;
			}

			const int64 iIndex = Enum->GetIndexByValue(iValue);

			MenuBuilder.AddMenuEntry(
				Enum->GetDisplayNameTextByIndex(iIndex),
				Enum->GetToolTipTextByIndex(iIndex),
				FSlateIcon(),
				FUIAction(CreateAction(iValue))
			);
		}
		MenuBuilder.EndSection();
		return MenuBuilder.MakeWidget();
	}

	int32 GetArrayNum(const TSharedPtr<INiagaraDistributionAdapter>& Distribution)
	{
		return Distribution ? Distribution->GetArrayValues().Num() / Distribution->GetNumChannels() : 0;
	}

	TArray<float> GetArrayElement(const TSharedPtr<INiagaraDistributionAdapter>& Distribution, int32 Index)
	{
		TArray<float> Value;
		if (Distribution && Index >= 0 && Index < GetArrayNum(Distribution))
		{
			const int32 NumChannels = Distribution->GetNumChannels();
			Value.AddDefaulted(NumChannels);

			const int32 ArrayOffset = NumChannels * Index;
			TConstArrayView<float> ArrayData = Distribution->GetArrayValues();
			for (int32 i = 0; i < NumChannels; ++i)
			{
				Value[i] = ArrayData[ArrayOffset + i];
			}
		}
		return Value;
	}

	void SetArrayElement(const TSharedPtr<INiagaraDistributionAdapter>& Distribution, int32 Index, TConstArrayView<float> Value)
	{
		if (Distribution && Index >= 0 && Index < GetArrayNum(Distribution))
		{
			TArray<float> ArrayData(Distribution->GetArrayValues());

			const int32 NumChannels = Distribution->GetNumChannels();
			const int32 ArrayOffset = NumChannels * Index;
			for (int32 i = 0; i < NumChannels; ++i)
			{
				ArrayData[ArrayOffset + i] = Value[i];
			}

			Distribution->SetArrayValues(ArrayData);
		}
	}

	void SwapArrayElement(const TSharedPtr<INiagaraDistributionAdapter>& Distribution, int32 IndexA, int32 IndexB)
	{
		if (Distribution && IndexA >= 0 && IndexA < GetArrayNum(Distribution) && IndexB >= 0 && IndexB < GetArrayNum(Distribution))
		{
			TArray<float> ArrayData(Distribution->GetArrayValues());

			const int32 NumChannels = Distribution->GetNumChannels();
			const int32 ArrayOffsetA = NumChannels * IndexA;
			const int32 ArrayOffsetB = NumChannels * IndexB;
			for (int32 i = 0; i < NumChannels; ++i)
			{
				Swap(ArrayData[ArrayOffsetA + i], ArrayData[ArrayOffsetB + i]);
			}

			Distribution->SetArrayValues(ArrayData);
		}
	}

	void InsertArrayElement(const TSharedPtr<INiagaraDistributionAdapter>& Distribution, int32 Index)
	{
		if (Distribution && Index >= 0 && Index <= GetArrayNum(Distribution))
		{
			const int32 NumChannels = Distribution->GetNumChannels();
			const int32 ArrayOffset = NumChannels * Index;

			TArray<float> ArrayData(Distribution->GetArrayValues());
			if (ArrayOffset == ArrayData.Num())
			{
				for (int32 i = 0; i < NumChannels; ++i)
				{
					ArrayData.Emplace(1.0f);
				}
			}
			else
			{
				for (int32 i = 0; i < NumChannels; ++i)
				{
					ArrayData.Insert(1.0f, ArrayOffset);
				}
			}

			Distribution->SetArrayValues(ArrayData);
		}
	}

	void DuplicateArrayElement(const TSharedPtr<INiagaraDistributionAdapter>& Distribution, int32 Index)
	{
		if (Distribution && Index >= 0 && Index < GetArrayNum(Distribution))
		{
			const int32 NumChannels = Distribution->GetNumChannels();
			const int32 ArrayOffset = NumChannels * Index;

			TArray<float> ElementToDuplicate = GetArrayElement(Distribution, Index);

			TArray<float> ArrayData(Distribution->GetArrayValues());
			ArrayData.InsertZeroed(ArrayOffset, NumChannels);
			for (int32 i = 0; i < NumChannels; ++i)
			{
				ArrayData[ArrayOffset + i] = ElementToDuplicate[i];
			}

			Distribution->SetArrayValues(ArrayData);
		}
	}

	void RemoveArrayElement(const TSharedPtr<INiagaraDistributionAdapter>& Distribution, int32 Index)
	{
		if (Distribution && Index >= 0 && Index < GetArrayNum(Distribution))
		{
			const int32 NumChannels = Distribution->GetNumChannels();
			const int32 ArrayOffset = NumChannels * Index;

			TArray<float> ArrayData(Distribution->GetArrayValues());
			ArrayData.RemoveAt(ArrayOffset, NumChannels);

			Distribution->SetArrayValues(ArrayData);
		}
	}

	class SArrayButton : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SArrayButton) {}
			SLATE_ARGUMENT(FText, ToolTip)
			SLATE_ARGUMENT_DEFAULT( const FSlateBrush*, Image ) = FAppStyle::GetBrush("Default");
			SLATE_ARGUMENT_DEFAULT( bool, IsFocusable ) = false;
			SLATE_ATTRIBUTE( bool, IsEnabled )
			SLATE_EVENT(FSimpleDelegate, OnClicked)
		SLATE_END_ARGS()

		void Construct( const FArguments& InArgs )
		{
			OnClickedDelegate = InArgs._OnClicked;
			ChildSlot
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(22.0f)
				.HeightOverride(22.0f)
				.ToolTipText(InArgs._ToolTip)
				[
					SNew(SButton)
					.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
					.OnClicked(this, &SArrayButton::OnClicked)
					.ContentPadding(0.0f)
					.IsFocusable(InArgs._IsFocusable)
					.IsEnabled(InArgs._IsEnabled)
					[ 
						SNew( SImage )
						.Image( InArgs._Image )
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
			]; 
		}

		FReply OnClicked() const
		{
			OnClickedDelegate.ExecuteIfBound();
			return FReply::Handled();
		}

		FSimpleDelegate OnClickedDelegate;
	};

	class SArrayToggleButton : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SArrayToggleButton) {}
			SLATE_ARGUMENT(FText, ToolTip)
			SLATE_ARGUMENT_DEFAULT( const FSlateBrush*, Image ) = FAppStyle::GetBrush("Default");
			SLATE_ARGUMENT_DEFAULT( bool, IsFocusable ) = false;
			SLATE_ATTRIBUTE( bool, IsEnabled )
			SLATE_EVENT(FSimpleDelegate, OnClicked)
		SLATE_END_ARGS()

		void Construct( const FArguments& InArgs )
		{
			OnClickedDelegate = InArgs._OnClicked;
			ChildSlot
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(22.0f)
				.HeightOverride(22.0f)
				.ToolTipText(InArgs._ToolTip)
				[
					SNew(SButton)
					.ButtonStyle( FAppStyle::Get(), "ToggleButton" )
					.OnClicked(this, &SArrayToggleButton::OnClicked)
					.ContentPadding(0.0f)
					.IsFocusable(InArgs._IsFocusable)
					.IsEnabled(InArgs._IsEnabled)
					[ 
						SNew( SImage )
						.Image( InArgs._Image )
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
			]; 
		}

		FReply OnClicked() const
		{
			OnClickedDelegate.ExecuteIfBound();
			return FReply::Handled();
		}

		FSimpleDelegate OnClickedDelegate;
	};

	class SArrayEntryWidget : public STableRow<TSharedPtr<int32>>
	{
	public:
		SLATE_BEGIN_ARGS(SArrayEntryWidget) {}
			SLATE_ARGUMENT(TSharedPtr<INiagaraDistributionAdapter>, DistributionAdapter)
			SLATE_ARGUMENT(TSharedPtr<int32>, RowIndexPtr)
			SLATE_EVENT(FSimpleDelegate, OnValueChanged)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
		{
			DistributionAdapter = InArgs._DistributionAdapter;
			RowIndexPtr = InArgs._RowIndexPtr;
			OnValueChangedDelegate = InArgs._OnValueChanged;

			STableRow::Construct(
				STableRow<TSharedPtr<int32>>::FArguments()
				.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow")
				, InOwnerTableView
			);

			if (DistributionAdapter.IsValid() && RowIndexPtr.IsValid())
			{
				FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
				const FNiagaraTypeDefinition TypeDef = DistributionAdapter->GetExpressionTypeDef();
				TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(TypeDef);
				if (TypeEditorUtilities.IsValid())
				{
					DistributionAdapter->OnDistributionValueChanged().AddSP(this, &SArrayEntryWidget::OnExternalChange);

					ElementData = GetArrayElement(DistributionAdapter, *RowIndexPtr);

					ValueStructOnScope = MakeShared<FStructOnScope>(TypeDef.GetStruct(), reinterpret_cast<uint8*>(ElementData.GetData()));

					FNiagaraInputParameterCustomization CustomizationOptions;
					CustomizationOptions.bBroadcastValueChangesOnCommitOnly = true;
					ValueParameterEditor = TypeEditorUtilities->CreateParameterEditor(TypeDef, EUnit::Unspecified, CustomizationOptions);
					ValueParameterEditor->UpdateInternalValueFromStruct(ValueStructOnScope.ToSharedRef());
					ValueParameterEditor->SetOnValueChanged(
						SNiagaraParameterEditor::FOnValueChange::CreateSP(this, &SArrayEntryWidget::OnValueChanged)
					);

					ChildSlot
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						. AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							. AutoWidth()
							[
								SNew(SBox)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(FMargin(0, 0, 5, 0))
								[
									SNew(STextBlock)
									.Font(IDetailLayoutBuilder::GetDetailFont())
									.Text(FText::Format(LOCTEXT("ArrayEntryFormat", "[{0}]"), FText::AsNumber(*RowIndexPtr)))
								]
							]
							+ SHorizontalBox::Slot()
							. FillWidth(1.0f)
							[
								ValueParameterEditor.ToSharedRef()
							]
							+ SHorizontalBox::Slot()
							. AutoWidth()
							[
								SNew(SNiagaraDistributionArrayEditorPrivate::SArrayButton)
								.Image(FAppStyle::GetBrush(TEXT("Icons.ArrowUp")))
								.ToolTip(LOCTEXT("MoveElementUp", "Move this element up one"))
								.IsEnabled(this, &SArrayEntryWidget::CanMoveElementUp)
								.OnClicked(this, &SArrayEntryWidget::MoveElementUp)
							]
							+ SHorizontalBox::Slot()
							. AutoWidth()
							[
								SNew(SNiagaraDistributionArrayEditorPrivate::SArrayButton)
								.Image(FAppStyle::GetBrush(TEXT("Icons.ArrowDown")))
								.ToolTip(LOCTEXT("MoveElementDown", "Move this element down one"))
								.IsEnabled(this, &SArrayEntryWidget::CanMoveElementDown)
								.OnClicked(this, &SArrayEntryWidget::MoveElementDown)
							]
							//+ SHorizontalBox::Slot()
							//. AutoWidth()
							//[
								//SNew(SNiagaraDistributionArrayEditorPrivate::SArrayButton)
								//.Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")))
								//.ToolTip(LOCTEXT("InsertElement", "Inset a new element above this one"))
								//.OnClicked(this, &SArrayEntryWidget::InsertElementAbove)
							//]
							//+ SHorizontalBox::Slot()
							//.AutoWidth()
							//[
							//	SNew(SNiagaraDistributionArrayEditorPrivate::SArrayButton)
							//	.Image(FAppStyle::GetBrush(TEXT("Icons.Minus")))
							//	.ToolTip(LOCTEXT("RemoveElement", "Remove this element"))
							//	.OnClicked(this, &SArrayEntryWidget::RemoveElement)
							//]
							+ SHorizontalBox::Slot()
							. AutoWidth()
							[
								PropertyCustomizationHelpers::MakeInsertDeleteDuplicateButton(
									FExecuteAction::CreateSP(this, &SArrayEntryWidget::InsertElementAbove),
									FExecuteAction::CreateSP(this, &SArrayEntryWidget::RemoveElement),
									FExecuteAction::CreateSP(this, &SArrayEntryWidget::DuplicateElementAbove)
								)
							]
						]
					];
				}
			}
		}

		bool CanMoveElementUp() const
		{
			return RowIndexPtr && *RowIndexPtr > 0;
		}
		
		bool CanMoveElementDown() const
		{
			return RowIndexPtr && *RowIndexPtr < GetArrayNum(DistributionAdapter) - 1;
		}

		void MoveElementUp()
		{
			const int32 Index = *RowIndexPtr;
			SwapArrayElement(DistributionAdapter, Index, Index - 1);
			OnValueChangedDelegate.ExecuteIfBound();
		}

		void MoveElementDown()
		{
			const int32 Index = *RowIndexPtr;
			SwapArrayElement(DistributionAdapter, Index, Index + 1);
			OnValueChangedDelegate.ExecuteIfBound();
		}

		void InsertElementAbove()
		{
			const int32 Index = *RowIndexPtr;
			InsertArrayElement(DistributionAdapter, Index);
			OnValueChangedDelegate.ExecuteIfBound();
		}

		void DuplicateElementAbove()
		{
			const int32 Index = *RowIndexPtr;
			DuplicateArrayElement(DistributionAdapter, Index);
			OnValueChangedDelegate.ExecuteIfBound();
		}

		void RemoveElement()
		{
			const int32 Index = *RowIndexPtr;
			RemoveArrayElement(DistributionAdapter, Index);
			OnValueChangedDelegate.ExecuteIfBound();
		}

		void OnValueChanged() const
		{
			ValueParameterEditor->UpdateStructFromInternalValue(ValueStructOnScope.ToSharedRef());
			SetArrayElement(DistributionAdapter, *RowIndexPtr, ElementData);
		}

		void OnExternalChange()
		{
			if (*RowIndexPtr >= GetArrayNum(DistributionAdapter))
			{
				// This can happen when we remove an element
				return;
			}

			TArray<float> NewElementData = GetArrayElement(DistributionAdapter, *RowIndexPtr);
			for (int32 i = 0; i < ElementData.Num(); ++i)
			{
				ElementData[i] = NewElementData[i];
			}
			ValueParameterEditor->UpdateInternalValueFromStruct(ValueStructOnScope.ToSharedRef());
		}

		TSharedPtr<INiagaraDistributionAdapter>		DistributionAdapter;
		TSharedPtr<int32>							RowIndexPtr;
		TArray<float>								ElementData;
		FSimpleDelegate								OnValueChangedDelegate;

		TSharedPtr<FStructOnScope>					ValueStructOnScope;
		TSharedPtr<SNiagaraParameterEditor>			ValueParameterEditor;
	};
}

void SNiagaraDistributionArrayEditor::Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter)
{
	DistributionAdapter = InDistributionAdapter;

	RefreshRows();

	SAssignNew(ArrayListViewWidget, SListView<TSharedPtr<int32>>)
		.ListItemsSource(&ArrayEntryItems)
		.OnGenerateRow(this, &SNiagaraDistributionArrayEditor::MakeArrayEntryWidget)
		.Visibility(EVisibility::Visible)
		.SelectionMode(ESelectionMode::Single)
		.ConsumeMouseWheel(EConsumeMouseWheel::Never)
		.ScrollbarVisibility(EVisibility::Collapsed)
		;

	FSlimHorizontalToolBarBuilder ToolBarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	{
		ToolBarBuilder.SetStyle(&FAppStyle::Get(), "DetailsView.ExtensionToolBar");
		ToolBarBuilder.SetIsFocusable(false);
		ToolBarBuilder.BeginSection("Array");

		ToolBarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraDistributionArrayEditor::AddArrayElement)),
			NAME_None,
			FText(),
			LOCTEXT("AddElementToEnd", "Add a new element to the end of the array"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.PlusCircle"),
			EUserInterfaceActionType::Button
		);

		ToolBarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraDistributionArrayEditor::RemoveAllArrayElements)),
			NAME_None,
			FText(),
			LOCTEXT("DeleteAllElements", "Delete all elements in the array"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
			EUserInterfaceActionType::Button
		);

		ToolBarBuilder.EndSection();
	}

	TSharedRef<SGridPanel> GridPanel = SNew(SGridPanel);
	int32 GridPanelY = 0;

	if (DistributionAdapter->AllowLookupValueMode())
	{
		GridPanel->AddSlot(0, GridPanelY)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("LookupValueMode", "Lookup Value: "))
		];

		GridPanel->AddSlot(1, GridPanelY)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SNiagaraDistributionLookupValueWidget, DistributionAdapter)
			.ShowValueOnly(true)
		];
		++GridPanelY;
	}

	if (DistributionAdapter->AllowInterpolationMode())
	{
		GridPanel->AddSlot(0, GridPanelY)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("InterpolationMode", "Interpolation: "))
		];

		GridPanel->AddSlot(1, GridPanelY)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &SNiagaraDistributionArrayEditor::OnGetInterpolationModeOptions)
			.ContentPadding(1)
			//-TODO:.ToolTip(this, &SNiagaraDistributionArrayEditor::GetInterpolationModeToolTip)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &SNiagaraDistributionArrayEditor::GetInterpolationModeText)
			]
		];
		++GridPanelY;
	}

	if (DistributionAdapter->AllowAddressMode())
	{
		GridPanel->AddSlot(0, GridPanelY)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("AddessMode", "Address Mode: "))
		];

		GridPanel->AddSlot(1, GridPanelY)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &SNiagaraDistributionArrayEditor::OnGetAddressModeOptions)
			.ContentPadding(1)
			//-TODO:.ToolTip(this, &SNiagaraDistributionArrayEditor::GetAddressModeToolTip)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &SNiagaraDistributionArrayEditor::GetAddressModeText)
			]
		];
		++GridPanelY;
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		. AutoHeight()
		[
			GridPanel
		]
		//////////////////////////////////////////////////////
		+ SVerticalBox::Slot()
		. AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			. AutoWidth()
			[
				SNew(SBox)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0,0,5,0)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(this, &SNiagaraDistributionArrayEditor::GetArrayHeaderText)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				ToolBarBuilder.MakeWidget()
			]
		]
		+ SVerticalBox::Slot()
		. AutoHeight()
		[
			ArrayListViewWidget.ToSharedRef()
		]
	];
}

int32 SNiagaraDistributionArrayEditor::GetArrayNum() const
{
	return SNiagaraDistributionArrayEditorPrivate::GetArrayNum(DistributionAdapter);
}

FText SNiagaraDistributionArrayEditor::GetArrayHeaderText() const
{
	const int32 NumElements = GetArrayNum();
	return FText::Format(NumElements > 1 ? LOCTEXT("NumArrayElements", "{0} array elements") : LOCTEXT("NumArrayElement", "{0} array element"), FText::AsNumber(NumElements));
}

void SNiagaraDistributionArrayEditor::AddArrayElement()
{
	const int32 NumEntries = SNiagaraDistributionArrayEditorPrivate::GetArrayNum(DistributionAdapter);
	if (NumEntries == 0)
	{
		SNiagaraDistributionArrayEditorPrivate::InsertArrayElement(DistributionAdapter, 0);
	}
	else
	{
		SNiagaraDistributionArrayEditorPrivate::DuplicateArrayElement(DistributionAdapter, NumEntries - 1);
	}
	RefreshRows();
}

void SNiagaraDistributionArrayEditor::RemoveAllArrayElements()
{
	DistributionAdapter->SetArrayValues({});

	RefreshRows();
}

void SNiagaraDistributionArrayEditor::SetInterpolationMode(ENiagaraDistributionInterpolationMode Mode)
{
	DistributionAdapter->SetInterpolationMode(Mode);
}

FText SNiagaraDistributionArrayEditor::GetInterpolationModeText() const
{
	return UEnum::GetDisplayValueAsText(DistributionAdapter->GetInterpolationMode());
}

FText SNiagaraDistributionArrayEditor::GetInterpolationModeToolTip() const
{
	return SNiagaraDistributionArrayEditorPrivate::GetEnumToolTipTextByValue(DistributionAdapter->GetInterpolationMode());
}

void SNiagaraDistributionArrayEditor::SetAddressMode(ENiagaraDistributionAddressMode Mode)
{
	DistributionAdapter->SetAddressMode(Mode);
}

FText SNiagaraDistributionArrayEditor::GetAddressModeText() const
{
	return UEnum::GetDisplayValueAsText(DistributionAdapter->GetAddressMode());
}

FText SNiagaraDistributionArrayEditor::GetAddressModeToolTip() const
{
	return SNiagaraDistributionArrayEditorPrivate::GetEnumToolTipTextByValue(DistributionAdapter->GetAddressMode());
}

void SNiagaraDistributionArrayEditor::RefreshRows()
{
	const int32 NumElements = GetArrayNum();
	if (ArrayEntryItems.Num() > NumElements)
	{
		ArrayEntryItems.SetNum(NumElements);
	}
	else
	{
		while (ArrayEntryItems.Num() < NumElements)
		{
			ArrayEntryItems.Emplace(MakeShared<int32>(ArrayEntryItems.Num()));
		}
	}
	if (ArrayListViewWidget.IsValid())
	{
		ArrayListViewWidget->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SNiagaraDistributionArrayEditor::MakeArrayEntryWidget(const TSharedPtr<int32> RowIndexPtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	return
		SNew(SNiagaraDistributionArrayEditorPrivate::SArrayEntryWidget, OwnerTable)
		.DistributionAdapter(DistributionAdapter)
		.RowIndexPtr(RowIndexPtr)
		.OnValueChanged(this, &SNiagaraDistributionArrayEditor::RefreshRows)
		;
}

TSharedRef<SWidget> SNiagaraDistributionArrayEditor::OnGetInterpolationModeOptions()
{
	return SNiagaraDistributionArrayEditorPrivate::GetEnumMenuOptions(
		StaticEnum<ENiagaraDistributionInterpolationMode>(),
		[&](int64 iValue)
		{
			return FExecuteAction::CreateSP(this, &SNiagaraDistributionArrayEditor::SetInterpolationMode, ENiagaraDistributionInterpolationMode(iValue));
		}
	);
}

TSharedRef<SWidget> SNiagaraDistributionArrayEditor::OnGetAddressModeOptions()
{
	return SNiagaraDistributionArrayEditorPrivate::GetEnumMenuOptions(
		StaticEnum<ENiagaraDistributionAddressMode>(),
		[&](int64 iValue)
		{
			return FExecuteAction::CreateSP(this, &SNiagaraDistributionArrayEditor::SetAddressMode, ENiagaraDistributionAddressMode(iValue));
		}
	);
}

#undef LOCTEXT_NAMESPACE
