// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessDistribution.h"

#include "Misc/NotifyHook.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SCompoundWidget.h"

#include "SNiagaraDistributionArrayEditor.generated.h"

class INiagaraDistributionAdapter;
class ITableRow;

USTRUCT()
struct FNiagaraDistributionArray
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Array")
	TArray<float> FloatData;

	UPROPERTY(EditAnywhere, Category="Array")
	TArray<FVector2f> Vector2Data;

	UPROPERTY(EditAnywhere, Category="Array")
	TArray<FVector3f> Vector3Data;

	UPROPERTY(EditAnywhere, Category="Array")
	TArray<FVector4f> Vector4Data;
};

class SNiagaraDistributionArrayEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDistributionArrayEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter);

private:
	int32 GetArrayNum() const;
	FText GetArrayHeaderText() const;

	void AddArrayElement();
	void RemoveAllArrayElements();

	void RefreshRows();

	TSharedRef<ITableRow> MakeArrayEntryWidget(const TSharedPtr<int32> RowIndexPtr, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedRef<SWidget> OnGetInterpolationModeOptions();
	TSharedRef<SWidget> OnGetAddressModeOptions();

	void SetInterpolationMode(ENiagaraDistributionInterpolationMode Mode);
	FText GetInterpolationModeText() const;
	FText GetInterpolationModeToolTip() const;

	void SetAddressMode(ENiagaraDistributionAddressMode Mode);
	FText GetAddressModeText() const;
	FText GetAddressModeToolTip() const;

private:
	TSharedPtr<INiagaraDistributionAdapter>		DistributionAdapter;
	FNiagaraDistributionArray					ArrayStruct;

	TSharedPtr<SListView<TSharedPtr<int32>>>	ArrayListViewWidget;
	TArray<TSharedPtr<int32>>					ArrayEntryItems;
};
