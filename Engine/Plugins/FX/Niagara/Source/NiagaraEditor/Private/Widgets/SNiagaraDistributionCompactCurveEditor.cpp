// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraDistributionCompactCurveEditor.h"

#include "NiagaraEditorStyle.h"
#include "ScopedTransaction.h"
#include "SEnumCombo.h"
#include "Widgets/INiagaraDistributionAdapter.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SLeafWidget.h"

#define LOCTEXT_NAMESPACE "NiagaraDistributionCompactCurveEditor"

class SNiagaraCompactCurveView : public SLeafWidget
{
	SLATE_BEGIN_ARGS(SNiagaraCompactCurveView)
		: _Width(20)
		, _Height(20)
		, _CurveColor(FSlateColor(EStyleColor::White))
		, _CurveStateSerialNumber(0)
		{ }
		SLATE_ARGUMENT(float, Width)
		SLATE_ARGUMENT(float, Height)
		SLATE_ARGUMENT(FSlateColor, CurveColor)
		SLATE_ATTRIBUTE(FKeyHandle, SelectedKeyHandle)
		SLATE_ATTRIBUTE(int32, CurveStateSerialNumber)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FRichCurve* InCurveToDisplay)
	{
		Width = InArgs._Width;
		Height = InArgs._Height;
		CurveColor = InArgs._CurveColor;
		SelectedKeyHandle = InArgs._SelectedKeyHandle;
		CurveStateSerialNumber = InArgs._CurveStateSerialNumber;
		CurveToDisplay = InCurveToDisplay;

		KeyColor = FSlateColor(EStyleColor::AccentGray);
		SelectedKeyColor = FSlateColor(EStyleColor::Select);
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		FKeyHandle NewSelectedKeyHandle = SelectedKeyHandle.Get();
		int32 NewCurveStateSerialNumber = CurveStateSerialNumber.Get(0);

		if (CurveStateSerialNumberCache != NewCurveStateSerialNumber || AllottedGeometryCache != AllottedGeometry)
		{
			SelectedKeyHandleCache = NewSelectedKeyHandle;
			CurveStateSerialNumberCache = NewCurveStateSerialNumber;
			AllottedGeometryCache = AllottedGeometry;
			CacheLines(AllottedGeometry.Size.X, AllottedGeometry.Size.Y, true, true, true);
		}
		else if (SelectedKeyHandleCache != NewSelectedKeyHandle)
		{
			SelectedKeyHandleCache = NewSelectedKeyHandle;
			CacheLines(AllottedGeometry.Size.X, AllottedGeometry.Size.Y, false, false, true);
		}

		FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), CurvePointsCache, ESlateDrawEffect::None, CurveColor.GetColor(InWidgetStyle), true, 2.0f);
		for (const TArray<FVector2D>& KeyLinePoints : KeyLinesCache)
		{
			FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), KeyLinePoints, ESlateDrawEffect::None, KeyColor.GetColor(InWidgetStyle), true, 2.0f);
		}
		FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), SelectedKeyLineCache, ESlateDrawEffect::None, SelectedKeyColor.GetColor(InWidgetStyle), true, 3.0f);
		return LayerId;
	}

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return FVector2D(Width, Height);
	}

private:
	void CacheLines(float InWidth, float InHeight, bool bCacheCurve, bool bCacheKeys, bool bCacheSelectedKey) const
	{
		if (CurveToDisplay == nullptr)
		{
			return;
		}

		float TimeMin;
		float TimeMax;
		float ValueMin;
		float ValueMax;
		CurveToDisplay->GetTimeRange(TimeMin, TimeMax);
		CurveToDisplay->GetValueRange(ValueMin, ValueMax);

		float TimeRange = TimeMax != TimeMin ? TimeMax - TimeMin : 1;
		float ValueRange = ValueMax != ValueMin ? ValueMax - ValueMin : 1;

		if (bCacheCurve)
		{
			CurvePointsCache.Empty();
			int32 Points = CurveToDisplay->GetNumKeys() * 20;
			float TimeIncrement = TimeRange / (Points - 1);
			for (int32 i = 0; i < Points; i++)
			{
				float Time = TimeMin + i * TimeIncrement;
				float Value = CurveToDisplay->Eval(Time);

				float NormalizedX = (Time - TimeMin) / TimeRange;
				float NormalizedY = (Value - ValueMin) / ValueRange;
				CurvePointsCache.Add(FVector2D(NormalizedX * InWidth, (1 - NormalizedY) * InHeight));
			}
		}

		if (bCacheKeys)
		{
			KeyLinesCache.Empty();
			for (auto KeyIterator(CurveToDisplay->GetKeyIterator()); KeyIterator; ++KeyIterator)
			{
				const FRichCurveKey& Key = *KeyIterator;
				TArray<FVector2D>& KeyLinePoints = KeyLinesCache.AddDefaulted_GetRef();

				float NormalizedX = (Key.Time - TimeMin) / TimeRange;
				KeyLinePoints.Add(FVector2D(NormalizedX * InWidth, 0));
				KeyLinePoints.Add(FVector2D(NormalizedX * InWidth, InHeight));
			}
		}

		if (bCacheSelectedKey)
		{
			SelectedKeyLineCache.Empty();
			for (auto KeyHandleIterator(CurveToDisplay->GetKeyHandleIterator()); KeyHandleIterator; ++KeyHandleIterator)
			{
				if (*KeyHandleIterator == SelectedKeyHandleCache)
				{
					const FRichCurveKey& SelectedKey = CurveToDisplay->GetKey(SelectedKeyHandleCache);
					float NormalizedX = (SelectedKey.Time - TimeMin) / TimeRange;
					SelectedKeyLineCache.Add(FVector2D(NormalizedX * InWidth, 0));
					SelectedKeyLineCache.Add(FVector2D(NormalizedX * InWidth, InHeight));
					break;
				}
			}
		}
	}

	const FRichCurve* CurveToDisplay = nullptr;
	mutable TArray<FVector2D> CurvePointsCache;
	mutable TArray<TArray<FVector2D>> KeyLinesCache; 
	mutable TArray<FVector2D> SelectedKeyLineCache;
	mutable FGeometry AllottedGeometryCache;
	mutable FKeyHandle SelectedKeyHandleCache;
	mutable int32 CurveStateSerialNumberCache = 0;
	float Width;
	float Height;
	FSlateColor CurveColor;
	FSlateColor KeyColor;
	FSlateColor SelectedKeyColor;
	TAttribute<FKeyHandle> SelectedKeyHandle;
	TAttribute<int32> CurveStateSerialNumber;
};

class SNiagaraCompactCurveKeyHandle : public SBorder
{
	DECLARE_DELEGATE_OneParam(FOnMoved, float /* LocalDelta */);

	SLATE_BEGIN_ARGS(SNiagaraCompactCurveKeyHandle)
		: _Size(10, 10)
	{ }
		SLATE_ARGUMENT(FVector2f, Size)
		SLATE_ATTRIBUTE(bool, IsSelected)
		SLATE_EVENT(FSimpleDelegate, OnSelected)
		SLATE_EVENT(FOnMoved, OnMoved)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Size = InArgs._Size;
		IsSelected = InArgs._IsSelected;
		OnSelectedDelegate = InArgs._OnSelected;
		OnMovedDelegate = InArgs._OnMoved;
		SBorder::Construct(SBorder::FArguments()
			.BorderImage(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.DistributionEditor.CurveKeyHandle"))
			.BorderBackgroundColor(this, &SNiagaraCompactCurveKeyHandle::GetHandleColor)
			.Padding(0)
			[
				SNew(SBox)
				.WidthOverride(Size.X)
				.HeightOverride(Size.Y)
			]);
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			FVector2D MouseLocation = MouseEvent.GetScreenSpacePosition();
			LastMouseLocation = MouseLocation.X;
			return FReply::Handled().CaptureMouse(SharedThis(this));
		}
		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (this->HasMouseCapture())
		{
			if (IsSelected.Get() == false)
			{
				OnSelectedDelegate.ExecuteIfBound();
			}
			return FReply::Handled().ReleaseMouseCapture();
		}
		return FReply::Unhandled();
	}

	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (this->HasMouseCapture())
		{
			FVector2D MouseLocation = MouseEvent.GetScreenSpacePosition();
			OnMovedDelegate.ExecuteIfBound((MouseLocation.X - LastMouseLocation) / MyGeometry.Scale);
			LastMouseLocation = MouseLocation.X;
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

private:
	FSlateColor GetHandleColor() const
	{
		return IsSelected.Get()
			? FSlateColor(EStyleColor::Select)
			: FSlateColor(EStyleColor::White);
	}

private:
	FVector2f Size;
	TAttribute<bool> IsSelected;
	FSimpleDelegate OnSelectedDelegate;
	FOnMoved OnMovedDelegate;
	float LastMouseLocation;
};

class SNiagaraCompactCurveKeySelector : public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam(FOnKeySelected, FKeyHandle /* SelectedKeyHandle */);
	DECLARE_DELEGATE_TwoParams(FOnKeyMoved, FKeyHandle /* SelectedKeyHandle */, float /* TimeDelta */)

	SLATE_BEGIN_ARGS(SNiagaraCompactCurveKeySelector)
		: _HandleSize(10, 10)
		, _CurveStateSerialNumber(0)
	{ }
		SLATE_ARGUMENT(FVector2f, HandleSize)
		SLATE_ATTRIBUTE(FKeyHandle, SelectedKeyHandle)
		SLATE_ATTRIBUTE(int32, CurveStateSerialNumber)
		SLATE_EVENT(FOnKeySelected, OnKeySelected)
		SLATE_EVENT(FOnKeyMoved, OnKeyMoved)
	SLATE_END_ARGS()

private:
	struct FKeyWidgetData
	{
		FKeyHandle Handle;
		TSharedPtr<SNiagaraCompactCurveKeyHandle> HandleWidget;
		FMargin BorderPadding;
	};

public:
	void Construct(const FArguments& InArgs, FRichCurve* InEditCurve)
	{
		EditCurve = InEditCurve;
		HandleSize = InArgs._HandleSize;
		SelectedKeyHandle = InArgs._SelectedKeyHandle;
		CurveStateSerialNumber = InArgs._CurveStateSerialNumber;
		OnKeySelectedDelegate = InArgs._OnKeySelected;
		OnKeyMovedDelegate = InArgs._OnKeyMoved;
		LocalWidthCache = 100;

		ChildSlot
		[
			SAssignNew(BorderOverlay, SOverlay)
		];

		UpdateKeyWidgets();
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
		int32 CurrentCurveStateSerialNumber = CurveStateSerialNumber.Get();
		LocalWidthCache = AllottedGeometry.GetLocalSize().X - HandleSize.X;
		if (CachedCurveStateSerialNumber != CurrentCurveStateSerialNumber || AllottedGeometry != AllottedGeometryCache)
		{
			CachedCurveStateSerialNumber = CurrentCurveStateSerialNumber;
			UpdateKeyWidgets();
		}
		AllottedGeometryCache = AllottedGeometry;
	}

	void RequestWidgetUpdate() { bNeedsWidgetUpdate = true; }

private:
	void UpdateKeyWidgets()
	{
		if (KeyWidgetData.Num() != EditCurve->GetNumKeys() || bNeedsWidgetUpdate)
		{
			BorderOverlay->ClearChildren();
			KeyWidgetData.Empty();
			for (auto KeyHandleIterator = EditCurve->GetKeyHandleIterator(); KeyHandleIterator; KeyHandleIterator++)
			{
				int32 DataIndex = KeyWidgetData.Num();
				FKeyWidgetData& KeyWidgetDataItem = KeyWidgetData.AddDefaulted_GetRef();
				KeyWidgetDataItem.Handle = *KeyHandleIterator;
				TAttribute<FMargin> BorderPadding = TAttribute<FMargin>::CreateSP(this, &SNiagaraCompactCurveKeySelector::GetKeyBorderPadding, DataIndex);
				BorderOverlay->AddSlot()
				.Padding(BorderPadding)
				.HAlign(HAlign_Left)
				[
					SAssignNew(KeyWidgetDataItem.HandleWidget, SNiagaraCompactCurveKeyHandle)
					.Size(HandleSize)
					.IsSelected(this, &SNiagaraCompactCurveKeySelector::GetKeyIsSelected, KeyWidgetDataItem.Handle)
					.OnSelected(this, &SNiagaraCompactCurveKeySelector::KeyHandleSelected, KeyWidgetDataItem.Handle)
					.OnMoved(this, &SNiagaraCompactCurveKeySelector::KeyHandleMoved, KeyWidgetDataItem.Handle)
				];
			}
			bNeedsWidgetUpdate = false;
		}

		float TimeMin;
		float TimeMax;
		float ValueMin;
		float ValueMax;
		EditCurve->GetTimeRange(TimeMin, TimeMax);
		EditCurve->GetValueRange(ValueMin, ValueMax);
		float TimeRange = TimeMax != TimeMin ? TimeMax - TimeMin : 1;
		float ValueRange = ValueMax != ValueMin ? ValueMax - ValueMin : 1;
		for (FKeyWidgetData& KeyWidgetDataItem : KeyWidgetData)
		{
			const FRichCurveKey& Key = EditCurve->GetKey(KeyWidgetDataItem.Handle);
			float NormalizedX = (Key.Time - TimeMin) / TimeRange;
			KeyWidgetDataItem.BorderPadding = FMargin(LocalWidthCache * NormalizedX, 0, 0, 0);
		}
	}

	FMargin GetKeyBorderPadding(int32 KeyWidgetDataIndex) const
	{
		return KeyWidgetDataIndex < KeyWidgetData.Num() ? KeyWidgetData[KeyWidgetDataIndex].BorderPadding : FMargin(0);
	}

	bool GetKeyIsSelected(FKeyHandle KeyHandle) const
	{
		return KeyHandle == SelectedKeyHandle.Get();
	}

	void KeyHandleSelected(FKeyHandle KeyHandle)
	{
		OnKeySelectedDelegate.ExecuteIfBound(KeyHandle);
	}

	void KeyHandleMoved(float LocalDelta, FKeyHandle KeyHandle)
	{
		float TimeMin;
		float TimeMax;
		EditCurve->GetTimeRange(TimeMin, TimeMax);
		float TimeRange = TimeMax - TimeMin;
		float NormalizedDelta = FMath::IsNearlyZero(LocalWidthCache) ? 0.0f : LocalDelta / LocalWidthCache;
		if (SelectedKeyHandle.Get() != KeyHandle)
		{
			OnKeySelectedDelegate.ExecuteIfBound(KeyHandle);
		}
		OnKeyMovedDelegate.ExecuteIfBound(KeyHandle, NormalizedDelta * TimeRange);
	}

private:
	FRichCurve* EditCurve = nullptr;
	FVector2f HandleSize;
	TAttribute<FKeyHandle> SelectedKeyHandle;
	TAttribute<int32> CurveStateSerialNumber;
	FOnKeySelected OnKeySelectedDelegate;
	FOnKeyMoved OnKeyMovedDelegate;

	FGeometry AllottedGeometryCache;
	float LocalWidthCache;

	TSharedPtr<SOverlay> BorderOverlay;
	int32 CachedCurveStateSerialNumber = INDEX_NONE;
	TArray<FKeyWidgetData> KeyWidgetData;

	bool bNeedsWidgetUpdate = false;
};

void SNiagaraDistributionCompactCurveEditor::Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter, int32 InChannelIndex)
{
	DistributionAdapter = InDistributionAdapter;
	ChannelIndex = InChannelIndex;
	const FRichCurve* EditCurvePtr = DistributionAdapter->GetCurveValue(ChannelIndex);
	if (EditCurvePtr != nullptr)
	{
		EditCurve = *EditCurvePtr;
	}

	if (EditCurve.GetNumKeys() == 0)
	{
		EditCurve.AddKey(0, 0);
	}

	UpdateSelectedKeyIndex(0);

	float KeyHandleWidth = 16;
	float KeyHandleHeight = 16;
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 3)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 3, 0)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 0, 0, 2)
				[
					SNew(SNumericEntryBox<int32>)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.Value(this, &SNiagaraDistributionCompactCurveEditor::GetSelectedKeyIndex)
					.OnValueChanged(this, &SNiagaraDistributionCompactCurveEditor::SelectedKeyIndexChanged)
					.OnValueCommitted(this, &SNiagaraDistributionCompactCurveEditor::SelectedKeyIndexComitted)
					.AllowSpin(true)
					.MinValue(0)
					.MaxValue(this, &SNiagaraDistributionCompactCurveEditor::GetMaxKeyIndex)
					.MinSliderValue(0)
					.MaxSliderValue(this, &SNiagaraDistributionCompactCurveEditor::GetMaxKeyIndex)
					.Delta(1)
					.MinDesiredValueWidth(8)
					.LabelVAlign(VAlign_Center)
					.Label()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
						.Text(LOCTEXT("KeyLabel", "Key"))
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0, 0, 2, 0)
					[
						SNew(SButton)
						.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
						.OnClicked(this, &SNiagaraDistributionCompactCurveEditor::AddKeyButtonClicked)
						.ToolTipText(LOCTEXT("AddKeyToolTip", "Add a key"))
						.ContentPadding(0.0f)
						[ 
							SNew( SImage )
							.Image( FAppStyle::GetBrush("Icons.PlusCircle") )
							.ColorAndOpacity( FSlateColor::UseForeground() )
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0, 0, 2, 0)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnClicked(this, &SNiagaraDistributionCompactCurveEditor::DeleteKeyButtonClicked)
						.ToolTipText(LOCTEXT("DeleteKeyToolTip", "Delete this key"))
						.ContentPadding(0.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Delete"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 0, 2, 0)
						[
							SNew(SButton)
								.ButtonStyle(FAppStyle::Get(), "SimpleButton")
								.OnClicked(this, &SNiagaraDistributionCompactCurveEditor::FlipCurve, ECurveFlipDirection::Horizontal)
								.ToolTipText(LOCTEXT("FlipHorizontalToolTip", "Flip curve horizontally"))
								.ContentPadding(0.0f)
								[
									SNew(SImage)
										.Image(FAppStyle::GetBrush("Icons.FlipHorizontal"))
										.ColorAndOpacity(FSlateColor::UseForeground())
								]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 0, 2, 0)
						[
							SNew(SButton)
								.ButtonStyle(FAppStyle::Get(), "SimpleButton")
								.OnClicked(this, &SNiagaraDistributionCompactCurveEditor::FlipCurve, ECurveFlipDirection::Vertical)
								.ToolTipText(LOCTEXT("FlipVerticalToolTip", "Flip curve vertically"))
								.ContentPadding(0.0f)
								[
									SNew(SImage)
										.Image(FAppStyle::GetBrush("Icons.FlipVertical"))
										.ColorAndOpacity(FSlateColor::UseForeground())
								]
						]
				]
			]
			// Curve view and key selector.
			+ SHorizontalBox::Slot()
			.Padding(2)
			[
				SNew(SGridPanel)
				.FillRow(0, 1)
				.FillColumn(0, 1)
				// Value Labels
				+ SGridPanel::Slot(1, 0)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.Text(this, &SNiagaraDistributionCompactCurveEditor::GetCurveValueMaxText)
					]
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Bottom)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.Text(this, &SNiagaraDistributionCompactCurveEditor::GetCurveValueMinText)
					]
				]
				// Curve view
				+ SGridPanel::Slot(0, 0)
				.Padding(FMargin(KeyHandleWidth / 2, 3, KeyHandleWidth / 2, 3))
				[
					SNew(SNiagaraCompactCurveView, &EditCurve)
					.Width(1000)
					.Height(50)
					.CurveColor(InArgs._CurveColor)
					.SelectedKeyHandle(this, &SNiagaraDistributionCompactCurveEditor::GetSelectedKeyHandle)
					.CurveStateSerialNumber(this, &SNiagaraDistributionCompactCurveEditor::GetCurveStateSerialNumber)
				]
				// Time Labels
				+ SGridPanel::Slot(0, 2)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.Text(this, &SNiagaraDistributionCompactCurveEditor::GetCurveTimeMinText)
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.Text(this, &SNiagaraDistributionCompactCurveEditor::GetCurveTimeMaxText)
					]
				]
				// key selector
				+ SGridPanel::Slot(0, 1)
				.Padding(0, 0, 0, 3)
				[
					SAssignNew(KeySelectorWidget, SNiagaraCompactCurveKeySelector, &EditCurve)
					.HandleSize(FVector2f(KeyHandleWidth, KeyHandleHeight))
					.SelectedKeyHandle(this, &SNiagaraDistributionCompactCurveEditor::GetSelectedKeyHandle)
					.CurveStateSerialNumber(this, &SNiagaraDistributionCompactCurveEditor::GetCurveStateSerialNumber)
					.OnKeySelected(this, &SNiagaraDistributionCompactCurveEditor::OnKeySelectorKeySelected)
					.OnKeyMoved(this, &SNiagaraDistributionCompactCurveEditor::OnKeySelectorKeyMoved)
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 0, 5, 0)
			[
				SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.Value(this, &SNiagaraDistributionCompactCurveEditor::GetSelectedKeyTime)
				.OnValueChanged(this, &SNiagaraDistributionCompactCurveEditor::SelectedKeyTimeChanged)
				.OnValueCommitted(this, &SNiagaraDistributionCompactCurveEditor::SelectedKeyTimeComitted)
				.OnBeginSliderMovement(this, &SNiagaraDistributionCompactCurveEditor::BeginSliderMovement)
				.OnEndSliderMovement(this, &SNiagaraDistributionCompactCurveEditor::EndSliderMovement)
				.AllowSpin(true)
				.MinValue(TOptional<float>())
				.MaxValue(TOptional<float>())
				.MinSliderValue(TOptional<float>())
				.MaxSliderValue(TOptional<float>())
				.LabelVAlign(VAlign_Center)
				.Label()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.Text(LOCTEXT("TimeLabel", "Time"))
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(0, 0, 5, 0)
			[
				SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.Value(this, &SNiagaraDistributionCompactCurveEditor::GetSelectedKeyValue)
				.OnValueChanged(this, &SNiagaraDistributionCompactCurveEditor::SelectedKeyValueChanged)
				.OnValueCommitted(this, &SNiagaraDistributionCompactCurveEditor::SelectedKeyValueComitted)
				.OnBeginSliderMovement(this, &SNiagaraDistributionCompactCurveEditor::BeginSliderMovement)
				.OnEndSliderMovement(this, &SNiagaraDistributionCompactCurveEditor::EndSliderMovement)
				.AllowSpin(true)
				.MinValue(TOptional<float>())
				.MaxValue(TOptional<float>())
				.MinSliderValue(TOptional<float>())
				.MaxSliderValue(TOptional<float>())
				.LabelVAlign(VAlign_Center)
				.Label()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.Text(LOCTEXT("ValueLabel", "Value"))
				]
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0, 0, 3, 0)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.Text(LOCTEXT("InterpLabel", "Interp"))
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SEnumComboBox, StaticEnum<ERichCurveInterpMode>())
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.ContentPadding(FMargin(2, 0))
					.CurrentValue(this, &SNiagaraDistributionCompactCurveEditor::GetSelectedKeyInterpMode)
					.OnEnumSelectionChanged(this, &SNiagaraDistributionCompactCurveEditor::SelectedKeyInterpModeChanged)
				]
			]
		]
	];
}

TOptional<int32> SNiagaraDistributionCompactCurveEditor::GetSelectedKeyIndex() const
{
	return SelectedKeyIndex;
}

TOptional<int32> SNiagaraDistributionCompactCurveEditor::GetMaxKeyIndex() const
{
	return EditCurve.GetNumKeys() - 1;
}

void SNiagaraDistributionCompactCurveEditor::SelectedKeyIndexChanged(int32 InValue)
{
	UpdateSelectedKeyIndex(InValue);
}

void SNiagaraDistributionCompactCurveEditor::SelectedKeyIndexComitted(int32 InValue, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		UpdateSelectedKeyIndex(InValue);
	}
}

void SNiagaraDistributionCompactCurveEditor::UpdateSelectedKeyIndex(int32 NewKeyIndex)
{
	SelectedKeyIndex = NewKeyIndex;
	int32 KeyIndex = 0;
	for (auto KeyHandleIterator = EditCurve.GetKeyHandleIterator(); KeyHandleIterator; KeyHandleIterator++)
	{
		if (KeyIndex == NewKeyIndex)
		{
			SelectedKeyHandle = *KeyHandleIterator;
			break;
		}
		KeyIndex++;
	}
	CurveStateSerialNumber++;
}

TOptional<float> SNiagaraDistributionCompactCurveEditor::GetSelectedKeyTime() const
{
	UpdateCachedValues();
	return KeyTimeCache;
}

void SNiagaraDistributionCompactCurveEditor::SelectedKeyTimeChanged(float InValue)
{
	UpdateSelectedKeyTime(InValue);
}

void SNiagaraDistributionCompactCurveEditor::SelectedKeyTimeComitted(float InValue, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		UpdateSelectedKeyTime(InValue);
	}
}

void SNiagaraDistributionCompactCurveEditor::UpdateSelectedKeyTime(float NewKeyTime)
{
	if (GetSelectedKeyTime() != NewKeyTime)
	{
		EditCurve.SetKeyTime(SelectedKeyHandle, NewKeyTime);
		DistributionAdapter->SetCurveValue(ChannelIndex, EditCurve);

		int32 NewKeyIndex = INDEX_NONE;
		int32 KeyIndex = 0;
		for (auto KeyHandleIterator = EditCurve.GetKeyHandleIterator(); KeyHandleIterator; KeyHandleIterator++)
		{
			if (SelectedKeyHandle == *KeyHandleIterator)
			{
				NewKeyIndex = KeyIndex;
				break;
			}
			KeyIndex++;
		}

		SelectedKeyIndex = NewKeyIndex;
		KeyTimeCache = NewKeyTime;
		CurveStateSerialNumber++;
	}
}

TOptional<float> SNiagaraDistributionCompactCurveEditor::GetSelectedKeyValue() const
{
	UpdateCachedValues();
	return KeyValueCache;
}

void SNiagaraDistributionCompactCurveEditor::SelectedKeyValueChanged(float InValue)
{
	UpdateSelectedKeyValue(InValue);
}

void SNiagaraDistributionCompactCurveEditor::SelectedKeyValueComitted(float InValue, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		UpdateSelectedKeyValue(InValue);
	}
}

void SNiagaraDistributionCompactCurveEditor::UpdateSelectedKeyValue(float NewKeyValue)
{
	if (GetSelectedKeyValue() != NewKeyValue)
	{
		EditCurve.SetKeyValue(SelectedKeyHandle, NewKeyValue);
		DistributionAdapter->SetCurveValue(ChannelIndex, EditCurve);
		KeyValueCache = NewKeyValue;
		CurveStateSerialNumber++;
	}
}

int32 SNiagaraDistributionCompactCurveEditor::GetSelectedKeyInterpMode() const
{
	UpdateCachedValues();
	return KeyInterpModeCache;
}

void SNiagaraDistributionCompactCurveEditor::SelectedKeyInterpModeChanged(int32 InNewValue, ESelectInfo::Type InInfo)
{
	if (GetSelectedKeyInterpMode() != InNewValue)
	{
		EditCurve.SetKeyInterpMode(SelectedKeyHandle, (ERichCurveInterpMode)InNewValue);
		DistributionAdapter->SetCurveValue(ChannelIndex, EditCurve);
		KeyInterpModeCache = InNewValue;
		CurveStateSerialNumber++;
	}
}

void SNiagaraDistributionCompactCurveEditor::BeginSliderMovement()
{
	DistributionAdapter->BeginContinuousChange();
}

void SNiagaraDistributionCompactCurveEditor::EndSliderMovement(float Value)
{
	DistributionAdapter->EndContinuousChange();
}

FReply SNiagaraDistributionCompactCurveEditor::AddKeyButtonClicked()
{
	int32 NumKeys = EditCurve.GetNumKeys();
	int32 NewSelectedKeyIndex;
	if (NumKeys == 0)
	{
		EditCurve.AddKey(0, 0);
		NewSelectedKeyIndex = 0;
	}
	else if (NumKeys == 1)
	{
		FRichCurveKey FirstKey = EditCurve.GetFirstKey();
		EditCurve.AddKey(FirstKey.Time + 1, FirstKey.Value);
		NewSelectedKeyIndex = 1;
	}
	else
	{
		if (SelectedKeyIndex < NumKeys - 1)
		{
			FKeyHandle NextKeyHandle = EditCurve.GetNextKey(SelectedKeyHandle);
			FRichCurveKey SelectedKey = EditCurve.GetKey(SelectedKeyHandle);
			FRichCurveKey NextKey = EditCurve.GetKey(NextKeyHandle);
			EditCurve.AddKey((SelectedKey.Time + NextKey.Time) / 2, (SelectedKey.Value + NextKey.Value) / 2);
			NewSelectedKeyIndex = SelectedKeyIndex + 1;
		}
		else
		{
			FKeyHandle PreviousKeyHandle = EditCurve.GetPreviousKey(SelectedKeyHandle);
			FRichCurveKey SelectedKey = EditCurve.GetKey(SelectedKeyHandle);
			FRichCurveKey PreviousKey = EditCurve.GetKey(PreviousKeyHandle);
			EditCurve.AddKey(SelectedKey.Time + (SelectedKey.Time - PreviousKey.Time), SelectedKey.Value + (SelectedKey.Value - PreviousKey.Value));
			NewSelectedKeyIndex = SelectedKeyIndex;
		}
	}

	DistributionAdapter->SetCurveValue(ChannelIndex, EditCurve);
	UpdateSelectedKeyIndex(NewSelectedKeyIndex);
	CurveStateSerialNumber++;
	return FReply::Handled();
}

FReply SNiagaraDistributionCompactCurveEditor::DeleteKeyButtonClicked()
{
	if (EditCurve.GetNumKeys() > 1)
	{
		EditCurve.DeleteKey(GetSelectedKeyHandle());
		DistributionAdapter->SetCurveValue(ChannelIndex, EditCurve);
		int32 NewKeyIndex = FMath::Clamp(SelectedKeyIndex - 1, 0, EditCurve.GetNumKeys() - 1);
		UpdateSelectedKeyIndex(NewKeyIndex);
		CurveStateSerialNumber++;
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SNiagaraDistributionCompactCurveEditor::FlipCurve(ECurveFlipDirection Direction)
{
	if (EditCurve.GetNumKeys() <= 0)
	{
		return FReply::Unhandled();
	}

	int32 NumKeys = EditCurve.GetNumKeys();
	int32 NewSelectedKeyIndex = SelectedKeyIndex;

	// Intialize the range
	float RangeMin = FLT_MAX;
	float RangeMax = -FLT_MAX;

	// If flipping horizontally
	if (Direction == ECurveFlipDirection::Horizontal)
	{
		// Get the range
		float MinTime, MaxTime;
		EditCurve.GetTimeRange(MinTime, MaxTime);
		RangeMin = FMath::Min(MinTime, RangeMin);
		RangeMax = FMath::Max(MaxTime, RangeMax);

		TArray<TPair<float, float>> TimeValuePairs;
		TArray<ERichCurveInterpMode> InterpModes;
		TimeValuePairs.SetNum(NumKeys);
		InterpModes.SetNum(NumKeys);

		// Get the (flipped x, original y) value of all points
		FKeyHandle KeyHandle = EditCurve.GetFirstKeyHandle();
		for (int32 Index = 0; Index <= NumKeys - 1; ++Index)
		{
			FRichCurveKey Key = EditCurve.GetKey(KeyHandle);
			TimeValuePairs[Index].Key = RangeMax - Key.Time + RangeMin;
			TimeValuePairs[Index].Value = Key.Value;
			InterpModes[Index] = EditCurve.GetKeyInterpMode(KeyHandle);
			KeyHandle = EditCurve.GetNextKey(KeyHandle);
		}

		// Delete all existing keys
		EditCurve.Reset();

		// Add the flipped points back in
		FKeyHandle NewKeyHandle;
		for (int32 Index = 0; Index <= NumKeys - 1; ++Index)
		{
			NewKeyHandle = EditCurve.AddKey(TimeValuePairs[Index].Key, TimeValuePairs[Index].Value);
			EditCurve.SetKeyInterpMode(NewKeyHandle, InterpModes[Index]);
			//NOTE: key tangents are not handled right now since distribution curve editor currently don't support tangent editing from user
		}

		NewSelectedKeyIndex = NumKeys - 1 - SelectedKeyIndex;
		if (KeySelectorWidget) 
		{
			KeySelectorWidget->RequestWidgetUpdate();
		}
	}

	// If flipping vertically
	if (Direction == ECurveFlipDirection::Vertical)
	{
		// Get the range
		float MinVal, MaxVal;
		EditCurve.GetValueRange(MinVal, MaxVal);
		RangeMin = FMath::Min(MinVal, RangeMin);
		RangeMax = FMath::Max(MaxVal, RangeMax);

		// Flip the y value of all keys
		FKeyHandle KeyHandle = EditCurve.GetFirstKeyHandle();
		for (int32 Index = 0; Index <= NumKeys - 1; ++Index)
		{
			EditCurve.SetKeyValue(KeyHandle, RangeMax - EditCurve.GetKey(KeyHandle).Value + RangeMin);
			KeyHandle = EditCurve.GetNextKey(KeyHandle);
		}
	}

	DistributionAdapter->SetCurveValue(ChannelIndex, EditCurve);
	UpdateSelectedKeyIndex(NewSelectedKeyIndex);
	CurveStateSerialNumber++;
	return FReply::Handled();
}

void SNiagaraDistributionCompactCurveEditor::OnKeySelectorKeySelected(FKeyHandle InSelectedKeyHandle)
{
	int32 KeyIndex = 0;
	for (auto KeyHandleIterator = EditCurve.GetKeyHandleIterator(); KeyHandleIterator; KeyHandleIterator++)
	{
		if (*KeyHandleIterator == InSelectedKeyHandle)
		{
			SelectedKeyHandle = *KeyHandleIterator;
			SelectedKeyIndex = KeyIndex;
			CurveStateSerialNumber++;
			break;
		}
		KeyIndex++;
	}
}

void SNiagaraDistributionCompactCurveEditor::OnKeySelectorKeyMoved(FKeyHandle InSelectedKeyHandle, float TimeDelta)
{
	if (SelectedKeyHandle == InSelectedKeyHandle)
	{
		UpdateSelectedKeyTime(GetSelectedKeyTime().GetValue() + TimeDelta);
	}
}

FText SNiagaraDistributionCompactCurveEditor::GetCurveTimeMinText() const
{
	UpdateCachedValues();
	return CurveTimeMinTextCache;
}

FText SNiagaraDistributionCompactCurveEditor::GetCurveTimeMaxText() const
{
	UpdateCachedValues();
	return CurveTimeMaxTextCache;
}

FText SNiagaraDistributionCompactCurveEditor::GetCurveValueMinText() const
{
	UpdateCachedValues();
	return CurveValueMinTextCache;
}

FText SNiagaraDistributionCompactCurveEditor::GetCurveValueMaxText() const
{
	UpdateCachedValues();
	return CurveValueMaxTextCache;
}

void SNiagaraDistributionCompactCurveEditor::UpdateCachedValues() const
{
	if (CacheCurveStateSerialNumber != CurveStateSerialNumber)
	{
		CacheCurveStateSerialNumber = CurveStateSerialNumber;
		KeyTimeCache = EditCurve.GetKeyTime(SelectedKeyHandle);
		KeyValueCache = EditCurve.GetKeyValue(SelectedKeyHandle);
		KeyInterpModeCache = (int32)EditCurve.GetKeyInterpMode(SelectedKeyHandle);

		float MinTime;
		float MaxTime;
		float MinValue;
		float MaxValue;
		EditCurve.GetTimeRange(MinTime, MaxTime);
		EditCurve.GetValueRange(MinValue, MaxValue);
		CurveTimeMinTextCache = FText::AsNumber(MinTime);
		CurveTimeMaxTextCache = FText::AsNumber(MaxTime);
		CurveValueMinTextCache = FText::AsNumber(MinValue);
		CurveValueMaxTextCache = FText::AsNumber(MaxValue);
	}
}

#undef LOCTEXT_NAMESPACE