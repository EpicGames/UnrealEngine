// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "Styling/StyleColors.h"
#include "Widgets/SCompoundWidget.h"

class INiagaraDistributionAdapter;
class SNiagaraCompactCurveKeySelector;

class SNiagaraDistributionCompactCurveEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDistributionCompactCurveEditor)
		: _CurveColor(FSlateColor(EStyleColor::White))
		{ }
		SLATE_ARGUMENT(FSlateColor, CurveColor)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter, int32 InChannelIndex);

private:
	/** Curve flip direction types **/
	enum class ECurveFlipDirection : uint8
	{
		Horizontal,
		Vertical
	};

private:
	TOptional<int32> GetSelectedKeyIndex() const;
	TOptional<int32> GetMaxKeyIndex() const;
	void SelectedKeyIndexChanged(int32 InValue);
	void SelectedKeyIndexComitted(int32 InValue, ETextCommit::Type CommitInfo);
	void UpdateSelectedKeyIndex(int32 NewKeyIndex);

	TOptional<float> GetSelectedKeyTime() const;
	void SelectedKeyTimeChanged(float InValue);
	void SelectedKeyTimeComitted(float InValue, ETextCommit::Type CommitInfo);
	void UpdateSelectedKeyTime(float NewKeyTime);

	TOptional<float> GetSelectedKeyValue() const;
	void SelectedKeyValueChanged(float InValue);
	void SelectedKeyValueComitted(float InValue, ETextCommit::Type CommitInfo);
	void UpdateSelectedKeyValue(float NewKeyValue);

	int32 GetSelectedKeyInterpMode() const;
	void SelectedKeyInterpModeChanged(int32 InNewValue, ESelectInfo::Type InInfo);

	void BeginSliderMovement();
	void EndSliderMovement(float Value);

	FReply AddKeyButtonClicked();
	FReply DeleteKeyButtonClicked();
	FReply FlipCurve(ECurveFlipDirection Direction);

	FKeyHandle GetSelectedKeyHandle() const { return SelectedKeyHandle; }
	int32 GetCurveStateSerialNumber() const { return CurveStateSerialNumber; }

	void OnKeySelectorKeySelected(FKeyHandle InSelectedKeyHandle);
	void OnKeySelectorKeyMoved(FKeyHandle InSelectedKeyHandle, float TimeDelta);

	FText GetCurveTimeMinText() const;
	FText GetCurveTimeMaxText() const;
	FText GetCurveValueMinText() const;
	FText GetCurveValueMaxText() const;

	void UpdateCachedValues() const;

private:
	TSharedPtr<SNiagaraCompactCurveKeySelector> KeySelectorWidget;
	TSharedPtr<INiagaraDistributionAdapter> DistributionAdapter;
	int32 ChannelIndex;

	FRichCurve EditCurve;
	FKeyHandle SelectedKeyHandle;

	int32 SelectedKeyIndex = INDEX_NONE;
	int32 CurveStateSerialNumber = 0;

	mutable int32 CacheCurveStateSerialNumber = INDEX_NONE;
	mutable TOptional<float> KeyTimeCache;
	mutable TOptional<float> KeyValueCache;
	mutable int32 KeyInterpModeCache;
	mutable FText CurveTimeMinTextCache;
	mutable FText CurveTimeMaxTextCache;
	mutable FText CurveValueMinTextCache;
	mutable FText CurveValueMaxTextCache;
};