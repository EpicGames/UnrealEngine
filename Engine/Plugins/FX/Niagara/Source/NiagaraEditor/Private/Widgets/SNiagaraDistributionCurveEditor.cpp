// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraDistributionCurveEditor.h"

#include "INiagaraDistributionAdapter.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorUtilities.h"

#include "Engine/Engine.h"
#include "Curves/CurveFloat.h"
#include "Curves/RichCurve.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/AxisDisplayInfo.h"
#include "Tree/SCurveEditorTree.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "Tree/SCurveEditorTreePin.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SNiagaraExpandedToggle.h"
#include "CurveEditor.h"
#include "CurveEditorCommands.h"
#include "ICurveEditorBounds.h"
#include "SCurveEditorPanel.h"
#include "SCurveKeyDetailPanel.h"
#include "EditorFontGlyphs.h"
#include "PropertyHandle.h"
#include "RichCurveEditorModel.h"
#include "SResizeBox.h"

#define LOCTEXT_NAMESPACE "NiagaraCurveEditor"

namespace
{
	
	class FNiagaraDistributionCurveEditorModel : public FRichCurveEditorModel
	{
	public:
		FNiagaraDistributionCurveEditorModel(TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter, int32 InChannelIndex)
			: FRichCurveEditorModel(GetOwnerFromDistribution(InDistributionAdapter.Get()))
			, DistributionAdapter(InDistributionAdapter)
			, ChannelIndex(InChannelIndex)
		{
		}

		virtual bool IsValid() const override { return DistributionAdapter.IsValid() && DistributionAdapter->GetCurveValue(ChannelIndex) != nullptr; }
		virtual FRichCurve& GetRichCurve() { return *DistributionAdapter->GetCurveValue(ChannelIndex); }
		virtual const FRichCurve& GetReadOnlyRichCurve() const override { return *DistributionAdapter->GetCurveValue(ChannelIndex); }

	private:
		static UObject* GetOwnerFromDistribution(INiagaraDistributionAdapter& InDistribution)
		{
			TArray<UObject*> Owners;
			InDistribution.GetOwners(Owners);
			return Owners.Num() == 1 ? Owners[0] : nullptr;
		}

	private:
		TSharedPtr<INiagaraDistributionAdapter>	DistributionAdapter;
		int32 ChannelIndex = 0;
	};

	struct FNiagaraDistributionCurveEditorBounds : public ICurveEditorBounds
	{
		FNiagaraDistributionCurveEditorBounds(TSharedRef<FNiagaraDistributionCurveEditorOptions> InCurveEditorOptions)
			: CurveEditorOptions(InCurveEditorOptions)
		{
		}
	
		virtual void GetInputBounds(double& OutMin, double& OutMax) const
		{
			OutMin = CurveEditorOptions->GetViewMinInput();
			OutMax = CurveEditorOptions->GetViewMaxInput();
		}
	
		virtual void SetInputBounds(double InMin, double InMax)
		{
			CurveEditorOptions->SetInputViewRange((float)InMin, (float)InMax);
		}
	
	private:
		TSharedRef<FNiagaraDistributionCurveEditorOptions> CurveEditorOptions;
	};

	struct FNiagaraDistributionCurveEditorDetailsTreeItem : public ICurveEditorTreeItem, TSharedFromThis<FNiagaraDistributionCurveEditorDetailsTreeItem>
	{
		void Initialize(TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter, int InChannelIndex)
		{
			DistributionAdapter = InDistributionAdapter;
			ChannelIndex = InChannelIndex;

			CurveName		= DistributionAdapter->GetChannelDisplayName(ChannelIndex);
			CurveToolTip	= DistributionAdapter->GetChannelToolTip(ChannelIndex);
			CurveColor		= DistributionAdapter->GetChannelColor(ChannelIndex).GetSpecifiedColor();
		}

		virtual TSharedPtr<SWidget> GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& TableRow) override
		{
			if (InColumnName == ColumnNames.PinHeader)
			{
				return SNew(SCurveEditorTreePin, InCurveEditor, InTreeItemID, TableRow);
			}

			if (InColumnName == ColumnNames.Label)
			{
				TSharedRef<SHorizontalBox> LabelBox =
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0, 0, 5, 0)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
						.Text(FEditorFontGlyphs::Circle)
						.ColorAndOpacity(FSlateColor(CurveColor))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.VAlign(VAlign_Center)
						.MinDesiredHeight(22)
						[
							SNew(STextBlock)
							.Text(CurveName)
						]
					];

				if (!CurveToolTip.IsEmpty())
				{
					LabelBox->SetToolTipText(CurveToolTip);
				}

				return LabelBox;
			}

			return nullptr;
		}

		virtual void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override
		{
			TArray<UObject*> OwnerObjects;
			DistributionAdapter->GetOwners(OwnerObjects);
			if (OwnerObjects.Num() != 1)
			{
				return;
			}

			TUniquePtr<FNiagaraDistributionCurveEditorModel> NewCurve = MakeUnique<FNiagaraDistributionCurveEditorModel>(DistributionAdapter.ToSharedRef(), ChannelIndex);
			NewCurve->SetShortDisplayName(CurveName);
			NewCurve->SetColor(CurveColor);
			NewCurve->OnCurveModified().AddSP(this, &FNiagaraDistributionCurveEditorDetailsTreeItem::OnCurveChanged);
			OutCurveModels.Add(MoveTemp(NewCurve));
		}

		virtual bool PassesFilter(const FCurveEditorTreeFilter* InFilter) const override { return true; }

	private:
		void OnCurveChanged()
		{
			DistributionAdapter->NotifyCurveValueChangedExternally(ChannelIndex);
		}

	private:
		TSharedPtr<INiagaraDistributionAdapter>	DistributionAdapter;
		int										ChannelIndex = 0;
		FText									CurveName;
		FText									CurveToolTip;
		FLinearColor							CurveColor;
	};

	class SNiagaraDistributionCurveKeySelector : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SNiagaraDistributionCurveKeySelector) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<FCurveEditor> InCurveEditor, const TArray<FCurveEditorTreeItemID>& InCurveTreeItemIds, TSharedPtr<SCurveEditorTree> InCurveEditorTree)
		{
			CurveEditor = InCurveEditor;
			OrderedCurveTreeItemIds = InCurveTreeItemIds;
			CurveEditorTree = InCurveEditorTree;
			ChildSlot
			[
				SNew(SHorizontalBox)

				// Zoom to fit button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 4, 0)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &SNiagaraDistributionCurveKeySelector::ZoomToFitClicked)
					.ToolTipText(LOCTEXT("ZoomToFitToolTip", "Zoom to fit all keys"))
					.Content()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Text(FEditorFontGlyphs::Expand)
					]
				]

				// Previous curve button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 1, 0)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &SNiagaraDistributionCurveKeySelector::PreviousCurveClicked)
					.ToolTipText(LOCTEXT("PreviousCurveToolTip", "Select the previous curve"))
					.Visibility(this, &SNiagaraDistributionCurveKeySelector::GetNextPreviousCurveButtonVisibility)
					.Content()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowUp"))
					]
				]

				// Next curve button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 4, 0)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &SNiagaraDistributionCurveKeySelector::NextCurveClicked)
					.ToolTipText(LOCTEXT("NextCurveToolTip", "Select the next curve"))
					.Visibility(this, &SNiagaraDistributionCurveKeySelector::GetNextPreviousCurveButtonVisibility)
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowDown"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]

				// Previous key button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 1, 0)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &SNiagaraDistributionCurveKeySelector::PreviousKeyClicked)
					.ToolTipText(LOCTEXT("PreviousKeyToolTip", "Select the previous key for the selected curve."))
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowLeft"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]

				// Next key button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 4, 0)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &SNiagaraDistributionCurveKeySelector::NextKeyClicked)
					.ToolTipText(LOCTEXT("NextKeyToolTip", "Select the next key for the selected curve."))
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.ArrowRight"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]

				// Add key button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 1, 0)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &SNiagaraDistributionCurveKeySelector::AddKeyClicked)
					.ToolTipText(LOCTEXT("AddKeyToolTip", "Add a key to the selected curve."))
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]

				// Delete key button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &SNiagaraDistributionCurveKeySelector::DeleteKeyClicked)
					.ToolTipText(LOCTEXT("DeleteKeyToolTip", "Delete the currently selected keys."))
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];
		}

	private:
		void GetActiveCurveModelAndSelectedKeys(TOptional<FCurveModelID>& OutActiveCurveModelId, TArray<FKeyHandle>& OutSelectedKeyHandles)
		{
			const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& CurveTreeSelection = CurveEditor->GetTree()->GetSelection();
			if (CurveTreeSelection.Num() != 0)
			{
				// If there are curves selected in the tree use those first.
				FCurveEditorTreeItemID FirstSelectedCurveTreeItemId;
				for (int32 i = OrderedCurveTreeItemIds.Num() - 1; i >= 0; i--)
				{
					const FCurveEditorTreeItemID& CurveTreeItemId = OrderedCurveTreeItemIds[i];
					if (CurveTreeSelection.Contains(CurveTreeItemId))
					{
						FirstSelectedCurveTreeItemId = CurveTreeItemId;
						break;
					}
				}

				TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(FirstSelectedCurveTreeItemId).GetOrCreateCurves(CurveEditor.Get());
				if (CurveModelIds.Num() == 1)
				{
					OutActiveCurveModelId = CurveModelIds[0];

					const FKeyHandleSet* SelectedKeyHandleSet = CurveEditor->GetSelection().GetAll().Find(OutActiveCurveModelId.GetValue());
					if (SelectedKeyHandleSet != nullptr)
					{
						OutSelectedKeyHandles = SelectedKeyHandleSet->AsArray();
					}
				}
			}
			else
			{
				// Otherwise check if there are keys selected and if so use first curve with selected keys.
				FCurveEditorSelection& CurveEditorSelection = CurveEditor->GetSelection();
				if (CurveEditorSelection.IsEmpty() == false)
				{
					for (int32 i = OrderedCurveTreeItemIds.Num() - 1; i >= 0; i--)
					{
						const FCurveEditorTreeItemID& CurveTreeItemId = OrderedCurveTreeItemIds[i];
						TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(CurveTreeItemId).GetCurves();
						if (CurveModelIds.Num() == 1)
						{
							const FKeyHandleSet* SelectedKeyHandleSet = CurveEditorSelection.GetAll().Find(CurveModelIds[0]);
							if (SelectedKeyHandleSet != nullptr)
							{
								OutActiveCurveModelId = CurveModelIds[0];
								OutSelectedKeyHandles = SelectedKeyHandleSet->AsArray();
								break;
							}
						}
					}
				}
				else
				{
					// Otherwise just use the first pinned curve.
					const TSet<FCurveModelID>& PinnedCurveIds = CurveEditor->GetPinnedCurves();
					if (PinnedCurveIds.Num() > 0)
					{
						for (FCurveEditorTreeItemID& CurveTreeItemId : OrderedCurveTreeItemIds)
						{
							TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(CurveTreeItemId).GetOrCreateCurves(CurveEditor.Get());
							if (CurveModelIds.Num() == 1 && PinnedCurveIds.Contains(CurveModelIds[0]))
							{
								OutActiveCurveModelId = CurveModelIds[0];
								break;
							}
						}
					}
				}
			}
		}

		struct FKeyHandlePositionPair
		{
			FKeyHandle Handle;
			FKeyPosition Position;
		};

		void GetSortedKeyHandlessAndPositionsForModel(FCurveModel& InCurveModel, TArray<FKeyHandlePositionPair>& OutSortedKeyHandlesAndPositions)
		{
			const TArray<FKeyHandle> KeyHandles = InCurveModel.GetAllKeys();

			TArray<FKeyPosition> KeyPositions;
			KeyPositions.AddDefaulted(KeyHandles.Num());
			InCurveModel.GetKeyPositions(KeyHandles, KeyPositions);

			for (int32 i = 0; i < KeyHandles.Num(); i++)
			{
				FKeyHandlePositionPair& KeyHandlePositionPair = OutSortedKeyHandlesAndPositions.AddDefaulted_GetRef();
				KeyHandlePositionPair.Handle = KeyHandles[i];
				KeyHandlePositionPair.Position = KeyPositions[i];
			}

			OutSortedKeyHandlesAndPositions.Sort([](const FKeyHandlePositionPair& A, const FKeyHandlePositionPair& B) { return A.Position.InputValue < B.Position.InputValue; });
		}

		void GetOrderedActiveCurveModelIds(TArray<FCurveModelID>& OutOrderedActiveCurveModelIds)
		{
			if (CurveEditor->GetTreeSelection().Num() > 0)
			{
				// If there are curves selected in the tree then those are the only active ones.
				for (const FCurveEditorTreeItemID& OrderedCurveTreeItemId : OrderedCurveTreeItemIds)
				{
					if(CurveEditor->GetTreeSelectionState(OrderedCurveTreeItemId) != ECurveEditorTreeSelectionState::None)
					{
						TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(OrderedCurveTreeItemId).GetOrCreateCurves(CurveEditor.Get());
						if (CurveModelIds.Num() == 1)
						{
							OutOrderedActiveCurveModelIds.Add(CurveModelIds[0]);
						}
					}
				}
			}
			else
			{
				// Otherwise the active curves are the pinned curves.
				for (const FCurveEditorTreeItemID& OrderedCurveTreeItemId : OrderedCurveTreeItemIds)
				{
					TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(OrderedCurveTreeItemId).GetCurves();
					if (CurveModelIds.Num() == 1 && CurveEditor->IsCurvePinned(CurveModelIds[0]))
					{
						OutOrderedActiveCurveModelIds.Add(CurveModelIds[0]);
					}
				}
			}
		}

		enum class ENavigateDirection
		{
			Previous,
			Next
		};

		void NavigateToAdjacentCurve(ENavigateDirection Direction)
		{
			if (CurveEditorTree.IsValid())
			{
				if (CurveEditor->GetTreeSelection().Num() == 0)
				{
					CurveEditorTree->SetItemSelection(OrderedCurveTreeItemIds[0], true);
				}
				else
				{
					int32 TargetSelectedTreeItemIndex = INDEX_NONE;
					int32 StartIndex = Direction == ENavigateDirection::Previous ? 0 : OrderedCurveTreeItemIds.Num() - 1;
					int32 EndIndex = Direction == ENavigateDirection::Previous ? OrderedCurveTreeItemIds.Num() : -1;
					int32 IndexOffset = Direction == ENavigateDirection::Previous ? 1 : -1;
					for (int32 i = StartIndex; i != EndIndex; i += IndexOffset)
					{
						if (CurveEditor->GetTreeSelectionState(OrderedCurveTreeItemIds[i]) != ECurveEditorTreeSelectionState::None)
						{
							TargetSelectedTreeItemIndex = i;
							break;
						}
					}

					if (TargetSelectedTreeItemIndex != INDEX_NONE)
					{
						int32 IndexToSelect = Direction == ENavigateDirection::Previous 
							? TargetSelectedTreeItemIndex - 1
							: TargetSelectedTreeItemIndex + 1;

						if (IndexToSelect > OrderedCurveTreeItemIds.Num() - 1)
						{
							IndexToSelect = 0;
						}
						else if(IndexToSelect < 0)
						{
							IndexToSelect = OrderedCurveTreeItemIds.Num() - 1;
						}

						CurveEditorTree->ClearSelection();
						CurveEditorTree->SetItemSelection(OrderedCurveTreeItemIds[IndexToSelect], true);
					}
				}
			}
		}

		void NavigateToAdjacentKey(ENavigateDirection Direction)
		{
			TOptional<FCurveModelID> ActiveCurveModelId;
			TArray<FKeyHandle> SelectedKeyHandles;
			GetActiveCurveModelAndSelectedKeys(ActiveCurveModelId, SelectedKeyHandles);

			TOptional<FCurveModelID> CurveModelIdToSelect;
			TOptional<FKeyHandle> KeyHandleToSelect;
			if (ActiveCurveModelId.IsSet())
			{
				FCurveModel* ActiveCurveModel = CurveEditor->GetCurves()[ActiveCurveModelId.GetValue()].Get();
				TArray<FKeyHandlePositionPair> ActiveSortedKeyHandlePositionPairs;
				GetSortedKeyHandlessAndPositionsForModel(*ActiveCurveModel, ActiveSortedKeyHandlePositionPairs);

				if (SelectedKeyHandles.Num() != 0)
				{
					// If there's currently a selected key on the active curve then we want to use that as the target for navigating.
					int32 TargetSelectedKeyIndex = INDEX_NONE;
					int32 StartIndex = Direction == ENavigateDirection::Previous ? 0 : ActiveSortedKeyHandlePositionPairs.Num() - 1;
					int32 EndIndex = Direction == ENavigateDirection::Previous ? ActiveSortedKeyHandlePositionPairs.Num() : -1;
					int32 IndexOffset = Direction == ENavigateDirection::Previous ? 1 : -1;
					for (int32 i = StartIndex; i != EndIndex; i += IndexOffset)
					{
						if (SelectedKeyHandles.Contains(ActiveSortedKeyHandlePositionPairs[i].Handle))
						{
							TargetSelectedKeyIndex = i;
							break;
						}
					}

					if (TargetSelectedKeyIndex != INDEX_NONE)
					{
						if(Direction == ENavigateDirection::Previous && TargetSelectedKeyIndex > 0)
						{
							// If we're navigating previous and we're not at the first key we can just select the previous key on this curve.
							CurveModelIdToSelect = ActiveCurveModelId;
							KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs[TargetSelectedKeyIndex - 1].Handle;
						}
						else if (Direction == ENavigateDirection::Next && TargetSelectedKeyIndex < ActiveSortedKeyHandlePositionPairs.Num() - 1)
						{
							// If we're navigating next and we're not at the last key we can just select the next key on this curve.
							CurveModelIdToSelect = ActiveCurveModelId;
							KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs[TargetSelectedKeyIndex + 1].Handle;
						}
						else
						{
							// Otherwise we're going to need to navigate to another curve since we're at the start or end of the current curve.
							TArray<FCurveModelID> OrderedActiveCurveModelIds;
							GetOrderedActiveCurveModelIds(OrderedActiveCurveModelIds);

							// Find the adjacent curve with keys so we can select a key there.
							FCurveModel* CurveModelToSelect = nullptr;
							int32 CurrentIndex = OrderedActiveCurveModelIds.IndexOfByKey(ActiveCurveModelId);
							int32 CurrentIndexOffset = Direction == ENavigateDirection::Previous ? -1 : 1;
							while (CurveModelIdToSelect.IsSet() == false)
							{
								CurrentIndex += CurrentIndexOffset;
								if (CurrentIndex < 0)
								{
									CurrentIndex = OrderedActiveCurveModelIds.Num() - 1;
								}
								else if (CurrentIndex > OrderedActiveCurveModelIds.Num() - 1)
								{
									CurrentIndex = 0;
								}

								FCurveModel* CurrentCurveModel = CurveEditor->GetCurves()[OrderedActiveCurveModelIds[CurrentIndex]].Get();
								if (CurrentCurveModel->GetNumKeys() > 0)
								{
									CurveModelIdToSelect = OrderedActiveCurveModelIds[CurrentIndex];
									CurveModelToSelect = CurrentCurveModel;
								}
							}

							if (CurveModelIdToSelect == ActiveCurveModelId)
							{
								// There were no other active curves with keys to select from so just wrap around the currently active curve.
								if (Direction == ENavigateDirection::Previous)
								{
									KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs.Last().Handle;
								}
								else
								{
									KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs[0].Handle;
								}
							}
							else if(CurveModelToSelect != nullptr)
							{
								// We're selecting a key on a different curve so we need to sort the positions and select the first or last based
								// on the navigation direction.
								TArray<FKeyHandlePositionPair> SortedKeyHandlePositionPairs;
								GetSortedKeyHandlessAndPositionsForModel(*CurveModelToSelect, SortedKeyHandlePositionPairs);
								if (Direction == ENavigateDirection::Previous)
								{
									KeyHandleToSelect = SortedKeyHandlePositionPairs.Last().Handle;
								}
								else
								{
									KeyHandleToSelect = SortedKeyHandlePositionPairs[0].Handle;
								}
							}
						}
					}
				}
				else if(ActiveSortedKeyHandlePositionPairs.Num() > 0)
				{
					// There weren't any keys already selected so just select the first key on the active curve.
					CurveModelIdToSelect = ActiveCurveModelId;
					KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs[0].Handle;
				}
			}

			if (CurveModelIdToSelect.IsSet() && KeyHandleToSelect.IsSet())
			{
				FCurveEditorSelection& CurveEditorSelection = CurveEditor->GetSelection();
				CurveEditorSelection.Clear();
				CurveEditorSelection.Add(CurveModelIdToSelect.GetValue(), ECurvePointType::Key, KeyHandleToSelect.GetValue());

				TArray<FCurveModelID> CurvesToFit;
				CurvesToFit.Add(CurveModelIdToSelect.GetValue());
				CurveEditor->ZoomToFitCurves(CurvesToFit);
			}
		}

		FReply ZoomToFitClicked()
		{
			TArray<FCurveModelID> ActiveCurveModelIds;
			GetOrderedActiveCurveModelIds(ActiveCurveModelIds);
			CurveEditor->ZoomToFitCurves(ActiveCurveModelIds);
			return FReply::Handled();
		}

		FReply PreviousCurveClicked()
		{
			NavigateToAdjacentCurve(ENavigateDirection::Previous);
			return FReply::Handled();
		}

		FReply NextCurveClicked()
		{
			NavigateToAdjacentCurve(ENavigateDirection::Next);
			return FReply::Handled();
		}

		FReply PreviousKeyClicked()
		{
			NavigateToAdjacentKey(ENavigateDirection::Previous);
			return FReply::Handled();
		}

		FReply NextKeyClicked()
		{
			NavigateToAdjacentKey(ENavigateDirection::Next);
			return FReply::Handled();
		}

		EVisibility GetNextPreviousCurveButtonVisibility() const
		{
			return OrderedCurveTreeItemIds.Num() == 1 ? EVisibility::Collapsed : EVisibility::Visible;
		}

		FReply AddKeyClicked()
		{
			TOptional<FCurveModelID> CurveModelIdForAdd;
			TArray<FKeyHandle> SelectedKeyHandles;
			GetActiveCurveModelAndSelectedKeys(CurveModelIdForAdd, SelectedKeyHandles);

			if(CurveModelIdForAdd.IsSet())
			{
				FCurveModel* CurveModelForAdd = CurveEditor->GetCurves()[CurveModelIdForAdd.GetValue()].Get();

				FKeyPosition NewKeyPosition;
				FKeyAttributes NewKeyAttributes = CurveEditor->GetDefaultKeyAttributes().Get();
				NewKeyAttributes.SetInterpMode(RCIM_Cubic);
				NewKeyAttributes.SetTangentMode(RCTM_Auto);
				if (CurveModelForAdd->GetNumKeys() == 0)
				{
					// If there are no keys, add one at 0, 0.
					NewKeyPosition.InputValue = 0.0f;
					NewKeyPosition.OutputValue = 0.0f;
				}
				else if (CurveModelForAdd->GetNumKeys() == 1)
				{
					// If there's a single key, add the new key at the same value, but time + 1.
					const TArray<FKeyHandle> KeyHandles = CurveModelForAdd->GetAllKeys();
				
					TArray<FKeyPosition> KeyPositions;
					KeyPositions.AddDefaulted();
					CurveModelForAdd->GetKeyPositions(KeyHandles, KeyPositions);
						
					NewKeyPosition.InputValue = KeyPositions[0].InputValue + 1;
					NewKeyPosition.OutputValue = KeyPositions[0].OutputValue;
				}
				else
				{
					TArray<FKeyHandlePositionPair> SortedKeyHandlePositionPairs;
					GetSortedKeyHandlessAndPositionsForModel(*CurveModelForAdd, SortedKeyHandlePositionPairs);

					int32 IndexToAddAfter = INDEX_NONE;
					if (SelectedKeyHandles.Num() > 0)
					{
						for (int32 i = SortedKeyHandlePositionPairs.Num() - 1; i >= 0; i--)
						{
							if (SelectedKeyHandles.Contains(SortedKeyHandlePositionPairs[i].Handle))
							{
								IndexToAddAfter = i;
								break;
							}
						}
					}

					if(IndexToAddAfter == INDEX_NONE)
					{
						IndexToAddAfter = SortedKeyHandlePositionPairs.Num() - 1;
					}

					if (IndexToAddAfter == SortedKeyHandlePositionPairs.Num() - 1)
					{
						const FKeyPosition& TargetKeyPosition = SortedKeyHandlePositionPairs[IndexToAddAfter].Position;
						const FKeyPosition& PreviousKeyPosition = SortedKeyHandlePositionPairs[IndexToAddAfter - 1].Position;
						NewKeyPosition.InputValue = TargetKeyPosition.InputValue + (TargetKeyPosition.InputValue - PreviousKeyPosition.InputValue);
						NewKeyPosition.OutputValue = TargetKeyPosition.OutputValue + (TargetKeyPosition.OutputValue - PreviousKeyPosition.OutputValue);
					}
					else
					{
						const FKeyPosition& TargetKeyPosition = SortedKeyHandlePositionPairs[IndexToAddAfter].Position;
						const FKeyPosition& NextKeyPosition = SortedKeyHandlePositionPairs[IndexToAddAfter + 1].Position;
						NewKeyPosition.InputValue = TargetKeyPosition.InputValue + ((NextKeyPosition.InputValue - TargetKeyPosition.InputValue) / 2);
						NewKeyPosition.OutputValue = TargetKeyPosition.OutputValue + ((NextKeyPosition.OutputValue - TargetKeyPosition.OutputValue) / 2);
					}
				}

				FScopedTransaction Transaction(LOCTEXT("AddKey", "Add Key"));
				TOptional<FKeyHandle> NewKeyHandle = CurveModelForAdd->AddKey(NewKeyPosition, NewKeyAttributes);
				if(NewKeyHandle.IsSet())
				{
					FCurveEditorSelection& CurveEditorSelection = CurveEditor->GetSelection();
					CurveEditorSelection.Clear();
					CurveEditorSelection.Add(CurveModelIdForAdd.GetValue(), ECurvePointType::Key, NewKeyHandle.GetValue());

					TArray<FCurveModelID> CurvesToFit;
					CurvesToFit.Add(CurveModelIdForAdd.GetValue());
					CurveEditor->ZoomToFitCurves(CurvesToFit);
				}
			}
			return FReply::Handled();
		}

		FReply DeleteKeyClicked()
		{
			CurveEditor->DeleteSelection();
			return FReply::Handled();
		}

	private:
		TSharedPtr<FCurveEditor> CurveEditor;
		TArray<FCurveEditorTreeItemID> OrderedCurveTreeItemIds;
		TSharedPtr<SCurveEditorTree> CurveEditorTree;
	};
}

void SNiagaraDistributionCurveEditor::Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter)
{
	DistributionAdapter = InDistributionAdapter;
	CurveEditorOptions	= InArgs._CurveEditorOptions;

	if (CurveEditorOptions == nullptr)
	{
		CurveEditorOptions = MakeShared<FNiagaraDistributionCurveEditorOptions>();
	}

	if (CurveEditorOptions->GetNeedsInitializeView())
	{
		InitializeView();
	}

	UNiagaraEditorSettings* EditorSettings = GetMutableDefault<UNiagaraEditorSettings>();

	CurveEditor = MakeShared<FCurveEditor>();

	FCurveEditorInitParams InitParams;
	CurveEditor->InitCurveEditor(InitParams);
	CurveEditor->InputSnapEnabledAttribute = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateUObject(EditorSettings, &UNiagaraEditorSettings::IsCurveInputSnapEnabled));
	CurveEditor->OutputSnapEnabledAttribute = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateUObject(EditorSettings, &UNiagaraEditorSettings::IsCurveOutputSnapEnabled));
	CurveEditor->OnInputSnapEnabledChanged.BindUObject(EditorSettings, &UNiagaraEditorSettings::SetCurveInputSnapEnabled);
	CurveEditor->OnOutputSnapEnabledChanged.BindUObject(EditorSettings, &UNiagaraEditorSettings::SetCurveOutputSnapEnabled);
	CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");

	// Initialize our bounds at slightly larger than default to avoid clipping the tabs on the color widget.
	TUniquePtr<ICurveEditorBounds> EditorBounds = MakeUnique<FNiagaraDistributionCurveEditorBounds>(CurveEditorOptions.ToSharedRef());
	CurveEditor->SetBounds(MoveTemp(EditorBounds));

	const int NumChannels = DistributionAdapter->GetNumChannels();
	const bool bIsSingleCurve = NumChannels == 1;

	CurveEditorPanel =
		SNew(SCurveEditorPanel, CurveEditor.ToSharedRef())
		.MinimumViewPanelHeight(50.0f)
		.TreeContent()
		[
			bIsSingleCurve ? SNullWidget::NullWidget : SAssignNew(CurveEditorTree, SCurveEditorTree, CurveEditor).SelectColumnWidth(0)
		];

	TArray<FCurveEditorTreeItemID> TreeItemIds;
	for (int iChannel=0; iChannel < NumChannels; ++iChannel)
	{
		TSharedRef<FNiagaraDistributionCurveEditorDetailsTreeItem> TreeItem = MakeShared<FNiagaraDistributionCurveEditorDetailsTreeItem>();
		TreeItem->Initialize(DistributionAdapter.ToSharedRef(), iChannel);
		FCurveEditorTreeItem* NewItem = CurveEditor->AddTreeItem(FCurveEditorTreeItemID::Invalid());
		NewItem->SetStrongItem(TreeItem);
		TreeItemIds.Add(NewItem->GetID());
		for (const FCurveModelID& CurveModel : NewItem->GetOrCreateCurves(CurveEditor.Get()))
		{
			CurveEditor->PinCurve(CurveModel);
		}
	}

	const float KeySelectorLeftPadding = bIsSingleCurve ? 0 : 7;

	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	TSharedRef<INiagaraEditorWidgetProvider> WidgetProvider = NiagaraEditorModule.GetWidgetProvider();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 3)
		[
			CreateToolbarWidget()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SVerticalResizeBox)
			.ContentHeight(CurveEditorOptions.ToSharedRef(), &FNiagaraDistributionCurveEditorOptions::GetHeight)
			.ContentHeightChanged(CurveEditorOptions.ToSharedRef(), &FNiagaraDistributionCurveEditorOptions::SetHeight)
			.Content()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(0, 0, 0, 5)
				[
					CurveEditorPanel.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(KeySelectorLeftPadding, 0, 8, 0)
					[
						WidgetProvider->CreateCurveKeySelector(CurveEditor, TreeItemIds, CurveEditorTree)
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(0, 0, 3, 0)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("KeyLabel", "Key Data"))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						CurveEditorPanel->GetKeyDetailsView().ToSharedRef()
					]
				]
			]
		]
	];
}

void SNiagaraDistributionCurveEditor::InitializeView()
{
	bool bHasKeys = false;
	float ViewMinInput = TNumericLimits<float>::Max();
	float ViewMaxInput = TNumericLimits<float>::Lowest();
	float ViewMinOutput = TNumericLimits<float>::Max();
	float ViewMaxOutput = TNumericLimits<float>::Lowest();

	for (int i=0; i < DistributionAdapter->GetNumChannels(); ++i)
	{
		const FRealCurve* Curve = DistributionAdapter->GetCurveValue(i);
		for (auto KeyIterator = Curve->GetKeyHandleIterator(); KeyIterator; ++KeyIterator)
		{
			float KeyTime = Curve->GetKeyTime(*KeyIterator);
			float KeyValue = Curve->GetKeyValue(*KeyIterator);
			ViewMinInput = FMath::Min(ViewMinInput, KeyTime);
			ViewMaxInput = FMath::Max(ViewMaxInput, KeyTime);
			ViewMinOutput = FMath::Min(ViewMinOutput, KeyValue);
			ViewMaxOutput = FMath::Max(ViewMaxOutput, KeyValue);
			bHasKeys = true;
		}
	}

	if (bHasKeys == false)
	{
		ViewMinInput = 0;
		ViewMaxInput = 1;
		ViewMinOutput = 0;
		ViewMaxOutput = 1;
	}

	if (FMath::IsNearlyEqual(ViewMinInput, ViewMaxInput))
	{
		if (FMath::IsWithinInclusive(ViewMinInput, 0.0f, 1.0f))
		{
			ViewMinInput = 0;
			ViewMaxInput = 1;
		}
		else
		{
			ViewMinInput -= 0.5f;
			ViewMaxInput += 0.5f;
		}
	}

	if (FMath::IsNearlyEqual(ViewMinOutput, ViewMaxOutput))
	{
		if (FMath::IsWithinInclusive(ViewMinOutput, 0.0f, 1.0f))
		{
			ViewMinOutput = 0;
			ViewMaxOutput = 1;
		}
		else
		{
			ViewMinOutput -= 0.5f;
			ViewMaxOutput += 0.5f;
		}
	}

	float ViewInputRange = ViewMaxInput - ViewMinInput;
	float ViewOutputRange = ViewMaxOutput - ViewMinOutput;
	float ViewInputPadding = ViewInputRange * .05f;
	float ViewOutputPadding = ViewOutputRange * .05f;

	CurveEditorOptions->InitializeView(
		ViewMinInput - ViewInputPadding,
		ViewMaxInput + ViewInputPadding,
		ViewMinOutput - ViewOutputPadding,
		ViewMaxOutput + ViewOutputPadding);
}

TSharedRef<SWidget> SNiagaraDistributionCurveEditor::CreateToolbarWidget()
{
	FToolBarBuilder ToolBarBuilder(CurveEditor->GetCommands(), FMultiBoxCustomization::None, nullptr, true);
	ToolBarBuilder.SetStyle(&FAppStyle::Get(), "CurveEditorToolBar");
	ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().ToggleInputSnapping);
	ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().ToggleOutputSnapping);
	ToolBarBuilder.AddSeparator();
	ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().FlipCurveHorizontal);
	ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().FlipCurveVertical);
	ToolBarBuilder.AddSeparator();

	const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();
	if (EditorSettings->GetCurveTemplates().Num() > 0)
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TSharedRef<INiagaraEditorWidgetProvider> WidgetProvider = NiagaraEditorModule.GetWidgetProvider();

		ToolBarBuilder.AddWidget(
			SNew(SBox)
			.VAlign(VAlign_Center)
			.Padding(FMargin(3, 0, 5, 0))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CurveTemplateLabel", "Templates"))
			]
		);

		for (const FNiagaraCurveTemplate& CurveTemplate : EditorSettings->GetCurveTemplates())
		{
			UCurveFloat* FloatCurveAsset = Cast<UCurveFloat>(CurveTemplate.CurveAsset.TryLoad());
			if (FloatCurveAsset == nullptr)
			{
				continue;
			}

			FText CurveDisplayName =
				CurveTemplate.DisplayNameOverride.IsEmpty()
				? FText::FromName(FloatCurveAsset->GetFName()) //FText::FromString(FName::NameToDisplayString(FloatCurveAsset->GetName(), false))
				: FText::FromString(CurveTemplate.DisplayNameOverride);

			TWeakObjectPtr<UCurveFloat> WeakCurveAsset(FloatCurveAsset);
			ToolBarBuilder.AddWidget(
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SNiagaraDistributionCurveEditor::ApplyCurveTemplate, WeakCurveAsset)
				.ToolTipText(FText::Format(LOCTEXT("ApplyCurveTemplateFormat", "{0}\nClick to apply this template to the selected curves."), CurveDisplayName))
				.ContentPadding(FMargin(3, 10, 3, 0))
				.Content()
				[
					WidgetProvider->CreateCurveThumbnail(FloatCurveAsset->FloatCurve)
				]
			);
		}
	}

	return ToolBarBuilder.MakeWidget();
}

FReply SNiagaraDistributionCurveEditor::ApplyCurveTemplate(TWeakObjectPtr<UCurveFloat> WeakCurveAsset)
{
	UCurveFloat* FloatCurveAsset = WeakCurveAsset.Get();
	if (FloatCurveAsset != nullptr)
	{
		TArray<FCurveModelID> CurveModelIdsToSet;
		if (CurveEditor->GetRootTreeItems().Num() == 1)
		{
			const FCurveEditorTreeItem& TreeItem = CurveEditor->GetTreeItem(CurveEditor->GetRootTreeItems()[0]);
			for (const FCurveModelID& CurveModelId : TreeItem.GetCurves())
			{
				CurveModelIdsToSet.Add(CurveModelId);
			}
		}
		else
		{
			for (const TPair<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& TreeItemSelectionState : CurveEditor->GetTreeSelection())
			{
				if (TreeItemSelectionState.Value != ECurveEditorTreeSelectionState::None)
				{
					const FCurveEditorTreeItem& TreeItem = CurveEditor->GetTreeItem(TreeItemSelectionState.Key);
					for (const FCurveModelID& CurveModelId : TreeItem.GetCurves())
					{
						CurveModelIdsToSet.Add(CurveModelId);
					}
				}
			}
		}

		if (CurveModelIdsToSet.Num() > 0)
		{
			FScopedTransaction ApplyTemplateTransaction(LOCTEXT("ApplyCurveTemplateTransaction", "Apply curve template"));
			for (const FCurveModelID& CurveModelId : CurveModelIdsToSet)
			{
				FCurveModel* CurveModel = CurveEditor->GetCurves()[CurveModelId].Get();
				if (CurveModel != nullptr)
				{
					const TArray<FKeyHandle> KeyHandles = CurveModel->GetAllKeys();
					CurveModel->RemoveKeys(KeyHandles, 0.0);

					const FRichCurve& FloatCurve = FloatCurveAsset->FloatCurve;
					for (auto KeyIterator = FloatCurve.GetKeyHandleIterator(); KeyIterator; ++KeyIterator)
					{
						const FRichCurveKey& Key = FloatCurve.GetKey(*KeyIterator);
						FKeyPosition KeyPosition;
						KeyPosition.InputValue = Key.Time;
						KeyPosition.OutputValue = Key.Value;
						FKeyAttributes KeyAttributes;
						KeyAttributes.SetInterpMode(Key.InterpMode);
						KeyAttributes.SetTangentMode(Key.TangentMode);
						KeyAttributes.SetArriveTangent(Key.ArriveTangent);
						KeyAttributes.SetLeaveTangent(Key.LeaveTangent);
						CurveModel->AddKey(KeyPosition, KeyAttributes);
					}
				}
			}
			CurveEditor->ZoomToFit();
		}
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
