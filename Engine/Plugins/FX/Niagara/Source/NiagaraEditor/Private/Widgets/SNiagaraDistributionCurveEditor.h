// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/SlateStructs.h"
#include "Widgets/SCompoundWidget.h"

class FCurveEditor;
class SCurveEditorPanel;
class SCurveEditorTree;
class UCurveFloat;
class INiagaraDistributionAdapter;

class FNiagaraDistributionCurveEditorOptions
{
public:
	void InitializeView(float InViewMinInput, float InViewMaxInput, float InViewMinOutput, float InViewMaxOutput)
	{
		ViewMinInput = InViewMinInput;
		ViewMaxInput = InViewMaxInput;
		ViewMinOutput = InViewMinOutput;
		ViewMaxOutput = InViewMaxOutput;
		bNeedsInitializeView = false;
	}

	bool GetNeedsInitializeView() const { return bNeedsInitializeView; }

	float GetViewMinInput() const { return ViewMinInput; }
	float GetViewMaxInput() const { return ViewMaxInput; }
	void SetInputViewRange(float InViewMinInput, float InViewMaxInput) { ViewMinInput = InViewMinInput; ViewMaxInput = InViewMaxInput; }

	float GetViewMinOutput() const { return ViewMinOutput; }
	float GetViewMaxOutput() const { return ViewMaxOutput; }
	void SetOutputViewRange(float InViewMinOutput, float InViewMaxOutput) { ViewMinOutput = InViewMinOutput; ViewMaxOutput = InViewMaxOutput; }

	bool GetIsGradientVisible() const { return bIsGradientVisible; }
	void SetIsGradientVisible(bool bInIsGradientVisible) { bIsGradientVisible = bInIsGradientVisible; }

	float GetTimelineLength() const { return ViewMaxInput - ViewMinInput; }

	float GetHeight() const { return Height; }
	void SetHeight(float InHeight) { Height = InHeight; }

private:
	float ViewMinInput = 0.0f;
	float ViewMaxInput = 1.0f;
	float ViewMinOutput = 0.0f;
	float ViewMaxOutput = 1.0f;
	bool bIsGradientVisible = true;
	bool bNeedsInitializeView = true;
	float Height = 180.0f;
};

class SNiagaraDistributionCurveEditor : public SCompoundWidget
{
public:

public:
	SLATE_BEGIN_ARGS(SNiagaraDistributionCurveEditor)
		{ }
		SLATE_ARGUMENT(TSharedPtr<FNiagaraDistributionCurveEditorOptions>, CurveEditorOptions)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InCurveEditorAdapter);

private:
	void InitializeView();
	TSharedRef<SWidget> CreateToolbarWidget();
	FReply ApplyCurveTemplate(TWeakObjectPtr<UCurveFloat> WeakCurveAsset);

private:
	TSharedPtr<INiagaraDistributionAdapter>				DistributionAdapter;
	TSharedPtr<FNiagaraDistributionCurveEditorOptions>	CurveEditorOptions;
	TSharedPtr<FCurveEditor>							CurveEditor;
	TSharedPtr<SCurveEditorPanel>						CurveEditorPanel;
	TSharedPtr<SCurveEditorTree>						CurveEditorTree;
};
