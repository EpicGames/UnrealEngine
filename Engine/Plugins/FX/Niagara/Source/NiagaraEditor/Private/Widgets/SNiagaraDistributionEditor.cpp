// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraDistributionEditor.h"

#include "NiagaraEditorModule.h"
#include "Curves/CurveOwnerInterface.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "NiagaraEditorStyle.h"
#include "SColorGradientEditor.h"
#include "SNiagaraParameterEditor.h"
#include "TypeEditorUtilities/NiagaraFloatTypeEditorUtilities.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/NiagaraDistributionEditorUtilities.h"
#include "Widgets/SNiagaraColorEditor.h"
#include "Widgets/SNiagaraDistributionArrayEditor.h"
#include "Widgets/SNiagaraDistributionCurveEditor.h"
#include "Widgets/SNiagaraDistributionCompactCurveEditor.h"
#include "Widgets/SNiagaraDistributionLookupValueWidget.h"
#include "Widgets/SNiagaraParameterName.h"

#include "Stateless/SNiagaraStatelessExpressionWidget.h"

#define LOCTEXT_NAMESPACE "NiagaraDistributionEditor"

namespace SNiagaraDistributionEditorPrivate
{
	static bool GUseCompactEditor = false;
	static FAutoConsoleVariableRef CVarUseCompactEditor(
		TEXT("fx.Niagara.UI.Distribution.UseCompactEditor"),
		GUseCompactEditor,
		TEXT("When enabled we use the compact editor vs the curve editor.")
	);
}

class SNiagaraDistributionModeSelector : public SComboButton
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDistributionModeSelector) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter)
	{
		DistributionAdapter = InDistributionAdapter;
		SComboButton::Construct(SComboButton::FArguments()
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ContentPadding(FMargin(0))
			.OnGetMenuContent(this, &SNiagaraDistributionModeSelector::OnGetMenuContent)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(this, &SNiagaraDistributionModeSelector::GetModeIcon)
				.ToolTipText(this, &SNiagaraDistributionModeSelector::GetModeToolTip)
			]);
	}

private:
	const FSlateBrush* GetModeIcon() const
	{
		UpdateCachedValues();
		return ModeIconCache;
	}

	FText GetModeToolTip() const
	{
		UpdateCachedValues();
		return ModeToolTipCache;
	}

	TSharedRef<SWidget> OnGetMenuContent()
	{
		FMenuBuilder MenuBuilder(true, nullptr);
		TArray<ENiagaraDistributionEditorMode> SupportedModes;
		DistributionAdapter->GetSupportedDistributionModes(SupportedModes);
		for (ENiagaraDistributionEditorMode SupportedMode : SupportedModes)
		{
			MenuBuilder.AddMenuEntry(
				FNiagaraDistributionEditorUtilities::DistributionModeToDisplayName(SupportedMode),
				FNiagaraDistributionEditorUtilities::DistributionModeToToolTipText(SupportedMode),
				FNiagaraDistributionEditorUtilities::DistributionModeToIcon(SupportedMode),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraDistributionModeSelector::DistributionModeSelected, SupportedMode),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SNiagaraDistributionModeSelector::IsDistributionModeSelected, SupportedMode)
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		return MenuBuilder.MakeWidget();
	}

	void UpdateCachedValues() const
	{
		ENiagaraDistributionEditorMode CurrentMode = DistributionAdapter->GetDistributionMode();
		if (ModeCache.IsSet() == false || ModeCache.GetValue() != CurrentMode)
		{
			ModeCache = CurrentMode;
			ModeIconCache = FNiagaraDistributionEditorUtilities::DistributionModeToIconBrush(CurrentMode);
			ModeToolTipCache = FText::Format(LOCTEXT("ModeSelectorToolTip", "{0} - {1}"),
				FNiagaraDistributionEditorUtilities::DistributionModeToDisplayName(CurrentMode),
				FNiagaraDistributionEditorUtilities::DistributionModeToToolTipText(CurrentMode));
		}
	}

	bool IsDistributionModeSelected(ENiagaraDistributionEditorMode InSelectedMode) const
	{
		UpdateCachedValues();
		return ModeCache == InSelectedMode;
	}

	void DistributionModeSelected(ENiagaraDistributionEditorMode InSelectedMode)
	{
		DistributionAdapter->SetDistributionMode(InSelectedMode);
		UpdateCachedValues();
	}

private:
	TSharedPtr<INiagaraDistributionAdapter> DistributionAdapter;

	mutable TOptional<ENiagaraDistributionEditorMode> ModeCache;
	mutable const FSlateBrush* ModeIconCache = nullptr;
	mutable FText ModeToolTipCache;
};

class SNiagaraDistributionBindingEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDistributionBindingEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter)
	{
		DistributionAdapter = InDistributionAdapter;

		ChildSlot
		[
			SNew(SComboButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ContentPadding(FMargin(0))
			.OnGetMenuContent(this, &SNiagaraDistributionBindingEditor::OnGetMenuContent)
			.ButtonContent()
			[
				SNew(SNiagaraParameterName)
				.ParameterName(this, &SNiagaraDistributionBindingEditor::GetBindingName)
				.IsReadOnly(true)
			]
		];
	}

private:
	FName GetBindingName() const
	{
		FNiagaraVariableBase Binding = DistributionAdapter->GetBindingValue();
		return Binding.IsValid() ? Binding.GetName() : NAME_None;
	}

	void SetBinding(FNiagaraVariableBase Binding)
	{
		DistributionAdapter->SetBindingValue(Binding);
	}

	TSharedRef<SWidget> OnGetMenuContent()
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		const TArray<FNiagaraVariableBase> AvailableBindings = DistributionAdapter->GetAvailableBindings();
		if (AvailableBindings.Num() > 0)
		{
			for (const FNiagaraVariableBase& Variable : AvailableBindings)
			{
				TSharedRef<SWidget> Widget = SNew(SNiagaraParameterName)
					.ParameterName(Variable.GetName())
					.IsReadOnly(true);

				MenuBuilder.AddMenuEntry(
					FUIAction(
						FExecuteAction::CreateSP(this, &SNiagaraDistributionBindingEditor::SetBinding, Variable)
					),
					Widget
				);
			}
		}
		else
		{
			const FNiagaraTypeDefinition AllowedTypeDef = DistributionAdapter->GetExpressionTypeDef();
			if (AllowedTypeDef.IsValid())
			{
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("NoBindings", "No bindings of type '{0}' found"), AllowedTypeDef.GetNameText()),
					LOCTEXT("NoBindingsDesc", "No bindings for the property type were found."),
					FSlateIcon(),
					FUIAction()
				);
			}
			else
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("InvalidTypeDef", "Bindings Unavailable"),
					LOCTEXT("InvalidTypeDefDesc", "Bindings can not be set on this property."),
					FSlateIcon(),
					FUIAction()
				);
			}
		}
		return MenuBuilder.MakeWidget();
	}

private:
	TSharedPtr<INiagaraDistributionAdapter> DistributionAdapter;
};

class SNiagaraDistributionValueEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDistributionValueEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter)
	{
		DistributionAdapter = InDistributionAdapter;
		bool bIsUniform = FNiagaraDistributionEditorUtilities::IsUniform(DistributionAdapter->GetDistributionMode());
		bool bIsColor = FNiagaraDistributionEditorUtilities::IsColor(DistributionAdapter->GetDistributionMode());
		bool bIsConstant = FNiagaraDistributionEditorUtilities::IsConstant(DistributionAdapter->GetDistributionMode());
		bool bLayoutHorizontal = bIsUniform || bIsColor;

		static const FText MinLabelText = LOCTEXT("MinLabel", "Min");
		static const FText MaxLabelText = LOCTEXT("MaxLabel", "Max");

		TSharedPtr<SWidget> ChildWidget;
		if (bIsConstant)
		{
			TSharedPtr<SWidget> ValueWidget;
			if (bIsUniform)
			{
				ValueWidget = ConstructCustomizableFloatWidget(0, 0);
			}
			else if (bIsColor)
			{
				ValueWidget = ConstructColorWidget(DistributionAdapter->GetNumChannels(), 0, true, FText());
			}
			else
			{
				ValueWidget = ConstructVectorWidget(DistributionAdapter->GetNumChannels(), 0);
			}
			ChildWidget = SNew(SBox)
				.HAlign(HAlign_Left)
				[
					ValueWidget.ToSharedRef()
				];
		}
		else
		{
			if (bIsUniform)
			{
				ChildWidget = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0, 0, 10, 0)
					[
						ConstructFloatWidget(0, 0, MinLabelText)
					]
					+ SHorizontalBox::Slot()
					[
						ConstructFloatWidget(0, 1, MaxLabelText)
					];
			}
			else if (bIsColor)
			{
				ChildWidget = SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0, 0, 10, 0)
					[
						ConstructColorWidget(DistributionAdapter->GetNumChannels(), 0, true, MinLabelText)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ConstructColorWidget(DistributionAdapter->GetNumChannels(), 1, false, MaxLabelText)
					];
			}
			else
			{
				if (bIsConstant)
				{
					ChildWidget = ConstructVectorWidget(DistributionAdapter->GetNumChannels(), 0);
				}
				else
				{
					ChildWidget = SNew(SGridPanel)
						.FillColumn(1, 1)
						+ SGridPanel::Slot(0, 0)
						.VAlign(VAlign_Center)
						.Padding(0, 0, 5, 3)
						[
							SNew(STextBlock)
							.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
							.Text(MinLabelText)
						]
						+ SGridPanel::Slot(1, 0)
						.Padding(0, 0, 0, 3)
						[
							ConstructVectorWidget(DistributionAdapter->GetNumChannels(), 0)
						]
						+ SGridPanel::Slot(0, 1)
						.VAlign(VAlign_Center)
						.Padding(0, 0, 5, 0)
						[
							SNew(STextBlock)
							.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
							.Text(MaxLabelText)
						]
						+ SGridPanel::Slot(1, 1)
						[
							ConstructVectorWidget(DistributionAdapter->GetNumChannels(), 1)
						];
				}
			}
		}

		ChildSlot
		[
			ChildWidget.ToSharedRef()
		];
	}

private:
	bool GetColorComponentsExpanded() const
	{
		return bColorComponentsExpanded;
	}

	void ColorComponentsExandedChanged(bool bExpanded)
	{
		bColorComponentsExpanded = bExpanded;
	}

	TSharedRef<SWidget> ConstructFloatWidget(int32 ChannelIndex, int32 ValueIndex, FText LabelText)
	{
		TSharedRef<SWidget> LabelWidget = LabelText.IsEmpty()
			? SNullWidget::NullWidget
			: SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(LabelText);
		EUnit DisplayUnit = DistributionAdapter->GetDisplayUnit();
		
		return
			SNew(SNumericEntryBox<float>)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			.Value(this, &SNiagaraDistributionValueEditor::GetValue, ChannelIndex, ValueIndex)
			.OnValueChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, ChannelIndex, ValueIndex)
			.OnValueCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, ChannelIndex, ValueIndex)
			.OnBeginSliderMovement(this, &SNiagaraDistributionValueEditor::BeginValueChange)
			.OnEndSliderMovement(this, &SNiagaraDistributionValueEditor::EndValueChange)
			.AllowSpin(true)
			.MinValue(TOptional<float>())
			.MaxValue(TOptional<float>())
			.MinSliderValue(TOptional<float>())
			.MaxSliderValue(TOptional<float>())
			.BroadcastValueChangesPerKey(false)
			.TypeInterface(MakeShareable(new TNumericUnitTypeInterface<float>(DisplayUnit)))
			.MinDesiredValueWidth(SNiagaraDistributionEditor::DefaultInputSize - 18)
			.LabelVAlign(EVerticalAlignment::VAlign_Center)
			.Label()
			[
				LabelWidget
			];
	}

	TSharedRef<SWidget> ConstructCustomizableFloatWidget(int32 ChannelIndex, int32 ValueIndex)
	{
		FNiagaraEditorModule& NiagaraEditorModule = FNiagaraEditorModule::Get();
		TSharedPtr<INiagaraEditorTypeUtilities> TypeUtilities = NiagaraEditorModule.GetTypeUtilities(FNiagaraTypeDefinition::GetFloatDef());
		FNiagaraInputParameterCustomization WidgetCustomization = DistributionAdapter->GetWidgetCustomization();
		EUnit DisplayUnit = DistributionAdapter->GetDisplayUnit();
		return SNew(SNiagaraFloatParameterEditor, DisplayUnit, WidgetCustomization)
			.Value(this, &SNiagaraDistributionValueEditor::GetValueFloat, ChannelIndex, ValueIndex)
			.OnValueChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, ChannelIndex, ValueIndex)
			.OnBeginValueChange(this, &SNiagaraDistributionValueEditor::BeginValueChange)
			.OnEndValueChange(this, &SNiagaraDistributionValueEditor::EndValueChange);
	}

	typedef SNumericVectorInputBox<float, UE::Math::TVector2<float>, 2> SNumericVectorInputBox2;
	typedef SNumericVectorInputBox<float, UE::Math::TVector<float>, 3> SNumericVectorInputBox3;
	typedef SNumericVectorInputBox<float, UE::Math::TVector4<float>, 4> SNumericVectorInputBox4;

	TSharedRef<SWidget> ConstructVectorWidget(int32 ChannelCount, int32 ValueIndex)
	{
		const EUnit DisplayUnit = DistributionAdapter->GetDisplayUnit();
		if (ChannelCount == 2)
		{
			return SNew(SBox)
				.MinDesiredWidth(2 * SNiagaraDistributionEditor::DefaultInputSize)
				[
					SNew(SNumericVectorInputBox2)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.AllowSpin(true)
					.bColorAxisLabels(true)
					.X(this, &SNiagaraDistributionValueEditor::GetValue, 0, ValueIndex)
					.XDisplayName(this, &SNiagaraDistributionValueEditor::GetChannelDisplayName, 0)
					.OnXChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, 0, ValueIndex)
					.OnXCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, 0, ValueIndex)
					.Y(this, &SNiagaraDistributionValueEditor::GetValue, 1, ValueIndex)
					.YDisplayName(this, &SNiagaraDistributionValueEditor::GetChannelDisplayName, 1)
					.OnYChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, 1, ValueIndex)
					.OnYCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, 1, ValueIndex)
					.OnBeginSliderMovement(this, &SNiagaraDistributionValueEditor::BeginValueChange)
					.OnEndSliderMovement(this, &SNiagaraDistributionValueEditor::EndValueChange)
					.TypeInterface(MakeShareable(new TNumericUnitTypeInterface<float>(DisplayUnit)))
				];
		}
		if (ChannelCount == 3)
		{
			return SNew(SBox)
				.MinDesiredWidth(3 * SNiagaraDistributionEditor::DefaultInputSize)
				[
					SNew(SNumericVectorInputBox3)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.AllowSpin(true)
					.bColorAxisLabels(true)
					.X(this, &SNiagaraDistributionValueEditor::GetValue, 0, ValueIndex)
					.XDisplayName(this, &SNiagaraDistributionValueEditor::GetChannelDisplayName, 0)
					.OnXChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, 0, ValueIndex)
					.OnXCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, 0, ValueIndex)
					.Y(this, &SNiagaraDistributionValueEditor::GetValue, 1, ValueIndex)
					.YDisplayName(this, &SNiagaraDistributionValueEditor::GetChannelDisplayName, 1)
					.OnYChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, 1, ValueIndex)
					.OnYCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, 1, ValueIndex)
					.Z(this, &SNiagaraDistributionValueEditor::GetValue, 2, ValueIndex)
					.ZDisplayName(this, &SNiagaraDistributionValueEditor::GetChannelDisplayName, 2)
					.OnZChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, 2, ValueIndex)
					.OnZCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, 2, ValueIndex)
					.OnBeginSliderMovement(this, &SNiagaraDistributionValueEditor::BeginValueChange)
					.OnEndSliderMovement(this, &SNiagaraDistributionValueEditor::EndValueChange)
					.TypeInterface(MakeShareable(new TNumericUnitTypeInterface<float>(DisplayUnit)))
				];
		}
		if (ChannelCount == 4)
		{
			return SNew(SBox)
				.MinDesiredWidth(4 * SNiagaraDistributionEditor::DefaultInputSize)
				[
					SNew(SNumericVectorInputBox4)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.AllowSpin(true)
					.bColorAxisLabels(true)
					.X(this, &SNiagaraDistributionValueEditor::GetValue, 0, ValueIndex)
					.XDisplayName(this, &SNiagaraDistributionValueEditor::GetChannelDisplayName, 0)
					.OnXChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, 0, ValueIndex)
					.OnXCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, 0, ValueIndex)
					.Y(this, &SNiagaraDistributionValueEditor::GetValue, 1, ValueIndex)
					.YDisplayName(this, &SNiagaraDistributionValueEditor::GetChannelDisplayName, 1)
					.OnYChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, 1, ValueIndex)
					.OnYCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, 1, ValueIndex)
					.Z(this, &SNiagaraDistributionValueEditor::GetValue, 2, ValueIndex)
					.ZDisplayName(this, &SNiagaraDistributionValueEditor::GetChannelDisplayName, 2)
					.OnZChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, 2, ValueIndex)
					.OnZCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, 2, ValueIndex)
					.W(this, &SNiagaraDistributionValueEditor::GetValue, 3, ValueIndex)
					.WDisplayName(this, &SNiagaraDistributionValueEditor::GetChannelDisplayName, 3)
					.OnWChanged(this, &SNiagaraDistributionValueEditor::ValueChanged, 3, ValueIndex)
					.OnWCommitted(this, &SNiagaraDistributionValueEditor::ValueCommitted, 3, ValueIndex)
					.OnBeginSliderMovement(this, &SNiagaraDistributionValueEditor::BeginValueChange)
					.OnEndSliderMovement(this, &SNiagaraDistributionValueEditor::EndValueChange)
					.TypeInterface(MakeShareable(new TNumericUnitTypeInterface<float>(DisplayUnit)))
				];
		}
		return SNullWidget::NullWidget;
	}

	TSharedRef<SWidget> ConstructColorWidget(int32 ChannelCount, int32 ValueIndex, bool bShowExpander, FText LabelText)
	{
		return SNew(SNiagaraColorEditor)
			.ShowAlpha(ChannelCount == 4)
			.ShowExpander(bShowExpander)
			.ExpandComponents(this, &SNiagaraDistributionValueEditor::GetColorComponentsExpanded)
			.Color(this, &SNiagaraDistributionValueEditor::GetColorValue, ValueIndex)
			.LabelText(LabelText)
			.MinDesiredColorBlockWidth(SNiagaraDistributionEditor::DefaultInputSize)
			.OnColorChanged(this, &SNiagaraDistributionValueEditor::ColorValueChanged, ValueIndex)
			.OnBeginEditing(this, &SNiagaraDistributionValueEditor::BeginValueChange)
			.OnEndEditing(this, &SNiagaraDistributionValueEditor::EndValueChange, 0.0f)
			.OnCancelEditing(this, &SNiagaraDistributionValueEditor::CancelValueChange, ValueIndex)
			.OnExpandComponentsChanged(this, &SNiagaraDistributionValueEditor::ColorComponentsExandedChanged);
	}

	TOptional<float> GetValue(int32 ChannelIndex, int32 ValueIndex) const
	{
		return DistributionAdapter->GetConstantOrRangeValue(ChannelIndex, ValueIndex);
	}

	float GetValueFloat(int32 ChannelIndex, int32 ValueIndex) const
	{
		return DistributionAdapter->GetConstantOrRangeValue(ChannelIndex, ValueIndex);
	}

	void ValueChanged(float Value, int32 ChannelIndex, int32 ValueIndex)
	{
		DistributionAdapter->SetConstantOrRangeValue(ChannelIndex, ValueIndex, Value);
	}
	
	void ValueCommitted(float Value, ETextCommit::Type CommitInfo, int32 ChannelIndex, int32 ValueIndex)
	{
		if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
		{
			DistributionAdapter->SetConstantOrRangeValue(ChannelIndex, ValueIndex, Value);
		}
	}

	FText GetChannelDisplayName(int32 ChannelIndex) const
	{
		return DistributionAdapter->GetChannelDisplayName(ChannelIndex);
	}

	FLinearColor GetColorValue(int32 ValueIndex) const
	{
		if (DistributionAdapter->GetNumChannels() == 3)
		{
			return FLinearColor(
				DistributionAdapter->GetConstantOrRangeValue(0, ValueIndex),
				DistributionAdapter->GetConstantOrRangeValue(1, ValueIndex), 
				DistributionAdapter->GetConstantOrRangeValue(2, ValueIndex));
		}
		else if (DistributionAdapter->GetNumChannels() == 4)
		{
			return FLinearColor(
				DistributionAdapter->GetConstantOrRangeValue(0, ValueIndex),
				DistributionAdapter->GetConstantOrRangeValue(1, ValueIndex),
				DistributionAdapter->GetConstantOrRangeValue(2, ValueIndex),
				DistributionAdapter->GetConstantOrRangeValue(3, ValueIndex));
		}
		return FLinearColor::White;
	}

	void ColorValueChanged(FLinearColor InColorValue, int32 ValueIndex)
	{
		UpdateColorValue(ValueIndex, InColorValue);
	}

	void BeginValueChange()
	{
		DistributionAdapter->BeginContinuousChange();
	}

	void EndValueChange(float Value)
	{
		DistributionAdapter->EndContinuousChange();
	}

	void CancelValueChange(FLinearColor OriginalColor, int32 ValueIndex)
	{
		UpdateColorValue(ValueIndex, OriginalColor);
		DistributionAdapter->CancelContinuousChange();
	}

	void UpdateColorValue(int32 ValueIndex, const FLinearColor& InColorValue)
	{
		TArray<float> ColorValues{ InColorValue.R, InColorValue.G, InColorValue.B };
		if (DistributionAdapter->GetNumChannels() == 4)
		{
			ColorValues.Add(InColorValue.A);
		}
		DistributionAdapter->SetConstantOrRangeValues(ValueIndex, ColorValues);
	}

private:
	TSharedPtr<INiagaraDistributionAdapter> DistributionAdapter;
	bool bColorComponentsExpanded = false;
};

class FNiagaraDistributionColorCurveOwner : public FCurveOwnerInterface
{
public:
	FNiagaraDistributionColorCurveOwner(TSharedPtr<INiagaraDistributionAdapter> InDistributionAdapter)
	{
		DistributionAdapter = InDistributionAdapter;
		if (DistributionAdapter->GetDistributionMode() == ENiagaraDistributionEditorMode::ColorGradient &&
			DistributionAdapter->GetNumChannels() >= 3)
		{
			// The gradient editor requires an alpha curve, so we need to create 4 curve infos here even if the distribution
			// only has 3 channels.
			for (int32 ChannelIndex = 0; ChannelIndex < 4; ++ChannelIndex)
			{
				FRichCurve& EditCurve = EditCurves.Add_GetRef(ChannelIndex < InDistributionAdapter->GetNumChannels()
					? *DistributionAdapter->GetCurveValue(ChannelIndex)
					: FRichCurve());
				ConstCurveInfos.Add(FRichCurveEditInfoConst(&EditCurve, *DistributionAdapter->GetChannelDisplayName(ChannelIndex).ToString()));
				CurveInfos.Add(FRichCurveEditInfo(&EditCurve, *DistributionAdapter->GetChannelDisplayName(ChannelIndex).ToString()));
			}
		}
	}

	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const override { return ConstCurveInfos; }
	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override
	{
		for (const FRichCurveEditInfoConst& ConstCurve: ConstCurveInfos)
		{
			Curves.Add(ConstCurve);
		}
	}

	virtual TArray<FRichCurveEditInfo> GetCurves() override { return CurveInfos; }

	virtual void ModifyOwner() override
	{
		DistributionAdapter->ModifyOwners();
	}

	virtual TArray<const UObject*> GetOwners() const override
	{
		static TArray<const UObject*> Owners;
		return Owners;
	}

	virtual void MakeTransactional() override { }

	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override
	{
		for (const FRichCurveEditInfo& ChangedCurveEditInfo : ChangedCurveEditInfos)
		{
			int32 CurveIndex = EditCurves.IndexOfByPredicate([ChangedCurveEditInfo]
				(const FRichCurve& EditCurve) { return &EditCurve == ChangedCurveEditInfo.CurveToEdit; });
			if (CurveIndex != INDEX_NONE && CurveIndex < DistributionAdapter->GetNumChannels())
			{
				DistributionAdapter->SetCurveValue(CurveIndex, EditCurves[CurveIndex]);
			}
		}
	}

	virtual bool IsLinearColorCurve() const override
	{
		return true;
	}

	virtual FLinearColor GetLinearColorValue(float InTime) const override
	{
		return FLinearColor(
			EditCurves[0].Eval(InTime),
			EditCurves[1].Eval(InTime),
			EditCurves[2].Eval(InTime),
			EditCurves[3].Eval(InTime));
	}

	virtual bool HasAnyAlphaKeys() const override
	{
		return EditCurves[3].GetNumKeys() > 0;
	}

	virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override
	{
		return CurveInfos.Contains(CurveInfo);
	}

	virtual FLinearColor GetCurveColor(FRichCurveEditInfo CurveInfo) const override
	{
		return FLinearColor::White;
	}

private:
	TSharedPtr<INiagaraDistributionAdapter> DistributionAdapter;
	TArray<FRichCurve> EditCurves;
	TArray<FRichCurveEditInfoConst> ConstCurveInfos;
	TArray<FRichCurveEditInfo> CurveInfos;
};

class SNiagaraDistributionGradientEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDistributionGradientEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter)
	{
		DistributionAdapter = InDistributionAdapter;
		ColorCurveOwner = MakeShared<FNiagaraDistributionColorCurveOwner>(DistributionAdapter);
		TSharedRef<SColorGradientEditor> GradientEditor = SNew(SColorGradientEditor)
			.ViewMinInput(0.0f)
			.ViewMaxInput(1.0f)
			.ClampStopsToViewRange(true);

		GradientEditor->SetCurveOwner(ColorCurveOwner.Get());
		ChildSlot
		[
			SNew(SBox)
			.Padding(2)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					GradientEditor
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SNiagaraDistributionLookupValueWidget, DistributionAdapter)
				]
			]
		];
	}

private:
	TSharedPtr<INiagaraDistributionAdapter> DistributionAdapter;
	TSharedPtr<FNiagaraDistributionColorCurveOwner> ColorCurveOwner;
};

void SNiagaraDistributionEditor::Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter)
{
	DistributionAdapter = InDistributionAdapter;
	DistributionAdapter->OnDistributionEditorModeChanged().AddSP(this, &SNiagaraDistributionEditor::OnDistributionModeChanged);
	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, 5, 0)
		[
			SNew(SNiagaraDistributionModeSelector, InDistributionAdapter)
		]
		+ SHorizontalBox::Slot()
		[
			SAssignNew(ContentBox, SBox)
			[
				ConstructContentForMode()
			]
		]
	];
}

const float SNiagaraDistributionEditor::DefaultInputSize = 125.0f;

void  SNiagaraDistributionEditor::OnDistributionModeChanged()
{
	ContentBox->SetContent(ConstructContentForMode());
}

TSharedRef<SWidget> SNiagaraDistributionEditor::ConstructContentForMode()
{
	ENiagaraDistributionEditorMode Mode = DistributionAdapter->GetDistributionMode();
	if (FNiagaraDistributionEditorUtilities::IsArray(Mode))
	{
		return SNew(SNiagaraDistributionArrayEditor, DistributionAdapter.ToSharedRef());
	}
	else if (FNiagaraDistributionEditorUtilities::IsBinding(Mode))
	{
		return SNew(SNiagaraDistributionBindingEditor, DistributionAdapter.ToSharedRef());
	}
	else if (FNiagaraDistributionEditorUtilities::IsExpression(Mode))
	{
		return SNew(SNiagaraStatelessExpressionWidget, DistributionAdapter.ToSharedRef());
	}
	else if (FNiagaraDistributionEditorUtilities::IsConstant(Mode) || FNiagaraDistributionEditorUtilities::IsRange(Mode))
	{
		return SNew(SNiagaraDistributionValueEditor, DistributionAdapter.ToSharedRef());
	}
	else if (FNiagaraDistributionEditorUtilities::IsCurve(Mode))
	{
		TSharedRef<SVerticalBox> CurveBox = SNew(SVerticalBox);

		if (SNiagaraDistributionEditorPrivate::GUseCompactEditor)
		{
			int32 NumberOfChannels = FNiagaraDistributionEditorUtilities::IsUniform(Mode) ? 1 : DistributionAdapter->GetNumChannels();
			for (int32 ChannelIndex = 0; ChannelIndex < NumberOfChannels; ChannelIndex++)
			{
				CurveBox->AddSlot()
				.AutoHeight()
				.Padding(0, 0, 0, 3)
				[
					SNew(SNiagaraDistributionCompactCurveEditor, DistributionAdapter.ToSharedRef(), ChannelIndex)
					.CurveColor(DistributionAdapter->GetChannelColor(ChannelIndex))
				];
			}
		}
		else
		{
			CurveBox->AddSlot()
			.AutoHeight()
			.Padding(0, 0, 0, 3)
			[
				SNew(SNiagaraDistributionCurveEditor, DistributionAdapter.ToSharedRef())
			];
		}

		CurveBox->AddSlot()
		.AutoHeight()
		[
			SNew(SNiagaraDistributionLookupValueWidget, DistributionAdapter)
		];

		return CurveBox;
	}
	else if (FNiagaraDistributionEditorUtilities::IsGradient(Mode))
	{
		return SNew(SNiagaraDistributionGradientEditor, DistributionAdapter.ToSharedRef());
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

#undef LOCTEXT_NAMESPACE
