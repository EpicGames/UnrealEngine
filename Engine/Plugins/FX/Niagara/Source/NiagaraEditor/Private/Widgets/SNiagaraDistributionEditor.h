// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/INiagaraDistributionAdapter.h"
#include "Widgets/SCompoundWidget.h"

struct FSlateBrush;
struct FSlateIcon;
class INiagaraDistributionAdapter;
class SBox;

class SNiagaraDistributionEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDistributionEditor) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraDistributionAdapter> InDistributionAdapter);

	static const float DefaultInputSize;

private:
	void OnDistributionModeChanged();

	TSharedRef<SWidget> ConstructContentForMode();

private:
	TSharedPtr<INiagaraDistributionAdapter> DistributionAdapter;
	TSharedPtr<SBox> ContentBox;
};