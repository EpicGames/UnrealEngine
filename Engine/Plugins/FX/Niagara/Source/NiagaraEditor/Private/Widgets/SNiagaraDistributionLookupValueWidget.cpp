// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraDistributionLookupValueWidget.h"

#include "Widgets/INiagaraDistributionAdapter.h"
#include "Widgets/SNiagaraParameterName.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "SNiagaraDistributionLookupValueWidget"

void SNiagaraDistributionLookupValueWidget::Construct(const FArguments& InArgs, TSharedPtr<INiagaraDistributionAdapter> InDistributionAdapter)
{
	DistributionAdapter = InDistributionAdapter;
	if (DistributionAdapter == nullptr || !DistributionAdapter->AllowLookupValueMode())
	{
		return;
	}

	LookupValueModeEnum = DistributionAdapter->GetLookupValueModeEnum();
	if (LookupValueModeEnum == nullptr)
	{
		return;
	}

	TSharedRef<SHorizontalBox> BoxWidget = SNew(SHorizontalBox);
	if (InArgs._ShowValueOnly == false)
	{
		BoxWidget->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			.Text(LOCTEXT("LookupValueMode", "Lookup Value"))
		];
	}

	BoxWidget->AddSlot()
	.VAlign(VAlign_Center)
	[
		SNew(SComboButton)
		.OnGetMenuContent(this, &SNiagaraDistributionLookupValueWidget::OnGetLookupValueModeOptions)
		.ContentPadding(1)
		.ButtonContent()
		[
			SNew(SNiagaraParameterName)
			.ParameterName(this, &SNiagaraDistributionLookupValueWidget::GetLookupValueModeName)
			.IsReadOnly(true)
		]
	];

	ChildSlot
	[
		BoxWidget
	];
}

TSharedRef<SWidget> SNiagaraDistributionLookupValueWidget::OnGetLookupValueModeOptions()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.BeginSection("Actions");
	if (LookupValueModeEnum)
	{
		for (int64 iValue = 0; iValue < LookupValueModeEnum->GetMaxEnumValue(); ++iValue)
		{
			if (LookupValueModeEnum->IsValidEnumValue(iValue) == false)
			{
				continue;
			}

			const int64 iIndex = LookupValueModeEnum->GetIndexByValue(iValue);

			MenuBuilder.AddMenuEntry(
				LookupValueModeEnum->GetDisplayNameTextByIndex(iIndex),
				LookupValueModeEnum->GetToolTipTextByIndex(iIndex),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraDistributionLookupValueWidget::SetLookupValueMode, uint8(iValue))
				)
			);
		}
	}
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

FName SNiagaraDistributionLookupValueWidget::GetLookupValueModeName() const
{
	const uint8 Mode = DistributionAdapter->GetLookupValueMode();
	return FName(LookupValueModeEnum->GetDisplayNameTextByValue(Mode).ToString());
}

FText SNiagaraDistributionLookupValueWidget::GetLookupValueModeToolTip() const
{
	const uint8 Mode = DistributionAdapter->GetLookupValueMode();
	const int64 Index = LookupValueModeEnum->GetIndexByValue(Mode);
	return Index != INDEX_NONE ? LookupValueModeEnum->GetToolTipTextByIndex(Index) : FText();
}

void SNiagaraDistributionLookupValueWidget::SetLookupValueMode(uint8 Mode)
{
	DistributionAdapter->SetLookupValueMode(Mode);
}

#undef LOCTEXT_NAMESPACE
