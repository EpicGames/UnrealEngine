// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stateless/NiagaraStatelessDistribution.h"

#include "Misc/NotifyHook.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SCompoundWidget.h"

class INiagaraDistributionAdapter;

class SNiagaraDistributionLookupValueWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDistributionLookupValueWidget) { }
		SLATE_ARGUMENT_DEFAULT(bool, ShowValueOnly) = false;
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedPtr<INiagaraDistributionAdapter> InDistributionAdapter);

private:
	TSharedRef<SWidget> OnGetLookupValueModeOptions();

	FName GetLookupValueModeName() const;
	FText GetLookupValueModeToolTip() const;

	void SetLookupValueMode(uint8 Mode);

protected:
	TSharedPtr<INiagaraDistributionAdapter>	DistributionAdapter;
	const UEnum*							LookupValueModeEnum = nullptr;
};
