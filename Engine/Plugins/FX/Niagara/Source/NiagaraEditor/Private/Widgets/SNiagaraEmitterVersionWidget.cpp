// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraEmitterVersionWidget.h"

#include "AssetToolsModule.h"
#include "Editor.h"
#include "NiagaraEmitter.h"
#include "NiagaraVersionMetaData.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "NiagaraEmitterVersionWidget"

void SNiagaraEmitterVersionWidget::Construct(const FArguments& InArgs, UNiagaraEmitter* InEmitter, UNiagaraVersionMetaData* InMetadata, const FString& InBasePackageName)
{
	Emitter = InEmitter;
	SNiagaraVersionWidget::Construct(InArgs, InEmitter, InMetadata);
	BasePackageName = InBasePackageName;
}

FText SNiagaraEmitterVersionWidget::GetInfoHeaderText() const
{
	if (Emitter == nullptr)
	{
		return LOCTEXT("NiagaraManageVersionWrongAsset", "Versioning is not available for the selected asset type. Currently, Niagara only supports versioning for standalone emitter or script assets.\nYou probably see this window because you opened it in another asset and the editor remembers you choice of opened tabs.");
	}
	
	if (Emitter->IsVersioningEnabled())
	{
		return LOCTEXT("NiagaraManageVersionInfoHeader", "Here you can see and manage available emitter versions. Adding versions allows you to make changes without breaking existing child assets.\nThe exposed version is the one that users will see when adding the emitter to a system. If the exposed version has a higher major version than the one used in an asset, the user will see a prompt to upgrade to the new version.");	
	}
	return LOCTEXT("NiagaraManageVersionInfoHeaderDisabled", "Versioning is not yet enabled for this emitter. Enabling versioning allows you to make changes to a parent emitter without breaking exising usages from child emitters. But it also forces users to manually upgrade to new versions, as changes are no longer pushed out automatically.\n\nDo you want to enable versioning for this emitter? This will convert the current asset to version 1.0 and automatically exposes it.");
}

void SNiagaraEmitterVersionWidget::ExecuteSaveAsAssetAction(FNiagaraAssetVersion AssetVersion)
{
	FString Name;
	FString PackageName;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, TEXT(""), PackageName, Name);
	
	UNiagaraEmitter* NewAssetEmitter = Cast<UNiagaraEmitter>(AssetToolsModule.Get().DuplicateAssetWithDialogAndTitle(Name, FPackageName::GetLongPackagePath(PackageName), Emitter, LOCTEXT("SaveVersionAsAssetTitle", "Create Emitter As")));
	if (NewAssetEmitter != nullptr)
	{
		NewAssetEmitter->DisableVersioning(AssetVersion.VersionGuid);
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAssetEmitter);
	}
}

#undef LOCTEXT_NAMESPACE
