// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraExpandedToggle.h"

#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"

void SNiagaraExpandedToggle::Construct(const FArguments& InArgs)
{
	Expanded = InArgs._Expanded;
	OnExpandedChangedDelegate = InArgs._OnExpandedChanged;
	ChildSlot
	[
		SNew(SCheckBox)
		.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("TransparentCheckBox"))
		.IsChecked(this, &SNiagaraExpandedToggle::GetExpandedCheckBoxState)
		.OnCheckStateChanged(this, &SNiagaraExpandedToggle::ExpandedCheckBoxStateChanged)
		[
			SNew(SImage)
			.Image(this, &SNiagaraExpandedToggle::GetExpandedBrush)
		]
	];
}

ECheckBoxState SNiagaraExpandedToggle::GetExpandedCheckBoxState() const
{
	return Expanded.Get(false) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraExpandedToggle::ExpandedCheckBoxStateChanged(ECheckBoxState InCheckState)
{
	bool bExpanded = InCheckState == ECheckBoxState::Checked;
	if (Expanded.Get(false) != bExpanded)
	{
		if (Expanded.IsBound() == false)
		{
			Expanded = bExpanded;
		}
		ExpandedBrushCache.Reset();
		OnExpandedChangedDelegate.ExecuteIfBound(bExpanded);
	}
}

const FSlateBrush* SNiagaraExpandedToggle::GetExpandedBrush() const
{
	if (ExpandedBrushCache.IsSet() == false)
	{
		ExpandedBrushCache = Expanded.Get(false)
			? FCoreStyle::Get().GetBrush("TreeArrow_Expanded")
			: FCoreStyle::Get().GetBrush("TreeArrow_Collapsed");
	}
	return ExpandedBrushCache.GetValue();
}