// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"

class SNiagaraExpandedToggle : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnExpandedChanged, bool /* bExpanded */);

public:
	SLATE_BEGIN_ARGS(SNiagaraExpandedToggle)
		: _Expanded(false)
		{ }
		SLATE_ATTRIBUTE(bool, Expanded)
		SLATE_EVENT(FOnExpandedChanged, OnExpandedChanged)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

private:
	ECheckBoxState GetExpandedCheckBoxState() const;
	void ExpandedCheckBoxStateChanged(ECheckBoxState InCheckState);
	const FSlateBrush* GetExpandedBrush() const;

private:
	TAttribute<bool> Expanded;
	FOnExpandedChanged OnExpandedChangedDelegate;
	mutable TOptional<const FSlateBrush*> ExpandedBrushCache;
};