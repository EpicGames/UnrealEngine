// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraFilterBox.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "Styling/StyleColors.h"
#include "ViewModels/NiagaraCurveSelectionViewModel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "NiagaraFilter"

TMap<EScriptSource, bool> SNiagaraSourceFilterBox::SourceState
{
	{EScriptSource::Niagara, true},
	{EScriptSource::Game, true},
	{EScriptSource::Plugins, false},
	{EScriptSource::Developer, false},
};

TOptional<ENiagaraScriptTemplateSpecification> SNiagaraTemplateTabBox::CachedActiveTab;

void SNiagaraSourceFilterCheckBox::Construct(const FArguments& Args, EScriptSource InSource)
{
	Source = InSource;
	OnSourceStateChanged = Args._OnSourceStateChanged;
	OnShiftClicked = Args._OnShiftClicked;
	
	UEnum* ScriptSourceEnum = StaticEnum<EScriptSource>();
	FText DisplayName = ScriptSourceEnum->GetDisplayNameTextByValue((int64) Source);
	FText ToolTipText = LOCTEXT("SourceFilterToolTip", "Display actions from source: {0}.\nUse Shift+Click to exclusively select this filter.");
	ToolTipText = FText::Format(ToolTipText, DisplayName);
	
	SCheckBox::FArguments ParentArgs;
	ParentArgs
	.Style(FAppStyle::Get(), "ContentBrowser.FilterButton")
	.IsChecked(Args._IsChecked)
	.OnCheckStateChanged(FOnCheckStateChanged::CreateLambda([this](ECheckBoxState NewState)
    {
        OnSourceStateChanged.ExecuteIfBound(Source, NewState == ECheckBoxState::Checked ? true : false);
    }));
	SCheckBox::Construct(ParentArgs);

	SetToolTipText(ToolTipText);
	
	SetContent(
		SNew(SBorder)
		 .Padding(1.0f)
		.BorderImage(FNiagaraEditorStyle::Get().GetBrush("GraphActionMenu.FilterCheckBox.Border"))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("ContentBrowser.FilterImage"))
				.ColorAndOpacity(this, &SNiagaraSourceFilterCheckBox::GetScriptSourceColor)
			]
			+SHorizontalBox::Slot()
			.Padding(TAttribute<FMargin>(this, &SNiagaraSourceFilterCheckBox::GetFilterNamePadding))
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(DisplayName)
				.ColorAndOpacity_Raw(this, &SNiagaraSourceFilterCheckBox::GetTextColor)
				.TextStyle(FNiagaraEditorStyle::Get(), "GraphActionMenu.ActionFilterTextBlock")
			]
		]
    );
}

FReply SNiagaraSourceFilterCheckBox::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = SCheckBox::OnMouseButtonUp(MyGeometry, MouseEvent);
	
	if(FSlateApplication::Get().GetModifierKeys().IsShiftDown() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bool bExecuted = OnShiftClicked.ExecuteIfBound(Source, !IsChecked());
		return FReply::Handled().ReleaseMouseCapture();
	}

	return Reply;
}

FSlateColor SNiagaraSourceFilterCheckBox::GetTextColor() const
{
	const float DimFactor = 0.75f;
	return IsHovered() ? FLinearColor(DimFactor, DimFactor, DimFactor, 1.0f) : IsChecked() ? FLinearColor::White : FLinearColor::Gray;
}

FSlateColor SNiagaraSourceFilterCheckBox::GetScriptSourceColor() const
{
	FLinearColor ScriptSourceColor = FNiagaraEditorUtilities::GetScriptSourceColor(Source);
	if(!IsChecked())
	{
		ScriptSourceColor.A = 0.1f;
	}

	return ScriptSourceColor;
}

FMargin SNiagaraSourceFilterCheckBox::GetFilterNamePadding() const
{
	return bIsPressed ? FMargin(2,2,1,0) : FMargin(2,1,1,1);	
}

void SNiagaraSourceFilterBox::Construct(const FArguments& Args)
{
	OnFiltersChanged = Args._OnFiltersChanged;
	
	TSharedRef<SHorizontalBox> SourceContainer = SNew(SHorizontalBox);
    UEnum* ScriptSourceEnum = StaticEnum<EScriptSource>();

    // a hard coded "Show all" button
    SourceContainer->AddSlot()
	.AutoWidth()
    .Padding(5.f)
    [
        SNew(SBorder)
        .BorderImage(FNiagaraEditorStyle::Get().GetBrush("GraphActionMenu.FilterCheckBox.Border"))
        .ToolTipText(LOCTEXT("ShowAllToolTip", "Show all"))
        .Padding(3.f)
        [
            SNew(SCheckBox)
            .Style(FNiagaraEditorStyle::Get(), "GraphActionMenu.FilterCheckBox")
            .BorderBackgroundColor_Lambda([=]() -> FSlateColor
            {
                bool bChecked = true;
                for(int32 SourceIndex = 0; SourceIndex < (int32) EScriptSource::Unknown; SourceIndex++)
                {
                    bChecked &= SourceState[(EScriptSource) ScriptSourceEnum->GetValueByIndex(SourceIndex)];
                }
    
                return bChecked ? FLinearColor::White : FSlateColor::UseForeground();
            })  
            .IsChecked_Lambda([=]() -> ECheckBoxState
            {
                bool bChecked = true;
                for(int32 SourceIndex = 0; SourceIndex < (int32) EScriptSource::Unknown; SourceIndex++)
                {
                    bChecked &= SourceState[(EScriptSource) ScriptSourceEnum->GetValueByIndex(SourceIndex)];
                }
    
                return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
            })
            .OnCheckStateChanged(FOnCheckStateChanged::CreateLambda([this, ScriptSourceEnum](ECheckBoxState NewState)
            {
            	bool bAnyChange = false;
            	// we always want to "Show all" so we always set the source filters to true
            	for(int32 SourceIndex = 0; SourceIndex < (int32) EScriptSource::Unknown; SourceIndex++)
            	{
            		// we are assuming the map has a key value pair for every enum entry (aside from Unknown) 
					if(SourceState[(EScriptSource) ScriptSourceEnum->GetValueByIndex(SourceIndex)] == false)
					{
						bAnyChange = true;
					}
				}

            	if(bAnyChange)
            	{
	                for(int32 SourceIndex = 0; SourceIndex < (int32) EScriptSource::Unknown; SourceIndex++)
	                {
	                    SourceState.Add((EScriptSource) ScriptSourceEnum->GetValueByIndex(SourceIndex), true);
	                }
	                
	                BroadcastFiltersChanged();
            	}
            }))
            [
                SNew(SHorizontalBox)
                + SHorizontalBox::Slot()
                .HAlign(HAlign_Center)
                .VAlign(VAlign_Center)
                .Padding(2.f)
                [
                    SNew(STextBlock)
                    .Text(LOCTEXT("ShowAll", "Show all"))
                    .ShadowOffset(0.0f)
                    .ColorAndOpacity_Lambda([=]() -> FSlateColor
                    {
                        bool bChecked = true;
                        for(int32 SourceIndex = 0; SourceIndex < (int32) EScriptSource::Unknown; SourceIndex++)
                        {
                            bChecked &= SourceState[(EScriptSource) ScriptSourceEnum->GetValueByIndex(SourceIndex)];
                        }
                        return bChecked ? FLinearColor::Black : FLinearColor::Gray;
                    })
                    .TextStyle(FNiagaraEditorStyle::Get(), "GraphActionMenu.ActionFilterTextBlock")
                ]
            ]	
        ]
    ];

    // create a button for every source option
    for(int32 SourceIndex = 0; SourceIndex < (int32) EScriptSource::Unknown; SourceIndex++)
    {
        SourceContainer->AddSlot()
    	.AutoWidth()
    	.Padding(2.f)
    	[
    		SNew(SBorder)
    		.BorderImage(FAppStyle::GetBrush(TEXT("NoBorder")))
			.Padding(3.f)
    		[
    			SNew(SNiagaraSourceFilterCheckBox, (EScriptSource) ScriptSourceEnum->GetValueByIndex(SourceIndex))
    			.IsChecked_Lambda([=]()
    			{
    				return SourceState[(EScriptSource)ScriptSourceEnum->GetValueByIndex(SourceIndex)] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
    			})
    			.OnSourceStateChanged_Lambda([this](EScriptSource Source, bool bState)
    			{
    				SourceState.Add(Source, bState);
    				BroadcastFiltersChanged();
    			})
    			.OnShiftClicked_Lambda([this](EScriptSource ChangedSource, bool bState)
    			{
    				TArray<EScriptSource> Keys;
    				SourceState.GenerateKeyArray(Keys);
				
    				for(EScriptSource& Source : Keys)
    				{
    					if(Source == ChangedSource)
    					{
    						SourceState.Add(Source, true);
    					}
    					else
    					{
    						SourceState.Add(Source, false);
    					}
    				}

    				BroadcastFiltersChanged();
    			})
    		]
    	];
    }

	ChildSlot
    [
        SourceContainer
    ];
}

bool SNiagaraSourceFilterBox::IsFilterActive(EScriptSource Source) const
{
	if(SourceState.Contains(Source))
	{
		return SourceState[Source];
	}

	return true;
}

ECheckBoxState SNiagaraSourceFilterBox::OnIsFilterActive(EScriptSource Source) const
{
	return IsFilterActive(Source) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SNiagaraTemplateTabBox::Construct(const FArguments& InArgs, FNiagaraTemplateTabOptions InTabOptions)
{
	TabOptions = InTabOptions;

	OnTabActivatedDelegate = InArgs._OnTabActivated;
	Class = InArgs._Class;
	
	TabContainer = SNew(SSegmentedControl<ENiagaraScriptTemplateSpecification>)
	.Value(TAttribute<ENiagaraScriptTemplateSpecification>::CreateSP(this, &SNiagaraTemplateTabBox::GetActiveTab))
	.OnValueChanged(this, &SNiagaraTemplateTabBox::OnTabActivated);

	checkf(TabOptions.GetNumAvailableTabs() >= 1, TEXT("At least one tab option needs to be set to true."));
	
	// we restore the cached active tab, if it is available in the current context
	if(CachedActiveTab.IsSet() && TabOptions.IsTabAvailable(CachedActiveTab.GetValue()))
	{
		ActiveTab = CachedActiveTab;
	}	
	
	if(TabOptions.IsTabAvailable(ENiagaraScriptTemplateSpecification::Template))
	{
		// we set the currently active tab using the first available tab in a set order
		if(!ActiveTab.IsSet())
		{
			ActiveTab = ENiagaraScriptTemplateSpecification::Template;
		}
		
		TabContainer->AddSlot(ENiagaraScriptTemplateSpecification::Template, false)
		.Text(DetermineControlLabel(ENiagaraScriptTemplateSpecification::Template))
		.ToolTip(DetermineControlTooltip(ENiagaraScriptTemplateSpecification::Template));		
	}

	if(TabOptions.IsTabAvailable(ENiagaraScriptTemplateSpecification::None))
	{
		if(!ActiveTab.IsSet())
		{
			ActiveTab = ENiagaraScriptTemplateSpecification::None;
		}		
		
		TabContainer->AddSlot(ENiagaraScriptTemplateSpecification::None, false)
		.Text(DetermineControlLabel(ENiagaraScriptTemplateSpecification::None))
		.ToolTip(DetermineControlTooltip(ENiagaraScriptTemplateSpecification::None));
	}

	if(TabOptions.IsTabAvailable(ENiagaraScriptTemplateSpecification::Behavior))
	{
		if(!ActiveTab.IsSet())
		{
			ActiveTab = ENiagaraScriptTemplateSpecification::Behavior;
		}

		TabContainer->AddSlot(ENiagaraScriptTemplateSpecification::Behavior, false)
		.Text(DetermineControlLabel(ENiagaraScriptTemplateSpecification::Behavior))
		.ToolTip(DetermineControlTooltip(ENiagaraScriptTemplateSpecification::Behavior));
	}

	TabContainer->RebuildChildren();
	// we cache the active tab if we have more than 1 tab available so we can activate it for other instances as it was intentionally navigated to
	if(TabOptions.GetNumAvailableTabs() > 1 && ActiveTab.IsSet())
	{
		CachedActiveTab = ActiveTab.GetValue();	
	}

	ChildSlot
	[
		TabContainer.ToSharedRef()
	];
}

ENiagaraScriptTemplateSpecification SNiagaraTemplateTabBox::GetActiveTab() const
{
	if(ActiveTab.IsSet())
	{
		return ActiveTab.GetValue();
	}
	
	return ENiagaraScriptTemplateSpecification::None;	
}

bool SNiagaraTemplateTabBox::GetActiveTab(ENiagaraScriptTemplateSpecification& OutTemplateSpecification) const
{
	if(ActiveTab.IsSet())
	{
		OutTemplateSpecification = ActiveTab.GetValue();
		return true;
	}	

	return false;
}

void SNiagaraTemplateTabBox::OnTabActivated(ENiagaraScriptTemplateSpecification AssetTab)
{
	if(ActiveTab != AssetTab)
	{
		ActiveTab = AssetTab;
		CachedActiveTab = ActiveTab;
		OnTabActivatedDelegate.ExecuteIfBound(AssetTab);
	}
}

void SNiagaraTemplateTabBox::OnTabActivated(ECheckBoxState NewState, ENiagaraScriptTemplateSpecification AssetTab)
{
	if(ActiveTab != AssetTab)
	{
		ActiveTab = AssetTab;
		CachedActiveTab = ActiveTab;
		OnTabActivatedDelegate.ExecuteIfBound(AssetTab);
	}
}

FSlateColor SNiagaraTemplateTabBox::GetBackgroundColor(ENiagaraScriptTemplateSpecification TemplateSpecification) const
{
	if(ActiveTab == TemplateSpecification)
	{
		return FCoreStyle::Get().GetSlateColor("SelectionColor");
	}

	return FLinearColor::Transparent;
}

FSlateColor SNiagaraTemplateTabBox::GetTabForegroundColor(ENiagaraScriptTemplateSpecification TemplateSpecification) const
{
	if(ActiveTab == TemplateSpecification)
	{
		return FLinearColor::Black;
	}

	return FLinearColor::White;
}

FText SNiagaraTemplateTabBox::DetermineControlLabel(ENiagaraScriptTemplateSpecification TemplateSpecification) const
{
	if(Class.IsValid())
	{
		if(Class == UNiagaraEmitter::StaticClass())
		{
			switch(TemplateSpecification)
			{
			case ENiagaraScriptTemplateSpecification::None:
				return LOCTEXT("EmitterNoneTabLabel", "Parent Emitters");
			case ENiagaraScriptTemplateSpecification::Template:
				return LOCTEXT("EmitterTemplateTabLabel", "Templates");
			case ENiagaraScriptTemplateSpecification::Behavior:
				return LOCTEXT("EmitterLearningExampleTabLabel", "Learning Examples");
			default:
				return FText::GetEmpty();
			}
		}
		else if(Class == UNiagaraSystem::StaticClass())
		{
			switch(TemplateSpecification)
			{
			case ENiagaraScriptTemplateSpecification::None:
				return LOCTEXT("SystemNoneTabLabel", "Standard");
			case ENiagaraScriptTemplateSpecification::Template:
				return LOCTEXT("SystemTemplateTabLabel", "Templates");
			case ENiagaraScriptTemplateSpecification::Behavior:
				return LOCTEXT("SystemLearningExampleTabLabel", "Learning Examples");
			default:
				return FText::GetEmpty();
			}
		}				
	}

	check(false);
	return FText::GetEmpty();
}

FText SNiagaraTemplateTabBox::DetermineControlTooltip(ENiagaraScriptTemplateSpecification TemplateSpecification) const
{
	if(Class.IsValid())
	{
		if(Class == UNiagaraEmitter::StaticClass())
		{
			switch(TemplateSpecification)
			{
			case ENiagaraScriptTemplateSpecification::None:
				return LOCTEXT("EmitterNoneTabTooltip", "Parent Emitters are inherited as children and will receive changes from the parent emitter,\n"
					"and are meant to serve as art directed initial behaviors which can be propagated throughout a project quickly and easily.\n"
					"Over time, a library of parent emitters can be used to speed up the construction of complex effects specific to your project.");
			case ENiagaraScriptTemplateSpecification::Template:
				return LOCTEXT("EmitterTemplateTabTooltip", "Templates are intended as starting points for building functional emitters of different types,\n"
			"Emitter templates are copied into a system as a unique emitter with no inheritance");
			case ENiagaraScriptTemplateSpecification::Behavior:
				return LOCTEXT("EmitterBehaviorExampleTabTooltip", "Learning Examples are intended to serve as a guide to how Niagara works at a feature level.\n"
			"Each example shows a simplified setup used to achieve specific outcomes and are intended as starting points, building blocks, or simply as reference.\n"
			"Learning Example emitters are copied into a system as a unique emitter with no inheritance");
			default:
				return FText::GetEmpty();
			}
		}
		else if(Class == UNiagaraSystem::StaticClass())
		{
			switch(TemplateSpecification)
			{
			case ENiagaraScriptTemplateSpecification::None:
				return LOCTEXT("SystemNoneTabTooltip", "By default, any system will be categorized as a standard system.");
			case ENiagaraScriptTemplateSpecification::Template:
				return LOCTEXT("SystemTemplateTabTooltip", "Template systems ");
			case ENiagaraScriptTemplateSpecification::Behavior:
				return LOCTEXT("SystemBehaviorExampleTabTooltip", "Learning Example systems serve to demonstrate different Niagara features that rely on multiple emitters interacting in some way.");
			default:
				return FText::GetEmpty();				
			}
		}	
	}

	check(false);
	return FText::GetEmpty();
}

bool SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions::IsTabAvailable(ENiagaraScriptTemplateSpecification AssetTab) const
{
	if(TabData.Contains(AssetTab))
	{
		return TabData[AssetTab];
	}

	return false;
}

int32 SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions::GetNumAvailableTabs() const
{
	TArray<bool> StateArray;
	TabData.GenerateValueArray(StateArray);

	int32 NumAvailableTabs = 0;
	for(const bool& State : StateArray)
	{
		if(State)
		{
			NumAvailableTabs++;
		}
	}

	return NumAvailableTabs;
}

bool SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions::GetOnlyAvailableTab(ENiagaraScriptTemplateSpecification& OutTab) const
{
	int32 ActiveCount = 0;
	ENiagaraScriptTemplateSpecification Tab = ENiagaraScriptTemplateSpecification::None;
	for(const auto& TabEntry : TabData)
	{
		if(TabEntry.Value == true)
		{
			ActiveCount++;
			Tab = TabEntry.Key;
		}
	}

	if(ActiveCount == 1)
	{
		OutTab = Tab;
		return true;
	}

	return false;
}

bool SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions::GetOnlyShowTemplates() const
{
	ENiagaraScriptTemplateSpecification Tab;
	bool bFound = GetOnlyAvailableTab(Tab);
	return bFound && Tab == ENiagaraScriptTemplateSpecification::Template;
}

const TMap<ENiagaraScriptTemplateSpecification, bool>& SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions::GetTabData() const
{
	return TabData;
}

bool SNiagaraFilterBox::FFilterOptions::IsAnyFilterActive()
{
	return bAddLibraryFilter || bAddSourceFilter || (bAddTemplateFilter && TabOptions.GetNumAvailableTabs() > 0);
}

void SNiagaraFilterBox::FFilterOptions::SetAddSourceFilter(bool bInAddSourceFilter)
{
	this->bAddSourceFilter = bInAddSourceFilter;
}

void SNiagaraFilterBox::FFilterOptions::SetAddLibraryFilter(bool bInAddLibraryFilter)
{
	this->bAddLibraryFilter = bInAddLibraryFilter;
}

void SNiagaraFilterBox::FFilterOptions::SetAddTemplateFilter(bool bInAddTemplateFilter)
{
	this->bAddTemplateFilter = bInAddTemplateFilter;
}

bool SNiagaraFilterBox::FFilterOptions::GetAddSourceFilter() const
{
	return bAddSourceFilter;
}

bool SNiagaraFilterBox::FFilterOptions::GetAddLibraryFilter() const
{
	return bAddLibraryFilter;
}

bool SNiagaraFilterBox::FFilterOptions::GetAddTemplateFilter() const
{
	return bAddTemplateFilter;
}

SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions SNiagaraFilterBox::FFilterOptions::GetTabOptions() const
{
	return TabOptions;
}

void SNiagaraFilterBox::FFilterOptions::SetTabOptions(const SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions& InTabOptions)
{
	this->TabOptions = InTabOptions;
}

void SNiagaraFilterBox::Construct(const FArguments& InArgs, FFilterOptions InFilterOptions)
{
	TSharedPtr<SVerticalBox> MainContainer = SNew(SVerticalBox);

	TSharedPtr<SHorizontalBox> FirstRowContainer = SNew(SHorizontalBox);
	
	if(InFilterOptions.GetAddSourceFilter())
	{
		SAssignNew(SourceFilterBox, SNiagaraSourceFilterBox)
		.OnFiltersChanged(InArgs._OnSourceFiltersChanged);
		
		FirstRowContainer->AddSlot()
		.AutoWidth()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3, 1)
			[		
				SNew(STextBlock)
				.Text(LOCTEXT("ActionSourceLabel", "Source Filtering"))
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3, 1)
			[				
				SourceFilterBox.ToSharedRef()
			]
		];
	}

	if(InFilterOptions.GetAddLibraryFilter())
	{
		SAssignNew(LibraryOnlyToggleHeader, SNiagaraLibraryOnlyToggleHeader)
		.bShowHeaderLabel(false)
		.LibraryOnly(InArgs._bLibraryOnly)
		.LibraryOnlyChanged(InArgs._OnLibraryOnlyChanged);

		FirstRowContainer->AddSlot()
		.MaxWidth(150.f)
		[				
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5, 1)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LibraryVisiblityLabel", "Library Filtering"))
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.Padding(5, 1)
			[
				LibraryOnlyToggleHeader.ToSharedRef()
			]     
		];
	}

	MainContainer->AddSlot()
	[
		FirstRowContainer.ToSharedRef()	
	];

	if(InFilterOptions.GetAddTemplateFilter())
	{
		ensureMsgf(InArgs._Class != nullptr, TEXT("If the template filter is added, the class must be valid as depending on class the control labels and tooltips are generated."));
		
		TemplateTabBox = SNew(SNiagaraTemplateTabBox, InFilterOptions.GetTabOptions())
		.OnTabActivated(InArgs._OnTabActivated)
		.Class(InArgs._Class);
		
		// the template filter is a whole-row filter
		MainContainer->AddSlot()
		.AutoHeight()
		[
			// we only show the filter if we have more than 1 tab available
			SNew(SVerticalBox)
			.Visibility(InFilterOptions.GetTabOptions().GetNumAvailableTabs() > 1 ? EVisibility::Visible : EVisibility::Collapsed)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3, 1)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TemplateTabFilterLabel", "Asset Type Filtering"))
			]
			+ SVerticalBox::Slot()
			.Padding(3, 1)
			[
				TemplateTabBox.ToSharedRef()	
			]     
		];
	}
	
	ChildSlot
	[
		MainContainer.ToSharedRef()
	];
}

bool SNiagaraFilterBox::IsSourceFilterActive(EScriptSource Source) const
{
	bool bSuccess = false;

	if(SourceFilterBox.IsValid())
	{
		bSuccess = SourceFilterBox->IsFilterActive(Source);
	}

	return bSuccess;
}

bool SNiagaraFilterBox::GetActiveTemplateTab(ENiagaraScriptTemplateSpecification& OutTemplateSpecification) const
{
	bool bSuccess = false;

	if(TemplateTabBox.IsValid())
	{
		bSuccess = TemplateTabBox->GetActiveTab(OutTemplateSpecification);
	}

	return bSuccess;
}

#undef LOCTEXT_NAMESPACE
