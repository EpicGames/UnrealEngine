// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGeneratedCodeView.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ISequencer.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraEmitter.h"
#include "NiagaraEmitterHandle.h"
#include "NiagaraScript.h"
#include "NiagaraSettings.h"
#include "NiagaraSystem.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemScriptViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/NiagaraHLSLSyntaxHighlighter.h"

#define LOCTEXT_NAMESPACE "NiagaraGeneratedCodeView"

void SNiagaraGeneratedCodeView::Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel, TSharedPtr<SDockTab> InOwnerTab)
{
	OwnerTab = InOwnerTab;

	TabState = 0;
	ScriptEnum = StaticEnum<ENiagaraScriptUsage>();
	ensure(ScriptEnum);
	SyntaxHighlighter = FNiagaraHLSLSyntaxHighlighter::Create();

	SystemViewModel = InSystemViewModel;
	SystemViewModel->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().AddSP(this, &SNiagaraGeneratedCodeView::SystemSelectionChanged);
	SystemViewModel->GetSystemScriptViewModel()->OnSystemCompiled().AddRaw(this, &SNiagaraGeneratedCodeView::OnCodeCompiled);

	TSharedRef<SWidget> HeaderContentsFirstLine = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.OnClicked(this, &SNiagaraGeneratedCodeView::OnCopyPressed)
			.Text(LOCTEXT("CopyOutput", "Copy"))
			.ToolTipText(LOCTEXT("CopyOutputToolitp", "Press this button to put the contents of this tab in the clipboard."))
		]
		+ SHorizontalBox::Slot()
		[
			SNullWidget::NullWidget
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(2, 4, 2, 4)
		[
			SAssignNew(SearchBox, SSearchBox)
			.OnTextCommitted(this, &SNiagaraGeneratedCodeView::OnSearchTextCommitted)
			.HintText(NSLOCTEXT("SearchBox", "HelpHint", "Search For Text"))
			.OnTextChanged(this, &SNiagaraGeneratedCodeView::OnSearchTextChanged)
			.SelectAllTextWhenFocused(false)
			.DelayChangeNotificationsWhileTyping(true)
			.MinDesiredWidth(200)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2, 4, 2, 4)
		[
			SAssignNew(SearchFoundMOfNText, STextBlock)
			.MinDesiredWidth(25)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 4, 2, 4)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.IsFocusable(false)
			.ToolTipText(LOCTEXT("UpToolTip", "Focus to previous found search term"))
			.OnClicked(this, &SNiagaraGeneratedCodeView::SearchUpClicked)
			.Content()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.Text(FText::FromString(FString(TEXT("\xf062"))))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 4, 2, 4)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.IsFocusable(false)
			.ToolTipText(LOCTEXT("DownToolTip", "Focus to next found search term"))
			.OnClicked(this, &SNiagaraGeneratedCodeView::SearchDownClicked)
			.Content()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.Text(FText::FromString(FString(TEXT("\xf063"))))
			]
		];

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight() // Header block
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					HeaderContentsFirstLine						
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(ScriptNameContainer, SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2, 4, 2, 4)
					[
						SAssignNew(ScriptNameCombo, SComboButton)
						.OnGetMenuContent(this, &SNiagaraGeneratedCodeView::MakeScriptMenu)
						.ComboButtonStyle(FAppStyle::Get(), "GenericFilters.ComboButtonStyle")
						.ForegroundColor(FLinearColor::White)
						.ContentPadding(0)
						.ToolTipText(LOCTEXT("ScriptsToolTip", "Select a script to view below."))
						.HasDownArrow(true)
						.ContentPadding(FMargin(1, 0))
						.ButtonContent()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(2, 0, 0, 0)
								[
									SNew(STextBlock)
									.TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
									.Text(LOCTEXT("Scripts", "Scripts"))
								]
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(20, 4, 2, 4)
					[
						SNew(STextBlock)
						.MinDesiredWidth(25)
						.Text(this, &SNiagaraGeneratedCodeView::GetCurrentScriptNameText)
					]
					// We'll insert here on UI updating..
				]
				+ SVerticalBox::Slot()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoDataText", "Generated code not available.  Either data was pulled from DDC (do a Full Rebuild of the System) or the script failed to be generated."))
					.Visibility_Lambda([&]() {
						if (TabHasScriptData())
							return EVisibility::Collapsed;
						return EVisibility::Visible;
					})
				]
			]
		]
		+ SVerticalBox::Slot() // Text body block
		[
			SAssignNew(TextBodyContainer, SVerticalBox)
		]
	]; // this->ChildSlot
	UpdateUI();
}

void SNiagaraGeneratedCodeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	//If our parent tab is in the foreground then do the expensive slate UI update if needed.
	if (OwnerTab && OwnerTab->IsForeground() && bUIUpdatePending)
	{
		UpdateUI_Internal();
	}
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

FText SNiagaraGeneratedCodeView::GetCurrentScriptNameText() const
{
	if (TabState < (uint32)GeneratedCode.Num())
	{
		return GeneratedCode[TabState].UsageName;
	}
	else
	{
		return FText::GetEmpty();
	}
}

TSharedRef<SWidget> SNiagaraGeneratedCodeView::MakeScriptMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (int32 i = 0; i < GeneratedCode.Num(); i++)
	{
		MenuBuilder.AddMenuEntry(
			GeneratedCode[i].UsageName,
			FText::Format(LOCTEXT("MakeScriptMenuTooltip", "View {0}"), GeneratedCode[i].UsageName),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraGeneratedCodeView::OnTabChanged, (uint32)i)));
	}

	return MenuBuilder.MakeWidget();
}

FReply SNiagaraGeneratedCodeView::SearchDownClicked()
{
	if (ActiveFoundTextEntries.Num() > 0)
	{
		CurrentFoundTextEntry++;
		if (CurrentFoundTextEntry == ActiveFoundTextEntries.Num())
		{
			CurrentFoundTextEntry = 0;
		}
	}
	GeneratedCode[TabState].Text->AdvanceSearch(false);

	SetSearchMofN();

	return FReply::Handled();
}

FReply SNiagaraGeneratedCodeView::SearchUpClicked()
{
	if (ActiveFoundTextEntries.Num() > 0)
	{
		CurrentFoundTextEntry--;
		if (CurrentFoundTextEntry < 0)
		{
			CurrentFoundTextEntry = ActiveFoundTextEntries.Num() - 1;
		}
	}
	GeneratedCode[TabState].Text->AdvanceSearch(true);
	
	SetSearchMofN();

	return FReply::Handled();
}

FReply SNiagaraGeneratedCodeView::OnCopyPressed()
{
	if (TabState < (uint32)GeneratedCode.Num())
	{
		FPlatformApplicationMisc::ClipboardCopy(*GeneratedCode[TabState].Hlsl.ToString());
	}
	return FReply::Handled();
}

void SNiagaraGeneratedCodeView::OnSearchTextChanged(const FText& InFilterText)
{
	DoSearch(InFilterText);
}

void SNiagaraGeneratedCodeView::DoSearch(const FText& InFilterText)
{
	const FText OldText = GeneratedCode[TabState].Text->GetSearchText();
	GeneratedCode[TabState].Text->SetSearchText(InFilterText);
	GeneratedCode[TabState].Text->BeginSearch(InFilterText, ESearchCase::IgnoreCase, false);

	FString SearchString = InFilterText.ToString();
	ActiveFoundTextEntries.Empty();
	if (SearchString.IsEmpty())
	{
		SetSearchMofN();
		return;
	}

	ActiveFoundTextEntries.Empty();
	for (int32 i = 0; i < GeneratedCode[TabState].HlslByLines.Num(); i++)
	{
		const FString& Line = GeneratedCode[TabState].HlslByLines[i];
		int32 FoundPos = Line.Find(SearchString, ESearchCase::IgnoreCase);
		while (FoundPos != INDEX_NONE && ActiveFoundTextEntries.Num() < 1000) // guard against a runaway loop
		{
			ActiveFoundTextEntries.Add(FTextLocation(i, FoundPos));
			int32 LastPos = FoundPos + SearchString.Len();
			if (LastPos < Line.Len())
			{
				FoundPos = Line.Find(SearchString, ESearchCase::IgnoreCase, ESearchDir::FromStart, LastPos);
			}
			else
			{
				FoundPos = INDEX_NONE;
			}
		}
	}

	if (ActiveFoundTextEntries.Num() > 0 && OldText.CompareTo(InFilterText) != 0)
	{
		CurrentFoundTextEntry = 0;
		//GeneratedCode[TabState].Text->ScrollTo(ActiveFoundTextEntries[CurrentFoundTextEntry]);
	}
	else if (ActiveFoundTextEntries.Num() == 0)
	{
		CurrentFoundTextEntry = INDEX_NONE;
	}

	SetSearchMofN();
}

void SNiagaraGeneratedCodeView::SetSearchMofN()
{
	SearchFoundMOfNText->SetText(FText::Format(LOCTEXT("MOfN", "{0} of {1}"), FText::AsNumber(CurrentFoundTextEntry + 1), FText::AsNumber(ActiveFoundTextEntries.Num())));
	//SearchFoundMOfNText->SetText(FText::Format(LOCTEXT("MOfN", "{1} found"), FText::AsNumber(CurrentFoundTextEntry), FText::AsNumber(ActiveFoundTextEntries.Num())));
}

void SNiagaraGeneratedCodeView::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType)
{
	OnSearchTextChanged(InFilterText);
	if (ActiveFoundTextEntries.Num() > 0)
	{
		CurrentFoundTextEntry++;
		if (CurrentFoundTextEntry == ActiveFoundTextEntries.Num())
		{
			CurrentFoundTextEntry = 0;
		}
	}

	GeneratedCode[TabState].Text->AdvanceSearch(true);

	SetSearchMofN();
}

void SNiagaraGeneratedCodeView::OnCodeCompiled()
{
	UpdateUI();
}

void SNiagaraGeneratedCodeView::SystemSelectionChanged()
{
	UpdateUI();
}

void SNiagaraGeneratedCodeView::UpdateUI()
{
	enum EScriptDisplayType
	{
		HLSL_SCRIPT_TYPE,
		GPU_SCRIPT_TYPE,
		ASSEMBLY_SCRIPT_TYPE,
		EVVM_ASSEMBLY_SCRIPT_TYPE,
	};

	//Mark the UI as needing an update. We now do the very expensive slate updates from Tick() iside UIUpdate_Internal.
	//This way we can avoid refreshing this UI constantly when this tab isn't even in view.
	bUIUpdatePending = true;

	TArray<UNiagaraScript*> Scripts;
	TArray<EScriptDisplayType> ScriptDisplayTypes;
	UNiagaraSystem& System = SystemViewModel->GetSystem();
	Scripts.Add(System.GetSystemSpawnScript());
	Scripts.Add(System.GetSystemUpdateScript());

	TArray<FGuid> SelectedEmitterHandleIds = SystemViewModel->GetSelectionViewModel()->GetSelectedEmitterHandleIds();
	if(SelectedEmitterHandleIds.Num() == 1)
	{
		TSharedPtr<FNiagaraEmitterHandleViewModel> SelectedEmitterHandleViewModel = SystemViewModel->GetEmitterHandleViewModelById(SelectedEmitterHandleIds[0]);

		FNiagaraEmitterHandle* Handle = SelectedEmitterHandleViewModel.IsValid() ? SelectedEmitterHandleViewModel->GetEmitterHandle() : nullptr;
		const FVersionedNiagaraEmitterData* EmitterData = Handle ? Handle->GetEmitterData() : nullptr;
		if (EmitterData)
		{
			TArray<UNiagaraScript*> EmitterScripts;
			EmitterData->GetScripts(EmitterScripts);
			Scripts.Append(EmitterScripts);
		}
	}

	// Mark the scripts with the correct display type and copy references for the non-gpu scripts for the assembly view.
	int32 OriginalScriptCount = Scripts.Num();
	ScriptDisplayTypes.AddUninitialized(OriginalScriptCount);
	for (int32 i = 0; i < OriginalScriptCount; i++)
	{
		UNiagaraScript* Script = Scripts[i];
		if (Script->GetUsage() == ENiagaraScriptUsage::ParticleGPUComputeScript)
		{
			ScriptDisplayTypes[i] = GPU_SCRIPT_TYPE;
		}
		else
		{
			ScriptDisplayTypes[i] = HLSL_SCRIPT_TYPE;

			Scripts.Add(Script);
			ScriptDisplayTypes.Add(ASSEMBLY_SCRIPT_TYPE);

			Scripts.Add(Script);
			ScriptDisplayTypes.Add(EVVM_ASSEMBLY_SCRIPT_TYPE);
		}
	}
		
	GeneratedCode.SetNum(Scripts.Num());

	if (TabState >= (uint32)GeneratedCode.Num())
	{
		TabState = 0;
	}

	TextBodyContainer->ClearChildren();

	for (int32 i = 0; i < GeneratedCode.Num(); i++)
	{
		GeneratedCode[i].Hlsl = FText::GetEmpty();

		if (Scripts[i] != nullptr)
		{
			const FNiagaraVMExecutableData& ExeData = Scripts[i]->GetVMExecutableData();

			GeneratedCode[i].Usage = Scripts[i]->Usage;
			GeneratedCode[i].UsageId = Scripts[i]->GetUsageId();

			if (ExeData.IsValid())
			{
				const FString* SourceText = nullptr;
				bool bAddLineNumbers = false;

				FText AssemblyIdText = FText::GetEmpty();

				switch (ScriptDisplayTypes[i])
				{
					// GPU combined spawn / update script
					case GPU_SCRIPT_TYPE:
						SourceText = &ExeData.LastHlslTranslationGPU;
						bAddLineNumbers = true;
						break;

					case ASSEMBLY_SCRIPT_TYPE:
						SourceText = &ExeData.LastAssemblyTranslation;
						AssemblyIdText = LOCTEXT("IsAssembly", "Assembly");
						break;

					case EVVM_ASSEMBLY_SCRIPT_TYPE:
						SourceText = &ExeData.LastExperimentalAssemblyScript;
						AssemblyIdText = LOCTEXT("IsEVVMAssembly", "EVVM Assembly");
						break;

					default:
						SourceText = &ExeData.LastHlslTranslation;
						bAddLineNumbers = true;
						break;
				}

				if (Scripts[i]->Usage == ENiagaraScriptUsage::ParticleEventScript)
				{
					FText EventName;
					if (FNiagaraEditorUtilities::TryGetEventDisplayName(Scripts[i]->GetTypedOuter<UNiagaraEmitter>(), Scripts[i]->GetUsageId(), EventName) == false)
					{
						EventName = NSLOCTEXT("NiagaraNodeOutput", "UnknownEventName", "Unknown");
					}
					GeneratedCode[i].UsageName = FText::Format(LOCTEXT("UsageNameEvent", "{0}-{1}{2}"), ScriptEnum->GetDisplayNameTextByValue((int64)Scripts[i]->Usage), EventName, AssemblyIdText);
				}
				// GPU combined spawn / update script
				else if (ScriptDisplayTypes[i] == GPU_SCRIPT_TYPE && i == GeneratedCode.Num() - 1 && Scripts[i]->IsParticleSpawnScript())
				{
					GeneratedCode[i].UsageName = LOCTEXT("UsageNameGPU", "GPU Spawn/Update");
				}
				else
				{
					GeneratedCode[i].UsageName = FText::Format(LOCTEXT("UsageName", "{0}{1}"), ScriptEnum->GetDisplayNameTextByValue((int64)Scripts[i]->Usage), AssemblyIdText);
				}

				if (SourceText && SourceText->Len() > 0)
				{
					if (bAddLineNumbers)
					{
						FString SumString;
						TArray<FString> OutputByLines;

						SourceText->ParseIntoArrayLines(OutputByLines, false);

						SumString.Reserve(SourceText->Len() + OutputByLines.Num() * 10);
						GeneratedCode[i].HlslByLines.Reset(OutputByLines.Num());

						for (int32 k = 0; k < OutputByLines.Num(); k++)
						{
							GeneratedCode[i].HlslByLines.Add(FString::Printf(TEXT("/*%04d*/\t\t%s\r\n"), k, *OutputByLines[k]));
							SumString.Append(GeneratedCode[i].HlslByLines[k]);
						}
						GeneratedCode[i].Hlsl = FText::FromString(SumString);
					}
					else
					{
						GeneratedCode[i].Hlsl = FText::FromString(*SourceText);
						SourceText->ParseIntoArrayLines(GeneratedCode[i].HlslByLines, false);
					}
				}
			}
		}
		else
		{
			GeneratedCode[i].Usage = ENiagaraScriptUsage::ParticleSpawnScript;
			GeneratedCode[i].UsageName = LOCTEXT("UsageNameInvalid", "Invalid");
		}

		//We now do the very expensive slate stuff inside Update UI_Internal
	}
}

void SNiagaraGeneratedCodeView::UpdateUI_Internal()
{
	bUIUpdatePending = false;

	for (int32 i = 0; i < GeneratedCode.Num(); i++)
	{
		if (!GeneratedCode[i].HorizontalScrollBar.IsValid())
		{
			GeneratedCode[i].HorizontalScrollBar = SNew(SScrollBar)
				.Orientation(Orient_Horizontal)
				.Thickness(FVector2D(12.0f, 12.0f));
		}

		if (!GeneratedCode[i].VerticalScrollBar.IsValid())
		{
			GeneratedCode[i].VerticalScrollBar = SNew(SScrollBar)
				.Orientation(Orient_Vertical)
				.Thickness(FVector2D(12.0f, 12.0f));
		}
		
		if (!GeneratedCode[i].Container.IsValid())
		{
			SAssignNew(GeneratedCode[i].Container, SVerticalBox)
				.Visibility(this, &SNiagaraGeneratedCodeView::GetViewVisibility, (uint32)i)
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SAssignNew(GeneratedCode[i].Text, SMultiLineEditableTextBox)
						.ClearTextSelectionOnFocusLoss(false)
						.IsReadOnly(true)
						.Marshaller(SyntaxHighlighter)
						.SearchText(this, &SNiagaraGeneratedCodeView::GetSearchText)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						GeneratedCode[i].VerticalScrollBar.ToSharedRef()
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					GeneratedCode[i].HorizontalScrollBar.ToSharedRef()
				];
		}

		GeneratedCode[i].Text->SetText(GeneratedCode[i].Hlsl);

		TextBodyContainer->AddSlot()
			[
				GeneratedCode[i].Container.ToSharedRef()
			];		
	}

	DoSearch(SearchBox->GetText());
}

SNiagaraGeneratedCodeView::~SNiagaraGeneratedCodeView()
{
	if (SystemViewModel.IsValid())
	{
		if (SystemViewModel->GetSelectionViewModel())
		{
			SystemViewModel->GetSelectionViewModel()->OnEmitterHandleIdSelectionChanged().RemoveAll(this);
		}

		if (SystemViewModel->GetSystemScriptViewModel().IsValid())
		{
			SystemViewModel->GetSystemScriptViewModel()->OnSystemCompiled().RemoveAll(this);
		}
	}
	
}

FText SNiagaraGeneratedCodeView::GetSearchText() const
{
	return SearchBox->GetText();
}

void SNiagaraGeneratedCodeView::OnTabChanged(uint32 Tab)
{
	TabState = Tab;
	DoSearch(SearchBox->GetText());
}


bool SNiagaraGeneratedCodeView::TabHasScriptData() const
{
	return !GeneratedCode[TabState].Hlsl.IsEmpty();
}

bool SNiagaraGeneratedCodeView::GetTabCheckedState(uint32 Tab) const
{
	return TabState == Tab ? true : false;
}

EVisibility SNiagaraGeneratedCodeView::GetViewVisibility(uint32 Tab) const
{
	return TabState == Tab ? EVisibility::Visible : EVisibility::Collapsed;
}


#undef LOCTEXT_NAMESPACE
