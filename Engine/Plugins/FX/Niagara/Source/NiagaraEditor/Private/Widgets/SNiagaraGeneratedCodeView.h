// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "TickableEditorObject.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "NiagaraDataSet.h"
#include "Templates/SharedPointer.h"
#include "Containers/Map.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Docking/SDockTab.h"

class FNiagaraHLSLSyntaxHighlighter;

class SNiagaraGeneratedCodeView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGeneratedCodeView)
	{}

	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel, TSharedPtr<SDockTab> InOwnerTab);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);
	virtual ~SNiagaraGeneratedCodeView();

	void OnCodeCompiled();

protected:

	void UpdateUI();
	void UpdateUI_Internal();

	TSharedPtr<SDockTab> OwnerTab;

	struct TabInfo
	{
		FText UsageName;
		FText Hlsl;
		ENiagaraScriptUsage Usage;
		FGuid UsageId;

		TArray<FString> HlslByLines;
		TSharedPtr<SMultiLineEditableTextBox> Text;
		TSharedPtr<SScrollBar> HorizontalScrollBar;
		TSharedPtr<SScrollBar> VerticalScrollBar;
		TSharedPtr<SVerticalBox> Container;
	};

	TArray<TabInfo> GeneratedCode;
	TSharedPtr<SComboButton> ScriptNameCombo;
	TSharedPtr<SHorizontalBox> ScriptNameContainer;
	TSharedPtr<SVerticalBox> TextBodyContainer;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<STextBlock> SearchFoundMOfNText;
	TArray<FTextLocation> ActiveFoundTextEntries;
	TSharedPtr<FNiagaraHLSLSyntaxHighlighter> SyntaxHighlighter;
	int32 CurrentFoundTextEntry;

	void SetSearchMofN();

	void SystemSelectionChanged();
	void OnTabChanged(uint32 Tab);
	bool GetTabCheckedState(uint32 Tab) const;
	EVisibility GetViewVisibility(uint32 Tab) const;
	bool TabHasScriptData() const;
	FReply OnCopyPressed();
	void OnSearchTextChanged(const FText& InFilterText);
	void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType);
	FReply SearchUpClicked();
	FReply SearchDownClicked();
	TSharedRef<SWidget> MakeScriptMenu();
	void DoSearch(const FText& InFilterText);
	FText GetCurrentScriptNameText() const;

	FText GetSearchText() const;

	uint32 TabState;

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;

	UEnum* ScriptEnum;

	bool bUIUpdatePending = false;
};


