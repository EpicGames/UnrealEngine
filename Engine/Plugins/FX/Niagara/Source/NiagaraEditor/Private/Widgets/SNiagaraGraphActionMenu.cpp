// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraGraphActionMenu.h"
#include "EdGraphSchema_Niagara.h"
#include "EdGraph/EdGraph.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "NiagaraActions.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "SNiagaraGraphActionWidget.h"
#include "Config/NiagaraFavoriteActionsConfig.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SItemSelector.h"

#define LOCTEXT_NAMESPACE "NiagaraGraphActionMenu"

bool SNiagaraGraphActionMenu::bLibraryOnly = true;
const FName SNiagaraGraphActionMenu::FavoriteActionsProfile = FName("NiagaraGraphActions");

SNiagaraGraphActionMenu::~SNiagaraGraphActionMenu()
{
	OnClosedCallback.ExecuteIfBound();
}

void SNiagaraGraphActionMenu::Construct( const FArguments& InArgs )
{
	this->GraphObj = InArgs._GraphObj;
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->NewNodePosition = InArgs._NewNodePosition;
	this->OnClosedCallback = InArgs._OnClosedCallback;
	this->AutoExpandActionMenu = InArgs._AutoExpandActionMenu;

	SAssignNew(FilterBox, SNiagaraFilterBox, SNiagaraFilterBox::FFilterOptions())
	.bLibraryOnly(this, &SNiagaraGraphActionMenu::GetLibraryOnly)
	.OnLibraryOnlyChanged(this, &SNiagaraGraphActionMenu::SetLibraryOnly)
	.OnSourceFiltersChanged(this, &SNiagaraGraphActionMenu::TriggerRefresh);
	
	// Build the widget layout
	SBorder::Construct( SBorder::FArguments()
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		.Padding(5)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
	        .Padding(1.0f)
	        .AutoHeight()
	        [
	        	FilterBox.ToSharedRef()
	        ]
            + SVerticalBox::Slot()
            [   
				// Achieving fixed width by nesting items within a fixed width box.
				SNew(SBox)
				.WidthOverride(450)
				.HeightOverride(400)
				[
					SAssignNew(ActionSelector, SNiagaraMenuActionSelector)
					.Items(CollectAllActions())
					.OnGetCategoriesForItem(this, &SNiagaraGraphActionMenu::OnGetCategoriesForItem)
					.OnGetSectionsForItem(this, &SNiagaraGraphActionMenu::OnGetSectionsForItem)
					.OnCompareSectionsForEquality(this, &SNiagaraGraphActionMenu::OnCompareSectionsForEquality)
					.OnCompareSectionsForSorting(this, &SNiagaraGraphActionMenu::OnCompareSectionsForSorting)
					.OnCompareCategoriesForEquality(this, &SNiagaraGraphActionMenu::OnCompareCategoriesForEquality)
					.OnCompareCategoriesForSorting(this, &SNiagaraGraphActionMenu::OnCompareCategoriesForSorting)
					.OnCompareItemsForSorting(this, &SNiagaraGraphActionMenu::OnCompareItemsForSorting)
					.OnDoesItemMatchFilterText_Lambda([](const FText& FilterText, const TSharedPtr<FNiagaraAction_NewNode>& Item)
					{
                        return FNiagaraEditorUtilities::DoesItemMatchFilterText(FilterText, Item);              
					})
					.OnGenerateWidgetForSection(this, &SNiagaraGraphActionMenu::OnGenerateWidgetForSection)
					.OnGenerateWidgetForCategory(this, &SNiagaraGraphActionMenu::OnGenerateWidgetForCategory)
					.OnGenerateWidgetForItem(this, &SNiagaraGraphActionMenu::OnGenerateWidgetForItem)
					.OnGetItemWeight_Lambda([](const TSharedPtr<FNiagaraAction_NewNode>& InCurrentAction, const TArray<FString>& InFilterTerms)
					{
						return FNiagaraEditorUtilities::GetWeightForItem(InCurrentAction, SNiagaraGraphActionMenu::FavoriteActionsProfile, InFilterTerms);
					})
					.OnItemActivated(this, &SNiagaraGraphActionMenu::OnItemActivated)
					.AllowMultiselect(false)
					.OnDoesItemPassCustomFilter(this, &SNiagaraGraphActionMenu::DoesItemPassCustomFilter)
					.OnDoesSectionPassCustomFilter(this, &SNiagaraGraphActionMenu::DoesSectionPassCustomFilter)
					.ClickActivateMode(EItemSelectorClickActivateMode::SingleClick)
					.ExpandInitially(false)
					.OnItemRowHoverEvent_Lambda([](const TSharedPtr<FNiagaraAction_NewNode>& Item, bool bHover)
					{
						Item->bIsHovered = bHover;
					})
					.OnGetSectionData_Lambda([](const ENiagaraMenuSections& Section)
					{
						if(Section == ENiagaraMenuSections::Suggested)
						{
                            return SNiagaraMenuActionSelector::FSectionData(SNiagaraMenuActionSelector::FSectionData::List, true);
                        }

                        return SNiagaraMenuActionSelector::FSectionData(SNiagaraMenuActionSelector::FSectionData::Tree, false);
					})
				]
			]
		]
	);
}


TArray<TSharedPtr<FNiagaraAction_NewNode>> SNiagaraGraphActionMenu::CollectAllActions()
{
	OwnerOfTemporaries =  NewObject<UEdGraph>((UObject*)GetTransientPackage());
	
	const UEdGraphSchema_Niagara* Schema = Cast<UEdGraphSchema_Niagara>(GraphObj->GetSchema());

	TArray<TSharedPtr<FNiagaraAction_NewNode>> Actions = Schema->GetGraphActions(GraphObj, DraggedFromPins.Num() > 0 ? DraggedFromPins[0] : nullptr, OwnerOfTemporaries);
	return Actions;
}

TSharedRef<SWidget> SNiagaraGraphActionMenu::GetFilterTextBox()
{
	return ActionSelector->GetSearchBox();
}

TArray<FString> SNiagaraGraphActionMenu::OnGetCategoriesForItem(const TSharedPtr<FNiagaraAction_NewNode>& Item)
{	
	return Item->Categories;
}

TArray<ENiagaraMenuSections> SNiagaraGraphActionMenu::OnGetSectionsForItem(const TSharedPtr<FNiagaraAction_NewNode>& Item)
{	
	TArray<ENiagaraMenuSections> Sections{ENiagaraMenuSections::General};

	if(Item->FavoritesActionData.IsSet())
	{
		FNiagaraFavoriteActionsProfile& ActionsProfile = UNiagaraFavoriteActionsConfig::Get()->GetActionsProfile(SNiagaraGraphActionMenu::FavoriteActionsProfile);
		TOptional<bool> bIsFavorite = ActionsProfile.IsFavorite(Item->FavoritesActionData.GetValue());
		if(bIsFavorite.IsSet() && bIsFavorite.GetValue())
		{
			Sections.Add(ENiagaraMenuSections::Suggested);
		}
	}
		
	return Sections;
}

bool SNiagaraGraphActionMenu::OnCompareSectionsForEquality(const ENiagaraMenuSections& SectionA, const ENiagaraMenuSections& SectionB)
{
	return SectionA == SectionB;
}

bool SNiagaraGraphActionMenu::OnCompareSectionsForSorting(const ENiagaraMenuSections& SectionA, const ENiagaraMenuSections& SectionB)
{
	return SectionA < SectionB;
}

bool SNiagaraGraphActionMenu::OnCompareCategoriesForEquality(const FString& CategoryA, const FString& CategoryB)
{
	return CategoryA.Compare(CategoryB) == 0;
}

bool SNiagaraGraphActionMenu::OnCompareCategoriesForSorting(const FString& CategoryA,
	const FString& CategoryB)
{
	return CategoryA.Compare(CategoryB) < 0;
}

bool SNiagaraGraphActionMenu::OnCompareItemsForEquality(const TSharedPtr<FNiagaraAction_NewNode>& ItemA, const TSharedPtr<FNiagaraAction_NewNode>& ItemB)
{
	return ItemA->DisplayName.EqualTo(ItemB->DisplayName);
}

bool SNiagaraGraphActionMenu::OnCompareItemsForSorting(const TSharedPtr<FNiagaraAction_NewNode>& ItemA, const TSharedPtr<FNiagaraAction_NewNode>& ItemB)
{
	return ItemA->DisplayName.CompareTo(ItemB->DisplayName) == -1;
}

TSharedRef<SWidget> SNiagaraGraphActionMenu::OnGenerateWidgetForSection(const ENiagaraMenuSections& Section)
{
	UEnum* SectionEnum = StaticEnum<ENiagaraMenuSections>();
	FText TextContent = SectionEnum->GetDisplayNameTextByValue((int64) Section);
	
	return SNew(STextBlock)
		.Text(TextContent)
		.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.AssetPickerAssetCategoryText");
}

TSharedRef<SWidget> SNiagaraGraphActionMenu::OnGenerateWidgetForCategory(const FString& Category)
{
	FText TextContent = FText::FromString(Category);

	return SNew(SRichTextBlock)
		.Text(TextContent)
		.DecoratorStyleSet(&FAppStyle::Get())
		.TextStyle(FNiagaraEditorStyle::Get(), "ActionMenu.HeadingTextBlock");
}

TSharedRef<SWidget> SNiagaraGraphActionMenu::OnGenerateWidgetForItem(const TSharedPtr<FNiagaraAction_NewNode>& Item)
{
	FCreateNiagaraWidgetForActionData ActionData(Item);
	ActionData.HighlightText = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &SNiagaraGraphActionMenu::GetFilterText));
	ActionData.FavoriteActionsProfileName = SNiagaraGraphActionMenu::FavoriteActionsProfile;
	ActionData.PreviewMovieViewModel = ViewModel ? ViewModel->GetPreviewMovieViewModel() : nullptr;
	return SNew(SNiagaraActionWidget, ActionData);
}

void SNiagaraGraphActionMenu::OnItemActivated(const TSharedPtr<FNiagaraAction_NewNode>& Item)
{
	Item->CreateNode(GraphObj, DraggedFromPins, NewNodePosition);
	FSlateApplication::Get().DismissAllMenus();	
}

void SNiagaraGraphActionMenu::TriggerRefresh(const TMap<EScriptSource, bool>& SourceState)
{
	ActionSelector->RefreshAllCurrentItems();

	TArray<bool> States;
	SourceState.GenerateValueArray(States);

	int32 NumActive = 0;
	for(bool& State : States)
	{
		if(State == true)
		{
			NumActive++;
		}
	}

	ActionSelector->ExpandTree();
}

void SNiagaraGraphActionMenu::SetLibraryOnly(bool bInIsLibraryOnly)
{
	bLibraryOnly = bInIsLibraryOnly;
	ActionSelector->RefreshAllCurrentItems(true);
}

bool SNiagaraGraphActionMenu::DoesItemPassCustomFilter(const TSharedPtr<FNiagaraAction_NewNode>& Item)
{
	bool bLibraryConditionFulfilled = (bLibraryOnly && Item->bIsInLibrary) || !bLibraryOnly;
	return FilterBox->IsSourceFilterActive(Item->SourceData.Source) && bLibraryConditionFulfilled;
}

bool SNiagaraGraphActionMenu::DoesSectionPassCustomFilter(const ENiagaraMenuSections& Section)
{
	if(Section == ENiagaraMenuSections::Suggested && !ActionSelector->IsSearching())
	{
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
