// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphActionWidget.h"

#include "NiagaraActions.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SNiagaraParameterName.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "Config/NiagaraFavoriteActionsConfig.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SToolTip.h"
#include "Widgets/IToolTip.h"

#define LOCTEXT_NAMESPACE "NiagaraGraphActionWidget"

void SNiagaraGraphActionWidget::Construct(const FArguments& InArgs, const FCreateWidgetForActionData* InCreateData)
{
	ActionPtr = InCreateData->Action;
	MouseButtonDownDelegate = InCreateData->MouseButtonDownDelegate;

	TSharedPtr<FNiagaraMenuAction> NiagaraAction = StaticCastSharedPtr<FNiagaraMenuAction>(InCreateData->Action);

	TSharedPtr<SWidget> NameWidget;
	if (NiagaraAction->GetParameterVariable().IsSet())
	{
		NameWidget = SNew(SNiagaraParameterName)
            .ParameterName(NiagaraAction->GetParameterVariable()->GetName())
            .IsReadOnly(true)
            .HighlightText(InArgs._HighlightText)
            .DecoratorHAlign(HAlign_Right)
            .DecoratorPadding(FMargin(7.0f, 0.0f, 0.0f, 0.0f))
            .Decorator()
            [
                SNew(STextBlock)
                .TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterName.TypeText")
                .Text(NiagaraAction->GetParameterVariable()->GetType().GetNameText())
            ];
	}
	else
	{
		NameWidget = SNew(STextBlock)
            .Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
            .Text(InCreateData->Action->GetMenuDescription())
            .HighlightText(InArgs._HighlightText);
	}

	this->ChildSlot
    [
        SNew(SHorizontalBox)
        .ToolTipText(InCreateData->Action->GetTooltipDescription())
        + SHorizontalBox::Slot()
        .FillWidth(1)
        .VAlign(VAlign_Center)
        [
            NameWidget.ToSharedRef()
        ]
        + SHorizontalBox::Slot()
        .AutoWidth()
        .VAlign(VAlign_Center)
        .HAlign(HAlign_Right)
        [
            SNew(SImage)
            .Image(NiagaraAction->IsExperimental ? FAppStyle::GetBrush("Icons.Info") : nullptr)
            .Visibility(NiagaraAction->IsExperimental ? EVisibility::Visible : EVisibility::Collapsed)
            .ToolTipText(NiagaraAction->IsExperimental ? LOCTEXT("ScriptExperimentalToolTip", "This script is experimental, use with care!") : FText::GetEmpty())
        ]
    ];
}

FReply SNiagaraGraphActionWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseButtonDownDelegate.Execute(ActionPtr))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SNiagaraActionWidget::Construct(const FArguments& InArgs, const FCreateNiagaraWidgetForActionData& InCreateData)
{
	ActionPtr = InCreateData.Action;
	FavoriteProfileName = InCreateData.FavoriteActionsProfileName;
	PreviewMovieViewModel = InCreateData.PreviewMovieViewModel;

	TSharedPtr<SWidget> NameWidget;
	if (ActionPtr->GetParameterVariable().IsSet())
	{
		TSharedRef<SNiagaraParameterName> ParameterNameWidget = SNew(SNiagaraParameterName)
			.ParameterName(ActionPtr->GetParameterVariable()->GetName())
			.IsReadOnly(true)
			.HighlightText(InCreateData.HighlightText)
			.DecoratorHAlign(HAlign_Right)
			.DecoratorPadding(FMargin(7.0f, 0.0f, 0.0f, 0.0f));
		
			if(InArgs._bShowTypeIfParameter)
			{
				ParameterNameWidget->UpdateDecorator( SNew(STextBlock)
               .Text(ActionPtr->GetParameterVariable()->GetType().GetNameText())
               .HighlightText(InCreateData.HighlightText)
               .TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterName.TypeText"));
			}

		NameWidget = ParameterNameWidget;
	}
	else
	{
		NameWidget = SNew(STextBlock)
			.Text(ActionPtr->DisplayName)
			.WrapTextAt(300.f)
			.TextStyle(FNiagaraEditorStyle::Get(), "ActionMenu.ActionTextBlock")
			.HighlightText(InCreateData.HighlightText);
	}

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		.ToolTip(this, &SNiagaraActionWidget::GetCustomTooltip)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			NameWidget.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
           SNew(SSpacer)
		]
		+ SHorizontalBox::Slot()
        .AutoWidth()
        .VAlign(VAlign_Center)
        .HAlign(HAlign_Right)
        [
            SNew(SImage)
            .Image(ActionPtr->bIsExperimental ? FAppStyle::GetBrush("Icons.Info") : nullptr)
            .Visibility(ActionPtr->bIsExperimental ? EVisibility::Visible : EVisibility::Collapsed)
            .ToolTipText(ActionPtr->bIsExperimental ? LOCTEXT("ScriptExperimentalToolTip", "This script is experimental, use with care!") : FText::GetEmpty())
        ]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CreateFavoriteActionWidget()
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
           SNew(SSeparator)
           .SeparatorImage(FAppStyle::Get().GetBrush("Separator"))
           .Orientation(EOrientation::Orient_Vertical)
			.Visibility(ActionPtr->SourceData.bDisplaySource ? EVisibility::Visible : EVisibility::Collapsed)
		]
       + SHorizontalBox::Slot()
       .HAlign(HAlign_Fill)
       .VAlign(VAlign_Center)
       .Padding(5, 0, 0, 0)
       .AutoWidth()
       [
           SNew(SBox)
           .WidthOverride(90.f)
           .Visibility(ActionPtr->SourceData.bDisplaySource ? EVisibility::Visible : EVisibility::Collapsed)
           [
               SNew(STextBlock)
               .Text(ActionPtr->SourceData.SourceText)
               .ColorAndOpacity(FNiagaraEditorUtilities::GetScriptSourceColor(ActionPtr->SourceData.Source))
               .TextStyle(FNiagaraEditorStyle::Get(), "GraphActionMenu.ActionSourceTextBlock")
           ]
       ]
    ];
}

TSharedRef<SWidget> SNiagaraActionWidget::CreateFavoriteActionWidget()
{	
	return
		SNew(SBox)
		.WidthOverride(16.f)
		.HeightOverride(16.f)
		[
			SNew(SButton)
			.ContentPadding(FMargin(0.f))
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
			.OnClicked(this, &SNiagaraActionWidget::OnFavoriteStateChanged)
			.Visibility(this, &SNiagaraActionWidget::OnShouldShowFavoriteButton)
			.ToolTipText(this, &SNiagaraActionWidget::GetFavoriteButtonTooltipText)
			[
				SNew(SImage)
				.Image(this, &SNiagaraActionWidget::GetFavoriteBrush)
			]
		];
}

FReply SNiagaraActionWidget::OnFavoriteStateChanged()
{
	// We still want the button to consume the input, but generally this will never happen as the button isn't displayed
	if(FavoriteProfileName.IsSet() == false || ActionPtr->FavoritesActionData.IsSet() == false)
	{
		return FReply::Handled();
	}
	
	FNiagaraFavoriteActionsProfile& ActionsProfile = UNiagaraFavoriteActionsConfig::Get()->GetActionsProfile(FavoriteProfileName.GetValue());
	ActionsProfile.ToggleFavoriteAction(ActionPtr->FavoritesActionData.GetValue());

	return FReply::Handled();
}

ECheckBoxState SNiagaraActionWidget::OnGetIsFavorite() const
{
	if(FavoriteProfileName.IsSet() == false || ActionPtr->FavoritesActionData.IsSet() == false)
	{
		return ECheckBoxState::Undetermined;
	}
		
	FNiagaraFavoriteActionsProfile& ActionsProfile = UNiagaraFavoriteActionsConfig::Get()->GetActionsProfile(FavoriteProfileName.GetValue());
	return ActionsProfile.IsFavorite(ActionPtr->FavoritesActionData.GetValue()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

EVisibility SNiagaraActionWidget::OnShouldShowFavoriteButton() const
{
	if(FavoriteProfileName.IsSet() == false || ActionPtr->FavoritesActionData.IsSet() == false)
	{
		return EVisibility::Collapsed;
	}

	return ActionPtr->bIsHovered ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* SNiagaraActionWidget::GetFavoriteBrush() const
{
	if(FavoriteProfileName.IsSet() == false || ActionPtr->FavoritesActionData.IsSet() == false)
	{
		return FAppStyle::GetNoBrush();
	}
	
	FNiagaraFavoriteActionsProfile& ActionsProfile = UNiagaraFavoriteActionsConfig::Get()->GetActionsProfile(FavoriteProfileName.GetValue());	
	if(ActionsProfile.IsFavorite(ActionPtr->FavoritesActionData.GetValue()))
	{
		return FAppStyle::GetBrush("Icons.Star");
	}
	
	return FAppStyle::GetBrush("Icons.Star.Outline");
}

FText SNiagaraActionWidget::GetFavoriteButtonTooltipText() const
{
	if(FavoriteProfileName.IsSet() == false || ActionPtr->FavoritesActionData.IsSet() == false)
	{
		return FText::GetEmpty();
	}

	FNiagaraFavoriteActionsProfile& ActionsProfile = UNiagaraFavoriteActionsConfig::Get()->GetActionsProfile(FavoriteProfileName.GetValue());
	if(ActionsProfile.IsFavorite(ActionPtr->FavoritesActionData.GetValue()))
	{
		return LOCTEXT("UnfavoriteButtonTooltip", "Remove this action from favorites");
	}

	return LOCTEXT("FavoriteButtonTooltip", "Add this action to favorites");
}

TSharedPtr<IToolTip> SNiagaraActionWidget::GetCustomTooltip() const
{
	using namespace FNiagaraEditorUtilities::Tooltips;
	
	if(PreviewMovieViewModel.IsValid() && ActionPtr->PreviewMoviePath.IsSet())
	{
		PreviewMovieViewModel.Pin()->PlayMovie(ActionPtr->PreviewMoviePath.GetValue());
	}
		
	if(TooltipCache.IsValid() == false)
	{
		FCreateTooltipArguments CreateTooltipArguments;
		CreateTooltipArguments.bTryCreatePreviewMovie = GetDefault<UNiagaraEditorSettings>()->bDisplayPreviewMoviesInTooltips;
		CreateTooltipArguments.PreviewMovieOwner = PreviewMovieViewModel.IsValid() ? PreviewMovieViewModel.Pin() : nullptr;
		TooltipCache = CreateTooltipForNiagaraAction(ActionPtr.ToSharedRef(), CreateTooltipArguments);
	}		
		
	return TooltipCache.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
