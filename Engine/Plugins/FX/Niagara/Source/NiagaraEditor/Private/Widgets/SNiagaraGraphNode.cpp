// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphNode.h"

#include "EdGraphSchema_Niagara.h"
#include "GraphEditorSettings.h"
#include "IDocumentation.h"
#include "Rendering/DrawElements.h"
#include "SGraphPin.h"
#include "NiagaraEditorSettings.h"
#include "SCommentBubble.h"
#include "TutorialMetaData.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "SNiagaraGraphNode"

const FSlateBrush* SNiagaraGraphNode::CachedInnerIcon = nullptr;
const FSlateBrush* SNiagaraGraphNode::CachedOuterIcon = nullptr;

SNiagaraGraphNode::SNiagaraGraphNode() : SGraphNode()
{
	NiagaraNode = nullptr;
}

SNiagaraGraphNode::~SNiagaraGraphNode()
{
	if (NiagaraNode.IsValid())
	{
		NiagaraNode->OnVisualsChanged().RemoveAll(this);
	}
}

void SNiagaraGraphNode::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	GraphNode = InGraphNode;
	NiagaraNode = Cast<UNiagaraNode>(InGraphNode);
	CompactTitle = NiagaraNode->GetCompactTitle();
	bShowPinNamesInCompactMode = NiagaraNode->ShouldShowPinNamesInCompactMode();
	CompactNodeTitleFontSizeOverride = NiagaraNode->GetCompactModeFontSizeOverride();
	RegisterNiagaraGraphNode(InGraphNode);
	UpdateGraphNode();	
}

void SNiagaraGraphNode::HandleNiagaraNodeChanged(UNiagaraNode* InNode)
{
	check(InNode == NiagaraNode);
	UpdateGraphNode();
}

void SNiagaraGraphNode::RegisterNiagaraGraphNode(UEdGraphNode* InNode)
{
	NiagaraNode = Cast<UNiagaraNode>(InNode);
	NiagaraNode->OnVisualsChanged().AddSP(this, &SNiagaraGraphNode::HandleNiagaraNodeChanged);
}

void SNiagaraGraphNode::UpdateGraphNode()
{
	check(NiagaraNode.IsValid());
	if(ShouldDrawCompact())
	{
		UpdateGraphNodeCompact();
	}
	else
	{
		SGraphNode::UpdateGraphNode();
	}
	LastSyncedNodeChangeId = NiagaraNode->GetChangeId();
}

void SNiagaraGraphNode::CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox)
{
	NiagaraNode->AddWidgetsToInputBox(InputBox);
}

void SNiagaraGraphNode::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	NiagaraNode->AddWidgetsToOutputBox(OutputBox);
}

void SNiagaraGraphNode::UpdateErrorInfo()
{
	const UNiagaraEditorSettings* NiagaraEditorSettings = GetDefault<UNiagaraEditorSettings>();
	ErrorMsg.Reset();
	if (NiagaraNode.IsValid())
	{
		if (NiagaraEditorSettings->IsReferenceableClass(GraphNode->GetClass()) == false || 
			(NiagaraNode->GetReferencedAsset() != nullptr && NiagaraEditorSettings->IsAllowedAssetObjectByClassUsage(*NiagaraNode->GetReferencedAsset()) == false))
		{
			ErrorMsg = FString(TEXT("UNSUPPORTED!"));
			ErrorColor = FAppStyle::GetColor("ErrorReporting.BackgroundColor");
		}
	}

	if(ErrorMsg.IsEmpty())
	{
		SGraphNode::UpdateErrorInfo();
	}
}

void SNiagaraGraphNode::CreatePinWidgets()
{
	// the default implementation will create default pin widgets for all our logical pins
	SGraphNode::CreatePinWidgets();

	TSet<TSharedRef<SWidget>> AllPins;
	GetPins(AllPins);

	LoadCachedIcons();

	// then we override the pin widget images for wildcards & numerics
	for(TSharedRef<SWidget>& Widget : AllPins)
	{
		if(TSharedPtr<SGraphPin> Pin = StaticCastSharedRef<SGraphPin>(Widget))
		{
			UEdGraphPin* SourcePin = Pin->GetPinObj();
			
			// Split pins should be drawn as normal pins, the inner properties are not promotable
			if(!SourcePin || SourcePin->ParentPin)
			{
				continue;
			}

			if(SourcePin->PinType == UEdGraphSchema_Niagara::TypeDefinitionToPinType(FNiagaraTypeDefinition::GetWildcardDef()) || SourcePin->PinType == UEdGraphSchema_Niagara::TypeDefinitionToPinType(FNiagaraTypeDefinition::GetGenericNumericDef()))
			{
				if(TSharedPtr<SLayeredImage> PinImage = StaticCastSharedPtr<SLayeredImage>(Pin->GetPinImageWidget()))
				{
					// Set the image to use the outer icon, which will be the connect pin type color
					PinImage->SetLayerBrush(0, CachedOuterIcon);

					// Set the inner image to be wildcard color, which is grey by default
					PinImage->AddLayer(CachedInnerIcon, GetDefault<UGraphEditorSettings>()->WildcardPinTypeColor);
				}
			}			
		}
	}
}

TSharedRef<SWidget> SNiagaraGraphNode::CreateTitleRightWidget()
{
	if(NiagaraNode.IsValid())
	{
		return NiagaraNode->CreateTitleRightWidget();
	}

	return SGraphNode::CreateTitleRightWidget();
}

void SNiagaraGraphNode::UpdateGraphNodeCompact()
{
	InputPins.Empty();
	OutputPins.Empty();

	// error handling set-up
	SetupErrorReporting();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

 	TSharedPtr<SToolTip> NodeToolTip = SNew( SToolTip );
	if (!SWidget::GetToolTip().IsValid())
	{
		NodeToolTip = IDocumentation::Get()->CreateToolTip( TAttribute< FText >( this, &SGraphNode::GetNodeTooltip ), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName() );
		SetToolTip(NodeToolTip);
	}

	// Setup a meta tag for this node
	FGraphNodeMetaData TagMeta(TEXT("Graphnode"));
	PopulateMetaTag(&TagMeta);
	
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode)
		.Text(this, &SNiagaraGraphNode::GetNodeCompactTitle);

	TSharedRef<SHorizontalBox> NodeContent = SNew(SHorizontalBox);

	EVerticalAlignment PinVerticalAlignment = VAlign_Center;
	
	// If at least 1 input pin exists, we want to sync left & right pin box sizes so that the title stays in the middle
	// If we don't have an input, like for constant getters or BP-like variable getters, we don't want to enlarge the left pin box
	bool bAnyInputExists = false;
	for(UEdGraphPin* Pin : GraphNode->GetAllPins())
	{
		if(Pin->Direction == EGPD_Input && Pin->bHidden == false)
		{
			bAnyInputExists = true;
			break;
		}
	}

	NodeContent->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(PinVerticalAlignment)
		.AutoWidth()
		.Padding(/* left */ 0.f, 0.f, 5.f, /* bottom */ 0.f)
		[
			// LEFT
			SAssignNew(LeftNodeBox, SVerticalBox)
		];
	
	NodeContent->AddSlot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(bAnyInputExists ? 5.f : 25.f, 0.f, 5.f, 0.f)
		[
			// MIDDLE
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				SNew(STextBlock)
					.TextStyle( FAppStyle::Get(), "Graph.CompactNode.Title" )
					.Text( NodeTitle.Get(), &SNodeTitle::GetHeadTitle )
					.WrapTextAt(128.0f)
					.Font(GetCompactNodeTitleFont())
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				NodeTitle.ToSharedRef()
			]
		];
	
	NodeContent->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(PinVerticalAlignment)
		.AutoWidth()
		.Padding(5.f, 0.f, 0.f, 0.f)
		[
			// RIGHT
			SAssignNew(RightNodeBox, SVerticalBox)
		];

	//
	//             ______________________
	//            | (>) L |      | R (>) |
	//            | (>) E |      | I (>) |
	//            | (>) F |   +  | G (>) |
	//            | (>) T |      | H (>) |
	//            |       |      | T (>) |
	//            |_______|______|_______|
	//
	TSharedRef<SVerticalBox> InnerVerticalBox =
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			// NODE CONTENT AREA
			SNew( SOverlay)
			+SOverlay::Slot()
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("Graph.VarNode.Body") )
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("Graph.VarNode.Gloss") )
			]
			+SOverlay::Slot()
			.Padding( FMargin(0,3) )
			[
				NodeContent
			]
		];
	
	TSharedPtr<SWidget> EnabledStateWidget = GetEnabledStateWidget();
	if (EnabledStateWidget.IsValid())
	{
		InnerVerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.Padding(FMargin(2, 0))
			[
				EnabledStateWidget.ToSharedRef()
			];
	}

	InnerVerticalBox->AddSlot()
		.AutoHeight()
		.Padding( FMargin(5.0f, 1.0f) )
		[
			ErrorReporting->AsWidget()
		];

	this->GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		InnerVerticalBox
	];

	CreatePinWidgets();

	// We hide input pin labels by default, unless the node specified we want to show them in compact mode
	if(bShowPinNamesInCompactMode == false)
	{
		for (auto InputPin: this->InputPins)
		{
			if (InputPin->GetPinObj()->ParentPin == nullptr)
			{
				InputPin->SetShowLabel(false);
			}
		}

	}

	// Output pin names are generally hidden here as compact mode is generally used for simple operations that have only 1 output
	for (auto OutputPin : this->OutputPins)
	{
		if (OutputPin->GetPinObj()->ParentPin == nullptr)
		{
			OutputPin->SetShowLabel(false);
		}
	}
	
	// Create comment bubble
	TSharedPtr<SCommentBubble> CommentBubble;
	const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

	SAssignNew( CommentBubble, SCommentBubble )
	.GraphNode( GraphNode )
	.Text( this, &SGraphNode::GetNodeComment )
	.OnTextCommitted( this, &SGraphNode::OnCommentTextCommitted )
	.ColorAndOpacity( CommentColor )
	.AllowPinning( true )
	.EnableTitleBarBubble( true )
	.EnableBubbleCtrls( true )
	.GraphLOD( this, &SGraphNode::GetCurrentLOD )
	.IsGraphNodeHovered(this, &SGraphNode::IsHovered);

	GetOrAddSlot( ENodeZone::TopCenter )
	.SlotOffset2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetOffset2f ))
	.SlotSize2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetSize2f ))
	.AllowScaling( TAttribute<bool>( CommentBubble.Get(), &SCommentBubble::IsScalingAllowed ))
	.VAlign( VAlign_Top )
	[
		CommentBubble.ToSharedRef()
	];

	CreateInputSideAddButton(LeftNodeBox);
	CreateOutputSideAddButton(RightNodeBox);
}

TAttribute<FSlateFontInfo> SNiagaraGraphNode::GetCompactNodeTitleFont()
{
	const FTextBlockStyle& CompactStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("Graph.CompactNode.Title");
	
	if(CompactNodeTitleFontSizeOverride.IsSet())
	{
		FSlateFontInfo FontInfo = CompactStyle.Font;
		FontInfo.Size = CompactNodeTitleFontSizeOverride.GetValue();
		return FontInfo;
	}

	return CompactStyle.Font;
}

void SNiagaraGraphNode::LoadCachedIcons()
{
	static const FName PromotableTypeOuterName("Kismet.VariableList.PromotableTypeOuterIcon");
	static const FName PromotableTypeInnerName("Kismet.VariableList.PromotableTypeInnerIcon");

	// Outer ring icons
	if(!CachedOuterIcon)
	{
		CachedOuterIcon = FAppStyle::GetBrush(PromotableTypeOuterName);
	}

	if(!CachedInnerIcon)
	{
		CachedInnerIcon = FAppStyle::GetBrush(PromotableTypeInnerName);
	}
}

#undef LOCTEXT_NAMESPACE
