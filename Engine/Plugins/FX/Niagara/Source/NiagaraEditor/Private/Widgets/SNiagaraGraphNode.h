// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"
#include "NiagaraNode.h"

/** A graph node widget representing a niagara node. */
class SNiagaraGraphNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphNode) {}
	SLATE_END_ARGS();

	SNiagaraGraphNode();
	virtual ~SNiagaraGraphNode();

	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	//~ SGraphNode api
	virtual void UpdateGraphNode() override;
	virtual void CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox) override;
	virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
	virtual void UpdateErrorInfo() override;
	virtual void CreatePinWidgets() override;
	virtual TSharedRef<SWidget> CreateTitleRightWidget() override;

protected:
	void UpdateGraphNodeCompact();
	virtual bool ShouldDrawCompact() const { return CompactTitle.IsEmpty() == false; }
	
	FText GetNodeCompactTitle() const { return CompactTitle; }
	/** To allow customization of the node title font size */
	TAttribute<FSlateFontInfo> GetCompactNodeTitleFont();

	void LoadCachedIcons();

	void RegisterNiagaraGraphNode(UEdGraphNode* InNode);
	void HandleNiagaraNodeChanged(UNiagaraNode* InNode);
	TWeakObjectPtr<UNiagaraNode> NiagaraNode;
	FGuid LastSyncedNodeChangeId;
	
	FText CompactTitle;
	bool bShowPinNamesInCompactMode = false;
	TOptional<float> CompactNodeTitleFontSizeOverride;
	
	static const FSlateBrush* CachedOuterIcon;
	static const FSlateBrush* CachedInnerIcon;
};