// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphNodeConvert.h"
#include "NiagaraNodeConvert.h"
#include "ViewModels/NiagaraConvertNodeViewModel.h"
#include "ViewModels/NiagaraConvertPinViewModel.h"
#include "ViewModels/NiagaraConvertPinSocketViewModel.h"
#include "SNiagaraConvertPinSocket.h"
#include "GraphEditorSettings.h"
#include "Rendering/DrawElements.h"
#include "Widgets/Input/SCheckBox.h"
#include "SGraphPin.h"

#define LOCTEXT_NAMESPACE "SNiagaraGraphNodeConvert"


void SNiagaraGraphNodeConvert::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	GraphNode = InGraphNode; 
	RegisterNiagaraGraphNode(InGraphNode);

	UpdateGraphNode();
}

void SNiagaraGraphNodeConvert::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	DefaultTitleAreaWidget->AddSlot()
	.Padding(5)
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Right)
	[
		SNew(SCheckBox)
		.OnCheckStateChanged(this, &SNiagaraGraphNodeConvert::ToggleShowWiring)
		.IsChecked(this, &SNiagaraGraphNodeConvert::GetToggleButtonChecked)
		.Cursor(EMouseCursor::Default)
		.ToolTipText(LOCTEXT("ToggleConvertNode_Tooltip", "Toggle visibility of convert node wiring."))
		.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SNiagaraGraphNodeConvert::GetToggleButtonArrow)
			]
		]
	];
}

FReply SNiagaraGraphNodeConvert::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if(InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		if (UNiagaraNodeConvert* ConvertNode = Cast<UNiagaraNodeConvert>(GraphNode))
		{
			ConvertNode->SetWiringShown(true);
		}
		OnDoubleClick.ExecuteIfBound(GraphNode);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SNiagaraGraphNodeConvert::ToggleShowWiring(const ECheckBoxState NewState)
{
	if (UNiagaraNodeConvert* ConvertNode = Cast<UNiagaraNodeConvert>(GraphNode))
	{
		ConvertNode->SetWiringShown(NewState == ECheckBoxState::Unchecked ? false : true);
	}
}

ECheckBoxState SNiagaraGraphNodeConvert::GetToggleButtonChecked() const
{
	if (UNiagaraNodeConvert* ConvertNode = Cast<UNiagaraNodeConvert>(GraphNode))
	{
		return ConvertNode->IsWiringShown() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Checked;
}

const FSlateBrush* SNiagaraGraphNodeConvert::GetToggleButtonArrow() const
{
	if (UNiagaraNodeConvert* ConvertNode = Cast<UNiagaraNodeConvert>(GraphNode))
	{
		return FAppStyle::GetBrush(ConvertNode->IsWiringShown() ? TEXT("Icons.ChevronUp") : TEXT("Icons.ChevronDown"));
	}
	return FAppStyle::GetBrush(TEXT("Icons.ChevronUp"));
}

TSharedRef<SWidget> ConstructPinSocketsRecursive(const TArray<TSharedRef<FNiagaraConvertPinSocketViewModel>>& SocketViewModels)
{
	TSharedRef<SVerticalBox> SocketBox = SNew(SVerticalBox);
	for (const TSharedRef<FNiagaraConvertPinSocketViewModel>& SocketViewModel : SocketViewModels)
	{
		if(SocketViewModel->CanBeConnected())
		{
			SocketBox->AddSlot()
			.AutoHeight()
			.Padding(TAttribute<FMargin>(SocketViewModel, &FNiagaraConvertPinSocketViewModel::GetSlotMargin))
			[
				SNew(SNiagaraConvertPinSocket, SocketViewModel)
				.Visibility(SocketViewModel, &FNiagaraConvertPinSocketViewModel::GetSocketVisibility)
			];
		}

		if (SocketViewModel->GetChildSockets().Num() > 0)
		{
			SocketBox->AddSlot()
			.AutoHeight()
			.Padding(TAttribute<FMargin>(SocketViewModel, &FNiagaraConvertPinSocketViewModel::GetChildSlotMargin))
			[
				ConstructPinSocketsRecursive(SocketViewModel->GetChildSockets())
			];
		}
	}
	return SocketBox;
}

TSharedRef<SWidget> ConstructPinSockets(TSharedRef<FNiagaraConvertPinViewModel> PinViewModel)
{
	return ConstructPinSocketsRecursive(PinViewModel->GetSocketViewModels());
}

void SNiagaraGraphNodeConvert::UpdateGraphNode()
{
	UNiagaraNodeConvert* ConvertNode = Cast<UNiagaraNodeConvert>(GraphNode);
	if (ConvertNode != nullptr)
	{
		ConvertNodeViewModel = MakeShareable(new FNiagaraConvertNodeViewModel(*ConvertNode));
	}
	SNiagaraGraphNode::UpdateGraphNode();
	
	// set visibility of add pins
	if (InputPins.Num() > 0 && OutputPins.Num() > 0 && ConvertNode)
	{
		InputPins.Last()->SetVisibility(ConvertNode->IsWiringShown() ? EVisibility::Visible : EVisibility::Collapsed);
		OutputPins.Last()->SetVisibility(ConvertNode->IsWiringShown() ? EVisibility::Visible : EVisibility::Collapsed);
	}
}

void SNiagaraGraphNodeConvert::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	TSharedPtr<FNiagaraConvertPinViewModel> ConvertPinViewModel = GetViewModelForPinWidget(PinToAdd);
	if (ConvertPinViewModel.IsValid() == false)
	{
		SGraphNode::AddPin(PinToAdd);
	}
	else
	{
		PinToAdd->SetOwner(SharedThis(this));

		const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
		const bool bAdvancedParameter = (PinObj != nullptr) && PinObj->bAdvancedView;
		if (bAdvancedParameter)
		{
			PinToAdd->SetVisibility(TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced));
		}

		float WirePadding = 20;
		float SocketPinPadding = 30;

		if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
		{
			FMargin InputPinPadding = Settings->GetInputPinPadding();
			InputPinPadding.Bottom = 3;
			InputPinPadding.Right += WirePadding;

			FMargin InputSocketPadding = InputPinPadding;
			InputSocketPadding.Top = 0;
			InputSocketPadding.Left += SocketPinPadding;

			LeftNodeBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(InputPinPadding)
				[
					PinToAdd
				];
			LeftNodeBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Top)
				.Padding(InputSocketPadding)
				[
					ConstructPinSockets(ConvertPinViewModel.ToSharedRef())
				];

			InputPins.Add(PinToAdd);
		}
		else // Direction == EEdGraphPinDirection::EGPD_Output
		{
			FMargin OutputPinPadding = Settings->GetOutputPinPadding();
			OutputPinPadding.Bottom = 3;
			OutputPinPadding.Left += WirePadding;

			FMargin OutputSocketPadding = OutputPinPadding;
			OutputSocketPadding.Top = 0;
			OutputSocketPadding.Right += SocketPinPadding;

			RightNodeBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(OutputPinPadding)
				[
					PinToAdd
				];
			RightNodeBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				.Padding(OutputSocketPadding)
				[
					ConstructPinSockets(ConvertPinViewModel.ToSharedRef())
				];
			OutputPins.Add(PinToAdd);
		}
	}
}

float DirectionOffset = 100;

int32 SNiagaraGraphNodeConvert::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 NewLayerId = SGraphNode::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	UNiagaraNodeConvert* ConvertNode = Cast<UNiagaraNodeConvert>(GraphNode);
	if (ConvertNode != nullptr && ConvertNode->IsWiringShown() == false)
	{
		return NewLayerId;
	}

	NewLayerId++;

	for (const TSharedRef<FNiagaraConvertConnectionViewModel>& ConnectionViewModel : ConvertNodeViewModel->GetConnectionViewModels())
	{
		FVector2D AbsStart = ConnectionViewModel->SourceSocket->GetAbsoluteConnectionPosition();
		FVector2D LocalStart = AllottedGeometry.AbsoluteToLocal(AbsStart) ;
		FVector2D StartDirection = FVector2D(DirectionOffset, 0);

		FVector2D AbsEnd = ConnectionViewModel->DestinationSocket->GetAbsoluteConnectionPosition();
		FVector2D LocalEnd = AllottedGeometry.AbsoluteToLocal(AbsEnd);
		FVector2D EndDirection = FVector2D(DirectionOffset, 0);

		bool bAllValuesValid = (AbsStart.X != -FLT_MAX) && (AbsStart.Y != -FLT_MAX) && (AbsEnd.X != -FLT_MAX) && (AbsEnd.Y != -FLT_MAX);

		if (ConnectionViewModel->SourceSocket->GetSocketVisibility().IsVisible() &&
			ConnectionViewModel->DestinationSocket->GetSocketVisibility().IsVisible() &&
			bAllValuesValid)
		{
			FSlateDrawElement::MakeSpline(
				OutDrawElements,
				NewLayerId,
				AllottedGeometry.ToPaintGeometry(),
				LocalStart,
				StartDirection,
				LocalEnd,
				EndDirection,
				2);
		}
		
	}

	if (ConvertNodeViewModel->GetDraggedSocketViewModel().IsValid())
	{
		FVector2D AbsStart = ConvertNodeViewModel->GetDraggedSocketViewModel()->GetAbsoluteConnectionPosition();
		FVector2D LocalStart = AllottedGeometry.AbsoluteToLocal(AbsStart);
		FVector2D StartDirection = FVector2D(DirectionOffset, 0);

		FVector2D AbsEnd = ConvertNodeViewModel->GetDraggedSocketViewModel()->GetAbsoluteDragPosition() + FVector2D(Inverse(Args.GetWindowToDesktopTransform()));
		FVector2D LocalEnd = AllottedGeometry.AbsoluteToLocal(AbsEnd);
		FVector2D EndDirection = FVector2D(DirectionOffset, 0);

		// Swap directions if going backwards
		if (ConvertNodeViewModel->GetDraggedSocketViewModel()->GetDirection() == EGPD_Output)
		{
			FVector2D Temp = LocalEnd;
			LocalEnd = LocalStart;
			LocalStart = Temp;
		}

		bool bAllValuesValid = (AbsStart.X != -FLT_MAX) && (AbsStart.Y != -FLT_MAX) && (AbsEnd.X != -FLT_MAX) && (AbsEnd.Y != -FLT_MAX);

		if (bAllValuesValid)
		{
			FSlateDrawElement::MakeSpline(
				OutDrawElements,
				NewLayerId,
				AllottedGeometry.ToPaintGeometry(),
				LocalStart,
				StartDirection,
				LocalEnd,
				EndDirection,
				2);
		}
	}

	return NewLayerId;
}

TSharedPtr<FNiagaraConvertPinViewModel> SNiagaraGraphNodeConvert::GetViewModelForPinWidget(TSharedRef<SGraphPin> GraphPin)
{
	const TArray<TSharedRef<FNiagaraConvertPinViewModel>>& ViewModels = GraphPin->GetDirection() == EGPD_Input
		? ConvertNodeViewModel->GetInputPinViewModels()
		: ConvertNodeViewModel->GetOutputPinViewModels();

	auto FindByPin = [GraphPin](const TSharedRef<FNiagaraConvertPinViewModel> PinViewModel) { return &PinViewModel->GetGraphPin() == GraphPin->GetPinObj(); };
	const TSharedRef<FNiagaraConvertPinViewModel>* ViewModelPtr = ViewModels.FindByPredicate(FindByPin);

	return ViewModelPtr != nullptr
		? *ViewModelPtr
		: TSharedPtr<FNiagaraConvertPinViewModel>();
}

#undef LOCTEXT_NAMESPACE
