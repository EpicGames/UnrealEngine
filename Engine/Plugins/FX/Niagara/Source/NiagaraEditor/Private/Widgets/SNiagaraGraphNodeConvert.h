// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SNiagaraGraphNode.h"

class FNiagaraConvertNodeViewModel;
class FNiagaraConvertPinViewModel;

/** A graph node widget representing a niagara convert node. */
class SNiagaraGraphNodeConvert : public SNiagaraGraphNode
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphNodeConvert) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	//~ SGraphNode api
	virtual void UpdateGraphNode() override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

protected:
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;

	void ToggleShowWiring(const ECheckBoxState NewState);
	ECheckBoxState GetToggleButtonChecked() const;
	const FSlateBrush* GetToggleButtonArrow() const;

private:
	TSharedPtr<FNiagaraConvertPinViewModel> GetViewModelForPinWidget(TSharedRef<SGraphPin> GraphPin);

private:
	TSharedPtr<FNiagaraConvertNodeViewModel> ConvertNodeViewModel;
};