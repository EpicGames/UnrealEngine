// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphNodeCustomHlsl.h"

#include "EdGraphSchema_Niagara.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraGraph.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SNiagaraGraphNodeCustomHlsl"

SNiagaraGraphNodeCustomHlsl::SNiagaraGraphNodeCustomHlsl() : SNiagaraGraphNode()
{

}

void SNiagaraGraphNodeCustomHlsl::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	GraphNode = InGraphNode;
	SyntaxHighlighter = FNiagaraHLSLSyntaxHighlighter::Create();
	RegisterNiagaraGraphNode(InGraphNode);
	UpdateGraphNode();
}

bool SNiagaraGraphNodeCustomHlsl::IsNameReadOnly() const
{
	return false;
}

void SNiagaraGraphNodeCustomHlsl::RequestRenameOnSpawn()
{
	// We only want to initiate the rename if this is a uniquely added node.
	UNiagaraNodeCustomHlsl* CustomNode = CastChecked<UNiagaraNodeCustomHlsl>(GraphNode);
	UNiagaraGraph* Graph = CastChecked<UNiagaraGraph>(CustomNode->GetGraph());
	TArray<UNiagaraNodeCustomHlsl*> CustomNodes;
	Graph->GetNodesOfClass<UNiagaraNodeCustomHlsl>(CustomNodes);

	int32 NumMatches = 0;
	for (UNiagaraNodeCustomHlsl* Node : CustomNodes)
	{
		if (Node == CustomNode)
		{
			continue;
		}

		bool bNeedsSync = Node->Signature.Name == CustomNode->Signature.Name;
		if (bNeedsSync)
		{
			NumMatches++;
		}
	}
	if (NumMatches == 0)
	{
		RequestRename();
	}
}

void SNiagaraGraphNodeCustomHlsl::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	DefaultTitleAreaWidget->AddSlot()
	.Padding(5)
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Right)
	[
		SNew(SCheckBox)
		.OnCheckStateChanged(this, &SNiagaraGraphNodeCustomHlsl::ToggleShowShaderCode)
		.IsChecked(this, &SNiagaraGraphNodeCustomHlsl::GetToggleButtonChecked)
		.Cursor(EMouseCursor::Default)
		.ToolTipText(LOCTEXT("ToggleShaderCode_Tooltip", "Toggle visibility of shader code."))
		.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SNiagaraGraphNodeCustomHlsl::GetToggleButtonArrow)
			]
		]
	];
}

FReply SNiagaraGraphNodeCustomHlsl::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry,	const FPointerEvent& InMouseEvent)
{
	if(InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		UNiagaraNodeCustomHlsl* CustomNode = CastChecked<UNiagaraNodeCustomHlsl>(GraphNode);
		CustomNode->SetShaderCodeShown(true);
		OnDoubleClick.ExecuteIfBound(CustomNode);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SNiagaraGraphNodeCustomHlsl::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	UNiagaraNodeCustomHlsl* CustomNode = CastChecked<UNiagaraNodeCustomHlsl>(GraphNode);

	TAttribute<FText> GetText;
	GetText.BindUObject(CustomNode, &UNiagaraNodeCustomHlsl::GetHlslText);
	
	FOnTextCommitted TextCommit;
	TextCommit.BindUObject(CustomNode, &UNiagaraNodeCustomHlsl::OnCustomHlslTextCommitted);

	SAssignNew(ShaderTextBox, SMultiLineEditableTextBox)
		.AutoWrapText(false)
		.Margin(FMargin(5, 5, 5, 5))
		.Text(GetText)
		.Marshaller(SyntaxHighlighter)
		.OnKeyDownHandler(this, &SNiagaraGraphNodeCustomHlsl::OnKeyDown)
		.OnKeyCharHandler(this, &SNiagaraGraphNodeCustomHlsl::OnShaderTextKeyChar)
		.OnTextCommitted(TextCommit)
		.Visibility(this, &SNiagaraGraphNodeCustomHlsl::GetShaderTextVisibility);

	// the menu anchor is a bit of a hack to show the auto-complete window.
	// Main advantage is that it has all the layout logic inside to place the popup window correctly and auto-closes it.
	// Disadvantage is that it's always at the bottom of the node and not where the current line cursor is.
	SAssignNew(MenuAnchor, SMenuAnchor)
		.Placement(MenuPlacement_BelowAnchor)
		.Method(EPopupMethod::CreateNewWindow)
		.Visibility(EVisibility::HitTestInvisible)
		.OnGetMenuContent_Lambda([this]()
		{
			return SNew(STextBlock)
			.Margin(5)
			.Text_Lambda([this]()
			{
				TStringBuilder<256> Options;
				for (int i = 0; (i + AutoCompletePageIndex * 9) < FunctionPrototypes.Num() && i < 9; i++)
				{
					const FString& Prototype = FunctionPrototypes[i + AutoCompletePageIndex * 9].Get<0>();
					Options.Appendf(TEXT("\n(%i) %s"), i + 1, *Prototype);
				}
				FText HiddenText = FText::GetEmpty();
				if (FunctionPrototypes.Num() > (AutoCompletePageIndex + 1) * 9)
				{
					HiddenText = LOCTEXT("Autocomplete_HiddenFormat", "\n(0) more...");
				}
				return FText::Format(LOCTEXT("Autocomplete_Format", "Select auto-complete option:{0}{1}"), FText::FromString(Options.ToString()), HiddenText);
			});
		});
	
	MainBox->AddSlot()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(10.f, 5.f, 10.f, 10.f))
		[
			ShaderTextBox.ToSharedRef()
		]
	];

	
	MainBox->AddSlot()
	.MaxHeight(0)
	.AutoHeight()
	[
		MenuAnchor.ToSharedRef()
	];
}

bool IsValidIdentifierChar(const TCHAR& Char)
{
	return ((Char >= TCHAR('0') && Char <= TCHAR('9')) ||
			(Char >= TCHAR('A') && Char <= TCHAR('Z')) ||
			(Char >= TCHAR('a') && Char <= TCHAR('z')) ||
			Char == TCHAR('_'));
}

FString SNiagaraGraphNodeCustomHlsl::GetIdentifierUnderCursor() const
{
	FString CurrentLine;
	ShaderTextBox->GetCurrentTextLine(CurrentLine);
	FTextLocation CursorLocation = ShaderTextBox->GetCursorLocation();
	int32 Offset = CursorLocation.GetOffset();
	int32 StartIndex = Offset - 1;
	while (StartIndex >= 0)
	{
		if (IsValidIdentifierChar(CurrentLine[StartIndex]))
		{
			StartIndex--;
		}
		else
		{
			break;
		}
	}
	StartIndex++;

	if (StartIndex >= 0 && Offset >= 0 && StartIndex < Offset)
	{
		return CurrentLine.Mid(StartIndex, Offset - StartIndex);
	}
	return FString();
}

FReply SNiagaraGraphNodeCustomHlsl::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Tab)
	{
		constexpr int TabToSpaceLen = 4;

		const FTextLocation CursorLocation = ShaderTextBox->GetCursorLocation();
		if (InKeyEvent.IsShiftDown())
		{
			const FTextLocation NewCursorLocation(CursorLocation.GetLineIndex(), FMath::Max(AlignDown(CursorLocation.GetOffset() - 1, TabToSpaceLen), 0));

			FString CurrentLine;
			ShaderTextBox->GetCurrentTextLine(CurrentLine);

			if (NewCursorLocation.GetOffset() != CursorLocation.GetOffset())
			{
				bool bTrimWhitespace = true;
				for (int32 i = 0; i < CursorLocation.GetOffset() && bTrimWhitespace; ++i)
				{
					bTrimWhitespace = FChar::IsWhitespace(CurrentLine[i]);
				}

				if (bTrimWhitespace)
				{
					CurrentLine.RemoveAt(NewCursorLocation.GetOffset(), CursorLocation.GetOffset() - NewCursorLocation.GetOffset());

					// We lack methods exposed to select or delete data from the ShaderTextBox so we rebuild all the text
					TArray<FString> AllLines;
					ShaderTextBox->GetText().ToString().ParseIntoArrayLines(AllLines, false);
					AllLines[CursorLocation.GetLineIndex()] = CurrentLine;

					const FString NewText = FString::Join(AllLines, TEXT("\n"));
					ShaderTextBox->SetText(FText::FromString(NewText));
				}
			}

			ShaderTextBox->GoTo(NewCursorLocation);

			return FReply::Handled();
		}
		else
		{
			const FTextLocation NewCursorLocation(CursorLocation.GetLineIndex(), Align(CursorLocation.GetOffset() + 1, TabToSpaceLen));
			const FString SpaceString = FString::ChrN(NewCursorLocation.GetOffset() - CursorLocation.GetOffset(), TEXT(' '));
			ShaderTextBox->InsertTextAtCursor(SpaceString);
			ShaderTextBox->GoTo(NewCursorLocation);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SNiagaraGraphNodeCustomHlsl::OnShaderTextKeyChar(const FGeometry&, const FCharacterEvent& InCharacterEvent)
{
	const TCHAR Character = InCharacterEvent.GetCharacter();

	if (bIsAutoCompleteActive)
	{
		if (Character == TCHAR('0') && FunctionPrototypes.Num() > (AutoCompletePageIndex + 1) * 9)
		{
			AutoCompletePageIndex++;
			return FReply::Handled();
		}
		if ((Character >= TCHAR('1') && Character <= TCHAR('9')))
		{
			int32 SelectedOption = (Character - TCHAR('1')) + AutoCompletePageIndex * 9;
			if (AutoCompleteOptions.IsValidIndex(SelectedOption))
			{
				ShaderTextBox->InsertTextAtCursor(AutoCompleteOptions[SelectedOption]);
				bIsAutoCompleteActive = false;
				MenuAnchor->SetIsOpen(false);
				return FReply::Handled();
			}
		}
	}
	MenuAnchor->SetIsOpen(false);
	bIsAutoCompleteActive = false;
	
	if (Character == TEXT('.'))
	{
		// Show auto-complete hint for data interfaces
		FString Identifier = GetIdentifierUnderCursor();
		if (Identifier.Len() > 0)
		{
			UNiagaraNodeCustomHlsl* CustomNode = CastChecked<UNiagaraNodeCustomHlsl>(GraphNode);
			FPinCollectorArray NodeInputPins;
			CustomNode->GetInputPins(NodeInputPins);
			for (UEdGraphPin* Pin : NodeInputPins)
			{
				if (Pin->GetName().Equals(Identifier, ESearchCase::IgnoreCase))
				{
					FunctionPrototypes = UEdGraphSchema_Niagara::GetDataInterfaceFunctionPrototypes(Pin);
					if (FunctionPrototypes.Num() > 0)
					{
						bIsAutoCompleteActive = true;
						AutoCompletePageIndex = 0;
						AutoCompleteOptions.Empty();

						for (int i = 0; i < FunctionPrototypes.Num(); i++)
						{
							FString Prototype = FunctionPrototypes[i].Get<1>();
							int32 NameDot = 0;
							if (Prototype.FindChar('.', NameDot))
							{
								Prototype.RightChopInline(NameDot + 1);
							}
							Prototype.TrimStartAndEndInline();
							AutoCompleteOptions.Add(Prototype);
						}						
						MenuAnchor->SetIsOpen(true, false);
						break;
					}
				}
			}
		}
		ShaderTextBox->InsertTextAtCursor(TEXT("."));
		return FReply::Handled();
	}
	
	// Let SMultiLineEditableTextBox::OnKeyChar handle it.
	return FReply::Unhandled();
}

EVisibility SNiagaraGraphNodeCustomHlsl::GetShaderTextVisibility() const
{
	UNiagaraNodeCustomHlsl* CustomNode = CastChecked<UNiagaraNodeCustomHlsl>(GraphNode);
	return CustomNode->IsShaderCodeShown() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SNiagaraGraphNodeCustomHlsl::ToggleShowShaderCode(const ECheckBoxState NewCheckedState)
{
	UNiagaraNodeCustomHlsl* CustomNode = CastChecked<UNiagaraNodeCustomHlsl>(GraphNode);
	CustomNode->SetShaderCodeShown(NewCheckedState == ECheckBoxState::Unchecked ? false : true);
}

ECheckBoxState SNiagaraGraphNodeCustomHlsl::GetToggleButtonChecked() const
{
	UNiagaraNodeCustomHlsl* CustomNode = CastChecked<UNiagaraNodeCustomHlsl>(GraphNode);
	return CustomNode->IsShaderCodeShown() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

const FSlateBrush* SNiagaraGraphNodeCustomHlsl::GetToggleButtonArrow() const
{
	UNiagaraNodeCustomHlsl* CustomNode = CastChecked<UNiagaraNodeCustomHlsl>(GraphNode);
	return FAppStyle::GetBrush(CustomNode->IsShaderCodeShown() ? TEXT("Icons.ChevronUp") : TEXT("Icons.ChevronDown"));
}

#undef LOCTEXT_NAMESPACE
