// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/NiagaraHLSLSyntaxHighlighter.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "SNiagaraGraphNode.h"

class SMultiLineEditableTextBox;

/** A graph node widget representing a niagara input node. */
class SNiagaraGraphNodeCustomHlsl : public SNiagaraGraphNode
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphNodeCustomHlsl) {}
	SLATE_END_ARGS();

	SNiagaraGraphNodeCustomHlsl();

	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	//~ SGraphNode api
	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	virtual bool IsNameReadOnly() const override;
	virtual void RequestRenameOnSpawn() override;
	virtual bool ShouldAllowCulling() const override { return false; }
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
protected:

	TSharedPtr<FNiagaraHLSLSyntaxHighlighter> SyntaxHighlighter;
	TSharedPtr<SMultiLineEditableTextBox> ShaderTextBox;
	TSharedPtr<SMenuAnchor> MenuAnchor;

private:
	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	FReply OnShaderTextKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent);
	FString GetIdentifierUnderCursor() const;

	EVisibility GetShaderTextVisibility() const;
	void ToggleShowShaderCode(const ECheckBoxState NewCheckedState);
	ECheckBoxState GetToggleButtonChecked() const;
	const FSlateBrush* GetToggleButtonArrow() const;

	bool bIsAutoCompleteActive = false;
	TArray<FString> AutoCompleteOptions;
	TArray<TTuple<FString, FString, FText>> FunctionPrototypes;
	int32 AutoCompletePageIndex = 0;
};
