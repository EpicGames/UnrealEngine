// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphNodeFunctionCallWithSpecifiers.h"
#include "ISinglePropertyView.h"
#include "Modules/ModuleManager.h"
#include "NiagaraNodeFunctionCall.h"
#include "GraphEditorSettings.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "NiagaraConstants.h"

#define LOCTEXT_NAMESPACE "SNiagaraGraphNodeFunctionCallWithSpecifiers"

void SNiagaraGraphNodeFunctionCallWithSpecifiers::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	GraphNode = InGraphNode;
	this->FunctionSpecifiers = &(dynamic_cast<UNiagaraNodeFunctionCall*>(GraphNode)->FunctionSpecifiers);
	RegisterNiagaraGraphNode(InGraphNode);
	this->UpdateGraphNode();
}

TSharedRef<SWidget> SNiagaraGraphNodeFunctionCallWithSpecifiers::CreateNodeContentArea()
{
	TSharedPtr<SWidget> FunctionSpecifierWidget = nullptr;

	UNiagaraNodeFunctionCall* FuncNode = CastChecked<UNiagaraNodeFunctionCall>(GraphNode);
	if(FuncNode)
	{
		FunctionSpecifierWidget = INiagaraDataInterfaceNodeActionProvider::GetCustomFunctionSpecifierWidget(FuncNode->GetDIClass(), FuncNode);
	}
	
	//Fall back to default if there is no override for this DI function.
	if(FunctionSpecifierWidget == nullptr)
	{
		TSharedPtr<SVerticalBox> DefaultFunctionSpecifierWidget = SNew(SVerticalBox);
		for (TTuple<FName, FName>& Entry : *FunctionSpecifiers)
		{
			DefaultFunctionSpecifierWidget->AddSlot()
				.VAlign(VAlign_Center)
				[
					SNew(SNiagaraFunctionSpecifier, Entry.Key, Entry.Value, *FunctionSpecifiers)
					.OnValueNameChanged(this, &SNiagaraGraphNodeFunctionCallWithSpecifiers::OnValueNameChanged)
				];
		}
		FunctionSpecifierWidget = DefaultFunctionSpecifierWidget;
	}

	TSharedRef<SWidget> ContentAreaWidget = SGraphNode::CreateNodeContentArea();
	TSharedPtr<SVerticalBox> VertContainer = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.Padding(Settings->GetInputPinPadding())
		[
			FunctionSpecifierWidget.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ContentAreaWidget
		];
	TSharedRef<SWidget> RetWidget = VertContainer.ToSharedRef();
	return RetWidget;
}

void SNiagaraGraphNodeFunctionCallWithSpecifiers::OnValueNameChanged()
{
	Cast<UNiagaraNodeFunctionCall>(GraphNode)->MarkNodeRequiresSynchronization(__FUNCTION__, true);
}

void SNiagaraFunctionSpecifier::Construct(const FArguments& InArgs, FName InAttributeName, FName InValueName, TMap<FName, FName>& InSpecifiers)
{
	OnValueNameChanged = InArgs._OnValueNameChanged;

	AttributeName = InAttributeName;
	ValueName = InValueName;
	Specifiers = &InSpecifiers;

	ChildSlot
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoWidth()
			.Padding(5)
			[
				SNew(STextBlock)
				.Text(FText::FromName(AttributeName))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoWidth()
			.Padding(5)
			[
				SNew(SEditableTextBox)
				.Text(FText::FromName(ValueName))
				.OnVerifyTextChanged(this, &SNiagaraFunctionSpecifier::VerifyNameTextChanged)
				.OnTextCommitted(this, &SNiagaraFunctionSpecifier::OnValueNameCommitted)
			]
		];
}

bool SNiagaraFunctionSpecifier::VerifyNameTextChanged(const FText& NewText, FText& OutErrorMessage) const
{
	if (NewText.ToString().Len() >= FNiagaraConstants::MaxParameterLength)
	{
		OutErrorMessage = FText::FormatOrdered(LOCTEXT("NameToLongError", "Name cannot exceed {0} characters."), FNiagaraConstants::MaxParameterLength);
		return false;
	}
	return true;
}

void SNiagaraFunctionSpecifier::OnValueNameCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	ValueName = FName(*InText.ToString());
	if (Specifiers)
	{
		Specifiers->Add(AttributeName, ValueName);
		OnValueNameChanged.ExecuteIfBound();
	}
}

#undef LOCTEXT_NAMESPACE