// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SNiagaraGraphNode.h"

/** A graph node widget representing a function call with function specifiers. */
class SNiagaraGraphNodeFunctionCallWithSpecifiers : public SNiagaraGraphNode
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphNodeFunctionCallWithSpecifiers) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;

	void OnValueNameChanged();
	
	TMap<FName, FName>* FunctionSpecifiers;
};

class SNiagaraFunctionSpecifier : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraFunctionSpecifier) {}
	SLATE_EVENT(FSimpleDelegate, OnValueNameChanged)
	SLATE_END_ARGS();


	void Construct(const FArguments& InArgs, FName InAttributeName, FName InValueName, TMap<FName, FName>& InSpecifiers);

	bool VerifyNameTextChanged(const FText& NewText, FText& OutErrorMessage) const;
	void OnValueNameCommitted(const FText& InText, ETextCommit::Type InCommitType);

	FSimpleDelegate OnValueNameChanged;

	FName AttributeName;
	FName ValueName;
	TMap<FName, FName>* Specifiers;
};