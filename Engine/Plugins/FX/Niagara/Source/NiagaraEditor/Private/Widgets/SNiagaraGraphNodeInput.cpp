// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphNodeInput.h"
#include "NiagaraNodeInput.h"
#include "NiagaraGraph.h"
#include "Framework/Commands/Commands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SNiagaraGraphNodeInput"

class FNiagaraGraphNodeInputCommands
	: public TCommands<FNiagaraGraphNodeInputCommands>
{
public:

	/**
	* Default constructor.
	*/
	FNiagaraGraphNodeInputCommands()
		: TCommands<FNiagaraGraphNodeInputCommands>("NiagaraInputNodeEditor", NSLOCTEXT("Contexts", "NiagaraInputNodeEditor", "Niagara Input Node Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{ }

public:

	// TCommands interface

	virtual void RegisterCommands() override
	{
		UI_COMMAND(ToggleExposed, "Expose Input", "If true then this is exposed as an input to it's caller - turning this off makes sense when the input is defined in a function script and should not be visible to the caller.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ToggleRequired, "Suppress Default Value", "If true then this input is required to be set by the caller. For optional values (e.g. values behind an edit condition) this can be set to false, so the translator uses the default value instead.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ToggleCanAutoBind, "CanAutoBind", "Toggles whether or not this parameter can auto-bind to system parameters and emitter attributes. Will never auto bind to custom parameters.", EUserInterfaceActionType::ToggleButton, FInputChord());
		UI_COMMAND(ToggleHidden, "Advanced Display", "If this input should be hidden in the advanced pin section of it's caller.", EUserInterfaceActionType::ToggleButton, FInputChord());
	}

public:
	TSharedPtr< FUICommandInfo > ToggleExposed;
	TSharedPtr< FUICommandInfo > ToggleRequired;
	TSharedPtr< FUICommandInfo > ToggleCanAutoBind;
	TSharedPtr< FUICommandInfo > ToggleHidden;
};


SNiagaraGraphNodeInput::SNiagaraGraphNodeInput() : SNiagaraGraphNode(),
	ToolkitCommands(new FUICommandList()),
	bRequestedSyncExposureOptions(false)
{

}

void SNiagaraGraphNodeInput::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	GraphNode = InGraphNode; 
	RegisterNiagaraGraphNode(InGraphNode);

	FNiagaraGraphNodeInputCommands::Register();
	BindCommands();
	UpdateGraphNode();
}

bool SNiagaraGraphNodeInput::IsNameReadOnly() const
{
	const UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	if (InputNode->Usage == ENiagaraInputNodeUsage::Parameter ||
		InputNode->Usage == ENiagaraInputNodeUsage::RapidIterationParameter)
	{
		return false;
	}
	return true;
}

void SNiagaraGraphNodeInput::RequestRenameOnSpawn()
{
	// We only want to initiate the rename if this is a uniquely added node.
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	UNiagaraGraph* Graph = CastChecked<UNiagaraGraph>(InputNode->GetGraph());
	TArray<UNiagaraNodeInput*> InputNodes;
	Graph->GetNodesOfClass<UNiagaraNodeInput>(InputNodes);

	int32 NumMatches = 0;
	for (UNiagaraNodeInput* Node : InputNodes)
	{
		if (Node == InputNode)
		{
			continue;
		}

		bool bNeedsSync = Node->ReferencesSameInput(InputNode);
		if (bNeedsSync)
		{
			NumMatches++;
		}
	}
	if (NumMatches == 0)
	{
		RequestRename();
	}
}

TSharedRef<SWidget> SNiagaraGraphNodeInput::CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SGraphNode::CreateTitleWidget(NodeTitle)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(10.0f, 0.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SAssignNew(ExposureOptionsMenuAnchor, SMenuAnchor)
			.OnGetMenuContent(this, &SNiagaraGraphNodeInput::GenerateExposureOptionsMenu)
			.OnMenuOpenChanged(this, &SNiagaraGraphNodeInput::ExposureOptionsMenuOpenChanged)
			.Placement(MenuPlacement_ComboBox)
			[
				SNew(SButton)
				.ClickMethod(EButtonClickMethod::MouseDown)
				.VAlign(VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "FlatButton")
				.ToolTipText(LOCTEXT("ShowExposureOptions_Tooltip", "Set the Exposure Options for this Input node."))
				.OnClicked(this, &SNiagaraGraphNodeInput::HandleExposureOptionsMenuButtonClicked)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("ComboButton.Arrow"))
						.Visibility(this, &SNiagaraGraphNodeInput::GetExposureOptionsVisibility)
					]
				]
			]
		];
}

void SNiagaraGraphNodeInput::BindCommands()
{
	const FNiagaraGraphNodeInputCommands& Commands = FNiagaraGraphNodeInputCommands::Get();

	ToolkitCommands->MapAction(
		Commands.ToggleExposed,
		FExecuteAction::CreateRaw(this, &SNiagaraGraphNodeInput::HandleExposedActionExecute),
		FCanExecuteAction::CreateRaw(this, &SNiagaraGraphNodeInput::HandleExposedActionCanExecute),
		FIsActionChecked::CreateRaw(this, &SNiagaraGraphNodeInput::HandleExposedActionIsChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleRequired,
		FExecuteAction::CreateRaw(this, &SNiagaraGraphNodeInput::HandleRequiredActionExecute),
		FCanExecuteAction::CreateRaw(this, &SNiagaraGraphNodeInput::HandleRequiredActionCanExecute),
		FIsActionChecked::CreateRaw(this, &SNiagaraGraphNodeInput::HandleRequiredActionIsChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleCanAutoBind,
		FExecuteAction::CreateRaw(this, &SNiagaraGraphNodeInput::HandleAutoBindActionExecute),
		FCanExecuteAction::CreateRaw(this, &SNiagaraGraphNodeInput::HandleAutoBindActionCanExecute),
		FIsActionChecked::CreateRaw(this, &SNiagaraGraphNodeInput::HandleAutoBindActionIsChecked));

	ToolkitCommands->MapAction(
		Commands.ToggleHidden,
		FExecuteAction::CreateRaw(this, &SNiagaraGraphNodeInput::HandleHiddenActionExecute),
		FCanExecuteAction::CreateRaw(this, &SNiagaraGraphNodeInput::HandleHiddenActionCanExecute),
		FIsActionChecked::CreateRaw(this, &SNiagaraGraphNodeInput::HandleHiddenActionIsChecked));
}

void SNiagaraGraphNodeInput::HandleExposedActionExecute()
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	const FScopedTransaction Transaction(LOCTEXT("ExposedChanged", "Toggle Input Node Exposure."));
	InputNode->Modify();
	InputNode->ExposureOptions.bExposed = !InputNode->ExposureOptions.bExposed;
	SynchronizeGraphNodes();
}

bool SNiagaraGraphNodeInput::HandleExposedActionCanExecute() const
{
	return true;
}

bool SNiagaraGraphNodeInput::HandleExposedActionIsChecked() const
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	return InputNode->ExposureOptions.bExposed;
}

void SNiagaraGraphNodeInput::HandleRequiredActionExecute()
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	const FScopedTransaction Transaction(LOCTEXT("RequiredChanged", "Toggle Input Node Required."));
	InputNode->Modify();
	InputNode->ExposureOptions.bRequired = !InputNode->ExposureOptions.bRequired;
	bRequestedSyncExposureOptions = true; // Deferred b/c updating InputNode pins with menu open crashes.
}

bool SNiagaraGraphNodeInput::HandleRequiredActionCanExecute() const
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	return InputNode->ExposureOptions.bExposed;
}

bool SNiagaraGraphNodeInput::HandleRequiredActionIsChecked() const
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	return InputNode->ExposureOptions.bRequired;
}

void SNiagaraGraphNodeInput::HandleAutoBindActionExecute()
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	const FScopedTransaction Transaction(LOCTEXT("AutoBindChanged", "Toggle Input Node Auto Bind."));
	InputNode->Modify();
	InputNode->ExposureOptions.bCanAutoBind = !InputNode->ExposureOptions.bCanAutoBind;
	bRequestedSyncExposureOptions = true; // Deferred b/c updating InputNode pins with menu open crashes.
}
bool SNiagaraGraphNodeInput::HandleAutoBindActionCanExecute() const
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	return InputNode->ExposureOptions.bExposed;
}
bool SNiagaraGraphNodeInput::HandleAutoBindActionIsChecked() const
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	return InputNode->ExposureOptions.bCanAutoBind;
}

void SNiagaraGraphNodeInput::HandleHiddenActionExecute()
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	const FScopedTransaction Transaction(LOCTEXT("HiddenChanged", "Toggle Input Node Hidden."));
	InputNode->Modify();
	InputNode->ExposureOptions.bHidden = !InputNode->ExposureOptions.bHidden;
	bRequestedSyncExposureOptions = true; // Deferred b/c updating InputNode pins with menu open crashes.
}
bool SNiagaraGraphNodeInput::HandleHiddenActionCanExecute() const
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	return InputNode->ExposureOptions.bExposed;
}
bool SNiagaraGraphNodeInput::HandleHiddenActionIsChecked() const
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	return InputNode->ExposureOptions.bHidden;
}

void SNiagaraGraphNodeInput::SynchronizeGraphNodes()
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	UNiagaraGraph* Graph = CastChecked<UNiagaraGraph>(InputNode->GetGraph());
	TArray<UNiagaraNodeInput*> InputNodes;
	Graph->GetNodesOfClass<UNiagaraNodeInput>(InputNodes);

	for (UNiagaraNodeInput* Node : InputNodes)
	{
		if (Node == InputNode)
		{
			Node->NotifyExposureOptionsChanged(); // We need to regenerate pins potentially.
			continue;
		}

		bool bNeedsSync = Node->ReferencesSameInput(InputNode);
		if (bNeedsSync)
		{
			Node->Modify();
			Node->ExposureOptions = InputNode->ExposureOptions;
			Node->NotifyExposureOptionsChanged(); // We need to regenerate pins potentially.
		}
	}
}


TSharedRef<SWidget> SNiagaraGraphNodeInput::GenerateExposureOptionsMenu() const
{
	FMenuBuilder OptionsMenuBuilder(true, ToolkitCommands);
	{
		OptionsMenuBuilder.BeginSection("InputNodeExposureOptions", LOCTEXT("OptionsMenuHeader", "Exposure Options"));
		{
			OptionsMenuBuilder.AddMenuEntry(FNiagaraGraphNodeInputCommands::Get().ToggleExposed);
			OptionsMenuBuilder.AddMenuEntry(FNiagaraGraphNodeInputCommands::Get().ToggleRequired);
			OptionsMenuBuilder.AddMenuEntry(FNiagaraGraphNodeInputCommands::Get().ToggleHidden);
		}

		OptionsMenuBuilder.EndSection();
	}
	return OptionsMenuBuilder.MakeWidget();
}

void SNiagaraGraphNodeInput::ExposureOptionsMenuOpenChanged(bool bOpened)
{
	// It isn't safe to trigger pins to update until the menu is going away.
	if (bRequestedSyncExposureOptions && !bOpened)
	{
		SynchronizeGraphNodes();
		bRequestedSyncExposureOptions = false;
	}
}

EVisibility SNiagaraGraphNodeInput::GetExposureOptionsVisibility() const
{
	UNiagaraNodeInput* InputNode = CastChecked<UNiagaraNodeInput>(GraphNode);
	return InputNode->Usage == ENiagaraInputNodeUsage::Parameter ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SNiagaraGraphNodeInput::HandleExposureOptionsMenuButtonClicked()
{
	if (ExposureOptionsMenuAnchor->ShouldOpenDueToClick())
	{
		ExposureOptionsMenuAnchor->SetIsOpen(true);
	}
	else
	{
		ExposureOptionsMenuAnchor->SetIsOpen(false);
	}

	return FReply::Handled();
}
#undef LOCTEXT_NAMESPACE