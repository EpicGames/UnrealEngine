// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SNiagaraGraphNode.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Framework/Commands/UICommandList.h"

/** A graph node widget representing a niagara input node. */
class SNiagaraGraphNodeInput : public SNiagaraGraphNode
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphNodeInput) {}
	SLATE_END_ARGS();

	SNiagaraGraphNodeInput();

	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	//~ SGraphNode api
	virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle) override;
	virtual bool IsNameReadOnly() const override;
	virtual void RequestRenameOnSpawn() override;

protected:

	/**
	* Generates the Exposure options menu widget.
	*/
	TSharedRef<SWidget> GenerateExposureOptionsMenu() const;

	void ExposureOptionsMenuOpenChanged(bool bOpened);

	void BindCommands();

	void HandleExposedActionExecute();
	bool HandleExposedActionCanExecute() const;
	bool HandleExposedActionIsChecked() const;

	void HandleRequiredActionExecute();
	bool HandleRequiredActionCanExecute() const;
	bool HandleRequiredActionIsChecked() const;

	void HandleAutoBindActionExecute();
	bool HandleAutoBindActionCanExecute() const;
	bool HandleAutoBindActionIsChecked() const;

	void HandleHiddenActionExecute();
	bool HandleHiddenActionCanExecute() const;
	bool HandleHiddenActionIsChecked() const;

	EVisibility GetExposureOptionsVisibility() const;

	void SynchronizeGraphNodes();
private:

	// Callback for clicking the Exposure Options menu button.
	FReply HandleExposureOptionsMenuButtonClicked();

	/** List of UI commands for this toolkit.  */
	TSharedRef<FUICommandList> ToolkitCommands;

	// Holds the anchor for the view options menu.
	TSharedPtr<SMenuAnchor> ExposureOptionsMenuAnchor;

	// Used to defer pin regeneration when exposure properties change.
	bool bRequestedSyncExposureOptions;
};
