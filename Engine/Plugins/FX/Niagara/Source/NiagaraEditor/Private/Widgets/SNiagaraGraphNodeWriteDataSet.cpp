// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphNodeWriteDataSet.h"
#include "NiagaraNodeInput.h"
#include "NiagaraGraph.h"
#include "Framework/Commands/Commands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Images/SImage.h"
#include "NiagaraNodeWriteDataSet.h"
#include "GraphEditorSettings.h"

#define LOCTEXT_NAMESPACE "SNiagaraGraphNodeWriteDataSet"


void SNiagaraGraphNodeWriteDataSet::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	GraphNode = InGraphNode; 
	RegisterNiagaraGraphNode(InGraphNode);

	this->UpdateGraphNode();
}

TSharedRef<SWidget> SNiagaraGraphNodeWriteDataSet::CreateNodeContentArea()
{
	TSharedRef<SWidget> ContentAreaWidget = SGraphNode::CreateNodeContentArea();
	TSharedPtr<SVerticalBox> VertContainer = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoWidth()
			.Padding(Settings->GetInputPinPadding())
			[
				SNew(STextBlock)
				.Text(LOCTEXT("EventName","Event Name"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.AutoWidth()
			[
				SNew(SEditableTextBox)
				.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitType)
				{
					if (UNiagaraNodeWriteDataSet* WriteDataSetNode = Cast<UNiagaraNodeWriteDataSet>(GraphNode))
					{
						WriteDataSetNode->EventName = FName(*NewText.ToString());
					}
				})
				.Text_Lambda([this]()
				{
					if (UNiagaraNodeWriteDataSet* WriteDataSetNode = Cast<UNiagaraNodeWriteDataSet>(GraphNode))
					{
						return FText::FromName(WriteDataSetNode->EventName);
					}
					return FText();
				})
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ContentAreaWidget
		];
	TSharedRef<SWidget> RetWidget = VertContainer.ToSharedRef();
	return RetWidget;
}



#undef LOCTEXT_NAMESPACE
