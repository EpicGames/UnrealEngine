// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SNiagaraGraphNode.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Framework/Commands/UICommandList.h"

/** A graph node widget representing a niagara write data set node. */
class SNiagaraGraphNodeWriteDataSet : public SNiagaraGraphNode
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphNodeWriteDataSet) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;
};
