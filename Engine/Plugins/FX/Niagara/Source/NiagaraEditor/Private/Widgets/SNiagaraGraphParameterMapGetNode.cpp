// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphParameterMapGetNode.h"
#include "NiagaraNodeParameterMapGet.h"
#include "Widgets/Input/SButton.h"
#include "GraphEditorSettings.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "SGraphPin.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraScriptVariable.h"
#include "SDropTarget.h"
#include "NiagaraEditorStyle.h"


#define LOCTEXT_NAMESPACE "SNiagaraGraphParameterMapGetNode"


void SNiagaraGraphParameterMapGetNode::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	BackgroundBrush = FAppStyle::GetBrush("Graph.Pin.Background");
	BackgroundHoveredBrush = FAppStyle::GetBrush("PlainBorder");

	GraphNode = InGraphNode; 
	RegisterNiagaraGraphNode(InGraphNode);

	UpdateGraphNode();
}

void SNiagaraGraphParameterMapGetNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner(SharedThis(this));

	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bAdvancedParameter = (PinObj != nullptr) && PinObj->bAdvancedView;
	const bool bInvisiblePin = (PinObj != nullptr) && PinObj->bDefaultValueIsReadOnly;
	if (bAdvancedParameter)
	{
		PinToAdd->SetVisibility(TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced));
	}

	// Save the UI building for later...
	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		if (bInvisiblePin)
		{
			//PinToAdd->SetOnlyShowDefaultValue(true);
			PinToAdd->SetPinColorModifier(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
		}
		InputPins.Add(PinToAdd);
	}
	else // Direction == EEdGraphPinDirection::EGPD_Output
	{
		OutputPins.Add(PinToAdd);
	}	
}

TSharedRef<SWidget> SNiagaraGraphParameterMapGetNode::CreateNodeContentArea()
{
	// NODE CONTENT AREA
	return 	SNew(SDropTarget)
		.OnDropped(this, &SNiagaraGraphParameterMapGetNode::OnDroppedOnTarget)
		.OnAllowDrop(this, &SNiagaraGraphParameterMapGetNode::OnAllowDrop)
		.Content()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(0, 3))
			[
				SAssignNew(PinContainerRoot, SVerticalBox)
			]
		];
}

void SNiagaraGraphParameterMapGetNode::CreatePinWidgets()
{
	SGraphNode::CreatePinWidgets();
	
	UNiagaraNodeParameterMapGet* GetNode = Cast<UNiagaraNodeParameterMapGet>(GraphNode);
	
	TSet<TSharedRef<SGraphPin>> AddedPins;

	auto AddRowWidgetToNode = [this](TSharedPtr<SWidget> RowWidget)
	{
		TSharedRef<SBorder> Border = SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(0, 3))
			//.OnMouseButtonDown(this, &SNiagaraGraphParameterMapGetNode::OnBorderMouseButtonDown, i)
			[
				RowWidget.ToSharedRef()
			];
		
		Border->SetBorderImage(TAttribute<const FSlateBrush*>::Create(TAttribute<const FSlateBrush*>::FGetter::CreateRaw(this, &SNiagaraGraphParameterMapGetNode::GetBackgroundBrush, RowWidget)));

		PinContainerRoot->AddSlot()
		.AutoHeight()
		[
			Border
		];
	};
	
	// add the source pin on the first row
	TSharedPtr<SGraphPin> SourcePin = nullptr;
	if(InputPins.Num() > 0)
	{
		SourcePin = InputPins[0];
	}
		
	TSharedPtr<SWidget> SourceRowWidget = SNew(SHorizontalBox)
		.Visibility(EVisibility::Visible)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.FillWidth(1.0f)
		.Padding(Settings->GetInputPinPadding())
		[
			(SourcePin.IsValid() ? SourcePin.ToSharedRef() : SNullWidget::NullWidget)
		]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(Settings->GetOutputPinPadding())
		[
			SNullWidget::NullWidget
		];

	if(SourcePin.IsValid())
	{
		AddedPins.Add(InputPins[0]);
	}

	AddRowWidgetToNode(SourceRowWidget);
	
	// Deferred pin adding to line up input/output pins by persistent guid mapping.
	for (int32 i = 0; i < OutputPins.Num(); i++)
	{
		TSharedRef<SGraphPin> OutputPin = OutputPins[i];
		UEdGraphPin* SrcOutputPin = OutputPin->GetPinObj(); 

		UEdGraphPin* MatchingInputPin = GetNode->GetDefaultPin(SrcOutputPin);

		TSharedPtr<SGraphPin> InputPin = nullptr;
		for (TSharedRef<SGraphPin> Pin : InputPins)
		{
			UEdGraphPin* SrcInputPin = Pin->GetPinObj();
			if (SrcInputPin == MatchingInputPin)
			{
				InputPin = Pin;
				Pin->SetShowLabel(false);
			}
		}

		TSharedPtr<SWidget> ParameterRow = SNew(SHorizontalBox)
			.Visibility(EVisibility::Visible)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(1.0f)
			.Padding(Settings->GetInputPinPadding())
			[
				(InputPin.IsValid() ? InputPin.ToSharedRef() : SNullWidget::NullWidget)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.Padding(Settings->GetOutputPinPadding())
			[
				OutputPin
			];

		if(InputPin.IsValid())
		{
			AddedPins.Add(InputPin.ToSharedRef());
		}
		AddedPins.Add(OutputPin);

		AddRowWidgetToNode(ParameterRow);
	}

	// for each unadded input pin (for example orphaned pins), we add them back at the bottom
	for (int32 i = 0; i < InputPins.Num(); i++)
	{
		if(!AddedPins.Contains(InputPins[i]))
		{
			TSharedRef<SGraphPin> UnaddedInputPin = InputPins[i];
			// since we don't want to waste space, we look for some output pin we also haven't added yet
			TSharedRef<SGraphPin>* SomeUnaddedOutputPin = OutputPins.FindByPredicate([&](TSharedRef<SGraphPin> Pin)
			{
				return !AddedPins.Contains(Pin);
			});

			TSharedPtr<SWidget> LeftoverPinContainer = SNew(SHorizontalBox)
				.Visibility(EVisibility::Visible)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.FillWidth(1.0f)
				.Padding(Settings->GetInputPinPadding())
				[
					UnaddedInputPin
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.Padding(Settings->GetOutputPinPadding())
				[
					(SomeUnaddedOutputPin != nullptr ? (*SomeUnaddedOutputPin) : SNullWidget::NullWidget)
				];

			AddedPins.Add(UnaddedInputPin);
			if(SomeUnaddedOutputPin != nullptr)
			{
				AddedPins.Add(*SomeUnaddedOutputPin);
			}
			
			AddRowWidgetToNode(LeftoverPinContainer);
		}
	}
}


const FSlateBrush* SNiagaraGraphParameterMapGetNode::GetBackgroundBrush(TSharedPtr<SWidget> Border) const
{
	return Border->IsHovered() ? BackgroundHoveredBrush	: BackgroundBrush;
}


FReply SNiagaraGraphParameterMapGetNode::OnBorderMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, int32 InWhichPin)
{
	if (InWhichPin >= 0 && InWhichPin < OutputPins.Num() + 1)
	{
		UNiagaraNodeParameterMapGet* GetNode = Cast<UNiagaraNodeParameterMapGet>(GraphNode);
		if (GetNode)
		{
			UNiagaraGraph* Graph = GetNode->GetNiagaraGraph();
			if (Graph && InWhichPin > 0)
			{
				const UEdGraphSchema_Niagara* Schema = Graph->GetNiagaraSchema();
				if (Schema)
				{
					FNiagaraVariable Var = Schema->PinToNiagaraVariable(OutputPins[InWhichPin-1]->GetPinObj());
					TObjectPtr<UNiagaraScriptVariable>* PinAssociatedScriptVariable = Graph->GetAllMetaData().Find(Var);
					if (PinAssociatedScriptVariable != nullptr)
					{
						Graph->OnSubObjectSelectionChanged().Broadcast(*PinAssociatedScriptVariable);
					}
				}
			}
		}

	}
	return FReply::Unhandled();
}

FReply SNiagaraGraphParameterMapGetNode::OnDroppedOnTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragDropOperation = InDragDropEvent.GetOperation();
	if (DragDropOperation)
	{
		UNiagaraNodeParameterMapBase* MapNode = Cast<UNiagaraNodeParameterMapBase>(GraphNode);
		if (MapNode != nullptr && MapNode->HandleDropOperation(DragDropOperation))
		{
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

bool SNiagaraGraphParameterMapGetNode::OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	UNiagaraNodeParameterMapBase* MapNode = Cast<UNiagaraNodeParameterMapBase>(GraphNode);
	return MapNode != nullptr && MapNode->CanHandleDropOperation(DragDropOperation);
}

#undef LOCTEXT_NAMESPACE
