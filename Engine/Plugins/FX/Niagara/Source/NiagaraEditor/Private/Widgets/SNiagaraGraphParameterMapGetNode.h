// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SNiagaraGraphNode.h"

class SNiagaraGraphParameterMapGetNode : public SNiagaraGraphNode
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphParameterMapGetNode) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	virtual void CreatePinWidgets() override;

protected:
	FReply OnBorderMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, int32 InWhichPin);
	const FSlateBrush* GetBackgroundBrush(TSharedPtr<SWidget> Border) const;
	FReply OnDroppedOnTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);
	bool OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation);
	TSharedPtr<SVerticalBox> PinContainerRoot;

	const FSlateBrush* BackgroundBrush;
	const FSlateBrush* BackgroundHoveredBrush;

};