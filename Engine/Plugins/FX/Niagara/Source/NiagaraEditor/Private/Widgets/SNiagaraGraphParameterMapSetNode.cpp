// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphParameterMapSetNode.h"
#include "NiagaraNodeParameterMapSet.h"
#include "GraphEditorSettings.h"
#include "SGraphPin.h"
#include "EdGraphSchema_Niagara.h"
#include "SDropTarget.h"


#define LOCTEXT_NAMESPACE "SNiagaraGraphParameterMapSetNode"


void SNiagaraGraphParameterMapSetNode::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{

	GraphNode = InGraphNode; 
	RegisterNiagaraGraphNode(InGraphNode);

	UpdateGraphNode();
}

TSharedRef<SWidget> SNiagaraGraphParameterMapSetNode::CreateNodeContentArea()
{
	TSharedRef<SWidget> ExistingContent = SNiagaraGraphNode::CreateNodeContentArea();
	// NODE CONTENT AREA
	return 	SNew(SDropTarget)
		.OnDropped(this, &SNiagaraGraphParameterMapSetNode::OnDroppedOnTarget)
		.OnAllowDrop(this, &SNiagaraGraphParameterMapSetNode::OnAllowDrop)
		.Content()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(FMargin(0, 3))
			[
				ExistingContent
			]
		];
}

FReply SNiagaraGraphParameterMapSetNode::OnDroppedOnTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragDropOperation = InDragDropEvent.GetOperation();
	if (DragDropOperation)
	{
		UNiagaraNodeParameterMapBase* MapNode = Cast<UNiagaraNodeParameterMapBase>(GraphNode);
		if (MapNode != nullptr && MapNode->HandleDropOperation(DragDropOperation))
		{
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

bool SNiagaraGraphParameterMapSetNode::OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	UNiagaraNodeParameterMapBase* MapNode = Cast<UNiagaraNodeParameterMapBase>(GraphNode);
	return MapNode != nullptr && MapNode->CanHandleDropOperation(DragDropOperation);
}

#undef LOCTEXT_NAMESPACE
