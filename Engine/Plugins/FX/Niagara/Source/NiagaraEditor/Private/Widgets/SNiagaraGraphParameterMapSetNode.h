// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SNiagaraGraphNode.h"

class SNiagaraGraphParameterMapSetNode : public SNiagaraGraphNode
{
public:
	SLATE_BEGIN_ARGS(SNiagaraGraphParameterMapSetNode) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;

protected:
	FReply OnDroppedOnTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);
	bool OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation);

};