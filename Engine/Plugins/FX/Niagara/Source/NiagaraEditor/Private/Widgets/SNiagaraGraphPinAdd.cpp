// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraGraphPinAdd.h"

#include "EdGraphSchema_Niagara.h"
#include "NiagaraConstants.h"
#include "NiagaraNodeCustomHlsl.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraNodeWithDynamicPins.h"
#include "NiagaraScript.h"
#include "NiagaraScriptSource.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "Widgets/SNiagaraParameterMenu.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "NiagaraGraphPinAdd"

void SNiagaraGraphPinAdd::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SetShowLabel(false);
	OwningNode = Cast<UNiagaraNodeWithDynamicPins>(InGraphPinObj->GetOwningNode());
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
	TSharedPtr<SHorizontalBox> PinBox = GetFullPinHorizontalRowWidget().Pin();
	if (PinBox.IsValid())
	{
		if (InGraphPinObj->Direction == EGPD_Input)
		{
			PinBox->AddSlot()
			[
				ConstructAddButton()
			];
		}
		else
		{
			PinBox->InsertSlot(0)
			[
				ConstructAddButton()
			];
		}
	}
}

TSharedRef<SWidget>	SNiagaraGraphPinAdd::ConstructAddButton()
{
	AddButton = SNew(SComboButton)
		.HasDownArrow(false)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ForegroundColor(FSlateColor::UseForeground())
		.OnGetMenuContent(this, &SNiagaraGraphPinAdd::OnGetAddButtonMenuContent)
		.ContentPadding(FMargin(2))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ToolTipText(LOCTEXT("AddPinButtonToolTip", "Connect this pin to add a new typed pin, or choose from the drop-down."))
		.ButtonContent()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::GetBrush("Plus"))
		];

	return AddButton.ToSharedRef();
}

TSharedRef<SWidget> SNiagaraGraphPinAdd::OnGetAddButtonMenuContent()
{
	if (OwningNode == nullptr)
	{
		return SNullWidget::NullWidget;
	}

	UEdGraphPin* AddPin = GetPinObj();
	const UEdGraphPin* ConstAddPin = AddPin;

	if(OwningNode->IsA<UNiagaraNodeParameterMapBase>())
	{
		// Collect args for add menu widget construct
		UNiagaraGraph* OwningNodeGraph = CastChecked<UNiagaraGraph>(OwningNode->GetOuter());
		TArray<UNiagaraGraph*> InGraphs = { OwningNodeGraph };

		UNiagaraScriptSource* OwningScriptSource = CastChecked<UNiagaraScriptSource>(OwningNodeGraph->GetOuter());
		UNiagaraScript* OwningScript = CastChecked<UNiagaraScript>(OwningScriptSource->GetOuter());

		TArray<TSharedPtr<FNiagaraScriptViewModel>> ExistingViewModels;
		TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>::GetAllViewModelsForObject(OwningScript, ExistingViewModels);
		if (ExistingViewModels.Num() != 1)
		{
			ensureMsgf(false, TEXT("Missing or duplicate script view models detected.  Can not create SNiagaraGraphPinAdd button menu content."));
			return SNullWidget::NullWidget;
		}

		TSharedPtr<FNiagaraScriptViewModel> ScriptViewModel = ExistingViewModels[0];
		const bool bSkipSubscribedLibraries = false;

		// Collect all parameter names that are already on this map node and cull them from the add menu to be summoned.
		TArray<UEdGraphPin*> Pins;
		if (OwningNode->GetPinDirectionForNewParameters() == EEdGraphPinDirection::EGPD_Input)
		{
			OwningNode->GetInputPins(Pins);
		}
		else // PinDirectionForNewParameters == EEdGraphPinDirection::EGDP_Output
		{
			OwningNode->GetOutputPins(Pins);
		}

		TSet<FName> AdditionalCulledParameterNames;
		const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();
		for (UEdGraphPin* Pin : Pins)
		{
			if (OwningNode->IsAddPin(Pin) == false && Schema->PinToTypeDefinition(Pin) != FNiagaraTypeDefinition::GetParameterMapDef())
			{
				AdditionalCulledParameterNames.Add(Pin->GetFName());
			}
		}

		// Create the add menu
		TSharedRef<SNiagaraAddParameterFromPanelMenu> MenuWidget = SNew(SNiagaraAddParameterFromPanelMenu)
		.Graphs(InGraphs)
		.AvailableParameterDefinitions(ScriptViewModel->GetAvailableParameterDefinitions(bSkipSubscribedLibraries))
		.SubscribedParameterDefinitions(ScriptViewModel->GetSubscribedParameterDefinitions())
		.OnNewParameterRequested_UObject(OwningNode, &UNiagaraNodeWithDynamicPins::AddParameter, ConstAddPin->Direction)
		.OnSpecificParameterRequested_UObject(OwningNode, &UNiagaraNodeWithDynamicPins::AddExistingParameter, ConstAddPin)
		.OnAddScriptVar_UObject(OwningNode, &UNiagaraNodeWithDynamicPins::AddParameter, ConstAddPin->Direction)
		.OnAddParameterDefinitions(ScriptViewModel.ToSharedRef(), &FNiagaraScriptViewModel::SubscribeToParameterDefinitions)
		.OnAllowMakeType_UObject(OwningNode, &UNiagaraNodeWithDynamicPins::AllowNiagaraTypeForAddPin) 
		.AllowCreatingNew(true)
		.ShowNamespaceCategory(false)
		.ShowGraphParameters(true)
		.AutoExpandMenu(false)
		.IsParameterRead(AddPin->Direction == EEdGraphPinDirection::EGPD_Input ? false : true)
		.ForceCollectEngineNamespaceParameterActions(true)
		.AdditionalCulledParameterNames(AdditionalCulledParameterNames);

		AddButton->SetMenuContentWidgetToFocus(MenuWidget->GetSearchBox());
		return MenuWidget;
	}
	else
	{
		TSharedRef<SNiagaraAddParameterFromPinMenu> MenuWidget = SNew(SNiagaraAddParameterFromPinMenu)
		.NiagaraNode(OwningNode)
		.AddPin(AddPin);

		AddButton->SetMenuContentWidgetToFocus(MenuWidget->GetSearchBox());
		return MenuWidget;
	}
}


#undef LOCTEXT_NAMESPACE
