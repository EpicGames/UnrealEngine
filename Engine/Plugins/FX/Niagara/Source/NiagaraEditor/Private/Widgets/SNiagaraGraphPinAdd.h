// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "NiagaraTypes.h"

class UNiagaraNodeWithDynamicPins;
struct FNiagaraTypeDefinition;

/** A graph pin for adding additional pins a dynamic niagara node. */
class SNiagaraGraphPinAdd : public SGraphPin
{
	SLATE_BEGIN_ARGS(SNiagaraGraphPinAdd) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	
	/** Returns the owning node of this add pin. */
	UNiagaraNodeWithDynamicPins* GetOwningNode() { return OwningNode; }

private:
	TSharedRef<SWidget>	ConstructAddButton();

	TSharedRef<SWidget> OnGetAddButtonMenuContent();
	
	TSharedPtr<class SComboButton> AddButton;

private:
	UNiagaraNodeWithDynamicPins* OwningNode;
};