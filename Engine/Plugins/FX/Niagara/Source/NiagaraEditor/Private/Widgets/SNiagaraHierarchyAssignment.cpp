// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraHierarchyAssignment.h"

#include "NiagaraEditorUtilities.h"
#include "NiagaraNodeAssignment.h"
#include "SlateOptMacros.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SNiagaraHierarchyAssignment::Construct(const FArguments& InArgs, UNiagaraNodeAssignment& InAssignmentNode)
{
	if(InAssignmentNode.GetAssignmentTargets().Num() > 0)
	{
		TSharedRef<SHorizontalBox> DisplayWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString("Set:"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			FNiagaraParameterUtilities::GetParameterWidget(InAssignmentNode.GetAssignmentTargets()[0], false, false)
		];

		if(InAssignmentNode.GetAssignmentTargets().Num() > 1)
		{
			DisplayWidget->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FormatOrdered(FText::FromString("(+{0})"), InAssignmentNode.GetAssignmentTargets().Num() - 1))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];
		}
		
		ChildSlot
		[
			DisplayWidget
		];
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
