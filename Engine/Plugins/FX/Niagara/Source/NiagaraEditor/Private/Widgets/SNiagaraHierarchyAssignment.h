// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraTypes.h"
#include "Widgets/SCompoundWidget.h"

/**
 * 
 */
class SNiagaraHierarchyAssignment : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraHierarchyAssignment)
	{
	}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, class UNiagaraNodeAssignment& InAssignmentNode);
};
