// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraHierarchyModuleInput.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraScriptVariable.h"
#include "SlateOptMacros.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

#define LOCTEXT_NAMESPACE "SNiagaraHierarchyModuleInput"

void SNiagaraHierarchyModuleInput::Construct(const FArguments& InArgs, TSharedRef<FNiagaraModuleInputViewModel> InputViewModel)
{
	InputViewModelWeakPtr = InputViewModel;
	FNiagaraParameterUtilities::FNiagaraParameterWidgetOptions Options;
	Options.bShowVisibilityConditionIcon = true;
	Options.bShowEditConditionIcon = true;
	Options.bShowAdvanced = true;
	Options.NameOverride = TAttribute<FText>::CreateSP(this, &SNiagaraHierarchyModuleInput::GetDisplayNameOverride);
	Options.NameOverrideVisibility = TAttribute<EVisibility>::CreateSP(this, &SNiagaraHierarchyModuleInput::GetDisplayNameOverrideVisibility);
	Options.NameOverrideTooltip = TAttribute<FText>::CreateSP(this, &SNiagaraHierarchyModuleInput::GetDisplayNameOverrideTooltip);

	FInputData InputData = InputViewModel->GetInputData().GetValue();

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			FNiagaraParameterUtilities::GetParameterWidget(FNiagaraVariable(InputData.Type, InputData.InputName), InputData.MetaData, Options)
		]
	];
}

FText SNiagaraHierarchyModuleInput::GetDisplayNameOverride() const
{
	if(InputViewModelWeakPtr.IsValid())
	{
		return InputViewModelWeakPtr.Pin()->GetSummaryInputNameOverride();
	}

	return FText::GetEmpty();
}

EVisibility SNiagaraHierarchyModuleInput::GetDisplayNameOverrideVisibility() const
{
	return GetDisplayNameOverride().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; 
}

FText SNiagaraHierarchyModuleInput::GetDisplayNameOverrideTooltip() const
{
	return FText::FormatOrdered(LOCTEXT("InputNameOverrideTooltip", "This input will be displayed as {0} in the summary view."), GetDisplayNameOverride());
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
