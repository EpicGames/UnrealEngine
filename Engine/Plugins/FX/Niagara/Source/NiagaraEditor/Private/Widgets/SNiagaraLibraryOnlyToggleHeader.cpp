// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraLibraryOnlyToggleHeader.h"
#include "SGraphActionMenu.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

void SNiagaraLibraryOnlyToggleHeader::Construct(const FArguments& InArgs)
{
	LibraryOnly = InArgs._LibraryOnly;
	LibraryOnlyChanged = InArgs._LibraryOnlyChanged;
	ChildSlot
	[
		SNew(SHorizontalBox)

		// Search context description
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Visibility(InArgs._bShowHeaderLabel ? EVisibility::Visible : EVisibility::Collapsed)
			.Text(InArgs._HeaderLabelText)
		]

		// Library Only Toggle
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &SNiagaraLibraryOnlyToggleHeader::OnCheckStateChanged)
			.IsChecked(this, &SNiagaraLibraryOnlyToggleHeader::GetCheckState)
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("LibraryOnlyToggle", "LibraryOnly", "Library Only"))
			]
		]
	];
}

void SNiagaraLibraryOnlyToggleHeader::SetActionMenu(TSharedRef<SGraphActionMenu> InActionMenu)
{
	ActionMenuWeak = InActionMenu;
}

void SNiagaraLibraryOnlyToggleHeader::OnCheckStateChanged(ECheckBoxState InCheckState)
{
	LibraryOnlyChanged.ExecuteIfBound(InCheckState == ECheckBoxState::Checked);
	TSharedPtr<SGraphActionMenu> ActionMenu = ActionMenuWeak.Pin();
	if (ActionMenu.IsValid())
	{
		ActionMenu->RefreshAllActions(true, false);
	}
}

ECheckBoxState SNiagaraLibraryOnlyToggleHeader::GetCheckState() const
{
	return LibraryOnly.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}