// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraTypes.h"
#include "NiagaraEditorStyle.h"
#include "EdGraphSchema_Niagara.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Input/Events.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

/** A widget which allows the user to enter a digit or choose a number from a drop down menu. Adapted from SNumericDropDown. */
template<typename NumericType>
class SNiagaraNumericDropDown : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnValueChanged, NumericType );

public:
	/** Represents a named numeric value for display in the drop down menu. */
	class FNamedValue
	{
	public:
		/** Creates a new FNamedValue 
		  * @InValue The numeric value to be assigned
		  * @InName The display text for the value in the UI 
		  * @InDescription The description of the value used in tooltips or wherever a longer description is needed. */
		FNamedValue( NumericType InValue, FText InName, FText InDescription )
		{
			Value = InValue;
			Name = InName;
			Description = InDescription;
		}

		NumericType GetValue() const
		{
			return Value;
		}

		FText GetName() const
		{
			return Name;
		}

		FText GetDescription() const
		{
			return Description;
		}

	private:
		NumericType Value;
		FText Name;
		FText Description;
	};

public:
	SLATE_BEGIN_ARGS( SNiagaraNumericDropDown<NumericType> )
		: _DropDownValues()
		, _bAllowTyping(true)
		, _MinDesiredValueWidth( 40 )
		, _bShowNamedValue(false)
		{}

		/** The values which are used to populate the drop down menu. */
		SLATE_ATTRIBUTE( TArray<FNamedValue>, DropDownValues )
		/** If set to false, will disable the text box and only allow choosing a value from the dropdown. */
		SLATE_ARGUMENT( bool, bAllowTyping )
		/** Controls the minimum width for the text box portion of the control. */
		SLATE_ATTRIBUTE( float, MinDesiredValueWidth )
		/** Toggle to show the drop down text value if the value matches the numeric value. */
		SLATE_ATTRIBUTE( bool, bShowNamedValue )
		/** The type to display for the pill icon. */
		SLATE_ATTRIBUTE( FNiagaraTypeDefinition, PillType )
		/** The value displayed by the control. */
		SLATE_ATTRIBUTE( NumericType, Value )
		/** The callback for when the value changes. */
		SLATE_EVENT( FOnValueChanged, OnValueChanged )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		DropDownValues = InArgs._DropDownValues;
		bAllowTyping = InArgs._bAllowTyping;
		bShowNamedValue = InArgs._bShowNamedValue;
		Value = InArgs._Value;
		OnValueChanged = InArgs._OnValueChanged;

		TSharedPtr<SWidget> ValueWidget;

		if(bAllowTyping)
		{
			ValueWidget = SNew(SEditableTextBox)
			.Padding(0)
			.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.Stack.NumericDropdownInput")
			.MinDesiredWidth( InArgs._MinDesiredValueWidth )
			.RevertTextOnEscape(true)
			.SelectAllTextWhenFocused(true)
			.Text( this, &SNiagaraNumericDropDown<NumericType>::GetValueText )
			.OnTextCommitted( this, &SNiagaraNumericDropDown<NumericType>::ValueTextComitted );
		}
		else
		{
			ValueWidget = SNew(STextBlock)
			.MinDesiredWidth( InArgs._MinDesiredValueWidth )
			.Text( this, &SNiagaraNumericDropDown<NumericType>::GetValueText )
			.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.Stack.IntegerAsEnum");
		}
		
		ChildSlot
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
			.VAlign( EVerticalAlignment::VAlign_Center )
			.AutoWidth()
			.Padding( FMargin( 0, 0, 3, 0 ) )
			[
				SNew(SImage)
				.ColorAndOpacity(UEdGraphSchema_Niagara::GetTypeColor(InArgs._PillType.Get(FNiagaraTypeDefinition::GetFloatDef())))
				.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.TypeIconPill"))
				.Visibility(bAllowTyping ? EVisibility::Visible : EVisibility::Collapsed)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( SComboButton )
				.OnGetMenuContent( this, &SNiagaraNumericDropDown<NumericType>::BuildMenu )
				.ButtonContent()
				[
					ValueWidget.ToSharedRef()
				]
			]
		];

		SetToolTipText(TAttribute<FText>::CreateSP(this, &SNiagaraNumericDropDown::GetValueTooltip));
	}

private:

	FText GetValueText() const
	{
		if (bShowNamedValue.Get())
		{
			for ( FNamedValue DropDownValue : DropDownValues.Get() )
			{
				if (FMath::IsNearlyEqual(static_cast<float>(DropDownValue.GetValue()), static_cast<float>(Value.Get())))
				{
					return DropDownValue.GetName();
				}
			}
		}
		return FText::AsNumber( Value.Get() );
	}

	void ValueTextComitted( const FText& InNewText, ETextCommit::Type InTextCommit )
	{
		if ( InNewText.IsNumeric() )
		{
			NumericType NewValue;
			TTypeFromString<NumericType>::FromString( NewValue, *InNewText.ToString() );
			OnValueChanged.ExecuteIfBound(NewValue);
		}
	}

	FText GetValueTooltip() const
	{
		if (bShowNamedValue.Get())
		{
			for ( FNamedValue DropDownValue : DropDownValues.Get() )
			{
				if (FMath::IsNearlyEqual(static_cast<float>(DropDownValue.GetValue()), static_cast<float>(Value.Get())))
				{
					return DropDownValue.GetDescription();
				}
			}
		}
		
		return FText::GetEmpty();
	}

	TSharedRef<SWidget> BuildMenu()
	{
		FMenuBuilder MenuBuilder( true, NULL );

		for ( FNamedValue DropDownValue : DropDownValues.Get() )
		{
			FUIAction MenuAction(FExecuteAction::CreateSP(this, &SNiagaraNumericDropDown::SetValue, DropDownValue.GetValue()));
			MenuBuilder.AddMenuEntry(DropDownValue.GetName(), DropDownValue.GetDescription(), FSlateIcon(), MenuAction);
		}

		return MenuBuilder.MakeWidget();
	}

	void SetValue( NumericType InValue )
	{
		FSlateApplication::Get().ClearKeyboardFocus( EFocusCause::Cleared );
		OnValueChanged.ExecuteIfBound(InValue);
	}

	TAttribute<TArray<FNamedValue>> DropDownValues;
	bool bAllowTyping = true;
	TAttribute<bool> bShowNamedValue;
	TAttribute<NumericType> Value;
	FOnValueChanged OnValueChanged;
};
