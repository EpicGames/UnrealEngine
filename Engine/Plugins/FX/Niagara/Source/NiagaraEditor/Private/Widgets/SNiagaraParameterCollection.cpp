// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraParameterCollection.h"
#include "NiagaraEditorModule.h"
#include "ViewModels/NiagaraParameterCollectionViewModel.h"
#include "ViewModels/NiagaraParameterViewModel.h"
#include "NiagaraTypes.h"
#include "INiagaraEditorTypeUtilities.h"
#include "SNiagaraParameterEditor.h"
#include "NiagaraEditorStyle.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSplitter.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Framework/Commands/GenericCommands.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "NiagaraParameterCollectionEditor"

class SSimpleExpander : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSimpleExpander)
		:_IsExpanded(false)
	{}
		SLATE_ARGUMENT(bool, IsExpanded)
		SLATE_NAMED_SLOT(FArguments, Header)
		SLATE_NAMED_SLOT(FArguments, Body)

	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs)
	{
		bIsExpanded = InArgs._IsExpanded;
		ExpandedImage = FCoreStyle::Get().GetBrush("TreeArrow_Expanded");
		CollapsedImage = FCoreStyle::Get().GetBrush("TreeArrow_Collapsed");

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0)
				[
					SNew(SButton)
					.ButtonStyle(FCoreStyle::Get(), "NoBorder")
					.OnClicked(this, &SSimpleExpander::ExpandButtonClicked)
					.ForegroundColor(FSlateColor::UseForeground())
					[
						SNew(SImage)
						.Image(this, &SSimpleExpander::GetExpandButtonImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot()
				[
					InArgs._Header.Widget
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.Visibility(this, &SSimpleExpander::GetBodyVisibility)
				[
					InArgs._Body.Widget
				]
			]
		];
	}
private:
	FReply ExpandButtonClicked()
	{
		bIsExpanded = !bIsExpanded;
		return FReply::Handled();
	}

	const FSlateBrush* GetExpandButtonImage() const
	{
		return bIsExpanded ? ExpandedImage : CollapsedImage;
	}

	EVisibility GetBodyVisibility() const
	{
		return bIsExpanded ? EVisibility::Visible : EVisibility::Collapsed;
	}

private:
	bool bIsExpanded;
	const FSlateBrush* ExpandedImage;
	const FSlateBrush* CollapsedImage;
};


	

//////////////////////////////////////////////////////////////////////////
// FParamCollectionDragDropAction

class FParamCollectionDragDropAction : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FParamCollectionDragDropAction, FDragDropOperation)

	FText BodyText;
	FScopedTransaction Transaction;

	// FDragDropOperation interface
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;
	virtual void Construct() override;
	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
	// End of FDragDropOperation interface

	FText GetBodyText() const { return BodyText; }

	void SetHoverTargetItem(TSharedRef<INiagaraParameterViewModel> DropItem, EItemDropZone DropZone);

	void SetDefaultTooltip();

	void SetCanDropHere(bool bCanDropHere)
	{
		MouseCursor = bCanDropHere ? EMouseCursor::TextEditBeam : EMouseCursor::SlashedCircle;
	}

	static TSharedRef<FParamCollectionDragDropAction> New();

protected:
	FParamCollectionDragDropAction();
};

TSharedPtr<SWidget> FParamCollectionDragDropAction::GetDefaultDecorator() const
{
	return SNew(SBox)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			[
				SNew(STextBlock)
				.ColorAndOpacity(FAppStyle::GetColor("DefaultForeground"))
				.Text(this, &FParamCollectionDragDropAction::GetBodyText)
			]
		];
}

void FParamCollectionDragDropAction::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition());
	}
}


void FParamCollectionDragDropAction::Construct()
{
	MouseCursor = EMouseCursor::GrabHandClosed;

	SetDefaultTooltip();

	FDragDropOperation::Construct();
}

void FParamCollectionDragDropAction::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	if (!bDropWasHandled)
	{
		Transaction.Cancel();
	}
}

void FParamCollectionDragDropAction::SetDefaultTooltip()
{
	BodyText = LOCTEXT("DragDropHoverDefault", "Cannot drop here");
}

void FParamCollectionDragDropAction::SetHoverTargetItem(TSharedRef<INiagaraParameterViewModel> DropItem, EItemDropZone DropZone)
{
	switch (DropZone)
	{
	default:
	case EItemDropZone::AboveItem:
		BodyText = FText::Format(LOCTEXT("DragDropHoverTextBefore", "Place before {0}"), DropItem->GetNameText());
		break;
	case EItemDropZone::OntoItem:
		BodyText = FText::Format(LOCTEXT("DragDropHoverTextOnto", "Place onto {0}"), DropItem->GetNameText());
		break;
	case EItemDropZone::BelowItem:
		BodyText = FText::Format(LOCTEXT("DragDropHoverTextAfter", "Place after {0}"), DropItem->GetNameText());
		break;
	}
}

TSharedRef<FParamCollectionDragDropAction> FParamCollectionDragDropAction::New()
{
	// Create the drag-drop op containing the key
	TSharedRef<FParamCollectionDragDropAction> Operation = MakeShareable(new FParamCollectionDragDropAction);
	Operation->Construct();

	return Operation;
}

FParamCollectionDragDropAction::FParamCollectionDragDropAction()
: Transaction(LOCTEXT("MovedParametersInList", "Reorder parameters"))
{

}


void SNiagaraParameterCollection::Construct(const FArguments& InArgs, TSharedRef<INiagaraParameterCollectionViewModel> InCollection)
{
	Collection = InCollection;
	Collection->OnCollectionChanged().AddSP(this, &SNiagaraParameterCollection::ViewModelCollectionChanged);
	Collection->GetSelection().OnSelectedObjectsChanged().AddSP(this, &SNiagaraParameterCollection::ViewModelSelectionChanged);
	Collection->OnExpandedChanged().AddSP(this, &SNiagaraParameterCollection::ViewModelIsExpandedChanged);

	NameColumnWidth = InArgs._NameColumnWidth;
	ContentColumnWidth = InArgs._ContentColumnWidth;
	OnNameColumnWidthChanged = InArgs._OnNameColumnWidthChanged;
	OnContentColumnWidthChanged = InArgs._OnContentColumnWidthChanged;

	BindCommands();

	bUpdatingListSelectionFromViewModel = false;

	SAssignNew(ParameterListView, SListView<TSharedRef<INiagaraParameterViewModel>>)
		.ListItemsSource(&Collection->GetParameters())
		.OnGenerateRow(this, &SNiagaraParameterCollection::OnGenerateRowForParameter)
		.OnSelectionChanged(this, &SNiagaraParameterCollection::OnParameterListSelectionChanged);

	if (Collection->GetParameters().Num() > 0)
	{
		ParameterListView->SetSelection(Collection->GetParameters()[0], ESelectInfo::Direct);
	}

	ChildSlot
	[
		SAssignNew(ExpandableArea, SExpandableArea)
		.InitiallyCollapsed(false)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
		.OnAreaExpansionChanged(this, &SNiagaraParameterCollection::AreaExpandedChanged)
		.Padding(0)
		.HeaderContent()
		[
			SAssignNew(HeaderBox, SBox)
			[
				//~ Title
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Collection.ToSharedRef(), &INiagaraParameterCollectionViewModel::GetDisplayName)
				]
				//~ Add button
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SAssignNew(AddButton, SComboButton)
					.HasDownArrow(false)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ForegroundColor(FSlateColor::UseForeground())
					.OnGetMenuContent(this, &SNiagaraParameterCollection::GetAddMenuContent)
					.Visibility(Collection.ToSharedRef(), &INiagaraParameterCollectionViewModel::GetAddButtonVisibility)
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 1, 2, 1)
						[
							SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(FAppStyle::GetBrush("Plus"))
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "SmallText")
							.Text(Collection.ToSharedRef(), &INiagaraParameterCollectionViewModel::GetAddButtonText)
							.Visibility(this, &SNiagaraParameterCollection::GetAddButtonTextVisibility)
						]
					]
				]
			]
		]
		.BodyContent()
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				ParameterListView.ToSharedRef()
			]
		]
	];

}

void SNiagaraParameterCollection::BindCommands()
{
	Commands = MakeShareable(new FUICommandList());
	Commands->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(Collection.ToSharedRef(), &INiagaraParameterCollectionViewModel::DeleteSelectedParameters),
		FCanExecuteAction::CreateSP(Collection.ToSharedRef(), &INiagaraParameterCollectionViewModel::CanDeleteParameters));
}

FReply SNiagaraParameterCollection::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SNiagaraParameterCollection::ViewModelCollectionChanged()
{
	ParameterListView->RequestListRefresh();
}

void SNiagaraParameterCollection::ViewModelSelectionChanged()
{
	if (FNiagaraEditorUtilities::ArrayMatchesSet(ParameterListView->GetSelectedItems(), Collection->GetSelection().GetSelectedObjects()) == false)
	{
		bUpdatingListSelectionFromViewModel = true;
		{
			ParameterListView->ClearSelection();
			for (TSharedRef<INiagaraParameterViewModel> Parameter : Collection->GetSelection().GetSelectedObjects())
			{
				ParameterListView->SetItemSelection(Parameter, true);
			}
		}
		bUpdatingListSelectionFromViewModel = false;
	}
}

void SNiagaraParameterCollection::ViewModelIsExpandedChanged()
{
	// we ignore the expansion state for the root entry, otherwise the user wouldn't be able to see anything in a collapsed state
	ExpandableArea->SetExpanded(true);
}

void SNiagaraParameterCollection::AreaExpandedChanged(bool bIsExpanded)
{
	Collection->SetIsExpanded(bIsExpanded);
}

EVisibility SNiagaraParameterCollection::GetAddButtonTextVisibility() const
{
	return HeaderBox->IsHovered() || AddButton->IsOpen() ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SNiagaraParameterCollection::GetAddMenuContent()
{
	FMenuBuilder AddMenuBuilder(true, nullptr);
	TSortedMap<FString, TArray<TSharedPtr<FNiagaraTypeDefinition>>> SubmenusToAdd;
	for (TSharedPtr<FNiagaraTypeDefinition> AvailableType : Collection->GetAvailableTypesSorted())
	{
		if (AvailableType->GetStruct() != nullptr || AvailableType->GetEnum() != nullptr)
		{
			FText SubmenuText = FNiagaraEditorUtilities::GetTypeDefinitionCategory(*AvailableType);
			if (SubmenuText.IsEmptyOrWhitespace())
			{
				AddMenuBuilder.AddMenuEntry
	            (
	                AvailableType->GetNameText(),
	                FText(),
	                FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(Collection.ToSharedRef(), &INiagaraParameterCollectionViewModel::AddParameter, AvailableType))
	            );
			}
			else
			{
				SubmenusToAdd.FindOrAdd(SubmenuText.ToString()).Add(AvailableType);
			}
		}
	}
	for (const auto& Entry : SubmenusToAdd)
	{
		TArray<TSharedPtr<FNiagaraTypeDefinition>> SubmenuEntries = Entry.Value;
		AddMenuBuilder.AddSubMenu(FText::FromString(Entry.Key), FText(), FNewMenuDelegate::CreateLambda([SubmenuEntries, this](FMenuBuilder& InSubMenuBuilder)
        {
			for (TSharedPtr<FNiagaraTypeDefinition> AvailableType : SubmenuEntries)
			{
				InSubMenuBuilder.AddMenuEntry
                (
                    AvailableType->GetNameText(),
                    FText(),
                    FSlateIcon(),
                    FUIAction(FExecuteAction::CreateSP(Collection.ToSharedRef(), &INiagaraParameterCollectionViewModel::AddParameter, AvailableType))
                );
			}
        }));
	}
	return AddMenuBuilder.MakeWidget();
}

FReply SNiagaraParameterCollection::DeleteParameter(TSharedRef<INiagaraParameterViewModel> Item) const
{
	Collection->DeleteParameters({FNiagaraVariable(Item->GetType(), Item->GetName())});
	return FReply::Handled();
}

TSharedRef<ITableRow> SNiagaraParameterCollection::OnGenerateRowForParameter(TSharedRef<INiagaraParameterViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	// Name widget
	TSharedPtr<SHorizontalBox> NameWidget;
	SAssignNew(NameWidget, SHorizontalBox);

	if (Item->IsOptional())
	{
		NameWidget->AddSlot()
			.AutoWidth()
			.Padding(FMargin(3, 0, 0, 0))
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(Item, &INiagaraParameterViewModel::SetProvided)
				.IsChecked(Item, &INiagaraParameterViewModel::IsProvided)
				//How do I grey all the other stuff out if it's optional but not provided?
			];
	}
	if (Item->CanRenameParameter())
	{
		NameWidget->AddSlot()
			.AutoWidth()
			.Padding(FMargin(3, 0, 0, 0))
			[
				SNew(SInlineEditableTextBlock)
				.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterInlineEditableText")
				.Text(Item, &INiagaraParameterViewModel::GetNameText)
				.OnVerifyTextChanged(Item, &INiagaraParameterViewModel::VerifyNodeNameTextChanged)
				.OnTextCommitted(Item, &INiagaraParameterViewModel::NameTextCommitted)
				.IsSelected(this, &SNiagaraParameterCollection::IsItemSelected, Item)
				.IsEnabled(TAttribute<bool>(Item, &INiagaraParameterViewModel::IsEditingEnabled))
			];
	}
	else
	{
		NameWidget->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(3, 0, 0, 0))
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
				.Text(Item, &INiagaraParameterViewModel::GetNameText)
			];
	}

//#define DEBUG_SORT_ORDER
#if defined(DEBUG_SORT_ORDER)
	TSharedPtr<SWidget> SortOrderWidget;
	
	SortOrderWidget = SNew(SSpinBox<int32>)
		.Delta(1)
		.MinValue(0)
		.MaxValue(INT_MAX)
		.IsEnabled(false)
		.Value(Item, &INiagaraParameterViewModel::GetSortOrder);
#endif


	// Details and parameter editor widgets.
	TSharedPtr<SWidget> CustomValueEditor;
	TSharedPtr<SWidget> DetailsWidget;
	if (Item->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::Struct)
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		FNiagaraTypeDefinition ParameterType = Item->GetType();
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(ParameterType);
		TSharedPtr<SNiagaraParameterEditor> ParameterEditor;
		if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanCreateParameterEditor())
		{
			ParameterEditor = TypeEditorUtilities->CreateParameterEditor(ParameterType);
			ParameterEditor->UpdateInternalValueFromStruct(Item->GetDefaultValueStruct());
			ParameterEditor->SetOnBeginValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(
				this, &SNiagaraParameterCollection::ParameterEditorBeginValueChange, Item));
			ParameterEditor->SetOnEndValueChange(SNiagaraParameterEditor::FOnValueChange::CreateSP(
				this, &SNiagaraParameterCollection::ParameterEditorEndValueChange, Item));
			ParameterEditor->SetOnValueChanged(SNiagaraParameterEditor::FOnValueChange::CreateSP(
				this, &SNiagaraParameterCollection::ParameterEditorValueChanged, ParameterEditor.ToSharedRef(), Item));
		}

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;

		TSharedRef<IStructureDetailsView> StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(
			DetailsViewArgs,
			FStructureDetailsViewArgs(),
			nullptr);

		StructureDetailsView->SetStructureData(Item->GetDefaultValueStruct());
		StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(Item, &INiagaraParameterViewModel::NotifyDefaultValuePropertyChanged);
		
		Item->OnDefaultValueChanged().AddSP(this, &SNiagaraParameterCollection::ParameterViewModelDefaultValueChanged, Item, ParameterEditor, StructureDetailsView);
		Item->OnTypeChanged().AddSP(this, &SNiagaraParameterCollection::ParameterViewModelTypeChanged);

		CustomValueEditor = ParameterEditor;
		DetailsWidget = StructureDetailsView->GetWidget();
		DetailsWidget->SetEnabled(Item->IsEditingEnabled());
	}
	else if (Item->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::DataInterface)
	{
		FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
		TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> TypeEditorUtilities = NiagaraEditorModule.GetTypeUtilities(Item->GetType());
		if (TypeEditorUtilities.IsValid() && TypeEditorUtilities->CanCreateDataInterfaceEditor())
		{
			CustomValueEditor = TypeEditorUtilities->CreateDataInterfaceEditor(Item->GetDefaultValueDataInterface(),
				INiagaraEditorTypeUtilities::FNotifyValueChanged::CreateSP(Item, &INiagaraParameterViewModel::NotifyDefaultValueChanged));
		}

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.NotifyHook = this;

		TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObject(Item->GetDefaultValueDataInterface());
		DetailsWidget = DetailsView;
	}
	else if (Item->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::ObjectAsset)
	{
		const FNiagaraTypeDefinition ParameterType = Item->GetType();

		CustomValueEditor = SNew(SObjectPropertyEntryBox)
			.AllowedClass(ParameterType.GetClass())
			.ObjectPath_Lambda(
				[Item]() -> FString
				{
					TObjectPtr<UObject> DefaultValue = Item->GetDefaultValueObjectAsset();
					return DefaultValue ? DefaultValue.GetPathName() : FString();
				}
			)
			.DisplayBrowse(true)
			.DisplayUseSelected(true)
			.DisplayThumbnail(true)
			.EnableContentPicker(true)
			.OnObjectChanged_Lambda(
				[Item](const FAssetData& AssetData)
				{
					Item->SetDefaultValueObjectAsset(AssetData.GetAsset());
				}
			);
		DetailsWidget = SNullWidget::NullWidget;
	}
	else
	{
		DetailsWidget = SNullWidget::NullWidget;
	}

	TSharedRef<SWidget> ParameterDeleteButton = SNew(SButton)
		.OnClicked(this, &SNiagaraParameterCollection::DeleteParameter, Item)
		.Visibility(this, &SNiagaraParameterCollection::ShowParameterDeleteButton, Item)
		.Content()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Delete"))
		];
	
	if (CustomValueEditor.IsValid())
	{
		// Should the value of the parameter be enabled?
		CustomValueEditor->SetEnabled(Item->IsEditingEnabled());

		return SNew(STableRow<TSharedRef<INiagaraParameterViewModel>>, OwnerTable)
		//.IsEnabled(TAttribute<bool>(Item, &INiagaraParameterViewModel::IsEditingEnabled))
		.ToolTipText(TAttribute<FText>(Item, &INiagaraParameterViewModel::GetTooltip))
		.OnCanAcceptDrop(this, &SNiagaraParameterCollection::OnItemCanAcceptDrop)
		.OnAcceptDrop(this, &SNiagaraParameterCollection::OnItemAcceptDrop)
		.OnDragDetected(this, &SNiagaraParameterCollection::OnItemDragDetected)
		.OnDragEnter(this, &SNiagaraParameterCollection::OnItemDragEnter, Item)
		.OnDragLeave(this, &SNiagaraParameterCollection::OnItemDragLeave, Item)
		.Padding(FMargin(2, 3, 2, 3))
		.Content()
		[
			SNew(SSimpleExpander)
			.IsExpanded(false)
			.Header()
			[
				SNew(SSplitter)
				+ SSplitter::Slot()
				.Value(NameColumnWidth)
				.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SNiagaraParameterCollection::ParameterNameColumnWidthChanged))
				[
					NameWidget.ToSharedRef()
				]
				+ SSplitter::Slot()
				.Value(ContentColumnWidth)
				.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SNiagaraParameterCollection::ParameterContentColumnWidthChanged))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(FMargin(3, 0, 0, 0))
					[
						CustomValueEditor.ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(4.f)
					[
						ParameterDeleteButton
					]
#if defined(DEBUG_SORT_ORDER)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(FMargin(3, 0, 0, 0))
					[
						SortOrderWidget.ToSharedRef()
					]
#endif
				]
			]
			.Body()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(25, 2, 0, 0)
				[
					DetailsWidget.ToSharedRef()
				]
			]
		];
	}
	else
	{
		return SNew(STableRow<TSharedRef<INiagaraParameterViewModel>>, OwnerTable)
		//.IsEnabled(TAttribute<bool>(Item, &INiagaraParameterViewModel::IsEditingEnabled))
		.ToolTipText(TAttribute<FText>(Item, &INiagaraParameterViewModel::GetTooltip))
		.OnCanAcceptDrop(this, &SNiagaraParameterCollection::OnItemCanAcceptDrop)
		.OnAcceptDrop(this, &SNiagaraParameterCollection::OnItemAcceptDrop)
		.OnDragDetected(this, &SNiagaraParameterCollection::OnItemDragDetected)
		.OnDragEnter(this, &SNiagaraParameterCollection::OnItemDragEnter, Item)
		.OnDragLeave(this, &SNiagaraParameterCollection::OnItemDragLeave, Item)
		.Padding(FMargin(2))
		.Content()
		[
			SNew(SSimpleExpander)
			.IsExpanded(true)
			.Header()
			[
				SNew(SSplitter)
				+ SSplitter::Slot()
				.Value(NameColumnWidth)
				.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SNiagaraParameterCollection::ParameterNameColumnWidthChanged))
				[
					NameWidget.ToSharedRef()
				]
				+ SSplitter::Slot()
				.Value(ContentColumnWidth)
				.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SNiagaraParameterCollection::ParameterContentColumnWidthChanged))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.Padding(4.f)
					[
						ParameterDeleteButton
					]
#if defined(DEBUG_SORT_ORDER)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(FMargin(3, 0, 0, 0))
					[
						SortOrderWidget.ToSharedRef()
					]
#endif
				]
			]
			.Body()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(25, 2, 0, 0)
				[
					DetailsWidget.ToSharedRef()
				]
			]
		];
	}
}

TSharedRef<SWidget> SNiagaraParameterCollection::OnGenerateWidgetForTypeComboBox(TSharedPtr<FNiagaraTypeDefinition> Item)
{
	return SNew(STextBlock)
		.Text(Collection->GetTypeDisplayName(Item));
}

void SNiagaraParameterCollection::ParameterViewModelDefaultValueChanged(TSharedRef<INiagaraParameterViewModel> Item, TSharedPtr<SNiagaraParameterEditor> ParameterEditor, TSharedRef<IStructureDetailsView> StructureDetailsView)
{
	if (ParameterEditor.IsValid())
	{
		ParameterEditor->UpdateInternalValueFromStruct(Item->GetDefaultValueStruct());

		// Only update the details view if the parameter editor isn't currently the exclusive editor.  This hack is necessary because the details 
		// view closes all color pickers when it's changed! */
		if (ParameterEditor->GetIsEditingExclusively() == false)
		{
			StructureDetailsView->SetStructureData(Item->GetDefaultValueStruct());
		}
	}
}


FReply SNiagaraParameterCollection::OnItemDragDetected(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
{
	const TSet<TSharedRef<INiagaraParameterViewModel>>& SelectedItems = Collection->GetSelection().GetSelectedObjects();
	if (SelectedItems.Num() > 0)
	{
		bool bAllAreMovable = true;
		for (const TSharedRef<INiagaraParameterViewModel>& ParamViewModel : SelectedItems)
		{
			if (ParamViewModel->CanChangeSortOrder() == false)
			{
				bAllAreMovable = false;
			}
		}
		
		if (bAllAreMovable)
		{
			TSharedRef<FParamCollectionDragDropAction> Operation = FParamCollectionDragDropAction::New();

			return FReply::Handled().BeginDragDrop(Operation);
		}
	}

	return FReply::Unhandled();
}

void SNiagaraParameterCollection::OnItemDragEnter(const FDragDropEvent& DragDropEvent, TSharedRef<INiagaraParameterViewModel> DropItem)
{
	
}

void SNiagaraParameterCollection::OnItemDragLeave(const FDragDropEvent& DragDropEvent, TSharedRef<INiagaraParameterViewModel> DropItem)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return;
	}

	if (Operation->IsOfType<FParamCollectionDragDropAction>())
	{
		// Inform the Drag and Drop operation that we are hovering over nothing.
		TSharedPtr<FParamCollectionDragDropAction> DragConnectionOp = StaticCastSharedPtr<FParamCollectionDragDropAction>(Operation);
		DragConnectionOp->SetDefaultTooltip();
	}
}

TOptional<EItemDropZone> SNiagaraParameterCollection::OnItemCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedRef<INiagaraParameterViewModel> DropItem)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	
	// In sorting order, onto doesn't make any sense so we don't support it.
	if (DropZone == EItemDropZone::OntoItem)
	{
		DropZone = EItemDropZone::AboveItem;
	}
	
	if (!Operation.IsValid())
	{
		return DropZone;
	}

	if (Operation->IsOfType<FParamCollectionDragDropAction>())
	{
		// Inform the Drag and Drop operation that we are hovering over this entry if it isn't a selected item.
		TSharedPtr<FParamCollectionDragDropAction> DragConnectionOp = StaticCastSharedPtr<FParamCollectionDragDropAction>(Operation);
		if (!IsItemSelected(DropItem))
		{
			DragConnectionOp->SetHoverTargetItem(DropItem, DropZone);
		}
		else // Otherwise, this is an invalid drop
		{
			DragConnectionOp->SetDefaultTooltip();
		}
	}
	return DropZone;
}

FReply SNiagaraParameterCollection::OnItemAcceptDrop(FDragDropEvent const& DragDropEvent, EItemDropZone DropZone, TSharedRef<INiagaraParameterViewModel> DropItem)
{
	bool bWasDropHandled = false;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid() && Operation->IsOfType<FParamCollectionDragDropAction>())
	{
		// Doesn't make sense to drop onto yourself, so ignore those drops
		if (!IsItemSelected(DropItem))
		{
			const auto& FrameDragDropOp = StaticCastSharedPtr<FParamCollectionDragDropAction>(Operation);
			TArray<TSharedRef<INiagaraParameterViewModel>> SelectedItems = ParameterListView->GetSelectedItems();
			INiagaraParameterCollectionViewModel::SortViewModels(SelectedItems);

			TArray<TSharedRef<INiagaraParameterViewModel>> AllItems = Collection->GetParameters();

			// Remove the se;ected items...
			for (int32 i = 0; i < SelectedItems.Num(); i++)
			{
				AllItems.Remove(SelectedItems[i]);
			}

			// Figure out where in the list we want to insert
			int32 ItemIdx = AllItems.Find(DropItem);

			if (DropZone == EItemDropZone::BelowItem)
			{
				ItemIdx++;
			}

			// Insert all the items into the list at the target location.
			AllItems.Insert(SelectedItems, ItemIdx);

			// Tell everyone their new sort order
			for (int32 i = 0; i < AllItems.Num(); i++)
			{
				AllItems[i]->SetSortOrder(i);
			}
			// Refreshing will re-build the list, taking into account the proper sort order.
			Collection->RefreshParameterViewModels();
			bWasDropHandled = true;
		}
	}

	return bWasDropHandled ? FReply::Handled() : FReply::Unhandled();
}

EVisibility SNiagaraParameterCollection::ShowParameterDeleteButton(TSharedRef<INiagaraParameterViewModel> Item) const
{
	// if we can add, we can also remove, unless it's an externally provided parameter
	if (Item->IsEditingEnabled())
	{
		return Collection->GetAddButtonVisibility();
	}

	return EVisibility::Hidden;
}

void SNiagaraParameterCollection::ParameterViewModelTypeChanged()
{
	if (ParameterListView.IsValid())
	{
		ParameterListView->RequestListRefresh();
	}
}

void SNiagaraParameterCollection::OnParameterListSelectionChanged(TSharedPtr<INiagaraParameterViewModel> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (bUpdatingListSelectionFromViewModel == false)
	{
		Collection->GetSelection().SetSelectedObjects(ParameterListView->GetSelectedItems());
	}
}

bool SNiagaraParameterCollection::IsItemSelected(TSharedRef<INiagaraParameterViewModel> Item)
{
	return Collection->GetSelection().GetSelectedObjects().Contains(Item);
}

void SNiagaraParameterCollection::ParameterEditorBeginValueChange(TSharedRef<INiagaraParameterViewModel> Item)
{
	Item->NotifyBeginDefaultValueChange();
}

void SNiagaraParameterCollection::ParameterEditorEndValueChange(TSharedRef<INiagaraParameterViewModel> Item)
{
	Item->NotifyEndDefaultValueChange();
}

void SNiagaraParameterCollection::ParameterEditorValueChanged(TSharedRef<SNiagaraParameterEditor> ParameterEditor, TSharedRef<INiagaraParameterViewModel> Item)
{
	ParameterEditor->UpdateStructFromInternalValue(Item->GetDefaultValueStruct());
	Item->NotifyDefaultValueChanged();
}

void SNiagaraParameterCollection::ParameterNameColumnWidthChanged(float Width)
{
	if (NameColumnWidth.IsBound() == false)
	{
		NameColumnWidth.Set(Width);
	}
	OnNameColumnWidthChanged.ExecuteIfBound(Width);
}

void SNiagaraParameterCollection::ParameterContentColumnWidthChanged(float Width)
{
	if (ContentColumnWidth.IsBound() == false)
	{
		ContentColumnWidth.Set(Width);
	}
	OnContentColumnWidthChanged.ExecuteIfBound(Width);
}

void SNiagaraParameterCollection::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (PropertyChangedEvent.GetNumObjectsBeingEdited() == 0)
	{
		return;
	}

	for (TSharedRef<INiagaraParameterViewModel> Parameter : Collection->GetParameters())
	{
		if (Parameter->GetDefaultValueType() == INiagaraParameterViewModel::EDefaultValueType::DataInterface)
		{
			UObject* ParameterObject = Parameter->GetDefaultValueDataInterface();
			const UObject* ChangedObject = PropertyChangedEvent.GetObjectBeingEdited(0);
			const UObject* CurrentObject = ChangedObject;
			bool ParameterIsInObjectChain = false;
			while (ParameterIsInObjectChain == false && CurrentObject != nullptr)
			{
				if (ParameterObject == CurrentObject)
				{
					ParameterIsInObjectChain = true;
				}
				else
				{
					CurrentObject = CurrentObject->GetOuter();
				}
			}
			if (ParameterIsInObjectChain)
			{
				// Calling this could lead to the entire script being recompiled and the parameters list being reset.
				Parameter->NotifyDefaultValuePropertyChanged(PropertyChangedEvent);
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE // NiagaraParameterCollectionEditor
