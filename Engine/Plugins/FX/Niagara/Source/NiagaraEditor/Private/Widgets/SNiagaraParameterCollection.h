// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Misc/NotifyHook.h"
#include "Widgets/Views/SListView.h"


class INiagaraParameterCollectionViewModel;
class INiagaraParameterViewModel;
class IStructureDetailsView;
class SNiagaraParameterEditor;
struct FNiagaraTypeDefinition;
struct FNiagaraVariable;

class FUICommandList;
class ITableRow;
class STableViewBase;
class SExpandableArea;
class SBox;
class SComboButton;

/** A widget for editing parameter collections. */
class SNiagaraParameterCollection : public SCompoundWidget, public FNotifyHook
{
public:
	DECLARE_DELEGATE_OneParam(FOnColumnWidthChanged, float);

public:
	SLATE_BEGIN_ARGS(SNiagaraParameterCollection)
		: _NameColumnWidth(.3f)
		, _ContentColumnWidth(.7f)
	{}
		SLATE_ATTRIBUTE(float, NameColumnWidth)
		SLATE_ATTRIBUTE(float, ContentColumnWidth)
		SLATE_EVENT(FOnColumnWidthChanged, OnNameColumnWidthChanged)
		SLATE_EVENT(FOnColumnWidthChanged, OnContentColumnWidthChanged)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<INiagaraParameterCollectionViewModel> InCollection);

protected:
	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	/** Sets up the commands for the editor. */
	void BindCommands();

	/** Handles the view model parameter collection changing. */
	void ViewModelCollectionChanged();

	/** Handles the view model selection changing .*/
	void ViewModelSelectionChanged();

	/** Handles the view model expanded state changing .*/
	void ViewModelIsExpandedChanged();

	/** Handles the expander widget expanded state changing .*/
	void AreaExpandedChanged(bool bIsExpanded);

	/** Provides the visibility of the add button text. */
	EVisibility GetAddButtonTextVisibility() const;

	/** Gets the content for the add menu. */
	TSharedRef<SWidget> GetAddMenuContent();

	FReply DeleteParameter(TSharedRef<INiagaraParameterViewModel> Item) const;

	/** Generates a row for the parameter list view. */
	TSharedRef<ITableRow> OnGenerateRowForParameter(TSharedRef<INiagaraParameterViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Generates a widget for an entry in the type combo box. */
	TSharedRef<SWidget> OnGenerateWidgetForTypeComboBox(TSharedPtr<FNiagaraTypeDefinition> Item);

	/** Handles a change to a property by a parameter details view. */
	void ParameterViewModelDefaultValueChanged(TSharedRef<INiagaraParameterViewModel> Item, TSharedPtr<SNiagaraParameterEditor> ParameterEditor, TSharedRef<IStructureDetailsView> StructureDetailsView);

	/** Called when the type of a parameter changes. */
	void ParameterViewModelTypeChanged();

	/** Called whenever the list view selection changes. */
	void OnParameterListSelectionChanged(TSharedPtr<INiagaraParameterViewModel> SelectedItem, ESelectInfo::Type SelectInfo);

	/** Gets whether or not a parameter is selected. */
	bool IsItemSelected(TSharedRef<INiagaraParameterViewModel> Item);

	/** Called when a parameter editor begins a continuous change to the parameter value. */
	void ParameterEditorBeginValueChange(TSharedRef<INiagaraParameterViewModel> Item);

	/** Called when a parameter editor ends a continuous change to the parameter value. */
	void ParameterEditorEndValueChange(TSharedRef<INiagaraParameterViewModel> Item);

	/** Called when a parameter editor changes the parameter value. */
	void ParameterEditorValueChanged(TSharedRef<SNiagaraParameterEditor> ParameterEditor, TSharedRef<INiagaraParameterViewModel> Item);

	/** Called when the width of the parameter name column changes. */
	void ParameterNameColumnWidthChanged(float Width);

	/** Called when the width of the parameter name column changes. */
	void ParameterContentColumnWidthChanged(float Width);

	//~ FNotifyHook interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

	//~ Drag and drop interfaces for the view models 
	FReply OnItemDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void OnItemDragEnter(const FDragDropEvent& DragDropEvent, TSharedRef<INiagaraParameterViewModel> DropItem);
	void OnItemDragLeave(const FDragDropEvent& DragDropEvent, TSharedRef<INiagaraParameterViewModel> DropItem);
	TOptional<EItemDropZone> OnItemCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedRef<INiagaraParameterViewModel> DropItem);
	FReply OnItemAcceptDrop(FDragDropEvent const& DragDropEvent, EItemDropZone DropZone, TSharedRef<INiagaraParameterViewModel> DropItem);

	EVisibility ShowParameterDeleteButton(TSharedRef<INiagaraParameterViewModel> Item) const;

private:
	/** The view model for the parameter collection. */
	TSharedPtr<INiagaraParameterCollectionViewModel> Collection;

	/** The outer expander widget. */
	TSharedPtr<SExpandableArea> ExpandableArea;

	/** The box widget containing the header content. */
	TSharedPtr<SBox> HeaderBox;

	/** The button which adds parameters. */
	TSharedPtr<SComboButton> AddButton;

	/** The list view which displays the parameters. */
	TSharedPtr<SListView<TSharedRef<INiagaraParameterViewModel>>> ParameterListView;

	/** The commands registered for the parameter editor. */
	TSharedPtr<FUICommandList> Commands;

	/** A flag to prevent reentrancy when synchronizing selection between the UI and the view model. */
	bool bUpdatingListSelectionFromViewModel;

	/** The width coefficient of the name column. */
	TAttribute<float> NameColumnWidth;

	/** The width coefficient of the second column. */
	TAttribute<float> ContentColumnWidth;

	/** Delegate which is called when the name column width changes. */
	FOnColumnWidthChanged OnNameColumnWidthChanged;

	/** Delegate which is called when the second column width changes. */
	FOnColumnWidthChanged OnContentColumnWidthChanged;
};
