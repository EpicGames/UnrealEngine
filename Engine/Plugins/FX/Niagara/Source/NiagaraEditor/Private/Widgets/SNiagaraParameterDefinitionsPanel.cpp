// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraParameterDefinitionsPanel.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "EdGraphSchema_Niagara.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraScriptVariable.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/SNiagaraPinTypeSelector.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "ViewModels/NiagaraParameterDefinitionsPanelViewModel.h"

#define LOCTEXT_NAMESPACE "NiagaraParameterDefinitionsPanel"

TArray<FNiagaraParameterDefinitionsPanelCategory> SNiagaraParameterDefinitionsPanel::DefaultCategoryNoLibraries = { FNiagaraParameterDefinitionsPanelCategory(FText::FromString("No Parameter Definitions"), FGuid()) };
//@todo(ng) touchup

NIAGARAEDITOR_API SNiagaraParameterDefinitionsPanel::~SNiagaraParameterDefinitionsPanel()
{
	ParameterDefinitionsPanelViewModel->GetOnRequestRefreshDelegate().Unbind();
}

NIAGARAEDITOR_API void SNiagaraParameterDefinitionsPanel::Construct(const FArguments& InArgs, const TSharedPtr<INiagaraParameterDefinitionsPanelViewModel>& InParameterDefinitionsPanelViewModel, const TSharedPtr<FUICommandList>& InToolkitCommands)
{
	ToolkitCommands = InToolkitCommands;
	ParameterDefinitionsPanelViewModel = InParameterDefinitionsPanelViewModel;
	ParameterDefinitionsPanelViewModel->GetOnRequestRefreshDelegate().BindSP(this, &SNiagaraParameterDefinitionsPanel::Refresh);

	SAssignNew(ItemSelector, SNiagaraParameterDefinitionsPanelSelector)
		.Items(GetViewedParameterDefinitionsItems())
		.OnGetCategoriesForItem(this, &SNiagaraParameterDefinitionsPanel::OnGetCategoriesForItem)
		.OnCompareCategoriesForEquality(this, &SNiagaraParameterDefinitionsPanel::OnCompareCategoriesForEquality)
		.OnCompareCategoriesForSorting(this, &SNiagaraParameterDefinitionsPanel::OnCompareCategoriesForSorting)
		.OnCompareItemsForEquality(this, &SNiagaraParameterDefinitionsPanel::OnCompareItemsForEquality)
		.OnCompareItemsForSorting(this, &SNiagaraParameterDefinitionsPanel::OnCompareItemsForSorting)
		.OnDoesItemMatchFilterText(this, &SNiagaraParameterDefinitionsPanel::OnDoesItemMatchFilterText)
		.OnGenerateWidgetForCategory(this, &SNiagaraParameterDefinitionsPanel::OnGenerateWidgetForCategory)
		.OnGenerateWidgetForItem(this, &SNiagaraParameterDefinitionsPanel::OnGenerateWidgetForItem)
		.OnContextMenuOpening(this, &SNiagaraParameterDefinitionsPanel::OnContextMenuOpening)
		.OnItemSelected(this, &SNiagaraParameterDefinitionsPanel::OnParameterItemSelected)
		.OnItemsDragged(this, &SNiagaraParameterDefinitionsPanel::OnParameterItemsDragged)
		.AllowMultiselect(false)
		.DefaultCategories(GetDefaultCategories())
		.ClearSelectionOnClick(true)
		.CategoryRowStyle(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
		.OnGetCategoryBackgroundImage(this, &SNiagaraParameterDefinitionsPanel::GetCategoryBackgroundImage)
		.CategoryBorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
		.CategoryChildSlotPadding(FMargin(0.0f, 2.0f, 0.0f, 0.0f))
		.CategoryBorderBackgroundPadding(FMargin(0.0f, 3.0f))
		.SearchBoxAdjacentContent()
		[
			CreateAddLibraryButton()
		];
	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(300)
		[
			ItemSelector.ToSharedRef()
		]
	];
}

TArray<FNiagaraParameterDefinitionsPanelCategory> SNiagaraParameterDefinitionsPanel::OnGetCategoriesForItem(const FNiagaraParameterDefinitionsPanelItem& Item)
{
	return { FNiagaraParameterDefinitionsPanelCategory(Item.ParameterDefinitionsNameText, Item.ParameterDefinitionsUniqueId) };
}

bool SNiagaraParameterDefinitionsPanel::OnCompareCategoriesForEquality(const FNiagaraParameterDefinitionsPanelCategory& CategoryA, const FNiagaraParameterDefinitionsPanelCategory& CategoryB) const
{
	return CategoryA.ParameterDefinitionsUniqueId == CategoryB.ParameterDefinitionsUniqueId;
}

bool SNiagaraParameterDefinitionsPanel::OnCompareCategoriesForSorting(const FNiagaraParameterDefinitionsPanelCategory& CategoryA, const FNiagaraParameterDefinitionsPanelCategory& CategoryB) const
{
	return CategoryA.ParameterDefinitionsNameText.CompareTo(CategoryB.ParameterDefinitionsNameText) > 0;
}

TArray<FNiagaraParameterDefinitionsPanelItem> SNiagaraParameterDefinitionsPanel::GetViewedParameterDefinitionsItems() const
{
	TArray<FNiagaraParameterDefinitionsPanelItem> OutItems;
	const TArray<UNiagaraParameterDefinitions*> ParameterDefinitionsAssets = ParameterDefinitionsPanelViewModel->GetParameterDefinitionsAssets();
	for (const UNiagaraParameterDefinitions* ParameterDefinitions : ParameterDefinitionsAssets)
	{
		for (const UNiagaraScriptVariable* ScriptVar : ParameterDefinitions->GetParametersConst())
		{
			const FNiagaraNamespaceMetadata NamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(ScriptVar->Variable.GetName());
			const FText ParameterDefinitionsNameText = FText::FromString(*ParameterDefinitions->GetName());
			const FGuid& ParameterDefinitionsId = ParameterDefinitions->GetDefinitionsUniqueId();
			OutItems.Emplace(ScriptVar, NamespaceMetaData, ParameterDefinitionsNameText, ParameterDefinitionsId);
		}
	}
	return OutItems;
}

bool SNiagaraParameterDefinitionsPanel::OnCompareItemsForEquality(const FNiagaraParameterDefinitionsPanelItem& ItemA, const FNiagaraParameterDefinitionsPanelItem& ItemB) const
{
	return ItemA.GetVariable() == ItemB.GetVariable();
}

bool SNiagaraParameterDefinitionsPanel::OnCompareItemsForSorting(const FNiagaraParameterDefinitionsPanelItem& ItemA, const FNiagaraParameterDefinitionsPanelItem& ItemB) const
{
	return ItemA.GetVariable().GetName().LexicalLess(ItemB.GetVariable().GetName());
}

bool SNiagaraParameterDefinitionsPanel::OnDoesItemMatchFilterText(const FText& FilterText, const FNiagaraParameterDefinitionsPanelItem& Item)
{
	return Item.GetVariable().GetName().ToString().Contains(FilterText.ToString());
}

TSharedRef<SWidget> SNiagaraParameterDefinitionsPanel::OnGenerateWidgetForCategory(const FNiagaraParameterDefinitionsPanelCategory& Category)
{
	return SNew(SBox)
	.Padding(FMargin(3, 0, 0, 0))
	[
		SNew(SRichTextBlock)
		.Text(Category.ParameterDefinitionsNameText)
		.DecoratorStyleSet(&FAppStyle::Get())
		.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
	];
}

TSharedRef<SWidget> SNiagaraParameterDefinitionsPanel::OnGenerateWidgetForItem(const FNiagaraParameterDefinitionsPanelItem& Item)
{
	// Generate the icon widget.
	FText			   IconToolTip = FText::GetEmpty();
	FSlateBrush const* IconBrush = FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
	const FLinearColor TypeColor = UEdGraphSchema_Niagara::GetTypeColor(Item.GetVariable().GetType());
	FSlateColor        IconColor = FSlateColor(TypeColor);
	FString			   IconDocLink, IconDocExcerpt;
	FSlateBrush const* SecondaryIconBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateColor        SecondaryIconColor = IconColor;
	TSharedRef<SWidget> IconWidget = SNew(SNiagaraIconWidget)
	.IconToolTip(IconToolTip)
	.IconBrush(IconBrush)
	.IconColor(IconColor)
	.DocLink(IconDocLink)
	.DocExcerpt(IconDocExcerpt)
	.SecondaryIconBrush(SecondaryIconBrush) 
	.SecondaryIconColor(SecondaryIconColor);

	TSharedPtr<SNiagaraParameterNameTextBlock> ParameterNameTextBlock = SNew(SNiagaraParameterNameTextBlock)
	.ParameterText(FText::FromName(Item.GetVariable().GetName()))
	.HighlightText(ItemSelector.ToSharedRef(), &SNiagaraParameterDefinitionsPanelSelector::GetFilterTextNoRef)
	.IsSelected(ItemSelector.ToSharedRef(), &SNiagaraParameterDefinitionsPanelSelector::IsItemSelected, Item)
	.IsReadOnly(true);

	// Finalize the item widget.
	return SNew(SHorizontalBox)
	// icon slot
	+ SHorizontalBox::Slot()
	.AutoWidth()
	[
		IconWidget
	]
	// name slot
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.Padding(5, 0)
	[
		ParameterNameTextBlock.ToSharedRef()
	];
}	

TSharedPtr<SWidget> SNiagaraParameterDefinitionsPanel::OnContextMenuOpening()
{
	const TArray<FNiagaraParameterDefinitionsPanelItem>& SelectedItems = ItemSelector->GetSelectedItems();
	const TArray<FNiagaraParameterDefinitionsPanelCategory>& SelectedCategories = ItemSelector->GetSelectedCategories();
	const bool bShouldCloseWindowAfterMenuSelection = true;
	// Only create context menus when a single item is selected.
	if (SelectedItems.Num() == 1)
	{
		// Create a menu with all relevant operations for a parameter.
		const FNiagaraParameterPanelItemBase& SelectedItemBase = SelectedItems[0];

		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, ToolkitCommands);
		MenuBuilder.BeginSection("Edit", LOCTEXT("EditMenuHeader", "Edit"));
		{
			const FText CopyReferenceToolTip = LOCTEXT("LibraryCopyReferenceToolTip", "Copy a string reference for this parameter to the clipboard.\nThis reference can be used in expressions and custom HLSL nodes.");
			MenuBuilder.AddMenuEntry(
				FGenericCommands::Get().Copy,
				NAME_None,
				LOCTEXT("CopyReference", "Copy Reference"),
				CopyReferenceToolTip);

			FText CopyParameterMetaDataToolTip;
			const bool bCanCopyParameterMetaData = ParameterDefinitionsPanelViewModel->GetCanCopyParameterMetaDataAndToolTip(SelectedItemBase, CopyParameterMetaDataToolTip);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CopyParameterMetadata", "Copy Metadata"),
				CopyParameterMetaDataToolTip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(ParameterDefinitionsPanelViewModel->AsShared(), &INiagaraImmutableParameterPanelViewModel::CopyParameterMetaData, SelectedItemBase),
					FCanExecuteAction::CreateLambda([bCanCopyParameterMetaData](){ return bCanCopyParameterMetaData; })));
		}

		MenuBuilder.EndSection();
		return MenuBuilder.MakeWidget();
	}
	else if (SelectedCategories.Num() == 1)
	{
		// Create a menu with all relevant operations for a parameter definitions.
		const FNiagaraParameterDefinitionsPanelCategory& SelectedCategory = SelectedCategories[0];
		if (SelectedCategory.ParameterDefinitionsUniqueId.IsValid() == false)
		{
			// If the "no definitions" default category is selected, do not make a context menu.
			return SNullWidget::NullWidget;
		}

		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, ToolkitCommands);
		MenuBuilder.BeginSection("Edit", LOCTEXT("EditMenuHeader", "Edit"));
		{
			FText UnsubscribeToolTip;
			const bool bCanUnsubscribe = ParameterDefinitionsPanelViewModel->GetCanRemoveParameterDefinitionsAndToolTip(SelectedCategory, UnsubscribeToolTip);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("UnlinkAllParameters", "Unlink All Parameters from Definitions"),
				UnsubscribeToolTip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SNiagaraParameterDefinitionsPanel::RemoveParameterDefinitions, SelectedCategory),
					FCanExecuteAction::CreateLambda([bCanUnsubscribe](){ return bCanUnsubscribe; })));

			FText SubscribeAllToolTip;
			const bool bCanSynchronizeAll = ParameterDefinitionsPanelViewModel->GetCanSubscribeAllParametersToDefinitionsAndToolTip(SelectedCategory, SubscribeAllToolTip);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("LinkAllParameters", "Link All Parameters to Definitions"),
				SubscribeAllToolTip,
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SNiagaraParameterDefinitionsPanel::SetAllParametersToSynchronizeWithParameterDefinitions, SelectedCategory)));
		}

		MenuBuilder.EndSection();
		return MenuBuilder.MakeWidget();
	}
	// More than one item selected, do not return a context menu.
	return SNullWidget::NullWidget;
}

void SNiagaraParameterDefinitionsPanel::OnParameterItemSelected(const FNiagaraParameterDefinitionsPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const
{
	ParameterDefinitionsPanelViewModel->OnParameterItemSelected(SelectedItem, SelectInfo);
}

FReply SNiagaraParameterDefinitionsPanel::OnParameterItemsDragged(const TArray<FNiagaraParameterDefinitionsPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const
{
	return ParameterDefinitionsPanelViewModel->OnParameterItemsDragged(DraggedItems, MouseEvent);
}

void SNiagaraParameterDefinitionsPanel::Refresh(bool bExpandCategories)
{
	ItemSelector->RefreshItemsAndDefaultCategories(GetViewedParameterDefinitionsItems(), GetDefaultCategories()); 
}

const TArray<FNiagaraParameterDefinitionsPanelCategory> SNiagaraParameterDefinitionsPanel::GetDefaultCategories() const
{
	if (ParameterDefinitionsPanelViewModel->GetParameterDefinitionsAssets().Num() == 0)
	{
		return DefaultCategoryNoLibraries;
	}
	return TArray<FNiagaraParameterDefinitionsPanelCategory>();
}

void SNiagaraParameterDefinitionsPanel::OnCategoryActivated(const FNiagaraParameterDefinitionsPanelCategory& ActivatedCategory) const
{
	const UNiagaraParameterDefinitions* FoundLibraryForCategory = ParameterDefinitionsPanelViewModel->FindSubscribedParameterDefinitionsById(ActivatedCategory.ParameterDefinitionsUniqueId);
	if(FoundLibraryForCategory == nullptr)
	{
		ensureMsgf(false, TEXT("Tried to activate category but failed to find parameter definitions associated with category by Guid!"));
		return;
	}

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(const_cast<UNiagaraParameterDefinitions*>(FoundLibraryForCategory), EToolkitMode::Standalone);
}

void SNiagaraParameterDefinitionsPanel::AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const
{
	ParameterDefinitionsPanelViewModel->AddParameterDefinitions(NewParameterDefinitions);
}

TSharedRef<SWidget> SNiagaraParameterDefinitionsPanel::CreateAddLibraryButton()
{
	const bool bButtonEnabled = GetAvailableParameterDefinitionsAssetsToAdd().Num() > 0;

	TSharedPtr<SComboButton> Button;
	SAssignNew(AddParameterDefinitionsButton, SComboButton)
	.IsEnabled(bButtonEnabled) //@todo(ng) tooltip you cannot add parameter libraries if there are none valid
	.ButtonStyle(FAppStyle::Get(), "RoundButton")
	.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
	.ContentPadding(FMargin(2, 0))
	.OnGetMenuContent(this, &SNiagaraParameterDefinitionsPanel::CreateAddLibraryMenu)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.HasDownArrow(false)
	.ButtonContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(0, 1))
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Plus"))
		]
	];

	return AddParameterDefinitionsButton.ToSharedRef();
}

void SNiagaraParameterDefinitionsPanel::RemoveParameterDefinitions(const FNiagaraParameterDefinitionsPanelCategory CategoryToRemove) const
{
	ParameterDefinitionsPanelViewModel->RemoveParameterDefinitions(CategoryToRemove);
}

void SNiagaraParameterDefinitionsPanel::SetAllParametersToSynchronizeWithParameterDefinitions(const FNiagaraParameterDefinitionsPanelCategory CategoryToSync) const
{
	ParameterDefinitionsPanelViewModel->SubscribeAllParametersToDefinitions(CategoryToSync);
}

const TArray<UNiagaraParameterDefinitions*> SNiagaraParameterDefinitionsPanel::GetAvailableParameterDefinitionsAssetsToAdd() const
{
	const bool bSkipSubscribed = true;
	return ParameterDefinitionsPanelViewModel->GetAvailableParameterDefinitionsAssets(bSkipSubscribed);
}

TSharedRef<SWidget> SNiagaraParameterDefinitionsPanel::CreateAddLibraryMenu()
{
	// helper lambda to make the widgets for each parameter definitions.
	auto MakeWidgetForParameterDefinitions = [](const UNiagaraParameterDefinitions* ParameterDefinitions)->TSharedRef<SWidget> {
		return SNew(STextBlock)
		.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterName.TypeText")
		.Text(FText::FromString(*ParameterDefinitions->GetName()));
	};

	FMenuBuilder MenuBuilder = FMenuBuilder(true, nullptr);
	for(UNiagaraParameterDefinitions* ParameterDefinitions : GetAvailableParameterDefinitionsAssetsToAdd())
	{
		FUIAction Action = FUIAction(FExecuteAction::CreateSP(this, &SNiagaraParameterDefinitionsPanel::AddParameterDefinitions, ParameterDefinitions));
		TSharedRef<SWidget> Widget = MakeWidgetForParameterDefinitions(ParameterDefinitions);
		MenuBuilder.AddMenuEntry(Action, Widget);
	}
	return MenuBuilder.MakeWidget();
}

const FSlateBrush* SNiagaraParameterDefinitionsPanel::GetCategoryBackgroundImage(bool bIsCategoryHovered, bool bIsCategoryExpanded) const
{
	if (bIsCategoryHovered)
	{
		return bIsCategoryExpanded ? FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered") : FAppStyle::GetBrush("DetailsView.CollapsedCategory_Hovered");
	}
	else
	{
		return bIsCategoryExpanded ? FAppStyle::GetBrush("DetailsView.CategoryTop") : FAppStyle::GetBrush("DetailsView.CollapsedCategory");
	}
}

#undef LOCTEXT_NAMESPACE /*"NiagaraParameterDefinitionsPanel"*/
