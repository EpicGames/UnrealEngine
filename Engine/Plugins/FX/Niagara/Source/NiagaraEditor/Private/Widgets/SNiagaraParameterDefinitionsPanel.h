// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Internationalization/Text.h"
#include "NiagaraTypes.h"
#include "NiagaraActions.h"
#include "NiagaraParameterPanelTypes.h"
#include "Framework/Commands/Commands.h"
#include "Widgets/SItemSelector.h"
#include "NiagaraParameterDefinitionsBase.h"

typedef SItemSelector<FNiagaraParameterDefinitionsPanelCategory, FNiagaraParameterDefinitionsPanelItem> SNiagaraParameterDefinitionsPanelSelector;

class INiagaraParameterDefinitionsPanelViewModel;
class SComboButton;
class UNiagaraParameterDefinitions;

class FNiagaraParameterDefinitionsPanelCommands : public TCommands<FNiagaraParameterDefinitionsPanelCommands>
{
public:
	/** Constructor */
	FNiagaraParameterDefinitionsPanelCommands()
		: TCommands<FNiagaraParameterDefinitionsPanelCommands>(TEXT("NiagaraParameterDefinitionsPanelCommands"), NSLOCTEXT("Contexts", "NiagaraParameterDefinitionsPanel", "NiagaraParameterDefinitionsPanel"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	// Basic operations
	TSharedPtr<FUICommandInfo> DeleteItem;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

/** A widget for viewing and editing UNiagaraParameterDefinitions provided by an INiagaraParameterDefinitionsPanelViewModel */
class SNiagaraParameterDefinitionsPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraParameterDefinitionsPanel)
	{}
	SLATE_END_ARGS();

	NIAGARAEDITOR_API virtual ~SNiagaraParameterDefinitionsPanel() override;

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, const TSharedPtr<INiagaraParameterDefinitionsPanelViewModel>& InParameterDefinitionsPanelViewModel, const TSharedPtr<FUICommandList>& InToolkitCommands);

	//~ Begin SItemSelector
	TArray<FNiagaraParameterDefinitionsPanelCategory> OnGetCategoriesForItem(const FNiagaraParameterDefinitionsPanelItem& Item);
	bool OnCompareCategoriesForEquality(const FNiagaraParameterDefinitionsPanelCategory& CategoryA, const FNiagaraParameterDefinitionsPanelCategory& CategoryB) const;
	bool OnCompareCategoriesForSorting(const FNiagaraParameterDefinitionsPanelCategory& CategoryA, const FNiagaraParameterDefinitionsPanelCategory& CategoryB) const;
	TArray<FNiagaraParameterDefinitionsPanelItem> GetViewedParameterDefinitionsItems() const;
	bool OnCompareItemsForEquality(const FNiagaraParameterDefinitionsPanelItem& ItemA, const FNiagaraParameterDefinitionsPanelItem& ItemB) const;
	bool OnCompareItemsForSorting(const FNiagaraParameterDefinitionsPanelItem& ItemA, const FNiagaraParameterDefinitionsPanelItem& ItemB) const;
	bool OnDoesItemMatchFilterText(const FText& FilterText, const FNiagaraParameterDefinitionsPanelItem& Item);
	TSharedRef<SWidget> OnGenerateWidgetForCategory(const FNiagaraParameterDefinitionsPanelCategory& Category);
	TSharedRef<SWidget> OnGenerateWidgetForItem(const FNiagaraParameterDefinitionsPanelItem& Item);
	TSharedPtr<SWidget> OnContextMenuOpening();
	const TArray<FNiagaraParameterDefinitionsPanelCategory> GetDefaultCategories() const;
	void OnCategoryActivated(const FNiagaraParameterDefinitionsPanelCategory& ActivatedCategory) const;
	//~ End SItemSelector

	void Refresh(bool bExpandCategories);
	void AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const;
	void OnParameterItemSelected(const FNiagaraParameterDefinitionsPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const;
	FReply OnParameterItemsDragged(const TArray<FNiagaraParameterDefinitionsPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const;

private:
	TSharedRef<SWidget> CreateAddLibraryButton();
	void RemoveParameterDefinitions(const FNiagaraParameterDefinitionsPanelCategory CategoryToRemove) const;
	void SetAllParametersToSynchronizeWithParameterDefinitions(const FNiagaraParameterDefinitionsPanelCategory CategoryToSync) const;
	const TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitionsAssetsToAdd() const;
	TSharedRef<SWidget> CreateAddLibraryMenu();
	const FSlateBrush* GetCategoryBackgroundImage(bool bIsCategoryHovered, bool bIsCategoryExpanded) const;

private:
	TSharedPtr<INiagaraParameterDefinitionsPanelViewModel> ParameterDefinitionsPanelViewModel;

	TSharedPtr<SNiagaraParameterDefinitionsPanelSelector> ItemSelector;

	TSharedPtr<FUICommandList> ToolkitCommands;

	TSharedPtr<SComboButton> AddParameterDefinitionsButton;

	static TArray<FNiagaraParameterDefinitionsPanelCategory> DefaultCategoryNoLibraries;
};
