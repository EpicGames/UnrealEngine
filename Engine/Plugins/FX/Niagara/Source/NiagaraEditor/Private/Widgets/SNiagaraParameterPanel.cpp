// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraParameterPanel.h"

#include "DetailLayoutBuilder.h"
#include "EdGraphSchema_Niagara.h"
#include "EditorFontGlyphs.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "NiagaraActions.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraTypes.h"
#include "SDropTarget.h"
#include "Widgets/SNiagaraPinTypeSelector.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/SNiagaraParameterName.h"
#include "Widgets/SItemSelector.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "NiagaraParameterPanel"


SNiagaraParameterPanel::~SNiagaraParameterPanel()
{
	ParameterPanelViewModel->GetOnRequestRefreshDelegate().Unbind();
	ParameterPanelViewModel->GetOnRequestRefreshNextTickDelegate().Unbind();
	ParameterPanelViewModel->GetOnSelectParameterItemByNameDelegate().Unbind();
	ParameterPanelViewModel->GetOnNotifyParameterPendingRenameDelegate().Unbind();
	ParameterPanelViewModel->GetOnNotifyParameterPendingNamespaceModifierRenameDelegate().Unbind();
	ParameterPanelViewModel->GetParametersWithNamespaceModifierRenamePendingDelegate().Unbind();

	// Unregister all commands for right click on action node
	ToolkitCommands->UnmapAction(FNiagaraParameterPanelCommands::Get().DeleteItem);
	ToolkitCommands->UnmapAction(FGenericCommands::Get().Rename);
	ToolkitCommands->UnmapAction(FGenericCommands::Get().Copy);
}

void FNiagaraParameterPanelCommands::RegisterCommands()
{
	UI_COMMAND(DeleteItem, "Delete", "Delete this parameter", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete));
}

void SNiagaraParameterPanel::Construct(const FArguments& InArgs, const TSharedPtr<INiagaraParameterPanelViewModel>& InParameterPanelViewModel, const TSharedPtr<FUICommandList>& InToolkitCommands)
{
	bShowParameterSynchronizingWithLibraryIcon = InArgs._ShowParameterSynchronizingWithLibraryIcon;
	bShowParameterSynchronizingWithLibraryIconExternallyReferenced = InArgs._ShowParameterSynchronizingWithLibraryIconExternallyReferenced;
	bShowParameterReferenceCounter = InArgs._ShowParameterReferenceCounter;

	ParameterPanelViewModel = InParameterPanelViewModel;
	ToolkitCommands = InToolkitCommands;
	bPendingRefresh = false;
	bParameterItemsPendingChange = false;
	ParametersWithNamespaceModifierRenamePending = MakeShared<TArray<FName>>();

	ParameterPanelViewModel->GetOnRequestRefreshDelegate().BindSP(this, &SNiagaraParameterPanel::Refresh);
	ParameterPanelViewModel->GetOnRequestRefreshNextTickDelegate().BindSP(this, &SNiagaraParameterPanel::RefreshNextTick);
	ParameterPanelViewModel->GetOnExternalSelectionChangedDelegate().AddSP(this, &SNiagaraParameterPanel::HandleExternalSelectionChanged);
	ParameterPanelViewModel->GetOnSelectParameterItemByNameDelegate().BindSP(this, &SNiagaraParameterPanel::SelectParameterItemByName);
	ParameterPanelViewModel->GetOnNotifyParameterPendingRenameDelegate().BindSP(this, &SNiagaraParameterPanel::AddParameterPendingRename);
	ParameterPanelViewModel->GetOnNotifyParameterPendingNamespaceModifierRenameDelegate().BindSP(this, &SNiagaraParameterPanel::AddParameterPendingNamespaceModifierRename);
	ParameterPanelViewModel->GetParametersWithNamespaceModifierRenamePendingDelegate().BindSP(this, &SNiagaraParameterPanel::GetParametersWithNamespaceModifierRenamePending);

	if (ParameterPanelViewModel->UsesCategoryFilteringForInitialExpansion())
	{
		FilterCategoryExpandedDelegate.BindRaw(ParameterPanelViewModel.Get(), &INiagaraImmutableParameterPanelViewModel::IsCategoryExpandedByDefault);
	}
	
	SAssignNew(ItemSelector, SNiagaraParameterPanelSelector)
	.PreserveSelectionOnRefresh(true)
	.PreserveExpansionOnRefresh(true)
	.DefaultCategories(GetDefaultCategories())
	.Items(ParameterPanelViewModel->GetViewedParameterItems())
	.OnGetCategoriesForItem(this, &SNiagaraParameterPanel::OnGetCategoriesForItem)
	.OnCompareCategoriesForEquality(this, &SNiagaraParameterPanel::OnCompareCategoriesForEquality)
	.OnCompareCategoriesForSorting(this, &SNiagaraParameterPanel::OnCompareCategoriesForSorting)
	.OnCompareItemsForEquality(this, &SNiagaraParameterPanel::OnCompareItemsForEquality)
	.OnCompareItemsForSorting(this, &SNiagaraParameterPanel::OnCompareItemsForSorting)
	.OnDoesItemMatchFilterText(this, &SNiagaraParameterPanel::OnDoesItemMatchFilterText)
	.OnGenerateWidgetForCategory(this, &SNiagaraParameterPanel::OnGenerateWidgetForCategory)
	.OnGenerateWidgetForItem(this, &SNiagaraParameterPanel::OnGenerateWidgetForItem)
	.OnContextMenuOpening(this, &SNiagaraParameterPanel::OnContextMenuOpening)
	.OnItemSelected(this, &SNiagaraParameterPanel::OnParameterItemSelected)
	.OnItemsDragged(this, &SNiagaraParameterPanel::OnParameterItemsDragged)
	.OnItemActivated(this, &SNiagaraParameterPanel::OnParameterItemActived)
	.AllowMultiselect(ParameterPanelViewModel->GetAllowMultiSelect())
	.ClearSelectionOnClick(true)
	.CategoryRowStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.Parameters.TableRow")
	.OnGetCategoryBackgroundImage(this, &SNiagaraParameterPanel::GetCategoryBackgroundImage)
	.OnGetCategoryExpandedInitially(FilterCategoryExpandedDelegate)
	.CategoryBorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
	.CategoryChildSlotPadding(FMargin(0.0f, 2.0f, 0.0f, 0.0f))
	.CategoryBorderBackgroundPadding(FMargin(0.0f, 3.0f))
	.OnGetKeyForItem(this, &SNiagaraParameterPanel::OnGetKeyForItem)
	.OnGetKeyForCategory(this, &SNiagaraParameterPanel::OnGetKeyForCategory)
	.OnCategoryExpansionChanged(this, &SNiagaraParameterPanel::OnCategoryExpansionChanged)
	.ExpandInitially(false)
	.SearchBoxAdjacentContent()
	[
		InArgs._SearchAdjacentWidget.Widget
	];

	// Finalize the widget
	ChildSlot
		[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4, 2)
		[
			SAssignNew(SectionSelectorBox, SWrapBox)
			.UseAllottedSize(true)
			.InnerSlotPadding(FVector2D(4, 4))
		]
		+ SVerticalBox::Slot()
		.Padding(0, 2)
		[
			// Drop target
			SNew(SDropTarget)
			.OnDropped(this, &SNiagaraParameterPanel::HandleDragDropOperation)
			.OnAllowDrop(this, &SNiagaraParameterPanel::GetCanHandleDragDropOperation)
			.Content()
			[
				ItemSelector.ToSharedRef()
			]
		]
	];

	// Register all commands for right click on action node
	FNiagaraParameterPanelCommands::Register();
	ToolkitCommands->MapAction(FNiagaraParameterPanelCommands::Get().DeleteItem,
		FExecuteAction::CreateSP(this, &SNiagaraParameterPanel::DeleteSelectedItems),
		FCanExecuteAction::CreateSP(this, &SNiagaraParameterPanel::CanDeleteSelectedItems));
	ToolkitCommands->MapAction(FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SNiagaraParameterPanel::RequestRenameSelectedItem),
		FCanExecuteAction::CreateSP(this, &SNiagaraParameterPanel::CanRequestRenameSelectedItem));
	ToolkitCommands->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SNiagaraParameterPanel::CopyParameterReference),
		FCanExecuteAction::CreateSP(this, &SNiagaraParameterPanel::CanCopyParameterReference));

	ConstructSectionButtons();
}


void SNiagaraParameterPanel::ConstructSectionButtons()
{
	SectionSelectorBox->ClearChildren();

	SectionSelectorBox->SetVisibility(ParameterPanelViewModel->GetShowSections() ? EVisibility::Visible : EVisibility::Collapsed);
	for (const INiagaraParameterPanelViewModel::FSectionDesc& Section : ParameterPanelViewModel->GetSections())
	{
		SectionSelectorBox->AddSlot()
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.OnCheckStateChanged(this, &SNiagaraParameterPanel::OnSectionChecked, Section.DisplayName)
				.IsChecked(this, &SNiagaraParameterPanel::GetSectionCheckState, Section.DisplayName)
				.IsEnabled(this, &SNiagaraParameterPanel::GetSectionEnabled, Section.DisplayName)
				.ToolTipText(Section.Description)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "SmallText")
					.Text(Section.DisplayName)
				]
			];
	}
}

ECheckBoxState SNiagaraParameterPanel::GetSectionCheckState(FText Section) const
{
	return Section.IdenticalTo(ParameterPanelViewModel->GetActiveSection()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


bool SNiagaraParameterPanel::GetSectionEnabled(FText Section) const
{
	return ParameterPanelViewModel->GetSectionEnabled(Section);
}


void SNiagaraParameterPanel::OnSectionChecked(ECheckBoxState CheckState, FText Section)
{
	if (CheckState == ECheckBoxState::Checked)
	{
		ParameterPanelViewModel->SetActiveSection(Section);
		Refresh();
	}
}

TArray<FNiagaraParameterPanelCategory> SNiagaraParameterPanel::OnGetCategoriesForItem(const FNiagaraParameterPanelItem& Item)
{
	return {Item.NamespaceMetaData};
}

bool SNiagaraParameterPanel::OnCompareCategoriesForEquality(const FNiagaraParameterPanelCategory& CategoryA, const FNiagaraParameterPanelCategory& CategoryB) const
{
	return CategoryA.NamespaceMetaData == CategoryB.NamespaceMetaData;
}

bool SNiagaraParameterPanel::OnCompareCategoriesForSorting(const FNiagaraParameterPanelCategory& CategoryA, const FNiagaraParameterPanelCategory& CategoryB) const
{
	return CategoryA.NamespaceMetaData.SortId < CategoryB.NamespaceMetaData.SortId;
}

const FGuid& SNiagaraParameterPanel::OnGetKeyForCategory(const FNiagaraParameterPanelCategory& Category) const
{
	return Category.NamespaceMetaData.GetGuid();
}

void SNiagaraParameterPanel::OnCategoryExpansionChanged(const FNiagaraParameterPanelCategory& Category, bool bIsExpanded)
{
	ParameterPanelViewModel->UpdateCategoryExpansionState(Category, bIsExpanded);
}

bool SNiagaraParameterPanel::OnCompareItemsForEquality(const FNiagaraParameterPanelItem& ItemA, const FNiagaraParameterPanelItem& ItemB) const
{
	return ItemA.GetVariable()  == ItemB.GetVariable();
}

bool SNiagaraParameterPanel::OnCompareItemsForSorting(const FNiagaraParameterPanelItem& ItemA, const FNiagaraParameterPanelItem& ItemB) const
{
	return ItemA.GetVariable().GetName().LexicalLess(ItemB.GetVariable().GetName());
}

const FNiagaraVariableBase& SNiagaraParameterPanel::OnGetKeyForItem(const FNiagaraParameterPanelItem& Item) const
{
	return Item.GetVariable();
}

bool SNiagaraParameterPanel::OnDoesItemMatchFilterText(const FText& FilterText, const FNiagaraParameterPanelItem& Item)
{
	return Item.GetVariable().GetName().ToString().Contains(FilterText.ToString());
}

TSharedRef<SWidget> SNiagaraParameterPanel::OnGenerateWidgetForCategory(const FNiagaraParameterPanelCategory& Category)
{
	const FText& CategoryText = Category.NamespaceMetaData.DisplayNameLong.IsEmptyOrWhitespace() == false ?
		Category.NamespaceMetaData.DisplayNameLong : Category.NamespaceMetaData.DisplayName;
	TSharedRef<SHorizontalBox> ParameterPanelCategoryHorizontalBox = SNew(SHorizontalBox);
	
	ParameterPanelCategoryHorizontalBox->AddSlot()
	.VAlign(VAlign_Center)
	.Padding(3, 0, 0, 0)
	[

		SNew(SRichTextBlock)
		.Text(CategoryText)
		.ToolTip(SNew(SToolTip).Text(Category.NamespaceMetaData.Description))
		.DecoratorStyleSet(&FNiagaraEditorStyle::Get())
		.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.Parameters.HeaderText")
	];

	if (GetCanAddParametersToCategory(Category))
	{
		FText AddNewText = LOCTEXT("AddNewParameter", "Add Parameter");
		FName MetaDataTag = TEXT("AddNewParameter");

		ParameterPanelCategoryHorizontalBox->AddSlot()
		.AutoWidth()
		.Padding(0.0f, 4.0f, 3.0f, 4.0f)
		[
			CreateAddToCategoryButton(Category, AddNewText, MetaDataTag)
		];
	}

	return ParameterPanelCategoryHorizontalBox;
}

TSharedRef<SWidget> SNiagaraParameterPanel::OnGenerateWidgetForItem(const FNiagaraParameterPanelItem& Item)
{
	// Generate the icon widget.
	FText			   IconToolTip = Item.ScriptVariable->Variable.GetType().GetNameText();
	FSlateBrush const* IconBrush = Item.GetVariable().GetType().IsStatic() ? FNiagaraEditorStyle::Get().GetBrush(TEXT("NiagaraEditor.StaticIcon")) : FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
	const FLinearColor TypeColor = UEdGraphSchema_Niagara::GetTypeColor(Item.GetVariable().GetType());
	FSlateColor        IconColor = FSlateColor(TypeColor);
	FString			   IconDocLink, IconDocExcerpt;
	FSlateBrush const* SecondaryIconBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateColor        SecondaryIconColor = IconColor;
	TSharedRef<SWidget> IconWidget = SNew(SScaleBox)
	[
		SNew(SNiagaraIconWidget)
		.IconToolTip(IconToolTip)
		.IconBrush(IconBrush)
		.IconColor(IconColor)
		.DocLink(IconDocLink)
		.DocExcerpt(IconDocExcerpt)
		.SecondaryIconBrush(SecondaryIconBrush)
		.SecondaryIconColor(SecondaryIconColor)
	];

	// Generate the parameter name decorator widget.
	bool bItemReadOnly = Item.NamespaceMetaData.IsValid() == false || Item.NamespaceMetaData.Options.Contains(ENiagaraNamespaceMetadataOptions::PreventEditingName) || Item.bExternallyReferenced;
	
	TSharedPtr<SHorizontalBox> ParameterNameTextBlockDecorator = SNew(SHorizontalBox);
	if ( bShowParameterSynchronizingWithLibraryIcon && (bShowParameterSynchronizingWithLibraryIconExternallyReferenced || !Item.bExternallyReferenced) )
	{
		if (Item.ScriptVariable->GetIsSubscribedToParameterDefinitions())
		{
			if (Item.ScriptVariable->GetIsOverridingParameterDefinitionsDefaultValue())
			{
				ParameterNameTextBlockDecorator->AddSlot()
				.AutoWidth()
				[
					SNew(SBox)
					.MinDesiredWidth(20.0f)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
						.Text(FEditorFontGlyphs::Share_Alt)
						.ToolTipText(LOCTEXT("ParameterDefinitionDefaultValueOverridingToolTip", "Parameter is overriding the linked Parameter Definition default value."))
					]
				];
			}

			ParameterNameTextBlockDecorator->AddSlot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(20.0f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
					.Text(FEditorFontGlyphs::Book)
					.ToolTipText(LOCTEXT("ParameterDefinitionSubscribedToolTip", "Parameter is linked to a Parameter Definition."))
				]
			];
		}
		else if (
			Item.DefinitionMatchState == EParameterDefinitionMatchState::MatchingOneDefinition ||
			Item.DefinitionMatchState == EParameterDefinitionMatchState::MatchingMoreThanOneDefinition ||
			Item.DefinitionMatchState == EParameterDefinitionMatchState::MatchingDefinitionNameButNotType)
		{
			FText LinkedParameterToolTipText;
			if (Item.DefinitionMatchState == EParameterDefinitionMatchState::MatchingOneDefinition)
			{
				LinkedParameterToolTipText = LOCTEXT("ParameterDefinitionMatchToolTip", "Parameter has the same name and type as a Parameter Definition, but is not linked to the Parameter Definition.");
			}
			else if (Item.DefinitionMatchState == EParameterDefinitionMatchState::MatchingMoreThanOneDefinition)
			{
				LinkedParameterToolTipText = LOCTEXT("ParameterDefinitionMultipleMatchToolTip", "Parameter has the same name and type as multiple Parameter Definitions.");
			}
			else if (Item.DefinitionMatchState == EParameterDefinitionMatchState::MatchingDefinitionNameButNotType)
			{
				LinkedParameterToolTipText = LOCTEXT("ParameterDefinitionMultipleNameAliasToolTip", "Parameter has the same name but not the same type as a Parameter Definition.");
			}
			else
			{
				ensureMsgf(false, TEXT("Encountered unexepcted definition match state when getting tooltip for linked parameter icon!"));
			}

			ParameterNameTextBlockDecorator->AddSlot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(20.0f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
					.Text(FEditorFontGlyphs::Book)
					.ColorAndOpacity(FLinearColor::Red)
					.ToolTipText(LinkedParameterToolTipText)
					.ShadowOffset(FVector2D(1, 1))
				]
			];
		}
	}

	if (Item.bExternallyReferenced)
	{
		ParameterNameTextBlockDecorator->AddSlot()
		.AutoWidth()
		[
			SNew(SBox)
			.MinDesiredWidth(20.0f)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
				.Text(FEditorFontGlyphs::Lock)
				.ToolTipText(LOCTEXT("LockedToolTip", "This parameter is used in a referenced external graph and can't be edited directly."))
			]
		];
	}
	if (Item.bSourcedFromCustomStackContext)
	{
		ParameterNameTextBlockDecorator->AddSlot()
		.AutoWidth()
		[
			SNew(SBox)
			.MinDesiredWidth(20.0f)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
				.Text(FEditorFontGlyphs::Database)
				.ToolTipText(LOCTEXT("DataInterfaceSourceToolTip", "This parameter is a child variable of an existing Data Interface, meant to be used in Simulation Stage based stacks where the parent Data Interface is the Iteration Source."))
			]
		];
	}


	// Generate the parameter name widget.
	TSharedPtr<SNiagaraParameterNameTextBlock> ParameterNameTextBlock = SNew(SNiagaraParameterNameTextBlock)
	.ParameterText(FText::FromName(Item.GetVariable().GetName()))
	.HighlightText(ItemSelector.ToSharedRef(), &SNiagaraParameterPanelSelector::GetFilterTextNoRef)
	.ToolTipText_Lambda([Item]
	{
		return Item.ScriptVariable->Metadata.Description;
	})
	.OnTextCommitted(this, &SNiagaraParameterPanel::OnParameterNameTextCommitted, Item)
	.OnVerifyTextChanged(this, &SNiagaraParameterPanel::OnParameterNameTextVerifyChanged, Item)
	.IsSelected(ItemSelector.ToSharedRef(), &SNiagaraParameterPanelSelector::IsItemSelected, Item)
	.IsReadOnly(bItemReadOnly);

	// Bind the rename delegate to the parameter name text block.
	Item.GetOnRequestRename().BindSP(ParameterNameTextBlock.ToSharedRef(), &SNiagaraParameterNameTextBlock::EnterEditingMode);
	Item.GetOnRequestRenameNamespaceModifier().BindSP(ParameterNameTextBlock.ToSharedRef(), &SNiagaraParameterNameTextBlock::EnterNamespaceModifierEditingMode);

	// Finalize the item widget.
	TSharedPtr<SHorizontalBox> ItemWidgetHorizontalBox = SNew(SHorizontalBox);

	ItemWidgetHorizontalBox->AddSlot()
	[
		SNew(SHorizontalBox)
		// icon slot
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			IconWidget
		]
		// name slot
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(5, 0)
		[
			ParameterNameTextBlock.ToSharedRef()
		]
		// name slot decorators
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			ParameterNameTextBlockDecorator.ToSharedRef()
		]
	];

	if (bShowParameterReferenceCounter)
	{
		TMap<FName, TArray<FNiagaraParameterReferencePath>> PathByParent;
		for (const FNiagaraParameterReferencePath& Path : Item.ReferencePaths)
		{
			if (const UObject* SourceObject = Path.SourceObject.Get())
			{
				FName ParentName = SourceObject->GetFName();
				const UNiagaraGraph* SourceGraph = Cast<const UNiagaraGraph>(SourceObject);
				if (SourceGraph && SourceGraph->GetOwningEmitter().Emitter)
				{
					ParentName = SourceGraph->GetOwningEmitter().Emitter.GetFName();
				}
				PathByParent.FindOrAdd(ParentName).Add(Path);
			}
		}

		FString DetailReferences;
		for (const TPair<FName, TArray<FNiagaraParameterReferencePath>>& Pair : PathByParent)
		{
			DetailReferences += "\n" + Pair.Key.GetDisplayNameEntry()->GetPlainNameString() + ":\n";
			for (const FNiagaraParameterReferencePath& Path : Pair.Value)
			{
				FString Prefix = (Path.bRead && Path.bWrite) ? "R+W" : (Path.bRead ? "R" : "W");
				DetailReferences += FString("\t") + Prefix + ": " +  Path.ModuleName.ToString() + "\n";
			}
		}
		FText TooltipBaseText = LOCTEXT("ReferenceCountTooltip", "Shows the total number of references for this parameter across all selected emitters in the format 'Reads / Writes'.\nThis includes usages inside module scripts. If no emitter is selected then it only shows usages in the system script.");
		FText FormattedTooltip = DetailReferences.IsEmpty() ? TooltipBaseText : FText::Format(LOCTEXT("DetailReferenceFmt", "{0}\nDetailed references:\n{1}"), TooltipBaseText, FText::FromString(DetailReferences));
		
		ItemWidgetHorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "RoundButton")
			.ToolTipText(FormattedTooltip)
			.ForegroundColor(FSlateColor::UseForeground())
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.Parameters.ReadColor"))
					.Text(FText::AsNumber(Item.ReadReferenceCount))
					.Font(FCoreStyle::GetDefaultFontStyle(FName("Bold"), 10))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Margin(FMargin(2, 0))
					.Text(FText::FromString(TEXT("|")))
					.Font(FCoreStyle::GetDefaultFontStyle(FName("Regular"), 10))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.ColorAndOpacity(FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.Parameters.WriteColor"))
					.Text(FText::AsNumber(Item.WriteReferenceCount))
					.Font(FCoreStyle::GetDefaultFontStyle(FName("Bold"), 10))
				]
				
			]
		];
	}

	return SNew(SBox)
	.MinDesiredHeight(20.0f)
	[
		ItemWidgetHorizontalBox.ToSharedRef()
	];
}

TSharedPtr<SWidget> SNiagaraParameterPanel::OnContextMenuOpening()
{
	const TArray<FNiagaraParameterPanelItem>& SelectedItems = ItemSelector->GetSelectedItems();
	return ParameterPanelViewModel->CreateContextMenuForItems(SelectedItems, ToolkitCommands);
}

void SNiagaraParameterPanel::OnParameterItemSelected(const FNiagaraParameterPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const
{
	ParameterPanelViewModel->OnParameterItemSelected(SelectedItem, SelectInfo);
}

FReply SNiagaraParameterPanel::OnParameterItemsDragged(const TArray<FNiagaraParameterPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const
{
	return ParameterPanelViewModel->OnParameterItemsDragged(DraggedItems, MouseEvent);
}

void SNiagaraParameterPanel::OnParameterItemActived(const FNiagaraParameterPanelItem& ActivatedItem) const
{
	ParameterPanelViewModel->OnParameterItemActivated(ActivatedItem);
}

const TArray<FNiagaraParameterPanelCategory>& SNiagaraParameterPanel::GetDefaultCategories() const
{
	return ParameterPanelViewModel->GetDefaultCategories();
}

void SNiagaraParameterPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if(!ItemSelector->IsPendingRefresh())
	{ 
		if (bPendingRefresh)
		{
			bPendingRefresh = false;
			Refresh(bRunCategoryExpansionFilter);
			bRunCategoryExpansionFilter = false;
		}
		else if (bParameterItemsPendingChange)
		{
			bParameterItemsPendingChange = false;
			ProcessParameterItemsPendingChange();
		}
	}
}

void SNiagaraParameterPanel::HandleExternalSelectionChanged(const UObject* Obj)
{
	if (Obj && Obj->IsA< UNiagaraScriptVariable>())
	{
		const UNiagaraScriptVariable* Var = Cast< UNiagaraScriptVariable>(Obj);
		if (Var)
		{
			SelectParameterItemByName(Var->Variable.GetName());
		}
	}
}

TSharedRef<SWidget> SNiagaraParameterPanel::CreateAddToCategoryButton(const FNiagaraParameterPanelCategory& Category, FText AddNewText, FName MetaDataTag)
{
	TSharedPtr<SComboButton> Button;
	SAssignNew(Button, SComboButton)
	.ButtonStyle(FAppStyle::Get(), "RoundButton")
	.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
	.ContentPadding(FMargin(2, 0))
	.OnGetMenuContent(this, &SNiagaraParameterPanel::OnGetParameterMenu, Category)
	.IsEnabled(this, &SNiagaraParameterPanel::GetCanAddParametersToCategory, Category)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.HasDownArrow(false)
	.ButtonContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(0, 1))
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Plus"))
		]
	];


	CategoryToButtonMap.FindOrAdd(Category.NamespaceMetaData, Button) = Button;
	return Button.ToSharedRef();
}

TSharedRef<SWidget> SNiagaraParameterPanel::OnGetParameterMenu(FNiagaraParameterPanelCategory Category)
{
	FMenuAndSearchBoxWidgets MenuAndSearchBoxWidgets = ParameterPanelViewModel->GetParameterMenu(Category);
	if (MenuAndSearchBoxWidgets.MenuSearchBoxWidget.IsValid())
	{
		CategoryToButtonMap.FindChecked(Category.NamespaceMetaData)->SetMenuContentWidgetToFocus(MenuAndSearchBoxWidgets.MenuSearchBoxWidget.ToSharedRef());
	}
	return MenuAndSearchBoxWidgets.MenuWidget.ToSharedRef();
}

bool SNiagaraParameterPanel::GetCanAddParametersToCategory(FNiagaraParameterPanelCategory Category) const
{
	return ParameterPanelViewModel->GetCanAddParametersToCategory(Category);
}

void SNiagaraParameterPanel::DeleteSelectedItems() const
{
	ParameterPanelViewModel->DeleteParameters(ItemSelector->GetSelectedItems());
}

bool SNiagaraParameterPanel::CanDeleteSelectedItems() const
{
	bool bCanDelete = true;
	FText Unused;
	for (const FNiagaraParameterPanelItem& Item : ItemSelector->GetSelectedItems())
	{
		bCanDelete &= ParameterPanelViewModel->GetCanDeleteParameterAndToolTip(Item, Unused);
	}
	return bCanDelete;
}

void SNiagaraParameterPanel::RequestRenameSelectedItem() const
{
	if (CanRequestRenameSelectedItem() == false)
	{
		return;
	}

	TArray<FNiagaraParameterPanelItem> SelectedItems = ItemSelector->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		SelectedItems[0].RequestRename();
	}
}

bool SNiagaraParameterPanel::CanRequestRenameSelectedItem() const
{
	const TArray<FNiagaraParameterPanelItem>& SelectedItems = ItemSelector->GetSelectedItems();
	// it's possible the request rename delegate has not been bound yet as the widget for it might not have been created at this point
	if (SelectedItems.Num() == 1 && SelectedItems[0].GetOnRequestRename().IsBound())
	{
		FText Unused;
		bool bCheckEmptyNameText = false;
		return ParameterPanelViewModel->GetCanRenameParameterAndToolTip(SelectedItems[0], Unused, bCheckEmptyNameText, Unused);
	}
	return false;
}

void SNiagaraParameterPanel::CopyParameterReference() const
{
	const TArray<FNiagaraParameterPanelItem>& SelectedItems = ItemSelector->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		ParameterPanelViewModel->CopyParameterReference(SelectedItems[0]);
	}
}

bool SNiagaraParameterPanel::CanCopyParameterReference() const
{
	const TArray<FNiagaraParameterPanelItem>& SelectedItems = ItemSelector->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		FText Unused;
		return ParameterPanelViewModel->GetCanCopyParameterReferenceAndToolTip(SelectedItems[0], Unused);
	}
	return false;
}

void SNiagaraParameterPanel::OnParameterNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, const FNiagaraParameterPanelItem ItemToBeRenamed) const
{
	ParameterPanelViewModel->RenameParameter(ItemToBeRenamed, FName(*NewText.ToString()));
}

bool SNiagaraParameterPanel::OnParameterNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage, const FNiagaraParameterPanelItem ItemToBeRenamed) const
{
	bool bCheckEmptyNameText = true;
	return ParameterPanelViewModel->GetCanRenameParameterAndToolTip(ItemToBeRenamed, InNewText, bCheckEmptyNameText, OutErrorMessage);
}

void SNiagaraParameterPanel::Refresh(bool bInRunCategoryExpansionFilter)
{
	ItemSelector->RefreshItemsAndDefaultCategories(ParameterPanelViewModel->GetViewedParameterItems(), ParameterPanelViewModel->GetDefaultCategories());
	if (bInRunCategoryExpansionFilter && FilterCategoryExpandedDelegate.IsBound() && ParameterPanelViewModel->UsesCategoryFilteringForInitialExpansion())
	{	
		ItemSelector->ExpandTreeByFilter(FilterCategoryExpandedDelegate);
	}
}

void SNiagaraParameterPanel::RefreshNextTick(bool bInRunCategoryExpansionFilter)
{
	bPendingRefresh = true;

	// We want to make sure that if anyone requested the expansion filter, it doesn't get cleared out, so only set if it isn't already true...
	if (!bRunCategoryExpansionFilter)
	{
		bRunCategoryExpansionFilter = bInRunCategoryExpansionFilter;
	}
}


void SNiagaraParameterPanel::SelectParameterItemByName(const FName ParameterName) const
{
	for (const FNiagaraParameterPanelItem& Item : ParameterPanelViewModel->GetCachedViewedParameterItems())
	{
		if (Item.GetVariable().GetName() == ParameterName)
		{
			const TArray<FNiagaraParameterPanelItem> ItemsToSelect = { Item };
			ItemSelector->SetSelectedItems(ItemsToSelect, true);
			ItemSelector->RequestScrollIntoView(Item);
			
			return;
		}
	}
}

void SNiagaraParameterPanel::AddParameterPendingRename(const FName ParameterName)
{
	bParameterItemsPendingChange = true;
	SelectParameterItemByName(ParameterName);
	ParametersWithRenamePending.Add(ParameterName);
}

void SNiagaraParameterPanel::AddParameterPendingNamespaceModifierRename(const FName ParameterName)
{
	bParameterItemsPendingChange = true;
	SelectParameterItemByName(ParameterName);
	ParametersWithNamespaceModifierRenamePending->Add(ParameterName);
}

TSharedPtr<TArray<FName>> SNiagaraParameterPanel::GetParametersWithNamespaceModifierRenamePending()
{
	return ParametersWithNamespaceModifierRenamePending;
}

void SNiagaraParameterPanel::ProcessParameterItemsPendingChange()
{
	auto FindItemByParameterName = [](const TArray<FNiagaraParameterPanelItem>& ItemsToSearch, const FName ParameterName)->const FNiagaraParameterPanelItem* {
		return ItemsToSearch.FindByPredicate([ParameterName](const FNiagaraParameterPanelItem& Item) {return Item.GetVariable().GetName() == ParameterName; });
	};

	const TArray<FNiagaraParameterPanelItem> SelectedItems = ItemSelector->GetSelectedItems();
 	for (const FName PendingName : ParametersWithRenamePending)
	{
		if (const FNiagaraParameterPanelItem* PendingItemPtr = FindItemByParameterName(SelectedItems, PendingName))
		{
			const FNiagaraParameterPanelItem& PendingItem = *PendingItemPtr;
			const TArray<FNiagaraParameterPanelItem> ItemsToSelect = { PendingItem };
			ItemSelector->RequestScrollIntoView(PendingItem);
			PendingItemPtr->RequestRename();
		}
	}

	const TArray<FName>& PendingNames = *ParametersWithNamespaceModifierRenamePending.Get();
	for (const FName PendingName : PendingNames)
	{
		if (const FNiagaraParameterPanelItem* PendingItemPtr = FindItemByParameterName(SelectedItems, PendingName))
		{
			const FNiagaraParameterPanelItem& PendingItem = *PendingItemPtr;
			const TArray<FNiagaraParameterPanelItem> ItemsToSelect = { PendingItem };
			ItemSelector->RequestScrollIntoView(PendingItem);
			PendingItemPtr->RequestRenameNamespaceModifier();
		}
	}

	ParametersWithRenamePending.Reset();
	ParametersWithNamespaceModifierRenamePending->Reset();
}

const FSlateBrush* SNiagaraParameterPanel::GetCategoryBackgroundImage(bool bIsCategoryHovered, bool bIsCategoryExpanded) const
{
	if (bIsCategoryHovered)
	{
		return bIsCategoryExpanded ? FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered") : FAppStyle::GetBrush("DetailsView.CollapsedCategory_Hovered");
	}
	else
	{
		return bIsCategoryExpanded ? FAppStyle::GetBrush("DetailsView.CategoryTop") : FAppStyle::GetBrush("DetailsView.CollapsedCategory");
	}
}

TSharedRef<SWidget> SNiagaraParameterPanel::GetViewOptionsMenu()
{
	FMenuBuilder MenuBuilder(false, nullptr);

	auto ToggleShowAdvancedCategoriesActionLambda = [this]() {
		UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
		Settings->SetDisplayAdvancedParameterPanelCategories(!Settings->GetDisplayAdvancedParameterPanelCategories());
		// Force a refresh as the SItemSelector categories need to be regenerated.
		Refresh();
	};

	auto GetShowAdvancedCategoriesCheckStateActionLambda = []() {
		UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
		return Settings->GetDisplayAdvancedParameterPanelCategories() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;;
	};

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowAdvancedCategoriesLabel", "Show Advanced Categories"),
		LOCTEXT("ShowAdvancedCategoriesToolTip", "Display advanced categories for the parameter panel."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(ToggleShowAdvancedCategoriesActionLambda),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda(GetShowAdvancedCategoriesCheckStateActionLambda)),
		NAME_None,
		EUserInterfaceActionType::Check
	);

	return MenuBuilder.MakeWidget();
}

FReply SNiagaraParameterPanel::HandleDragDropOperation(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragDropOperation = InDragDropEvent.GetOperation();
	if (DragDropOperation)
	{
		return ParameterPanelViewModel->HandleDragDropOperation(DragDropOperation);
	}

	return FReply::Unhandled();
}

bool SNiagaraParameterPanel::GetCanHandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	return ParameterPanelViewModel->GetCanHandleDragDropOperation(DragDropOperation);
}

const FSlateBrush* SNiagaraParameterPanel::GetViewOptionsBorderBrush()
{
	UNiagaraEditorSettings* Settings = GetMutableDefault<UNiagaraEditorSettings>();
	return Settings->GetDisplayAdvancedParameterPanelCategories()
		? FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Stack.DepressedHighlightedButtonBrush")
		: FAppStyle::GetBrush("NoBrush");
}

#undef LOCTEXT_NAMESPACE // "NiagaraParameterPanel"
