// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraParameterPanelPaletteItem.h"
#include "NiagaraActions.h"
#include "TutorialMetaData.h"
#include "EdGraphSchema_Niagara.h"

#define LOCTEXT_NAMESPACE "NiagaraParameterMapPalleteItem"

void SNiagaraParameterPanelPaletteItem::Construct(const FArguments& InArgs/*, FCreateWidgetForActionData* const InCreateData*/, TSharedRef<SWidget> ParameterNameViewWidget)
{
// 	check(InCreateData->Action.IsValid());
// 	TSharedPtr<FNiagaraScriptVarAndViewInfoAction> Action = StaticCastSharedPtr<FNiagaraScriptVarAndViewInfoAction>(InCreateData->Action);
// 	ActionPtr = InCreateData->Action;

	FTutorialMetaData TagMeta("PaletteItem");

	const FLinearColor TypeColor = FLinearColor(1, 1, 1, 0);/* UEdGraphSchema_Niagara::GetTypeColor(Action->GetScriptVarType());*/
	FSlateBrush const* IconBrush = FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
	FSlateBrush const* SecondaryBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateColor        IconColor = FSlateColor(TypeColor);
	FSlateColor        SecondaryIconColor = IconColor;
	FText			   IconToolTip = FText::GetEmpty();
	FString			   IconDocLink, IconDocExcerpt;
	TSharedRef<SWidget> IconWidget = CreateIconWidget(IconToolTip, IconBrush, IconColor, IconDocLink, IconDocExcerpt, SecondaryBrush, SecondaryIconColor);
	IconWidget->SetEnabled(true);

	static const FName BoldFontName = FName("Bold");
	static const FName ItalicFontName = FName("Italic");
	const FName FontType = ItalicFontName;
	FSlateFontInfo Font = FCoreStyle::GetDefaultFontStyle(FontType, 10);

	// now, create the actual widget
	ChildSlot
	[
		SNew(SHorizontalBox)
		.AddMetaData<FTutorialMetaData>(TagMeta)
		//.ToolTipText(Action->GetTooltipDescription())
		// icon slot
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			IconWidget
		]
		+SHorizontalBox::Slot()
		[
			ParameterNameViewWidget
		]
	];
}

#undef LOCTEXT_NAMESPACE // "SNiagaraParameterPanelPalleteItem"
