// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEditorCommon.h"
#include "SGraphPalette.h"

struct FCreateWidgetForActionData;

class SNiagaraParameterPanelPaletteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS(SNiagaraParameterPanelPaletteItem)
	{}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<SWidget> ParameterNameViewWidget);
};