// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraPinTypeSelector.h"

#include "NiagaraEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SNiagaraParameterMenu.h"
#include "NiagaraNode.h"
#include "NiagaraEditorUtilities.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "NiagaraPinTypeSelector"

void SNiagaraPinTypeSelector::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPin)
{
	ensure(InGraphPin);
	Pin = InGraphPin;

	ChildSlot
	[
		SAssignNew(SelectorButton, SComboButton)
		.ContentPadding(0.f)
		.HasDownArrow(false)
		.ButtonStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.Module.Pin.TypeSelector.Button")
		.ToolTipText(GetTooltipText())
		.OnGetMenuContent(this, &SNiagaraPinTypeSelector::GetMenuContent)
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Module.Pin.TypeSelector"))
		]
	];
}

TSharedRef<SWidget> SNiagaraPinTypeSelector::GetMenuContent()
{
	TArray<TWeakObjectPtr<UNiagaraGraph>> Graphs;
	UNiagaraNode* NiagaraNode = Cast<UNiagaraNode>(Pin->GetOwningNode());
	Graphs.Add(NiagaraNode->GetNiagaraGraph());

	TSharedRef<SNiagaraChangePinTypeMenu> MenuWidget = SNew(SNiagaraChangePinTypeMenu)
	.PinToModify(Pin)
	.AutoExpandMenu(true);

	SelectorButton->SetMenuContentWidgetToFocus(MenuWidget->GetSearchBox());
	return MenuWidget;
}

FText SNiagaraPinTypeSelector::GetTooltipText() const
{
	return LOCTEXT("PinTypeSelectorTooltip", "Choose a different type for this pin");
}

#undef LOCTEXT_NAMESPACE