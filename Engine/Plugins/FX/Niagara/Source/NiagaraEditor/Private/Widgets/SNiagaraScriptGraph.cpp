// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraScriptGraph.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraNode.h"
#include "NiagaraNodeInput.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraNodeReroute.h"
#include "NiagaraScriptSource.h"

#include "GraphEditor.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Text/TextLayout.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Text/STextBlock.h"
#include "NiagaraEditorSettings.h"
#include "EdGraphSchema_Niagara.h"
#include "ScopedTransaction.h"
#include "NiagaraNodeFactory.h"
#include "NiagaraEditorCommands.h"
#include "NiagaraNodeOutput.h"
#include "Widgets/Input/SButton.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "GraphEditorActions.h"
#include "EditorFontGlyphs.h"
#include "NiagaraEditorSettings.h"
#include "SGraphPanel.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/SNiagaraGraphActionMenu.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptGraph"

struct FNiagaraScriptZoomLevelsContainer : public FZoomLevelsContainer
{
	struct FZoomLevelEntry
	{
	public:
		FZoomLevelEntry(float InZoomAmount, const FText& InDisplayText, EGraphRenderingLOD::Type InLOD)
			: DisplayText(FText::Format(LOCTEXT("Zoom", "Zoom {0}"), InDisplayText))
			, ZoomAmount(InZoomAmount)
			, LOD(InLOD)
		{
		}

	public:
		FText DisplayText;
		float ZoomAmount;
		EGraphRenderingLOD::Type LOD;
	};
	
	FNiagaraScriptZoomLevelsContainer()
	{
		ZoomLevels.Reserve(20);
		ZoomLevels.Add(FZoomLevelEntry(0.100f, FText::FromString(TEXT("-12")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.125f, FText::FromString(TEXT("-11")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.150f, FText::FromString(TEXT("-10")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.175f, FText::FromString(TEXT("-9")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.200f, FText::FromString(TEXT("-8")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.225f, FText::FromString(TEXT("-7")), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.250f, FText::FromString(TEXT("-6")), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.375f, FText::FromString(TEXT("-5")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.500f, FText::FromString(TEXT("-4")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.675f, FText::FromString(TEXT("-3")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.750f, FText::FromString(TEXT("-2")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(0.875f, FText::FromString(TEXT("-1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.000f, FText::FromString(TEXT("1:1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.250f, FText::FromString(TEXT("+1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.375f, FText::FromString(TEXT("+2")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FZoomLevelEntry(1.500f, FText::FromString(TEXT("+3")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(1.675f, FText::FromString(TEXT("+4")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(1.750f, FText::FromString(TEXT("+5")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(1.875f, FText::FromString(TEXT("+6")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FZoomLevelEntry(2.000f, FText::FromString(TEXT("+7")), EGraphRenderingLOD::FullyZoomedIn));
	}

	float GetZoomAmount(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].ZoomAmount;
	}

	int32 GetNearestZoomLevel(float InZoomAmount) const override
	{
		for (int32 ZoomLevelIndex=0; ZoomLevelIndex < GetNumZoomLevels(); ++ZoomLevelIndex)
		{
			if (InZoomAmount <= GetZoomAmount(ZoomLevelIndex))
			{
				return ZoomLevelIndex;
			}
		}

		return GetDefaultZoomLevel();
	}
	
	FText GetZoomText(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].DisplayText;
	}
	
	int32 GetNumZoomLevels() const override
	{
		return ZoomLevels.Num();
	}
	
	int32 GetDefaultZoomLevel() const override
	{
		return 12;
	}

	EGraphRenderingLOD::Type GetLOD(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].LOD;
	}

	virtual EGraphZoomLimitHandling GetZoomLimitHandling() const override
	{
		return GetDefault<UNiagaraEditorSettings>()->GetScriptEditorZoomLimitHandling();
	}
	
	TArray<FZoomLevelEntry> ZoomLevels;
};

void SNiagaraScriptGraph::Construct(const FArguments& InArgs, TSharedRef<FNiagaraScriptGraphViewModel> InViewModel, const FAssetData& InEditedAsset)
{
	EditedAsset = InEditedAsset;
	
	UpdateViewModel(InViewModel);
	bUpdatingGraphSelectionFromViewModel = false;

	GraphTitle = InArgs._GraphTitle;
	SetForegroundColor(InArgs._ForegroundColor);
	bShowHeader = InArgs._ShowHeader;
	IsEditable = InArgs._IsEditable;
	DisplayAsReadOnly = InArgs._DisplayAsReadOnly;

	GraphEditor = ConstructGraphEditor();
	if (InArgs._ZoomToFitOnLoad)
	{
		GraphEditor->ZoomToFit(false);
	}

	ChildSlot
	[
		GraphEditor.ToSharedRef()
	];

	CurrentFocusedSearchMatchIndex = 0;

	AddAssetListeners();
}

SNiagaraScriptGraph::~SNiagaraScriptGraph()
{
	ClearListeners();
}

void SNiagaraScriptGraph::UpdateViewModel(TSharedRef<FNiagaraScriptGraphViewModel> InNewModel)
{
	bool bKeepOldGraphView = false;
	FVector2f Location;
	float ZoomAmount =1.0f;
	FGuid BookmarkId = FGuid();

	// remove old listeners
	if (ViewModel)
	{
		ViewModel->GetNodeSelection()->OnSelectedObjectsChanged().RemoveAll(this);
		ViewModel->OnNodesPasted().RemoveAll(this);
		ViewModel->OnGraphChanged().RemoveAll(this);

		if (GraphEditor.IsValid())
		{
			bKeepOldGraphView = true;
			GraphEditor->GetViewLocation(Location, ZoomAmount);
			GraphEditor->GetViewBookmark(BookmarkId);
		}
	}

	// set model and listeners
	ViewModel = InNewModel;
	ViewModel->GetNodeSelection()->OnSelectedObjectsChanged().AddSP(this, &SNiagaraScriptGraph::ViewModelSelectedNodesChanged);
	ViewModel->OnNodesPasted().AddSP(this, &SNiagaraScriptGraph::NodesPasted);
	ViewModel->OnGraphChanged().AddSP(this, &SNiagaraScriptGraph::GraphChanged);

	RecreateGraphWidget();

	if (bKeepOldGraphView && GraphEditor.IsValid())
	{
		GraphEditor->SetViewLocation(Location, ZoomAmount, BookmarkId);
	}
}

void SNiagaraScriptGraph::RecreateGraphWidget()
{
	GraphEditor = ConstructGraphEditor();
	ChildSlot
    [
        GraphEditor.ToSharedRef()
    ];
}

TSharedRef<SGraphEditor> SNiagaraScriptGraph::ConstructGraphEditor()
{
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText", "NIAGARA");

	TSharedRef<SWidget> TitleBarWidget =
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
		.HAlign(HAlign_Fill)
		[
			// Error Indicator and Title
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 0.0f, 3.0f, 0.0f)
					[
						SNew(SErrorText)
						.Visibility(ViewModel.ToSharedRef(), &FNiagaraScriptGraphViewModel::GetGraphErrorTextVisible)
						.BackgroundColor(ViewModel.ToSharedRef(), &FNiagaraScriptGraphViewModel::GetGraphErrorColor)
						.ToolTipText(ViewModel.ToSharedRef(), &FNiagaraScriptGraphViewModel::GetGraphErrorMsgToolTip)
						.ErrorText(ViewModel->GetGraphErrorText())
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(ViewModel.ToSharedRef(), &FNiagaraScriptGraphViewModel::GetDisplayName)
						.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
						.Justification(ETextJustify::Center)
						.Visibility(bShowHeader ? EVisibility::Visible : EVisibility::Collapsed)
					]
				]
				// optional info for downstream assets
				+ SVerticalBox::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
					.ColorAndOpacity(this, &SNiagaraScriptGraph::GetScriptSubheaderColor)
					.Visibility(this, &SNiagaraScriptGraph::GetScriptSubheaderVisibility)
					.HAlign(HAlign_Fill)
					[
						SNew(STextBlock)
						.Text(this, &SNiagaraScriptGraph::GetScriptSubheaderText)
						.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
						.Justification(ETextJustify::Center)
					]
				]
			]
			// Search Box
			+SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Top)
			.Padding(3)
			[
				SNew(SHorizontalBox)
				.Visibility(this, &SNiagaraScriptGraph::GetGraphSearchBoxVisibility)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(250)
					[
						SAssignNew(SearchBox, SSearchBox)
						.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.ScriptGraph.SearchBoxStyle")
						.HintText(LOCTEXT("GraphSearchBoxHint", "Search Nodes and Pins in Graph"))
						.SearchResultData(this, &SNiagaraScriptGraph::GetSearchResultData)
						.OnTextChanged(this, &SNiagaraScriptGraph::OnSearchTextChanged)
						.OnTextCommitted(this, &SNiagaraScriptGraph::OnSearchBoxTextCommitted)
						.DelayChangeNotificationsWhileTyping(true)
						.OnSearch(this, &SNiagaraScriptGraph::OnSearchBoxSearch)
						.OnKeyDownHandler(this, &SNiagaraScriptGraph::HandleGraphSearchBoxKeyDown)
					]
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.IsFocusable(false)
					.ForegroundColor(FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.Stack.FlatButtonColor"))
					.ToolTipText(LOCTEXT("CloseGraphSearchBox", "Close Graph search box"))
					.OnClicked(this, &SNiagaraScriptGraph::CloseGraphSearchBoxPressed)
					.ContentPadding(3)
					.Content()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.Text(FEditorFontGlyphs::Times)
					]
				]
			]
		];

	SGraphEditor::FGraphEditorEvents Events;
	Events.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &SNiagaraScriptGraph::GraphEditorSelectedNodesChanged);
	Events.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &SNiagaraScriptGraph::OnNodeDoubleClicked);
	Events.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &SNiagaraScriptGraph::OnNodeTitleCommitted);
	Events.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &SNiagaraScriptGraph::OnVerifyNodeTextCommit);
	Events.OnSpawnNodeByShortcutAtLocation = SGraphEditor::FOnSpawnNodeByShortcutAtLocation::CreateSP(this, &SNiagaraScriptGraph::OnSpawnGraphNodeByShortcut);
	Events.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &SNiagaraScriptGraph::OnCreateActionMenu);

	Commands = MakeShared<FUICommandList>();
	Commands->Append(ViewModel->GetCommands());
	Commands->MapAction(
		FNiagaraEditorCommands::Get().FindInCurrentView,
		FExecuteAction::CreateRaw(this, &SNiagaraScriptGraph::FocusGraphSearchBox));
	Commands->MapAction(
		FGraphEditorCommands::Get().CreateComment,
		FExecuteAction::CreateRaw(this, &SNiagaraScriptGraph::OnCreateComment));
	// Alignment Commands
	Commands->MapAction(FGraphEditorCommands::Get().AlignNodesTop,
		FExecuteAction::CreateSP(this, &SNiagaraScriptGraph::OnAlignTop)
	);

	Commands->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle,
		FExecuteAction::CreateSP(this, &SNiagaraScriptGraph::OnAlignMiddle)
	);

	Commands->MapAction(FGraphEditorCommands::Get().AlignNodesBottom,
		FExecuteAction::CreateSP(this, &SNiagaraScriptGraph::OnAlignBottom)
	);

	Commands->MapAction(FGraphEditorCommands::Get().AlignNodesLeft,
		FExecuteAction::CreateSP(this, &SNiagaraScriptGraph::OnAlignLeft)
	);

	Commands->MapAction(FGraphEditorCommands::Get().AlignNodesCenter,
		FExecuteAction::CreateSP(this, &SNiagaraScriptGraph::OnAlignCenter)
	);

	Commands->MapAction(FGraphEditorCommands::Get().AlignNodesRight,
		FExecuteAction::CreateSP(this, &SNiagaraScriptGraph::OnAlignRight)
	);

	Commands->MapAction(FGraphEditorCommands::Get().StraightenConnections,
		FExecuteAction::CreateSP(this, &SNiagaraScriptGraph::OnStraightenConnections)
	);

	// Distribution Commands
	Commands->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally,
		FExecuteAction::CreateSP(this, &SNiagaraScriptGraph::OnDistributeNodesH)
	);

	Commands->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically,
		FExecuteAction::CreateSP(this, &SNiagaraScriptGraph::OnDistributeNodesV)
	);
	
	TSharedRef<SGraphEditor> CreatedGraphEditor = SNew(SGraphEditor)
		.AdditionalCommands(Commands.ToSharedRef())
		.Appearance(AppearanceInfo)
		.TitleBar(TitleBarWidget)
		.GraphToEdit(ViewModel->GetGraph())
		.GraphEvents(Events)
		.ShowGraphStateOverlay(false)
		.IsEditable(IsEditable)
		.DisplayAsReadOnly(DisplayAsReadOnly);

	// Set a niagara node factory.
	CreatedGraphEditor->SetNodeFactory(MakeShareable(new FNiagaraNodeFactory()));
	CreatedGraphEditor->GetGraphPanel()->SetZoomLevelsContainer<FNiagaraScriptZoomLevelsContainer>();
	CreatedGraphEditor->ZoomToFit(false);

	return CreatedGraphEditor;
}

void SNiagaraScriptGraph::ViewModelSelectedNodesChanged()
{
	if (FNiagaraEditorUtilities::SetsMatch(GraphEditor->GetSelectedNodes(), ViewModel->GetNodeSelection()->GetSelectedObjectsResolved()) == false)
	{
		bUpdatingGraphSelectionFromViewModel = true;
		GraphEditor->ClearSelectionSet();
		for (UObject* SelectedNode : ViewModel->GetNodeSelection()->GetSelectedObjectsResolved())
		{
			UEdGraphNode* GraphNode = Cast<UEdGraphNode>(SelectedNode);
			if (GraphNode != nullptr)
			{
				GraphEditor->SetNodeSelection(GraphNode, true);
			}
		}
		bUpdatingGraphSelectionFromViewModel = false;
	}
}

void SNiagaraScriptGraph::GraphEditorSelectedNodesChanged(const TSet<UObject*>& SelectedNodes)
{
	if (bUpdatingGraphSelectionFromViewModel == false)
	{
		if (SelectedNodes.Num() == 0)
		{
			ViewModel->GetNodeSelection()->ClearSelectedObjects();
		} 
		else 
		{
			ViewModel->GetNodeSelection()->SetSelectedObjects(SelectedNodes);
		}
	}
}

void SNiagaraScriptGraph::OnNodeDoubleClicked(UEdGraphNode* ClickedNode)
{
	UNiagaraNode* NiagaraNode = Cast<UNiagaraNode>(ClickedNode);
	if (NiagaraNode != nullptr)
	{
		NiagaraNode->OpenReferencedAsset();
	}
}

void SNiagaraScriptGraph::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		// When you request rename on spawn but accept the value, we want to not add a transaction if they just hit "Enter".
		bool bRename = true;
		if (NodeBeingChanged->IsA(UNiagaraNodeInput::StaticClass()))
		{
			FName CurrentName = Cast<UNiagaraNodeInput>(NodeBeingChanged)->Input.GetName();
			if (CurrentName.ToString().Equals(NewText.ToString(), ESearchCase::CaseSensitive))
			{
				bRename = false;
			}
		}

		if (bRename)
		{
			const FScopedTransaction Transaction(LOCTEXT("RenameNode", "Rename Node"));
			NodeBeingChanged->Modify();
			NodeBeingChanged->OnRenameNode(NewText.ToString());

			// If the node isn't a NiagarNode, like say a comment, go ahead and mark it dirty
			if (!NodeBeingChanged->IsA(UNiagaraNode::StaticClass()))
			{
				ViewModel->GetGraph()->NotifyGraphNeedsRecompile();
			}
		}
	}
}

bool SNiagaraScriptGraph::OnVerifyNodeTextCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage)
{
	bool bValid = true;
	UNiagaraNodeInput* InputNodeBeingChanged = Cast<UNiagaraNodeInput>(NodeBeingChanged);
	if (InputNodeBeingChanged != nullptr)
	{
		return FNiagaraEditorUtilities::VerifyNameChangeForInputOrOutputNode(*InputNodeBeingChanged, InputNodeBeingChanged->Input.GetName(), NewText.ToString(), OutErrorMessage);
	}
	return bValid;
}

FReply SNiagaraScriptGraph::OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition)
{
	if (ViewModel.Get() == nullptr)
	{
		return FReply::Unhandled();
	}
	UNiagaraGraph* Graph = ViewModel->GetGraph();
	if (Graph == nullptr)
	{
		return FReply::Unhandled();
	}

	const UNiagaraEditorSettings* Settings = GetDefault<UNiagaraEditorSettings>();
	if (Settings == nullptr)
	{
		return FReply::Unhandled();
	}

	for (int32 i = 0; i < Settings->GraphCreationShortcuts.Num(); i++)
	{
		if (Settings->GraphCreationShortcuts[i].Input.GetRelationship(InChord) == FInputChord::ERelationshipType::Same)
		{
			const UEdGraphSchema_Niagara* Schema = GetDefault<UEdGraphSchema_Niagara>();

			UEdGraph* OwnerOfTemporaries = NewObject<UEdGraph>((UObject*)GetTransientPackage());
			TArray<UObject*> SelectedObjects;
			TArray<TSharedPtr<FNiagaraAction_NewNode>> Actions = Schema->GetGraphActions(Graph, nullptr, OwnerOfTemporaries);

			for (int32 ActionIdx = 0; ActionIdx < Actions.Num(); ActionIdx++)
			{
				TSharedPtr<FNiagaraAction_NewNode>& NiagaraAction = Actions[ActionIdx];
				
				bool bMatch = false;
				bool bCanMatch = true;
				TArray<FString> CategoryParsedTerms;
				Settings->GraphCreationShortcuts[i].Name.ParseIntoArray(CategoryParsedTerms, TEXT("::"), true);
				FString ActionDisplayName = CategoryParsedTerms.Last();

				// if we have more than one term, the shortcut has at least one category specified. In that case, we require the category chain to be identical
				if(CategoryParsedTerms.Num() > 1 && CategoryParsedTerms.Num() - 1 == NiagaraAction->Categories.Num())
				{
					for(int32 CategoryIndex = 0; CategoryIndex < CategoryParsedTerms.Num() - 1; CategoryIndex++)
					{
						if(!CategoryParsedTerms[CategoryIndex].Equals(NiagaraAction->Categories[CategoryIndex], ESearchCase::IgnoreCase))
						{
							bCanMatch = false;
							break;
						}
					}					
				}

				// we can match either only via DisplayName or via Category1::...:::ActionDisplayName 
				if (bCanMatch && NiagaraAction->DisplayName.ToString().Equals(ActionDisplayName, ESearchCase::IgnoreCase))
				{
					bMatch = true;
				}
				
				if (bMatch)
				{
					FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));
					TArray<UEdGraphPin*> Pins;
					NiagaraAction->CreateNode(Graph, Pins, FDeprecateSlateVector2D(InPosition));
					return FReply::Handled();
				}					
			}
		}
	}

	return FReply::Unhandled();

	/*
	if (Graph == nullptr)
	{
	return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));

	TArray<UEdGraphNode*> OutNodes;
	FVector2D NodeSpawnPos = InPosition;
	FBlueprintSpawnNodeCommands::Get().GetGraphActionByChord(InChord, InGraph, NodeSpawnPos, OutNodes);

	TSet<const UEdGraphNode*> NodesToSelect;

	for (UEdGraphNode* CurrentNode : OutNodes)
	{
	NodesToSelect.Add(CurrentNode);
	}

	// Do not change node selection if no actions were performed
	if(OutNodes.Num() > 0)
	{
	Graph->SelectNodeSet(NodesToSelect, true);
}
	else
	{
		Transaction.Cancel();
	}

	return FReply::Handled();
	*/
}

FActionMenuContent SNiagaraScriptGraph::OnCreateActionMenu(UEdGraph* Graph, const FVector2f& Position, const TArray<UEdGraphPin*>& DraggedPins, bool bAutoExpandActionMenu, SGraphEditor::FActionMenuClosed OnClosed)
{
	FActionMenuContent Content;
	
	TSharedPtr<SNiagaraGraphActionMenu> ActionMenu = SNew(SNiagaraGraphActionMenu)
	.GraphObj(Graph)
	.NewNodePosition(Position)
	.DraggedFromPins(DraggedPins)
	.AutoExpandActionMenu(bAutoExpandActionMenu)
	.OnClosedCallback(OnClosed);

	Content.Content = ActionMenu.ToSharedRef();
	Content.WidgetToFocus = ActionMenu->GetFilterTextBox();
	return Content;
}


void SNiagaraScriptGraph::NodesPasted(const TSet<UEdGraphNode*>& PastedNodes)
{
	if (PastedNodes.Num() != 0)
	{
		PositionPastedNodes(PastedNodes);
		GraphEditor->NotifyGraphChanged();
	}
}

void SNiagaraScriptGraph::PositionPastedNodes(const TSet<UEdGraphNode*>& PastedNodes)
{
	FVector2f AvgNodePosition(0.0f, 0.0f);

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		AvgNodePosition.X += PastedNode->NodePosX;
		AvgNodePosition.Y += PastedNode->NodePosY;
	}

	float InvNumNodes = 1.0f / float(PastedNodes.Num());
	AvgNodePosition.X *= InvNumNodes;
	AvgNodePosition.Y *= InvNumNodes;

	FVector2f PasteLocation = GraphEditor->GetPasteLocation2f();
	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		PastedNode->NodePosX = (PastedNode->NodePosX - AvgNodePosition.X) + PasteLocation.X;
		PastedNode->NodePosY = (PastedNode->NodePosY - AvgNodePosition.Y) + PasteLocation.Y;

		PastedNode->SnapToGrid(16);
	}
}

void SNiagaraScriptGraph::GraphChanged()
{
	TSharedPtr<SGraphEditor> NewGraphEditor;
	TSharedPtr<SWidget> NewChildWidget;

	if (ViewModel->GetGraph() != nullptr)
	{
		NewGraphEditor = ConstructGraphEditor();
		NewChildWidget = NewGraphEditor;
	}
	else
	{
		NewGraphEditor = nullptr;
		NewChildWidget = SNullWidget::NullWidget;
	}
	
	GraphEditor = NewGraphEditor;
	ChildSlot
	[
		NewChildWidget.ToSharedRef()
	];
}

void SNiagaraScriptGraph::FocusGraphElement(const INiagaraScriptGraphFocusInfo* FocusInfo)
{
	checkf(FocusInfo->GetFocusType() != INiagaraScriptGraphFocusInfo::ENiagaraScriptGraphFocusInfoType::None, TEXT("Failed to assign focus type to FocusInfo parameter!"));

	if (FocusInfo->GetFocusType() == INiagaraScriptGraphFocusInfo::ENiagaraScriptGraphFocusInfoType::Node)
	{
		const FNiagaraScriptGraphNodeToFocusInfo* NodeFocusInfo = static_cast<const FNiagaraScriptGraphNodeToFocusInfo*>(FocusInfo);
		const FGuid& NodeGuidToMatch = NodeFocusInfo->GetNodeGuidToFocus();
		TObjectPtr<UEdGraphNode> const* NodeToFocus = ViewModel->GetGraph()->Nodes.FindByPredicate([&NodeGuidToMatch](const UEdGraphNode* Node) {return Node->NodeGuid == NodeGuidToMatch; });
		if (NodeToFocus != nullptr && *NodeToFocus != nullptr)
		{
			GetGraphEditor()->JumpToNode(*NodeToFocus);
			return;
		}
		ensureMsgf(false, TEXT("Failed to find Node with matching GUID when focusing graph element. Was the graph edited out from underneath us?"));
		return;
	}
	else if (FocusInfo->GetFocusType() == INiagaraScriptGraphFocusInfo::ENiagaraScriptGraphFocusInfoType::Pin)
	{
		const FNiagaraScriptGraphPinToFocusInfo* PinFocusInfo = static_cast<const FNiagaraScriptGraphPinToFocusInfo*>(FocusInfo);
		const FGuid& PinGuidToMatch = PinFocusInfo->GetPinGuidToFocus();
		for (const UEdGraphNode* Node : ViewModel->GetGraph()->Nodes)
		{
			const UEdGraphPin* const* PinToFocus = Node->Pins.FindByPredicate([&PinGuidToMatch](const UEdGraphPin* Pin) {return Pin->PersistentGuid == PinGuidToMatch; });
			if (PinToFocus != nullptr && *PinToFocus != nullptr)
			{
				GetGraphEditor()->JumpToPin(*PinToFocus);
				return;
			}
		}
		ensureMsgf(false, TEXT("Failed to find Pin with matching GUID when focusing graph element. Was the graph edited out from underneath us?"));
		return;
	}
	checkf(false, TEXT("Requested focus for a graph element without specifying a Node or Pin to focus!"));
}

bool NodeMatchesSearch(UNiagaraNode* Node, const FString& SearchTextString)
{
	if (Node->IsA<UNiagaraNodeReroute>())
	{
		// Ignore reroute nodes.
		return false;
	}

	if (Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString().Contains(SearchTextString))
	{
		return true;
	}
	
	if (Node->IsA<UNiagaraNodeParameterMapBase>())
	{
		UNiagaraNodeParameterMapBase* ParameterMapNode = CastChecked<UNiagaraNodeParameterMapBase>(Node);
		for (UEdGraphPin* Pin : ParameterMapNode->Pins)
		{
			if (ParameterMapNode->IsAddPin(Pin))
			{
				if (Pin->GetDisplayName().ToString().Contains(SearchTextString))
				{
					return true;
				}
			}
			else
			{
				if (FNiagaraParameterUtilities::DoesParameterNameMatchSearchText(Pin->PinName, SearchTextString))
				{
					return true;
				}
			}
		}
	}
	else
	{
		if (Node->Pins.ContainsByPredicate([&SearchTextString](UEdGraphPin* Pin) { return Pin->GetDisplayName().ToString().Contains(SearchTextString); }))
		{
			return true;
		}
	}
	return false;
}

void SNiagaraScriptGraph::OnSearchTextChanged(const FText& SearchText)
{
	if (!CurrentSearchText.EqualTo(SearchText))
	{
		CurrentSearchResults.Empty();
		CurrentSearchText = SearchText;

		TArray<UNiagaraNode*> AllNodes;
		ViewModel->GetGraph()->GetNodesOfClass<UNiagaraNode>(AllNodes);

		TArray<UNiagaraNodeOutput*> OutputNodes;
		ViewModel->GetGraph()->GetNodesOfClass<UNiagaraNodeOutput>(OutputNodes);

		TArray<UNiagaraNode*> TraversedNodes;
		for (UNiagaraNodeOutput* OutputNode : OutputNodes)
		{
			TArray<UNiagaraNode*> OutputNodeTraversal;
			UNiagaraGraph::BuildTraversal(OutputNodeTraversal, OutputNode, false);
			TraversedNodes.Append(OutputNodeTraversal);
		}

		AllNodes.RemoveAll([&TraversedNodes](UNiagaraNode* Node) { return TraversedNodes.Contains(Node); });

		TArray<UNiagaraNode*> OrderedSearchNodes;
		OrderedSearchNodes.Append(TraversedNodes);
		OrderedSearchNodes.Append(AllNodes);

		FString SearchTextString = SearchText.ToString();
		for (UNiagaraNode* SearchNode : OrderedSearchNodes)
		{
			if(NodeMatchesSearch(SearchNode, SearchTextString))
			{
				CurrentSearchResults.Add(MakeShared<FNiagaraScriptGraphNodeToFocusInfo>(SearchNode->NodeGuid));
			}
		}

		CurrentFocusedSearchMatchIndex = 0;
		if (CurrentSearchResults.Num() > 0)
		{
			FocusGraphElement(CurrentSearchResults[0].Get());
		}
	}
}

void SNiagaraScriptGraph::OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		if (CurrentSearchText.CompareTo(NewText) == 0)
		{
			OnSearchBoxSearch(SSearchBox::Next);
		}
		else
		{
			OnSearchTextChanged(NewText);
		}
	}
}

TOptional<SSearchBox::FSearchResultData> SNiagaraScriptGraph::GetSearchResultData() const
{
	if (CurrentSearchText.IsEmpty() || CurrentSearchResults.Num() == 0)
	{
		return TOptional<SSearchBox::FSearchResultData>();
	}
	return TOptional<SSearchBox::FSearchResultData>({ CurrentSearchResults.Num(), CurrentFocusedSearchMatchIndex + 1 });
}

FText SNiagaraScriptGraph::GetScriptSubheaderText() const
{
	int32 SearchLimit = GetDefault<UNiagaraEditorSettings>()->GetAssetStatsSearchLimit();
	int32 AffectedAssets = GetScriptAffectedAssets();
	bool bVersioned = false;
	if (UNiagaraScriptSource* ScriptSource = ViewModel->GetScriptSource())
	{
		if (UNiagaraScript* Script = ScriptSource->GetTypedOuter<UNiagaraScript>())
		{
			bVersioned = Script->IsVersioningEnabled();
		}
	}
	FText VersionText = bVersioned ? LOCTEXT("ScriptSubheaderVersionInfoText", "(across all versions)") : FText();
	FText LimitReachedText = AffectedAssets >= SearchLimit ? LOCTEXT("ScriptSubheaderLimitReachedText", "more than ") : FText();
	return FText::Format(LOCTEXT("ScriptSubheaderText", "Note: editing this script will affect {0}{1} dependent {1}|plural(one=asset,other=assets)! {2}"), LimitReachedText, FText::AsNumber(AffectedAssets), VersionText);
}

EVisibility SNiagaraScriptGraph::GetScriptSubheaderVisibility() const
{
	return bShowHeader && (GetScriptAffectedAssets() > 0) ? EVisibility::Visible : EVisibility::Collapsed;
}

FLinearColor SNiagaraScriptGraph::GetScriptSubheaderColor() const
{
	return GetScriptAffectedAssets() >= 5 ? FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.ScriptGraph.AffectedAssetsWarningColor") : FLinearColor::White;
}

int32 SNiagaraScriptGraph::GetScriptAffectedAssets() const
{
	if (!EditedAsset.IsValid())
	{
		return 0;
	}
	const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();
	if (EditorSettings->GetDisplayAffectedAssetStats() == false)
	{
		return 0;
	}
	
	if (!ScriptAffectedAssets.IsSet())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		if (AssetRegistry.IsLoadingAssets())
		{
			// We are still discovering assets, wait a bit
			return 0;
		}

		ScriptAffectedAssets = FNiagaraEditorUtilities::GetReferencedAssetCount(EditedAsset, [](const FAssetData& AssetToCheck)
		{
			if (AssetToCheck.GetClass() == UNiagaraSystem::StaticClass())
			{
				return FNiagaraEditorUtilities::ETrackAssetResult::Count;
			}
			if (AssetToCheck.GetClass() == UNiagaraEmitter::StaticClass() || AssetToCheck.GetClass() == UNiagaraScript::StaticClass())
			{
				return FNiagaraEditorUtilities::ETrackAssetResult::CountRecursive;
			}
			return FNiagaraEditorUtilities::ETrackAssetResult::Ignore;
		});
	}
	return ScriptAffectedAssets.Get(0);
}

void SNiagaraScriptGraph::ResetAssetCount(const FAssetData&)
{
	ScriptAffectedAssets.Reset();
}

void SNiagaraScriptGraph::AddAssetListeners()
{
	if (EditedAsset.IsValid())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.OnAssetUpdated().AddRaw(this, &SNiagaraScriptGraph::ResetAssetCount);
		AssetRegistry.OnAssetAdded().AddRaw(this, &SNiagaraScriptGraph::ResetAssetCount);
		AssetRegistry.OnAssetRemoved().AddRaw(this, &SNiagaraScriptGraph::ResetAssetCount);
	}
}

void SNiagaraScriptGraph::ClearListeners()
{
	if (EditedAsset.IsValid())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.OnAssetUpdated().RemoveAll(this);
		AssetRegistry.OnAssetAdded().RemoveAll(this);
		AssetRegistry.OnAssetRemoved().RemoveAll(this);
	}
}

FReply SNiagaraScriptGraph::CloseGraphSearchBoxPressed()
{
	bGraphSearchBoxActive = false;
	return FReply::Handled();
}

FReply SNiagaraScriptGraph::HandleGraphSearchBoxKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		CloseGraphSearchBoxPressed();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}


void SNiagaraScriptGraph::OnAlignTop()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignTop();
	}
}

void SNiagaraScriptGraph::OnAlignMiddle()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignMiddle();
	}
}

void SNiagaraScriptGraph::OnAlignBottom()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignBottom();
	}
}

void SNiagaraScriptGraph::OnAlignLeft()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignLeft();
	}
}

void SNiagaraScriptGraph::OnAlignCenter()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignCenter();
	}
}

void SNiagaraScriptGraph::OnAlignRight()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignRight();
	}
}

void SNiagaraScriptGraph::OnStraightenConnections()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnStraightenConnections();
	}
}

void SNiagaraScriptGraph::OnDistributeNodesH()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnDistributeNodesH();
	}
}

void SNiagaraScriptGraph::OnDistributeNodesV()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnDistributeNodesV();
	}
}


void SNiagaraScriptGraph::FocusGraphSearchBox()
{
	bGraphSearchBoxActive = true;

	if (SearchBox.IsValid())
	{
		FWidgetPath WidgetToFocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchBox.ToSharedRef(), WidgetToFocusPath, EVisibility::All);
		FSlateApplication::Get().SetKeyboardFocus(WidgetToFocusPath, EFocusCause::SetDirectly);
	}
}

void SNiagaraScriptGraph::OnCreateComment()
{
	FNiagaraSchemaAction_NewComment CommentAction = FNiagaraSchemaAction_NewComment(GraphEditor);
	if (CommentAction.PerformAction(ViewModel->GetGraph(), nullptr, GraphEditor->GetPasteLocation2f(), false))
	{
		ViewModel->GetGraph()->NotifyGraphNeedsRecompile();
	}
}

void SNiagaraScriptGraph::OnSearchBoxSearch(SSearchBox::SearchDirection Direction)
{
	if (CurrentSearchResults.Num() > 0)
	{
		if (Direction == SSearchBox::Next)
		{
			CurrentFocusedSearchMatchIndex = CurrentFocusedSearchMatchIndex < CurrentSearchResults.Num() - 1 ? CurrentFocusedSearchMatchIndex + 1 : 0;
			FocusGraphElement(CurrentSearchResults[CurrentFocusedSearchMatchIndex].Get());
		}
		else if (Direction == SSearchBox::Previous)
		{
			CurrentFocusedSearchMatchIndex = CurrentFocusedSearchMatchIndex > 0 ? CurrentFocusedSearchMatchIndex - 1 : CurrentSearchResults.Num() - 1;
			FocusGraphElement(CurrentSearchResults[CurrentFocusedSearchMatchIndex].Get());
		}
	}
}

#undef LOCTEXT_NAMESPACE // "NiagaraScriptGraph"
