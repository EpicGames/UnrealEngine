// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraScriptInputPreviewPanel.h"

#include "DataHierarchyEditorStyle.h"
#include "DataHierarchyViewModelBase.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraObjectSelection.h"
#include "NiagaraScriptSource.h"
#include "NiagaraScriptVariable.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Toolkits/NiagaraScriptToolkit.h"
#include "ViewModels/HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

#define LOCTEXT_NAMESPACE "NiagaraEditor"

void SNiagaraScriptInputPreviewPanel::Construct(const FArguments& InArgs, TSharedRef<FNiagaraScriptToolkit> InScriptToolkit, TSharedRef<FNiagaraObjectSelection> InVariableObjectSelection)
{
	ScriptToolkit = InScriptToolkit;
	VariableObjectSelection = InVariableObjectSelection;

	SetupDelegates();

	// Search Box is meaningless without row selection & functionality
	// If we add functionality in the future, this already works, so we only skip adding it to the UI for now
	SAssignNew(SearchBox, SSearchBox)
	.OnTextChanged(this, &SNiagaraScriptInputPreviewPanel::OnSearchTextChanged)
	.OnTextCommitted(this, &SNiagaraScriptInputPreviewPanel::OnSearchTextCommitted)
	.OnSearch(SSearchBox::FOnSearch::CreateSP(this, &SNiagaraScriptInputPreviewPanel::OnSearchButtonClicked))
	.DelayChangeNotificationsWhileTyping(true)
	.SearchResultData(this, &SNiagaraScriptInputPreviewPanel::GetSearchResultData);
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(4.f)
		[
			SNew(SButton)
			.OnClicked(this, &SNiagaraScriptInputPreviewPanel::SummonHierarchyEditor)
			.ButtonStyle(FAppStyle::Get(), "RoundButton")
			[
				SNew(STextBlock)
				.Text(LOCTEXT("EditHierarchy_ScriptInputs", "Edit Input Hierarchy"))
			]
		]
		+ SVerticalBox::Slot()
		[
			SAssignNew(TreeView, STreeView<UHierarchyElement*>)
			.TreeItemsSource(&RootArray)
			.OnGenerateRow(this, &SNiagaraScriptInputPreviewPanel::OnGenerateRow)
			.OnGetChildren(this, &SNiagaraScriptInputPreviewPanel::OnGetChildren)
			.SelectionMode(ESelectionMode::Type::None)
		]
	];

	Refresh();
}

SNiagaraScriptInputPreviewPanel::~SNiagaraScriptInputPreviewPanel()
{
	RemoveDelegates();
}

void SNiagaraScriptInputPreviewPanel::Refresh()
{
	if(ScriptToolkit.IsValid() == false || ScriptToolkit.Pin()->GetHierarchyViewModel() == nullptr || ScriptToolkit.Pin()->GetHierarchyViewModel()->GetHierarchyRoot() == nullptr)
	{
		return;
	}
	
	TWeakObjectPtr<UHierarchyRoot> Root = ScriptToolkit.Pin()->GetHierarchyViewModel()->GetHierarchyRoot();
	if(Root.IsValid())
	{
		TransientLeftoverParameters.Empty();
		RootArray.Empty();
		// We don't want to display the root itself; we add the root's explicitly added children here
		RootArray.Append(Root->GetChildren());

		// Next, we want to take care of input parameters not added to the hierarchy
		TArray<UNiagaraHierarchyScriptParameter*> HierarchyScriptParameters;
		Root->GetChildrenOfType(HierarchyScriptParameters, true);

		TArray<UNiagaraScriptVariable*> ScriptVariablesInHierarchy;
		Algo::Transform(HierarchyScriptParameters, ScriptVariablesInHierarchy, [](UNiagaraHierarchyScriptParameter* HierarchyScriptParameter)
		{
			return HierarchyScriptParameter->GetScriptVariable();
		});
		
		TArray<UNiagaraScriptVariable*> AllInputScriptVariables;
		Cast<UNiagaraScriptSource>(ScriptToolkit.Pin()->EditedNiagaraScript.GetScriptData()->GetSource())->NodeGraph->GetAllInputScriptVariables(AllInputScriptVariables);
		
		AllInputScriptVariables.RemoveAll([&ScriptVariablesInHierarchy](UNiagaraScriptVariable* ScriptVariableCandidate)
		{
			return ScriptVariablesInHierarchy.Contains(ScriptVariableCandidate);
		});

		// We construct a transient list of script parameters for all the leftover parameters
		for(UNiagaraScriptVariable* LeftoverScriptVariable : AllInputScriptVariables)
		{
			// They need to have an outer that has the graph as the outer to properly function, so we create transient script parameters under the hierarchy root
			UNiagaraHierarchyScriptParameter* LeftoverTransientScriptParameter = NewObject<UNiagaraHierarchyScriptParameter>(Root.Get(), NAME_None, RF_Transient);
			LeftoverTransientScriptParameter->Initialize(*LeftoverScriptVariable);
			TransientLeftoverParameters.Add(LeftoverTransientScriptParameter);
		}

		// Since they haven't been added to the hierarchy, we sort them lexicographically
		Algo::Sort(TransientLeftoverParameters, [](TObjectPtr<UNiagaraHierarchyScriptParameter> HierarchyScriptParameterA, TObjectPtr<UNiagaraHierarchyScriptParameter> HierarchyScriptParameterB)
		{
			return HierarchyScriptParameterA->ToString() < HierarchyScriptParameterB->ToString();
		});
		
		RootArray.Append(TransientLeftoverParameters);
		
		TreeView->RequestTreeRefresh();
	}
}

TSharedRef<ITableRow> SNiagaraScriptInputPreviewPanel::OnGenerateRow(UHierarchyElement* Item, const TSharedRef<STableViewBase>& TableViewBase) const
{
	TSharedPtr<STableRow<UHierarchyElement*>> TableRow;
	if(UHierarchyCategory* Category = Cast<UHierarchyCategory>(Item))
	{		
		TableRow = SNew(STableRow<UHierarchyElement*>, TableViewBase)
		.ToolTipText_UObject(Category, &UHierarchyCategory::GetTooltip)
		.Style(FDataHierarchyEditorStyle::Get(), "HierarchyEditor.Row.Category")
		.Padding(FMargin(0.f, 6.f))
		[
			SNew(SRichTextBlock)
			.Text_UObject(Item, &UHierarchyElement::ToText)
			.TextStyle(&FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.Parameters.HeaderText"))
		];
	}
	else if(UNiagaraHierarchyScriptParameter* HierarchyScriptParameter = Cast<UNiagaraHierarchyScriptParameter>(Item))
	{
		TableRow = SNew(STableRow<UHierarchyElement*>, TableViewBase)
		.ToolTipText_UObject(HierarchyScriptParameter, &UNiagaraHierarchyScriptParameter::GetTooltip)
		.Padding(FMargin(0.f, 2.f))
		[
			FNiagaraParameterUtilities::GetParameterWidget(HierarchyScriptParameter->GetVariable().Get(FNiagaraVariable()), true, false)
		];
	}

	return TableRow.ToSharedRef();
}

void SNiagaraScriptInputPreviewPanel::OnGetChildren(UHierarchyElement* Item, TArray<UHierarchyElement*>& OutChildren) const
{
	OutChildren.Append(Item->GetChildrenMutable());
}

FReply SNiagaraScriptInputPreviewPanel::SummonHierarchyEditor() const
{
	if(ScriptToolkit.IsValid())
	{
		ScriptToolkit.Pin()->GetTabManager()->TryInvokeTab(FNiagaraScriptToolkit::HierarchyEditor_ParametersTabId);
	}

	return FReply::Handled();
}

void SNiagaraScriptInputPreviewPanel::OnSearchTextChanged(const FText& Text)
{
	SearchResults.Empty();
	FocusedSearchResult.Reset();
	TreeView->ClearSelection();

	if(!Text.IsEmpty())
	{
		FString TextAsString = Text.ToString();;

		TArray<FSearchItem> SearchItems;
		for(UHierarchyElement* RootItem : RootArray)
		{
			GenerateSearchItems(RootItem, {}, SearchItems);
			
		}
		
		for(const FSearchItem& SearchItem : SearchItems)
		{
			if(SearchItem.GetEntry()->ToString().Contains(TextAsString))
			{
				SearchResults.Add(SearchItem);
			}
		}

		ExpandSourceSearchResults();
		SelectNextSourceSearchResult();
	}
	else
	{
		TreeView->ClearExpandedItems();
	}
}

void SNiagaraScriptInputPreviewPanel::OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	bool bIsShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	if(CommitType == ETextCommit::OnEnter)
	{
		if(bIsShiftDown == false)
		{
			SelectNextSourceSearchResult();
		}
		else
		{
			SelectPreviousSourceSearchResult();
		}
	}
}

void SNiagaraScriptInputPreviewPanel::OnSearchButtonClicked(SSearchBox::SearchDirection SearchDirection)
{
	if(SearchDirection == SSearchBox::Next)
	{
		SelectNextSourceSearchResult();
	}
	else
	{
		SelectPreviousSourceSearchResult();
	}
}

void SNiagaraScriptInputPreviewPanel::GenerateSearchItems(UHierarchyElement* InRoot, TArray<UHierarchyElement*> ParentChain, TArray<FSearchItem>& OutSearchItems)
{
	TConstArrayView<const TObjectPtr<UHierarchyElement>> Children = InRoot->GetChildren();
	ParentChain.Add(InRoot);
	OutSearchItems.Add(FSearchItem{ParentChain});
	for(UHierarchyElement* Child : Children)
	{
		GenerateSearchItems(Child, ParentChain, OutSearchItems);
	}
}

void SNiagaraScriptInputPreviewPanel::ExpandSourceSearchResults()
{
	TreeView->ClearExpandedItems();

	for(FSearchItem& SearchResult : SearchResults)
	{
		for(UHierarchyElement* EntryInPath : SearchResult.Path)
		{
			TreeView->SetItemExpansion(EntryInPath, true);
		}
	}
}

void SNiagaraScriptInputPreviewPanel::SelectNextSourceSearchResult()
{
	if(SearchResults.IsEmpty())
	{
		return;
	}
	
	if(!FocusedSearchResult.IsSet())
	{
		FocusedSearchResult = SearchResults[0];
	}
	else
	{
		int32 CurrentSearchResultIndex = SearchResults.Find(FocusedSearchResult.GetValue());
		if(SearchResults.IsValidIndex(CurrentSearchResultIndex+1))
		{
			FocusedSearchResult = SearchResults[CurrentSearchResultIndex+1];
		}
		else
		{
			FocusedSearchResult = SearchResults[0];
		}
	}

	TreeView->ClearSelection();
	TreeView->RequestScrollIntoView(FocusedSearchResult.GetValue().GetEntry());
	TreeView->SetItemSelection(FocusedSearchResult.GetValue().GetEntry(), true);
}

void SNiagaraScriptInputPreviewPanel::SelectPreviousSourceSearchResult()
{
	if(SearchResults.IsEmpty())
	{
		return;
	}
	
	if(!FocusedSearchResult.IsSet())
	{
		FocusedSearchResult = SearchResults[0];
	}
	else
	{
		int32 CurrentSearchResultIndex = SearchResults.Find(FocusedSearchResult.GetValue());
		if(SearchResults.IsValidIndex(CurrentSearchResultIndex-1))
		{
			FocusedSearchResult = SearchResults[CurrentSearchResultIndex-1];
		}
		else
		{
			FocusedSearchResult = SearchResults[SearchResults.Num()-1];
		}
	}

	TreeView->ClearSelection();
	TreeView->RequestScrollIntoView(FocusedSearchResult.GetValue().GetEntry());
	TreeView->SetItemSelection(FocusedSearchResult.GetValue().GetEntry(), true);
}

TOptional<SSearchBox::FSearchResultData> SNiagaraScriptInputPreviewPanel::GetSearchResultData() const
{
	if(SearchResults.Num() > 0)
	{
		SSearchBox::FSearchResultData SearchResultData;
		SearchResultData.NumSearchResults = SearchResults.Num();

		if(FocusedSearchResult.IsSet())
		{
			// we add one just to make it look nicer as this is merely for cosmetic purposes
			SearchResultData.CurrentSearchResultIndex = SearchResults.Find(FocusedSearchResult.GetValue()) + 1;
		}
		else
		{
			SearchResultData.CurrentSearchResultIndex = INDEX_NONE;
		}

		return SearchResultData;
	}

	return TOptional<SSearchBox::FSearchResultData>();
}

void SNiagaraScriptInputPreviewPanel::SetupDelegates()
{
	Cast<UNiagaraScriptSource>(ScriptToolkit.Pin()->EditedNiagaraScript.GetScriptData()->GetSource())->NodeGraph->OnParametersChanged().AddSP(this, &SNiagaraScriptInputPreviewPanel::OnParametersChanged);

	ScriptToolkit.Pin()->GetHierarchyViewModel()->GetOnHierarchyChangedDelegate().AddSP(this, &SNiagaraScriptInputPreviewPanel::RefreshOnHierarchyChanged);
}

void SNiagaraScriptInputPreviewPanel::RemoveDelegates()
{
	if(ScriptToolkit.IsValid())
	{
		Cast<UNiagaraScriptSource>(ScriptToolkit.Pin()->EditedNiagaraScript.GetScriptData()->GetSource())->NodeGraph->OnParametersChanged().RemoveAll(this);

		if(UNiagaraScriptParametersHierarchyViewModel* HierarchyViewModel = ScriptToolkit.Pin()->GetHierarchyViewModel())
		{
			HierarchyViewModel->GetOnHierarchyChangedDelegate().RemoveAll(this);
		}
	}
}

void SNiagaraScriptInputPreviewPanel::OnParametersChanged(TOptional<TInstancedStruct<FNiagaraParametersChangedData>> ParametersChangedData)
{
	Refresh();
}

FString SNiagaraScriptInputPreviewPanel::GetReferencerName() const
{
	return "NiagaraScriptInputParametersPanel";
}

void SNiagaraScriptInputPreviewPanel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(TransientLeftoverParameters);
}

void SNiagaraScriptInputPreviewPanel::PostUndo(bool bSuccess)
{
	Refresh();
}

void SNiagaraScriptInputPreviewPanel::PostRedo(bool bSuccess)
{
	Refresh();
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
