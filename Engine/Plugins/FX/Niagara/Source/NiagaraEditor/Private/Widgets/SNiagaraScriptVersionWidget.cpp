// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraScriptVersionWidget.h"

#include "AssetToolsModule.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Modules/ModuleManager.h"
#include "NiagaraActions.h"
#include "NiagaraVersionMetaData.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "NiagaraScriptVersionWidget"

void SNiagaraScriptVersionWidget::Construct(const FArguments& InArgs, UNiagaraScript* InScript,	UNiagaraVersionMetaData* InMetadata, const FString& InBasePackageName)
{
	Script = InScript;
	BasePackageName = InBasePackageName;
	SNiagaraVersionWidget::Construct(InArgs, InScript, InMetadata);
}

FText SNiagaraScriptVersionWidget::GetInfoHeaderText() const
{
	if (Script->IsVersioningEnabled())
	{
		return LOCTEXT("NiagaraManageVersionInfoHeader", "Here you can see and manage available script versions. Adding versions allows you to make changes without breaking existing assets.\nThe exposed version is the one that users will see when adding the module, function or dynamic input to the stack. If the exposed version has a higher major version than the one used in an asset, the user will see a prompt to upgrade to the new version.");	
	}
	return LOCTEXT("NiagaraManageVersionInfoHeaderDisabled", "Versioning is not yet enabled for this script. Enabling versioning allows you to make changes to the module without breaking exising usages. But it also forces users to manually upgrade to new versions, as changes are no longer pushed out automatically.\n\nDo you want to enable versioning for this script? This will convert the current script to version 1.0 and automatically exposes it.");
}

void SNiagaraScriptVersionWidget::ExecuteSaveAsAssetAction(FNiagaraAssetVersion AssetVersion)
{
	FString Name;
	FString PackageName;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, TEXT(""), PackageName, Name);
	
	UNiagaraScript* NewAssetScript = Cast<UNiagaraScript>(AssetToolsModule.Get().DuplicateAssetWithDialogAndTitle(Name, FPackageName::GetLongPackagePath(PackageName), Script, LOCTEXT("SaveVersionAsAssetTitle", "Create Script As")));
	
	if (NewAssetScript != nullptr)
	{
		NewAssetScript->DisableVersioning(AssetVersion.VersionGuid);
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAssetScript);
	}
}

#undef LOCTEXT_NAMESPACE
