// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "NiagaraActions.h"
#include "SNiagaraVersionWidget.h"

class SNiagaraScriptVersionWidget : public SNiagaraVersionWidget
{
public:
	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, UNiagaraScript* InScript, UNiagaraVersionMetaData* InMetadata, const FString& BasePackageName);

protected:
	NIAGARAEDITOR_API virtual FText GetInfoHeaderText() const override;
	NIAGARAEDITOR_API virtual void ExecuteSaveAsAssetAction(FNiagaraAssetVersion AssetVersion) override;

private:
	UNiagaraScript* Script = nullptr;
	FString BasePackageName;
};
