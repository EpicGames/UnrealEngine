// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraSelectedObjectsDetails.h"
#include "NiagaraObjectSelection.h"

#include "Customizations/NiagaraTypeCustomizations.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraScriptVariable.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "NiagaraSelectedObjectsDetails"


void SNiagaraSelectedObjectsDetails::Construct(const FArguments& InArgs, TSharedRef<FNiagaraObjectSelection> InSelectedObjects)
{
	bAllowEditingLibraryOwnedScriptVars = InArgs._AllowEditingLibraryScriptVariables;
	bViewingLibrarySubscribedScriptVar = false;
	LastSetSelectedObjectsArrayIdx = 0;
	SelectedObjectsArray.Push(InSelectedObjects);
	SelectedObjectsArray[0]->OnSelectedObjectsChanged().AddSP(this, &SNiagaraSelectedObjectsDetails::SelectedObjectsChangedFirst);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObjects(SelectedObjectsArray[0]->GetSelectedObjectsResolved().Array());
	DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SNiagaraSelectedObjectsDetails::PropertyIsReadOnly));
	DetailsView->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SNiagaraSelectedObjectsDetails::DetailsPanelIsEnabled)));
	//@todo(ng) re-enable once this is implemented.
	//DetailsView->SetIsCustomRowReadOnlyDelegate(FIsCustomRowReadOnly::CreateSP(this, &SNiagaraSelectedObjectsDetails::CustomRowIsReadOnly)); 
	DetailsView->OnFinishedChangingProperties().AddRaw(this, &SNiagaraSelectedObjectsDetails::OnDetailsPanelFinishedChangingProperties);

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];
} 

void SNiagaraSelectedObjectsDetails::Construct(const FArguments& InArgs, TSharedRef<FNiagaraObjectSelection> InSelectedObjects, TSharedRef<FNiagaraObjectSelection> InSelectedObjects2)
{
	bAllowEditingLibraryOwnedScriptVars = InArgs._AllowEditingLibraryScriptVariables;
	bViewingLibrarySubscribedScriptVar = false;
	LastSetSelectedObjectsArrayIdx = 0;
	SelectedObjectsArray.Push(InSelectedObjects);
	SelectedObjectsArray.Push(InSelectedObjects2);
	SelectedObjectsArray[0]->OnSelectedObjectsChanged().AddSP(this, &SNiagaraSelectedObjectsDetails::SelectedObjectsChangedFirst);
	SelectedObjectsArray[1]->OnSelectedObjectsChanged().AddSP(this, &SNiagaraSelectedObjectsDetails::SelectedObjectsChangedSecond);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObjects(SelectedObjectsArray[0]->GetSelectedObjectsResolved().Array());
	DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SNiagaraSelectedObjectsDetails::PropertyIsReadOnly));
	DetailsView->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SNiagaraSelectedObjectsDetails::DetailsPanelIsEnabled)));
	//@todo(ng) re-enable once this is implemented.
	//DetailsView->SetIsCustomRowReadOnlyDelegate(FIsCustomRowReadOnly::CreateSP(this, &SNiagaraSelectedObjectsDetails::CustomRowIsReadOnly)); 
	DetailsView->OnFinishedChangingProperties().AddRaw(this, &SNiagaraSelectedObjectsDetails::OnDetailsPanelFinishedChangingProperties);

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];
}

void SNiagaraSelectedObjectsDetails::PostUndo(bool bSuccess)
{
	RefreshDetails();
}

void SNiagaraSelectedObjectsDetails::SelectedObjectsChanged()
{
	// Do not update selected object info flags if editing library owned script vars is enabled as these flags are not used.
	if (bAllowEditingLibraryOwnedScriptVars == false)
	{
		UpdateSelectedObjectInfoFlags(SelectedObjectsArray[LastSetSelectedObjectsArrayIdx]);
	}
	DetailsView->SetObjects(SelectedObjectsArray[LastSetSelectedObjectsArrayIdx]->GetSelectedObjectsResolved().Array());
}

void SNiagaraSelectedObjectsDetails::SelectedObjectsChangedFirst()
{
	// Do not update selected object info flags if editing library owned script vars is enabled as these flags are not used.
	if (bAllowEditingLibraryOwnedScriptVars == false)
	{
		UpdateSelectedObjectInfoFlags(SelectedObjectsArray[0]);
	}
	DetailsView->SetObjects(SelectedObjectsArray[0]->GetSelectedObjectsResolved().Array());
	LastSetSelectedObjectsArrayIdx = 0;
}

void SNiagaraSelectedObjectsDetails::RefreshDetails()
{
	DetailsView->ForceRefresh();
}

// TODO: Instead have a delegate that takes an array argument? This seems a bit dodgy..
void SNiagaraSelectedObjectsDetails::SelectedObjectsChangedSecond()
{
	// Do not update selected object info flags if editing library owned script vars is enabled as these flags are not used.
	if (bAllowEditingLibraryOwnedScriptVars == false)
	{
		UpdateSelectedObjectInfoFlags(SelectedObjectsArray[1]);
	}
	DetailsView->SetObjects(SelectedObjectsArray[1]->GetSelectedObjectsResolved().Array());
	LastSetSelectedObjectsArrayIdx = 1;
}

void SNiagaraSelectedObjectsDetails::OnDetailsPanelFinishedChangingProperties(const FPropertyChangedEvent& InEvent)
{
	if (OnFinishedChangingPropertiesDelegate.IsBound())
	{
		OnFinishedChangingPropertiesDelegate.Broadcast(InEvent);
	}
}

bool SNiagaraSelectedObjectsDetails::DetailsPanelIsEnabled() const
{
	const UNiagaraScriptVariable* ScriptVar = GetSelectedScriptVar();
	if (ScriptVar == nullptr)
	{
		return true;
	}
	else if (ScriptVar->GetOuter()->IsA<UNiagaraParameterDefinitions>())
	{
		if (bAllowEditingLibraryOwnedScriptVars)
		{
			return true;
		}
		return false;
	}

	return true;
}

bool SNiagaraSelectedObjectsDetails::PropertyIsReadOnly(const FPropertyAndParent& PropertyAndParent) const
{
	if (bAllowEditingLibraryOwnedScriptVars)
	{
		// When editing parameter libraries, all properties are editable.
		return false;
	}
	else if (bViewingLibraryOwnedScriptVar)
	{
		// Do not allow editing library owned script vars if bAllowEditingLibraryOwnedScriptVars is not set.
		return true;
	}
	else if (bViewingLibrarySubscribedScriptVar == false)
	{
		// If we are not viewing a library script var, allow editing all properties.
		return false;
	}
	
	const FProperty& Property = PropertyAndParent.Property;
	const FName& PropertyName = Property.GetFName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultMode))
	{
		// Always allow editing the default value mode as this is necessary to override synchronized library default value.
		return false;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, Metadata))
	{
		// Always allow editing the metadata at this level as the FNiagaraVariableMetaDataCustomization decides which struct properties are editable.
		return false;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraScriptVariable, DefaultBinding))
	{
		if (const UNiagaraScriptVariable* ScriptVar = GetSelectedScriptVar())
		{
			if (ScriptVar->GetIsOverridingParameterDefinitionsDefaultValue())
			{
				// Allow editing the default binding if the script variable is set to override its default value.
				return false;
			}
		}
	}

	// Otherwise the property is read only.
	return true;
}

bool SNiagaraSelectedObjectsDetails::CustomRowIsReadOnly(const FName InRowName, const FName InParentName) const
{
	if (bAllowEditingLibraryOwnedScriptVars)
	{
		// When editing parameter libraries, all properties are editable.
		return false;
	}
	else if (bViewingLibraryOwnedScriptVar)
	{
		// Do not allow editing library owned script vars if bAllowEditingLibraryOwnedScriptVars is not set.
		return true;
	}
	else if (bViewingLibrarySubscribedScriptVar == false)
	{
		// If we are not viewing a library synchronizing script var, allow editing all properties.
		return false;
	}

	if (InRowName == FNiagaraEditorStrings::DefaultModeCustomRowName)
	{
		// Always allow editing the default value mode as this is necessary to override synchronized library default value.
		return false;
	}
	else if (InRowName == FNiagaraEditorStrings::DefaultValueCustomRowName)
	{
		if (const UNiagaraScriptVariable* ScriptVar = GetSelectedScriptVar())
		{
			if (ScriptVar->GetIsOverridingParameterDefinitionsDefaultValue())
			{
				// Allow editing the default binding if the script variable is set to override its default value.
				return false;
			}
		}
		return true;
	}

	checkf(false, TEXT("Encountered unknown custom row name when setting custom row read only! Update this method!"));
	return true;
}

void SNiagaraSelectedObjectsDetails::UpdateSelectedObjectInfoFlags(const TSharedPtr<FNiagaraObjectSelection>& SelectedObjects)
{
	for (const UObject* Obj : SelectedObjects->GetSelectedObjectsResolved())
	{
		if (const UNiagaraScriptVariable* ScriptVar = Cast<const UNiagaraScriptVariable>(Obj))
		{
			bViewingLibrarySubscribedScriptVar = ScriptVar->GetIsSubscribedToParameterDefinitions();
			bViewingLibraryOwnedScriptVar = ScriptVar->GetOuter()->IsA<UNiagaraParameterDefinitions>();
		}
		else
		{
			bViewingLibrarySubscribedScriptVar = false;
			bViewingLibraryOwnedScriptVar = false;
		}
	}
}

const UNiagaraScriptVariable* SNiagaraSelectedObjectsDetails::GetSelectedScriptVar() const
{
	//@todo(ng) avoid this check during level travel! 
	for(const TSharedPtr<FNiagaraObjectSelection>& ObjectSelection : SelectedObjectsArray)
	{
		for (const UObject* Obj : ObjectSelection->GetSelectedObjectsResolved())
		{
			if (const UNiagaraScriptVariable* ScriptVar = Cast<const UNiagaraScriptVariable>(Obj))
			{
				return ScriptVar;
			}
		}
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE // "NiagaraSelectedObjectsDetails"
