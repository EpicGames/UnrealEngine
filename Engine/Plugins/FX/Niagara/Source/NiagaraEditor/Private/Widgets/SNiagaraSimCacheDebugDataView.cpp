// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSimCacheDebugDataView.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/SNiagaraParameterName.h"
#include "NiagaraSimCacheDebugData.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "NiagaraSimCacheDebugDataView"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Niagara::SimCache::DebugDataUI
{
static FName NAME_ParameterName("ParameterName");
static FName NAME_ParameterValue("ParameterValue");

class SParameterStoreItemWidget : public SMultiColumnTableRow<TSharedPtr<FNiagaraVariableBase>>
{
public:
	SLATE_BEGIN_ARGS(SParameterStoreItemWidget){}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraParameterStore>,	ParameterStore)
		SLATE_ARGUMENT(TSharedPtr<FNiagaraVariableBase>,	ParameterVariable)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		ParameterStore = InArgs._ParameterStore;
		ParameterVariable = InArgs._ParameterVariable;
	
		SMultiColumnTableRow<TSharedPtr<FNiagaraVariableBase>>::Construct( FSuperRowType::FArguments(), InOwnerTable);
	}

	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName)
	{
		TSharedPtr<SWidget> ContentWidget;

		if (ParameterStore && ParameterVariable)
		{
			if ( ColumnName == NAME_ParameterName )
			{
				ContentWidget =
					SNew(SNiagaraParameterName)
					.ParameterName(ParameterVariable->GetName())
					.IsReadOnly(true);
			}
			else if ( ColumnName == NAME_ParameterValue)
			{
				if (const uint8* ParameterData = ParameterStore->GetParameterData(*ParameterVariable))
				{
					const FString ValueString = ParameterVariable->GetType().ToString(ParameterData);
					ContentWidget =
						SNew(STextBlock)
						.Text(FText::FromString(ValueString));
				}
			}
		}

		if (ContentWidget == nullptr)
		{
			ContentWidget =
				SNew(STextBlock)
				.Text(LOCTEXT("UnknownColumn", "Unknown Column"));
		}

		return
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 5.0f, 0.0f)
			[
				ContentWidget.ToSharedRef()
			];
	}

	TSharedPtr<FNiagaraParameterStore>	ParameterStore;
	TSharedPtr<FNiagaraVariableBase>	ParameterVariable;
};

class SParameterStoreListView : public SListView<TSharedPtr<FNiagaraVariableBase>>
{
	SLATE_BEGIN_ARGS(SParameterStoreListView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SListView::Construct(SListView::FArguments()
			//.Clipping(EWidgetClipping::OnDemand)
			.ListItemsSource(&ParameterVariables)
			.OnGenerateRow(this, &SParameterStoreListView::OnGenerateRowForEntry)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(NAME_ParameterName)
				.DefaultLabel(LOCTEXT("ParameterName", "Parameter Name"))
				.ManualWidth(200)
				+ SHeaderRow::Column(NAME_ParameterValue)
				.DefaultLabel(LOCTEXT("ParameterValue", "Parameter Value"))
				.ManualWidth(200)
			)
		);
	}

	void SetParameterStore(const FNiagaraParameterStore& InParameterStore)
	{
		ParameterVariables.Empty();
		ParameterStore = MakeShared<FNiagaraParameterStore>(InParameterStore);

		for (const FNiagaraVariableBase& Variable : ParameterStore->ReadParameterVariables())
		{
			if (Variable.IsDataInterface() || Variable.IsUObject())
			{
				continue;
			}
			ParameterVariables.Emplace(MakeShared<FNiagaraVariableBase>(Variable));
		}
		RequestListRefresh();
	}

	TSharedRef<ITableRow> OnGenerateRowForEntry(TSharedPtr<FNiagaraVariableBase> ParameterVariable, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return
			SNew(SParameterStoreItemWidget, OwnerTable)
			.ParameterVariable(ParameterVariable)
			.ParameterStore(ParameterStore);
	}

	TSharedPtr<FNiagaraParameterStore>			ParameterStore;
	TArray<TSharedPtr<FNiagaraVariableBase>>	ParameterVariables;
};

} //namespace UE::Niagara::SimCache::DebugDataUI

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void SNiagaraSimCacheDebugDataView::Construct(const FArguments& InArgs)
{
	SimCacheViewModel = InArgs._SimCacheViewModel;

	// Attach handlers for the cache data changing
	SimCacheViewModel->OnViewDataChanged().AddSP(this, &SNiagaraSimCacheDebugDataView::RefreshContents);
	SimCacheViewModel->OnSimCacheChanged().AddSP(this, &SNiagaraSimCacheDebugDataView::RefreshContents);
	SimCacheViewModel->OnBufferChanged().AddSP(this, &SNiagaraSimCacheDebugDataView::RefreshContents);

	if (const FNiagaraSimCacheDebugDataFrame* FrameData = GetCurrentFrameData())
	{
		if (FrameData->DebugParameterStores.Num() > 0)
		{
			SelectedParameterStoreName = FrameData->DebugParameterStores.CreateConstIterator().Key();
		}
	}

	// Create our details view
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowObjectLabel = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObject(const_cast<UNiagaraSimCacheDebugData*>(SimCacheViewModel->GetCacheDebugData()));
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ParameterStoreSelection", "Parameter Store Selection:"))
					.Margin(FMargin(0.0, 0.0, 5.0, 0.0))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SComboButton)
					.OnGetMenuContent(this, &SNiagaraSimCacheDebugDataView::GetParameterStoreSelectionMenu)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(
							TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([this]() { return FText::FromString(SelectedParameterStoreName); }))
						)
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(OverrideParametersWidget, UE::Niagara::SimCache::DebugDataUI::SParameterStoreListView)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 6.0f, 0.0, 0.0f)
		[
			DetailsView.ToSharedRef()
		]
	];

	RefreshContents();
};

const FNiagaraSimCacheDebugDataFrame* SNiagaraSimCacheDebugDataView::GetCurrentFrameData() const
{
	const UNiagaraSimCacheDebugData* DebugData = SimCacheViewModel ? SimCacheViewModel->GetCacheDebugData() : nullptr;
	if (DebugData)
	{
		const int32 FrameIndex = SimCacheViewModel->GetFrameIndex();
		if (DebugData->Frames.IsValidIndex(FrameIndex))
		{
			return &DebugData->Frames[FrameIndex];
		}
	}
	return nullptr;
}

TSharedRef<SWidget> SNiagaraSimCacheDebugDataView::GetParameterStoreSelectionMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	if (const FNiagaraSimCacheDebugDataFrame* FrameData = GetCurrentFrameData())
	{
		for ( auto It=FrameData->DebugParameterStores.CreateConstIterator(); It; ++It)
		{
			MenuBuilder.AddMenuEntry(
				FText::FromString(It->Key),
				FText(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda(
						[this, SelectedValue=It->Key]()
						{
							SelectedParameterStoreName = SelectedValue;
							RefreshContents();
						}
					)
				)
			);
		}
	}

	return MenuBuilder.MakeWidget();
}

void SNiagaraSimCacheDebugDataView::RefreshContents()
{
	const FNiagaraParameterStore* ParameterStore = nullptr;
	if (const FNiagaraSimCacheDebugDataFrame* FrameData = GetCurrentFrameData())
	{
		ParameterStore = FrameData->DebugParameterStores.Find(SelectedParameterStoreName);
	}

	OverrideParametersWidget->SetParameterStore(ParameterStore ? *ParameterStore : FNiagaraParameterStore());

	DetailsView->SetObject(const_cast<UNiagaraSimCacheDebugData*>(SimCacheViewModel->GetCacheDebugData()));
}

void SNiagaraSimCacheDebugDataView::RefreshContents(bool)
{
	RefreshContents();
}

#undef LOCTEXT_NAMESPACE
