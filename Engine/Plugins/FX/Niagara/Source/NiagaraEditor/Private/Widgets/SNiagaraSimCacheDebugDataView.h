// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class IDetailsView;

class FNiagaraSimCacheViewModel;
struct FNiagaraSimCacheDebugDataFrame;
namespace UE::Niagara::SimCache::DebugDataUI
{
	class SParameterStoreListView;
};

class SNiagaraSimCacheDebugDataView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheDebugDataView) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheViewModel>, SimCacheViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	const FNiagaraSimCacheDebugDataFrame* GetCurrentFrameData() const;
	TSharedRef<SWidget> GetParameterStoreSelectionMenu();

	void RefreshContents();
	void RefreshContents(bool);

private:
	TSharedPtr<FNiagaraSimCacheViewModel>	SimCacheViewModel;
	TSharedPtr<IDetailsView>				DetailsView;

	TSharedPtr<UE::Niagara::SimCache::DebugDataUI::SParameterStoreListView> OverrideParametersWidget;
	FString	SelectedParameterStoreName;

};
