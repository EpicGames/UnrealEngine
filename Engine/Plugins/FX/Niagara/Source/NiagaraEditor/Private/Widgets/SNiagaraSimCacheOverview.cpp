// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSimCacheOverview.h"

#include "SNiagaraSimCacheTreeView.h"

#define LOCTEXT_NAMESPACE "NiagaraSimCacheOverview"

class SNiagaraSimCacheBufferItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheBufferItem) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheOverviewItem>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Item = InArgs._Item;

		RefreshContent();
	}

	void RefreshContent()
	{
		ChildSlot
		.Padding(2.0f)
		[
			Item->GetRowWidget()
		];
	}

	TSharedPtr<FNiagaraSimCacheOverviewItem> Item;
};


void SNiagaraSimCacheOverview::OnSimCacheChanged()
{
	if(BufferListView.IsValid())
	{
		BufferListView->RebuildList();
	}
	RebuildMainWidget();
}

void SNiagaraSimCacheOverview::OnViewDataChanged(bool)
{
	RebuildMainWidget();
}

void SNiagaraSimCacheOverview::Construct(const FArguments& InArgs)
{
	ViewModel = InArgs._SimCacheViewModel;

	SAssignNew(TreeViewWidget, SNiagaraSimCacheTreeView)
	.SimCacheViewModel(ViewModel);

	SAssignNew(BufferListView, SListView<TSharedRef<FNiagaraSimCacheOverviewItem>>)
	.ListItemsSource(ViewModel->GetBufferEntries())
	.OnGenerateRow(this, &SNiagaraSimCacheOverview::OnGenerateRowForItem)
	.OnSelectionChanged(this, &SNiagaraSimCacheOverview::OnListSelectionChanged)
	.SelectionMode(ESelectionMode::Single);

	ViewModel.Get()->OnSimCacheChanged().AddSP(this, &SNiagaraSimCacheOverview::OnSimCacheChanged);
	ViewModel.Get()->OnViewDataChanged().AddSP(this, &SNiagaraSimCacheOverview::OnViewDataChanged);

	ChildSlot
	[
		SAssignNew(MainWidget, SSplitter)
		.Orientation(Orient_Vertical)
	];

	RebuildMainWidget();
}

void SNiagaraSimCacheOverview::RebuildMainWidget()
{
	// Skip rebuild if we both things active
	const bool bNeedsComponentFilter = ViewModel->IsComponentFilterActive() && ViewModel->IsCacheValid();
	const int32 CurrentNumChildren = MainWidget->GetChildren()->Num();
	const int32 RequiredNumChildren = bNeedsComponentFilter ? 2 : 1;
	if (CurrentNumChildren == RequiredNumChildren)
	{
		return;
	}

	//-OPT: We can avoid a total rebuild here
	while (MainWidget->GetChildren()->Num() > 0)
	{
		MainWidget->RemoveAt(0);
	}

	const float MinSplitterSlotSize = 30.0f;

	MainWidget->AddSlot()
		.Value(0.2f)
		.MinSize(MinSplitterSlotSize)
		[

			// Cache Buffers
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+SSplitter::Slot()
			.Resizable(false)
			.SizeRule(SSplitter::SizeToContent)
			[
				// Header
				SNew(SBorder)
				.BorderImage(FAppStyle::GetNoBrush())
				.Padding(5.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CacheBufferSelection", "Cache Buffer Selection"))
				]
			]
			+SSplitter::Slot()
			.Resizable(false)
			[
				// List View
				BufferListView.ToSharedRef()
			]
		];

	if (bNeedsComponentFilter)
	{
		MainWidget->AddSlot()
			.Value(0.8f)
			.MinSize(MinSplitterSlotSize)
			[
				// Component Details
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				+SSplitter::Slot()
				.Resizable(false)
				.SizeRule(SSplitter::SizeToContent)
				[
					// Header
					SNew(SBorder)
					.BorderImage(FAppStyle::GetNoBrush())
					.Padding(5.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ComponentTreeView", "Component Details"))
					]
				]
				+SSplitter::Slot()
				.Resizable(false)
				[
					// Tree View
					TreeViewWidget.ToSharedRef()
				]
			];
	}
}

TSharedRef<ITableRow> SNiagaraSimCacheOverview::OnGenerateRowForItem(TSharedRef<FNiagaraSimCacheOverviewItem> Item, const TSharedRef<STableViewBase>& Owner)
{
	return SNew(STableRow<TSharedRef<FNiagaraSimCacheOverviewItem>>, Owner)
	.Style(FNiagaraEditorStyle::Get(), "NiagaraEditor.SimCache.ComponentItem")
	.Padding(1.0f)
	[
		SNew(SNiagaraSimCacheBufferItem)
		.Item(Item)
	];
}

void SNiagaraSimCacheOverview::OnListSelectionChanged(TSharedPtr<FNiagaraSimCacheOverviewItem> Item, ESelectInfo::Type)
{
	if (Item.IsValid())
	{
		switch (Item->GetType())
		{
			case ENiagaraSimCacheOverviewItemType::System:
				ViewModel->SetSelectedSystemInstance();
				break;

			case ENiagaraSimCacheOverviewItemType::Emitter:
				ViewModel->SetSelectedEmitter(Item->GetEmitterName());
				break;

			case ENiagaraSimCacheOverviewItemType::DataInterface:
				ViewModel->SetSelectedDataInterface(Item->GetDataInterface());
				break;

			case ENiagaraSimCacheOverviewItemType::DebugData:
				ViewModel->SetSelectedDebugData();
				break;
		}
	}
}

#undef LOCTEXT_NAMESPACE