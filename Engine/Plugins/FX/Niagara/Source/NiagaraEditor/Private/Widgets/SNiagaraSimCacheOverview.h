// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Templates/SharedPointer.h"

class FNiagaraSimCacheViewModel;
class SNiagaraSimCacheTreeView;
class SSplitter;

enum class ENiagaraSimCacheOverviewItemType
{
	System,
	Emitter,
	Component,
	DataInterface,
	DebugData,
	MAX,
};

struct FNiagaraSimCacheOverviewItem : public TSharedFromThis<FNiagaraSimCacheOverviewItem>
{
	FNiagaraSimCacheOverviewItem() = default;
	virtual ~FNiagaraSimCacheOverviewItem() = default;

	FText GetDisplayNameText() { return DisplayName; }
	void SetDisplayName(const FText& NewName) { DisplayName = NewName; }

	virtual FName GetEmitterName() { return EmitterName; }
	void SetEmitterName(FName InEmitterName) { EmitterName = InEmitterName; }

	virtual FNiagaraVariableBase GetDataInterface() { return FNiagaraVariableBase(); }

	virtual ENiagaraSimCacheOverviewItemType GetType () { checkNoEntry(); return ENiagaraSimCacheOverviewItemType::MAX; }

	virtual TSharedRef<SWidget> GetRowWidget() { return SNew(STextBlock).Text(GetDisplayNameText()); }

	FName EmitterName;
	FText DisplayName;
};

struct FNiagaraSimCacheOverviewSystemItem : FNiagaraSimCacheOverviewItem
{
	virtual ENiagaraSimCacheOverviewItemType GetType () override { return ENiagaraSimCacheOverviewItemType::System; }
};

struct FNiagaraSimCacheOverviewEmitterItem : FNiagaraSimCacheOverviewItem
{
	virtual ENiagaraSimCacheOverviewItemType GetType() override {return ENiagaraSimCacheOverviewItemType::Emitter; }
};

struct FNiagaraSimCacheOverviewDataInterfaceItem : FNiagaraSimCacheOverviewItem
{
	virtual ENiagaraSimCacheOverviewItemType GetType() override {return ENiagaraSimCacheOverviewItemType::DataInterface; }
	virtual FNiagaraVariableBase GetDataInterface() override { return DataInterfaceReference; }

	FNiagaraVariableBase DataInterfaceReference;
};

struct FNiagaraSimCacheOverviewDebugDataItem : FNiagaraSimCacheOverviewItem
{
	virtual ENiagaraSimCacheOverviewItemType GetType() override { return ENiagaraSimCacheOverviewItemType::DebugData; }
};

class SNiagaraSimCacheOverview : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheOverview) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheViewModel>, SimCacheViewModel)
	SLATE_END_ARGS()

	void OnSimCacheChanged();
	void OnViewDataChanged(bool);
	void Construct(const FArguments& InArgs);

	void RebuildMainWidget();
	TSharedRef<ITableRow> OnGenerateRowForItem(TSharedRef<FNiagaraSimCacheOverviewItem> Item, const TSharedRef<STableViewBase>& Owner);
	void OnListSelectionChanged(TSharedPtr<FNiagaraSimCacheOverviewItem> Item, ESelectInfo::Type SelectInfo);

	TSharedPtr<SSplitter> MainWidget;

	TSharedPtr<SListView<TSharedRef<FNiagaraSimCacheOverviewItem>>> BufferListView;

	TSharedPtr<SNiagaraSimCacheTreeView> TreeViewWidget;

	TSharedPtr<FNiagaraSimCacheViewModel> ViewModel;
};
