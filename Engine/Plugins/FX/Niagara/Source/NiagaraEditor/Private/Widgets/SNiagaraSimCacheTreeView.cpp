// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSimCacheTreeView.h"

#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "NiagaraSimCacheTreeView"

class SNiagaraSimCacheTreeItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheTreeItem) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheTreeItem>, Item)
		SLATE_ARGUMENT(TWeakPtr<SNiagaraSimCacheTreeView>, Owner)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Item = InArgs._Item;
		Owner = InArgs._Owner;
	
		RefreshContent();
	}

	void RefreshContent() 
	{
		ChildSlot
		.Padding(2.0f)
		[
			Item->GetRowWidget()
		];
	}

	TSharedPtr<FNiagaraSimCacheTreeItem> Item;
	TWeakPtr<SNiagaraSimCacheTreeView> Owner;
};

//// Filter Widget /////


void SNiagaraSimCacheTreeViewFilterWidget::Construct(const FArguments& InArgs, TWeakPtr<FNiagaraSimCacheTreeItem> InTreeItem,
	TWeakPtr<SNiagaraSimCacheTreeView> InTreeView)
{
	WeakTreeItem = InTreeItem;
	WeakTreeView = InTreeView;

	ChildSlot
	.HAlign(HAlign_Center)
	.Padding(3.0f)
	[
		// Filter controls
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f)
		[
			// Clear All
			SNew(SButton)
			.Text(LOCTEXT("ClearAll", "Clear All"))
			.OnClicked(this, &SNiagaraSimCacheTreeViewFilterWidget::OnClearAllReleased)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f)
		[
			// Select All
			SNew(SButton)
			.Text(LOCTEXT("SelectAll", "Select All"))
			.OnClicked(this, &SNiagaraSimCacheTreeViewFilterWidget::OnSelectAllReleased)
		]
	];
}

FReply SNiagaraSimCacheTreeViewFilterWidget::OnClearAllReleased()
{
	TSharedPtr<SNiagaraSimCacheTreeView> TreeView = WeakTreeView.Pin();

	if(TreeView.IsValid())
	{
		TreeView->ClearFilterSelection();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SNiagaraSimCacheTreeViewFilterWidget::OnSelectAllReleased()
{
	TSharedPtr<SNiagaraSimCacheTreeView> TreeView = WeakTreeView.Pin();

	if(TreeView.IsValid())
	{
		TreeView->SelectAll();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

//// Visibility Widget //////

void SSimCacheTreeViewVisibilityWidget::Construct(const FArguments& InArgs, TWeakPtr<FNiagaraSimCacheTreeItem> InTreeItem, TWeakPtr<SNiagaraSimCacheTreeView> InWeakTreeView)
{
	WeakTreeItem = InTreeItem;
	WeakTreeView = InWeakTreeView;

	ChildSlot
	.HAlign(HAlign_Center)
	.Padding(1.0f)
	[
		SNew(SCheckBox)
		.OnCheckStateChanged(this, &SSimCacheTreeViewVisibilityWidget::OnCheckStateChanged)
		.IsChecked(this, &SSimCacheTreeViewVisibilityWidget::GetCheckedState)
	];
	
}
	
void SSimCacheTreeViewVisibilityWidget::OnCheckStateChanged(ECheckBoxState NewState)
{
	TSharedPtr<SNiagaraSimCacheTreeView> TreeView = WeakTreeView.Pin();
	TSharedPtr<FNiagaraSimCacheTreeItem> TreeItem = WeakTreeItem.Pin();
	
	if(TreeView.IsValid() && TreeItem.IsValid())
	{
		TreeView->VisibilityButtonClicked(TreeItem.ToSharedRef());
	}
}

ECheckBoxState SSimCacheTreeViewVisibilityWidget::GetCheckedState() const
{
	TSharedPtr<SNiagaraSimCacheTreeView> TreeView = WeakTreeView.Pin();
	TSharedPtr<FNiagaraSimCacheTreeItem> TreeItem = WeakTreeItem.Pin();

	if (TreeView.IsValid() && TreeItem.IsValid())
	{
		return TreeView->GetFilterCheckedState(TreeItem);
	}

	return ECheckBoxState::Unchecked;
}

///// Tree View Widget


void SNiagaraSimCacheTreeView::SetupRootEntries()
{
	TArray<TSharedRef<FNiagaraSimCacheTreeItem>>* RootEntries = ViewModel->GetSelectedRootEntries();

	if(RootEntries && !RootEntries->IsEmpty())
	{
		TreeView->SetItemExpansion((*RootEntries)[0], true);
	}
}

void SNiagaraSimCacheTreeView::Construct(const FArguments& InArgs)
{
	ViewModel = InArgs._SimCacheViewModel;

	ViewModel->OnBufferChanged().AddSP(this, &SNiagaraSimCacheTreeView::OnBufferChanged);
	ViewModel->OnSimCacheChanged().AddSP(this, &SNiagaraSimCacheTreeView::OnSimCacheChanged);

	ViewModel->BuildEntries(SharedThis(this));
	
	TreeView = SNew(STreeView<TSharedRef<FNiagaraSimCacheTreeItem>>)
	.SelectionMode(ESelectionMode::Single)
	.TreeItemsSource(ViewModel->GetSelectedRootEntries())
	.OnGenerateRow(this, &SNiagaraSimCacheTreeView::OnGenerateRow)
	.OnGeneratePinnedRow(this, &SNiagaraSimCacheTreeView::OnGenerateRow)
	.ShouldStackHierarchyHeaders(true)
	.OnGetChildren(this, &SNiagaraSimCacheTreeView::OnGetChildren);

	SetupRootEntries();

	ChildSlot
	[
		TreeView.ToSharedRef()
	];
}


TSharedRef<ITableRow> SNiagaraSimCacheTreeView::OnGenerateRow(TSharedRef<FNiagaraSimCacheTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	static const char* ItemStyles[] =
	{
		"NiagaraEditor.SimCache.SystemItem",
		"NiagaraEditor.SimCache.EmitterItem",
		"NiagaraEditor.SimCache.ComponentItem",
		"NiagaraEditor.SimCache.DataInterfaceItem",
		"NiagaraEditor.SimCache.DebugData"
	};
	static_assert(UE_ARRAY_COUNT(ItemStyles) == int(ENiagaraSimCacheOverviewItemType::MAX), "Mismatch on style count");

	ENiagaraSimCacheOverviewItemType StyleType = Item->GetType();
	
	return SNew(STableRow<TSharedRef<FNiagaraSimCacheTreeItem>>, OwnerTable)
	.Style(FNiagaraEditorStyle::Get(), ItemStyles[static_cast<int32>(StyleType)])
	[
		SNew(SNiagaraSimCacheTreeItem)
		.Item(Item)
		.Owner(SharedThis(this))
	];
}

void SNiagaraSimCacheTreeView::OnGetChildren(TSharedRef<FNiagaraSimCacheTreeItem> InItem,
	TArray<TSharedRef<FNiagaraSimCacheTreeItem>>& OutChildren)
{
	OutChildren = InItem.Get().Children;
}

void SNiagaraSimCacheTreeView::OnBufferChanged()
{
	TreeView->RequestTreeRefresh();

	TArray<TSharedRef<FNiagaraSimCacheTreeItem>>* RootEntries = ViewModel->GetSelectedRootEntries();
	if (RootEntries->Num() > 0)
	{
		TreeView->SetItemExpansion((*RootEntries)[0], true);
	}
}

void SNiagaraSimCacheTreeView::OnSimCacheChanged()
{
	ViewModel->BuildEntries(SharedThis(this));
	SetupRootEntries();
}

ECheckBoxState SNiagaraSimCacheTreeView::GetFilterCheckedState(TSharedPtr<FNiagaraSimCacheTreeItem> InItem) const
{
	// Leaf node we get the state directly
	if (InItem->Children.Num() == 0)
	{
		return ViewModel->IsComponentFiltered(InItem->GetFilterName()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	// Recurse children to figure out the state
	TOptional<ECheckBoxState> IsChecked;
	for (const TSharedRef<FNiagaraSimCacheTreeItem>& ChildItem : InItem->Children)
	{
		const ECheckBoxState ChildCheckState = GetFilterCheckedState(ChildItem);
		if (!IsChecked.IsSet())
		{
			IsChecked = ChildCheckState;
		}
		else if (ChildCheckState != IsChecked.GetValue())
		{
			return ECheckBoxState::Undetermined;
		}
	}
	return IsChecked.Get(ECheckBoxState::Unchecked);
}

void SNiagaraSimCacheTreeView::VisibilityButtonClicked(TSharedRef<FNiagaraSimCacheTreeItem> InItem)
{
	// Leaf node, apply the toggle directly
	if (InItem->Children.Num() == 0)
	{
		ViewModel->ToggleComponentFiltered(InItem->GetFilterName());
		return;
	}

	// Toggle children
	for (const TSharedRef<FNiagaraSimCacheTreeItem>& ChildItem : InItem->Children)
	{
		VisibilityButtonClicked(ChildItem);
	}
}

bool SNiagaraSimCacheTreeView::IsItemSelected(TSharedRef<FNiagaraSimCacheTreeItem> InItem)
{
	return TreeView->GetSelectedItems().Contains(InItem);
}

void SNiagaraSimCacheTreeView::ClearFilterSelection()
{
	ViewModel->SetAllComponentFiltered(false);
}

void SNiagaraSimCacheTreeView::SelectAll()
{
	if(!ViewModel->IsCacheValid())
	{
		return;
	}

	ViewModel->SetAllComponentFiltered(true);
}

bool SNiagaraSimCacheTreeView::ShouldShowComponentView() const
{
	if (ViewModel)
	{
		switch (ViewModel->GetSelectionMode())
		{
			case FNiagaraSimCacheViewModel::ESelectionMode::SystemInstance:
			case FNiagaraSimCacheViewModel::ESelectionMode::Emitter:
				return true;
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
