// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/NiagaraSimCacheViewModel.h"

#include "NiagaraEditorStyle.h"
#include "Widgets/SNiagaraPinTypeSelector.h"
#include "EdGraphSchema_Niagara.h"
#include "SNiagaraSimCacheOverview.h"
#include "Widgets/Layout/SScaleBox.h"

class SNiagaraSimCacheTreeView;
class FNiagaraSimCacheViewModel;
struct FNiagaraSimCacheTreeItem;
struct FNiagaraSimCacheOverviewItem;

///// Widget for controlling filters on the tree view
class SNiagaraSimCacheTreeViewFilterWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheTreeViewFilterWidget) {}
	SLATE_END_ARGS()

	
	void Construct(const FArguments& InArgs, TWeakPtr<FNiagaraSimCacheTreeItem> InTreeItem, TWeakPtr<SNiagaraSimCacheTreeView> InTreeView);

protected:
	
	FReply OnClearAllReleased();
	FReply OnSelectAllReleased();

private:
	TWeakPtr<FNiagaraSimCacheTreeItem> WeakTreeItem;
	TWeakPtr<SNiagaraSimCacheTreeView> WeakTreeView;

};

///// Widget for managing visibility of a component in the tree view.
class SSimCacheTreeViewVisibilityWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSimCacheTreeViewVisibilityWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FNiagaraSimCacheTreeItem> InTreeItem, TWeakPtr<SNiagaraSimCacheTreeView> InTreeView);

protected:
	void OnCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState GetCheckedState() const;
	
private:
	TWeakPtr<FNiagaraSimCacheTreeItem> WeakTreeItem;
	TWeakPtr<SNiagaraSimCacheTreeView> WeakTreeView;
};

class SNiagaraSimCacheTreeView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheTreeView) {}
	SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheViewModel>, SimCacheViewModel)
SLATE_END_ARGS()

	void OnSimCacheChanged();
	void SetupRootEntries();
	void Construct(const FArguments& InArgs);
	ECheckBoxState GetFilterCheckedState(TSharedPtr<FNiagaraSimCacheTreeItem> InItem) const;
	void VisibilityButtonClicked(TSharedRef<FNiagaraSimCacheTreeItem> InItem);
	bool IsItemSelected(TSharedRef<FNiagaraSimCacheTreeItem> InItem);
	void ClearFilterSelection();
	void SelectAll();
	bool ShouldShowComponentView() const;

private:
	TSharedRef<ITableRow> OnGenerateRow(TSharedRef<FNiagaraSimCacheTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	void OnGetChildren(TSharedRef<FNiagaraSimCacheTreeItem> InItem, TArray<TSharedRef<FNiagaraSimCacheTreeItem>>& OutChildren);

	void OnBufferChanged();

	TSharedPtr<STreeView<TSharedRef<FNiagaraSimCacheTreeItem>>> TreeView;

	TArray<TSharedRef<FNiagaraSimCacheTreeItem>> ActiveRootEntries;

	TSharedPtr<FNiagaraSimCacheViewModel> ViewModel;
};

struct FNiagaraSimCacheTreeItem : FNiagaraSimCacheOverviewItem
{
	explicit FNiagaraSimCacheTreeItem(TWeakPtr<SNiagaraSimCacheTreeView> InOwner)
	{
		Owner = InOwner;
	}

	virtual ~FNiagaraSimCacheTreeItem() override
	{
		Children.Empty();
	}

	FString GetFilterName()
	{
		return FilterName;
	}

	void SetFilterName(FString NewName)
	{
		FilterName = NewName;
	}

	virtual FNiagaraSimCacheTreeItem* GetRootItem()
	{
		return this;
	}

	void AddChild(TSharedRef<FNiagaraSimCacheTreeItem> NewChild)
	{
		Children.AddUnique(NewChild);
	}

	virtual ENiagaraSimCacheOverviewItemType GetType () override { return ENiagaraSimCacheOverviewItemType::System; }

	virtual TSharedRef<SWidget> GetRowWidget() override
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(GetDisplayNameText())
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SNiagaraSimCacheTreeViewFilterWidget, SharedThis(this), Owner)
				.Visibility_Lambda([this]()
				{
					if (TSharedPtr<SNiagaraSimCacheTreeView> TreeView = Owner.Pin())
					{
						return TreeView->ShouldShowComponentView() ? EVisibility::Visible : EVisibility::Collapsed;
					}
					return EVisibility::Collapsed;
				})
			];
	}

	FText DisplayName;
	FString FilterName;
	TSharedPtr<SSimCacheTreeViewVisibilityWidget> VisibilityWidget;
	TArray<TSharedRef<FNiagaraSimCacheTreeItem>> Children;
	TWeakPtr<SNiagaraSimCacheTreeView> Owner;
	
};

struct FNiagaraSimCacheEmitterTreeItem : FNiagaraSimCacheTreeItem
{
	explicit FNiagaraSimCacheEmitterTreeItem(TWeakPtr<SNiagaraSimCacheTreeView> InOwner): FNiagaraSimCacheTreeItem(InOwner)
	{
	}

	virtual ~FNiagaraSimCacheEmitterTreeItem() override = default;
	
	virtual ENiagaraSimCacheOverviewItemType GetType () override { return ENiagaraSimCacheOverviewItemType::Emitter; }
};

struct FNiagaraSimCacheDataInterfaceTreeItem : FNiagaraSimCacheTreeItem
{
	explicit FNiagaraSimCacheDataInterfaceTreeItem(TWeakPtr<SNiagaraSimCacheTreeView> InOwner): FNiagaraSimCacheTreeItem(InOwner)
	{
	}

	virtual ~FNiagaraSimCacheDataInterfaceTreeItem() override = default;
	
	virtual ENiagaraSimCacheOverviewItemType GetType () override { return ENiagaraSimCacheOverviewItemType::DataInterface; }

	FNiagaraVariableBase DataInterfaceReference;
};

struct FNiagaraSimCacheDebugDataTreeItem : FNiagaraSimCacheTreeItem
{
	explicit FNiagaraSimCacheDebugDataTreeItem(TWeakPtr<SNiagaraSimCacheTreeView> InOwner) : FNiagaraSimCacheTreeItem(InOwner)
	{
	}

	virtual ~FNiagaraSimCacheDebugDataTreeItem() override = default;

	virtual ENiagaraSimCacheOverviewItemType GetType() override { return ENiagaraSimCacheOverviewItemType::DebugData; }
};

struct FNiagaraSimCacheComponentTreeItem : FNiagaraSimCacheTreeItem
{
	explicit FNiagaraSimCacheComponentTreeItem(TWeakPtr<SNiagaraSimCacheTreeView> InOwner) : FNiagaraSimCacheTreeItem(InOwner)
	{
	}

	virtual ~FNiagaraSimCacheComponentTreeItem() override = default;

	virtual ENiagaraSimCacheOverviewItemType GetType () override { return ENiagaraSimCacheOverviewItemType::Component; }

	virtual TSharedRef<SWidget> GetRowWidget() override
	{
		TSharedRef<SHorizontalBox> Contents = SNew(SHorizontalBox);

		{
			FSlateBrush const* IconBrush = TypeDef.IsSet() && TypeDef.GetValue().IsStatic() ? FNiagaraEditorStyle::Get().GetBrush(TEXT("NiagaraEditor.StaticIcon")) : FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
			const FLinearColor TypeColor = TypeDef.IsSet() ? UEdGraphSchema_Niagara::GetTypeColor(TypeDef.GetValue()) : FLinearColor::Black;
			const FText ToolTipText = TypeDef.IsSet() ? TypeDef.GetValue().GetNameText() : FText();
			
			Contents->AddSlot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SScaleBox)
				[
					SNew(SNiagaraIconWidget)
					.IconToolTip(ToolTipText)
					.IconBrush(IconBrush)
					.IconColor(TypeColor)
					.SecondaryIconBrush(FAppStyle::GetBrush(TEXT("NoBrush")))
					.SecondaryIconColor(TypeColor)
				]
			];
		}
		
		
		Contents->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.FillWidth(1.0f)
		.Padding(5, 0)
		[
			SNew(STextBlock)
			.Text(GetDisplayNameText())
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		];

		Contents->AddSlot()
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SAssignNew(VisibilityWidget, SSimCacheTreeViewVisibilityWidget, SharedThis(this), Owner)
			.Visibility_Lambda([this]()
			{
				if (TSharedPtr<SNiagaraSimCacheTreeView> TreeView = Owner.Pin())
				{
					return TreeView->ShouldShowComponentView() ? EVisibility::Visible : EVisibility::Collapsed;
				}
				return EVisibility::Collapsed;
			})
		];
		
		return Contents;
	}

	TOptional<FNiagaraTypeDefinition> TypeDef;
};



