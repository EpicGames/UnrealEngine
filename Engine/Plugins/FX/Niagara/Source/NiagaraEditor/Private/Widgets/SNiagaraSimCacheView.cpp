// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSimCacheView.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "SNiagaraSimCacheDebugDataView.h"

#include "CoreMinimal.h"
#include "NiagaraEditorModule.h"

#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NiagaraSimCacheView"

static const FName NAME_Instance("Instance");

class SSimCacheDataBufferRowWidget : public SMultiColumnTableRow<TSharedPtr<int32>>
{
public:
	SLATE_BEGIN_ARGS(SSimCacheDataBufferRowWidget) {}
		SLATE_ARGUMENT(TSharedPtr<int32>, RowIndexPtr)
		SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheViewModel>, SimCacheViewModel)
	SLATE_END_ARGS()

	

	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		RowIndexPtr = InArgs._RowIndexPtr;
		SimCacheViewModel = InArgs._SimCacheViewModel;

		SMultiColumnTableRow<TSharedPtr<int32>>::Construct(
			FSuperRowType::FArguments()
			.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow"),
			InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		
		if(!SimCacheViewModel.Get()->IsCacheValid())
		{
			return SNullWidget::NullWidget;
		}

		const int32 InstanceIndex = *RowIndexPtr;
		
		if (InColumnName == NAME_Instance)
		{
			return SNew(STextBlock)
				.Text(FText::AsNumber(InstanceIndex));
		}
		
		return SNew(STextBlock)
			.Text(SimCacheViewModel->GetComponentText(InColumnName, InstanceIndex));
		
	}

	TSharedPtr<int32>						RowIndexPtr;
	TSharedPtr<FNiagaraSimCacheViewModel>   SimCacheViewModel;
};

void SNiagaraSimCacheView::Construct(const FArguments& InArgs)
{
	SimCacheViewModel = InArgs._SimCacheViewModel;

	SimCacheViewModel->OnViewDataChanged().AddSP(this, &SNiagaraSimCacheView::OnViewDataChanged);
	SimCacheViewModel->OnSimCacheChanged().AddSP(this, &SNiagaraSimCacheView::OnSimCacheChanged);
	SimCacheViewModel->OnBufferChanged().AddSP(this, &SNiagaraSimCacheView::OnBufferChanged);

	HeaderRowWidget = SNew(SHeaderRow);

	UpdateListView();

	TSharedRef<SScrollBar> HorizontalScrollBar =
		SNew(SScrollBar)
		.AlwaysShowScrollbar(true)
		.Thickness(12.0f)
		.Orientation(Orient_Horizontal);
	TSharedRef<SScrollBar> VerticalScrollBar =
		SNew(SScrollBar)
		.AlwaysShowScrollbar(true)
		.Thickness(12.0f)
		.Orientation(Orient_Vertical);
	CustomDisplayScrollBar =
		SNew(SScrollBar)
		.AlwaysShowScrollbar(false)
		.Thickness(12.0f)
		.Orientation(Orient_Vertical);

	//// Main Spreadsheet View
	SAssignNew(ListViewWidget, SListView<TSharedPtr<int32>>)
		.ListItemsSource(&RowItems)
		.OnGenerateRow(this, &SNiagaraSimCacheView::MakeRowWidget)
		.Visibility(EVisibility::Visible)
		.SelectionMode(ESelectionMode::Single)
		.ExternalScrollbar(VerticalScrollBar)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always)
		.AllowOverscroll(EAllowOverscroll::No)
		.HeaderRow(HeaderRowWidget);

	// Widget
	ChildSlot
	[
		SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				.ExternalScrollbar(HorizontalScrollBar)
				+ SScrollBox::Slot()
				[
					// switcher for spreadsheet / data interfaces
					SAssignNew(SwitchWidget, SWidgetSwitcher)
					.WidgetIndex_Lambda(
						[this]
						{
							switch (SimCacheViewModel->GetSelectionMode())
							{
								case FNiagaraSimCacheViewModel::ESelectionMode::SystemInstance:
								case FNiagaraSimCacheViewModel::ESelectionMode::Emitter:			return 0;
								default:															return 1;
							}
						}
					)
					+SWidgetSwitcher::Slot()
					[
						ListViewWidget.ToSharedRef()
					]
					+SWidgetSwitcher::Slot()
					[
						SNullWidget::NullWidget
					]
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SWidgetSwitcher)
					.WidgetIndex_Lambda(
						[this]
						{
							switch (SimCacheViewModel->GetSelectionMode())
							{
								case FNiagaraSimCacheViewModel::ESelectionMode::SystemInstance:
								case FNiagaraSimCacheViewModel::ESelectionMode::Emitter:			return 0;
								case FNiagaraSimCacheViewModel::ESelectionMode::DataInterface:		return 1;
								case FNiagaraSimCacheViewModel::ESelectionMode::DebugData:			return 2;
								default:
									checkNoEntry();
									return 0;
							}
						}
					)
				+SWidgetSwitcher::Slot()
				[
					VerticalScrollBar
				]
				+SWidgetSwitcher::Slot()
				[
					CustomDisplayScrollBar.ToSharedRef()
				]
				+SWidgetSwitcher::Slot()
				[
					SNullWidget::NullWidget
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			HorizontalScrollBar
		]
	];
};

TSharedRef<ITableRow> SNiagaraSimCacheView::MakeRowWidget(const TSharedPtr<int32> RowIndexPtr, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return
		SNew(SSimCacheDataBufferRowWidget, OwnerTable)
		.RowIndexPtr(RowIndexPtr)
		.SimCacheViewModel(SimCacheViewModel);
}

void SNiagaraSimCacheView::UpdateCustomDisplayWidget()
{
	for (TSharedPtr<SWidget> Widget : CustomDisplayWidgets)
	{
		SwitchWidget->RemoveSlot(Widget.ToSharedRef());
	}
	CustomDisplayWidgets.Empty();

	TSharedRef<SVerticalBox> WidgetBox = SNew(SVerticalBox);

	TOptional<FText> MissingCustomDisplayText;
	switch (SimCacheViewModel->GetSelectionMode())
	{
		case FNiagaraSimCacheViewModel::ESelectionMode::DataInterface:
		{
			const FNiagaraVariableBase DIVariable = SimCacheViewModel->GetSelectedDataInterface();

			FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::GetModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
			for (TSharedRef<INiagaraDataInterfaceSimCacheVisualizer> Visualizer : NiagaraEditorModule.FindDataInterfaceCacheVisualizer(DIVariable.GetType().GetClass()))
			{
				if (const UObject* DataObject = SimCacheViewModel->GetSelectedDataInterfaceStorage())
				{
					TSharedPtr<SWidget> VisualizerWidget = Visualizer->CreateWidgetFor(DataObject, SimCacheViewModel);
					CustomDisplayWidgets.Add(VisualizerWidget);
					WidgetBox->AddSlot()
						.AutoHeight()
						.AttachWidget(VisualizerWidget.ToSharedRef());
				}
			}

			if (CustomDisplayWidgets.Num() == 0)
			{
				MissingCustomDisplayText = FText::Format(LOCTEXT("NoDataInterfaceVisualizer", "No valid visualizer found for data interface '{0}'"), DIVariable.GetType().GetNameText());
			}
			break;
		}

		case FNiagaraSimCacheViewModel::ESelectionMode::DebugData:
		{
			if (SimCacheViewModel->GetCacheDebugData())
			{
				TSharedPtr<SWidget> DebugDataWidget =
					SNew(SNiagaraSimCacheDebugDataView)
					.SimCacheViewModel(SimCacheViewModel);

				CustomDisplayWidgets.Add(DebugDataWidget);

				WidgetBox->AddSlot()
					.AutoHeight()
					.AttachWidget(DebugDataWidget.ToSharedRef());
			}
			else
			{
				MissingCustomDisplayText = LOCTEXT("NoDebugData", "Data Data not found inside cache");
			}
			break;
		}
	}

	if (MissingCustomDisplayText.IsSet())
	{
		TSharedPtr<SWidget> VisualizerWidget = SNew(SBox)
		   .Padding(10)
		   [
			   SNew(STextBlock)
			   .Text(MissingCustomDisplayText.GetValue())
		   ];
		CustomDisplayWidgets.Add(VisualizerWidget);
		WidgetBox->AddSlot()
			.AutoHeight()
			.AttachWidget(VisualizerWidget.ToSharedRef());
	}

	SwitchWidget->AddSlot(1).AttachWidget(
		SNew(SScrollBox)
		.Orientation(Orient_Vertical)
		.ExternalScrollbar(CustomDisplayScrollBar)
		 + SScrollBox::Slot()
		[
			WidgetBox
		]
	);
}

void SNiagaraSimCacheView::UpdateListView()
{
	GenerateColumns();
	GenerateRows();
	SortRows();

	if (ListViewWidget)
	{
		ListViewWidget->RequestListRefresh();
	}
}

void SNiagaraSimCacheView::GenerateColumns()
{
	// Invalid early out
	if (SimCacheViewModel->IsCacheValid() == false)
	{
		HeaderRowWidget->ClearColumns();
		HeaderRowWidget->RefreshColumns();
		return;
	}

	// Do we need to update our columns?
	bool bRebuildColumns = true;

	TConstArrayView<FNiagaraSimCacheViewModel::FComponentInfo> SelectedComponents = SimCacheViewModel->GetSelectedComponentInfos();
	if (HeaderRowWidget->GetColumns().Num() == SelectedComponents.Num() + 1)
	{
		bRebuildColumns = false;
		for (int32 i = 0; i < SelectedComponents.Num(); ++i)
		{
			if (HeaderRowWidget->GetColumns()[i + 1].ColumnId != SelectedComponents[i].Name)
			{
				bRebuildColumns = true;
				break;
			}
		}
	}

	//  Give columns a width to prevent them from being shrunk when filtering. 
	if (bRebuildColumns)
	{
		constexpr float ManualWidth = 125.0f;
		HeaderRowWidget->ClearColumns();

		if (SimCacheViewModel->IsCacheValid())
		{
			// Generate instance count column
			HeaderRowWidget->AddColumn(
				SHeaderRow::Column(NAME_Instance)
				.DefaultLabel(FText::FromName(NAME_Instance))
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.VAlignHeader(EVerticalAlignment::VAlign_Fill)
				.HAlignCell(EHorizontalAlignment::HAlign_Center)
				.VAlignCell(EVerticalAlignment::VAlign_Fill)
				.ManualWidth(ManualWidth)
				.SortMode(this, &SNiagaraSimCacheView::GetColumnSortMode, NAME_Instance)
				.OnSort(this, &SNiagaraSimCacheView::OnColumnNameSortModeChanged)
			);

			// Generate a column for each component
			for (const FNiagaraSimCacheViewModel::FComponentInfo& ComponentInfo : SelectedComponents)
			{
				HeaderRowWidget->AddColumn(
					SHeaderRow::Column(ComponentInfo.Name)
					.DefaultLabel(FText::FromName(ComponentInfo.Name))
					.HAlignHeader(EHorizontalAlignment::HAlign_Center)
					.VAlignHeader(EVerticalAlignment::VAlign_Fill)
					.HAlignCell(EHorizontalAlignment::HAlign_Center)
					.VAlignCell(EVerticalAlignment::VAlign_Fill)
					.FillWidth(1.0f)
					.ManualWidth(ManualWidth)
					.ShouldGenerateWidget(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SNiagaraSimCacheView::GetShouldGenerateWidget, ComponentInfo.Name)))
					.SortMode(this, &SNiagaraSimCacheView::GetColumnSortMode, ComponentInfo.Name)
					.OnSort(this, &SNiagaraSimCacheView::OnColumnNameSortModeChanged)
				);
			}
		}
	}

	HeaderRowWidget->RefreshColumns();
}

void SNiagaraSimCacheView::GenerateRows()
{
	if (SimCacheViewModel->IsCacheValid() == false )
	{
		RowItems.Empty();
		return;
	}
	
	RowItems.Reset(SimCacheViewModel->GetNumInstances());
	for (int32 i = 0; i < SimCacheViewModel->GetNumInstances(); ++i)
	{
		RowItems.Emplace(MakeShared<int32>(i));
	}
}

void SNiagaraSimCacheView::SortRows()
{
	if (RowItems.Num() == 0)
	{
		return;
	}

	// Is the column name valid?
	const int32 ColumnIndex = SimCacheViewModel->GetSelectedComponentInfos().IndexOfByPredicate([ColumnName=SortColumnName](const FNiagaraSimCacheViewModel::FComponentInfo& ComponentInfo) { return ComponentInfo.Name == ColumnName; });
	if (ColumnIndex == INDEX_NONE)
	{
		SortColumnName = NAME_Instance;
	}

	// Simple sort
	if (SortColumnName == NAME_Instance)
	{
		if ( SortMode == EColumnSortMode::Ascending )
		{
			RowItems.Sort([](const TSharedPtr<int32>& Lhs, const TSharedPtr<int32>& Rhs) { return *Lhs <= *Rhs; });
		}
		else
		{
			RowItems.Sort([](const TSharedPtr<int32>& Lhs, const TSharedPtr<int32>& Rhs) { return *Lhs  > *Rhs; });
		}
	}
	// Complex sort
	else
	{
		const bool bAscending = SortMode == EColumnSortMode::Ascending;
		RowItems.Sort(
			[ViewModel=SimCacheViewModel.Get(), bAscending, ColumnIndex](const TSharedPtr<int32>& Lhs, const TSharedPtr<int32>& Rhs)
			{
				return ViewModel->CompareComponent(ColumnIndex, *Lhs, *Rhs, bAscending);
			}
		);
	}
}

void SNiagaraSimCacheView::OnSimCacheChanged()
{
	UpdateListView();
	UpdateCustomDisplayWidget();
}

void SNiagaraSimCacheView::OnViewDataChanged(const bool bFullRefresh)
{
	UpdateListView();
}

void SNiagaraSimCacheView::OnBufferChanged()
{
	UpdateListView();
	UpdateCustomDisplayWidget();
}

bool SNiagaraSimCacheView::GetShouldGenerateWidget(FName Name)
{
	if(!SimCacheViewModel->IsComponentFilterActive())
	{
		return true;
	}

	return SimCacheViewModel->IsComponentFiltered(Name);
}

EColumnSortMode::Type SNiagaraSimCacheView::GetColumnSortMode(FName ColumnName) const
{
	return ColumnName == SortColumnName ? SortMode : EColumnSortMode::None;
}

void SNiagaraSimCacheView::OnColumnNameSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortMode		= InSortMode;
	SortColumnName	= ColumnId;

	SortRows();
	if (ListViewWidget)
	{
		ListViewWidget->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
