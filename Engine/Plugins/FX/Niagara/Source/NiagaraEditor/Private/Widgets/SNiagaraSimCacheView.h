// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FNiagaraSimCacheViewModel;
class ITableRow;
class STableViewBase;
class SHeaderRow;
class SWidgetSwitcher;
class SScrollBar;

class SNiagaraSimCacheView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheView) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraSimCacheViewModel>, SimCacheViewModel)
	SLATE_END_ARGS()

	using FBufferSelectionInfo = TPair<int32, FText>;

	void Construct(const FArguments& InArgs);

	TSharedRef<ITableRow> MakeRowWidget(const TSharedPtr<int32> RowIndexPtr, const TSharedRef<STableViewBase>& OwnerTable) const;

private:
	void UpdateListView();
	void GenerateColumns();
	void GenerateRows();
	void SortRows();

	void OnSimCacheChanged();

	void OnViewDataChanged(const bool bFullRefresh);
	void OnBufferChanged();

	bool GetShouldGenerateWidget(FName Name);
	void UpdateCustomDisplayWidget();

	EColumnSortMode::Type GetColumnSortMode(FName ColumnName) const;
	void OnColumnNameSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

private:
	TArray<TSharedPtr<int32>>					RowItems;
	TSharedPtr<FNiagaraSimCacheViewModel>		SimCacheViewModel;
	
	TSharedPtr<SHeaderRow>						HeaderRowWidget;
	TSharedPtr<SListView<TSharedPtr<int32>>>	ListViewWidget;
	TSharedPtr<SWidgetSwitcher>					SwitchWidget;
	TArray<TSharedPtr<SWidget>>					CustomDisplayWidgets;
	TSharedPtr<SScrollBar>						CustomDisplayScrollBar;

	EColumnSortMode::Type						SortMode = EColumnSortMode::Ascending;
	FName										SortColumnName;
};
