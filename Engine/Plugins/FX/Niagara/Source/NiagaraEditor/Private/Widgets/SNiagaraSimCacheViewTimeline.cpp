// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSimCacheViewTimeline.h"

#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Styling/AppStyle.h"
#include "EditorWidgetsModule.h"


#define LOCTEXT_NAMESPACE "NigaraSimCacheViewTimeline"

void SNiagaraSimCacheViewTimeline::Construct(const FArguments& InArgs)
{
	WeakViewModel = InArgs._WeakViewModel;
}

int32 SNiagaraSimCacheViewTimeline::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId,
	const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FNiagaraSimCacheViewModel* ViewModel = WeakViewModel.Pin().Get();

	if( !ViewModel )
	{
		return LayerId;
	}

	const int NumFrames = ViewModel->GetNumFrames();
	const int CurrentFrame = ViewModel->GetFrameIndex();
	const float NormalizedTime = NumFrames > 0 ? float(CurrentFrame) / float(NumFrames) : 0;

	//////////////////////////////////////////////////////////////////////////
	// Draw boxes to show frames
	{
		const FSlateBrush* BoxBrush = FAppStyle::GetBrush("Sequencer.Section.BackgroundTint");
		const FLinearColor BoxTints[2] = { FLinearColor(0.5f, 0.5f, 0.5f, 1.0f), FLinearColor(0.3f, 0.3f, 0.3f, 1.0f) };
		const FLinearColor CurrentTint(0.2f, 0.2f, 1.0f, 1.0f);
		if (NumFrames > 0)
		{
			const float UStep = AllottedGeometry.Size.X / float(NumFrames);

			for (int32 i = 0; i < NumFrames; ++i)
			{
				const FVector2D BoxLocation(UStep * float(i), 0.0f);
				const FVector2D BoxSize(UStep, AllottedGeometry.Size.Y);
				const FLinearColor& Tint = (i == CurrentFrame) ? CurrentTint : BoxTints[i & 1];
				FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, AllottedGeometry.ToPaintGeometry(BoxSize, FSlateLayoutTransform(BoxLocation)), BoxBrush, ESlateDrawEffect::None, Tint);
			}
		}
		else
		{
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, AllottedGeometry.ToPaintGeometry(), BoxBrush, ESlateDrawEffect::None, BoxTints[0]);
		}
	}

	//////////////////////////////////////////////////////////////////////////
	// Draw a line to show the current time
	{
		TArray<FVector2D> LinePoints;
		LinePoints.AddUninitialized(2);

		const float LineXPos = AllottedGeometry.Size.X * NormalizedTime;
		LinePoints[0] = FVector2D(LineXPos, 0.0f);
		LinePoints[1] = FVector2D(LineXPos, AllottedGeometry.Size.Y);

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			FLinearColor::White
		);
	}

	return LayerId;
}

FReply SNiagaraSimCacheViewTimeline::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return OnTimelineScrubbed(MyGeometry, MouseEvent);
}

FReply SNiagaraSimCacheViewTimeline::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return OnTimelineScrubbed(MyGeometry, MouseEvent);
}

FReply SNiagaraSimCacheViewTimeline::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// TODO Implement dragging on timeline
	return FReply::Unhandled();
}

FReply SNiagaraSimCacheViewTimeline::OnTimelineScrubbed(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( FNiagaraSimCacheViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		const FVector2D LocalLocation = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		const float NormalizedTime = FMath::Clamp( LocalLocation.X / MyGeometry.Size.X, 0.0f, 1.0f);

		const int32 NewFrameIndex = FMath::Floor(NormalizedTime * float(ViewModel->GetNumFrames()));

		ViewModel->SetFrameIndex(NewFrameIndex);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
