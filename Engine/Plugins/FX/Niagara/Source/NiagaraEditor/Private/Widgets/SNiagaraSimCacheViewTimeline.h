// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SNiagaraSimCacheViewTimeline : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheViewTimeline) {}
		SLATE_ARGUMENT(TWeakPtr<class FNiagaraSimCacheViewModel>, WeakViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// SWidget interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	// End of SWidget interface

	FReply OnTimelineScrubbed(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	
private:
	TWeakPtr<FNiagaraSimCacheViewModel> WeakViewModel;

};
