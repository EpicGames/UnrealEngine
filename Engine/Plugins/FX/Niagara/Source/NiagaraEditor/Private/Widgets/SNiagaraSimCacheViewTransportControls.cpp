// Copyright Epic Games, Inc. All Rights Reserved.


#include "SNiagaraSimCacheViewTransportControls.h"

#include "Modules/ModuleManager.h"
#include "EditorWidgetsModule.h"
#include "SlateOptMacros.h"
#include "ITransportControl.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "Widgets/Input/SNumericEntryBox.h"

void SNiagaraSimCacheViewTransportControls::Construct(const FArguments& InArgs)
{
	WeakViewModel = InArgs._WeakViewModel;
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::Get().LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");

	TransportControls = MakeTransportControlsWidget(EditorWidgetsModule);

	ChildSlot
	[
		TransportControls.ToSharedRef()
	];
}

TSharedRef<SWidget> SNiagaraSimCacheViewTransportControls::MakeTransportControlsWidget(
	FEditorWidgetsModule& EditorWidgetsModule)
{
	FTransportControlArgs TransportControlArgs;
	TransportControlArgs.OnBackwardEnd.BindSP(this, &SNiagaraSimCacheViewTransportControls::OnTransportBackwardEnd);
	TransportControlArgs.OnBackwardStep.BindSP(this, &SNiagaraSimCacheViewTransportControls::OnTransportBackwardStep);
	TransportControlArgs.OnForwardStep.BindSP(this, &SNiagaraSimCacheViewTransportControls::OnTransportForwardStep);
	TransportControlArgs.OnForwardEnd.BindSP(this, &SNiagaraSimCacheViewTransportControls::OnTransportForwardEnd);
	

	TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::BackwardEnd));
	TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::BackwardStep));
	TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(FOnMakeTransportWidget::CreateSP(this, &SNiagaraSimCacheViewTransportControls::CreateFrameWidget)));
	TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::ForwardStep));
	TransportControlArgs.WidgetsToCreate.Add(FTransportControlWidget(ETransportControlWidgetType::ForwardEnd));
	
	TransportControlArgs.bAreButtonsFocusable = false;

	return EditorWidgetsModule.CreateTransportControl(TransportControlArgs);
}

TOptional<int32> SNiagaraSimCacheViewTransportControls::GetCurrentFrame() const
{
	if ( FNiagaraSimCacheViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		return ViewModel->GetFrameIndex();
	}

	return 0;
}

void SNiagaraSimCacheViewTransportControls::OnFrameIndexChanged(const int32 InFrameIndex, ETextCommit::Type Arg)
{
	if (FNiagaraSimCacheViewModel* ViewModel = WeakViewModel.Pin().Get())
	{
		int32 NewFrameIndex = FMath::Max(0, InFrameIndex);
		NewFrameIndex = FMath::Min(NewFrameIndex, ViewModel->GetNumFrames() - 1);
		ViewModel->SetFrameIndex(NewFrameIndex);
	}
}

TOptional<int32> SNiagaraSimCacheViewTransportControls::GetMaxFrameIndex() const
{
	if (FNiagaraSimCacheViewModel* ViewModel = WeakViewModel.Pin().Get())
	{
		return ViewModel->GetNumFrames() - 1;
	}

	return 0;
}

TSharedRef<SWidget> SNiagaraSimCacheViewTransportControls::CreateFrameWidget()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0.5f)
		[
			SNew(SNumericEntryBox<int32>)
			.MinDesiredValueWidth(25)
			.AllowSpin(false)
			.MinValue(0)
			.MaxValue(this, &SNiagaraSimCacheViewTransportControls::GetMaxFrameIndex)
			.Value(this, &SNiagaraSimCacheViewTransportControls::GetCurrentFrame)
			.OnValueCommitted(this, &SNiagaraSimCacheViewTransportControls::OnFrameIndexChanged)
		];
	
}

FReply SNiagaraSimCacheViewTransportControls::OnTransportBackwardEnd()
{
	if ( FNiagaraSimCacheViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		ViewModel->SetFrameIndex(0);
		
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SNiagaraSimCacheViewTransportControls::OnTransportBackwardStep()
{
	if ( FNiagaraSimCacheViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		ViewModel->SetFrameIndex(FMath::Max(ViewModel->GetFrameIndex() -1, 0));
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SNiagaraSimCacheViewTransportControls::OnTransportForwardStep()
{
	if ( FNiagaraSimCacheViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		ViewModel->SetFrameIndex(FMath::Min(ViewModel->GetFrameIndex() + 1, ViewModel->GetNumFrames() - 1));
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SNiagaraSimCacheViewTransportControls::OnTransportForwardEnd()
{
	if ( FNiagaraSimCacheViewModel* ViewModel = WeakViewModel.Pin().Get() )
	{
		ViewModel->SetFrameIndex(ViewModel->GetNumFrames() - 1);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}