// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class SNiagaraSimCacheViewTransportControls : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheViewTransportControls) {}
		SLATE_ARGUMENT(TWeakPtr<class FNiagaraSimCacheViewModel>, WeakViewModel)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	NIAGARAEDITOR_API void Construct(const FArguments& InArgs);

	
	// Transport Controls
	NIAGARAEDITOR_API TSharedRef<SWidget> MakeTransportControlsWidget(class FEditorWidgetsModule& EditorWidgetsModule);
	NIAGARAEDITOR_API TOptional<int32> GetCurrentFrame() const;
	NIAGARAEDITOR_API void OnFrameIndexChanged(const int32 InFrameIndex, ETextCommit::Type Arg);
	NIAGARAEDITOR_API TOptional<int32> GetMaxFrameIndex() const;
	// Custom transport widget for frame selection
	NIAGARAEDITOR_API TSharedRef<SWidget> CreateFrameWidget();
	
	NIAGARAEDITOR_API FReply OnTransportBackwardEnd();
	NIAGARAEDITOR_API FReply OnTransportBackwardStep();
	NIAGARAEDITOR_API FReply OnTransportForwardStep();
	NIAGARAEDITOR_API FReply OnTransportForwardEnd();

private:
	TWeakPtr<FNiagaraSimCacheViewModel> WeakViewModel;
	TSharedPtr<SWidget>	TransportControls;
};
