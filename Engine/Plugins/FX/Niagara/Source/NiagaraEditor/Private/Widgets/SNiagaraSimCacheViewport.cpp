// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSimCacheViewport.h"

#include "AdvancedPreviewScene.h"
#include "ViewModels/NiagaraSimCacheViewModel.h"
#include "SEditorViewportToolBarMenu.h"
#include "SNiagaraSimCacheViewportToolbar.h"
#include "NiagaraComponent.h"
#include "NiagaraEditorCommands.h"

//////////////////////////////////////////////////////////////////////////
// Viewport Client
//////////////////////////////////////////////////////////////////////////


class FNiagaraSimCacheViewportClient : public FEditorViewportClient
{
public:

	FNiagaraSimCacheViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SNiagaraSimCacheViewport>& InNiagaraEditorViewport);

	// FEditorViewportClient interface
	virtual FLinearColor GetBackgroundColor() const override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(FViewport* Viewport,FCanvas* Canvas) override;
	virtual bool ShouldOrbitCamera() const override;
	virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 ViewIndex = INDEX_NONE) override;
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override { return false; }
	virtual bool CanCycleWidgetMode() const override { return false; }

	FAdvancedPreviewScene* AdvancedPreviewScene = nullptr;

	TWeakPtr<SNiagaraSimCacheViewport> SimCacheViewportPrt;
};

FNiagaraSimCacheViewportClient::FNiagaraSimCacheViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SNiagaraSimCacheViewport>& InNiagaraEditorViewport):
	FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InNiagaraEditorViewport))
{
	SimCacheViewportPrt = InNiagaraEditorViewport;

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(80,80,80);
	DrawHelper.GridColorMajor = FColor(72,72,72);
	DrawHelper.GridColorMinor = FColor(64,64,64);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
	ShowWidget(false);

	FEditorViewportClient::SetViewMode(VMI_Lit);

	EngineShowFlags.SetSnap(0);

	OverrideNearClipPlane(1.0f);

	FNiagaraSimCacheViewportClient::SetIsSimulateInEditorViewport(true);
}

FLinearColor FNiagaraSimCacheViewportClient::GetBackgroundColor() const
{
	return FEditorViewportClient::GetBackgroundColor();
}

void FNiagaraSimCacheViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Tick the preview scene world.
	PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}

void FNiagaraSimCacheViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{

	FEditorViewportClient::Draw(InViewport, Canvas);
}

bool FNiagaraSimCacheViewportClient::ShouldOrbitCamera() const
{
	return bUsingOrbitCamera;
}

FSceneView* FNiagaraSimCacheViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	return FEditorViewportClient::CalcSceneView(ViewFamily, StereoViewIndex);
}

//////////////////////////////////////////////////////////////////////////
// Viewport Widget
//////////////////////////////////////////////////////////////////////////

void SNiagaraSimCacheViewport::Construct(FArguments InArgs)
{
	AdvancedPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));
	AdvancedPreviewScene->SetFloorVisibility(false, true);
	AdvancedPreviewScene->SetEnvironmentVisibility(false, true);
	
	SEditorViewport::Construct(SEditorViewport::FArguments());

	Client->EngineShowFlags.SetGrid(false);
}

void SNiagaraSimCacheViewport::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (PreviewComponent)
	{
		Collector.AddReferencedObject(PreviewComponent);
	}
}

void SNiagaraSimCacheViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime,
                                    const float InDeltaTime)
{
	SEditorViewport::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SNiagaraSimCacheViewport::SetPreviewComponent(UNiagaraComponent* NewPreviewComponent)
{
	if (PreviewComponent != nullptr)
	{
		AdvancedPreviewScene->RemoveComponent(PreviewComponent);
	}
	
	PreviewComponent = NewPreviewComponent;

	if (PreviewComponent != nullptr)
	{
		AdvancedPreviewScene->AddComponent(PreviewComponent, PreviewComponent->GetRelativeTransform());
	}
}

TSharedRef<SEditorViewport> SNiagaraSimCacheViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SNiagaraSimCacheViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SNiagaraSimCacheViewport::OnFloatingButtonClicked()
{
}

TSharedRef<FEditorViewportClient> SNiagaraSimCacheViewport::MakeEditorViewportClient()
{
	SimCacheViewportClient = MakeShareable( new FNiagaraSimCacheViewportClient(*AdvancedPreviewScene.Get(), SharedThis(this)));

	SimCacheViewportClient->SetViewLocation(FVector::ZeroVector);
	SimCacheViewportClient->SetViewRotation(FRotator::ZeroRotator);
	SimCacheViewportClient->SetViewLocationForOrbiting(FVector::ZeroVector);
	SimCacheViewportClient->bSetListenerPosition = false;

	SimCacheViewportClient->SetRealtime(true);
	SimCacheViewportClient->VisibilityDelegate.BindSP(this, &SNiagaraSimCacheViewport::IsVisible);
	// Default to orbit camera for parity with the Niagara editor viewport.
	SimCacheViewportClient->bUsingOrbitCamera = true;
	
	return SimCacheViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SNiagaraSimCacheViewport::BuildViewportToolbar()
{
	return SNew(SNiagaraSimCacheViewportToolbar, SharedThis(this));
}

EVisibility SNiagaraSimCacheViewport::OnGetViewportContentVisibility() const
{
	return SEditorViewport::OnGetViewportContentVisibility();
}

void SNiagaraSimCacheViewport::ToggleOrbit()
{
	SimCacheViewportClient->ToggleOrbitCamera(!SimCacheViewportClient->bUsingOrbitCamera);
}

bool SNiagaraSimCacheViewport::IsToggleOrbitChecked()
{
	return SimCacheViewportClient->bUsingOrbitCamera;
}

void SNiagaraSimCacheViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FNiagaraEditorCommands& Commands = FNiagaraEditorCommands::Get();

	CommandList->MapAction(
		Commands.ToggleOrbit,
		FExecuteAction::CreateSP(this, &SNiagaraSimCacheViewport::ToggleOrbit),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SNiagaraSimCacheViewport::IsToggleOrbitChecked));
}

void SNiagaraSimCacheViewport::OnFocusViewportToSelection()
{
	SEditorViewport::OnFocusViewportToSelection();
}

bool SNiagaraSimCacheViewport::IsVisible() const
{
	return SEditorViewport::IsVisible();
}
