// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AdvancedPreviewScene.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "UObject/GCObject.h"
#include "SEditorViewport.h"

class UNiagaraComponent;

class SNiagaraSimCacheViewport : public SEditorViewport, public FGCObject, public ICommonEditorViewportToolbarInfoProvider
{
	SLATE_BEGIN_ARGS(SNiagaraSimCacheViewport) {}
	SLATE_END_ARGS()
	
public:
	void Construct(FArguments InArgs);

	/** FGCObject interface **/
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;

	virtual FString GetReferencerName() const override
	{
		return TEXT("SNiagaraSimCacheViewport");
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void SetPreviewComponent (UNiagaraComponent* NewPreviewComponent);
	UNiagaraComponent* GetPreviewComponent () const { return PreviewComponent; }

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	TSharedRef<FAdvancedPreviewScene> GetPreviewScene() const {return AdvancedPreviewScene.ToSharedRef();}

protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	void ToggleOrbit();
	bool IsToggleOrbitChecked();
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;

private:
	virtual bool IsVisible() const override;
	
	TSharedPtr<FAdvancedPreviewScene> AdvancedPreviewScene;

	TSharedPtr<class FNiagaraSimCacheViewportClient> SimCacheViewportClient;

	TObjectPtr<UNiagaraComponent> PreviewComponent = nullptr;
};
