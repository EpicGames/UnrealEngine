// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSimCacheViewportToolbar.h"

#include "NiagaraEditorCommands.h"

void SNiagaraSimCacheViewportToolbar::Construct(const FArguments& InArgs, TSharedPtr<SNiagaraSimCacheViewport> InViewport)
{
	SCommonEditorViewportToolbarBase::Construct(SCommonEditorViewportToolbarBase::FArguments(), InViewport);
}

void SNiagaraSimCacheViewportToolbar::ExtendOptionsMenu(FMenuBuilder& OptionsMenuBuilder) const
{
	SCommonEditorViewportToolbarBase::ExtendOptionsMenu(OptionsMenuBuilder);

	OptionsMenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleOrbit);
}

void SNiagaraSimCacheViewportToolbar::ExtendLeftAlignedToolbarSlots(TSharedPtr<SHorizontalBox> MainBoxPtr,
	TSharedPtr<SViewportToolBar> ParentToolBarPtr) const
{
	SCommonEditorViewportToolbarBase::ExtendLeftAlignedToolbarSlots(MainBoxPtr, ParentToolBarPtr);
}
