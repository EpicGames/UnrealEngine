// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SCommonEditorViewportToolbarBase.h"
#include "SNiagaraSimCacheViewport.h"

class SNiagaraSimCacheViewportToolbar final : public SCommonEditorViewportToolbarBase
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSimCacheViewportToolbar) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SNiagaraSimCacheViewport> InViewport);

	// SCommonEditorViewportToolbarBase interface
	virtual void ExtendOptionsMenu(FMenuBuilder& OptionsMenuBuilder) const override;
	/** Used for a custom realtime button as the default realtime button is for viewport realtime, not simulation realtime and overrides only allow for direct setting */
	virtual void ExtendLeftAlignedToolbarSlots(TSharedPtr<SHorizontalBox> MainBoxPtr, TSharedPtr<SViewportToolBar> ParentToolBarPtr) const override;
	virtual bool GetShowScalabilityMenu() const override
	{
		return true;
	}
	// End of SCommonEditorViewportToolbarBase
	
};
