// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraStatelessEmitterTemplateWidgets.h"
#include "Stateless/NiagaraStatelessCommon.h"
#include "Stateless/NiagaraStatelessEmitterTemplate.h"
#include "ViewModels/NiagaraStatelessEmitterTemplateViewModel.h"
#include "Widgets/NiagaraHLSLSyntaxHighlighter.h"
#include "Widgets/SNiagaraParameterName.h"
#include "NiagaraEditorUtilities.h"

#include "Components/VerticalBox.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"


#define LOCTEXT_NAMESPACE "NiagaraStatelessEmitterTemplateWidgets"

/////////////////////////////////////////////////////////////////////////////////////////////////////////////// 

void SNiagaraStatelessEmitterTemplateModules::Construct(const FArguments& InArgs)
{
	ViewModel = InArgs._ViewModel;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = ViewModel.Get();
	DetailsViewArgs.bAllowSearch = false;

	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(ViewModel->GetTemplateForEdit());

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f, 2.0f))
		[
			DetailsView
		]
	];
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////// 

void SNiagaraStatelessEmitterTemplateFeatures::Construct(const FArguments& InArgs)
{
	ViewModel = InArgs._ViewModel;
	ViewModel->OnTemplateChanged().AddSP(this, &SNiagaraStatelessEmitterTemplateFeatures::OnRebuildWidget);

	ChildSlot
	[
		SAssignNew(GridPanel, SGridPanel)
	];

	OnRebuildWidget();
}

void SNiagaraStatelessEmitterTemplateFeatures::OnRebuildWidget()
{
	GridPanel->ClearChildren();

	const TPair<ENiagaraStatelessFeatureMask, ENiagaraStatelessFeatureMask> FeatureMasks = ViewModel->GetFeatureMaskRange();
	const auto GetEnabledText = [](bool bEnabled) { return bEnabled ? LOCTEXT("Enabled", "Enabled") : LOCTEXT("Disabled", "Disabled"); };

	UEnum* Enum = StaticEnum<ENiagaraStatelessFeatureMask>();
	int32 iRow = 0;

	// Setup header
	{
		int32 iColumn = 0;
		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Feature", "Feature"))
		];
		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AllModulesEnabled", "All Modules Enabled"))
		];
		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("RequriedModulesEnabled", "Required Modules Enabled"))
		];
		++iRow;
	}

	// Generated rows
	for (int32 i = 0; i < Enum->NumEnums() - 1; ++i)		// Note: -1 to skip the implicit MAX field
	{
		if (Enum->HasMetaData(TEXT("Hidden"), i))
		{
			continue;
		}

		int32 iColumn = 0;

		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(Enum->GetDisplayNameTextByIndex(i))
		];		

		const bool bMinEnabled = (int32(FeatureMasks.Key) & int32(Enum->GetValueByIndex(i))) != 0;
		const bool bMaxEnabled = (int32(FeatureMasks.Value) & int32(Enum->GetValueByIndex(i))) != 0;

		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(GetEnabledText(bMinEnabled))
		];		

		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(GetEnabledText(bMaxEnabled))
		];

		++iRow;
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////// 

void SNiagaraStatelessEmitterTemplateOutputVariables::Construct(const FArguments& InArgs)
{
	ViewModel = InArgs._ViewModel;
	ViewModel->OnTemplateChanged().AddSP(this, &SNiagaraStatelessEmitterTemplateOutputVariables::OnRebuildWidget);

	ChildSlot
	[
		SNew(SScrollBox)
		+SScrollBox::Slot().Padding(5)
		[
			SAssignNew(GridPanel, SGridPanel)
		]
	];

	OnRebuildWidget();
}

void SNiagaraStatelessEmitterTemplateOutputVariables::OnRebuildWidget()
{
	GridPanel->ClearChildren();

	int32 iRow = 0;
	{
		int32 iColumn = 0;
		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Variable", "Variable"))
		];
		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Implicit", "Implicit"))
		];
		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Module", "Module"))
		];
		GridPanel->AddSlot(iColumn++, iRow)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("GPU", "GPU"))
		];
	}
	++iRow;

	const FText TextTrue = LOCTEXT("True", "True");
	const FText TextFalse = LOCTEXT("False", "False");

	for (const FNiagaraStatelessEmitterTemplateViewModel::FOutputVariable& OutputVariable : ViewModel->GetOututputVariables())
	{
		int32 iColumn = 0;

		GridPanel->AddSlot(iColumn++, iRow)
		[
			FNiagaraParameterUtilities::GetParameterWidget(FNiagaraVariable(OutputVariable.Variable), true, false)
		];
		GridPanel->AddSlot(iColumn++, iRow)
		[
			SNew(STextBlock).Text(OutputVariable.bIsImplicit ? TextTrue : TextFalse)
		];
		GridPanel->AddSlot(iColumn++, iRow)
		[
			SNew(STextBlock).Text(OutputVariable.bIsModule ? TextTrue : TextFalse)
		];
		GridPanel->AddSlot(iColumn++, iRow)
		[
			SNew(STextBlock).Text(OutputVariable.bIsShader ? TextTrue : TextFalse)
		];
		++iRow;
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////// 

void SNiagaraStatelessEmitterTemplateCodeView::Construct(const FArguments& InArgs)
{
	ViewModel = InArgs._ViewModel;
	ViewModel->OnTemplateChanged().AddSP(this, &SNiagaraStatelessEmitterTemplateCodeView::OnRebuildWidget);

	SyntaxHighlighter = FNiagaraHLSLSyntaxHighlighter::Create();

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("CopyToClipboard", "Copy To Clipboard"))
				.OnPressed(this, &SNiagaraStatelessEmitterTemplateCodeView::OnCopyToClipboard)
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SScrollBox)
			+SScrollBox::Slot().Padding(5)
			[
				SNew(SMultiLineEditableText)
				.Text(this, &SNiagaraStatelessEmitterTemplateCodeView::GetHlslText)
				.TextStyle(FAppStyle::Get(), "MessageLog")
				.IsReadOnly(true)
				.Marshaller(SyntaxHighlighter)
			]
		]
	];

	OnRebuildWidget();
}

void SNiagaraStatelessEmitterTemplateCodeView::OnRebuildWidget()
{
	HlslAsString = ViewModel->GenerateComputeTemplateHLSL();
	HlslAsText = FText::FromString(HlslAsString);
}

FText SNiagaraStatelessEmitterTemplateCodeView::GetHlslText() const
{
	return HlslAsText;
}

void SNiagaraStatelessEmitterTemplateCodeView::OnCopyToClipboard()
{
	FPlatformApplicationMisc::ClipboardCopy(*HlslAsString);
}

#undef LOCTEXT_NAMESPACE
