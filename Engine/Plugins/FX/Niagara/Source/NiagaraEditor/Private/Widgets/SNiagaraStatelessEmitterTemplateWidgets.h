// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Templates/SharedPointer.h"

class FNiagaraStatelessEmitterTemplateViewModel;
class SGridPanel;
class ITextLayoutMarshaller;

/////////////////////////////////////////////////////////////////////////////////////////////////////////////// 

class SNiagaraStatelessEmitterTemplateModules : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStatelessEmitterTemplateModules) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraStatelessEmitterTemplateViewModel>, ViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	TSharedPtr<FNiagaraStatelessEmitterTemplateViewModel> ViewModel;
};

/////////////////////////////////////////////////////////////////////////////////////////////////////////////// 

class SNiagaraStatelessEmitterTemplateFeatures : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStatelessEmitterTemplateFeatures) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraStatelessEmitterTemplateViewModel>, ViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void OnRebuildWidget();

private:
	TSharedPtr<FNiagaraStatelessEmitterTemplateViewModel>	ViewModel;
	TSharedPtr<SGridPanel>									GridPanel;
};

/////////////////////////////////////////////////////////////////////////////////////////////////////////////// 

class SNiagaraStatelessEmitterTemplateOutputVariables : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStatelessEmitterTemplateOutputVariables) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraStatelessEmitterTemplateViewModel>, ViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void OnRebuildWidget();

private:
	TSharedPtr<FNiagaraStatelessEmitterTemplateViewModel>	ViewModel;
	TSharedPtr<SGridPanel>									GridPanel;
};

/////////////////////////////////////////////////////////////////////////////////////////////////////////////// 

class SNiagaraStatelessEmitterTemplateCodeView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraStatelessEmitterTemplateCodeView) {}
		SLATE_ARGUMENT(TSharedPtr<FNiagaraStatelessEmitterTemplateViewModel>, ViewModel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void OnRebuildWidget();

	FText GetHlslText() const;

	void OnCopyToClipboard();

private:
	TSharedPtr<FNiagaraStatelessEmitterTemplateViewModel>	ViewModel;
	TSharedPtr<ITextLayoutMarshaller>						SyntaxHighlighter;
	FString													HlslAsString;
	FText													HlslAsText;
};

