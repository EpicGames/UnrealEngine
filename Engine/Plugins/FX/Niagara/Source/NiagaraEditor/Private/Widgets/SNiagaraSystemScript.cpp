// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSystemScript.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraSystemScriptViewModel.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "ViewModels/NiagaraScriptInputCollectionViewModel.h"
#include "SNiagaraParameterCollection.h"
#include "Widgets/SNiagaraScriptGraph.h"

#include "Widgets/Layout/SSplitter.h"

void SNiagaraSystemScript::Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModel = InSystemViewModel;
	ChildSlot
	[
		SNew(SNiagaraScriptGraph, SystemViewModel->GetSystemScriptViewModel()->GetGraphViewModel())
	];
}
