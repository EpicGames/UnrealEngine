// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FNiagaraSystemViewModel;

/** A widget for editing the System script. */
class SNiagaraSystemScript : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSystemScript) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

private:
	/** The System view model that owns the System script being edited. */
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
};
