// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNiagaraSystemUserParameters.h"

#include "NiagaraConstants.h"
#include "NiagaraEditorData.h"
#include "NiagaraEditorUtilities.h"
#include "Customizations/NiagaraComponentDetails.h"
#include "Styling/StyleColors.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "Widgets/SNiagaraParameterMenu.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "NiagaraSystemUserParameters"

void SNiagaraSystemUserParameters::Construct(const FArguments& InArgs, TSharedPtr<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModel = InSystemViewModel;
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	TSharedRef<IDetailsView> ObjectDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	ObjectDetailsView->RegisterInstancedCustomPropertyLayout(UNiagaraSystem::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraSystemUserParameterDetails::MakeInstance));
	ObjectDetailsView->SetObject(&InSystemViewModel->GetSystem());
	
	ChildSlot
	[
		SNew(SBox)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("UserParameters")))
		[
			ObjectDetailsView
		]		
	];
}

#undef LOCTEXT_NAMESPACE
