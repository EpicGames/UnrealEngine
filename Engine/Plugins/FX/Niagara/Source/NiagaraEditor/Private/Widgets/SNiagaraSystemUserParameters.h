// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ViewModels/NiagaraParameterPanelViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SNiagaraParameterMenu.h"
#include "ViewModels/TNiagaraViewModelManager.h"

class SNiagaraSystemUserParameters : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SNiagaraSystemUserParameters ){}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TSharedPtr<FNiagaraSystemViewModel> InSystemViewModel);
	
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override {}

private:
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;
};
