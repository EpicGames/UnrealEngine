// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSystemViewport.h"

#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneMenus.h"
#include "AssetViewerSettings.h"
#include "CanvasItem.h"
#include "DrawDebugHelpers.h"
#include "EditorViewportCommands.h"
#include "EngineUtils.h"
#include "ImageUtils.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "NiagaraComponent.h"
#include "NiagaraEditorCommands.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraEffectType.h"
#include "NiagaraSystemMemReport.h"
#include "NiagaraPerfBaseline.h"
#include "NiagaraSettings.h"
#include "NiagaraShader.h"
#include "NiagaraSystemImpl.h"
#include "NiagaraSystemInstance.h"
#include "NiagaraSystemInstanceController.h"
#include "NiagaraSystemGpuComputeProxy.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraSystemEditorViewportToolbarSections.h"
#include "PreviewProfileController.h"
#include "UnrealEdGlobals.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Canvas.h"
#include "Engine/Font.h"
#include "Engine/TextureCube.h"
#include "Slate/SceneViewport.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ToolMenus.h"
#include "Toolkits/SystemToolkitModes/NiagaraSystemToolkitModeBase.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Components/LineBatchComponent.h"


#define LOCTEXT_NAMESPACE "SNiagaraSystemViewport"

class UNiagaraSystemEditorData;

namespace UE::NiagaraSystemEditor::Private
{
void ExtendCameraSubmenu(FName InCameraOptionsSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InCameraOptionsSubmenuName);

	Submenu->AddDynamicSection(
		"EditorCameraExtensionDynamicSection",
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InDynamicMenu)
			{
				FToolMenuSection& PositioningSection = InDynamicMenu->FindOrAddSection(
					"NavigationOptions", LOCTEXT("NavigationOptionsLabel", "Navigation Options")
				);

				PositioningSection.AddMenuEntry(FNiagaraEditorCommands::Get().ToggleOrbit);
			}
		)
	);
}

}


/** Viewport Client for the preview viewport */
class FNiagaraSystemViewportClient : public FEditorViewportClient, public TSharedFromThis<FNiagaraSystemViewportClient>
{
public:
	DECLARE_DELEGATE_OneParam(FOnScreenShotCaptured, UTexture2D*);

public:
	FNiagaraSystemViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SNiagaraSystemViewport>& InNiagaraEditorViewport, FOnScreenShotCaptured InOnScreenShotCaptured);
	virtual ~FNiagaraSystemViewportClient() override;

	void InitDelegates();
	
	// FEditorViewportClient interface
	virtual FLinearColor GetBackgroundColor() const override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(FViewport* Viewport,FCanvas* Canvas) override;
	virtual bool ShouldOrbitCamera() const override;
	virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 ViewIndex = INDEX_NONE) override;
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override { return false; }
	virtual bool CanCycleWidgetMode() const override { return false; }

	void FocusViewport(FBox FocusBounds, bool bInstant);

	void SetOrbitModeFromSettings();
	virtual void SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport) override;
	UNiagaraSystemEditorData* GetSystemEditorData() const;

	void DrawInstructionCounts(UNiagaraSystem* ParticleSystem, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight);
	void DrawParticleCounts(UNiagaraComponent* Component, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight);
	void DrawEmitterExecutionOrder(UNiagaraComponent* Component, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight);
	void DrawGpuTickInformation(UNiagaraComponent* Component, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight);
	void DrawMemoryInfo(UNiagaraComponent* Component, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight);
	void DrawStatelessInfo(UNiagaraComponent* Component, FViewport* InViewport, FCanvas* Canvas, UFont* Font, const float FontHeight);
	void SetUpdateViewportFocus(bool bUpdate) { bUpdateViewportFocus = bUpdate; }

	TWeakPtr<SNiagaraSystemViewport> NiagaraViewportPtr;
	bool bCaptureScreenShot;
	TWeakObjectPtr<UObject> ScreenShotOwner;

	FAdvancedPreviewScene* AdvancedPreviewScene = nullptr;

	FOnScreenShotCaptured OnScreenShotCaptured;

private:
	void SaveSharedViewportSettingsToConfig() const;
	void LoadSharedSettingsFromConfig();
	void SavePerAssetViewportSettings() const;
	void LoadPerAssetViewportSettings();

private:
	bool bUpdateViewportFocus = false;

	FNiagaraSystemMemReport	MemReport;
	double					MemReportLastCheckedTime = 0.0;
};

FNiagaraSystemViewportClient::FNiagaraSystemViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SNiagaraSystemViewport>& InNiagaraEditorViewport, FOnScreenShotCaptured InOnScreenShotCaptured)
	: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InNiagaraEditorViewport))
	, AdvancedPreviewScene(&InPreviewScene)
	, OnScreenShotCaptured(InOnScreenShotCaptured)
{
	NiagaraViewportPtr = InNiagaraEditorViewport;
	
	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(80,80,80);
	DrawHelper.GridColorMajor = FColor(72,72,72);
	DrawHelper.GridColorMinor = FColor(64,64,64);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
	ShowWidget(false);

	FEditorViewportClient::SetViewMode(VMI_Lit);

	EngineShowFlags.SetSnap(0);

	OverrideNearClipPlane(1.0f);

	bCaptureScreenShot = false;

	bDrawAxesGame = true;

	//This seems to be needed to get the correct world time in the preview.
	FNiagaraSystemViewportClient::SetIsSimulateInEditorViewport(true);

	LoadSharedSettingsFromConfig();
	LoadPerAssetViewportSettings();
}

FNiagaraSystemViewportClient::~FNiagaraSystemViewportClient()
{
	SaveSharedViewportSettingsToConfig();

	if(NiagaraViewportPtr.IsValid() && NiagaraViewportPtr.Pin()->GetSystemViewModel().IsValid())
	{
		NiagaraViewportPtr.Pin()->GetSystemViewModel().Pin()->OnPreSave().RemoveAll(this);
	}
}

void FNiagaraSystemViewportClient::InitDelegates()
{
	if(NiagaraViewportPtr.IsValid() && NiagaraViewportPtr.Pin()->GetSystemViewModel().IsValid())
	{
		NiagaraViewportPtr.Pin()->GetSystemViewModel().Pin()->OnPreSave().AddSP(this, &FNiagaraSystemViewportClient::SavePerAssetViewportSettings);
	}
}

void FNiagaraSystemViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Tick the preview scene world.
	PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}

void FNiagaraSystemViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	TSharedPtr<SNiagaraSystemViewport> NiagaraViewport = NiagaraViewportPtr.Pin();
	UNiagaraSystem* ParticleSystem = NiagaraViewport.IsValid() ? NiagaraViewport->GetPreviewComponent()->GetAsset() : nullptr;
	UNiagaraComponent* Component = NiagaraViewport.IsValid() ? NiagaraViewport->GetPreviewComponent() : nullptr;

	FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = NiagaraViewport.IsValid() ? NiagaraViewport->GetPreviewComponent()->GetSystemInstanceController() : nullptr;
	if (Component && SystemInstanceController.IsValid() && SystemInstanceController->GetAge() > 0.0)
	{
		FBox Bounds = Component->Bounds.GetBox();
		if (bUpdateViewportFocus && Bounds.IsValid)
		{
			// Provide some minimum bounds for the instant snap
			Bounds += Bounds.GetCenter() - FVector(100.0);
			Bounds += Bounds.GetCenter() + FVector(100.0);

			const bool bCachedShouldOrbitCamera = ShouldOrbitCamera();
			FocusViewport(Bounds, true);
			ToggleOrbitCamera(bCachedShouldOrbitCamera);
			SetUpdateViewportFocus(false);
		}

		UWorld* World = Component->GetWorld();
		if (World && ParticleSystem && !ParticleSystem->bFixedBounds)
		{
			FNiagaraSystemInstance* SystemInstance = SystemInstanceController->GetSystemInstance_Unsafe();
			const TArray<FNiagaraEmitterHandle>& EmitterHandles = ParticleSystem->GetEmitterHandles();
			for (int32 i=0; i < EmitterHandles.Num(); ++i)
			{
				if (!EmitterHandles[i].GetIsEnabled() || !EmitterHandles[i].GetDebugShowBounds())
				{
					continue;
				}
				const FNiagaraEmitterInstance& EmitterInstance = SystemInstance->GetEmitters()[i].Get();
				const FTransform& Transform = Component->GetComponentTransform();
				const FBox EmitterBounds = EmitterInstance.GetBounds();
				DrawDebugBox(World, EmitterBounds.GetCenter() + Transform.GetLocation(), EmitterBounds.GetExtent(), Transform.GetRotation(), FColor::Green);
			}
		}
	}

	const bool bDrawBounds = NiagaraViewport.IsValid() && NiagaraViewport->GetDrawElement(SNiagaraSystemViewport::EDrawElements::Bounds);
	if (bDrawBounds)
	{
		EngineShowFlags.SetBounds(true);
		EngineShowFlags.Game = 1;
	}
	else
	{
		EngineShowFlags.SetBounds(false);
		EngineShowFlags.Game = 0;
	}

	if (ParticleSystem && Component)
	{
		UWorld* World = Component->GetWorld();

		//-TODO:Stateless: Temporary to draw some debug information
		for (const FNiagaraEmitterHandle& EmitterHandle : ParticleSystem->GetEmitterHandles())
		{
			UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandle.GetStatelessEmitter();
			if (!StatelessEmitter || EmitterHandle.GetEmitterMode() != ENiagaraEmitterMode::Stateless)
			{
				continue;
			}

			StatelessEmitter->DrawModuleDebug(World, Component->GetComponentTransform());
		}

		// Draw Streaming Bounds if enabled
		if (bDrawBounds)
		{
			const FBox StreamingBounds = ParticleSystem->GetInitialStreamingBounds();
			if (StreamingBounds.IsValid)
			{
				DrawDebugBox(
					World,
					StreamingBounds.GetCenter(),
					StreamingBounds.GetExtent(),
					FQuat::Identity,
					FColor::Green
				);
			}
		}
	}

	FEditorViewportClient::Draw(InViewport, Canvas);

	if (NiagaraViewport.IsValid() )
	{
		float CurrentX = 10.0f;
		float CurrentY = 50.0f;
		UFont* Font = GEngine->GetSmallFont();
		const float FontHeight = Font->GetMaxCharHeight() * 1.1f;

		if (NiagaraViewport->GetDrawElement(SNiagaraSystemViewport::EDrawElements::EmitterExecutionOrder) && Component)
		{
			DrawEmitterExecutionOrder(Component, Canvas, CurrentX, CurrentY, Font, FontHeight);
			CurrentY += FontHeight;
		}
		if (NiagaraViewport->GetDrawElement(SNiagaraSystemViewport::EDrawElements::GpuTickInformation) && Component)
		{
			DrawGpuTickInformation(Component, Canvas, CurrentX, CurrentY, Font, FontHeight);
			CurrentY += FontHeight;
		}
		if (NiagaraViewport->GetDrawElement(SNiagaraSystemViewport::EDrawElements::InstructionCounts) && ParticleSystem)
		{
			DrawInstructionCounts(ParticleSystem, Canvas, CurrentX, CurrentY, Font, FontHeight);
			CurrentY += FontHeight;
		}
		if (NiagaraViewport->GetDrawElement(SNiagaraSystemViewport::EDrawElements::MemoryInfo) && Component)
		{
			DrawMemoryInfo(Component, Canvas, CurrentX, CurrentY, Font, FontHeight);
			CurrentY += FontHeight;
		}
		if (NiagaraViewport->GetDrawElement(SNiagaraSystemViewport::EDrawElements::ParticleCounts) && Component)
		{
			DrawParticleCounts(Component, Canvas, CurrentX, CurrentY, Font, FontHeight);
			CurrentY += FontHeight;
		}
		if (NiagaraViewport->GetDrawElement(SNiagaraSystemViewport::EDrawElements::StatelessInfo) && Component)
		{
			UFont* TinyFont = GEngine->GetTinyFont();
			DrawStatelessInfo(Component, InViewport, Canvas, TinyFont, TinyFont->GetMaxCharHeight() * 1.1f);
		}
		if (NiagaraViewport->GetDrawElement(SNiagaraSystemViewport::EDrawElements::OriginAxis) && Component)
		{
			UWorld* World = Component->GetWorld();
			ULineBatchComponent* LineBatcher = World->GetLineBatcher(UWorld::ELineBatcherType::World);

			const FVector XAxis(1, 0, 0);
			const FVector YAxis(0, 1, 0);
			const FVector ZAxis(0, 0, 1);

			FMatrix ArrowMatrix = FMatrix(XAxis, YAxis, ZAxis, FVector::ZeroVector);
			LineBatcher->DrawDirectionalArrow(ArrowMatrix, FColorList::Red, 50.f, 4.0f, SDPG_World);

			ArrowMatrix = FMatrix(YAxis, ZAxis, XAxis, FVector::ZeroVector);
			LineBatcher->DrawDirectionalArrow(ArrowMatrix, FColorList::Green, 50.f, 4.0f, SDPG_World);

			ArrowMatrix = FMatrix(ZAxis, XAxis, YAxis, FVector::ZeroVector);
			LineBatcher->DrawDirectionalArrow(ArrowMatrix, FColorList::Blue, 50.f, 4.0f, SDPG_World);
		}
	}

	if (bCaptureScreenShot && ScreenShotOwner.IsValid() && OnScreenShotCaptured.IsBound())
	{
		int32 SrcWidth = InViewport->GetSizeXY().X;
		int32 SrcHeight = InViewport->GetSizeXY().Y;
		// Read the contents of the viewport into an array.
		TArray<FColor> OrigBitmap;
		if (InViewport->ReadPixels(OrigBitmap))
		{
			check(OrigBitmap.Num() == SrcWidth * SrcHeight);

			// Resize image to enforce max size.
			TArray<FColor> ScaledBitmap;
			int32 ScaledWidth = 512;
			int32 ScaledHeight = 512;
			FImageUtils::CropAndScaleImage(SrcWidth, SrcHeight, ScaledWidth, ScaledHeight, OrigBitmap, ScaledBitmap);

			// Compress.
			FCreateTexture2DParameters Params;
			Params.bDeferCompression = true;

			UTexture2D* ThumbnailImage = FImageUtils::CreateTexture2D(ScaledWidth, ScaledHeight, ScaledBitmap, ScreenShotOwner.Get(), TEXT("ThumbnailTexture"), RF_NoFlags, Params);

			OnScreenShotCaptured.Execute(ThumbnailImage);
			NiagaraViewport->GetPreviewComponent()->MarkRenderStateDirty();
		}

		bCaptureScreenShot = false;
		ScreenShotOwner.Reset();
	}
}

void FNiagaraSystemViewportClient::DrawInstructionCounts(UNiagaraSystem* ParticleSystem, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight)
{
	Canvas->DrawShadowedString(CurrentX, CurrentY, TEXT("Instruction Counts"), Font, FLinearColor::White);
	CurrentY += FontHeight;

	for (const FNiagaraEmitterHandle& EmitterHandle : ParticleSystem->GetEmitterHandles())
	{
		FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
		if (EmitterData == nullptr)
		{
			continue;
		}

		Canvas->DrawShadowedString(CurrentX + 10.0f, CurrentY, *FString::Printf(TEXT("Emitter %s"), *EmitterHandle.GetName().ToString()), Font, FLinearColor::White);
		CurrentY += FontHeight;

		TArray<UNiagaraScript*> EmitterScripts;
		EmitterData->GetScripts(EmitterScripts);

		for (UNiagaraScript* Script : EmitterScripts)
		{
			if (Script->GetUsage() == ENiagaraScriptUsage::ParticleGPUComputeScript)
			{
				FNiagaraShaderScript* ShaderScript = Script->GetRenderThreadScript();
				if (ShaderScript != nullptr && ShaderScript->GetBaseVMScript() != nullptr)
				{
					TConstArrayView<FSimulationStageMetaData> SimulationStageMetaData = ShaderScript->GetBaseVMScript()->GetSimulationStageMetaData();
					for (int32 iSimStageIndex = 0; iSimStageIndex < SimulationStageMetaData.Num(); ++iSimStageIndex)
					{
						FNiagaraShaderRef Shader = ShaderScript->GetShaderGameThread(iSimStageIndex);
						if (Shader.IsValid())
						{
							FColor DisplayColor = FColor(196, 196, 196);
							FString StageName = SimulationStageMetaData[iSimStageIndex].SimulationStageName.ToString();
							Canvas->DrawShadowedString(CurrentX + 20.0f, CurrentY, *FString::Printf(TEXT("GPU StageName(%s) Stage(%d) = %u"), *StageName, iSimStageIndex, Shader->GetNumInstructions()), Font, DisplayColor);
							CurrentY += FontHeight;
						}
					}
				}
			}
			else
			{
				const uint32 NumInstructions = Script->GetVMExecutableData().LastOpCount;
				if (NumInstructions > 0)
				{
					Canvas->DrawShadowedString(CurrentX + 20.0f, CurrentY, *FString::Printf(TEXT("%s = %u"), *Script->GetName(), NumInstructions), Font, FLinearColor::White);
					CurrentY += FontHeight;
				}
			}
		}
	}
}

void FNiagaraSystemViewportClient::DrawParticleCounts(UNiagaraComponent* Component, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight)
{
	// Show particle counts
	FNiagaraSystemInstanceControllerPtr SystemInstanceController = Component->GetSystemInstanceController();
	FNiagaraSystemInstance* SystemInstance = SystemInstanceController.IsValid() ? SystemInstanceController->GetSoloSystemInstance() : nullptr;
	if (SystemInstance)
	{
		FCanvasTextItem TextItem(FVector2D(CurrentX, CurrentY), FText::FromString(TEXT("Particle Counts")), Font, FLinearColor::White);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.Draw(Canvas);
		CurrentY += FontHeight;

		for (const FNiagaraEmitterInstanceRef& EmitterInstance : SystemInstance->GetEmitters())
		{
			const FNiagaraEmitterHandle& EmitterHandle = EmitterInstance->GetEmitterHandle();
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();

			const FName EmitterName = EmitterHandle.GetName();
			const int32 CurrentCount = EmitterInstance->GetNumParticles();
			const int32 MaxCount = EmitterData ? EmitterData->GetMaxParticleCountEstimate() : 0;
			const bool IsIsolated = EmitterHandle.IsIsolated();
			const bool IsEnabled = EmitterHandle.GetIsEnabled();
			const ENiagaraExecutionState ExecutionState = EmitterInstance->GetExecutionState();
			const FString EmitterExecutionString = UEnum::GetValueAsString(ExecutionState);
			const int32 EmitterExecutionStringValueIndex = EmitterExecutionString.Find(TEXT("::"));
			const TCHAR* EmitterExecutionText = EmitterExecutionStringValueIndex == INDEX_NONE ? *EmitterExecutionString : *EmitterExecutionString + EmitterExecutionStringValueIndex + 2;
			const TCHAR* EmitterMode = EmitterHandle.GetEmitterMode() == ENiagaraEmitterMode::Stateless ? TEXT("[Lightweight]") : TEXT("");

			TextItem.Text = FText::FromString(FString::Printf(TEXT("%i Current, %i Max (est.) - [%s] [%s] %s"), CurrentCount, MaxCount, *EmitterName.ToString(), EmitterExecutionText, EmitterMode));
			TextItem.Position = FVector2D(CurrentX, CurrentY);
			TextItem.bOutlined = IsIsolated;
			TextItem.OutlineColor = FLinearColor(0.7f, 0.0f, 0.0f);
			TextItem.SetColor(IsEnabled ? FLinearColor::White : FLinearColor::Gray);
			TextItem.Draw(Canvas);
			CurrentY += FontHeight;
		}
	}
	else if ( UNiagaraSystem* NiagaraSystem = Component->GetAsset() )
	{
		const bool bHasOutstandingCompilation = NiagaraSystem->HasOutstandingCompilationRequests();

		for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles())
		{
			FVersionedNiagaraEmitterData* EmitterData = EmitterHandle.GetEmitterData();
			if (EmitterData == nullptr)
			{
				continue;
			}

			// we assume that if the emitter isn't ready to run and we have compilations still oustanding that we're waiting on that
			const bool bEmitterCompiling = bHasOutstandingCompilation && !EmitterData->IsReadyToRun();
			if (!bEmitterCompiling)
			{
				continue;
			}

			FString CompilingText = FString::Printf(TEXT("%s - %s"), *FText::FromString("Compiling Emitter").ToString(), *EmitterHandle.GetUniqueInstanceName());
			Canvas->DrawShadowedString(CurrentX, CurrentY, *CompilingText, Font, FLinearColor::Yellow);
			CurrentY += FontHeight;
		}
	}
}

void FNiagaraSystemViewportClient::DrawEmitterExecutionOrder(UNiagaraComponent* Component, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight)
{
	UNiagaraSystem* NiagaraSystem = Component->GetAsset();
	if (NiagaraSystem == nullptr)
		return;

	Canvas->DrawShadowedString(CurrentX, CurrentY, TEXT("Emitter Execution Order"), Font, FLinearColor::White);
	CurrentY += FontHeight;

	if ( NiagaraSystem->IsReadyToRun() )
	{
		TConstArrayView<FNiagaraEmitterExecutionIndex> ExecutionOrder = NiagaraSystem->GetEmitterExecutionOrder();
		int32 DisplayIndex = 0;
		for (const FNiagaraEmitterExecutionIndex& EmitterExecIndex : ExecutionOrder)
		{
			const FNiagaraEmitterHandle& EmitterHandle = NiagaraSystem->GetEmitterHandle(EmitterExecIndex.EmitterIndex);
			if (FVersionedNiagaraEmitterData* NiagaraEmitter = EmitterHandle.GetEmitterData())
			{
				if (EmitterExecIndex.bStartNewOverlapGroup)
				{
					Canvas->DrawShadowedString(CurrentX, CurrentY, TEXT("<Overlap Barrier>"), Font, FLinearColor::White);
					CurrentY += FontHeight;
				}
				Canvas->DrawShadowedString(CurrentX, CurrentY, *FString::Printf(TEXT("%d - %s"), ++DisplayIndex, NiagaraEmitter->GetDebugSimName()), Font, FLinearColor::White);
				CurrentY += FontHeight;
			}
		}
	}
}

void FNiagaraSystemViewportClient::DrawGpuTickInformation(UNiagaraComponent* Component, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight)
{
	FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = Component->GetSystemInstanceController();
	if (SystemInstanceController.IsValid() == false)
	{
		return;
	}

	FNiagaraSystemInstance* SystemInstance = SystemInstanceController->GetSystemInstance_Unsafe();
	if (SystemInstance == nullptr)
	{
		return;
	}

	Canvas->DrawShadowedString(CurrentX, CurrentY, TEXT("Gpu Tick Information"), Font, FLinearColor::White);
	CurrentY += FontHeight;

	if (SystemInstance->IsReadyToRun())
	{
		if (const FNiagaraSystemGpuComputeProxy* ComputeProxy = SystemInstance->GetSystemGpuComputeProxy())
		{
			static UEnum* GpuComputeTickStageEnum = StaticEnum<ENiagaraGpuComputeTickStage::Type>();
			const ENiagaraGpuComputeTickStage::Type GpuTickStage = ComputeProxy->GetComputeTickStage();
			const FLinearColor TextColor = GpuTickStage == ENiagaraGpuComputeTickStage::First ? FLinearColor::White : FLinearColor::Yellow;
			Canvas->DrawShadowedString(CurrentX + 5.0f, CurrentY, *FString::Printf(TEXT("GpuTickStage = %s"), *GpuComputeTickStageEnum->GetNameStringByValue(GpuTickStage)), Font, TextColor);
			CurrentY += FontHeight;
		}
		else
		{
			Canvas->DrawShadowedString(CurrentX + 5.0f, CurrentY, TEXT("No GPU Emitters"), Font, FLinearColor::White);
			CurrentY += FontHeight;
		}
		if (SystemInstance->RequiresGlobalDistanceField())
		{
			Canvas->DrawShadowedString(CurrentX + 5.0f, CurrentY, TEXT("RequiresGlobalDistanceField"), Font, FLinearColor::White);
			CurrentY += FontHeight;
		}
		if (SystemInstance->RequiresDepthBuffer())
		{
			Canvas->DrawShadowedString(CurrentX + 5.0f, CurrentY, TEXT("RequiresDepthBuffer"), Font, FLinearColor::White);
			CurrentY += FontHeight;
		}
		if (SystemInstance->RequiresEarlyViewData())
		{
			Canvas->DrawShadowedString(CurrentX + 5.0f, CurrentY, TEXT("RequiresEarlyViewData"), Font, FLinearColor::White);
			CurrentY += FontHeight;
		}
		if (SystemInstance->RequiresViewUniformBuffer())
		{
			Canvas->DrawShadowedString(CurrentX + 5.0f, CurrentY, TEXT("RequiresViewUniformBuffer"), Font, FLinearColor::White);
			CurrentY += FontHeight;
		}
		if (SystemInstance->RequiresRayTracingScene())
		{
			Canvas->DrawShadowedString(CurrentX + 5.0f, CurrentY, TEXT("RequiresRayTracingScene"), Font, FLinearColor::White);
			CurrentY += FontHeight;
		}
		if (SystemInstance->RequiresCurrentFrameNDC())
		{
			Canvas->DrawShadowedString(CurrentX + 5.0f, CurrentY, TEXT("RequiresCurrentFrameNDC"), Font, FLinearColor::White);
			CurrentY += FontHeight;
		}
	}
}

void FNiagaraSystemViewportClient::DrawMemoryInfo(UNiagaraComponent* Component, FCanvas* Canvas, float& CurrentX, float& CurrentY, UFont* Font, const float FontHeight)
{
	// Update costs
	const double CurrentTime = FPlatformTime::Seconds();
	const double RefreshTime = 2.0;
	if ((CurrentTime - MemReportLastCheckedTime) > RefreshTime)
	{
		MemReportLastCheckedTime = CurrentTime;
		MemReport.GenerateReport(FNiagaraSystemMemReport::EReportType::Basic, Component->GetAsset());
	}

	FResourceSizeEx ComponentResSize = FResourceSizeEx(EResourceSizeMode::EstimatedTotal);
	Component->GetResourceSizeEx(ComponentResSize);
	const uint64 MemComponentCost = ComponentResSize.GetTotalMemoryBytes();

	// Title
	Canvas->DrawShadowedString(CurrentX, CurrentY, TEXT("Memory Info (Approximate)"), Font, FLinearColor::White);
	CurrentY += FontHeight;
	CurrentX += 8.0f;

	// System Mem Information
	for (const FNiagaraSystemMemReport::FNode& Node : MemReport.GetNodes())
	{
		Canvas->DrawShadowedString(
			CurrentX + (Node.Depth * 8.0f), CurrentY,
			*FString::Printf(TEXT("- %s %s = %skb (%s bytes)"), *Node.ObjectClass.ToString(), *Node.ObjectName.ToString(), *FString::FormatAsNumber(FMath::DivideAndRoundUp(uint32(Node.InclusiveSizeBytes), 1024u)), *FString::FormatAsNumber(Node.InclusiveSizeBytes)),
			Font, FLinearColor::White
		);
		CurrentY += FontHeight;
	}

	// Data Interface Mem
	const uint32 DIMemory = uint32(MemReport.GetDataInterfaceSizeBytes());
	Canvas->DrawShadowedString(
		CurrentX, CurrentY,
		*FString::Printf(TEXT("- DataInterface = %skb (%s bytes) - Included In NiagaraSystem Mem"), *FString::FormatAsNumber(FMath::DivideAndRoundUp(DIMemory, 1024u)), *FString::FormatAsNumber(DIMemory)),
		Font, FLinearColor::White
	);
	CurrentY += FontHeight;

	// Component Mem Information
	Canvas->DrawShadowedString(
		CurrentX, CurrentY,
		*FString::Printf(TEXT("- Component = %skb (%s bytes)"), *FString::FormatAsNumber(FMath::DivideAndRoundUp(uint32(MemComponentCost), 1024u)), *FString::FormatAsNumber(uint32(MemComponentCost))),
		Font, FLinearColor::White
	);
	CurrentY += FontHeight;
}

void FNiagaraSystemViewportClient::DrawStatelessInfo(UNiagaraComponent* Component, FViewport* InViewport, FCanvas* Canvas, UFont* Font, const float FontHeight)
{
	UNiagaraSystem* NiagaraSystem = Component->GetAsset();
	if (!NiagaraSystem)
	{
		return;
	}

	int32 NumStatelessEmitters = 0;
	int32 NumEmitters = 0;

	for (const FNiagaraEmitterHandle& EmitterHandle : NiagaraSystem->GetEmitterHandles())
	{
		if (EmitterHandle.GetIsEnabled())
		{
			++NumEmitters;
			UNiagaraStatelessEmitter* StatelessEmitter = EmitterHandle.GetEmitterMode() == ENiagaraEmitterMode::Stateless ? EmitterHandle.GetStatelessEmitter() : nullptr;
			if (StatelessEmitter)
			{
				++NumStatelessEmitters;
			}
		}
	}

	if (NumStatelessEmitters > 0)
	{
		const FVector2D ScaledViewportSize = FVector2D(InViewport->GetSizeXY()) / Canvas->GetDPIScale();
		FCanvasTextItem TextItem(FVector2D::ZeroVector, FText::GetEmpty(), Font, FLinearColor::White);

		FString StatelessInfo = FString::Printf(TEXT("Stateless: %d / %d Emitters "), NumStatelessEmitters, NumEmitters);

		if (const TCHAR* SystemStateModeString = NiagaraSystem->GetSystemStateModeString())
		{
			StatelessInfo.Append(SystemStateModeString);
		}

		int32 Width = 0;
		int32 Height = 0;
		UCanvas::ClippedStrLen(Font, 1.0f, 1.0f, Width, Height, *StatelessInfo);

		TextItem.Text = FText::FromString(StatelessInfo);
		TextItem.EnableShadow(FLinearColor::Black);
		TextItem.Draw(Canvas, ScaledViewportSize.X - 5 - float(Width), ScaledViewportSize.Y - 5 - float(Height));
	}
}

void FNiagaraSystemViewportClient::SetOrbitModeFromSettings()
{
	const UNiagaraSettings* Settings = GetDefault<UNiagaraSettings>();
	check(Settings);

	UNiagaraSystemEditorData* EditorData = GetSystemEditorData();
	if (EditorData)
	{
		bUsingOrbitCamera = EditorData->GetAssetViewportSettings().bUseOrbitMode;
	}
	else
	{
		bUsingOrbitCamera = Settings->bSystemViewportInOrbitMode;
	}
}

bool FNiagaraSystemViewportClient::ShouldOrbitCamera() const
{
	return bUsingOrbitCamera;
}


FLinearColor FNiagaraSystemViewportClient::GetBackgroundColor() const
{
	if (AdvancedPreviewScene != nullptr)
	{
		return AdvancedPreviewScene->GetBackgroundColor();
	}

	FLinearColor BackgroundColor = FLinearColor::Black;
	return BackgroundColor;
}

void FNiagaraSystemViewportClient::FocusViewport(FBox FocusBounds, bool bInstant)
{
	if (FocusBounds.IsValid == false)
	{
		return;
	}

	UNiagaraSystemEditorData* SystemEditorData = GetSystemEditorData();
	if (SystemEditorData && SystemEditorData->GetViewportManualFocusDistanceEnabled())
	{
		const float Distance = SystemEditorData->GetViewportManualFocusDistance();
		const FVector HDistance = FVector(Distance * 0.5f);
		FocusBounds = FBox(FocusBounds.GetCenter() - HDistance, FocusBounds.GetCenter() + HDistance);
	}
	FocusViewportOnBox(FocusBounds, bInstant);
}

FSceneView* FNiagaraSystemViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	FSceneView* SceneView = FEditorViewportClient::CalcSceneView(ViewFamily);
	FFinalPostProcessSettings::FCubemapEntry& CubemapEntry = *new(SceneView->FinalPostProcessSettings.ContributingCubemaps) FFinalPostProcessSettings::FCubemapEntry;
	CubemapEntry.AmbientCubemap = GUnrealEd->GetThumbnailManager()->AmbientCubemap;
	CubemapEntry.AmbientCubemapTintMulScaleValue = FLinearColor::White;
	return SceneView;
}

void FNiagaraSystemViewportClient::SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport)
{
	bIsSimulateInEditorViewport = bInIsSimulateInEditorViewport;

// 	if (bInIsSimulateInEditorViewport)
// 	{
// 		TSharedRef<FPhysicsManipulationEdModeFactory> Factory = MakeShareable(new FPhysicsManipulationEdModeFactory);
// 		FEditorModeRegistry::Get().RegisterMode(FBuiltinEditorModes::EM_Physics, Factory);
// 	}
// 	else
// 	{
// 		FEditorModeRegistry::Get().UnregisterMode(FBuiltinEditorModes::EM_Physics);
// 	}
}

UNiagaraSystemEditorData* FNiagaraSystemViewportClient::GetSystemEditorData() const
{
	TSharedPtr<SNiagaraSystemViewport> NiagaraViewport = NiagaraViewportPtr.Pin();
	if (NiagaraViewport.IsValid() && NiagaraViewport->GetPreviewComponent() && NiagaraViewport->GetPreviewComponent()->GetAsset())
	{
		return Cast<UNiagaraSystemEditorData>(NiagaraViewport->GetPreviewComponent()->GetAsset()->GetEditorData());
	}
	return nullptr;
}

void FNiagaraSystemViewportClient::SaveSharedViewportSettingsToConfig() const
{
	if(GUnrealEd && GetDefault<UNiagaraEditorSettings>())
	{
		const FEngineShowFlags& EditorShowFlagsToSave = IsInGameView() ? LastEngineShowFlags : EngineShowFlags;
		const FEngineShowFlags& GameShowFlagsToSave = IsInGameView() ? EngineShowFlags : LastEngineShowFlags;
		
		FNiagaraViewportSharedSettings ViewportSharedSettings = GetDefault<UNiagaraEditorSettings>()->GetViewportSharedSettings();
		ViewportSharedSettings.ViewportType = ViewportType;
		ViewportSharedSettings.PerspViewModeIndex = GetPerspViewMode();
		ViewportSharedSettings.OrthoViewModeIndex = GetOrthoViewMode();
		ViewportSharedSettings.EditorShowFlagsString = EditorShowFlagsToSave.ToString();
		ViewportSharedSettings.GameShowFlagsString = GameShowFlagsToSave.ToString();
		ViewportSharedSettings.ExposureSettings = ExposureSettings;
		ViewportSharedSettings.FOVAngle = FOVAngle;
		ViewportSharedSettings.bShowOnScreenStats = ShouldShowStats();
		SaveRealtimeStateToConfig(ViewportSharedSettings.bIsRealtime);
		
		GetMutableDefault<UNiagaraEditorSettings>()->SetViewportSharedSettings(ViewportSharedSettings);
	}
}

void FNiagaraSystemViewportClient::LoadSharedSettingsFromConfig()
{
	const FNiagaraViewportSharedSettings& ViewportSharedSettings = GetDefault<UNiagaraEditorSettings>()->GetViewportSharedSettings();

	FEngineShowFlags EditorShowFlags(ESFIM_Editor);
	FEngineShowFlags GameShowFlags(ESFIM_Game);

	EditorShowFlags.DisableAdvancedFeatures();

	if(!ViewportSharedSettings.EditorShowFlagsString.IsEmpty())
	{
		EditorShowFlags.SetFromString(*ViewportSharedSettings.EditorShowFlagsString);
	}

	if(!ViewportSharedSettings.GameShowFlagsString.IsEmpty())
	{
		GameShowFlags.SetFromString(*ViewportSharedSettings.GameShowFlagsString);
	}

	if(ViewportSharedSettings.ViewportType == LVT_Perspective)
	{
		ApplyViewMode(ViewportSharedSettings.PerspViewModeIndex, true, EditorShowFlags);
		ApplyViewMode(ViewportSharedSettings.PerspViewModeIndex, true, GameShowFlags);
	}
	else
	{
		ApplyViewMode(ViewportSharedSettings.OrthoViewModeIndex, false, EditorShowFlags);
		ApplyViewMode(ViewportSharedSettings.OrthoViewModeIndex, false, GameShowFlags);
	}

	// Disabling some features for orthographic views. 
	if(ViewportSharedSettings.ViewportType != LVT_Perspective)
	{
		EditorShowFlags.MotionBlur = 0;
		EditorShowFlags.Fog = 0;
		EditorShowFlags.SetDepthOfField(false);
		GameShowFlags.MotionBlur = 0;
		GameShowFlags.Fog = 0;
		GameShowFlags.SetDepthOfField(false);
	}
	
	ViewportType = ViewportSharedSettings.ViewportType;
	EngineShowFlags = EditorShowFlags;
	LastEngineShowFlags = GameShowFlags;
	ExposureSettings = ViewportSharedSettings.ExposureSettings;
	SetRealtime(ViewportSharedSettings.bIsRealtime);
	SetShowStats(ViewportSharedSettings.bShowOnScreenStats);
	ViewFOV = FOVAngle = ViewportSharedSettings.FOVAngle;
	SetViewModes(ViewportSharedSettings.PerspViewModeIndex, ViewportSharedSettings.OrthoViewModeIndex );
}

void FNiagaraSystemViewportClient::SavePerAssetViewportSettings() const
{
	if(UNiagaraSystemEditorData* SystemEditorData = GetSystemEditorData())
	{
		FNiagaraPerAssetViewportSettings CurrentAssetSettings = SystemEditorData->GetAssetViewportSettings();
		CurrentAssetSettings.ViewLocation = GetViewLocation();
		CurrentAssetSettings.ViewRotation = GetViewRotation();
		SystemEditorData->SetAssetViewportSettings(CurrentAssetSettings);
	}
}

void FNiagaraSystemViewportClient::LoadPerAssetViewportSettings()
{
	SetOrbitModeFromSettings();
	
	const FNiagaraPerAssetViewportSettings& AssetViewportSettings = NiagaraViewportPtr.Pin()->GetSystemViewModel().Pin()->GetEditorData().GetAssetViewportSettings();
	SetViewLocation(AssetViewportSettings.ViewLocation);
	SetViewRotation(AssetViewportSettings.ViewRotation);
}

//////////////////////////////////////////////////////////////////////////

void SNiagaraSystemViewport::Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel)
{
	SystemViewModel = InSystemViewModel;
	
	const UNiagaraEditorSettings* Settings = GetDefault<UNiagaraEditorSettings>();
	
	DrawFlags = 0;
	DrawFlags |= Settings->IsShowParticleCountsInViewport() ? EDrawElements::ParticleCounts : 0;
	DrawFlags |= Settings->IsShowInstructionsCount() ? EDrawElements::InstructionCounts : 0;
	DrawFlags |= Settings->IsShowEmitterExecutionOrder() ? EDrawElements::EmitterExecutionOrder : 0;
	DrawFlags |= Settings->IsShowGpuTickInformation() ? EDrawElements::GpuTickInformation : 0;
	DrawFlags |= Settings->IsShowMemoryInfo() ? EDrawElements::MemoryInfo : 0;
	DrawFlags |= Settings->IsShowStatelessInfo() ? EDrawElements::StatelessInfo : 0;

	bShowBackground = false;
	PreviewComponent = nullptr;
	AdvancedPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));
	AdvancedPreviewScene->SetFloorVisibility(false);

	float Pitch = -40.0;
	float Yaw = 128.0;
	float Roll = 0.0;
	AdvancedPreviewScene->SetLightDirection(FRotator(Pitch, Yaw, Roll));

	OnThumbnailCapturedDelegate = InArgs._OnThumbnailCaptured;
	Sequencer = InArgs._Sequencer;
	AssetEditorToolkit = InArgs._AssetEditorToolkit;

	SEditorViewport::Construct( SEditorViewport::FArguments() );

	Client->EngineShowFlags.SetGrid(Settings->IsShowGridInViewport());

	// Register for preview platform changes
	if (UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine))
	{
		OnPreviewFeatureLevelChangedHandle = EditorEngine->OnPreviewShaderPlatformChanged().AddLambda(
			[WeakWorld=TWeakObjectPtr<UWorld>(Client->GetWorld())](EShaderPlatform NewShaderPlatform)
			{
				if (UWorld* World = WeakWorld.Get())
				{
					World->ShaderPlatformChanged(NewShaderPlatform);
				}
			}
		);
	}
	
	UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(AdvancedPreviewScene, Client);
}

SNiagaraSystemViewport::~SNiagaraSystemViewport()
{
	if (OnPreviewFeatureLevelChangedHandle.IsValid())
	{
		if (UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine))
		{
			EditorEngine->OnPreviewShaderPlatformChanged().Remove(OnPreviewFeatureLevelChangedHandle);
		}
	}
}

void SNiagaraSystemViewport::CreateThumbnail(UObject* InScreenShotOwner, TOptional<FGuid> InEmitterToCaptureThumbnailFor)
{
	if (SystemViewportClient.IsValid() && PreviewComponent != nullptr)
	{
		PreviewComponent->MarkRenderStateDirty();

		// If we want to capture the thumbnail for a specific emitter in a system, we make sure isolation state is correctly handled
		if(InEmitterToCaptureThumbnailFor.IsSet() && SystemViewModel.IsValid() && SystemViewModel.Pin()->GetEditMode() == ENiagaraSystemViewModelEditMode::SystemAsset)
		{
			if(TSharedPtr<FNiagaraEmitterHandleViewModel> EditableEmitterHandleViewModel = SystemViewModel.Pin()->GetEmitterHandleViewModelById(InEmitterToCaptureThumbnailFor.GetValue()))
			{
				SystemViewModel.Pin()->CacheIsolatedEmitterState();
				EditableEmitterHandleViewModel->SetIsIsolated(true);
			}
		}
		
		EmitterToCaptureThumbnailFor = InEmitterToCaptureThumbnailFor;
		SystemViewportClient->bCaptureScreenShot = true;
		SystemViewportClient->ScreenShotOwner = InScreenShotOwner;
	}
}


void SNiagaraSystemViewport::AddReferencedObjects( FReferenceCollector& Collector )
{
	if (PreviewComponent != nullptr)
	{
		Collector.AddReferencedObject(PreviewComponent);
	}
}


bool SNiagaraSystemViewport::GetDrawElement(EDrawElements Element) const
{
	return (DrawFlags & Element) != 0;
}

void SNiagaraSystemViewport::ToggleDrawElement(EDrawElements Element)
{
	DrawFlags = DrawFlags ^ Element;
}

bool SNiagaraSystemViewport::IsToggleOrbitChecked() const
{
	return SystemViewportClient->bUsingOrbitCamera;
}

void SNiagaraSystemViewport::ToggleOrbit()
{
	bool bNewOrbitSetting = !SystemViewportClient->bUsingOrbitCamera;
	SystemViewportClient->ToggleOrbitCamera(bNewOrbitSetting);

	if (PreviewComponent && PreviewComponent->GetAsset())
	{
		UNiagaraSystemEditorData* EditorData = Cast<UNiagaraSystemEditorData>(PreviewComponent->GetAsset()->GetEditorData());
		if (EditorData)
		{
			EditorData->SetUseOrbitMode(bNewOrbitSetting);
		}
	}
}

void SNiagaraSystemViewport::ToggleMotion()
{
	bMotionEnabled = !bMotionEnabled;
	if (PreviewComponent && bMotionEnabled == false)
	{
		PreviewComponent->SetComponentToWorld(FTransform::Identity);
	}
}

bool SNiagaraSystemViewport::IsMotionEnabled() const
{
	return bMotionEnabled;
}

bool SNiagaraSystemViewport::IsFloorEnabled() const
{
	return AdvancedPreviewScene ? AdvancedPreviewScene->IsFloorEnabled() : false;
}

void SNiagaraSystemViewport::ToggleFloorEnabled()
{
	if ( AdvancedPreviewScene )
	{
		AdvancedPreviewScene->SetFloorVisibility(!AdvancedPreviewScene->IsFloorEnabled());
	}
}

float SNiagaraSystemViewport::GetFloorOffset() const
{
	return AdvancedPreviewScene ? AdvancedPreviewScene->GetFloorOffset() : 0.0f;
}

void SNiagaraSystemViewport::SetFloorOffset(float Offset)
{
	if (AdvancedPreviewScene)
	{
		AdvancedPreviewScene->SetFloorOffset(Offset);
		RefreshViewport();
	}
}

bool SNiagaraSystemViewport::IsManualFocusDistanceEnabled() const
{
	UNiagaraSystemEditorData* SystemEditorData = SystemViewportClient->GetSystemEditorData();
	return SystemEditorData ? SystemEditorData->GetViewportManualFocusDistanceEnabled() : false;
}

void SNiagaraSystemViewport::ToggleManualFocusDistance()
{
	if ( UNiagaraSystemEditorData* SystemEditorData = SystemViewportClient->GetSystemEditorData() )
	{
		SystemEditorData->SetViewportManualFocusDistanceEnabled(!SystemEditorData->GetViewportManualFocusDistanceEnabled());
	}
}

void SNiagaraSystemViewport::SetManualFocusDistance(float Distance)
{
	if (UNiagaraSystemEditorData* SystemEditorData = SystemViewportClient->GetSystemEditorData())
	{
		SystemEditorData->SetViewportManualFocusDistance(Distance);
	}
}

float SNiagaraSystemViewport::GetManualFocusDistance() const
{
	UNiagaraSystemEditorData* SystemEditorData = SystemViewportClient->GetSystemEditorData();
	return SystemEditorData ? SystemEditorData->GetViewportManualFocusDistance() : 0.0f;
}

void SNiagaraSystemViewport::RefreshViewport()
{
	//reregister the preview components, so if the preview material changed it will be propagated to the render thread
	PreviewComponent->MarkRenderStateDirty();
	SceneViewport->InvalidateDisplay();
}

void SNiagaraSystemViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SEditorViewport::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );

	// this marks the end of the transition, so we restore orbit mode if needed
	if(bIsViewTransitioning && !SystemViewportClient->GetViewTransform().IsPlaying())
	{
		if(bShouldActivateOrbitAfterTransitioning)
		{
			SystemViewportClient->ToggleOrbitCamera(true);
		}

		bShouldActivateOrbitAfterTransitioning = false;
		bIsViewTransitioning = false;
	}

	if (PreviewComponent && bMotionEnabled)
	{
		const float MotionTime = PreviewComponent->GetDesiredAge();

		FVector Location;
		Location.X = MotionRadius * FMath::Sin(FMath::DegreesToRadians(MotionRate) * MotionTime);
		Location.Y = 0.0f;
		Location.Z = MotionRadius * FMath::Cos(FMath::DegreesToRadians(MotionRate) * MotionTime);
		PreviewComponent->SetComponentToWorld(FTransform(Location));
	}
}

void SNiagaraSystemViewport::SetPreviewComponent(UNiagaraComponent* NiagaraComponent)
{
	if (PreviewComponent != nullptr)
	{
		AdvancedPreviewScene->RemoveComponent(PreviewComponent);
		PreviewComponent->SetGpuComputeDebug(false);
	}
	PreviewComponent = NiagaraComponent;

	if (PreviewComponent != nullptr)
	{
		PreviewComponent->SetGpuComputeDebug(true);
		AdvancedPreviewScene->AddComponent(PreviewComponent, PreviewComponent->GetRelativeTransform());

		SystemViewportClient->SetUpdateViewportFocus(true);
	}

	SystemViewportClient->SetOrbitModeFromSettings();
}


void SNiagaraSystemViewport::ToggleRealtime()
{
	SystemViewportClient->ToggleRealtime();
}

/*
TSharedRef<FUICommandList> SNiagaraSystemViewport::GetSystemEditorCommands() const
{
	check(SystemEditorPtr.IsValid());
	return SystemEditorPtr.GetToolkitCommands();
}
*/

void SNiagaraSystemViewport::OnAddedToTab( const TSharedRef<SDockTab>& OwnerTab )
{
	ParentTab = OwnerTab;
}

bool SNiagaraSystemViewport::IsVisible() const
{
	return ViewportWidget.IsValid() && (!ParentTab.IsValid() || ParentTab.Pin()->IsForeground()) && SEditorViewport::IsVisible() ;
}

void SNiagaraSystemViewport::OnScreenShotCaptured(UTexture2D* ScreenShot)
{
	OnThumbnailCapturedDelegate.ExecuteIfBound(ScreenShot, EmitterToCaptureThumbnailFor);
	EmitterToCaptureThumbnailFor.Reset();
}

void SNiagaraSystemViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	// Unbind the CycleTransformGizmos since niagara currently doesn't use the gizmos and it prevents resetting the system with
	// spacebar when the viewport is focused.
	CommandList->UnmapAction(FEditorViewportCommands::Get().CycleTransformGizmos);

	const FNiagaraEditorCommands& Commands = FNiagaraEditorCommands::Get();

	// Add the commands to the toolkit command list so that the toolbar buttons can find them

	CommandList->MapAction(
		Commands.TogglePreviewGrid,
		FExecuteAction::CreateSP( this, &SNiagaraSystemViewport::TogglePreviewGrid ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SNiagaraSystemViewport::IsTogglePreviewGridChecked )
	);

	CommandList->MapAction(
		Commands.ToggleOriginAxis,		
		FExecuteAction::CreateLambda([Viewport = this]()
		{
			Viewport->ToggleDrawElement(EDrawElements::OriginAxis);
			Viewport->RefreshViewport();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Viewport = this]() -> bool { return Viewport->GetDrawElement(EDrawElements::OriginAxis); })
	);

	CommandList->MapAction(
		Commands.ToggleInstructionCounts,
		FExecuteAction::CreateLambda([Viewport=this]()
		{
			Viewport->ToggleDrawElement(EDrawElements::InstructionCounts);
			GetMutableDefault<UNiagaraEditorSettings>()->SetShowInstructionsCount(Viewport->GetDrawElement(EDrawElements::InstructionCounts));
			Viewport->RefreshViewport();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Viewport=this]() -> bool { return Viewport->GetDrawElement(EDrawElements::InstructionCounts); })
	);

	CommandList->MapAction(
		Commands.ToggleParticleCounts,
		FExecuteAction::CreateLambda([Viewport = this]()
		{
			Viewport->ToggleDrawElement(EDrawElements::ParticleCounts);
			GetMutableDefault<UNiagaraEditorSettings>()->SetShowParticleCountsInViewport(Viewport->GetDrawElement(EDrawElements::ParticleCounts));
			Viewport->RefreshViewport();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Viewport = this]() -> bool { return Viewport->GetDrawElement(EDrawElements::ParticleCounts); })
	);

	CommandList->MapAction(
		Commands.ToggleEmitterExecutionOrder,
		FExecuteAction::CreateLambda([Viewport = this]()
		{
			Viewport->ToggleDrawElement(EDrawElements::EmitterExecutionOrder);
			GetMutableDefault<UNiagaraEditorSettings>()->SetShowEmitterExecutionOrder(Viewport->GetDrawElement(EDrawElements::EmitterExecutionOrder));
			Viewport->RefreshViewport();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Viewport = this]() -> bool { return Viewport->GetDrawElement(EDrawElements::EmitterExecutionOrder); })
	);

	CommandList->MapAction(
		Commands.ToggleGpuTickInformation,
		FExecuteAction::CreateLambda([Viewport = this]()
		{
			Viewport->ToggleDrawElement(EDrawElements::GpuTickInformation);
			GetMutableDefault<UNiagaraEditorSettings>()->SetShowGpuTickInformation(Viewport->GetDrawElement(EDrawElements::GpuTickInformation));
			Viewport->RefreshViewport();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Viewport = this]() -> bool { return Viewport->GetDrawElement(EDrawElements::GpuTickInformation); })
	);

	CommandList->MapAction(
		Commands.ToggleMemoryInfo,
		FExecuteAction::CreateLambda([Viewport = this]()
		{
			Viewport->ToggleDrawElement(EDrawElements::MemoryInfo);
			GetMutableDefault<UNiagaraEditorSettings>()->SetShowMemoryInfo(Viewport->GetDrawElement(EDrawElements::MemoryInfo));
			Viewport->RefreshViewport();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Viewport = this]() -> bool { return Viewport->GetDrawElement(EDrawElements::MemoryInfo); })
	);

	CommandList->MapAction(
		Commands.ToggleStatelessInfo,
		FExecuteAction::CreateLambda([Viewport = this]()
		{
			Viewport->ToggleDrawElement(EDrawElements::StatelessInfo);
			GetMutableDefault<UNiagaraEditorSettings>()->SetShowStatelessInfo(Viewport->GetDrawElement(EDrawElements::StatelessInfo));
			Viewport->RefreshViewport();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Viewport = this]() -> bool { return Viewport->GetDrawElement(EDrawElements::StatelessInfo); })
	);

	CommandList->MapAction(
		Commands.TogglePreviewBackground,
		FExecuteAction::CreateSP( this, &SNiagaraSystemViewport::TogglePreviewBackground ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SNiagaraSystemViewport::IsTogglePreviewBackgroundChecked )
	);

	CommandList->MapAction(
		Commands.ToggleOrbit,
		FExecuteAction::CreateSP(this, &SNiagaraSystemViewport::ToggleOrbit),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SNiagaraSystemViewport::IsToggleOrbitChecked));

	CommandList->MapAction(
		Commands.ToggleMotion,
		FExecuteAction::CreateSP(this, &SNiagaraSystemViewport::ToggleMotion),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SNiagaraSystemViewport::IsMotionEnabled));
}

void SNiagaraSystemViewport::OnFocusViewportToSelection()
{
	if( PreviewComponent )
	{
		// FocusViewportOnBox disables orbit, so remember our state
		const bool bIsOrbit = SystemViewportClient->ShouldOrbitCamera();

		SystemViewportClient->FocusViewport(PreviewComponent->Bounds.GetBox(), false);

		// this will reactivate orbit mode after the transition is done, if needed
		bIsViewTransitioning = true;
		bShouldActivateOrbitAfterTransitioning = bIsOrbit;
	}
}

void SNiagaraSystemViewport::TogglePreviewGrid()
{
	SystemViewportClient->SetShowGrid();
	GetMutableDefault<UNiagaraEditorSettings>()->SetShowGridInViewport(SystemViewportClient->EngineShowFlags.Grid);
	RefreshViewport();
}

bool SNiagaraSystemViewport::IsTogglePreviewGridChecked() const
{
	return SystemViewportClient->IsSetShowGridChecked();
}

void SNiagaraSystemViewport::TogglePreviewBackground()
{
	bShowBackground = !bShowBackground;
	// @todo DB: Set the background mesh for the preview viewport.
	RefreshViewport();
}

bool SNiagaraSystemViewport::IsTogglePreviewBackgroundChecked() const
{
	return bShowBackground;
}

TSharedRef<FEditorViewportClient> SNiagaraSystemViewport::MakeEditorViewportClient()
{
	SystemViewportClient = MakeShareable( new FNiagaraSystemViewportClient(*AdvancedPreviewScene.Get(), SharedThis(this),
		FNiagaraSystemViewportClient::FOnScreenShotCaptured::CreateSP(this, &SNiagaraSystemViewport::OnScreenShotCaptured) ) );

	SystemViewportClient->InitDelegates();
	
	// Settings that aren't kept in the config are 
	SystemViewportClient->SetAllowCinematicControl(true);
	// SystemViewportClient->SetViewLocation( FVector::ZeroVector );
	// SystemViewportClient->SetViewRotation( FRotator::ZeroRotator );
	SystemViewportClient->SetViewLocationForOrbiting( FVector::ZeroVector );
	SystemViewportClient->VisibilityDelegate.BindSP( this, &SNiagaraSystemViewport::IsVisible );
	SystemViewportClient->bSetListenerPosition = false;
	
	return SystemViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SNiagaraSystemViewport::BuildViewportToolbar()
{
	// Register the viewport toolbar if another viewport hasn't already (it's shared).
	const FName ViewportToolbarName = "NiagaraSystemEditor.ViewportToolbar";

	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
			ViewportToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
		);

		ViewportToolbarMenu->StyleName = "ViewportToolbar";

		// Add the left-aligned part of the viewport toolbar.
		{
			FToolMenuSection& LeftSection = ViewportToolbarMenu->AddSection("Left");
		}

		// Add the right-aligned part of the viewport toolbar.
		{
			FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			// Add the "Camera" submenu.
			{
				RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowLensControls()));
				const FName SubmenuName = UToolMenus::JoinMenuPaths(ViewportToolbarName, "Camera");
				UE::NiagaraSystemEditor::Private::ExtendCameraSubmenu(SubmenuName);
			}

			// Add the "View Modes" sub menu.
			{
				// Stay backward-compatible with the old viewport toolbar.
				{
					const FName ParentSubmenuName = "UnrealEd.ViewportToolbar.View";
					// Create our parent menu.
					if (!UToolMenus::Get()->IsMenuRegistered(ParentSubmenuName))
					{
						UToolMenus::Get()->RegisterMenu(ParentSubmenuName);
					}

					// Register our ToolMenu here first, before we create the submenu, so we can set our parent.
					UToolMenus::Get()->RegisterMenu("NiagaraSystemEditor.ViewportToolbar.ViewModes", ParentSubmenuName);
				}

				RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			}

			RightSection.AddEntry(UE::NiagaraSystemEditor::CreateShowSubmenu());
			RightSection.AddEntry(UE::UnrealEd::CreatePerformanceAndScalabilitySubmenu());

			// Add Preview Scene Submenu
			{
				const FName AssetViewerProfileMenuName = "NiagaraSystemEditor.ViewportToolbar.AssetViewerProfile";
				RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
				UE::AdvancedPreviewScene::Menus::ExtendAdvancedPreviewSceneSettings(AssetViewerProfileMenuName);
				UE::NiagaraSystemEditor::ExtendPreviewSceneSettingsSubmenu(AssetViewerProfileMenuName);
				UE::UnrealEd::ExtendPreviewSceneSettingsWithTabEntry(AssetViewerProfileMenuName);
			}
		}
	}

	FToolMenuContext ViewportToolbarContext;
	{
		ViewportToolbarContext.AppendCommandList(AdvancedPreviewScene->GetCommandList());
		ViewportToolbarContext.AppendCommandList(GetCommandList());

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject = NewObject<UUnrealEdViewportToolbarContext>();
			ContextObject->Viewport = SharedThis(this);
			ContextObject->AssetEditorToolkit = AssetEditorToolkit;
			ContextObject->PreviewSettingsTabId = FNiagaraSystemToolkitModeBase::PreviewSettingsTabId;

			// Hook up our toolbar's filter for supported view modes.
			ContextObject->IsViewModeSupported = UE::UnrealEd::IsViewModeSupportedDelegate::CreateLambda(
				[](EViewModeIndex ViewModeIndex) -> bool
				{
					// This code is taken from SViewportToolBar::IsViewModeSupported
					// SSCSEditorViewportToolBar does not override it, so we just take it as-is
					// TODO: maybe create a private function for it, or move IsViewModeSupported to SEditorViewport

					switch (ViewModeIndex)
					{
					case VMI_PrimitiveDistanceAccuracy:
					case VMI_MaterialTextureScaleAccuracy:
					case VMI_RequiredTextureResolution:
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					case VMI_Lit_Wireframe:
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
						return false;
					default:
						return true;
					}
				}
			);

			ViewportToolbarContext.AddObject(ContextObject);
		}
	}

	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
}

TSharedPtr<IPreviewProfileController> SNiagaraSystemViewport::CreatePreviewProfileController()
{
	return MakeShared<FPreviewProfileController>();
}

EVisibility SNiagaraSystemViewport::OnGetViewportContentVisibility() const
{
	EVisibility BaseVisibility = SEditorViewport::OnGetViewportContentVisibility();
	if (BaseVisibility != EVisibility::Visible)
	{
		return BaseVisibility;
	}
	return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SNiagaraSystemViewport::GetViewportCompileStatusText() const
{
	if (PreviewComponent && PreviewComponent->GetAsset())
	{
		if (PreviewComponent->GetAsset()->HasOutstandingCompilationRequests())
		{
			return LOCTEXT("Compiling", "Compiling...");
		}
		bool bCompilationFailed = false;
		PreviewComponent->GetAsset()->ForEachScript([&bCompilationFailed](UNiagaraScript* Script)
		{
			if (Script && Script->GetLastCompileStatus() == ENiagaraScriptCompileStatus::NCS_Error)
			{
				bCompilationFailed = true;
			}
		});
		if (bCompilationFailed)
		{
			return LOCTEXT("CompilingFailed", "Compilation Failed!");
		}
	}
	return FText();
}

void SNiagaraSystemViewport::PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay)
{
	Overlay->AddSlot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	[
		SAssignNew(CompileText, STextBlock)
		.Text_Raw(this, &SNiagaraSystemViewport::GetViewportCompileStatusText)
		.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.Viewport.CompileOverlay")
		.ColorAndOpacity(FLinearColor::White)
		.ShadowOffset(FVector2D(1.5, 1.5))
		.ShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f))
	];
}


TSharedRef<class SEditorViewport> SNiagaraSystemViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SNiagaraSystemViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SNiagaraSystemViewport::OnFloatingButtonClicked()
{
}


//////////////////////////////////////////////////////////////////////////

/** Viewport Client for the Niagara baseline viewport */
class FNiagaraBaselineViewportClient : public FEditorViewportClient
{
public:
	FNiagaraBaselineViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SNiagaraBaselineViewport>& InNiagaraEditorViewport);

	// FEditorViewportClient interface
	virtual FLinearColor GetBackgroundColor() const override;
	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(FViewport* Viewport,FCanvas* Canvas) override;
	virtual bool ShouldOrbitCamera() const override;
	virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE) override;
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override { return false; }
	virtual bool CanCycleWidgetMode() const override { return false; }

	virtual void SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport)override;

	TWeakPtr<SNiagaraBaselineViewport> NiagaraViewportPtr;
	FAdvancedPreviewScene* AdvancedPreviewScene = nullptr;
};

FNiagaraBaselineViewportClient::FNiagaraBaselineViewportClient(FAdvancedPreviewScene& InPreviewScene, const TSharedRef<SNiagaraBaselineViewport>& InNiagaraEditorViewport)
	: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InNiagaraEditorViewport))
	, AdvancedPreviewScene(&InPreviewScene)
{
	NiagaraViewportPtr = InNiagaraEditorViewport;

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = false;
	DrawHelper.GridColorAxis = FColor(80,80,80);
	DrawHelper.GridColorMajor = FColor(72,72,72);
	DrawHelper.GridColorMinor = FColor(64,64,64);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
	ShowWidget(false);

	SetViewMode(VMI_Lit);
	
	EngineShowFlags.SetSnap(0);

	OverrideNearClipPlane(1.0f);
	bUsingOrbitCamera = false;
//
// 	float PreviewDistance = 1000.0f;
// 	FRotator PreviewAngle(45.0f, 0.0f, 0.0f);
// 	SetViewLocation( PreviewAngle.Vector() * -PreviewDistance );
// 	SetViewRotation( PreviewAngle );
// 	SetViewLocationForOrbiting(FVector::ZeroVector, PreviewDistance);


	//This seems to be needed to get the correct world time in the preview.
	FNiagaraBaselineViewportClient::SetIsSimulateInEditorViewport(true);
}


void FNiagaraBaselineViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	if (UWorld* World = PreviewScene->GetWorld())
	{
		if (!World->GetBegunPlay())
		{
			for (TActorIterator<ANiagaraPerfBaselineActor> It(World); It; ++It)
			{
				if (ANiagaraPerfBaselineActor* BaselineActor = *It)
				{
					It->DispatchBeginPlay();
				}
			}
			World->SetBegunPlay(true);

			// Simulate behavior from GameEngine.cpp
			World->bWorldWasLoadedThisTick = false;
			World->bTriggerPostLoadMap = true;
		}

		// Tick the preview scene world.
		World->Tick(LEVELTICK_All, DeltaSeconds);
	}
}

void FNiagaraBaselineViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	FEditorViewportClient::Draw(InViewport, Canvas);
}

bool FNiagaraBaselineViewportClient::ShouldOrbitCamera() const
{
	return bUsingOrbitCamera;
}

FLinearColor FNiagaraBaselineViewportClient::GetBackgroundColor() const
{
	if (AdvancedPreviewScene != nullptr)
	{
		return AdvancedPreviewScene->GetBackgroundColor();
	}

	FLinearColor BackgroundColor = FLinearColor::Black;
	return BackgroundColor;
}

FSceneView* FNiagaraBaselineViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	FSceneView* SceneView = FEditorViewportClient::CalcSceneView(ViewFamily);
	FFinalPostProcessSettings::FCubemapEntry& CubemapEntry = *new(SceneView->FinalPostProcessSettings.ContributingCubemaps) FFinalPostProcessSettings::FCubemapEntry;
	CubemapEntry.AmbientCubemap = GUnrealEd->GetThumbnailManager()->AmbientCubemap;
	CubemapEntry.AmbientCubemapTintMulScaleValue = FLinearColor::White;
	return SceneView;
}

void FNiagaraBaselineViewportClient::SetIsSimulateInEditorViewport(bool bInIsSimulateInEditorViewport)
{
	bIsSimulateInEditorViewport = bInIsSimulateInEditorViewport;
}

//////////////////////////////////////////////////////////////////////////

void SNiagaraBaselineViewport::Construct(const FArguments& InArgs)
{
	AdvancedPreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));
	AdvancedPreviewScene->SetFloorVisibility(false);
	AdvancedPreviewScene->SetEnvironmentVisibility(true);

	SEditorViewport::Construct( SEditorViewport::FArguments() );
}

void SNiagaraBaselineViewport::RefreshViewport()
{
	SceneViewport->InvalidateDisplay();
}

void SNiagaraBaselineViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SEditorViewport::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );

	int32 NumBaselines = 0;
	for (auto It = TActorIterator<ANiagaraPerfBaselineActor>(AdvancedPreviewScene->GetWorld()); It; ++It)
	{
		if(ANiagaraPerfBaselineActor* Actor = *It)
		{
			++NumBaselines;
		}
	}

	//Kill the window when all tests are done.
	if (NumBaselines == 0)
	{
		OwnerWindow->RequestDestroyWindow();
	}
}

bool SNiagaraBaselineViewport::IsVisible() const
{
	return true;//ViewportWidget.IsValid() && OwnerWindow.IsValid() && OwnerWindow->IsVisible() && SEditorViewport::IsVisible();
}

void SNiagaraBaselineViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	// Unbind the CycleTransformGizmos since niagara currently doesn't use the gizmos and it prevents resetting the system with
	// spacebar when the viewport is focused.
	CommandList->UnmapAction(FEditorViewportCommands::Get().CycleTransformGizmos);
}

void SNiagaraBaselineViewport::OnFocusViewportToSelection()
{

}

TSharedRef<FEditorViewportClient> SNiagaraBaselineViewport::MakeEditorViewportClient()
{
	SystemViewportClient = MakeShareable( new FNiagaraBaselineViewportClient(*AdvancedPreviewScene.Get(), SharedThis(this)) );

	SystemViewportClient->SetViewLocation( FVector::ZeroVector );
	SystemViewportClient->SetViewRotation( FRotator(0.0f, 0.0f, 0.0f) );
	SystemViewportClient->SetViewLocationForOrbiting( FVector::ZeroVector, 750.0f );
	SystemViewportClient->bSetListenerPosition = false;

	SystemViewportClient->SetRealtime( true );
	SystemViewportClient->SetGameView(false);
	SystemViewportClient->VisibilityDelegate.BindSP( this, &SNiagaraBaselineViewport::IsVisible );

	return SystemViewportClient.ToSharedRef();
}

EVisibility SNiagaraBaselineViewport::OnGetViewportContentVisibility() const
{
	EVisibility BaseVisibility = SEditorViewport::OnGetViewportContentVisibility();
	if (BaseVisibility != EVisibility::Visible)
	{
		return BaseVisibility;
	}
	return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SNiagaraBaselineViewport::Init(TSharedPtr<SWindow>& InOwnerWindow)
{
	OwnerWindow = InOwnerWindow;
}

bool SNiagaraBaselineViewport::AddBaseline(UNiagaraEffectType* EffectType)
{
	check(EffectType && EffectType->IsPerfBaselineValid() == false);

	if (UNiagaraBaselineController* Controller = EffectType->GetPerfBaselineController())
	{
		if (UNiagaraSystem* System = Controller->GetSystem())
		{
			if (System->bFixedBounds == false)
			{
				UE_LOGF(LogNiagaraEditor, Warning, "Niagara System shouldn't be used as a perf baseline as it does not have fixed bounds. %ls", *System->GetName());
			}

			//Also generate the baseline actor in the preview world.
			if (UWorld* BaselineWorld = AdvancedPreviewScene->GetWorld())
			{
				BaselineWorld->GetWorldSettings()->SetIsTemporarilyHiddenInEditor(false);
				EffectType->SpawnBaselineActor(BaselineWorld);
				return true;
			}
		}
		else
		{
			UE_LOGF(LogNiagaraEditor, Warning, "Baseline Niagara System missing!. Effect Type: %ls", *EffectType->GetName());
			return false;
		}
	}
	return false;

}

#undef LOCTEXT_NAMESPACE
