// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/GCObject.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "PreviewScene.h"
#include "Framework/Commands/UICommandList.h"
#include "EditorViewportClient.h"
#include "ISequencerModule.h"
#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "NiagaraPerfBaseline.h"

class FNiagaraSystemViewModel;
class UNiagaraComponent;
class FAssetEditorToolkit;
class FNiagaraSystemEditorViewportClient;
class FNiagaraSystemInstance;
class UNiagaraEffectType;

/**
 * Niagara Editor Preview viewport widget
 */
class SNiagaraSystemViewport : public SEditorViewport, public FGCObject, public ICommonEditorViewportToolbarInfoProvider
{
public:
	DECLARE_DELEGATE_TwoParams(FOnThumbnailCaptured, UTexture2D*, TOptional<FGuid> /** An optional emitter handle guid */);

public:
	SLATE_BEGIN_ARGS( SNiagaraSystemViewport ){}
		SLATE_EVENT(FOnThumbnailCaptured, OnThumbnailCaptured)
		/** So we can retrieve different data from sequencer to display in the viewport */
		SLATE_ARGUMENT(TWeakPtr<ISequencer>, Sequencer)
		SLATE_ARGUMENT(TWeakPtr<FAssetEditorToolkit>, AssetEditorToolkit)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);
	virtual ~SNiagaraSystemViewport() override;
	
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SNiagaraSystemViewport");
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	void RefreshViewport();
	
	void SetPreviewComponent(UNiagaraComponent* NiagaraComponent);
	
	void ToggleRealtime();
	
	/** If true, render background object in the preview scene. */
	bool bShowBackground;

	TSharedRef<class FAdvancedPreviewScene> GetPreviewScene() const { return AdvancedPreviewScene.ToSharedRef(); }
	TWeakPtr<FNiagaraSystemViewModel> GetSystemViewModel() { return SystemViewModel; }
	
	/** The material editor has been added to a tab */
	void OnAddedToTab( const TSharedRef<SDockTab>& OwnerTab );
	
	/** Event handlers */
	void TogglePreviewGrid();
	bool IsTogglePreviewGridChecked() const;
	void TogglePreviewBackground();
	bool IsTogglePreviewBackgroundChecked() const;
	class UNiagaraComponent *GetPreviewComponent()	{ return PreviewComponent;  }

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	/** Draw flag types */
	enum EDrawElements
	{
		OriginAxis				= 0x010,
		Bounds					= 0x020,
		InstructionCounts		= 0x040,
		ParticleCounts			= 0x080,
		EmitterExecutionOrder	= 0x100,
		GpuTickInformation		= 0x200,
		MemoryInfo				= 0x400,
		StatelessInfo			= 0x800,
	};

	bool GetDrawElement(EDrawElements Element) const;
	void ToggleDrawElement(EDrawElements Element);
	/** By specifying an emitter guid, we can record thumbnails for individual emitters within a system.*/
	void CreateThumbnail(UObject* InScreenShotOwner, TOptional<FGuid> EmitterToCaptureThumbnailFor);

	bool IsToggleOrbitChecked() const;
	void ToggleOrbit();

	bool IsMotionEnabled() const;
	void ToggleMotion();

	float GetMotionRate() const { return MotionRate; }
	void SetMotionRate(float Rate) { MotionRate = Rate; }

	float GetMotionRadius() const { return MotionRadius; }
	void SetMotionRadius(float Radius) { MotionRadius = Radius; }

	bool IsFloorEnabled() const;
	void ToggleFloorEnabled();
	float GetFloorOffset() const;
	void SetFloorOffset(float Offset);

	bool IsManualFocusDistanceEnabled() const;
	void ToggleManualFocusDistance();
	void SetManualFocusDistance(float Distance);
	float GetManualFocusDistance() const;

protected:	
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	virtual void PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay) override;
	FText GetViewportCompileStatusText() const;

private:
	virtual bool IsVisible() const override;
	void OnScreenShotCaptured(UTexture2D* ScreenShot);

	/** The parent tab where this viewport resides */
	TWeakPtr<SDockTab> ParentTab;

	TWeakPtr<ISequencer> Sequencer = nullptr;

	TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit = nullptr;

	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;
	
	/** Preview Scene - uses advanced preview settings */
	TSharedPtr<class FAdvancedPreviewScene> AdvancedPreviewScene;

	TSharedPtr<STextBlock> CompileText;

	TOptional<FGuid> EmitterToCaptureThumbnailFor;

	/** Pointer back to the material editor tool that owns us */
	//TWeakPtr<INiagaraSystemEditor> SystemEditorPtr;
	
	TObjectPtr<class UNiagaraComponent> PreviewComponent;
	
	/** Level viewport client */
	TSharedPtr<class FNiagaraSystemViewportClient> SystemViewportClient;

	uint32 DrawFlags = 0;

	FOnThumbnailCaptured OnThumbnailCapturedDelegate;

	/** Used on tick to determine if a view transition was active so we can restore view settings at the end of it */
	bool bIsViewTransitioning = false;

	/** True if orbit mode was active before we started a view transition. Used to restore orbit mode at the the end of the transition */
	bool bShouldActivateOrbitAfterTransitioning = false;

	/** Motion Parameters */
	bool bMotionEnabled = false;
	float MotionRate = 90.0f;
	float MotionRadius = 200.0f;

	FDelegateHandle OnPreviewFeatureLevelChangedHandle;
};

#if NIAGARA_PERF_BASELINES

/** Niagara Baseline Display Viewport */
class SNiagaraBaselineViewport : public SEditorViewport
{
public:
	SLATE_BEGIN_ARGS(SNiagaraBaselineViewport) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void RefreshViewport();

	void Init(TSharedPtr<SWindow>& InOwnerWindow);

	bool AddBaseline(UNiagaraEffectType* EffectType);

protected:
	/** SEditorViewport interface */
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual EVisibility OnGetViewportContentVisibility() const override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;

private:
	virtual bool IsVisible() const override;

	/** Preview Scene - uses advanced preview settings */
	TSharedPtr<class FAdvancedPreviewScene> AdvancedPreviewScene;

	/** Level viewport client */
	TSharedPtr<class FNiagaraBaselineViewportClient> SystemViewportClient;

	TSharedPtr<SWindow> OwnerWindow;
};

#endif
