// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraVersionWidget.h"

#include "AssetToolsModule.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "NiagaraActions.h"
#include "NiagaraVersionMetaData.h"
#include "PropertyEditorModule.h"
#include "SGraphActionMenu.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "NiagaraVersionWidget"

FNiagaraVersionMenuAction::FNiagaraVersionMenuAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords, FOnExecuteStackAction InAction, int32 InSectionID, FNiagaraAssetVersion InVersion)
	: FNiagaraMenuAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords), MoveTemp(InAction), InSectionID)
	, AssetVersion(InVersion)
{}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText SNiagaraVersionWidget::FormatVersionLabel(const FNiagaraAssetVersion& Version) const
{
	FText Format = Version == VersionedObject->GetExposedVersion() ? FText::FromString("{0}.{1} (exposed)") : FText::FromString("{0}.{1}");
	return FText::Format(Format, Version.MajorVersion, Version.MinorVersion);
}

FText SNiagaraVersionWidget::GetInfoHeaderText() const
{
	return FText();
}

void SNiagaraVersionWidget::ExecuteSaveAsAssetAction(FNiagaraAssetVersion AssetVersion)
{
}

SNiagaraVersionWidget::~SNiagaraVersionWidget()
{
	GEditor->UnregisterForUndo(this);
}

void SNiagaraVersionWidget::Construct(const FArguments& InArgs, FNiagaraVersionedObject* InVersionedObject, UNiagaraVersionMetaData* InMetadata)
{
	OnVersionDataChanged = InArgs._OnVersionDataChanged;
	OnChangeToVersion = InArgs._OnChangeToVersion;
	
	FDetailsViewArgs DetailsArgs;
	DetailsArgs.bHideSelectionTip = true;
	DetailsArgs.bAllowSearch = false;
	DetailsArgs.NotifyHook = this;
	
	if (InVersionedObject == nullptr)
	{
		ChildSlot
	    [
		 SNew(SBox)
	        .MinDesiredWidth(400)
			.Padding(10)
	        [
	            SNew(SVerticalBox)

	            // the top description text
	            + SVerticalBox::Slot()
	            .AutoHeight()
	            [
            		SNew(STextBlock)
            		.AutoWrapText(true)
					.Text(this, &SNiagaraVersionWidget::GetInfoHeaderText)
	            ]
	        ]
	    ];
		return;
	}

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	VersionSettingsDetails = PropertyModule.CreateDetailView(DetailsArgs);

	UpdateEditedObject(InVersionedObject, InMetadata);
	
	// the list of available versions
	SAssignNew(VersionListWidget, SGraphActionMenu)
	    .OnActionSelected(this, &SNiagaraVersionWidget::OnActionSelected)
		.OnCollectAllActions(this, &SNiagaraVersionWidget::CollectAllVersionActions)
	    .OnCollectStaticSections_Lambda([](TArray<int32>& StaticSectionIDs) {StaticSectionIDs.Add(1);})
	    .OnGetSectionTitle_Lambda([](int32) {return LOCTEXT("NiagaraVersionManagementTitle", "Versions");})
	    .AutoExpandActionMenu(true)
	    .ShowFilterTextBox(false)
	    .OnCreateCustomRowExpander_Static(&SNiagaraVersionWidget::CreateCustomActionExpander)
	    .UseSectionStyling(true)
		.AlphaSortItems(false)
	    .OnGetSectionWidget(this, &SNiagaraVersionWidget::GetVersionSelectionHeaderWidget)
		.OnContextMenuOpening(this, &SNiagaraVersionWidget::OnVersionContextMenuOpening)
	    .OnCreateWidgetForAction_Lambda([](const FCreateWidgetForActionData* InData)
	    {
	        TSharedPtr<FNiagaraMenuAction> NiagaraAction = StaticCastSharedPtr<FNiagaraMenuAction>(InData->Action);
	        return SNew(STextBlock).Text(NiagaraAction->GetMenuDescription());
	    });

	ChildSlot
    [
	 SNew(SBox)
        .MinDesiredWidth(400)
		.Padding(10)
        [
            SNew(SVerticalBox)

            // the top description text
            + SVerticalBox::Slot()
            .AutoHeight()
            [
            	SNew(STextBlock)
            	.AutoWrapText(true)
				.Text(this, &SNiagaraVersionWidget::GetInfoHeaderText)
            ]

            // the main part of the widget
            + SVerticalBox::Slot()
            .FillHeight(1)
            .Padding(FMargin(5, 15))
            .VAlign(VAlign_Fill)
            [
            	// display button to enable versioning or versioning details
	            SNew(SWidgetSwitcher)
		        .WidgetIndex(this, &SNiagaraVersionWidget::GetDetailWidgetIndex)
		        + SWidgetSwitcher::Slot()
		        [
					SNew(SOverlay)

					+ SOverlay::Slot()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Center)
					[
						// enable versioning button
				        SNew(SButton)
		                .ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
		                .ButtonStyle(FAppStyle::Get(), "FlatButton.Dark")
		                .ContentPadding(FMargin(6, 2))
		                .TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
		                .Text(LOCTEXT("NiagaraAEnableVersioning", "Enable versioning"))
		                .OnClicked(this, &SNiagaraVersionWidget::EnableVersioning)
	                ]
		        ]
		        + SWidgetSwitcher::Slot()
		        [
			        SNew(SHorizontalBox)

		            // version selector
		            + SHorizontalBox::Slot()
		            .AutoWidth()
		            [
		                SNew(SVerticalBox)

		                // version list
		                + SVerticalBox::Slot()
		                .FillHeight(1)
		                [
		                    SNew(SBox)
		                    .MaxDesiredWidth(200)
		                    .MinDesiredWidth(200)
		                    [
		                        VersionListWidget.ToSharedRef()
		                    ]
		                ]
		            ]

		            // separator
		            + SHorizontalBox::Slot()
		            .AutoWidth()
		            .Padding(5)
		            [
		                SNew(SSeparator)
		                .Orientation(Orient_Vertical)
		            ]

		            // version details
		            + SHorizontalBox::Slot()
		            [
		                VersionSettingsDetails.ToSharedRef()
		            ]
		        ]
            ]
        ]
    ];

	// select the exposed version
	FText ItemName = FormatVersionLabel(VersionedObject->GetExposedVersion());
	VersionListWidget->SelectItemByName(FName(ItemName.ToString()));

	GEditor->RegisterForUndo(this);
}

void SNiagaraVersionWidget::UpdateEditedObject(FNiagaraVersionedObject* InVersionedObject, UNiagaraVersionMetaData* InMetadata)
{
	VersionedObject = InVersionedObject;
	VersionMetadata = InMetadata;
	VersionSettingsDetails->SetObject(VersionMetadata);
}

void SNiagaraVersionWidget::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	// update script data from details view changes
	TSharedPtr<FNiagaraVersionDataAccessor> ObjectData = VersionedObject->GetVersionDataAccessor(SelectedVersion);
	check(ObjectData);
	
	ObjectData->GetVersionChangeDescription() = VersionMetadata->ChangeDescription;
	ObjectData->IsDeprecated() = VersionMetadata->bDeprecated;
	ObjectData->GetDeprecationMessage() = VersionMetadata->DeprecationMessage;
	ObjectData->GetUpdateScriptExecutionType() = VersionMetadata->UpdateScriptExecution;
	ObjectData->GetScriptAsset() = VersionMetadata->ScriptAsset;
	ObjectData->GetPythonUpdateScript() = VersionMetadata->PythonUpdateScript;
	ObjectData->GetUpdateUtilityClass() = VersionMetadata->UpdateUtilityClass;

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraVersionMetaData, bIsExposedVersion) && VersionMetadata->bIsExposedVersion)
	{
		ExecuteExposeAction(ObjectData->GetObjectVersion());
	}

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraVersionMetaData, bIsVisibleInVersionSelector))
	{
		ObjectData->GetObjectVersion().bIsVisibleInVersionSelector = VersionMetadata->bIsVisibleInVersionSelector;
		VersionListWidget->RefreshAllActions(true);
		VersionInListSelected(ObjectData->GetObjectVersion());
	}

	OnVersionDataChanged.ExecuteIfBound(&PropertyChangedEvent, SelectedVersion);
}

void SNiagaraVersionWidget::PostUndo(bool bSuccess)
{
	if (VersionListWidget)
	{
		VersionListWidget->RefreshAllActions(true);
	}
}

void SNiagaraVersionWidget::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void SNiagaraVersionWidget::SetOnVersionDataChanged(FOnVersionPropertyChangedDelegate InOnVersionDataChanged)
{
	OnVersionDataChanged = InOnVersionDataChanged;
}

TSharedRef<SExpanderArrow> SNiagaraVersionWidget::CreateCustomActionExpander(const FCustomExpanderData& ActionMenuData)
{
	return SNew(SExpanderArrow, ActionMenuData.TableRow)
        .Visibility(EVisibility::Hidden);
}

TSharedRef<SWidget> SNiagaraVersionWidget::OnGetAddVersionMenu()
{
	FNiagaraAssetVersion LatestVersion = VersionedObject->GetAllAvailableVersions().Last();
	FMenuBuilder MenuBuilder(true, nullptr);

	FText Label = FText::Format(LOCTEXT("NiagaraAddMajorVersion", "New major version ({0}.0)"), LatestVersion.MajorVersion + 1);
	FText Tooltip = LOCTEXT("NiagaraAddMajorVersion_Tooltip", "Adds a new major version. This should be used for breaking changes (e.g. adding a new parameter).");
	FUIAction UIAction(FExecuteAction::CreateSP(this, &SNiagaraVersionWidget::AddNewMajorVersion), FCanExecuteAction());
	MenuBuilder.AddMenuEntry(Label, Tooltip, FSlateIcon(), UIAction);

	Label = FText::Format(LOCTEXT("NiagaraAddMinorVersion", "New minor version ({0}.{1})"), LatestVersion.MajorVersion, LatestVersion.MinorVersion + 1);
	Tooltip = LOCTEXT("NiagaraAddMinorVersion_Tooltip", "Adds a new minor version. This should be used for non-breaking changes (e.g. adding a comment).");
	UIAction = FUIAction(FExecuteAction::CreateSP(this, &SNiagaraVersionWidget::AddNewMinorVersion), FCanExecuteAction());
	MenuBuilder.AddMenuEntry(Label, Tooltip, FSlateIcon(), UIAction);

	return MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> SNiagaraVersionWidget::OnVersionContextMenuOpening()
{
	TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
	VersionListWidget->GetSelectedActions(SelectedActions);
	if (SelectedActions.Num() != 1)
	{
		return SNullWidget::NullWidget;
	}
	FNiagaraVersionMenuAction* SelectedAction = static_cast<FNiagaraVersionMenuAction*>(SelectedActions[0].Get());
	
	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.BeginSection("Edit", LOCTEXT("EditMenuHeader", "Edit"));
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("ExposeVersion", "Expose version"), LOCTEXT("ExposeVersion_Tooltip", "Exposing this version will make it the default version for any new assets."), FSlateIcon(),
            FUIAction(
                FExecuteAction::CreateSP(this, &SNiagaraVersionWidget::ExecuteExposeAction, SelectedAction->AssetVersion),
                FCanExecuteAction::CreateSP(this, &SNiagaraVersionWidget::CanExecuteExposeAction, SelectedAction->AssetVersion)
            ));

		MenuBuilder.AddMenuEntry(LOCTEXT("SaveAsAsset", "Save as new asset..."), LOCTEXT("SaveAsAssetVersion_Tooltip", "Creates a new asset with this version as starting point."), FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SNiagaraVersionWidget::ExecuteSaveAsAssetAction, SelectedAction->AssetVersion)));

		MenuBuilder.AddSeparator();
		
		MenuBuilder.AddMenuEntry(LOCTEXT("DeleteVersion", "Delete version"), LOCTEXT("DeleteVersion_Tooltip", "Deletes this version and all associated data. This will break existing usages of that version!"), FSlateIcon(),
            FUIAction(
                FExecuteAction::CreateSP(this, &SNiagaraVersionWidget::ExecuteDeleteAction, SelectedAction->AssetVersion),
                FCanExecuteAction::CreateSP(this, &SNiagaraVersionWidget::CanExecuteDeleteAction, SelectedAction->AssetVersion)
            ));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

int32 SNiagaraVersionWidget::GetDetailWidgetIndex() const
{
	return VersionedObject->IsVersioningEnabled() ? 1 : 0;
}

FReply SNiagaraVersionWidget::EnableVersioning()
{
	FScopedTransaction Transaction(LOCTEXT("EnableVersioning", "Enable versioning"));
	VersionedObject->EnableVersioning();
	OnVersionDataChanged.ExecuteIfBound(nullptr, SelectedVersion);
	
	return FReply::Handled();
}

bool SNiagaraVersionWidget::CanExecuteDeleteAction(FNiagaraAssetVersion AssetVersion)
{
	return AssetVersion != VersionedObject->GetExposedVersion() && (AssetVersion.MajorVersion != 1 || AssetVersion.MinorVersion != 0);
}

bool SNiagaraVersionWidget::CanExecuteExposeAction(FNiagaraAssetVersion AssetVersion)
{
	return AssetVersion != VersionedObject->GetExposedVersion();
}

void SNiagaraVersionWidget::ExecuteDeleteAction(FNiagaraAssetVersion AssetVersion)
{
	FScopedTransaction Transaction(LOCTEXT("DeleteVersion", "Delete version"));
	OnChangeToVersion.ExecuteIfBound(VersionedObject->GetExposedVersion().VersionGuid);
	VersionedObject->DeleteVersion(AssetVersion.VersionGuid);
	
	VersionListWidget->RefreshAllActions(true);
	VersionInListSelected(VersionedObject->GetExposedVersion());
	OnVersionDataChanged.ExecuteIfBound(nullptr, SelectedVersion);
}

void SNiagaraVersionWidget::ExecuteExposeAction(FNiagaraAssetVersion AssetVersion)
{
	FScopedTransaction Transaction(LOCTEXT("ExposeVersion", "Expose version"));
	VersionedObject->ExposeVersion(AssetVersion.VersionGuid);

	VersionListWidget->RefreshAllActions(true);
	VersionInListSelected(AssetVersion);
	OnVersionDataChanged.ExecuteIfBound(nullptr, SelectedVersion);
}

TSharedRef<ITableRow> SNiagaraVersionWidget::HandleVersionViewGenerateRow(TSharedRef<FNiagaraAssetVersion> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return
        SNew(STableRow<TSharedRef<FText>>, OwnerTable)
        [
            SNew(STextBlock)
            .Text(FormatVersionLabel(Item.Get()))
        ];
}

void SNiagaraVersionWidget::OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions,	ESelectInfo::Type)
{
	for (TSharedPtr<FEdGraphSchemaAction> Action : SelectedActions)
	{
		FNiagaraMenuAction* SelectedAction = (FNiagaraMenuAction*) Action.Get();
		if (SelectedAction)
		{
			SelectedAction->ExecuteAction();
		}
	}
}

TSharedRef<SWidget> SNiagaraVersionWidget::GetVersionSelectionHeaderWidget(TSharedRef<SWidget>, int32)
{
	// creates the add version button
	return SNew(SComboButton)
        .ButtonStyle(FAppStyle::Get(), "RoundButton")
        .ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
        .ContentPadding(FMargin(2, 0))
        .OnGetMenuContent(this, &SNiagaraVersionWidget::OnGetAddVersionMenu)
        .HAlign(HAlign_Center)
        .VAlign(VAlign_Center)
        .HasDownArrow(false)
        .ButtonContent()
        [
            SNew(SHorizontalBox)

            + SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(FMargin(0, 1))
            [
                SNew(SImage)
                .Image(FAppStyle::GetBrush("Plus"))
            ]

            + SHorizontalBox::Slot()
            .VAlign(VAlign_Center)
            .AutoWidth()
            .Padding(FMargin(2,0,0,0))
            [
                SNew(STextBlock)
                .Font(IDetailLayoutBuilder::GetDetailFontBold())
                .Text(LOCTEXT("NiagaraVersionManagementAdd", "Add version"))
                .ShadowOffset(FVector2D(1,1))
            ]
        ];
}

void SNiagaraVersionWidget::CollectAllVersionActions(FGraphActionListBuilderBase& OutAllActions)
{
	TArray<FNiagaraAssetVersion> AssetVersions = VersionedObject->GetAllAvailableVersions();
	for (FNiagaraAssetVersion& Version : AssetVersions)
	{
		FText MenuDesc = FormatVersionLabel(Version);
		FText Tooltip;
		auto ExecAction = FNiagaraMenuAction::FOnExecuteStackAction::CreateSP(this, &SNiagaraVersionWidget::VersionInListSelected, Version);
		TSharedPtr<FNiagaraMenuAction> Action = MakeShared<FNiagaraVersionMenuAction>(FText(), MenuDesc, Tooltip, 0, FText(), ExecAction, 1, Version);
		OutAllActions.AddAction(Action);
	}
}

void SNiagaraVersionWidget::VersionInListSelected(FNiagaraAssetVersion InSelectedVersion)
{
	SelectedVersion = InSelectedVersion.VersionGuid;
	TSharedPtr<FNiagaraVersionDataAccessor> ObjectData = VersionedObject->GetVersionDataAccessor(SelectedVersion);
	check(ObjectData);
	
	VersionMetadata->VersionGuid = InSelectedVersion.VersionGuid;
	VersionMetadata->ChangeDescription = ObjectData->GetVersionChangeDescription();
	VersionMetadata->bDeprecated = ObjectData->IsDeprecated();
	VersionMetadata->DeprecationMessage = ObjectData->GetDeprecationMessage();
	VersionMetadata->bIsExposedVersion = InSelectedVersion == VersionedObject->GetExposedVersion();
	VersionMetadata->bIsVisibleInVersionSelector = InSelectedVersion.bIsVisibleInVersionSelector;

	VersionMetadata->UpdateScriptExecution = ObjectData->GetUpdateScriptExecutionType();
	VersionMetadata->ScriptAsset = ObjectData->GetScriptAsset();
	VersionMetadata->PythonUpdateScript = ObjectData->GetPythonUpdateScript();
	VersionMetadata->UpdateUtilityClass = ObjectData->GetUpdateUtilityClass();

	VersionSettingsDetails->SetObject(VersionMetadata, true);
}

void SNiagaraVersionWidget::AddNewMajorVersion()
{
	FScopedTransaction Transaction(LOCTEXT("AddMajorVersion", "Add major version"));
	FNiagaraAssetVersion LatestVersion = VersionedObject->GetAllAvailableVersions().Last();
	FGuid NewVersion = VersionedObject->AddNewVersion(LatestVersion.MajorVersion + 1, 0);

	VersionListWidget->RefreshAllActions(true);
	
	// select the new version
	FText ItemName = FormatVersionLabel(VersionedObject->GetVersionDataAccessor(NewVersion)->GetObjectVersion());
	VersionListWidget->SelectItemByName(FName(ItemName.ToString()));

	OnChangeToVersion.ExecuteIfBound(NewVersion);
	OnVersionDataChanged.ExecuteIfBound(nullptr, SelectedVersion);
}

void SNiagaraVersionWidget::AddNewMinorVersion()
{
	FScopedTransaction Transaction(LOCTEXT("AddMinorVersion", "Add minor version"));
	FNiagaraAssetVersion LatestVersion = VersionedObject->GetAllAvailableVersions().Last();
	FGuid NewVersion = VersionedObject->AddNewVersion(LatestVersion.MajorVersion, LatestVersion.MinorVersion + 1);

	VersionListWidget->RefreshAllActions(true);
	
	// select the new version
	FText ItemName = FormatVersionLabel(VersionedObject->GetVersionDataAccessor(NewVersion)->GetObjectVersion());
	VersionListWidget->SelectItemByName(FName(ItemName.ToString()));

	OnChangeToVersion.ExecuteIfBound(NewVersion);
	OnVersionDataChanged.ExecuteIfBound(nullptr, SelectedVersion);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE
