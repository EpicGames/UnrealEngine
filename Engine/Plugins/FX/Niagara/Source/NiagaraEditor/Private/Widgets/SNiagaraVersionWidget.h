// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "NiagaraActions.h"
#include "EdGraph/EdGraphSchema.h"
#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

struct FCustomExpanderData;
struct FNiagaraAssetVersion;
class SGraphActionMenu;
class SExpanderArrow;
class STableViewBase;
class ITableRow;
class IDetailsView;
class UNiagaraVersionMetaData;
class UNiagaraScript;

DECLARE_DELEGATE_OneParam(FOnSwitchToVersionDelegate, FGuid);
DECLARE_DELEGATE_TwoParams(FOnVersionPropertyChangedDelegate, const FPropertyChangedEvent*, FGuid);

struct FNiagaraVersionMenuAction : FNiagaraMenuAction
{
	FNiagaraVersionMenuAction() {}
	FNiagaraVersionMenuAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords, FOnExecuteStackAction InAction, int32 InSectionID, FNiagaraAssetVersion InVersion);

	FNiagaraAssetVersion AssetVersion;
};

class SNiagaraVersionWidget : public SCompoundWidget, FNotifyHook, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SNiagaraVersionWidget)
	{}

	/** Called when the version data of the script was edited by the user */
	SLATE_EVENT(FOnVersionPropertyChangedDelegate, OnVersionDataChanged)

	/** Called when the user does something that prompts the editor to change the current active version, e.g. delete a version or add a new version */
    SLATE_EVENT(FOnSwitchToVersionDelegate, OnChangeToVersion)
	
	SLATE_END_ARGS()

	NIAGARAEDITOR_API virtual ~SNiagaraVersionWidget() override;

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, FNiagaraVersionedObject* InVersionedObject, UNiagaraVersionMetaData* InMetadata);

	NIAGARAEDITOR_API void UpdateEditedObject(FNiagaraVersionedObject* InVersionedObject, UNiagaraVersionMetaData* InMetadata);

	//~ Begin FNotifyHook Interface
	NIAGARAEDITOR_API virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	// End of FNotifyHook

	//~ Begin FEditorUndoClient interface
	NIAGARAEDITOR_API virtual void PostUndo(bool bSuccess) override;
	NIAGARAEDITOR_API virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	/** See OnVersionDataChanged event */
	NIAGARAEDITOR_API void SetOnVersionDataChanged(FOnVersionPropertyChangedDelegate InOnVersionDataChanged);

protected:
	NIAGARAEDITOR_API virtual FText GetInfoHeaderText() const;
	NIAGARAEDITOR_API virtual void ExecuteSaveAsAssetAction(FNiagaraAssetVersion AssetVersion);
	
private:
	FNiagaraVersionedObject* VersionedObject = nullptr;
	bool bAssetVersionsChanged = false;
	UNiagaraVersionMetaData* VersionMetadata = nullptr;
	
	FOnVersionPropertyChangedDelegate OnVersionDataChanged;
	FOnSwitchToVersionDelegate OnChangeToVersion;
	TSharedPtr<IDetailsView> VersionSettingsDetails;
	TSharedPtr<SGraphActionMenu> VersionListWidget;
	FGuid SelectedVersion;

	void AddNewMajorVersion();
	void AddNewMinorVersion();
	
	FText FormatVersionLabel(const FNiagaraAssetVersion& Version) const;
	TSharedRef<ITableRow> HandleVersionViewGenerateRow(TSharedRef<FNiagaraAssetVersion> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions, ESelectInfo::Type InSelectionType);
	TSharedRef<SWidget> GetVersionSelectionHeaderWidget(TSharedRef<SWidget> RowWidget, int32 SectionID);
	void CollectAllVersionActions(FGraphActionListBuilderBase& OutAllActions);
	void VersionInListSelected(FNiagaraAssetVersion SelectedVersion);
	static TSharedRef<SExpanderArrow> CreateCustomActionExpander(const FCustomExpanderData& ActionMenuData);
	TSharedRef<SWidget> OnGetAddVersionMenu();
	TSharedPtr<SWidget> OnVersionContextMenuOpening();
	int32 GetDetailWidgetIndex() const;
	FReply EnableVersioning();

	// context menu actions
	bool CanExecuteDeleteAction(FNiagaraAssetVersion AssetVersion);
	bool CanExecuteExposeAction(FNiagaraAssetVersion AssetVersion);
	void ExecuteDeleteAction(FNiagaraAssetVersion AssetVersion);
	void ExecuteExposeAction(FNiagaraAssetVersion AssetVersion);
};
