// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraScalabilityContext.h"
#include "IDetailChildrenBuilder.h"
#include "ISinglePropertyView.h"
#include "NiagaraEmitterDetailsCustomization.h"
#include "NiagaraMeshRendererProperties.h"
#include "NiagaraObjectSelection.h"
#include "Modules/ModuleManager.h"
#include "Stateless/NiagaraStatelessEmitter.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "PropertyEditorModule.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/Stack/NiagaraStackEmitterPropertiesGroup.h"
#include "ViewModels/Stack/NiagaraStackRoot.h"
#include "ViewModels/Stack/NiagaraStackStatelessEmitterGroup.h"
#include "ViewModels/Stack/NiagaraStackSystemSettingsGroup.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "NiagaraScalabilityContext"

void SNiagaraScalabilityContext::Construct(const FArguments& InArgs, UNiagaraSystemScalabilityViewModel& InScalabilityViewModel)
{
	ScalabilityViewModel = &InScalabilityViewModel;
	
	ScalabilityViewModel->GetSystemViewModel().Pin()->GetSelectionViewModel()->OnEntrySelectionChanged().AddSP(this, &SNiagaraScalabilityContext::UpdateScalabilityContent);
	ScalabilityViewModel->GetSystemViewModel().Pin()->GetOverviewGraphViewModel()->GetNodeSelection()->OnSelectedObjectsChanged().AddSP(this, &SNiagaraScalabilityContext::UpdateScalabilityContent);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.ViewIdentifier = "ScalabilityContextDetailsView";
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	// important: we register a generic details customization to overwrite the global customization for this object. This is because using "EditCategory" in a customization
	// puts a category from default into custom categories. This in turn will ignore the property visible delegate.
	// without this, a few properties from the mesh renderer properties make it into the details panel even though the delegate returns false for them.
	DetailsView->RegisterInstancedCustomPropertyLayout(UNiagaraMeshRendererProperties::StaticClass(), DetailsView->GetGenericLayoutDetailsDelegate());
	DetailsView->RegisterInstancedCustomPropertyLayout(UNiagaraEmitter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraEmitterScalabilityDetails::MakeInstance));

	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SNiagaraScalabilityContext::FilterScalabilityProperties));
	
	UpdateScalabilityContent();

	ChildSlot
	[
		SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([this]
		{
			return DetailsView->GetSelectedObjects().Num() == 1 ? 0 : 1;  
		})
		+ SWidgetSwitcher::Slot()
		[
			DetailsView.ToSharedRef()
		]
		+ SWidgetSwitcher::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(10.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("EmptyScalabilityContextDescription", "This tab displays scalability settings of the selected emitters, system, or renderers.\nPlease select emitter or system properties or a renderer."))
			.ColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f, 0.5f))
			.AutoWrapText(true)
		]
	];
}

SNiagaraScalabilityContext::~SNiagaraScalabilityContext()
{
	if(ScalabilityViewModel.IsValid() && ScalabilityViewModel->GetSystemViewModel().IsValid())
	{
		if(ScalabilityViewModel->GetSystemViewModel().Pin()->GetSelectionViewModel())
		{
			ScalabilityViewModel->GetSystemViewModel().Pin()->GetSelectionViewModel()->OnEntrySelectionChanged().RemoveAll(this);
		}

		if(ScalabilityViewModel->GetSystemViewModel().Pin()->GetOverviewGraphViewModel())
		{
			ScalabilityViewModel->GetSystemViewModel().Pin()->GetOverviewGraphViewModel()->GetNodeSelection()->OnSelectedObjectsChanged().RemoveAll(this);
		}
	}
}

void SNiagaraScalabilityContext::SetObject(UObject* Object)
{
	if(Object == nullptr)
	{
		DetailsView->SetObject(nullptr);
	}
	else if(UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Object))
	{
		DetailsView->SetObject(Emitter);
	}
	else if (UNiagaraStatelessEmitter* StatelessEmitter = Cast<UNiagaraStatelessEmitter>(Object))
	{
		DetailsView->SetObject(StatelessEmitter);
	}
	else if(UNiagaraSystem* System = Cast<UNiagaraSystem>(Object))
	{
		DetailsView->SetObject(System);
	}
	else if(UNiagaraStackRendererItem* RendererItem = Cast<UNiagaraStackRendererItem>(Object))
	{
		DetailsView->SetObject(RendererItem->GetRendererProperties());
	}
}

void SNiagaraScalabilityContext::UpdateScalabilityContent()
{
	UObject* NewSelection = nullptr;

	TArray<UNiagaraStackEntry*> StackEntries;
	ScalabilityViewModel->GetSystemViewModel().Pin()->GetSelectionViewModel()->GetSelectedEntries(StackEntries);

	TSet<UObject*> SelectedNodes = ScalabilityViewModel->GetSystemViewModel().Pin()->GetOverviewGraphViewModel()->GetNodeSelection()->GetSelectedObjectsResolved();

	// if we select a node, the stack entries will be updated to a niagara stack root, which we want to ignore
	if(StackEntries.Num() == 1 && !StackEntries[0]->IsA<UNiagaraStackRoot>())
	{
		UNiagaraStackEntry* StackEntry = StackEntries[0];

		if(StackEntry->IsA(UNiagaraStackRendererItem::StaticClass()))
		{
			NewSelection = StackEntry;
		}
		else if(StackEntry->IsA(UNiagaraStackEmitterPropertiesGroup::StaticClass()))
		{
			UNiagaraEmitter* Emitter = Cast<UNiagaraStackEmitterPropertiesGroup>(StackEntry)->GetEmitterViewModel()->GetEmitter().Emitter;
			NewSelection = Emitter;
		}
		else if (StackEntry->IsA(UNiagaraStackStatelessEmitterGroup::StaticClass()))
		{
			UNiagaraStatelessEmitter* Emitter = Cast<UNiagaraStackStatelessEmitterGroup>(StackEntry)->GetStatelessEmitter();
			NewSelection = Emitter;
		}
		else if(StackEntry->IsA(UNiagaraStackSystemPropertiesGroup::StaticClass()))
		{
			UNiagaraSystem& System = Cast<UNiagaraStackSystemPropertiesGroup>(StackEntry)->GetSystemViewModel()->GetSystem();
			NewSelection = &System;
		}
	}
	else if(SelectedNodes.Num() == 1)
	{
		UObject* SelectedNode = SelectedNodes.Array()[0];

		if(UNiagaraOverviewNode* OverviewNode = Cast<UNiagaraOverviewNode>(SelectedNode))
		{
			if(FNiagaraEmitterHandle* EmitterHandle = OverviewNode->TryGetEmitterHandle())
			{
				if (EmitterHandle->GetEmitterMode() == ENiagaraEmitterMode::Standard)
				{
					NewSelection = EmitterHandle->GetInstance().Emitter;
				}
				else
				{
					NewSelection = EmitterHandle->GetStatelessEmitter();
				}
			}
			else if(UNiagaraSystem* System = OverviewNode->GetOwningSystem())
			{
				NewSelection = System;
			}
		}
	}	
	
	SetObject(NewSelection);
}

bool SNiagaraScalabilityContext::FilterScalabilityProperties(const FPropertyAndParent& InPropertyAndParent) const
{
	auto ShouldPropertyBeVisible = [](const FProperty& InProperty)
	{
		return InProperty.HasMetaData("DisplayInScalabilityContext");		
	};

	auto IsParentPropertyVisible = [&](const TArray<const FProperty*> ParentProperties)
	{
		for(const FProperty* Property : ParentProperties)
		{
			if(ShouldPropertyBeVisible(*Property))
			{
				return true;
			}
		}

		return false;
	};
	
	if (InPropertyAndParent.Property.IsA<FStructProperty>())
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(&InPropertyAndParent.Property);
		for (TFieldIterator<FProperty> PropertyIt(StructProperty->Struct); PropertyIt; ++PropertyIt)
		{
			if (ShouldPropertyBeVisible(**PropertyIt))
			{
				return true;
			}
		}
	}

	bool bShowProperty = ShouldPropertyBeVisible(InPropertyAndParent.Property);
	bShowProperty = bShowProperty || IsParentPropertyVisible(InPropertyAndParent.ParentProperties);
	return  bShowProperty;
}

#undef LOCTEXT_NAMESPACE
