// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "NiagaraSystem.h"
#include "ViewModels/NiagaraSystemScalabilityViewModel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SCompoundWidget.h"

struct FPropertyAndParent;

class SNiagaraScalabilityContext : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SNiagaraScalabilityContext)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UNiagaraSystemScalabilityViewModel& InScalabilityViewModel);
	virtual ~SNiagaraScalabilityContext() override;
	
	void SetObject(UObject* Object);

	void UpdateScalabilityContent();
private:
	bool FilterScalabilityProperties(const FPropertyAndParent& InPropertyAndParent) const;
	TWeakObjectPtr<UNiagaraSystemScalabilityViewModel> ScalabilityViewModel;
	TSharedPtr<IDetailsView> DetailsView;
};
