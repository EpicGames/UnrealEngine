// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraSystemResolvedScalabilitySettings.h"
#include "Stateless/NiagaraStatelessEmitter.h"

#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "ISinglePropertyView.h"
#include "NiagaraEditorStyle.h"
#include "NiagaraObjectSelection.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateIconFinder.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraOverviewGraphViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SExpandableButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/Stack/NiagaraStackEmitterPropertiesGroup.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackSystemSettingsGroup.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "NiagaraScalabilitySystem"

static const TCHAR* ColumnPropertyName = TEXT("Scalability Setting");
static const TCHAR* ColumnDefaultValue = TEXT("Default");
static const TCHAR* ColumnResolvedValue = TEXT("Override");
static const TCHAR* ColumnLocate = TEXT("Locate");


class FNiagaraEmptyPlatformCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;
};

TSharedRef<IPropertyTypeCustomization> FNiagaraEmptyPlatformCustomization::MakeInstance()
{
	return MakeShared<FNiagaraEmptyPlatformCustomization>();
}

void FNiagaraEmptyPlatformCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle,
                                                         FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FNiagaraEmptyPlatformCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle,
                                                           IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

bool IsEmitterScalabilityValueOverridden(FName PropertyName, const FNiagaraEmitterScalabilityOverride& ActiveOverrides)
{
	if(PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraEmitterScalabilityOverride, SpawnCountScale))
	{
		return ActiveOverrides.bOverrideSpawnCountScale;
	}

	// if new overrides get added, this function has to be extended
	check(false);
	return false;
}

bool IsSystemScalabilityValueOverridden(FName PropertyName, const FNiagaraSystemScalabilityOverride& ActiveOverrides)
{
	if(PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraSystemScalabilitySettings, MaxDistance))
	{
		return ActiveOverrides.bOverrideDistanceSettings;
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraSystemScalabilitySettings, MaxInstances))
	{
		return ActiveOverrides.bOverrideInstanceCountSettings;
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraSystemScalabilitySettings, MaxSystemInstances))
	{
		return ActiveOverrides.bOverridePerSystemInstanceCountSettings;
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraSystemScalabilitySettings, VisibilityCulling))
	{
		return ActiveOverrides.bOverrideVisibilitySettings;
	}
	// cull proxy mode and max system proxies are controlled via the same override bool
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraSystemScalabilitySettings, CullProxyMode))
	{
		return ActiveOverrides.bOverrideCullProxySettings;
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraSystemScalabilitySettings, MaxSystemProxies))
	{
		return ActiveOverrides.bOverrideCullProxySettings;
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(FNiagaraSystemScalabilitySettings, BudgetScaling))
	{
		return ActiveOverrides.bOverrideGlobalBudgetScalingSettings;
	}

	// if new overrides get added, this function has to be extended
	check(false);
	return false;
}

void SScalabilityResolvedRow::Construct(const FArguments& InArgs, TSharedRef<FScalabilityRowData> InScalabilityRowData, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<SNiagaraSystemResolvedScalabilitySettings>& InParentWidget)
{
	ScalabilityRowData = InScalabilityRowData;
	ParentWidget = InParentWidget;

	auto Args = FSuperRowType::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));
	SMultiColumnTableRow<TSharedPtr<FScalabilityRowData>>::Construct(Args, OwnerTableView);
}

TSharedRef<SWidget> SScalabilityResolvedRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	auto IconColorActive = [&](bool bActive) -> FLinearColor
	{
		return bActive ? FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.Scalability.System.Feature.Active") : FNiagaraEditorStyle::Get().GetColor("NiagaraEditor.Scalability.System.Feature.Inactive");
	};

	static FNiagaraSystemScalabilityOverride EmptySystemOverride;
	static FNiagaraEmitterScalabilityOverride EmptyEmitterOverride;

	const FNiagaraSystemScalabilityOverride* CurrentSystemOverride = &EmptySystemOverride;
	const FNiagaraEmitterScalabilityOverride* CurrentEmitterOverride = &EmptyEmitterOverride;
	
	bool bIsResolvedNodeEditable = ScalabilityRowData->ResolvedTreeNode->CreatePropertyHandle()->IsEditable();
	bool bIsDefaultNodeEditable = false;

	if(ScalabilityRowData->DefaultTreeNode.IsValid())
	{
		bIsDefaultNodeEditable = ScalabilityRowData->DefaultTreeNode->CreatePropertyHandle()->IsEditable();
	}
	
	if(UNiagaraSystem* System = Cast<UNiagaraSystem>(ScalabilityRowData->OwningObject))
	{
		for(FNiagaraSystemScalabilityOverride& SystemScalabilityOverride : System->GetScalabilityOverrides().Overrides)
		{
			if(ParentWidget.Pin()->GetSystemViewModel()->GetScalabilityViewModel()->IsPlatformActive(SystemScalabilityOverride.Platforms))
			{
				CurrentSystemOverride = &SystemScalabilityOverride;
			}
		}
	}

	if(UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(ScalabilityRowData->OwningObject))
	{
		for(FNiagaraEmitterScalabilityOverride& EmitterScalabilityOverride : Emitter->GetEmitterData(ScalabilityRowData->OwningObjectVersion)->ScalabilityOverrides.Overrides)
		{
			if(ParentWidget.Pin()->GetSystemViewModel()->GetScalabilityViewModel()->IsPlatformActive(EmitterScalabilityOverride.Platforms))
			{
				CurrentEmitterOverride = &EmitterScalabilityOverride;
			}
		}
	}
	else if (UNiagaraStatelessEmitter* StatelessEmitter = Cast<UNiagaraStatelessEmitter>(ScalabilityRowData->OwningObject))
	{
		for (FNiagaraEmitterScalabilityOverride& EmitterScalabilityOverride : StatelessEmitter->GetScalabilityOverrides().Overrides)
		{
			if (ParentWidget.Pin()->GetSystemViewModel()->GetScalabilityViewModel()->IsPlatformActive(EmitterScalabilityOverride.Platforms))
			{
				CurrentEmitterOverride = &EmitterScalabilityOverride;
			}
		}
	}

	// we need to check whether the property is overridden so we can determine whether to check the resolved or default property for a scalability property being active

	bool bIsPropertyOverridden = false;
	if(ScalabilityRowData->OwningObject->IsA<UNiagaraEmitter>())
	{
		bIsPropertyOverridden = IsEmitterScalabilityValueOverridden(ScalabilityRowData->ResolvedTreeNode->GetNodeName(), *CurrentEmitterOverride);
	}
	else if(ScalabilityRowData->OwningObject->IsA<UNiagaraSystem>())
	{
		bIsPropertyOverridden = IsSystemScalabilityValueOverridden(ScalabilityRowData->ResolvedTreeNode->GetNodeName(), *CurrentSystemOverride);
	}
	
	TSharedPtr<SWidget> RowContent;
	if(ColumnId == ColumnPropertyName)
	{
		RowContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock).Text(FText::FromName(ScalabilityRowData->ScalabilityAttribute)).ColorAndOpacity(IconColorActive(bIsPropertyOverridden ? bIsResolvedNodeEditable : bIsDefaultNodeEditable))
		];
	}
	else if(ColumnId == ColumnDefaultValue)
	{
		if(ScalabilityRowData->DefaultPropertyHandle.IsValid())
		{
			FText DefaultValueText;
			ScalabilityRowData->DefaultPropertyHandle->GetValueAsDisplayText(DefaultValueText);			
			RowContent = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(DefaultValueText).ColorAndOpacity(IconColorActive(bIsDefaultNodeEditable))
			];
		}
		else
		{
			RowContent = SNullWidget::NullWidget;
		}
	}
	// for the override value, we only add an entry if an override is active 
	else if(ColumnId == ColumnResolvedValue)
	{	
		if(bIsPropertyOverridden)
		{
			FText ResolvedValueText;
			ScalabilityRowData->ResolvedPropertyHandle->GetValueAsDisplayText(ResolvedValueText);
			
			RowContent = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			.VAlign(VAlign_Center)
			[
				SNew(SImage).Image(FNiagaraEditorStyle::Get().GetBrush("NiagaraEditor.Scalability.System.Feature.Override"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(ResolvedValueText).ColorAndOpacity(IconColorActive(bIsResolvedNodeEditable))
			];			
		}
		else
		{
			RowContent = SNullWidget::NullWidget;
		}
	}
	// @Todo this won't be called currently as we removed the column from the header row
	else if(ColumnId == ColumnLocate)
	{
		RowContent = SNew(SButton).OnClicked(this, &SScalabilityResolvedRow::NavigateToValue, ScalabilityRowData);
	}

	return SNew(SBox).MinDesiredHeight(24.f)
		[
			RowContent.ToSharedRef()
		];
}

FReply SScalabilityResolvedRow::NavigateToValue(TSharedPtr<FScalabilityRowData> RowData)
{
	if(!RowData.IsValid())
	{
		return FReply::Handled();
	}
	
	//ParentWidget.Pin()->GetSystemViewModel()->GetScalabilityViewModel()->NavigateToScalabilityProperty(RowData->OwningObject, FName(RowData->DefaultPropertyHandle->GeneratePathToProperty()));
	return FReply::Unhandled();
}

void SNiagaraSystemResolvedScalabilitySettings::RebuildWidget()
{
	ResolvedScalabilityContainer->ClearChildren();

	TArray<UNiagaraStackEntry*> StackEntries;
	SystemViewModel->GetSelectionViewModel()->GetSelectedEntries(StackEntries);

	UObject* NewSelection = nullptr;
	FGuid ObjectVersion;
	if(StackEntries.Num() == 1)
	{
		UNiagaraStackEntry* StackEntry = StackEntries[0];
		
		if(StackEntry->IsA(UNiagaraStackEmitterPropertiesGroup::StaticClass()))
		{
			UNiagaraEmitter* Emitter = Cast<UNiagaraStackEmitterPropertiesGroup>(StackEntry)->GetEmitterViewModel()->GetEmitter().Emitter;
			ObjectVersion = Cast<UNiagaraStackEmitterPropertiesGroup>(StackEntry)->GetEmitterViewModel()->GetEmitter().Version;
			NewSelection = Emitter;
		}
		else if(StackEntry->IsA(UNiagaraStackSystemPropertiesGroup::StaticClass()))
		{
			NewSelection = System.Get();
		}
	}
	
	TSet<UNiagaraOverviewNode*> SelectedOverviewNodes;

	for(UObject* SelectedNode : SystemViewModel->GetOverviewGraphViewModel()->GetNodeSelection()->GetSelectedObjectsResolved())
	{
		if(UNiagaraOverviewNode* OverviewNode = Cast<UNiagaraOverviewNode>(SelectedNode))
		{
			SelectedOverviewNodes.Add(OverviewNode);
		}
	}

	// we prioritize stack selection
	if(NewSelection != nullptr)
	{
		if(UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(NewSelection))
		{
			Emitter->OnPropertiesChanged().RemoveAll(this);
			Emitter->OnPropertiesChanged().AddSP(this, &SNiagaraSystemResolvedScalabilitySettings::RebuildWidget);

			ResolvedScalabilityContainer->AddSlot()
			.AutoHeight()
			[
				GenerateResolvedScalabilityTable(Emitter, ObjectVersion)
			];
		}
		if(UNiagaraSystem* NiagaraSystem = Cast<UNiagaraSystem>(NewSelection))
		{
			ResolvedScalabilityContainer->AddSlot()
			.AutoHeight()
			[
				GenerateResolvedScalabilityTable(NiagaraSystem, FGuid())
			];
		}
	}
	// we use node selection if we don't have any stack selection
	else if(SelectedOverviewNodes.Num() > 0)
	{
		for(UObject* SelectedNode : SelectedOverviewNodes)
		{
			if(UNiagaraOverviewNode* OverviewNode = Cast<UNiagaraOverviewNode>(SelectedNode))
			{
				if(FNiagaraEmitterHandle* Handle = OverviewNode->TryGetEmitterHandle())
				{
					FVersionedNiagaraEmitter VersionedEmitter = Handle->GetInstance();
					VersionedEmitter.Emitter->OnPropertiesChanged().RemoveAll(this);
					VersionedEmitter.Emitter->OnPropertiesChanged().AddSP(this, &SNiagaraSystemResolvedScalabilitySettings::RebuildWidget);
					ResolvedScalabilityContainer->AddSlot()
					.AutoHeight()
					[
						GenerateResolvedScalabilityTable(VersionedEmitter.Emitter, VersionedEmitter.Version)
					];
				}
				else
				{
					ResolvedScalabilityContainer->AddSlot()
					.AutoHeight()
					[
						GenerateResolvedScalabilityTable(OverviewNode->GetOwningSystem(), FGuid())
					];
				}
			}
		}
	}
	else
	{
		ResolvedScalabilityContainer->AddSlot()
		[
			GenerateResolvedScalabilityTable(&SystemViewModel->GetSystem(), FGuid())
		];
		
		for(const TSharedRef<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel : SystemViewModel->GetEmitterHandleViewModels())
		{
			if(EmitterHandleViewModel->IsValid() == false)
			{
				continue;
			}
			
			FVersionedNiagaraEmitter VersionedEmitter = EmitterHandleViewModel->GetEmitterHandle()->GetInstance();
			VersionedEmitter.Emitter->OnPropertiesChanged().RemoveAll(this);
			VersionedEmitter.Emitter->OnPropertiesChanged().AddSP(this, &SNiagaraSystemResolvedScalabilitySettings::RebuildWidget);
			ResolvedScalabilityContainer->AddSlot()
			[
				GenerateResolvedScalabilityTable(VersionedEmitter.Emitter, VersionedEmitter.Version)
			];	
		}
	}
	
	ChildSlot
	[
		ResolvedScalabilityContainer.ToSharedRef()
	];
}

TSharedRef<ITableRow> SNiagaraSystemResolvedScalabilitySettings::GenerateScalabilityValueRow(TSharedRef<FScalabilityRowData> ScalabilityRowData, const TSharedRef<STableViewBase>& TableViewBase)
{
	return SNew(SScalabilityResolvedRow, ScalabilityRowData, TableViewBase, SharedThis(this));
}

TSharedRef<SWidget> SNiagaraSystemResolvedScalabilitySettings::GenerateResolvedScalabilityTable(UObject* Object, const FGuid& Version)
{
	FString DisplayName = Object->GetName();

	FPropertyEditorModule& EditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FPropertyRowGeneratorArgs Args;
	TSharedRef<IPropertyRowGenerator> ResolvedPropertyGenerator = EditorModule.CreatePropertyRowGenerator(Args);
	ResolvedPropertyGenerator->RegisterInstancedCustomPropertyTypeLayout(FName(FNiagaraPlatformSet::StaticStruct()->GetName()), FOnGetPropertyTypeCustomizationInstance::CreateStatic(FNiagaraEmptyPlatformCustomization::MakeInstance));

	TSharedRef<IPropertyRowGenerator> DefaultPropertyGenerator = EditorModule.CreatePropertyRowGenerator(Args);
	DefaultPropertyGenerator->RegisterInstancedCustomPropertyTypeLayout(FName(FNiagaraPlatformSet::StaticStruct()->GetName()), FOnGetPropertyTypeCustomizationInstance::CreateStatic(FNiagaraEmptyPlatformCustomization::MakeInstance));

	TSharedPtr<FStructOnScope> DefaultValueStruct = nullptr;
	TSharedPtr<FStructOnScope> ResolvedValueStruct = nullptr;

	if(UNiagaraEmitter* Emitter = Cast<UNiagaraEmitter>(Object))
	{
		if(UNiagaraEffectType* EffectType = System->GetEffectType())
		{
			const FNiagaraEmitterScalabilitySettings* DefaultEmitterScalabilitySettings = nullptr;

			for(const FNiagaraEmitterScalabilitySettings& EmitterScalabilitySettings : EffectType->GetEmitterScalabilitySettings().Settings)
			{
				// for the effect type's settings we haven't bound the platform QL & profile overrides, so we use a customized function to check if a platformset is active under scalability preview
				if(GetSystemViewModel()->GetScalabilityViewModel()->IsPlatformActive(EmitterScalabilitySettings.Platforms))
				{
					DefaultEmitterScalabilitySettings = &EmitterScalabilitySettings;
				}
			}

			if(DefaultEmitterScalabilitySettings != nullptr)
			{
				DefaultValueStruct = MakeShared<FStructOnScope>(FNiagaraEmitterScalabilitySettings::StaticStruct(), (uint8*) DefaultEmitterScalabilitySettings);			
			}
		}
		const FNiagaraEmitterScalabilitySettings& EmitterScalabilitySettings = Emitter->GetEmitterData(Version)->GetScalabilitySettings();
		ResolvedValueStruct = MakeShared<FStructOnScope>(FNiagaraEmitterScalabilitySettings::StaticStruct(), (uint8*) &EmitterScalabilitySettings);
	}
	else if(UNiagaraSystem* NiagaraSystem = Cast<UNiagaraSystem>(Object))
	{
		if(UNiagaraEffectType* EffectType = NiagaraSystem->GetEffectType())
		{
			const FNiagaraSystemScalabilitySettings* DefaultSystemScalabilitySettings = nullptr;
			for(const FNiagaraSystemScalabilitySettings& SystemScalabilitySettings : EffectType->GetSystemScalabilitySettings().Settings)
			{
				// see above
				if(GetSystemViewModel()->GetScalabilityViewModel()->IsPlatformActive(SystemScalabilitySettings.Platforms))
				{
					DefaultSystemScalabilitySettings = &SystemScalabilitySettings;
				}
			}

			if(DefaultSystemScalabilitySettings != nullptr)
			{
				DefaultValueStruct = MakeShared<FStructOnScope>(FNiagaraSystemScalabilitySettings::StaticStruct(), (uint8*) DefaultSystemScalabilitySettings);			
			}
		}
		const FNiagaraSystemScalabilitySettings& SystemScalabilitySettings = NiagaraSystem->GetScalabilitySettings();
		ResolvedValueStruct = MakeShared<FStructOnScope>(FNiagaraSystemScalabilitySettings::StaticStruct(), (uint8*) &SystemScalabilitySettings);
	}

	ensure(ResolvedValueStruct.IsValid());
	
    ResolvedPropertyGenerator->SetStructure(ResolvedValueStruct);

	// we are caching the default handles so we can later find the default handle corresponding to a resolved handle
	TMap<FString, TSharedRef<IDetailTreeNode>> DefaultHandles;
	if(DefaultValueStruct.IsValid())
	{
		DefaultPropertyGenerator->SetStructure(DefaultValueStruct);
		TArray<TSharedRef<IDetailTreeNode>> DefaultTreeNodes = DefaultPropertyGenerator->GetRootTreeNodes();
		
		for(TSharedRef<IDetailTreeNode> RootNode : DefaultTreeNodes)
		{
			if(RootNode->GetNodeName().IsEqual("Scalability"))
			{
				TArray<TSharedRef<IDetailTreeNode>> Children;
				RootNode->GetChildren(Children);

				for(const TSharedRef<IDetailTreeNode>& Child : Children)
				{
					TSharedPtr<IPropertyHandle> ChildPropertyHandle = Child->CreatePropertyHandle();
				
					if(ChildPropertyHandle->HasMetaData("DisplayInScalabilityValuesBar"))
					{
						DefaultHandles.Add(ChildPropertyHandle->GetProperty()->GetName(), Child);
					}
				}
			}
		}
	}	

	// we now create the scalability row data necessary to create a list view
	TArray<TSharedRef<FScalabilityRowData>> ScalabilityValues;
	const TArray<TSharedRef<IDetailTreeNode>> RootTreeNodes = ResolvedPropertyGenerator->GetRootTreeNodes();
	for(TSharedRef<IDetailTreeNode> RootNode : RootTreeNodes)
	{
		if(RootNode->GetNodeName().IsEqual("Scalability"))
		{
			TArray<TSharedRef<IDetailTreeNode>> Children;
			RootNode->GetChildren(Children);

			for(const TSharedRef<IDetailTreeNode>& Child : Children)
			{
				TSharedPtr<IPropertyHandle> ChildPropertyHandle = Child->CreatePropertyHandle();
				
				if(ChildPropertyHandle->HasMetaData("DisplayInScalabilityValuesBar"))
				{
					TSharedRef<FScalabilityRowData> ScalabilityValue = MakeShared<FScalabilityRowData>();
					ScalabilityValue->ScalabilityAttribute = FName(ChildPropertyHandle->GetPropertyDisplayName().ToString());
					ScalabilityValue->ResolvedPropertyHandle = ChildPropertyHandle.ToSharedRef();
					ScalabilityValue->ResolvedTreeNode = Child;
					ScalabilityValue->OwningObject = Object;
					ScalabilityValue->OwningObjectVersion = Version;
					//ScalabilityValue->bIsOverridden = Object->IsA<UNiagaraEmitter>() ? IsEmitterScalabilityValueOverridden(Child->GetNodeName(), )

					// if we found the same handle in the default handles, we have both handles needed to display default and override data
					if(DefaultHandles.Contains(ChildPropertyHandle->GetProperty()->GetName()))
					{
						ScalabilityValue->DefaultTreeNode = DefaultHandles[ChildPropertyHandle->GetProperty()->GetName()];
						ScalabilityValue->DefaultPropertyHandle = DefaultHandles[ChildPropertyHandle->GetProperty()->GetName()]->CreatePropertyHandle();
					}
					
					// @todo
					// ScalabilityValue->SourceProperties = ;
					ScalabilityValues.Add(ScalabilityValue);
				}
			}
		}
	}

	// we cache the objects to maintain their lifetimes while used
	FRequiredInstanceInformation InstanceInfo;
	InstanceInfo.DefaultPropertyRowGenerator = DefaultPropertyGenerator;
	InstanceInfo.ResolvedPropertyRowGenerator = ResolvedPropertyGenerator;
	for (const TSharedRef<FScalabilityRowData>& Value : ScalabilityValues)
	{
		InstanceInfo.ScalabilityValues->Add(Value);
	}
	InstanceInformation.Add(Object, InstanceInfo);
	const FSlateBrush* AssetIcon = FSlateIconFinder::FindIconBrushForClass(Object->GetClass());
	
	TSharedRef<SWidget> Widget = SNew(SExpandableArea)
	.HeaderContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(3.f)
		[
			SNew(SImage)
			.Image(AssetIcon)
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SInlineEditableTextBlock)
			.Text(FText::FromString(DisplayName))
			.Style(FAppStyle::Get(), "DetailsView.NameTextBlockStyle")
			.IsReadOnly(true)
		]
	]
	.BodyContent()
	[	
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
		[
			SNew(SListView<TSharedRef<FScalabilityRowData>>)
			.ListItemsSource(InstanceInformation[Object].ScalabilityValues)
			.OnGenerateRow(this, &SNiagaraSystemResolvedScalabilitySettings::GenerateScalabilityValueRow)
			.HeaderRow
			(
				SNew(SHeaderRow)
                + SHeaderRow::Column(ColumnPropertyName)
                	.DefaultLabel(LOCTEXT("ScalabilitySettingLabel", "Scalability Setting"))
                	.DefaultTooltip(LOCTEXT("ScalabilitySettingLabelTooltip", "Scalability Settings can be turned on or off and generally can be overridden."))
                	.FillWidth(0.5f)
                + SHeaderRow::Column(ColumnDefaultValue)
                	.DefaultLabel(LOCTEXT("ScalabilityDefaultValueLabel", "Default"))
                	.DefaultTooltip(LOCTEXT("ScalabilityDefaultValueLabelTooltip", "Default values come from the effect type assigned to a system."))
                	.FillWidth(0.2f)
                + SHeaderRow::Column(ColumnResolvedValue)
                	.DefaultLabel(LOCTEXT("ScalabilityOverrideValueLabel", "Override"))
                	.DefaultTooltip(LOCTEXT("ScalabilityOverrideValueLabelTooltip", "Overrides can be specified in a system or emitter to override an effect type's default scalability settings."))
                	.FillWidth(0.25f)
                	// @Todo: Add locate functionality. This is involved as the active scalability tab has no functionality yet to select certain rows
                	//+ SHeaderRow::Column(ColumnLocate)
					// .DefaultLabel(LOCTEXT("ScalabilityLocateValueLabel", ""))
					// .DefaultTooltip(LOCTEXT("BlueprintWarningBehaviorHeaderTooltip", "Determines what happens when the warning is raised - warnings can be treated more strictly or suppressed entirely"))
					// .FixedWidth(20.f)
            )
			
		]
	];

	return Widget;
}

SNiagaraSystemResolvedScalabilitySettings::~SNiagaraSystemResolvedScalabilitySettings()
{
	System->OnScalabilityChanged().RemoveAll(this);
}

void SNiagaraSystemResolvedScalabilitySettings::Construct(const FArguments& InArgs, UNiagaraSystem& InSystem, TSharedPtr<FNiagaraSystemViewModel> InSystemViewModel)
{
	System = &InSystem;
	SystemViewModel = InSystemViewModel;

	ResolvedScalabilityContainer = SNew(SVerticalBox);

	SystemViewModel->GetOverviewGraphViewModel()->GetNodeSelection()->OnSelectedObjectsChanged().AddSP(this, &SNiagaraSystemResolvedScalabilitySettings::RebuildWidget);
	SystemViewModel->GetSelectionViewModel()->OnEntrySelectionChanged().AddSP(this, &SNiagaraSystemResolvedScalabilitySettings::RebuildWidget);
	System->OnScalabilityChanged().AddSP(this, &SNiagaraSystemResolvedScalabilitySettings::RebuildWidget);
	
	RebuildWidget();
}

#undef LOCTEXT_NAMESPACE
