// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraSystem.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class IDetailTreeNode;
class IPropertyRowGenerator;

struct FScalabilityRowData
{
	FName ScalabilityAttribute;
	TSharedPtr<IDetailTreeNode> ResolvedTreeNode;
	TSharedPtr<IDetailTreeNode> DefaultTreeNode;
	TSharedPtr<IPropertyHandle> ResolvedPropertyHandle;
	TSharedPtr<IPropertyHandle> DefaultPropertyHandle;
	TArray<FProperty*> SourceProperties;
	UObject* OwningObject;
	FGuid OwningObjectVersion;
};

class SScalabilityResolvedRow : public SMultiColumnTableRow<TSharedPtr<FScalabilityRowData>>
{
public:
	SLATE_BEGIN_ARGS(SScalabilityResolvedRow) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TSharedRef<FScalabilityRowData> InScalabilityRowData, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<class SNiagaraSystemResolvedScalabilitySettings>& ParentWidget);
	
	/** SMultiColumnTableRow interface */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

	FReply NavigateToValue(TSharedPtr<FScalabilityRowData> RowData);
private:
	TSharedPtr<FScalabilityRowData> ScalabilityRowData;
	TWeakPtr<class SNiagaraSystemResolvedScalabilitySettings> ParentWidget;
};

class SNiagaraSystemResolvedScalabilitySettings : public SCompoundWidget
{
	// struct used to maintain lifetime
	struct FRequiredInstanceInformation
	{
		TSharedPtr<IPropertyRowGenerator> DefaultPropertyRowGenerator;
		TSharedPtr<IPropertyRowGenerator> ResolvedPropertyRowGenerator;
		TSharedRef<UE::Slate::Containers::TObservableArray<TSharedRef<FScalabilityRowData>>> ScalabilityValues = MakeShared<UE::Slate::Containers::TObservableArray<TSharedRef<FScalabilityRowData>>>();
	};
public:
	SLATE_BEGIN_ARGS(SNiagaraSystemResolvedScalabilitySettings)
	{}
	SLATE_END_ARGS()

	virtual ~SNiagaraSystemResolvedScalabilitySettings();
	
	void Construct(const FArguments& InArgs, UNiagaraSystem& InSystem, TSharedPtr<FNiagaraSystemViewModel> SystemViewModel);
	void RebuildWidget();
	TSharedPtr<FNiagaraSystemViewModel> GetSystemViewModel() { return SystemViewModel; }
private:
	TWeakObjectPtr<UNiagaraSystem> System;
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
	TSharedPtr<SVerticalBox> ResolvedScalabilityContainer;
private:
	TMap<UObject*, FRequiredInstanceInformation> InstanceInformation;
	
	/** Generates an expandable widget for an emitter or system */
	TSharedRef<SWidget> GenerateResolvedScalabilityTable(UObject* Object, const FGuid& Version);
	
	TSharedRef<ITableRow> GenerateScalabilityValueRow(TSharedRef<FScalabilityRowData> ScalabilityRowData, const TSharedRef< class STableViewBase >& TableViewBase);
};
