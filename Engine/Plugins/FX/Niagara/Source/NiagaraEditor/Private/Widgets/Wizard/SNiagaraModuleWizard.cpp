// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Wizard/SNiagaraModuleWizard.h"

#include "NiagaraClipboard.h"
#include "NiagaraNodeOp.h"
#include "NiagaraNodeParameterMapGet.h"
#include "NiagaraNodeParameterMapSet.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"
#include "Framework/Application/SlateApplication.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"

#define LOCTEXT_NAMESPACE "FNiagaraWizard"

using namespace UE::Niagara::Wizard;

TSharedRef<SWidget> FModuleWizardPage::GetContent()
{
	return SNullWidget::NullWidget;
}

void FModuleWizardPage::RefreshContent()
{
}

TArray<FModuleWizardModel::FModuleCreationEntry> FModuleWizardModel::GetModulesToCreate(UNiagaraNodeOutput* ProvidedOutputNode, int32 ProvidedTargetIndex, TSharedPtr<FNiagaraSystemViewModel> SystemModel, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel)
{
	TArray<FModuleCreationEntry> Result;
	Result.Add({ProvidedOutputNode, ProvidedTargetIndex});
	return Result;
}

void FModuleWizardModel::GenerateNewModuleContent(TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel, const TArray<const UNiagaraNodeFunctionCall*>& PreviouslyCreatedModules)
{
	ScratchPadScriptViewModel->SetScriptName(FText::FromString(TEXT("Read")));
}

bool FModuleWizardModel::UpdateModuleInputs(UNiagaraClipboardContent* NewModule, const TArray<const UNiagaraNodeFunctionCall*>& PreviouslyCreatedModules)
{
	return false;
}

void SNiagaraModuleWizard::Construct(const FArguments& InArgs, TSharedRef<FModuleWizardModel> InModel)
{
	Model = InModel;
	OnCreateModule = InArgs._OnCreateModule;

	SWizard::FArguments SuperArgs;
	SuperArgs
	.ShowPageList(false)
	.ShowBreadcrumbs(true)
	.ShowPageTitle(false)
	.FinishButtonText(LOCTEXT("FinishButtonLabel", "Create Module"))
	.FinishButtonToolTip(LOCTEXT("FinishButtonTooltip", "Create the module and close the wizard"))
	.OnFinished(this, &SNiagaraModuleWizard::OnFinished)
	.OnCanceled(this, &SNiagaraModuleWizard::CloseContainingWindow)
	.CanFinish(this, &SNiagaraModuleWizard::CanFinish);

	for (int i = 0; i < Model->Pages.Num(); i++)
	{
		TSharedRef<FModuleWizardPage> Page = Model->Pages[i];
		FWizardPage::FArguments PageArgs = SWizard::Page()
			.CanShow_Lambda([this, Index = i]()
			{
				return Index == 0 ? true : Model->Pages[Index - 1]->CanGoToNextPage();
			})
			.OnEnter(Page, &FModuleWizardPage::RefreshContent)
			.Name(Page->Name)
			[
				Page->GetContent()
			];
		SuperArgs.Slots.Add(new FWizardPage(PageArgs));
	}
	SWizard::Construct(SuperArgs);
}

TArray<const UNiagaraNodeFunctionCall*> SNiagaraModuleWizard::AddModulesToStack(TSharedPtr<FNiagaraSystemViewModel> SystemViewModel, UNiagaraNodeOutput* OutputNode, int32 TargetIndex, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel, UNiagaraStackEditorData* StackEditorData)
{
	// the wizard can potentially create more than one module
	TArray<const UNiagaraNodeFunctionCall*> CreatedModules;
	TArray<FModuleWizardModel::FModuleCreationEntry> TargetModules = Model->GetModulesToCreate(OutputNode, TargetIndex, SystemViewModel, EmitterViewModel);
	for (const FModuleWizardModel::FModuleCreationEntry& Module : TargetModules)
	{
		check(Module.OutputNode);
		TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel = SystemViewModel->GetScriptScratchPadViewModel()->CreateNewScript(ENiagaraScriptUsage::Module, Module.OutputNode->GetUsage(), FNiagaraTypeDefinition());
		if (!ScratchPadScriptViewModel.IsValid())
		{
			return CreatedModules;
		}
		Model->GenerateNewModuleContent(ScratchPadScriptViewModel, CreatedModules);
		UNiagaraNodeFunctionCall* NewModule = FNiagaraStackGraphUtilities::AddScriptModuleToStack(ScratchPadScriptViewModel->GetOriginalScript(), *Module.OutputNode, Module.TargetIndex);

		// open newly create scratch pad?
		//SystemModel->GetScriptScratchPadViewModel()->FocusScratchPadScriptViewModel(ScratchPadScriptViewModel.ToSharedRef());
		
		UNiagaraStackScriptHierarchyRoot* FunctionHierarchyRoot = NewObject<UNiagaraStackScriptHierarchyRoot>(GetTransientPackage()); 
		UNiagaraStackEntry::FRequiredEntryData RequiredEntryData(SystemViewModel.ToSharedRef(), EmitterViewModel, NAME_None, NAME_None, *StackEditorData);
		FunctionHierarchyRoot->Initialize(RequiredEntryData, *NewModule, *NewModule, FString());
		FunctionHierarchyRoot->RefreshChildren();

		// Reset all direct inputs on this function to initialize data interfaces and default dynamic inputs.
		TArray<UNiagaraStackFunctionInput*> StackFunctionInputs;
		FunctionHierarchyRoot->GetUnfilteredChildrenOfType(StackFunctionInputs, true);
		for (UNiagaraStackFunctionInput* StackFunctionInput : StackFunctionInputs)
		{
			if (StackFunctionInput != nullptr && StackFunctionInput->CanReset())
			{
				StackFunctionInput->Reset();
			}
		}

		// allow the wizard model to change the inputs from the default
		UNiagaraClipboardContent* ModuleInputs = UNiagaraClipboardContent::Create();
		FunctionHierarchyRoot->ToClipboardFunctionInputs(ModuleInputs, MutableView(ModuleInputs->FunctionInputs));
		if (Model->UpdateModuleInputs(ModuleInputs, CreatedModules))
		{
			FunctionHierarchyRoot->SetValuesFromClipboardFunctionInputs(ModuleInputs->FunctionInputs);
		}
		FunctionHierarchyRoot->Finalize();
		FunctionHierarchyRoot->ConditionalBeginDestroy();
		SystemViewModel->NotifyDataObjectChanged(TArray<UObject*>(), ENiagaraDataObjectChange::Unknown);
	
		FNiagaraStackGraphUtilities::RelayoutGraph(*OutputNode->GetGraph());
		CreatedModules.Add(NewModule);
	}
	return CreatedModules;
}

bool SNiagaraModuleWizard::CanFinish() const
{
	for (TSharedRef<FModuleWizardPage> Page : Model->Pages)
	{
		if (!Page->CanCompleteWizard())
		{
			return false;
		}
	}
	return true;
}

void SNiagaraModuleWizard::OnFinished()
{
	if (Model.IsValid())
	{
		for (TSharedRef<FModuleWizardPage> Page : Model->Pages)
		{
			Page->RefreshContent();
		}
	}
	OnCreateModule.ExecuteIfBound();

	CloseContainingWindow();
}

void SNiagaraModuleWizard::CloseContainingWindow()
{
	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}


UEdGraphPin* Utilities::AddReadParameterPin(const FNiagaraTypeDefinition& Type, const FName& Name, UNiagaraNodeParameterMapGet* MapGetNode)
{
	UEdGraphPin* Pin = MapGetNode->AddParameterPin(FNiagaraVariable(Type, Name), EGPD_Output);
	MapGetNode->CancelEditablePinName(FText::GetEmpty(), Pin);
	return Pin;
}

UEdGraphPin* Utilities::AddWriteParameterPin(const FNiagaraTypeDefinition& Type, const FName& Name, UNiagaraNodeParameterMapSet* MapSetNode)
{
	UEdGraphPin* Pin = MapSetNode->AddParameterPin(FNiagaraVariable(Type, Name), EGPD_Input);
	MapSetNode->CancelEditablePinName(FText::GetEmpty(), Pin);
	return Pin;
}

TSharedRef<IDetailsView> Utilities::CreateDetailsView()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bShowObjectLabel = false;

	return PropertyModule.CreateDetailView(DetailsViewArgs);
}

UNiagaraNodeFunctionCall* Utilities::CreateDataInterfaceFunctionNode(const TSubclassOf<UNiagaraDataInterface>& DataInterfaceClass, const FName& FunctionName, UNiagaraGraph* Graph)
{
	UNiagaraDataInterface* DataInterfaceCDO = DataInterfaceClass.GetDefaultObject();
	TArray<FNiagaraFunctionSignature> Functions;
	DataInterfaceCDO->GetFunctionSignatures(Functions);
	for (FNiagaraFunctionSignature& Sig : Functions)
	{
		if (Sig.Name == FunctionName)
		{
			UNiagaraNodeFunctionCall* FuncNode = NewObject<UNiagaraNodeFunctionCall>(Graph);
			FuncNode->Signature = Sig;
			FuncNode->SetFlags(RF_Transactional);
			Graph->AddNode(FuncNode, false, false);

			FuncNode->CreateNewGuid();
			FuncNode->PostPlacedNewNode();
			FuncNode->AllocateDefaultPins();
			return FuncNode;
		}
	}
	return nullptr;
}

UNiagaraNodeFunctionCall* Utilities::CreateFunctionCallNode(UNiagaraScript* FunctionScript, UNiagaraGraph* Graph)
{
	if (FunctionScript)
	{
		UNiagaraNodeFunctionCall* FuncNode = NewObject<UNiagaraNodeFunctionCall>(Graph);
		FuncNode->FunctionScript = FunctionScript;
		FuncNode->SelectedScriptVersion = FunctionScript->GetExposedVersion().VersionGuid;
		FuncNode->SetFlags(RF_Transactional);
		Graph->AddNode(FuncNode, false, false);

		FuncNode->CreateNewGuid();
		FuncNode->PostPlacedNewNode();
		FuncNode->AllocateDefaultPins();
		return FuncNode;
	}
	return nullptr;
}

UNiagaraNodeOp* Utilities::CreateOpNode(const FName& OpName, UNiagaraGraph* Graph)
{
	UNiagaraNodeOp* NewOpNode = NewObject<UNiagaraNodeOp>(Graph);
	NewOpNode->OpName = OpName;
	NewOpNode->SetFlags(RF_Transactional);
	Graph->AddNode(NewOpNode, false, false);
	NewOpNode->CreateNewGuid();
	NewOpNode->PostPlacedNewNode();
	NewOpNode->AllocateDefaultPins();
	return NewOpNode;
}

void Utilities::SetDefaultBinding(UNiagaraGraph* Graph, const FName& VarName, const FName& DefaultBinding)
{
	if (UNiagaraScriptVariable* ScriptVariable = Graph->GetScriptVariable(VarName))
	{
		ScriptVariable->DefaultMode = ENiagaraDefaultMode::Binding;
		ScriptVariable->DefaultBinding.SetName(DefaultBinding);
		Graph->ScriptVariableChanged(ScriptVariable->Variable);
	}
}

void Utilities::SetTooltip(UNiagaraGraph* Graph, const FName& VarName, const FText& Tooltip)
{
	if (Graph)
	{
		if (UNiagaraScriptVariable* ScriptVariable = Graph->GetScriptVariable(VarName))
		{
			ScriptVariable->Metadata.Description = Tooltip;
			Graph->ScriptVariableChanged(ScriptVariable->Variable);
		}
	}
}

#undef LOCTEXT_NAMESPACE
