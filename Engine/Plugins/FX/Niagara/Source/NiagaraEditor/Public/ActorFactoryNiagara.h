// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryNiagara.generated.h"

class AActor;
struct FAssetData;

UCLASS(MinimalAPI, config = Editor, collapsecategories, hidecategories = Object)
class UActorFactoryNiagara : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	//~ End UActorFactory Interface
};



