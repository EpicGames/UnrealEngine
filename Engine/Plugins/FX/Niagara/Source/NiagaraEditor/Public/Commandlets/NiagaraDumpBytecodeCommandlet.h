// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "NiagaraDumpBytecodeCommandlet.generated.h"

class UNiagaraScript;
class FNiagaraShaderScript;

UCLASS(config=Editor)
class UNiagaraDumpByteCodeCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	int32 Main(const FString& Params) override;

	/** The folder in which the commandlet's output files will be stored */
	FString AuditOutputFolder;

	/** Only assets in this collection will be considered. If this is left blank, no assets will be filtered by collection */
	FString FilterCollection;

	/** Package paths to include */
	TArray<FName> PackagePaths;

	/** Whether to force rapid iteration parameters to be baked out */
	bool ForceBakedRapidIteration = false;

	/** Whether to force attributes to be trimmed to their minimal set */
	bool ForceAttributeTrimming = false;

	/** Whether the translated hlsl should also be dumped */
	bool DumpTranslatedHlsl = false;

	/** Include GPU scripts */
	bool DumpGpuScripts = false;

	/** */
	bool IncludeDeveloperFolder = false;

private:
	void ProcessNiagaraScripts();
	void ProcessBatch(TArray<FAssetData>& BatchAssets);
	void DumpByteCode(const UNiagaraScript* Script, const FString& PathName, const FString& HashName, const FString& FilePath);
	void DumpShaderScript(const UNiagaraScript* GPUComputeScript, const FString& PathName, const FString& HashName, const FString& FilePath);

	struct FScriptMetaData
	{
		FString SystemHash;
		FString FullName;
		int32 RegisterCount;
		int32 OpCount;
		int32 ConstantCount;
		int32 AttributeCount;
	};

	TArray<FScriptMetaData> ScriptMetaData;

	struct FShaderMetaData
	{
		FString SystemHash;
		FString FullName;
		int32 PermutationIndex;
		FString VertexFactory;
		FString ShaderType;
		uint32 ByteCount;
		FString ShaderHash;
	};
	TArray<FShaderMetaData> ShaderMetaData;
};