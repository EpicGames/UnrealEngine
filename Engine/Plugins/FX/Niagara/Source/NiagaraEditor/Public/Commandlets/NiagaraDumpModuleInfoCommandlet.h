// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "NiagaraDumpModuleInfoCommandlet.generated.h"

class UNiagaraScript;

UCLASS(config=Editor)
class UNiagaraDumpModuleInfoCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	int32 Main(const FString& Params) override;

	/** The folder in which the commandlet's output files will be stored */
	FString AuditOutputFolder;

	/** Only assets in this collection will be considered. If this is left blank, no assets will be filtered by collection */
	FString FilterCollection;

	/** Package paths to include */
	TArray<FName> PackagePaths;

private:
	void ProcessNiagaraScripts();
	void ProcessNiagaraParameterDefinitions();
};