// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "NiagaraStatelessAuditCommandlet.generated.h"

class UNiagaraSystem;

UCLASS(config=Editor)
class UNiagaraStatelessAuditCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

private:
	struct FModuleUsage
	{
		bool			bIsExperimental = false;
		bool			bIsEarlyAccess = false;

		int32			UsageCount = 0;
		TArray<FString>	SystemNames;
		TArray<FString>	EmitterNames;
		TArray<FString> SystemAndEmitterNames;
	};

public:
	int32 Main(const FString& Params) override;

private:
	void ParseParameters(const FString& Params);
	void ProcessSystem(UNiagaraSystem* NiagaraSystem);
	void WriteResults() const;

	TUniquePtr<FArchive> GetOutputFile(const TCHAR* Filename) const;

private:
	FString						AuditOutputFolder;
	TMap<FName, FModuleUsage>	ModuleUsageMap;
	bool						bAnyExperimentalModules = false;
	bool						bAnyEarlyAccessModules = false;
};
