// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "NiagaraSystemAuditCommandlet.generated.h"

UCLASS(config=Editor)
class UNiagaraSystemAuditCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	/** All Niagara systems with warmup time */
	TSet<FString> NiagaraSystemsWithWarmup;
	/** All Niagara systems with lights and the details */
	TSet<FString> NiagaraSystemsWithLights;
	/** All Niagara systems with events */
	TSet<FString> NiagaraSystemsWithEvents;
	/** All Niagara systems with GPU emitters */
	TSet<FString> NiagaraSystemsWithGPUEmitters;
	/** All Niagara systems with prerequisites */
	TSet<FString> NiagaraSystemsWithPrerequisites;
	/** All Niagara systems that contain emitters with dynamic bounds */
	TSet<FString> NiagaraSystemsWithDynamicBounds;
	/** All Niagara systems that contain the requested data interface */
	TSet<FString> NiagaraSystemsWithDataInterfaceToFind;
	/** All Niagara Systems & Emitters with simulation stages enabled*/
	TSet<FString> NiagaraSystemsWithSimulationStages;
	/** All Niagara systems with compression enabled */
	TSet<FString> NiagaraSystemsWithCompression;
	/** All Niagara systems with stateless emitters enabled */
	TSet<FString> NiagaraSystemsWithStatelessEmitters;

	/** All Validation Issues */
	TUniquePtr<FArchive> NiagaraValidationIssues;

	/** Optional list of data interface usage */
	struct FDataInterfaceUsage
	{
		int32			UsageCount = 0;
		TSet<FName>		Systems;
	};
	bool bCaptureDataInterfaceUsage = false;
	TSet<FName> NiagaraDataInterfaceUsageToCapture;
	TMap<FName, FDataInterfaceUsage>  NiagaraDataInterfaceUsage;

	/** Detailed renderer information. */
	bool bRendererDetailed = false;
	TSet<FString> NiagaraRibbonRenderers;

	/** The folder in which the commandlet's output files will be stored */
	FString AuditOutputFolder;

	/** Only assets in this collection will be considered. If this is left blank, no assets will be filtered by collection */
	FString FilterCollection;

	/** Package paths to include */
	TArray<FName> PackagePaths;

	/** List of assets to only include (i.e. process only these) */
	TSet<FName> IncludeOnlyPackages;

	/** Systems using specific data interfaces to find */
	TSet<UClass*> DataInterfacesToFind;

	/** DeviceProfiles we want to disable GPU emitters for. */
	TArray<class UDeviceProfile*> DeviceProfilesToDisableGpu;

	UPROPERTY()
	TArray<TObjectPtr<UPackage>> PackagesToSave;

	/** Entry point */
	int32 Main(const FString& Params) override;

	/** Process all referenced Niagara systems */
	bool ProcessNiagaraSystems();

	/** Dump the results of the audit */
	virtual void DumpResults();

	/** Generic function to handle dumping values to a CSV file */
	bool DumpSimpleSet(TSet<FString>& InSet, const TCHAR* InShortFilename, const TCHAR* OptionalHeader = nullptr);

	/** Gets an archive to write to an output file */
	FArchive* GetOutputFile(const TCHAR* InShortFilename);

private:
	TArray<class UNiagaraDataInterface*> GetDataInterfaces(class UNiagaraSystem* System);
};
