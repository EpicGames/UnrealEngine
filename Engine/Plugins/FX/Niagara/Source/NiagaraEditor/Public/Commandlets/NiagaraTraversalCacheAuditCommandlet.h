// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "NiagaraTypes.h"

#include "NiagaraTraversalCacheAuditCommandlet.generated.h"

struct FNiagaraEmitterHandle;
class  IConsoleVariable;
class  UEdGraphPin;
class  UNiagaraNodeFunctionCall;
class  UNiagaraScript;
class  UNiagaraSystem;

UCLASS(config=Editor)
class UNiagaraTraversalCacheAuditCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	int32 Main(const FString& Params) override;

private:
	struct FFunctionData
	{
		UNiagaraNodeFunctionCall* FunctionCallNode = nullptr;
		TArray<FNiagaraVariable> InputVariables;
		TSet<FNiagaraVariable> HiddenInputVariables;
		TArray<UEdGraphPin*> InputPins;
		TSet<UEdGraphPin*> HiddenInputPins;
	};

	struct FScriptData
	{
		UNiagaraScript* Script = nullptr;
		const FNiagaraEmitterHandle* EmitterHandle = nullptr;
		TArray<FFunctionData> FunctionDatas;
	};

	struct FSystemData
	{
		FString SystemPath;
		UNiagaraSystem* System = nullptr;
		TArray<FScriptData> ScriptDatas;
	};

	struct FAuditResultFunctionData
	{
		FString FunctionName;
		FName FunctionScriptPath = NAME_None;
		bool bFunctionEnabled = true;

		TArray<FNiagaraVariableBase> AddedInputVariables;
		TArray<FNiagaraVariableBase> RemovedInputVariables;
		TArray<FNiagaraVariableBase> AddedHiddenInputVariables;
		TArray<FNiagaraVariableBase> RemovedHiddenInputVariables;

		TArray<FNiagaraVariableBase> AddedStaticInputPinVariables;
		TArray<FNiagaraVariableBase> RemovedStaticInputPinVariables;
		TArray<FNiagaraVariableBase> AddedHiddenStaticInputPinVariables;
		TArray<FNiagaraVariableBase> RemovedHiddenStaticInputPinVariables;

		int32 CountDifferences() const;
	};

	struct FAuditResultScriptData
	{
		FString ScriptName;
		FString EmitterName;
		TArray<FAuditResultFunctionData> FunctionDatas;
		int32 CountDifferences() const;
	};

	struct FAuditResultSystemData
	{
		FString SystemPath;
		TArray<FAuditResultScriptData> ScriptDatas;
		int32 CountDifferences() const;
	};

	struct FAuditResults
	{
		TArray<FAuditResultSystemData> SystemDatas;
		int32 ErrorCount = 0;
	};

	void GatherAllSystemPaths(TArray<FString>& OutSystemPaths) const;
	void CollectFunctionData(UNiagaraSystem& System, const FNiagaraEmitterHandle* EmitterHandle, UNiagaraScript* Script, TArray<FFunctionData>& OutFunctionDatas) const;
	void CollectScriptData(UNiagaraSystem* System, const FNiagaraEmitterHandle* EmitterHandle, UNiagaraScript* Script, TArray<FScriptData>& OutScriptDatas) const;
	void CollectScriptDataForSystem(FSystemData& SystemData) const;
	void AuditScript(UNiagaraSystem& AuditSystem, FScriptData& AuditScriptData, TArray<FAuditResultScriptData>& OutScriptResults) const;
	void LogSystemResult(const FAuditResultSystemData& SystemResult, int32 SystemResultIndex) const;
	void AuditSystems(const TArray<FString>& SystemPaths, FAuditResults& Results) const;
	void WriteCSVFile(const FAuditResults& Results) const;

private:
	IConsoleVariable* EnableTraversalCacheCVar = nullptr;

	int32 MaxErrorCount = 0;
	int32 BatchSize = 100;
	bool bWriteCSV = true;

	static FCriticalSection ResultCS;
};