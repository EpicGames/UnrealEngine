// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorConfigBase.h"
#include "NiagaraFavoriteActionsConfig.generated.h"

#define UE_API NIAGARAEDITOR_API

USTRUCT()
struct FNiagaraActionIdentifier
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FName> Names;

	UPROPERTY()
	TArray<FGuid> Guids;	
	
	bool IsValid() const
	{
		return Guids.Num() > 0 || Names.Num() > 0;
	}
	
	bool operator==(const FNiagaraActionIdentifier& OtherIdentity) const
	{
		if(Guids.Num() != OtherIdentity.Guids.Num() || Names.Num() != OtherIdentity.Names.Num())
		{
			return false;
		}

		for(int32 GuidIndex = 0; GuidIndex < Guids.Num(); GuidIndex++)
		{
			if(Guids[GuidIndex] != OtherIdentity.Guids[GuidIndex])
			{
				return false;
			}
		}

		for(int32 NameIndex = 0; NameIndex < Names.Num(); NameIndex++)
		{
			if(!Names[NameIndex].IsEqual(OtherIdentity.Names[NameIndex]))
			{
				return false;
			}
		}

		return true;
	}

	bool operator!=(const FNiagaraActionIdentifier& OtherIdentity) const
	{
		return !(*this == OtherIdentity);
	}
};

inline uint32 GetTypeHash(const FNiagaraActionIdentifier& Identity)
{
	uint32 Hash = 0;
	
	for(const FGuid& Guid : Identity.Guids)
	{
		Hash = HashCombine(Hash, GetTypeHash(Guid));
	}
	
	for(const FName& Name : Identity.Names)
	{
		Hash = HashCombine(Hash, GetTypeHash(Name));
	}
	
	return Hash;
}

struct FNiagaraFavoritesActionData
{
	FNiagaraActionIdentifier ActionIdentifier;
	bool bFavoriteByDefault = false;
};

USTRUCT()
struct FNiagaraFavoriteActionsProfile
{
	GENERATED_BODY()
public:
	
	UE_API bool IsFavorite(FNiagaraFavoritesActionData InAction);
	UE_API void ToggleFavoriteAction(FNiagaraFavoritesActionData InAction);
	
private:
	/** Explicitly favorited actions */
	UPROPERTY()
	TSet<FNiagaraActionIdentifier> FavoriteActions;

	/** For unfavorited actions */
	UPROPERTY()
	TSet<FNiagaraActionIdentifier> UnfavoriteActions;
};
/**
 * 
 */
UCLASS(MinimalAPI, EditorConfig="FavoriteNiagaraActions")
class UNiagaraFavoriteActionsConfig : public UEditorConfigBase
{
	GENERATED_BODY()

public:
	bool HasActionsProfile(FName ProfileName) const { return Profiles.Contains(ProfileName); }
	UE_API FNiagaraFavoriteActionsProfile& GetActionsProfile(FName ProfileName);
	
	static UE_API UNiagaraFavoriteActionsConfig* Get();
	static UE_API void Shutdown();
	
private:
	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FNiagaraFavoriteActionsProfile> Profiles;

private:
	static UE_API TStrongObjectPtr<UNiagaraFavoriteActionsConfig> Instance;
};

#undef UE_API
