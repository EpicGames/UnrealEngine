// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API NIAGARAEDITOR_API

class FNiagaraSimCacheViewModel;
class SWidget;

/**
 * Implementations can be registered with the niagara editor module to provide custom visualizations for data interfaces stored in sim caches.
 * This is needed since data interfaces can store any custom uobject, so the sim cache editor has no knowledge how to display the data stored within.
 *
 * See FNiagaraDataChannelCacheVisualizer for an example implementation.
 */
class INiagaraDataInterfaceSimCacheVisualizer
{
public:
	virtual ~INiagaraDataInterfaceSimCacheVisualizer() = default;
	
	UE_API virtual TSharedPtr<SWidget> CreateWidgetFor(const UObject* CachedData, TSharedPtr<FNiagaraSimCacheViewModel> ViewModel);
};

#undef UE_API
