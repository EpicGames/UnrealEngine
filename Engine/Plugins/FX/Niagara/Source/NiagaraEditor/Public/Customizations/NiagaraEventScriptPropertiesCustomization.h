// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "EdGraph/EdGraphSchema.h"
#include "EditorUndoClient.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "NiagaraEventScriptPropertiesCustomization.generated.h"


USTRUCT()
struct FNiagaraStackAssetAction_EventSource : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();
	
	FName EmitterName;
	FName EventName;
	FName EventTypeName;
	FGuid EmitterGuid;

	// Simple type info
	static FName StaticGetTypeId() { static FName Type("FNiagaraStackAssetAction_EventSource"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	FNiagaraStackAssetAction_EventSource()
		: FEdGraphSchemaAction()
	{}

	FNiagaraStackAssetAction_EventSource(FName InEmitterName, FName InEventName, FName InEventTypeName, FGuid InEmitterGuid,
		FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords)), 
		EmitterName(InEmitterName), EventName(InEventName), EventTypeName(InEventTypeName), EmitterGuid(InEmitterGuid)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override
	{
		return nullptr;
	}
	//~ End FEdGraphSchemaAction Interface

};

class FNiagaraEventScriptPropertiesCustomization : public IPropertyTypeCustomization, public FEditorUndoClient
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance(TWeakObjectPtr<UNiagaraSystem> InSystem, FVersionedNiagaraEmitterWeakPtr InEmitter);

	FNiagaraEventScriptPropertiesCustomization(TWeakObjectPtr<UNiagaraSystem> InSystem,	FVersionedNiagaraEmitterWeakPtr InEmitter);

	virtual ~FNiagaraEventScriptPropertiesCustomization() override;

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	//~ FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
protected:
	TSharedRef<SWidget> OnGetMenuContent() const;
	FText OnGetButtonText() const;
	FText GetProviderText(const FName& InEmitterName, const FName& InEventName) const;
	void ChangeEventSource(FGuid InEmitterId, FName InEmitterName, FName InEventName);
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);
	void OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedActions, ESelectInfo::Type InSelectionType);
	bool GetSpawnNumberEnabled() const;
	bool GetUseRandomSpawnNumber() const;
	EVisibility GetMinSpawnNumberVisible() const;
	bool GetUpdateInitialValuesEnabled() const;
	void ResolveEmitterName();
	void ComputeErrorVisibility();
	EVisibility GetErrorVisibility() const;
	FText GetErrorText() const;
	FText GetErrorTextTooltip() const;

	TArray<FName> GetEventNames(const FVersionedNiagaraEmitter& Emitter) const;

	void OnUpdateInitialValuesChanged();

private:
	TSharedPtr<IPropertyHandle> HandleSrcID;
	TSharedPtr<IPropertyHandle> HandleEventName;
	TSharedPtr<IPropertyHandle> HandleSpawnNumber;
	TSharedPtr<IPropertyHandle> HandleExecutionMode;
	TSharedPtr<IPropertyHandle> HandleMaxEvents;
	TSharedPtr<IPropertyHandle> HandleUseRandomSpawnNumber;
	TSharedPtr<IPropertyHandle> HandleMinSpawnNumber;
	TSharedPtr<IPropertyHandle> HandleUpdateInitialValues;

	TWeakObjectPtr<UNiagaraSystem> System;
	FVersionedNiagaraEmitterWeakPtr Emitter;

	FName CachedEmitterName;
	EVisibility CachedVisibility;
};
