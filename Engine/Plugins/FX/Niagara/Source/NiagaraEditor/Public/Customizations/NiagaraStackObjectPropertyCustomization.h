// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Widgets/SWidget.h"

class FMenuBuilder;
class UNiagaraStackPropertyRow;

/**
 *  This class allows you to customize the UI for a UNiagaraStackObject's properties.
 *  Register this customization in the NiagaraEditorModule by providing the class it is customizing and your customization class.
 */
class FNiagaraStackObjectPropertyCustomization : public TSharedFromThis<FNiagaraStackObjectPropertyCustomization>
{
public:
	virtual ~FNiagaraStackObjectPropertyCustomization() = default;

	/** If specified, will generate a custom name widget for a given property row. Only applicable for rows that generate typical name & value widgets. */
	virtual TOptional<TSharedPtr<SWidget>> GenerateNameWidget(UNiagaraStackPropertyRow* PropertyRow) const { return TOptional<TSharedPtr<SWidget>>(); }

	/** If specified, adds custom entries to the right-click context menu for a given property row. */
	virtual void GenerateContextMenuActions(UNiagaraStackPropertyRow* PropertyRow, FMenuBuilder& MenuBuilder) const {}
};
