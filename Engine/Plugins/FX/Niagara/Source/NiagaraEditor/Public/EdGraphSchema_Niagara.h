// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphSchema.h"
#include "ConnectionDrawingPolicy.h"
#include "NiagaraActions.h"

#include "EdGraphSchema_Niagara.generated.h"

class IAssetReferenceFilter;
class UEdGraph;
struct FNiagaraVariable;
struct FNiagaraVariableInfo;
struct FNiagaraTypeDefinition;
enum class ENiagaraDataType : uint8;
struct FNiagaraVariant;

/** Action to add a node to the graph */
USTRUCT()
struct FNiagaraSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	/** Template of node we want to create */
	UPROPERTY()
	TObjectPtr<class UEdGraphNode> NodeTemplate;

	UPROPERTY()
	FName InternalName;

	FNiagaraSchemaAction_NewNode() 
		: FEdGraphSchemaAction()
		, NodeTemplate(nullptr)
	{}

	FNiagaraSchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FName InInternalName, FText InToolTip, const int32 InGrouping, FText InKeywords = FText(), int32 InSectionID = 0)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, InKeywords, InSectionID)
		, NodeTemplate(nullptr), InternalName(InInternalName)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	NIAGARAEDITOR_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	NIAGARAEDITOR_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	NIAGARAEDITOR_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	//~ End FEdGraphSchemaAction Interface

	template <typename NodeType>
	static NodeType* SpawnNodeFromTemplate(class UEdGraph* ParentGraph, NodeType* InTemplateNode, const UE::Slate::FDeprecateVector2DParameter& Location, bool bSelectNewNode = true)
	{
		FNiagaraSchemaAction_NewNode Action;
		Action.NodeTemplate = InTemplateNode;

		return Cast<NodeType>(Action.PerformAction(ParentGraph, NULL, Location, bSelectNewNode));
	}
};

USTRUCT()
struct FNiagaraSchemaAction_NewComment : public FEdGraphSchemaAction
{
public:
	GENERATED_USTRUCT_BODY();

	FNiagaraSchemaAction_NewComment()
		: FEdGraphSchemaAction()
		, GraphEditor(nullptr)
	{}

	FNiagaraSchemaAction_NewComment(const TSharedPtr<SGraphEditor>& InGraphEditor)
		: FEdGraphSchemaAction()
		, GraphEditor(InGraphEditor)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	NIAGARAEDITOR_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

private:
	/** SGraphEditor to use for getting new comment boxes dimensions */
	TSharedPtr<SGraphEditor> GraphEditor;
};

UCLASS(MinimalAPI)
class UEdGraphSchema_Niagara : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	// Allowable PinType.PinCategory values
	static NIAGARAEDITOR_API const FName PinCategoryType;
	static NIAGARAEDITOR_API const FName PinCategoryMisc;
	static NIAGARAEDITOR_API const FName PinCategoryClass;
	static NIAGARAEDITOR_API const FName PinCategoryEnum;
	static NIAGARAEDITOR_API const FName PinCategoryStaticType;
	static NIAGARAEDITOR_API const FName PinCategoryStaticClass;
	static NIAGARAEDITOR_API const FName PinCategoryStaticEnum;

	static NIAGARAEDITOR_API bool IsStaticPin(const UEdGraphPin* Pin);

	//~ Begin EdGraphSchema Interface
	NIAGARAEDITOR_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	NIAGARAEDITOR_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	NIAGARAEDITOR_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	NIAGARAEDITOR_API virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	NIAGARAEDITOR_API virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	NIAGARAEDITOR_API virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	NIAGARAEDITOR_API virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const override;
	NIAGARAEDITOR_API virtual FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	NIAGARAEDITOR_API virtual void ResetPinToAutogeneratedDefaultValue(UEdGraphPin* Pin, bool bCallModifyCallbacks = true) const override;
	NIAGARAEDITOR_API virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override; 
	virtual bool ShouldAlwaysPurgeOnModification() const override { return false; }
	NIAGARAEDITOR_API virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	NIAGARAEDITOR_API virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;

	/**
	* Sets the string to the specified pin; even if it is invalid it is still set.
	*
	* @param	Pin			   	The pin on which to set the default value.
	* @param	NewDefaultValue	The new default value.
	* @param   bMarkAsModified Marks the container of the value as modified. For Niagara Schema, whether or not to notify the Node owning the Pin that the Pin has changed.
	*/
	NIAGARAEDITOR_API virtual void TrySetDefaultValue(UEdGraphPin& Pin, const FString& NewDefaultValue, bool bMarkAsModified = true) const override;

	//~ End EdGraphSchema Interface

	static NIAGARAEDITOR_API FLinearColor GetTypeColor(const FNiagaraTypeDefinition& Type);

	NIAGARAEDITOR_API TArray<TSharedPtr<FNiagaraAction_NewNode>> GetGraphActions(const UEdGraph* CurrentGraph, const UEdGraphPin* FromPin, UEdGraph* OwnerOfTemporaries) const;
	NIAGARAEDITOR_API void PromoteSinglePinToParameter(UEdGraphPin* SourcePin);
	static NIAGARAEDITOR_API bool CanPromoteSinglePinToParameter(const UEdGraphPin* SourcePin);
	NIAGARAEDITOR_API void ToggleNodeEnabledState(class UNiagaraNode* InNode) const;
	NIAGARAEDITOR_API void RefreshNode(UNiagaraNode* InNode) const;

	/** 
	  * Creates a niagara variable using the name, type, and default value stored on an ed graph pin.
	  * Pin The pin to create a variable from.
	  * bNeedsValue Whether or not the returned variable must be allocated to a valid value. When true if the pin doesn't have a valid default value itself the variable will be reset to default before return.
	  * returns The newly created variable.
	  */
	static NIAGARAEDITOR_API FNiagaraVariable PinToNiagaraVariable(const UEdGraphPin* Pin, bool bNeedsValue=false, ENiagaraStructConversion StructConversion = ENiagaraStructConversion::UserFacing);

	/** 
	  * Tries to get a default value string for a graph pin from a Niagara variable.
	  * @param Variable The variable to get a default pin value for.
	  * @param OutPinDefaultValue The pin default value string if this call was successful, otherwise empty.
	  * @returns Whether or not we could get a default value form the supplied variable.
	  */
	NIAGARAEDITOR_API bool TryGetPinDefaultValueFromNiagaraVariable(const FNiagaraVariable& Variable, FString& OutPinDefaultValue) const;

	/**
	  * Tries to get a default value string for a graph pin from a Niagara variant.
	  * @param Variable The variable.
	  * @param Value The variant containing the data for the variable.
	  * @param OutPinDefaultValue The pin default value string if this call was successful, otherwise empty.
	  * @returns Whether or not we could get a default value form the supplied variable.
	  */
	NIAGARAEDITOR_API bool TryGetPinDefaultValueFromNiagaraVariant(const FNiagaraVariableBase& Variable, const FNiagaraVariant& Value, FString& OutPinDefaultValue) const;

	static NIAGARAEDITOR_API void ConvertIllegalPinsInPlace(UEdGraphPin* Pin);
	static NIAGARAEDITOR_API FNiagaraTypeDefinition PinToTypeDefinition(const UEdGraphPin* Pin, ENiagaraStructConversion StructConversion = ENiagaraStructConversion::UserFacing);
	static NIAGARAEDITOR_API FNiagaraTypeDefinition PinTypeToTypeDefinition(const FEdGraphPinType& PinType);
	static NIAGARAEDITOR_API FEdGraphPinType TypeDefinitionToPinType(FNiagaraTypeDefinition TypeDef);
	
	static NIAGARAEDITOR_API bool IsPinWildcard(const UEdGraphPin* Pin);
	
	// Fast path helper to skip doing full type conversion
	static NIAGARAEDITOR_API bool IsPinStatic(const UEdGraphPin* Pin);
	static NIAGARAEDITOR_API FPinConnectionResponse GetWildcardConnectionResponse(const UEdGraphPin* PinA, const UEdGraphPin* PinB);
	
	NIAGARAEDITOR_API bool IsSystemConstant(const FNiagaraVariable& Variable)const;

	NIAGARAEDITOR_API class UNiagaraParameterCollection* VariableIsFromParameterCollection(const FNiagaraVariable& Var)const;
	NIAGARAEDITOR_API class UNiagaraParameterCollection* VariableIsFromParameterCollection(const FString& VarName, bool bAllowPartialMatch, FNiagaraVariable& OutVar)const;

	NIAGARAEDITOR_API bool IsValidNiagaraPropertyType(const FProperty* Property) const;
	NIAGARAEDITOR_API FNiagaraTypeDefinition GetTypeDefForProperty(const FProperty* Property) const;

	static NIAGARAEDITOR_API const FLinearColor NodeTitleColor_Attribute;
	static NIAGARAEDITOR_API const FLinearColor NodeTitleColor_Constant;
	static NIAGARAEDITOR_API const FLinearColor NodeTitleColor_SystemConstant;
	static NIAGARAEDITOR_API const FLinearColor NodeTitleColor_FunctionCall;
	static NIAGARAEDITOR_API const FLinearColor NodeTitleColor_CustomHlsl;
	static NIAGARAEDITOR_API const FLinearColor NodeTitleColor_Event; 
	static NIAGARAEDITOR_API const FLinearColor NodeTitleColor_TranslatorConstant;
	static NIAGARAEDITOR_API const FLinearColor NodeTitleColor_RapidIteration;

	static NIAGARAEDITOR_API const FText ReplaceExistingInputConnectionsText;
	static NIAGARAEDITOR_API const FText TypesAreNotCompatibleText;
	static NIAGARAEDITOR_API const FText ConvertText;
	static NIAGARAEDITOR_API const FText ConvertLossyText;
	static NIAGARAEDITOR_API const FText PinNotConnectableText;
	static NIAGARAEDITOR_API const FText SameNodeConnectionForbiddenText;
	static NIAGARAEDITOR_API const FText DirectionsNotCompatibleText;
	static NIAGARAEDITOR_API const FText AddPinIncompatibleTypeText;
	static NIAGARAEDITOR_API const FText CircularConnectionFoundText;
	
	NIAGARAEDITOR_API bool PinTypesValidForNumericConversion(FEdGraphPinType AType, FEdGraphPinType BType) const;	

	// Return type is Name, Prototype, Description
	static NIAGARAEDITOR_API TArray<TTuple<FString, FString, FText>> GetDataInterfaceFunctionPrototypes(const UEdGraphPin* GraphPin);

private:
	NIAGARAEDITOR_API void GetNumericConversionToSubMenuActions(class UToolMenu* Menu, const FName SectionName, UEdGraphPin* InGraphPin);
	NIAGARAEDITOR_API void GetNumericConversionToSubMenuActionsAll(class UToolMenu* Menu, const FName SectionName, UNiagaraNode* InGraphPin);
	NIAGARAEDITOR_API void ConvertNumericPinToType(UEdGraphPin* InPin, FNiagaraTypeDefinition TypeDef);
	NIAGARAEDITOR_API void ConvertNumericPinToTypeAll(UNiagaraNode* InPin, FNiagaraTypeDefinition TypeDef);
	NIAGARAEDITOR_API void ConvertPinToType(UEdGraphPin* InPin, FNiagaraTypeDefinition TypeDef) const;

	NIAGARAEDITOR_API void GenerateDataInterfacePinMenu(UToolMenu* ToolMenu, const FName SectionName, const UEdGraphPin* GraphPin, FNiagaraTypeDefinition TypeDef) const;

	static NIAGARAEDITOR_API bool CheckCircularConnection(TSet<const UEdGraphNode*>& VisitedNodes, const UEdGraphNode* InNode, const UEdGraphNode* InTestNode);

	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);
};

class FNiagaraConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
public:
	FNiagaraConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraph);
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;

protected:
	bool ShouldChangeTangentForReroute(class UNiagaraNodeReroute* Node);
	bool GetAverageConnectedPosition(class UNiagaraNodeReroute* Reroute, EEdGraphPinDirection Direction, FVector2D& OutPos) const;
	bool FindPinCenter(UEdGraphPin* Pin, FVector2D& OutCenter) const;
protected:
	class UNiagaraGraph* GraphObj;
	
	float DefaultDataWireThickness;
	float DefaultExecutionWireThickness;
	
	// Each time a Reroute is encountered, input geometry is compared to output geometry to see if the pins on the reroute need to be reversed
	TMap<class UNiagaraNodeReroute*, bool> RerouteToReversedDirectionMap;
};

