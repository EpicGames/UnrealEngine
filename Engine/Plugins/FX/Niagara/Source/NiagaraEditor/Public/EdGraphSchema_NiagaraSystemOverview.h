// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "ConnectionDrawingPolicy.h"
#include "EdGraphSchema_NiagaraSystemOverview.generated.h"

class IAssetReferenceFilter;

UCLASS(MinimalAPI)
class UEdGraphSchema_NiagaraSystemOverview : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	//~ Begin EdGraphSchema Interface
	NIAGARAEDITOR_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override { return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, FText()); }; //@TODO System Overview: write text response
	virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override { return false; };
	virtual bool ShouldAlwaysPurgeOnModification() const override { return false; }
	NIAGARAEDITOR_API virtual bool IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const override;
	NIAGARAEDITOR_API virtual int32 GetCurrentVisualizationCacheID() const override;
	NIAGARAEDITOR_API virtual void ForceVisualizationCacheClear() const override;
	NIAGARAEDITOR_API virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	NIAGARAEDITOR_API virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;

private:
	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);

	static NIAGARAEDITOR_API int32 CurrentCacheRefreshID;
};

