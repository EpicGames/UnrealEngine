// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/StringView.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraParameters.h"
#include "NiagaraScript.h"
#include "NiagaraShared.h"

class Error;
class FCompilerResultsLog;
class FNiagaraCompilationCopyData;
class FNiagaraCompileOptions;
class FNiagaraCompileRequestData;
class FNiagaraCompileRequestDuplicateData;
class FNiagaraPrecompileData;
class UEdGraphPin;
class UNiagaraDataInterface;
class UNiagaraGraph;
class UNiagaraNode;
struct FNiagaraTranslateResults;
struct FNiagaraTranslatorOutput;
struct FNiagaraVMExecutableData;

/** Defines information about the results of a Niagara script compile. */
struct FNiagaraCompileResults
{
	/** Whether or not the script compiled successfully for VectorVM */
	bool bVMSucceeded = false;
	/** Whether or not the script compiled successfully for GPU compute */
	bool bComputeSucceeded = false;
	
	/** The actual final compiled data.*/
	TSharedPtr<FNiagaraVMExecutableData> Data;
	float CompilerWallTime = 0.0f;
	float CompilerPreprocessTime = 0.0f;
	float CompilerWorkerTime = 0.0f;

	/** Tracking any compilation warnings or errors that occur.*/
	TArray<FNiagaraCompileEvent> CompileEvents;
	uint32 NumErrors = 0;
	uint32 NumWarnings = 0;
	FString DumpDebugInfoPath;
 
	static ENiagaraScriptCompileStatus CompileResultsToSummary(const FNiagaraCompileResults* CompileResults);
	void AppendCompileEvents(TArrayView<const FNiagaraCompileEvent> InCompileEvents)
	{
		CompileEvents.Reserve(CompileEvents.Num() + InCompileEvents.Num());
		for (const auto& CompileEvent : InCompileEvents)
		{
			CompileEvents.Add(CompileEvent);
			NumErrors += (CompileEvent.Severity == FNiagaraCompileEventSeverity::Error) ? 1 : 0;
			NumWarnings += (CompileEvent.Severity == FNiagaraCompileEventSeverity::Warning) ? 1 : 0;
		}
	}
};

//Interface for Niagara compilers.
// NOTE: the graph->hlsl translation step is now in FNiagaraHlslTranslator
//
class INiagaraCompiler
{
public:
	UE_DEPRECATED(5.4, "Please update to supply the GroupName directly as this will be removed in a future version.")
	virtual int32 CompileScript(const class FNiagaraCompileRequestData* InCompileRequest, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, FNiagaraTranslatorOutput* TranslatorOutput, FString& TranslatedHLSL) { return INDEX_NONE; };

	/** Starts the async compilation of a script and returns the job handle to retrieve the results */
	virtual int32 CompileScript(const FStringView GroupName, const FNiagaraCompileOptions& InOptions, const FNiagaraTranslateResults& InTranslateResults, const FNiagaraTranslatorOutput& TranslatorOutput, const FString& TranslatedHLSL) = 0;

	/** Returns the compile result for a given job id once the job has finished compiling. */
	virtual TOptional<FNiagaraCompileResults> GetCompileResult(int32 JobID, bool bWait = false) = 0;

	/** Adds an error to be reported to the user. Any error will lead to compilation failure. */
	virtual void Error(FText ErrorText) = 0 ;

	/** Adds a warning to be reported to the user. Warnings will not cause a compilation failure. */
	virtual void Warning(FText WarningText) = 0;
};

/** Defines information about the results of a Niagara script compile. */
struct FNiagaraTranslateResults
{
	/** Whether or not HLSL generation was successful */
	bool bHLSLGenSucceeded = false;;

	/** A results log with messages, warnings, and errors which occurred during the compile. */
	TArray<FNiagaraCompileEvent> CompileEvents;
	uint32 NumErrors = 0;
	uint32 NumWarnings = 0;

	TArray<FNiagaraCompileDependency> CompileDependencies;

	TArray<FNiagaraCompilerTag> CompileTags;

	TArray<FNiagaraCompilerTag> CompileTagsEditorOnly;

	/** A string representation of the compilation output. */
	FString OutputHLSL;

	static ENiagaraScriptCompileStatus TranslateResultsToSummary(const FNiagaraTranslateResults* CompileResults);
};

class FHlslNiagaraTranslatorOptions
{
public:
	ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim;

	/** Any parameters in these namespaces will be pulled from an "InstanceParameters" dataset rather than from the uniform table. */
	TArray<FString> InstanceParameterNamespaces;

	/** Whether or not to treat top-level module variables as external values for rapid iteration without need for compilation.*/
	bool bParameterRapidIteration = true;

	/** Should we disable debug switches during translation. */
	bool bDisableDebugSwitches = false;

	/** Whether or not to override top-level module variables with values from the constant override table. This is only used for variables that were candidates for rapid iteration.*/
	TArray<FNiagaraVariable> OverrideModuleConstants;
};

class INiagaraHlslTranslator
{
public:
	virtual ~INiagaraHlslTranslator() = default;

	virtual FNiagaraTranslateResults Translate(const FNiagaraCompileOptions& InCompileOptions, const FHlslNiagaraTranslatorOptions& Options) = 0;
	virtual const FNiagaraTranslatorOutput& GetTranslateOutput() const = 0;
	virtual const FString& GetTranslatedHLSL() const = 0;

	static NIAGARAEDITOR_API TUniquePtr<INiagaraHlslTranslator> CreateTranslator(const FNiagaraCompileRequestDataBase* InCompileData, const FNiagaraCompileRequestDuplicateDataBase* InDuplicateData);
	static NIAGARAEDITOR_API TUniquePtr<INiagaraHlslTranslator> CreateTranslator(const FNiagaraPrecompileData* InCompileData, const FNiagaraCompilationCopyData* InDuplicateData);

	static NIAGARAEDITOR_API FString BuildHLSLStructDecl(const FNiagaraTypeDefinition& Type, FText& OutErrorMessage, bool bGpuScript);
	static NIAGARAEDITOR_API bool IsBuiltInHlslType(const FNiagaraTypeDefinition& Type);
	static NIAGARAEDITOR_API FString GetStructHlslTypeName(const FNiagaraTypeDefinition& Type);
	static NIAGARAEDITOR_API FString GetPropertyHlslTypeName(const FProperty* Property);
	static NIAGARAEDITOR_API bool IsHlslBuiltinVector(const FNiagaraTypeDefinition& Type);
	static NIAGARAEDITOR_API TArray<FName> ConditionPropertyPath(const FNiagaraTypeDefinition& Type, const TArray<FName>& InPath);
	static NIAGARAEDITOR_API FString GetHlslDefaultForType(const FNiagaraTypeDefinition& Type);
};
