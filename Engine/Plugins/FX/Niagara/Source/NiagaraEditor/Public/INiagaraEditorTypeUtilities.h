// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Math/UnitConversion.h"
#include "NiagaraTypes.h"
#include "NiagaraVariant.h"
#include "NiagaraVariableMetaData.h"

class FStructOnScope;
class IPropertyHandle;
class SNiagaraParameterEditor;
class SWidget;
struct FNiagaraTypeDefinition;
struct FNiagaraVariable;
struct FNiagaraVariant;
struct FNiagaraInputParameterCustomization;
struct FNiagaraClipboardPortableValue;


class INiagaraEditorTypeUtilities
{
public:
	DECLARE_DELEGATE(FNotifyValueChanged);
public:
	virtual ~INiagaraEditorTypeUtilities() = default;

	virtual bool CanProvideDefaultValue() const = 0;

	virtual void UpdateVariableWithDefaultValue(FNiagaraVariable& Variable) const = 0;

	virtual bool CanCreateParameterEditor() const = 0;

	virtual TSharedPtr<SNiagaraParameterEditor> CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit = EUnit::Unspecified, const FNiagaraInputParameterCustomization& WidgetCustomization = FNiagaraInputParameterCustomization()) const = 0;

	virtual bool CanCreateDataInterfaceEditor() const = 0;

	virtual TSharedPtr<SWidget> CreateDataInterfaceEditor(UObject* DataInterface, FNotifyValueChanged DataInterfaceChangedHandler) const = 0;

	virtual bool CanHandlePinDefaults() const = 0;

	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const = 0;

	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const = 0;

	virtual bool CanSetValueFromDisplayName() const = 0;

	virtual bool SetValueFromDisplayName(const FText& TextValue, FNiagaraVariable& Variable) const = 0;

	virtual FText GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const = 0;

	virtual FText GetStackDisplayText(const FNiagaraVariable& Variable) const = 0;

	virtual bool SupportsClipboardPortableValues() const = 0;

	virtual bool TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const = 0;

	virtual bool CanUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType) const = 0;

	virtual bool TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const = 0;

	virtual bool CanBeSelectValue() const = 0;

	virtual int32 VariableToSelectNumericValue(const FNiagaraVariable& InVariableValue) const = 0;

	virtual FName GetDebugNameForSelectValue(const FNiagaraTypeDefinition& ValueType, int32 SelectValue) const = 0;
};

class FNiagaraEditorTypeUtilities : public INiagaraEditorTypeUtilities, public TSharedFromThis<FNiagaraEditorTypeUtilities, ESPMode::ThreadSafe>
{
public:
	DECLARE_DELEGATE(FNotifyValueChanged);
public:
	//~ INiagaraEditorTypeUtilities
	virtual bool CanProvideDefaultValue() const override { return false; }
	virtual void UpdateVariableWithDefaultValue(FNiagaraVariable& Variable) const override { }
	virtual bool CanCreateParameterEditor() const override { return false; }
	virtual TSharedPtr<SNiagaraParameterEditor> CreateParameterEditor(const FNiagaraTypeDefinition& ParameterType, EUnit DisplayUnit, const FNiagaraInputParameterCustomization& WidgetCustomization) const override { return TSharedPtr<SNiagaraParameterEditor>(); }
	virtual bool CanCreateDataInterfaceEditor() const override { return false; };
	virtual TSharedPtr<SWidget> CreateDataInterfaceEditor(UObject* DataInterface, FNotifyValueChanged DataInterfaceChangedHandler) const override { return TSharedPtr<SWidget>(); }
	virtual bool CanHandlePinDefaults() const override { return false; }
	virtual FString GetPinDefaultStringFromValue(const FNiagaraVariable& AllocatedVariable) const override { return FString(); }
	virtual bool SetValueFromPinDefaultString(const FString& StringValue, FNiagaraVariable& Variable) const override { return false; }
	virtual bool CanSetValueFromDisplayName() const override { return false; }
	virtual bool SetValueFromDisplayName(const FText& TextValue, FNiagaraVariable& Variable) const override { return false; }
	virtual FText GetSearchTextFromValue(const FNiagaraVariable& AllocatedVariable) const override { return FText(); }
	virtual FText GetStackDisplayText(const FNiagaraVariable& Variable) const override
	{
		FString DefaultString = GetPinDefaultStringFromValue(Variable);
		return FText::FromString(DefaultString.IsEmpty() ? "[?]" : DefaultString);
	}
	virtual bool SupportsClipboardPortableValues() const override { return false; }
	virtual bool TryUpdateClipboardPortableValueFromTypedValue(const FNiagaraTypeDefinition& InSourceType, const FNiagaraVariant& InSourceValue, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const override { return false; }
	virtual bool CanUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType) const override 
	{
		FNiagaraVariant Unused;
		return TryUpdateTypedValueFromClipboardPortableValue(InSourceClipboardPortableValue, InTargetType, Unused);
	}
	virtual bool TryUpdateTypedValueFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, const FNiagaraTypeDefinition& InTargetType, FNiagaraVariant& InTargetValue) const override { return false; }
	virtual bool CanBeSelectValue() const override { return false; }
	virtual int32 VariableToSelectNumericValue(const FNiagaraVariable& InVariableValue) const override
	{
		unimplemented();
		return INDEX_NONE;
	}
	virtual FName GetDebugNameForSelectValue(const FNiagaraTypeDefinition& ValueType, int32 SelectValue) const override
	{
		unimplemented();
		return NAME_None;
	}
};

class INiagaraEditorPropertyUtilities
{
public:
	virtual bool SupportsClipboardPortableValues() const = 0;
	virtual bool TryUpdateClipboardPortableValueFromProperty(const IPropertyHandle& InPropertyHandle, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const = 0;
	virtual bool TryUpdatePropertyFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, IPropertyHandle& InPropertyHandle) const = 0;
};

class FNiagaraEditorPropertyUtilities : public INiagaraEditorPropertyUtilities
{
public:
	virtual bool SupportsClipboardPortableValues() const override { return false; }
	virtual bool TryUpdateClipboardPortableValueFromProperty(const IPropertyHandle& InPropertyHandle, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const override { return false; }
	virtual bool TryUpdatePropertyFromClipboardPortableValue(const FNiagaraClipboardPortableValue& InSourceClipboardPortableValue, IPropertyHandle& InPropertyHandle) const override { return false; }
};

class INiagaraEditorExternalObjectUtilities
{
public:
	virtual ~INiagaraEditorExternalObjectUtilities() = default;
	virtual bool SupportsClipboardPortableValues() const = 0;
	virtual bool TryUpdateClipboardPortableValueFromObject(const UObject& InExternalObject, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const = 0;
};

class FNiagaraEditorExternalObjectUtilities : public INiagaraEditorExternalObjectUtilities
{
public:
	virtual bool SupportsClipboardPortableValues() const override { return false; }
	virtual bool TryUpdateClipboardPortableValueFromObject(const UObject& InExternalObject, FNiagaraClipboardPortableValue& InTargetClipboardPortableValue) const override { return false; }
};