// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditorDragDropAction.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "NiagaraEditorCommon.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "Config/NiagaraFavoriteActionsConfig.h"
#include "NiagaraActions.generated.h"

class UToolMenu;
class UGraphNodeContextMenuContext;


UENUM()
enum class ENiagaraMenuSections : uint8
{
	Suggested = 0,
	General = 1
};

UENUM()
enum class EScriptSource : uint8
{
	Niagara,
	Game,
	Plugins,
	Developer,
	Unknown
};

USTRUCT()
struct FNiagaraActionSourceData
{
	GENERATED_BODY()
	
	FNiagaraActionSourceData()
	{}
	FNiagaraActionSourceData(const EScriptSource& InSource, const FText& InSourceText, bool bInDisplaySource = false)
	{
		Source = InSource;
		SourceText = InSourceText;
		bDisplaySource = bInDisplaySource;
	}	
	
	EScriptSource Source = EScriptSource::Unknown;
	FText SourceText = FText::GetEmpty();
	bool bDisplaySource = false;	
};

USTRUCT()
struct FNiagaraMenuAction : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	DECLARE_DELEGATE(FOnExecuteStackAction);
	DECLARE_DELEGATE_RetVal(bool, FCanExecuteStackAction);

	FNiagaraMenuAction() {}
	NIAGARAEDITOR_API FNiagaraMenuAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords, FOnExecuteStackAction InAction, int32 InSectionID = 0);
	NIAGARAEDITOR_API FNiagaraMenuAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords, FOnExecuteStackAction InAction, FCanExecuteStackAction InCanPerformAction, int32 InSectionID = 0);

	void ExecuteAction()
	{
		if (CanExecute())
		{
			Action.ExecuteIfBound();
		}
	}

	bool CanExecute() const
	{
		// Fire the 'can execute' delegate if we have one, otherwise always return true
		return CanPerformAction.IsBound() ? CanPerformAction.Execute() : true;
	}

	bool IsExperimental = false;

	NIAGARAEDITOR_API TOptional<FNiagaraVariable> GetParameterVariable() const;
	NIAGARAEDITOR_API void SetParameterVariable(const FNiagaraVariable& InParameterVariable);
	void SetSectionId(const int32 NewSectionId) { SectionID = NewSectionId; };

private:
	TOptional<FNiagaraVariable> ParameterVariable;
	FOnExecuteStackAction Action;
	FCanExecuteStackAction CanPerformAction;
};

class FNiagaraMenuActionCollector
{
public:
	void AddAction(TSharedPtr<FNiagaraMenuAction> Action, int32 SortOrder);
	void AddAllActionsTo(FGraphActionListBuilderBase& ActionBuilder);

private:
	struct FCollectedAction
	{
		TSharedPtr<FNiagaraMenuAction> Action;
		int32 SortOrder;
	};

	TArray<FCollectedAction> Actions;
};

// new action hierarchy for the new menus. Prefer inheriting from this rather than the above
// this action does not have any use; inherit from it and provide your own functionality
USTRUCT()
struct FNiagaraMenuAction_Base
{
	GENERATED_USTRUCT_BODY();
	
	DECLARE_DELEGATE(FOnExecuteAction);
	DECLARE_DELEGATE_RetVal(bool, FCanExecuteAction);

	FNiagaraMenuAction_Base() {}
	NIAGARAEDITOR_API FNiagaraMenuAction_Base(FText DisplayName, TArray<FString> InNodeCategories, TOptional<FNiagaraFavoritesActionData> FavoritesActionData, FText InToolTip, FText InKeywords, float InIntrinsicWeightMultiplier = 1.f);

	NIAGARAEDITOR_API void UpdateFullSearchText();

	bool bIsExperimental = false;

	bool bIsInLibrary = true;
	
	/** Nested categories below a top level section. Can be empty */
	TArray<FString> Categories;

	/** The DisplayName used in lists */
	FText DisplayName;

	/** An alternate name used explicitly for search purposes. Example: Divide and '/'.*/
	TOptional<FText> AlternateSearchName;

	/** The Tooltip text for this action */
	FText ToolTip;

	/** Additional keywords that should be considered for searching */
	FText Keywords;

	/** Additional data about where this action originates. Useful to display additional data such as the owning module. */
	FNiagaraActionSourceData SourceData;

	/** A string that combines all kinds of search terms */
	FString FullSearchString;

	/** A multiplier intrinsic to the action. Can be used to tweak search relevance depending on context (like module actions in a module script) */
	float SearchWeightMultiplier = 1.f;
	
	TOptional<FNiagaraFavoritesActionData> FavoritesActionData;

	TOptional<FSoftObjectPath> PreviewMoviePath;
	
	mutable bool bIsHovered = false;
};

USTRUCT()
struct FNiagaraMenuAction_Generic : public FNiagaraMenuAction_Base
{
	GENERATED_BODY()

	FNiagaraMenuAction_Generic() {}

	FNiagaraMenuAction_Generic(FOnExecuteAction ExecuteAction, FCanExecuteAction InCanExecuteAction,
		FText InDisplayName, TArray<FString> InNodeCategories, TOptional<FNiagaraFavoritesActionData> FavoritesActionData,  FText InToolTip, FText InKeywords, float InIntrinsicWeightMultiplier = 1.f)
    : FNiagaraMenuAction_Base(InDisplayName, InNodeCategories, FavoritesActionData, InToolTip, InKeywords, InIntrinsicWeightMultiplier)
	{
		Action = ExecuteAction;
		CanExecuteAction = InCanExecuteAction;
	}

	FNiagaraMenuAction_Generic(FOnExecuteAction ExecuteAction,
        FText InDisplayName, TArray<FString> InNodeCategories, TOptional<FNiagaraFavoritesActionData> FavoritesActionData, FText InToolTip, FText InKeywords, float InIntrinsicWeightMultiplier = 1.f)
    : FNiagaraMenuAction_Base(InDisplayName, InNodeCategories, FavoritesActionData, InToolTip, InKeywords, InIntrinsicWeightMultiplier)
	{
		Action = ExecuteAction;
	}

	void Execute()
	{
		if(CanExecuteAction.IsBound())
		{
			if(CanExecuteAction.Execute())
			{
				Action.ExecuteIfBound();
			}
		}
		else
		{
			Action.ExecuteIfBound();
		}
	}

	NIAGARAEDITOR_API TOptional<FNiagaraVariable> GetParameterVariable() const;
	NIAGARAEDITOR_API void SetParameterVariable(const FNiagaraVariable& InParameterVariable);
protected:
	FOnExecuteAction Action;
	FCanExecuteAction CanExecuteAction;

	TOptional<FNiagaraVariable> ParameterVariable;
};

USTRUCT()
struct FNiagaraAction_NewNode : public FNiagaraMenuAction_Generic
{
	GENERATED_BODY()

	FNiagaraAction_NewNode() {}
	FNiagaraAction_NewNode(
		FText InDisplayName, TArray<FString> InNodeCategories, TOptional<FNiagaraFavoritesActionData> FavoritesActionData, FText InToolTip, FText InKeywords, float InIntrinsicWeightMultiplier = 1.f)
		: FNiagaraMenuAction_Generic(FOnExecuteAction(), InDisplayName, InNodeCategories, FavoritesActionData, InToolTip, InKeywords, InIntrinsicWeightMultiplier)
	{		
	}

	NIAGARAEDITOR_API class UEdGraphNode* CreateNode(UEdGraph* Graph, UEdGraphPin* FromPin, FVector2D NodePosition, bool bSelectNewNode = true) const;
	NIAGARAEDITOR_API class UEdGraphNode* CreateNode(UEdGraph* Graph, TArray<UEdGraphPin*>& FromPins, FVector2D NodePosition, bool bSelectNewNode = true) const;

	UPROPERTY()
	TWeakObjectPtr<class UEdGraphNode> WeakNodeTemplate = nullptr;
};

struct FNiagaraParameterAction : public FEdGraphSchemaAction
{
	FNiagaraParameterAction()
		: ScriptVar(nullptr)
		, Parameter(FNiagaraVariable())
		, ReferenceCollection()
		, bIsExternallyReferenced(false)
		, bIsSourcedFromCustomStackContext(false)
		, ParametersWithNamespaceModifierRenamePendingWeak()
	{
	}

	NIAGARAEDITOR_API FNiagaraParameterAction(const FNiagaraVariable& InParameter,
		const TArray<FNiagaraGraphParameterReferenceCollection>& InReferenceCollection,
		FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords,
		TSharedPtr<TArray<FName>> ParametersWithNamespaceModifierRenamePending,
		int32 InSectionID = 0);

	NIAGARAEDITOR_API FNiagaraParameterAction(const FNiagaraVariable& InParameter, 
		FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords,
		TSharedPtr<TArray<FName>> ParametersWithNamespaceModifierRenamePending,
		int32 InSectionID = 0);

	NIAGARAEDITOR_API FNiagaraParameterAction(const UNiagaraScriptVariable* InScriptVar,
		FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords,
		int32 InSectionID = 0);

	/** Simple type info. */
	static FName StaticGetTypeId() { return FNiagaraEditorStrings::FNiagaraParameterActionId; };
	virtual FName GetTypeId() const { return StaticGetTypeId(); };

	NIAGARAEDITOR_API const UNiagaraScriptVariable* GetScriptVar() const;

	NIAGARAEDITOR_API const FNiagaraVariable& GetParameter() const;

	NIAGARAEDITOR_API TArray<FNiagaraGraphParameterReferenceCollection>& GetReferenceCollection();

	NIAGARAEDITOR_API bool GetIsNamespaceModifierRenamePending() const;

	NIAGARAEDITOR_API void SetIsNamespaceModifierRenamePending(bool bIsNamespaceModifierRenamePending);

	NIAGARAEDITOR_API bool GetIsExternallyReferenced() const;

	NIAGARAEDITOR_API void SetIsExternallyReferenced(bool bInIsExternallyReferenced);

	NIAGARAEDITOR_API bool GetIsSourcedFromCustomStackContext() const;

	NIAGARAEDITOR_API void SetIsSourcedFromCustomStackContext(bool bInIsSourcedFromCustomStackContext);

private:
	const UNiagaraScriptVariable* ScriptVar;

	FNiagaraVariable Parameter;

	TArray<FNiagaraGraphParameterReferenceCollection> ReferenceCollection;

	bool bIsExternallyReferenced;

	bool bIsSourcedFromCustomStackContext;

	TWeakPtr<TArray<FName>> ParametersWithNamespaceModifierRenamePendingWeak;
};

struct FNiagaraScriptParameterAction : public FEdGraphSchemaAction
{
	FNiagaraScriptParameterAction() {}
	NIAGARAEDITOR_API FNiagaraScriptParameterAction(const FNiagaraVariable& InVariable, const FNiagaraVariableMetaData& InVariableMetaData);
};

class FNiagaraParameterGraphDragOperation : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FNiagaraParameterGraphDragOperation, FGraphSchemaActionDragDropAction)

	static NIAGARAEDITOR_API TSharedRef<FNiagaraParameterGraphDragOperation> New(const TSharedPtr<FEdGraphSchemaAction>& InActionNode);

	// FGraphEditorDragDropAction interface
	NIAGARAEDITOR_API virtual void HoverTargetChanged() override;
	NIAGARAEDITOR_API virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	NIAGARAEDITOR_API virtual FReply DroppedOnPanel(const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	// End of FGraphEditorDragDropAction

	/** Set if operation is modified by alt */
	void SetAltDrag(bool InIsAltDrag) { bAltDrag = InIsAltDrag; }

	/** Set if operation is modified by the ctrl key */
	void SetCtrlDrag(bool InIsCtrlDrag) { bControlDrag = InIsCtrlDrag; }

	/** Returns true if the drag operation is currently hovering over the supplied node */
	NIAGARAEDITOR_API bool IsCurrentlyHoveringNode(const UEdGraphNode* TestNode) const;

	const TSharedPtr<FEdGraphSchemaAction>& GetSourceAction() const { return SourceAction; }

protected:
	/** Constructor */
	NIAGARAEDITOR_API FNiagaraParameterGraphDragOperation();

	/** Structure for required node construction parameters */
	struct FNiagaraParameterNodeConstructionParams
	{
		FNiagaraParameterNodeConstructionParams() = delete;

		FNiagaraParameterNodeConstructionParams(
			const UE::Slate::FDeprecateVector2DParameter& InGraphPosition,
			UEdGraph* InGraph,
			const FNiagaraVariable& InParameter,
			const UNiagaraScriptVariable* InScriptVar)
			: GraphPosition(InGraphPosition)
			, Graph(InGraph)
			, Parameter(InParameter)
			, ScriptVar(InScriptVar)
		{};

		const FDeprecateSlateVector2D GraphPosition;
		UEdGraph* Graph;
		const FNiagaraVariable Parameter;
		const UNiagaraScriptVariable* ScriptVar;
	};

	static NIAGARAEDITOR_API void MakeGetMap(FNiagaraParameterNodeConstructionParams InParams);
	static NIAGARAEDITOR_API void MakeSetMap(FNiagaraParameterNodeConstructionParams InParams);
	static NIAGARAEDITOR_API void MakeStaticSwitch(FNiagaraParameterNodeConstructionParams InParams, const UNiagaraScriptVariable* ScriptVariable);

	NIAGARAEDITOR_API virtual EVisibility GetIconVisible() const override;
	NIAGARAEDITOR_API virtual EVisibility GetErrorIconVisible() const override;

	/** Was ctrl held down at start of drag */
	bool bControlDrag;
	/** Was alt held down at the start of drag */
	bool bAltDrag;
};

class FNiagaraParameterDragOperation : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FNiagaraParameterDragOperation, FDecoratedDragDropOp)

	FNiagaraParameterDragOperation(TSharedPtr<FEdGraphSchemaAction> InSourceAction)
		: SourceAction(InSourceAction)
	{
	}

	const TSharedPtr<FEdGraphSchemaAction>& GetSourceAction() const { return SourceAction; }

	NIAGARAEDITOR_API virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	void SetAdditionalText(FText InAdditionalText) { CurrentHoverText = InAdditionalText; }
	NIAGARAEDITOR_API EVisibility IsTextVisible() const;
private:
	TSharedPtr<FEdGraphSchemaAction> SourceAction;
	TOptional<FNiagaraVariable> TargetParameter;
};

class FNiagaraScriptDragOperation : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FNiagaraScriptDragOperation, FDecoratedDragDropOp)

		FNiagaraScriptDragOperation(UNiagaraScript* InPayloadScript, const FGuid& InVersion, const FText& InFriendlyName)
		: Script(InPayloadScript), Version(InVersion), FriendlyName(InFriendlyName)
	{
	}

	TWeakObjectPtr<UNiagaraScript> Script;
	FGuid Version;
	FText FriendlyName;
};


class INiagaraDataInterfaceNodeActionProvider
{
public:
	/** If a node displays its options 'inline' (title right widget), you can configure the way the combo button content is displayed and give it a tooltip. */
	struct FInlineMenuDisplayOptions
	{
		/** This needs to be true to have any effect, as it will determine widget generation in the title right area of the node. */
		bool bDisplayInline = false;
		FText DisplayName;
		const FSlateBrush* DisplayBrush = nullptr;
		FText TooltipText;
	};
	
	INiagaraDataInterfaceNodeActionProvider() = default;
	INiagaraDataInterfaceNodeActionProvider(INiagaraDataInterfaceNodeActionProvider&) = delete;
	INiagaraDataInterfaceNodeActionProvider(INiagaraDataInterfaceNodeActionProvider&&) = delete;
	INiagaraDataInterfaceNodeActionProvider& operator=(INiagaraDataInterfaceNodeActionProvider&) = delete;
	INiagaraDataInterfaceNodeActionProvider& operator=(INiagaraDataInterfaceNodeActionProvider&&) = delete;
	virtual ~INiagaraDataInterfaceNodeActionProvider() = default;

	/** Allows DIs to add context menu actions for the whole node. */
	virtual void GetNodeContextMenuActionsImpl(UToolMenu* Menu, UGraphNodeContextMenuContext* Context, const FNiagaraFunctionSignature& Signature) const {}

	virtual void GetInlineNodeContextMenuActionsImpl(UToolMenu* ToolMenu) const {}

	virtual FInlineMenuDisplayOptions GetInlineMenuDisplayOptionsImpl(UClass* DIClass, UEdGraphNode* Source) const { return {}; }
	
	/** Allows DIs to add actions for add pins on the DI function call nodes. */
	virtual void CollectAddPinActionsImpl(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin) const {}

	virtual TSharedPtr<SWidget> GetCustomFunctionSpecifierWidgetImpl(UNiagaraNodeFunctionCall* FunctionCallNode) const { return nullptr; }

	template<typename DIClass, typename ActionProviderClass>
	static void Register();

	template<typename DIClass>
	static void Unregister();

	template<typename DIClass>
	static void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context, const FNiagaraFunctionSignature& Signature);

	static NIAGARAEDITOR_API void GetNodeContextMenuActions(UClass* DIClass, UToolMenu* Menu, UGraphNodeContextMenuContext* Context, const FNiagaraFunctionSignature& Signature);

	static NIAGARAEDITOR_API void GetInlineNodeContextMenuActions(UClass* DIClass, UToolMenu* ToolMenu);
	
	static NIAGARAEDITOR_API FInlineMenuDisplayOptions GetInlineMenuDisplayOptions(UClass* DIClass, UEdGraphNode* Source);
	
	template<typename DIClass>
	static void CollectAddPinActions(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin);

	static NIAGARAEDITOR_API void CollectAddPinActions(UClass* DIClass, FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin);

	static NIAGARAEDITOR_API TSharedPtr<SWidget> GetCustomFunctionSpecifierWidget(UClass* DIClass, UNiagaraNodeFunctionCall* FunctionCallNode);

	/** All currently registered action providers. */
	static NIAGARAEDITOR_API TMap<FName, TUniquePtr<INiagaraDataInterfaceNodeActionProvider>> RegisteredActionProviders;
};

template<typename DIClass, typename ActionProviderClass>
void INiagaraDataInterfaceNodeActionProvider::Register()
{
	TUniquePtr<INiagaraDataInterfaceNodeActionProvider>& Provider = RegisteredActionProviders.FindOrAdd(DIClass::StaticClass()->GetFName());
	Provider = MakeUnique<ActionProviderClass>();
}

template<typename DIClass>
void INiagaraDataInterfaceNodeActionProvider::Unregister()
{
	RegisteredActionProviders.Remove(DIClass::StaticClass()->GetFName());
}

template<typename DIClass>
void INiagaraDataInterfaceNodeActionProvider::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context, const FNiagaraFunctionSignature& Signature)
{
	if(TUniquePtr<INiagaraDataInterfaceNodeActionProvider>* Provider = RegisteredActionProviders.Find(DIClass::StaticClass()->GetFName()))
	{
		(*Provider)->GetNodeContextMenuActionsImpl(Menu, Context, Signature);
	}
}

template<typename DIClass>
void INiagaraDataInterfaceNodeActionProvider::CollectAddPinActions(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin)
{
	if (TUniquePtr<INiagaraDataInterfaceNodeActionProvider>* Provider = RegisteredActionProviders.Find(DIClass::StaticClass()->GetFName()))
	{
		(*Provider)->CollectAddPinActionsImpl(Collector, AddPin);
	}
}

////////////////////////////////
/// Actions for engine data interfaces.
/////////////////////////////////

class FNiagaraDataInterfaceNodeActionProvider_DataChannelWrite : public INiagaraDataInterfaceNodeActionProvider
{
public:

	virtual void GetNodeContextMenuActionsImpl(UToolMenu* Menu, UGraphNodeContextMenuContext* Context, const FNiagaraFunctionSignature& Signature) const override;
	virtual void CollectAddPinActionsImpl(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin)const override;
	virtual void GetInlineNodeContextMenuActionsImpl(UToolMenu* ToolMenu) const override;
	virtual FInlineMenuDisplayOptions GetInlineMenuDisplayOptionsImpl(UClass* DIClass, UEdGraphNode* Source) const override;

private:
	static void AddDataChannelInitActions(UToolMenu* ToolMenu);
};

class FNiagaraDataInterfaceNodeActionProvider_DataChannelRead : public INiagaraDataInterfaceNodeActionProvider
{
public:

	virtual void GetNodeContextMenuActionsImpl(UToolMenu* Menu, UGraphNodeContextMenuContext* Context, const FNiagaraFunctionSignature& Signature) const override;
	virtual void GetInlineNodeContextMenuActionsImpl(UToolMenu* ToolMenu) const override;
	virtual FInlineMenuDisplayOptions GetInlineMenuDisplayOptionsImpl(UClass* DIClass, UEdGraphNode* Source) const override;
	virtual void CollectAddPinActionsImpl(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin)const override;
	virtual TSharedPtr<SWidget> GetCustomFunctionSpecifierWidgetImpl(UNiagaraNodeFunctionCall* FunctionCallNode) const override;

private:
	static void AddDataChannelInitActions(UToolMenu* ToolMenu);
};

class FNiagaraDataInterfaceNodeActionProvider_DataTable : public INiagaraDataInterfaceNodeActionProvider
{
public:
	virtual void GetInlineNodeContextMenuActionsImpl(UToolMenu* ToolMenu) const override;
	virtual FInlineMenuDisplayOptions GetInlineMenuDisplayOptionsImpl(UClass* DIClass, UEdGraphNode* Source) const override;

private:
	static void AddInitializeActions(UToolMenu* ToolMenu);
	static void AllocateNodePins(FSoftObjectPath DataTablePath, TWeakObjectPtr<UNiagaraNodeFunctionCall> WeakFunctionNode);
};
