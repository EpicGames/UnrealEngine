// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "NiagaraMessages.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraTypes.h"
#include "Engine/UserDefinedEnum.h"
#include "UObject/GCObject.h"
#include "UObject/SoftObjectPtr.h"
#include "NiagaraClipboard.generated.h"

class UHierarchyCategory;
class IPropertyHandle;
class UNiagaraDataInterface;
class UNiagaraScript;
class UNiagaraRendererProperties;
class UNiagaraNodeFunctionCall;
class UNiagaraScriptVariable;

UENUM()
enum class ENiagaraClipboardFunctionInputValueMode
{
	Local,
	Linked,
	Data,
	ObjectAsset,
	Expression,
	Dynamic,
	ResetToDefault // special paste mode where it resets to the default value
};

class UNiagaraClipboardFunction;

UCLASS(MinimalAPI)
class UNiagaraClipboardFunctionInput : public UObject
{
	GENERATED_BODY()

public:
	static NIAGARAEDITOR_API const UNiagaraClipboardFunctionInput* CreateLocalValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, TArray<uint8>& InLocalValueData);

	static NIAGARAEDITOR_API const UNiagaraClipboardFunctionInput* CreateLinkedValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, const FNiagaraVariableBase& InLinkedValue);

	static NIAGARAEDITOR_API const UNiagaraClipboardFunctionInput* CreateDataValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, UNiagaraDataInterface* InDataValue);

	static NIAGARAEDITOR_API const UNiagaraClipboardFunctionInput* CreateObjectAssetValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, UObject* InObject);

	static NIAGARAEDITOR_API const UNiagaraClipboardFunctionInput* CreateExpressionValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, const FString& InExpressionValue);

	static NIAGARAEDITOR_API const UNiagaraClipboardFunctionInput* CreateDynamicValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType, TOptional<bool> bInEditConditionValue, FString InDynamicValueName, UNiagaraScript* InDynamicValue, const FGuid& InScriptVersion);

	static NIAGARAEDITOR_API const UNiagaraClipboardFunctionInput* CreateDefaultInputValue(UObject* InOuter, FName InInputName, FNiagaraTypeDefinition InInputType);
	
	UPROPERTY()
	FName InputName;

	UPROPERTY()
	FNiagaraTypeDefinition InputType;

	UPROPERTY()
	bool bHasEditCondition;

	UPROPERTY()
	bool bEditConditionValue;

	UPROPERTY()
	ENiagaraClipboardFunctionInputValueMode ValueMode;

	UPROPERTY()
	TArray<uint8> Local;

	UPROPERTY()
	FNiagaraVariableBase Linked;

	UPROPERTY()
	TObjectPtr<UNiagaraDataInterface> Data;

	UPROPERTY()
	TObjectPtr<UObject> ObjectAsset;

	UPROPERTY()
	FString Expression;

	UPROPERTY()
	TObjectPtr<UNiagaraClipboardFunction> Dynamic;

	UPROPERTY()
	TArray<TObjectPtr<const UNiagaraClipboardFunctionInput>> ChildrenInputs;

	NIAGARAEDITOR_API bool CopyValuesFrom(const UNiagaraClipboardFunctionInput* InOther);

	const FNiagaraTypeDefinition& GetTypeDef() const { return InputType; };

};

UCLASS()
class UNiagaraClipboardRenderer : public UObject
{
	GENERATED_BODY()

public:
	static NIAGARAEDITOR_API const UNiagaraClipboardRenderer* CreateRenderer(UObject* InOuter, UNiagaraRendererProperties* Renderer, TOptional<FNiagaraStackNoteData> StackNoteData = TOptional<FNiagaraStackNoteData>(), TOptional<FText> AlternateDisplayName = TOptional<FText>());

	UPROPERTY()
	TObjectPtr<UNiagaraRendererProperties> RendererProperties;

	UPROPERTY()
	TOptional<FNiagaraStackNoteData> StackNoteData;

	UPROPERTY()
	TOptional<FText> AlternateDisplayName;
};

UENUM()
enum class ENiagaraClipboardFunctionScriptMode
{
	ScriptAsset,
	Assignment
};

UCLASS(MinimalAPI)
class UNiagaraClipboardFunction : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnPastedFunctionCallNode, UNiagaraNodeFunctionCall*, PastedFunctionCall);

	static NIAGARAEDITOR_API UNiagaraClipboardFunction* CreateScriptFunction(UObject* InOuter, FString InFunctionName, UNiagaraScript* InScript, const FGuid& InScriptVersion = FGuid(), const TOptional<FNiagaraStackNoteData>& InStackNote = TOptional<FNiagaraStackNoteData>());

	static NIAGARAEDITOR_API UNiagaraClipboardFunction* CreateAssignmentFunction(UObject* InOuter, FString InFunctionName, const TArray<FNiagaraVariable>& InAssignmentTargets, const TArray<FString>& InAssignmentDefaults, TOptional<FNiagaraStackNoteData> InStackNoteData = TOptional<FNiagaraStackNoteData>());

	UPROPERTY()
	FString FunctionName;

	UPROPERTY()
	FText DisplayName;

	UPROPERTY()
	ENiagaraClipboardFunctionScriptMode ScriptMode;

	UPROPERTY()
	TSoftObjectPtr<UNiagaraScript> Script;

	UPROPERTY()
	TArray<FNiagaraVariable> AssignmentTargets;

	UPROPERTY()
	TArray<FString> AssignmentDefaults;

	UPROPERTY()
	TArray<TObjectPtr<const UNiagaraClipboardFunctionInput>> Inputs;

	UPROPERTY()
	FOnPastedFunctionCallNode OnPastedFunctionCallNodeDelegate;

	UPROPERTY()
	FGuid ScriptVersion;
	
	UPROPERTY()
	FNiagaraStackNoteData StackNoteData;
};

USTRUCT()
struct FNiagaraClipboardScriptVariable
{
	GENERATED_BODY()

	FNiagaraClipboardScriptVariable() : ScriptVariable(nullptr)
	{}
	FNiagaraClipboardScriptVariable(const UNiagaraScriptVariable& InScriptVariable) : ScriptVariable(&InScriptVariable), OriginalChangeId(InScriptVariable.GetChangeId())
	{}

	UPROPERTY();
	TObjectPtr<const UNiagaraScriptVariable> ScriptVariable;

	/** We cache the original change Id here since deserialization of the clipboard will cause the change id to update.
	 *  Using the original change id, we can identify during pasting whether we have already pasted this script variable before. 
	 */
	UPROPERTY()
	FGuid OriginalChangeId;

	// since the contained variables are typically copies, we compare their change IDs instead
	bool operator==(const FNiagaraClipboardScriptVariable& OtherVariable) const
	{
		return OriginalChangeId == OtherVariable.OriginalChangeId;
	}
};

USTRUCT()
struct FNiagaraClipboardCurveCollection
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FRichCurve> Curves;
};

USTRUCT()
struct FNiagaraClipboardPortableValue
{
	GENERATED_BODY()

	class FErrorPipe : public FOutputDevice
	{
	public:

		int32 NumErrors;

		FErrorPipe()
			: FOutputDevice()
			, NumErrors(0)
		{
		}

		virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
		{
			NumErrors++;
		}
	};

	FNiagaraClipboardPortableValue()
		: ValueString(FString())
	{
	}

	bool IsValid() const { return ValueString.IsEmpty() == false; } // ValueType != ENiagaraClipboardPortableValueType::None; }

	void Reset()
	{
		*this = FNiagaraClipboardPortableValue();
	}

	static FNiagaraClipboardPortableValue NIAGARAEDITOR_API CreateFromStructValue(const UScriptStruct& TargetStruct, uint8* StructMemory);
	static FNiagaraClipboardPortableValue NIAGARAEDITOR_API CreateFromTypedValue(const FNiagaraTypeDefinition& InType, const FNiagaraVariant& InValue);
	static FNiagaraClipboardPortableValue NIAGARAEDITOR_API CreateFromPropertyHandle(const IPropertyHandle& InPropertyHandle);

	template<typename StructType>
	static FNiagaraClipboardPortableValue CreateFromStructValue(const StructType& InStructValue)
	{
		return CreateFromStructValue(*StructType::StaticStruct(), (uint8*)&InStructValue);
	}

	NIAGARAEDITOR_API bool TryUpdateStructValue(const UScriptStruct& TargetStruct, uint8* StructMemory) const;
	NIAGARAEDITOR_API bool CanUpdateTypedValue(const FNiagaraTypeDefinition& InTargetInputType) const;
	NIAGARAEDITOR_API bool TryUpdateTypedValue(const FNiagaraTypeDefinition& InTargetInputType, FNiagaraVariant& InValue) const;
	NIAGARAEDITOR_API bool TryUpdatePropertyHandle(IPropertyHandle& InTargetPropertyHandle) const;

	template<typename StructType>
	bool TryUpdateStructValue(StructType& OutStructValue) const 
	{
		return TryUpdateStructValue(*StructType::StaticStruct(), (uint8*)&OutStructValue);
	}

	UPROPERTY()
	FString ValueString;
};

UCLASS(MinimalAPI)
class UNiagaraClipboardContent : public UObject
{
	GENERATED_BODY()

public:
	static NIAGARAEDITOR_API UNiagaraClipboardContent* Create();

	UPROPERTY()
	TArray<TObjectPtr<const UNiagaraClipboardFunction>> Functions;

	UPROPERTY()
	TArray<TObjectPtr<const UNiagaraClipboardFunctionInput>> FunctionInputs;
	
	UPROPERTY()
	TArray<TObjectPtr<const UNiagaraClipboardRenderer>> Renderers;

	UPROPERTY()
	TArray<TObjectPtr<const UNiagaraScript>> Scripts;

	UPROPERTY()
	TArray<FNiagaraClipboardScriptVariable> ScriptVariables;

	UPROPERTY()
	TArray<TObjectPtr<const UObject>> StatelessModules;

	/** We expect nodes to be exported into this string using FEdGraphUtilities::ExportNodesToText */
	UPROPERTY()
	FString ExportedNodes;

	/** Markup MetaData to specify that if scripts are pasted from the clipboard to automatically fixup their order in the stack to satisfy dependencies. */
	UPROPERTY()
	mutable bool bFixupPasteIndexForScriptDependenciesInStack = false;

	UPROPERTY()
	FNiagaraStackNoteData StackNote;

	UPROPERTY()
	TArray<FNiagaraClipboardPortableValue> PortableValues;
};

class FNiagaraClipboard
{
public:
	NIAGARAEDITOR_API FNiagaraClipboard();

	NIAGARAEDITOR_API void SetClipboardContent(UNiagaraClipboardContent* ClipboardContent);

	NIAGARAEDITOR_API const UNiagaraClipboardContent* GetClipboardContent() const;

	NIAGARAEDITOR_API static FString ClipboardContentToString(UNiagaraClipboardContent* ClipboardContent);

	NIAGARAEDITOR_API static UNiagaraClipboardContent* ClipboardContentFromString(const FString& ClipboardText);
};

UCLASS(MinimalAPI)
class UNiagaraClipboardEditorScriptingUtilities : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API void TryGetInputByName(const TArray<UNiagaraClipboardFunctionInput*>& InInputs, FName InInputName, bool& bOutSucceeded, UNiagaraClipboardFunctionInput*& OutInput);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API void TryGetLocalValueAsFloat(const UNiagaraClipboardFunctionInput* InInput, bool& bOutSucceeded, float& OutValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API void TryGetLocalValueAsInt(const UNiagaraClipboardFunctionInput* InInput, bool& bOutSucceeded, int32& OutValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API void TrySetLocalValueAsInt(UNiagaraClipboardFunctionInput* InInput, bool& bOutSucceeded, int32 InValue, bool bLooseTyping = true);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API FName GetTypeName(const UNiagaraClipboardFunctionInput* InInput);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateColorLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FLinearColor InColorValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateFloatLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, float InLocalValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateMatrixLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FMatrix44f InMatrixValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateQuatLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FQuat4f InQuatValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateVec2LocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FVector2D InVec2Value);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateVec3LocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FVector InVec3Value);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateVec4LocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, FVector4f InVec4Value);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateIntLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, int32 InLocalValue);
	
	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateBoolLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, bool InBoolValue);
	
	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateStructLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, UUserDefinedStruct* InStructValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateEnumLocalValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditCoditionValue, UUserDefinedEnum* InEnumType, int32 InEnumValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateEnumLinkedValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditCoditionValue, UUserDefinedEnum* InEnumType, FName InLinkedValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateLinkedValueInput(UObject* InOuter, FName InInputName, FName InInputTypeName, bool bInHasEditCondition, bool bInEditConditionValue, FName InLinkedValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateDataValueInput(UObject* InOuter, FName InInputName, bool bInHasEditCondition, bool bInEditConditionValue, UNiagaraDataInterface* InDataValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateExpressionValueInput(UObject* InOuter, FName InInputName, FName InInputTypeName, bool bInHasEditCondition, bool bInEditConditionValue, const FString& InExpressionValue);

	UFUNCTION(BlueprintPure, Category = "Input")
	static NIAGARAEDITOR_API UNiagaraClipboardFunctionInput* CreateDynamicValueInput(UObject* InOuter, FName InInputName, FName InInputTypeName, bool bInHasEditCondition, bool bInEditConditionValue, FString InDynamicValueName, UNiagaraScript* InDynamicValue);

	static NIAGARAEDITOR_API FNiagaraTypeDefinition GetRegisteredTypeDefinitionByName(FName InTypeName);
};
