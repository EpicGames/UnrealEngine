// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"

// forward declarations for the GraphBridge
class UNiagaraGraph;
class UEdGraphPin;
class UNiagaraNodeCustomHlsl;
class UNiagaraNode;
class UNiagaraNodeOutput;
class UNiagaraNodeInput;
class UNiagaraNodeOp;
class UNiagaraNodeEmitter;
class UNiagaraNodeIf;
class UNiagaraNodeConvert;
class UNiagaraNodeSelect;
class UNiagaraNodeFunctionCall;
class UNiagaraNodeParameterMapGet;
class UNiagaraNodeParameterMapSet;
class UNiagaraNodeParameterMapFor;
class UNiagaraNodeStaticSwitch;
template<typename GraphBridge> struct TNiagaraParameterMapHistory;
template<typename GraphBridge> class TNiagaraParameterMapHistoryBuilder;
class FNiagaraCompileRequestData;
class FNiagaraCompileRequestDuplicateData;
struct FGraphTraversalHandle;
struct FNiagaraGraphFunctionAliasContext;
struct FNiagaraConvertConnection;
class UNiagaraParameterCollection;
template<typename PinType> struct TModuleScopedPin;
struct FNiagaraCustomHlslInclude;
struct FNiagaraFindInputNodeOptions;
class FCompileConstantResolver;
struct FNiagaraStaticVariableSearchContext;

// forward declarations for the DigestBridge
class FNiagaraCompilationGraph;
class FNiagaraCompilationPin;
class FNiagaraCompilationNodeCustomHlsl;
class FNiagaraCompilationNode;
class FNiagaraCompilationNodeOutput;
class FNiagaraCompilationNodeInput;
class FNiagaraCompilationNodeOp;
class FNiagaraCompilationNodeEmitter;
class FNiagaraCompilationNodeIf;
class FNiagaraCompilationNodeConvert;
class FNiagaraCompilationNodeSelect;
class FNiagaraCompilationNodeFunctionCall;
class FNiagaraCompilationNodeParameterMapGet;
class FNiagaraCompilationNodeParameterMapSet;
class FNiagaraCompilationNodeParameterMapFor;
class FNiagaraCompilationNodeStaticSwitch;
class FNiagaraCompilationInputPin;
class FNiagaraCompilationOutputPin;
class FNiagaraPrecompileData;
class FNiagaraCompilationCopyData;
struct FNiagaraDigestFunctionAliasContext;
struct FNiagaraCompilationCachedConnection;
class FNiagaraCompilationNPCHandle;
class FNiagaraFixedConstantResolver;
struct FNiagaraTraversalStateContext;
class FNiagaraDigestedParameterCollections;

// these two structures represent a way to abstract the implementation details of the graph structure
// that is being used for the different pieces of code related to compiling Niagara systems.  The GraphBridge
// handles the standard editor objects for graphs & nodes (UNiagaraGraph and UNiagaraNode respectively) with
// the remaining types following from that.  The DigestBridge represents the digested graphs and nodes that
// we use to be able to compile Niagara systems async where the graphs & nodes are FNiagaraCompilationGraph
// FNiagaraCompilationNode respectively.
//
// The bridge class is used as a template argument for the following classes (and supplementary functions/classes):
//	-FNiagaraParameterMapHistory
//	-FNiagaraParameterMapHistoryBuilder
//	-FNiagaraHlslTranslator
//
// This allows us to have the logic for these classes unified between the different graph representations
// without having to write two versions of the code.
struct FNiagaraCompilationGraphBridge
{
	// base types
	using FGraph = UNiagaraGraph;
	using FPin = UEdGraphPin;
	using FCustomHlslNode = UNiagaraNodeCustomHlsl;
	using FNode = UNiagaraNode;
	using FOutputNode = UNiagaraNodeOutput;
	using FInputNode = UNiagaraNodeInput;
	using FOpNode = UNiagaraNodeOp;
	using FEmitterNode = UNiagaraNodeEmitter;
	using FIfNode = UNiagaraNodeIf;
	using FConvertNode = UNiagaraNodeConvert;
	using FSelectNode = UNiagaraNodeSelect;
	using FFunctionCallNode = UNiagaraNodeFunctionCall;
	using FParamMapGetNode = UNiagaraNodeParameterMapGet;
	using FParamMapSetNode = UNiagaraNodeParameterMapSet;
	using FParamMapForNode = UNiagaraNodeParameterMapFor;
	using FStaticSwitchNode = UNiagaraNodeStaticSwitch;
	using FInputPin = UEdGraphPin;
	using FOutputPin = UEdGraphPin;
	using FParamMapHistory = TNiagaraParameterMapHistory<FNiagaraCompilationGraphBridge>;
	using FParamMapHistoryBuilder = TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationGraphBridge>;
	using FPrecompileData = FNiagaraCompileRequestData;
	using FCompilationCopy = FNiagaraCompileRequestDuplicateData;
	using FModuleScopedPin = TModuleScopedPin<FPin>;
	using FGraphTraversalHandle = FGraphTraversalHandle;
	using FGraphFunctionAliasContext = FNiagaraGraphFunctionAliasContext;
	using FConvertConnection = FNiagaraConvertConnection;
	using FParameterCollection = UNiagaraParameterCollection*;
	using FConstantResolver = FCompileConstantResolver;

	// additional data for extending the ParameterMapHistoryBuilder
	struct FParameterCollectionStore
	{
		void Append(const FParameterCollectionStore& Other);
		void Add(UNiagaraParameterCollection* Collection);

		TArray<UNiagaraParameterCollection*> Collections;
		/** Cached off contents of used parameter collections, in case they change during threaded compilation. */
		TArray<TArray<FNiagaraVariable>> CollectionVariables;
		/** Cached off contents of used parameter collections, in case they change during threaded compilation. */
		TArray<FName> CollectionNamespaces;
	};

	struct FAvailableParameterCollections
	{
		UNiagaraParameterCollection* FindCollection(const FNiagaraVariable& Variable) const;
		UNiagaraParameterCollection* FindMatchingCollection(FName VariableName, bool bAllowPartialMatch, FNiagaraVariable& OutVar) const;
	};

	// Used as a base class for the FNiagaraParameterMapHistoryBuilder to store additional data that is specific
	// to the graph representation
	class FBuilderExtraData
	{
	public:
		FBuilderExtraData();
		TUniquePtr<FAvailableParameterCollections> AvailableCollections;
	};

	static const FGraph* GetGraph(const FCompilationCopy* CompilationCopy);
	static const FNode* GetOwningNode(const FPin* Pin);
	static FNode* GetMutableOwningNode(const FPin* Pin);
	static const FGraph* GetOwningGraph(const FNode* Node);
	static bool CustomHlslReferencesTokens(const FCustomHlslNode* CustomNode, TConstArrayView<FStringView> TokenStrings);
	static void CustomHlslReferencesTokens(const FCustomHlslNode* CustomNode, TConstArrayView<FName> TokenStrings, TArrayView<bool> Results);
	static ENiagaraScriptUsage GetCustomHlslUsage(const FCustomHlslNode* CustomNode);
	static FString GetCustomHlslString(const FCustomHlslNode* CustomNode);
	static void GetCustomHlslIncludePaths(const FCustomHlslNode* CustomNode, TArray<FNiagaraCustomHlslInclude>& Includes);
	static const TArray<FConvertConnection>& GetConvertConnections(const FConvertNode* ConvertNode);

	// various cast functions
	static const FFunctionCallNode* AsFunctionCallNode(const FNode* Node);
	static const FInputNode* AsInputNode(const FNode* Node);
	static const FParamMapGetNode* AsParamMapGetNode(const FNode* Node);
	static const FCustomHlslNode* AsCustomHlslNode(const FNode* Node);
	static const FParamMapSetNode* AsParamMapSetNode(const FNode* Node);

	static bool GraphHasParametersOfType(const FGraph* Graph, const FNiagaraTypeDefinition& TypeDef);
	static TArray<FNiagaraVariableBase> GraphGetStaticSwitchInputs(const FGraph* Graph);
	static void FindOutputNodes(const FGraph* Graph, ENiagaraScriptUsage ScriptUsage, TArray<const FOutputNode*>& OutputNodes);
	static void FindOutputNodes(const FGraph* Graph, TArray<const FOutputNode*>& OutputNodes);
	static const FOutputNode* FindOutputNode(const FGraph* Graph, ENiagaraScriptUsage ScriptUsage, const FGuid& UsageId);
	static void BuildTraversal(const FGraph* Graph, const FNode* OutputNode, TArray<const FNode*>& TraversedNodes);
	static const FGraph* GetEmitterGraph(const FEmitterNode* EmitterNode);
	static FString GetEmitterUniqueName(const FEmitterNode* EmitterNode);
	static FNiagaraEmitterID GetEmitterID(const FEmitterNode* EmitterNode);
	static ENiagaraScriptUsage GetEmitterUsage(const FEmitterNode* EmitterNode);
	static FString GetEmitterName(const FEmitterNode* EmitterNode);
	static FString GetEmitterPathName(const FEmitterNode* EmitterNode);
	static FString GetEmitterHandleIdString(const FEmitterNode* EmitterNode);
	static const FGraph* GetFunctionNodeGraph(const FFunctionCallNode* FunctionCall);
	static FString GetFunctionFullName(const FFunctionCallNode* FunctionCall);
	static FString GetFunctionScriptName(const FFunctionCallNode* FunctionCall);
	static FString GetFunctionName(const FFunctionCallNode* FunctionCall);
	static ENiagaraScriptUsage GetFunctionUsage(const FFunctionCallNode* FunctionCall);
	static TOptional<ENiagaraDefaultMode> GetGraphDefaultMode(const FGraph* Graph, const FNiagaraVariable& Variable, FNiagaraScriptVariableBinding& Binding);
	static const FInputPin* GetDefaultPin(const FParamMapGetNode* GetNode, const FOutputPin* OutputPin);
	static bool IsStaticPin(const FPin* Pin);
	// retrieves all input pins (excluding any add pins that may be present)
	static TArray<const FInputPin*> GetInputPins(const FNode* Node);
	// retrieves all output pins (excluding both orphaned pins and add pins)
	static TArray<const FOutputPin*> GetOutputPins(const FNode* Node);
	// gets all pins assoicated with the node
	static TArray<const FPin*> GetPins(const FNode* Node);
	static FNiagaraTypeDefinition GetPinType(const FPin* Pin, ENiagaraStructConversion Conversion);
	static FText GetPinFriendlyName(const FPin* Pin);
	static FText GetPinDisplayName(const FPin* Pin);
	static FNiagaraVariable GetPinVariable(const FPin* Pin, bool bNeedsValue, ENiagaraStructConversion Conversion);
	static const FInputPin* GetPinAsInput(const FPin* Pin);
	static FNiagaraVariable GetInputVariable(const FInputNode* InputNode);
	static const TArray<FNiagaraVariable>& GetOutputVariables(const FOutputNode* OutputNode);
	static TArray<FNiagaraVariable> GetGraphOutputNodeVariables(const FGraph* Graph, ENiagaraScriptUsage Usage);
	static TArray<const FInputNode*> GetGraphInputNodes(const FGraph* Graph, const FNiagaraFindInputNodeOptions& Options);
	static const FOutputPin* GetLinkedOutputPin(const FInputPin* InputPin);
	static bool CanCreateConnection(const FOutputPin* OutputPin, const FInputPin* InputPin, FText& FailureMessage);
	static ENiagaraScriptUsage GetOutputNodeUsage(const FOutputNode* OutputNode);
	static FGuid GetOutputNodeUsageId(const FOutputNode* OutputNode);
	static ENiagaraScriptUsage GetOutputNodeScriptType(const FOutputNode* OutputNode);
	static FGuid GetOutputNodeScriptTypeId(const FOutputNode* OutputNode);
	static bool IsGraphEmpty(const FGraph* Graph);
	static void AddCollectionPaths(const FParamMapHistory& History, TArray<FString>& Paths);
	static bool NodeIsEnabled(const FNode* Node);
	static TOptional<ENiagaraDefaultMode> GraphGetDefaultMode(const FGraph* Graph, const FNiagaraVariableBase& Variable, FNiagaraScriptVariableBinding& Binding);
	static const FOutputPin* GetSelectOutputPin(const FSelectNode* SelectNode, const FNiagaraVariableBase& Variable);
	static FString GetNodeName(const FNode* Node);
	static FString GetNodeTitle(const FNode* Node);
	static const FInputPin* GetInputPin(const FNode* Node, int32 PinIndex);
	static int32 GetPinIndexById(TConstArrayView<const FPin*> Pins, const FGuid& PinId);
	static FString GetCollectionFullName(FParameterCollection Collection);
	static bool IsCollectionValid(FParameterCollection Collection);
	static UNiagaraDataInterface* GetCollectionDataInterface(FParameterCollection Collection, const FNiagaraVariable& Variable);
	static TObjectPtr<UObject> GetCollectionUObject(FParameterCollection Collection, const FNiagaraVariable& Variable);
	static FParameterCollection IsParameterCollectionParameter(const FParamMapHistory& ParamMapHistory, const FNiagaraVariable& InVar, bool& bMissingParameter);

	static const FOutputNode* AsOutputNode(const FNode* Node);
	static bool IsParameterMapGet(const FNode* Node);
	static TOptional<FName> GetOutputNodeStackContextOverride(const FOutputNode* OutputNode);
	static FString GetNodeClassName(const FNode* Node);
	static bool IsParameterMapPin(const FPin* Pin);
	static bool GetGraphReferencesStaticVariables(const FGraph* Graph, FNiagaraStaticVariableSearchContext& StaticVariableContext);
	static const FEmitterNode* GetNodeAsEmitter(const FNode* Node);

	static bool GetCustomNodeUsesImpureFunctions(const FCustomHlslNode* CustomNode);
};

struct FNiagaraCompilationDigestBridge
{
	using FGraph = FNiagaraCompilationGraph;
	using FPin = FNiagaraCompilationPin;
	using FCustomHlslNode = FNiagaraCompilationNodeCustomHlsl;
	using FNode = FNiagaraCompilationNode;
	using FOutputNode = FNiagaraCompilationNodeOutput;
	using FInputNode = FNiagaraCompilationNodeInput;
	using FOpNode = FNiagaraCompilationNodeOp;
	using FEmitterNode = FNiagaraCompilationNodeEmitter;
	using FIfNode = FNiagaraCompilationNodeIf;
	using FConvertNode = FNiagaraCompilationNodeConvert;
	using FSelectNode = FNiagaraCompilationNodeSelect;
	using FFunctionCallNode = FNiagaraCompilationNodeFunctionCall;
	using FParamMapGetNode = FNiagaraCompilationNodeParameterMapGet;
	using FParamMapSetNode = FNiagaraCompilationNodeParameterMapSet;
	using FParamMapForNode = FNiagaraCompilationNodeParameterMapFor;
	using FStaticSwitchNode = FNiagaraCompilationNodeStaticSwitch;
	using FInputPin = FNiagaraCompilationInputPin;
	using FOutputPin = FNiagaraCompilationOutputPin;
	using FParamMapHistory = TNiagaraParameterMapHistory<FNiagaraCompilationDigestBridge>;
	using FParamMapHistoryBuilder = TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationDigestBridge>;
	using FPrecompileData = FNiagaraPrecompileData;
	using FCompilationCopy = FNiagaraCompilationCopyData;
	using FModuleScopedPin = TModuleScopedPin<FPin>;
	using FGraphFunctionAliasContext = FNiagaraDigestFunctionAliasContext;
	using FConvertConnection = FNiagaraCompilationCachedConnection;
	using FParameterCollection = FNiagaraCompilationNPCHandle;
	using FConstantResolver = FNiagaraFixedConstantResolver;

	struct FParameterCollectionStore
	{
		void Append(const FParameterCollectionStore& Other);
		TArray<FNiagaraCompilationNPCHandle> Handles;
	};

	// Used as a base class for the FNiagaraParameterMapHistoryBuilder to store additional data that is specific
	// to the graph representation
	class FBuilderExtraData
	{
	public:
		FBuilderExtraData();
		TPimplPtr<FNiagaraTraversalStateContext> TraversalStateContext;
		TUniquePtr<FNiagaraDigestedParameterCollections> AvailableCollections;
	};

	static const FGraph* GetGraph(const FCompilationCopy* CompilationCopy);
	static const FNode* GetOwningNode(const FPin* Pin);
	static FNode* GetMutableOwningNode(const FPin* Pin);
	static const FGraph* GetOwningGraph(const FNode* Node);
	static bool CustomHlslReferencesTokens(const FCustomHlslNode* CustomNode, TConstArrayView<FStringView> TokenStrings);
	static void CustomHlslReferencesTokens(const FCustomHlslNode* CustomNode, TConstArrayView<FName> TokenStrings, TArrayView<bool> Results);
	static ENiagaraScriptUsage GetCustomHlslUsage(const FCustomHlslNode* CustomNode);
	static FString GetCustomHlslString(const FCustomHlslNode* CustomNode);
	static void GetCustomHlslIncludePaths(const FCustomHlslNode* CustomNode, TArray<FNiagaraCustomHlslInclude>& Includes);
	static const TArray<FConvertConnection>& GetConvertConnections(const FConvertNode* ConvertNode);

	// various cast functions
	static const FFunctionCallNode* AsFunctionCallNode(const FNode* Node);
	static const FInputNode* AsInputNode(const FNode* Node);
	static const FParamMapGetNode* AsParamMapGetNode(const FNode* Node);
	static const FCustomHlslNode* AsCustomHlslNode(const FNode* Node);
	static const FParamMapSetNode* AsParamMapSetNode(const FNode* Node);

	static bool GraphHasParametersOfType(const FGraph* Graph, const FNiagaraTypeDefinition& TypeDef);
	static TArray<FNiagaraVariableBase> GraphGetStaticSwitchInputs(const FGraph* Graph);
	static void FindOutputNodes(const FGraph* Graph, ENiagaraScriptUsage ScriptUsage, TArray<const FOutputNode*>& OutputNodes);
	static void FindOutputNodes(const FGraph* Graph, TArray<const FOutputNode*>& OutputNodes);
	static const FOutputNode* FindOutputNode(const FGraph* Graph, ENiagaraScriptUsage ScriptUsage, const FGuid& UsageId);
	static void BuildTraversal(const FGraph* Graph, const FNode* OutputNode, TArray<const FNode*>& TraversedNodes);
	static const FGraph* GetEmitterGraph(const FEmitterNode* EmitterNode);
	static FNiagaraEmitterID GetEmitterID(const FEmitterNode* EmitterNode);
	static FString GetEmitterUniqueName(const FEmitterNode* EmitterNode);
	static ENiagaraScriptUsage GetEmitterUsage(const FEmitterNode* EmitterNode);
	static FString GetEmitterName(const FEmitterNode* EmitterNode);
	static FString GetEmitterPathName(const FEmitterNode* EmitterNode);
	static FString GetEmitterHandleIdString(const FEmitterNode* EmitterNode);
	static const FGraph* GetFunctionNodeGraph(const FFunctionCallNode* FunctionCall);
	static FString GetFunctionFullName(const FFunctionCallNode* FunctionCall);
	static FString GetFunctionScriptName(const FFunctionCallNode* FunctionCall);
	static FString GetFunctionName(const FFunctionCallNode* FunctionCall);
	static ENiagaraScriptUsage GetFunctionUsage(const FFunctionCallNode* FunctionCall);
	static TOptional<ENiagaraDefaultMode> GetGraphDefaultMode(const FGraph* Graph, const FNiagaraVariable& Variable, FNiagaraScriptVariableBinding& Binding);
	static const FInputPin* GetDefaultPin(const FParamMapGetNode* GetNode, const FOutputPin* OutputPin);
	static bool IsStaticPin(const FPin* Pin);
	// retrieves all input pins (excluding any add pins that may be present)
	static TArray<const FInputPin*> GetInputPins(const FNode* Node);
	// retrieves all output pins (excluding both orphaned pins and add pins)
	static TArray<const FOutputPin*> GetOutputPins(const FNode* Node);
	// gets all pins assoicated with the node
	static TArray<const FPin*> GetPins(const FNode* Node);
	static FNiagaraTypeDefinition GetPinType(const FPin* Pin, ENiagaraStructConversion Conversion);
	static FText GetPinFriendlyName(const FPin* Pin);
	static FText GetPinDisplayName(const FPin* Pin);
	static FNiagaraVariable GetPinVariable(const FPin* Pin, bool bNeedsValue, ENiagaraStructConversion Conversion);
	static const FInputPin* GetPinAsInput(const FPin* Pin);
	static FNiagaraVariable GetInputVariable(const FInputNode* InputNode);
	static const TArray<FNiagaraVariable>& GetOutputVariables(const FOutputNode* OutputNode);
	static TArray<FNiagaraVariable> GetGraphOutputNodeVariables(const FGraph* Graph, ENiagaraScriptUsage Usage);
	static TArray<const FInputNode*> GetGraphInputNodes(const FGraph* Graph, const FNiagaraFindInputNodeOptions& Options);
	static const FOutputPin* GetLinkedOutputPin(const FInputPin* InputPin);
	static bool CanCreateConnection(const FOutputPin* OutputPin, const FInputPin* InputPin, FText& FailureMessage);
	static ENiagaraScriptUsage GetOutputNodeUsage(const FOutputNode* OutputNode);
	static FGuid GetOutputNodeUsageId(const FOutputNode* OutputNode);
	static ENiagaraScriptUsage GetOutputNodeScriptType(const FOutputNode* OutputNode);
	static FGuid GetOutputNodeScriptTypeId(const FOutputNode* OutputNode);
	static bool IsGraphEmpty(const FGraph* Graph);
	static void AddCollectionPaths(const FParamMapHistory& History, TArray<FString>& Paths);
	static bool NodeIsEnabled(const FNode* Node);
	static TOptional<ENiagaraDefaultMode> GraphGetDefaultMode(const FGraph* Graph, const FNiagaraVariableBase& Variable, FNiagaraScriptVariableBinding& Binding);
	static const FOutputPin* GetSelectOutputPin(const FSelectNode* SelectNode, const FNiagaraVariableBase& Variable);
	static FString GetNodeName(const FNode* Node);
	static FString GetNodeTitle(const FNode* Node);
	static const FInputPin* GetInputPin(const FNode* Node, int32 PinIndex);
	static int32 GetPinIndexById(TConstArrayView<const FInputPin*> Pins, const FGuid& PinId);
	static int32 GetPinIndexById(TConstArrayView<const FOutputPin*> Pins, const FGuid& PinId);
	static FString GetCollectionFullName(FParameterCollection Collection);
	static bool IsCollectionValid(FParameterCollection Collection);
	static UNiagaraDataInterface* GetCollectionDataInterface(FParameterCollection Collection, const FNiagaraVariable& Variable);
	static UObject* GetCollectionUObject(FParameterCollection Collection, const FNiagaraVariable& Variable);
	static FParameterCollection IsParameterCollectionParameter(const FParamMapHistory& ParamMapHistory, const FNiagaraVariable& InVar, bool& bMissingParameter);

	static const FOutputNode* AsOutputNode(const FNode* Node);
	static bool IsParameterMapGet(const FNode* Node);
	static TOptional<FName> GetOutputNodeStackContextOverride(const FOutputNode* OutputNode);
	static FString GetNodeClassName(const FNode* Node);
	static bool IsParameterMapPin(const FPin* Pin);
	static bool GetGraphReferencesStaticVariables(const FGraph* Graph, FNiagaraStaticVariableSearchContext& StaticVariableContext);
	static const FEmitterNode* GetNodeAsEmitter(const FNode* Node);

	static bool GetCustomNodeUsesImpureFunctions(const FCustomHlslNode* CustomNode);
};
