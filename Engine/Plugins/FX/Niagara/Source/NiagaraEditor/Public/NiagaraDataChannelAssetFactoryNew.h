// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "NiagaraDataChannelAssetFactoryNew.generated.h"

UCLASS(hidecategories = Object)
class UNiagaraDataChannelAssetFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;

	virtual bool CanCreateNew()const override;
	//~ Begin UFactory Interface
};
