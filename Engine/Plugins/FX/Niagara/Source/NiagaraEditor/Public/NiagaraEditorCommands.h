// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

UE_DECLARE_TCOMMANDS(class FNiagaraEditorCommands, NIAGARAEDITOR_API)

/**
* Defines commands for the niagara editor.
*/
class FNiagaraEditorCommands : public TCommands<FNiagaraEditorCommands>
{
public:
	FNiagaraEditorCommands()
		: TCommands<FNiagaraEditorCommands>
		(
			TEXT("NiagaraEditor"),
			NSLOCTEXT("Contexts", "NiagaraEditor", "Niagara Editor"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		)
	{ }

	NIAGARAEDITOR_API virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> Apply;
	TSharedPtr<FUICommandInfo> ApplyScratchPadChanges;
	TSharedPtr<FUICommandInfo> Discard;
	TSharedPtr<FUICommandInfo> Compile;
	TSharedPtr<FUICommandInfo> RefreshNodes;
	TSharedPtr<FUICommandInfo> ModuleVersioning;
	TSharedPtr<FUICommandInfo> EmitterVersioning;
	TSharedPtr<FUICommandInfo> ResetSimulation;
	TSharedPtr<FUICommandInfo> SelectNextUsage;
	TSharedPtr<FUICommandInfo> CreateAssetFromSelection;

	TSharedPtr<FUICommandInfo> OpenAddEmitterMenu;
	
	/** Toggles the preview pane's grid */
	TSharedPtr<FUICommandInfo> TogglePreviewGrid;
	TSharedPtr<FUICommandInfo> ToggleOriginAxis;
	TSharedPtr<FUICommandInfo> ToggleInstructionCounts;
	TSharedPtr<FUICommandInfo> ToggleParticleCounts;
	TSharedPtr<FUICommandInfo> ToggleEmitterExecutionOrder;
	TSharedPtr<FUICommandInfo> ToggleGpuTickInformation;
	TSharedPtr<FUICommandInfo> ToggleMemoryInfo;
	TSharedPtr<FUICommandInfo> ToggleStatelessInfo;

	/** Toggles the preview pane's background */
	TSharedPtr< FUICommandInfo > TogglePreviewBackground;

	/** Toggles the locking/unlocking of refreshing from changes*/
	TSharedPtr<FUICommandInfo> ToggleUnlockToChanges;

	TSharedPtr<FUICommandInfo> ToggleOrbit;
	TSharedPtr<FUICommandInfo> ToggleBounds;
	TSharedPtr<FUICommandInfo> ToggleBounds_SetFixedBounds_SelectedEmitters;
	TSharedPtr<FUICommandInfo> ToggleBounds_SetFixedBounds_System;
	TSharedPtr<FUICommandInfo> SaveThumbnailImage;

	TSharedPtr<FUICommandInfo> ToggleMotion;

	TSharedPtr<FUICommandInfo> ToggleStatPerformance;
	TSharedPtr<FUICommandInfo> ToggleStatPerformanceGPU;
	TSharedPtr<FUICommandInfo> ToggleProfileInEditMode;
	TSharedPtr<FUICommandInfo> ClearStatPerformance;
	TSharedPtr<FUICommandInfo> ToggleStatPerformanceTypeAvg;
	TSharedPtr<FUICommandInfo> ToggleStatPerformanceTypeMax;
	TSharedPtr<FUICommandInfo> ToggleStatPerformanceModePercent;
	TSharedPtr<FUICommandInfo> ToggleStatPerformanceModeAbsolute;

	TSharedPtr<FUICommandInfo> OpenDebugHUD;
	TSharedPtr<FUICommandInfo> OpenDebugOutliner;
	TSharedPtr<FUICommandInfo> OpenAttributeSpreadsheet;

	TSharedPtr<FUICommandInfo> ToggleAutoPlay;
	TSharedPtr<FUICommandInfo> ToggleResetSimulationOnChange;
	TSharedPtr<FUICommandInfo> ToggleResimulateOnChangeWhilePaused;
	TSharedPtr<FUICommandInfo> ToggleResetDependentSystems;

	TSharedPtr<FUICommandInfo> IsolateSelectedEmitters;
	TSharedPtr<FUICommandInfo> EnableSelectedEmitters;
	TSharedPtr<FUICommandInfo> DisableSelectedEmitters;
	TSharedPtr<FUICommandInfo> HideDisabledModules;

	TSharedPtr<FUICommandInfo> CollapseStackToHeaders;

	TSharedPtr<FUICommandInfo> FindInCurrentView;

	TSharedPtr<FUICommandInfo> ZoomToFit;
	TSharedPtr<FUICommandInfo> ZoomToFitAll;
};
