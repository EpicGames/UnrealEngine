// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "Delegates/Delegate.h"
#include "NiagaraEditorCommon.generated.h"

NIAGARAEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LogNiagaraEditor, Log, All);

/** Common Strings */
namespace FNiagaraEditorStrings
{ 
	extern const FName DefaultValueCustomRowName;
	extern const FName DefaultModeCustomRowName;

	extern const FName FNiagaraParameterActionId;
}

/** Information about a Niagara operation. */
class FNiagaraOpInfo
{
public:
	FNiagaraOpInfo()
		: Keywords(FText())
		, NumericOuputTypeSelectionMode(ENiagaraNumericOutputTypeSelectionMode::Largest)
		, bSupportsAddedInputs(false)
		, bNumericsCanBeIntegers(true)
		, bNumericsCanBeFloats(true)
		, bShowPinNamesInCompactMode(false)
		, bSupportsStaticResolution(false)
	{}

	FName Name;
	// An alternate name for search purposes, such as Divide and '/'
	TOptional<FName> AlternateSearchName;
	FText Category;
	FText FriendlyName;
	FText CompactName;
	TOptional<float> CompactNameFontSizeOverride;
	FText Description;
	FText Keywords;
	ENiagaraNumericOutputTypeSelectionMode NumericOuputTypeSelectionMode;
	TArray<FNiagaraOpInOutInfo> Inputs;
	TArray<FNiagaraOpInOutInfo> Outputs;


	/** If true then this operation supports a variable number of inputs */
	bool bSupportsAddedInputs;

	/** If integer pins are allowed on this op's numeric pins. */
	bool bNumericsCanBeIntegers;

	/** If float pins are allowed on this op's numeric pins. */
	bool bNumericsCanBeFloats;

	/** Only applicable if CompactName is set as that will activate compact node mode. */
	bool bShowPinNamesInCompactMode;

	/** 
	* The format that can generate the hlsl for the given number of inputs.
	* Used the placeholder {A} and {B} to chain the inputs together.
	*/
	FString AddedInputFormatting;

	/**
	* If added inputs are enabled then this filters the available pin types shown to the user.
	* If empty then all the default niagara types are shown.
	*/
	TArray<FNiagaraTypeDefinition> AddedInputTypeRestrictions;

	static TMap<FName, int32> OpInfoMap;
	static TArray<FNiagaraOpInfo> OpInfos;

	static void Init();
	static const FNiagaraOpInfo* GetOpInfo(FName OpName);
	static const TArray<FNiagaraOpInfo>& GetOpInfoArray();

	void BuildName(FString InName, FString InCategory);

	bool CreateHlslForAddedInputs(int32 InputCount, FString& HlslResult) const;

	DECLARE_DELEGATE_RetVal_OneParam(int32, FStaticVariableResolve, const TArray<int32>& );
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FInputTypeValidation, const TArray<FNiagaraTypeDefinition>&, FText& );;
	DECLARE_DELEGATE_RetVal_OneParam(FNiagaraTypeDefinition, FCustomNumericResolve, TConstArrayView<FNiagaraTypeDefinition> )

	FStaticVariableResolve StaticVariableResolveFunction;
	FInputTypeValidation InputTypeValidationFunction;
	FCustomNumericResolve CustomNumericResolveFunction;

	/** Whether or not you can upgrade type to static on connection.*/
	bool bSupportsStaticResolution;
};

/** Interface for struct representing information about where to focus in a Niagara Script Graph after opening the editor for it. */
struct INiagaraScriptGraphFocusInfo : public TSharedFromThis<INiagaraScriptGraphFocusInfo>
{
public:
	enum class ENiagaraScriptGraphFocusInfoType : uint8
	{
		None = 0,
		Node,
		Pin
	};

	INiagaraScriptGraphFocusInfo(const ENiagaraScriptGraphFocusInfoType InFocusType)
		: FocusType(InFocusType)
	{
	};

	const ENiagaraScriptGraphFocusInfoType& GetFocusType() const { return FocusType; };

	virtual ~INiagaraScriptGraphFocusInfo() = 0;
	
private:
	const ENiagaraScriptGraphFocusInfoType FocusType;
};

struct FNiagaraScriptGraphNodeToFocusInfo : public INiagaraScriptGraphFocusInfo
{
public:
	FNiagaraScriptGraphNodeToFocusInfo(const FGuid& InNodeGuidToFocus)
		: INiagaraScriptGraphFocusInfo(ENiagaraScriptGraphFocusInfoType::Node)
		, NodeGuidToFocus(InNodeGuidToFocus)
	{
	};

	const FGuid& GetNodeGuidToFocus() const { return NodeGuidToFocus; };

private:
	const FGuid NodeGuidToFocus;
};

struct FNiagaraScriptGraphPinToFocusInfo : public INiagaraScriptGraphFocusInfo
{
public:
	FNiagaraScriptGraphPinToFocusInfo(const FGuid& InPinGuidToFocus)
		: INiagaraScriptGraphFocusInfo(ENiagaraScriptGraphFocusInfoType::Pin)
		, PinGuidToFocus(InPinGuidToFocus)
	{
	};

	const FGuid& GetPinGuidToFocus() const { return PinGuidToFocus; };

private:
	const FGuid PinGuidToFocus;
};

struct FNiagaraScriptIDAndGraphFocusInfo
{
public:
	FNiagaraScriptIDAndGraphFocusInfo(const uint32& InScriptUniqueAssetID, const TSharedPtr<INiagaraScriptGraphFocusInfo>& InScriptGraphFocusInfo)
		: ScriptUniqueAssetID(InScriptUniqueAssetID)
		, ScriptGraphFocusInfo(InScriptGraphFocusInfo)
	{
	};

	const uint32& GetScriptUniqueAssetID() const { return ScriptUniqueAssetID; };

	const TSharedPtr<INiagaraScriptGraphFocusInfo>& GetScriptGraphFocusInfo() const { return ScriptGraphFocusInfo; };

private:
	const uint32 ScriptUniqueAssetID;
	const TSharedPtr<INiagaraScriptGraphFocusInfo> ScriptGraphFocusInfo;
};

/** Defines different flags to use in conjunction with OnStructureChanged delegates for stack entries and related classes. */
enum ENiagaraStructureChangedFlags
{
	/** The actual stack structure changed - used to invalidate or refresh previous state, like search results */
	StructureChanged = 1 << 0,
	/** Only filtering changed; we don't need to invalidate or refresh as much state */
	FilteringChanged = 1 << 1
	// add more flags here if needed
};

/** Defines different types of changes to data objects within a niagara system. */
enum class ENiagaraDataObjectChange
{
	/** An object was added somewhere in the system. */
	Added,
	/** An object was changed somewhere in the system. */
	Changed,
	/** An object ws removed somewhere in the system. */
	Removed,
	/** It's now known how the data object was changed. */
	Unknown
};

/** Defines a relationship of an FNiagaraVariable to a set of FNiagaraVariable parameter definitions. */
enum class EParameterDefinitionMatchState : uint8
{
	NoMatchingDefinitions = 0,
	MatchingMoreThanOneDefinition,
	MatchingOneDefinition,
	MatchingDefinitionNameButNotType,
};


USTRUCT()
struct FFunctionInputSummaryViewKey
{
	GENERATED_BODY()

private:
	UPROPERTY()
	FGuid FunctionGuid;

	UPROPERTY()
	FGuid InputGuid;

	UPROPERTY()
	FName InputName;
public:

	FFunctionInputSummaryViewKey() { }

	FFunctionInputSummaryViewKey(const FGuid& InFunctionGuid, const FGuid& InInputGuid)
		: FunctionGuid(InFunctionGuid), InputGuid(InInputGuid)
	{
		check(InFunctionGuid.IsValid());
		check(InInputGuid.IsValid());		
	}

	FFunctionInputSummaryViewKey(const FGuid& InFunctionGuid, const FName& InInputName)
		: FunctionGuid(InFunctionGuid), InputName(InInputName)
	{
		check(InFunctionGuid.IsValid());
		check(!InputName.IsNone());		
	}

	FGuid GetFunctionGuid() const { return FunctionGuid; }
	FGuid GetInputGuid() const { return InputGuid; }
	FName GetInputName() const { return InputName; }
	
	bool operator==(const FFunctionInputSummaryViewKey& Other) const
	{
		return FunctionGuid == Other.FunctionGuid && InputGuid == Other.InputGuid && InputName == Other.InputName;
	}

	friend uint32 GetTypeHash(const FFunctionInputSummaryViewKey& Key)
	{
		return HashCombine(
			GetTypeHash(Key.FunctionGuid),
			Key.InputGuid.IsValid()? GetTypeHash(Key.InputGuid) : GetTypeHash(Key.InputName));
	}
};

USTRUCT()
struct FFunctionInputSummaryViewMetadata
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool bVisible;
	UPROPERTY()
	FName DisplayName;
	UPROPERTY()
	FName Category;
	UPROPERTY()
	int32 SortIndex;
	
	FFunctionInputSummaryViewMetadata()
		: bVisible(false)
		, SortIndex(INDEX_NONE)
	{		
	}

	bool operator==(const FFunctionInputSummaryViewMetadata& Other) const
	{
		return
			bVisible == Other.bVisible &&
			DisplayName == Other.DisplayName &&
			Category == Other.Category &&
			SortIndex == Other.SortIndex;
	}
};
