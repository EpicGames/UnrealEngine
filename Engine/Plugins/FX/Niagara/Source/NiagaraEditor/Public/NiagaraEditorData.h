// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorDataBase.h"

#include "NiagaraEditorData.generated.h"

class UNiagaraParameterDefinitionsBase;
class UNiagaraScriptVariable;

/** An interface for editor only data which owns UNiagaraScriptVariables and supports synchronizing them with definitions. */
UCLASS()
class UNiagaraEditorParametersAdapter : public UNiagaraEditorParametersAdapterBase
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UNiagaraEditorParametersBase
	/** Synchronize all source script variables that have been changed or removed from the parameter definitions to all eligible destination script variables owned by the editor data.
	 *
	 *  @param TargetDefinitions			The set of parameter definitions that will be synchronized with the editor only parameters.
	 *	@param AllDefinitions				All parameter definitions in the project. Used to add new subscriptions to definitions if specified in Args.
	 *  @param AllDefinitionsParameterIds	All unique Ids of all parameter definitions.
	 *	@param Subscriber					The INiagaraParameterDefinitionsSubscriber that owns the editor data. Used to add new subscriptions to definitions if specified in Args.
	 *	@param Args							Additional arguments that specify how to perform the synchronization.
	 * @return								Returns an array of name pairs representing old names of script vars that were synced and the new names they inherited, respectively.
	 */
	virtual TArray<TTuple<FName /*SyncedOldName*/, FName /*SyncedNewName*/>> SynchronizeParametersWithParameterDefinitions(
		const TArray<UNiagaraParameterDefinitionsBase*> TargetDefinitions,
		const TArray<UNiagaraParameterDefinitionsBase*> AllDefinitions,
		const TSet<FGuid>& AllDefinitionsParameterIds,
		INiagaraParameterDefinitionsSubscriber* Subscriber,
		FSynchronizeWithParameterDefinitionsArgs Args
	) override;
	//~ End UNiagaraEditorParametersBase

	/** Get All UNiagaraScriptVariables owned directly by this editor data. */
	virtual TArray<TObjectPtr<UNiagaraScriptVariable>>& GetParameters() { return EditorOnlyScriptVars; };

	/** Synchronize a specific dest UNiagaraScriptVariable directly owned by the editor data with a specific source UNiagaraScriptVariable. */
	TOptional<TTuple<FName /*SyncedOldName*/, FName /*SyncedNewName*/>> SynchronizeEditorOnlyScriptVar(const UNiagaraScriptVariable* SourceScriptVar, UNiagaraScriptVariable* DestScriptVar = nullptr);

	/** Find a script variable with the same key as RemovedScriptVarId and unmark it as being sourced from a parameter definitions. */
	bool SynchronizeParameterDefinitionsScriptVariableRemoved(const FGuid& RemovedScriptVarId);

private:
	UPROPERTY()
	TArray<TObjectPtr<UNiagaraScriptVariable>> EditorOnlyScriptVars;
#endif
};
