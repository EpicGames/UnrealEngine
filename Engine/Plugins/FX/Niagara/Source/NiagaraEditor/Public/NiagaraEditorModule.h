// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIShaderPlatform.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "NiagaraTypes.h"
#include "INiagaraCompiler.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "NiagaraCompilationTypes.h"
#include "NiagaraPerfBaseline.h"
#include "NiagaraDebuggerCommon.h"
#include "NiagaraRendererProperties.h"
#include "Customizations/NiagaraDataInterfaceSimCacheVisualizer.h"
#include "Customizations/NiagaraStackObjectPropertyCustomization.h"
#include "Brushes/SlateImageBrush.h"
#include "NiagaraEditorModule.generated.h"

class UNiagaraStackObject;

namespace UE::Niagara::Wizard
{
	class FModuleWizardGenerator;
}

class FCurveEditor;
struct FCurveEditorTreeItemID;
class SCurveEditorTree;
struct FRichCurve;
class FNiagaraRecentAndFavoritesManager;
class IAssetTools;
class IAssetTypeActions;
class INiagaraEditorTypeUtilities;
class INiagaraEditorPropertyUtilities;
class INiagaraEditorExternalObjectUtilities;
class UNiagaraDataInterface;
class UNiagaraSettings;
class USequencerSettings;
class FNiagaraDataInterfaceError;
class UNiagaraStackViewModel;
class UNiagaraStackEntry;
class UNiagaraStackIssue;
class FNiagaraSystemViewModel;
class FNiagaraScriptMergeManager;
class FNiagaraCompileOptions;
class FNiagaraCompileRequestDataBase;
class FNiagaraCompileRequestDuplicateDataBase;
class FNiagaraDataInterfaceFeedback;
class UMovieSceneNiagaraParameterTrack;
struct IConsoleCommand;
class INiagaraEditorOnlyDataUtilities;
class FNiagaraEditorCommands;
struct FNiagaraScriptHighlight;
class FNiagaraClipboard;
class UNiagaraScratchPadViewModel;
class FHlslNiagaraCompiler;
class FNiagaraComponentBroker;
class INiagaraStackObjectIssueGenerator;
class UNiagaraEffectType;
class SNiagaraBaselineViewport;
class FParticlePerfStatsListener_NiagaraBaselineComparisonRender;
class FNiagaraDebugger;
class UNiagaraParameterDefinitions;
class UNiagaraReservedParametersManager;
class FNiagaraGraphDataCache;
class UNiagaraParameterCollection;
struct FNiagaraSystemAsyncCompileResults;
class ITargetPlatform;

namespace UE::Niagara::TraversalCache
{
	class FTraversalCache;
}

DECLARE_STATS_GROUP(TEXT("Niagara Editor"), STATGROUP_NiagaraEditor, STATCAT_Niagara);

extern NIAGARAEDITOR_API int32 GbShowNiagaraDeveloperWindows;
extern NIAGARAEDITOR_API int32 GbPreloadSelectablePluginAssetsOnDemand;
extern NIAGARAEDITOR_API int32 GbEnableExperimentalInlineDynamicInputs;
extern NIAGARAEDITOR_API int32 GbEnableCustomInlineDynamicInputFormats;

/* Defines methods for allowing external modules to supply widgets to the core editor module. */
class INiagaraEditorWidgetProvider
{
public:
	virtual TSharedRef<SWidget> CreateStackView(UNiagaraStackViewModel& StackViewModel) const = 0;
	virtual TSharedRef<SWidget> CreateSystemOverview(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, const FAssetData& EditedAsset) const = 0;
	virtual TSharedRef<SWidget> CreateStackIssueIcon(UNiagaraStackViewModel& StackViewModel, UNiagaraStackEntry& StackEntry) const = 0;

	virtual TSharedRef<SWidget> CreateScriptScratchPadManager(UNiagaraScratchPadViewModel& ScriptScratchPadViewModel) const = 0;
	virtual TSharedRef<SWidget> CreateCurveOverview(TSharedRef<FNiagaraSystemViewModel> SystemViewModel) const = 0;
	virtual TSharedRef<SWidget> CreateCurveKeySelector(TSharedPtr<FCurveEditor> InCurveEditor, const TArray<FCurveEditorTreeItemID>& InCurveTreeItemIds, TSharedPtr<SCurveEditorTree> InCurveEditorTree) const = 0;
	virtual TSharedRef<SWidget> CreateCurveThumbnail(const FRichCurve& Curve) const = 0;

	virtual FLinearColor GetColorForExecutionCategory(FName ExecutionCategory) const = 0;
};

/* Wrapper struct for tracking parameters that are reserved by parameter definitions assets. */
USTRUCT()
struct FReservedParameter
{
	GENERATED_BODY()

public:
	FReservedParameter()
		: Parameter(FNiagaraVariableBase())
		, ReservingDefinitionsAsset(nullptr)
	{};

	FReservedParameter(const FNiagaraVariableBase& InParameter, const UNiagaraParameterDefinitions* InReservingDefinitionsAsset)
		: Parameter(InParameter)
		, ReservingDefinitionsAsset(const_cast<UNiagaraParameterDefinitions*>(InReservingDefinitionsAsset))
	{};

	bool operator== (const FReservedParameter& Other) const { return Parameter == Other.GetParameter() && ReservingDefinitionsAsset == Other.GetReservingDefinitionsAsset(); };

	const FNiagaraVariableBase& GetParameter() const { return Parameter; };
	const UNiagaraParameterDefinitions* GetReservingDefinitionsAsset() const { return ReservingDefinitionsAsset; };

private:
	UPROPERTY(transient)
	FNiagaraVariableBase Parameter;
		 
	UPROPERTY(transient)
	TObjectPtr<UNiagaraParameterDefinitions> ReservingDefinitionsAsset;
};

USTRUCT()
struct FNiagaraRendererCreationInfo
{
	DECLARE_DELEGATE_RetVal_OneParam(UNiagaraRendererProperties*, FRendererFactory, UObject* OuterEmitter);

	GENERATED_BODY()

	FNiagaraRendererCreationInfo() = default;
	FNiagaraRendererCreationInfo(FText InDisplayName, const FTopLevelAssetPath& InRendererClassPath, FRendererFactory InFactory) : DisplayName(InDisplayName), RendererClassPath(InRendererClassPath), RendererFactory(InFactory)
	{}

	FNiagaraRendererCreationInfo(FText InDisplayName, FText InDescription, const FTopLevelAssetPath& InRendererClassPath, FRendererFactory InFactory) : DisplayName(InDisplayName), Description(InDescription), RendererClassPath(InRendererClassPath), RendererFactory(InFactory)
	{}

	UPROPERTY()
	FText DisplayName;

	UPROPERTY()
	FText Description;

	UPROPERTY()
	FTopLevelAssetPath RendererClassPath;
	
	FRendererFactory RendererFactory;
};

struct FNiagaraStackObjectCustomizationRegistry
{
	DECLARE_DELEGATE_RetVal(TSharedRef<FNiagaraStackObjectPropertyCustomization>, FOnGetStackObjectCustomizationInstance);
public:
	void RegisterStackObjectCustomization(UClass& Class, FOnGetStackObjectCustomizationInstance Customization);
	void UnregisterStackObjectCustomization(UClass& Class);

	TSharedPtr<const FNiagaraStackObjectPropertyCustomization> GetCustomizationForStackObject(UNiagaraStackObject& StackObject);
private:
	TMap<UClass*, TSharedPtr<FNiagaraStackObjectPropertyCustomization>> Customizations;
};

inline uint32 GetTypeHash(const FReservedParameter& ReservedParameter) { return GetTypeHash(ReservedParameter.GetParameter().GetName()); };

/** Niagara Editor module */
class FNiagaraEditorModule : public IModuleInterface,
	public IHasMenuExtensibility, public IHasToolBarExtensibility, public FGCObject
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(UMovieSceneNiagaraParameterTrack*, FOnCreateMovieSceneTrackForParameter, FNiagaraVariable);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCheckScriptToolkitsShouldFocusGraphElement, const FNiagaraScriptIDAndGraphFocusInfo*);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnScriptApplied, UNiagaraScript*, FGuid /** VersionGuid */);

public:
	FNiagaraEditorModule();

	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Get the instance of this module. */
	NIAGARAEDITOR_API static FNiagaraEditorModule& Get();

	NIAGARAEDITOR_API FNiagaraRecentAndFavoritesManager* GetRecentsManager();
	
	/** Start the compilation of the specified script. */
	virtual int32 CompileScript(const FNiagaraCompileRequestDataBase* InCompileRequest, const FNiagaraCompileRequestDuplicateDataBase* InCompileRequestDuplicate, const FNiagaraCompileOptions& InCompileOptions);
	virtual TSharedPtr<FNiagaraVMExecutableData> GetCompilationResult(int32 JobID, bool bWait, FNiagaraScriptCompileMetrics& ScriptMetrics);

	TSharedPtr<FNiagaraCompileRequestDataBase, ESPMode::ThreadSafe> Precompile(UObject* InObj, FGuid Version);
	TSharedPtr<FNiagaraCompileRequestDuplicateDataBase, ESPMode::ThreadSafe> PrecompileDuplicate(
		const FNiagaraCompileRequestDataBase* OwningSystemRequestData,
		UNiagaraSystem* OwningSystem,
		UNiagaraEmitter* OwningEmitter,
		UNiagaraScript* TargetScript,
		FGuid TargetVersion);
	TSharedPtr<FNiagaraGraphCachedDataBase, ESPMode::ThreadSafe> CacheGraphTraversal(const UObject* Obj, FGuid Version);

	FNiagaraCompilationTaskHandle RequestCompileSystem(UNiagaraSystem* System, bool bForce, const ITargetPlatform* TargetPlatform);
	bool PollSystemCompile(FNiagaraCompilationTaskHandle, FNiagaraSystemAsyncCompileResults&, bool /*bWait*/, bool /*bPeek*/);
	void AbortSystemCompile(FNiagaraCompilationTaskHandle);

	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override {return MenuExtensibilityManager;}
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override {return ToolBarExtensibilityManager;}

	/** Registers niagara editor type utilities for a specific type. */
	void RegisterTypeUtilities(FNiagaraTypeDefinition Type, TSharedRef<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> EditorUtilities);

	/** Registers niagara editor property utilities for a specific struct type. */
	void RegisterPropertyUtilities(const UScriptStruct* InStruct, TSharedRef<INiagaraEditorPropertyUtilities, ESPMode::ThreadSafe> PropertyUtilities);

	/** Registers niagara editor external object utilities for a specific class type, optionally allowing handling subtypes. */
	void RegisterExternalObjectUtilities(const UClass& InClass, bool bHandleSubclasses, TSharedRef<INiagaraEditorExternalObjectUtilities, ESPMode::ThreadSafe> ExternalObjectUtilities);

	/** Registers niagara wizards that can be used to add generated modules to the stack. */
	void RegisterModuleWizards(TSharedRef<UE::Niagara::Wizard::FModuleWizardGenerator> WizardGenerator);
	TConstArrayView<TSharedRef<UE::Niagara::Wizard::FModuleWizardGenerator>> GetModuleWizards() const { return ModuleWizards; }

	/** Register/unregister niagara editor settings. */
	void RegisterSettings();
	void UnregisterSettings();

	void InitializeTaggedAssetBrowserDetails();
	
	/** Gets Niagara editor type utilities for a specific type if there are any registered. */
	TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> NIAGARAEDITOR_API GetTypeUtilities(const FNiagaraTypeDefinition& Type) const;

	/** Gets Niagara editor property utilities for a specific struct if there are any registered. */
	TSharedPtr<INiagaraEditorPropertyUtilities, ESPMode::ThreadSafe> NIAGARAEDITOR_API GetPropertyUtilities(const UScriptStruct& Struct) const;

	/** Gets Niagara editor external object utilities for a specific class if there are any registered. */
	TSharedPtr<const INiagaraEditorExternalObjectUtilities, ESPMode::ThreadSafe> NIAGARAEDITOR_API GetExternalObjectUtilities(const UClass& Class) const;

	NIAGARAEDITOR_API void RegisterWidgetProvider(TSharedRef<INiagaraEditorWidgetProvider> InWidgetProvider);
	NIAGARAEDITOR_API void UnregisterWidgetProvider(TSharedRef<INiagaraEditorWidgetProvider> InWidgetProvider);

	TSharedRef<INiagaraEditorWidgetProvider> GetWidgetProvider() const;

	
	NIAGARAEDITOR_API void RegisterDataInterfaceCacheVisualizer(UClass* DataInterfaceClass, TSharedRef<INiagaraDataInterfaceSimCacheVisualizer> InCacheVisualizer);
	NIAGARAEDITOR_API void UnregisterDataInterfaceCacheVisualizer(UClass* DataInterfaceClass, TSharedRef<INiagaraDataInterfaceSimCacheVisualizer> InCacheVisualizer);
	TArrayView<TSharedRef<INiagaraDataInterfaceSimCacheVisualizer>> FindDataInterfaceCacheVisualizer(UClass* DataInterfaceClass);
	
	TSharedRef<FNiagaraScriptMergeManager> GetScriptMergeManager() const;

	// Object pooling methods used to prevent unnecessary object allocation during compiles
	NIAGARAEDITOR_API UObject* GetPooledDuplicateObject(UObject* Source, EFieldIteratorFlags::SuperClassFlags CopySuperProperties = EFieldIteratorFlags::ExcludeSuper);
	NIAGARAEDITOR_API void ReleaseObjectToPool(UObject* Obj);
	NIAGARAEDITOR_API void ClearObjectPool();

	/** Registers a new renderer creation delegate with the display name it's going to use for the UI. */
	NIAGARAEDITOR_API void RegisterRendererCreationInfo(FNiagaraRendererCreationInfo RendererCreationInfo);
	const TArray<FNiagaraRendererCreationInfo>& GetRendererCreationInfos() const { return RendererCreationInfo; }
	
	void RegisterParameterTrackCreatorForType(const UScriptStruct& StructType, FOnCreateMovieSceneTrackForParameter CreateTrack);
	void UnregisterParameterTrackCreatorForType(const UScriptStruct& StructType);
	bool CanCreateParameterTrackForType(const UScriptStruct& StructType);
	UMovieSceneNiagaraParameterTrack* CreateParameterTrackForType(const UScriptStruct& StructType, FNiagaraVariable Parameter);

	/** Niagara Editor app identifier string */
	static const FName NiagaraEditorAppIdentifier;

	/** The tab color scale for niagara editors. */
	static const FLinearColor WorldCentricTabColorScale;

	/** Get the niagara UI commands. */
	NIAGARAEDITOR_API const class FNiagaraEditorCommands& Commands();

	FOnCheckScriptToolkitsShouldFocusGraphElement& GetOnScriptToolkitsShouldFocusGraphElement() { return OnCheckScriptToolkitsShouldFocusGraphElement; };

	NIAGARAEDITOR_API TSharedPtr<FNiagaraSystemViewModel> GetExistingViewModelForSystem(UNiagaraSystem* InSystem);

	NIAGARAEDITOR_API const FNiagaraEditorCommands& GetCommands() const;

	void InvalidateCachedScriptAssetData();

	FNiagaraStackObjectCustomizationRegistry& GetStackObjectCustomizationRegistry() { return StackObjectCustomizationRegistry; }
	
	const TArray<UNiagaraScript*>& GetCachedTypeConversionScripts() const;

	NIAGARAEDITOR_API FNiagaraClipboard& GetClipboard() const;

	template<typename T>
	void EnqueueObjectForDeferredDestruction(TSharedRef<T> InObjectToDestruct)
	{
		TDeferredDestructionContainer<T>* ObjectInContainer = new TDeferredDestructionContainer<T>(InObjectToDestruct);
		EnqueueObjectForDeferredDestructionInternal(ObjectInContainer);
	}

	inline INiagaraStackObjectIssueGenerator* FindStackObjectIssueGenerator(FName StructName)
	{
		if (INiagaraStackObjectIssueGenerator** FoundGenerator = StackIssueGenerators.Find(StructName))
		{
			return *FoundGenerator;
		}
		return nullptr;
	}

#if WITH_NIAGARA_DEBUGGER
	TSharedPtr<FNiagaraDebugger> GetDebugger(){ return Debugger; }
#endif

	const TArray<TWeakObjectPtr<UNiagaraParameterDefinitions>>& GetCachedParameterDefinitionsAssets();
	const TArray<TWeakObjectPtr<UNiagaraParameterCollection>>& GetCachedParameterCollectionAssets();

	NIAGARAEDITOR_API void GetTargetSystemAndEmitterForDataInterface(UNiagaraDataInterface* InDataInterface, UNiagaraSystem*& OutOwningSystem, FVersionedNiagaraEmitter& OutOwningEmitter);
	NIAGARAEDITOR_API void GetDataInterfaceFeedbackSafe(UNiagaraDataInterface* InDataInterface, TArray<FNiagaraDataInterfaceError>& OutErrors, TArray<FNiagaraDataInterfaceFeedback>& Warnings, TArray<FNiagaraDataInterfaceFeedback>& Info);

	NIAGARAEDITOR_API void EnsureReservedDefinitionUnique(FGuid& UniqueId);

	FNiagaraGraphDataCache& GetGraphDataCache() const { return *GraphDataCache.Get(); }

	NIAGARAEDITOR_API UNiagaraParameterCollection* FindCollectionForVariable(const FString& VariableName);

	void PreloadSelectablePluginAssetsByClass(UClass* InClass);

	NIAGARAEDITOR_API void ScriptApplied(UNiagaraScript* Script, FGuid VersionGuid = FGuid()) const;
	
	/** Callback whenever a script is applied/updated in the editor. */
	FOnScriptApplied& OnScriptApplied();

	UPackage* GetTempPackage() { return TempPackage; }

	UE::Niagara::TraversalCache::FTraversalCache& GetParameterTraversalCache() { return *TraversalCache.Get(); }

	struct FMoviePreviewObjects
	{
		TStrongObjectPtr<UMediaPlayer> MediaPlayer;
		TStrongObjectPtr<UMediaTexture> MediaTexture;
		FSlateImageBrush* ImageBrush = nullptr;
	};
	
	const FMoviePreviewObjects& FindOrAddMoviePreviewObjects(const FAssetData& AssetData);
private:
	TMap<FAssetData, FMoviePreviewObjects> MoviePreviewObjects;
	
	class FDeferredDestructionContainerBase
	{
	public:
		virtual ~FDeferredDestructionContainerBase() = default;
	};

	template<typename T>
	class TDeferredDestructionContainer : public FDeferredDestructionContainerBase
	{
	public:
		TDeferredDestructionContainer(TSharedRef<const T> InObjectToDestruct)
			: ObjectToDestuct(InObjectToDestruct)
		{
		}

		virtual ~TDeferredDestructionContainer()
		{
			ObjectToDestuct.Reset();
		}

		TSharedPtr<const T> ObjectToDestuct;
	};

	template<class AssetType>
	class TAssetPreloadCache
	{
	public:
		void RefreshCache(bool bAllowLoading)
		{
			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			TArray<FAssetData> AssetData;

			FARFilter Filter;
			Filter.ClassPaths.Add(AssetType::StaticClass()->GetClassPathName());
			Filter.bRecursiveClasses = false;
			Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// Enumerate disk-only assets during cook to ensure deterministic results
			AssetRegistryModule.GetRegistry().GetAssets(Filter, AssetData);

			FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
			CachedAssets.Reset(AssetData.Num());
			for (const FAssetData& AssetDatum : AssetData)
			{
				if(AssetToolsModule.Get().IsAssetVisible(AssetDatum, true) == false)
				{
					continue;
				}
				
				if (AssetDatum.IsAssetLoaded() || (bAllowLoading && FPackageName::GetPackageMountPoint(AssetDatum.PackageName.ToString()) != NAME_None))
				{
					AssetType* Asset = nullptr;
					if (AssetDatum.IsAssetLoaded() == false && bForceLoadSilent)
					{
						Asset = Cast<AssetType>(LoadSilent(AssetDatum));
					}
					else
					{
						Asset = Cast<AssetType>(AssetDatum.GetAsset());
					}

					if (Asset != nullptr)
					{
						Asset->ConditionalPostLoad();
						CachedAssets.Add(MakeWeakObjectPtr(Asset));
					}
				}
			}
		};

		const TArray<TWeakObjectPtr<AssetType>>& Get() const { return CachedAssets; };

		void SetForceLoadSilent(bool bInForceLoadSilent) { bForceLoadSilent = bInForceLoadSilent; }

	private:
		static UObject* LoadSilent(const FAssetData& AssetData)
		{
			uint32 LoadFlags = LOAD_Quiet | LOAD_NoWarn;
			return StaticLoadObject(AssetType::StaticClass(), nullptr, *AssetData.GetObjectPathString(), nullptr, LoadFlags, nullptr, true);
		};

	private:
		TArray<TWeakObjectPtr<AssetType>> CachedAssets;
		bool bForceLoadSilent = false;
	};

	void RegisterDefaultRendererFactories();
	void OnNiagaraSettingsChangedEvent(const FName& PropertyName, const UNiagaraSettings* Settings);
	void OnPreGarbageCollection();
	void OnExecParticleInvoked(const TCHAR* InStr);
	void OnPostEngineInit();
	void OnDeviceProfileManagerUpdated();
	void OnPreviewPlatformChanged(EShaderPlatform NewShaderPlatform);
	void OnPreExit();
	void PostGarbageCollect();

	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return "FNiagaraEditorModule";
	}

	void TestCompileScriptFromConsole(const TArray<FString>& Arguments);

	void ValidateScriptVariableIds(const TArray<FString>& ScriptPathArgs, bool bFix = false);
	
	void ReinitializeStyle();

	void EnqueueObjectForDeferredDestructionInternal(FDeferredDestructionContainerBase* InObjectToDestruct);

	bool DeferredDestructObjects(float InDeltaTime);

	void RegisterStackIssueGenerator(FName StructName, INiagaraStackObjectIssueGenerator* Generator)
	{
		StackIssueGenerators.Add(StructName) = Generator;
	}

	void RegisterDefaultStackObjectCustomizations();

	void OnAssetRegistryLoadComplete();
	
	void OnAssetsPreDelete(const TArray<UObject*>& Objects);

	void OnModulesChanged(FName Module, EModuleChangeReason Reason);

	void RegisterDynamicCustomizations();
	void UnregisterDynamicCustomizations();
	
	void RegisterMenuExtensions();
	
	void RegisterMenuExtension_MigrateToUserAssetTags();

	void ConditionalRegisterDynamicStruct(const UScriptStruct* Struct);

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	/** All created asset type actions.  Cached here so that we can unregister it during shutdown. */
	TArray< TSharedPtr<IAssetTypeActions> > CreatedAssetTypeActions;
	TArray<TSharedRef<UE::Niagara::Wizard::FModuleWizardGenerator>> ModuleWizards;

	mutable UE::FMutex EditorUtilitiesGuard;

	TMap<FNiagaraTypeDefinition, TSharedRef<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe>> TypeToEditorUtilitiesMap;
	TSharedPtr<INiagaraEditorTypeUtilities, ESPMode::ThreadSafe> EnumTypeUtilities;
	TMap<const UScriptStruct*, TSharedRef<INiagaraEditorPropertyUtilities, ESPMode::ThreadSafe>> StructToPropertyUtilitiesMap;

	struct FRegisteredExternalObjectUtilities
	{
		const UClass* Class = nullptr;
		bool bHandleSubclasses = false;
		TSharedPtr<const INiagaraEditorExternalObjectUtilities, ESPMode::ThreadSafe> ExternalObjectUtilities;
	};

	TArray<FRegisteredExternalObjectUtilities> RegisteredExternalObjectUtilities;

	FOnScriptApplied OnScriptAppliedDelegate;

	FDelegateHandle CreateEmitterTrackEditorHandle;
	FDelegateHandle CreateSystemTrackEditorHandle;

	FDelegateHandle CreateBoolParameterTrackEditorHandle;
	FDelegateHandle CreateFloatParameterTrackEditorHandle;
	FDelegateHandle CreateIntegerParameterTrackEditorHandle;
	FDelegateHandle CreateVectorParameterTrackEditorHandle;
	FDelegateHandle CreateColorParameterTrackEditorHandle;

	FDelegateHandle ScriptCompilerHandle;
	FDelegateHandle CompileResultHandle;
	FDelegateHandle PrecompilerHandle;
	FDelegateHandle PrecompileDuplicatorHandle;
	FDelegateHandle GraphCacheTraversalHandle;
	FDelegateHandle RequestCompileSystemHandle;
	FDelegateHandle PollSystemCompileHandle;
	FDelegateHandle AbortSystemCompileHandle;

	FDelegateHandle DeviceProfileManagerUpdatedHandle;

	FDelegateHandle PreviewPlatformChangedHandle;
	FDelegateHandle PreviewFeatureLevelChangedHandle;

	FDelegateHandle AssetRegistryOnLoadCompleteHandle;
	
	FDelegateHandle OnAssetsPreDeleteHandle;

	FDelegateHandle DefaultTrackHandle;

	FDelegateHandle OnModulesChangedHandle;
	
	TObjectPtr<USequencerSettings> SequencerSettings;

	TSharedPtr<INiagaraEditorWidgetProvider> WidgetProvider;

	TMap<TObjectKey<UClass>, TArray<TSharedRef<INiagaraDataInterfaceSimCacheVisualizer>>> DataInterfaceVisualizers;

	TSharedPtr<FNiagaraScriptMergeManager> ScriptMergeManager;

	TSharedPtr<INiagaraEditorOnlyDataUtilities> EditorOnlyDataUtilities;

	TSharedPtr<FNiagaraComponentBroker> NiagaraComponentBroker;

	TMap<const UScriptStruct*, FOnCreateMovieSceneTrackForParameter> TypeToParameterTrackCreatorMap;

	FNiagaraStackObjectCustomizationRegistry StackObjectCustomizationRegistry;
	
	IConsoleCommand* TestCompileScriptCommand;
	IConsoleCommand* ValidateScriptVariableGuidsCommand;
	IConsoleCommand* ValidateAndFixScriptVariableGuidsCommand;
	IConsoleCommand* DumpRapidIterationParametersForAsset;
	IConsoleCommand* PreventSystemRecompileCommand;
	IConsoleCommand* PreventAllSystemRecompilesCommand;
	IConsoleCommand* UpgradeAllNiagaraAssetsCommand;
	IConsoleCommand* DumpCompileIdDataForAssetCommand;
	IConsoleCommand* LoadAllSystemsInFolderCommand;
	IConsoleCommand* DumpEmitterDependenciesCommand;

	FOnCheckScriptToolkitsShouldFocusGraphElement OnCheckScriptToolkitsShouldFocusGraphElement;

	mutable TOptional<TArray<FNiagaraScriptHighlight>> CachedScriptAssetHighlights;
	mutable TOptional<TArray<UNiagaraScript*>> TypeConversionScriptCache;

	bool bThumbnailRenderersRegistered;

	TSharedRef<FNiagaraClipboard> Clipboard;

	IConsoleCommand* ReinitializeStyleCommand;

	struct FActiveCompilation
	{
		TSharedPtr<FHlslNiagaraCompiler> Compiler;
		float TranslationTime = 0.0f;
	};

	TMap<int32, FActiveCompilation> ActiveCompilations;

	TArray<TSharedRef<const FDeferredDestructionContainerBase>> EnqueuedForDeferredDestruction;

	TMap<FName, INiagaraStackObjectIssueGenerator*> StackIssueGenerators;

	TMap<UClass*, TArray<UObject*>> ObjectPool;

	TArray<FNiagaraRendererCreationInfo> RendererCreationInfo;

#if NIAGARA_PERF_BASELINES
	void GeneratePerfBaselines(TArray<UNiagaraEffectType*>& BaselinesToGenerate);
	void OnPerfBaselineWindowClosed(const TSharedRef<SWindow>& ClosedWindow);

	/** Viewport used when generating the performance baselines for Niagara systems. */
	TSharedPtr<SNiagaraBaselineViewport> BaselineViewport;
#endif

#if WITH_NIAGARA_DEBUGGER
	TSharedPtr<FNiagaraDebugger> Debugger;
#endif

	UNiagaraReservedParametersManager* ReservedParametersManagerSingleton;

	// Set of Parameter Definitions Ids
	TSet<FGuid> ReservedDefinitionIds;

	TUniquePtr<FNiagaraGraphDataCache> GraphDataCache;

	TAssetPreloadCache<UNiagaraParameterCollection> ParameterCollectionAssetCache;
	TAssetPreloadCache<UNiagaraParameterDefinitions> ParameterDefinitionsAssetCache;

	TArray<UClass*> PluginAssetClassesPreloaded;

	TSharedPtr<FNiagaraRecentAndFavoritesManager> RecentAndFavoritesManager;

	UPackage* TempPackage = nullptr;

	TSharedPtr<UE::Niagara::TraversalCache::FTraversalCache> TraversalCache;

	//Set of dynamic structs we have done registrations for 
	TSet<FName> RegisteredDynamicStructs;
};
