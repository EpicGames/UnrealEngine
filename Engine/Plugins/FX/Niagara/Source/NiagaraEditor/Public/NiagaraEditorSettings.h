// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraSystem.h"
#include "NiagaraEmitter.h"
#include "NiagaraScript.h"
#include "NiagaraValidationRuleSet.h"
#include "Engine/DeveloperSettings.h"
#include "NiagaraSpawnShortcut.h"
#include "SNodePanel.h"
#include "Editor/UnrealEdTypes.h"
#include "Viewports.h"
#include "NiagaraEditorSettings.generated.h"

class FAssetEditorToolkit;
class FLayoutExtender;
class FWorkflowTabFactory;
class UCurveFloat;
enum class EScriptSource : uint8;
class UClass;

namespace FNiagaraEditorGuids
{
	const extern FGuid SystemNamespaceMetaDataGuid;
	const extern FGuid EmitterNamespaceMetaDataGuid;
	const extern FGuid ParticleAttributeNamespaceMetaDataGuid;
	const extern FGuid ModuleNamespaceMetaDataGuid;
	const extern FGuid ModuleOutputNamespaceMetaDataGuid;
	const extern FGuid ModuleLocalNamespaceMetaDataGuid;
	const extern FGuid TransientNamespaceMetaDataGuid;
	const extern FGuid StackContextNamespaceMetaDataGuid;
	const extern FGuid EngineNamespaceMetaDataGuid;
	const extern FGuid UserNamespaceMetaDataGuid;
	const extern FGuid ParameterCollectionNamespaceMetaDataGuid;
	const extern FGuid DataInstanceNamespaceMetaDataGuid;
	const extern FGuid StaticSwitchNamespaceMetaDataGuid;
}

UENUM()
enum class ENiagaraCategoryExpandState : uint8
{
	/** Categories will use the default expand / collapse state. */
	Default,
	/** Categories will use the default expand / collapse state unless they contain modified properties in which case they will expand. */
	DefaultExpandModified,
	/** Categories will ignore the default state and be collapsed. */
	CollapseAll,
	/** Categories will ignore the default state and be expanded. */
	ExpandAll,
};

UENUM()
enum class ENiagaraNamespaceMetadataOptions
{
	HideInScript,
	HideInSystem,
	AdvancedInScript,
	AdvancedInSystem,
	PreventEditingNamespace,
	PreventEditingNamespaceModifier,
	PreventEditingName,
	PreventCreatingInSystemEditor,
	HideInDefinitions,
};

UENUM()
enum class ENiagaraAddDefaultsTrackMode : uint8
{
	/** Adding a Niagara actor to a sequence will not add any additional subtracks by default */
	NoSubtracks,

	/** Adding a Niagara actor will add a component subtrack, but nothing else */
	ComponentTrackOnly,
	
	/** Adding a Niagara actor to a sequence will also add a life cycle track to it by default  */
	LifecycleTrack,
};

USTRUCT()
struct FNiagaraNamespaceMetadata
{
	GENERATED_BODY()

	NIAGARAEDITOR_API FNiagaraNamespaceMetadata();

	NIAGARAEDITOR_API FNiagaraNamespaceMetadata(TArray<FName> InNamespaces, FName InRequiredNamespaceModifier = NAME_None);

	//FNiagaraNamespaceMetadata(const FNiagaraNamespaceMetadata& Other);

	bool operator==(const FNiagaraNamespaceMetadata& Other) const
	{
		return
			Namespaces == Other.Namespaces &&
			RequiredNamespaceModifier == Other.RequiredNamespaceModifier &&
			DisplayName.IdenticalTo(Other.DisplayName) &&
			DisplayNameLong.IdenticalTo(Other.DisplayNameLong) &&
			Description.IdenticalTo(Other.Description) &&
			BackgroundColor == Other.BackgroundColor &&
			ForegroundStyle == Other.ForegroundStyle &&
			SortId == Other.SortId &&
			OptionalNamespaceModifiers == Other.OptionalNamespaceModifiers &&
			Options == Other.Options;
	}

	UPROPERTY()
	TArray<FName> Namespaces;

	UPROPERTY()
	FName RequiredNamespaceModifier;

	UPROPERTY()
	FText DisplayName;

	UPROPERTY()
	FText DisplayNameLong;

	UPROPERTY()
	FText Description;

	UPROPERTY()
	FLinearColor BackgroundColor;

	UPROPERTY()
	FName ForegroundStyle;

	UPROPERTY()
	int32 SortId;

	UPROPERTY()
	TArray<FName> OptionalNamespaceModifiers;

	UPROPERTY()
	TArray<ENiagaraNamespaceMetadataOptions> Options;

	FNiagaraNamespaceMetadata& SetDisplayName(FText InDisplayName)
	{
		DisplayName = InDisplayName;
		return *this;
	}

	FNiagaraNamespaceMetadata& SetDisplayNameLong(FText InDisplayNameLong)
	{
		DisplayNameLong = InDisplayNameLong;
		return *this;
	}

	FNiagaraNamespaceMetadata& SetDescription(FText InDescription)
	{
		Description = InDescription;
		return *this;
	}

	FNiagaraNamespaceMetadata& SetBackgroundColor(FLinearColor InBackgroundColor)
	{
		BackgroundColor = InBackgroundColor;
		return *this;
	}

	FNiagaraNamespaceMetadata& SetForegroundStyle(FName InForegroundStyle)
	{
		ForegroundStyle = InForegroundStyle;
		return *this;
	}

	FNiagaraNamespaceMetadata& SetSortId(int32 InSortId)
	{
		SortId = InSortId;
		return *this;
	}

	FNiagaraNamespaceMetadata& AddOptionalNamespaceModifier(FName InOptionalNamespaceModifier)
	{
		OptionalNamespaceModifiers.Add(InOptionalNamespaceModifier);
		return *this;
	}

	FNiagaraNamespaceMetadata& AddOption(ENiagaraNamespaceMetadataOptions Option)
	{
		Options.Add(Option);
		return *this;
	}

	FNiagaraNamespaceMetadata& SetGuid(const FGuid& NewGuid)
	{ 
		Guid = NewGuid; 
		return *this;
	}

	bool IsValid() const { return Namespaces.Num() > 0; }

	const FGuid& GetGuid() const { return Guid; }

private:
	UPROPERTY(Transient)
	FGuid Guid;
};

USTRUCT()
struct FNiagaraCurveTemplate
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, Category = Curve)
	FString DisplayNameOverride;

	UPROPERTY(EditAnywhere, Category = Curve, meta = (AllowedClasses = "/Script/Engine.CurveFloat"))
	FSoftObjectPath CurveAsset;
};

USTRUCT()
struct FNiagaraActionColors
{
	GENERATED_BODY()

	NIAGARAEDITOR_API FNiagaraActionColors();
	
	UPROPERTY(EditAnywhere, Category = Color)
	FLinearColor NiagaraColor;

	UPROPERTY(EditAnywhere, Category = Color)
	FLinearColor GameColor;

	UPROPERTY(EditAnywhere, Category = Color)
	FLinearColor PluginColor;

	UPROPERTY(EditAnywhere, Category = Color)
	FLinearColor DeveloperColor;
};


USTRUCT()
struct FNiagaraParameterPanelSectionStorage
{
	GENERATED_BODY()
		
	FNiagaraParameterPanelSectionStorage() {}

	FNiagaraParameterPanelSectionStorage(const FGuid& InGuid) : ParamStorageId(InGuid) 
	{
		check(InGuid.IsValid());
	}

	UPROPERTY()
	FGuid ParamStorageId;

	UPROPERTY()
	TArray<FGuid> ExpandedCategories;
};

/** Contains all the settings that should be shared across sessions. */
USTRUCT()
struct FNiagaraViewportSharedSettings
{
	GENERATED_USTRUCT_BODY()

	FNiagaraViewportSharedSettings() {}

	/** The viewport type */
	UPROPERTY(config)
	TEnumAsByte<ELevelViewportType> ViewportType = LVT_Perspective;

	/* View mode to set when this viewport is of type LVT_Perspective. */
	UPROPERTY(config)
	TEnumAsByte<EViewModeIndex> PerspViewModeIndex = VMI_Lit;

	/* View mode to set when this viewport is not of type LVT_Perspective. */
	UPROPERTY(config)
	TEnumAsByte<EViewModeIndex> OrthoViewModeIndex = VMI_BrushWireframe;

	/**
	 * A set of flags that determines visibility for various scene elements (FEngineShowFlags), converted to string form.
	 * These have to be saved as strings since FEngineShowFlags is too complex for UHT to parse correctly.
	 */
	UPROPERTY(config)
	FString EditorShowFlagsString;

	/**
	 * A set of flags that determines visibility for various scene elements (FEngineShowFlags), converted to string form.
	 * These have to be saved as strings since FEngineShowFlags is too complex for UHT to parse correctly.
	 */
	UPROPERTY(config)
	FString GameShowFlagsString;

	/** Setting to allow designers to override the automatic expose. */
	UPROPERTY(config)
	FExposureSettings ExposureSettings;

	/* Field of view angle for the viewport. */
	UPROPERTY(config)
	float FOVAngle = EditorViewportDefs::DefaultPerspectiveFOVAngle;

	/* Whether this viewport is updating in real-time. */
	UPROPERTY(config)
	bool bIsRealtime = true;
	
	/* Whether viewport statistics should be shown. */
	UPROPERTY(config)
	bool bShowOnScreenStats = true;

	UPROPERTY(config)
	bool bShowGridInViewport = false;

	UPROPERTY(config)
	bool bShowInstructionsCount = false;
	
	UPROPERTY(config)
	bool bShowParticleCountsInViewport = false;

	UPROPERTY(config)
	bool bShowEmitterExecutionOrder = false;

	UPROPERTY(config)
	bool bShowGpuTickInformation = false;

	UPROPERTY(config)
	bool bShowMemoryInfo = false;

	UPROPERTY(config)
	bool bShowStatelessInfo = true;
};

USTRUCT()
struct FNiagaraCurveEditorSharedSettings
{
	GENERATED_BODY()
	
	UPROPERTY(Config)
	bool bCurveInputSnapEnabled = false;

	UPROPERTY(Config)
	bool bCurveOutputSnapEnabled = false;
};

inline uint32 GetTypeHash(const FNiagaraNamespaceMetadata& NamespaceMetaData)
{
	return GetTypeHash(NamespaceMetaData.GetGuid());
}

UCLASS(config = Niagara, defaultconfig, meta=(DisplayName="Niagara"), MinimalAPI)
class UNiagaraEditorSettings : public UDeveloperSettings
{
public:
	GENERATED_UCLASS_BODY()

	/** Niagara script to duplicate as the base of all new script assets created. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	FSoftObjectPath DefaultScript;

	/** Niagara script to duplicate as the base of all new script assets created. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	FSoftObjectPath DefaultDynamicInputScript;

	/** Niagara script to duplicate as the base of all new script assets created. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	FSoftObjectPath DefaultFunctionScript;

	/** Niagara script to duplicate as the base of all new script assets created. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	FSoftObjectPath DefaultModuleScript;

	/** Niagara emitter to duplicate as the base for minimal emitters. Useful to set up default state. If empty, a truly empty emitter will be used. */
	UPROPERTY(config, EditAnywhere, DisplayName="Minimal Emitter", Category = Niagara, meta=(AllowedClasses="/Script/Niagara.NiagaraEmitter"))
	FSoftObjectPath DefaultEmptyEmitter;

	/** The configuration used in the Niagara Asset Wizard when creating an new Niagara Emitter or adding an emitter to a system. */
	UPROPERTY(config, EditAnywhere, DisplayName="Emitter Asset Wizard Configuration", Category = Niagara, meta=(AllowedClasses="/Script/UserAssetTagsEditor.TaggedAssetBrowserConfiguration"))
	FSoftObjectPath EmitterAssetWizardConfiguration;

	/** The configuration used in the Niagara Asset Wizard when creating a new Niagara System. */
	UPROPERTY(config, EditAnywhere, DisplayName="System Asset Wizard Configuration", Category = Niagara, meta=(AllowedClasses="/Script/UserAssetTagsEditor.TaggedAssetBrowserConfiguration"))
	FSoftObjectPath SystemAssetWizardConfiguration;

	/** Default material to use with new decal renderers. */
	UPROPERTY(config, EditAnywhere, Category = "Niagara Renderer", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath DefaultDecalRendererMaterial;

	/** Default mesh to use with new mesh renderers. */
	UPROPERTY(config, EditAnywhere, Category = "Niagara Renderer", meta = (AllowedClasses = "/Script/Engine.StaticMesh"))
	FSoftObjectPath DefaultMeshRendererMesh;

	/** Default material to use with new ribbon renderers. */
	UPROPERTY(config, EditAnywhere, Category = "Niagara Renderer", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath DefaultRibbonRendererMaterial;

	/** Default material to use with new sprite renderers. */
	UPROPERTY(config, EditAnywhere, Category = "Niagara Renderer", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath DefaultSpriteRendererMaterial;

	/** Whether we allow using previews in the editor. */
	UPROPERTY(Config, EditAnywhere, Category = "Preview")
	bool bDisplayPreviewMoviesInTooltips = true;

	/** The size of the movie previews used in tooltips. */
	UPROPERTY(Config, EditAnywhere, Category = "Preview", meta=(EditCondition="bDisplayPreviewMoviesInTooltips==true"))
	FVector2f TooltipPreviewMovieSize = FVector2f(128.f, 128.f);

	/** Niagara script which is required in the system update script to control system state. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	FSoftObjectPath RequiredSystemUpdateScript;

	/** Validation rules applied to all Niagara systems. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	TArray<TSoftObjectPtr<UNiagaraValidationRuleSet>> DefaultValidationRuleSets;

	/** Shortcut key bindings that if held down while doing a mouse click, will spawn the specified type of Niagara node.*/
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	TArray<FNiagaraSpawnShortcut> GraphCreationShortcuts;

	/** If true then emitter and system nodes will show a simplified representation on low zoom levels. This improves performance and readablity when zoomed out of the system overview graph. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	bool bSimplifyStackNodesAtLowResolution = true;

	/** The max number of chars before names on the low resolution nodes are truncated. */
	UPROPERTY(config, EditAnywhere, Category = Niagara, meta=(EditCondition="bSimplifyStackNodesAtLowResolution", UIMin=3))
	int32 LowResolutionNodeMaxNameChars = 7;
	
	/** If true then the system editor will zoom to fit all emitters when opening an asset. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	bool bAlwaysZoomToFitSystemGraph = true;

	UPROPERTY(config, EditAnywhere, Category = Niagara)
	ENiagaraCategoryExpandState RendererCategoryExpandState = ENiagaraCategoryExpandState::Default;
	
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	ENiagaraAddDefaultsTrackMode DefaultsSequencerSubtracks = ENiagaraAddDefaultsTrackMode::NoSubtracks;

	UPROPERTY(config, EditAnywhere, Category = "Niagara Graph")
	EGraphZoomLimitHandling ScriptGraphZoomLimitHandling = EGraphZoomLimitHandling::Default;

	/** Module Notes that were defined in a Module Script will expand by default, if true. Consider setting this to false if you are familiar with Niagara and don't need module notes. */
	UPROPERTY(config, EditAnywhere, Category = "Niagara Stack")
	bool bExpandModuleNotesByDefault = true;
	
	NIAGARAEDITOR_API TArray<float> GetPlaybackSpeeds() const;

	/** Gets whether or not auto-compile is enabled in the editors. */
	NIAGARAEDITOR_API bool GetAutoCompile() const;

	/** Sets whether or not auto-compile is enabled in the editors. */
	NIAGARAEDITOR_API void SetAutoCompile(bool bInAutoCompile);

	/** Gets whether or not simulations should start playing automatically when the emitter or system editor is opened, or when the data is changed in the editor. */
	NIAGARAEDITOR_API bool GetAutoPlay() const;

	/** Sets whether or not simulations should start playing automatically when the emitter or system editor is opened, or when the data is changed in the editor. */
	NIAGARAEDITOR_API void SetAutoPlay(bool bInAutoPlay);

	/** Gets whether or not the simulation should reset when a value on the emitter or system is changed. */
	NIAGARAEDITOR_API bool GetResetSimulationOnChange() const;

	/** Sets whether or not the simulation should reset when a value on the emitter or system is changed. */
	NIAGARAEDITOR_API void SetResetSimulationOnChange(bool bInResetSimulationOnChange);

	/** Gets whether or not to rerun the simulation to the current time when making modifications while paused. */
	NIAGARAEDITOR_API bool GetResimulateOnChangeWhilePaused() const;

	/** Sets whether or not to rerun the simulation to the current time when making modifications while paused. */
	NIAGARAEDITOR_API void SetResimulateOnChangeWhilePaused(bool bInResimulateOnChangeWhilePaused);

	/** Gets whether or not to reset all components that include the system that is currently being reset */
	NIAGARAEDITOR_API bool GetResetDependentSystemsWhenEditingEmitters() const;

	/** Sets whether or not to reset all components that include the system that is currently being reset */
	NIAGARAEDITOR_API void SetResetDependentSystemsWhenEditingEmitters(bool bInResetDependentSystemsWhenEditingEmitters);

	/** Gets whether or not to display advanced categories for the parameter panel. */
	NIAGARAEDITOR_API bool GetDisplayAdvancedParameterPanelCategories() const;

	/** Sets whether or not to display advanced categories for the parameter panel. */
	NIAGARAEDITOR_API void SetDisplayAdvancedParameterPanelCategories(bool bInDisplayAdvancedParameterPanelCategories);

	NIAGARAEDITOR_API bool GetDisplayAffectedAssetStats() const;
	NIAGARAEDITOR_API int32 GetAssetStatsSearchLimit() const;

	NIAGARAEDITOR_API bool IsShowGridInViewport() const;
	NIAGARAEDITOR_API void SetShowGridInViewport(bool bShowGridInViewport);
	
	NIAGARAEDITOR_API bool IsShowParticleCountsInViewport() const;
	NIAGARAEDITOR_API void SetShowParticleCountsInViewport(bool bShowParticleCountsInViewport);

	NIAGARAEDITOR_API FNiagaraNamespaceMetadata GetDefaultNamespaceMetadata() const;
	NIAGARAEDITOR_API FNiagaraNamespaceMetadata GetMetaDataForNamespaces(TArray<FName> Namespaces) const;
	NIAGARAEDITOR_API FNiagaraNamespaceMetadata GetMetaDataForId(const FGuid& NamespaceId) const;
	NIAGARAEDITOR_API const FGuid& GetIdForUsage(ENiagaraScriptUsage Usage) const;
	NIAGARAEDITOR_API const TArray<FNiagaraNamespaceMetadata>& GetAllNamespaceMetadata() const;

	NIAGARAEDITOR_API FNiagaraNamespaceMetadata GetDefaultNamespaceModifierMetadata() const;
	NIAGARAEDITOR_API FNiagaraNamespaceMetadata GetMetaDataForNamespaceModifier(FName NamespaceModifier) const;
	NIAGARAEDITOR_API const TArray<FNiagaraNamespaceMetadata>& GetAllNamespaceModifierMetadata() const;

	NIAGARAEDITOR_API const TArray<FNiagaraCurveTemplate>& GetCurveTemplates() const;
	
	// Begin UDeveloperSettings Interface
	NIAGARAEDITOR_API virtual FName GetCategoryName() const override;
	NIAGARAEDITOR_API virtual FText GetSectionText() const override;
	// END UDeveloperSettings Interface

	NIAGARAEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	const FNiagaraViewportSharedSettings& GetViewportSharedSettings() const { return ViewportSettings; }
	void SetViewportSharedSettings(const FNiagaraViewportSharedSettings& InViewportSharedSettings);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNiagaraEditorSettingsChanged, const FString&,
	                                     const UNiagaraEditorSettings*);

	/** Gets a multicast delegate which is called whenever one of the parameters in this settings object changes. */
	static NIAGARAEDITOR_API FOnNiagaraEditorSettingsChanged& OnSettingsChanged();

	const TMap<FString, FString>& GetHLSLKeywordReplacementsMap()const { return HLSLKeywordReplacements; }

	NIAGARAEDITOR_API FLinearColor GetSourceColor(EScriptSource Source) const;

	NIAGARAEDITOR_API FLinearColor GetDefaultNoteColor() const;

	NIAGARAEDITOR_API FNiagaraParameterPanelSectionStorage& FindOrAddParameterPanelSectionStorage(FGuid PanelSectionId, bool& bOutAdded);

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsClassAllowed, const UClass* /*InClass*/);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsClassPathAllowed, const FTopLevelAssetPath& /*InClassPath*/);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterAsset, const FTopLevelAssetPath& /*InAssetPath*/);
	DECLARE_DELEGATE_ThreeParams(FOnGetExternalTabsForAsset, TSharedRef<FAssetEditorToolkit> /*InHostToolkit*/, UObject& /*InAsset*/, TArray<TSharedRef<FWorkflowTabFactory>>& /*OutExternalTabSpawners*/);
	DECLARE_DELEGATE_TwoParams(FOnExternalExtendLayoutForAsset, TSharedRef<FLayoutExtender> /*InLayoutExtender*/, UObject& /*InAsset*/);
	
	/** Sets a delegate that allows external code to restrict which features can be used within the niagara editor by filtering which classes are visible in various menus.
	 * A non-visible class might still be referenceable, and can be selected through actions as copy pasting content. */
	NIAGARAEDITOR_API void SetOnIsClassVisible(const FOnIsClassAllowed& InOnIsClassAllowed);

	/** Sets a delegate that allows external code to restrict which features can be used within the niagara editor by filtering which classes are actually referenceable.
	 * A referenceable class is considered 'valid' content, but might or might not be visible in menus. */
	NIAGARAEDITOR_API void SetOnIsClassReferenceable(const FOnIsClassAllowed& InOnIsClassAllowed);

	/** Sets a delegate that allows external code to restrict which features can be used within the niagara editor by filtering which assets are allowed. */
	NIAGARAEDITOR_API void SetOnShouldFilterAssetByClassUsage(const FOnShouldFilterAsset& InOnShouldFilterAssetByClassUsage);

	/** Sets a delegate that allows external code to restrict what assets will show up in the Niagara Asset Browser. */
	NIAGARAEDITOR_API void SetOnShouldFilterAssetInNiagaraAssetBrowser(const FOnShouldFilterAsset& InOnShouldFilterAssetByClassUsage);

	NIAGARAEDITOR_API void SetOnGetExternalAssetTabSpawners(FOnGetExternalTabsForAsset InOnGetExternalTabsForAsset, FOnExternalExtendLayoutForAsset InOnExternalExtendLayoutForAsset);

	/** Returns whether or not the supplied class is visible for UI purposes in the current editor context. */
	NIAGARAEDITOR_API bool IsVisibleClass(const UClass* InClass) const;

	/** Returns whether or not the supplied class is valid to be used in the current editor context. */
	NIAGARAEDITOR_API bool IsReferenceableClass(const UClass* InClass) const;

	/** Returns whether or not the supplied niagara type definition can be used in the current editor context. */
	NIAGARAEDITOR_API bool IsVisibleTypeDefinition(const FNiagaraTypeDefinition& InTypeDefinition) const;

	NIAGARAEDITOR_API void GetExternalTabSpawnersForAsset(TSharedRef<FAssetEditorToolkit> InHostToolkit, UObject& InAsset, TArray<TSharedRef<FWorkflowTabFactory>>& OutExternalTabSpawners) const;

	NIAGARAEDITOR_API void ExternalExtendLayoutForAsset(TSharedRef<FLayoutExtender> InLayoutExtender, UObject& InAsset) const;

	NIAGARAEDITOR_API FAssetRegistryTag CreateClassUsageAssetRegistryTag(const UObject* SourceObject) const;

	NIAGARAEDITOR_API bool IsAllowedAssetByClassUsage(const FAssetData& InAssetData) const;
	NIAGARAEDITOR_API bool IsAllowedAssetObjectByClassUsage(const UObject& InAssetObject) const;
	NIAGARAEDITOR_API bool IsAllowedAssetInNiagaraAssetBrowser(const FAssetData& InAssetData) const;

	NIAGARAEDITOR_API bool GetUpdateStackValuesOnCommitOnly() const;

	NIAGARAEDITOR_API bool GetUseAutoExposure() const;
	NIAGARAEDITOR_API void SetUseAutoExposure(bool bInUseAutoExposure);
	
	NIAGARAEDITOR_API float GetExposureValue() const;
	NIAGARAEDITOR_API void SetAutoExposureValue(bool bInUseAutoExposure);

	bool GetForceSilentLoadingOfCachedAssets() const { return bForceSilentLoadingOfCachedAssets; }

	NIAGARAEDITOR_API EGraphZoomLimitHandling GetScriptEditorZoomLimitHandling() const;

	NIAGARAEDITOR_API bool IsCurveInputSnapEnabled() const;
	NIAGARAEDITOR_API bool IsCurveOutputSnapEnabled() const;
	NIAGARAEDITOR_API void SetCurveInputSnapEnabled(bool bEnabled);
	NIAGARAEDITOR_API void SetCurveOutputSnapEnabled(bool bEnabled);

private:
	NIAGARAEDITOR_API bool IsAllowedObjectByClassUsageInternal(const UObject& InObject, TSet<const UObject*>& CheckedObjects) const;
	NIAGARAEDITOR_API bool IsAllowedAssetObjectByClassUsageInternal(const UObject& InAssetObject, TSet<const UObject*>& CheckedAssetObjects) const;

	NIAGARAEDITOR_API void SetupNamespaceMetadata();
	NIAGARAEDITOR_API void BuildCachedPlaybackSpeeds() const;
	NIAGARAEDITOR_API bool ShouldTrackClassUsage(const UClass* InClass) const;
protected:
	FOnNiagaraEditorSettingsChanged SettingsChangedDelegate;
private:
	/** Whether or not auto-compile is enabled in the editors. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	bool bAutoCompile;

	/** Whether or not simulations should start playing automatically when the emitter or system editor is opened, or when the data is changed in the editor. */
	UPROPERTY(config, EditAnywhere, Category = SimulationOptions)
	bool bAutoPlay;

	/** Whether or not the simulation should reset when a value on the emitter or system is changed. */
	UPROPERTY(config, EditAnywhere, Category = SimulationOptions)
	bool bResetSimulationOnChange;

	/** Whether or not to rerun the simulation to the current time when making modifications while paused. */
	UPROPERTY(config, EditAnywhere, Category = SimulationOptions)
	bool bResimulateOnChangeWhilePaused;

	/** Whether or not to reset all components that include the system currently being reset. */
	UPROPERTY(config, EditAnywhere, Category = SimulationOptions)
	bool bResetDependentSystemsWhenEditingEmitters;

	/** Whether or not to display advanced categories for the parameter panel. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	bool bDisplayAdvancedParameterPanelCategories;

	/** If true, then the emitter and script editors will show an info message how many downstream asset are affected by a change. Gathering this information for large asset graphs can delay the opening of the asset editors a bit. */
	UPROPERTY(config, EditAnywhere, Category=Niagara)
	bool bDisplayAffectedAssetStats = true;

	/** The maximum amount of asset references that are searched before stopping. Setting this too high can lead to long load times when opening default assets (basically the same as disabling the breadth limit in the reference viewer). */
	UPROPERTY(config, EditAnywhere, Category=Niagara, meta=(EditCondition="bDisplayAffectedAssetStats", ClampMin=1))
	int32 AffectedAssetSearchLimit = 25;

	/** This affects numeric inputs for modules. When set to false, the inputs update the simulation while typing. When set to true, the simulation is only updated after submitting the change by pressing Enter. */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	bool bUpdateStackValuesOnCommitOnly = false;

	/** Speeds used for slowing down and speeding up the playback speeds */
	UPROPERTY(config, EditAnywhere, Category = Niagara)
	TArray<float> PlaybackSpeeds;

	UPROPERTY(config, EditAnywhere, Category = "Niagara Colors")
	FNiagaraActionColors ActionColors;

	UPROPERTY(Config, EditAnywhere, Category ="Niagara Colors")
	FLinearColor DefaultNoteColor = FLinearColor(FColor(56, 111, 75));

	/** This is built using PlaybackSpeeds, populated whenever it is accessed using GetPlaybackSpeeds() */
	mutable TOptional<TArray<float>> CachedPlaybackSpeeds;

	UPROPERTY(config)
	TMap<FString, FString> HLSLKeywordReplacements;

	UPROPERTY()
	TArray<FNiagaraNamespaceMetadata> NamespaceMetadata;

	UPROPERTY()
	TArray<FNiagaraNamespaceMetadata> NamespaceModifierMetadata;

	UPROPERTY()
	FNiagaraNamespaceMetadata DefaultNamespaceMetadata;

	UPROPERTY()
	FNiagaraNamespaceMetadata DefaultNamespaceModifierMetadata;

	UPROPERTY(config, EditAnywhere, Category = Niagara)
	TArray<FNiagaraCurveTemplate> CurveTemplates;

	UPROPERTY(config)
	FNiagaraCurveEditorSharedSettings CurveEditorSharedSettings;
	
	UPROPERTY(config)
	FNiagaraViewportSharedSettings ViewportSettings;
	
	UPROPERTY(config)
	TArray<FNiagaraParameterPanelSectionStorage> SystemParameterPanelSectionData;

	UPROPERTY(config)
	bool bForceSilentLoadingOfCachedAssets;
	
	FOnIsClassAllowed OnIsClassVisibleDelegate;
	FOnIsClassAllowed OnIsClassReferenceableDelegate;
	FOnIsClassPathAllowed OnIsClassPathAllowedDelegate;
	FOnShouldFilterAsset OnShouldFilterAssetByClassUsage;
	FOnShouldFilterAsset OnShouldFilterAssetInNiagaraAssetBrowser;
	FOnGetExternalTabsForAsset OnGetExternalTabsForAsset;
	FOnExternalExtendLayoutForAsset OnExternalExtendLayoutForAsset;

	TArray<UClass*> TrackedUsageBaseClasses;

public:
	NIAGARAEDITOR_API bool IsShowInstructionsCount() const;
	NIAGARAEDITOR_API void SetShowInstructionsCount(bool bShowInstructionsCount);
	NIAGARAEDITOR_API bool IsShowEmitterExecutionOrder() const;
	NIAGARAEDITOR_API void SetShowEmitterExecutionOrder(bool bShowEmitterExecutionOrder);
	NIAGARAEDITOR_API bool IsShowGpuTickInformation() const;
	NIAGARAEDITOR_API void SetShowGpuTickInformation(bool bShowGpuTickInformation);
	NIAGARAEDITOR_API bool IsShowMemoryInfo() const;
	NIAGARAEDITOR_API void SetShowMemoryInfo(bool bInShowInfo);
	NIAGARAEDITOR_API bool IsShowStatelessInfo() const;
	NIAGARAEDITOR_API void SetShowStatelessInfo(bool bInShowInfo);

};
