// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/** Manages the style which provides resources for niagara editor widgets. */
class FNiagaraEditorStyle : public FSlateStyleSet
{
public:
	static NIAGARAEDITOR_API void Register();
	static NIAGARAEDITOR_API void Unregister();
	static NIAGARAEDITOR_API void Shutdown();

	/** reloads textures used by slate renderer */
	static NIAGARAEDITOR_API void ReloadTextures();

	/** @return The Slate style set for niagara editor widgets */
	static NIAGARAEDITOR_API const FNiagaraEditorStyle& Get();

	static NIAGARAEDITOR_API void ReinitializeStyle();

private:	
	FNiagaraEditorStyle();
	void InitStats();
	void InitAssetPicker();
	void InitActionMenu();
	void InitEmitterHeader();
	void InitParameters();
	void InitParameterMapView();
	void InitCodeView();
	void InitSelectedEmitter();
	void InitToolbarIcons();
	void InitTabIcons();
	void InitIcons();
	void InitTextStyles();
	void InitOverview();
	void InitViewportStyle();
	void InitEmitterDetails();
	void InitAssetColors();
	void InitThumbnails();
	void InitClassIcon();
	void InitStackIcons();
	void InitStackWidgets();
	void InitNiagaraSequence();
	void InitPlatformSet();
	void InitDropTarget();
	void InitScriptGraph();
	void InitDebuggerStyle();
	void InitBakerStyle();
	void InitCommonColors();
	void InitOutlinerStyle();
	void InitScalabilityColors();
	void InitScalabilityIcons();
	void InitScratchStyle();
	void InitSimCacheEditor();

	static TSharedPtr<FNiagaraEditorStyle> NiagaraEditorStyle;
};
