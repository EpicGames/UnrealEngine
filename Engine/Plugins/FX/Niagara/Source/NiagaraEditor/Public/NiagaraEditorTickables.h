// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TickableEditorObject.h"



class FNiagaraShaderQueueTickable : FTickableEditorObject
{
public:
	static void ProcessQueue();

	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	virtual void Tick(float DeltaSeconds) override;

	virtual TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraShaderQueueTickable, STATGROUP_Tickables);
	}
};