// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "UObject/StructOnScope.h"
#include "Misc/Attribute.h"
#include "AssetRegistry/AssetData.h"
#include "NiagaraActions.h"
#include "NiagaraGraph.h"
#include "NiagaraEditorSettings.h"
#include "ViewModels/NiagaraSystemScalabilityViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "DataHierarchyViewModelBase.h"

#define UE_API NIAGARAEDITOR_API

class UNiagaraStackFunctionInput;
class UNiagaraClipboardContent;
class UNiagaraNodeInput;
class UNiagaraNodeOutput;
class UNiagaraNodeFunctionCall;
class UNiagaraNodeParameterMapGet;
struct FNiagaraVariable;
struct FNiagaraTypeDefinition;
class UNiagaraGraph;
class UNiagaraSystem;
class FNiagaraSystemViewModel;
struct FNiagaraEmitterHandle;
class UNiagaraEmitter;
class UNiagaraScript;
class FStructOnScope;
class UEdGraph;
class UEdGraphNode;
class SToolTip;
class SWidget;
class UNiagaraNode;
class UEdGraphSchema_Niagara;
class UEdGraphPin;
class FCompileConstantResolver;
class UNiagaraStackEditorData;
class FMenuBuilder;
class FNiagaraEmitterViewModel;
class FNiagaraEmitterHandleViewModel;
enum class ECheckBoxState : uint8;
enum class EScriptSource : uint8;
struct FNiagaraNamespaceMetadata;
class FNiagaraParameterHandle;
class INiagaraParameterDefinitionsSubscriberViewModel;
struct FNiagaraScriptVersionUpgradeContext;
struct FNiagaraScriptConversionContext;
class UUpgradeNiagaraEmitterContext;
struct FNiagaraMessageStore;
class UNiagaraStackNote;
class SToolTip;

enum class ENiagaraFunctionDebugState : uint8;

struct FRefreshAllScriptsFromExternalChangesArgs
{
	UNiagaraScript* OriginatingScript = nullptr;
	UNiagaraGraph* OriginatingGraph = nullptr;
	UNiagaraParameterDefinitions* OriginatingParameterDefinitions = nullptr;
};

struct FNiagaraEditorSharedTexts
{
	static UE_API const FText DebugDrawUIActionBaseText;
};

namespace FNiagaraEditorUtilities
{
	/** Determines if the contents of two sets matches */
	// TODO: Move this to TSet.
	template<typename ElementType>
	bool SetsMatch(const TSet<ElementType>& SetA, const TSet<ElementType>& SetB)
	{
		if (SetA.Num() != SetB.Num())
		{
			return false;
		}

		for (ElementType SetItemA : SetA)
		{
			if (SetB.Contains(SetItemA) == false)
			{
				return false;
			}
		}

		return true;
	}

	/** Determines if the contents of an array matches a set */
	// TODO: Move this to TSet.
	template<typename ElementType>
	bool ArrayMatchesSet(const TArray<ElementType>& Array, const TSet<ElementType>& Set)
	{
		if (Array.Num() != Set.Num())
		{
			return false;
		}

		for (ElementType ArrayItem : Array)
		{
			if (Set.Contains(ArrayItem) == false)
			{
				return false;
			}
		}

		return true;
	}

	/** Gets a set of the system constant names. */
	TSet<FName> GetSystemConstantNames();

	/** Resets the variables value to default, either based on the struct, or if available through registered type utilities. */
	void NIAGARAEDITOR_API ResetVariableToDefaultValue(FNiagaraVariable& Variable);

	/** Fills DefaultData with the types default, either based on the struct, or if available through registered type utilities. */
	void NIAGARAEDITOR_API GetTypeDefaultValue(const FNiagaraTypeDefinition& Type, TArray<uint8>& DefaultData);

	/** Sets up a niagara input node for parameter usage. */
	void InitializeParameterInputNode(UNiagaraNodeInput& InputNode, const FNiagaraTypeDefinition& Type, const UNiagaraGraph* Graph, FName InputName = FName(TEXT("NewInput")));

	/** Writes text to a specified location on disk.*/
	void NIAGARAEDITOR_API WriteTextFileToDisk(FString SaveDirectory, FString FileName, FString TextToSave, bool bAllowOverwriting = false);

	/** Gathers up the change Id's and optionally writes them to disk.*/
	void GatherChangeIds(UNiagaraEmitter& Emitter, TMap<FGuid, FGuid>& ChangeIds, const FString& InDebugName, bool bWriteToLogDir = false);
	void GatherChangeIds(UNiagaraGraph& Graph, TMap<FGuid, FGuid>& ChangeIds, const FString& InDebugName, bool bWriteToLogDir = false);

	/** Options for the GetParameterVariablesFromSystem function. */
	struct FGetParameterVariablesFromSystemOptions
	{
		FGetParameterVariablesFromSystemOptions()
			: bIncludeStructParameters(true)
			, bIncludeDataInterfaceParameters(true)
		{
		}

		bool bIncludeStructParameters;
		bool bIncludeDataInterfaceParameters;
	};

	/** Gets the niagara variables for the input parameters on a niagara System. */
	void GetParameterVariablesFromSystem(UNiagaraSystem& System, TArray<FNiagaraVariable>& ParameterVariables, FGetParameterVariablesFromSystemOptions Options = FGetParameterVariablesFromSystemOptions());

	/** Helper to clean up copy & pasted graphs.*/
	void FixUpPastedNodes(UEdGraph* Graph, TSet<UEdGraphNode*> PastedNodes);

	/** Helper to convert compile status to text.*/
	FText StatusToText(ENiagaraScriptCompileStatus Status);

	/** Helper method to union two distinct compiler statuses.*/
	ENiagaraScriptCompileStatus UnionCompileStatus(const ENiagaraScriptCompileStatus& StatusA, const ENiagaraScriptCompileStatus& StatusB);

	/** Returns whether the data in a niagara variable and a struct on scope match */
	bool DataMatches(const FNiagaraVariable& Variable, const FStructOnScope& StructOnScope);

	/** Returns whether the data in two niagara variables match. */
	bool DataMatches(const FNiagaraVariable& VariableA, const FNiagaraVariable& VariableB);

	/** Returns whether the data in two structs on scope matches. */
	bool DataMatches(const FStructOnScope& StructOnScopeA, const FStructOnScope& StructOnScopeB);

	void NIAGARAEDITOR_API CopyDataTo(FStructOnScope& DestinationStructOnScope, const FStructOnScope& SourceStructOnScope, bool bCheckTypes = true);

	TSharedPtr<SWidget> CreateInlineErrorText(TAttribute<FText> ErrorMessage, TAttribute<FText> ErrorTooltip);

	void CompileExistingEmitters(const TArray<FVersionedNiagaraEmitter>& AffectedEmitters);

	bool TryGetEventDisplayName(UNiagaraEmitter* Emitter, FGuid EventUsageId, FText& OutEventDisplayName);

	bool IsCompilableAssetClass(UClass* AssetClass);

	FText GetVariableTypeCategory(const FNiagaraVariable& Variable);

	FText GetTypeDefinitionCategory(const FNiagaraTypeDefinition& TypeDefinition);

	NIAGARAEDITOR_API bool AreTypesAssignable(const FNiagaraTypeDefinition& FromType, const FNiagaraTypeDefinition& ToType);

	void MarkDependentCompilableAssetsDirty(TArray<UObject*> InObjects);
	void MarkDependentCompilableAssetsDirty(const TArray<FAssetData>& InAssets);

	void ResolveNumerics(UNiagaraGraph* SourceGraph, bool bForceParametersToResolveNumerics, TArray<FNiagaraVariable>& ChangedNumericParams);

	void FixUpNumericPins(const UEdGraphSchema_Niagara* Schema, UNiagaraNode* Node);

	void SetStaticSwitchConstants(UNiagaraGraph* Graph, TArrayView<UEdGraphPin* const> CallInputs, const FCompileConstantResolver& ConstantResolver);

	bool ResolveConstantValue(UEdGraphPin* Pin, int32& Value);

	TSharedPtr<FStructOnScope> StaticSwitchDefaultIntToStructOnScope(int32 InStaticSwitchDefaultValue, FNiagaraTypeDefinition InSwitchType);

	void PreprocessFunctionGraph(const UEdGraphSchema_Niagara* Schema, UNiagaraGraph* Graph, TArrayView<UEdGraphPin* const> CallInputs, TArrayView<UEdGraphPin* const> CallOutputs, ENiagaraScriptUsage ScriptUsage, const FCompileConstantResolver& ConstantResolver);

	bool PODPropertyAppendCompileHash(const void* Container, FProperty* Property, FStringView PropertyName, struct FNiagaraCompileHashVisitor* InVisitor);
	bool NestedPropertiesAppendCompileHash(const void* Container, const UStruct* Struct, EFieldIteratorFlags::SuperClassFlags IteratorFlags, FStringView BaseName, struct FNiagaraCompileHashVisitor* InVisitor);

	/** Options for the GetScriptsByFilter function. 
	** @Param ScriptUsageToInclude Only return Scripts that have this usage
	** @Param (Optional) TargetUsageToMatch Only return Scripts that have this target usage (output node) 
	** @Param bIncludeDeprecatedScripts Whether or not to return Scripts that are deprecated (defaults to false) 
	** @Param bIncludeNonLibraryScripts Whether or not to return non-library scripts (defaults to false)
	*/
	struct FGetFilteredScriptAssetsOptions
	{
		enum ESuggestedFiltering
		{
			NoFiltering,
			OnlySuggested,
			NoSuggested
		};
		FGetFilteredScriptAssetsOptions()
			: ScriptUsageToInclude(ENiagaraScriptUsage::Module)
			, TargetUsageToMatch()
			, bIncludeDeprecatedScripts(false)
			, bIncludeNonLibraryScripts(false)
			, SuggestedFiltering(NoFiltering)
		{
		}

		ENiagaraScriptUsage ScriptUsageToInclude;
		TOptional<ENiagaraScriptUsage> TargetUsageToMatch;
		bool bIncludeDeprecatedScripts;
		bool bIncludeNonLibraryScripts;
		ESuggestedFiltering SuggestedFiltering;
	};

	NIAGARAEDITOR_API void GetFilteredScriptAssets(FGetFilteredScriptAssetsOptions InFilter, TArray<FAssetData>& OutFilteredScriptAssets); 

	NIAGARAEDITOR_API UNiagaraNodeOutput* GetScriptOutputNode(UNiagaraScript& Script);

	UNiagaraScript* GetScriptFromSystem(UNiagaraSystem& System, FGuid EmitterHandleId, ENiagaraScriptUsage Usage, FGuid UsageId);

	/**
	 * Gets an emitter handle from a system and an owned emitter.  This handle will become invalid if emitters are added or
	 * removed from the system, so in general this value should not be cached across frames.
	 * @param System The source system which owns the emitter handles.
	 * @param The emitter to search for in the system.
	 * @returns The emitter handle for the supplied emitter, or nullptr if the emitter isn't owned by this system.
	 */
	const FNiagaraEmitterHandle* GetEmitterHandleForEmitter(UNiagaraSystem& System, const FVersionedNiagaraEmitter& Emitter);

	NIAGARAEDITOR_API ENiagaraScriptLibraryVisibility GetScriptAssetVisibility(const FAssetData& ScriptAssetData);

	NIAGARAEDITOR_API bool GetIsInheritableFromAssetRegistryTags(const FAssetData& AssetData, bool& bUseInheritance);
	
	/** Used instead of reading the template tag directly for backwards compatibility reasons when changing from a bool template specifier to an enum */
	NIAGARAEDITOR_API bool GetTemplateSpecificationFromTag(const FAssetData& Data, ENiagaraScriptTemplateSpecification& OutTemplateSpecification);

	NIAGARAEDITOR_API bool IsScriptAssetInLibrary(const FAssetData& ScriptAssetData);

	NIAGARAEDITOR_API bool IsEnginePluginAsset(const FTopLevelAssetPath& InTopLevelAssetPath);

	NIAGARAEDITOR_API int32 GetWeightForItem(const TSharedPtr<FNiagaraMenuAction_Generic>& Item, FName FavoriteActionsProfile, const TArray<FString>& FilterTerms);

	NIAGARAEDITOR_API bool DoesItemMatchFilterText(const FText& FilterText, const TSharedPtr<FNiagaraMenuAction_Generic>& Item);
	
	NIAGARAEDITOR_API TTuple<EScriptSource, FText> GetScriptSource(const FAssetData& ScriptAssetData);

	NIAGARAEDITOR_API FLinearColor GetScriptSourceColor(EScriptSource ScriptSourceData);

	NIAGARAEDITOR_API FText FormatScriptName(FName Name, bool bIsInLibrary);

	NIAGARAEDITOR_API FText FormatScriptDescription(FText Description, const FSoftObjectPath& Path, bool bIsInLibrary);

	NIAGARAEDITOR_API FText FormatVariableDescription(FText Description, FText Name, FText Type);

	void ResetSystemsThatReferenceSystemViewModel(const FNiagaraSystemViewModel& ReferencedSystemViewModel);

	TArray<UNiagaraComponent*> GetComponentsThatReferenceSystem(const UNiagaraSystem& ReferencedSystem);

	TArray<UNiagaraComponent*> GetComponentsThatReferenceSystemViewModel(const FNiagaraSystemViewModel& ReferencedSystemViewModel);

	NIAGARAEDITOR_API const FGuid AddEmitterToSystem(UNiagaraSystem& InSystem, UNiagaraEmitter& InEmitterToAdd, FGuid EmitterVersion, bool bCreateCopy = true);

	void RemoveEmittersFromSystemByEmitterHandleId(UNiagaraSystem& InSystem, TSet<FGuid> EmitterHandleIdsToDelete);

	/** Kills all system instances using the referenced system. */
	void KillSystemInstances(const UNiagaraSystem& System);


	bool VerifyNameChangeForInputOrOutputNode(const UNiagaraNode& NodeBeingChanged, FName OldName, FString NewName, FText& OutErrorMessage);

	/**
	 * Adds a new Parameter to a target ParameterStore with an undo/redo transaction and name collision handling.
	 * @param NewParameterVariable The FNiagaraVariable to be added to TargetParameterStore. MUST be a unique object, do not pass an existing reference.
	 * @param TargetParameterStore The ParameterStore to receive NewVariable.
	 * @param ParameterStoreOwner The UObject to call Modify() on for the undo/redo transaction of adding NewVariable.
	 * @param StackEditorData The editor data used to mark the newly added FNiagaraVariable in the Stack for renaming.
	 * @returns Bool for whether adding the parameter succeeded.
	 */
	bool AddParameter(FNiagaraVariable& NewParameterVariable, FNiagaraParameterStore& TargetParameterStore, UObject& ParameterStoreOwner, UNiagaraStackEditorData* StackEditorData);

	struct FVariableAttributeBindingInfo
	{
		/** The actual binding that was found during traversal */
		FNiagaraVariableAttributeBinding* Binding = nullptr;
		/** The default binding as provided by the CDO. */
		FNiagaraVariableAttributeBinding* DefaultBinding = nullptr;
		ENiagaraRendererSourceDataMode SourceDataMode = ENiagaraRendererSourceDataMode::Particles;
		/** Immediate owning object, used to call Modify() */
		UObject* Owner = nullptr;
		/** The owning emitter, if it exists. */
		FVersionedNiagaraEmitter OwningEmitter;
	};

	/** Config struct to set what an action should be targeting. Can target the system, all emitters, or a specific emitter.
	 * If SpecificEmitterGuid is set, bIncludeEmitters should be true also.  */
	struct FNiagaraActionScopeConfig
	{
		FNiagaraActionScopeConfig() = default;
		FNiagaraActionScopeConfig(FGuid InSpecificEmitterGuid);
		
		bool bIncludeSystem = true;
		bool bIncludeEmitters = true;
		TOptional<FGuid> SpecificEmitterGuid;
	};
	
	TArray<FVariableAttributeBindingInfo> GetVariableAttributeBindingsForParameter(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable Parameter, const FNiagaraActionScopeConfig& Config);

	NIAGARAEDITOR_API bool AddEmitterContextMenuActions(FMenuBuilder& MenuBuilder, const TSharedPtr<FNiagaraEmitterHandleViewModel>& EmitterHandleViewModel);

	void ShowParentEmitterInContentBrowser(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);

	NIAGARAEDITOR_API void OpenParentEmitterForEdit(TSharedRef<FNiagaraEmitterViewModel> Emitter);
	ECheckBoxState GetSelectedEmittersEnabledCheckState(TSharedRef<FNiagaraSystemViewModel> SystemViewModel);
	void ToggleSelectedEmittersEnabled(TSharedRef<FNiagaraSystemViewModel> SystemViewModel);

	ECheckBoxState GetSelectedEmittersIsolatedCheckState(TSharedRef<FNiagaraSystemViewModel> SystemViewModel);
	void ToggleSelectedEmittersIsolated(TSharedRef<FNiagaraSystemViewModel> SystemViewModel);

	void CreateAssetFromEmitter(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);

	NIAGARAEDITOR_API void WarnWithToastAndLog(FText WarningMessage);
	NIAGARAEDITOR_API void InfoWithToastAndLog(FText WarningMessage, float ToastDuration = 5.0f);

	NIAGARAEDITOR_API FName GetUniqueObjectName(UObject* Outer, UClass* ObjectClass, const FString& CandidateName);

	template<typename T>
	FName GetUniqueObjectName(UObject* Outer, const FString& CandidateName)
	{
		return GetUniqueObjectName(Outer, T::StaticClass(), CandidateName);
	}

	NIAGARAEDITOR_API TArray<FName> DecomposeVariableNamespace(const FName& InVarNameToken, FName& OutName);

	NIAGARAEDITOR_API void  RecomposeVariableNamespace(const FName& InVarNameToken, const TArray<FName>& InParentNamespaces, FName& OutName);

	FString NIAGARAEDITOR_API GetNamespacelessVariableNameString(const FName& InVarName);

	void GetReferencingFunctionCallNodes(UNiagaraScript* Script, TArray<UNiagaraNodeFunctionCall*>& OutReferencingFunctionCallNodes);

	// Compare two FNiagaraVariable names for the sort priority relative to the first argument VarNameA. Sorting is ordered by namespace and then alphabetized. 
	bool GetVariableSortPriority(const FName& VarNameA, const FName& VarNameB);

	// Compare two FNiagaraNamespaceMetadata for the sort priority relative to the first argument A, where a lower number represents a higher priority.
	int32 GetNamespaceMetaDataSortPriority(const FNiagaraNamespaceMetadata& A, const FNiagaraNamespaceMetadata& B);

	// Get the sort priority of a registered namespace FName, where a lower number represents a higher priority.
	int32 GetNamespaceSortPriority(const FName& Namespace);

	const FNiagaraNamespaceMetadata GetNamespaceMetaDataForVariableName(const FName& VarName);

	const FNiagaraNamespaceMetadata GetNamespaceMetaDataForId(const FGuid& NamespaceId);

	const FGuid& GetNamespaceIdForUsage(ENiagaraScriptUsage Usage);

	// Convenience wrapper to get all discovered parameter definitions assets from asset registry.
	// NOTE: You are not guaranteed to get all parameter definitions in mounted directories if the asset registry has not finished discovering assets.
	TArray<UNiagaraParameterDefinitions*> GetAllParameterDefinitions();

	bool GetAvailableParameterDefinitions(const TArray<FString>& ExternalPackagePaths, TArray<FAssetData>& OutParameterDefinitionsAssetData);

	NIAGARAEDITOR_API void GetAvailableParameterCollections(TArray<UNiagaraParameterCollection*>& OutParameterCollections);

	NIAGARAEDITOR_API void GetAvailableParameterDefinitions(TArray<UNiagaraParameterDefinitions*>& OutParameterDefinitions);

	TSharedPtr<INiagaraParameterDefinitionsSubscriberViewModel> GetOwningLibrarySubscriberViewModelForGraph(const UNiagaraGraph* Graph);

	TArray<UNiagaraParameterDefinitions*> DowncastParameterDefinitionsBaseArray(const TArray<UNiagaraParameterDefinitionsBase*> BaseArray);
	
	// Executes python conversion script on the given source node
    UNiagaraClipboardContent* RunPythonConversionScript(FVersionedNiagaraScriptData& NewScriptVersionData, UNiagaraClipboardContent* NewScriptInputs, FVersionedNiagaraScriptData& OldScriptVersionData, UNiagaraClipboardContent* OldScriptInputs, FText& OutWarnings);

	// Executes python upgrade scripts on the given source node for all the given in-between versions
	void RunPythonUpgradeScripts(UNiagaraNodeFunctionCall* SourceNode, const TArray<FVersionedNiagaraScriptData*>& UpgradeVersionData, const FNiagaraScriptVersionUpgradeContext& UpgradeContext, FString& OutWarnings);

	// Executes python upgrade scripts on the given source node for all the given in-between versions
	void RunPythonUpgradeScripts(UUpgradeNiagaraEmitterContext* UpgradeContext);

	// Changes the referenced parent version and optionally runs the python upgrade scripts
	NIAGARAEDITOR_API void SwitchParentEmitterVersion(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel, TSharedRef<FNiagaraSystemViewModel> SystemViewModel, const FGuid& NewVersionGuid);

	void RefreshAllScriptsFromExternalChanges(FRefreshAllScriptsFromExternalChangesArgs Args);

	DECLARE_DELEGATE_OneParam(FNodeVisitor, UEdGraphNode* /*VisitedNode*/);
	void VisitAllNodesConnectedToInputs(UEdGraphNode* StartNode, FNodeVisitor Visitor);
	
	NIAGARAEDITOR_API float GetScalabilityTintAlpha(FNiagaraEmitterHandle* EmitterHandle);

	enum class ETrackAssetResult
	{
		// do not count this asset
		Ignore,

		// count this asset
		Count,

		// count this asset and also check assets referencing it
		CountRecursive
	};

	NIAGARAEDITOR_API int GetReferencedAssetCount(const FAssetData& SourceAsset, TFunction<ETrackAssetResult(const FAssetData&)> Predicate);

	/** Gets a list of the registered user variable types which are allowed in the current editor context.  This API should be
		called when providing a list types to the user instead of getting the type list directly from the type registry. */
	void GetAllowedUserVariableTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes);

	/** Gets a list of the registered system variable types which are allowed in the current editor context.  This API should be
		called when providing a list types to the user instead of getting the type list directly from the type registry. */
	void GetAllowedSystemVariableTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes);

	/** Gets a list of the registered emitter variable types which are allowed in the current editor context.  This API should be
		called when providing a list types to the user instead of getting the type list directly from the type registry. */
	void GetAllowedEmitterVariableTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes);

	/** Gets a list of the registered particle variable types which are allowed in the current editor context.  This API should be
		called when providing a list types to the user instead of getting the type list directly from the type registry. */
	void GetAllowedParticleVariableTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes);

	/** Gets a list of the registered parameter types which are allowed in the current editor context.  This API should be
		called when providing a list types to the user instead of getting the type list directly from the type registry. */
	void GetAllowedParameterTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes);

	/** Gets a list of the registered payload types which are allowed in the current editor context.  This API should be
		called when providing a list types to the user instead of getting the type list directly from the type registry. */
	void GetAllowedPayloadTypes(TArray<FNiagaraTypeDefinition>& OutAllowedTypes);

	bool IsEnumIndexVisible(const UEnum* Enum, int32 Index);

	/** Test to see id the global filter allows this class or not.   Classes that are not allowed can fail validation. */
	NIAGARAEDITOR_API bool IsClassVisibileInGlobalFilter(UClass* Class);

	void GetScriptMessageStores(UNiagaraScript* InScript, TArray<FNiagaraMessageSourceAndStore>& OutNiagaraMessageStores);

	NIAGARAEDITOR_API bool IsEditorDataInterfaceInstance(const UNiagaraDataInterface* DataInterface);

	NIAGARAEDITOR_API UNiagaraDataInterface* GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(const UNiagaraSystem& OwningSystem, UNiagaraDataInterface& EditorDataInterfaceInstance);

	NIAGARAEDITOR_API EAxisList::Type VectorComponentToAxis(int32 NumComponents, int32 ComponentIndex);

	namespace UserParameters
	{
		NIAGARAEDITOR_API void AddUserParameterToSystem(UNiagaraSystem& System, const FNiagaraVariableBase& UserParameter);

		NIAGARAEDITOR_API void RemoveUserParameterFromSystem(UNiagaraSystem& System, const FNiagaraVariableBase& UserParameter);

		NIAGARAEDITOR_API void RenameUserParameterForSystem(UNiagaraSystem& System, const FNiagaraVariableBase& OldParameter, const FNiagaraVariableBase& NewParameter);

		NIAGARAEDITOR_API void GetUserParameterDefaultForSystem(const UNiagaraSystem& System, const FNiagaraVariableWithOffset& UserParameterWithOffset, FNiagaraVariant& OutDefault);

		NIAGARAEDITOR_API TOptional<FNiagaraVariant> FindUserParameterDefaultForSystem(const UNiagaraSystem& System, const FNiagaraVariableBase& UserParameter);

		NIAGARAEDITOR_API void SetUserParameterDefaultForSystem(UNiagaraSystem& System, const FNiagaraVariableBase& UserParameter, const FNiagaraVariant& Default);

		FNiagaraVariable DuplicateUserParameter(FNiagaraVariable ParameterToDuplicate, UNiagaraSystem& System);

		TArray<FNiagaraVariable> GetReferencedUserParametersFromEmitter(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
		
		TArray<UNiagaraNodeParameterMapGet*> GetParameterMapGetNodesWithUserParameter(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable UserParameter, const FNiagaraActionScopeConfig& Config);

		struct FUserParameterBindingInfo
		{
			FNiagaraUserParameterBinding* Binding = nullptr;
			UObject* Owner = nullptr;
		};
		
		TArray<FUserParameterBindingInfo> GetUserParameterBindingsForUserParameter(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable UserParameter, const FNiagaraActionScopeConfig& Config);
		TArray<UNiagaraStackFunctionInput*> GetStackFunctionInputsWithLinkedParameter(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable Parameter, const FNiagaraActionScopeConfig& Config);
		
		NIAGARAEDITOR_API TObjectPtr<UNiagaraScriptVariable> GetScriptVariableForUserParameter(const FNiagaraVariable& UserParameter, TSharedPtr<FNiagaraSystemViewModel> SystemViewModel);
		NIAGARAEDITOR_API TObjectPtr<UNiagaraScriptVariable> GetScriptVariableForUserParameter(const FNiagaraVariable& UserParameter, UNiagaraSystem& System);
		NIAGARAEDITOR_API const UNiagaraScriptVariable* FindScriptVariableForUserParameter(const FGuid& UserParameterGuid, const UNiagaraSystem& System);
		NIAGARAEDITOR_API const UNiagaraScriptVariable* FindScriptVariableForUserParameter(const FNiagaraVariable& UserParameter, const UNiagaraSystem& System);
		
		void ReplaceUserParameterReferences(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable OldUserParameter, FNiagaraVariable NewUserParameter, const FNiagaraActionScopeConfig& Config);
		void DeleteUserParameterReferences(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, FNiagaraVariable UserParameterToDelete, const FNiagaraActionScopeConfig& Config);
	}

	namespace Preview
	{
		NIAGARAEDITOR_API TOptional<FSoftObjectPath> GetPreviewMovieObjectPath(const FAssetData& AssetData);
	}
	
	namespace Tooltips
	{
		struct FCreateTooltipArguments
		{
			/** If this is true, PreviewMovieOwner has to be valid as well. If false, will not attempt to create a movie. */
			bool bTryCreatePreviewMovie = false;
			TSharedPtr<FNiagaraPreviewMovieViewModel> PreviewMovieOwner;
		};

		/** An all purpose function to create a tooltip with a text and an optional movie. */
		NIAGARAEDITOR_API TSharedRef<SToolTip> CreateTooltip(FText Description, TOptional<FSoftObjectPath> PreviewMoviePath = TOptional<FSoftObjectPath>(), const FCreateTooltipArguments& CreateTooltipArguments = FCreateTooltipArguments());
		/** Creates a tooltip for a niagara action. Supports preview movies. */
		NIAGARAEDITOR_API TSharedRef<SToolTip> CreateTooltipForNiagaraAction(TSharedRef<FNiagaraMenuAction_Generic> Action, const FCreateTooltipArguments& CreateTooltipArguments = FCreateTooltipArguments());
		/** Creates a tooltip for a script. Supports preview movies.*/
		NIAGARAEDITOR_API TSharedRef<SToolTip> CreateTooltipForScript(UNiagaraScript& Script, FGuid VersionGuid = FGuid(), const FCreateTooltipArguments& CreateTooltipArguments = FCreateTooltipArguments());
		
		NIAGARAEDITOR_API TSharedRef<SToolTip> CreateStackNoteTooltip(UNiagaraStackNote& StackNote);
		NIAGARAEDITOR_API FText GetMinimalEmitterCreationTooltip();
	}
	
	namespace Scripts
	{
		namespace Validation
		{
			TMap<FGuid, TArray<FNiagaraVariableBase>> ValidateScriptVariableIds(UNiagaraScript* Script, FGuid VersionGuid);
			TMap<FNiagaraVariableBase, FGuid> FixupDuplicateScriptVariableGuids(UNiagaraScript* Script);
		}
	}

	namespace HierarchyEditor
	{		
		namespace Scripts
		{
			TSharedRef<SWidget> GenerateRowContentForScriptParameterHierarchyEditor(TSharedRef<FHierarchyElementViewModel> HierarchyItem);
		}
	}
}

namespace FNiagaraParameterUtilities
{
	bool DoesParameterNameMatchSearchText(FName ParameterName, const FString& SearchTextString);

	FText FormatParameterNameForTextDisplay(FName ParameterName);

	bool GetNamespaceEditData(
		FName InParameterName,
		FNiagaraParameterHandle& OutParameterHandle,
		FNiagaraNamespaceMetadata& OutNamespaceMetadata,
		FText& OutErrorMessage);

	bool GetNamespaceModifierEditData(
		FName InParameterName,
		FNiagaraParameterHandle& OutParameterHandle,
		FNiagaraNamespaceMetadata& OutNamespaceMetadata,
		FText& OutErrorMessage);

	enum class EParameterContext : uint8
	{
		Script,
		System,
		Definitions,
	};

	struct FChangeNamespaceMenuData
	{
		bool bCanChange;
		FText CanChangeToolTip;
		FName NamespaceParameterName;
		FNiagaraNamespaceMetadata Metadata;
	};

	NIAGARAEDITOR_API void GetChangeNamespaceMenuData(FName InParameterName, EParameterContext InParameterContext, TArray<FChangeNamespaceMenuData>& OutChangeNamespaceMenuData);

	NIAGARAEDITOR_API TSharedRef<SWidget> CreateNamespaceMenuItemWidget(FName Namespace, FText ToolTip);

	NIAGARAEDITOR_API bool TestCanChangeNamespaceWithMessage(FName ParameterName, const FNiagaraNamespaceMetadata& NewNamespaceMetadata, FText& OutMessage);

	NIAGARAEDITOR_API FName ChangeNamespace(FName ParameterName, const FNiagaraNamespaceMetadata& NewNamespaceMetadata);

	NIAGARAEDITOR_API int32 GetNumberOfNamePartsBeforeEditableModifier(const FNiagaraNamespaceMetadata& NamespaceMetadata);

	NIAGARAEDITOR_API void GetOptionalNamespaceModifiers(FName ParameterName, EParameterContext InParameterContext, TArray<FName>& OutOptionalNamespaceModifiers);

	NIAGARAEDITOR_API FName GetEditableNamespaceModifierForParameter(FName ParameterName);

	NIAGARAEDITOR_API bool TestCanSetSpecificNamespaceModifierWithMessage(FName InParameterName, FName InNamespaceModifier, FText& OutMessage);

	NIAGARAEDITOR_API FName SetSpecificNamespaceModifier(FName InParameterName, FName InNamespaceModifier);

	NIAGARAEDITOR_API bool TestCanSetCustomNamespaceModifierWithMessage(FName InParameterName, FText& OutMessage);

	NIAGARAEDITOR_API FName SetCustomNamespaceModifier(FName InParameterName);

	NIAGARAEDITOR_API FName SetCustomNamespaceModifier(FName InParameterName, TSet<FName>& CurrentParameterNames);

	NIAGARAEDITOR_API bool TestCanRenameWithMessage(FName ParameterName, FText& OutMessage);

	/** A simple utility function to generate a parameter widget. */
	NIAGARAEDITOR_API TSharedRef<SWidget> GetParameterWidget(FNiagaraVariable Variable, bool bAddTypeIcon, bool bShowValue);

	struct FNiagaraParameterWidgetOptions
	{
		bool bAddTypeIcon = false;
		bool bShowValue = false;
		bool bShowVisibilityConditionIcon = false;
		bool bShowEditConditionIcon = false;
		bool bShowAdvanced = false;
		TOptional<TAttribute<FText>> NameOverride;
		TOptional<TAttribute<EVisibility>> NameOverrideVisibility;
		TOptional<TAttribute<FText>> NameOverrideTooltip;
	};
	
	/** A more advanced utility function that also makes use of parameter metadata to generate a parameter widget */
	NIAGARAEDITOR_API TSharedRef<SWidget> GetParameterWidget(FNiagaraVariable Variable, FNiagaraVariableMetaData MetaData, FNiagaraParameterWidgetOptions Options);

	/** Creates a tooltip based on a parameter. Also shows the value, if allocated and enabled. */
	NIAGARAEDITOR_API TSharedRef<SToolTip> GetTooltipWidget(FNiagaraVariable Variable, bool bShowValue = true, TSharedPtr<SWidget> AdditionalVerticalWidget = nullptr);

	NIAGARAEDITOR_API void FilterToRelevantStaticVariables(TConstArrayView<FNiagaraVariable> InVars, TArray<FNiagaraVariable>& OutVars, FName InOldEmitterAlias, FName InNewEmitterAlias, bool bFilterByEmitterAliasAndConvertToUnaliased);
};

namespace FNiagaraParameterDefinitionsUtilities
{
	TArray<const UNiagaraScriptVariable*>  FindReservedParametersByName(const FName ParameterName);
	int32 GetNumParametersReservedForName(const FName ParameterName);
	EParameterDefinitionMatchState GetDefinitionMatchStateForParameter(const FNiagaraVariableBase& Parameter);
	void TrySubscribeScriptVarToDefinitionByName(UNiagaraScriptVariable* ScriptVar, INiagaraParameterDefinitionsSubscriberViewModel* OwningDefinitionSubscriberViewModel);
};

class FNiagaraEnumIndexVisibilityCache
{
public:
	static bool GetVisibility(const UEnum* InEnum, int32 InIndex);

private:
	struct FEnumIndexPair
	{
		FEnumIndexPair(const UEnum* InEnum, int32 InIndex) : Enum(InEnum), Index(InIndex) { }
		bool operator==(const FEnumIndexPair& Other) const
		{
			return Enum == Other.Enum && Index == Other.Index;
		}
		const UEnum* Enum;
		int32 Index;
	};

	friend inline uint32 GetTypeHash(const FEnumIndexPair& EnumIndexPair)
	{
		return HashCombineFast(GetTypeHash(EnumIndexPair.Enum), GetTypeHash(EnumIndexPair.Index));
	}

	static TMap<FEnumIndexPair, bool> Cache;
	static FCriticalSection CacheLock;
};

#undef UE_API
