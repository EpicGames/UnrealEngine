// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEditorDataBase.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraStackSection.h"
#include "DataHierarchyViewModelBase.h"
#include "NiagaraEmitterEditorData.generated.h"

class UNiagaraStackEditorData;


/** Editor only UI data for emitters. */
UCLASS(MinimalAPI)
class UNiagaraEmitterEditorData : public UNiagaraEditorDataBase
{
	GENERATED_BODY()

public: 
	struct PrivateMemberNames
	{
		static const FName SummarySections;
	};

public:
	UNiagaraEmitterEditorData(const FObjectInitializer& ObjectInitializer);
	
	virtual void PostLoad() override;
	virtual void PostLoadFromOwner(UObject* InOwner) override;
#if WITH_EDITORONLY_DATA
	NIAGARAEDITOR_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	NIAGARAEDITOR_API UNiagaraStackEditorData& GetStackEditorData() const;

	TRange<float> GetPlaybackRange() const;

	void SetPlaybackRange(TRange<float> InPlaybackRange);

	NIAGARAEDITOR_API FSimpleMulticastDelegate& OnSummaryViewStateChanged();
	
	bool ShouldShowSummaryView() const { return bShowSummaryView; }
	NIAGARAEDITOR_API void SetShowSummaryView(bool bInShouldShowSummaryView);
	NIAGARAEDITOR_API void ToggleShowSummaryView();
	
	UHierarchyRoot* GetSummaryRoot() const { return SummaryViewRoot; }
	TConstArrayView<const TObjectPtr<UHierarchySection>> GetSummarySections() const;

	UTexture2D* GetThumbnail() const { return EmitterThumbnail; }
	void SetThumbnail(UTexture2D* InThumbnail) { EmitterThumbnail = InThumbnail; OnPersistentDataChanged().Broadcast(); }
private:
	UPROPERTY(Instanced)
	TObjectPtr<UNiagaraStackEditorData> StackEditorData;

	UPROPERTY()
	float PlaybackRangeMin;

	UPROPERTY()
	float PlaybackRangeMax;

	UPROPERTY()
	uint32 bShowSummaryView : 1;

	/** Stores metadata for filtering function inputs when in Filtered/Simple view. */
	UPROPERTY()
	TMap<FFunctionInputSummaryViewKey, FFunctionInputSummaryViewMetadata> SummaryViewFunctionInputMetadata_DEPRECATED;

	UPROPERTY()
	TArray<FNiagaraStackSection> SummarySections_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UHierarchyRoot> SummaryViewRoot = nullptr;

	UPROPERTY()
	TObjectPtr<UTexture2D> EmitterThumbnail = nullptr;
	
	FSimpleMulticastDelegate OnSummaryViewStateChangedDelegate;
	
	void StackEditorDataChanged();

	void PostLoad_TransferSummaryDataToNewFormat();
	void PostLoad_TransferEmitterThumbnailImage(UObject* Owner);
	void PostLoad_TransferModuleStackNotesToNewFormat(UObject* Owner);
};


