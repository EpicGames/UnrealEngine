// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "NiagaraSettings.h"
#include "FrontendFilterBase.h"

#include "NiagaraEmitterFactoryNew.generated.h"

/** A factory for niagara emitter assets. */
UCLASS(hidecategories = Object, MinimalAPI)
class UNiagaraEmitterFactoryNew : public UFactory
{
	GENERATED_BODY()

public:
	UNiagaraEmitterFactoryNew();
	
	TWeakObjectPtr<UNiagaraEmitter> EmitterToCopy;
	bool bUseInheritance;
	bool bAddDefaultModulesAndRenderersToEmptyEmitter;

private:
	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual bool ConfigurePropertiesAsync(FOnFactoryConfigurePropertiesAsyncComplete OnComplete, FOnFactoryConfigurePropertiesAsyncCancelled OnCancelled) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

public:
	NIAGARAEDITOR_API static void InitializeEmitter(UNiagaraEmitter* NewEmitter, bool bAddDefaultModulesAndRenderers);

private:
	void OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type Method);
	bool OnAdditionalShouldFilterAsset(const FAssetData& AssetData) const;
	void OnExtendAddFilterMenu(UToolMenu* ToolMenu) const;
	TArray<TSharedRef<FFrontendFilter>> OnGetExtraFrontendFilters() const;

};



