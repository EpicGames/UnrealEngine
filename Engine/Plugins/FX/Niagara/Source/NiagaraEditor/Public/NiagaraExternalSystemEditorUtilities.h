// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraVariant.h"
#include "NiagaraEditorUtilities.h"

#include "JsonObjectConverter.h"
#include "JsonSchema/JsonSchemaPropertyFilter.h"

#include "NiagaraParameterMapHistoryFwd.h"
#include "NiagaraVariableMetaData.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "NiagaraExternalSystemEditorUtilities.generated.h"

class UNiagaraStackModuleItem;
class UNiagaraStackFunctionInput;

#define UE_API NIAGARAEDITOR_API

/**
 * Indirection for the property reflection behind UNiagaraExternalEditUtilities' external-edit calls. Holding only
 * plain TFunctions (no external-plugin types in the signatures) lets NiagaraEditor build without any provider
 * plugin. An installable provider supplies a richer implementation while loaded. SetPropertyProvider fills
 * any member left unbound with the default JsonUtilities-based implementation, so an override can replace just the
 * operations it cares about and the active provider's callables are always safe to invoke.
 */
struct FNiagaraExternalEditPropertyProvider
{
	/** JSON Schema string for the editable properties of a struct or class. */
	TFunction<FString(const UStruct* Struct, bool bUserVisibleOnly)> ListStructProperties;

	/** The named properties of an object as a JSON value string. */
	TFunction<FString(const UObject* Object, const TArray<FName>& PropertyNames)> GetObjectProperties;

	/** Applies a JSON property string to an object; returns true on success. */
	TFunction<bool(UObject* Object, const FString& PropertiesJson)> SetObjectProperties;
};

//Tools for external code to modify Niagara Systems.
//Provides minimal public API required for info about and modification of Niagara internals.
//
//EXPERIMENTAL! DO NOT USE EXCEPT FOR TESTING! EVERYTHING HERE IS SUBJECT TO CHANGE WITHOUT NOTICE!
//EXPERIMENTAL! DO NOT USE EXCEPT FOR TESTING! EVERYTHING HERE IS SUBJECT TO CHANGE WITHOUT NOTICE!
//EXPERIMENTAL! DO NOT USE EXCEPT FOR TESTING! EVERYTHING HERE IS SUBJECT TO CHANGE WITHOUT NOTICE!


/** Base class for instanced values used in Niagara External System Editor. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_InstancedValue : public FInstancedStruct
{
	GENERATED_BODY()
};

/**
An instanced struct type that holds the value of a Niagara Variable.
Can contain specialized types (Object, Enum, DataInterface wrappers) or standard value types.
*/
USTRUCT(BlueprintInternalUseOnly, meta=(
	BaseStruct="/Script/NiagaraEditor.NiagaraExt_VariableValueBase",
	AdditionalTypes="/Script/Niagara.NiagaraFloat,/Script/Niagara.NiagaraBool,/Script/Niagara.NiagaraInt32,/Script/Niagara.NiagaraPosition,/Script/Niagara.NiagaraID,/Script/CoreUObject.Vector,/Script/CoreUObject.Vector2D,/Script/CoreUObject.Vector4,/Script/CoreUObject.Quat,/Script/CoreUObject.LinearColor,/Script/CoreUObject.Rotator"
))
struct FNiagaraExt_VariableValue : public FNiagaraExt_InstancedValue
{
	GENERATED_BODY()

	UE_API void Get(FNiagaraVariant& OutVariant, FNiagaraExternalEditContext& Context)const;
	UE_API void Set(const FNiagaraTypeDefinition& TypeDef, const FNiagaraVariant& Variant);
};

/** Base struct for variable value types. Excluded from schema - only derived types are valid. */
USTRUCT(BlueprintInternalUseOnly, meta=(EDAIncludeStructsInSchema, ExcludeBaseStruct))
struct FNiagaraExt_VariableValueBase
{
	GENERATED_BODY()
};

/**
Wrapper for UEnum values. USE THIS when the Niagara variable type is an Enum.
Provides type-safe enum reference and string value selection.
*/
USTRUCT(BlueprintInternalUseOnly, meta=(
	ToolTip="Wrapper for UEnum references. Use this struct when passing enum values to Niagara variables."
))
struct FNiagaraExt_VariableValue_Enum : public FNiagaraExt_VariableValueBase
{
	GENERATED_BODY()

	/** Reference to the UEnum type. */
	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UEnum> Enum;

	/** The selected enum name (e.g., \"MyEnum::Name1\"). */
	UPROPERTY(EditAnywhere, Category="Variable", meta=(ToolTip="The enum value name"))
	FName EnumName;

	/** Display value for EnumName. Often more descriptive and useful than the internal EnumName. For informational purposes only, the actual set value of the enum is EnumName. */
	UPROPERTY(EditAnywhere, Category="Variable")
	FText DisplayName;
};

/**
Wrapper for UNiagaraDataInterface values. USE THIS when the Niagara variable type is a DataInterface.
Provides class selection and instance reference.
*/
USTRUCT(BlueprintInternalUseOnly, meta=(
	ToolTip="Wrapper for DataInterface references. Use this struct when passing DataInterface instances to Niagara variables."
))
struct FNiagaraExt_VariableValue_DataInterface : public FNiagaraExt_VariableValueBase
{
	GENERATED_BODY()

	/** The DataInterface class type */
	UPROPERTY(EditAnywhere, Category="Variable", meta=(
		ToolTip="The class of the DataInterface"
	))
	TSubclassOf<UNiagaraDataInterface> DataInterfaceClass;

	/** The DataInterface instance */
	UPROPERTY(EditAnywhere, Category="Variable", meta=(
		ToolTip="Reference to the DataInterface object"
	))
	TObjectPtr<UNiagaraDataInterface> DataInterface;
};

/**
Wrapper for generic UObject values. USE THIS when the Niagara variable type is a UObject reference (textures, materials, etc.).
Provides class constraint and object reference. Do NOT use simple types for object references.
*/
USTRUCT(BlueprintInternalUseOnly, meta=(
	ToolTip="Wrapper for UObject references. Use this struct when passing UObject instances (textures, materials, etc.) to Niagara variables."
))
struct FNiagaraExt_VariableValue_Object : public FNiagaraExt_VariableValueBase
{
	GENERATED_BODY()

	/** The expected UObject class type (e.g., UTexture2D, UMaterialInterface) */
	UPROPERTY(EditAnywhere, Category="Variable", meta=(
		ToolTip="The class type constraint for the object reference"
	))
	TSubclassOf<UObject> ObjectClass;

	/** The UObject instance reference */
	UPROPERTY(EditAnywhere, Category="Variable", meta=(
		ToolTip="Reference to the UObject instance"
	))
	TObjectPtr<UObject> Object;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_Variable
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Variable")
	FName Name;

 	UPROPERTY()
 	FNiagaraTypeDefinition Type;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_VariableInst : public FNiagaraExt_Variable
{
	GENERATED_BODY()

	UPROPERTY()
	FNiagaraExt_VariableValue Value;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_UserVariable : public FNiagaraExt_Variable
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Variable")
	FNiagaraExt_VariableValue DefaultValue;

	UPROPERTY(EditAnywhere, Category="Variable")
	FText Description;
};

/** A single parameter entry for a SetParameters (UNiagaraNodeAssignment) module. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_SetParameterEntry
{
	GENERATED_BODY()

	/** The variable to assign (name + type). */
	UPROPERTY(EditAnywhere, Category="Parameter")
	FNiagaraExt_Variable Variable;

	UPROPERTY(EditAnywhere, Category="Parameter")
	FNiagaraExt_VariableValue DefaultValue;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_UserVariables
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Variables")
	TArray<FNiagaraExt_UserVariable> UserVariables;
};


/* We expose the internals of Niagara in 3 layers.
*
* 1. Property Schema: Top level properties that can be modified for Systems,Emitters,Modules,Renderers,DataInterfaces. Constant and may need calling only once.
* 2. Topology: The currently topology of the Niagara System, Emitters, Modules and their Dynamic input chains. Changes infrequently.
* 3. Values: The current values of properties and module inputs. Changes frequently.
*/

//////////////////////////////////////////////////////////////////////////
// Diagnostics Layer - Compile state and stack issues.

/** Mirror of ENiagaraScriptCompileStatus for external/AI use. */
UENUM(BlueprintType)
enum class ENiagaraExt_ScriptCompileStatus : uint8
{
	Unknown,
	Dirty,
	Error,
	UpToDate,
	BeingCreated,
	UpToDateWithWarnings,
	ComputeUpToDateWithWarnings,
};

/** Mirror of FNiagaraCompileEventSeverity for external/AI use. */
UENUM(BlueprintType)
enum class ENiagaraExt_CompileEventSeverity : uint8
{
	Log,
	Display,
	Warning,
	Error,
};

/** Mirror of EStackIssueSeverity for external/AI use. */
UENUM(BlueprintType)
enum class ENiagaraExt_StackIssueSeverity : uint8
{
	Error,
	Warning,
	Info,
	None,
};

/** Mirror of EStackIssueFixStyle for external/AI use. */
UENUM(BlueprintType)
enum class ENiagaraExt_StackIssueFixStyle : uint8
{
	Fix,
	Link,
};

/** A single compile event produced during Niagara script translation/compilation. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_CompileEvent
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	ENiagaraExt_CompileEventSeverity Severity = ENiagaraExt_CompileEventSeverity::Log;

	/** The compile event message. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	FString Message;

	/** Optional short description for the event. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	FString ShortDescription;

	/** Guid of the node that produced this event (may be zero if not associated with a node). */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	FGuid NodeGuid;

	/** Guid of the pin that produced this event (may be zero). */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	FGuid PinGuid;

	/**
	 * True if this event originated on a dependency script and propagated to this one, rather
	 * than being raised directly by this script's own compile. When true, the fix for the
	 * underlying problem is typically in a different script — inspect the referenced scripts
	 * before editing this one.
	 */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	bool bFromScriptDependency = false;
};

/**
 * Compile information for a single script (e.g. SystemSpawn, EmitterUpdate, etc.) within a Niagara System.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_ScriptCompileInfo
{
	GENERATED_BODY()

	/** Name of the emitter that owns this script, or NAME_None for system-level scripts. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	FName EmitterName;

	/**
	 * Script role name from ENiagaraScriptUsage (e.g. SystemSpawnScript, ParticleUpdateScript).
	 * Matches the ScriptName carried on FNiagaraExt_StackIssue::Location so compile events and
	 * stack issues can be cross-referenced by script. NAME_None if the usage could not be named.
	 */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	FName ScriptName;

	/** Last known compile status of this script. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	ENiagaraExt_ScriptCompileStatus LastCompileStatus = ENiagaraExt_ScriptCompileStatus::Unknown;

	/** Error message string from the last compile attempt (empty if no error). */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	FString ErrorSummary;

	/** All compile events (errors, warnings, log messages) from the last compile of this script. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	TArray<FNiagaraExt_CompileEvent> CompileEvents;
};

/**
 * Aggregate compile state for a Niagara System, including per-script breakdown.
 * bIsCompiling and bIsStale may both be true simultaneously when a compile is in flight.
 * When bIsCompiling is true, the returned data reflects the last completed compile and may be stale.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_SystemCompileState
{
	GENERATED_BODY()

	/** Aggregate compile status across all scripts (worst-case union). */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	ENiagaraExt_ScriptCompileStatus AggregateStatus = ENiagaraExt_ScriptCompileStatus::Unknown;

	/** True if the system currently has active or pending compile requests. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	bool bIsCompiling = false;

	/** True if bIsCompiling is true or any per-script status is Dirty. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	bool bIsStale = false;

	/** True if any script status is Error or any CompileEvent has Severity == Error. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	bool bHasErrors = false;

	/** True if any CompileEvent has Severity == Warning. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	bool bHasWarnings = false;

	/**
	 * Per-script compile info. Order: SystemSpawn, SystemUpdate, then each emitter's scripts
	 * in handle order (ParticleSpawn, ParticleUpdate, ParticleGPUCompute, event handlers,
	 * simulation stages — whatever the emitter has). EmitterSpawn/EmitterUpdate scripts are
	 * intentionally omitted because their bodies are inlined into SystemSpawn/SystemUpdate at
	 * compile time and they never carry their own compile status; any errors from their
	 * contents appear on the corresponding system-level script.
	 */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Compile")
	TArray<FNiagaraExt_ScriptCompileInfo> Scripts;
};

// Note: FNiagaraExt_StackIssueFix, FNiagaraExt_StackIssue, and FNiagaraExt_StackIssues are declared
// after FNiagaraExt_StackItemReference below (they reference it as a UPROPERTY field type).

/** Schema describing a Niagara System asset's top level properties. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_SystemSchema
{
	GENERATED_BODY()

	/** Json Schema describing editable properties of a Niagara System. */
	UPROPERTY(EditAnywhere, Category="Schema")
	FString PropertySchema;
};

/** Schema describing a Niagara Emitter asset's top level properties. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_EmitterSchema
{
	GENERATED_BODY()

	/** Json Schema describing editable properties of a Niagara Emitter. */
	UPROPERTY(EditAnywhere, Category="Schema")
	FString PropertySchema;
};

/** Schema describing a Niagara Renderer. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_RendererSchema
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Schema")
	TSubclassOf<UNiagaraRendererProperties> RendererClass;

	/** Json Schema describing editable properties of a the Niagara Renderer class. */
	UPROPERTY(EditAnywhere, Category="Schema")
	FString PropertySchema;
};

/** Schema describing a Niagara Data Interface. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_DataInterfaceSchema
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Schema")
	TSubclassOf<UNiagaraDataInterface> DataInterfaceClass;

	/** Json Schema describing editable properties of a the data interface class. */
	UPROPERTY(EditAnywhere, Category="Schema")
	FString PropertySchema;
};

/** Schema describing a single Niagara Module input. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackInputSchema
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Schema")
	FName Name;

 	UPROPERTY()
 	FNiagaraTypeDefinition Type;

	UPROPERTY(EditAnywhere, Category="Schema")
	FText Category;

	/** True if this input can be set with a hlsl expression. */
	UPROPERTY(EditAnywhere, Category="Schema")
	bool bSupportsExpressions = false;

	UPROPERTY(EditAnywhere, Category="Schema")
	FNiagaraVariableMetaData MetaData;
};

/** Schema describing a Niagara Module asset and its inputs. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_ModuleSchema
{
	GENERATED_BODY()

	/** The script asset this schema describes. Read-only — populated by GetModuleSchema variants. */
	UPROPERTY(EditAnywhere, Category="Schema")
	TObjectPtr<const UNiagaraScript> Asset;

	UPROPERTY(EditAnywhere, Category="Schema")
	TArray<FNiagaraExt_StackInputSchema> Inputs;

	UPROPERTY(EditAnywhere, Category="Schema")
	TArray<FNiagaraExt_Variable> Outputs;
};

/** Schema describing a Niagara Dynamic Input asset. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_DynamicInputSchema : public FNiagaraExt_ModuleSchema
{
	GENERATED_BODY()
};

//////////////////////////////////////////////////////////////////////////
// Data Layer - Defines the actual values of object properties and stack module inputs.

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_SystemData
{
	GENERATED_BODY()

	/** Json values corresponding to the schema for a Niagara System. */
	UPROPERTY(EditAnywhere, Category="Data")
	FString PropertyValues;
};

/**
 * Emitter property data: a single JSON-string blob (PropertyValues) containing the full
 * FVersionedNiagaraEmitterData serialisation — Name, bIsEnabled, SimTarget, bLocalSpace,
 * RandomSeed, FixedBounds, CalculateBoundsMode, Platforms, ScalabilityOverrides, etc.
 *
 * This struct intentionally exposes one opaque field rather than a list of typed properties,
 * because the underlying property set evolves with engine versions and is large.
 *
 * For typed access to common metadata (SimTarget, EmitterName, bEnabled, RendererClasses)
 * prefer FNiagaraExt_EmitterSummary via GetEmitterSummary — it returns named fields directly
 * and avoids a JSON parse step. Use GetEmitterData when you need the full property set or
 * a less-common field that the summary doesn't expose.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_EmitterData
{
	GENERATED_BODY()

	/** JSON blob matching the emitter property schema. Fields use C++ PascalCase (SimTarget, RandomSeed, …). Parse to read individual fields. */
	UPROPERTY(EditAnywhere, Category="Data")
	FString PropertyValues;
};

/** Top level properties for a Niagara Module. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_ModuleData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Data")
	FName ModuleName;

	UPROPERTY(EditAnywhere, Category="Data")
	bool Enabled = true;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_RendererData
{
	GENERATED_BODY()

	/** Json values corresponding to the schema for a Niagara Renderer. */
	UPROPERTY(EditAnywhere, Category="Data")
	FString PropertyValues;
};

/**
 * Current value of a Niagara Stack Input. Instanced - the concrete variant is one of:
 * local literal (typed - float, bool, int32, Vector3f, LinearColor, etc.), linked binding,
 * HLSL expression, dynamic-input reference, data-interface reference, or enum wrapper.
 */
USTRUCT(BlueprintInternalUseOnly,
meta=(
BaseStruct="/Script/NiagaraEditor.NiagaraExt_StackInputData",
AdditionalTypes="/Script/Niagara.NiagaraFloat,/Script/Niagara.NiagaraBool,/Script/Niagara.NiagaraInt32,/Script/Niagara.NiagaraPosition,/Script/Niagara.NiagaraID,/Script/CoreUObject.Vector3f,/Script/CoreUObject.Vector2f,/Script/CoreUObject.Vector4f,/Script/CoreUObject.Quat4f,/Script/CoreUObject.LinearColor,/Script/CoreUObject.Rotator"
))
struct FNiagaraExt_StackInputValue : public FNiagaraExt_InstancedValue
{
	GENERATED_BODY()

	void InitStackInputFromValue(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExternalEditContext& Context)const;
	void InitFromStackInput(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExternalEditContext& Context);
};

/** Base struct for stack input data types. Excluded from schema - only derived types are valid. */
USTRUCT(BlueprintInternalUseOnly, meta=(EDAIncludeStructsInSchema, ExcludeBaseStruct))
struct FNiagaraExt_StackInputData
{
	GENERATED_BODY()
};

/** A special case of Local input that stores an Enum value as a string rather than it's raw int. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackInputData_Enum : public FNiagaraExt_StackInputData
{
	GENERATED_BODY()

	/** Reference to the UEnum type. */
	UPROPERTY(EditAnywhere, Category="Variable")
	TObjectPtr<UEnum> Enum;

	/** The selected enum name (e.g., \"MyEnum::Name1\"). */
	UPROPERTY(EditAnywhere, Category="Variable", meta=(ToolTip="The enum value name"))
	FName EnumName;

	/** Display value for EnumName. Often more descriptive and useful than the internal EnumName. For informational purposes only, the actual set value of the enum is EnumName. */
	UPROPERTY(EditAnywhere, Category="Variable")
	FText DisplayName;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackInputData_Linked : public FNiagaraExt_StackInputData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Input")
	FNiagaraExt_Variable LinkedVariable;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackInputData_HlslExpression : public FNiagaraExt_StackInputData
{
	GENERATED_BODY()

	/** A single rvalue hlsl expression defining the value of this input. MUST be a single rvalue with no variable definitions or return statements etc. For example a color could use the expression "float4(1.0, 0.0, 0.0, 1.0)" */
	UPROPERTY(EditAnywhere, Category="Input")
	FString HlslExpression;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackInputData_DataInterface : public FNiagaraExt_StackInputData
{
	GENERATED_BODY()

	/** Json values corresponding to the schema for this data interface. */
	UPROPERTY(EditAnywhere, Category="Input")
	FString PropertyValues;
};

/**
Stack input for a dynamic input.
*/
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackInputData_DynamicInput : public FNiagaraExt_StackInputData
{
	GENERATED_BODY()

	/** The dynamic input asset defining this dynamic input. */
	UPROPERTY(EditAnywhere, Category="Input")
	TObjectPtr<UNiagaraScript> DynamicInputAsset;
};

/** An unsupported input type */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackInputData_Unsupported : public FNiagaraExt_StackInputData
{
	GENERATED_BODY()
};


//////////////////////////////////////////////////////////////////////////
//Topology Layer - Defines how Niagara Systems, Emitters, Modules and their inputs are arranged.

/**
 * Topology-only description of a single stack input.
 * Contains structural metadata (name, type, visibility, editability).
 * No resolved value payload — call GetStackInputData for values.
 * If bIsDynamic is true, call GetDynamicInputChain to get the full topology and values for the chain.
 *
 * bIsVisible and bIsEditable combine three gating axes: the runtime hidden flag (set by static-
 * switch / conditional logic in the executing graph), VisibleCondition metadata, and EditCondition
 * metadata. Hidden or VisibleCondition-failing inputs are neither visible nor editable;
 * EditCondition-failing inputs are visible but not editable. SetStackInputData refuses writes
 * when bIsEditable is false.
 */
USTRUCT(BlueprintInternalUseOnly, meta=(EDAIncludeStructsInSchema))
struct FNiagaraExt_StackInputTopology : public FNiagaraExt_Variable
{
	GENERATED_BODY()

	/** True when visible in the stack UI. False if hidden by static-switch / conditional logic or by a VisibleCondition. */
	UPROPERTY(EditAnywhere, Category="Input")
	bool bIsVisible = true;

	/** True when the user can edit the value. Requires bIsVisible AND a passing EditCondition. SetStackInputData refuses writes when false. */
	UPROPERTY(EditAnywhere, Category="Input")
	bool bIsEditable = true;

	/** True when this input's value is provided by a dynamic input script. Call GetDynamicInputChain to retrieve the full chain topology and values. */
	UPROPERTY(EditAnywhere, Category="Input")
	bool bIsDynamic = false;

	/**
	 * True when this input is a static-switch parameter. Changing it recompiles the module and
	 * reshapes the visible/editable sub-input set — previously-hidden sub-inputs may become
	 * exposed, and vice versa.
	 */
	UPROPERTY(EditAnywhere, Category="Input")
	bool bIsStaticSwitch = false;
};

/**
 * A single resolved input value paired with its name.
 * Used within FNiagaraExt_ModuleInputValues to return all input values for a module in one call.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackInputValueEntry
{
	GENERATED_BODY()

	/** The input parameter name (matches the Name field in FNiagaraExt_StackInputTopology). */
	UPROPERTY(EditAnywhere, Category="Input")
	FName Name;

	/** The resolved value for this input. */
	UPROPERTY(EditAnywhere, Category="Input")
	FNiagaraExt_StackInputValue Value;
};

/**
 * All resolved input values for a single module.
 * Returned by GetModuleInputValues — one entry per input on the module.
 *
 * The Inputs array on this struct mirrors the Inputs array on FNiagaraExt_ModuleTopology
 * one-to-one: same names, same walk order. Pair this with topology when you need both
 * structure and values.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_ModuleInputValues
{
	GENERATED_BODY()

	/** The module name (matches ModuleName in FNiagaraExt_ModuleTopology). */
	UPROPERTY(EditAnywhere, Category="Data")
	FName ModuleName;

	/** One entry per input on this module, in walk order. Mirrors FNiagaraExt_ModuleTopology::Inputs. */
	UPROPERTY(EditAnywhere, Category="Data")
	TArray<FNiagaraExt_StackInputValueEntry> Inputs;
};

// Forward declaration so FNiagaraExt_DynamicInputChainRef can name the payload type.
struct FNiagaraExt_DynamicInputChain;

/**
 * FInstancedStruct wrapper for FNiagaraExt_DynamicInputChain.
 * Declared before the payload struct to break the UHT self-reference restriction that prevents
 * TArray<FNiagaraExt_DynamicInputChain> from appearing inside FNiagaraExt_DynamicInputChain itself.
 * Use .Get() / .GetMutable() to access the wrapped chain entry.
 */
USTRUCT(BlueprintInternalUseOnly, meta=(BaseStruct="/Script/NiagaraEditor.NiagaraExt_DynamicInputChain"))
struct FNiagaraExt_DynamicInputChainRef : public FNiagaraExt_InstancedValue
{
	GENERATED_BODY()

	UE_API FNiagaraExt_DynamicInputChainRef();
	UE_API const FNiagaraExt_DynamicInputChain& Get() const;
	UE_API FNiagaraExt_DynamicInputChain& GetMutable();
};

/**
 * A single entry in a dynamic-input chain, carrying both topology metadata and the resolved value.
 * Nested child entries are accessible via Inputs (one per direct input of the dynamic input script).
 *
 * The FInstancedStruct wrapper pattern (FNiagaraExt_DynamicInputChainRef declared before this struct)
 * avoids the UHT "struct recursion via arrays" restriction that blocks direct self-reference.
 * The schema generator detects the self-reference and emits a typed stub at the recursion boundary;
 * the wire format itself recurses to arbitrary depth.
 */
USTRUCT(BlueprintInternalUseOnly, meta=(EDAIncludeStructsInSchema))
struct FNiagaraExt_DynamicInputChain
{
	GENERATED_BODY()

	/** Input parameter name. */
	UPROPERTY(EditAnywhere, Category="Chain")
	FName Name;

	/** Niagara type definition for this input. */
	UPROPERTY()
	FNiagaraTypeDefinition Type;

	/** True when visible in the stack UI. False if hidden by static-switch / conditional logic or by a VisibleCondition. */
	UPROPERTY(EditAnywhere, Category="Chain")
	bool bIsVisible = true;

	/** True when the user can edit the value. Requires bIsVisible AND a passing EditCondition. SetStackInputData refuses writes when false. */
	UPROPERTY(EditAnywhere, Category="Chain")
	bool bIsEditable = true;

	/**
	 * True when this input is a static-switch parameter of the dynamic input script. Same semantics
	 * as FNiagaraExt_StackInputTopology::bIsStaticSwitch — changing it recompiles the chain level
	 * and reshapes which sibling inputs are visible/editable.
	 */
	UPROPERTY(EditAnywhere, Category="Chain")
	bool bIsStaticSwitch = false;

	/** Resolved value for this input. */
	UPROPERTY(EditAnywhere, Category="Chain")
	FNiagaraExt_StackInputValue Value;

	/** Direct inputs of the dynamic input script at this level, each of which may itself be dynamic. */
	UPROPERTY(EditAnywhere, Category="Chain")
	TArray<FNiagaraExt_DynamicInputChainRef> Inputs;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_ModuleTopology
{
	GENERATED_BODY()

	/** Use as ModuleName in FNiagaraExt_StackItemReference. */
	UPROPERTY(EditAnywhere, Category="Topology")
	FName ModuleName;

	/** Disabled modules stay in the stack but don't execute. */
	UPROPERTY(EditAnywhere, Category="Topology")
	bool Enabled = true;

	UPROPERTY(EditAnywhere, Category="Topology")
	TObjectPtr<UNiagaraScript> ModuleScript;

	/** True if this module is a SetParameters (UNiagaraNodeAssignment) module rather than a regular script module. */
	UPROPERTY(EditAnywhere, Category="Topology")
	bool bIsSetParametersModule = false;

	/** All inputs on this module in walk order. Always populated. */
	UPROPERTY(EditAnywhere, Category="Topology")
	TArray<FNiagaraExt_StackInputTopology> Inputs;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_ScriptStackTopology
{
	GENERATED_BODY()

	/** Which of the six script stacks this represents: SystemSpawnScript, SystemUpdateScript, EmitterSpawnScript, EmitterUpdateScript, ParticleSpawnScript, ParticleUpdateScript. */
	UPROPERTY(EditAnywhere, Category="Topology")
	FName ScriptName;

	/** Modules in the stack, in execution order. Always populated. */
	UPROPERTY(EditAnywhere, Category="Topology")
	TArray<FNiagaraExt_ModuleTopology> Modules;
};

/**
 * Topology-only renderer reference: class and index only.
 * For resolved renderer property values call GetRendererData with the same reference.
 *
 * The RendererIndex field matches FNiagaraExt_StackItemReference::RendererIndex — assign it
 * directly when constructing a stack reference for renderer-targeted calls.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_RendererRef
{
	GENERATED_BODY()

	/** Index of this renderer within its owning emitter's renderer list. Pair with FNiagaraExt_StackItemReference::RendererIndex. */
	UPROPERTY(EditAnywhere, Category="Topology")
	int32 RendererIndex = INDEX_NONE;

	UPROPERTY(EditAnywhere, Category="Topology")
	TSubclassOf<UNiagaraRendererProperties> RendererClass;
};

/**
 * Lightweight metadata summary for a single emitter.
 * Contains only the fields that are cheap to read without a full child walk.
 * For structural detail (script stacks, module inputs) use GetEmitterTopology.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_EmitterSummary
{
	GENERATED_BODY()

	/** Use as EmitterName in FNiagaraExt_StackItemReference. */
	UPROPERTY(EditAnywhere, Category="Summary")
	FName EmitterName;

	/** Disabled emitters still exist but produce no particles. */
	UPROPERTY(EditAnywhere, Category="Summary")
	bool bEnabled = true;

	/** CPUSim or GPUComputeSim. */
	UPROPERTY(EditAnywhere, Category="Summary")
	ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim;

	/** Distinct renderer classes on this emitter (de-duplicated set). */
	UPROPERTY(EditAnywhere, Category="Summary")
	TArray<TSubclassOf<UNiagaraRendererProperties>> RendererClasses;
};

/**
 * Lightweight metadata summary for the whole system.
 * Contains system name, user variables, and one FNiagaraExt_EmitterSummary per emitter.
 * For structural detail use GetEmitterTopology per emitter.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_SystemSummary
{
	GENERATED_BODY()

	/** The system asset name. */
	UPROPERTY(EditAnywhere, Category="Summary")
	FName SystemName;

	/** User-exposed parameters on the system (User.* namespace). */
	UPROPERTY(EditAnywhere, Category="Summary")
	TArray<FNiagaraExt_UserVariable> UserVariables;

	/** One entry per emitter, in handle order. */
	UPROPERTY(EditAnywhere, Category="Summary")
	TArray<FNiagaraExt_EmitterSummary> Emitters;
};

/**
 * Describes a Niagara Emitter's full structure (topology only — no resolved input values).
 * All script stacks and renderer references are always populated.
 * For emitter-level property values call GetEmitterData.
 * For per-module resolved input values call GetEmitterInputValues.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_EmitterTopology
{
	GENERATED_BODY()

	/** Use as EmitterName in FNiagaraExt_StackItemReference. */
	UPROPERTY(EditAnywhere, Category="Topology")
	FName EmitterName;

	/** Disabled emitters still exist but produce no particles. */
	UPROPERTY(EditAnywhere, Category="Topology")
	bool bEnabled = true;

	/** CPUSim or GPUComputeSim. Affects which Data Interfaces and features are available. */
	UPROPERTY(EditAnywhere, Category="Topology")
	ENiagaraSimTarget SimTarget = ENiagaraSimTarget::CPUSim;

	/** Distinct renderer classes on this emitter (de-duplicated set). */
	UPROPERTY(EditAnywhere, Category="Topology")
	TArray<TSubclassOf<UNiagaraRendererProperties>> RendererClasses;

	/** Emitter Spawn stack: modules run once when the emitter is created. */
	UPROPERTY(EditAnywhere, Category="Topology")
	FNiagaraExt_ScriptStackTopology EmitterSpawnScript;

	/** Emitter Update stack: modules run once per frame at emitter scope (spawn rates, bursts, lifecycle). */
	UPROPERTY(EditAnywhere, Category="Topology")
	FNiagaraExt_ScriptStackTopology EmitterUpdateScript;

	/** Particle Spawn stack: modules run once per newly-spawned particle. */
	UPROPERTY(EditAnywhere, Category="Topology")
	FNiagaraExt_ScriptStackTopology ParticleSpawnScript;

	/** Particle Update stack: modules run once per particle per frame (forces, velocity, colour-over-life). */
	UPROPERTY(EditAnywhere, Category="Topology")
	FNiagaraExt_ScriptStackTopology ParticleUpdateScript;

	/** Renderer references (class + index). For property values call GetRendererData. */
	UPROPERTY(EditAnywhere, Category="Topology")
	TArray<FNiagaraExt_RendererRef> Renderers;
};

/**
 * Dependency rollup for a Niagara System.
 * Contains the four distinct asset sets used across all emitters.
 * Returned by GetSystemDependencies; not embedded in topology structs.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_SystemDependencies
{
	GENERATED_BODY()

	/** Distinct renderer classes used across all emitters in this system. */
	UPROPERTY(EditAnywhere, Category="Dependencies")
	TArray<TSubclassOf<UNiagaraRendererProperties>> UsedRenderers;

	/** Distinct Data Interface classes referenced by any module or dynamic input in this system. */
	UPROPERTY(EditAnywhere, Category="Dependencies")
	TArray<TSubclassOf<UNiagaraDataInterface>> UsedDataInterfaces;

	/** Distinct module script assets used across all emitters. */
	UPROPERTY(EditAnywhere, Category="Dependencies")
	TArray<TObjectPtr<UNiagaraScript>> UsedModules;

	/** Distinct dynamic-input script assets used across all module inputs. */
	UPROPERTY(EditAnywhere, Category="Dependencies")
	TArray<TObjectPtr<UNiagaraScript>> UsedDynamicInputs;
};

//////////////////////////////////////////////////////////////////////////

struct FNiagaraEmitterHandle;
class UNiagaraStackScriptItemGroup;
class UNiagaraStackModuleItem;
class UNiagaraStackFunctionInput;
class UNiagaraRendererProperties;

//A reference to an item inside a Niagara System such as an emitter, script, module, input or renderer.
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackItemReference
{
	GENERATED_BODY()

	FNiagaraExt_StackItemReference() = default;
	FNiagaraExt_StackItemReference(UNiagaraSystem* InSystem, FName InEmitterName=NAME_None, FName InScriptName=NAME_None, FName InModuleName=NAME_None)
		:System(InSystem), EmitterName(InEmitterName), ScriptName(InScriptName), ModuleName(InModuleName)
	{
	}

	// Copy constructor and assignment clear cached UObject pointers so that a copy with a
	// different InputNameStack always re-resolves rather than returning a stale cached input.
	FNiagaraExt_StackItemReference(const FNiagaraExt_StackItemReference& Other)
		: System(Other.System), EmitterName(Other.EmitterName), ScriptName(Other.ScriptName)
		, ModuleName(Other.ModuleName), RendererIndex(Other.RendererIndex), InputNameStack(Other.InputNameStack)
	{
		// Intentionally do NOT copy CachedScript, CachedModule, CachedInput.
	}
	FNiagaraExt_StackItemReference& operator=(const FNiagaraExt_StackItemReference& Other)
	{
		if (this != &Other)
		{
			System = Other.System;
			EmitterName = Other.EmitterName;
			ScriptName = Other.ScriptName;
			ModuleName = Other.ModuleName;
			RendererIndex = Other.RendererIndex;
			InputNameStack = Other.InputNameStack;
			// Intentionally do NOT copy CachedScript, CachedModule, CachedInput.
			CachedScript = nullptr;
			CachedModule = nullptr;
			CachedInput = nullptr;
		}
		return *this;
	}

	UPROPERTY(EditAnywhere, Category="Stack Item")
	TObjectPtr<UNiagaraSystem> System;

	UPROPERTY(EditAnywhere, Category="Stack Item")
	FName EmitterName;

	UPROPERTY(EditAnywhere, Category="Stack Item")
	FName ScriptName;

	/** Name of the module to reference. Requires a valid ScriptName. */
	UPROPERTY(EditAnywhere, Category="Stack Item")
	FName ModuleName;

	/** Index of a renderer to reference. Requires a valid EmitterName. */
	UPROPERTY(EditAnywhere, Category="Stack Item")
	int32 RendererIndex = INDEX_NONE;

	/** Input Name stack. Requires a valid ScriptName and ModuleName. For a top level module input this will be a single FName. For a dynamic input this could be a chain of several nested dynamic inputs. */
	UPROPERTY(EditAnywhere, Category="Stack Item")
	TArray<FName> InputNameStack;

	UE_API UNiagaraSystem* GetSystem(FNiagaraExternalEditContext& Context)const;
	FNiagaraEmitterHandle* GetEmitter(FNiagaraExternalEditContext& Context, bool bRequired=true)const;
	UNiagaraStackScriptItemGroup* GetScript(FNiagaraExternalEditContext& Context, bool bRequired=true)const;
	UNiagaraStackModuleItem* GetModule(FNiagaraExternalEditContext& Context, bool bRequired=true)const;
	UNiagaraStackFunctionInput* GetInput(FNiagaraExternalEditContext& Context, bool bRequired=true)const;
	UNiagaraRendererProperties* GetRenderer(FNiagaraExternalEditContext& Context, bool bRequired=true)const;

	void SetEmitter(FNiagaraEmitterHandle* Emitter);
	void SetEmitter(FName InEmitterName);

	void SetScript(UNiagaraStackScriptItemGroup* Script);
	void SetScript(FName InScriptName);

	void SetModule(UNiagaraStackModuleItem* Module);
	void SetModule(FName InModuleName);

	void SetInput(UNiagaraStackFunctionInput* Input);
	void SetInput(FName Name);

	void PushInput(UNiagaraStackFunctionInput* Input);
	void PushInput(FName Name);

	void SetRenderer(int32 InRendererIndex);

private:

	// IMPORTANT: These caches are only valid for a single operation and should never be stored long-term.
	// Emitter is not cached to avoid dangling pointers into the emitter handles array.
	// UObject pointers use weak references to detect stale objects.
	mutable TWeakObjectPtr<UNiagaraStackScriptItemGroup> CachedScript = nullptr;
	mutable TWeakObjectPtr<UNiagaraStackModuleItem> CachedModule = nullptr;
	mutable TWeakObjectPtr<UNiagaraStackFunctionInput> CachedInput = nullptr;
};

//////////////////////////////////////////////////////////////////////////
// Diagnostics Layer (Stack) - declared here because they reference FNiagaraExt_StackItemReference.

/**
 * A single auto-fix action associated with a stack issue.
 * Style == Fix means the action can be applied programmatically via ApplyStackIssueFix.
 * Style == Link is a navigation hint for a human user (e.g. "open this asset") and cannot
 * be triggered by tool calls; ApplyStackIssueFix will refuse and return bApplied == false.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackIssueFix
{
	GENERATED_BODY()

	/** Unique identifier for this fix, used as the FixId parameter in ApplyStackIssueFix. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	FString FixId;

	/** Human-readable description of what this fix does. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	FString Description;

	/**
	 * Fix = can be applied programmatically via ApplyStackIssueFix. You should prefer acting on
	 *     Fix-style entries yourself rather than surfacing them to a human.
	 * Link = navigation hint intended for a human user. Cannot be applied by tool calls —
	 *     ApplyStackIssueFix will reject Link-style fixes.
	 */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	ENiagaraExt_StackIssueFixStyle Style = ENiagaraExt_StackIssueFixStyle::Fix;
};

/**
 * A single warning or error issue on a Niagara stack entry, as reported by the stack validator.
 * Location and StackDisplayPath help the AI identify where the issue is in the stack.
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackIssue
{
	GENERATED_BODY()

	/** Unique identifier for this issue, used as the IssueId parameter in ApplyStackIssueFix. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	FString IssueId;

	/** Severity of this issue. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	ENiagaraExt_StackIssueSeverity Severity = ENiagaraExt_StackIssueSeverity::None;

	/** Short description of the issue (e.g. "Missing required input"). */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	FString ShortDescription;

	/** Detailed description of the issue with context. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	FString LongDescription;

	/** True if the user is able to dismiss this issue in the UI. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	bool bCanBeDismissed = false;

	/** True if this issue has been dismissed by the user and is hidden in the UI. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	bool bIsDismissed = false;

	/**
	 * Structured reference to the emitter/script/module that owns this issue.
	 * EmitterName, ScriptName, and ModuleName may be NAME_None if the issue is above that level.
	 */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	FNiagaraExt_StackItemReference Location;

	/**
	 * Human-readable breadcrumb path from the stack root to the entry that owns this issue,
	 * joined with '/'. Example: "FireEmitter/ParticleUpdate/AddVelocity".
	 */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	FString StackDisplayPath;

	/** Available auto-fix actions for this issue. May be empty. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	TArray<FNiagaraExt_StackIssueFix> Fixes;
};

/**
 * Container for all stack issues across a Niagara System.
 * Counter invariant: NumErrors + NumWarnings + NumInfos <= Issues.Num()
 * (issues with Severity == None are not counted in any counter).
 */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_StackIssues
{
	GENERATED_BODY()

	/** Count of issues with Severity == Error. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	int32 NumErrors = 0;

	/** Count of issues with Severity == Warning. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	int32 NumWarnings = 0;

	/** Count of issues with Severity == Info. */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	int32 NumInfos = 0;

	/** All stack issues across the system (both dismissed and undismissed). */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	TArray<FNiagaraExt_StackIssue> Issues;
};

/** Return value for ApplyStackIssueFix: bApplied + AppliedFixDescription report what ran. */
USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExt_ApplyStackIssueFixResult
{
	GENERATED_BODY()

	/**
	 * True if the fix delegate executed. False on any error path (system null, IssueId or FixId
	 * not found, ambiguous match, Link-style fix, or unbound delegate). When false, inspect the
	 * toolset context's error list for the specific reason.
	 */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	bool bApplied = false;

	/**
	 * Human-readable description of the fix that ran, copied from FNiagaraExt_StackIssueFix::Description.
	 * Empty when bApplied is false.
	 */
	UPROPERTY(EditAnywhere, Category="Diagnostics|Stack")
	FString AppliedFixDescription;
};

USTRUCT(BlueprintInternalUseOnly)
struct FNiagaraExternalEditContext
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Context")
	TArray<FText> Errors;

	FNiagaraExternalEditContext() = default;
	UE_API FNiagaraExternalEditContext(UNiagaraSystem* InSystem);
	UE_API FNiagaraExternalEditContext(const FNiagaraExt_StackItemReference& ItemRef);
		UNiagaraSystem* GetSystem();
	TSharedPtr<FNiagaraSystemViewModel> GetSystemViewModel();

	/**
	 * Returns a lazily-created non-data-only VM for diagnostic reads (stack issues). Distinct from
	 * the primary SystemViewModel because data-only VMs intentionally clear their stack-issue arrays.
	 * The two VMs coexist for the context's lifetime so items resolved through the primary VM
	 * (e.g. cached UNiagaraStackModuleItem pointers on FNiagaraExt_StackItemReference) remain valid.
	 */
	UE_API TSharedPtr<FNiagaraSystemViewModel> GetDiagnosticsSystemViewModel();

	// Dependency accumulators — populated as side-effects of the dependency walk.
	// Only meaningful when GetSystemDependencies is called; not used by topology walks.
	TSet<TSubclassOf<UNiagaraRendererProperties>> UsedRenderers;
	TSet<TSubclassOf<UNiagaraDataInterface>> UsedDataInterfaces;
	TSet<UNiagaraScript*> UsedModules;
	TSet<UNiagaraScript*> UsedDynamicInputs;

	bool HasErrors()const { return Errors.Num() > 0; }

	void Error(FText InError)
	{
		Errors.Emplace(InError);
	}

	bool CheckSystem(const UNiagaraSystem* System);
	bool CheckScriptAsset(const UNiagaraScript* ScriptAsset);

private:

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
	TSharedPtr<FNiagaraSystemViewModel> DiagnosticsSystemViewModel;
};

/**
* A C++ and Blueprint accessible library of functions allowing for the inspection and modification of Niagara Systems.
*/
UCLASS(MinimalAPI)
class UNiagaraExternalEditUtilities : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Creates a new Niagara System asset.
	 *
	 * @param AssetName - Name for the new system asset (required)
	 * @param AssetPath - Package path where the asset will be created (required)
	 * @param TemplateSystem - Optional template system to copy. If null, an empty system will be created.
	 * @param Context - Edit context for error reporting
	 * @return The newly created system, or nullptr if creation failed
	 */
	UE_API static UNiagaraSystem* CreateNiagaraSystem(const FString& AssetName, const FString& AssetPath, UNiagaraSystem* TemplateSystem, FNiagaraExternalEditContext& Context);
	UE_API static void GetAvailableDynamicInputs(const FNiagaraTypeDefinition& Type, TArray<UNiagaraScript*>& OutDynamicInputScripts, FNiagaraExternalEditContext& Context);

	// Override the installable provider (see FNiagaraExternalEditPropertyProvider). Process-wide state with no
	// locking; install and reset only at module startup/shutdown.
	UE_API static void SetPropertyProvider(const FNiagaraExternalEditPropertyProvider& Provider);
	UE_API static void ResetPropertyProvider();
	UE_API static const FNiagaraExternalEditPropertyProvider& GetPropertyProvider();


	//Schemas Layer - Defines constant data about editable properties and inputs based on specific classes or assets rather than a specific System being edited.
	UE_API static void GetSystemSchema(FNiagaraExt_SystemSchema& OutSchema);
	UE_API static void GetEmitterSchema(FNiagaraExt_EmitterSchema& OutSchema);
	UE_API static void GetRendererSchema(TSubclassOf<UNiagaraRendererProperties> RendererClass, FNiagaraExt_RendererSchema& OutSchema);
	UE_API static void GetDataInterfaceSchema(TSubclassOf<UNiagaraDataInterface> DataInterfaceClass, FNiagaraExt_DataInterfaceSchema& OutSchema);
 	UE_API static void GetStackInputSchema(const FNiagaraExt_StackItemReference& InputReference, FNiagaraExt_StackInputSchema& OutSchema, FNiagaraExternalEditContext& Context);
 	UE_API static void GetModuleSchema(const FNiagaraExt_StackItemReference& ModuleReference, FNiagaraExt_ModuleSchema& OutSchema, FNiagaraExternalEditContext& Context);
	UE_API static void GetDynamicInputSchema(const FNiagaraExt_StackItemReference& DynamicInputReference, FNiagaraExt_DynamicInputSchema& OutSchema, FNiagaraExternalEditContext& Context);

	//Ideally we'll be able to inspect these without the need for an owning system but that isn't currently possible.
// 	static void GetStackInputSchema(const TObjectPtr<UNiagaraScript> ModuleAsset, FNiagaraExt_Variable& Input, FNiagaraExt_StackInputSchema& OutSchema);
 	UE_API static void GetModuleSchema(const UNiagaraScript* ModuleAsset, FNiagaraExt_ModuleSchema& OutSchema, FNiagaraExternalEditContext& Context);
 	UE_API static void GetDynamicInputSchema(const UNiagaraScript* ModuleAsset, FNiagaraExt_DynamicInputSchema& OutSchema, FNiagaraExternalEditContext& Context);

	//Schema Layer END

	// --- Summary tier --------------------------------------------------------
	// Cheap metadata-only calls. No child walk, no dependency sets.

	/** Returns system name, user variables, and one FNiagaraExt_EmitterSummary per emitter. */
	UE_API static void GetSystemSummary(UNiagaraSystem* System, FNiagaraExt_SystemSummary& OutSummary, FNiagaraExternalEditContext& Context);

	/** Returns emitter metadata: name, enabled, sim target, de-duplicated renderer classes. */
	UE_API static void GetEmitterSummary(const FNiagaraExt_StackItemReference& EmitterRef, FNiagaraExt_EmitterSummary& OutSummary, FNiagaraExternalEditContext& Context);

	// --- Topology tier -------------------------------------------------------
	// Full structural walk. Every field always populated. No resolved input values.

	/** Returns full emitter topology: four script stacks with all modules and inputs, renderer references. */
	UE_API static void GetEmitterTopology(const FNiagaraExt_StackItemReference& EmitterRef, FNiagaraExt_EmitterTopology& OutTopology, FNiagaraExternalEditContext& Context);

	/** Returns script stack topology: all modules and their inputs in execution order. */
	UE_API static void GetScriptStackTopology(const FNiagaraExt_StackItemReference& ScriptRef, FNiagaraExt_ScriptStackTopology& OutTopology, FNiagaraExternalEditContext& Context);

	/** Returns module topology: metadata and all inputs (name/type/visibility only, no values). */
	UE_API static void GetModuleTopology(const FNiagaraExt_StackItemReference& ModuleRef, FNiagaraExt_ModuleTopology& OutTopology, FNiagaraExternalEditContext& Context);

	/** Returns stack input topology: name, type, visibility, editability. No value payload. */
	UE_API static void GetStackInputTopology(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExt_StackInputTopology& OutTopology, FNiagaraExternalEditContext& Context);

	// --- Data tier (scope-level) ---------------------------------------------
	// Resolved input values across all modules in the given scope.

	/** Returns all resolved input values for every module across all four emitter script stacks. */
	UE_API static void GetEmitterInputValues(const FNiagaraExt_StackItemReference& EmitterRef, TArray<FNiagaraExt_ModuleInputValues>& OutValues, FNiagaraExternalEditContext& Context);

	/** Returns all resolved input values for every module in the given script stack. */
	UE_API static void GetScriptStackInputValues(const FNiagaraExt_StackItemReference& ScriptRef, TArray<FNiagaraExt_ModuleInputValues>& OutValues, FNiagaraExternalEditContext& Context);

	/** Returns resolved input values for a single module. */
	UE_API static void GetModuleInputValues(const FNiagaraExt_StackItemReference& ModuleRef, FNiagaraExt_ModuleInputValues& OutValues, FNiagaraExternalEditContext& Context);

	// --- Dynamic-input chain endpoint ----------------------------------------

	/**
	 * Returns the full recursive chain for a dynamic input: topology metadata and resolved values at every level.
	 * The starting input must have value mode Dynamic; surfaces an error otherwise.
	 * The returned FNiagaraExt_DynamicInputChainRef wraps the root chain entry; nested entries are
	 * accessible via Inputs (FInstancedStruct wrapper pattern required for UHT-safe self-reference).
	 */
	UE_API static void GetDynamicInputChain(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExt_DynamicInputChainRef& OutChain, FNiagaraExternalEditContext& Context);

	// --- Dependency endpoint -------------------------------------------------

	/** Returns the four Used* sets (renderers, data interfaces, modules, dynamic inputs) gathered across all emitters. */
	UE_API static void GetSystemDependencies(UNiagaraSystem* System, FNiagaraExt_SystemDependencies& OutDependencies, FNiagaraExternalEditContext& Context);

	//Topology Layer END

public:
	//Data - Access actual data values of Systems, Emitters and module inputs.
	UE_API static void GetUserVariables(UNiagaraSystem* System, FNiagaraExt_UserVariables& OutVariables, FNiagaraExternalEditContext& Context);
	UE_API static void GetSystemData(UNiagaraSystem* System, FNiagaraExt_SystemData& OutData, FNiagaraExternalEditContext& Context);
	UE_API static void GetEmitterData(const FNiagaraExt_StackItemReference& EmitterRef, FNiagaraExt_EmitterData& OutData, FNiagaraExternalEditContext& Context);
	UE_API static void GetRendererData(const FNiagaraExt_StackItemReference& RendererRef, FNiagaraExt_RendererData& OutData, FNiagaraExternalEditContext& Context);
	UE_API static void GetStackInputData(const FNiagaraExt_StackItemReference& StackInputRef, FNiagaraExt_StackInputValue& OutData, FNiagaraExternalEditContext& Context);

	// Diagnostics Layer - Read compile state and stack issues.

	/**
	 * Reads the aggregate and per-script compile state of a Niagara System without triggering a
	 * recompile. When bIsCompiling is true in OutState, the data reflects the last completed compile.
	 * Never blocks or flushes compile requests.
	 */
	UE_API static void GetSystemCompileState(
		UNiagaraSystem* System,
		FNiagaraExt_SystemCompileState& OutState,
		FNiagaraExternalEditContext& Context);

	/**
	 * Collects all stack issues (errors, warnings, info) from the system and emitter stacks.
	 * Includes dismissed issues.
	 *
	 * Preconditions: System is non-null and no compile is in flight. Errors on Context if either
	 * precondition is violated.
	 */
	UE_API static void GetStackIssues(
		UNiagaraSystem* System,
		FNiagaraExt_StackIssues& OutIssues,
		FNiagaraExternalEditContext& Context);

	/**
	 * Re-locates a live stack issue/fix pair by IssueId+FixId and executes the Fix-style delegate
	 * inside a scoped transaction.
	 *
	 * Preconditions: System is non-null and no compile is in flight. Errors on Context (without
	 * applying) if either precondition is violated, if IssueId is not found, if FixId is not
	 * found, if the pair is ambiguous, or if the fix style is Link.
	 *
	 * Fills OutResult.bApplied and OutResult.AppliedFixDescription on success.
	 */
	UE_API static void ApplyStackIssueFix(
		UNiagaraSystem* System,
		const FString& IssueId,
		const FString& FixId,
		FNiagaraExt_ApplyStackIssueFixResult& OutResult,
		FNiagaraExternalEditContext& Context);

	// Diagnostics Layer END

	//Edit Operations
	UE_API static void AddUserVariable(UNiagaraSystem* System, const FNiagaraExt_UserVariable& Variable, FNiagaraExternalEditContext& Context);
	UE_API static void RemoveUserVariable(UNiagaraSystem* System, const FNiagaraExt_Variable& Variable, FNiagaraExternalEditContext& Context);

	UE_API static void AddEmitter(UNiagaraEmitter* TemplateEmitter, FName EmitterName, FNiagaraExt_EmitterTopology& OutTopology, FNiagaraExternalEditContext& Context);
 	UE_API static void RemoveEmitter(const FNiagaraExt_StackItemReference& EmitterToRemoveRef, FNiagaraExternalEditContext& Context);

	UE_API static void AddRenderer(const FNiagaraExt_StackItemReference& NewRendererLocation, const TSubclassOf<UNiagaraRendererProperties> RendererClass, FNiagaraExt_RendererRef& OutRef, FNiagaraExternalEditContext& Context);
	UE_API static void RemoveRenderer(const FNiagaraExt_StackItemReference& RendererToRemove, FNiagaraExternalEditContext& Context);

	UE_API static void AddModule(const FNiagaraExt_StackItemReference& NewModuleLocationRef, const UNiagaraScript* ModuleAsset, FNiagaraExt_ModuleTopology& OutTopology, FNiagaraExternalEditContext& Context);
	UE_API static void AddSetParametersModule(const FNiagaraExt_StackItemReference& NewModuleLocationRef, const TArray<FNiagaraExt_SetParameterEntry>& Parameters, FNiagaraExt_ModuleTopology& OutTopology, FNiagaraExternalEditContext& Context);
	UE_API static void AddSetParameterEntry(const FNiagaraExt_StackItemReference& ModuleRef, const FNiagaraExt_SetParameterEntry& Entry, FNiagaraExt_ModuleTopology& OutTopology, FNiagaraExternalEditContext& Context);
	UE_API static void RemoveSetParameterEntry(const FNiagaraExt_StackItemReference& ModuleRef, FName ParameterName, FNiagaraExternalEditContext& Context);
	UE_API static void RemoveModule(const FNiagaraExt_StackItemReference& ModuleToRemove, FNiagaraExternalEditContext& Context);
	UE_API static void SetModuleEnabled(const FNiagaraExt_StackItemReference& ModuleRef, bool bEnabled, FNiagaraExternalEditContext& Context);

	UE_API static void SetSystemData(UNiagaraSystem* System, const FNiagaraExt_SystemData& InData, FNiagaraExternalEditContext& Context);
	UE_API static void SetEmitterData(const FNiagaraExt_StackItemReference& EmitterItemRef, const FNiagaraExt_EmitterData& InData, FNiagaraExternalEditContext& Context);
	UE_API static void SetRendererData(const FNiagaraExt_StackItemReference& RendererItemRef, const FNiagaraExt_RendererData& InData, FNiagaraExternalEditContext& Context);
	UE_API static void SetStackInputData(const FNiagaraExt_StackItemReference& StackItemRef, const FNiagaraExt_StackInputValue& InData, FNiagaraExternalEditContext& Context);

private:
	static void GetEmitterDataInternal(UNiagaraSystem* System, FNiagaraEmitterHandle& Emitter, FNiagaraExt_EmitterData& OutData, FNiagaraExternalEditContext& Context);
	static void GetRendererDataInternal(FNiagaraEmitterHandle& Emitter, UNiagaraRendererProperties* Renderer, FNiagaraExt_RendererData& OutData, FNiagaraExternalEditContext& Context);

	/** Enumerates all compilable scripts in the system in canonical order and populates OutScripts. */
	static void EnumerateSystemScripts(
		UNiagaraSystem* System,
		TArray<FNiagaraExt_ScriptCompileInfo>& OutScripts);

	/** Walks all stack view models and collects issues into OutIssues, including dismissed ones. */
	static void CollectStackIssues(
		FNiagaraSystemViewModel& SystemVM,
		UNiagaraSystem* System,
		FNiagaraExt_StackIssues& OutIssues);

	// Internal helpers for walk recursion
	static void WalkScriptStackForInputValues(const FNiagaraExt_StackItemReference& ScriptRef, TArray<FNiagaraExt_ModuleInputValues>& OutValues, FNiagaraExternalEditContext& Context);

	// Internal helper for dynamic-input chain walk
	static void WalkDynamicInputChain(UNiagaraStackFunctionInput* Input, const FNiagaraExt_StackItemReference& InputRef, FNiagaraExt_DynamicInputChainRef& OutChain, FNiagaraExternalEditContext& Context);

	// Builds parameter map histories for a script allowing some inspection of script schemas without needing an owning system.
	static bool BuildParameterMapHistoriesFromScript(const UNiagaraScript* Script, TArray<FNiagaraParameterMapHistory>& OutHistories, FNiagaraExternalEditContext& Context);
};

#undef UE_API
