// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "EdGraph/EdGraph.h"
#include "NiagaraScript.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraVariableMetaData.h"
#include "StructUtils/InstancedStruct.h"
#include "NiagaraGraph.generated.h"

class UNiagaraNodeStaticSwitch;
class UNiagaraParameterDefinitions;
class UNiagaraScriptVariable;
class UHierarchyRoot;
struct FNiagaraCompileHashVisitorDebugInfo;
struct FNiagaraScriptVariableData;
struct FNiagaraStaticVariableSearchContext;
struct FSynchronizeWithParameterDefinitionsArgs;

/** This is the type of action that occurred on a given Niagara graph. Note that this should follow from EEdGraphActionType, leaving some slop for growth. */
enum ENiagaraGraphActionType
{
	GRAPHACTION_GenericNeedsRecompile = 0x1 << 16,
};

struct FInputPinsAndOutputPins
{
	TArray<UEdGraphPin*> InputPins;
	TArray<UEdGraphPin*> OutputPins;
};

class UNiagaraNode;
class UNiagaraGraph;

USTRUCT()
struct FNiagaraGraphParameterReference
{
	GENERATED_USTRUCT_BODY()
public:
	FNiagaraGraphParameterReference() {}
	FNiagaraGraphParameterReference(const FGuid& InKey, UObject* InValue): Key(InKey), Value(InValue){}

	UPROPERTY()
	FGuid Key;

	UPROPERTY()
	TWeakObjectPtr<UObject> Value;

	// If false then it is just a technical reference (e.g. setting the default value)
	UPROPERTY()
	bool bIsUserFacing = true;

	inline bool operator==(const FNiagaraGraphParameterReference& Other)const
	{
		return Other.Key == Key && Other.Value == Value;
	}
};

USTRUCT()
struct FNiagaraGraphParameterReferenceCollection
{

	GENERATED_USTRUCT_BODY()
public:
	FNiagaraGraphParameterReferenceCollection() = default;
	FNiagaraGraphParameterReferenceCollection(bool bInCreated);

	/** All the references in the graph. */
	UPROPERTY()
	TArray<FNiagaraGraphParameterReference> ParameterReferences;

	/** Returns true if this parameter was initially created by the user. */
	bool WasCreatedByUser() const;

private:
	/** Whether this parameter was initially created by the user. */
	UPROPERTY()
	bool bCreatedByUser = false;
};


/** Container for UNiagaraGraph cached data for managing CompileIds and Traversals.*/
USTRUCT()
struct FNiagaraGraphScriptUsageInfo
{
	GENERATED_USTRUCT_BODY()

public:
	FNiagaraGraphScriptUsageInfo();

	/** A guid which is generated when this usage info is created.  Allows for forced recompiling when the cached ids are invalidated. */
	UPROPERTY()
	FGuid BaseId;

	/** The context in which this sub-graph traversal will be used.*/
	UPROPERTY()
	ENiagaraScriptUsage UsageType;
	
	/** The particular instance of the usage type. Event scripts, for example, have potentially multiple graphs.*/
	UPROPERTY()
	FGuid UsageId;

	/** The hash that we calculated last traversal. */
	UPROPERTY()
	FNiagaraCompileHash CompileHash;

	/** The hash that we calculated last traversal. */
	UPROPERTY()
	FNiagaraCompileHash CompileHashFromGraph;

	/** The hash of values that could feed into downstream scripts.*/
	UPROPERTY()
	FNiagaraCompileHash ReferenceHashFromGraph;

	UPROPERTY(Transient)
	TArray<FNiagaraCompileHashVisitorDebugInfo> CompileLastObjects;


	/** The traversal of output to input nodes for this graph. This is not a recursive traversal, it just includes nodes from this graph.*/
	UPROPERTY()
	TArray<TObjectPtr<UNiagaraNode>> Traversal;

	void PostLoad(UObject* Owner);

	bool IsValid() const;

private:
	UPROPERTY()
	TArray<uint8> DataHash_DEPRECATED;

	UPROPERTY()
	FGuid GeneratedCompileId_DEPRECATED;
};

struct FNiagaraGraphFunctionAliasContext
{
	// the usage as defined in the compilation request (same for all translation stages)
	ENiagaraScriptUsage CompileUsage;

	// the usage as defined in the current translation stage
	ENiagaraScriptUsage ScriptUsage;
	
	void AppendPermutationString(FStringBuilderBase& StringBuilder) const;
	void AddPermutationIdentifier(const FName& Name, FStringView Value);
	void AddPendingStaticSwitchVariable(const FNiagaraVariable& Variable, uint8 SwitchType);

	struct FStaticSwitchVariable
	{
		FNiagaraVariable Variable;
		uint8 SwitchType;

		friend bool operator==(const FStaticSwitchVariable& Lhs, const FStaticSwitchVariable& Rhs)
		{
			return Lhs.Variable == Rhs.Variable && Lhs.SwitchType == Rhs.SwitchType;
		}
	};

	TConstArrayView<FStaticSwitchVariable> ReadPendingStaticSwitchVariables() const
	{
		return CompilerStaticSwitchVariables;
	}

private:
	// the set of identifiers from internal graph nodes that permute the function
	TMap<FName, FString> PermutationIdentifiers;

	TArray<FStaticSwitchVariable> CompilerStaticSwitchVariables;
};

/** Options for the FindInputNodes function */
struct FNiagaraFindInputNodeOptions
{
	FNiagaraFindInputNodeOptions()
		: bSort(false)
		, bIncludeParameters(true)
		, bIncludeAttributes(true)
		, bIncludeSystemConstants(true)
		, bIncludeTranslatorConstants(false)
		, bFilterDuplicates(false)
		, bFilterByScriptUsage(false)
		, TargetScriptUsage(ENiagaraScriptUsage::Function)
	{
	}

	/** Whether or not to sort the nodes, defaults to false. */
	bool bSort;
	/** Whether or not to include parameters, defaults to true. */
	bool bIncludeParameters;
	/** Whether or not to include attributes, defaults to true. */
	bool bIncludeAttributes;
	/** Whether or not to include system parameters, defaults to true. */
	bool bIncludeSystemConstants;
	/** Whether or not to include translator parameters, defaults to false. */
	bool bIncludeTranslatorConstants;
	/** Whether of not to filter out duplicate nodes, defaults to false. */
	bool bFilterDuplicates;
	/** Whether or not to limit to nodes connected to an output node of the specified script type.*/
	bool bFilterByScriptUsage;
	/** The specified script usage required for an input.*/
	ENiagaraScriptUsage TargetScriptUsage;
	/** The specified id within the graph of the script usage*/
	FGuid TargetScriptUsageId;
};

USTRUCT()
struct FNiagaraParametersChangedData
{
	GENERATED_BODY()
};

USTRUCT()
struct FNiagaraParameterCreatedData : public FNiagaraParametersChangedData
{
	GENERATED_BODY()

	FNiagaraParameterCreatedData() = default;
	explicit FNiagaraParameterCreatedData(UNiagaraScriptVariable* ScriptVariable) : NewScriptVariable(ScriptVariable) { }
	
	TWeakObjectPtr<const UNiagaraScriptVariable> NewScriptVariable;
};

USTRUCT()
struct FNiagaraParameterRenamedData : public FNiagaraParametersChangedData
{
	GENERATED_BODY()
	
	TWeakObjectPtr<const UNiagaraScriptVariable> OldScriptVariable;
	TWeakObjectPtr<const UNiagaraScriptVariable> NewScriptVariable;
};

UCLASS(MinimalAPI)
class UNiagaraGraph : public UEdGraph
{
	GENERATED_BODY()

public:		
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnParametersChanged, TOptional<TInstancedStruct<FNiagaraParametersChangedData>> ParametersChangedData)
	DECLARE_MULTICAST_DELEGATE(FOnDataInterfaceChanged);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSubObjectSelectionChanged, const UObject*);
	DECLARE_DELEGATE_RetVal_OneParam(TArray<UNiagaraParameterDefinitions*> /*AvailableParameterDefinitions*/, FOnGetParameterDefinitionsForDetailsCustomization, bool /*bSkipSubscribedParameterDefinitions*/)

	UNiagaraGraph();
	
	//~ Begin UObject Interface
	virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void BeginDestroy() override;
	//~ End UObjet Interface
	
	/** Get the source that owns this graph */
	class UNiagaraScriptSource* GetSource() const;

	/** Determine if there are any nodes in this graph.*/
	bool IsEmpty() const { return Nodes.Num() == 0; }

	/** Creates a transient copy of this graph for compilation purposes. */
	UNiagaraGraph* CreateCompilationCopy(const TArray<ENiagaraScriptUsage>& CompileUsages);
	void ReleaseCompilationCopy();

	/** Returns if this graph is for compilation purposes only. **/
	bool IsCompilationCopy() const { return bIsForCompilationOnly; };
			
	/** Find the first output node bound to the target usage type.*/
	class UNiagaraNodeOutput* FindOutputNode(ENiagaraScriptUsage TargetUsageType, FGuid TargetUsageId = FGuid()) const;
	NIAGARAEDITOR_API class UNiagaraNodeOutput* FindEquivalentOutputNode(ENiagaraScriptUsage TargetUsageType, FGuid TargetUsageId = FGuid()) const;

	/** Find all output nodes.*/
	void FindOutputNodes(TArray<UNiagaraNodeOutput*>& OutputNodes) const;
	void FindOutputNodes(ENiagaraScriptUsage TargetUsageType, TArray<UNiagaraNodeOutput*>& OutputNodes) const;
	void FindEquivalentOutputNodes(ENiagaraScriptUsage TargetUsageType, TArray<UNiagaraNodeOutput*>& OutputNodes) const;

	using FFindInputNodeOptions = FNiagaraFindInputNodeOptions;

	/** Finds input nodes in the graph with. */
	void FindInputNodes(TArray<class UNiagaraNodeInput*>& OutInputNodes, FFindInputNodeOptions Options = FFindInputNodeOptions()) const;

	/** Returns a list of variable inputs for all static switch nodes in the graph. */
	TArray<FNiagaraVariable> NIAGARAEDITOR_API FindStaticSwitchInputs(bool bReachableOnly = false, const TArray<FNiagaraVariable>& InStaticVars = TArray<FNiagaraVariable>()) const;

	/** Get an in-order traversal of a graph by the specified target output script usage.*/
	NIAGARAEDITOR_API void BuildTraversal(TArray<class UNiagaraNode*>& OutNodesTraversed, ENiagaraScriptUsage TargetUsage, FGuid TargetUsageId, bool bEvaluateStaticSwitches = false) const;
	static void BuildTraversal(TArray<class UNiagaraNode*>& OutNodesTraversed, UNiagaraNode* FinalNode, bool bEvaluateStaticSwitches = false);

	/** Generates a list of unique input and output parameters for when this script is used as a function. */
	void GetParameters(TArray<FNiagaraVariable>& Inputs, TArray<FNiagaraVariable>& Outputs) const;

	/** Generates a list of all input parameters - including parameters in the Input namespace & Static Switch inputs. */
	void GetAllInputScriptVariables(TArray<UNiagaraScriptVariable*>& OutScriptVariables) const;
	
	/** Returns the index of this variable in the output node of the graph. INDEX_NONE if this is not a valid attribute. */
	int32 GetOutputNodeVariableIndex(const FNiagaraVariable& Attr)const;
	void GetOutputNodeVariables(TArray< FNiagaraVariable >& OutAttributes)const;
	void GetOutputNodeVariables(ENiagaraScriptUsage InTargetScriptUsage, TArray< FNiagaraVariable >& OutAttributes)const;

	bool HasNumericParameters()const;

	bool HasParameterMapParameters()const;

	NIAGARAEDITOR_API bool GetPropertyMetadata(FName PropertyName, FString& OutValue) const;

	/** Signal to listeners that the graph has changed */
	void NotifyGraphNeedsRecompile();
		
	/** Notifies the graph that a contained data interface has changed. */
	void NotifyGraphDataInterfaceChanged();

	/** Get all referenced graphs in this specified graph, including this graph. */
	void GetAllReferencedGraphs(TArray<const UNiagaraGraph*>& Graphs) const;

	/** Gather all the change ids of external references for this specific graph traversal.*/
	void GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector);

	/** Determine if another item has been synchronized with this graph.*/
	bool IsOtherSynchronized(const FGuid& InChangeId) const;

	/** Identify that this graph has undergone changes that will require synchronization with a compiled script.*/
	void MarkGraphRequiresSynchronization(FString Reason);

	/** A change was made to the graph that external parties should take note of. The ChangeID will be updated.*/
	virtual void NotifyGraphChanged() override;

	/** Each graph is given a Change Id that occurs anytime the graph's content is manipulated. This key changing induces several important activities, including being a 
	value that third parties can poll to see if their cached handling of the graph needs to potentially adjust to changes. Furthermore, for script compilation we cache 
	the changes that were produced during the traversal of each output node, which are referred to as the CompileID.*/
	FGuid GetChangeID() const { return ChangeId; }

	/** Gets the current compile data hash associated with the output node traversal specified by InUsage and InUsageId. If the usage is not found, an invalid hash is returned.*/
	FNiagaraCompileHash GetCompileDataHash(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const;

	/** Gets the current reference data hash associated with the output node traversal specified by InUsage and InUsageId (Static variables & switches). If the usage is not found, an invalid hash is returned.*/
	FNiagaraCompileHash GetCompileReferencedDataHash(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const;

	/** Gets the current base id associated with the output node traversal specified by InUsage and InUsageId. If the usage is not found, an invalid guid is returned. */
	FGuid GetBaseId(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const;

	/** Forces the base compile id for the supplied script.  This should only be used to keep things consistent after an emitter merge. */
	void ForceBaseId(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, const FGuid InForcedBaseId);

	/** Walk through the graph for an ParameterMapGet nodes and see if any of them specify a default for VariableName.*/
	UEdGraphPin* FindParameterMapDefaultValuePin(const FName VariableName, ENiagaraScriptUsage InUsage, ENiagaraScriptUsage InParentUsage) const;
	void MultiFindParameterMapDefaultValuePins(TConstArrayView<FName> VariableNames, ENiagaraScriptUsage InUsage, ENiagaraScriptUsage InParentUsage, TArrayView<UEdGraphPin*> DefaultPins) const;

	/** Find all matching default pins for VariableName, irrespective of usage. */
	TArray<UEdGraphPin*> FindParameterMapDefaultValuePins(const FName VariableName) const;

	/** Rebuilds the internally stored ParameterRefrenceMap if it has been marked dirty.  Used to explicitly trigger the rebuild rather than
	impliticly rebuilding it through a call to GetParameterReferenceMap(). */
	NIAGARAEDITOR_API void ConditionalRefreshParameterReferences();

	const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& GetParameterReferenceMap() const; // NOTE: The const is a lie! (This indirectly calls RefreshParameterReferences, which can recreate the entire map)

	/** Gets the meta-data associated with this variable, if it exists.*/
	NIAGARAEDITOR_API TOptional<FNiagaraVariableMetaData> GetMetaData(const FNiagaraVariable& InVar) const;

	/** Sets the meta-data associated with this variable. Creates a new UNiagaraScriptVariable if the target variable cannot be found. Illegal to call on FNiagaraVariables that are Niagara Constants. */
	void SetMetaData(const FNiagaraVariable& InVar, const FNiagaraVariableMetaData& MetaData);

	// These functions are not supported for compilation copies
	using FScriptVariableMap = TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>>;

	NIAGARAEDITOR_API const FScriptVariableMap& GetAllMetaData() const;
	NIAGARAEDITOR_API FScriptVariableMap& GetAllMetaData();

	UHierarchyRoot* GetScriptParameterHierarchyRoot() const { return ParameterHierarchyRoot; }

	UNiagaraScriptVariable* GetScriptVariable(FNiagaraVariable Parameter) const;
	NIAGARAEDITOR_API UNiagaraScriptVariable* GetScriptVariable(FName ParameterName) const;
	NIAGARAEDITOR_API UNiagaraScriptVariable* GetScriptVariable(FGuid VariableGuid) const;
	NIAGARAEDITOR_API TArray<FGuid> GetChildScriptVariableGuidsForInput(FGuid VariableGuid) const;
	TArray<UNiagaraScriptVariable*> GetChildScriptVariablesForInput_Deprecated(FGuid VariableGuid) const;

	/** Adds parameter to the VariableToScriptVariable map.*/
	UNiagaraScriptVariable* AddParameter(const FNiagaraVariable& Parameter, bool bIsStaticSwitch = false);
	UNiagaraScriptVariable* AddParameter(const FNiagaraVariable& Parameter, const FNiagaraVariableMetaData& ParameterMetaData, bool bIsStaticSwitch, bool bNotifyChanged);
	UNiagaraScriptVariable* AddParameter(const UNiagaraScriptVariable* InScriptVar);

	/** Adds an FNiagaraGraphParameterReference to the ParameterToReferenceMap. */
	void AddParameterReference(const FNiagaraVariable& Parameter, FNiagaraGraphParameterReference& NewParameterReference);

	/** Remove parameter from map and all the pins associated. */
	void RemoveParameter(const FNiagaraVariable& Parameter, bool bAllowDeleteStaticSwitch = false);

	/**
	* Rename parameter from map and all the pins associated.
	*
	* @param Parameter				The target FNiagaraVariable key to lookup the canonical UNiagaraScriptVariable to rename.
	* @param NewName				The new name to apply.
	* @param bFromStaticSwitch		Whether the target parameter is from a static switch. Used to determine whether to fixup binding strings.
	* @param bMerged				Whether or not the rename ended up merging with a different parameter because the names are the same.
	* @param bSuppressEvents		Whether or not to invoke UEdGraphNode::OnPinRenamed() when setting NewName on a node pin. If true, do not call UEdGraphNode::OnPinRenamed().
	* @return						true if the new name was applied. 
	*/
	NIAGARAEDITOR_API bool RenameParameter(const FNiagaraVariable& Parameter, FName NewName, bool bRenameRequestedFromStaticSwitch = false, bool* bMerged = nullptr, bool bSuppressEvents = false);

	/** Rename a pin inline in a graph. If this is the only instance used in the graph, then rename them all, otherwise make a duplicate. */
	bool RenameParameterFromPin(const FNiagaraVariable& Parameter, FName NewName, UEdGraphPin* InPin);

	bool RenameStaticSwitch(UNiagaraNodeStaticSwitch* SwitchNode, FName NewName);

	/** Changes the type of existing graph parameters.
	 *  Optionally creates orphaned pins for any connection that won't be kept, but tries to keep connections as long as types are matching.
	 *  Changing multiple parameters at once helps with maintaining connections.
	 *  CAUTION: Do not allow orphaned pins in the stack graphs, as they aren't user facing.
	 */
	NIAGARAEDITOR_API void ChangeParameterType(const TArray<FNiagaraVariable>& ParametersToChange, const FNiagaraTypeDefinition& NewType, bool bAllowOrphanedPins = false);

	void ReplaceScriptReferences(UNiagaraScript* OldScript, UNiagaraScript* NewScript);

	/** Gets a delegate which is called whenever a parameter is added or removed */
	FOnParametersChanged& OnParametersChanged();
	
	/** Gets a delegate which is called whenever a contained data interfaces changes. */
	FOnDataInterfaceChanged& OnDataInterfaceChanged();

	/** Gets a delegate which is called whenever a custom subobject in the graph is selected*/
	FOnSubObjectSelectionChanged& OnSubObjectSelectionChanged();

	void ForceGraphToRecompileOnNextCheck();

	/** Add a listener for OnGraphNeedsRecompile events */
	FDelegateHandle AddOnGraphNeedsRecompileHandler(const FOnGraphChanged::FDelegate& InHandler);

	/** Remove a listener for OnGraphNeedsRecompile events */
	void RemoveOnGraphNeedsRecompileHandler(FDelegateHandle Handle);

	FNiagaraTypeDefinition GetCachedNumericConversion(const class UEdGraphPin* InPin);

	const class UEdGraphSchema_Niagara* GetNiagaraSchema() const;

	void InvalidateNumericCache();

	/** If this graph is the source of a function call, it can add a string to the function name to discern it from different
	  * function calls to the same graph. For example, if the graph contains static switches and two functions call it with
	  * different switch parameters, the final function names in the hlsl must be different.
	  */
	void UpdateFunctionAliasByContext(FNiagaraGraphFunctionAliasContext& FunctionAliasContext) const;

	/** In order to support reducing the number of times we need to fully generate the CompileId this function is introduced
	  * to work with LastBuiltScriptVersionId & bHasValidLastBuiltScriptVersionId.
	  */
	void ConditionalRebuildCompileIdCache();

	void RebuildCachedCompileIds();

	void CopyCachedReferencesMap(UNiagaraGraph* TargetGraph);

	bool IsPinVisualWidgetProviderRegistered() const;

	void ScriptVariableChanged(FNiagaraVariable Variable);

	FVersionedNiagaraEmitter GetOwningEmitter() const;

	/** Synchronize all the properties of DestScriptVar to those of SourceScriptVar, as well as propagating those changes through the graph (pin variable names and default values on pins.) 
	 *  If DestScriptVar is not set, find a script variable with the same key as the SourceScriptVar.
	 *  Returns bool to signify if DestScriptVar was modified.
	 */
	bool SynchronizeScriptVariable(const UNiagaraScriptVariable* SourceScriptVar, UNiagaraScriptVariable* DestScriptVar = nullptr, bool bIgnoreChangeId = false);

	/** Find a script variable with the same key as RemovedScriptVarId and unmark it as being sourced from a parameter definitions. */
	bool SynchronizeParameterDefinitionsScriptVariableRemoved(const FGuid RemovedScriptVarId);

	/** Synchronize all source script variables that have been changed or removed from the parameter definitions to all eligible destination script variables owned by the graph. 
	 * 
	 *  @param TargetDefinitions			The set of parameter definitions that will be synchronized with the graph parameters.
	 *	@param AllDefinitions				All parameter definitions in the project. Used to add new subscriptions to definitions if specified in Args.
	 *  @param AllDefinitionsParameterIds	All unique Ids of all parameter definitions. 
	 *	@param Subscriber					The INiagaraParameterDefinitionsSubscriber that owns the graph. Used to add new subscriptions to definitions if specified in Args.
	 *	@param Args							Additional arguments that specify how to perform the synchronization. 
	 */
	void SynchronizeParametersWithParameterDefinitions(
		const TArray<UNiagaraParameterDefinitions*> TargetDefinitions,
		const TArray<UNiagaraParameterDefinitions*> AllDefinitions,
		const TSet<FGuid>& AllDefinitionsParameterIds,
		INiagaraParameterDefinitionsSubscriber* Subscriber,
		FSynchronizeWithParameterDefinitionsArgs Args
	);

	/** Rename all assignment and map set node pins. 
	 *  Used when synchronizing definitions with source scripts of systems and emitters.
	 */
	void RenameAssignmentAndSetNodePins(const FName OldName, const FName NewName);

	/** Go through all known parameter names in this graph and generate a new unique one.*/
	FName MakeUniqueParameterName(const FName& InName);

	static FName MakeUniqueParameterNameAcrossGraphs(const FName& InName, TArray<TWeakObjectPtr<UNiagaraGraph>>& InGraphs);

	static FName StandardizeName(FName Name, ENiagaraScriptUsage Usage, bool bIsGet, bool bIsSet);

	/** Helper to get a map of variables to all input/output pins with the same name. */
	const TMap<FNiagaraVariable, FInputPinsAndOutputPins> NIAGARAEDITOR_API CollectVarsToInOutPinsMap() const;

	void GetAllScriptVariableGuids(TArray<FGuid>& VariableGuids) const;
	void GetAllVariables(TArray<FNiagaraVariable>& Variables) const;
	TOptional<bool> IsStaticSwitch(const FNiagaraVariable& Variable) const;
	TOptional<int32> GetStaticSwitchDefaultValue(const FNiagaraVariable& Variable) const;
	TOptional<ENiagaraDefaultMode> GetDefaultMode(const FNiagaraVariable& Variable, FNiagaraScriptVariableBinding* Binding = nullptr) const;
	TOptional<FGuid> GetScriptVariableGuid(const FNiagaraVariable& Variable) const;
	TOptional<FNiagaraVariable> GetVariable(const FNiagaraVariable& Variable) const;
	bool HasVariable(const FNiagaraVariable& Variable) const;

	void SetIsStaticSwitch(const FNiagaraVariable& Variable, bool InValue);

	bool ReferencesStaticVariable(FNiagaraStaticVariableSearchContext& SearchContext) const;

	/** Migrates the deprecated data from the provided OwnerData and populates the ParameterHierarchyRoot */
	void MigrateParameterScriptDataToHierarchyRoot(FVersionedNiagaraScriptData& OwnerData);

protected:
	void RebuildNumericCache();
	bool bNeedNumericCacheRebuilt;
	TMap<TPair<FGuid, UEdGraphNode*>, FNiagaraTypeDefinition> CachedNumericConversions;
	void ResolveNumerics(TMap<UNiagaraNode*, bool>& VisitedNodes, UEdGraphNode* Node);
	bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor, const TArray<UNiagaraNode*>& InTraversal) const;
	UNiagaraScriptVariable* CreateScriptVariableInternal(const FNiagaraVariable& Parameter, const FNiagaraVariableMetaData& ParameterMetaData, bool bIsStaticSwitch);

private:
	virtual void NotifyGraphChanged(const FEdGraphEditAction& InAction) override;

	/** Find parameters in the graph. */
	void RefreshParameterReferences() const;

	/** Marks the found parameter collections as invalid so they're rebuilt the next time they're requested. */
	void InvalidateCachedParameterData();

	/** A delegate that broadcasts a notification whenever the graph needs recompile due to structural change. */
	FOnGraphChanged OnGraphNeedsRecompile;

	/** Find all nodes in the graph that can be reached during compilation. */
	TArray<UEdGraphNode*> FindReachableNodes(const TArray<FNiagaraVariable>& InStaticVars) const;

	/** Compares the values on the default pins with the metadata and syncs the two if necessary */
	void ValidateDefaultPins();

	void StandardizeParameterNames();

	static bool VariableLess(const FNiagaraVariable& Lhs, const FNiagaraVariable& Rhs);
	TOptional<FNiagaraScriptVariableData> GetScriptVariableData(const FNiagaraVariable& Variable) const;

	void FixupReferenceCollectionsPostRename(const FNiagaraVariable& OrigVariable, const FNiagaraVariable& DestVariable, bool bParameterMerged, bool bSuppressEvents);

	void PostLoad_LWCFixup(int32 NiagaraVersion);
	void PostLoad_ManageScriptVariables(int32 NiagaraVersion);

private:
	/** The current change identifier for this graph overall. Used to sync status with UNiagaraScripts.*/
	UPROPERTY()
	FGuid ChangeId;

	/** Internal value used to invalidate a DDC key for the script no matter what.*/
	UPROPERTY()
	FGuid ForceRebuildId;

	UPROPERTY()
	FGuid LastBuiltTraversalDataChangeId;

	/** The script version that was used when the cached CompileId was generated, a change
	 *	in script version will invalidate the cached CompileId and a new one will be generated.
	 *	Will be initialized to an invalid guid but it won't be used until a valid script has been
	 *	assigned (as dictated by bHasValidLastBuiltScriptVersionId) */
	UPROPERTY()
	FGuid LastBuiltScriptVersionId;

	UPROPERTY()
	TArray<FNiagaraGraphScriptUsageInfo> CachedUsageInfo;

	/** Storage of meta-data for variables defined for use explicitly with this graph.*/
	UPROPERTY()
	mutable TMap<FNiagaraVariable, FNiagaraVariableMetaData> VariableToMetaData_DEPRECATED;

	/** Storage of variables defined for use with this graph.*/
	UPROPERTY()
	TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>> VariableToScriptVariable;

	UPROPERTY()
	TObjectPtr<UHierarchyRoot> ParameterHierarchyRoot;
	
	/** A map of parameters in the graph to their referencers. */
	UPROPERTY(Transient)
	mutable TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection> ParameterToReferencesMap;

	UPROPERTY(Transient)
	mutable TArray<FNiagaraScriptVariableData> CompilationScriptVariables;

	/** Used in conjunction with LastBuiltScriptVersionId to note that we've got a valid script Id stored.
	 *	Works around things without having to add a custom version. */
	UPROPERTY()
	bool bHasValidLastBuiltScriptVersionId = false;

	FOnParametersChanged OnParametersChangedDelegate;
	FOnDataInterfaceChanged OnDataInterfaceChangedDelegate;
	FOnSubObjectSelectionChanged OnSelectedSubObjectChanged;
	FOnGetParameterDefinitionsForDetailsCustomization OnGetParameterDefinitionsForDetailsCustomizationDelegate;

	/** Whether currently renaming a parameter to prevent recursion. */
	bool bIsRenamingParameter;

	mutable bool bParameterReferenceRefreshPending;

	bool bIsForCompilationOnly = false;
};
