// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/LruCache.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorUtilities.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

class UEdGraphPin;
class UNiagaraGraph;
class UNiagaraNodeFunctionCall;

class FNiagaraGraphDataCache
{
public:
	FNiagaraGraphDataCache();

	void GetStackFunctionInputs(const UNiagaraNodeFunctionCall& FunctionCallNode, TConstArrayView<FNiagaraVariable> StaticVars, TArray<FNiagaraVariable>& OutInputVariables, FCompileConstantResolver ConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled);
	void GetStackFunctionInputs(const UNiagaraNodeFunctionCall& FunctionCallNode, TConstArrayView<FNiagaraVariable> StaticVars, TArray<FNiagaraVariable>& OutInputVariables, TSet<FNiagaraVariable>& OutHiddenVariables, FCompileConstantResolver ConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled);

	void NIAGARAEDITOR_API InvalidateCache();

private:
	void GetStackFunctionInputsInternal(
		const UNiagaraNodeFunctionCall& FunctionCallNode,
		const UNiagaraGraph* CalledGraph,
		TConstArrayView<FNiagaraVariable> StaticVars,
		TArray<FNiagaraVariable>& OutInputVariables,
		FCompileConstantResolver ConstantResolver,
		FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options,
		bool bIgnoreDisabled,
		bool bFilterForCompilation);

	struct FStackFunctionInputPinKey
	{
		FStackFunctionInputPinKey(const UNiagaraNodeFunctionCall& FunctionCallNode, const UNiagaraGraph* CalledGraph, TConstArrayView<FNiagaraVariable> StaticVazriables, const FCompileConstantResolver& InConstantResolver, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions InOptions, bool bInIgnoreDisabled, bool bInFilterForCompilation);
		bool operator==(const FStackFunctionInputPinKey& Other) const;

		friend uint32 GetTypeHash(const FStackFunctionInputPinKey& Key)
		{
			return Key.Hash;
		}

		const FGuid& GetGraphChangeId() const
		{
			return FunctionCallGraphChangeId;
		}

	private:
		FObjectKey FunctionCallGraphKey;
		FGuid FunctionCallGraphChangeId;
		TArray<FNiagaraVariable> ContextStaticVariables;
		uint32 ConstantResolverHash = 0;
		uint8 InputPinOption = 0;
		bool bIgnoreDisabled = false;
		bool bFilterForCompilation = false;

		int32 Hash = 0;
	};

	struct FStackFunctionInputPinValue
	{
		TArray<FNiagaraVariable> InputVariables;
	};

	// cache holding a finite number (specific value pulled from a cvar) of results to previous calls
	// to retrieve the input pins
	TLruCache<FStackFunctionInputPinKey, FStackFunctionInputPinValue> StackFunctionInputPinCache;
};