// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraShared.h"
#include "NiagaraCommon.h"
#include "Logging/TokenizedMessage.h"
#include "TickableEditorObject.h"
#include "UObject/ObjectKey.h"
#include "Templates/SharedPointer.h"
#include "NiagaraMessages.h"

class FNiagaraScriptToolkit;

//Extension of message token to allow opening the asset editor when clicking on the linked asset name.
class FNiagaraCompileEventToken : public IMessageToken
{
public:
	/** Factory method, tokens can only be constructed as shared refs */
	static TSharedRef<FNiagaraCompileEventToken> Create(
		  const FString& InScriptAssetPath
		, const FText& InMessage
		, const TOptional<const FGuid>& InNodeGUID = TOptional<const FGuid>()
		, const TOptional<const FGuid>& InPinGUID = TOptional<const FGuid>()
	);

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::AssetName;
	}
	/** End IMessageToken interface */

private:
	/** Private constructor */
	FNiagaraCompileEventToken(
		  const FString& InScriptAssetPath
		, const FText& InMessage
		, const TOptional<const FGuid>& InNodeGUID
		, const TOptional<const FGuid>& InPinGUID
	);

	/**
	 * Find and open an asset in editor
	 * @param	Token			The token that was clicked
	 * @param	InAssetPath		The asset to find
	 */
	static void OpenScriptAssetByPathAndFocusNodeOrPinIfSet(
		  const TSharedRef<IMessageToken>& Token
		, FString InScriptAssetPath
		, const TOptional<const FGuid> InNodeGUID
		, const TOptional<const FGuid> InPinGUID
	);

	/** The script asset path to open the editor toolkit for. */
	const FString ScriptAssetPath;

	/** The optional Node or Pin GUID to find and focus after opening the script asset */
	const TOptional<const FGuid> NodeGUID;
	const TOptional<const FGuid> PinGUID;
};

class INiagaraMessageRegistrationHandle : public TSharedFromThis<INiagaraMessageRegistrationHandle>
{
public:
	DECLARE_DELEGATE_OneParam(FOnRequestRefresh, const TArray<TSharedRef<const INiagaraMessage>>& /** NewMessages*/)

	INiagaraMessageRegistrationHandle(const uint32 InTopicBitfield = 0)
		: TopicBitfield(InTopicBitfield)
	{};

	virtual ~INiagaraMessageRegistrationHandle() = default;

	virtual TArray<TSharedRef<const INiagaraMessage>> FilterMessages(const TArray<TSharedRef<const INiagaraMessage>>& Messages, const uint32& DesiredTopicBitfield) const = 0;

	FOnRequestRefresh& GetOnRequestRefresh() { return RefreshDelegate; };
	uint32 GetTopicBitfield() const { return TopicBitfield; };

protected:
	const uint32 TopicBitfield;
	FOnRequestRefresh RefreshDelegate;
};

class FNiagaraMessageTopicRegistrationHandle : public INiagaraMessageRegistrationHandle
{
public:
	FNiagaraMessageTopicRegistrationHandle(const uint32 InTopicBitfield)
		: INiagaraMessageRegistrationHandle(InTopicBitfield)
	{};

	virtual TArray<TSharedRef<const INiagaraMessage>> FilterMessages(const TArray<TSharedRef<const INiagaraMessage>>& Messages, const uint32& DesiredTopicBitfield) const override;
};

class FNiagaraMessageObjectRegistrationHandle : public INiagaraMessageRegistrationHandle
{
public:
	FNiagaraMessageObjectRegistrationHandle(const FObjectKey& InObjectKey)
		: INiagaraMessageRegistrationHandle(INT32_MAX) //Pure object registrations listen for every topic
		, ObjectKey(InObjectKey)
	{};

	virtual TArray<TSharedRef<const INiagaraMessage>> FilterMessages(const TArray<TSharedRef<const INiagaraMessage>>& Messages, const uint32& DesiredTopicBitfield) const override;

protected:
	const FObjectKey ObjectKey;
};

class FNiagaraMessageManager : FTickableEditorObject
{
public:

	NIAGARAEDITOR_API static FNiagaraMessageManager* Get();

	NIAGARAEDITOR_API void AddMessage(const TSharedRef<const INiagaraMessage>& InMessage, const FGuid& InMessageAssetKey);

	NIAGARAEDITOR_API void AddMessageJob(TUniquePtr<const INiagaraMessageJob>&& InMessageJob, const FGuid& InMessageJobAssetKey);

	void ClearAssetMessages(const FGuid& AssetKey);

	void ClearAssetMessagesForTopic(const FGuid& AssetKey, const FName& Topic);

	void ClearAssetMessagesForObject(const FGuid& AssetKey, const FObjectKey& ObjectKeys);

	static const TOptional<const FString> GetStringForScriptUsageInStack(const ENiagaraScriptUsage InScriptUsage);

	NIAGARAEDITOR_API void RegisterMessageTopic(FName TopicName);
	NIAGARAEDITOR_API void RegisterAdditionalMessageLogTopic(FName MessageLogTopicName);
	uint32 GetMessageTopicBitflag(FName TopicName);

	NIAGARAEDITOR_API FNiagaraMessageTopicRegistrationHandle::FOnRequestRefresh& SubscribeToAssetMessagesByTopic(
		  const FText& DebugNameText
		, const FGuid& MessageAssetKey
		, const TArray<FName>& MessageTopics
		, FGuid& OutMessageManagerRegistrationKey);

	NIAGARAEDITOR_API FNiagaraMessageTopicRegistrationHandle::FOnRequestRefresh& SubscribeToAssetMessagesByObject(
		  const FText& DebugNameText
		, const FGuid& MessageAssetKey
		, const FObjectKey& ObjectKey
		, FGuid& OutMessageManagerRegistrationKey);

	void Unsubscribe(const FText& DebugNameText, const FGuid& MessageAssetKey, FGuid& MessageManagerRegistrationKey);

	const TArray<FName>& GetAdditionalMessageLogTopics() { return AdditionalMessageLogTopics; };

	//~ Begin FTickableEditorObject Interface.
	virtual void Tick(float DeltaSeconds) override;

	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; };

	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FNiagaraMessageManager, STATGROUP_Tickables); };
	//~ End FTickableEditorObject Interface

private:

	struct FAssetMessageInfo
	{
		FAssetMessageInfo()
			: bDirty(false)
			, DirtyTopicBitfield(0)
		{};

		TArray<TSharedRef<const INiagaraMessage>> Messages;
		TMap<FGuid, TSharedPtr<INiagaraMessageRegistrationHandle>> RegistrationKey_To_RegistrationHandleMap;
		bool bDirty;
		uint32 DirtyTopicBitfield;
	};

	struct FMessageJobAndAssetKey
	{
		FMessageJobAndAssetKey(const FGuid& InAssetKey)
			: AssetKey(InAssetKey)
		{};

// 		FMessageJobAndAssetKey(TUniquePtr<const INiagaraMessageJob>&& InMessageJob, const FGuid& InAssetKey) //@todo(ng) impl
// 			: MessageJob(MoveTemp(InMessageJob))
// 			, AssetKey(InAssetKey)
// 		{};

		TUniquePtr<const INiagaraMessageJob> MessageJob;
		const FGuid AssetKey;
	};

	FNiagaraMessageManager();

	void DoMessageJobsTick();
	void TryFlushMessagesTick();

	void FlushMessages();
	static void SetRefreshTimerElapsed();

	uint32 MakeBitfieldForMessageTopics(const FText& DebugNameText, const TArray<FName>& MessageTopics);

private:

	static constexpr double MaxJobWorkTime = .02f; // do message jobs at 50 fps.
	static constexpr double RefreshHysterisisTime = 2.0f; // do not consecutively request refresh subscribers more often than every 2 seconds.

	static bool bRefreshTimeElapsed;
	static bool bNeedFlushMessages;
	static uint32 NextTopicBitflag; // every time a topic is registered, assign this bitflag to the topic and then binary increment this bitflag.
	static FNiagaraMessageManager* Singleton;

	FTimerHandle RefreshTimerHandle;
	TArray<FMessageJobAndAssetKey> MessageJobs;
	TMap<const FGuid, FAssetMessageInfo> AssetToMessageInfoMap;
	TMap<const FName, uint32> RegisteredTopicToBitflagsMap;
	TArray<FName> AdditionalMessageLogTopics;
};
