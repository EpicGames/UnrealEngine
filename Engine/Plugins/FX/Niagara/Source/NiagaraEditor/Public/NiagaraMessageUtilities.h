// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraMessages.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"

class INiagaraMessage;

namespace FNiagaraMessageUtilities
{
	FText MakePostCompileSummaryText(const FText& CompileObjectNameText, ENiagaraScriptCompileStatus LatestCompileStatus, const int32& WarningCount, const int32& ErrorCount);

	UNiagaraStackEntry::FStackIssue MessageToStackIssue(TSharedRef<const INiagaraMessage> InMessage, FString InStackEditorDataKey);

	UNiagaraStackEntry::FStackIssue StackMessageToStackIssue(const FNiagaraStackMessage& InMessage, FString InStackEditorDataKey, const TArray<FLinkNameAndDelegate>& InLinks);

	FText GetShortDescriptionFromSeverity(EStackIssueSeverity Severity);
}
