// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/Guid.h"
#include "UObject/ObjectKey.h"
#include "NiagaraCommon.h"
#include "NiagaraShared.h"
#include "NiagaraGraph.h"
#include "NiagaraMessageDataBase.h"
#include "NiagaraMessages.generated.h"

struct FNiagaraMessageTopics
{
	static NIAGARAEDITOR_API const FName CompilerTopicName;
	static NIAGARAEDITOR_API const FName ObjectTopicName;
};

UENUM()
enum class ENiagaraMessageSeverity : uint8
{
	CriticalError = 0,
	Error = 1,
	PerformanceWarning = 2,
	Warning = 3,
	Info = 4
};

//Struct for passing around script asset info from compile event message job to message types
struct FNiagaraScriptNameAndAssetPath
{
public:
	FNiagaraScriptNameAndAssetPath(const FString& InScriptNameString, const FString& InScriptAssetPathString)
		: ScriptNameString(InScriptNameString)
		, ScriptAssetPathString(InScriptAssetPathString)
	{};

	const FString ScriptNameString;
	const FString ScriptAssetPathString;
};

// Struct for passing around named simple delegates.
struct FLinkNameAndDelegate
{
public:
	FLinkNameAndDelegate() = default;

	FLinkNameAndDelegate(const FText& InLinkNameText, const FSimpleDelegate& InLinkDelegate)
		: LinkNameText(InLinkNameText)
		, LinkDelegate(InLinkDelegate)
	{};

	FText LinkNameText;
	FSimpleDelegate LinkDelegate;
};

/** 
 * Interface for view-agnostic message that holds limited lifetime information on a message (e.g. a weak pointer to an asset.)
 */
class INiagaraMessage
{
public:
	INiagaraMessage(const TArray<FObjectKey>& InAssociatedObjectKeys = TArray<FObjectKey>())
		: AssociatedObjectKeys(InAssociatedObjectKeys)
		, MessageTopicBitflag(0)
	{};

	virtual ~INiagaraMessage() = default;

	virtual FText GenerateMessageText() const = 0;

	/** Can optionally be overriden to give the message a title/short description. */
	virtual FText GenerateMessageTitle() const;

	virtual const FSimpleDelegate& GetDismissHandler() const { return DismissHandler; }
	
	virtual TSharedRef<FTokenizedMessage> GenerateTokenizedMessage() const = 0;

	virtual void GenerateLinks(TArray<FText>& OutLinkDisplayNames, TArray<FSimpleDelegate>& OutLinkNavigationActions) const = 0;

	virtual const FName GetMessageTopic() const = 0;

	const TArray<FObjectKey>& GetAssociatedObjectKeys() const { return AssociatedObjectKeys; };

	const uint32 GetMessageTopicBitflag() const;

	/** Can optionally be overridden to allow a message to only be logged instead of appearing elsewhere, like the stack. */
	virtual bool ShouldOnlyLog() const { return false; }

protected:
	const TArray<FObjectKey> AssociatedObjectKeys;
	mutable uint32 MessageTopicBitflag;
	FSimpleDelegate DismissHandler;
};

class FNiagaraMessageCompileEvent : public INiagaraMessage
{
public:
	FNiagaraMessageCompileEvent(
		const FNiagaraCompileEvent& InCompileEvent
		, TArray<FNiagaraScriptNameAndAssetPath>& InContextScriptNamesAndAssetPaths
		, TOptional<const FText>& InOwningScriptNameAndUsageText
		, TOptional<const FNiagaraScriptNameAndAssetPath>& InCompiledScriptNameAndAssetPath
		, const TArray<FObjectKey>& InAssociatedObjectKeys
		);

	virtual FText GenerateMessageText() const override;

	virtual FText GenerateMessageTitle() const override;

	virtual TSharedRef<FTokenizedMessage> GenerateTokenizedMessage() const override;

	virtual void GenerateLinks(TArray<FText>& OutLinkDisplayNames, TArray<FSimpleDelegate>& OutLinkNavigationActions) const override;

	virtual const FName GetMessageTopic() const override { return FNiagaraMessageTopics::CompilerTopicName; };

	virtual bool ShouldOnlyLog() const override { return CompileEvent.Severity == FNiagaraCompileEventSeverity::Log; }
	
	const FNiagaraCompileEvent& GetCompileEvent() const { return CompileEvent; }
private:
	const FNiagaraCompileEvent CompileEvent;
	const TArray<FNiagaraScriptNameAndAssetPath> ContextScriptNamesAndAssetPaths;
	const TOptional<const FText> OwningScriptNameAndUsageText;
	const TOptional<const FNiagaraScriptNameAndAssetPath> CompiledScriptNameAndAssetPath;
	const TArray<FObjectKey> AssociatedObjectKeys;
};

class FNiagaraMessageText : public INiagaraMessage
{
public:
	FNiagaraMessageText(const FText& InMessageText, const FText& InShortDescription, const EMessageSeverity::Type& InMessageSeverity, const FName& InTopicName, const FSimpleDelegate& InDismissHandler = FSimpleDelegate(), const TArray<FObjectKey>& InAssociatedObjectKeys = TArray<FObjectKey>())
		: INiagaraMessage(InAssociatedObjectKeys)
		, MessageText(InMessageText)
		, ShortDescription(InShortDescription)
		, MessageSeverity(InMessageSeverity)
		, TopicName(InTopicName)
		, DismissHandler(InDismissHandler)
	{
	};

	FNiagaraMessageText(const FText& InMessageText, const EMessageSeverity::Type& InMessageSeverity, const FName& InTopicName, const FSimpleDelegate& InDismissHandler = FSimpleDelegate(), const TArray<FObjectKey>& InAssociatedObjectKeys = TArray<FObjectKey>())
		: INiagaraMessage(InAssociatedObjectKeys)
		, MessageText(InMessageText)
		, MessageSeverity(InMessageSeverity)
		, TopicName(InTopicName)
		, DismissHandler(InDismissHandler)
	{
	};

	virtual FText GenerateMessageText() const override;

	virtual FText GenerateMessageTitle() const override;

	virtual const FSimpleDelegate& GetDismissHandler() const { return DismissHandler; }
	
	virtual TSharedRef<FTokenizedMessage> GenerateTokenizedMessage() const override;

	virtual void GenerateLinks(TArray<FText>& OutLinkDisplayNames, TArray<FSimpleDelegate>& OutLinkNavigationActions) const override { }

	virtual const FName GetMessageTopic() const override { return TopicName; };

private:
	const FText MessageText;
	const FText ShortDescription;
	const EMessageSeverity::Type MessageSeverity;
	const FName TopicName;
	const FSimpleDelegate DismissHandler;
};

class FNiagaraMessageTextWithLinks : public FNiagaraMessageText
{
public:
	FNiagaraMessageTextWithLinks(
	  const FText& InMessageText
	, const FText& InShortDescription  
	, const EMessageSeverity::Type& InMessageSeverity
	, const FName& InTopicName
	, const FSimpleDelegate& InDismissHandler
	, const TArray<FLinkNameAndDelegate>& InLinks
	, const TArray<FObjectKey>& InAssociatedObjectKeys = TArray<FObjectKey>())
		: FNiagaraMessageText(InMessageText, InShortDescription, InMessageSeverity, InTopicName, InDismissHandler, InAssociatedObjectKeys)
		, Links(InLinks)
	{
	};

	virtual TSharedRef<FTokenizedMessage> GenerateTokenizedMessage() const override;

	virtual void GenerateLinks(TArray<FText>& OutLinkDisplayNames, TArray<FSimpleDelegate>& OutLinkNavigationActions) const override;

private:
	const TArray<FLinkNameAndDelegate> Links;
};

/**
 * Interface for "slow task" message generation jobs that should be time sliced to avoid stalling the UI.
 */
class INiagaraMessageJob
{
public:
	virtual TSharedRef<const INiagaraMessage> GenerateNiagaraMessage() const = 0;

	virtual ~INiagaraMessageJob() = default;
};

class FNiagaraMessageJobCompileEvent : public INiagaraMessageJob
{
public:
	FNiagaraMessageJobCompileEvent(
		const FNiagaraCompileEvent& InCompileEvent
		, const TWeakObjectPtr<const UNiagaraScript>& InOriginatingScriptWeakObjPtr
		, FGuid InCompiledScriptVersion = FGuid()
		, const TOptional<const FString>& InOwningScriptNameString = TOptional<const FString>()
		, const TOptional<const FString>& InSourceScriptAssetPath = TOptional<const FString>()
		);

	virtual TSharedRef<const INiagaraMessage> GenerateNiagaraMessage() const override;

private:

	bool RecursiveGetScriptNamesAndAssetPathsFromContextStack(
		TArray<FGuid>& InContextStackNodeGuids
		, FGuid NodeGuid
		, const UNiagaraGraph* InGraphToSearch
		, TArray<FNiagaraScriptNameAndAssetPath>& OutContextScriptNamesAndAssetPaths
		, TOptional<const FString>& OutEmitterName
		, TOptional<const FText>& OutFailureReason
		, TArray<FObjectKey>& OutContextNodeObjectKeys
		) const;

	const FNiagaraCompileEvent CompileEvent;
	const TWeakObjectPtr<const UNiagaraScript> OriginatingScriptWeakObjPtr;
	FGuid CompiledScriptVersion;
	TOptional<const FString> OwningScriptNameString;
	TOptional<const FString> SourceScriptAssetPath;
};

USTRUCT()
struct FNiagaraStackMessage
{
	GENERATED_BODY()

	NIAGARAEDITOR_API FNiagaraStackMessage();

	NIAGARAEDITOR_API FNiagaraStackMessage(const FText& MessageText, const FText& ShortDescription, ENiagaraMessageSeverity Severity, bool bAllowDismissal, FGuid Guid = FGuid::NewGuid());
	
	UPROPERTY()
	FText MessageText;

	UPROPERTY()
	FText ShortDescription;

	UPROPERTY()
	ENiagaraMessageSeverity MessageSeverity = ENiagaraMessageSeverity::Info;

	UPROPERTY()
	bool bAllowDismissal = true;

	UPROPERTY()
	FGuid Guid;
};

USTRUCT()
struct FNiagaraStackNoteData
{
	GENERATED_BODY()

	UPROPERTY()
	FText MessageHeader;

	UPROPERTY()
	FLinearColor Color = FLinearColor::Black;
	
	UPROPERTY()
	FText Message;

	/** If true, this note should be used inline for a stack entry, instead of adding additional rows. */
	UPROPERTY()
	bool bInlineNote = false;

	bool operator==(const FNiagaraStackNoteData& Other) const
	{
		return MessageHeader.EqualTo(Other.MessageHeader) && Message.EqualTo(Other.Message); 
	}

	bool IsValid() const
	{
		return MessageHeader.IsEmpty() == false || Message.IsEmpty() == false;
	}
};

struct FGenerateNiagaraMessageInfo
{
public:
	FGenerateNiagaraMessageInfo() = default;

	void SetAssociatedObjectKeys(const TArray<FObjectKey>& InAssociatedObjectKeys) { AssociatedObjectKeys = InAssociatedObjectKeys; };
	const TArray<FObjectKey>& GetAssociatedObjectKeys() const { return AssociatedObjectKeys; };
	void SetLinks(const TArray<FLinkNameAndDelegate>& InLinks) { Links = InLinks; };
	const TArray<FLinkNameAndDelegate>& GetLinks() const { return Links; };
	void SetDismissHandler(const FSimpleDelegate& InDismissHandler) { DismissHandler = InDismissHandler; }
	const FSimpleDelegate& GetDismissHandler() const { return DismissHandler; }

private:
	TArray<FObjectKey> AssociatedObjectKeys;
	TArray<FLinkNameAndDelegate> Links;
	FSimpleDelegate DismissHandler;
};

UCLASS(abstract, MinimalAPI)
class UNiagaraMessageData : public UNiagaraMessageDataBase
{
	GENERATED_BODY()

public:
	virtual TSharedRef<const INiagaraMessage> GenerateNiagaraMessage(const FGenerateNiagaraMessageInfo& InGenerateInfo = FGenerateNiagaraMessageInfo()) const PURE_VIRTUAL(UNiagaraMessageData::GenerateNiagaraMessage, return TSharedRef<const INiagaraMessage>(static_cast<INiagaraMessage*>(nullptr)););
};

UCLASS(MinimalAPI)
class UNiagaraMessageDataText : public UNiagaraMessageData
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Init(const FText& InMessageText, const ENiagaraMessageSeverity InMessageSeverity, const FName& InTopicName);
	
	NIAGARAEDITOR_API void Init(const FText& InMessageText, const FText& InShortDescription, const ENiagaraMessageSeverity InMessageSeverity, const FName& InTopicName);

	NIAGARAEDITOR_API virtual TSharedRef<const INiagaraMessage> GenerateNiagaraMessage(const FGenerateNiagaraMessageInfo& InGenerateInfo = FGenerateNiagaraMessageInfo()) const override;

	virtual bool GetAllowDismissal() const override { return bAllowDismissal; }
	void SetAllowDismissal(bool bInAllowDismissal) { bAllowDismissal = bInAllowDismissal; }
	
private:

	/* Marking those FTexts explicitly as editoronly_data will make localization not pick these up. */	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FText MessageText;

	UPROPERTY()
	FText ShortDescription;
#endif
	UPROPERTY()
	ENiagaraMessageSeverity MessageSeverity;

	UPROPERTY()
	bool bAllowDismissal;
	
	UPROPERTY()
	FName TopicName;
};
