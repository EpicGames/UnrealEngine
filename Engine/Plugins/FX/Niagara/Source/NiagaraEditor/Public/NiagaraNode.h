// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNode.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraParameterMapHistory.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Misc/Guid.h"
#include "UObject/UnrealType.h"
#include "EdGraph/EdGraphSchema.h"
#include "Widgets/SBoxPanel.h"
#include "NiagaraNode.generated.h"

struct FNiagaraCompilationGraphBridge;
class UEdGraphPin;
class UEdGraphSchema_Niagara;
class INiagaraCompiler;
struct FNiagaraGraphFunctionAliasContext;
class FSHA1;
template<typename T> class TNiagaraHlslTranslator;

typedef TArray<UEdGraphPin*, TInlineAllocator<16>> FPinCollectorArray;

UCLASS(MinimalAPI)
class UNiagaraNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()
protected:

	using FTranslator = TNiagaraHlslTranslator<FNiagaraCompilationGraphBridge>;

	NIAGARAEDITOR_API bool ReallocatePins(bool bMarkNeedsResynchronizeOnChange = true);

	NIAGARAEDITOR_API bool CompileInputPins(FTranslator* Translator, TArray<int32>& OutCompiledInputs) const;

	virtual void OnPostSynchronizationInReallocatePins() {}
	
public:

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodeVisualsChanged, UNiagaraNode*);

	//~ Begin UObject interface
	NIAGARAEDITOR_API virtual void PostLoad() override;
	//~ End UObject interface

	//~ Begin EdGraphNode Interface
	NIAGARAEDITOR_API virtual void PostPlacedNewNode() override;
	NIAGARAEDITOR_API virtual void AutowireNewNode(UEdGraphPin* FromPin) override;	
	NIAGARAEDITOR_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	NIAGARAEDITOR_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	NIAGARAEDITOR_API virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	NIAGARAEDITOR_API virtual void OnRenameNode(const FString& NewName) override;
	NIAGARAEDITOR_API virtual void OnPinRemoved(UEdGraphPin* InRemovedPin) override;
	NIAGARAEDITOR_API virtual void NodeConnectionListChanged() override;	
	NIAGARAEDITOR_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override; 
	NIAGARAEDITOR_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	NIAGARAEDITOR_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	NIAGARAEDITOR_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	//~ End EdGraphNode Interface

	virtual TSharedRef<SWidget> CreateTitleRightWidget();
	
	/** If this does not return an empty title, compact mode will be activated putting the text into the center of the node. Should be short. */
	virtual FText GetCompactTitle() const { return FText::GetEmpty(); }
	virtual bool ShouldShowPinNamesInCompactMode() { return false; }
	virtual TOptional<float> GetCompactModeFontSizeOverride() const { return {}; }

	/** Virtual function to allow for custom widgets in the input or output box */
	NIAGARAEDITOR_API virtual void AddWidgetsToInputBox(TSharedPtr<SVerticalBox> InputBox);
	NIAGARAEDITOR_API virtual void AddWidgetsToOutputBox(TSharedPtr<SVerticalBox> OutputBox);
	
	/** Get the Niagara graph that owns this node */
	NIAGARAEDITOR_API const class UNiagaraGraph* GetNiagaraGraph()const;
	NIAGARAEDITOR_API class UNiagaraGraph* GetNiagaraGraph();

	/** Get the source object */
	NIAGARAEDITOR_API class UNiagaraScriptSource* GetSource()const;

	/** Gets the asset referenced by this node, or nullptr if there isn't one. */
	virtual UObject* GetReferencedAsset() const { return nullptr; }
	virtual void OpenReferencedAsset() const {}

	/** Refreshes the node due to external changes, e.g. the underlying function changed for a function call node. Return true if the graph changed.*/
	virtual bool RefreshFromExternalChanges() { return false; }

	NIAGARAEDITOR_API virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const;

	NIAGARAEDITOR_API UEdGraphPin* GetInputPin(int32 InputIndex) const;
	NIAGARAEDITOR_API UEdGraphPin* GetOutputPin(int32 OutputIndex) const;

	template<typename ContainerType>
	void GetInputPins(ContainerType& OutInputPins) const
	{
		GetInputPinsInternal<ContainerType>(OutInputPins);
	}
	template<typename ContainerType>
	void GetOutputPins(ContainerType& OutOutputPins) const
	{
		GetOutputPinsInternal<ContainerType>(OutOutputPins);
	}

	NIAGARAEDITOR_API UEdGraphPin* GetPinByPersistentGuid(const FGuid& InGuid) const;
	NIAGARAEDITOR_API virtual void ResolveNumerics(const UEdGraphSchema_Niagara* Schema, bool bSetInline, TMap<TPair<FGuid, class UEdGraphNode*>, FNiagaraTypeDefinition>* PinCache);

	/** Apply any node-specific logic to determine if it is safe to add this node to the graph. This is meant to be called only in the Editor before placing the node.*/
	NIAGARAEDITOR_API virtual bool CanAddToGraph(UNiagaraGraph* TargetGraph, FString& OutErrorMsg) const;

	/** Request a pin type change to a specific type. */
	NIAGARAEDITOR_API void RequestNewPinType(UEdGraphPin* PinToChange, FNiagaraTypeDefinition NewType);

	/** Determine whether we are allowed to change a pin's type from UI. Enables the type conversion widget. */
	virtual bool AllowExternalPinTypeChanges(const UEdGraphPin* InGraphPin) const { return false; }
	/** Determine whether or not a pin can be changed to a certain type.
	 *  Used to populate the type conversion menu if external pin type changes are allowed or for wildcard responses */
	virtual bool AllowNiagaraTypeForPinTypeChange(const FNiagaraTypeDefinition& InType, UEdGraphPin* Pin) const { return true; }
	NIAGARAEDITOR_API virtual void GetWildcardPinHoverConnectionTextAddition(const UEdGraphPin* WildcardPin, const UEdGraphPin* OtherPin, ECanCreateConnectionResponse ConnectionResponse, FString& OutString) const;
	
	/** Gets which mode to use when deducing the type of numeric output pins from the types of the input pins. */
	NIAGARAEDITOR_API virtual ENiagaraNumericOutputTypeSelectionMode GetNumericOutputTypeSelectionMode() const;

	/** Convert the type of an existing numeric pin to a more known type.*/
	NIAGARAEDITOR_API virtual bool ConvertNumericPinToType(UEdGraphPin* InGraphPin, FNiagaraTypeDefinition TypeDef);

	/** Determine whether or not a pin should be renamable. */
	virtual bool IsPinNameEditable(const UEdGraphPin* GraphPinObj) const { return false; }

	/** Determine whether or not a specific pin should immediately be opened for rename.*/
	virtual bool IsPinNameEditableUponCreation(const UEdGraphPin* GraphPinObj) const { return false; }

	/** Verify that the potential rename has produced acceptable results for a pin.*/
	virtual bool VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const { return false; }

	/** Verify that the potential rename has produced acceptable results for a pin.*/
	virtual bool CommitEditablePinName(const FText& InName,  UEdGraphPin* InGraphPinObj, bool bSuppressEvents = false) { return false; }
	/** Notify the rename was cancelled.*/
	virtual bool CancelEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj) { return false; }

	/** Returns whether or not the supplied pin has a rename pending. */
	NIAGARAEDITOR_API bool GetIsPinRenamePending(const UEdGraphPin* Pin);

	/** Sets whether or not the supplied pin has a rename pending. */
	NIAGARAEDITOR_API void SetIsPinRenamePending(const UEdGraphPin* Pin, bool bInIsRenamePending);

	NIAGARAEDITOR_API bool IsParameterMapPin(const UEdGraphPin* Pin) const;
	
	/** Adds the current node information to the parameter map history
	 *
	 *  @Param	OutHistory				The resulting history
	 *  @Param	bRecursive				If true then the history of all of this node's input pins will also be added
	 *  @Param	bFilterForCompilation	If true, some nodes like static switches or reroute nodes are jumped over. The ui usually sets this to false to follow all possible paths.  
	 */
	NIAGARAEDITOR_API virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const;
	
	/** Go through all the external dependencies of this node in isolation and add them to the reference id list.*/
	virtual void GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const {};

	/** Traces one of this node's output pins to its source output pin.
	 *
	 *  @Param	LocallyOwnedOutputPin	The pin to trace, must be a child of this node
	 *  @Param	bFilterForCompilation	If true, some nodes like static switches or reroute nodes are jumped over. The ui usually sets this to false to follow all possible paths.  
	 */
	NIAGARAEDITOR_API virtual UEdGraphPin* GetTracedOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bFilterForCompilation, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace = nullptr) const;

	/** Traces a node's output pins to its source output pin.
	*
	*  @Param	LocallyOwnedOutputPin	The pin to trace
	*  @Param	bFilterForCompilation	If true, some nodes like static switches or reroute nodes are jumped over. The ui usually sets this to false to follow all possible paths.  
	*/
	static NIAGARAEDITOR_API UEdGraphPin* TraceOutputPin(UEdGraphPin* LocallyOwnedOutputPin, bool bFilterForCompilation = true, TArray<const UNiagaraNode*>* OutNodesVisitedDuringTrace = nullptr);

	/** Allows a node to replace a pin that is about to be compiled with another pin. This can be used for either optimizations or features such as the static switch. Returns true if the pin was successfully replaced, false otherwise. */
	NIAGARAEDITOR_API virtual bool SubstituteCompiledPin(FTranslator* Translator, UEdGraphPin** LocallyOwnedPin);

	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* LocallyOwnedOutputPin) const override { return nullptr; }
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* LocallyOwnedOutputPin, ENiagaraScriptUsage InUsage) const { return nullptr; }

	/** Identify that this node has undergone changes that will require synchronization with a compiled script.*/
	NIAGARAEDITOR_API void MarkNodeRequiresSynchronization(FString Reason, bool bRaiseGraphNeedsRecompile);

	/** Get the change id for this node. This change id is updated whenever the node is manipulated in a way that should force a recompile.*/
	const FGuid& GetChangeId() const { return ChangeId; }

	/** Set the change id for this node to an explicit value. This should only be called by internal code. */
	NIAGARAEDITOR_API void ForceChangeId(const FGuid& InId, bool bRaiseGraphNeedsRecompile);

	NIAGARAEDITOR_API FOnNodeVisualsChanged& OnVisualsChanged();

	virtual void UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const { };

	/** Old style compile hash code. To be removed in the future.*/
	NIAGARAEDITOR_API virtual void UpdateCompileHashForNode(FSHA1& HashState) const;

	/** Entry point for generating the compile hash.*/
	NIAGARAEDITOR_API bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;

	static NIAGARAEDITOR_API bool SetPinDefaultToTypeDefaultIfUnset(UEdGraphPin* InPin);

protected:
	/** Pin type changes are handled by the individual subclasses to account for pin sets (like the if node or select node) */
	virtual bool OnNewPinTypeRequested(UEdGraphPin* PinToChange, FNiagaraTypeDefinition NewType) { return false; }
	
	/** Go through all class members for a given UClass on this object and hash them into the visitor.*/
	NIAGARAEDITOR_API virtual bool GenerateCompileHashForClassMembers(const UClass* InClass, FNiagaraCompileHashVisitor* InVisitor) const;

	/** Write out the specific entries for UNiagaraNode into the visitor hash. */
	NIAGARAEDITOR_API bool NiagaraNodeAppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;

	/** Write out the specific entries of this pin to the visitor hash.*/
	NIAGARAEDITOR_API virtual bool PinAppendCompileHash(const UEdGraphPin* InPin, FNiagaraCompileHashVisitor* InVisitor) const;
	
	/** Helper function to hash arbitrary UProperty entries (Arrays, Maps, Structs, etc).*/
	NIAGARAEDITOR_API virtual bool NestedPropertiesAppendCompileHash(const void* Container, const UStruct* Struct, EFieldIteratorFlags::SuperClassFlags IteratorFlags, const FString& BaseName, FNiagaraCompileHashVisitor* InVisitor) const;
	
	/** For a simple Plain old data type UProperty, hash the data.*/
	NIAGARAEDITOR_API virtual bool PODPropertyAppendCompileHash(const void* Container, FProperty* Property, const FString& PropertyName, FNiagaraCompileHashVisitor* InVisitor) const;

	NIAGARAEDITOR_API virtual int32 CompileInputPin(FTranslator* Translator, UEdGraphPin* Pin) const;
	NIAGARAEDITOR_API virtual bool IsValidPinToCompile(UEdGraphPin* Pin) const;

	NIAGARAEDITOR_API void NumericResolutionByPins(const UEdGraphSchema_Niagara* Schema, TArrayView<UEdGraphPin* const> InputPins, TArrayView<UEdGraphPin* const> OutputPins,
		bool bFixInline, TMap<TPair<FGuid, UEdGraphNode*>, FNiagaraTypeDefinition>* PinCache);
		
	NIAGARAEDITOR_API virtual FNiagaraTypeDefinition ResolveCustomNumericType(const TArray<FNiagaraTypeDefinition>& NonNumericInputs) const;

	/** Route input parameter map to output parameter map if it exists. Note that before calling this function,
		the input pins should have been visited already.*/
	NIAGARAEDITOR_API virtual void RouteParameterMapAroundMe(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive) const;

	/** Basically routes the pin through the parameter map builder so that it looks like a regular pin. Handles visiting the node connected as well to the input pin.*/
	NIAGARAEDITOR_API virtual void RegisterPassthroughPin(FNiagaraParameterMapHistoryBuilder& OutHistory, UEdGraphPin* InputPin, UEdGraphPin* OutputPin, bool bFilterForCompilation, bool bVisitInputPin) const;

	/** If the pin is a known name (like Engine.DeltaTime) this tries to return a default tooltip for it. */
	NIAGARAEDITOR_API bool GetTooltipTextForKnownPin(const UEdGraphPin& Pin, FText& OutTooltip) const;
	
#if WITH_EDITORONLY_DATA
	NIAGARAEDITOR_API virtual void GatherForLocalization(FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags) const override;
#endif

	template<typename ContainerType>
	void GetInputPinsInternal(ContainerType& OutInputPins) const
	{
		OutInputPins.Reset(Pins.Num());

		for (int32 PinIndex = 0; PinIndex < Pins.Num(); PinIndex++)
		{
			if (Pins[PinIndex]->Direction == EGPD_Input)
			{
				OutInputPins.Add(Pins[PinIndex]);
			}
		}
	}

	template<typename ContainerType>
	void GetOutputPinsInternal(ContainerType& OutOutputPins) const
	{
		OutOutputPins.Reset(Pins.Num());

		for (int32 PinIndex = 0; PinIndex < Pins.Num(); PinIndex++)
		{
			if (Pins[PinIndex]->Direction == EGPD_Output)
			{
				OutOutputPins.Add(Pins[PinIndex]);
			}
		}
	}

	NIAGARAEDITOR_API virtual void GetCompilationInputPins(FPinCollectorArray& InputPins) const;
	NIAGARAEDITOR_API virtual void GetCompilationOutputPins(FPinCollectorArray& OutputPins) const;

	/** The current change identifier for this node. Used to sync status with UNiagaraScripts.*/
	UPROPERTY()
	FGuid ChangeId;

	FOnNodeVisualsChanged VisualsChangedDelegate;

	TArray<FGuid> PinsGuidsWithRenamePending;
};
