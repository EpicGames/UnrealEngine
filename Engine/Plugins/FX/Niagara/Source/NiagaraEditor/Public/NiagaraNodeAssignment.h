// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraNode.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeAssignment.generated.h"

class SComboButton;
class SWidget;
class UNiagaraScript;

UCLASS(MinimalAPI)
class UNiagaraNodeAssignment : public UNiagaraNodeFunctionCall
{
	GENERATED_BODY()

public:
	int32 NumTargets() const { return AssignmentTargets.Num(); }

	NIAGARAEDITOR_API int32 FindAssignmentTarget(const FName& InName);
	NIAGARAEDITOR_API int32 FindAssignmentTarget(const FName& InName, const FNiagaraTypeDefinition& TypeDef);

	/** Set the assignment target and default value. This does not call RefreshFromExternalChanges, which you will need to do to update the internal graph. Returns true if values were changed.*/
	NIAGARAEDITOR_API bool SetAssignmentTarget(int32 Idx, const FNiagaraVariable& InVar, const FString* InDefaultValue = nullptr);
	NIAGARAEDITOR_API int32 AddAssignmentTarget(const FNiagaraVariable& InVar, const FString* InDefaultValue = nullptr);

	/** This will rename an existing input but will not refresh the underlying script.  RefreshFromExternalChanges must be called if this function returns true. */
	NIAGARAEDITOR_API bool RenameAssignmentTarget(FName OldName, FName NewName);

	const FNiagaraVariable& GetAssignmentTarget(int32 Idx) const { return AssignmentTargets[Idx]; }

	const TArray<FNiagaraVariable>& GetAssignmentTargets() const { return AssignmentTargets; }
	const TArray<FString>& GetAssignmentDefaults() const { return AssignmentDefaultValues; }

	//~ Begin EdGraphNode Interface
	NIAGARAEDITOR_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static NIAGARAEDITOR_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	NIAGARAEDITOR_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	NIAGARAEDITOR_API virtual void AllocateDefaultPins() override;

	//~ UNiagaraNodeFunctionCall interface
	virtual bool AllowDynamicPins() const override { return true; }
	NIAGARAEDITOR_API virtual bool RefreshFromExternalChanges() override;
	NIAGARAEDITOR_API virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;
	NIAGARAEDITOR_API virtual void GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const override;

	NIAGARAEDITOR_API void AddParameter(FNiagaraVariable InVar, FString InDefaultValue);
	NIAGARAEDITOR_API void RemoveParameter(const FNiagaraVariable& InVar);

	NIAGARAEDITOR_API void UpdateUsageBitmaskFromOwningScript();

	NIAGARAEDITOR_API virtual FText GetTooltipText() const override;

	FSimpleMulticastDelegate& OnAssignmentTargetsChanged() { return AssignmentTargetsChangedDelegate; }

	NIAGARAEDITOR_API TSharedRef<SWidget> CreateAddParameterMenu(const TSharedPtr<SComboButton>& AddButton);

protected:

	UPROPERTY()
	FNiagaraVariable AssignmentTarget_DEPRECATED;

	UPROPERTY()
	FString AssignmentDefaultValue_DEPRECATED;

	UPROPERTY()
	TArray<FNiagaraVariable> AssignmentTargets;

	UPROPERTY()
	TArray<FString> AssignmentDefaultValues;

	UPROPERTY()
	FString OldFunctionCallName;

	FText Title;

	FSimpleMulticastDelegate AssignmentTargetsChangedDelegate;

private:
	NIAGARAEDITOR_API void GenerateScript();

	NIAGARAEDITOR_API void InitializeScript(UNiagaraScript* NewScript);

	NIAGARAEDITOR_API int32 CalculateScriptUsageBitmask();

	NIAGARAEDITOR_API void RefreshTitle();

	NIAGARAEDITOR_API bool PostLoad_LWCFixup(int32 NiagaraVersion);
};

