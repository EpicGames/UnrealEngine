// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeInput.h"
#include "NiagaraNodeCustomHlsl.generated.h"

class UNiagaraScript;
struct FNiagaraCustomHlslInclude;

UCLASS(MinimalAPI)
class UNiagaraNodeCustomHlsl : public UNiagaraNodeFunctionCall
{
	GENERATED_UCLASS_BODY()

public:
	const FString& GetCustomHlsl() const;
	void SetCustomHlsl(const FString& InCustomHlsl);

	bool IsShaderCodeShown() const;
	void SetShaderCodeShown(bool bInShown);

	void GetIncludeFilePaths(TArray<FNiagaraCustomHlslInclude>& OutCustomHlslIncludeFilePaths) const;

	UPROPERTY()
	ENiagaraScriptUsage ScriptUsage;

	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;

	FText GetHlslText() const;
	void OnCustomHlslTextCommitted(const FText& InText, ETextCommit::Type InType);

	bool GetTokens(TArray<FStringView>& OutTokens, bool IncludeComments = true, bool IncludeWhitespace = true) const;

	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;
	virtual void GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const override;

	// Replace items in the tokens array if they start with the src string or optionally src string and a namespace delimiter
	static uint32 ReplaceExactMatchTokens(TArray<FString>& Tokens, FStringView SrcString, FStringView ReplaceString, bool bAllowNamespaceSeparation);

	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const override;
	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType, EEdGraphPinDirection InDirection) const override;

	virtual bool ReferencesVariable(const FNiagaraVariableBase& InVar) const;

	static bool GetTokensFromString(const FString& InHlsl, TArray<FStringView>& OutTokens, bool IncludeComments = true, bool IncludeWhitespace = true);

	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	void InitAsCustomHlslDynamicInput(const FNiagaraTypeDefinition& OutputType);

	bool CallsImpureDataInterfaceFunctions() const;

protected:
	virtual bool AllowDynamicPins() const override { return true; }
	virtual bool GetValidateDataInterfaces() const override { return false; }
	virtual bool VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const override;
	virtual bool IsPinNameEditableUponCreation(const UEdGraphPin* Pin) const override;
	virtual bool IsPinNameEditable(const UEdGraphPin* Pin) const override;
	virtual bool CommitEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj, bool bSuppressEvents = false) override;
	virtual bool CancelEditablePinName(const FText& InName, UEdGraphPin* InGraphPinObj) override;

	virtual bool CanModifyPin(const UEdGraphPin* Pin) const override { return UNiagaraNodeWithDynamicPins::CanModifyPin(Pin); }
	virtual bool CanRenamePin(const UEdGraphPin* Pin) const override { return UNiagaraNodeWithDynamicPins::CanRenamePin(Pin); }
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const override { return UNiagaraNodeWithDynamicPins::CanRemovePin(Pin); }
	virtual bool CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const override {	return UNiagaraNodeWithDynamicPins::CanMovePin(Pin, DirectionToMove); }

	/** Called when a new typed pin is added by the user. */
	virtual void OnNewTypedPinAdded(UEdGraphPin*& NewPin) override;

	/** Called when a pin is renamed. */
	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldPinName) override;
	
	/** Removes a pin from this node with a transaction. */
	virtual void RemoveDynamicPin(UEdGraphPin* Pin) override;

	virtual void MoveDynamicPin(UEdGraphPin* Pin, int32 DirectionToMove) override;

	void RebuildSignatureFromPins();
	UEdGraphPin* PinPendingRename;

private:
	UPROPERTY(EditAnywhere, Category = "HLSL", meta = (MultiLine = true))
	FString CustomHlsl;

	// Links to hlsl files that will be included by the translator. These external files are not watched by the engine, so changes to them do not automatically trigger a recompile of Niagara scripts.
	UPROPERTY(EditAnywhere, Category = "HLSL")
	TArray<FFilePath> AbsoluteIncludeFilePaths;

	// Links to hlsl files that will be included by the translator. These paths are resolved with the virtual shader paths registered in the engine.
	// For example, /Plugin/FX/Niagara maps to /Engine/Plugins/FX/Niagara/Shaders. Custom mappings can be added via AddShaderSourceDirectoryMapping().
	UPROPERTY(EditAnywhere, Category = "HLSL")
	TArray<FString> VirtualIncludeFilePaths;
	
	// Is the shader code UI shown?
	UPROPERTY()
	bool bIsShaderCodeShown;
};
