// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraNode.h"
#include "NiagaraNodeDataSetBase.generated.h"

UCLASS(MinimalAPI)
class UNiagaraNodeDataSetBase : public UNiagaraNode
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY()
	FNiagaraDataSetID DataSet;

	UPROPERTY()
	TArray<FNiagaraVariable> Variables;
	
	UPROPERTY()
	TArray<FString> VariableFriendlyNames;

	UPROPERTY()
	TObjectPtr<const UStruct> ExternalStructAsset;

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin EdGraphNode Interface
	virtual FLinearColor GetNodeTitleColor() const override;
	//~ End EdGraphNode Interface

	//~ Begin UNiagaraNode Interface
	virtual bool RefreshFromExternalChanges() override;
	//~ End UNiagaraNode Interface

	bool InitializeFromStruct(const UStruct* ReferenceStruct);
		
protected:
	void AddParameterMapPins();
	bool InitializeFromStructInternal(const UStruct* PayloadStruct);

	bool IsSynchronizedWithStruct(bool bIgnoreConditionVar, FString* Issues, bool bLogIssues = true) const;
	virtual bool SynchronizeWithStruct();
	bool GetSupportedNiagaraTypeDef(const FProperty* Property, FNiagaraTypeDefinition& TypeDef) const;

	static const FName ConditionVarName;
	static const FName ParamMapInVarName;
	static const FName ParamMapOutVarName;
};

