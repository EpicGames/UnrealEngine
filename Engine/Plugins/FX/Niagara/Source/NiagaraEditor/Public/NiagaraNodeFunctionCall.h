// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraMessages.h"
#include "NiagaraMessageStore.h"
#include "NiagaraNodeWithDynamicPins.h"
#include "NiagaraNodeInput.h"
#include "UpgradeNiagaraScriptResults.h"

#include "NiagaraNodeFunctionCall.generated.h"

class UNiagaraScript;
class UNiagaraMessageData;

USTRUCT()
struct FNiagaraPropagatedVariable
{
	GENERATED_USTRUCT_BODY()

public:

	FNiagaraPropagatedVariable() : FNiagaraPropagatedVariable(FNiagaraVariable()) {}

	FNiagaraPropagatedVariable(FNiagaraVariable SwitchParameter) : SwitchParameter(SwitchParameter), PropagatedName(FString()) {}

	UPROPERTY()
	FNiagaraVariable SwitchParameter;

	/** If set, then this overrides the name of the switch parameter when propagating. */
	UPROPERTY()
	FString PropagatedName;

	FNiagaraVariable ToVariable() const
	{
		FNiagaraVariable Copy = SwitchParameter;
		if (!PropagatedName.IsEmpty())
		{
			Copy.SetName(FName(*PropagatedName));
		}
		return Copy;
	}

	bool operator==(const FNiagaraPropagatedVariable& Other)const
	{
		return SwitchParameter == Other.SwitchParameter;
	}
};

UCLASS(MinimalAPI)
class UNiagaraNodeFunctionCall : public UNiagaraNodeWithDynamicPins
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FOnInputsChanged);

	UPROPERTY(EditAnywhere, Category = "Function", meta = (ForceShowEngineContent = true, ForceShowPluginContent = true))
	TObjectPtr<UNiagaraScript> FunctionScript;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, Category = "Version Details")
	FGuid SelectedScriptVersion;

	/** 
	 * A path to a script asset which can be used to assign the function script the first time that
	 * default pins are generated. This is used so that the function nodes can be populated in the graph context
	 * menu without having to load all of the actual script assets.
	 */
	UPROPERTY(Transient, meta = (SkipForCompileHash = "true"))
	FName FunctionScriptAssetObjectPath;

	/** Some functions can be provided a signature directly rather than a script. */
	UPROPERTY()
	FNiagaraFunctionSignature Signature;

	UPROPERTY(VisibleAnywhere, Category = "Function")
	TMap<FName, FName> FunctionSpecifiers;

	/** All the input values the function propagates to the next higher caller instead of forcing the user to set them directly. */
	UPROPERTY()
	TArray<FNiagaraPropagatedVariable> PropagatedStaticSwitchParameters;

	/** Can be used by the ui after a version change to display change notes */
	UPROPERTY(meta = (SkipForCompileHash = "true"))
	FGuid PreviousScriptVersion;

	/** Can be used by the ui after a version change to display change notes */
	UPROPERTY(meta = (SkipForCompileHash = "true"))
	FString PythonUpgradeScriptWarnings;

	UPROPERTY()
	ENiagaraFunctionDebugState DebugState;

	/** Controls whether the debug state of the current function gets propagated into this function call. */
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bInheritDebugStatus = true;

	NIAGARAEDITOR_API bool HasValidScriptAndGraph() const;

	//Begin UObject interface
	virtual void PostLoad()override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//End UObject interface
	
	//~ Begin UNiagaraNode Interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual UObject* GetReferencedAsset() const override;
	virtual void OpenReferencedAsset() const override;
	virtual bool RefreshFromExternalChanges() override;
	virtual ENiagaraNumericOutputTypeSelectionMode GetNumericOutputTypeSelectionMode() const override;
	virtual bool CanAddToGraph(UNiagaraGraph* TargetGraph, FString& OutErrorMsg) const override;
	virtual void GatherExternalDependencyData(ENiagaraScriptUsage InUsage, const FGuid& InUsageId, FNiagaraScriptHashCollector& HashCollector) const override;
	virtual void UpdateCompileHashForNode(FSHA1& HashState) const override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual TSharedRef<SWidget> CreateTitleRightWidget() override;
	//End UNiagaraNode interface

	virtual void UpdateReferencedStaticsHashForNode(FSHA1& HashState) const;

	//~ Begin EdGraphNode Interface
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	
	/** Returns true if this node is deprecated */
	virtual bool IsDeprecated() const override;
	//~ End EdGraphNode Interface

	//~ Begin UNiagaraNodeWithDynamicPins Interface
	virtual void CollectAddPinActions(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin) const override;
	//~ End UNiagaraNodeWithDynamicPins Interface

	/** When overriding an input value, this updates which variable guid was bound to which input name, so it can be reassigned when the input is renamed.*/
	void UpdateInputNameBinding(const FGuid& BoundVariableGuid, const FName& BoundName);

	bool FindAutoBoundInput(const UNiagaraNodeInput* InputNode, const UEdGraphPin* PinToAutoBind, FNiagaraVariable& OutFoundVar, ENiagaraInputNodeUsage& OutNodeUsage) const;

	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;

	/** if bDeferOverridePinUpdate is true then it is the caller's responsibility to call UpdateOverridePins() */
	NIAGARAEDITOR_API void ChangeScriptVersion(FGuid NewScriptVersion, const FNiagaraScriptVersionUpgradeContext& UpgradeContext, bool bShowNotesInStack = false, bool bDeferOverridePinUpdate = false);
	NIAGARAEDITOR_API void UpdateOverridePins(const FNiagaraScriptVersionUpgradeContext& UpgradeContext);

	FString GetFunctionName() const { return FunctionDisplayName; }
	NIAGARAEDITOR_API UNiagaraGraph* GetCalledGraph() const;
	NIAGARAEDITOR_API ENiagaraScriptUsage GetCalledUsage() const;
	NIAGARAEDITOR_API UNiagaraScriptSource* GetFunctionScriptSource() const;
	NIAGARAEDITOR_API FVersionedNiagaraScriptData* GetScriptData() const;

	/** Walk through the internal script graph for an ParameterMapGet nodes and see if any of them specify a default for VariableName.*/
	UEdGraphPin* FindParameterMapDefaultValuePin(const FName VariableName, ENiagaraScriptUsage InParentUsage, FCompileConstantResolver ConstantResolver) const;
	void FindParameterMapDefaultValuePins(TConstArrayView<FName> VariableNames, ENiagaraScriptUsage InParentUsage, const FCompileConstantResolver& ConstantResolver, TArrayView<UEdGraphPin*> DefaultPins);

	/** Attempts to find the input pin for a static switch with the given name in the internal script graph. Returns nullptr if no such pin can be found. */
	UEdGraphPin* FindStaticSwitchInputPin(const FName& VariableName) const;

	/** checks to see if this called function contains any debug switches */
	NIAGARAEDITOR_API bool ContainsDebugSwitch() const;

	/** Tries to rename this function call to a new name.  The actual name that gets applied might be different due to conflicts with existing
		nodes with the same name. */
	void SuggestName(FString SuggestedName, bool bForceSuggestion = false);

	FOnInputsChanged& OnInputsChanged();

	FNiagaraPropagatedVariable* FindPropagatedVariable(const FNiagaraVariable& Variable);
	void RemovePropagatedVariable(const FNiagaraVariable& Variable);
	void CleanupPropagatedSwitchValues();

	/** Does any automated data manipulated required to update DI function call nodes to the current version. */
	void UpgradeDIFunctionCalls();

	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;

	// Messages API
	FNiagaraMessageStore& GetMessageStore() { return MessageStore; }

	// Custom Notes API - Deprecated. Used for transferring to stack editor data
	const TArray<FNiagaraStackMessage>& GetDeprecatedCustomNotes() const { return StackMessages; }
	
	TArray<FGuid> GetBoundPinGuidsByName(FName InputName) const;

	/** Adds a static switch pin to this function call node by variable id and sets it's default value using the supplied data and marks it as
		orphaned. This allows a previously available static switch value to be retained on the node even if the the switch is no longer exposed. */
	void AddOrphanedStaticSwitchPinForDataRetention(FNiagaraVariableBase StaticSwitchVariable, const FString& StaticSwitchPinDefault);

	virtual bool GetValidateDataInterfaces() const { return true; };

	void RemoveAllDynamicPins();

	//If this is a DI function call, return the UClass of the DI. returns null for non DI function calls.
	UClass* GetDIClass()const;

	void SetFunctionSpecifier(FName Key, FName Value);

protected:
	UEdGraphPin* AddStaticSwitchInputPin(FNiagaraVariable Input);

	virtual bool AllowDynamicPins() const override { return Signature.VariadicInput() || Signature.VariadicOutput(); }
	virtual bool CanModifyPin(const UEdGraphPin* Pin) const override;
	virtual bool CanRenamePin(const UEdGraphPin* Pin) const override { return AllowDynamicPins() && Super::CanRenamePin(Pin); }
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const override { return AllowDynamicPins() && Super::CanRemovePin(Pin); }
	virtual bool CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const override { return AllowDynamicPins() && Super::CanMovePin(Pin, DirectionToMove); }

	virtual void OnNewTypedPinAdded(UEdGraphPin*& NewPin) override { Super::OnNewTypedPinAdded(NewPin); RefreshSignature(); }
	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldPinName) override { Super::OnPinRenamed(RenamedPin, OldPinName); RefreshSignature(); }
	virtual void RemoveDynamicPin(UEdGraphPin* Pin) override { Super::RemoveDynamicPin(Pin); RefreshSignature(); }
	virtual void MoveDynamicPin(UEdGraphPin* Pin, int32 MoveAmount)override { Super::MoveDynamicPin(Pin, MoveAmount); RefreshSignature(); }

	void RefreshSignature();

	bool IsBaseSignatureOfDataInterfaceFunction(const UEdGraphPin* Pin) const;
	
	/** Resets the node name based on the referenced script or signature. Guaranteed unique within a given graph instance.*/
	void ComputeNodeName(FString SuggestedName = FString(), bool bForceSuggestion = false);

	void SetPinAutoGeneratedDefaultValue(UEdGraphPin& FunctionInputPin, UNiagaraNodeInput& FunctionScriptInputNode);

	bool IsValidPropagatedVariable(const FNiagaraVariable& Variable) const;

	void UpdateNodeErrorMessage();

	void FixupFunctionScriptVersion();

	void UpdatePinTooltips();

	void UpdateStaticSwitchPinsWithPersistentGuids();

	/** Adjusted every time that we compile this script. Lets us know that we might differ from any cached versions.*/
	UPROPERTY(meta = (SkipForCompileHash="true"))
	FGuid CachedChangeId;

	/** If a script version we reference goes away we select a fallback version, but save the original version to generate warnings. */
	UPROPERTY()
	FGuid InvalidScriptVersionReference;

	UPROPERTY()
	FString FunctionDisplayName;

	/* Marking those properties explicitly as editoronly_data will make localization not pick these up. */
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (SkipForCompileHash="true"))
	TMap<FGuid, TObjectPtr<UNiagaraMessageData>> MessageKeyToMessageMap_DEPRECATED;

	UPROPERTY(meta = (SkipForCompileHash="true"))
	FNiagaraMessageStore MessageStore;
	
	UPROPERTY(meta = (SkipForCompileHash="true"))
	TArray<FNiagaraStackMessage> StackMessages;
#endif
	UPROPERTY(meta = (SkipForCompileHash="true"))
	TMap<FGuid, FName> BoundPinNames;

	FOnInputsChanged OnInputsChangedDelegate;
};

