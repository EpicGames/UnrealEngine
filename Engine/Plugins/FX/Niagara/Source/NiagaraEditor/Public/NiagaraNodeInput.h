// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraNode.h"
#include "NiagaraNodeInput.generated.h"

class UEdGraphPin;
class UNiagaraGraph;


USTRUCT()
struct FNiagaraInputExposureOptions
{
	GENERATED_USTRUCT_BODY()
		
	FNiagaraInputExposureOptions()
		: bExposed(1)
		, bRequired(1)
		, bCanAutoBind(0)
		, bHidden(0)
	{}

	/** If true then this is exposed as an input to it's caller - turning this off makes sense when the input is defined in a function script and should not be visible to the caller. */
	UPROPERTY(EditAnywhere, DisplayName="Expose Input", Category = Exposure)
	uint32 bExposed : 1;

	/** If true then this input is required to be set by the caller. For optional values (e.g. values behind an edit condition) this can be set to false, so the translator uses the default value instead. */
	UPROPERTY(EditAnywhere, DisplayName="Suppress Default Value", Category = Exposure, meta = (editcondition = "bExposed"))
	uint32 bRequired : 1;

	/** If this input can auto-bind to system parameters and emitter attributes. Will never auto bind to custom parameters. */
	UPROPERTY()
	uint32 bCanAutoBind : 1;

	/** If true then this input will be shown in the advanced pin section of the caller node. Only works in function scripts, does nothing in dynamic inputs or module scripts. */
	UPROPERTY(EditAnywhere, DisplayName="Advanced Display", Category = Exposure, meta = (editcondition = "bExposed"))
	uint32 bHidden : 1;
};


UCLASS(MinimalAPI)
class UNiagaraNodeInput : public UNiagaraNode
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Input, meta = (SkipForCompileHash = "true"))
	FNiagaraVariable Input;

	UPROPERTY(EditAnywhere, Category = Input)
	ENiagaraInputNodeUsage Usage;

	/** Controls where this input is relative to others in the calling node. */
	UPROPERTY(EditAnywhere, Category = Input, meta = (SkipForCompileHash = "true"))
	int32 CallSortPriority;

	/** Controls this inputs exposure to callers. */
	UPROPERTY(EditAnywhere, Category = Input, meta=(EditCondition="Usage == ENiagaraInputNodeUsage::Parameter", EditConditionHides))
	FNiagaraInputExposureOptions ExposureOptions;

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin)override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual void OnRenameNode(const FString& NewName) override;
	//~ End EdGraphNode Interface

	/** Notifies the node that the type of it's input has been changed externally. */
	void NotifyInputTypeChanged();

	/** Notifies the node that the exposure options have changed externally.*/
	void NotifyExposureOptionsChanged();

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	
	bool IsExposed()const { return ExposureOptions.bExposed && Usage == ENiagaraInputNodeUsage::Parameter; }
	bool IsRequired()const { return IsExposed() && ExposureOptions.bRequired; }
	bool IsHidden()const { return IsExposed() && ExposureOptions.bHidden; }
	bool CanAutoBind()const { return IsExposed() && ExposureOptions.bCanAutoBind; }

	bool ReferencesSameInput(UNiagaraNodeInput* Other) const;

	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;
	virtual void UpdateFunctionAliasForContext(FNiagaraGraphFunctionAliasContext& InOutFunctionAliasContext) const override;

	static const FLinearColor TitleColor_Attribute;
	static const FLinearColor TitleColor_Constant;

	/** Generate a unique name based off of the existing names in the system.*/
	static FName GenerateUniqueName(const UNiagaraGraph* Graph, FName& ProposedName, ENiagaraInputNodeUsage Usage);

	/** Generate a unique sort index based off the existing nodes in the system.*/
	static int32 GenerateNewSortPriority(const UNiagaraGraph* Graph, FName& ProposedName, ENiagaraInputNodeUsage Usage);

	/** Given an array of nodes, sort them in place by their sort order, then lexicographically if the same.*/
	static void SortNodes(TArray<UNiagaraNodeInput*>& InOutNodes);

	bool IsDataInterface() const;
	bool IsObjectAsset() const;

	UNiagaraDataInterface* GetDataInterface() const;
	void SetDataInterface(UNiagaraDataInterface* InDataInterface);
	
	TObjectPtr<UObject> GetObjectAsset() const;
	void SetObjectAsset(TObjectPtr<UObject> Object);

private:
	void DataInterfaceChanged();
	void ValidateDataInterface();

private:
	UPROPERTY(meta = (SkipForCompileHash = "true"))
	TObjectPtr<class UNiagaraDataInterface> DataInterface;

	UPROPERTY(meta = (CompileHashObjectPath = "true"))
	TObjectPtr<UObject> ObjectAsset;
};
