// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "NiagaraEditorCommon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SOverlay.h"
#include "NiagaraNode.h"
#include "NiagaraNodeWithDynamicPins.h"
#include "NiagaraNodeOp.generated.h"

class SGraphNode;
class SGraphPin;
class SVerticalBox;

USTRUCT()
struct FAddedPinData
{
	GENERATED_BODY()
		
	/** The data type of the added pin */
	UPROPERTY()
	FEdGraphPinType PinType;

	/** The name type of the added pin */
	UPROPERTY()
	FName PinName;
};

UCLASS(MinimalAPI)
class UNiagaraNodeOp : public UNiagaraNodeWithDynamicPins
{
	GENERATED_UCLASS_BODY()

public:

	/** Name of operation */
	UPROPERTY()
	FName OpName;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	TArray<FAddedPinData> AddedPins;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	bool bAllStatic;

	//~ Begin UObject interface
	virtual void PostLoad() override;
	//~ End UObject interface

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	//~ End EdGraphNode Interface

	//~ Begin UNiagaraNode Interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;
	virtual bool RefreshFromExternalChanges() override;
	virtual ENiagaraNumericOutputTypeSelectionMode GetNumericOutputTypeSelectionMode() const override;
	virtual bool GenerateCompileHashForClassMembers(const UClass* InClass, FNiagaraCompileHashVisitor* InVisitor) const override;
	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const;
	virtual FText GetCompactTitle() const override;
	virtual bool ShouldShowPinNamesInCompactMode() override;
	virtual TOptional<float> GetCompactModeFontSizeOverride() const override;
	//~ End UNiagaraNode Interface

	//~ Begin UNiagaraNodeWithDynamicPins Interface
	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	//~ End UNiagaraNodeWithDynamicPins Interface


protected:
	//~ Begin EdGraphNode Interface
	virtual void OnPinRemoved(UEdGraphPin* PinToRemove) override;
	//~ End EdGraphNode Interface

	//~ Begin UNiagaraNodeWithDynamicPins Interface
	virtual bool AllowDynamicPins() const override;
	virtual bool CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const override { return false; }
	virtual void OnNewTypedPinAdded(UEdGraphPin*& NewPin) override;
	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldName) override;
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const override;
	//~ End UNiagaraNodeWithDynamicPins Interface

	virtual void OnPostSynchronizationInReallocatePins() override;
	virtual FNiagaraTypeDefinition ResolveCustomNumericType(const TArray<FNiagaraTypeDefinition>& NonNumericInputs) const override;

	void HandleStaticInputPinUpgrade(UEdGraphPin* InputPin);
	void HandleStaticOutputPinUpgrade();
	
private:
	FName GetUniqueAdditionalPinName() const;
};



