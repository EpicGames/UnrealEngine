// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraNode.h"
#include "NiagaraScript.h"
#include "NiagaraNodeOutput.generated.h"

UCLASS(MinimalAPI)
class UNiagaraNodeOutput : public UNiagaraNode
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(VisibleAnywhere, Category = Output, meta = (SkipForCompileHash = "true"))
	TArray<FNiagaraVariable> Outputs;

	UPROPERTY()
	ENiagaraScriptUsage ScriptType;

	UPROPERTY()
	FGuid ScriptTypeId;

public:
	UPROPERTY()
	int32 ScriptTypeIndex_DEPRECATED;

public:

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual bool CanUserDeleteNode() const override;
	virtual bool CanDuplicateNode() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const { return true; }
	//~ End EdGraphNode Interface

	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;

	/** Notifies the node that it's output variables have been modified externally. */
	void NotifyOutputVariablesChanged();

	virtual void Compile(FTranslator* Translator, TArray<int32>& OutputExpressions) const override;
	const TArray<FNiagaraVariable>& GetOutputs() const {return Outputs;}
	
	/** Gets the usage of this graph root. */
	ENiagaraScriptUsage GetUsage() const { return ScriptType; }
	void SetUsage(ENiagaraScriptUsage InUsage) { ScriptType = InUsage; }

	/** Gets the id of the usage of this script.  For use when there are multiple scripts with the same usage e.g. Events. */
	FGuid GetUsageId() const { return ScriptTypeId; }
	void SetUsageId(FGuid InId) { ScriptTypeId = InId; }

	/** Gets the Stack Context replacement name if it is different than the default for this stage. Note that this doesn't work in cloned graphs during translation, but will work in general editor evaluation.*/
	TOptional<FName> GetStackContextOverride() const; 
	/** Gets the Stack Context replacement names for the owning emitter if it is different than the default for this stage. Note that this doesn't work in cloned graphs during translation, but will work in general editor evaluation.*/
	TArray<FName> GetAllStackContextOverrides() const;

protected:
	virtual int32 CompileInputPin(FTranslator* Translator, UEdGraphPin* Pin) const override;

	/** Removes a pin from this node with a transaction. */
	virtual void RemoveOutputPin(UEdGraphPin* Pin);

private:

	/** Gets the display text for a pin. */
	FText GetPinNameText(UEdGraphPin* Pin) const;

	bool VerifyPinNameTextChanged(const FText& InText, FText& OutErrorMessage, UEdGraphPin* Pin) const;

	/** Called when a pin's name text is committed. */
	void PinNameTextCommitted(const FText& Text, ETextCommit::Type CommitType, UEdGraphPin* Pin);
};

