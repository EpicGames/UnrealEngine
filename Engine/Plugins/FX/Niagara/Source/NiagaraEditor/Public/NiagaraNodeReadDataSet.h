// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraNodeDataSetBase.h"
#include "NiagaraNodeReadDataSet.generated.h"

UCLASS(MinimalAPI)
class UNiagaraNodeReadDataSet : public UNiagaraNodeDataSetBase
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End EdGraphNode Interface

	//~Begin UNiagaraGraph Interface
	virtual bool CanAddToGraph(UNiagaraGraph* TargetGraph, FString& OutErrorMsg) const override;

	//~End UNiagaraGraph Interface
	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;

	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;
};

