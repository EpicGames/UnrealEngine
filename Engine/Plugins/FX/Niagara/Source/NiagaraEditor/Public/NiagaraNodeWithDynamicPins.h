// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraNode.h"
#include "Styling/SlateTypes.h"
#include "NiagaraActions.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "NiagaraNodeWithDynamicPins.generated.h"

class UEdGraphPin;

DECLARE_DELEGATE_OneParam(FOnAddParameter, const FNiagaraVariable&);

/** A base node for niagara nodes with pins which can be dynamically added and removed by the user. */
UCLASS(Abstract)
class UNiagaraNodeWithDynamicPins : public UNiagaraNode
{
public:
	GENERATED_BODY()

	//~ UEdGraphNode interface
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }

	/** Requests a new pin be added to the node with the specified direction, type, and name. */
	UEdGraphPin* RequestNewTypedPin(EEdGraphPinDirection Direction, const FNiagaraTypeDefinition& Type, FName InName);

	/** Requests a new pin be added to the node with the specified direction and type. */
	UEdGraphPin* RequestNewTypedPin(EEdGraphPinDirection Direction, const FNiagaraTypeDefinition& Type);

	/** Helper to identify if a pin is an Add pin.*/
	static bool IsAddPin(const UEdGraphPin* Pin);

	/** Helper to identify if a pin is an Add pin.*/
	bool IsExecPin(const UEdGraphPin* Pin) const;

	/** Determine whether or not a Niagara type is supported for an Add Pin possibility.*/
	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType) const;
	virtual bool AllowNiagaraTypeForAddPin(const FNiagaraTypeDefinition& InType, EEdGraphPinDirection InDirection) const { return AllowNiagaraTypeForAddPin(InType); };

	/** Request a new pin. */
	void AddParameter(FNiagaraVariable Parameter, TEnumAsByte<EEdGraphPinDirection> Direction);
	void AddParameter(const UNiagaraScriptVariable* ScriptVar, TEnumAsByte<EEdGraphPinDirection> Direction);
	UEdGraphPin* AddParameterPin(FNiagaraVariable Parameter, TEnumAsByte<EEdGraphPinDirection> Direction);
	UEdGraphPin* AddParameterPin(const UNiagaraScriptVariable* ScriptVar, TEnumAsByte<EEdGraphPinDirection> Direction);
	void AddExistingParameter(FNiagaraVariable Parameter, const UEdGraphPin* AddPin);

	/** Convenience method to determine whether this Node is a Map Get or Map Set when adding a parameter through the parameter panel. */
	virtual EEdGraphPinDirection GetPinDirectionForNewParameters() { return EEdGraphPinDirection::EGPD_MAX; };

	/** Virtual functions to allow for custom actions in Add Pin menus. */
	virtual void CollectAddPinActions(FNiagaraMenuActionCollector& Collector, UEdGraphPin* AddPin)const {}

protected:
	virtual bool AllowDynamicPins() const { return true; }

	/** Creates an add pin on the node for the specified direction. */
	void CreateAddPin(EEdGraphPinDirection Direction);

	/** Called when a new typed pin is added by the user. */
	virtual void OnNewTypedPinAdded(UEdGraphPin*& NewPin) { }
	
	/** Called in subclasses to restrict renaming.*/
	/** Verify that the potential rename has produced acceptable results for a pin.*/
	virtual bool VerifyEditablePinName(const FText& InName, FText& OutErrorMessage, const UEdGraphPin* InGraphPinObj) const override { OutErrorMessage = FText::GetEmpty(); return true; }

	/** Called when a pin is renamed. */
	virtual void OnPinRenamed(UEdGraphPin* RenamedPin, const FString& OldPinName) { }

	virtual bool CanRenamePinFromContextMenu(const UEdGraphPin* Pin) const { return CanRenamePin(Pin); }

	/** Called to determine if this node can modify a given pin at all. */
	virtual bool CanModifyPin(const UEdGraphPin* Pin) const;
	
	/** Called to determine if a pin can be renamed by the user. */
	virtual bool CanRenamePin(const UEdGraphPin* Pin) const;

	/** Called to determine if a pin can be removed by the user. */
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const;
	
	/** Called to determine if a pin can be moved by the user. Negative values for up, positive for down. */
	virtual bool CanMovePin(const UEdGraphPin* Pin, int32 DirectionToMove) const;

	/** Removes a pin from this node with a transaction. */
	virtual void RemoveDynamicPin(UEdGraphPin* Pin);

	/** Moves a pin among the pins of the same direction. Negative values for up, positive for down. */
	virtual void MoveDynamicPin(UEdGraphPin* Pin, int32 MoveAmount);

	virtual bool OnVerifyTextChanged(const FText& NewText, FText& OutMessage) { return true; };

	virtual bool IsValidPinToCompile(UEdGraphPin* Pin) const override;

	virtual void GetCompilationInputPins(FPinCollectorArray& InputPins) const override;
	virtual void GetCompilationOutputPins(FPinCollectorArray& OutputPins) const override;

private:

	/** Gets the display text for a pin. */
	FText GetPinNameText(UEdGraphPin* Pin) const;

	/** Called when a pin's name text is committed. */
	void PinNameTextCommitted(const FText& Text, ETextCommit::Type CommitType, UEdGraphPin* Pin);
	
	void RenameDynamicPinFromMenu(UEdGraphPin* Pin);

	void RemoveDynamicPinFromMenu(UEdGraphPin* Pin);

	void MoveDynamicPinFromMenu(UEdGraphPin* Pin, int32 DirectionToMove);
	
public:
	/** The sub category for add pins. */
	static const FName AddPinSubCategory;
};
