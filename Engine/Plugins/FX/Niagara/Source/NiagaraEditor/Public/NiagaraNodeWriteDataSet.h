// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraNodeDataSetBase.h"
#include "NiagaraNodeWriteDataSet.generated.h"

UCLASS(MinimalAPI)
class UNiagaraNodeWriteDataSet : public UNiagaraNodeDataSetBase
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, Category = DataSet)
	FName EventName;

	//TODO: DIRECT AND APPEND-CONSUME 
// 	UPROPERTY(EditAnywhere, Category = DataSet)
// 	ENiagaraDataSetAccessMode AccessMode;
	
	//~ Begin UObject interface
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End EdGraphNode Interface

	virtual void Compile(FTranslator* Translator, TArray<int32>& Outputs) const override;

	virtual void BuildParameterMapHistory(FNiagaraParameterMapHistoryBuilder& OutHistory, bool bRecursive = true, bool bFilterForCompilation = true) const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual bool SynchronizeWithStruct() override;
protected:
	void AddConditionPin(int32 PinIndex = -1);
};

