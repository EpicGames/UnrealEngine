// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEditorUtilities.h"

/** A set of selected objects which calls a delegate any time it is changed. */
template<typename SelectedItemType>
class TNiagaraSelection
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnSelectedObjectsChanged);

	/** Gets the set of selected objects. */
	const TSet<SelectedItemType>& GetSelectedObjects() const
	{
		return SelectedObjects;
	}

	const void* GetAdditionalSelectionInfo() const
	{
		return AdditionalSelectionInfo;
	}

	/** Replaces the currently selected set of objects with the supplied object. */
	void SetSelectedObject(SelectedItemType SelectedObject, const void* InSelectionInfo)
	{
		AdditionalSelectionInfo = InSelectionInfo;
		if (SelectedObjects.Num() == 1 && SelectedObjects.Contains(SelectedObject))
		{
			// Refresh the delegate, in case a different object selection has been used in 
			// a shared panel (but using a different selection, so this selection would not change)
			OnSelectedObjectsChangedDelegate.Broadcast();
			return;
		}

		SelectedObjects.Empty();
		SelectedObjects.Add(SelectedObject);
		OnSelectedObjectsChangedDelegate.Broadcast();
	}

	/** Replaces the currently selected set of objects with the supplied object. */
	void SetSelectedObject(SelectedItemType SelectedObject)
	{
		SetSelectedObject(SelectedObject, nullptr);
	}

	/** Replaces the currently selected set of objects with the supplied set. */
	void SetSelectedObjects(const TSet<SelectedItemType>& InSelectedObjects)
	{
		if (FNiagaraEditorUtilities::SetsMatch(SelectedObjects, InSelectedObjects) == false)
		{
			SelectedObjects.Empty();
			AdditionalSelectionInfo = nullptr;
			SelectedObjects = InSelectedObjects;
			OnSelectedObjectsChangedDelegate.Broadcast();
		}
	}

	/** Replaces the currently selected set of objects with the supplied array. */
	void SetSelectedObjects(const TArray<SelectedItemType>& InSelectedObjects)
	{
		if (FNiagaraEditorUtilities::ArrayMatchesSet(InSelectedObjects, SelectedObjects) == false)
		{
			SelectedObjects.Empty();
			AdditionalSelectionInfo = nullptr;
			SelectedObjects.Append(InSelectedObjects);
			OnSelectedObjectsChangedDelegate.Broadcast();
		}
	}

	/** Empties the currently selected set of objects. */
	void ClearSelectedObjects()
	{
		if (SelectedObjects.Num() > 0)
		{
			SelectedObjects.Empty();
			AdditionalSelectionInfo = nullptr;
			OnSelectedObjectsChangedDelegate.Broadcast();
		}
	}

	/** Gets a multicast delegate which is called any time the set of selected objects is changed. */
	FOnSelectedObjectsChanged& OnSelectedObjectsChanged()
	{
		return OnSelectedObjectsChangedDelegate;
	}

	/** Refresh all views subscribed to OnSelectedObjectsChanged. */
	void Refresh()
	{
		OnSelectedObjectsChangedDelegate.Broadcast();
	}

protected:
	/** The set of selected objects. */
	TSet<SelectedItemType> SelectedObjects;

	/** The delegate which is called whenever the set of selected objects changes. */
	FOnSelectedObjectsChanged OnSelectedObjectsChangedDelegate;

	/** Any additional data payload for the selection. */
	const void* AdditionalSelectionInfo = nullptr;
};

class FNiagaraObjectSelection : public TNiagaraSelection<FWeakObjectPtr>
{
public:

	UObject* GetFirstSelectedObject() const
	{
		for (const FWeakObjectPtr& Obj : SelectedObjects)
		{
			if (Obj.IsValid())
			{
				return Obj.Get();
			}
		}
		return nullptr;
	}
	
	TSet<UObject*> GetSelectedObjectsResolved() const
	{
		TSet<UObject*> Objects;
		for (const FWeakObjectPtr& Obj : SelectedObjects)
		{
			if (Obj.IsValid())
			{
				Objects.Add(Obj.Get());
			}
		}
		return Objects;
	}
	
	void SetSelectedObject(UObject* SelectedObject, const void* InSelectionInfo)
	{
		AdditionalSelectionInfo = InSelectionInfo;
		if (SelectedObjects.Num() == 1 && SelectedObjects.Contains(SelectedObject))
		{
			// Refresh the delegate, in case a different object selection has been used in 
			// a shared panel (but using a different selection, so this selection would not change)
			OnSelectedObjectsChangedDelegate.Broadcast();
			return;
		}

		SelectedObjects.Empty();
		SelectedObjects.Add(SelectedObject);
		OnSelectedObjectsChangedDelegate.Broadcast();
	}

	/** Replaces the currently selected set of objects with the supplied object. */
	void SetSelectedObject(UObject* SelectedObject)
	{
		SetSelectedObject(SelectedObject, nullptr);
	}

	/** Replaces the currently selected set of objects with the supplied set. */
	void SetSelectedObjects(const TSet<UObject*>& InSelectedObjects)
	{
		if (FNiagaraEditorUtilities::SetsMatch(GetSelectedObjectsResolved(), InSelectedObjects) == false)
		{
			SelectedObjects.Empty();
			AdditionalSelectionInfo = nullptr;
			for (UObject* Obj : InSelectedObjects)
			{
				SelectedObjects.Add(Obj);
			}
			OnSelectedObjectsChangedDelegate.Broadcast();
		}
	}

	/** Replaces the currently selected set of objects with the supplied array. */
	void SetSelectedObjects(const TArray<UObject*>& InSelectedObjects)
	{
		if (FNiagaraEditorUtilities::ArrayMatchesSet(InSelectedObjects, GetSelectedObjectsResolved()) == false)
		{
			SelectedObjects.Empty();
			AdditionalSelectionInfo = nullptr;
			for (UObject* Obj : InSelectedObjects)
			{
				SelectedObjects.Add(Obj);
			}
			OnSelectedObjectsChangedDelegate.Broadcast();
		}
	}
};