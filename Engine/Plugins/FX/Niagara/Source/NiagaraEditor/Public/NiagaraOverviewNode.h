// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EdGraph/EdGraphNode.h"
#include "NiagaraOverviewNode.generated.h"

class UNiagaraSystem;
class FNiagaraEmitterHandleViewModel;

UCLASS(MinimalAPI)
class UNiagaraOverviewNode : public UEdGraphNode
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API UNiagaraOverviewNode();
	NIAGARAEDITOR_API void Initialize(UNiagaraSystem* InOwningSystem);
	NIAGARAEDITOR_API void Initialize(UNiagaraSystem* InOwningSystem, FGuid InEmitterHandleGuid);
	void UpdateStatus();
	NIAGARAEDITOR_API const FGuid GetEmitterHandleGuid() const;
	NIAGARAEDITOR_API struct FNiagaraEmitterHandle* TryGetEmitterHandle() const;
	
	//~ Begin UEdGraphNode Interface
	/** Gets the name of this node, shown in title bar */
	NIAGARAEDITOR_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	NIAGARAEDITOR_API virtual FLinearColor GetNodeTitleColor() const override;

	/** Whether or not this node can be deleted by user action */
	NIAGARAEDITOR_API virtual bool CanUserDeleteNode() const override;

	/** Whether or not this node can be safely duplicated (via copy/paste, etc...) in the graph */
	NIAGARAEDITOR_API virtual bool CanDuplicateNode() const override;

	NIAGARAEDITOR_API virtual void OnRenameNode(const FString& NewName) override;

	NIAGARAEDITOR_API virtual bool GetCanRenameNode() const override;

	NIAGARAEDITOR_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;

	NIAGARAEDITOR_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	//~ End UEdGraphNode Interface

	NIAGARAEDITOR_API UNiagaraSystem* GetOwningSystem() const;

	void RequestRename() { bRenamePending = true; }
	void RenameStarted() { bRenamePending = false; }
	bool IsRenamePending() const { return bRenamePending; }

private:
	UPROPERTY()
	TObjectPtr<UNiagaraSystem> OwningSystem;

	UPROPERTY()
	FGuid EmitterHandleGuid;

	bool bRenamePending;

	static NIAGARAEDITOR_API bool bColorsAreInitialized;
	static NIAGARAEDITOR_API FLinearColor SystemColor;
	static NIAGARAEDITOR_API FLinearColor EmitterColor;
	static NIAGARAEDITOR_API FLinearColor StatelessEmitterColor;
	static NIAGARAEDITOR_API FLinearColor IsolatedColor;
	static NIAGARAEDITOR_API FLinearColor NotIsolatedColor;
};
