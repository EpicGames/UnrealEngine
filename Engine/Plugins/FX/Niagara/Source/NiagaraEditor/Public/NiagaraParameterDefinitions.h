// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraParameterDefinitionsBase.h"
#include "NiagaraTypes.h"

#include "NiagaraParameterDefinitions.generated.h"

class UNiagaraScriptVariable;


/** Helper structs to map between external parameter libraries and a set of their script variables to local script variables. */
USTRUCT()
struct FScriptVarBindingNameSubscription
{
public:
	GENERATED_BODY()

	UPROPERTY()
	FGuid ExternalScriptVarId;

	UPROPERTY()
	TArray<FGuid> InternalScriptVarIds;
};

USTRUCT()
struct FParameterDefinitionsBindingNameSubscription
{
public:
	GENERATED_BODY()

	FParameterDefinitionsBindingNameSubscription() 
		: SubscribedParameterDefinitions(nullptr)
		, BindingNameSubscriptions(TArray<FScriptVarBindingNameSubscription>())
	{};

	UPROPERTY()
	TObjectPtr<UNiagaraParameterDefinitions> SubscribedParameterDefinitions;

	UPROPERTY()
	TArray<FScriptVarBindingNameSubscription> BindingNameSubscriptions;
};


/** Collection of Niagara parameters and their metadata, which is synced to all other assets in the project.
 *  This can be used to enforce consistent naming and default values for commonly used parameters.
 */
UCLASS(MinimalAPI)
class UNiagaraParameterDefinitions : public UNiagaraParameterDefinitionsBase
{
public:
	GENERATED_BODY()

	UNiagaraParameterDefinitions(const FObjectInitializer& ObjectInitializer);
	~UNiagaraParameterDefinitions();

	//~ UObject Interface
	virtual void PostLoad() override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	void AddParameter(const FNiagaraVariable& NewVariable);
	bool HasParameter(const FNiagaraVariable& Variable);
	void FindOrAddParameter(const FNiagaraVariable& Variable);
	void RemoveParameter(const FNiagaraVariable& VariableToRemove);
	void RenameParameter(const FNiagaraVariable& VariableToRename, const FName NewName);
	const TArray<UNiagaraScriptVariable*>& GetParametersConst() const;
	virtual int32 GetChangeIdHash() const override;
	virtual TSet<FGuid> GetParameterIds() const override;

	/** Getters for script variables. 
	 *  NOTE: These public getters are implemented to support details views. Editing UNiagaraScriptVariable definitions publicly should be done via the add/remove/rename interface otherwise.
	 */
	NIAGARAEDITOR_API UNiagaraScriptVariable* GetScriptVariable(const FNiagaraVariable& Var);
	UNiagaraScriptVariable* GetScriptVariable(const FGuid& ScriptVarId);

	//~ Begin Subscribed ParameterDefinitions methods
	void SubscribeBindingNameToExternalParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions, const FGuid& ExternalScriptVarId, const FGuid& InternalScriptVarId);
	void UnsubscribeBindingNameFromExternalParameterDefinitions(const FGuid& InternalScriptVarToUnsubscribeId);

	/** Synchronize all parameter names in subscribed external parameter libraries to local parameters. */
	void SynchronizeWithSubscribedParameterDefinitions();

	/** Get all parameter libraries under the editor and niagara packages, along with those that are in the same package as the UObject this viewmodel is editing. */
	TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions) const;
	//~ End Subscribed ParameterDefinitions methods

	void NotifyParameterDefinitionsChanged();

	bool GetIsPromotedToTopInAddMenus() const { return bPromoteToTopInAddMenus; };
	
	int32 GetMenuSortOrder() const { return MenuSortOrder; };

private:
	const TArray<UNiagaraParameterDefinitions*> GetSubscribedParameterDefinitions() const;

	// If true then these parameters will appear as top level entry in add menus (e.g. in the module editor)
	UPROPERTY(EditAnywhere, Category = "Definition Preferences")
	bool bPromoteToTopInAddMenus;

	// Defines the sort order in add menus. Entries with smaller numbers are displayed first.
	UPROPERTY(EditAnywhere, Category = "Definition Preferences")
	int32 MenuSortOrder;

	UPROPERTY()
	TArray<TObjectPtr<UNiagaraScriptVariable>> ScriptVariables;

	UPROPERTY()
	TArray<FParameterDefinitionsBindingNameSubscription> ExternalParameterDefinitionsSubscriptions;
};
