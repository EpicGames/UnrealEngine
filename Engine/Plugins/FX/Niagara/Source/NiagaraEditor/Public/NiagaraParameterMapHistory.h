// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraCompilationBridge.h"
#include "NiagaraEmitter.h"
#include "NiagaraParameterMapHistoryFwd.h"
#include "NiagaraScript.h"
#include "Templates/Tuple.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

class UEdGraphPin;
class UEdGraphNode;
class FNiagaraCompilationNode;
class FNiagaraCompilationPin;
class UNiagaraGraph;
class FNiagaraHlslTranslator;
class UNiagaraNodeEmitter;
class UNiagaraNodeOutput;
class UNiagaraParameterCollection;

class FCompileConstantResolver
{
public:
	FCompileConstantResolver() : Emitter(FVersionedNiagaraEmitter()), System(nullptr), Translator(nullptr), Usage(ENiagaraScriptUsage::Function), DebugState(ENiagaraFunctionDebugState::NoDebug) {}
	FCompileConstantResolver(const FVersionedNiagaraEmitter& Emitter, ENiagaraScriptUsage Usage, ENiagaraFunctionDebugState DebugState = ENiagaraFunctionDebugState::NoDebug) : Emitter(Emitter), System(nullptr), Translator(nullptr), Usage(Usage), DebugState(DebugState) {}
	FCompileConstantResolver(const UNiagaraSystem* System, ENiagaraScriptUsage Usage, ENiagaraFunctionDebugState DebugState = ENiagaraFunctionDebugState::NoDebug) : Emitter(FVersionedNiagaraEmitter()), System(System), Translator(nullptr), Usage(Usage), DebugState(DebugState) {}
	FCompileConstantResolver(const FNiagaraHlslTranslator* Translator, ENiagaraFunctionDebugState DebugState = ENiagaraFunctionDebugState::NoDebug) : Emitter(FVersionedNiagaraEmitter()), System(nullptr), Translator(Translator), Usage(ENiagaraScriptUsage::Function), DebugState(DebugState) {}

	const UNiagaraEmitter* GetEmitter() const { return Emitter.Emitter; }
	const UNiagaraSystem* GetSystem() const { return System; }
	const FNiagaraHlslTranslator* GetTranslator() const { return Translator; }
	ENiagaraScriptUsage GetUsage() const { return Usage; }
	ENiagaraFunctionDebugState GetDebugState() const { return DebugState; }

	bool ResolveConstant(FNiagaraVariable& OutConstant) const;

	ENiagaraFunctionDebugState CalculateDebugState() const;
	FCompileConstantResolver WithDebugState(ENiagaraFunctionDebugState InDebugState) const;
	FCompileConstantResolver WithUsage(ENiagaraScriptUsage ScriptUsage) const;
	FCompileConstantResolver AsEmitter(const FVersionedNiagaraEmitter& InEmitter) const;

	// returns a hash of the data that is used when resolving constants using this resolver.  The specific
	// Emitter/System/Translator shouldn't be included but the values that it might use from those objects
	// should be
	uint32 BuildTypeHash() const;

private:
	FVersionedNiagaraEmitter Emitter;
	const UNiagaraSystem* System;
	const FNiagaraHlslTranslator* Translator;
	ENiagaraScriptUsage Usage;
	ENiagaraFunctionDebugState DebugState;
};

template<typename PinType>
struct TModuleScopedPin
{
	TModuleScopedPin() = default;
	TModuleScopedPin(const PinType* InPin, FName InName = NAME_None)
	: Pin(InPin)
	, ModuleName(InName)
	{}

	bool operator==(const TModuleScopedPin& Rhs) const
	{
		return Pin == Rhs.Pin && ModuleName == Rhs.ModuleName;
	}

	const PinType* Pin = nullptr;
	FName ModuleName = NAME_None;

	friend uint32 GetTypeHash(const TModuleScopedPin& ScopedPin)
	{
		return GetTypeHash(TTuple<const PinType*, FName>(ScopedPin.Pin, ScopedPin.ModuleName));
	}
};

struct FGraphTraversalHandle
{
public:
	FGraphTraversalHandle() {}
	FGraphTraversalHandle(const TArray<FGuid>& InputPath, TArray<FString> InFriendlyPath) : Path(InputPath), FriendlyPath(InFriendlyPath) {}

	void Append(const FGuid& Guid, const FString& FriendlyName)
	{
		FriendlyPath.Add(FriendlyName);
		Path.Add(Guid);
	}

	void PushNode(const UEdGraphNode* Node);
	void PushNode(const FNiagaraCompilationNode* Node);
	void PopNode();

	void PushPin(const UEdGraphPin* Pin);
	void PushPin(const FNiagaraCompilationPin* Pin);
	void PopPin();

	bool operator==(const FGraphTraversalHandle& Rhs) const
	{
		return Path == Rhs.Path;
	}

	friend uint32 GetTypeHash(const FGraphTraversalHandle& Ref)
	{
		uint32 Value = 0;
		for (int32 i = 0; i < Ref.Path.Num(); i++)
			Value = HashCombine(Value, GetTypeHash(Ref.Path[i]));
		return Value;
	}

	FString ToString(bool bVerbose = false) const
	{
		FString Output;
		for (const FString& Str : FriendlyPath)
			Output += TEXT("/") + Str;

		if (bVerbose)
		{
			Output += " GUID: ";
			for (const FGuid& Guid : Path)
				Output += TEXT("/") + Guid.ToString();
		}
		return Output;
	}
private:
	TArray<FGuid > Path;
	TArray<FString> FriendlyPath;
};

struct FNiagaraStaticVariableSearchContext
{
	/** Whether we are running a limited history run where we only care about collecting static variables */
	bool bCollectingStaticVariablesOnly = false;

	bool bIncludeReferencedGraphs = true;

	using FGraphKey = TTuple<TObjectKey<UNiagaraGraph>, FGuid>;
	TMap<FGraphKey, bool> CachedResults;
};

namespace FNiagaraParameterUtilities
{
	/** Get the first namespace entry for this variable. Optionally includes the trailing period.*/
	FString GetNamespace(const FNiagaraVariable& InVar, bool bIncludeDelimiter = true);


	FName ResolveEmitterAlias(const FName& InName, const FString& InAlias);


	/**
	* Remove the Particles namespace if it exists.
	*/
	FNiagaraVariable ResolveAsBasicAttribute(const FNiagaraVariable& InVar, bool bSanitizeInput = true);


	/**
	* Reverses ResolveAsBasicAttribute.
	*/
	FNiagaraVariable BasicAttributeToNamespacedAttribute(const FNiagaraVariable& InVar, bool bSanitizeInput = true);


	/** Prepends the namespace string to the variable name.*/
	FNiagaraVariable VariableToNamespacedVariable(const FNiagaraVariable& InVar, FString Namespace);

	/**
	* Does this parameter start with the "Module" namespace? Note that the emitter namespace is an alias
	* for all non-funcion/module script types and will be specialized to the function call node's name using the module.
	*/
	bool IsAliasedModuleParameter(const FNiagaraVariable& InVar);

	/**
	* Does this parameter start with the "Emitter" namespace? Note that the emitter namespace is an alias
	* for Emitter and System script types and will be specialized to the name of that specific emitter.
	*/
	bool IsAliasedEmitterParameter(const FNiagaraVariable& InVar);
	bool IsAliasedEmitterParameter(const FString& InVarName);
	/** Is this parameter in the special "System" namespace?*/
	bool IsSystemParameter(const FNiagaraVariable& InVar);
	/** Is this parameter in the special "Engine" namespace?*/
	bool IsEngineParameter(const FNiagaraVariable& InVar);
	/*** Is per instance engine parameter. */
	bool IsPerInstanceEngineParameter(const FNiagaraVariable& InVar, const FString& EmitterAlias);
	bool IsUserParameter(const FNiagaraVariable& InVar);
	bool IsRapidIterationParameter(const FNiagaraVariable& InVar);
	bool SplitRapidIterationParameterName(const FNiagaraVariable& InVar, ENiagaraScriptUsage InUsage, FString& EmitterName, FString& FunctionCallName, FString& InputName);

	/** Take an input string and make it hlsl safe.*/
	FString MakeSafeNamespaceString(const FString& InStr);

	/** Does the variable start with this namespace?*/
	bool IsInNamespace(const FNiagaraVariableBase& InVar, const FString& Namespace);

	void GetValidNamespacesForReading(const UNiagaraScript* InScript, TArray<FString>& OutputNamespaces);
	void GetValidNamespacesForReading(ENiagaraScriptUsage InScriptUsage, int32 InUsageBitmask, TArray<FString>& OutputNamespaces);
	bool IsValidNamespaceForReading(ENiagaraScriptUsage InScriptUsage, int32 InUsageBitmask, FString Namespace);
	bool IsWrittenToScriptUsage(const FNiagaraVariable& InVar, ENiagaraScriptUsage InUsage, bool bAllowDataInterfaces);

	/** Are we required to export this variable as an external constant?*/
	bool IsExportableExternalConstant(const FNiagaraVariable& InVar, const UNiagaraScript* InScript);

	/** Does this variable belong in a namespace that needs to come in as an external constant to this script?*/
	bool IsExternalConstantNamespace(const FNiagaraVariable& InVar, const UNiagaraScript* InScript, const FGuid& VersionGuid);
	bool IsExternalConstantNamespace(const FNiagaraVariable& InVar, ENiagaraScriptUsage InUsage, int32 InUsageBitmask);

	/** Take a non-namespaced variable and move it to an appropriate external constant namespace for this script type.*/
	FNiagaraVariable MoveToExternalConstantNamespaceVariable(const FNiagaraVariable& InVar, const UNiagaraScript* InScript);
	FNiagaraVariable MoveToExternalConstantNamespaceVariable(const FNiagaraVariable& InVar, ENiagaraScriptUsage InUsage);
	/**
	* Does this parameter start with the "Particles" namespace?
	*/
	bool IsAttribute(const FNiagaraVariableBase& InVar);

	/** Does this parameter contain the "Initial" namespace as one of its intermediate namespaces?*/
	bool IsInitialValue(const FNiagaraVariableBase& InVar);

	/** Does this variable name contain the "Initial" namespace as one of its intermediate namespaces? */
	bool IsInitialName(const FName& InVariableName);

	/** Does this parameter contain the "Previous" namespace as one of its intermediate namespaces?*/
	bool IsPreviousValue(const FNiagaraVariableBase& InVar);

	/** Does this variable name contain the "Previous" namespace as one of its intermediate namespaces? */
	bool IsPreviousName(const FName& InVariableName);

	/** Does this parameter contain the "Initial" namespace as one of its intermediate namespaces? If so, remove the "Initial" namespace and return the original value.*/
	FNiagaraVariable GetSourceForInitialValue(const FNiagaraVariable& InVar);

	/** Does this parameter contain the "Initial" namespace as one of its intermediate namespaces? If so, remove the "Initial" namespace and return the original value.*/
	FName GetSourceForInitialValue(const FName& InVariableName);

	/** Does this parameter contain the "Previous" namespace as one of its intermediate namespaces? If so, remove the "Previous" namespace and return the original value.*/
	FNiagaraVariable GetSourceForPreviousValue(const FNiagaraVariable& InVar);

	/** Does this parameter contain the "Previous" namespace as one of its intermediate namespaces? If so, remove the "Previous" namespace and return the original value.*/
	FName GetSourceForPreviousValue(const FName& InVariableName);

	FNiagaraVariable ConvertVariableToRapidIterationConstantName(FNiagaraVariable InVar, const TCHAR* InEmitterName, ENiagaraScriptUsage InUsage);
};

/** Traverses a Niagara node graph to identify the variables that have been written and read from a parameter map. 
* 	This class is meant to aid in UI and compilation of the graph. There are several main script types and each one interacts
*	slightly differently with the history depending on context.
*/
template<typename GraphBridge>
struct TNiagaraParameterMapHistory
{
public:
	using FPin = typename GraphBridge::FPin;
	using FNode = typename GraphBridge::FNode;
	using FOutputNode = typename GraphBridge::FOutputNode;
	using FInputPin = typename GraphBridge::FInputPin;
	using FParameterCollection = typename GraphBridge::FParameterCollection;
	using FParameterCollectionStore = typename GraphBridge::FParameterCollectionStore;

	using FModuleScopedPin = TModuleScopedPin<FPin>;

	TNiagaraParameterMapHistory();

	ENiagaraScriptUsage OriginatingScriptUsage;
	FGuid UsageGuid;
	FName UsageName;

	/** The variables that have been identified during the traversal. */
	TArray<FNiagaraVariable> Variables;

	/** The metadata associated with each variable identified during the traversal. Only gathered by FNiagaraParameterMapHistoryWithMetaDataBuilder. */
	TArray<FNiagaraVariableMetaData> VariableMetaData;

	TArray<TArray<FString> > PerVariableConstantValue;
	TArray<TArray<FString> > PerVariableConstantDefaultValue;

	TArray<FNiagaraVariable> VariablesWithOriginalAliasesIntact;

	/** Used parameter collections identified during the traversal. TODO: Need to ensure these cannot be GCd if the asset is deleted while it's being used in an in flight compilation. */
	FParameterCollectionStore EncounteredParameterCollections;
	
	/** Are there any warnings that were encountered during the traversal of the graph for a given variable? */
	TArray<FString> PerVariableWarnings;

	/** For each variable that was found, identify the pins that wrote to them in order from first to last write.*/
	TArray<TArray<TModuleScopedPin<FPin>>> PerVariableWriteHistory;

	/** For each variable that was found, identify the pins that read them from the map in order from first to last read. First of the pair has the read pin, second of the pair has the last set that wrote to the pin.*/
	struct FReadHistory
	{
		TModuleScopedPin<FPin> ReadPin;
		TModuleScopedPin<FPin> PreviousWritePin;
	};

	TArray<TArray<FReadHistory>> PerVariableReadHistory; 

	/** List of pins that manipulated the parameter map from input to output. */
	TArray<const FPin*> MapPinHistory;

	/** List of nodes that manipulated the parameter map from input to output.*/
	TArray<const FNode*> MapNodeVisitations;

	/** For each node in MapNodeVisitations, record the start index and end index of variables added within the body of the node, i.e. a Get node will record just the values it pulls out directly. A function call, however, will record all sub-graph traversals.*/
	TArray<TTuple<uint32, uint32> > MapNodeVariableMetaData;

	/** List of emitter namespaces encountered as this parameter map was built.*/
	TArray<FString> EmitterNamespacesEncountered;

	/** List of all the custom iteration source override namespaces encountered */
	TArray<FName> IterationNamespaceOverridesEncountered;

	/** List of additional DataSets to be written that were encountered during traversal. */
	TArray<FNiagaraDataSetID> AdditionalDataSetWrites;

	TMap<FGraphTraversalHandle, FString> PinToConstantValues;
	bool bParameterCollectionsSkipped = false;
	void RegisterConstantPin(const FGraphTraversalHandle& InTraversalPath, const FPin* InPin, const FString& InValue);

	bool IsVariableFromCustomIterationNamespaceOverride(const FNiagaraVariable& InVar) const;
	
	/**
	* Called in a depth-first traversal to identify a given Niagara Parameter Map pin that was touched during traversal.
	*/
	int32 RegisterParameterMapPin(const FPin* Pin);

	void RegisterConstantVariableWrite(const FString& InValue, int32 InVarIdx, bool bIsSettingDefault, bool bIsLinkNotValue);

	uint32 BeginNodeVisitation(const FNode* Node);
	void EndNodeVisitation(uint32 IndexFromBeginNode);

	/**
	* Find a variable by name with no concern for type.
	*/
	int32 FindVariableByName(const FName& VariableName, bool bAllowPartialMatch = false) const;

	/**
	* Find a variable by both name and type. 
	*/
	int32 FindVariable(const FName& VariableName, const FNiagaraTypeDefinition& Type) const;


	/**
	* Add a variable outside the normal get/set node paradigm.
	*/
	int32 AddExternalVariable(const FNiagaraVariable& InVar);

	/**
	* Get the pin that added the parameter map to the graph.
	*/
	const FPin* GetOriginalPin() const;

	/**
	* Get the output pin that we traced to build this history object.
	*/
	const FPin* GetFinalPin() const;


	/** Called to determine if a given variable should be output from a script. It is not static as it requires the overall context to include emitter namespaces visited for system scripts.*/
	bool IsPrimaryDataSetOutput(const FNiagaraVariableBase& InVar, const UNiagaraScript* InScript,   bool bAllowDataInterfaces = false,  bool bAllowStatics = false) const;
	bool IsPrimaryDataSetOutput(const FNiagaraVariableBase& InVar, ENiagaraScriptUsage InUsage, bool bAllowDataInterfaces = false, bool bAllowStatics = false) const;

	/** Get the output node associated with this graph.*/
	const FOutputNode* GetFinalOutputNode() const;

	/**
	* Helper to add a variable to the known list for a parameter map.
	*/
	int32 AddVariable(const FNiagaraVariable& InVar, const FNiagaraVariable& InAliasedVar, FName ModuleName, const FPin* InPin, TOptional<FNiagaraVariableMetaData> InMetaData = TOptional<FNiagaraVariableMetaData>());

	/** Get the default value for this variable.*/
	const FInputPin* GetDefaultValuePin(int32 VarIdx) const;

	bool ShouldIgnoreVariableDefault(const FNiagaraVariable& Var)const;

	void ConditionalUpdateAliasedVariable(int32 VariableIndex, const FNiagaraVariableBase& InAliasedVar);
};

template<typename GraphBridge>
class TNiagaraParameterMapHistoryBuilder : public GraphBridge::FBuilderExtraData
{
public:
	using FHistory = TNiagaraParameterMapHistory<GraphBridge>;
	using FOutputNode = typename GraphBridge::FOutputNode;
	using FNode = typename GraphBridge::FNode;
	using FPin = typename GraphBridge::FPin;
	using FInputPin = typename GraphBridge::FInputPin;
	using FOutputPin = typename GraphBridge::FOutputPin;
	using FGraph = typename GraphBridge::FGraph;
	using FEmitterNode = typename GraphBridge::FEmitterNode;
	using FConstantResolver = typename GraphBridge::FConstantResolver;
	using FParameterCollection = typename GraphBridge::FParameterCollection;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodeVisited, const FNode* Node);

	/** Collection of the build histories from the graph traversal.*/
	TArray<FHistory> Histories;

	/** Constructor*/
	TNiagaraParameterMapHistoryBuilder();

	virtual ~TNiagaraParameterMapHistoryBuilder() = default;

	/** Add a new parameter map to the array.*/
	int32 CreateParameterMap();

	/** Called in order to set up the correct initial context for an Output node and invokes the UNiagaraNodeOutput's BuildParameterMapHistory method.*/
	void BuildParameterMaps(const FOutputNode* OutputNode, bool bRecursive = true);

	/**
	* Called first during a node's visitation during traversal to identify that a node has been visited.
	*/
	void RegisterNodeVisitation(const FNode* Node);

	/** Important. Must be called for each routing of the parameter map. This feeds the list used by TraceParameterMapOutputPin.*/
	int32 RegisterParameterMapPin(int32 WhichParameterMap, const FPin* Pin);

	int32 RegisterConstantPin(int32 WhichConstant, const FPin* Pin);
	int32 RegisterConstantVariableWrite(int32 WhichParamMapIdx, int32 WhichConstant, int32 WhichVarIdx, bool bIsSettingDefault, bool bLinkNotValue);

	int32 RegisterConstantFromInputPin(const FInputPin* InputPin, const FString& PinDefaultValue);

	int32 GetConstantFromInputPin(const FInputPin* InputPin) const;
	int32 GetConstantFromOutputPin(const FOutputPin* OutputPin) const;
	int32 GetConstantFromVariableRead(const FNiagaraVariableBase& InVar);
	int32 AddOrGetConstantFromValue(const FString& Value);
	FString GetConstant(int32 InIndex);

	/** Records a write to a DataSet in the appropriate parameter map history */
	void RegisterDataSetWrite(int32 WhichParameterMap, const FNiagaraDataSetID& DataSet);

	uint32 BeginNodeVisitation(int32 WhichParameterMap, const FNode* Node);
	void EndNodeVisitation(int32 WhichParameterMap, uint32 IndexFromBeginNode);

	/** Trace back a pin to whom it was connected to to find the current parameter map to use.*/
	int32 TraceParameterMapOutputPin(const FOutputPin* OutputPin);

	void BeginTranslation(const UNiagaraScript* Script);

	void EndTranslation(const UNiagaraScript* Script);

	void BeginTranslation(const FString& EmitterUniqueName);
	void EndTranslation(const FString& EmitterUniqueName);

	void BeginTranslation(const UNiagaraEmitter* Emitter);

	void EndTranslation(const UNiagaraEmitter* Emitter);

	void BeginUsage(ENiagaraScriptUsage InUsage, FName InStageName = FName());
	void EndUsage();


	FGraphTraversalHandle ActivePath;

	/**
	* Record that we have entered a new function scope.
	*/
	void EnterFunction(const FString& InNodeName, const FGraph* InGraph, const FNode* Node);

	/**
	* Record that we have exited a function scope.
	*/
	void ExitFunction(const FNode* Node);

	/**
	* Record that we have entered an emitter scope.
	*/
	void EnterEmitter(const FString& InEmitterName, const FGraph* InGraph, const FNode* Node);

	/**
	* Record that we have exited an emitter scope.
	*/
	void ExitEmitter(const FString& InEmitterName, const FNode* Node);

	/**
	* Use the current alias map to resolve any aliases in this input variable name.
	*/
	FNiagaraVariable ResolveAliases(const FNiagaraVariable& InVar) const;
	
	/**
	* Has RegisterNodeVisitation been called yet on the owning node of this pin?
	*/
	bool GetPinPreviouslyVisited(const FPin* InPin) const;

	/**
	* Has RegisterNodeVisitation been called on the input node yet?
	*/
	bool GetNodePreviouslyVisited(const FNode* Node) const;

	/** If we haven't already visited the owning nodes, do so.*/
	void VisitInputPins(const FNode*, bool bFilterForCompilation);
	
	/** If we haven't already visited the owning node, do so.*/
	void VisitInputPin(const FInputPin* Pin, bool bFilterForCompilation);

	/**
	* Record that a pin writes to the parameter map. The pin name is expected to be the namespaced parameter map version of the name. If any aliases are in place, they are removed.
	*/
	int32 HandleVariableWrite(int32 ParameterMapIndex, const FInputPin* InPin);
	
	/**
	* Record that a variable write to the parameter map. The var name is expected to be the namespaced parameter map version of the name. If any aliases are in place, they are removed.
	*/
	int32 HandleVariableWrite(int32 ParameterMapIndex, const FNiagaraVariable& Var);

	/**
	* Record that a pin reads from the parameter map. The pin name is expected to be the namespaced parameter map version of the name. If any aliases are in place, they are removed.
	*/
	int32 HandleVariableRead(int32 ParameterMapIndex, const FPin* InPin, bool RegisterReadsAsVariables, const FInputPin* InDefaultPin, bool bFilterForCompilation, bool& OutUsedDefault);

	int32 HandleExternalVariableRead(int32 ParamMapIdx, const FName& InVarName);

	void HandleParameterCollectionRead(const FNiagaraVariableBase& InVar, int32 ParamMapIndex);


	/**
	* Virtual wrapper to call correct AddVariable method on the target ParameterMapHistory.
	*/
	virtual int32 AddVariableToHistory(FHistory& History, const FNiagaraVariable& InVar, const FNiagaraVariable& InAliasedVar, const FPin* InPin);
	
	/**
	* Get the string that the "Module" namespace maps to currently (if it exists)
	*/
	const FString* GetModuleAlias() const;

	/**
	* Get the string that the "Emitter" namespace maps to currently (if it exists)
	*/
	const FString* GetEmitterAlias() const;

	/** Get the node calling this sub-graph.*/
	const FNode* GetCallingContext() const;

	/** Are we currently in a top-level function call context (ie a node in the main graph or an argument into a function for the main graph.)*/
	bool InTopLevelFunctionCall(ENiagaraScriptUsage InFilterScriptType) const;

	/** Helper method to identify any matching input nodes from the calling context node to the input variable.*/
	int32 FindMatchingParameterMapFromContextInputs(const FNiagaraVariable& InVar) const;

	int32 FindMatchingStaticFromContextInputs(const FNiagaraVariable& InVar) const;

	/** In some cases, we don't want all the variables encountered in a traversal. In this case, you can filter 
	*  the map history to only include variables that are relevant to the specific script type. For instance, a System 
	*  script doesn't really care about the Particles namespace.
	*/
	void EnableScriptAllowList(bool bInEnable, ENiagaraScriptUsage InScriptType);

	bool HasCurrentUsageContext() const;
	ENiagaraScriptUsage GetCurrentUsageContext()const;
	ENiagaraScriptUsage GetBaseUsageContext()const;
	bool ContextContains(ENiagaraScriptUsage InUsage) const;

	bool GetIgnoreDisabled() const { return bIgnoreDisabled; }
	void SetIgnoreDisabled(bool bInIgnore) { bIgnoreDisabled = bInIgnore; }

	bool GetIgnoreStaticRapidIterationParameters() const { return bIgnoreStaticRapidIterationParameters; }
	void SetIgnoreStaticRapidIterationParameters(bool bInIgnore) { bIgnoreStaticRapidIterationParameters = bInIgnore; }

	bool IsInEncounteredFunctionNamespace(const FNiagaraVariable& InVar) const;
	bool IsInEncounteredEmitterNamespace(const FNiagaraVariable& InVar) const;

	/** Register any user or other external variables that could possibly be encountered but may not be declared explicitly. */
	void RegisterEncounterableVariables(TConstArrayView<FNiagaraVariable> Variables);
	const TArray<FNiagaraVariable>& GetEncounterableVariables() const {	return EncounterableExternalVariables;}

	void RegisterExternalStaticVariables(TConstArrayView<FNiagaraVariable> Variables);

	TPimplPtr<FConstantResolver, EPimplPtrMode::DeepCopy> ConstantResolver;

	/** An optional emitter handle guid you can register. This will skip parameter map building of all emitters that don't match this one, if set.
	 * Reason for this is to be able to build parameter map history for a system, while excluding certain emitters.
	 * Useful for emitter context UI that should retrieve system parameters & emitter parameters, but only emitter parameters of the given emitter.
	 */
	TOptional<FGuid> ExclusiveEmitterHandle;

	bool bShouldBuildSubHistories = true;

	int32 MaxGraphDepthTraversal = INDEX_NONE;
	int32 CurrentGraphDepth = 0;

	void SetConstantByStaticVariable(int32& OutValue, const FInputPin* InDefaultPin);
	void SetConstantByStaticVariable(int32& OutValue, const FNiagaraVariable& Var);
	int32 FindStaticVariable(const FNiagaraVariable& Var) const;

	TArray<FNiagaraVariable> StaticVariables;
	TArray<bool> StaticVariableExportable; // Should we export out these static variables to calling context?

	void GetContextuallyVisitedNodes(TArray<const FNode*>& OutVistedNodes)
	{
		if (ContextuallyVisitedNodes.Num() > 0)
			OutVistedNodes.Append(ContextuallyVisitedNodes.Last());
	}

	void IncludeStaticVariablesOnly()
	{
		StaticVariableSearchContext.bCollectingStaticVariablesOnly = true;
	}

	bool ShouldProcessDepthTraversal(const FGraph* Graph);

	FOnNodeVisited OnNodeVisitedDelegate;

protected:
	/** Internal helper function to decide whether or not we should use this static variable when merging with other
		graph traversal static variables, like Emitter graphs merging with System graphs. */
	bool IsStaticVariableExportableToOuterScopeBasedOnCurrentContext(const FNiagaraVariable& Var) const;

	/**
	* Generate the internal alias map from the current traversal state.
	*/
	void BuildCurrentAliases();

	/** Helper function called when variables are added to enable the filtering specified in EnableScriptAllowList.*/
	bool ShouldTrackVariable(const FNiagaraVariable& InVar);

	/** Helper method used to take in input script type and determine if the passed in namespaced variable is worth tracking.*/
	bool IsNamespacedVariableRelevantToScriptType(const FNiagaraVariable& InVar, ENiagaraScriptUsage ScriptType);

	bool AddParameterCollection(FParameterCollection Collection, int32 ParamMapIndex);

	/** Contains the hierarchy of nodes leading to the current graph being processed. Usually made up of FunctionCall and Emitter nodes.*/
	TArray<const FNode*> CallingContext;

	/** Contains the hierarchy of graphs leading to and including the current graph being processed. Is not in sync with CallingContext as there is an additional 0th entry for the NodeGraphDeepCopy. */
	TArray<const FGraph*> CallingGraphContext;

	TArray< TMap<const FPin*, int32> > PinToConstantIndices;
	TArray<FString> Constants;
	TArray< TMap<FNiagaraVariableBase, int32> > VariableToConstantIndices;

	/** Tracker for each context level of the parameter map index associated with a given pin. Used to trace parameter maps through the graph.*/
	TArray<TMap<const FPin*, int32> > PinToParameterMapIndices;
	/** List of previously visited nodes per context. Note that the same node may be visited multiple times across all graph traversals, but only one time per context level.*/
	TArray<TArray<const FNode*> > ContextuallyVisitedNodes;
	/** Contains the hierarchy of emitter node names leading to the current graph being processed.*/
	TArray<FName> EmitterNameContextStack;
	/** Contains the hierarchy of function call node names leading to the current graph being processed.*/
	TArray<FName> FunctionNameContextStack;
	/** Keeps track of the script usage at the current context level. This allows us to make some decisions about relevence.*/
	TArray<ENiagaraScriptUsage> RelevantScriptUsageContext;
	/** Resolved alias map for the current context level. Rebuilt by BuildCurrentAliases.*/
	FNiagaraAliasContext ResolveAliasContext;
	TArray<FName> ScriptUsageContextNameStack;

	TArray<TArray<FString> > EncounteredFunctionNames;
	TArray<FString> EncounteredEmitterNames;

	
	/** Whether or not the script allow list is active.*/
	bool bFilterByScriptAllowList;
	/** What the script type is that we should be filtering to if the allow list is enabled.*/
	ENiagaraScriptUsage FilterScriptType;

	/** Whether or not to ignore disabled nodes.*/
	bool bIgnoreDisabled;

	/** Whether we want to include ParameterCollection information */
	bool bIncludeParameterCollectionInfo;

	/** Whether or not we want to look up the rapid iteration form of a variable or just the regular form. 
	see comments in FNiagaraStackGraphUtilities::GetStackFunctionInputPinsWithoutCache for reasoning behind this.*/
	bool bIgnoreStaticRapidIterationParameters = false;

	TArray<FNiagaraVariable> EncounterableExternalVariables;

	FNiagaraStaticVariableSearchContext StaticVariableSearchContext;
};

template<typename GraphBridge>
class TNiagaraParameterMapHistoryWithMetaDataBuilder : public TNiagaraParameterMapHistoryBuilder<GraphBridge>
{
public:
	using FGraph = typename GraphBridge::FGraph;
	using FPin = typename GraphBridge::FPin;
	using FHistory = typename TNiagaraParameterMapHistoryBuilder<GraphBridge>::FHistory;
	using TNiagaraParameterMapHistoryBuilder<GraphBridge>::GetModuleAlias;
	using TNiagaraParameterMapHistoryBuilder<GraphBridge>::CallingGraphContext;

	//** Add a graph to the calling graph context stack. Generally used to prime the context stack with the node graph deep copy during precompilation. */
	void AddGraphToCallingGraphContextStack(const FGraph* InGraph);

	/**
	* Virtual wrapper to call correct AddVariable method on the target ParameterMapHistory.
	*/
	virtual int32 AddVariableToHistory(FHistory& History, const FNiagaraVariable& InVar, const FNiagaraVariable& InAliasedVar, const FPin* InPin) override;
};

struct FNiagaraGraphCachedBuiltHistory : public FNiagaraGraphCachedDataBase
{
	virtual ~FNiagaraGraphCachedBuiltHistory() = default;
	virtual void GetStaticVariables(TArray<FNiagaraVariable>& OutVars) override { OutVars = StaticVariables; }

	virtual bool IsValidForSystem(const UNiagaraSystem* InSystem) const override;
	virtual void SetSourceSystem(const UNiagaraSystem* InSystem) override;

	virtual bool IsValidForEmitter(const FVersionedNiagaraEmitterData* InEmitterData) const override;
	virtual void SetSourceEmitter(const FVersionedNiagaraEmitterData* InEmitterData) override;

	TArray<FNiagaraVariable> StaticVariables;
	TArray<FGuid> SourceAssetChangeIds;
};
