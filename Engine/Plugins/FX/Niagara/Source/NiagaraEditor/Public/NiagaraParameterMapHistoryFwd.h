// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FNiagaraCompilationGraphBridge;
struct FNiagaraCompilationDigestBridge;

template<typename GraphBridge>
struct TNiagaraParameterMapHistory;

template<typename GraphBridge>
class TNiagaraParameterMapHistoryBuilder;

template<typename GraphBridge>
class TNiagaraParameterMapHistoryWithMetaDataBuilder;

using FNiagaraParameterMapHistory = TNiagaraParameterMapHistory<FNiagaraCompilationGraphBridge>;
using FNiagaraParameterMapHistoryBuilder = TNiagaraParameterMapHistoryBuilder<FNiagaraCompilationGraphBridge>;
using FNiagaraParameterMapHistoryWithMetaDataBuilder = TNiagaraParameterMapHistoryWithMetaDataBuilder<FNiagaraCompilationGraphBridge>;
