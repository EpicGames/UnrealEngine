// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraTypes.h"
#include "NiagaraGraph.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorCommon.h"
#include "Misc/Guid.h"
#include "NiagaraScriptVariable.h"

enum class EParameterDefinitionMatchState : uint8;

struct FNiagaraParameterPanelItemBase
{
public:
	FNiagaraParameterPanelItemBase() = default;

	FNiagaraParameterPanelItemBase(
		const UNiagaraScriptVariable* InScriptVariable,
		const FNiagaraNamespaceMetadata& InNamespaceMetaData
	)
		: ScriptVariable(InScriptVariable)
		, NamespaceMetaData(InNamespaceMetaData)
	{
	}

	/* Equality operator to support AddUnique when gathering via parameter panel view models. */
	bool operator== (const FNiagaraParameterPanelItemBase & Other) const { return GetVariable() == Other.GetVariable(); };

	/* Simple getters to reduce clutter. */
	const FNiagaraVariable& GetVariable() const { return ScriptVariable->Variable; };
	const FNiagaraVariableMetaData& GetVariableMetaData() const { return ScriptVariable->Metadata; };

public:
	const UNiagaraScriptVariable* ScriptVariable;
	FNiagaraNamespaceMetadata NamespaceMetaData;
};

struct FNiagaraParameterReferencePath
{
	FNiagaraParameterReferencePath() = default;
	FNiagaraParameterReferencePath(const UObject* InSourceObject, FName InModuleName)
		: SourceObject(InSourceObject)
		, ModuleName(InModuleName)
	{
	}

	TWeakObjectPtr<const UObject> SourceObject;
	FName ModuleName = NAME_None;
	bool bRead = false;
	bool bWrite = false;

	bool operator==(const FNiagaraParameterReferencePath& Other) const
	{
		return Other.SourceObject == SourceObject && Other.ModuleName == ModuleName;
	};
};

struct FNiagaraParameterPanelItem : public FNiagaraParameterPanelItemBase
{
public:
	DECLARE_DELEGATE(FOnRequestRename);
	DECLARE_DELEGATE(FOnRequestRenameNamespaceModifier);

	FNiagaraParameterPanelItem() = default;

public:
	/* NOTE: Const is a lie, but necessary evil to allow binding during CreateWidget type methods where this is passed as a const ref. */
	FOnRequestRename& GetOnRequestRename() const { return OnRequestRenameDelegate; };
	FOnRequestRenameNamespaceModifier& GetOnRequestRenameNamespaceModifier() const { return OnRequestRenameNamespaceModifierDelegate; };

	void RequestRename() const
	{
		if ( ensure(OnRequestRenameDelegate.IsBound()) )
		{
			OnRequestRenameDelegate.Execute();
		}
	}

	void RequestRenameNamespaceModifier() const
	{
		if ( ensure(OnRequestRenameNamespaceModifierDelegate.IsBound()) )
		{
			OnRequestRenameNamespaceModifierDelegate.Execute();
		}
	}

	void AddToReadCount(const FNiagaraParameterReferencePath& SourcePath)
	{
		ReadReferenceCount++;
		FNiagaraParameterReferencePath* RefPath = ReferencePaths.FindByKey(SourcePath);
		if (!RefPath)
		{
			RefPath = &ReferencePaths.Add_GetRef(SourcePath);
		}
		RefPath->bRead = true;
	}

	void AddToWriteCount(const FNiagaraParameterReferencePath& SourcePath)
	{
		WriteReferenceCount++;
		FNiagaraParameterReferencePath* RefPath = ReferencePaths.FindByKey(SourcePath);
		if (!RefPath)
		{
			RefPath = &ReferencePaths.Add_GetRef(SourcePath);
		}
		RefPath->bWrite = true;
	}

	/* For script variables; if true, the variable is sourced from a script that is not owned by the emitter/system the parameter panel is referencing. */
	bool bExternallyReferenced = false;

	/* For script variables; if true, the variable is a member of a custom stack context for an emitter/system. */
	bool bSourcedFromCustomStackContext = false;

	/* Count of read references to the variable in graphs viewed by a parameter panel view model. */
	int32 ReadReferenceCount = 0;

	/* Count of write references to the variable in graphs viewed by a parameter panel view model. */
	int32 WriteReferenceCount = 0;

	/* A detailed list of unique references (i.e. modules) to display in the parameters panel */
	TArray<FNiagaraParameterReferencePath> ReferencePaths;

	/* The relation of this parameter item to all parameter definitions it is matching. Whether the parameter item is subscribed to a definition is tracked by the UNiagaraScriptVariable's bSubscribedToParameterDefinitions member. */
	EParameterDefinitionMatchState DefinitionMatchState = EParameterDefinitionMatchState::NoMatchingDefinitions;

private:
	mutable FOnRequestRename OnRequestRenameDelegate;
	mutable FOnRequestRenameNamespaceModifier OnRequestRenameNamespaceModifierDelegate;
};

struct FNiagaraParameterPanelCategory
{
public:
	FNiagaraParameterPanelCategory() = default;

	FNiagaraParameterPanelCategory(const FNiagaraNamespaceMetadata& InNamespaceMetadata)
		:NamespaceMetaData(InNamespaceMetadata)
	{};

	bool operator== (const FNiagaraParameterPanelCategory& Other) const {return NamespaceMetaData == Other.NamespaceMetaData;};

public:
	const FNiagaraNamespaceMetadata NamespaceMetaData;
};

struct FNiagaraParameterDefinitionsPanelItem : public FNiagaraParameterPanelItemBase
{
public:
	FNiagaraParameterDefinitionsPanelItem() = default;

	FNiagaraParameterDefinitionsPanelItem(
		const UNiagaraScriptVariable* InScriptVariable,
		const FNiagaraNamespaceMetadata& InNamespaceMetaData,
		const FText& InParameterDefinitionsNameText,
		const FGuid& InParameterDefinitionsUniqueId
	)
		: FNiagaraParameterPanelItemBase(InScriptVariable, InNamespaceMetaData)
		, ParameterDefinitionsNameText(InParameterDefinitionsNameText)
		, ParameterDefinitionsUniqueId(InParameterDefinitionsUniqueId)
	{};

public:
	FText ParameterDefinitionsNameText;
	FGuid ParameterDefinitionsUniqueId;
};

struct FNiagaraParameterDefinitionsPanelCategory
{
public:
	FNiagaraParameterDefinitionsPanelCategory() = default;

	FNiagaraParameterDefinitionsPanelCategory(const FText& InParameterDefinitionsNameText, const FGuid& InParameterDefinitionsUniqueId)
		: ParameterDefinitionsNameText(InParameterDefinitionsNameText)
		, ParameterDefinitionsUniqueId(InParameterDefinitionsUniqueId)
	{};

	bool operator== (const FNiagaraParameterDefinitionsPanelCategory & Other) const { return ParameterDefinitionsUniqueId == Other.ParameterDefinitionsUniqueId; };

	FText ParameterDefinitionsNameText;
	FGuid ParameterDefinitionsUniqueId;
};
