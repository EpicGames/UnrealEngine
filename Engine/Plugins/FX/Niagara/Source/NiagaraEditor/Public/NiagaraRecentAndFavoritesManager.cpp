// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRecentAndFavoritesManager.h"

#include "ModuleDescriptor.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEmitter.h"
#include "NiagaraSystem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Interfaces/IPluginManager.h"
#include "UObject/PackageResourceManager.h"

void FNiagaraRecentAndFavoritesManager::Initialize()
{
	RecentAndFavoriteEmittersAndSystem = MakeUnique<FMainMRUFavoritesList>(TEXT("NiagaraRecent_EmittersAndSystems"), TEXT("NiagaraFavorites_EmittersAndSystems"),10);
	RecentAndFavoriteModules = MakeUnique<FMainMRUFavoritesList>(TEXT("NiagaraRecent_Scripts"), TEXT("NiagaraFavorites_Scripts"),30);
	RecentAndFavoriteParameters = MakeUnique<FMainMRUFavoritesList>(TEXT("NiagaraRecent_Parameters"), TEXT("NiagaraFavorites_Parameters"), 10);

	RecentAndFavoriteEmittersAndSystem->ReadFromINI();
	RecentAndFavoriteModules->ReadFromINI();
	RecentAndFavoriteParameters->ReadFromINI();
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &FNiagaraRecentAndFavoritesManager::OnAssetRemoved);
	AssetRegistryModule.Get().OnAssetRenamed().AddSP(this, &FNiagaraRecentAndFavoritesManager::OnAssetRenamed);

	IPluginManager& PluginManager = IPluginManager::Get();
	PluginManager.OnLoadingPhaseComplete().AddRaw(this, &FNiagaraRecentAndFavoritesManager::OnPluginLoadingPhaseComplete);
	
	IPackageResourceManager::GetOnClearPackageResourceManagerDelegate().AddSP(this, &FNiagaraRecentAndFavoritesManager::OnPackageResourceManagerShutdown);
}

void FNiagaraRecentAndFavoritesManager::OnAssetAdded(const FAssetData& AssetData) const
{
	if(AssetData.GetClass() == UNiagaraEmitter::StaticClass() || AssetData.GetClass() == UNiagaraSystem::StaticClass())
	{
		RecentAndFavoriteEmittersAndSystem->AddMRUItem(AssetData.PackageName.ToString());
	}
	else if(AssetData.GetClass() == UNiagaraScript::StaticClass())
	{
		RecentAndFavoriteModules->AddMRUItem(AssetData.PackageName.ToString());
	}
}


void FNiagaraRecentAndFavoritesManager::OnAssetRemoved(const FAssetData& AssetData) const
{
	if(FMainMRUFavoritesList* RecentAssetsList = GetRecentsList(AssetData))
	{
		FString Identifier = AssetData.PackageName.ToString();

		// We need this because FindMRUItemIdx has a check() for non valid long package names
		if (FPackageName::IsValidLongPackageName(Identifier))
		{
			if (RecentAssetsList->FindMRUItemIdx(Identifier) != INDEX_NONE)
			{
				RecentAssetsList->RemoveMRUItem(Identifier);
			}

			if (RecentAssetsList->ContainsFavoritesItem(Identifier))
			{
				RecentAssetsList->RemoveFavoritesItem(Identifier);
			}
		}
	}
}

void FNiagaraRecentAndFavoritesManager::OnAssetOpenedInEditor(UObject* Object, IAssetEditorInstance* AssetEditorInstance) const
{
	FAssetData AssetData(Object);
	OnAssetAdded(AssetData);
}

void FNiagaraRecentAndFavoritesManager::OnAssetRenamed(const FAssetData& AssetData, const FString& AssetOldName) const
{
	FString OldPathName = FPaths::GetBaseFilename(AssetOldName, false);
	FString NewPathName = FPaths::GetBaseFilename(AssetData.GetObjectPathString(), false);

	// We need this early exit because FindMRUItemIdx has a check() for non valid long package names
	if (!FPackageName::IsValidLongPackageName(OldPathName) || !FPackageName::IsValidLongPackageName(NewPathName))
	{
		return;
	}
	
	if(FMainMRUFavoritesList* RecentAssetsList = GetRecentsList(AssetData))
	{
		// If the asset did not previously exist in the recents list, we have nothing to do
		if(RecentAssetsList->FindMRUItemIdx(OldPathName) == INDEX_NONE)
		{
			return;
		}

		// Otherwise remove the old name of the asset, and re-add it with the new name
		// NOTE: This has an unintentional side effect of bringing it to the top of the MRU list that can't be avoided
		RecentAssetsList->RemoveMRUItem(OldPathName);
		RecentAssetsList->AddMRUItem(NewPathName);
	}
}

void FNiagaraRecentAndFavoritesManager::OnPluginLoadingPhaseComplete(ELoadingPhase::Type LoadingPhase, bool bSuccess)
{
	if(LoadingPhase == ELoadingPhase::PostEngineInit)
	{
		// GEditor might not always exist
		if(GEditor)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetOpenedInEditor().AddSP(this, &FNiagaraRecentAndFavoritesManager::OnAssetOpenedInEditor);
			IPluginManager& PluginManager = IPluginManager::Get();
			PluginManager.OnLoadingPhaseComplete().RemoveAll(this);

			GEditor->OnEditorClose().AddSP(this, &FNiagaraRecentAndFavoritesManager::OnEditorClose);
		}
	}
}

void FNiagaraRecentAndFavoritesManager::Shutdown()
{
	Save();
	
	RecentAndFavoriteEmittersAndSystem.Reset();
	RecentAndFavoriteModules.Reset();
	RecentAndFavoriteParameters.Reset();
}

void FNiagaraRecentAndFavoritesManager::Save() const
{
	if(bCanCheckForPackages)
	{
		// Remove assets that have never been saved to disk
		for(int CurRecentIndex = 0; CurRecentIndex < RecentAndFavoriteEmittersAndSystem->GetNumItems(); ++CurRecentIndex)
		{
			const FString& RecentAsset = RecentAndFavoriteEmittersAndSystem->GetMRUItem(CurRecentIndex);
			// TODO If the engine is closing already the PackageManager might be nullptr
			if(!FPackageName::DoesPackageExist(RecentAsset))
			{
				RecentAndFavoriteEmittersAndSystem->RemoveMRUItem(CurRecentIndex);
				--CurRecentIndex;
			}
		}
		
		for(int CurRecentIndex = 0; CurRecentIndex < RecentAndFavoriteModules->GetNumItems(); ++CurRecentIndex)
		{
			const FString& RecentAsset = RecentAndFavoriteModules->GetMRUItem(CurRecentIndex);
			// TODO If the engine is closing already the PackageManager might be nullptr
			if(!FPackageName::DoesPackageExist(RecentAsset))
			{
				RecentAndFavoriteModules->RemoveMRUItem(CurRecentIndex);
				--CurRecentIndex;
			}
		}
	}

	
	RecentAndFavoriteEmittersAndSystem->WriteToINI();
	RecentAndFavoriteModules->WriteToINI();
	RecentAndFavoriteParameters->WriteToINI();
}

void FNiagaraRecentAndFavoritesManager::EmitterUsed(const UNiagaraEmitter& Emitter)
{
	const FAssetData EmitterAssetData(&Emitter);
	OnAssetAdded(EmitterAssetData);
}

void FNiagaraRecentAndFavoritesManager::SystemUsed(const UNiagaraSystem& System)
{
	const FAssetData SystemAssetData(&System);
	OnAssetAdded(SystemAssetData);
}

void FNiagaraRecentAndFavoritesManager::ModuleScriptUsed(const UNiagaraScript& Script)
{
	const FAssetData ScriptAssetData(&Script);
	OnAssetAdded(ScriptAssetData);
}

void FNiagaraRecentAndFavoritesManager::ParameterUsed(const FNiagaraVariable& VariableBase)
{
	RecentAndFavoriteParameters->AddMRUItem(VariableBase.ToString());
}

FMainMRUFavoritesList* FNiagaraRecentAndFavoritesManager::GetRecentsList(const FAssetData& AssetData) const
{
	FMainMRUFavoritesList* RecentAssetsList = nullptr;
	if(AssetData.GetClass() == UNiagaraEmitter::StaticClass() || AssetData.GetClass() == UNiagaraSystem::StaticClass())
	{
		RecentAssetsList = RecentAndFavoriteEmittersAndSystem.Get();
	}
	else if(AssetData.GetClass() == UNiagaraScript::StaticClass())
	{
		RecentAssetsList = RecentAndFavoriteModules.Get();
	}

	return RecentAssetsList;
}

void FNiagaraRecentAndFavoritesManager::OnEditorClose()
{
	Save();
	GEditor->OnEditorClose().RemoveAll(this);
}

void FNiagaraRecentAndFavoritesManager::OnPackageResourceManagerShutdown()
{
	bCanCheckForPackages = false;
	IPackageResourceManager::GetOnClearPackageResourceManagerDelegate().RemoveAll(this);
}
