// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModuleDescriptor.h"
#include "NiagaraSystem.h"
#include "MRUFavoritesList.h"
#include "Subsystems/AssetEditorSubsystem.h"

class UNiagaraScript;
class UNiagaraSystem;
class UNiagaraEmitter;
struct FNiagaraVariable;

class FNiagaraRecentAndFavoritesManager : public TSharedFromThis<FNiagaraRecentAndFavoritesManager>
{
public:
	void Initialize();
	void Shutdown();
	
	void Save() const;

	void EmitterUsed(const UNiagaraEmitter& Emitter);
	void SystemUsed(const UNiagaraSystem& System);
	void ModuleScriptUsed(const UNiagaraScript& Script);
	void ParameterUsed(const FNiagaraVariable& VariableBase);

	const FMainMRUFavoritesList* GetRecentEmitterAndSystemsList() const { return RecentAndFavoriteEmittersAndSystem.Get(); }
	const FMainMRUFavoritesList* GetRecentScriptsList() const { return RecentAndFavoriteModules.Get(); }
	const FMainMRUFavoritesList* GetRecentParametersList() const { return RecentAndFavoriteParameters.Get(); }

private:
	FMainMRUFavoritesList* GetRecentsList(const FAssetData& AssetData) const;
	void OnEditorClose();
	void OnPackageResourceManagerShutdown();

	void OnAssetAdded(const FAssetData& AssetData) const;
	void OnAssetRemoved(const FAssetData& AssetData) const;
	void OnAssetOpenedInEditor(UObject* Object, IAssetEditorInstance* AssetEditorInstance) const;
	void OnAssetRenamed(const FAssetData& AssetData, const FString& String) const;

	void OnPluginLoadingPhaseComplete(ELoadingPhase::Type LoadingPhase, bool bSuccess);

private:
	TUniquePtr<FMainMRUFavoritesList> RecentAndFavoriteEmittersAndSystem;
	TUniquePtr<FMainMRUFavoritesList> RecentAndFavoriteModules;
	TUniquePtr<FMainMRUFavoritesList> RecentAndFavoriteParameters;
	
	bool bCanCheckForPackages = true;
};

