// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "NiagaraSettings.h"
#include "NiagaraCommon.h"
#include "NiagaraScriptFactoryNew.generated.h"

UCLASS(hidecategories=Object, ABSTRACT)
class UNiagaraScriptFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	// End UFactory Interface

public:
	static void InitializeScript(UNiagaraScript* Script);

protected:

	/** Gets the script asset to create a new one from that based on the script usage type. */
	const FSoftObjectPath& GetDefaultScriptFromSettings(const class UNiagaraEditorSettings* Settings);

	/** Give child factory classes the ability to define which asset action it represents. */
	virtual FName GetAssetTypeActionName() const { return NAME_None; };

	/** Give child factory classes the ability to define which script usage type it represent. */
	virtual ENiagaraScriptUsage GetScriptUsage() const;
};

/**
 * Niagara module script factory.
 */
UCLASS()
class UNiagaraModuleScriptFactory: public UNiagaraScriptFactoryNew
{
	GENERATED_UCLASS_BODY()

	// Begin UFactory Interface
	virtual FName GetNewAssetThumbnailOverride() const override;
	// End UFactory Interface

protected:

	// Begin UNiagaraScriptFactoryNew Interface
	virtual FName GetAssetTypeActionName() const override;
	virtual ENiagaraScriptUsage GetScriptUsage() const override;
	// End UNiagaraScriptFactoryNew Interface
};

/**
 * Niagara function factory.
 */
UCLASS()
class UNiagaraFunctionScriptFactory: public UNiagaraScriptFactoryNew
{
	GENERATED_UCLASS_BODY()

	// Begin UFactory Interface
	virtual FName GetNewAssetThumbnailOverride() const override;
	// End UFactory Interface

protected:
	// Begin UNiagaraScriptFactoryNew Interface
	virtual FName GetAssetTypeActionName() const override;
	virtual ENiagaraScriptUsage GetScriptUsage() const override;
	// End UNiagaraScriptFactoryNew Interface
};

/**
 * Niagara dynamic input script factory.
 */
UCLASS()
class UNiagaraDynamicInputScriptFactory: public UNiagaraScriptFactoryNew
{
	GENERATED_UCLASS_BODY()

	// Begin UFactory Interface
	virtual FName GetNewAssetThumbnailOverride() const override;
	// End UFactory Interface

protected:
	// Begin UNiagaraScriptFactoryNew Interface
	virtual FName GetAssetTypeActionName() const override;
	virtual ENiagaraScriptUsage GetScriptUsage() const override;
	// End UNiagaraScriptFactoryNew Interface
};
