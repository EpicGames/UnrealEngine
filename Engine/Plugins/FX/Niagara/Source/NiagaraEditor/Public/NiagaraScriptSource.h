// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "NiagaraScriptSourceBase.h"
#include "INiagaraCompiler.h"
#include "NiagaraParameterMapHistory.h"
#include "GraphEditAction.h"
#include "NiagaraScriptSource.generated.h"

struct FNiagaraScriptHashCollector;
struct FNiagaraVMExecutableDataId;

UCLASS(MinimalAPI)
class UNiagaraScriptSource final : public UNiagaraScriptSourceBase
{
	GENERATED_BODY()

public:
	/** Graph for particle update expression */
	UPROPERTY()
	TObjectPtr<class UNiagaraGraph>	NodeGraph = nullptr;
	
	bool bIsCompilationCopy = false;
	bool bIsReleased = false;
	
	// UObject interface
	virtual void PostLoad() override;

	UNiagaraScriptSource* CreateCompilationCopy(const TArray<ENiagaraScriptUsage>& CompileUsages);

	void ReleaseCompilationCopy();

	// UNiagaraScriptSourceBase interface.
	//virtual ENiagaraScriptCompileStatus Compile(UNiagaraScript* ScriptOwner, FString& OutGraphLevelErrorMessages) override;
	virtual bool IsSynchronized(const FGuid& InChangeId) override;
	virtual void MarkNotSynchronized(FString Reason) override;

	virtual FGuid GetChangeID() override;
	FVersionedNiagaraEmitter GetOuterEmitter() const;

	virtual void RegisterVMCompilationIdDependencies(FNiagaraScriptHashCollector& Collector, ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const override;
	virtual void ComputeVMCompilationId(FNiagaraVMExecutableDataId& Id, FNiagaraScriptHashCollector& HashCollector, ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const override;

	// Will conditionally refresh the graph's CompileId
	virtual void RefreshGraphCompileId() override;
	
	virtual void ReportAnalyticsData(FNiagaraScriptSourceAnalytics& InData) const override;

	virtual FGuid GetCompileBaseId(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const override;

	virtual FNiagaraCompileHash GetCompileHash(ENiagaraScriptUsage InUsage, const FGuid& InUsageId) const override;

	virtual void PostLoadFromEmitter(FVersionedNiagaraEmitter OwningEmitter) override;

	virtual TMap<FName, UNiagaraDataInterface*> ComputeObjectNameMap(UNiagaraSystem& System, ENiagaraScriptUsage Usage, FGuid UsageId, FString EmitterUniqueName) const override;

	NIAGARAEDITOR_API virtual bool AddModuleIfMissing(FString ModulePath, ENiagaraScriptUsage Usage, bool& bOutFoundModule)override;

	virtual void MigrateParameterDataToHierarchyRoot(FVersionedNiagaraScriptData& OwnerData) override;

	void FixupRenamedParameters(UNiagaraNodeFunctionCall* FunctionCallNode, TConstArrayView<FNiagaraVariable> ModuleInputVariables, FNiagaraParameterStore& RapidIterationParameters, const TArray<FNiagaraVariable>& OldRapidIterationVariables, const FVersionedNiagaraEmitter& Emitter, ENiagaraScriptUsage ScriptUsage) const;
	void InitializeNewParameters(UNiagaraNodeFunctionCall* FunctionCallNode, TConstArrayView<FNiagaraVariable> ModuleInputVariables, FNiagaraParameterStore& RapidIterationParameters, const FVersionedNiagaraEmitter& VersionedEmitter, ENiagaraScriptUsage ScriptUsage, TSet<FNiagaraVariableBase>& ValidRapidIterationParameters) const;

	virtual void CleanUpOldAndInitializeNewRapidIterationParameters(const FVersionedNiagaraEmitter& Emitter, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FNiagaraParameterStore& RapidIterationParameters) const override;
	virtual void ForceGraphToRecompileOnNextCheck() override;
	virtual void RefreshFromExternalChanges() override;

	virtual void CollectDataInterfaces(TArray<const UNiagaraDataInterfaceBase*>& DataInterfaces) const override;

	/** Synchronize all source script variables that have been changed or removed from the parameter definitions to all eligible destination script variables owned by the graph.
	 *
	 *  @param TargetDefinitions			The set of parameter definitions that will be synchronized with the graph parameters.
	 *	@param AllDefinitions				All parameter definitions in the project. Used to add new subscriptions to definitions if specified in Args.
	 *  @param AllDefinitionsParameterIds	All unique Ids of all parameter definitions.
	 *	@param Subscriber					The INiagaraParameterDefinitionsSubscriber that owns the graph. Used to add new subscriptions to definitions if specified in Args.
	 *	@param Args							Additional arguments that specify how to perform the synchronization.
	 */
	virtual void SynchronizeGraphParametersWithParameterDefinitions(
		const TArray<UNiagaraParameterDefinitionsBase*> TargetDefinitions,
		const TArray<UNiagaraParameterDefinitionsBase*> AllDefinitions,
		const TSet<FGuid>& AllDefinitionsParameterIds,
		INiagaraParameterDefinitionsSubscriber* Subscriber,
		FSynchronizeWithParameterDefinitionsArgs Args
	) override;

	/** Rename all graph assignment and map set node pins.
	 *  Used when synchronizing definitions with source scripts of systems and emitters.
	 */
	virtual void RenameGraphAssignmentAndSetNodePins(const FName OldName, const FName NewName) override;

	virtual void GetLinkedPositionTypeInputs(const TArray<FNiagaraVariable>& ParametersToCheck, TSet<FNiagaraVariable>& OutLinkedParameters) override;
	virtual void ChangedLinkedInputTypes(const FNiagaraVariable& ParametersToChange, const FNiagaraTypeDefinition& NewType) override;
	virtual void ReplaceScriptReferences(UNiagaraScript* OldScript, UNiagaraScript* NewScript) override;
private:
	void OnGraphChanged(const FEdGraphEditAction &Action);
	void OnGraphDataInterfaceChanged();
};
