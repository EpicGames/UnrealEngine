// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraTypes.h"
#include "NiagaraVariableMetaData.h"
#include "NiagaraVariant.h"
#include "NiagaraScriptVariable.generated.h"

class UNiagaraScriptVariable;

USTRUCT()
struct FNiagaraScriptVariableData
{
	GENERATED_USTRUCT_BODY()

	FNiagaraScriptVariableData() = default;

	NIAGARAEDITOR_API void Init(const FNiagaraVariable& InVar, const FNiagaraVariableMetaData& InVarMetaData);
	NIAGARAEDITOR_API void InitFrom(const UNiagaraScriptVariable& Source, bool bCreateNewGuid = true);
	NIAGARAEDITOR_API void InitFrom(const FNiagaraScriptVariableData& Source, bool bCreateNewGuid = true);

	/** The default mode. Can be Value, Binding or Custom. */
	UPROPERTY(EditAnywhere, Category = "Default Value")
	ENiagaraDefaultMode DefaultMode = ENiagaraDefaultMode::Value;

	/** The default binding. Only used if DefaultMode == ENiagaraDefaultMode::Binding. */
	UPROPERTY(EditAnywhere, Category = "Default Value")
	FNiagaraScriptVariableBinding DefaultBinding;

	/** Variable type, name and data. The data is not persistent, but used as a buffer when interfacing elsewhere. */
	UPROPERTY()
	FNiagaraVariable Variable;

	/** The metadata associated with this script variable. */
	UPROPERTY(EditAnywhere, Category = "Variable", meta=(ShowOnlyInnerProperties))
	FNiagaraVariableMetaData Metadata;

	/** Entry point for generating the compile hash.*/
	NIAGARAEDITOR_API bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;

	bool GetIsStaticSwitch() const { return bIsStaticSwitch; };
	void SetIsStaticSwitch(bool bInIsStaticSwitch) { bIsStaticSwitch = bInIsStaticSwitch; };

	bool GetIsSubscribedToParameterDefinitions() const { return bSubscribedToParameterDefinitions; };
	NIAGARAEDITOR_API void SetIsSubscribedToParameterDefinitions(bool bInSubscribedToParameterDefinitions);

	bool GetIsOverridingParameterDefinitionsDefaultValue() const { return bOverrideParameterDefinitionsDefaultValue; };
	NIAGARAEDITOR_API void SetIsOverridingParameterDefinitionsDefaultValue(bool bInOverridingParameterDefinitionsDefaultValue);

	const FGuid& GetChangeId() const { return ChangeId; };
	void SetChangeId(const FGuid& NewId) { ChangeId = NewId; };
	void UpdateChangeId() { ChangeId = FGuid::NewGuid(); };

	const FNiagaraVariant& GetDefaultValueVariant() const { return DefaultValueVariant; };

	void SetDefaultValueData(const uint8* Data)
	{
		check(Data);
		AllocateData();
		DefaultValueVariant.SetBytes(Data, DefaultValueVariant.GetNumBytes());
		Variable.SetData(GetDefaultValueData());
	}

	const uint8* GetDefaultValueData() const
	{
		if (DefaultValueVariant.GetMode() == ENiagaraVariantMode::Bytes)
		{
			return DefaultValueVariant.GetBytes();
		}
		return nullptr;
	}

	void CopyDefaultValueDataTo(uint8* Dest) const
	{
		check(Variable.GetSizeInBytes() == DefaultValueVariant.GetNumBytes());
		check(IsDataAllocated());
		FMemory::Memcpy(Dest, DefaultValueVariant.GetBytes(), DefaultValueVariant.GetNumBytes());
	}

	void SetStaticSwitchDefaultValue(const int32 Value)
	{
		StaticSwitchDefaultValue = Value;
	}

	int32 GetStaticSwitchDefaultValue() const
	{
		return StaticSwitchDefaultValue;
	}

private:
	void AllocateData()
	{
		//It is possible the underlying variable does not have a valid type. Do not allocate until this is valid.
		if (Variable.GetType().IsValid())
		{
			if (DefaultValueVariant.GetNumBytes() != Variable.GetSizeInBytes())
			{
				// Allocate the correct size in bytes, and set the variant mode to bytes by invoking the ctor.
				TArray<uint8> InitialAllocationBytes;
				InitialAllocationBytes.SetNumZeroed(Variable.GetSizeInBytes());
				DefaultValueVariant = FNiagaraVariant(InitialAllocationBytes);
			}
			Variable.AllocateData();
		}
	}

	bool IsDataAllocated() const
	{
		return DefaultValueVariant.GetNumBytes() > 0 && DefaultValueVariant.GetNumBytes() == Variable.GetSizeInBytes() && Variable.IsDataAllocated();
	}

private:
	UPROPERTY(EditAnywhere, Category = "Hidden")
	FNiagaraVariant DefaultValueVariant;

	UPROPERTY()
	int32 StaticSwitchDefaultValue = 0;

	UPROPERTY()
	bool bIsStaticSwitch = false;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	bool bSubscribedToParameterDefinitions = false;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	FGuid ChangeId;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	bool bOverrideParameterDefinitionsDefaultValue = false;
};

/*
* Used to store variable data and metadata per graph. 
*/
UCLASS(MinimalAPI)
class UNiagaraScriptVariable : public UObject
{
public:
	GENERATED_UCLASS_BODY()

	DECLARE_DELEGATE_OneParam(FOnChanged, const UNiagaraScriptVariable* /*ThisScriptVariable*/);

	NIAGARAEDITOR_API virtual void PostLoad() override;

	NIAGARAEDITOR_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	NIAGARAEDITOR_API void Init(const FNiagaraVariable& InVar, const FNiagaraVariableMetaData& InVarMetaData);
	NIAGARAEDITOR_API void InitFrom(const UNiagaraScriptVariable* Value, bool bCreateNewGuid = true);

	/** The default mode. Can be Value, Binding or Custom. */
	UPROPERTY(EditAnywhere, Category = "Default Value")
	ENiagaraDefaultMode DefaultMode; 

	/** The default binding. Only used if DefaultMode == ENiagaraDefaultMode::Binding. */
	UPROPERTY(EditAnywhere, Category = "Default Value")
	FNiagaraScriptVariableBinding DefaultBinding; 

	/** Variable type, name and data. The data is not persistent, but used as a buffer when interfacing elsewhere. */
	UPROPERTY()
	FNiagaraVariable Variable;

	/** The metadata associated with this script variable. */
	UPROPERTY(EditAnywhere, Category = "Variable", meta=(ShowOnlyInnerProperties))
	FNiagaraVariableMetaData Metadata;

	/** Entry point for generating the compile hash.*/
	NIAGARAEDITOR_API bool AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const;

	bool GetIsStaticSwitch() const { return bIsStaticSwitch; };
	void SetIsStaticSwitch(bool bInIsStaticSwitch) { bIsStaticSwitch = bInIsStaticSwitch; };

	bool GetIsSubscribedToParameterDefinitions() const { return bSubscribedToParameterDefinitions; };
	NIAGARAEDITOR_API void SetIsSubscribedToParameterDefinitions(bool bInSubscribedToParameterDefinitions);

	bool GetIsOverridingParameterDefinitionsDefaultValue() const { return bOverrideParameterDefinitionsDefaultValue; };
	NIAGARAEDITOR_API void SetIsOverridingParameterDefinitionsDefaultValue(bool bInOverridingParameterDefinitionsDefaultValue);

	const FGuid& GetChangeId() const { return ChangeId; };
	void SetChangeId(const FGuid& NewId) { ChangeId = NewId; };
	void UpdateChangeId() { ChangeId = FGuid::NewGuid(); }; 

	const FNiagaraVariant& GetDefaultValueVariant() const { return DefaultValueVariant; };

	void SetDefaultValueData(const uint8* Data)
	{
		check(Data);
		AllocateData();
		DefaultValueVariant.SetBytes(Data, DefaultValueVariant.GetNumBytes());
		Variable.SetData(GetDefaultValueData());
	}

	const uint8* GetDefaultValueData() const
	{
		if(DefaultValueVariant.GetMode() == ENiagaraVariantMode::Bytes)
		{ 
			return DefaultValueVariant.GetBytes();
		}
		return nullptr;
	}

	void CopyDefaultValueDataTo(uint8* Dest) const
	{
		check(Variable.GetSizeInBytes() == DefaultValueVariant.GetNumBytes());
		check(IsDataAllocated());
		FMemory::Memcpy(Dest, DefaultValueVariant.GetBytes(), DefaultValueVariant.GetNumBytes());
	}

	void SetStaticSwitchDefaultValue(const int32 Value)
	{
		StaticSwitchDefaultValue = Value;
	}

	int32 GetStaticSwitchDefaultValue() const
	{
		return StaticSwitchDefaultValue;
	}

	static NIAGARAEDITOR_API bool DefaultsAreEquivalent(const UNiagaraScriptVariable* ScriptVarA, const UNiagaraScriptVariable* ScriptVarB);

	static NIAGARAEDITOR_API FGuid GenerateStableGuid(const UNiagaraScriptVariable* ScriptVariable);

private:
	void AllocateData()
	{
		//It is possible the underlying variable does not have a valid type. Do not allocate until this is valid.
		if(Variable.GetType().IsValid())
		{ 
			if (DefaultValueVariant.GetNumBytes() != Variable.GetSizeInBytes())
			{
				// Allocate the correct size in bytes, and set the variant mode to bytes by invoking the ctor.
				TArray<uint8> InitialAllocationBytes;
				InitialAllocationBytes.SetNumZeroed(Variable.GetSizeInBytes());
				DefaultValueVariant = FNiagaraVariant(InitialAllocationBytes);
			}
			Variable.AllocateData();
		}
	}

	bool IsDataAllocated() const
	{
		return DefaultValueVariant.GetNumBytes() > 0 && DefaultValueVariant.GetNumBytes() == Variable.GetSizeInBytes() && Variable.IsDataAllocated();
	}

private:
	UPROPERTY(EditAnywhere, Category = "Hidden")
	FNiagaraVariant DefaultValueVariant;

	UPROPERTY()
	int32 StaticSwitchDefaultValue;

	UPROPERTY()
	bool bIsStaticSwitch;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	bool bSubscribedToParameterDefinitions;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	FGuid ChangeId;

	UPROPERTY(meta = (SkipForCompileHash = "true"))
	bool bOverrideParameterDefinitionsDefaultValue;
}; 
