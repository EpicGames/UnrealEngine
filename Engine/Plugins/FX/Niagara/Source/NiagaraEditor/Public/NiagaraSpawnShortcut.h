// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraScript.h"
#include "Framework/Commands/InputChord.h"
#include "NiagaraSpawnShortcut.generated.h"

USTRUCT()
struct FNiagaraSpawnShortcut
{
	GENERATED_USTRUCT_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Shortcut)
	FString Name;
	UPROPERTY(EditAnywhere, Category = Shortcut)
	FInputChord Input;
};
