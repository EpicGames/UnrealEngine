// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEditorDataBase.h"
#include "NiagaraMessages.h"
#include "NiagaraNodeFunctionCall.h"

#include "NiagaraStackEditorData.generated.h"
struct FStackIssue;

/* Defines different modes for inline display in the stack. */
UENUM()
enum class ENiagaraStackEntryInlineDisplayMode
{
	Expression,
	GraphHorizontal,
	GraphVertical,
	GraphHybrid,
	None
};

/* Defines stack options for stateless modules */
USTRUCT()
struct FNiagaraStatelessModuleEditorData
{
	GENERATED_BODY()

	UPROPERTY()
	bool bShowWhenDisabled = false;
};

/** Editor only UI data for emitters. */
UCLASS()
class UNiagaraStackEditorData : public UNiagaraEditorDataBase
{
	GENERATED_BODY()

public:
	/*
	* Gets whether or not a stack entry has a rename pending.
	* @param StackEntryKey A unique key for the stack entry.
	*/
	bool GetStackEntryIsRenamePending(const FString& StackEntryKey) const;

	/*
	* Sets whether or not a stack entry has a rename pending.
	* @param StackEntryKey A unique key for the stack entry.
	* @param bIsRenamePending Whether or not the stack entry has a rename pending.
	*/
	void SetStackEntryIsRenamePending(const FString& StackEntryKey, bool bIsRenamePending);

	/*
	 * Gets whether or not a stack entry is Expanded.
	 * @param bIsExpandedDefault The default value to return if the expanded state hasn't been set for the stack entry.
	 * @param StackItemKey A unique key for the entry.
	 */
	NIAGARAEDITOR_API bool GetStackEntryIsExpanded(const FString& StackEntryKey, bool bIsExpandedDefault) const;

	/*
	 * Sets whether or not a stack entry is Expanded.
	 * @param StackEntryKey A unique key for the entry.
	 * @param bIsExpanded Whether or not the entry is expanded.
	 */
	void SetStackEntryIsExpanded(const FString& StackEntryKey, bool bIsExpanded);

	/*
	* Gets whether or not a stack entry is Expanded.
	* @param bIsExpandedDefault The default value to return if the expanded state hasn't been set for the stack entry.
	* @param StackItemKey A unique key for the entry.
	*/
	NIAGARAEDITOR_API bool GetStackEntryIsExpandedInOverview(const FString& StackEntryKey, bool bIsExpandedDefault) const;

	/*
	* Sets whether or not a stack entry is Expanded.
	* @param StackEntryKey A unique key for the entry.
	* @param bIsExpanded Whether or not the entry is expanded.
	*/
	void SetStackEntryIsExpandedInOverview(const FString& StackEntryKey, bool bIsExpanded);

	/*
	 * Gets whether or not a stack entry was Expanded before triggering a stack search.
	 * @param bWasExpandedPreSearchDefault The default value to to return if the pre-search expanded state hasn't been set for the stack entry.
	 * @param StackEntryKey a unique key for the entry.
	 */
	bool GetStackEntryWasExpandedPreSearch(const FString& StackEntryKey, bool bWasExpandedPreSearchDefault) const;

	/*
	 * Sets whether or not a stack entry was Expanded before a stack search was triggered.
	 * @param StackEntryKey A unique key for the entry.
	 * @param bWasExpandedPreSearch Whether or not the entry was expanded pre-search.
	 */
	void SetStackEntryWasExpandedPreSearch(const FString& StackEntryKey, bool bWasExpandedPreSearch);

	const TMap<FString, FNiagaraStackNoteData>& GetAllStackNotes();
	NIAGARAEDITOR_API TOptional<FNiagaraStackNoteData> GetStackNote(const FString& StackEntryKey);
	NIAGARAEDITOR_API bool HasStackNote(const FString& StackEntryKey);
	NIAGARAEDITOR_API void AddOrReplaceStackNote(const FString& StackEntryKey, FNiagaraStackNoteData NewStackMessage, bool bBroadcast = true);
	NIAGARAEDITOR_API void DeleteStackNote(const FString& StackEntryKey);
	
	void TransferDeprecatedStackNotes(const UNiagaraNodeFunctionCall& FunctionCallNode);

	/*
	 * Gets whether or not this entry should display all of its content inline. 
	 */
	ENiagaraStackEntryInlineDisplayMode GetStackEntryInlineDisplayMode(const FString& StackEntryKey) const;

	/*
	 * Sets whether or not this entry should display all of its content inline.
	 */
	void SetStackEntryInlineDisplayMode(const FString& StackEntryKey, ENiagaraStackEntryInlineDisplayMode InlineDisplayMode);

	/*
	* Gets whether or not a stack item is showing advanced items.
	* @param StackItemKey A unique key for the entry.
	* @param bIsExpandedDefault The default value to return if the expanded state hasn't been set for the stack entry.
	*/
	bool GetStackItemShowAdvanced(const FString& StackEntryKey, bool bShowAdvancedDefault) const;

	/*
	* Sets whether or not a stack entry is showing advanced items.
	* @param StackEntryKey A unique key for the entry.
	* @param bIsExpanded Whether or not the entry is expanded.
	*/
	void SetStackItemShowAdvanced(const FString& StackEntryKey, bool bShowAdanced);

	/*
	* Gets the active section for a stack entry.
	* @param StackItemKey A unique key for the entry.
	* @param ActiveSectionDefault The default value to return if an active section has not been set before.
	*/
	FText GetStackEntryActiveSection(const FString& StackEntryKey, FText ActiveSectionDefault) const;

	/*
	* Sets the active section for a stack entry.
	* @param StackItemKey A unique key for the entry.
	* @param SelectedSection The selected section for the entry.
	*/
	void SetStackEntryActiveSection(const FString& StackEntryKey, FText ActiveSection);

	void ClearStackEntryActiveSection(const FString& StackEntryKey);
	
	/*
	* Gets a stack entry's display name. Returns null if none is found.
	* @param StackEntryKey A unique key for the stack entry.
	*/
	NIAGARAEDITOR_API const FText* GetStackEntryDisplayName(const FString& StackEntryKey) const;

	/*
	* Gets a map of all renamed stack entry keys to their display name.
	*/
	const TMap<FString, FText>& GetAllStackEntryDisplayNames() const { return StackEntryKeyToDisplayName; }

	/*
	* Sets a stack entry's display name.
	* @param StackEntryKey A unique key for the stack entry.
	* @param InDisplayName The display name to set for this entry.
	*/
	void SetStackEntryDisplayName(const FString& StackEntryKey, const FText& InDisplayName);

	/** Gets whether or not only modified properties should be shown in the stack. */
	bool GetShowOnlyModified() const;

	/** Sets whether or not only modified properties should be shown in the stack. */
	void SetShowOnlyModified(bool bInShowOnlyModified);
	
	/* Gets whether or not all advanced items should be shown in the stack. */
	bool GetShowAllAdvanced() const;

	/* Sets whether or not all advanced items should be shown in the stack. */
	void SetShowAllAdvanced(bool bInShowAllAdvanced);

	/* Gets whether or not item outputs should be shown in the stack. */
	bool GetShowOutputs() const;

	/* Sets whether or not item outputs should be shown in the stack. */
	void SetShowOutputs(bool bInShowOutputs);

	/* Gets whether or not item linked script inputs should be shown in the stack. */
	bool GetShowLinkedInputs() const;

	/* Sets whether or not item linked script inputs should be shown in the stack. */
	void SetShowLinkedInputs(bool bInShowLinkedInputs);

	/* Gets whether or not only modules that have issues should be shown in the stack. */
	bool GetShowOnlyIssues() const;

	/* Sets whether or not only modules that haves issues should be shown in the stack. */
	void SetShowOnlyIssues(bool bInShowIssues);

	/* Gets the last scroll position for the associated stack. */
	double GetLastScrollPosition() const;

	/* Sets the last scroll position for the associated stack. */
	void SetLastScrollPosition(double InLastScrollPosition);

	/*
	* @param Issue the issue to be dismissed (not fixed).
	*/
	void DismissStackIssue(FString IssueId);

	/* Restores all the dismissed issues so that the user can see them and choose what to do. */
	NIAGARAEDITOR_API void UndismissAllIssues();

	/* Gets a reference to the dismissed stack issue array */
	NIAGARAEDITOR_API const TArray<FString>& GetDismissedStackIssueIds();

	/*
	* Gets whether or not a stateless module item should be shown when it's disabled.
	* @param StackEntryKey A unique key for the entry.
	*/
	bool GetStatelessModuleShowWhenDisabled(const FString& StackEntryKey) const;

	/*
	* Sets whether or not a stateless module item should be shown when it's disabled.
	* @param StackEntryKey A unique key for the entry.
	* @param bInShowWhenDisabled Whether or not the item should be shown when disabled.
	*/
	void SetStatelessModuleShowWhenDisabled(const FString& StackEntryKey, bool bInShowWhenDisabled);

	UPROPERTY()
	bool bHideDisabledModules = false;

private:
	TMap<FString, bool> StackEntryKeyToRenamePendingMap;

	UPROPERTY()
	TMap<FString, bool> StackEntryKeyToExpandedMap;

	TMap<FString, bool> StackEntryKeyToPreSearchExpandedMap;

	UPROPERTY()
	TMap<FString, ENiagaraStackEntryInlineDisplayMode> StackEntryKeyToInlineDisplayModeMap;

	TMap<FString, bool> StackItemKeyToShowAdvancedMap;

	UPROPERTY()
	TMap<FString, bool> StackEntryKeyToExpandedOverviewMap;

	TMap<FString, FText> StackEntryKeyToActiveSectionMap;

	UPROPERTY()
	TMap<FString, FNiagaraStackNoteData> StackNotes;

	UPROPERTY()
	TMap<FString, FNiagaraStatelessModuleEditorData> StackEntryKeyToStatelessModuleEditorData;

	/* Marking those FTexts explicitly as editoronly_data will make localization not pick these up.
	 * This is a workaround. EditorDataBase in system & emitter is already flagged as editor only, but it doesn't propagate properly */
#if WITH_EDITORONLY_DATA
	
	UPROPERTY()
	TMap<FString, FText> StackEntryKeyToDisplayName;
	
#endif

	bool bShowOnlyModified = false;
	
	bool bShowAllAdvanced = false;

	bool bShowOutputs = false;

	bool bShowLinkedInputs = false;

	bool bShowOnlyIssues = false;

	double LastScrollPosition;

	UPROPERTY()
	TArray<FString> DismissedStackIssueIds;
};