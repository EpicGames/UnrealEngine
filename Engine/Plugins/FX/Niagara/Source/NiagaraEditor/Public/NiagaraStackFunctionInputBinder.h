// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "NiagaraScriptSourceBase.h"
#include "EdGraphSchema_Niagara.h"
#include "NiagaraGraph.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "ViewModels/Stack/NiagaraStackGraphUtilities.h"

class UNiagaraScript;
struct FNiagaraEmitterHandle;
class UNiagaraNodeFunctionCall;
class UEdGraphPin;

class FNiagaraStackFunctionInputBinder
{
public:
	NIAGARAEDITOR_API FNiagaraStackFunctionInputBinder();

	bool TryBind(
		UNiagaraScript* InScript,
		TArray<UNiagaraScript*> InDependentScripts, 
		FCompileConstantResolver InConstantResolver,
		FString InOwningEmitterUniqueName, 
		UNiagaraNodeFunctionCall* InFunctionCallNode, 
		FName InMetaDataKey, 
		FString InMetaDataValue, 
		TOptional<FNiagaraTypeDefinition> InInputType,
		bool bInIsRequired,
		FText& OutErrorMessage);

	NIAGARAEDITOR_API bool TryBind(
		UNiagaraScript* InScript,
		TArray<UNiagaraScript*> InDependentScripts,
		FCompileConstantResolver InConstantResolver,
		FString InOwningEmitterUniqueName,
		UNiagaraNodeFunctionCall* InFunctionCallNode,
		FName InInputName,
		TOptional<FNiagaraTypeDefinition> InInputType,
		bool bInIsRequired,
		FText& OutErrorMessage);

	bool IsValid() const;

	void Reset();

	FName GetInputName() const;

	FNiagaraTypeDefinition GetInputType() const;

	UNiagaraNodeFunctionCall* GetFunctionCallNode() const;

	template<typename ValueType>
	ValueType GetValue() const
	{
		checkf(sizeof(ValueType) == InputType.GetSize(), TEXT("ValueType size doesn't match bound value size"));
		ValueType Value;
		FMemory::Memcpy(&Value, GetData().GetData(), InputType.GetSize());
		return Value;
	}
	
	template<typename ValueType>
	void SetValue(const ValueType& InValue)
	{
		SetData((uint8*)&InValue, sizeof(ValueType));
	}

	TArray<uint8> GetData() const;

	NIAGARAEDITOR_API void SetData(const uint8* InValue, int32 InSize);

private:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInputMatchesPredicate, FNiagaraVariable);

private:
	bool TryBindInternal(
		UNiagaraScript* InScript,
		TArray<UNiagaraScript*> InDependentScripts,
		FCompileConstantResolver InConstantResolver,
		FString InOwningEmitterUniqueName,
		UNiagaraNodeFunctionCall* InFunctionCallNode,
		FInputMatchesPredicate InputMatchesCallback,
		FText InputMatchDescription,
		TOptional<FNiagaraTypeDefinition> InInputType,
		bool bInIsRequired,
		FText& OutErrorMessage);

	FGuid GetChangeIdFromFunctionScript() const;

	void RefreshGraphPins() const;

private:
	TWeakObjectPtr<UNiagaraScript> Script;

	TArray<TWeakObjectPtr<UNiagaraScript>> DependentScripts;

	TWeakObjectPtr<UNiagaraNodeFunctionCall> FunctionCallNode;

	FNiagaraTypeDefinition InputType;

	FName InputName;

	FGuid InputVariableGuid;

	FNiagaraParameterHandle AliasedParameterHandle;

	mutable FGuid ValidScriptGraphChangeIdForOverridePin;

	mutable FGuid ValidScriptGraphChangeIdForDefaultPin;

	FNiagaraVariable RapidIterationParameter;

	mutable UEdGraphPin* DefaultPin;

	mutable UEdGraphPin* OverridePin;
};

template<>
inline bool FNiagaraStackFunctionInputBinder::GetValue<bool>() const
{
	check(InputType.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()));
	TArray<uint8> ValueData = GetData();
	FNiagaraBool* BoolStruct = (FNiagaraBool*)ValueData.GetData();
	return BoolStruct->GetValue();
}

template<>
inline void FNiagaraStackFunctionInputBinder::SetValue<bool>(const bool& bInValue)
{
	check(InputType.IsSameBaseDefinition(FNiagaraTypeDefinition::GetBoolDef()));
	FNiagaraBool BoolStruct(bInValue);
	SetData((uint8*)(&BoolStruct), sizeof(FNiagaraBool));
}