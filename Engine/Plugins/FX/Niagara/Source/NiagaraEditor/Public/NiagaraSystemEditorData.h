// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "NiagaraEditorDataBase.h"
#include "NiagaraEditorData.h"
#include "DataHierarchyViewModelBase.h"

#include "NiagaraSystemEditorData.generated.h"

class UNiagaraStackEditorData;
class UNiagaraSystem;
class UEdGraph;

/** Editor only folder data for emitters in a system. */
UCLASS()
class UNiagaraSystemEditorFolder : public UObject
{
	GENERATED_BODY()

public:
	const FName GetFolderName() const;

	void SetFolderName(FName InFolderName);

	const TArray<UNiagaraSystemEditorFolder*>& GetChildFolders() const;

	void AddChildFolder(UNiagaraSystemEditorFolder* ChildFolder);

	void RemoveChildFolder(UNiagaraSystemEditorFolder* ChildFolder);

	const TArray<FGuid>& GetChildEmitterHandleIds() const;

	void AddChildEmitterHandleId(FGuid ChildEmitterHandleId);

	void RemoveChildEmitterHandleId(FGuid ChildEmitterHandleId);

private:
	UPROPERTY()
	FName FolderName;

	UPROPERTY()
	TArray<TObjectPtr<UNiagaraSystemEditorFolder>> ChildFolders;

	UPROPERTY()
	TArray<FGuid> ChildEmitterHandleIds;
};

/** View settings that are saved per asset and aren't shared between different Niagara viewports. */
USTRUCT()
struct FNiagaraPerAssetViewportSettings
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FVector ViewLocation = FVector::ZeroVector;

	UPROPERTY()
	FRotator ViewRotation = FRotator::ZeroRotator;

	UPROPERTY()
	bool bUseOrbitMode = true;
};

/** Editor only data for systems. */
UCLASS(MinimalAPI)
class UNiagaraSystemEditorData : public UNiagaraEditorDataBase
{
	GENERATED_BODY()

	DECLARE_MULTICAST_DELEGATE(FOnUserParameterScriptVariablesSynced)
public:
	NIAGARAEDITOR_API UNiagaraSystemEditorData(const FObjectInitializer& ObjectInitializer);

	NIAGARAEDITOR_API virtual void PostInitProperties() override;

	NIAGARAEDITOR_API virtual void PostLoadFromOwner(UObject* InOwner) override;
#if WITH_EDITORONLY_DATA
	static NIAGARAEDITOR_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	/** Gets the root folder for UI folders for emitters. */
	NIAGARAEDITOR_API UNiagaraSystemEditorFolder& GetRootFolder() const;

	/** Gets the stack editor data for the system. */
	NIAGARAEDITOR_API UNiagaraStackEditorData& GetStackEditorData() const;

	const FTransform& GetOwnerTransform() const {
		return OwnerTransform;
	}

	void SetOwnerTransform(const FTransform& InTransform) {
		OwnerTransform = InTransform;
	}

	NIAGARAEDITOR_API TRange<float> GetPlaybackRange() const;

	NIAGARAEDITOR_API void SetPlaybackRange(TRange<float> InPlaybackRange);

	FFrameRate GetPlaybackFrameRate() const { return PlaybackFrameRate; }

	void SetPlaybackFrameRate(const FFrameRate InPlaybackFrameRate);

	bool GetLockPlaybackFrameRate() const { return bLockPlaybackFrameRate; }

	void SetLockPlaybackFrameRate(bool bInLockPlaybackFrameRate);

	NIAGARAEDITOR_API UEdGraph* GetSystemOverviewGraph() const;

	NIAGARAEDITOR_API const FNiagaraGraphViewSettings& GetSystemOverviewGraphViewSettings() const;

	NIAGARAEDITOR_API void SetSystemOverviewGraphViewSettings(const FNiagaraGraphViewSettings& InOverviewGraphViewSettings);

	NIAGARAEDITOR_API bool GetOwningSystemIsPlaceholder() const;

	NIAGARAEDITOR_API void SetOwningSystemIsPlaceholder(bool bInSystemIsPlaceholder, UNiagaraSystem& OwnerSystem);

	NIAGARAEDITOR_API void SynchronizeOverviewGraphWithSystem(UNiagaraSystem& OwnerSystem);

	NIAGARAEDITOR_API void InitOnSyncScriptVariables(UNiagaraSystem& System);
	
	NIAGARAEDITOR_API void SyncUserScriptVariables(UNiagaraSystem* System);
	FOnUserParameterScriptVariablesSynced& OnUserParameterScriptVariablesSynced() { return OnUserParameterScriptVariablesSyncedDelegate; }
	
	NIAGARAEDITOR_API UNiagaraScriptVariable* FindOrAddUserScriptVariable(FNiagaraVariable UserParameter, UNiagaraSystem& System);
	NIAGARAEDITOR_API const UNiagaraScriptVariable* FindUserScriptVariable(FGuid UserParameterGuid) const;
	NIAGARAEDITOR_API const UNiagaraScriptVariable* FindUserScriptVariable(FNiagaraVariable UserParameter, const UNiagaraSystem& System) const;
	NIAGARAEDITOR_API bool RenameUserScriptVariable(FNiagaraVariable OldVariable, FName NewName);
	NIAGARAEDITOR_API bool RemoveUserScriptVariable(FNiagaraVariable Variable);

	NIAGARAEDITOR_API const FNiagaraPerAssetViewportSettings& GetAssetViewportSettings() const;
	NIAGARAEDITOR_API void SetAssetViewportSettings(FNiagaraPerAssetViewportSettings InSettings);
	NIAGARAEDITOR_API void SetUseOrbitMode(bool bInUseOrbitMode);
	
	bool GetViewportManualFocusDistanceEnabled() const { return bViewportManualFocusDistanceEnabled; }
	void SetViewportManualFocusDistanceEnabled(bool bEnabled) { bViewportManualFocusDistanceEnabled = bEnabled; }

	float GetViewportManualFocusDistance() const { return ViewportManualFocusDistance; }
	void SetViewportManualFocusDistance(float Distance) { ViewportManualFocusDistance = FMath::Max(Distance, 1.0f); }

	/** Contains the root ids for organizing user parameters. */
	UPROPERTY()
	TObjectPtr<UHierarchyRoot> UserParameterHierarchy;
	
private:
	NIAGARAEDITOR_API void UpdatePlaybackRangeFromEmitters(UNiagaraSystem& OwnerSystem);
	void PostLoad_TransferModuleStackNotesToNewFormat(UObject* InOwner);
private:
	UPROPERTY(Instanced)
	TObjectPtr<UNiagaraSystemEditorFolder> RootFolder;

	UPROPERTY(Instanced)
	TObjectPtr<UNiagaraStackEditorData> StackEditorData;

	UPROPERTY()
	FTransform OwnerTransform;

	UPROPERTY()
	float PlaybackRangeMin;

	UPROPERTY()
	float PlaybackRangeMax;

	UPROPERTY()
	FFrameRate PlaybackFrameRate = FFrameRate(60, 1);

	UPROPERTY()
	bool bLockPlaybackFrameRate = true;

	/** Graph presenting overview of the current system and its emitters. */
	UPROPERTY()
	TObjectPtr<UEdGraph> SystemOverviewGraph;

	UPROPERTY()
	FNiagaraGraphViewSettings OverviewGraphViewSettings;

	UPROPERTY()
	FNiagaraPerAssetViewportSettings AssetViewportSettings;
	
	UPROPERTY()
	bool bSystemIsPlaceholder;

	UPROPERTY()
	TArray<TObjectPtr<UNiagaraScriptVariable>> UserParameterMetaData;
	
	FOnUserParameterScriptVariablesSynced OnUserParameterScriptVariablesSyncedDelegate;

	UPROPERTY()
	bool bViewportManualFocusDistanceEnabled = false;

	UPROPERTY()
	float ViewportManualFocusDistance = 100.0f;
};
