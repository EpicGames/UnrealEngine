// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraTypes.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "FrontendFilterBase.h"
#include "NiagaraSystemFactoryNew.generated.h"

class UNiagaraSystem;
class UNiagaraEmitter;

UCLASS(hidecategories = Object)
class UNiagaraSystemFactoryNew : public UFactory
{
	GENERATED_BODY()

public:
	UNiagaraSystemFactoryNew();
	
	TWeakObjectPtr<UNiagaraSystem> SystemToCopy;
	TArray<FVersionedNiagaraEmitter> EmittersToAddToNewSystem;

private:
	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual bool ConfigurePropertiesAsync(FOnFactoryConfigurePropertiesAsyncComplete OnComplete, FOnFactoryConfigurePropertiesAsyncCancelled OnCancelled) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

public:
	NIAGARAEDITOR_API static void InitializeSystem(UNiagaraSystem* System, bool bCreateDefaultNodes);

private:
	void TryAssignDefaultEffectType(UNiagaraSystem* System);

	void OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type ActivationMethod);
	bool OnAdditionalShouldFilterAsset(const FAssetData& AssetData) const;
	void OnExtendAddFilterMenu(UToolMenu* ToolMenu) const;
	TArray<TSharedRef<FFrontendFilter>> OnGetExtraFrontendFilters() const;
};



