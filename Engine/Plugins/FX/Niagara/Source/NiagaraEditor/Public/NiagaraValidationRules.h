// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraValidationRule.h"
#include "NiagaraPlatformSet.h"
#include "NiagaraRendererProperties.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackViewModel.h"
#include "NiagaraValidationRules.generated.h"

class UNiagaraEffectType;
class UNiagaraScript;
class UNiagaraStackModuleItem;
struct FNiagaraPlatformSetConflictInfo;
class UNiagaraStackRendererItem;

namespace NiagaraValidation
{
	NIAGARAEDITOR_API bool HasValidationRules(UNiagaraSystem* NiagaraSystem);
	NIAGARAEDITOR_API void ValidateAllRulesInSystem(TSharedPtr<FNiagaraSystemViewModel> ViewModel, TFunction<void(const FNiagaraValidationResult& Result)> ResultCallback);
	
	template<typename T>
	TArray<T*> GetStackEntries(UNiagaraStackViewModel* StackViewModel, bool bRefresh = false)
	{
		TArray<T*> Results;
		TArray<UNiagaraStackEntry*> EntriesToCheck;
		if (UNiagaraStackEntry* RootEntry = StackViewModel->GetRootEntry())
		{
			if (bRefresh)
			{
				RootEntry->RefreshChildren();
			}
			RootEntry->GetUnfilteredChildren(EntriesToCheck);
		}
		while (EntriesToCheck.Num() > 0)
		{
			UNiagaraStackEntry* Entry = EntriesToCheck.Pop();
			if (T* ItemToCheck = Cast<T>(Entry))
			{
				Results.Add(ItemToCheck);
			}
			Entry->GetUnfilteredChildren(EntriesToCheck);
		}
		return Results;
	}

	template<typename T>
	TArray<T*> GetAllStackEntriesInSystem(TSharedPtr<FNiagaraSystemViewModel> ViewModel, bool bRefresh = false)
	{
		TArray<T*> Results;
		Results.Append(NiagaraValidation::GetStackEntries<T>(ViewModel->GetSystemStackViewModel(), bRefresh));
		TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> EmitterHandleViewModels = ViewModel->GetEmitterHandleViewModels();
		for (TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleModel : EmitterHandleViewModels)
		{
			if (EmitterHandleModel->GetIsEnabled())
			{
				Results.Append(NiagaraValidation::GetStackEntries<T>(EmitterHandleModel.Get().GetEmitterStackViewModel(), bRefresh));
			}
		}
		return Results;
	}

	// helper function to retrieve a single stack entry from the system or emitter view model
	template<typename T>
	T* GetStackEntry(UNiagaraStackViewModel* StackViewModel, bool bRefresh = false)
	{
		TArray<T*> StackEntries = NiagaraValidation::GetStackEntries<T>(StackViewModel, bRefresh);
		if (StackEntries.Num() > 0)
		{
			return StackEntries[0];
		}
		return nullptr;
	}

	// helper function to get renderer stack item
	NIAGARAEDITOR_API UNiagaraStackRendererItem* GetRendererStackItem(UNiagaraStackViewModel* StackViewModel, UNiagaraRendererProperties* RendererProperties);

	// --------------------------------------------------------------------------------------------------------------------------------------------
	// Common fixes and links
	NIAGARAEDITOR_API void AddGoToFXTypeLink(FNiagaraValidationResult& Result, UNiagaraEffectType* FXType);
	NIAGARAEDITOR_API FNiagaraValidationFix MakeDisableGPUSimulationFix(FVersionedNiagaraEmitterWeakPtr WeakEmitterPtr);
	NIAGARAEDITOR_API TArray<FNiagaraPlatformSetConflictInfo> GatherPlatformSetConflicts(const FNiagaraPlatformSet* SetA, const FNiagaraPlatformSet* SetB);
	NIAGARAEDITOR_API FString GetPlatformConflictsString(TConstArrayView<FNiagaraPlatformSetConflictInfo> ConflictInfos, int MaxPlatformsToShow = 4);
	NIAGARAEDITOR_API FString GetPlatformConflictsString(const FNiagaraPlatformSet& PlatformSetA, const FNiagaraPlatformSet& PlatformSetB, int MaxPlatformsToShow = 4);
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterHandleViewModel> GetEmitterViewModel(const FNiagaraValidationContext& Context, const UNiagaraEmitter* NiagaraEmitter);
	NIAGARAEDITOR_API TOptional<int32> GetModuleStaticInt32Value(const UNiagaraStackModuleItem* Module, FName ParameterName);
	NIAGARAEDITOR_API void SetModuleStaticInt32Value(UNiagaraStackModuleItem* Module, FName ParameterName, int32 NewValue);
	NIAGARAEDITOR_API bool StructContainsUObjectProperty(UStruct* Struct);
}

/** This validation rule ensures that systems don't have a warmup time set. */
UCLASS(Category = "Validation", DisplayName = "No Warmup Time")
class UNiagaraValidationRule_NoWarmupTime : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Error;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule ensures that emitters do not use events. */
UCLASS(Category = "Validation", DisplayName = "No Events")
class UNiagaraValidationRule_NoEvents : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	UPROPERTY(EditAnywhere, Category = Validation)
	FNiagaraPlatformSet Platforms;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule ensures that GPU emitters have fixed bounds set. */
UCLASS(Category = "Validation", DisplayName = "Fixed GPU Bounds Set")
class UNiagaraValidationRule_FixedGPUBoundsSet : public UNiagaraValidationRule
{
	GENERATED_BODY()
public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

USTRUCT()
struct FNiagaraValidationRule_EmitterCountAndPlatformSet
{
	GENERATED_BODY()

	/** Name to display if we fail the limit check */
	UPROPERTY(EditAnywhere, Category = Validation)
	FString RuleName;

	UPROPERTY(EditAnywhere, Category = Validation)
	FNiagaraPlatformSet Platforms;

	UPROPERTY(EditAnywhere, Category = Validation, meta = (DisplayName = "Include Regular Emitters"))
	bool bIncludeStateful = true;

	UPROPERTY(EditAnywhere, Category = Validation, meta = (DisplayName = "Include Lightweight Emitters"))
	bool bIncludeStateless = true;

	UPROPERTY(EditAnywhere, Category = Validation)
	int32 EmitterCountLimit = 8;
};

/** This validation rule can be used to apply budgets for emitter count. */
UCLASS(Category = "Validation", DisplayName = "Emitter Count")
class UNiagaraValidationRule_EmitterCount : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	UPROPERTY(EditAnywhere, Category = Validation)
	TArray<FNiagaraValidationRule_EmitterCountAndPlatformSet> EmitterCountLimits;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

USTRUCT()
struct FNiagaraValidationRule_RendererCountAndPlatformSet
{
	GENERATED_BODY()

	/** Name to display if we fail the limit check */
	UPROPERTY(EditAnywhere, Category = Validation)
	FString RuleName;

	UPROPERTY(EditAnywhere, Category = Validation)
	FNiagaraPlatformSet Platforms;

	UPROPERTY(EditAnywhere, Category = Validation)
	int32 RendererCountLimit = 8;
};

/** This validation rule can be used to apply budgets for renderer count. */
UCLASS(Category = "Validation", DisplayName = "Renderer Count")
class UNiagaraValidationRule_RendererCount : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	UPROPERTY(EditAnywhere, Category = Validation)
	TArray<FNiagaraValidationRule_RendererCountAndPlatformSet> RendererCountLimits;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule can ban the use of certain renderers on all or a subset of platforms. */
UCLASS(Category = "Validation", DisplayName = "Banned Renderers")
class UNiagaraValidationRule_BannedRenderers : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:

	//Platforms this validation rule will apply to.
	UPROPERTY(EditAnywhere, Category = Validation)
	FNiagaraPlatformSet Platforms;

	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	UPROPERTY(EditAnywhere, Category = Validation)
	TArray<TSubclassOf<UNiagaraRendererProperties>> BannedRenderers;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** Validation rule to check for lightweight usage. */
UCLASS(Category = "Validation", DisplayName = "Lightweight Validation")
class UNiagaraValidationRule_Lightweight : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	// Platforms this validation rule will apply to.
	UPROPERTY(EditAnywhere, Category = Validation)
	FNiagaraPlatformSet Platforms;

	// When set if we have an emitter present it will be flagged at this severity
	UPROPERTY(EditAnywhere, Category = Validation)
	TOptional<ENiagaraValidationSeverity> UsedWithEmitter;

	// When set if an experimental module is found it will be flagged at this severity
	UPROPERTY(EditAnywhere, Category = Validation)
	TOptional<ENiagaraValidationSeverity> UsingExperimentalModule;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule can ban the use of certain modules on all or a subset of platforms. */
UCLASS(Category = "Validation", DisplayName = "Banned Modules")
class UNiagaraValidationRule_BannedModules : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:

	//Platforms this validation rule will apply to.
	UPROPERTY(EditAnywhere, Category=Validation)
	FNiagaraPlatformSet Platforms;

	UPROPERTY(EditAnywhere, Category = Validation)
	bool bBanOnGpu = true;

	UPROPERTY(EditAnywhere, Category = Validation)
	bool bBanOnCpu = true;

	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	UPROPERTY(EditAnywhere, Category = Validation)
	TArray<TObjectPtr<UNiagaraScript>> BannedModules;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule can ban the use of certain data interfaces on all or a subset of platforms. */
UCLASS(Category = "Validation", DisplayName = "Banned DataInterfaces")
class UNiagaraValidationRule_BannedDataInterfaces : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	UPROPERTY(EditAnywhere, Category = Validation)
	bool bBanOnGpu = true;

	UPROPERTY(EditAnywhere, Category = Validation)
	bool bBanOnCpu = true;

	UPROPERTY(EditAnywhere, Category = Validation)
	FNiagaraPlatformSet Platforms;

	UPROPERTY(EditAnywhere, Category = Validation)
	TArray<TSubclassOf<UNiagaraDataInterface>> BannedDataInterfaces;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** Checks to see if renderers have sorting enabled on them or not. */
UCLASS(Category = "Validation", DisplayName = "Renderer Sorting Enabled")
class UNiagaraValidationRule_RendererSortingEnabled : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	UPROPERTY(EditAnywhere, Category = Validation)
	FNiagaraPlatformSet Platforms;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule that can be used to ban GPU usage on the provided platforms or warn that GPU emitters might now work correctly. */
UCLASS(Category = "Validation", DisplayName = "Gpu Usage")
class UNiagaraValidationRule_GpuUsage : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	UPROPERTY(EditAnywhere, Category = Validation)
	FNiagaraPlatformSet Platforms;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/**
This validation rule is for ribbon renderers to ensure they are not used in situations that can cause compatability or performance issues.
i.e. Don't use a ribbon renderer with a GPU emitter / enable GPU ribbon init on lower end devices.
*/
UCLASS(Category = "Validation", DisplayName = "Ribbon Renderer")
class UNiagaraValidationRule_RibbonRenderer : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	/** When enable validation will fail if used by a GPU emitter. */
	UPROPERTY(EditAnywhere, Category = Validation)
	bool bFailIfUsedByGPUSimulation = true;

	/** When enable validation will fail if used by a CPU emitter and GPU init is enabled on the renderer. */
	UPROPERTY(EditAnywhere, Category = Validation)
	bool bFailIfUsedByGPUInit = true;

	UPROPERTY(EditAnywhere, Category = Validation)
	FNiagaraPlatformSet Platforms;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule always fails and can be used to mark a default/test effect type as stand-in that must be changed. Effectively forces the user to choose a correct effect type for a system. */
UCLASS(Category = "Validation", DisplayName = "Invalid Effect Type")
class UNiagaraValidationRule_InvalidEffectType : public UNiagaraValidationRule
{
	GENERATED_BODY()
public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule will check if a system has an effect type assigned. Useful for default validation set rules that are enforced globally. */
UCLASS(Category = "Validation", DisplayName = "Has Effect Type")
class UNiagaraValidationRule_HasEffectType : public UNiagaraValidationRule
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;
	
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule will check if a system uses emitters that are tagged as Deprecated using the Niagara Asset Tags.
 *  This is distinct from a Niagara Emitter version that is marked as deprecated, but might have a new, non-deprecated version. */
UCLASS(Category = "Validation", DisplayName = "Check for Deprecated Emitters", meta=(DeprecationMessage="This validation rule is no longer valid. Delete it from your setup"), Deprecated)
class UDEPRECATED_NiagaraValidationRule_CheckDeprecatedEmitters : public UNiagaraValidationRule
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;
	
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule checks for various common issue with Large World Coordinates like mixing vector and position types. */
UCLASS(Category = "Validation", DisplayName = "Large World Coordinates")
class UNiagaraValidationRule_LWC : public UNiagaraValidationRule
{
	GENERATED_BODY()
public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule is used by the collision module to check that renderers don't use any opaque or masked materials when depth buffer collisions are used. */
UCLASS(Category = "Validation", DisplayName = "Validate GPU Depth Collision Module")
class UNiagaraValidationRule_NoOpaqueRenderMaterial : public UNiagaraValidationRule
{
	GENERATED_BODY()
public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule is used by modules or effect types to warn that they don't support systems with fixed delta time ticks. */
UCLASS(Category = "Validation", DisplayName = "No Fixed DT Tick Support")
class UNiagaraValidationRule_NoFixedDeltaTime : public UNiagaraValidationRule
{
	GENERATED_BODY()
public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** This validation rule can be used to enforce a budget on the number of simulation stages and the iterations that may execute. */
UCLASS(Category = "Validation", DisplayName = "Simulation Stage Budget")
class UNiagaraValidationRule_SimulationStageBudget : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;

	UPROPERTY(EditAnywhere, Category = Validation)
	bool bMaxSimulationStagesEnabled = false;

	UPROPERTY(EditAnywhere, Category = Validation)
	bool bMaxIterationsPerStageEnabled = false;

	UPROPERTY(EditAnywhere, Category = Validation)
	bool bMaxTotalIterationsEnabled = false;

	/** How do we want to repro the error in the stack */
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	/** Maximum number of simulation stages allowed, where 0 means no simulation stages. */
	UPROPERTY(EditAnywhere, Category = Validation, meta=(EditCondition="bMaxSimulationStagesEnabled"))
	int32 MaxSimulationStages = 0;

	/**
	Maximum number of iterations a single stage is allowed to execute.
	Note: Can only check across explicit counts, dynamic bindings will be ignored.
	*/
	UPROPERTY(EditAnywhere, Category = Validation, meta = (EditCondition = "bMaxIterationsPerStageEnabled"))
	int32 MaxIterationsPerStage = 1;

	/**
	Maximum total iterations across all the enabled simulation stages.
	Note: Can only check across explicit counts, dynamic bindings will be ignored.
	*/
	UPROPERTY(EditAnywhere, Category = Validation, meta = (EditCondition = "bMaxTotalIterationsEnabled"))
	int32 MaxTotalIterations = 1;
};

/** Validation rule to check for unwanted tick dependencies.  */
UCLASS(Category = "Validation", DisplayName = "Tick Dependency Check")
class UNiagaraValidationRule_TickDependencyCheck : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;

	/** How do we want to repro the error in the stack */
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Info;

	/** Check that the actor component interface isn't adding a tick dependency on the CPU. */
	UPROPERTY(EditAnywhere, Category = Validation)
	bool bCheckActorComponentInterface = true;

	/** Check that the camera data interface isn't adding a tick dependency on the CPU. */
	UPROPERTY(EditAnywhere, Category = Validation)
	bool bCheckCameraDataInterface = true;

	/** Check that the skeletal mesh interface isn't adding a tick dependency on the CPU. */
	UPROPERTY(EditAnywhere, Category = Validation)
	bool bCheckSkeletalMeshInterface = true;

	/** If the system uses one of these effect types the rule will not be run. */
	UPROPERTY(EditAnywhere, Category = Validation)
	TArray<TSoftObjectPtr<UNiagaraEffectType>> EffectTypesToExclude;
};

/** This validation rule checks to see if you have exposed user data interfaces. */
UCLASS(Category = "Validation", DisplayName = "User Data Interfaces")
class UNiagaraValidationRule_UserDataInterfaces : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;

	/** Only include data interfaces that contain exposed UObject properties in them. */
	UPROPERTY(EditAnywhere, Category = Validation)
	bool bOnlyIncludeExposedUObjects = false;

	/** List data interfaces to validate against, if this list is empty all data interfaces will be included. */
	UPROPERTY(EditAnywhere, Category = Validation)
	TArray<TSubclassOf<UNiagaraDataInterface>> BannedDataInterfaces;

	/** List data interfaces that we always allow. */
	UPROPERTY(EditAnywhere, Category = Validation)
	TArray<TSubclassOf<UNiagaraDataInterface>> AllowDataInterfaces;

	/** How do we want to repro the error in the stack */
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;
};

/** This validation rule checks that a module is only used once per emitter/system stack. */
UCLASS(Category = "Validation", DisplayName = "Singleton Module")
class UNiagaraValidationRule_SingletonModule : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;

	/** How do we want to repro the error in the stack */
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	/** If true then the check is not emitter-wide, but only within the same context (e.g. particle update). */
	UPROPERTY(EditAnywhere, Category = Validation)
	bool bCheckDetailedUsageContext = false;
};

/** This validation rule checks that map for nodes are not used with cpu scripts (as they only work on gpu). */
UCLASS(Category = "Validation", DisplayName = "MapFor on CPU Check")
class UNiagaraValidationRule_NoMapForOnCpu : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;

	/** How do we want to report the error in the stack */
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Error;

private:

	struct FGraphCheckResult
	{
		FGuid ChangeID;
		bool bContainsMapForNode = false;
	};
	mutable TMap<FObjectKey, FGraphCheckResult> CachedResults;
};

/** This validation rule checks that a module is only used in emitters with the configured runtime target. */
UCLASS(Category = "Validation", DisplayName = "Module SimTarget Restriction")
class UNiagaraValidationRule_ModuleSimTargetRestriction : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;

	/** How do we want to report the error in the stack */
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Error;
	
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraSimTarget SupportedSimTarget = ENiagaraSimTarget::CPUSim;
};

/** Checks that the materials assigned to renderers have the correct usage with flags enabled. */
UCLASS(Category = "Validation", DisplayName = "Material Usage")
class UNiagaraValidationRule_MaterialUsage : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity FailedUsageSeverity = ENiagaraValidationSeverity::Error;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** Checks that all emitters with a parent emitter are using the latest version of the parent. */
UCLASS(Category = "Validation", DisplayName = "Require Latest Parent Emitter Version")
class UNiagaraValidationRule_RequireLatestParentEmitterVersion : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Warning;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};

/** Checks that all emitters with a parent emitter are using one of a set of allowed parent emitters. */
UCLASS(Category = "Validation", DisplayName = "Require Parent Emitter(s)")
class UNiagaraValidationRule_RequireParentEmitter : public UNiagaraValidationRule
{
	GENERATED_BODY()

public:

	/** The collection of allowed parent emitters */
	UPROPERTY(EditAnywhere, Category = Validation)
	TArray<TSoftObjectPtr<UNiagaraEmitter>> AllowedParentEmitters;

	/** True if emitters must have a parent emitter, false to allow no parent emitter */
	UPROPERTY(EditAnywhere, Category = Validation)
	bool bMustHaveParentEmitter = true;

	UPROPERTY(EditAnywhere, Category = Validation)
	ENiagaraValidationSeverity Severity = ENiagaraValidationSeverity::Error;

	virtual void CheckValidity(const FNiagaraValidationContext& Context, TArray<FNiagaraValidationResult>& OutResults) const override;
};
