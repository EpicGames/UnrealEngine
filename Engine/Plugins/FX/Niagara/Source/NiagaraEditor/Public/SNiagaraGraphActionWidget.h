// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "NiagaraActions.h"
#include "SlateFwd.h"
#include "SGraphActionMenu.h"
#include "EdGraph/EdGraphSchema.h"
#include "ViewModels/NiagaraPreviewMovieViewModel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FCreateWidgetForActionData;

DECLARE_DELEGATE_RetVal_OneParam(bool, FOnActionClicked, TSharedPtr<FNiagaraMenuAction_Generic>)

struct FCreateNiagaraWidgetForActionData
{
	FCreateNiagaraWidgetForActionData(const TSharedPtr<FNiagaraMenuAction_Generic> InAction)
		: Action(InAction)
	{}
	const TSharedPtr<FNiagaraMenuAction_Generic> Action; 
	/** True if we want to use the mouse delegate */
	bool bHandleMouseButtonDown;
	/** The delegate to determine if the current action is selected in the row */
	FIsSelected IsRowSelectedDelegate;
	/** The text to highlight */
	TAttribute<FText> HighlightText;
	/** True if the widget should be read only - no renaming allowed */
	bool bIsReadOnly = true;
	/** If specified, adds a 'favorite' toggle to the widget */
	TOptional<FName> FavoriteActionsProfileName;
	/** Needs to be supplied for preview movie playback to work. */
	TSharedPtr<FNiagaraPreviewMovieViewModel> PreviewMovieViewModel;
};

/** Custom widget for GraphActionMenu */
class SNiagaraGraphActionWidget : public SCompoundWidget
{
	public:

	SLATE_BEGIN_ARGS( SNiagaraGraphActionWidget ) {}
		SLATE_ATTRIBUTE(FText, HighlightText)
SLATE_END_ARGS()

NIAGARAEDITOR_API void Construct(const FArguments& InArgs, const FCreateWidgetForActionData* InCreateData);
	NIAGARAEDITOR_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	private:
	/** The item that we want to display with this widget */
	TWeakPtr<struct FEdGraphSchemaAction> ActionPtr;
	/** Delegate executed when mouse button goes down */
	FCreateWidgetMouseButtonDown MouseButtonDownDelegate;
};

/** Custom widget for GraphActionMenu */
class SNiagaraActionWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SNiagaraActionWidget )
		: _bShowTypeIfParameter(true)
		{} 
		SLATE_ARGUMENT(bool, bShowTypeIfParameter)
	SLATE_END_ARGS()

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, const FCreateNiagaraWidgetForActionData& InCreateData);
	//virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

private:
	TSharedRef<SWidget> CreateFavoriteActionWidget();
	FReply OnFavoriteStateChanged();
	ECheckBoxState OnGetIsFavorite() const;
	EVisibility OnShouldShowFavoriteButton() const;
	const FSlateBrush* GetFavoriteBrush() const;
	FText GetFavoriteButtonTooltipText() const;
	TSharedPtr<IToolTip> GetCustomTooltip() const;
private:
	/** The item that we want to display with this widget */
	TSharedPtr<struct FNiagaraMenuAction_Generic> ActionPtr;
	TWeakPtr<FNiagaraPreviewMovieViewModel> PreviewMovieViewModel;
	/** Delegate executed when mouse button goes down */
	FOnActionClicked MouseButtonDownDelegate;
	FNiagaraActionSourceData SourceData;
	TOptional<FName> FavoriteProfileName;
	mutable TSharedPtr<IToolTip> TooltipCache;
};
