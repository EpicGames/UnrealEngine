// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "UObject/StructOnScope.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"

/** Base class for inline parameter editors. These editors are expected to maintain an internal value which
	is populated from a parameter struct. */
class SNiagaraParameterEditor : public SCompoundWidget
{
public:
	DECLARE_DELEGATE(FOnValueChange);

public:
	SLATE_BEGIN_ARGS(SNiagaraParameterEditor) 
		: _HAlign(HAlign_Left)
		, _VAlign(VAlign_Center)
	{ }
		SLATE_ARGUMENT(EHorizontalAlignment, HAlign)
		SLATE_ARGUMENT(EVerticalAlignment, VAlign)
		SLATE_ARGUMENT(TOptional<float>, MinimumDesiredWidth)
		SLATE_ARGUMENT(TOptional<float>, MaximumDesiredWidth)
	SLATE_END_ARGS();

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs);

	/** Updates the internal value of the widget from a struct. */
	virtual void UpdateInternalValueFromStruct(TSharedRef<FStructOnScope> Struct) = 0;

	/** Updates a struct from the internal value of the widget. */
	virtual void UpdateStructFromInternalValue(TSharedRef<FStructOnScope> Struct) = 0;
	
	/** Gets whether this is currently the exclusive editor of this parameter, meaning that the corresponding details view
		should not be updated.  This hack is necessary because the details view closes all color pickers when
		it's changed! */
	NIAGARAEDITOR_API bool GetIsEditingExclusively();

	/** Sets the OnBeginValueChange delegate which is run when a continuous internal value change begins. */
	NIAGARAEDITOR_API void SetOnBeginValueChange(FOnValueChange InOnBeginValueChange);

	/** Sets the OnBeginValueChange delegate which is run when a continuous internal value change ends. */
	NIAGARAEDITOR_API void SetOnEndValueChange(FOnValueChange InOnEndValueChange);

	/** Sets the OnValueChanged delegate which is run when the internal value changes. */
	NIAGARAEDITOR_API void SetOnValueChanged(FOnValueChange InOnValueChanged);

	/** Gets an optional minimum desired width for this parameter editor. */
	NIAGARAEDITOR_API const TOptional<float>& GetMinimumDesiredWidth() const;

	/** Sets the minimum desired width for this parameter editor.  If the option parameter is not set it clears the current value. */
	NIAGARAEDITOR_API void SetMinimumDesiredWidth(TOptional<float> InMinimumDesiredWidth);

	/** Gets an optional maximum desired width for this parameter editor. */
	NIAGARAEDITOR_API const TOptional<float>& GetMaximumDesiredWidth() const;

	/** Sets the maximum desired width for this parameter editor.  If the option parameter is not set it clears the current value. */
	NIAGARAEDITOR_API void SetMaximumDesiredWidth(TOptional<float> InMaximumDesiredWidth);

	/** Gets the desired horizontal alignment of this parameter editor for it's parent container. */
	NIAGARAEDITOR_API EHorizontalAlignment GetHorizontalAlignment() const;

	/** Gets the desired horizontal alignment of this parameter editor for it's parent container. */
	NIAGARAEDITOR_API EVerticalAlignment GetVerticalAlignment() const;

	//~ SWidget interface
	NIAGARAEDITOR_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

	/** */
	virtual bool CanChangeContinuously() const { return false; }
protected:
	/** Sets whether this is currently the exclusive editor of this parameter, meaning that the corresponding details view
		should not be updated.  This hack is necessary because the details view closes all color pickers when
		it's changed! */
	NIAGARAEDITOR_API void SetIsEditingExclusively(bool bInIsEditingExclusively);

	/** Executes the OnBeginValueChange delegate */
	NIAGARAEDITOR_API void ExecuteOnBeginValueChange();

	/** Executes the OnEndValueChange delegate. */
	NIAGARAEDITOR_API void ExecuteOnEndValueChange();

	/** Executes the OnValueChanged delegate. */
	NIAGARAEDITOR_API void ExecuteOnValueChanged();

	template<typename NumericType>
	static TSharedPtr<TNumericUnitTypeInterface<NumericType>> GetTypeInterface(EUnit TypeUnit)
	{
		if (FUnitConversion::Settings().ShouldDisplayUnits())
		{
			return MakeShareable(new TNumericUnitTypeInterface<NumericType>(TypeUnit));
		}
		return MakeShareable(new TNumericUnitTypeInterface<NumericType>(EUnit::Unspecified));
	}

protected:
	static NIAGARAEDITOR_API const float DefaultInputSize;

private:
	//** Flag set when a continuous change is in progress to prevent invoking ExecuteOnBeginValueChange() or ExecuteOnEndValueChange() more than once for a change. */
	bool bContinousChangeActive;

	/** Whether this is currently the exclusive editor of this parameter, meaning that the corresponding details view
		should not be updated.  This hack is necessary because the details view closes all color pickers when
		it's changed! */
	bool bIsEditingExclusively;

	/** A delegate which is executed when a continuous change to the internal value begins. */
	FOnValueChange OnBeginValueChange;

	/** A delegate which is executed when a continuous change to the internal value ends. */
	FOnValueChange OnEndValueChange;

	/** A delegate which is executed when the internal value changes. */
	FOnValueChange OnValueChanged;

	/** The minimum desired width of this parameter editor. */
	TOptional<float> MinimumDesiredWidth;

	/** The maximum desired width of this parameter editor. */
	TOptional<float> MaximumDesiredWidth;

	/** The desired horizontal alignment of this parameter editor for it's parent container. */
	EHorizontalAlignment HorizontalAlignment;

	/** Sets the desired horizontal alignment of this parameter editor for it's parent container. */
	EVerticalAlignment VerticalAlignment;
};

