// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

template<typename TSourceItem, typename TTargetItem, typename TIndexable>
class TSharedRefCollectionBase
{
public:
	class FIterator
	{
	public:
		FIterator(const TIndexable* InSource, int32 InIndex)
			: Source(InSource)
			, Index(InIndex)
		{
		}

		bool operator!=(const FIterator& Other)
		{
			return Source != Other.Source || Index != Other.Index;
		}

		FIterator& operator++()
		{
			Index++;
			return *this;
		}

		const TSharedRef<TTargetItem> operator*() const
		{
			return (*Source)[Index];
		}

	private:
		const TIndexable* Source = nullptr;
		int32 Index = INDEX_NONE;
	};

	TSharedRefCollectionBase()
	{
	}

	TSharedRefCollectionBase(const TIndexable* InSource)
		: Source(InSource)
	{
	}

	FIterator begin() const
	{
		return Source != nullptr ? FIterator(Source, 0) : FIterator(nullptr, INDEX_NONE);
	}

	FIterator end() const
	{
		return Source != nullptr ? FIterator(Source, Source->Num()) : FIterator(nullptr, INDEX_NONE);
	}

	int32 Num() const 
	{
		return Source != nullptr ? Source->Num() : 0;
	}

	TSharedRef<TTargetItem> operator[](int32 Index) const
	{
		checkf(Source != nullptr, TEXT("Source is invalid"));
		checkf(Index >= 0 && Index < Source->Num(), TEXT("Index out of range"));
		return (*Source)[Index];
	}

	template<typename TPredicate>
	int32 IndexOfByPredicate(TPredicate Predicate)
	{
		return Source != nullptr ? Source->IndexOfByPredicate(Predicate) : INDEX_NONE;
	}

	template<typename TPredicate>
	TSharedPtr<TTargetItem> FindByPredicate(TPredicate Predicate)
	{
		int32 FoundIndex = IndexOfByPredicate(Predicate);
		return Source != nullptr && FoundIndex != INDEX_NONE ? (*Source)[FoundIndex] : TSharedPtr<TTargetItem>();
	}

	template<typename TPredicate>
	void FindAllByPredicate(TArray<TSharedRef<TTargetItem>>& OutFound, TPredicate Predicate)
	{
		if (Source == nullptr)
		{
			return;
		}
		for (const TSharedRef<TTargetItem>& Item : (*Source))
		{
			if (Predicate(Item))
			{
				OutFound.Add(Item);
			}
		}
	}

private:
	const TIndexable* Source = nullptr;
};

template<typename TItem>
class TSharedRefCollection : public TSharedRefCollectionBase<TItem, TItem, TArray<TSharedRef<TItem>>>
{
}
;

template<typename TItem>
class TConstSharedRefCollection : public TSharedRefCollectionBase<TItem, const TItem, TArray<TSharedRef<TItem>>>
{
};

class IAdapter
{
public:
	virtual ~IAdapter() = default;
	virtual bool IsValidAdapter() const = 0;
	virtual bool IsValidWriteAdapter() const = 0;
};

template<typename TSourceObject, typename TAdapter>
class TAdapterRefListOneSource
{
public:
	class IFactory
	{
	public:
		virtual ~IFactory() = default;
		virtual void CreateAdapters(TSourceObject* SourceObject, TArray<TSharedRef<TAdapter>>& OutAdapters) const = 0;
		virtual void CreateAdapters(const TSourceObject* SourceObject, TArray<TSharedRef<TAdapter>>& OutAdapters) const = 0;
	};

	void Initialize(TSourceObject* SourceObject, TSharedRef<const IFactory> InAdapterFactory)
	{
		SourceWeak = SourceObject;
		AdapterFactory = InAdapterFactory;
	}

	void Initialize(const TSourceObject* SourceObject, TSharedRef<const IFactory> InAdapterFactory)
	{
		SourceConstWeak = SourceObject;
		AdapterFactory = InAdapterFactory;
	}

	TSharedRefCollection<TAdapter> Get()
	{
		if (ensureMsgf(SourceWeak.IsSet(), TEXT("Adapter is only valid for const use.")))
		{
			if (AdapterList.IsSet() == false)
			{
				TArray<TSharedRef<TAdapter>> Adapters;
				AdapterFactory->CreateAdapters(SourceWeak.GetValue().Get(), Adapters);
				AdapterList.Emplace(Adapters);
			}
		}
		else
		{
			if (AdapterList.IsSet() == false)
			{
				AdapterList = TArray<TSharedRef<TAdapter>>();
			}
		}
		return TSharedRefCollection<TAdapter>(&AdapterList.GetValue());
	}

	TConstSharedRefCollection<TAdapter> Get() const
	{
		if (AdapterList.IsSet() == false)
		{
			TArray<TSharedRef<TAdapter>> Adapters;
			if (SourceWeak.IsSet())
			{
				AdapterFactory->CreateAdapters(SourceWeak.GetValue().Get(), Adapters);
			}
			else
			{
				AdapterFactory->CreateAdapters(SourceConstWeak.Get(), Adapters);
			}
			AdapterList.Emplace(Adapters);
		}
		return TConstSharedRefCollection<TAdapter>(&AdapterList.GetValue());
	}

private:
	TOptional<TWeakObjectPtr<TSourceObject>> SourceWeak;
	TWeakObjectPtr<const TSourceObject> SourceConstWeak;
	TSharedPtr<const IFactory> AdapterFactory;

	mutable TOptional<TArray<TSharedRef<TAdapter>>> AdapterList;
};

template<typename TSourceObject1, typename TSourceObject2, typename TAdapter>
class TAdapterRefListTwoSources
{
public:
	class IFactory
	{
	public:
		virtual ~IFactory() = default;
		virtual void CreateAdapters(TSourceObject1* SourceObject1, TSourceObject2* SourceObject2, TArray<TSharedRef<TAdapter>>& OutAdapters) const = 0;
		virtual void CreateAdapters(const TSourceObject1* SourceObject1, const TSourceObject2* SourceObject2, TArray<TSharedRef<TAdapter>>& OutAdapters) const = 0;
	};

	void Initialize(TSourceObject1* SourceObject1, TSourceObject2* SourceObject2, TSharedRef<const IFactory> InAdapterFactory)
	{
		SourceWeak1 = SourceObject1;
		SourceWeak2 = SourceObject2;
		AdapterFactory = InAdapterFactory;
	}

	void Initialize(const TSourceObject1* SourceObject1, const TSourceObject2* SourceObject2, TSharedRef<const IFactory> InAdapterFactory)
	{
		SourceConstWeak1 = SourceObject1;
		SourceConstWeak2 = SourceObject2;
		AdapterFactory = InAdapterFactory;
	}

	TSharedRefCollection<TAdapter> Get()
	{
		if (ensureMsgf(SourceWeak1.IsSet() && SourceWeak2.IsSet(), TEXT("Adapter is only valid for const use.")))
		{
			if (AdapterList.IsSet() == false)
			{
				TArray<TSharedRef<TAdapter>> Adapters;
				AdapterFactory->CreateAdapters(SourceWeak1.GetValue().Get(), SourceWeak2.GetValue().Get(), Adapters);
				AdapterList.Emplace(Adapters);
			}
		}
		else
		{
			if (AdapterList.IsSet() == false)
			{
				AdapterList = TArray<TSharedRef<TAdapter>>();
			}
		}
		return TSharedRefCollection<TAdapter>(&AdapterList.GetValue());
	}

	TConstSharedRefCollection<TAdapter> Get() const
	{
		if (AdapterList.IsSet() == false)
		{
			TArray<TSharedRef<TAdapter>> Adapters;
			if (SourceWeak1.IsSet() && SourceWeak2.IsSet())
			{
				AdapterFactory->CreateAdapters(SourceWeak1.GetValue().Get(), SourceWeak2.GetValue().Get(), Adapters);
			}
			else
			{
				AdapterFactory->CreateAdapters(SourceConstWeak1.Get(), SourceConstWeak2.Get(), Adapters);
			}
			AdapterList.Emplace(Adapters);
		}
		return TConstSharedRefCollection<TAdapter>(&AdapterList.GetValue());
	}

private:
	TOptional<TWeakObjectPtr<TSourceObject1>> SourceWeak1;
	TOptional<TWeakObjectPtr<TSourceObject2>> SourceWeak2;
	TWeakObjectPtr<const TSourceObject1> SourceConstWeak1;
	TWeakObjectPtr<const TSourceObject2> SourceConstWeak2;
	TSharedPtr<const IFactory> AdapterFactory;

	mutable TOptional<TArray<TSharedRef<TAdapter>>> AdapterList;
};

template<typename TSourceObject1, typename TSourceObject2, typename TSourceObject3, typename TAdapter>
class TAdapterRefListThreeSources
{
public:
	class IFactory
	{
	public:
		virtual ~IFactory() = default;
		virtual void CreateAdapters(TSourceObject1* SourceObject1, TSourceObject2* SourceObject2, TSourceObject3* SourceObject3, TArray<TSharedRef<TAdapter>>& OutAdapters) const = 0;
		virtual void CreateAdapters(const TSourceObject1* SourceObject1, const TSourceObject2* SourceObject2, const TSourceObject3* SourceObject3, TArray<TSharedRef<TAdapter>>& OutAdapters) const = 0;
	};

	void Initialize(TSourceObject1* SourceObject1, TSourceObject2* SourceObject2, TSourceObject3* SourceObject3, TSharedRef<const IFactory> InAdapterFactory)
	{
		SourceWeak1 = SourceObject1;
		SourceWeak2 = SourceObject2;
		SourceWeak3 = SourceObject3;
		AdapterFactory = InAdapterFactory;
	}

	void Initialize(const TSourceObject1* SourceObject1, const TSourceObject2* SourceObject2, const TSourceObject3* SourceObject3, TSharedRef<const IFactory> InAdapterFactory)
	{
		SourceConstWeak1 = SourceObject1;
		SourceConstWeak2 = SourceObject2;
		SourceConstWeak3 = SourceObject3;
		AdapterFactory = InAdapterFactory;
	}

	TSharedRefCollection<TAdapter> Get()
	{
		if (ensureMsgf(SourceWeak1.IsSet() && SourceWeak2.IsSet() && SourceWeak3.IsSet(), TEXT("Adapter is only valid for const use.")))
		{
			if (AdapterList.IsSet() == false)
			{
				TArray<TSharedRef<TAdapter>> Adapters;
				AdapterFactory->CreateAdapters(SourceWeak1.GetValue().Get(), SourceWeak2.GetValue().Get(), SourceWeak3.GetValue().Get(), Adapters);
				AdapterList.Emplace(Adapters);
			}
		}
		else
		{
			if (AdapterList.IsSet() == false)
			{
				AdapterList = TArray<TSharedRef<TAdapter>>();
			}
		}
		return TSharedRefCollection<TAdapter>(&AdapterList.GetValue());
	}

	TConstSharedRefCollection<TAdapter> Get() const
	{
		if (AdapterList.IsSet() == false)
		{
			TArray<TSharedRef<TAdapter>> Adapters;
			if (SourceWeak1.IsSet() && SourceWeak2.IsSet() && SourceWeak3.IsSet())
			{
				AdapterFactory->CreateAdapters(SourceWeak1.GetValue().Get(), SourceWeak2.GetValue().Get(), SourceWeak3.GetValue().Get(), Adapters);
			}
			else
			{
				AdapterFactory->CreateAdapters(SourceConstWeak1.Get(), SourceConstWeak2.Get(), SourceConstWeak3.Get(), Adapters);
			}
			AdapterList.Emplace(Adapters);
		}
		return TConstSharedRefCollection<TAdapter>(&AdapterList.GetValue());
	}

private:
	TOptional<TWeakObjectPtr<TSourceObject1>> SourceWeak1;
	TOptional<TWeakObjectPtr<TSourceObject2>> SourceWeak2;
	TOptional<TWeakObjectPtr<TSourceObject3>> SourceWeak3;
	TWeakObjectPtr<const TSourceObject1> SourceConstWeak1;
	TWeakObjectPtr<const TSourceObject2> SourceConstWeak2;
	TWeakObjectPtr<const TSourceObject3> SourceConstWeak3;
	TSharedPtr<const IFactory> AdapterFactory;

	mutable TOptional<TArray<TSharedRef<TAdapter>>> AdapterList;
};

template<typename TSourceObject, typename TAdapter>
class TAdapterPtr
{
public:
	class IFactory
	{
	public:
		virtual ~IFactory() = default;
		virtual TSharedRef<TAdapter> CreateAdapter(TSourceObject* InSourceObject) const = 0;
		virtual TSharedRef<TAdapter> CreateAdapter(const TSourceObject* InSourceObject) const = 0;
	};

	void Initialize(TSourceObject* InSourceObject, TSharedRef<const IFactory> InAdapterFactory)
	{
		SourceWeak = InSourceObject;
		AdapterFactory = InAdapterFactory;
	}

	void Initialize(const TSourceObject* InSourceObject, TSharedRef<const IFactory> InAdapterFactory)
	{
		SourceConstWeak = InSourceObject;
		AdapterFactory = InAdapterFactory;
	}

	TSharedRef<TAdapter> Get()
	{
		checkf(SourceWeak.IsSet(), TEXT("Adapter is only valid for const use."));
		if (Adapter.IsSet() == false)
		{
			Adapter.Emplace(AdapterFactory->CreateAdapter(SourceWeak.GetValue().Get()));
		}
		return Adapter.GetValue();
	}

	TSharedRef<const TAdapter> Get() const
	{
		if (Adapter.IsSet() == false)
		{
			if (SourceWeak.IsSet())
			{
				Adapter.Emplace(AdapterFactory->CreateAdapter(SourceWeak.GetValue().Get()));
			}
			else
			{
				Adapter.Emplace(AdapterFactory->CreateAdapter(SourceConstWeak.Get()));
			}
		}
		return Adapter.GetValue();
	}

private:
	TOptional<TWeakObjectPtr<TSourceObject>> SourceWeak;
	TWeakObjectPtr<const TSourceObject> SourceConstWeak;
	TSharedPtr<const IFactory> AdapterFactory;

	mutable TOptional<TSharedRef<TAdapter>> Adapter;
};