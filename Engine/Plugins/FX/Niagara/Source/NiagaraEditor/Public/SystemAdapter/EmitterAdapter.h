// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "SystemAdapter/AdapterShared.h"
#include "SystemAdapter/StackScriptAdapterCollection.h"

struct FNiagaraEmitterHandle;
class  UNiagaraSystem;

namespace UE::Niagara
{
	class FStackScriptAdapter;

	class FEmitterAdapter : public TSharedFromThis<FEmitterAdapter>, public IAdapter, public IStackScriptAdapterOwner
	{
	public:
		FEmitterAdapter()
		{
		}

		NIAGARAEDITOR_API static TSharedRef<FEmitterAdapter> Create(UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle);

		static TSharedRef<FEmitterAdapter> Create(const UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle);

		NIAGARAEDITOR_API static TSharedRef<const FEmitterAdapter> CreateConst(const UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle);

		NIAGARAEDITOR_API virtual bool IsValidAdapter() const override;

		NIAGARAEDITOR_API virtual bool IsValidWriteAdapter() const override;

		NIAGARAEDITOR_API FName GetName() const { return EmitterName; }

		NIAGARAEDITOR_API TSharedRef<FStackScriptAdapter> GetEmitterSpawnScript();

		NIAGARAEDITOR_API TSharedRef<const FStackScriptAdapter> GetEmitterSpawnScript() const;

		NIAGARAEDITOR_API TSharedRef<FStackScriptAdapter> GetEmitterUpdateScript();

		NIAGARAEDITOR_API TSharedRef<const FStackScriptAdapter> GetEmitterUpdateScript() const;

		NIAGARAEDITOR_API TSharedRef<FStackScriptAdapter> GetParticleSpawnScript();

		NIAGARAEDITOR_API TSharedRef<const FStackScriptAdapter> GetParticleSpawnScript() const;

		NIAGARAEDITOR_API TSharedRef<FStackScriptAdapter> GetParticleUpdateScript();

		NIAGARAEDITOR_API TSharedRef<const FStackScriptAdapter> GetParticleUpdateScript() const;

		virtual int32 GetNumScripts() const override;
		virtual TSharedRef<FStackScriptAdapter> GetScriptAt(int32 Index) override;

		NIAGARAEDITOR_API FScriptSharedRefCollection GetScripts();

		NIAGARAEDITOR_API FScriptConstSharedRefCollection GetScripts() const;

	private:
		void Initialize(UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle);

		void Initialize(const UNiagaraSystem* InOwningSystem, const FNiagaraEmitterHandle& InEmitterHandle);

	private:
		TOptional<TWeakObjectPtr<UNiagaraSystem>> OwningSystemWeak;
		TWeakObjectPtr<const UNiagaraSystem> OwningSystemConstWeak;
		FName EmitterName;
		FGuid EmitterHandleId;

		TAdapterPtr<UNiagaraSystem, FStackScriptAdapter> EmitterSpawnScript;
		TAdapterPtr<UNiagaraSystem, FStackScriptAdapter> EmitterUpdateScript;

		TAdapterPtr<UNiagaraSystem, FStackScriptAdapter> ParticleSpawnScript;
		TAdapterPtr<UNiagaraSystem, FStackScriptAdapter> ParticleUpdateScript;

		FStackScriptAdapterCollection StackScriptAdapterCollection;
	};

	typedef TSharedRef<FEmitterAdapter> FEmitterAdapterRef;
	typedef TSharedRef<const FEmitterAdapter> FEmitterAdapterConstRef;

} // namespace UE::Niagara