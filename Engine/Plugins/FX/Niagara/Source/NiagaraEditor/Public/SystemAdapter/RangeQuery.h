// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"

namespace UE::RangeQuery
{

template<typename TItem>
class IRangeQueryIterator
{
public:
	virtual ~IRangeQueryIterator() = default;

	virtual bool IsValidIterator() const = 0;
	virtual void NextItem() = 0;
	virtual TItem GetItem() const = 0;
};

template<typename TSourceItem, typename TTargetItem>
class IRangeQueryWrapperIterator : public IRangeQueryIterator<TTargetItem>
{
public:
	virtual void SetSourceIterator(TSharedRef<IRangeQueryIterator<TSourceItem>> InSourceIterator) = 0;
};

template<typename TItem>
class TRangeQuery
{
public:
	class FIterator
	{
	public:
		FIterator(IRangeQueryIterator<TItem>* InSourceIterator)
			: SourceIterator(InSourceIterator)
		{
		}

		bool operator!=(const FIterator& Other) const
		{
			return IsValidIterator() != Other.IsValidIterator();
		}

		FIterator& operator++()
		{
			SourceIterator->NextItem();
			return *this;
		}

		TItem operator*() const
		{
			return SourceIterator->GetItem();
		}

	private:
		bool IsValidIterator() const
		{
			return SourceIterator != nullptr && SourceIterator->IsValidIterator();
		}

	private:
		IRangeQueryIterator<TItem>* SourceIterator = nullptr;
	};

	TRangeQuery()
	{
	}

	TRangeQuery(TSharedRef<IRangeQueryIterator<TItem>> InSourceIterator)
		: SourceIterator(InSourceIterator)
	{
	}

	const TSharedPtr<IRangeQueryIterator<TItem>>& GetSourceIterator() const { return SourceIterator; }
	void SetSourceIterator(TSharedRef<IRangeQueryIterator<TItem>> InSourceIterator)
	{
		SourceIterator = InSourceIterator;
	}

	FIterator begin() const
	{
		return FIterator(SourceIterator.Get());
	}

	FIterator end() const
	{
		return FIterator(nullptr);
	}

	TRangeQuery<TItem>& Add(TSharedRef<IRangeQueryWrapperIterator<TItem, TItem>> InWrapper)
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		InWrapper->SetSourceIterator(SourceIterator.ToSharedRef());
		SourceIterator = InWrapper;
		return *this;
	}

	template<typename TTargetItem>
	TRangeQuery<TTargetItem> Add(TSharedRef<IRangeQueryWrapperIterator<TItem, TTargetItem>> InWrapper)
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		InWrapper->SetSourceIterator(SourceIterator.ToSharedRef());
		return TRangeQuery<TTargetItem>(InWrapper);
	}

	TRangeQuery<TItem>& operator+(TSharedRef<IRangeQueryWrapperIterator<TItem, TItem>> InWrapper)
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		InWrapper->SetSourceIterator(SourceIterator.ToSharedRef());
		SourceIterator = InWrapper;
		return *this;
	}

	template<typename TTargetItem>
	TRangeQuery<TTargetItem> operator+(TSharedRef<IRangeQueryWrapperIterator<TItem, TTargetItem>> InWrapper)
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		InWrapper->SetSourceIterator(SourceIterator.ToSharedRef());
		return TRangeQuery<TTargetItem>(InWrapper);
	}

	TItem First() const
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		for (const TItem& Item : *this)
		{
			return Item;
		}
		return nullptr;
	}

	template<typename TDefault>
	TDefault FirstOrDefault(TDefault InDefault) const
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		for (const TItem& Item : *this)
		{
			return Item;
		}
		return InDefault;
	}

	template<typename TArrayItem>
	void AppendToArray(TArray<TArrayItem>& OutArray)
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		for (const TItem& Item : *this)
		{
			OutArray.Add(Item);
		}
	}

private:
	TSharedPtr<IRangeQueryIterator<TItem>> SourceIterator;
};

template<typename TItem, typename TIndexable>
class TRangeQueryIndexableIterator : public IRangeQueryIterator<TItem>
{
public:
	static TSharedRef<TRangeQueryIndexableIterator<TItem, TIndexable>> CreateAsRef(const TIndexable* InIndexableRange)
	{
		TSharedRef<TRangeQueryIndexableIterator<TItem, TIndexable>> Iterator = MakeShared<TRangeQueryIndexableIterator<TItem, TIndexable>>();
		Iterator->SetSource(InIndexableRange, false);
		return Iterator;
	}

	static TSharedRef<TRangeQueryIndexableIterator<TItem, TIndexable>> CreateAsCopy(const TIndexable& InIndexableRange)
	{
		TIndexable* IndexableRangeCopy = new TIndexable();
		*IndexableRangeCopy = InIndexableRange;
		
		TSharedRef<TRangeQueryIndexableIterator<TItem, TIndexable>> Iterator = MakeShared<TRangeQueryIndexableIterator<TItem, TIndexable>>();
		Iterator->SetSource(IndexableRangeCopy, true);
		return Iterator;
	}

	TRangeQueryIndexableIterator()
	{
	}

	~TRangeQueryIndexableIterator()
	{
		if (bOwnsRange)
		{
			delete IndexableRange;
		}
	}

	virtual bool IsValidIterator() const override
	{
		return Index >= 0 && IndexableRange != nullptr && Index < IndexableRange->Num();
	}

	virtual void NextItem() override
	{
		Index++;
	}

	virtual TItem GetItem() const override
	{
		return (*IndexableRange)[Index];
	}

private:
	void SetSource(const TIndexable* InIndexableRange, bool bInOwnsRange)
	{
		IndexableRange = InIndexableRange;
		Index = 0;
		bOwnsRange = bInOwnsRange;
	}

private:
	const TIndexable* IndexableRange = nullptr;
	int32 Index = INDEX_NONE;
	bool bOwnsRange = false;
};

template<typename TItem>
class TRangeQueryPredicateIterator : public IRangeQueryWrapperIterator<TItem, TItem>
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(bool, FPredicate, const TItem& /* Item */);

public:
	TRangeQueryPredicateIterator(TSharedRef<IRangeQueryIterator<TItem>> InSourceIterator, FPredicate InPredicate)
		: SourceIterator(InSourceIterator)
		, Predicate(InPredicate)
	{
	}

	TRangeQueryPredicateIterator(FPredicate InPredicate)
		: Predicate(InPredicate)
	{
	}

	virtual void SetSourceIterator(TSharedRef<IRangeQueryIterator<TItem>> InSourceIterator) override
	{
		SourceIterator = InSourceIterator;
	}

	virtual bool IsValidIterator() const override
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		if (bMovedToInitialValidItem == false)
		{
			if (SourceIterator->IsValidIterator() && Predicate.Execute(SourceIterator->GetItem()) == false)
			{
				MoveToNextValidItem();
			}
			bMovedToInitialValidItem = true;
		}
		return SourceIterator->IsValidIterator();
	}

	virtual void NextItem() override
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		MoveToNextValidItem();
	}

	virtual TItem GetItem() const override
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		return SourceIterator->GetItem();
	}

private:
	void MoveToNextValidItem() const
	{
		SourceIterator->NextItem();
		while (SourceIterator->IsValidIterator() && Predicate.Execute(SourceIterator->GetItem()) == false)
		{
			SourceIterator->NextItem();
		}
	}

private:
	TSharedPtr<IRangeQueryIterator<TItem>> SourceIterator;
	FPredicate Predicate;
	mutable bool bMovedToInitialValidItem = false;
};

template<typename TItem, typename TSelectedItem>
class TRangeQuerySelectIterator : public IRangeQueryWrapperIterator<TItem, TSelectedItem>
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(TSelectedItem, FSelecter, const TItem&);

public:
	TRangeQuerySelectIterator(TSharedRef<IRangeQueryIterator<TItem>> InSourceIterator, FSelecter InSelecter)
		: SourceIterator(InSourceIterator)
		, Selecter(InSelecter)
	{
	}

	TRangeQuerySelectIterator(FSelecter InSelecter)
		: Selecter(InSelecter)
	{
	}

	virtual void SetSourceIterator(TSharedRef<IRangeQueryIterator<TItem>> InSourceIterator) override
	{
		SourceIterator = InSourceIterator;
	}

	virtual bool IsValidIterator() const override
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		return SourceIterator->IsValidIterator();
	}

	virtual void NextItem() override
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		SourceIterator->NextItem();
	}

	virtual TSelectedItem GetItem() const override
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		return Selecter.Execute(SourceIterator->GetItem());
	}

private:
	TSharedPtr<IRangeQueryIterator<TItem>> SourceIterator;
	FSelecter Selecter;
};

template<typename TItem, typename TSelectedItem>
class TRangeQuerySelectRangeIterator : public IRangeQueryWrapperIterator<TItem, TSelectedItem>
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<IRangeQueryIterator<TSelectedItem>>, FRangeSelecter, TItem);

public:
	TRangeQuerySelectRangeIterator(TSharedRef<IRangeQueryIterator<TItem>> InSourceIterator, FRangeSelecter InRangeSelecter)
		: SourceIterator(InSourceIterator)
		, RangeSelecter(InRangeSelecter)
	{
	}

	TRangeQuerySelectRangeIterator(FRangeSelecter InRangeSelecter)
		: RangeSelecter(InRangeSelecter)
	{

	}

	virtual void SetSourceIterator(TSharedRef<IRangeQueryIterator<TItem>> InSourceIterator) override
	{
		SourceIterator = InSourceIterator;
	}

	virtual bool IsValidIterator() const override
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		if (bMovedToInitialValidRangeIterator == false)
		{
			if (SourceIterator->IsValidIterator())
			{
				SelectedRangeIterator.Emplace(RangeSelecter.Execute(SourceIterator->GetItem()));
				if (SelectedRangeIterator.GetValue()->IsValidIterator() == false)
				{
					MoveToNextValidRangeIterator();
				}
			}
			bMovedToInitialValidRangeIterator = true;
		}
		return SelectedRangeIterator.IsSet() && SelectedRangeIterator.GetValue()->IsValidIterator();
	}

	virtual void NextItem() override
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		checkf(SelectedRangeIterator.IsSet(), TEXT("Iterator is not valid."));
		SelectedRangeIterator.GetValue()->NextItem();
		if (SelectedRangeIterator.GetValue()->IsValidIterator() == false && SourceIterator->IsValidIterator())
		{
			MoveToNextValidRangeIterator();
		}
	}

	virtual TSelectedItem GetItem() const override
	{
		checkf(SourceIterator.IsValid(), TEXT("Source iterator is not set."));
		checkf(SelectedRangeIterator.IsSet(), TEXT("Iterator is not valid."));
		return SelectedRangeIterator.GetValue()->GetItem();
	}

private:
	void MoveToNextValidRangeIterator() const
	{
		SourceIterator->NextItem();
		if (SourceIterator->IsValidIterator())
		{
			SelectedRangeIterator.Emplace(RangeSelecter.Execute(SourceIterator->GetItem()));
			while (SelectedRangeIterator.GetValue()->IsValidIterator() == false && SourceIterator->IsValidIterator())
			{
				SourceIterator->NextItem();
				if (SourceIterator->IsValidIterator())
				{
					SelectedRangeIterator.Emplace(RangeSelecter.Execute(SourceIterator->GetItem()));
				}
			}
		}
	}

private:
	TSharedPtr<IRangeQueryIterator<TItem>> SourceIterator;
	mutable bool bMovedToInitialValidRangeIterator = false;
	mutable TOptional<TSharedRef<IRangeQueryIterator<TSelectedItem>>> SelectedRangeIterator;
	FRangeSelecter RangeSelecter;
};

template<typename TItem>
TSharedRef<IRangeQueryWrapperIterator<TItem, TItem>> Where(typename TRangeQueryPredicateIterator<TItem>::FPredicate Predicate)
{
	return MakeShared<TRangeQueryPredicateIterator<TItem>>(Predicate);
};

template<typename TItem, typename TPredicate>
TSharedRef<IRangeQueryWrapperIterator<TItem, TItem>> Where(TPredicate Predicate)
{
	return MakeShared<TRangeQueryPredicateIterator<TItem>>(
		TRangeQueryPredicateIterator<TItem>::FPredicate::CreateLambda([Predicate](const TItem& Item) { return Predicate(Item); }));
};

template<typename TItem>
TSharedRef<IRangeQueryWrapperIterator<TItem, TItem>> WhereValid()
{
	return MakeShared<TRangeQueryPredicateIterator<TItem>>(
		TRangeQueryPredicateIterator<TItem>::FPredicate::CreateLambda([](const TItem& Item) { return Item.IsValid(); }));
}

template<typename TItem, typename TSelectedItem>
TSharedRef<IRangeQueryWrapperIterator<TItem, TSelectedItem>> Select(typename TRangeQuerySelectIterator<TItem, TSelectedItem>::FSelecter Selecter)
{
	return MakeShared<TRangeQuerySelectIterator<TItem, TSelectedItem>>(Selecter);
};

template<typename TItem, typename TSelectedItem, typename TSelecter>
TSharedRef<IRangeQueryWrapperIterator<TItem, TSelectedItem>> Select(TSelecter Selecter)
{
	return MakeShared<TRangeQuerySelectIterator<TItem, TSelectedItem>>(
		TRangeQuerySelectIterator<TItem, TSelectedItem>::FSelecter::CreateLambda([Selecter](const TItem& Item) { return Selecter(Item); }));
};

template<typename TItem, typename TSelectedItem>
TSharedRef<IRangeQueryWrapperIterator<TItem, TSelectedItem>> SelectRange(typename TRangeQuerySelectRangeIterator<TItem, TSelectedItem>::FRangeSelecter RangeSelecter)
{
	return MakeShared<TRangeQuerySelectRangeIterator<TItem, TSelectedItem>>(RangeSelecter);
};

template<typename TItem, typename TSelectedItem, typename TRangeSelecter>
TSharedRef<IRangeQueryWrapperIterator<TItem, TSelectedItem>> SelectRange(TRangeSelecter RangeSelecter)
{
	return MakeShared<TRangeQuerySelectRangeIterator<TItem, TSelectedItem>>(
		TRangeQuerySelectRangeIterator<TItem, TSelectedItem>::FRangeSelecter::CreateLambda([RangeSelecter](const TItem& Item) { return RangeSelecter(Item); }));
};

template<typename TItem, typename TIterable>
TSharedRef<IRangeQueryIterator<TItem>> MakeIterator(const TIterable* Iterable)
{
	return TRangeQueryIndexableIterator<TItem, TIterable>::CreateAsRef(Iterable);
}

template<typename TItem, typename TIterable>
TSharedRef<IRangeQueryIterator<TItem>> MakeIterator(const TIterable& Iterable)
{
	return TRangeQueryIndexableIterator<TItem, TIterable>::CreateAsCopy(Iterable);
}

template<typename TItem, typename TIterable>
TRangeQuery<TItem> MakeQuery(const TIterable* Iterable)
{
	return TRangeQuery<TItem>(MakeIterator<TItem, TIterable>(Iterable));
}

template<typename TItem, typename TIterable>
TRangeQuery<TItem> MakeQuery(const TIterable& Iterable)
{
	return TRangeQuery<TItem>(MakeIterator<TItem, TIterable>(Iterable));
}

} // namespace UE::RangeQuery
