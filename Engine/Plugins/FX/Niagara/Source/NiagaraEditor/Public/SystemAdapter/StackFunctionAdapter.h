// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SystemAdapter/AdapterShared.h"
#include "NiagaraTypes.h"

class UNiagaraNodeFunctionCall;
class UNiagaraScript;
class UNiagaraSystem;

namespace UE::Niagara
{
	class FStackFunctionInputAdapter;

	class FStackFunctionAdapter : public IAdapter
	{
	public:
		enum class EFunctionType
		{
			ScriptModule,
			SetParametersModule,
		};

		NIAGARAEDITOR_API static TSharedRef<FStackFunctionAdapter> Create(UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, UNiagaraScript* InOwningScript, UNiagaraNodeFunctionCall* InFunctionCallNode);

		static TSharedRef<FStackFunctionAdapter> Create(const UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, const UNiagaraScript* InOwningScript, const UNiagaraNodeFunctionCall* InFunctionCallNode);

		NIAGARAEDITOR_API static TSharedRef<const FStackFunctionAdapter> CreateConst(const UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, const UNiagaraScript* InOwningScript, const UNiagaraNodeFunctionCall* InFunctionCallNode);

		NIAGARAEDITOR_API virtual bool IsValidAdapter() const override;

		NIAGARAEDITOR_API virtual bool IsValidWriteAdapter() const override;

		EFunctionType GetType() const { return Type; }

		NIAGARAEDITOR_API FString GetFunctionName() const;

		NIAGARAEDITOR_API UNiagaraScript* GetModuleScript() const;

		NIAGARAEDITOR_API void GetSetParameters(TArray<FNiagaraVariableBase>& OutParameters) const;

		NIAGARAEDITOR_API bool SetsParameter(const FNiagaraVariableBase& InParameter) const;

		NIAGARAEDITOR_API TSharedRefCollection<FStackFunctionInputAdapter> GetInputs();

		NIAGARAEDITOR_API TConstSharedRefCollection<FStackFunctionInputAdapter> GetInputs() const;

		NIAGARAEDITOR_API TSharedPtr<FStackFunctionInputAdapter> FindInputByName(FName InputName);

		NIAGARAEDITOR_API TSharedPtr<const FStackFunctionInputAdapter> FindInputByName(FName InputName) const;

	private:
		void Initialize(UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, UNiagaraScript* InOwningScript, UNiagaraNodeFunctionCall* InFunctionCallNode);

		void Initialize(const UNiagaraSystem* InOwningSystem, FGuid InOwningEmitterHandleId, const UNiagaraScript* InOwningScript, const UNiagaraNodeFunctionCall* InFunctionCallNode);

	private:
		TOptional<TWeakObjectPtr<UNiagaraNodeFunctionCall>> FunctionCallNodeWeak;
		TWeakObjectPtr<const UNiagaraNodeFunctionCall> FunctionCallNodeConstWeak;
		EFunctionType Type = EFunctionType::ScriptModule;

		TAdapterRefListThreeSources<UNiagaraSystem, UNiagaraScript, UNiagaraNodeFunctionCall, FStackFunctionInputAdapter> Inputs;
	};

	typedef TSharedRef<FStackFunctionAdapter> FStackFunctionAdapterRef;
	typedef TSharedRef<const FStackFunctionAdapter> FStackFunctionAdapterConstRef;
	typedef TSharedPtr<FStackFunctionAdapter> FStackFunctionAdapterPtr;
	typedef TSharedPtr<const FStackFunctionAdapter> FStackFunctionAdapterConstPtr;

} // namespace UE::Niagara