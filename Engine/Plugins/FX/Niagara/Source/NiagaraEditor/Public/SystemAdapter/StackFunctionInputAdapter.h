// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "SystemAdapter/AdapterShared.h"

class UNiagaraDataInterface;
class UNiagaraNodeFunctionCall;

namespace UE::Niagara
{
	class FStackFunctionAdapter;

	class FStackFunctionInputAdapter : public IAdapter
	{
	public:
		enum class EValueMode
		{
			LocalValue,
			LinkedValue,
			DataValue,
			ObjectValue,
			DynamicValue,
			Unknown
		};

		NIAGARAEDITOR_API static TSharedRef<FStackFunctionInputAdapter> Create(UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter);

		static TSharedRef<FStackFunctionInputAdapter> Create(const UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter);

		NIAGARAEDITOR_API static TSharedRef<const FStackFunctionInputAdapter> CreateConst(const UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter);

		NIAGARAEDITOR_API virtual bool IsValidAdapter() const override;

		NIAGARAEDITOR_API virtual bool IsValidWriteAdapter() const override;

		NIAGARAEDITOR_API FName GetName() const;

		NIAGARAEDITOR_API const FNiagaraTypeDefinition& GetType() const;

		NIAGARAEDITOR_API EValueMode GetValueMode() const { return ValueMode; }

		NIAGARAEDITOR_API bool TryGetLocalValue(bool& OutValue) const;

		NIAGARAEDITOR_API bool TryGetLocalValue(int32& OutValue) const;

		NIAGARAEDITOR_API bool TryGetLocalValue(float& OutValue) const;

		NIAGARAEDITOR_API bool TryGetLocalValue(FVector2f& OutValue) const;

		NIAGARAEDITOR_API bool TryGetLocalValue(FVector3f& OutValue) const;

		NIAGARAEDITOR_API bool TryGetLocalValue(FVector4f& OutValue) const;

		NIAGARAEDITOR_API bool TryGetLocalValueData(TNotNull<UStruct*> ValueType, const uint8*& OutData) const;

		template<typename TValue>
		bool TryGetLocalValue(TValue& OutValue) const
		{
			const uint8* ValueData;
			if (TryGetLocalValueData(TValue::StaticStruct(), ValueData))
			{
				TValue::StaticStruct()->CopyScriptStruct(&OutValue, ValueData);
				return true;
			}
			return false;
		}

		NIAGARAEDITOR_API FNiagaraVariableBase GetLinkedValue() const;

		NIAGARAEDITOR_API UNiagaraDataInterface* GetDataValue() const;

		NIAGARAEDITOR_API UObject* GetObjectValue() const;

		NIAGARAEDITOR_API TSharedPtr<FStackFunctionAdapter> GetDynamicValue() const;

	private:
		void Initialize(UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter);

		void Initialize(const UNiagaraNodeFunctionCall* InFunctionCallNode, const FNiagaraVariableBase& InInputParameter);

	private:
		TOptional<TWeakObjectPtr<UNiagaraNodeFunctionCall>> FunctionCallNodeWeak;
		TWeakObjectPtr<const UNiagaraNodeFunctionCall> FunctionCallNodeConstWeak;
		FNiagaraVariableBase InputParameter;
		FName InputName;
		EValueMode ValueMode = EValueMode::Unknown;
	};

	typedef TSharedRef<FStackFunctionInputAdapter> FStackFunctionInputAdapterRef;
	typedef TSharedRef<const FStackFunctionInputAdapter> FStackFunctionInputAdapterConstRef;
	typedef TSharedPtr<FStackFunctionInputAdapter> FStackFunctionInputAdapterPtr;
	typedef TSharedPtr<const FStackFunctionInputAdapter> FStackFunctionInputAdapterConstPtr;

} // namespace UE::Niagara