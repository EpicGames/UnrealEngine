// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdapterShared.h"
#include "NiagaraCommon.h"
#include "NiagaraTypes.h"

class UNiagaraNodeFunctionCall;
class UNiagaraScript;

namespace UE::Niagara
{
	class FStackFunctionAdapter;

	class FStackScriptAdapter : public IAdapter
	{
	public:
		NIAGARAEDITOR_API static TSharedRef<FStackScriptAdapter> Create(UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, UNiagaraScript* InScript);

		static TSharedRef<FStackScriptAdapter> Create(const UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, const UNiagaraScript* InScript);

		NIAGARAEDITOR_API static TSharedRef<const FStackScriptAdapter> CreateConst(const UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, const UNiagaraScript* InScript);

		NIAGARAEDITOR_API virtual bool IsValidAdapter() const override;

		NIAGARAEDITOR_API virtual bool IsValidWriteAdapter() const override;

		NIAGARAEDITOR_API ENiagaraScriptUsage GetUsage() const;

		NIAGARAEDITOR_API FGuid GetUsageId() const;

		NIAGARAEDITOR_API TSharedRefCollection<FStackFunctionAdapter> GetModules();

		NIAGARAEDITOR_API TConstSharedRefCollection<FStackFunctionAdapter> GetModules() const;

		NIAGARAEDITOR_API TSharedPtr<FStackFunctionAdapter> FindModuleByName(const FName& InName);

		NIAGARAEDITOR_API TSharedPtr<const FStackFunctionAdapter> FindModuleByName(const FName& InName) const;

		NIAGARAEDITOR_API void FindSetParametersModulesBySetParameter(const FNiagaraVariableBase& InParameter, TArray<TSharedRef<FStackFunctionAdapter>>& OutSetParametersModules);

		NIAGARAEDITOR_API void FindSetParametersModulesBySetParameter(const FNiagaraVariableBase& InParameter, TArray<TSharedRef<const FStackFunctionAdapter>>& OutSetParametersModules) const;

		NIAGARAEDITOR_API TSharedPtr<FStackFunctionAdapter> FindFirstSetParametersModuleBySetParameter(const FNiagaraVariableBase& InParameter);

		NIAGARAEDITOR_API TSharedPtr<const FStackFunctionAdapter> FindFirstSetParametersModuleBySetParameter(const FNiagaraVariableBase& InParameter) const;

	private:
		void Initialize(UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, UNiagaraScript* InScript);

		void Initialize(const UNiagaraSystem* InOwningSystem, const FGuid& InOwningEmitterHandleId, const UNiagaraScript* InScript);

	private:
		TOptional<TWeakObjectPtr<UNiagaraScript>> ScriptWeak;
		TWeakObjectPtr<const UNiagaraScript> ScriptConstWeak;

		TAdapterRefListTwoSources<UNiagaraSystem, UNiagaraScript, FStackFunctionAdapter> Modules;
	};

	typedef TSharedRef<FStackScriptAdapter> FStackScriptAdapterRef;
	typedef TSharedRef<const FStackScriptAdapter> FStackScriptAdapterConstRef;
	typedef TSharedPtr<FStackScriptAdapter> FStackScriptAdapterPtr;
	typedef TSharedPtr<const FStackScriptAdapter> FStackScriptAdapterConstPtr;

} // namespace UE::Niagara