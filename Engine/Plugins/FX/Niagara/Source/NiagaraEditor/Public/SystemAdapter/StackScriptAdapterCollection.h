// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SystemAdapter/AdapterShared.h"

namespace UE::Niagara
{
	class FStackScriptAdapter;

	class IStackScriptAdapterOwner
	{
	public:
		virtual ~IStackScriptAdapterOwner() = default;

		virtual int32 GetNumScripts() const = 0;
		virtual TSharedRef<FStackScriptAdapter> GetScriptAt(int32 Index) = 0;
	};

	class FStackScriptAdapterCollection
	{
	public:
		void Initialize(TSharedRef<IStackScriptAdapterOwner> InScriptOwner);

		NIAGARAEDITOR_API int32 Num() const;

		NIAGARAEDITOR_API TSharedRef<FStackScriptAdapter> operator[](int32 Index) const;

	private:
		TWeakPtr<IStackScriptAdapterOwner> ScriptOwnerWeak;
	};

	class FScriptSharedRefCollection : public TSharedRefCollectionBase<FStackScriptAdapter, FStackScriptAdapter, FStackScriptAdapterCollection>
	{
	};
	
	class FScriptConstSharedRefCollection : public TSharedRefCollectionBase<FStackScriptAdapter, const FStackScriptAdapter, FStackScriptAdapterCollection>
	{
	};

} // namespace UE::Niagara