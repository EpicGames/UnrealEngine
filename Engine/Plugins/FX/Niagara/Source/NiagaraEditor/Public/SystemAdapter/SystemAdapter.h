// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraParameterStore.h"
#include "SystemAdapter/AdapterShared.h"
#include "SystemAdapter/StackScriptAdapterCollection.h"
#include "Templates/SharedPointer.h"

class  UNiagaraSystem;

namespace UE::Niagara
{
	class FEmitterAdapter;
	class FParameterAdapter;
	class FStackScriptAdapter;
	class IStackScriptAdapterCollection;

	class FUserParameterAdapter : public IAdapter
	{
		friend class FUserParameterAdapterList;

	public:
		FUserParameterAdapter()
		{
		}

		NIAGARAEDITOR_API static TSharedRef<FUserParameterAdapter> Create(UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset);

		static TSharedRef<FUserParameterAdapter> Create(const UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset);

		NIAGARAEDITOR_API static TSharedRef<const FUserParameterAdapter> CreateConst(const UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset);

		NIAGARAEDITOR_API virtual bool IsValidAdapter() const override;

		NIAGARAEDITOR_API virtual bool IsValidWriteAdapter() const override;

		NIAGARAEDITOR_API FName GetName() const;

		NIAGARAEDITOR_API FName GetNamespacedName() const { return UserParameterWithOffset.GetName(); }

		NIAGARAEDITOR_API bool TryGetValue(bool &OutValue) const;

		NIAGARAEDITOR_API bool TryGetValue(int32& OutValue) const;

		NIAGARAEDITOR_API bool TryGetValue(float& OutValue) const;

		NIAGARAEDITOR_API bool TryGetValue(FVector2f& OutValue) const;

		NIAGARAEDITOR_API bool TryGetValue(FVector3f& OutValue) const;

		NIAGARAEDITOR_API bool TryGetValue(FVector4f& OutValue) const;

		NIAGARAEDITOR_API bool TryGetValue(FVector& OutValue) const;

		NIAGARAEDITOR_API bool TryGetValue(FLinearColor& OutValue) const;

		NIAGARAEDITOR_API bool TryGetValue(FQuat4f& OutValue) const;

		NIAGARAEDITOR_API bool TryGetValue(FNiagaraPosition& OutValue) const;

		NIAGARAEDITOR_API bool TryGetValueData(TNotNull<UStruct*> ValueType, const uint8*& OutData) const;
		
		template<typename TValue>
		bool TryGetValue(TValue& OutValue) const
		{
			const uint8* ValueData;
			if (TryGetValueData(TValue::StaticStruct(), ValueData))
			{
				TValue::StaticStruct()->CopyScriptStruct(&OutValue, ValueData);
				return true;
			}
			return false;
		}

	private:
		void Initialize(UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset);

		void Initialize(const UNiagaraSystem* InSystem, const FNiagaraVariableWithOffset& InUserParameterWithOffset);

	private:
		TOptional<TWeakObjectPtr<UNiagaraSystem>> SystemWeak;
		TWeakObjectPtr<const UNiagaraSystem> SystemConstWeak;
		FNiagaraVariableWithOffset UserParameterWithOffset;
		mutable FName NameCache = NAME_None;
	};

	class FSystemAdapter : public TSharedFromThis<FSystemAdapter>, public IAdapter, public IStackScriptAdapterOwner
	{
	public:
		FSystemAdapter()
		{
		}

		NIAGARAEDITOR_API static TSharedRef<FSystemAdapter> Create(UNiagaraSystem* InSystem);

		NIAGARAEDITOR_API static TSharedRef<const FSystemAdapter> CreateConst(const UNiagaraSystem* InSystem);

		NIAGARAEDITOR_API virtual bool IsValidAdapter() const override;

		NIAGARAEDITOR_API virtual bool IsValidWriteAdapter() const override;

		NIAGARAEDITOR_API TSharedRefCollection<FUserParameterAdapter> GetUserParameters();
	
		NIAGARAEDITOR_API TConstSharedRefCollection<FUserParameterAdapter> GetUserParameters() const;

		NIAGARAEDITOR_API TSharedPtr<FUserParameterAdapter> GetUserParameterByName(const FName& InName);

		NIAGARAEDITOR_API TSharedPtr<const FUserParameterAdapter> GetUserParameterByName(const FName& InName) const;

		NIAGARAEDITOR_API TSharedPtr<FUserParameterAdapter> GetUserParameterByNamespacedName(const FName& InNamespacedName);

		NIAGARAEDITOR_API TSharedPtr<const FUserParameterAdapter> GetUserParameterByNamespacedName(const FName& InNamespacedName) const;

		NIAGARAEDITOR_API TSharedRefCollection<FEmitterAdapter> GetEmitters();

		NIAGARAEDITOR_API TConstSharedRefCollection<FEmitterAdapter> GetEmitters() const;

		NIAGARAEDITOR_API TSharedPtr<FEmitterAdapter> GetEmitterByName(FName InEmitterName);

		NIAGARAEDITOR_API TSharedPtr<const FEmitterAdapter> GetEmitterByName(FName InEmitterName) const;

		NIAGARAEDITOR_API TSharedRef<FStackScriptAdapter> GetSpawnScript();

		NIAGARAEDITOR_API TSharedRef<const FStackScriptAdapter> GetSpawnScript() const;

		NIAGARAEDITOR_API TSharedRef<FStackScriptAdapter> GetUpdateScript();

		NIAGARAEDITOR_API TSharedRef<const FStackScriptAdapter> GetUpdateScript() const;

		virtual int32 GetNumScripts() const override;

		virtual TSharedRef<FStackScriptAdapter> GetScriptAt(int32 Index) override;

		NIAGARAEDITOR_API FScriptSharedRefCollection GetScripts();

		NIAGARAEDITOR_API FScriptConstSharedRefCollection GetScripts() const;

	private:
		void Initialize(UNiagaraSystem* InSystem);

		void Initialize(const UNiagaraSystem* InSystem);

	private:
		TOptional<TWeakObjectPtr<UNiagaraSystem>> SystemWeak;
		TWeakObjectPtr<const UNiagaraSystem> SystemConstWeak;

		TAdapterRefListOneSource<UNiagaraSystem, FUserParameterAdapter> UserParameters;
		TAdapterRefListOneSource<UNiagaraSystem, FEmitterAdapter> Emitters;
		TAdapterPtr<UNiagaraSystem, FStackScriptAdapter> SpawnScript;
		TAdapterPtr<UNiagaraSystem, FStackScriptAdapter> UpdateScript;
		FStackScriptAdapterCollection StackScriptAdapterCollection;
	};

} // namespace UE::Niagara