// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "Widgets/Docking/SDockTab.h"
#include "GraphEditAction.h"
class FDocumentTracker;

class FNiagaraSystemToolkit;
class FNiagaraObjectSelection;

class FNiagaraSystemToolkitModeBase : public FApplicationMode
{
public:
	FNiagaraSystemToolkitModeBase(FName InModeName, TWeakPtr<FNiagaraSystemToolkit> InSystemToolkit);
	virtual ~FNiagaraSystemToolkitModeBase() override;
	
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

	int GetActiveSelectionDetailsIndex() const;

	virtual void OnActiveDocumentChanged(TSharedPtr<class SDockTab> NewActiveTab);
	virtual void OnSystemSelectionChanged();

	virtual void PostActivateMode() override;
private:
	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_CurveEd(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Sequencer(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SystemScript(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SystemParameters(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SystemParameterDefinitions(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SelectedEmitterGraph(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_DebugCacheSpreadsheet(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_PreviewSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_GeneratedCode(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_MessageLog(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SystemOverview(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_ScratchPadScripts(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_ScriptStats(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Baker(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Versioning(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_UserParameters(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_UserParametersHierarchyEditor(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_SummaryViewEditor(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_ScratchPadHierarchyEditor(const FSpawnTabArgs& Args);


protected:
	TWeakPtr<FNiagaraSystemToolkit> SystemToolkit;
	FDelegateHandle DocChangedHandle;
	FDelegateHandle LastSelectionUpdateDelegate;
	FDelegateHandle LastParamPanelSelectionUpdateDelegate;
	FDelegateHandle LastSystemSelectionUpdateDelegate;
	FDelegateHandle LastGraphEditDelegate;
	FDelegateHandle UpdateSummaryViewHandle;
	FDelegateHandle UpdateScratchPadScriptHierarchyHandle;

	int32 SwitcherIdx;
	TSharedPtr<FNiagaraObjectSelection> ObjectSelection;
	FText ObjectSelectionSubHeaderText;

	EVisibility GetObjectSelectionSubHeaderTextVisibility() const;
	FText GetObjectSelectionSubHeaderText() const;
	EVisibility GetObjectSelectionNoSelectionTextVisibility() const;
	void UpdateSelectionForActiveDocument();
	void OnParameterPanelViewModelExternalSelectionChanged();
	void OnEditedScriptGraphChanged(const FEdGraphEditAction& InAction);
	
	TSharedRef<SWidget> CreateSummaryViewWidget() const;
	void UpdateSummaryViewOnSelectionChanged() const;
	void OnSummaryViewEditorClosed(TSharedRef<SDockTab> DockTab) const;

	FReply SummonScratchPadScriptHierarchyEditor();
	TSharedRef<SWidget> CreateScratchPadHierarchyWidget();
	void UpdateScratchPadActiveScriptChanged(TSharedPtr<SDockTab> DockTab);
	void OnScratchPadHierarchyEditorClosed(TSharedRef<SDockTab> DockTab);
	EVisibility GetSummonScratchPadHierarchyEditorButtonVisibility() const;
	
	TSharedPtr<class SBox> SummaryViewContainer;
	TSharedPtr<class SBox> ScratchPadHierarchyContainer;
	TWeakPtr<class FNiagaraScratchPadScriptViewModel> LastActiveScratchPadViewModel;

	TArray<TSharedRef<FWorkflowTabFactory>> ExternalTabSpawners;

public:
	NIAGARAEDITOR_API static const FName DefaultModeName;
	NIAGARAEDITOR_API static const FName ScalabilityModeName;
	
	NIAGARAEDITOR_API static const FName ViewportTabID;
	static const FName CurveEditorTabID;
	static const FName SequencerTabID;
	static const FName SystemScriptTabID;
	static const FName SystemParametersTabID;
	static const FName SystemParameterDefinitionsTabID;
	NIAGARAEDITOR_API static const FName DetailsTabID;
	static const FName SelectedEmitterGraphTabID;
	static const FName DebugCacheSpreadsheetTabID;
	static const FName PreviewSettingsTabId;
	static const FName GeneratedCodeTabID;
	static const FName MessageLogTabID;
	static const FName SystemOverviewTabID;
	static const FName ScratchPadScriptsTabID;
	static const FName ScriptStatsTabID;
	static const FName BakerTabID;
	static const FName VersioningTabID;
	static const FName UserParametersTabID;
	static const FName UserParametersHierarchyTabID;
	static const FName EmitterSummaryViewEditorTabID;
	static const FName ScratchPadHierarchyEditorTabID;
};

