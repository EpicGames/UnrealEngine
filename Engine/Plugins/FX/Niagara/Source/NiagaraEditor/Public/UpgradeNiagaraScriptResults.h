// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "NiagaraParameterMapHistory.h"
#include "UObject/Object.h"
#include "NiagaraTypes.h"
#include "ViewModels/NiagaraEmitterHandleViewModel.h"
#include "UpgradeNiagaraScriptResults.generated.h"

class UNiagaraStackModuleItem;
class UNiagaraClipboardFunctionInput;

UENUM()
enum class ENiagaraPythonScriptInputSource : uint32
{
	Input,

    Output,

    Local,

    InputOutput,

    InitialValueInput,

    // insert new script parameter usages before
    None UMETA(Hidden),
	
    Num UMETA(Hidden)
};

// ---------------------- Script upgrade data -------------------------------------

/** Wrapper for setting the value on a parameter of a UNiagaraScript, applied through a UUpgradeNiagaraScriptResults. */
UCLASS(BlueprintInternalUseOnly)
class UNiagaraPythonScriptModuleInput : public UObject
{
	GENERATED_BODY()

public:
	UNiagaraPythonScriptModuleInput() {};

	UPROPERTY()
	TObjectPtr<const UNiagaraClipboardFunctionInput> Input = nullptr;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    bool IsSet() const;
	
	UFUNCTION(BlueprintCallable, Category = "Scripting")
    bool IsLocalValue() const;
    
    UFUNCTION(BlueprintCallable, Category = "Scripting")
    bool IsLinkedValue() const;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    float AsFloat() const;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    int32 AsInt() const;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    bool AsBool() const;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    FVector2D AsVec2() const;
    
	UFUNCTION(BlueprintCallable, Category = "Scripting")
    FVector AsVec3() const;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    FVector4 AsVec4() const;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    FLinearColor AsColor() const;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    FQuat AsQuat() const;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    FString AsEnum() const;
    
    UFUNCTION(BlueprintCallable, Category = "Scripting")
    FString AsLinkedValue() const;
};

/**
 * Wrapper class for passing results back from the version upgrade python script.
 */
UCLASS(BlueprintType, MinimalAPI)
class UUpgradeNiagaraScriptResults : public UObject
{
	GENERATED_BODY()

public:

	NIAGARAEDITOR_API UUpgradeNiagaraScriptResults();

	NIAGARAEDITOR_API void Init();
	
	// Whether the converter process was cancelled due to an unrecoverable error in the python script process.
	UPROPERTY(BlueprintReadWrite, Category = "Scripting")
	bool bCancelledByPythonError = false;

	UPROPERTY(BlueprintReadWrite, Category = "Scripting")
	TArray<TObjectPtr<UNiagaraPythonScriptModuleInput>> OldInputs;

	UPROPERTY(BlueprintReadWrite, Category = "Scripting")
	TArray<TObjectPtr<UNiagaraPythonScriptModuleInput>> NewInputs;

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API UNiagaraPythonScriptModuleInput* GetOldInput(const FString& InputName);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetFloatInput(const FString& InputName, float Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetIntInput(const FString& InputName, int32 Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetBoolInput(const FString& InputName, bool Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetVec2Input(const FString& InputName, FVector2D Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetVec3Input(const FString& InputName, FVector Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetVec4Input(const FString& InputName, FVector4 Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetColorInput(const FString& InputName, FLinearColor Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetQuatInput(const FString& InputName, FQuat Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetEnumInput(const FString& InputName, FString Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
	NIAGARAEDITOR_API void SetEnumInputFromInt(const FString& InputName, int32 Value);
    
    UFUNCTION(BlueprintCallable, Category = "Scripting")
    NIAGARAEDITOR_API void SetLinkedInput(const FString& InputName, FString Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
	NIAGARAEDITOR_API void SetNewInput(const FString& InputName, UNiagaraPythonScriptModuleInput* Value);

	UFUNCTION(BlueprintCallable, Category = "Scripting")
	NIAGARAEDITOR_API void ResetToDefault(const FString& InputName);

	TArray<UNiagaraClipboardFunctionInput*> CopyOldInputs() const;
	TArray<UNiagaraClipboardFunctionInput*> CopyNewInputs() const;
	
private:
	UNiagaraPythonScriptModuleInput* GetNewInput(const FName& InputName) const;
	TArray<UNiagaraClipboardFunctionInput*> CopyInputs(const TArray<TObjectPtr<UNiagaraPythonScriptModuleInput>>& Source) const;

	// This is used as a placeholder object for python to interact with when a module input could not be found. Returning a nullptr instead would crash the script.
	UPROPERTY(Transient)
	TObjectPtr<UNiagaraPythonScriptModuleInput> DummyInput;
};

struct FNiagaraScriptVersionUpgradeContext {
	TFunction<void(UNiagaraClipboardContent*)> CreateClipboardCallback;
	TFunction<void(UNiagaraClipboardContent*, FText&)> ApplyClipboardCallback;
	FCompileConstantResolver ConstantResolver;
	bool bSkipPythonScript = false;
};



// ---------------------- Emitter upgrade data -------------------------------------

/** Wrapper for a module from the emitter stack. */
UCLASS(BlueprintInternalUseOnly)
class UNiagaraPythonModule : public UObject
{
	GENERATED_BODY()

public:
	UNiagaraPythonModule() {};

	void Init(UNiagaraStackModuleItem* InModuleItem);

	// Returns the raw underlying object
	UFUNCTION(BlueprintCallable, Category = "Scripting")
	UNiagaraStackModuleItem* GetObject() const;

	//TODO: extend the api here to fully support module actions

private:
	UPROPERTY()
	TObjectPtr<UNiagaraStackModuleItem> ModuleItem = nullptr;
};

/** Wrapper for an emitter stack. */
UCLASS(BlueprintInternalUseOnly)
class UNiagaraPythonEmitter : public UObject
{
	GENERATED_BODY()

public:
	UNiagaraPythonEmitter() {}

	void Init(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterViewModel);

	// Returns the raw underlying object
	UFUNCTION(BlueprintCallable, Category = "Scripting")
	UNiagaraEmitter* GetObject();

	// returns the emitter properties, such as determinism or interpolated spawning
	UFUNCTION(BlueprintCallable, Category = "Scripting")
	FVersionedNiagaraEmitterData GetProperties() const;

	// sets the new emitter properties
	UFUNCTION(BlueprintCallable, Category = "Scripting")
	void SetProperties(FVersionedNiagaraEmitterData Data);

	// returns a list of all modules contained in this emitter
	UFUNCTION(BlueprintCallable, Category = "Scripting")
	TArray<UNiagaraPythonModule*> GetModules() const;

	// returns true if the emitter contains a certain module
	UFUNCTION(BlueprintCallable, Category = "Scripting")
	bool HasModule(const FString& ModuleName) const;

	// returns a module by name
	UFUNCTION(BlueprintCallable, Category = "Scripting")
	UNiagaraPythonModule* GetModule(const FString& ModuleName) const;

	//TODO: extend the api here to support renderers

	bool IsValid() const { return EmitterViewModel.IsValid(); }

	TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterViewModel;
};

/**
 * Wrapper class for passing results back from the version upgrade python script.
 */
UCLASS(BlueprintType, MinimalAPI)
class UUpgradeNiagaraEmitterContext : public UObject
{
	GENERATED_BODY()

public:

	UUpgradeNiagaraEmitterContext() {}

	NIAGARAEDITOR_API void Init(UNiagaraPythonEmitter* InOldEmitter, UNiagaraPythonEmitter* InNewEmitter);
	NIAGARAEDITOR_API bool IsValid() const;
	NIAGARAEDITOR_API const TArray<FVersionedNiagaraEmitterData*>& GetUpgradeData() const;
	
	// Whether the converter process was cancelled due to an unrecoverable error in the python script process.
	UPROPERTY(BlueprintReadWrite, Category = "Scripting")
	bool bCancelledByPythonError = false;

	UPROPERTY(BlueprintReadWrite, Category = "Scripting")
	TObjectPtr<UNiagaraPythonEmitter> OldEmitter = nullptr;

	UPROPERTY(BlueprintReadWrite, Category = "Scripting")
	TObjectPtr<UNiagaraPythonEmitter> NewEmitter = nullptr;

private:
	TArray<FVersionedNiagaraEmitterData*> UpgradeVersionData;
};
