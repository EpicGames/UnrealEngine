// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraGraph.h"
#include "DataHierarchyViewModelBase.h"
#include "NiagaraScriptVariable.h"
#include "ViewModels/NiagaraScriptViewModel.h"
#include "NiagaraScriptParametersHierarchyViewModel.generated.h"

#define UE_API NIAGARAEDITOR_API

class UNiagaraGraph;

/** The refresh context is used to determine if hierarchy script variables should be removed. */
UCLASS()
class UNiagaraHierarchyScriptParameterRefreshContext : public UHierarchyDataRefreshContext
{
	GENERATED_BODY()

public:
	void SetNiagaraGraph(UNiagaraGraph* InGraph) { NiagaraGraph = InGraph; }
	const UNiagaraGraph* GetNiagaraGraph() const { return NiagaraGraph; }
private:
	UPROPERTY(Transient)
	TObjectPtr<UNiagaraGraph> NiagaraGraph;
};

/** A hierarchy script parameter is an optional object embedded in the hierarchy. */
UCLASS()
class UNiagaraHierarchyScriptParameter : public UHierarchyItem
{
	GENERATED_BODY()

public:
	void Initialize(const UNiagaraScriptVariable& InParameterScriptVariable);

	bool IsValid() const;
	
	virtual FString ToString() const override;
	FText GetTooltip() const;
	
	UNiagaraScriptVariable* GetScriptVariable() const;
	TOptional<FNiagaraVariable> GetVariable() const;
};

/** The category class used for the script hierarchy editor. It lets us add additional data later on. */
UCLASS()
class UNiagaraHierarchyScriptCategory : public UHierarchyCategory
{
	GENERATED_BODY()
};

/**
 * The view model that defines the script editor's hierarchy editor for input parameters.
 */
UCLASS(MinimalAPI)
class UNiagaraScriptParametersHierarchyViewModel : public UDataHierarchyViewModelBase
{
	GENERATED_BODY()
public:
	UE_API void Initialize(TSharedRef<FNiagaraScriptViewModel> InScriptViewModel);
	UE_API TSharedRef<FNiagaraScriptViewModel> GetScriptViewModel() const;

	/** UDataHierarchyViewModelBase */
	UE_API virtual UHierarchyRoot* GetHierarchyRoot() const override;
	/** We set the outer for the source root to the graph, so that both source & hierarchy items have the same outer. */
	UE_API virtual UObject* GetOuterForSourceRoot() const override;
	UE_API virtual TSubclassOf<UHierarchyCategory> GetCategoryDataClass() const override;
	UE_API virtual TSharedPtr<FHierarchyElementViewModel> CreateCustomViewModelForElement(UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> Parent) override;
	UE_API virtual void PrepareSourceItems(UHierarchyRoot* SourceRoot, TSharedPtr<FHierarchyRootViewModel> SourceRootViewModel) override;
	UE_API virtual void SetupCommands() override;
	UE_API virtual TSharedRef<FHierarchyDragDropOp> CreateDragDropOp(const FDataHierarchyDragDropContext& DragContext) override;
	virtual bool SupportsDetailsPanel() override { return true; }
	virtual bool SupportsMultiEditInDetailsPanel() override { return false; }
	UE_API virtual void FinalizeInternal() override;
private:
	UE_API void OnParametersChanged(TOptional<TInstancedStruct<FNiagaraParametersChangedData>> ParametersChangedData);

protected:
	UE_API virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
private:
	TWeakPtr<FNiagaraScriptViewModel> ScriptViewModelWeak;
};

class FNiagaraHierarchyScriptParameterDragDropOp : public FHierarchyDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FNiagaraHierarchyScriptParameterDragDropOp, FHierarchyDragDropOp)

	explicit FNiagaraHierarchyScriptParameterDragDropOp(const FDataHierarchyDragDropContext& InDragContext) : FHierarchyDragDropOp(InDragContext) {}

	virtual TSharedRef<SWidget> CreateCustomDecorator() const override;
};

struct FNiagaraHierarchyScriptCategoryViewModel : public FHierarchyCategoryViewModel
{
	FNiagaraHierarchyScriptCategoryViewModel(UHierarchyCategory* Category, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraScriptParametersHierarchyViewModel> ViewModel)
		: FHierarchyCategoryViewModel(Category, InParent, ViewModel) {}

protected:
	// We don't sort anything to allow free authoring of UI
	virtual void SortChildrenData() override {}
	virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone) override;
};

struct FNiagaraHierarchyScriptParameterViewModel : public FHierarchyItemViewModel
{
	FNiagaraHierarchyScriptParameterViewModel(UNiagaraHierarchyScriptParameter* ScriptParameter, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraScriptParametersHierarchyViewModel> ViewModel)
		: FHierarchyItemViewModel(ScriptParameter, InParent, ViewModel)	{}

protected:
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override;
	virtual UObject* GetObjectForEditing() override { return Cast<UNiagaraHierarchyScriptParameter>(GetDataMutable())->GetScriptVariable(); }
	/** We want to be able to edit in the details panel regardless of source or hierarchy item. */
	virtual bool AllowEditingInDetailsPanel() const override { return true; }
	virtual bool CanRenameInternal() override { return false; }
	virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType) override;
	
	virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel>, EItemDropZone ItemDropZone) override;
	
	virtual FResultWithUserFeedback CanHaveChildren() const override { return bIsForHierarchy == true; }
};

struct FNiagaraHierarchyScriptRootViewModel : public FHierarchyRootViewModel
{
	FNiagaraHierarchyScriptRootViewModel(UHierarchyRoot* Root, TWeakObjectPtr<UNiagaraScriptParametersHierarchyViewModel> ViewModel, bool bInIsForHierarchy)
		: FHierarchyRootViewModel(Root, ViewModel, bInIsForHierarchy)	{}

protected:
	// We don't sort anything to allow free authoring of UI
	virtual void SortChildrenData() override { }
};

#undef UE_API
