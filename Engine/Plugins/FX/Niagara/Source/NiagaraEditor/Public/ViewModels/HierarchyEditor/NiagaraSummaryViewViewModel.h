// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraNodeFunctionCall.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraSimulationStageBase.h"
#include "DataHierarchyViewModelBase.h"
#include "NiagaraSummaryViewViewModel.generated.h"

class UNiagaraStackEventHandlerPropertiesItem;
class FNiagaraEmitterViewModel;

UCLASS()
class UNiagaraHierarchySummaryDataRefreshContext : public UHierarchyDataRefreshContext
{
	GENERATED_BODY()

public:
	UPROPERTY(Transient)
	TArray<TObjectPtr<UNiagaraRendererProperties>> Renderers;

	TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel;
};

UCLASS(MinimalAPI)
class UNiagaraHierarchyModule : public UHierarchyItem
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API void Initialize(const UNiagaraNodeFunctionCall& ModuleNode);
};

UCLASS(MinimalAPI)
class UNiagaraHierarchyModuleInput : public UHierarchyItem
{
	GENERATED_BODY()

public:
	UNiagaraHierarchyModuleInput() {}
	virtual ~UNiagaraHierarchyModuleInput() override = default;

	NIAGARAEDITOR_API void Initialize(const UNiagaraNodeFunctionCall& FunctionCall, FGuid InputGuid);

	void SetDisplayNameOverride(const FText& InText) { DisplayNameOverride = InText; }
	FText GetDisplayNameOverride() const { return DisplayNameOverride; }

	FText GetTooltipOverride() const { return TooltipOverride; }
private:
	/** If specified, will override how this input is presented in the stack. */
	UPROPERTY(EditAnywhere, Category="Niagara")
	FText DisplayNameOverride;

	/** If specified, will override how the tooltip of this input in the stack. */
	UPROPERTY(EditAnywhere, Category="Niagara", meta = (MultiLine = "true"))
	FText TooltipOverride;
};

UCLASS(MinimalAPI)
class UNiagaraHierarchyAssignmentInput : public UHierarchyItem
{
	GENERATED_BODY()

public:
	UNiagaraHierarchyAssignmentInput() {}
	virtual ~UNiagaraHierarchyAssignmentInput() override = default;

	NIAGARAEDITOR_API void Initialize(const UNiagaraNodeAssignment& AssignmentNode, FName AssignmentTarget);

	FText GetTooltipOverride() const { return TooltipOverride; }
private:
	/** If specified, will override how the tooltip of this input in the stack. */
	UPROPERTY(EditAnywhere, Category="Niagara", meta = (MultiLine = "true"))
	FText TooltipOverride;
};

UCLASS(MinimalAPI)
class UNiagaraHierarchyEmitterProperties : public UHierarchyItem
{
	GENERATED_BODY()
public:
	void Initialize(const FVersionedNiagaraEmitter& Emitter);
};

UCLASS(MinimalAPI)
class UNiagaraHierarchyRenderer : public UHierarchyItem
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API void Initialize(const UNiagaraRendererProperties& Renderer);
};

UCLASS(MinimalAPI)
class UNiagaraHierarchyEventHandler : public UHierarchyItem
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API void Initialize(const FNiagaraEventScriptProperties& EventHandler);
};

UCLASS(MinimalAPI)
class UNiagaraHierarchyEventHandlerProperties : public UHierarchyItem
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API void Initialize(const FNiagaraEventScriptProperties& EventHandler);
	
	static NIAGARAEDITOR_API FHierarchyElementIdentity MakeIdentity(const FNiagaraEventScriptProperties& EventHandler);
};

UCLASS(MinimalAPI)
class UNiagaraHierarchySimStage : public UHierarchyItem
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API void Initialize(const UNiagaraSimulationStageBase& SimStage);
};

UCLASS(MinimalAPI)
class UNiagaraHierarchySimStageProperties : public UHierarchyItem
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API void Initialize(const UNiagaraSimulationStageBase& SimStage);
	
	static NIAGARAEDITOR_API FHierarchyElementIdentity MakeIdentity(const UNiagaraSimulationStageBase& SimStage);
};

UCLASS(MinimalAPI)
class UNiagaraHierarchyObjectProperty : public UHierarchyItem
{
	GENERATED_BODY()
public:
	/** To know what object this ObjectProperty is referring to, a persistent guid that can be mapped back to an object is required. */
	NIAGARAEDITOR_API void Initialize(FGuid ObjectGuid, FString PropertyName);
};

UCLASS(MinimalAPI)
class UNiagaraSummaryViewViewModel : public UDataHierarchyViewModelBase
{
	GENERATED_BODY()
public:
	UNiagaraSummaryViewViewModel() {}
	virtual ~UNiagaraSummaryViewViewModel() override = default;

	NIAGARAEDITOR_API void Initialize(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;
	
	NIAGARAEDITOR_API TSharedRef<FNiagaraEmitterViewModel> GetEmitterViewModel() const;
	
	NIAGARAEDITOR_API virtual UHierarchyRoot* GetHierarchyRoot() const override;
	NIAGARAEDITOR_API virtual TSharedPtr<FHierarchyElementViewModel> CreateCustomViewModelForElement(UHierarchyElement* ItemBase, TSharedPtr<FHierarchyElementViewModel> Parent) override;
	
	NIAGARAEDITOR_API virtual void PrepareSourceItems(UHierarchyRoot* SourceRoot, TSharedPtr<FHierarchyRootViewModel>) override;
	NIAGARAEDITOR_API virtual void SetupCommands() override;
	
	NIAGARAEDITOR_API virtual TSharedRef<FHierarchyDragDropOp> CreateDragDropOp(const FDataHierarchyDragDropContext& DragContext) override;
	
	virtual bool SupportsDetailsPanel() override { return true; }
	virtual bool SupportsMultiEditInDetailsPanel() override { return false; }
	NIAGARAEDITOR_API virtual TArray<TTuple<UClass*, FOnGetDetailCustomizationInstance>> GetInstanceCustomizations() override;

	NIAGARAEDITOR_API TMap<FGuid, UObject*> GetObjectsForProperties();

	NIAGARAEDITOR_API UNiagaraNodeFunctionCall* GetFunctionCallNode(const FGuid& NodeIdentity);
	NIAGARAEDITOR_API void ClearFunctionCallNodeCache(const FGuid& NodeIdentity);
	NIAGARAEDITOR_API TOptional<struct FInputData> GetInputData(const UNiagaraHierarchyModuleInput& Input);

private:
	NIAGARAEDITOR_API void OnScriptGraphChanged(const FEdGraphEditAction& Action, const UNiagaraScript& Script);
	NIAGARAEDITOR_API void OnRenderersChanged();
	NIAGARAEDITOR_API void OnSimStagesChanged();
	NIAGARAEDITOR_API void OnEventHandlersChanged();
protected:
	// The cache is used to speed up access across different inputs, as the view models for both regular inputs & modules, dynamic inputs & assignment nodes need to 'find' these nodes which is expensive
	TMap<FGuid, TWeakObjectPtr<UNiagaraNodeFunctionCall>> FunctionCallCache;
	TWeakPtr<FNiagaraEmitterViewModel> EmitterViewModelWeak;
};

/** The view model for both module nodes & dynamic input nodes */
struct FNiagaraFunctionViewModel : public FHierarchyItemViewModel
{
	FNiagaraFunctionViewModel(UNiagaraHierarchyModule* HierarchyModule, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyItemViewModel(HierarchyModule, InParent, ViewModel)
	{ }

	virtual ~FNiagaraFunctionViewModel() override;
	
	TWeakObjectPtr<UNiagaraNodeFunctionCall> GetFunctionCallNode() const;	
	
	bool IsFromBaseEmitter() const;

	bool IsDynamicInput() const { return bIsDynamicInput; }
private:	
	virtual void Initialize() override;
	virtual void RefreshChildrenDataInternal() override;
	void RefreshChildrenInputs(bool bClearCache = false);
	
	virtual FString ToString() const override;

	virtual FResultWithUserFeedback IsEditableByUser() override;
	virtual FResultWithUserFeedback CanHaveChildren() const override { return bIsForHierarchy == false; }
	virtual bool CanRenameInternal() override { return false; }
	virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel>, EItemDropZone ItemDropZone) override;

	virtual bool IsExpandedByDefaultInternal() const override { return false; }

	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override { ClearCache(); return GetFunctionCallNode().IsValid(); }
	
	void OnScriptApplied(UNiagaraScript* NiagaraScript, FGuid Guid);

	void ClearCache() const;
private:
	FDelegateHandle OnScriptAppliedHandle;
	mutable TOptional<bool> IsFromBaseEmitterCache;
	bool bIsDynamicInput = false;
};

struct FInputData
{
	FName InputName;
	FNiagaraTypeDefinition Type;
	FNiagaraVariableMetaData MetaData;
	bool bIsStatic = false;
	TArray<FGuid> ChildrenInputGuids;
	UNiagaraNodeFunctionCall* FunctionCallNode;
};

struct FNiagaraModuleInputViewModel : public FHierarchyItemViewModel
{	
	FNiagaraModuleInputViewModel(UNiagaraHierarchyModuleInput* ModuleInput, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyItemViewModel(ModuleInput, InParent, ViewModel) {}

public:
	TOptional<FInputData> GetInputData() const;
	FText GetSummaryInputNameOverride() const;
	
protected:
	TWeakObjectPtr<UNiagaraNodeFunctionCall> GetModuleNode() const;

	virtual FString ToString() const override;
	virtual TArray<FString> GetSearchTerms() const override;
	
	bool IsFromBaseEmitter() const;

	void ClearCache() const;

	void RefreshChildDynamicInputs(bool bClearCache = false);

	virtual bool IsExpandedByDefaultInternal() const override { return false; }
	virtual FResultWithUserFeedback CanHaveChildren() const override;
	virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType) override;
	virtual FResultWithUserFeedback IsEditableByUser() override;
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override { ClearCache(); return GetInputData().IsSet(); }
	virtual void RefreshChildrenDataInternal() override;
	
	virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel>, EItemDropZone ItemDropZone) override;
	virtual void PostAddFixup() override;

	virtual void AppendDynamicContextMenuForSingleElement(UToolMenu* ToolMenu) override;
private:
	TOptional<FInputData> FindInputDataInternal() const;
	void AddNativeChildrenInputs();
	bool CanAddNativeChildrenInputs() const;
	TArray<FHierarchyElementIdentity> GetNativeChildInputIdentities() const;
private:
	mutable TOptional<FInputData> InputDataCache;
	mutable TOptional<bool> IsFromBaseEmitterCache;
};

struct FNiagaraAssignmentInputViewModel : public FHierarchyItemViewModel
{	
	FNiagaraAssignmentInputViewModel(UNiagaraHierarchyAssignmentInput* ModuleInput, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyItemViewModel(ModuleInput, InParent, ViewModel) {}

	virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel>, EItemDropZone ItemDropZone) override;
	
	TWeakObjectPtr<UNiagaraNodeAssignment> GetAssignmentNode() const;
	TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData> GetInputData() const;

	virtual FResultWithUserFeedback CanHaveChildren() const override { return false; }

	virtual FString ToString() const override;
	virtual TArray<FString> GetSearchTerms() const override;
	
	bool IsFromBaseEmitter() const;

	void ClearCache() const;
protected:
	virtual bool IsExpandedByDefaultInternal() const override { return false; }
	virtual FResultWithUserFeedback IsEditableByUser() override;
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override { ClearCache(); return GetInputData().IsSet(); }
private:
	TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData> FindInputDataInternal() const;
private:
	mutable TWeakObjectPtr<UNiagaraNodeAssignment> AssignmentNodeCache;
	mutable TOptional<FNiagaraStackGraphUtilities::FMatchingFunctionInputData> InputDataCache;
	mutable TOptional<bool> IsFromBaseEmitterCache;
};

struct FNiagaraHierarchySummaryCategoryViewModel : public FHierarchyCategoryViewModel
{
	FNiagaraHierarchySummaryCategoryViewModel(UHierarchyCategory* Category, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyCategoryViewModel(Category, InParent, ViewModel) {}
	
	bool IsFromBaseEmitter() const;
	
protected:
	virtual bool IsExpandedByDefaultInternal() const override { return false; }
	virtual FResultWithUserFeedback IsEditableByUser() override;
	
	mutable TOptional<bool> IsFromBaseEmitterCache;
};

struct FNiagaraHierarchyPropertyViewModel : public FHierarchyItemViewModel
{
	FNiagaraHierarchyPropertyViewModel(UNiagaraHierarchyObjectProperty* ObjectProperty, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
		: FHierarchyItemViewModel(ObjectProperty, InParent, ViewModel) {}

	virtual FString ToString() const override;

	bool IsFromBaseEmitter() const;
protected:
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override;
	
	virtual FResultWithUserFeedback IsEditableByUser() override;
	
	mutable TOptional<bool> IsFromBaseEmitterCache;	
};

struct FNiagaraHierarchyRendererViewModel : public FHierarchyItemViewModel
{
	FNiagaraHierarchyRendererViewModel(UNiagaraHierarchyRenderer* Renderer, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyItemViewModel(Renderer, InParent, ViewModel) {}

	virtual FString ToString() const override;
	virtual bool CanRenameInternal() override { return false; }

	UNiagaraRendererProperties* GetRendererProperties() const;

	bool IsFromBaseEmitter() const;
protected:
	virtual void RefreshChildrenDataInternal() override;

	virtual bool IsExpandedByDefaultInternal() const override { return false; }
	
	virtual FResultWithUserFeedback IsEditableByUser() override;
	virtual FResultWithUserFeedback CanHaveChildren() const override { return bIsForHierarchy == false; }
	
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override { return GetRendererProperties() != nullptr;}
	
	mutable TOptional<bool> IsFromBaseEmitterCache;
};

/** Emitter Properties currently don't list their individual properties since it's a mix of data of FVersionedNiagaraEmitterData & actual properties on the emitter which requires customization. */
struct FNiagaraHierarchyEmitterPropertiesViewModel : public FHierarchyItemViewModel
{
	FNiagaraHierarchyEmitterPropertiesViewModel(UNiagaraHierarchyEmitterProperties* EmitterProperties, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyItemViewModel(EmitterProperties, InParent, ViewModel) {}

	virtual FString ToString() const override;
	virtual bool CanRenameInternal() override { return false; }

	bool IsFromBaseEmitter() const;
protected:
	virtual bool IsExpandedByDefaultInternal() const override { return false; }

	virtual FResultWithUserFeedback IsEditableByUser() override;
	virtual FResultWithUserFeedback CanHaveChildren() const override { return bIsForHierarchy == false; }
	
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override { return true; }
	
	mutable TOptional<bool> IsFromBaseEmitterCache;
};

struct FNiagaraHierarchyEventHandlerViewModel : public FHierarchyItemViewModel
{
	FNiagaraHierarchyEventHandlerViewModel(UNiagaraHierarchyEventHandler* EventHandler, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyItemViewModel(EventHandler, InParent, ViewModel) {}

	virtual FString ToString() const override;
	virtual bool CanRenameInternal() override { return false; }

	FNiagaraEventScriptProperties* GetEventScriptProperties() const;

	bool IsFromBaseEmitter() const;
protected:
	virtual void RefreshChildrenDataInternal() override;
	
	virtual FResultWithUserFeedback IsEditableByUser() override;
	virtual FResultWithUserFeedback CanHaveChildren() const override { return bIsForHierarchy == false; }
	
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override { return GetEventScriptProperties() != nullptr;}
	
	mutable TOptional<bool> IsFromBaseEmitterCache;
};

struct FNiagaraHierarchyEventHandlerPropertiesViewModel : public FHierarchyItemViewModel
{
	FNiagaraHierarchyEventHandlerPropertiesViewModel(UNiagaraHierarchyEventHandlerProperties* EventHandlerProperties, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyItemViewModel(EventHandlerProperties, InParent, ViewModel) {}

	virtual FString ToString() const override;
	virtual bool CanRenameInternal() override { return false; }

	FNiagaraEventScriptProperties* GetEventScriptProperties() const;

	bool IsFromBaseEmitter() const;
protected:
	virtual void RefreshChildrenDataInternal() override;

	virtual FResultWithUserFeedback IsEditableByUser() override;
	virtual FResultWithUserFeedback CanHaveChildren() const override { return bIsForHierarchy == false; }
	
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override { return GetEventScriptProperties() != nullptr;}

	mutable TOptional<bool> IsFromBaseEmitterCache;
};

struct FNiagaraHierarchySimStageViewModel : public FHierarchyItemViewModel
{
	FNiagaraHierarchySimStageViewModel(UNiagaraHierarchySimStage* SimStage, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyItemViewModel(SimStage, InParent, ViewModel) {}

	virtual FString ToString() const override;
	virtual bool CanRenameInternal() override { return false; }

	UNiagaraSimulationStageBase* GetSimStage() const;
	
	bool IsFromBaseEmitter() const;
protected:
	virtual void RefreshChildrenDataInternal() override;
	
	virtual FResultWithUserFeedback IsEditableByUser() override;
	virtual FResultWithUserFeedback CanHaveChildren() const override { return bIsForHierarchy == false; }
	
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override { return GetSimStage() != nullptr;}
	
	mutable TOptional<bool> IsFromBaseEmitterCache;
};

struct FNiagaraHierarchySimStagePropertiesViewModel : public FHierarchyItemViewModel
{
	FNiagaraHierarchySimStagePropertiesViewModel(UNiagaraHierarchySimStageProperties* SimStage, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UNiagaraSummaryViewViewModel> ViewModel)
	: FHierarchyItemViewModel(SimStage, InParent, ViewModel) {}

	virtual FString ToString() const override;
	virtual bool CanRenameInternal() override { return false; }

	UNiagaraSimulationStageBase* GetSimStage() const;

	bool IsFromBaseEmitter() const;
protected:
	virtual void RefreshChildrenDataInternal() override;

	virtual FResultWithUserFeedback IsEditableByUser() override;
	virtual FResultWithUserFeedback CanHaveChildren() const override { return bIsForHierarchy == false; }
	
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override { return GetSimStage() != nullptr;}

	mutable TOptional<bool> IsFromBaseEmitterCache;
};

class FNiagaraHierarchyInputParameterHierarchyDragDropOp : public FHierarchyDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FNiagaraInputParameterHierarchyDragDropOp, FHierarchyDragDropOp)

	explicit FNiagaraHierarchyInputParameterHierarchyDragDropOp(const FDataHierarchyDragDropContext& InDragContext) : FHierarchyDragDropOp(InDragContext) {}

	virtual TSharedRef<SWidget> CreateCustomDecorator() const override;
};

class SNiagaraHierarchyModule : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNiagaraHierarchyModule)
		{}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<struct FNiagaraFunctionViewModel> InModuleViewModel);
		
	FText GetModuleDisplayName() const;
	
private:
	TWeakPtr<struct FNiagaraFunctionViewModel> ModuleViewModel;
	TSharedPtr<SInlineEditableTextBlock> InlineEditableTextBlock;
};
