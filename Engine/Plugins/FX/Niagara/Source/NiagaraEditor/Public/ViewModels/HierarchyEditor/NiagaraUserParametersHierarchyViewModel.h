// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataHierarchyViewModelBase.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraUserParametersHierarchyViewModel.generated.h"

class FNiagaraSystemViewModel;

UCLASS()
class UNiagaraHierarchyUserParameterRefreshContext : public UHierarchyDataRefreshContext
{
	GENERATED_BODY()

public:
	void SetSystem(UNiagaraSystem* InSystem) { System = InSystem; }
	const UNiagaraSystem* GetSystem() const { return System; }
private:
	UPROPERTY(Transient)
	TObjectPtr<UNiagaraSystem> System = nullptr;
};

UCLASS()
class UNiagaraHierarchyUserParameter : public UHierarchyItem
{
	GENERATED_BODY()

public:
	UNiagaraHierarchyUserParameter() {}
	virtual ~UNiagaraHierarchyUserParameter() override = default;
	
	void Initialize(UNiagaraScriptVariable& InUserParameterScriptVariable);
	
	const FNiagaraVariable& GetUserParameter() const { return UserParameterScriptVariable->Variable; }

	virtual FString ToString() const override { return GetUserParameter().GetName().ToString(); }

	const UNiagaraScriptVariable* GetScriptVariable() const { return UserParameterScriptVariable; }
private:
	UPROPERTY()
	TObjectPtr<UNiagaraScriptVariable> UserParameterScriptVariable;
};

struct FNiagaraHierarchyUserParameterViewModel : public FHierarchyItemViewModel
{
	FNiagaraHierarchyUserParameterViewModel(UNiagaraHierarchyUserParameter* InItem, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel)
	: FHierarchyItemViewModel(InItem, InParent, InHierarchyViewModel) {}
	
	/** For editing in the details panel we want to handle the script variable that is represented by the hierarchy item, not the hierarchy item itself. */
	virtual UObject* GetObjectForEditing() override;
	/** We want to be able to edit in the details panel regardless of source or hierarchy item. */
	virtual bool AllowEditingInDetailsPanel() const override { return true; }

	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override;
};

UCLASS()
class UNiagaraUserParametersHierarchyViewModel : public UDataHierarchyViewModelBase
{
	GENERATED_BODY()
public:
	UNiagaraUserParametersHierarchyViewModel() {}
	virtual ~UNiagaraUserParametersHierarchyViewModel() override = default;

	TSharedRef<FNiagaraSystemViewModel> GetSystemViewModel() const;

	void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);
	
	virtual UHierarchyRoot* GetHierarchyRoot() const override;
	virtual TSharedPtr<FHierarchyElementViewModel> CreateCustomViewModelForElement(UHierarchyElement* ItemBase, TSharedPtr<FHierarchyElementViewModel> Parent) override;
	
	virtual void PrepareSourceItems(UHierarchyRoot* SourceRoot, TSharedPtr<FHierarchyRootViewModel>) override;
	virtual void SetupCommands() override;
	
	virtual TSharedRef<FHierarchyDragDropOp> CreateDragDropOp(const FDataHierarchyDragDropContext& DragContext) override;
	
	virtual bool SupportsDetailsPanel() override { return true; }
	virtual bool SupportsMultiEditInDetailsPanel() override { return false; }
	virtual TArray<TTuple<UClass*, FOnGetDetailCustomizationInstance>> GetInstanceCustomizations() override;
protected:
	virtual void FinalizeInternal() override;
private:
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModelWeak;
};

class FNiagaraUserParameterHierarchyDragDropOp : public FHierarchyDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FNiagaraUserParameterHierarchyDragDropOp, FHierarchyDragDropOp)

	explicit FNiagaraUserParameterHierarchyDragDropOp(const FDataHierarchyDragDropContext& InDragContext) : FHierarchyDragDropOp(InDragContext) {}

	virtual TSharedRef<SWidget> CreateCustomDecorator() const override;
};
