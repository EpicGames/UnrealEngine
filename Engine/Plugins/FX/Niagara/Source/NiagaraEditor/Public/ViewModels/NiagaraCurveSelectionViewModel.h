// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"
#include "NiagaraCurveSelectionViewModel.generated.h"

class FCompileConstantResolver;
class UNiagaraDataInterfaceCurveBase;
struct FNiagaraEmitterHandle;
class UNiagaraNodeFunctionCall;
class FNiagaraPlaceholderDataInterfaceHandle;
class UNiagaraScript;
class UNiagaraStackEditorData;
class UNiagaraSystem;
class FNiagaraSystemViewModel;
struct FRichCurve;

struct FNiagaraCurveSelectionTreeNodeDataId
{
	FNiagaraCurveSelectionTreeNodeDataId()
		: Object(nullptr)
	{
	}

	NIAGARAEDITOR_API bool operator==(const FNiagaraCurveSelectionTreeNodeDataId& Other) const;

	FName UniqueName;
	FGuid Guid;
	UObject* Object;

	static NIAGARAEDITOR_API FNiagaraCurveSelectionTreeNodeDataId FromUniqueName(FName UniqueName);
	static NIAGARAEDITOR_API FNiagaraCurveSelectionTreeNodeDataId FromGuid(FGuid Guid);
	static NIAGARAEDITOR_API FNiagaraCurveSelectionTreeNodeDataId FromObject(UObject* Object);
};

enum class ENiagaraCurveSelectionNodeStyleMode
{
	TopLevelObject,
	Script,
	Module,
	DynamicInput,
	DataInterface,
	CurveComponent
};

struct FNiagaraCurveSelectionTreeNode : TSharedFromThis<FNiagaraCurveSelectionTreeNode>
{
public:
	NIAGARAEDITOR_API FNiagaraCurveSelectionTreeNode();

	NIAGARAEDITOR_API const FNiagaraCurveSelectionTreeNodeDataId& GetDataId() const;

	NIAGARAEDITOR_API void SetDataId(const FNiagaraCurveSelectionTreeNodeDataId& InDataId);

	NIAGARAEDITOR_API FGuid GetNodeUniqueId() const;

	NIAGARAEDITOR_API FText GetDisplayName() const;

	NIAGARAEDITOR_API void SetDisplayName(FText InDisplayName);

	NIAGARAEDITOR_API FText GetSecondDisplayName() const;

	NIAGARAEDITOR_API void SetSecondDisplayName(FText InSecondDisplayName);

	NIAGARAEDITOR_API FText GetToolTipText() const;

	NIAGARAEDITOR_API void SetToolTipText(FText InToolTipText);

	NIAGARAEDITOR_API ENiagaraCurveSelectionNodeStyleMode GetStyleMode() const;

	NIAGARAEDITOR_API FName GetExecutionCategory() const;

	NIAGARAEDITOR_API FName GetExecutionSubcategory() const;

	NIAGARAEDITOR_API bool GetIsParameter() const;

	NIAGARAEDITOR_API void SetStyle(ENiagaraCurveSelectionNodeStyleMode InStyleMode, FName InExecutionCategory, FName InExecutionSubcategory, bool bInIsParameter);

	NIAGARAEDITOR_API TSharedPtr<FNiagaraCurveSelectionTreeNode> GetParent() const;

protected:
	NIAGARAEDITOR_API void SetParent(TSharedPtr<FNiagaraCurveSelectionTreeNode> InParent);

public:
	NIAGARAEDITOR_API const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>>& GetChildNodes() const;

	NIAGARAEDITOR_API void SetChildNodes(const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> InChildNodes);

	static NIAGARAEDITOR_API TSharedPtr<FNiagaraCurveSelectionTreeNode> FindNodeWithDataId(const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>>& Nodes, FNiagaraCurveSelectionTreeNodeDataId DataId);

	NIAGARAEDITOR_API TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> GetCurveDataInterface() const;

	NIAGARAEDITOR_API FRichCurve* GetCurve() const;

	NIAGARAEDITOR_API FName GetCurveName() const;

	NIAGARAEDITOR_API FLinearColor GetCurveColor() const;

	NIAGARAEDITOR_API bool GetCurveIsReadOnly() const;

	NIAGARAEDITOR_API void SetCurveDataInterface(UNiagaraDataInterfaceCurveBase* InCurveDataInterface);

	NIAGARAEDITOR_API void SetPlaceholderDataInterfaceHandle(TSharedPtr<FNiagaraPlaceholderDataInterfaceHandle> InPlaceholderDataInterfaceHandle);

	NIAGARAEDITOR_API void SetCurveData(UNiagaraDataInterfaceCurveBase* InCurveDataInterface, FRichCurve* InCurve, FName InCurveName, FLinearColor InCurveColor);

	NIAGARAEDITOR_API const TOptional<FObjectKey>& GetDisplayedObjectKey() const;

	NIAGARAEDITOR_API void SetDisplayedObjectKey(FObjectKey InDisplayedObjectKey);

	NIAGARAEDITOR_API bool GetShowInTree() const;

	NIAGARAEDITOR_API void SetShowInTree(bool bInShouldShowInTree);

	NIAGARAEDITOR_API bool GetIsExpanded() const;

	NIAGARAEDITOR_API void SetIsExpanded(bool bInIsExpanded);

	NIAGARAEDITOR_API bool GetIsEnabled() const;

	NIAGARAEDITOR_API void SetIsEnabled(bool bInIsEnabled);

	NIAGARAEDITOR_API bool GetIsEnabledAndParentIsEnabled() const;

	NIAGARAEDITOR_API const TArray<int32>& GetSortIndices() const;

	NIAGARAEDITOR_API void UpdateSortIndices(int32 Index);

	NIAGARAEDITOR_API void ResetCachedEnabledState();

	NIAGARAEDITOR_API FSimpleMulticastDelegate& GetOnCurveChanged();

	NIAGARAEDITOR_API void NotifyCurveChanged();

private:
	FNiagaraCurveSelectionTreeNodeDataId DataId;
	FGuid NodeUniqueId;
	FText DisplayName;
	FText SecondDisplayName;
	FText ToolTipText;
	TWeakPtr<FNiagaraCurveSelectionTreeNode> ParentWeak;
	TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> ChildNodes;

	TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> CurveDataInterface;
	TSharedPtr<FNiagaraPlaceholderDataInterfaceHandle> PlaceholderDataInterfaceHandle;
	FRichCurve* Curve;
	FName CurveName;
	FLinearColor CurveColor;

	ENiagaraCurveSelectionNodeStyleMode StyleMode;
	FName ExecutionCategory;
	FName ExecutionSubcategory;
	bool bIsParameter;
	TOptional<FObjectKey> DisplayedObjectKey;
	bool bShowInTree;
	bool bIsExpanded;
	bool bIsEnabled;
	mutable TOptional<bool> bIsEnabledAndParentIsEnabledCache;
	TArray<int32> SortIndices;

	FSimpleMulticastDelegate OnCurveChangedDelegate;
};

UCLASS(MinimalAPI)
class UNiagaraCurveSelectionViewModel : public UObject
{
public:
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRequestSelectNode, FGuid /* NodeIdToSelect */);

public:
	NIAGARAEDITOR_API void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

	NIAGARAEDITOR_API void Finalize();

	NIAGARAEDITOR_API const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>>& GetRootNodes();

	NIAGARAEDITOR_API void FocusAndSelectCurveDataInterface(UNiagaraDataInterfaceCurveBase& CurveDataInterface);

	NIAGARAEDITOR_API void Refresh();

	NIAGARAEDITOR_API void RefreshDeferred();

	NIAGARAEDITOR_API void Tick();

	NIAGARAEDITOR_API FSimpleMulticastDelegate& OnRefreshed();

	NIAGARAEDITOR_API FOnRequestSelectNode& OnRequestSelectNode();

private:
	TSharedRef<FNiagaraCurveSelectionTreeNode> CreateNodeForCurveDataInterface(const FNiagaraCurveSelectionTreeNodeDataId& DataId, UNiagaraDataInterfaceCurveBase& CurveDataInterface, bool bIsParameter) const;

	TSharedPtr<FNiagaraCurveSelectionTreeNode> CreateNodeForUserParameters(TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes, UNiagaraSystem& System) const;

	TSharedPtr<FNiagaraCurveSelectionTreeNode> CreateNodeForFunction(
		const TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes,
		UNiagaraNodeFunctionCall& FunctionCallNode, UNiagaraStackEditorData& StackEditorData,
		FName ExecutionCategory, FName ExecutionSubCategory,
		FName InputName, bool bIsParameterDynamicInput,
		FCompileConstantResolver& ConstantResolver, FGuid& OwningEmitterHandleId) const;

	TSharedPtr<FNiagaraCurveSelectionTreeNode> CreateNodeForScript(
		TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes,
		UNiagaraScript& Script, FString ScriptDisplayName, UNiagaraStackEditorData& StackEditorData,
		FName ExecutionCategory, FName ExecutionSubcategory,
		const FNiagaraEmitterHandle* OwningEmitterHandle) const;

	TSharedPtr<FNiagaraCurveSelectionTreeNode> CreateNodeForSystem(TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes, UNiagaraSystem& System) const;

	TSharedPtr<FNiagaraCurveSelectionTreeNode> CreateNodeForEmitter(TArray<TSharedRef<FNiagaraCurveSelectionTreeNode>> OldParentChildNodes, const FNiagaraEmitterHandle& EmitterHandle) const;

	void DataInterfaceCurveChanged(TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> ChangedCurveDataInterfaceWeak) const;

	void StackEditorDataChanged();

	void UserParametersChanged();

private:
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModelWeak;

	TSharedPtr<FNiagaraCurveSelectionTreeNode> RootCurveSelectionTreeNode;

	FSimpleMulticastDelegate OnRefreshedDelegate;

	FOnRequestSelectNode OnRequestSelectNodeDelegate;

	mutable bool bHandlingInternalCurveChanged;

	bool bRefreshPending;
};
