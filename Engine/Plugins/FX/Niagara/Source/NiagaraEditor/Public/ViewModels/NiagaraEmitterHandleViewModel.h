// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Types/SlateEnums.h"
#include "Styling/SlateTypes.h"
#include "Layout/Visibility.h"

class UNiagaraSystem;
class FNiagaraSystemViewModel;
struct FNiagaraEmitterHandle;
class FNiagaraEmitterInstance;
class FNiagaraEmitterViewModel;
class UNiagaraStackViewModel;
class UNiagaraStackEntry;
class UNiagaraMessageData;
enum class ENiagaraSystemViewModelEditMode;

/** The view model for the FNiagaraEmitterEditorWidget. */
class FNiagaraEmitterHandleViewModel : public TSharedFromThis<FNiagaraEmitterHandleViewModel>, public FGCObject
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnPropertyChanged);
	DECLARE_MULTICAST_DELEGATE(FOnNameChanged);
public:
	/** Creates a new emitter editor view model.  This must be initialized before it can be used. */
	FNiagaraEmitterHandleViewModel(bool bInIsForDataProcessingOnly);

	virtual ~FNiagaraEmitterHandleViewModel() override;

	/** Initializes the emitter editor view model with the supplied emitter handle and simulation.*/
	void Initialize(TSharedRef<FNiagaraSystemViewModel> InOwningSystemViewModel, int32 InEmitterHandleIndex, TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> InSimulation);

	/** Returns whether or not this view model represents a valid emitter handle. */
	bool IsValid() const;

	/** Resets the data in the view model. */
	void Reset();

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraEmitterHandleViewModel");
	}

	/** Sets the simulation for the emitter this handle references. */
	void SetSimulation(TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> InSimulation);

	/** Gets the id of the emitter handle. */
	NIAGARAEDITOR_API FGuid GetId() const;

	/** Gets the name of the emitter handle. */
	NIAGARAEDITOR_API FName GetName() const;

	/** Sets the name of the emitter handle. */
	NIAGARAEDITOR_API void SetName(FName InName);

	/** Gets the text representation of the emitter handle name. */
	NIAGARAEDITOR_API FText GetNameText() const;

	/** Gets whether or not this emitter can be renamed. */
	NIAGARAEDITOR_API bool CanRenameEmitter() const;

	/** Called when the contents of the name text control is committed. */
	NIAGARAEDITOR_API void OnNameTextComitted(const FText& InText, ETextCommit::Type CommitInfo);

	/** Prevent invalid name being set on emitter.*/
	NIAGARAEDITOR_API bool VerifyNameTextChanged(const FText& NewText, FText& OutErrorMessage);

	/** Called to get the error state of the emitter handle.*/
	FText GetErrorText() const;
	EVisibility GetErrorTextVisibility() const;
	FSlateColor GetErrorTextColor() const;

	/** Gets whether or not this emitter handle is enabled. */
	NIAGARAEDITOR_API bool GetIsEnabled() const;

	/** Sets whether or not this emitter handle is enabled. Returns true if state was changed.
	 * Requests a recompile by default. Useful to turn off for batch operations. */
	NIAGARAEDITOR_API bool SetIsEnabled(bool bInIsEnabled, bool bRequestRecompile = true);

	/** Gets whether or not the emitter for this handle has been isolated in the UI. */
	NIAGARAEDITOR_API bool GetIsIsolated() const;

	/** Sets whether or not this emitter is isolated. May affect other emitters in the system. */
	NIAGARAEDITOR_API void SetIsIsolated(bool InIsIsolated);

	NIAGARAEDITOR_API void ToggleIsIsolated();

	NIAGARAEDITOR_API void RequestCaptureThumbnail();
	
	/** Gets the check state for the is enabled check box. */
	NIAGARAEDITOR_API ECheckBoxState GetIsEnabledCheckState() const;

	/** Called when the check state of the enabled check box changes. */
	NIAGARAEDITOR_API void OnIsEnabledCheckStateChanged(ECheckBoxState InCheckState);

	/** Gets the emitter handled being view and edited by this view model. */
	NIAGARAEDITOR_API FNiagaraEmitterHandle* GetEmitterHandle();

	/** Gets the view model for the emitter this handle references. */
	NIAGARAEDITOR_API TSharedRef<FNiagaraEmitterViewModel> GetEmitterViewModel();

	/** Gets the stack view model which represents the emitter pointed to by this handle. */
	NIAGARAEDITOR_API UNiagaraStackViewModel* GetEmitterStackViewModel();

	/** Gets the current edit mode of the emitter's owning system. */
	NIAGARAEDITOR_API ENiagaraSystemViewModelEditMode GetOwningSystemEditMode() const;

	/** Gets whether or not this emitter handle has a rename pending. */
	NIAGARAEDITOR_API bool GetIsRenamePending() const;

	/** Sets whether or not this emitter handle has a rename pending. */
	NIAGARAEDITOR_API void SetIsRenamePending(bool bInIsRenamePending);

	void RequestEmitterSelection(bool bClearSelection);

	/** Gets a multicast delegate which is called any time a property on the handle changes. */
	FOnPropertyChanged& OnPropertyChanged();

	/** Gets a multicast delegate which is called any time this handle is renamed. */
	FOnNameChanged& OnNameChanged();

	void Cleanup();
	NIAGARAEDITOR_API void GetRendererEntries(TArray<UNiagaraStackEntry*>& InRenderingEntries);
	NIAGARAEDITOR_API TSharedRef<FNiagaraSystemViewModel> GetOwningSystemViewModel() const;

	/** Add a serialized message to the Emitter this viewmodel is managing. Returns a key to the new message. */
	NIAGARAEDITOR_API FGuid AddMessage(UNiagaraMessageData* NewMessage, const FGuid& InNewGuid = FGuid()) const;

	/** Remove a serialized message from the Emitter this viewmodel is managing. */
	NIAGARAEDITOR_API void RemoveMessage(const FGuid& MessageKey) const;

	void BeginDebugEmitter();

private:
	/** The system view model which owns this emitter handle view model. */
	TWeakPtr<FNiagaraSystemViewModel> OwningSystemViewModelWeak;

	/** The index of this handle in the system when it was initialized. */
	int32 EmitterHandleIndex;

	/** The emitter handle being displayed and edited by this view model. */
	FNiagaraEmitterHandle* EmitterHandle;

	/** The view model for emitter this handle references. */
	TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel;

	/** The stack view model which represents the emitter pointed to by this handle. */
	TObjectPtr<UNiagaraStackViewModel> EmitterStackViewModel;

	/** A multicast delegate which is called any time a property on the handle changes. */
	FOnPropertyChanged OnPropertyChangedDelegate;

	/** A multicast delegate which is called any time this emitter handle is renamed. */
	FOnNameChanged OnNameChangedDelegate;

	/** Gets whether or not this emitter has a pending rename. */
	bool bIsRenamePending;
};
