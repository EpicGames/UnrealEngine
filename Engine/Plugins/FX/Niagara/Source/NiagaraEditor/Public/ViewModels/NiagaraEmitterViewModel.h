// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "ViewModels/NiagaraParameterDefinitionsSubscriberViewModel.h"
#include "UObject/ObjectKey.h"
#include "IAssetTypeActions.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraEmitter.h"

class UNiagaraEmitter;
class UNiagaraScript;
class FNiagaraSystemViewModel;
class FNiagaraScriptViewModel;
class FNiagaraScriptGraphViewModel;
class FNiagaraEmitterInstance;
struct FNiagaraVariable;
struct FNiagaraParameterStore;
struct FEdGraphEditAction;
class SWindow;
class FNiagaraEmitterHandleViewModel;
struct FNiagaraEventScriptProperties;
class UNiagaraScriptVariable;
class UNiagaraParameterDefinitions;
class UNiagaraSummaryViewViewModel;
struct FNiagaraMessageSourceAndStore;
class UNiagaraSummaryViewViewModel;


/** The view model for the UNiagaraEmitter objects */
class FNiagaraEmitterViewModel 
	: public TSharedFromThis<FNiagaraEmitterViewModel>
	, public TNiagaraViewModelManager<UNiagaraEmitter, FNiagaraEmitterViewModel>
	, public INiagaraParameterDefinitionsSubscriberViewModel
	, public FGCObject
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnEmitterChanged);
	DECLARE_MULTICAST_DELEGATE(FOnPropertyChanged);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnScriptCompiled, UNiagaraScript*, const FGuid&);
	DECLARE_MULTICAST_DELEGATE(FOnParentRemoved);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnScriptGraphChanged, const FEdGraphEditAction&, const UNiagaraScript&);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnScriptParameterStoreChanged, const FNiagaraParameterStore&, const UNiagaraScript&);
	DECLARE_DELEGATE_OneParam(FOnEmitterSelectionRequested, bool /* bClearSelection*/)

public:
	/** Creates a new emitter editor view model.  It must be initialized before use. */
	FNiagaraEmitterViewModel(bool bInIsForDataProcessingOnly);
	virtual ~FNiagaraEmitterViewModel() override;

	/** Initialize this view model with an emitter and simulation. */
	bool Initialize(const FVersionedNiagaraEmitter& InEmitter, TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> InSimulation);

	/** Resets this view model to initial conditions. */
	void Reset();

	//~ Begin NiagaraParameterDefinitionsSubscriberViewModel Interface
protected:
	virtual INiagaraParameterDefinitionsSubscriber* GetParameterDefinitionsSubscriber() override;
	//~ End NiagaraParameterDefinitionsSubscriberViewModel Interface

public:
	/** Gets the currently assigned simulation if there is one. */
	NIAGARAEDITOR_API TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> GetSimulation() const;

	/** Sets the current simulation for the emitter. */
	void SetSimulation(TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> InSimulation);

	/** Gets the emitter represented by this view model. */
	NIAGARAEDITOR_API FVersionedNiagaraEmitter GetEmitter();

	/** Gets whether or not this emitter has a parent emitter. */
	NIAGARAEDITOR_API bool HasParentEmitter() const;

	/** Gets the parent emitter for the emitter represented by this view model, if it has one. */
	NIAGARAEDITOR_API FVersionedNiagaraEmitter GetParentEmitter() const;

	/** Gets the text representation of the parent emitter name. */
	NIAGARAEDITOR_API FText GetParentNameText() const;

	/** Gets the text representation of the parent emitter path. */
	NIAGARAEDITOR_API FText GetParentPathNameText() const;

	NIAGARAEDITOR_API void CreateNewParentWindow(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);
	void UpdateParentEmitter(const TArray<FAssetData>& ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod, TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);

	/** Removes the parent emitter from this emitter. */
	NIAGARAEDITOR_API void RemoveParentEmitter();

	/** Gets text representing stats for the emitter. */
	//~ TODO: Instead of a single string here, we should probably have separate controls with tooltips etc.
	NIAGARAEDITOR_API FText GetStatsText() const;
	
	/** Geta a view model for the update/spawn Script. */
	TSharedRef<FNiagaraScriptViewModel> GetSharedScriptViewModel();

	NIAGARAEDITOR_API UNiagaraSummaryViewViewModel* GetSummaryHierarchyViewModel();

	/* Get the latest status of this view-model's script compilation.*/
	ENiagaraScriptCompileStatus GetLatestCompileStatus();

	/** Gets a multicast delegate which is called when the emitter for this view model changes to a different emitter. */
	FOnEmitterChanged& OnEmitterChanged();

	/** Gets a delegate which is called when a property on the emitter changes. */
	NIAGARAEDITOR_API FOnPropertyChanged& OnPropertyChanged();

	/** Gets a delegate which is called when the shared script is compiled. */
	FOnScriptCompiled& OnScriptCompiled();

	/** Gets a delegate which is called when this emitters parent is removed. */
	FOnParentRemoved& OnParentRemoved();

	/** Gets a multicast delegate which is called any time a graph on a script owned by this emitter changes. */
	FOnScriptGraphChanged& OnScriptGraphChanged();

	/** Gets a multicast delegate which is called any time a parameter store on a script owned by this emitter changes. */
	FOnScriptParameterStoreChanged& OnScriptParameterStoreChanged();

	FOnEmitterSelectionRequested& OnEmitterSelectionRequested();

	/** Gets editor specific data which can be stored per emitter.  If this data hasn't been created the default version will be returned. */
	NIAGARAEDITOR_API const UNiagaraEmitterEditorData& GetEditorData() const;

	/** Gets editor specific data which is stored per emitter.  If this data hasn't been created then it will be created. */
	NIAGARAEDITOR_API UNiagaraEmitterEditorData& GetEditorData();

	void GetEmitterMessageStores(TArray<FNiagaraMessageSourceAndStore>& OutMessageStores);

	/** Add an event script to the owned emitter. Sets the Usage, UsageID and Source of the EventScriptProperties. */
	NIAGARAEDITOR_API void AddEventHandler(FNiagaraEventScriptProperties& EventScriptProperties, bool bResetGraphForOutput = false);

	void Cleanup();

	/** When the parent emitter version is changed, this is set to generate the changelist in the stack. */
	FGuid PreviousEmitterVersion;

	/** Can be used by the ui after a version change to display change notes */
	FString PythonUpgradeScriptWarnings;
	
private:
	/** Sets this view model to a different emitter. */
	void SetEmitter(FVersionedNiagaraEmitter InEmitter);

	void OnVMCompiled(FVersionedNiagaraEmitter InEmitter);
	void OnGPUCompiled(FVersionedNiagaraEmitter InEmitter);

	void AddScriptEventHandlers();

	void RemoveScriptEventHandlers();

	void ScriptGraphChanged(const FEdGraphEditAction& InAction, const UNiagaraScript* OwningScript);

	void ScriptParameterStoreChanged(const FNiagaraParameterStore* ChangedParameterStore, const UNiagaraScript* OwningScript);

	void OnEmitterPropertiesChanged();

	void OnSummaryViewHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload);

	void UpdateSelectionOnSummaryViewStateChanged() const;

	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	
	/** The text format stats display .*/
	static const FText StatsFormat;

	/** The text format stats to only display particles count. */
	static const FText StatsParticleCountFormat;

	/** The text format stats to display when an emitter is disabled due to scalability. */
	static const FText ParticleDisabledDueToScalability;

	/** The emitter object being displayed by the control .*/
	FVersionedNiagaraEmitterWeakPtr EmitterWeakPtr;

	/** The runtime simulation for the emitter being displayed by the control */
	TWeakPtr<FNiagaraEmitterInstance, ESPMode::ThreadSafe> Simulation;
	
	/** The view model for the update/spawn/event script. */
	TSharedPtr<FNiagaraScriptViewModel> SharedScriptViewModel;

	TObjectPtr<UNiagaraSummaryViewViewModel> SummaryViewHierarchyViewModel;

	/** A flag to prevent reentrancy when updating selection sets. */
	bool bUpdatingSelectionInternally;

	/** A multicast delegate which is called whenever the emitter for this view model is changed to a different emitter. */
	FOnEmitterChanged OnEmitterChangedDelegate;

	/** A multicast delegate which is called whenever a property on the emitter changes. */
	FOnPropertyChanged OnPropertyChangedDelegate;

	FOnScriptCompiled OnScriptCompiledDelegate;

	/** A multicast delegate which is called when this emitters parent is removed. */
	FOnParentRemoved OnParentRemovedDelegate;

	FOnScriptGraphChanged OnScriptGraphChangedDelegate;

	FOnScriptParameterStoreChanged OnScriptParameterStoreChangedDelegate;

	FOnEmitterSelectionRequested OnEmitterSelectionRequestedDelegate;
	
	TNiagaraViewModelManager<UNiagaraEmitter, FNiagaraEmitterViewModel>::Handle RegisteredHandle;

	UEnum* ExecutionStateEnum;

	/** A mapping of script to the delegate handle for it's on graph changed delegate. */
	TMap<FObjectKey, FDelegateHandle> ScriptToOnGraphChangedHandleMap;
	TMap<FObjectKey, FDelegateHandle> ScriptToRecompileHandleMap;

	/** A mapping of script to the delegate handle for it's on parameter map changed delegate. */
	TMap<FObjectKey, FDelegateHandle> ScriptToOnParameterStoreChangedHandleMap;

	TSharedPtr<SWindow> NewParentWindow;
};
