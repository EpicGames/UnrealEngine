// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EditorUndoClient.h"
#include "Styling/SlateColor.h"
#include "Framework/Commands/UICommandList.h"

class UNiagaraGraph;
class FNiagaraSystemViewModel;
class FNiagaraObjectSelection;
struct FNiagaraGraphViewSettings;

/** A view model for editing a niagara system in a graph editor. */
class FNiagaraOverviewGraphViewModel : public TSharedFromThis<FNiagaraOverviewGraphViewModel>, public FEditorUndoClient
{
public:
	/** A multicast delegate which is called when nodes are pasted in the graph which supplies the pasted nodes. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodesPasted, const TSet<UEdGraphNode*>&);

	/** Create a new view model with the supplied system editor data and graph widget. */
	FNiagaraOverviewGraphViewModel();

	virtual ~FNiagaraOverviewGraphViewModel();

	void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);
	
	NIAGARAEDITOR_API TSharedRef<FNiagaraSystemViewModel> GetSystemViewModel();

	NIAGARAEDITOR_API const TSharedRef<FNiagaraSystemViewModel> GetSystemViewModel() const;

	/** Gets the display text for this graph. */
	NIAGARAEDITOR_API FText GetDisplayName() const;

	/** Gets the graph which is used to edit and view the system */
	NIAGARAEDITOR_API UEdGraph* GetGraph() const;

	/** Gets commands used for editing the graph. */
	NIAGARAEDITOR_API TSharedRef<FUICommandList> GetCommands();

	/** Gets the currently selected graph nodes. */
	NIAGARAEDITOR_API TSharedRef<FNiagaraObjectSelection> GetNodeSelection();

	NIAGARAEDITOR_API const FNiagaraGraphViewSettings& GetViewSettings() const;

	NIAGARAEDITOR_API void SetViewSettings(const FNiagaraGraphViewSettings& InOverviewGraphViewSettings);

	/** Gets a multicast delegate which is called when nodes are pasted in the graph. */
	NIAGARAEDITOR_API FOnNodesPasted& OnNodesPasted();

	//~ FEditorUndoClient interface.
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

	void UpdateOverviewGraphNodes();
private:
	void SetupCommands();

	void SelectAllNodes();
	void DeleteSelectedNodes();
	bool CanDeleteNodes() const;
	void CutSelectedNodes();
	bool CanCutNodes() const;
	void CopySelectedNodes();
	bool CanCopyNodes() const;
	void PasteNodes();
	bool CanPasteNodes() const;
	void DuplicateNodes();
	bool CanDuplicateNodes() const;
	void RenameNode();
	bool CanRenameNode() const;

	FText GetDisplayNameInternal() const;

	void GraphSelectionChanged();

	void SystemSelectionChanged();

private:

	/** The view model to interface with the system being viewed and edited by this view model. */
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;

	TWeakObjectPtr<UEdGraph> OverviewGraph;

	/** The display name for the overview graph. */
	mutable TOptional<FText> DisplayNameCache;

	/** Commands for editing the graph. */
	TSharedRef<FUICommandList> Commands;

	/** A multicast delegate which is called whenever nodes are pasted into the graph. */
	FOnNodesPasted OnNodesPastedDelegate;

	/** The set of nodes objects currently selected in the graph. */
	TSharedRef<FNiagaraObjectSelection> NodeSelection;

	bool bUpdatingSystemSelectionFromGraph;

	bool bUpdatingGraphSelectionFromSystem;

	/** Whether or not this view model is going to be used for data processing only and will not be shown in the UI. */
	bool bIsForDataProcessingOnly;
};
