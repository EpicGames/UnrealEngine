// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraParameterPanelViewModel.h"
#include "NiagaraToolkitCommon.h"
#include "Templates/SharedPointer.h"

class FNiagaraStandaloneScriptViewModel;
class FNiagaraSystemViewModel;
class UNiagaraParameterDefinitions;
class FNiagaraObjectSelection;

/** Interface for view models to the parameter panel. */
class INiagaraParameterDefinitionsPanelViewModel : public INiagaraImmutableParameterPanelViewModel
{
public:
	//~ Begin INiagaraImmutableParameterPanelViewModel interface
	NIAGARAEDITOR_API virtual const TArray<UNiagaraScriptVariable*> GetEditableScriptVariablesWithName(const FName ParameterName) const override;

	NIAGARAEDITOR_API virtual const TArray<FNiagaraGraphParameterReference> GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const override;

	//~ Begin Pure Virtual Methods
	virtual const TArray<UNiagaraGraph*> GetEditableGraphsConst() const override = 0;

	virtual const TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitionsAssets(bool bSkipSubscribedParameterDefinitions) const = 0;
	//~ End INiagaraImmutableParameterPanelViewModel interface

	virtual const TArray<UNiagaraParameterDefinitions*> GetParameterDefinitionsAssets() const = 0;
	virtual void AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const = 0;
	virtual void RemoveParameterDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete) const = 0;
	virtual bool GetCanRemoveParameterDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete, FText& OutCanUnsubscribeLibraryToolTip) const = 0;

	/** Find a subscribed Parameter Definitions asset with a matching Id GUID, or otherwise return nullptr. */
	virtual const UNiagaraParameterDefinitions* FindSubscribedParameterDefinitionsById(const FGuid& DefinitionsId) const = 0;

	// Find all external parameters with name matches to the definitions and set them to synchronize with all eligible definitions.
	virtual void SubscribeAllParametersToDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe) const = 0;
	virtual bool GetCanSubscribeAllParametersToDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe, FText& OutCanSubscribeParametersToolTip) const = 0;
	//~ End Pure Virtual Methods

	virtual void OnParameterItemSelected(const FNiagaraParameterDefinitionsPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const {};
	virtual FReply OnParameterItemsDragged(const TArray<FNiagaraParameterDefinitionsPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const { return FReply::Handled(); };
};

class FNiagaraScriptToolkitParameterDefinitionsPanelViewModel : public INiagaraParameterDefinitionsPanelViewModel
{
public:
	FNiagaraScriptToolkitParameterDefinitionsPanelViewModel(const TSharedPtr<FNiagaraStandaloneScriptViewModel>& InScriptViewModel);

	void Init(const FScriptToolkitUIContext& InUIContext);

	void Cleanup();

	//~ Begin INiagaraImmutableParameterPanelViewModel interface
	virtual const TArray<UNiagaraGraph*> GetEditableGraphsConst() const override;

	virtual const TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitionsAssets(bool bSkipSubscribedParameterDefinitions) const override;
	//~ End INiagaraImmutableParameterPanelViewModel interface

	//~ Begin INiagaraParameterDefinitionsPanelViewModel interface
	virtual const TArray<UNiagaraParameterDefinitions*> GetParameterDefinitionsAssets() const override;
	virtual void AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const override;
	virtual void RemoveParameterDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete) const override;
	virtual bool GetCanRemoveParameterDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete, FText& OutCanUnsubscribeLibraryToolTip) const override;
	virtual const UNiagaraParameterDefinitions* FindSubscribedParameterDefinitionsById(const FGuid& DefinitionsId) const override;
	virtual void SubscribeAllParametersToDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe) const override;
	virtual bool GetCanSubscribeAllParametersToDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe, FText& OutCanSubscribeParametersToolTip) const override;

	virtual void OnParameterItemSelected(const FNiagaraParameterDefinitionsPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const override;
	virtual FReply OnParameterItemsDragged(const TArray<FNiagaraParameterDefinitionsPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const override;
	//~ End INiagaraParameterDefinitionsPanelViewModel interface

private:
	TSharedPtr<FNiagaraStandaloneScriptViewModel> ScriptViewModel;
	TSharedPtr<FNiagaraObjectSelection> VariableObjectSelection;
	mutable FScriptToolkitUIContext UIContext;
};

class FNiagaraSystemToolkitParameterDefinitionsPanelViewModel : public INiagaraParameterDefinitionsPanelViewModel
{
public:
	FNiagaraSystemToolkitParameterDefinitionsPanelViewModel(const TSharedPtr<FNiagaraSystemViewModel>& InSystemViewModel);

	void Init(const FSystemToolkitUIContext& InUIContext);

	void Cleanup();

	//~ Begin INiagaraImmutableParameterPanelViewModel interface
	virtual const TArray<UNiagaraGraph*> GetEditableGraphsConst() const override;

	virtual const TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitionsAssets(bool bSkipSubscribedParameterDefinitions) const override;
	//~ End INiagaraImmutableParameterPanelViewModel interface

	//~ Begin INiagaraParameterDefinitionsPanelViewModel interface
	virtual const TArray<UNiagaraParameterDefinitions*> GetParameterDefinitionsAssets() const override;
	virtual void AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const override;
	virtual void RemoveParameterDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete) const override;
	virtual bool GetCanRemoveParameterDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToDelete, FText& OutCanUnsubscribeLibraryToolTip) const override;
	virtual const UNiagaraParameterDefinitions* FindSubscribedParameterDefinitionsById(const FGuid& DefinitionsId) const override;
	virtual void SubscribeAllParametersToDefinitions(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe) const override;
	virtual bool GetCanSubscribeAllParametersToDefinitionsAndToolTip(const FNiagaraParameterDefinitionsPanelCategory& CategoryToSubscribe, FText& OutCanSubscribeParametersToolTip) const override;

	virtual FReply OnParameterItemsDragged(const TArray<FNiagaraParameterDefinitionsPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const override;
	//~ End INiagaraParameterDefinitionsPanelViewModel interface

private:
	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
	TWeakPtr<FNiagaraSystemGraphSelectionViewModel> SystemGraphSelectionViewModelWeak;
	mutable FSystemToolkitUIContext UIContext;
};
