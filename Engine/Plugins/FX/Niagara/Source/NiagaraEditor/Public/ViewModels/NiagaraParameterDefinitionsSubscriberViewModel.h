// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "NiagaraCommon.h"
#include "NiagaraParameterDefinitionsDelegates.h"
#include "NiagaraParameterDefinitionsSubscriber.h"

class UNiagaraParameterDefinitions;
class UNiagaraParameterDefinitionsBase;
class UNiagaraScriptVariable;


/** Interface for viewmodels to classes that subscribe to UNiagaraParameterDefinitions. */
class INiagaraParameterDefinitionsSubscriberViewModel
{
public:
	virtual ~INiagaraParameterDefinitionsSubscriberViewModel() = default;
	
	void SubscribeToParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions);

	void UnsubscribeFromParameterDefinitions(const FGuid& ParameterDefinitionsToRemoveUniqueId);

	/** Synchronizes all Parameter Definitions UNiagaraScriptVariables with all graph parameters.  */
	void SynchronizeWithParameterDefinitions(FSynchronizeWithParameterDefinitionsArgs Args = FSynchronizeWithParameterDefinitionsArgs());

	/** Synchronizes a specific graph parameter with a subscribed parameter definition library's matching parameter (if it exists.)  */
	void SynchronizeScriptVarWithParameterDefinitions(UNiagaraScriptVariable* ScriptVarToSynchronize, bool bForce);

	/** Find all parameters owned by the object viewed by the INiagaraParameterDefinitionsSubscriberViewModel and mark them as synchronizing with the target Parameter Definitions, then synchronize them. */
	void SubscribeAllParametersToDefinitions(const FGuid& DefinitionsUniqueId);

	/** Find the parameter owned by the object viewed by the INiagaraParameterDefinitionsSubscriberViewModel and set its synchronizing state with a valid Parameter Definitions if possible. If set to synchronize, synchronize it immediately. */
	void SetParameterIsSubscribedToDefinitions(const FGuid& ScriptVarId, bool bIsSynchronizing);

	/** Find the parameter owned by the object viewed by the INiagaraParameterDefinitionsSubscriberViewModel and set its overriding state with a valid Parameter Definitions if possible. If set to no longer override, synchronize it immediately. */
	void SetParameterIsOverridingLibraryDefaultValue(const FGuid& ScriptVarId, bool bIsOverriding);

	TArray<UNiagaraParameterDefinitions*> GetSubscribedParameterDefinitions();

	/** Get all parameter libraries under the editor and niagara packages, along with those that are in the same package as the UObject this viewmodel is editing. */
	TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions);

	/** Get all UNiagaraScriptVariables held by the UNiagaraGraphs of the UNiagaraScriptSourceBases from GetAllSourceScripts(). */
	TArray<UNiagaraScriptVariable*> GetAllScriptVars();

	/** Find a subscribed Parameter Definitions with a matching Id GUID, or otherwise return nullptr. */
	UNiagaraParameterDefinitions* FindSubscribedParameterDefinitionsById(const FGuid& LibraryId);

	/** Find a viewed object owned UNiagaraScriptVariable with a matching Id GUID, or otherwise return nullptr. */
	UNiagaraScriptVariable* FindScriptVarById(const FGuid& ScriptVarId);

	/** Find a parameter definitions owned UNiagaraSCriptVariable with a matching parameter name FName, or otherwise return nullptr. */
	UNiagaraScriptVariable* FindSubscribedParameterDefinitionsScriptVarByName(const FName& ScriptVarName);

	/** Public passthrough to get the OnChanged() delegate owned by the viewed INiagaraParameterDefinitionsSubscriber. */
	FOnSubscribedParameterDefinitionsChanged& GetOnSubscribedParameterDefinitionsChangedDelegate();

protected:
	//~ Begin Pure Virtual Methods
	virtual INiagaraParameterDefinitionsSubscriber* GetParameterDefinitionsSubscriber() = 0;
	//~ End Pure Virtual Methods

	/** Get the path to the package of the UObject this viewmodel is editing. */
	FString GetSourceObjectPackagePathName();

};
