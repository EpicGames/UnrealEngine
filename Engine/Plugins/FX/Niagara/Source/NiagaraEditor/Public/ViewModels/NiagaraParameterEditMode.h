// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Defines edit modes for parameters in the view model. */
enum class ENiagaraParameterEditMode
{
	/** User can edit all aspects include name, type, and value, and can also add and remove. */
	EditAll,
	/** The user can only edit the value of the parameters. */
	EditValueOnly
};