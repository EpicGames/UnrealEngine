// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEditorCommon.h"
#include "NiagaraParameterPanelTypes.h"
#include "NiagaraToolkitCommon.h"
#include "Types/SlateEnums.h"
#include "EditorUndoClient.h"
#include "EdGraph/EdGraphSchema.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "NiagaraScript.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "Widgets/SNullWidget.h"

struct FCreateWidgetForActionData;
class FDelegateHandle;
class FMenuBuilder;
struct FNiagaraGraphParameterReference;
class FNiagaraObjectSelection;
class FNiagaraScriptViewModel;
class FNiagaraSystemGraphSelectionViewModel;
class FNiagaraSystemViewModel;
class FUICommandList;
class SComboButton;
class SEditableTextBox;
class UNiagaraGraph;
class UNiagaraNodeAssignment;
class UNiagaraParameterDefinitions;
class UNiagaraScript;
class UNiagaraScriptVariable;
class UNiagaraSystem;


namespace FNiagaraParameterUtilities
{
	enum class EParameterContext : uint8;
}

// NOTE: These utilities are not defined in the view model directly as they are shared between ParameterPanelViewModel and ParameterDefinitionsPanelViewModel.
namespace FNiagaraSystemToolkitParameterPanelUtilities
{
	FReply CreateDragEventForParameterItem(
		const FNiagaraParameterPanelItemBase& DraggedItem,
		const FPointerEvent& MouseEvent,
		const TArray<FNiagaraGraphParameterReference>& GraphParameterReferencesForItem,
		const TSharedPtr<TArray<FName>>& ParametersWithNamespaceModifierRenamePending
	);
}

namespace FNiagaraScriptToolkitParameterPanelUtilities
{
	FReply CreateDragEventForParameterItem(
		const FNiagaraParameterPanelItemBase& DraggedItem,
		const FPointerEvent& MouseEvent,
		const TArray<FNiagaraGraphParameterReference>& GraphParameterReferencesForItem,
		const TSharedPtr<TArray<FName>>& ParametersWithNamespaceModifierRenamePending
	);
}

namespace FNiagaraParameterPanelUtilities
{
	bool GetCanSetParameterNamespaceAndToolTipForScriptOrSystem(const FNiagaraParameterPanelItem& ItemToModify, const FName NewNamespace, FText& OutCanSetParameterNamespaceToolTip);

	bool GetCanSetParameterNamespaceModifierAndToolTipForScriptOrSystem(const TArray<FNiagaraParameterPanelItem>& CachedViewedItems, const FNiagaraParameterPanelItem& ItemToModify, const FName NamespaceModifier, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip);

	bool GetCanSetParameterCustomNamespaceModifierAndToolTipForScriptOrSystem(const FNiagaraParameterPanelItem& ItemToModify, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip);
}

struct FMenuAndSearchBoxWidgets
{
	TSharedPtr<SWidget> MenuWidget;
	TSharedPtr<SEditableTextBox> MenuSearchBoxWidget;
};

/** Base Interface for view models to SiagaraParameterPanel and SNiagaraParameterDefinitionsPanel. */
class INiagaraImmutableParameterPanelViewModel : public TSharedFromThis<INiagaraImmutableParameterPanelViewModel>, public FSelfRegisteringEditorUndoClient
{
public:
	/** Delegate to signal the view model's state has changed. */
	DECLARE_DELEGATE_OneParam(FOnRequestRefresh, bool);
	DECLARE_DELEGATE_OneParam(FOnRequestRefreshNextTick, bool);

	virtual ~INiagaraImmutableParameterPanelViewModel() = default;

	//~ Begin Pure Virtual Methods
	/** Returns a list of Graphs that are valid for operations to edit their variables and/or metadata.
	 *Should collect all Graphs that are currently selected, but also Graphs that are implicitly selected, e.g. the node graph for the script toolkit.
	*/
	virtual const TArray<UNiagaraGraph*> GetEditableGraphsConst() const = 0;

	virtual const TArray<UNiagaraScriptVariable*> GetEditableScriptVariablesWithName(const FName ParameterName) const = 0;

	virtual const TArray<FNiagaraGraphParameterReference> GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const = 0;
	//~ End Pure Virtual Methods

	//~ Begin FEditorUndoClient Interface 
	NIAGARAEDITOR_API virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); };
	//~ End FEditorUndoClient Interface 

	NIAGARAEDITOR_API virtual void CopyParameterReference(const FNiagaraParameterPanelItemBase& ItemToCopy) const;

	NIAGARAEDITOR_API virtual bool GetCanCopyParameterReferenceAndToolTip(const FNiagaraParameterPanelItemBase& ItemToCopy, FText& OutCanCopyParameterToolTip) const;

	NIAGARAEDITOR_API virtual void CopyParameterMetaData(const FNiagaraParameterPanelItemBase ItemToCopy) const;

	NIAGARAEDITOR_API virtual bool GetCanCopyParameterMetaDataAndToolTip(const FNiagaraParameterPanelItemBase& ItemToCopy, FText& OutCanCopyToolTip) const;

	// Refresh often encapsulates GUI rebuilding, so we have multiple API's for very specific use cases within the view model.

	// Plain refresh is a relatively fast refresh
	void Refresh() const { RefreshFull(false); };
	
	// RefreshFull allows for optional category tree expansion/collapse, which can be more expensive.
	NIAGARAEDITOR_API virtual void RefreshFull(bool bDoCategoryExpansion) const;

	// Enqueue light refresh next tick
	void RefreshNextTick() const { RefreshFullNextTick(false); }
	
	// Equeue a full refresh, parameterized by inputs next tick
	NIAGARAEDITOR_API virtual void RefreshFullNextTick(bool bDoCategoryExpansion) const;

	// Do an exhaustive refresh as context may be swapped out entirely
	virtual void RefreshDueToActiveDocumentChanged() {};

	FOnRequestRefresh& GetOnRequestRefreshDelegate() { return OnRequestRefreshDelegate; };
	FOnRequestRefreshNextTick& GetOnRequestRefreshNextTickDelegate() { return OnRequestRefreshNextTickDelegate; };

	// API for interacting with category expand/collapse within the view model
	virtual bool UsesCategoryFilteringForInitialExpansion() const { return false; }
	virtual bool IsCategoryExpandedByDefault(const FNiagaraParameterPanelCategory& Category) const { return false; };
	virtual void UpdateCategoryExpansionState(const FNiagaraParameterPanelCategory& Category, bool bIsExpanded) {};

protected:
	FOnRequestRefresh OnRequestRefreshDelegate;
	FOnRequestRefreshNextTick OnRequestRefreshNextTickDelegate;
};

/** Interface for view models to SiagaraParameterPanel. */
class INiagaraParameterPanelViewModel : public INiagaraImmutableParameterPanelViewModel
{
	
public:
	/** Helper struct for storing information about a section in the parameter panel (if used)*/
	struct FSectionDesc
	{
		FSectionDesc(const FGuid& InId, const FText& InName, const FText& InDesc) : SectionId(InId), DisplayName(InName), Description(InDesc) {}
		FGuid SectionId;
		FText DisplayName;
		FText Description;
	};

	/** Delegate to handle responses to external selection changes (e.g. TNiagaraSelection changes.) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnParameterPanelViewModelExternalSelectionChanged, const UObject*);

	/** Delegate to signal the ParameterPanel to select a parameter Item by name. */
	DECLARE_DELEGATE_OneParam(FOnSelectParameterItemByName, const FName /* ParameterName */);

	/** Delegate to signal the ParameterPanel to trigger pending parameter renames. */
	DECLARE_DELEGATE_OneParam(FOnNotifyParameterPendingRename, const FName /* ParameterName */);
	DECLARE_DELEGATE_OneParam(FOnNotifyParameterPendingNamespaceModifierRename, const FName /* ParameterName */);

	/** Delegate to get the name array representing parameters pending namespace modification. */
	DECLARE_DELEGATE_RetVal(TSharedPtr<TArray<FName>>, FOnGetParametersWithNamespaceModifierRenamePending);

	/** Delegate to get the names of all selected parameter items. */
	DECLARE_DELEGATE_RetVal(TArray<FName>, FOnGetSelectedParameterNames);
	
	DECLARE_MULTICAST_DELEGATE(FOnInvalidateCachedDependencies);
	
	INiagaraParameterPanelViewModel();
	virtual ~INiagaraParameterPanelViewModel() override;

	void RefreshFull(bool bDoCategoryExpansion) const;
	//~ Begin INiagaraImmutableParameterPanelViewModel interface
	virtual const TArray<UNiagaraGraph*> GetEditableGraphsConst() const = 0;

	//~ Begin Pure Virtual Methods
	virtual const TArray<UNiagaraScriptVariable*> GetEditableScriptVariablesWithName(const FName ParameterName) const = 0;

	virtual const TArray<FNiagaraGraphParameterReference> GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const = 0;

	virtual const TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions) const = 0;
	//~ End INiagaraImmutableParameterPanelViewModel interface

	virtual void AddParameter(FNiagaraVariable NewVariable, const FNiagaraParameterPanelCategory Category, const bool bRequestRename, const bool bMakeUniqueName)  = 0;

	virtual void FindOrAddParameter(FNiagaraVariable Variable, const FNiagaraParameterPanelCategory Category)  = 0;

	virtual bool CanAddType(const FNiagaraTypeDefinition& InType, TOptional<FNiagaraParameterPanelCategory> Category);

	virtual bool GetCanAddParametersToCategory(FNiagaraParameterPanelCategory Category) const = 0;

	virtual void DeleteParameters(const TArray<FNiagaraParameterPanelItem>& ItemsToDelete)  = 0;

	virtual void RenameParameter(const FNiagaraParameterPanelItem& ItemToRename, const FName NewName)  = 0;

	virtual TSharedPtr<SWidget> CreateContextMenuForItems(const TArray<FNiagaraParameterPanelItem>& Items, const TSharedPtr<FUICommandList>& ToolkitCommands) = 0;

	virtual FNiagaraParameterUtilities::EParameterContext GetParameterContext() const = 0;

	virtual TArray<FNiagaraVariable> GetEditableStaticSwitchParameters() const = 0;

	virtual TArray<FNiagaraParameterPanelItem> GetViewedParameterItems() const = 0;

	virtual const TArray<FNiagaraParameterPanelCategory>& GetDefaultCategories() const = 0;

	virtual FMenuAndSearchBoxWidgets GetParameterMenu(FNiagaraParameterPanelCategory Category)  = 0;

	virtual FReply HandleDragDropOperation(TSharedPtr<FDragDropOperation> DropOperation)  = 0;

	virtual bool GetSectionEnabled(FText Section) const { return true; }

	virtual bool GetCanHandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation) const = 0;

	virtual bool GetCanSetParameterNamespaceAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NewNamespace, FText& OutCanSetParameterNamespaceToolTip) const = 0;

	virtual bool GetCanSetParameterNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NamespaceModifier, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const = 0;

	virtual bool GetCanSetParameterCustomNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const = 0;
	//~ End Pure Virtual Methods

	virtual bool GetCanDeleteParameterAndToolTip(const FNiagaraParameterPanelItem& ItemToDelete, FText& OutCanDeleteParameterToolTip) const;

	virtual void ChangeParameterType(const TArray<FNiagaraParameterPanelItem> ItemsToModify, const FNiagaraTypeDefinition NewType) ;
	virtual bool GetCanChangeParameterType(const TArray<FNiagaraParameterPanelItem>& ItemToChange, FText& OutTooltip) const;
	virtual void GetChangeTypeSubMenu(FMenuBuilder& MenuBuilder, TArray<FNiagaraParameterPanelItem> Item) ;

	virtual void PasteParameterMetaData(const TArray<FNiagaraParameterPanelItem> SelectedItems);
	virtual bool GetCanPasteParameterMetaDataAndToolTip(FText& OutCanPasteToolTip);

	virtual bool GetAllowMultiSelect() const { return false; }

	virtual void DuplicateParameters(const TArray<FNiagaraParameterPanelItem> ItemToDuplicate) ;
	virtual bool GetCanDuplicateParameterAndToolTip(const TArray<FNiagaraParameterPanelItem>& ItemsToDuplicate, FText& OutCanDuplicateParameterToolTip) const;
	bool GetCanDebugParameters(const TArray<FNiagaraParameterPanelItem>& ItemsToDebug) const;

	virtual bool GetCanRenameParameterAndToolTip(const FNiagaraParameterPanelItem& ItemToRename, const FText& NewVariableNameText, bool bCheckEmptyNameText, FText& OutCanRenameParameterToolTip) const;

	virtual bool GetCanSubscribeParameterToLibraryAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const bool bSubscribing, FText& OutCanSubscribeParameterToolTip) const;

	virtual void SetParameterNamespace(const FNiagaraParameterPanelItem ItemToModify, FNiagaraNamespaceMetadata NewNamespaceMetaData, bool bDuplicateParameter) ;

	virtual void SetParameterNamespaceModifier(const FNiagaraParameterPanelItem ItemToModify, const FName NewNamespaceModifier, bool bDuplicateParameter) ;

	virtual void SetParameterCustomNamespaceModifier(const FNiagaraParameterPanelItem ItemToModify, bool bDuplicateParameter) ;

	virtual void GetChangeNamespaceSubMenu(FMenuBuilder& MenuBuilder, bool bDuplicateParameter, FNiagaraParameterPanelItem Item);

	virtual void GetChangeNamespaceModifierSubMenu(FMenuBuilder& MenuBuilder, bool bDuplicateParameter, FNiagaraParameterPanelItem Item) ;

	virtual void OnParameterItemSelected(const FNiagaraParameterPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const {};

	virtual FReply OnParameterItemsDragged(const TArray<FNiagaraParameterPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const { return FReply::Handled(); };

	virtual void OnParameterItemActivated(const FNiagaraParameterPanelItem& ActivatedItem) const;

	const TArray<FNiagaraParameterPanelItem>& GetCachedViewedParameterItems() const;
	
	void SelectParameterItemByName(const FName ParameterName, const bool bRequestRename);
	
	FOnParameterPanelViewModelExternalSelectionChanged& GetOnExternalSelectionChangedDelegate() { return OnParameterPanelViewModelExternalSelectionChangedDelegate; };

	FOnSelectParameterItemByName& GetOnSelectParameterItemByNameDelegate() { return OnSelectParameterItemByNameDelegate; };

	FOnNotifyParameterPendingRename& GetOnNotifyParameterPendingRenameDelegate() { return OnNotifyParameterPendingRenameDelegate; };
	FOnNotifyParameterPendingNamespaceModifierRename& GetOnNotifyParameterPendingNamespaceModifierRenameDelegate() { return OnNotifyParameterPendingNamespaceModifierRenameDelegate; };
	FOnGetParametersWithNamespaceModifierRenamePending& GetParametersWithNamespaceModifierRenamePendingDelegate() { return OnGetParametersWithNamespaceModifierRenamePendingDelegate; };
	FOnInvalidateCachedDependencies& GetOnInvalidateCachedDependencies(){ return OnInvalidateCachedDependenciesDelegate;}

	FText GetActiveSection() const
	{
		return Sections.IsValidIndex(ActiveSectionIndex) ? Sections[ActiveSectionIndex].DisplayName : FText::GetEmpty();
	}
	virtual bool GetShowSections() const { return false; }
	virtual bool GetNamespaceActive(const FName& InNamespace) const { return true; }
	virtual const TArray<FSectionDesc>& GetSections() const { return Sections; }
	virtual FText GetTooltipForSection(FText& InSection) const;
	void SetActiveSection(FText& InSection);
	virtual void SetActiveSection(int32 InSection);
	virtual void SetMainParameterPanelViewModel(TWeakPtr<INiagaraParameterPanelViewModel> InMainParameterPanelViewModel) { MainParameterPanelViewModel = InMainParameterPanelViewModel; }


	virtual bool IsVariableSelected(const FNiagaraVariableBase& InVar) const { return false; };

	virtual TSharedPtr<FNiagaraObjectSelection> GetVariableObjectSelection() { return TSharedPtr<FNiagaraObjectSelection>(); };
	
protected:
	TWeakPtr<INiagaraParameterPanelViewModel> MainParameterPanelViewModel;
	FOnParameterPanelViewModelExternalSelectionChanged OnParameterPanelViewModelExternalSelectionChangedDelegate;
	FOnSelectParameterItemByName OnSelectParameterItemByNameDelegate;
	FOnNotifyParameterPendingRename OnNotifyParameterPendingRenameDelegate;
	FOnNotifyParameterPendingNamespaceModifierRename OnNotifyParameterPendingNamespaceModifierRenameDelegate;
	FOnGetParametersWithNamespaceModifierRenamePending OnGetParametersWithNamespaceModifierRenamePendingDelegate;
    FOnInvalidateCachedDependencies OnInvalidateCachedDependenciesDelegate;

	virtual bool IncludeViewItemsInSelectParameterItem() const { return false; }
	
	void InvalidateCachedDependencies() const { OnInvalidateCachedDependenciesDelegate.Broadcast(); };
	
	/** SharedPtr to menu and searchbox widget retained to prevent the shared ref returned by GetParameterMenu from being invalidated. */
	mutable TSharedPtr<SWidget> ParameterMenuWidget;
	mutable TSharedPtr<SEditableTextBox> ParameterMenuSearchBoxWidget;

	/** Cached maps of parameters sent to SNiagaraParameterPanel, updated whenever GetViewedParameters is called. */
	mutable TArray<FNiagaraParameterPanelItem> CachedViewedItems; //@todo(ng) consider moving to tset in future

	/** Re-entrancy guard for adding parameters. */
	mutable bool bIsAddingParameter = false;

	/** Transient UNiagaraScriptVariables used to pass to new FNiagaraParameterPanelItems when the source FNiagaraVariable is not associated with a UNiagaraScriptVariable in a graph. */
	mutable TMap<FNiagaraVariable, TObjectPtr<UNiagaraScriptVariable>> TransientParameterToScriptVarMap;

	TArray<FSectionDesc> Sections;
	int32 ActiveSectionIndex;

	virtual void OnINiagaraParameterPanelViewModelSelectionChanged(UNiagaraScriptVariable* InVar) {};
};

class FNiagaraSystemToolkitParameterPanelViewModel : public INiagaraParameterPanelViewModel, public TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemToolkitParameterPanelViewModel>
{
public:
	/** Construct a SystemToolkit Parameter Panel View Model from a System View Model and an optional SystemGraphSelectionViewModel. */
	FNiagaraSystemToolkitParameterPanelViewModel(const TSharedPtr<FNiagaraSystemViewModel>& InSystemViewModel);

	void Init(const FSystemToolkitUIContext& InUIContext);

	void Cleanup();

	//~ Begin INiagaraImmutableParameterPanelViewModel interface
	virtual const TArray<UNiagaraGraph*> GetEditableGraphsConst() const override { return GetEditableGraphs(); };

	virtual const TArray<UNiagaraScriptVariable*> GetEditableScriptVariablesWithName(const FName ParameterName) const override;

	virtual const TArray<FNiagaraGraphParameterReference> GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const override;

	virtual const TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions) const override;
	//~ End INiagaraImmutableParameterPanelViewModel interface

	//~ Begin INiagaraParameterPanelViewModel interface
	virtual void AddParameter(FNiagaraVariable NewVariable, const FNiagaraParameterPanelCategory Category, const bool bRequestRename, const bool bMakeUniqueName)  override;

	virtual void FindOrAddParameter(FNiagaraVariable Variable, const FNiagaraParameterPanelCategory Category)  override;
	
	virtual bool GetCanAddParametersToCategory(FNiagaraParameterPanelCategory Category) const override;

	virtual void DeleteParameters(const TArray<FNiagaraParameterPanelItem>& ItemsToDelete)  override;

	virtual void RenameParameter(const FNiagaraParameterPanelItem& ItemToRename, const FName NewName)  override;

	virtual FReply OnParameterItemsDragged(const TArray<FNiagaraParameterPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const override;

	virtual TSharedPtr<SWidget> CreateContextMenuForItems(const TArray<FNiagaraParameterPanelItem>& Items, const TSharedPtr<FUICommandList>& ToolkitCommands) override;
	
	virtual FNiagaraParameterUtilities::EParameterContext GetParameterContext() const override;

	virtual TArray<FNiagaraVariable> GetEditableStaticSwitchParameters() const override;

	virtual TArray<FNiagaraParameterPanelItem> GetViewedParameterItems() const override;

	virtual const TArray<FNiagaraParameterPanelCategory>& GetDefaultCategories() const override;

	virtual FMenuAndSearchBoxWidgets GetParameterMenu(FNiagaraParameterPanelCategory Category)  override;

	virtual FReply HandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation)  override;

	virtual bool GetCanHandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation) const override;

	virtual bool GetCanSetParameterNamespaceAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NewNamespace, FText& OutCanSetParameterNamespaceToolTip) const override;

	virtual bool GetCanSetParameterNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NamespaceModifier, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const override;

	virtual bool GetCanSetParameterCustomNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const override;
	//~ End INiagaraParameterPanelViewModel interface

	TSharedRef<SWidget> CreateAddParameterMenuForAssignmentNode(UNiagaraNodeAssignment* AssignmentNode, const TSharedPtr<SComboButton>& AddButton);

	void DebugParameters(const TArray<FNiagaraParameterPanelItem> ItemToDuplicate) const;

	virtual void OnParameterItemSelected(const FNiagaraParameterPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const override;
	virtual bool IsVariableSelected(const FNiagaraVariableBase& InVar) const override;
	virtual void RefreshDueToActiveDocumentChanged() override;

	virtual bool GetSectionEnabled(FText Section) const override;

	virtual bool GetShowSections() const override { return true; }
	virtual bool GetNamespaceActive(const FName& InNamespace) const override { return true; }

	virtual TSharedPtr<FNiagaraObjectSelection> GetVariableObjectSelection() override { return VariableObjectSelection; };

	virtual void SetActiveSection(int32 InSection) override;

	virtual bool IsCategoryExpandedByDefault(const FNiagaraParameterPanelCategory& Category) const override;
	virtual bool UsesCategoryFilteringForInitialExpansion() const override { return true; }

	virtual void UpdateCategoryExpansionState(const FNiagaraParameterPanelCategory& Category, bool bIsExpanded) override;
protected:
	virtual void OnINiagaraParameterPanelViewModelSelectionChanged(UNiagaraScriptVariable* InVar) override;
	virtual bool IncludeViewItemsInSelectParameterItem() const { return true; }

private:

	bool  ShouldRouteThroughScratchParameterMap(const FNiagaraParameterPanelCategory* Category = nullptr, const FNiagaraVariableBase* Variable = nullptr);

	const TArray<UNiagaraGraph*> GetAllGraphsConst() const;

	TArray<UNiagaraGraph*> GetEditableGraphs() const;

	TArray<TWeakObjectPtr<UNiagaraGraph>> GetEditableEmitterScriptGraphs() const;

	TArray<FNiagaraEmitterHandle*> GetEditableEmitterHandles() const;

	void AddScriptVariable(const UNiagaraScriptVariable* NewScriptVar) ;

	void AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const;

	void RemoveParameterDefinitions(const FGuid& ParameterDefinitionsToRemoveId) const;

	void OnGraphChanged(const struct FEdGraphEditAction& InAction) const;

	void OnParameterRenamedExternally(const FNiagaraVariableBase& InOldVar, const FNiagaraVariableBase& InNewVar, UNiagaraEmitter* InOptionalEmitter);
	void OnParameterRemovedExternally(const FNiagaraVariableBase& InOldVar, UNiagaraEmitter* InOptionalEmitter); 

	void ReconcileOnGraphChangedBindings();

private:
	// Graphs viewed to gather UNiagaraScriptVariables that are displayed by the Parameter Panel.
	TWeakObjectPtr<UNiagaraGraph> SystemScriptGraph;

	mutable FNiagaraVariable SelectedVariable;

	TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
	TWeakPtr<FNiagaraSystemGraphSelectionViewModel> SystemGraphSelectionViewModelWeak;

	FDelegateHandle UserParameterStoreChangedHandle;
	TMap<uint32, FDelegateHandle> GraphIdToOnGraphChangedHandleMap;

	mutable FSystemToolkitUIContext UIContext;

	mutable TArray<FNiagaraParameterPanelCategory> CachedCurrentCategories;

	static TArray<FNiagaraParameterPanelCategory> DefaultCategories;
	static TArray<FNiagaraParameterPanelCategory> UserCategories;
	static TArray<FNiagaraParameterPanelCategory> DefaultAdvancedCategories;
	static TArray<FNiagaraParameterPanelCategory> DefaultScriptCategories;
	static TArray<FNiagaraParameterPanelCategory> DefaultAdvancedScriptCategories;

	TSharedPtr<FNiagaraObjectSelection> VariableObjectSelection;
	int32 ActiveScriptIdx = -1;
	int32 ActiveSystemIdx = -1;
	int32 LastActiveSystemSectionIdx = -1;

	TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemToolkitParameterPanelViewModel>::Handle RegisteredHandle;
};

class FNiagaraScriptToolkitParameterPanelViewModel : public INiagaraParameterPanelViewModel, public TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptToolkitParameterPanelViewModel>
{
public:
	/** Construct a ScriptToolkit Parameter Panel View Model from a Script View Model. */
	NIAGARAEDITOR_API FNiagaraScriptToolkitParameterPanelViewModel(TSharedPtr<FNiagaraScriptViewModel> InScriptViewModel);

	void Init(const FScriptToolkitUIContext& InUIContext);

	void Cleanup();

	//~ Begin INiagaraImmutableParameterPanelViewModel interface
	virtual const TArray<UNiagaraGraph*> GetEditableGraphsConst() const override { return GetEditableGraphs(); };

	virtual const TArray<UNiagaraScriptVariable*> GetEditableScriptVariablesWithName(const FName ParameterName) const override;

	virtual const TArray<FNiagaraGraphParameterReference> GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const override;

	virtual const TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions) const override;
	//~ End INiagaraImmutableParameterPanelViewModel interface

	//~ Begin INiagaraParameterPanelViewModel interface
	virtual void AddParameter(FNiagaraVariable NewVariable, const FNiagaraParameterPanelCategory Category, const bool bRequestRename, const bool bMakeUniqueName)  override;

	virtual void FindOrAddParameter(FNiagaraVariable Variable, const FNiagaraParameterPanelCategory Category)  override;

	virtual bool GetCanAddParametersToCategory(FNiagaraParameterPanelCategory Category) const override;

	virtual void DeleteParameters(const TArray<FNiagaraParameterPanelItem>& ItemToDelete)  override;

	virtual void RenameParameter(const FNiagaraParameterPanelItem& ItemToRename, const FName NewName)  override;
	
	virtual void ChangeParameterType(const TArray<FNiagaraParameterPanelItem> ItemsToModify, const FNiagaraTypeDefinition NewType)  override;

	virtual void DuplicateParameters(const TArray<FNiagaraParameterPanelItem> ItemsToDuplicate)  override;

	virtual void OnParameterItemSelected(const FNiagaraParameterPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const override;

	virtual FReply OnParameterItemsDragged(const TArray<FNiagaraParameterPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const override;

	virtual TSharedPtr<SWidget> CreateContextMenuForItems(const TArray<FNiagaraParameterPanelItem>& Items, const TSharedPtr<FUICommandList>& ToolkitCommands) override;

	virtual FNiagaraParameterUtilities::EParameterContext GetParameterContext() const override;

	virtual TArray<FNiagaraVariable> GetEditableStaticSwitchParameters() const override;

	virtual TArray<FNiagaraParameterPanelItem> GetViewedParameterItems() const override;

	virtual const TArray<FNiagaraParameterPanelCategory>& GetDefaultCategories() const override;

	virtual bool GetAllowMultiSelect() const override { return true; }


	virtual FMenuAndSearchBoxWidgets GetParameterMenu(FNiagaraParameterPanelCategory Category)  override;

	virtual FReply HandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation)  override;

	virtual bool GetCanHandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation) const override;

	virtual bool GetCanSetParameterNamespaceAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NewNamespace, FText& OutCanSetParameterNamespaceToolTip) const override;

	virtual bool GetCanSetParameterNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NamespaceModifier, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const override;

	virtual bool GetCanSetParameterCustomNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const override;
	//~ End INiagaraParameterPanelViewModel interface

	void RenameParameter(const UNiagaraScriptVariable* ScriptVarToRename, const FName NewName) ;

	void RenameParameter(const FNiagaraVariable& VariableToRename, const FName NewName) ;

private:
	void SetParameterIsOverridingLibraryDefaultValue(const FNiagaraParameterPanelItem ItemToModify, const bool bOverriding) ;

	TArray<UNiagaraGraph*> GetEditableGraphs() const;

	void AddScriptVariable(const UNiagaraScriptVariable* NewScriptVar) ;

	void AddParameterDefinitions(UNiagaraParameterDefinitions* NewParameterDefinitions) const;

	void RemoveParameterDefinitions(const FGuid& ParameterDefinitionsToRemoveId) const;

	void OnGraphChanged(const struct FEdGraphEditAction& InAction) const;

	void OnGraphSubObjectSelectionChanged(const UObject* Obj) const;

private:
	TSharedPtr<FNiagaraScriptViewModel> ScriptViewModel;
	mutable FScriptToolkitUIContext UIContext;

	FDelegateHandle OnGraphChangedHandle;
	FDelegateHandle OnGraphNeedsRecompileHandle;
	FDelegateHandle OnSubObjectSelectionHandle;

	TSharedPtr<FNiagaraObjectSelection> VariableObjectSelection;

	mutable TArray<FNiagaraParameterPanelCategory> CachedCurrentCategories;

	static TArray<FNiagaraParameterPanelCategory> DefaultCategories;
	static TArray<FNiagaraParameterPanelCategory> DefaultAdvancedCategories;

	TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptToolkitParameterPanelViewModel>::Handle RegisteredHandle;
};

class FNiagaraParameterDefinitionsToolkitParameterPanelViewModel : public INiagaraParameterPanelViewModel
{
public:
	/** Construct a ParameterDefinitionsToolkit Parameter Panel View Model from a Parameter Definitions. */
	FNiagaraParameterDefinitionsToolkitParameterPanelViewModel(UNiagaraParameterDefinitions* InParameterDefinitions, const TSharedPtr<FNiagaraObjectSelection>& InObjectSelection);

	void Init(const FParameterDefinitionsToolkitUIContext& InUIContext);

	void Cleanup();

	//~ Begin INiagaraImmutableParameterPanelViewModel interface
	//NOTE: The ParameterDefinitionsToolkitParameterPanelViewModel does not edit any graphs, so return an empty array.
	virtual const TArray<UNiagaraGraph*> GetEditableGraphsConst() const override { return TArray<UNiagaraGraph*>(); };

	virtual const TArray<UNiagaraScriptVariable*> GetEditableScriptVariablesWithName(const FName ParameterName) const override;

	virtual const TArray<FNiagaraGraphParameterReference> GetGraphParameterReferencesForItem(const FNiagaraParameterPanelItemBase& Item) const override;

	virtual const TArray<UNiagaraParameterDefinitions*> GetAvailableParameterDefinitions(bool bSkipSubscribedParameterDefinitions) const override;
	//~ End INiagaraImmutableParameterPanelViewModel interface

	//~ Begin INiagaraParameterPanelViewModel interface
	virtual void AddParameter(FNiagaraVariable NewVariable, const FNiagaraParameterPanelCategory Category, const bool bRequestRename, const bool bMakeUniqueName)  override;

	virtual void FindOrAddParameter(FNiagaraVariable Variable, const FNiagaraParameterPanelCategory Category)  override;

	virtual bool GetCanAddParametersToCategory(FNiagaraParameterPanelCategory Category) const override;

	virtual void DeleteParameters(const TArray<FNiagaraParameterPanelItem>& ItemToDelete)  override;

	virtual void RenameParameter(const FNiagaraParameterPanelItem& ItemToRename, const FName NewName)  override;

	virtual TSharedPtr<SWidget> CreateContextMenuForItems(const TArray<FNiagaraParameterPanelItem>& Items, const TSharedPtr<FUICommandList>& ToolkitCommands) override;

	virtual FNiagaraParameterUtilities::EParameterContext GetParameterContext() const override;

	virtual TArray<FNiagaraVariable> GetEditableStaticSwitchParameters() const override;

	virtual TArray<FNiagaraParameterPanelItem> GetViewedParameterItems() const override;

	virtual const TArray<FNiagaraParameterPanelCategory>& GetDefaultCategories() const override;

	virtual FMenuAndSearchBoxWidgets GetParameterMenu(FNiagaraParameterPanelCategory Category)  override;

	virtual FReply HandleDragDropOperation(TSharedPtr<FDragDropOperation> DropOperation)  override;

	virtual bool GetCanHandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation) const override;

	virtual void OnParameterItemSelected(const FNiagaraParameterPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const override;

	virtual bool GetCanRenameParameterAndToolTip(const FNiagaraParameterPanelItem& ItemToRename, const FText& NewVariableNameText, bool bCheckEmptyNameText, FText& OutCanRenameParameterToolTip) const override;
	
	virtual bool GetCanSetParameterNamespaceAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NewNamespace, FText& OutCanSetParameterNamespaceToolTip) const override;

	virtual bool GetCanSetParameterNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, const FName NamespaceModifier, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const override;

	virtual bool GetCanSetParameterCustomNamespaceModifierAndToolTip(const FNiagaraParameterPanelItem& ItemToModify, bool bDuplicateParameter, FText& OutCanSetParameterNamespaceModifierToolTip) const override;
	//~ End INiagaraParameterPanelViewModel interface

private:
	TWeakObjectPtr<UNiagaraParameterDefinitions> ParameterDefinitionsWeak;
	mutable FParameterDefinitionsToolkitUIContext UIContext;

	TSharedPtr<FNiagaraObjectSelection> VariableObjectSelection;

	static TArray<FNiagaraParameterPanelCategory> DefaultCategories;
};
