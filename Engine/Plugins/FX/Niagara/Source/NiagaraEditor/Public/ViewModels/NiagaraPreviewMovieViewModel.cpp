// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraPreviewMovieViewModel.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "UObject/UObjectGlobals.h"
#include "FileMediaSource.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraTrace.h"

FNiagaraPreviewMovieViewModel::FNiagaraPreviewMovieViewModel()
{
	MediaPlayer.Reset(NewObject<UMediaPlayer>(GetTransientPackage(), UMediaPlayer::StaticClass()));
	MediaTexture.Reset(NewObject<UMediaTexture>(GetTransientPackage()));
	MediaPlayer->SetLooping(true);
	MediaPlayer->AffectedByPIEHandling = false;
	MediaTexture->SetMediaPlayer(MediaPlayer.Get());
	
	const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>();
	ImageBrush = MakeUnique<FSlateImageBrush>(MediaTexture.Get(), EditorSettings->TooltipPreviewMovieSize);
}

FNiagaraPreviewMovieViewModel::~FNiagaraPreviewMovieViewModel()
{
	Shutdown();
}

void FNiagaraPreviewMovieViewModel::Initialize()
{
	if(const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>())
	{
		EditorSettings->OnSettingsChanged().AddSP(this, &FNiagaraPreviewMovieViewModel::OnEditorSettingsChanged);
	}
}

void FNiagaraPreviewMovieViewModel::Shutdown()
{
	if(const UNiagaraEditorSettings* EditorSettings = GetDefault<UNiagaraEditorSettings>())
	{
		EditorSettings->OnSettingsChanged().RemoveAll(this);
	}
}

bool FNiagaraPreviewMovieViewModel::IsPlayingPreviewMovieForAsset(const FAssetData& AssetData) const
{
	return IsPlayingPreviewMovie(FNiagaraEditorUtilities::Preview::GetPreviewMovieObjectPath(AssetData).Get(FSoftObjectPath()));
}

bool FNiagaraPreviewMovieViewModel::IsPlayingPreviewMovie(const FSoftObjectPath& InPreviewMoviePath) const
{
	return InPreviewMoviePath == CurrentPreviewMoviePath;
}

bool FNiagaraPreviewMovieViewModel::PlayMovieForAsset(const FAssetData& AssetData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(FNiagaraMoviePreviewViewModel::PlayMovieForAsset, NiagaraChannel)
	
	// If we are already playing this, return
	if(IsPlayingPreviewMovieForAsset(AssetData))
	{
		return false;
	}

	TOptional<FSoftObjectPath> PreviewMoviePath = FNiagaraEditorUtilities::Preview::GetPreviewMovieObjectPath(AssetData);

	if(PreviewMoviePath.IsSet() == false)
	{
		return false;
	}

	return PlayMovie(PreviewMoviePath.GetValue());
}

bool FNiagaraPreviewMovieViewModel::PlayMovie(const FSoftObjectPath& InPreviewMoviePath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(FNiagaraMoviePreviewViewModel::PlayMovie, NiagaraChannel)

	if(IsPlayingPreviewMovie(InPreviewMoviePath))
	{
		return false;
	}
	
	// Even if this fails, we update the current preview movie path so that further tries with the same path won't drain perf unnecessarily
	UFileMediaSource* MediaSource = Cast<UFileMediaSource>(InPreviewMoviePath.TryLoad());

	CurrentPreviewMoviePath = InPreviewMoviePath;
	
	if(MediaSource == nullptr)
	{
		return false;
	}
	
	MediaPlayer->OpenSource(MediaSource);
	MediaTexture->UpdateResource();
	return MediaPlayer->Play();
}

void FNiagaraPreviewMovieViewModel::OnEditorSettingsChanged(const FString& PropertyName, const UNiagaraEditorSettings* NiagaraEditorSettings)
{
	if(PropertyName == GET_MEMBER_NAME_CHECKED(UNiagaraEditorSettings, TooltipPreviewMovieSize))
	{
		ImageBrush->SetImageSize(NiagaraEditorSettings->TooltipPreviewMovieSize);
	}
}
