// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "MediaPlayer.h"
#include "MediaTexture.h"
#include "NiagaraEditorSettings.h"
#include "Brushes/SlateImageBrush.h"
#include "Templates/UniquePtr.h"
#include "AssetRegistry/AssetData.h"

class FNiagaraSystemViewModel;

class FNiagaraPreviewMovieViewModel : public TSharedFromThis<FNiagaraPreviewMovieViewModel>
{
public:
	FNiagaraPreviewMovieViewModel();
	~FNiagaraPreviewMovieViewModel();

	void Initialize();
	void Shutdown();
	
	bool IsPlayingPreviewMovieForAsset(const FAssetData& AssetData) const;
	bool IsPlayingPreviewMovie(const FSoftObjectPath& InPreviewMoviePath) const;
	bool PlayMovieForAsset(const FAssetData& AssetData);
	bool PlayMovie(const FSoftObjectPath& InPreviewMoviePath);
	const FSlateImageBrush* GetMoviePreviewImageBrush() const { return ImageBrush.Get(); }

protected:
	void OnEditorSettingsChanged(const FString& PropertyName, const UNiagaraEditorSettings* NiagaraEditorSettings);

private:
	TStrongObjectPtr<UMediaPlayer> MediaPlayer;
	TStrongObjectPtr<UMediaTexture> MediaTexture;
	TUniquePtr<FSlateImageBrush> ImageBrush;
	FSoftObjectPath CurrentPreviewMoviePath;
};
