// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/NiagaraScriptViewModel.h"
#include "GraphEditAction.h"
#include "HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"
#include "HierarchyEditor/NiagaraSummaryViewViewModel.h"

class INiagaraParameterPanelViewModel;
class FNiagaraScriptToolkitParameterPanelViewModel;
class FUICommandList;


class FNiagaraScratchPadScriptViewModel : public FNiagaraScriptViewModel, public FGCObject
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnRenamed);
	DECLARE_MULTICAST_DELEGATE(FOnPinnedChanged);
	DECLARE_MULTICAST_DELEGATE(FOnHasUnappliedChangesChanged);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnChangesApplied, TSharedRef<FNiagaraScratchPadScriptViewModel> /* The applied view model */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodeIDFocusRequested, FNiagaraScriptIDAndGraphFocusInfo*  /* FocusInfo */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPinIDFocusRequested, FNiagaraScriptIDAndGraphFocusInfo*  /* FocusInfo */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGraphSubObjectSelectionChanged, const UObject*);

	NIAGARAEDITOR_API FNiagaraScratchPadScriptViewModel(bool bInIsForDataProcessingOnly);

	NIAGARAEDITOR_API ~FNiagaraScratchPadScriptViewModel();

	void Initialize(UNiagaraScript* Script, UNiagaraScript* InEditScript, TWeakPtr<class FNiagaraSystemViewModel> InSystemViewModel);

	NIAGARAEDITOR_API TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter() const override;

	NIAGARAEDITOR_API bool IsValid() const;
	
	NIAGARAEDITOR_API void TransferFromOldWhenDoingApply(TSharedPtr< FNiagaraScratchPadScriptViewModel> InOldScriptVM);

	NIAGARAEDITOR_API void Finalize();

	//~ Begin FGCObject
	NIAGARAEDITOR_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	//~ End FGCObject

	NIAGARAEDITOR_API TArray<UNiagaraGraph*> GetEditableGraphs();

	NIAGARAEDITOR_API UNiagaraScriptParametersHierarchyViewModel* GetHierarchyViewModel();

	//~ Begin NiagaraParameterDefinitionsSubscriberViewModel Interface
protected:
	virtual INiagaraParameterDefinitionsSubscriber* GetParameterDefinitionsSubscriber() override { return &EditScript; };
	//~ End NiagaraParameterDefinitionsSubscriberViewModel Interface

	NIAGARAEDITOR_API void OnGraphSubObjectSelectionChanged(const UObject* Obj);

	FDelegateHandle ExternalSelectionChangedDelegate;

public:
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraScratchPadScriptViewModel");
	}

	NIAGARAEDITOR_API UNiagaraScript* GetOriginalScript() const;

	NIAGARAEDITOR_API const FVersionedNiagaraScript& GetEditScript() const;

	NIAGARAEDITOR_API TSharedPtr<INiagaraParameterPanelViewModel> GetParameterPanelViewModel() const;

	NIAGARAEDITOR_API TSharedPtr<FUICommandList> GetParameterPanelCommands() const;

	NIAGARAEDITOR_API FText GetToolTip() const;

	NIAGARAEDITOR_API bool GetIsPendingRename() const;

	NIAGARAEDITOR_API void SetIsPendingRename(bool bInIsPendingRename);

	NIAGARAEDITOR_API void SetScriptName(FText InScriptName);

	NIAGARAEDITOR_API bool GetIsPinned() const;

	NIAGARAEDITOR_API void SetIsPinned(bool bInIsPinned);

	NIAGARAEDITOR_API float GetEditorHeight() const;

	NIAGARAEDITOR_API void SetEditorHeight(float InEditorHeight);

	NIAGARAEDITOR_API bool HasUnappliedChanges() const;

	NIAGARAEDITOR_API void ApplyChanges();
	NIAGARAEDITOR_API void ApplyChangesAndSave();

	NIAGARAEDITOR_API void DiscardChanges();

	NIAGARAEDITOR_API FOnRenamed& OnRenamed();

	NIAGARAEDITOR_API FOnPinnedChanged& OnPinnedChanged();

	NIAGARAEDITOR_API FOnHasUnappliedChangesChanged& OnHasUnappliedChangesChanged();

	NIAGARAEDITOR_API FOnChangesApplied& OnChangesApplied();

	NIAGARAEDITOR_API FSimpleDelegate& OnRequestDiscardChanges();

	NIAGARAEDITOR_API void OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload);

	NIAGARAEDITOR_API FOnNodeIDFocusRequested& OnNodeIDFocusRequested();
	NIAGARAEDITOR_API FOnPinIDFocusRequested& OnPinIDFocusRequested();

	FOnGraphSubObjectSelectionChanged& OnGraphSelectionChanged() { return OnGraphSubObjectSelectionChangedDelegate;	};

	NIAGARAEDITOR_API void RaisePinFocusRequested(FNiagaraScriptIDAndGraphFocusInfo*  InFocusInfo);
	NIAGARAEDITOR_API void RaiseNodeFocusRequested(FNiagaraScriptIDAndGraphFocusInfo* InFocusInfo);

private:
	NIAGARAEDITOR_API FText GetDisplayNameInternal() const;

	NIAGARAEDITOR_API void OnScriptGraphChanged(const FEdGraphEditAction &Action);

	NIAGARAEDITOR_API void OnScriptPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent);

	FGuid GetScriptChangeID() const;

	bool bIsPendingRename;
	bool bIsPinned;
	float EditorHeight;

	UNiagaraScript* OriginalScript = nullptr;
	FVersionedNiagaraScript EditScript;

	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;

	bool bHasPendingChanges;
	FGuid LastAppliedChangeID;

	TSharedPtr<FUICommandList> ParameterPanelCommands;
	TSharedPtr<FNiagaraScriptToolkitParameterPanelViewModel> ParameterPaneViewModel;

	TObjectPtr<UNiagaraScriptParametersHierarchyViewModel> ScriptHierarchyViewModel;
	
	FDelegateHandle OnGraphNeedsRecompileHandle;

	FOnRenamed OnRenamedDelegate;
	FOnPinnedChanged OnPinnedChangedDelegate;
	FOnHasUnappliedChangesChanged OnHasUnappliedChangesChangedDelegate;
	FOnChangesApplied OnChangesAppliedDelegate;
	FSimpleDelegate	OnRequestDiscardChangesDelegate;
	FOnGraphSubObjectSelectionChanged OnGraphSubObjectSelectionChangedDelegate;

	FOnNodeIDFocusRequested OnNodeIDFocusRequestedDelegate;
	FOnPinIDFocusRequested OnPinIDFocusRequestedDelegate;
};
