// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraScratchPadViewModel.generated.h"

class FNiagaraSystemViewModel;
class FNiagaraScratchPadScriptViewModel;
class FNiagaraObjectSelection;
class UNiagaraScript;

UCLASS(MinimalAPI)
class UNiagaraScratchPadViewModel : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FOnScriptViewModelsChanged);
	DECLARE_MULTICAST_DELEGATE(FOnActiveScriptChanged);
	DECLARE_MULTICAST_DELEGATE(FOnScriptRenamed);
	DECLARE_MULTICAST_DELEGATE(FOnScriptDeleted);

public:
	NIAGARAEDITOR_API void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

	NIAGARAEDITOR_API void Finalize();

	NIAGARAEDITOR_API void RefreshScriptViewModels();

	NIAGARAEDITOR_API void ApplyScratchPadChanges();
	
	NIAGARAEDITOR_API const TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>>& GetScriptViewModels() const;

	NIAGARAEDITOR_API const TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>>& GetEditScriptViewModels() const;

	NIAGARAEDITOR_API TSharedPtr<FNiagaraScratchPadScriptViewModel> GetViewModelForScript(UNiagaraScript* InScript);
	NIAGARAEDITOR_API TSharedPtr<FNiagaraScratchPadScriptViewModel> GetViewModelForScript(FName InScriptName);

	NIAGARAEDITOR_API TSharedPtr<FNiagaraScratchPadScriptViewModel> GetViewModelForEditScript(UNiagaraScript* InEditScript);

	NIAGARAEDITOR_API const TArray<ENiagaraScriptUsage>& GetAvailableUsages() const;

	NIAGARAEDITOR_API FText GetDisplayNameForUsage(ENiagaraScriptUsage InUsage) const;

	NIAGARAEDITOR_API TSharedRef<FNiagaraObjectSelection> GetObjectSelection();

	NIAGARAEDITOR_API TSharedPtr<FNiagaraScratchPadScriptViewModel> GetActiveScriptViewModel();

	NIAGARAEDITOR_API void SetActiveScriptViewModel(TSharedRef<FNiagaraScratchPadScriptViewModel> InActiveScriptViewModel);

	NIAGARAEDITOR_API void FocusScratchPadScriptViewModel(TSharedRef<FNiagaraScratchPadScriptViewModel> InScriptViewModel);

	NIAGARAEDITOR_API void ResetActiveScriptViewModel();

	NIAGARAEDITOR_API void CopyActiveScript();

	NIAGARAEDITOR_API bool CanPasteScript() const;

	NIAGARAEDITOR_API void PasteScript();

	NIAGARAEDITOR_API void DeleteActiveScript();

	NIAGARAEDITOR_API TSharedPtr<FNiagaraScratchPadScriptViewModel> CreateNewScript(ENiagaraScriptUsage InScriptUsage, ENiagaraScriptUsage InTargetSupportedUsage, FNiagaraTypeDefinition InOutputType);

	NIAGARAEDITOR_API TSharedPtr<FNiagaraScratchPadScriptViewModel> CreateNewScriptAsDuplicate(const UNiagaraScript* ScriptToDuplicate);

	NIAGARAEDITOR_API void CreateAssetFromActiveScript();

	NIAGARAEDITOR_API bool CanSelectNextUsageForActiveScript();

	NIAGARAEDITOR_API void SelectNextUsageForActiveScript();

	NIAGARAEDITOR_API bool HasUnappliedChanges() const;

	NIAGARAEDITOR_API FOnScriptViewModelsChanged& OnScriptViewModelsChanged();

	NIAGARAEDITOR_API FOnScriptViewModelsChanged& OnEditScriptViewModelsChanged();

	NIAGARAEDITOR_API FOnActiveScriptChanged& OnActiveScriptChanged();

	NIAGARAEDITOR_API FOnScriptRenamed& OnScriptRenamed();

	NIAGARAEDITOR_API FOnScriptDeleted& OnScriptDeleted();

	NIAGARAEDITOR_API void OpenEditorForActive();

private:
	TSharedRef<FNiagaraSystemViewModel> GetSystemViewModel();

	TSharedRef<FNiagaraScratchPadScriptViewModel> CreateAndSetupScriptviewModel(UNiagaraScript* ScratchPadScript, UNiagaraScript* ScratchPadEditScript);

	void TearDownScriptViewModel(TSharedRef<FNiagaraScratchPadScriptViewModel> InScriptViewModel);

	void ResetActiveScriptViewModelInternal(bool bRefreshEditScriptViewModels);

	void RefreshEditScriptViewModels();

	void ScriptGraphNodeSelectionChanged(TWeakPtr<FNiagaraScratchPadScriptViewModel> InScriptViewModelWeak);

	void ScriptViewModelScriptRenamed(TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak);

	void ScriptViewModelPinnedChanged(TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak);

	void ScriptViewModelHasUnappliedChangesChanged();

	void ScriptViewModelChangesApplied(TSharedRef<FNiagaraScratchPadScriptViewModel> ViewModel);

	void ScriptViewModelRequestDiscardChanges(TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak);

	void ScriptViewModelVariableSelectionChanged(TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak);

	void ScriptViewModelGraphSelectionChanged(const UObject* Obj, TWeakPtr<FNiagaraScratchPadScriptViewModel> ScriptViewModelWeak);


private:
	TSharedPtr<FNiagaraObjectSelection> ObjectSelection;

	TSharedPtr<FNiagaraScratchPadScriptViewModel> ActiveScriptViewModel;

	TWeakPtr<FNiagaraSystemViewModel> SystemViewModelWeak;

	TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>> ScriptViewModels;

	TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>> PinnedScriptViewModels;

	TArray<TSharedRef<FNiagaraScratchPadScriptViewModel>> EditScriptViewModels;

	TArray<ENiagaraScriptUsage> AvailableUsages;

	mutable TOptional<bool> bHasUnappliedChangesCache;

	FOnScriptViewModelsChanged OnScriptViewModelsChangedDelegate;

	FOnScriptViewModelsChanged OnEditScriptViewModelsChangedDelegate;

	FOnActiveScriptChanged OnActiveScriptChangedDelegate;

	FOnScriptRenamed OnScriptRenamedDelegate;

	FOnScriptDeleted OnScriptDeletedDelegate;
};
