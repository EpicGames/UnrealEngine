// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/NiagaraParameterEditMode.h"
#include "INiagaraCompiler.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "EditorUndoClient.h"
#include "NiagaraParameterDefinitionsSubscriberViewModel.h"

class IAssetReferenceFilter;
class UNiagaraScript;
class UNiagaraScriptSource;
class UNiagaraScriptVariable;
class UNiagaraParameterDefinitions;
class INiagaraParameterCollectionViewModel;
class FNiagaraScriptGraphViewModel;
class FNiagaraScriptInputCollectionViewModel;
class FNiagaraScriptOutputCollectionViewModel;
class FNiagaraMetaDataCollectionViewModel;
class UNiagaraEmitter;
class FNiagaraObjectSelection;

template<typename ObjectType> class TAttribute;

/** A view model for Niagara scripts which manages other script related view models. */
class FNiagaraScriptViewModel
	: public TSharedFromThis<FNiagaraScriptViewModel>
	, public FEditorUndoClient
	, public TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>
	, public INiagaraParameterDefinitionsSubscriberViewModel
{
public:
	FNiagaraScriptViewModel(TAttribute<FText> DisplayName, ENiagaraParameterEditMode InParameterEditMode, bool bInIsForDataProcessingOnly);

	virtual ~FNiagaraScriptViewModel() override;

	//~ Begin NiagaraParameterDefinitionsSubscriberViewModel Interface
protected:
	virtual INiagaraParameterDefinitionsSubscriber* GetParameterDefinitionsSubscriber() override;
	//~ End NiagaraParameterDefinitionsSubscriberViewModel Interface

public:
	NIAGARAEDITOR_API FText GetDisplayName() const;

	NIAGARAEDITOR_API const TArray<FVersionedNiagaraScriptWeakPtr>& GetScripts() const;

	/** Sets the view model to a different script. */
	void SetScript(FVersionedNiagaraScript InScript);

	void SetScripts(FVersionedNiagaraEmitter InEmitter);

	virtual TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter() const { return {}; }

	/** Gets the view model for the input parameter collection. */
	TSharedRef<FNiagaraScriptInputCollectionViewModel> GetInputCollectionViewModel();
	
	/** Gets the view model for the output parameter collection. */
	TSharedRef<FNiagaraScriptOutputCollectionViewModel> GetOutputCollectionViewModel();

	/** Gets the view model for the graph. */
	NIAGARAEDITOR_API TSharedRef<FNiagaraScriptGraphViewModel> GetGraphViewModel();

	/** Gets the currently selected script variables. */
	NIAGARAEDITOR_API TSharedRef<FNiagaraObjectSelection> GetVariableSelection();

	/** Updates the script with the latest compile status. */
	void UpdateCompileStatus(ENiagaraScriptCompileStatus InAggregateCompileStatus, const FString& InAggregateCompileErrors,
		const TArray<ENiagaraScriptCompileStatus>& InCompileStatuses, const TArray<FString>& InCompileErrors, const TArray<FString>& InCompilePaths,
		const TArray<UNiagaraScript*>& InCompileSources);

	/** Compiles a script that isn't part of an emitter or System. */
	void CompileStandaloneScript(bool bForceCompile = false);

	/** Get the latest status of this view-model's script compilation.*/
	NIAGARAEDITOR_API virtual ENiagaraScriptCompileStatus GetLatestCompileStatus(FGuid VersionGuid = FGuid());

	/** Refreshes the nodes in the script graph, updating the pins to match external changes. */
	void RefreshNodes();

	//~ FEditorUndoClient Interface
	NIAGARAEDITOR_API virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

	UNiagaraScript* GetContainerScript(ENiagaraScriptUsage InUsage, FGuid InUsageId = FGuid());
	UNiagaraScript* GetScript(ENiagaraScriptUsage InUsage, FGuid InUsageId = FGuid());

	ENiagaraScriptCompileStatus GetScriptCompileStatus(ENiagaraScriptUsage InUsage, FGuid InUsageId) const;
	FText GetScriptErrors(ENiagaraScriptUsage InUsage, FGuid InUsageId) const;

	/** If this is editing a standalone script, returns the script being edited.*/
	virtual FVersionedNiagaraScript GetStandaloneScript();
	
	bool RenameParameter(const FNiagaraVariable TargetParameter, const FName NewName);

private:
	/** Handles the selection changing in the graph view model. */
	void GraphViewModelSelectedNodesChanged();

	/** Handles the selection changing in the input collection view model. */
	void InputViewModelSelectionChanged();

	/** Marks this script view model as dirty and marks the scripts as needing synchrnozation. */
	void MarkAllDirty(FString Reason);

	void SetScripts(UNiagaraScriptSource* InScriptSource, TArray<FVersionedNiagaraScript>& InScripts);

	/** Handles when values in the input parameter collection changes. */
	void InputParameterValueChanged(TConstArrayView<FName> ParameterName);

	/** Handles when values in the output parameter collection changes. */
	void OutputParameterValueChanged(TConstArrayView<FName> ParameterName);

protected:
	/** The script which provides the data for this view model. */
	TArray<FVersionedNiagaraScriptWeakPtr> Scripts;

	NIAGARAEDITOR_API virtual void OnVMScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion);

	NIAGARAEDITOR_API virtual void OnGPUScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion);

	TWeakObjectPtr<UNiagaraScriptSource> Source;

	/** The view model for the input parameter collection. */
	TSharedRef<FNiagaraScriptInputCollectionViewModel> InputCollectionViewModel;

	/** The view model for the output parameter collection .*/
	TSharedRef<FNiagaraScriptOutputCollectionViewModel> OutputCollectionViewModel;

	/** The view model for the graph. */
	TSharedRef<FNiagaraScriptGraphViewModel> GraphViewModel;

	/** The set of variables currently selected in the graph or the parameters panel. */
	TSharedRef<FNiagaraObjectSelection> VariableSelection;

	/** A flag for preventing reentrancy when synchronizing selection. */
	bool bUpdatingSelectionInternally;

	/** The stored latest compile status.*/
	ENiagaraScriptCompileStatus LastCompileStatus;
	
	/** The handle to the graph changed delegate needed for removing. */
	FDelegateHandle OnGraphChangedHandle;

	TArray<TNiagaraViewModelManager<UNiagaraScript, FNiagaraScriptViewModel>::Handle> RegisteredHandles;

	bool IsGraphDirty(FGuid VersionGuid) const;

	TArray<ENiagaraScriptCompileStatus> CompileStatuses;
	TArray<FString> CompileErrors;
	TArray<FString> CompilePaths;
	TArray<TPair<ENiagaraScriptUsage, FGuid>> CompileTypes;

	/** Whether or not this view model is going to be used for data processing only and will not be shown in the UI. */
	bool bIsForDataProcessingOnly;
};
