// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraSimCache.h"
#include "Widgets/SNiagaraSimCacheOverview.h"

class SNiagaraSimCacheTreeView;
struct FNiagaraSimCacheComponentTreeItem;
struct FNiagaraSimCacheTreeItem;
struct FNiagaraSimCacheOverviewItem;
struct FNiagaraSystemSimCacheCaptureReply;
struct FNiagaraSimCacheDataBuffersLayout;
class UNiagaraComponent;

class FNiagaraSimCacheViewModel : public TSharedFromThis<FNiagaraSimCacheViewModel>, public FGCObject
{
public:
	enum class ESelectionMode
	{
		SystemInstance,
		Emitter,
		DataInterface,
		DebugData,
	};

	struct FComponentInfo
	{
		FName	Name = NAME_None;
		uint32	ComponentOffset = INDEX_NONE;
		bool	bIsFiltered = true;
		bool	bIsFloat = false;
		bool	bIsHalf = false;
		bool	bIsInt32 = false;
		bool	bShowAsBool = false;
		UEnum*	Enum = nullptr;
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnViewDataChanged, bool)
	DECLARE_MULTICAST_DELEGATE(FOnSimCacheChanged)
	DECLARE_MULTICAST_DELEGATE(FOnBufferChanged)
	
	NIAGARAEDITOR_API FNiagaraSimCacheViewModel();
	NIAGARAEDITOR_API virtual ~FNiagaraSimCacheViewModel() override;

private:
	TArrayView<FComponentInfo> GetMutableSelectedComponentInfos();
public:
	NIAGARAEDITOR_API TConstArrayView<FComponentInfo> GetSelectedComponentInfos() const;

	int32 GetNumInstances() const { return NumInstances; }
	NIAGARAEDITOR_API int32 GetNumFrames() const;
	int32 GetFrameIndex() const { return FrameIndex; }
	NIAGARAEDITOR_API void SetFrameIndex(const int32 InFrameIndex);

	NIAGARAEDITOR_API FOnViewDataChanged& OnViewDataChanged();
	NIAGARAEDITOR_API FOnSimCacheChanged& OnSimCacheChanged();
	NIAGARAEDITOR_API FOnBufferChanged& OnBufferChanged();

	void Initialize(TWeakObjectPtr<UNiagaraSimCache> SimCache);

	ESelectionMode GetSelectionMode() const { return SelectionMode; }
	FName GetSelectedEmitter() const { return SelectedEmitterName; };
	FNiagaraVariableBase GetSelectedDataInterface() const { return SelectedDataInterface; };
	FText GetSelectedText() const;

	NIAGARAEDITOR_API const UObject* GetSelectedDataInterfaceStorage() const;

	void SetSelectedSystemInstance();
	void SetSelectedEmitter(FName EmitterName);
	void SetSelectedDataInterface(FNiagaraVariableBase DIVariable);
	void SetSelectedDebugData();

	bool IsComponentFilterActive() const;
	bool IsComponentFiltered(const FString& ComponentNameString) const;
	bool IsComponentFiltered(FName ComponentName) const;
	void SetComponentFiltered(const FString& ComponentNameString, bool bFiltered);
	void ToggleComponentFiltered(const FString& ComponentNameString);
	void SetAllComponentFiltered(bool bFiltered);

	void RefreshSelection();
private:
	void RefreshFromSelectionChanged();

public:
	FText GetComponentText(FName ComponentName, int32 InstanceIndex) const;
	bool CompareComponent(int32 ComponentIndex, int32 LhsInstance, int32 RhsInstance, bool bAscending) const;
	bool IsCacheValid() const;
	int32 GetNumEmitterLayouts() const;
	FName GetEmitterLayoutName(int32 Index) const;

	const UNiagaraSimCacheDebugData* GetCacheDebugData() const;

	// Construct entries for the tree view.
	void BuildEntries(TWeakPtr<SNiagaraSimCacheTreeView> OwningTreeView);
	TArray<TSharedRef<FNiagaraSimCacheTreeItem>>* GetSelectedRootEntries();
	TArray<TSharedRef<FNiagaraSimCacheOverviewItem>>* GetBufferEntries();

	bool CanCopyActiveToClipboard() const;
	void CopyActiveToClipboard() const;

	bool CanExportToDisk() const;
	void ExportToDisk() const;

	bool IsCreateAssetVisible() const;
	void CreateAsset();

	bool CanRemoveDebugData() const;
	void RemoveDebugData();

	UNiagaraComponent* GetPreviewComponent() { return PreviewComponent; }

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraSimCacheViewModel");
	}

private:
	void SetupPreviewComponentAndInstance();
	void BuildComponentInfos(const FName Name, const UScriptStruct* Struct, TArray<FComponentInfo>& ComponentInfos);
	void OnCacheModified(UNiagaraSimCache* SimCache);
	void UpdateCachedFrame();
	void UpdateCurrentEntries();

	void BuildTreeItemChildren(TSharedPtr<FNiagaraSimCacheTreeItem> InTreeItem, TWeakPtr<SNiagaraSimCacheTreeView> OwningTreeView);

	// Build out component infos for each buffer in this cache
	void UpdateComponentInfos();
	
	// The sim cache being viewed
	TObjectPtr<UNiagaraSimCache> SimCache = nullptr;

	// Component for preview scene
	TObjectPtr<UNiagaraComponent> PreviewComponent = nullptr;

	// Which frame of the cached sim is being viewed
	int32 FrameIndex = 0;

	// Handles what is selected currently
	ESelectionMode			SelectionMode = ESelectionMode::SystemInstance;
	FName					SelectedEmitterName;
	FNiagaraVariableBase	SelectedDataInterface;

	// Number of particles in the given frame
	int32 NumInstances = 0;

	// Cached Component infos for this system
	TArray<FComponentInfo>					SystemComponentInfos;
	TMap<FName, TArray<FComponentInfo>>		EmitterComponentInfos;
	// TODO: Component infos for Data Interfaces?

	TArray<TSharedRef<FNiagaraSimCacheTreeItem>> RootEntries;
	TArray<TSharedRef<FNiagaraSimCacheTreeItem>> SelectedRootEntries;

	TArray<TSharedRef<FNiagaraSimCacheOverviewItem>> BufferEntries;

	TArray<float>						FloatComponents;
	TArray<FFloat16>					HalfComponents;
	TArray<int32>						Int32Components;

	int32								FoundFloatComponents = 0;
	int32								FoundHalfComponents = 0;
	int32								FoundInt32Components = 0;

	bool								bDelegatesAdded = false;
	FOnViewDataChanged					OnViewDataChangedDelegate;
	FOnSimCacheChanged					OnSimCacheChangedDelegate;
	FOnBufferChanged					OnBufferChangedDelegate;
};
