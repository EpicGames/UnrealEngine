// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"
#include "Widgets/SNiagaraScriptGraph.h"
#include "NiagaraSystemEditorDocumentsViewModel.generated.h"

class FNiagaraSystemViewModel;
class FNiagaraScratchPadScriptViewModel;
class FNiagaraSystemGraphSelectionViewModel;
class FNiagaraObjectSelection;
class UNiagaraScript;
class UEdGraph;

UCLASS(MinimalAPI)
class UNiagaraSystemEditorDocumentsViewModel : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRequestOpenChildScript, UEdGraph* /* Graph */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRequestCloseChildScript, UEdGraph* /* Graph */);

	DECLARE_MULTICAST_DELEGATE_OneParam(FScriptToolkitsActiveDocumentChanged, TSharedPtr<SDockTab>);


public:
	NIAGARAEDITOR_API void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);
	NIAGARAEDITOR_API void Finalize();

	NIAGARAEDITOR_API void OpenChildScript(UEdGraph* Graph);
	NIAGARAEDITOR_API void CloseChildScript(UEdGraph* Graph);

	NIAGARAEDITOR_API TArray<class UNiagaraGraph*> GetEditableGraphsForActiveScriptDocument();
	NIAGARAEDITOR_API TArray<class UNiagaraGraph*> GetAllGraphsForActiveScriptDocument();
	NIAGARAEDITOR_API TArray<class UNiagaraGraph*> GetEditableGraphsForPrimaryDocument();
	NIAGARAEDITOR_API TArray<class UNiagaraGraph*> GetAllGraphsForPrimaryDocument();
	NIAGARAEDITOR_API void InitializePreTabManager(TSharedPtr<class FNiagaraSystemToolkit> InToolkit);
	NIAGARAEDITOR_API void InitializePostTabManager(TSharedPtr<class FNiagaraSystemToolkit> InToolkit);

	NIAGARAEDITOR_API void SetActiveDocumentTab(TSharedPtr<SDockTab> Tab);
	TWeakPtr<SDockTab> GetActiveDocumentTab() const { return ActiveDocumentTab; }
	FScriptToolkitsActiveDocumentChanged& OnActiveDocumentChanged() {
		return ActiveDocChangedDelegate;
	}

	NIAGARAEDITOR_API bool IsPrimaryDocumentActive() const;
	NIAGARAEDITOR_API TSharedPtr<class FNiagaraScratchPadScriptViewModel> GetActiveScratchPadViewModelIfSet();
	static NIAGARAEDITOR_API TSharedPtr < class FNiagaraScratchPadScriptViewModel> GetScratchPadViewModelFromGraph(FNiagaraSystemViewModel* InSysViewModel, UEdGraph* InTargetGraph);

	NIAGARAEDITOR_API void DrawAttentionToPrimaryDocument();
	void SetPrimaryDocumentID(const FName& TabId) { PrimaryDocumentTabId = TabId; }
	NIAGARAEDITOR_API void SwapEditableScripts(TSharedPtr < class FNiagaraScratchPadScriptViewModel> OldScriptViewModel, TSharedPtr < class FNiagaraScratchPadScriptViewModel> NewScriptViewModel);

	void CleanInvalidTabs() const;
protected:
	NIAGARAEDITOR_API TSharedPtr<SDockTab> OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause);

	NIAGARAEDITOR_API void NavigateTab(FDocumentTracker::EOpenDocumentCause InCause);

	NIAGARAEDITOR_API void CloseDocumentTab(const UObject* DocumentID);

	// Finds any open tabs containing the specified document and adds them to the specified array; returns true if at least one is found
	NIAGARAEDITOR_API bool FindOpenTabsContainingDocument(const UObject* DocumentID, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results);

	FName PrimaryDocumentTabId;


private:
	NIAGARAEDITOR_API TSharedRef<FNiagaraSystemViewModel> GetSystemViewModel() const;

	FScriptToolkitsActiveDocumentChanged ActiveDocChangedDelegate;

	TWeakPtr<FNiagaraSystemViewModel> SystemViewModelWeak;
	TWeakPtr< FNiagaraSystemGraphSelectionViewModel> SystemGraphSelectionVMWeak;

	NIAGARAEDITOR_API TSharedRef<class SNiagaraScratchPadScriptEditor> CreateGraphEditorWidget(TSharedRef<FTabInfo> InTabInfo, UEdGraph* InGraph);

private:
	TSharedPtr<FDocumentTracker> DocumentManager;
	/** Factory that spawns graph editors; used to look up all tabs spawned by it. */
	TWeakPtr<FDocumentTabFactory> GraphEditorTabFactoryPtr;

	TSharedPtr<class FTabManager> TabManager;

	TWeakPtr<SDockTab> ActiveDocumentTab;

	TSharedPtr<FNiagaraScratchPadScriptViewModel> ActiveDocumentTabScriptViewModel;


};
