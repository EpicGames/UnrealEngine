// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"


class UNiagaraGraph;
class FNiagaraSystemViewModel;
class UNiagaraSystemSelectionViewModel;

/** ViewModel providing selected graphs of system */
class FNiagaraSystemGraphSelectionViewModel : public TSharedFromThis<FNiagaraSystemGraphSelectionViewModel>
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnSelectedEmitterScriptGraphsRefreshed)

	FNiagaraSystemGraphSelectionViewModel() = default;

	void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

	const TArray<TWeakObjectPtr<UNiagaraGraph>> GetSelectedEmitterScriptGraphs() const;

	FOnSelectedEmitterScriptGraphsRefreshed& GetOnSelectedEmitterScriptGraphsRefreshedDelegate() { return OnSelectedEmitterScriptGraphsRefreshedDelegate; };

private:
	void RefreshSelectedEmitterScriptGraphs();

	TSharedRef<FNiagaraSystemViewModel> GetSystemViewModel();
	UNiagaraSystemSelectionViewModel* GetSystemSelectionViewModel();

private:
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModelWeak;
	TArray<TWeakObjectPtr<UNiagaraGraph>> SelectedEmitterScriptGraphs;
	FOnSelectedEmitterScriptGraphsRefreshed OnSelectedEmitterScriptGraphsRefreshedDelegate;
};
