// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMovieScenePlayer.h"

#include "NiagaraCommon.h"
#include "NiagaraEditorCommon.h"
#include "EditorUndoClient.h"
#include "UObject/GCObject.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "ISequencerModule.h"
#include "NiagaraPlatformSet.h"
#include "NiagaraSystemScalabilityViewModel.generated.h"

class FNiagaraSystemViewModel;

/** A view model for viewing and editing a UNiagaraSystem. */

UCLASS(MinimalAPI)
class UNiagaraSystemScalabilityViewModel : public UObject
{
	GENERATED_BODY()
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FScalabilityModeChanged, bool /* Active */)
	DECLARE_MULTICAST_DELEGATE_OneParam(FScalabilityPropertySelected, FName)

public:

	/** Creates a new view model with the supplied System and System instance. */
	NIAGARAEDITOR_API UNiagaraSystemScalabilityViewModel();
	
	/** Initializes this scalability view model with the supplied system view model. */
	NIAGARAEDITOR_API void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

	/** Returns whether or not this view model is initialized and safe to use. */
	NIAGARAEDITOR_API bool IsValid() const;
	
	NIAGARAEDITOR_API bool IsActive() const;

	TWeakPtr<FNiagaraSystemViewModel> GetSystemViewModel() const { return SystemViewModel; }
	FNiagaraPlatformSet* GetPreviewPlatformSet() { return PreviewPlatforms.Get(); }
	const FNiagaraPlatformSet* GetPreviewPlatformSet() const { return PreviewPlatforms.Get(); }
	
	int32 GetPreviewQualityLevelMask() const { return PreviewPlatforms->QualityLevelMask; }
	int32 GetPreviewQualityLevel() const { return FNiagaraPlatformSet::QualityLevelFromMask(PreviewPlatforms->QualityLevelMask); }
	TOptional<TObjectPtr<UDeviceProfile>> GetPreviewDeviceProfile() const { return PreviewDeviceProfile; }
	bool IsViewModeQualityEnabled(int32 QualityLevel) const { return PreviewPlatforms->IsEffectQualityEnabled(QualityLevel); }
	
	NIAGARAEDITOR_API void UpdatePreviewDeviceProfile(UDeviceProfile* DeviceProfile);
	NIAGARAEDITOR_API void UpdatePreviewQualityLevel(int32 QualityLevel);

	NIAGARAEDITOR_API bool IsPlatformActive(const FNiagaraPlatformSet& PlatformSet);

	//void NavigateToScalabilityProperty(UObject* Object, FName PropertyName);
	
private:
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;
	TSharedPtr<FNiagaraPlatformSet> PreviewPlatforms;
	TOptional<TObjectPtr<UDeviceProfile>> PreviewDeviceProfile;
public:
	FScalabilityModeChanged& OnScalabilityModeChanged() { return ScalabilityModeChangedDelegate; }
	FScalabilityPropertySelected& OnScalabilityPropertySelected() { return ScalabilityPropertySelectedDelegate; }
private:
	FScalabilityModeChanged ScalabilityModeChangedDelegate;
	FScalabilityPropertySelected ScalabilityPropertySelectedDelegate;
};
