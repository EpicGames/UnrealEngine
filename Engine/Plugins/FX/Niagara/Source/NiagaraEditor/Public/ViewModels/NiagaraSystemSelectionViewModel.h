// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectKey.h"
#include "NiagaraSystemSelectionViewModel.generated.h"


class FNiagaraSystemViewModel;
class UNiagaraStackEntry;
class UNiagaraStackSelection;
class UNiagaraStackViewModel;

UCLASS(MinimalAPI)
class UNiagaraSystemSelectionViewModel : public UObject
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnSelectionChanged);

private:
	struct FSelectionEntry
	{
		explicit FSelectionEntry(UNiagaraStackEntry* SelectedEntry);

		const TWeakObjectPtr<UNiagaraStackEntry> Entry;
		const FGuid EmitterGuid;
		const FString EditorDataKey;
	};

public:
	GENERATED_BODY()

	NIAGARAEDITOR_API void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

	NIAGARAEDITOR_API void Finalize();

	NIAGARAEDITOR_API bool ContainsEntry(UNiagaraStackEntry* StackEntry) const;

	NIAGARAEDITOR_API void GetSelectedEntries(TArray<UNiagaraStackEntry*>& OutSelectedEntries) const;

	NIAGARAEDITOR_API bool GetSystemIsSelected() const;

	NIAGARAEDITOR_API const TArray<FGuid>& GetSelectedEmitterHandleIds() const;

	NIAGARAEDITOR_API void UpdateSelectedEntries(const TArray<UNiagaraStackEntry*> InSelectedEntries, const TArray<UNiagaraStackEntry*> InDeselectedEntries, bool bClearCurrentSelection);

	NIAGARAEDITOR_API UNiagaraStackViewModel* GetSelectionStackViewModel();

	NIAGARAEDITOR_API void EmptySelection();

	NIAGARAEDITOR_API void RemoveEntriesFromSelection(const TArray<UNiagaraStackEntry*>& InEntriesToRemove);

	NIAGARAEDITOR_API void RemoveEntryFromSelectionByDisplayedObject(const UObject* InObject);

	NIAGARAEDITOR_API void AddEntriesToSelectionByDisplayedObjectsDeferred(const TArray<const UObject*>& InObjects);

	NIAGARAEDITOR_API void AddEntryToSelectionByDisplayedObjectDeferred(const UObject* InObjects);

	NIAGARAEDITOR_API void AddEntriesToSelectionByDisplayedObjectKeysDeferred(const TArray<FObjectKey>& InObjectKeys);

	NIAGARAEDITOR_API void AddEntryToSelectionByDisplayedObjectKeyDeferred(const FObjectKey& InObjectKey);

	NIAGARAEDITOR_API void AddEntryToSelectionBySelectionIdDeferred(const FGuid& InSelectionId);

	NIAGARAEDITOR_API bool Refresh();

	NIAGARAEDITOR_API void RefreshDeferred();

	NIAGARAEDITOR_API FOnSelectionChanged& OnEntrySelectionChanged();

	NIAGARAEDITOR_API FOnSelectionChanged& OnEmitterHandleIdSelectionChanged();

	NIAGARAEDITOR_API FOnSelectionChanged& OnSystemIsSelectedChanged();

	NIAGARAEDITOR_API void Tick();

private:
	TSharedRef<FNiagaraSystemViewModel> GetSystemViewModel();

	void RemoveEntryFromSelectionInternal(UNiagaraStackEntry* DeselectedEntry);

	void UpdateExternalSelectionState();

private:
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModelWeak;

	TArray<FSelectionEntry> SelectionEntries;

	bool bSystemIsSelected;

	TArray<FGuid> SelectedEmitterHandleIds;

	UPROPERTY()
	TObjectPtr<UNiagaraStackSelection> StackSelection;

	UPROPERTY()
	TObjectPtr<UNiagaraStackViewModel> SelectionStackViewModel;

	FOnSelectionChanged OnEntrySelectionChangedDelegate;

	FOnSelectionChanged OnEmitterHandleIdSelectionChangedDelegate;

	FOnSelectionChanged OnSystemIsSelectedChangedDelegate;

	TArray<FObjectKey> DeferredDisplayedObjectKeysToAddToSelection;

	TArray<FGuid> DeferredSelectionIdsToAddToSelection;

	bool bRefreshIsPending;
};
