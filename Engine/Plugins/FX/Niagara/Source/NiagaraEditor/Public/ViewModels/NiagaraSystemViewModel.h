// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "IMovieScenePlayer.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "NiagaraCommon.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraEditorModule.h"
#include "NiagaraPreviewMovieViewModel.h"
#include "NiagaraOverviewNode.h"
#include "NiagaraSystemScalabilityViewModel.h"
#include "NiagaraUserParameterPanelViewModel.h"
#include "TickableEditorObject.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "ViewModels/NiagaraParameterDefinitionsSubscriberViewModel.h"
#include "EventHandlers/ISignedObjectEventHandler.h"

struct FAssetData;
struct FEdGraphEditAction;
struct FNiagaraEmitterHandle;
class FNiagaraEmitterHandleViewModel;
class FNiagaraEmitterViewModel;
struct FNiagaraGraphParameterReference;
class FNiagaraOverviewGraphViewModel;
struct FNiagaraParameterStore;
class FNiagaraPlaceholderDataInterfaceManager;
class FNiagaraSystemGraphSelectionViewModel;
class FNiagaraSystemInstance;
class FNiagaraSystemScriptViewModel;
struct FNiagaraVariable;
class INiagaraParameterDefinitionsSubscriber;
class INiagaraParameterPanelViewModel;
class ISequencer;
class UMovieSceneNiagaraEmitterTrack;
class UNiagaraComponent;
class UNiagaraCurveSelectionViewModel;
class UNiagaraEditorParametersAdapter;
class UNiagaraEditorSettings;
class UNiagaraGraph;
class UNiagaraMessageData;
class UNiagaraNodeFunctionCall;
class UNiagaraOverviewNode;
class UNiagaraScratchPadViewModel;
class UNiagaraSequence;
class UNiagaraStackEntry;
class UNiagaraStackViewModel;
class UNiagaraSystem;
class UNiagaraSystemEditorData;
class UNiagaraSystemEditorDocumentsViewModel;
class UNiagaraSystemSelectionViewModel;
class UNiagaraUserParametersHierarchyViewModel;
enum class ENiagaraPerformanceStateScopeMode : int;

/** Defines different editing modes for this system view model. */
enum class ENiagaraSystemViewModelEditMode
{
	/** A system asset is being edited.  This assumes that emitters should be inheriting from a base version and that emitter editing will be restricted. */
	SystemAsset,
	/** An emitter asset is being edited.  In this mode the system scripts will not be editable and all emitter editing options are available. */
	EmitterAsset,
	/** Special case where an emitter asset is being edited during a merge. In this mode changes made to the emitter being edited don't have to be applied, but directly affect the source. */
	EmitterDuringMerge,
};

enum class ENiagaraGetGraphParameterReferencesMode
{
	AllGraphs,
	SelectedGraphs,
};

DECLARE_DELEGATE_RetVal(FName, FOnGetWorkflowMode);

/** Defines options for the niagara System view model */
struct FNiagaraSystemViewModelOptions
{
	/** Whether or not the user can edit emitters from the timeline. */
	bool bCanModifyEmittersFromTimeline = true;

	/** A delegate which is used to generate the content for the add menu in sequencer. */
	FOnGetAddMenuContent OnGetSequencerAddMenuContent;

	/** Whether or not the system represented by this view model can be automatically compiled.  True by default. */
	bool bCanAutoCompile = true;

	/** Whether or not the system represented by this view model can be simulated. True by default. */
	bool bCanSimulate = true;

	/** An optional unique id which associates this view model with messages in the message manager. */
	TOptional<const FGuid> MessageLogGuid;

	/** Gets the current editing mode for this system. */
	ENiagaraSystemViewModelEditMode EditMode = ENiagaraSystemViewModelEditMode::SystemAsset;

	/** Specifies that the view model is being constructed for data processing only and will not be displayed in the UI. */
	bool bIsForDataProcessingOnly = false;

	/**
	Should we compile the system for edit or not, this is only relevant if it's allowed.
	For example when bIsForDataProcessingOnly is true we won't allow edit mode.
	*/
	bool bCompileForEdit = true;
};

struct FNiagaraStackModuleData
{
	TWeakObjectPtr<UNiagaraNodeFunctionCall> WeakModuleNode;
	ENiagaraScriptUsage Usage;
	FGuid UsageId;
	int32 Index;
	FGuid EmitterHandleId;
};

/** A view model for viewing and editing a UNiagaraSystem. */
class FNiagaraSystemViewModel 
	: public TSharedFromThis<FNiagaraSystemViewModel>
	, public FGCObject
	, public FEditorUndoClient
	, public FTickableEditorObject
	, public TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>
	, public INiagaraParameterDefinitionsSubscriberViewModel
	, public UE::MovieScene::ISignedObjectEventHandler
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnEmitterHandleViewModelsChanged);

	DECLARE_MULTICAST_DELEGATE(FOnPostSequencerTimeChange);

	DECLARE_MULTICAST_DELEGATE(FOnSystemCompiled);

	DECLARE_MULTICAST_DELEGATE(FOnPinnedEmittersChanged);

	DECLARE_MULTICAST_DELEGATE(FOnPreClose);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRequestFocusTab, FName /* TabName */, bool /* DrawAttention */);

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnExternalRenameParameter, const FNiagaraVariableBase&, const FNiagaraVariableBase&, UNiagaraEmitter*);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnExternalRemoveParameter, const FNiagaraVariableBase&,  UNiagaraEmitter*);

	DECLARE_DELEGATE_OneParam(FOnChangeWorkflowMode, FName /* WorkflowMode */)
	DECLARE_DELEGATE_OneParam(FOnEmitterThumbnailRequested, FGuid EmitterGuid)
public:
	struct FEmitterHandleToDuplicate
	{
		FString SystemPath;
		FGuid EmitterHandleId;
		UNiagaraOverviewNode* OverviewNode = nullptr;
		bool operator==(const FEmitterHandleToDuplicate& Other) const
		{
			return SystemPath == Other.SystemPath && EmitterHandleId == Other.EmitterHandleId;
		}
	};
	
	/** Defines different multi-system reset modes for this system view model */
	enum class EMultiResetMode
	{
		/** Reset this instance. Then, if reset dependent systems is enabled in the editor through NiagaraEditorCommands, find all components that share this system and reset those components' system instances. */
		AllowResetAllInstances,
		/** Reset just this instance. */
		ResetThisInstance,
	};

	/** Defines different time reset modes for this system view model */
	enum class ETimeResetMode
	{
		/** If the current sequencer state and user settings allow, reset this system's time. */
		AllowResetTime,
		/** Keep this system's current time. */
		KeepCurrentTime,
	};

	/** Defines different initialization modes when resetting for this system view model */
	enum class EReinitMode
	{
		/** Reinitialize this system (pull in all changes) */
		ReinitializeSystem,
		/** Reset this system (do not pull in changes) */
		ResetSystem,
	};

	/** Creates a new view model with the supplied System and System instance. */
	NIAGARAEDITOR_API FNiagaraSystemViewModel();
	
	/** Initializes this system view model with the supplied system and options. */
	NIAGARAEDITOR_API void Initialize(UNiagaraSystem& InSystem, FNiagaraSystemViewModelOptions InOptions);

	/** Returns whether or not this view model is initialized and safe to use. */
	bool IsValid() const;

	NIAGARAEDITOR_API virtual ~FNiagaraSystemViewModel() override;
	
	//~ Begin NiagaraParameterDefinitionsSubscriberViewModel Interface
protected:
	virtual INiagaraParameterDefinitionsSubscriber* GetParameterDefinitionsSubscriber() override;
	//~ End NiagaraParameterDefinitionsSubscriberViewModel Interface

public:
	NIAGARAEDITOR_API FText GetDisplayName() const;

	/** Gets an array of the view models for the emitter handles owned by this System. */
	NIAGARAEDITOR_API const TArray<TSharedRef<FNiagaraEmitterHandleViewModel>>& GetEmitterHandleViewModels() const;

	/** Gets an emitter handle view model by ID. Returns an invalid shared ptr if it can't be found. */
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterHandleViewModel> GetEmitterHandleViewModelById(FGuid InEmitterHandleId);

	/** Gets an emitter handle view model for the given emitter. Returns an invalid shared ptr if it can't be found. */
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterHandleViewModel> GetEmitterHandleViewModelForEmitter(const FVersionedNiagaraEmitter& InEmitter) const;

	/** Gets the view model for the System script. */
	TSharedPtr<FNiagaraSystemScriptViewModel> GetSystemScriptViewModel();

	/** Gets a niagara component for previewing the simulated System. */
	NIAGARAEDITOR_API UNiagaraComponent* GetPreviewComponent();

	/** Gets the sequencer for this System for displaying the timeline. */
	TSharedPtr<ISequencer> GetSequencer();

	/** Get access to the underlying system*/
	NIAGARAEDITOR_API UNiagaraSystem& GetSystem() const;

	/** Gets whether or not emitters can be added from the timeline. */
	bool GetCanModifyEmittersFromTimeline() const;

	/** Gets the current editing mode for this system view model. */
	NIAGARAEDITOR_API ENiagaraSystemViewModelEditMode GetEditMode() const;

	/** Gets the current workflow mode for this system view model. */
	NIAGARAEDITOR_API FName GetWorkflowMode() const;

	/** Gets the delegate which is called any time the array of emitter handle view models changes. */
	NIAGARAEDITOR_API FOnGetWorkflowMode& OnGetWorkflowMode();

	NIAGARAEDITOR_API void SetWorkflowMode(FName WorkflowMode);
	
	/** Gets the delegate that is bound to the owning toolkit's SetCurrentMode function. */
	NIAGARAEDITOR_API FOnChangeWorkflowMode& OnChangeWorkflowMode();

	/** Gets the delegate that is bound to the owning toolkit's CreateThumbnail function. */
	FOnEmitterThumbnailRequested& OnEmitterThumbnailRequested();
	
	/** Adds a new emitter to the System from an emitter asset data. */
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterHandleViewModel> AddEmitterFromAssetData(const FAssetData& AssetData);

	/** Adds a new emitter to the System. */
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterHandleViewModel> AddEmitter(UNiagaraEmitter& Emitter, FGuid EmitterVersion);
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterHandleViewModel> AddEmitter(const FVersionedNiagaraEmitter& Emitter);

	/** Adds the default empty emitter to the system, or a truly empty emitter as a fallback. */
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterHandleViewModel> AddMinimalEmitter();

	/** Adds a truly empty emitter to the system. */
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterHandleViewModel> AddEmptyEmitter();

	/** Adds a stateless emitter to the system. */
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterHandleViewModel> AddStatelessEmitter();

	/** Deletes the emitters with the supplied ids from the system */
	NIAGARAEDITOR_API void DeleteEmitters(TSet<FGuid> EmitterHandleIdsToDelete);

	/** Gets a multicast delegate which is called any time the array of emitter handle view models changes. */
	NIAGARAEDITOR_API FOnEmitterHandleViewModelsChanged& OnEmitterHandleViewModelsChanged();

	/** Gets a multicast delegate which is called whenever we've received and handled a sequencer time update.*/
	FOnPostSequencerTimeChange& OnPostSequencerTimeChanged();

	/** Gets a multicast delegate which is called whenever the system has been compiled. */
	FOnSystemCompiled& OnSystemCompiled();

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNiagaraSystemViewModel");
	}

	//~ FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

	// ~ FTickableEditorObject
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return true; }
	virtual TStatId GetStatId() const override;

	//~ UE::MovieScene::ISignedObjectEventHandler interface
	virtual void OnModifiedIndirectly(UMovieSceneSignedObject* MovieSceneSignedObject) override;
	virtual void OnModifiedDirectly(UMovieSceneSignedObject* MovieSceneSignedObject) override;

	/** Resets the System instance to initial conditions. Tries to resets system simulation time. Does not reset all systems that share its emitters.
	 * Does not reinitialize the system to pull in changes. Calls into overloaded ResetSystem(). */
	void ResetSystem();

	/** Resets the System instance to initial conditions on the next tick. Tries to resets system simulation time. Does not reset all systems that share its emitters.
	 * Does not reinitialize the system to pull in changes. Calls into overloaded ResetSystem(). */
	void RequestResetSystem();

	/** Resets the system instance to initial conditions. Optionally resets system simulation time. Optionally resets all systems that share its emitters. 
	 * Optionally reinitializes the system to pull in changes.
	 * @param TimeResetMode Defines whether the system being reset should try to reset its time to 0 or keep its current time.
	 * @param MultiResetMode Defines whether the system being reset should try to reset all other systems with which it shares emitters along with resetting itself.
	 * @param ReinitMode Defines whether the system should reinitialize and pull in changes or reset and keep its current state.
	 */
	void ResetSystem(ETimeResetMode TimeResetMode, EMultiResetMode MultiResetMode, EReinitMode ReinitMode);

	/** This is specifically for UI with some unique behavior.
	 *  If at least 1 selected emitter is currently not in solo-mode, we assume the user wants to override the 'solo selection' so that all currently selected emitters enter solo mode
	 *  Since the UI functions can be activated without selection (i.e. clicking 'Isolate' in a overview node won't select the emitter), we always have to specify the guid of the emitter that triggered the call.
	 *  This is so we can work on all selected emitters + the emitter we clicked Isolate on.
	 */
	NIAGARAEDITOR_API void ToggleIsolateEmitterAndSelectedEmitters(FGuid OriginEmitter);
	NIAGARAEDITOR_API void IsolateSelectedEmitters();
	void EnableSelectedEmitters();
	void DisableSelectedEmitters();

	/** Update the compile for edit mode. */
	void SetCompileForEdit(bool bNewState);
	/** Are we running in compile for edit mode or not. */
	bool IsCompileForEdit() const { return bIsCompileForEdit; }
	
	/** Compiles the spawn and update scripts. */
	void CompileSystem(bool bForce);

	/* Get the latest status of this view-model's script compilation.*/
	ENiagaraScriptCompileStatus GetLatestCompileStatus() const;

	/** Gets editor specific data which can be stored per system.  If this data hasn't been created the default version will be returned. */
	NIAGARAEDITOR_API UNiagaraSystemEditorData& GetEditorData() const;
	
	/** Act as if the system has been fully destroyed although references might persist.*/
	void Cleanup();

	/** Reinitializes all System instances, and rebuilds emitter handle view models and tracks. */
	NIAGARAEDITOR_API void RefreshAll();
	
	NIAGARAEDITOR_API void ResetStack();

	/** Called to notify the system view model that one of the data objects in the system was modified. */
	void NotifyDataObjectChanged(TArray<UObject*> ChangedObjects, ENiagaraDataObjectChange ChangeType);

	/** Called to notify the system view model that one of the object references in the system was modified. */
	NIAGARAEDITOR_API void NotifyObjectAssetChanged(UNiagaraNode& OwningNode, FName VariableReadName, UObject* NewValue);

	/** Updates all selected emitter's fixed bounds with their current dynamic bounds. */
	void UpdateEmitterFixedBounds();

	/** Updates the current system's fixed bounds with its current dynamic bounds. */
	void UpdateSystemFixedBounds();

	/** Returns what level of performance stats we have available in the editor. */
	NIAGARAEDITOR_API ENiagaraPerformanceStateScopeMode GetPerformanceStateScopeMode() const;

	/** Are the performance stats enabled? */
	bool IsPerformanceStatsEnabled() const { return bPerformanceStatsEnabled; }
	/** Set if performance stats should be enabled or not */
	void SetPerformanceStatsEnabled(bool bEnabled) { bPerformanceStatsEnabled = bEnabled; }

	/** Clear the captures stats for the current system. */
	void ClearSystemStats();

	/** Clear the captures stats for all the emitters in the current system. */
	void ClearEmitterStats();

	/** Changes the version of an existing emitter in the system */
	bool ChangeEmitterVersion(const FVersionedNiagaraEmitter& Emitter, const FGuid& NewVersion);

	/** Isolates the supplied emitters.  This will remove all other emitters from isolation. */
	NIAGARAEDITOR_API void IsolateEmitters(TArray<FGuid> EmitterHandlesIdsToIsolate);

	void CacheIsolatedEmitterState();
	void RestoreIsolatedEmitterState();

	void SetSystenIsolationEnabled(bool bIsolationEnabled);

	/** Enable the supplied emitters. */
	NIAGARAEDITOR_API void EnableEmitters(TArray<FGuid> EmitterHandlesIdsToEnable);

	/** Disable the supplied emitters. */
	NIAGARAEDITOR_API void DisableEmitters(TArray<FGuid> EmitterHandlesIdsToDisable);

	/** Toggles the isolation state of a single emitter. */
	void ToggleEmitterIsolation(TSharedRef<FNiagaraEmitterHandleViewModel> InEmitterHandle);

	/** Export the current state of the system to text.*/
	void DumpToText(FString& ExportText);

	/** Keeps track of the emitters that are pinned in the stack UI, for persistence*/
	NIAGARAEDITOR_API TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> GetPinnedEmitterHandles();

	/** Sets an internal state when an emitter was pinned/unpinned in the stack UI */
	NIAGARAEDITOR_API void SetEmitterPinnedState(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleModel, bool bPinnedState);

	/** Returns a multicast delegate that is broadcast when an emitter pinned state has changed */
	NIAGARAEDITOR_API FOnPinnedEmittersChanged& GetOnPinnedEmittersChanged();

	/** Checks whether or not an emitter is pinned in the stack UI*/
	NIAGARAEDITOR_API bool GetIsEmitterPinned(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleModel);

	NIAGARAEDITOR_API void NotifyPreSave();
	NIAGARAEDITOR_API void NotifyPostSave();

	NIAGARAEDITOR_API void NotifyPreClose();

	NIAGARAEDITOR_API FSimpleMulticastDelegate& OnPreSave();
	NIAGARAEDITOR_API FOnPreClose& OnPreClose();

	FOnRequestFocusTab& OnRequestFocusTab();

	NIAGARAEDITOR_API void FocusTab(FName TabName, bool bDrawAttention = false);
	
	/** Gets the system toolkit command list. */
	NIAGARAEDITOR_API TSharedPtr<FUICommandList> GetToolkitCommands();
	
	/** Set the system toolkit command list. */
	void SetToolkitCommands(const TSharedRef<FUICommandList>& InToolkitCommands);

	/** Gets the stack module data for the provided module stack entry, for use in determining dependencies. */
	const TArray<FNiagaraStackModuleData>& GetStackModuleDataByModuleEntry(const UNiagaraStackEntry* ModuleEntry);

	/** Gets the stack module data for the provided emitter handled id, for use in determining dependencies. */
	const TArray<FNiagaraStackModuleData>& GetStackModuleDataByEmitterHandleId(FGuid EmitterHandleId);

	/** Get all non-event scripts which will execute for an emitter. */
	void GetOrderedScriptsForEmitterHandleId(FGuid EmitterHandleId, TArray<UNiagaraScript*>& OutScripts);

	/** Gets all non-event scripts which will execute for an emitter. */
	void GetOrderedScriptsForEmitter(const FVersionedNiagaraEmitter& Emitter, TArray<UNiagaraScript*>& OutScripts);

	/** Gets the ViewModel for the system overview graph. */
	NIAGARAEDITOR_API TSharedPtr<FNiagaraOverviewGraphViewModel> GetOverviewGraphViewModel() const;

	/** Gets the stack view model representing this system. */
	NIAGARAEDITOR_API UNiagaraStackViewModel* GetSystemStackViewModel();

	/** Gets the a view model representing the selected entries in the overview. */
	NIAGARAEDITOR_API UNiagaraSystemSelectionViewModel* GetSelectionViewModel();

	NIAGARAEDITOR_API UNiagaraScratchPadViewModel* GetScriptScratchPadViewModel();

	NIAGARAEDITOR_API UNiagaraCurveSelectionViewModel* GetCurveSelectionViewModel();

	NIAGARAEDITOR_API UNiagaraSystemScalabilityViewModel* GetScalabilityViewModel();

	NIAGARAEDITOR_API UNiagaraUserParametersHierarchyViewModel* GetUserParametersHierarchyViewModel();
	
	TArray<float> OnGetPlaybackSpeeds() const;
	
	/** Duplicates a set of emitters and refreshes everything.*/
	void DuplicateEmitters(TArray<FEmitterHandleToDuplicate> EmitterHandlesToDuplicate);

	/** Get the Guid Key for messages associated with this asset. */
	FGuid GetMessageLogGuid() const;

	/** Add a serialized message to the Emitter/System this viewmodel is managing. Returns a key to the new message. */
	NIAGARAEDITOR_API FGuid AddMessage(UNiagaraMessageData* NewMessage) const;

	/** Remove a serialized message from the Emitter/System this viewmodel is managing. */
	NIAGARAEDITOR_API void RemoveMessage(const FGuid& MessageKey) const;

	/** Add a serialized message to the target function call node inside a script this viewmodel is managing. Returns a key to the new message. */
	NIAGARAEDITOR_API FGuid AddStackMessage(UNiagaraMessageData* NewMessage, UNiagaraNodeFunctionCall* TargetFunctionCallNode) const;

	/** Remove a serialized message from the target function call node inside a script this viewmodel is managing. */
	NIAGARAEDITOR_API void RemoveStackMessage(const FGuid& MessageKey, UNiagaraNodeFunctionCall* TargetFunctionCallNode) const;

	void DismissMessageAndRefresh(FNiagaraMessageSourceAndStore MessageSourceAndStore, FGuid MessageKey);

	/** Helper to get the current System or Emitter's EditorParameters. */
	UNiagaraEditorParametersAdapter* GetEditorOnlyParametersAdapter() const;

	ENiagaraStatEvaluationType StatEvaluationType = ENiagaraStatEvaluationType::Average;
	ENiagaraStatDisplayMode StatDisplayMode = ENiagaraStatDisplayMode::Percent;

	void NotifyParameterRemovedExternally(const FNiagaraVariableBase& InVar, UNiagaraEmitter* InOptionalEmitter) { OnExternalRemoveDelegate.Broadcast(InVar, InOptionalEmitter); }
	void NotifyParameterRenamedExternally(const FNiagaraVariableBase& InOldVar, const FNiagaraVariableBase& InNewVar, UNiagaraEmitter* InOptionalEmitter) { OnExternalRenameDelegate.Broadcast(InOldVar, InNewVar, InOptionalEmitter); }

	FOnExternalRenameParameter& OnParameterRenamedExternally() { return OnExternalRenameDelegate; }
	FOnExternalRemoveParameter& OnParameterRemovedExternally() { return OnExternalRemoveDelegate; }

	TSharedRef<FNiagaraPlaceholderDataInterfaceManager> GetPlaceholderDataInterfaceManager();

	/** Gets whether or not this view model is for data processing only and will not be displayed in the UI. */
	bool GetIsForDataProcessingOnly() const;

	INiagaraParameterPanelViewModel* GetParameterPanelViewModel() const {
		return ParameterPanelViewModel.Pin().Get();
	}
	void SetParameterPanelViewModel(TSharedPtr<INiagaraParameterPanelViewModel> InVM);

	UNiagaraSystemEditorDocumentsViewModel* GetDocumentViewModel() { return EditorDocumentsViewModel;};

	void AddParameter(const FNiagaraVariableBase& InUserParameter);

	/** Rename a parameter within the RenameScope and update associated tracking metadata such as Parameter Definition synchronization. */
	bool RenameParameter(const FNiagaraVariable TargetParameter, const FName NewName, ENiagaraGetGraphParameterReferencesMode RenameScopeMode);

	bool DeleteParameter(const FNiagaraVariableBase Parameter);

	bool SetParameterDefaultValue(const FNiagaraVariableBase& InUserParameter, const FNiagaraVariant& InDefaultValue);

	const TSharedPtr<FNiagaraSystemGraphSelectionViewModel>& GetSystemGraphSelectionViewModel() const { return SystemGraphSelectionViewModel; };

	TSharedPtr<FNiagaraUserParameterPanelViewModel> GetUserParameterPanelViewModel() const { return UserParameterPanelViewModel; }

	TSharedPtr<FNiagaraPreviewMovieViewModel> GetPreviewMovieViewModel() const { return PreviewMovieViewModel; }
	
	/** Utility method to gather all graphs. */
	TArray<UNiagaraGraph*> GetAllGraphs();

	/** Utility method to gather all graphs marked as selected via the system selection viewmodel. */
	TArray<UNiagaraGraph*> GetSelectedGraphs();

	/** Utility method to gather graph parameter references */
	TArray<FNiagaraGraphParameterReference> GetGraphParameterReferences(const FNiagaraVariable& Parameter, ENiagaraGetGraphParameterReferencesMode Mode);

	void GetSystemMessageStores(TArray<FNiagaraMessageSourceAndStore>& OutMessageStores);

	void RefreshAssetMessagesDeferred();

private:
	/** Sends message jobs to FNiagaraMessageManager for all compile events from the last compile. */
	void SendLastCompileMessageJobs() const;

	void InvalidateCachedCompileStatus();

	void TickCompileStatus();

	/** Sets up the preview component and System instance. */
	void SetupPreviewComponentAndInstance();

	/** Resets the emitter handle view models and tracks to remove data from them.  This must be called before modifying the emitter handle collection to prevent accessing invalid data. */
	void ResetEmitterHandleViewModelsAndTracks();

	/** Rebuilds the emitter handle view models. */
	void RefreshEmitterHandleViewModels();

	/** Rebuilds the sequencer tracks. */
	void RefreshSequencerTracks();

	/** Updates the data in the sequencer tracks for the specified emitter ids. */
	void UpdateSequencerTracksForEmitters(const TArray<FGuid>& EmitterIdsRequiringUpdate);

	/** Gets the sequencer emitter track for the supplied emitter handle view model. */
	UMovieSceneNiagaraEmitterTrack* GetTrackForHandleViewModel(TSharedRef<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);

	/** Sets up the sequencer for this emitter. */
	void SetupSequencer();

	/** Callback function for when editor settings change. Used to snap to the closest available speed. */
	void SnapToNextSpeed(const FString& PropertyName, const UNiagaraEditorSettings* Settings);

	/** Updates the compiled versions of data interfaces when their sources change. */
	void UpdateCompiledDataInterfaces(UNiagaraDataInterface* ChangedDataInterface);

	/** Called whenever a property on the emitter handle changes. */
	void EmitterHandlePropertyChanged(FGuid OwningEmitterHandleId);

	/** Called whenever the name on an emitter handle changes. */
	void EmitterHandleNameChanged();

	/** Called whenever a property on the emitter changes. */
	void EmitterPropertyChanged();

	/** Called whenever we need to invalidate the parameter cache from the parameter panel updates*/
	void InvalidateCachedParams();

	/** 
	 * Called whenever a parameter store owned by the system changes.
	 * @param ChangedParameterStore The parameter store that changed.
	 * @param OwningScript The script that owns the parameter store, if there is one.
	 */
	void SystemParameterStoreChanged(const FNiagaraParameterStore* ChangedParameterStore, const UNiagaraScript* OwningScript);

	/** Called whenever an emitter's script graph changes. */
	void EmitterScriptGraphChanged(const FEdGraphEditAction& InAction, const UNiagaraScript& OwningScript, FGuid OwningEmitterHandleId);

	/** Called whenever the system script graph changes. */
	void SystemScriptGraphChanged(const FEdGraphEditAction& InAction);

	/**
	* Called whenever an emitter's parameter store owned by the system changes.
	* @param ChangedParameterStore The parameter store that changed.
	* @param OwningScript The script that owns the parameter store, if there is one.
	*/
	void EmitterParameterStoreChanged(const FNiagaraParameterStore& ChangedParameterStore, const UNiagaraScript& OwningScript);

	/** Updates the current simulation for a parameter changing, based on the current simulation options. */
	void UpdateSimulationFromParameterChange();

	/** Called when a script is compiled */
	void ScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion);

	/** Called when the sequencer movie scene object is changed. */
	void SequencerMovieSceneModified(const UMovieScene* MovieScene);

	/** Called whenever the data in the sequence is changed. */
	void SequencerDataChanged(EMovieSceneDataChangeType DataChangeType);

	/** Called whenever the global time in the sequencer is changed. */
	void SequencerTimeChanged();

	/** Called whenever the current selection in the system changes. */
	void SystemSelectionChanged();

	/** Called whenever the track selection in sequencer changes. */
	void SequencerTrackSelectionChanged(TArray<UMovieSceneTrack*> SelectedTracks);

	/** Called whenever the section selection in sequencer changes. */
	void SequencerSectionSelectionChanged(TArray<UMovieSceneSection*> SelectedSections);

	/** Updates the current emitter handle selection base on the sequencer selection. */
	void UpdateEmitterHandleSelectionFromSequencer();

	/** Updates the sequencer selection based on the current emitter handle selection. */
	void UpdateSequencerFromEmitterHandleSelection();

	/** Called when the system instance on the preview component changes. */
	void PreviewComponentSystemInstanceChanged();

	/** Called whenever the System instance is initialized. */
	void SystemInstanceInitialized();

	/** Called whenever the System instance is reset.*/
	void SystemInstanceReset();

	/** Adds event handler for the system's scripts. */
	void AddSystemEventHandlers();

	/** Removes event handlers for the system's scripts. */
	void RemoveSystemEventHandlers();

	/** builds stack module data for use in module dependencies */
	void BuildStackModuleData(UNiagaraScript* Script, FGuid InEmitterHandleId, TArray<FNiagaraStackModuleData>& OutStackModuleData);

	/** Returns true if our system has scripts pending compilation (including GPU compute scripts) */
	bool WaitingOnCompilation() const;

	/** Called when a property on the System is changed. */
	void SystemChanged(UNiagaraSystem* ChangedSystem);

	/** Called whenever one of the owned stack viewmodels structure changes. */
	void StackViewModelStructureChanged(ENiagaraStructureChangedFlags Flags);

	/** Called whenever one of the scripts in the scratch pad changes. */
	void ScratchPadScriptsChanged();

	/** Deferred refreshes the system and emitter view models the next time they are ticked. */
	void RefreshStackViewModels();

	/** Called whenever the map of messages associated with the managed Emitter/System changes. */
	void RefreshAssetMessages();

	const TArray<FNiagaraStackModuleData>& BuildAndCacheStackModuleData(FGuid EmitterHandleId, const FVersionedNiagaraEmitter& Emitter);

	/** Utility method to gather all emitter handle viewmodels marked as selected via the system selection viewmodel. */
	TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> GetSelectedEmitterHandleViewModels();

private:
	/** The System being viewed and edited by this view model. */
	UNiagaraSystem* System;

	/** The component used for previewing the System in a viewport. */
	TObjectPtr<UNiagaraComponent> PreviewComponent;

	/** The system instance currently simulating this system if available. */
	FNiagaraSystemInstance* SystemInstance;

	/** The view models for the emitter handles owned by the System. */
	TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> EmitterHandleViewModels;

	/** The view model for the System script. */
	TSharedPtr<FNiagaraSystemScriptViewModel> SystemScriptViewModel;

	/** The view model that owns and manages the singular movie player available to an asset. */
	TSharedPtr<FNiagaraPreviewMovieViewModel> PreviewMovieViewModel;
	
	/** Handles linking change events from the sequencer movie scene. */
	UE::MovieScene::TNonIntrusiveEventHandler<UE::MovieScene::ISignedObjectEventHandler> MovieSceneEventHandler;

	/** A niagara sequence for displaying this System in the sequencer timeline. */
	TObjectPtr<UNiagaraSequence >NiagaraSequence;

	/** The sequencer instance viewing and editing the niagara sequence. */
	TSharedPtr<ISequencer> Sequencer;

	/** Flag which indicates we are setting the sequencer time directly in an internal operation. */
	bool bSettingSequencerTimeDirectly;

	/** The previous play status for sequencer timeline. */
	EMovieScenePlayerStatus::Type PreviousSequencerStatus;

	/** The previous time for the sequencer timeline. */
	float PreviousSequencerTime;

	/** Whether or not the user can edit emitters from the timeline. */
	bool bCanModifyEmittersFromTimeline;

	/** Whether or not the system represented by this view model can be automatically compiled. */
	bool bCanAutoCompile;

	/** Whether or not the system requires a compilation*/
	bool bForceAutoCompileOnce;

	/** Whether or not the system represented by this view model can be simulated. */
	bool bCanSimulate;

	/** Whether we should use the CompileForEdit mode for the assigned system. */
	bool bSupportCompileForEdit;

	/** What mode compile for edit is currently in. */
	bool bIsCompileForEdit;

	/** The current editing mode for this view model. */
	ENiagaraSystemViewModelEditMode EditMode;

	/** A delegate to retrieve the current workflow mode from the toolkit. */
	FOnGetWorkflowMode OnGetWorkflowModeDelegate;

	/** A delegate that calls the toolkit's set current mode function */
	FOnChangeWorkflowMode OnChangeWorkflowModeDelegate;

	/** A delegate that calls the toolkit's Create Thumbnail function */
	FOnEmitterThumbnailRequested OnEmitterThumbnailRequestedDelegate;
	
	/** A delegate which is used to generate the content for the add menu in sequencer. */
	FOnGetAddMenuContent OnGetSequencerAddMenuContent;

	/** A multicast delegate which is called any time the array of emitter handle view models changes. */
	FOnEmitterHandleViewModelsChanged OnEmitterHandleViewModelsChangedDelegate;

	/** A multicast delegate which is called whenever we've received and handled a sequencer time update.*/
	FOnPostSequencerTimeChange OnPostSequencerTimeChangeDelegate;

	/** A multicast delegate which is called whenever the system has been compiled. */
	FOnSystemCompiled OnSystemCompiledDelegate;

	/** A multicast delegate which is called before the asset is saved. Useful to notify other systems that need to transfer data into the asset before saving is done.. */
	FSimpleMulticastDelegate OnPreSaveDelegate;
	
	/** A multicast delegate which is called whenever this has been notified it's owner will be closing. */
	FOnPreClose OnPreCloseDelegate;

	FOnExternalRenameParameter OnExternalRenameDelegate;
	FOnExternalRemoveParameter OnExternalRemoveDelegate;

	FOnRequestFocusTab OnRequestFocusTabDelegate;

	FDelegateHandle OnScriptAppliedDelegateHandle;

	/** A flag for preventing reentrancy when syncrhonizing sequencer data. */
	bool bUpdatingEmittersFromSequencerDataChange;

	/** A flag for preventing reentrancy when syncrhonizing sequencer data. */
	bool bUpdatingSequencerFromEmitterDataChange;

	/** A flag for preventing reentrancy when synchronizing system selection with sequencer selection */
	bool bUpdatingSystemSelectionFromSequencer;

	/** A flag for preventing reentrancy when synchronizing sequencer selection with system selection */
	bool bUpdatingSequencerSelectionFromSystem;

	/** A flag for preventing selection changes when resetting the sequencer tracks due to the emitter handles array changing. */
	bool bResetingSequencerTracks;

	TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::Handle RegisteredHandle;

	/** Array of FNiagaraEmitterHandleViewModel representing emitters that are pinned in the stack UI*/
	TArray<TSharedRef<FNiagaraEmitterHandleViewModel>> PinnedEmitterHandles;

	/** A single param delegate which is called when the pinned status is changed. */
	FOnPinnedEmittersChanged OnPinnedChangedDelegate;

	/** A cached reference to the niagara editor settings .*/
	UNiagaraEditorSettings* EditorSettings;

	/** A mapping of script to the delegate handle for it's on parameter map changed delegate. */
	TMap<FObjectKey, FDelegateHandle> ScriptToOnParameterStoreChangedHandleMap;

	/** A handle to the on parameter changed delegate for the user parameter store. */
	FDelegateHandle UserParameterStoreChangedHandle;

	/** A flag indicating that a reset has been request on the next tick */
	bool bResetRequestPending;

	TArray<FGuid> CachedEmitterIsolationState;
	
	/** The system toolkit commands. */
	TWeakPtr<class FUICommandList> ToolkitCommands;

	/** A flag which indicates that a compile has been requested, but has not completed. */
	bool bCompilePendingCompletion;

	/** The cache of stack module data for each emitter and for the system */
	TMap<FGuid, TArray<FNiagaraStackModuleData>> GuidToCachedStackModuleData;
	
	/** A handle to the on graph changed delegate for the system script. */
	FDelegateHandle SystemScriptGraphChangedHandle;

	/** A handle to the on graph needs recompile delegate for the system script. */
	FDelegateHandle SystemScriptGraphNeedsRecompileHandle;

	/** An array of emitter handle ids which need their sequencer tracks refreshed next frame. */
	TArray<FGuid> EmitterIdsRequiringSequencerTrackUpdate;

	/** GUID used when sending message jobs to FNiagaraMessageManager for notifying the FNiagaraMessageLogViewModel with the same GUID key */
	TOptional<const FGuid> SystemMessageLogGuidKey;

	/** ViewModel for the system overview graph */
	TSharedPtr<FNiagaraOverviewGraphViewModel> OverviewGraphViewModel;

	TObjectPtr<UNiagaraStackViewModel> SystemStackViewModel;
	
	TObjectPtr<UNiagaraSystemEditorDocumentsViewModel> EditorDocumentsViewModel;

	TObjectPtr<UNiagaraSystemSelectionViewModel> SelectionViewModel;

	TObjectPtr<UNiagaraScratchPadViewModel> ScriptScratchPadViewModel;

	TObjectPtr<UNiagaraCurveSelectionViewModel> CurveSelectionViewModel;

	TObjectPtr<UNiagaraSystemScalabilityViewModel> ScalabilityViewModel;

	TObjectPtr<UNiagaraUserParametersHierarchyViewModel> UserParametersHierarchyViewModel;
	
	TWeakPtr<INiagaraParameterPanelViewModel> ParameterPanelViewModel;

	TSharedPtr<FNiagaraPlaceholderDataInterfaceManager> PlaceholderDataInterfaceManager;

	TArray<UNiagaraScript*> ScriptsToCheckForStatus;
	TArray<ENiagaraScriptCompileStatus> ScriptCompileStatuses;

	TOptional<ENiagaraScriptCompileStatus> LatestCompileStatusCache;

	FDelegateHandle SystemChangedDelegateHandle;

	/** ObjectKeys for object viewed by this view model that supply messages. Used to invalidate the messages of these objects on refresh.*/
	TArray<FObjectKey> MessageSourceObjectKeys;

	/** Flag for when messages have been added/removed through the viewmodel to signal that the message manager needs to be refreshed. */
	mutable bool bPendingAssetMessagesChanged;

	/** Specifies that this view model is for data processing only and will not be displayed in the UI. */
	bool bIsForDataProcessingOnly;

	/** If performance stats should be enable in the editor or not. */
	bool bPerformanceStatsEnabled = false;

	/** ViewModel for caching the selected graph state and notifying when it changes. */
	TSharedPtr<FNiagaraSystemGraphSelectionViewModel> SystemGraphSelectionViewModel;

	TSharedPtr<FNiagaraUserParameterPanelViewModel> UserParameterPanelViewModel;
};
