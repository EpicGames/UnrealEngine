// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraSystem.h"
#include "ViewModels/TNiagaraViewModelManager.h"

class FNiagaraSystemViewModel;

class FNiagaraUserParameterPanelViewModel 
	: public TSharedFromThis<FNiagaraUserParameterPanelViewModel>
    , public TNiagaraViewModelManager<UNiagaraSystem, FNiagaraUserParameterPanelViewModel>
{
public:
	DECLARE_DELEGATE(FOnRefreshRequested)
    DECLARE_DELEGATE_OneParam(FOnParameterAdded, FNiagaraVariable);

	virtual ~FNiagaraUserParameterPanelViewModel();
	
	void Initialize(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel);

	void Refresh() const;

	FOnRefreshRequested& OnRefreshRequested() { return OnRefreshRequestedDelegate; }
    FOnParameterAdded& OnParameterAdded() { return OnParameterAddedDelegate; }
private:
	FOnRefreshRequested OnRefreshRequestedDelegate;
    FOnParameterAdded OnParameterAddedDelegate;
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModelWeak;
    TNiagaraViewModelManager<UNiagaraSystem, FNiagaraUserParameterPanelViewModel>::Handle RegisteredHandle;
};