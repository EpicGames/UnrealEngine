// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NiagaraActions.h"

/** Represents a single action for adding an item to a group in the stack. */
class INiagaraStackItemGroupAddAction
{
public:
	NIAGARAEDITOR_API static const FName FavoriteActionsProfile;
	
	/** Gets the category for this action. */
	virtual TArray<FString> GetCategories() const = 0;

	/** Gets the short display name for this action. */
	virtual FText GetDisplayName() const = 0;

	/** Gets a long description of what will happen if this add action is executed. */
	virtual FText GetDescription() const = 0;

	/** Gets a space separated string of keywords which expose additional search terms for this action. */
	virtual FText GetKeywords() const = 0;
	
	/** Required for favorites feature. */
	virtual TOptional<FNiagaraFavoritesActionData> GetFavoritesData() const = 0;

	/** If specified and valid, will display this movie on its tooltip. */
	virtual TOptional<FSoftObjectPath> GetPreviewMoviePath() const = 0;
	
	/** Indicates if this action represents a library action */
	virtual bool IsInLibrary() const = 0;

	/** Gets a source data struct that is useful to display additional information about an action */
	virtual FNiagaraActionSourceData GetSourceData() const = 0;

	virtual ~INiagaraStackItemGroupAddAction() = default;
};

/** Defines options for adding items to groups in the stack. */
struct FNiagaraStackItemGroupAddOptions
{
	/** Whether or not to include deprecated items. */
	bool bIncludeDeprecated		= false;

	/** Whether or not to include non-library items. */
	bool bIncludeNonLibrary		= false;
};

/** Defines utilities for generically handling adding items to groups in the stack. */
class INiagaraStackItemGroupAddUtilities
{
public:
	/** Defines different modes for adding to a stack group. */
	enum EAddMode
	{
		/** The group adds a new item directly. */
		AddDirectly,
		/** The group provides a list of actions to choose from for adding. */
		AddFromAction
	};

public:
	/** Gets the generic name for the type of item to add e.g. "Module" */
	virtual FText GetAddItemName() const = 0;

	virtual bool GetShowLabel() const = 0;

	/** Gets whether or not the add actions should be automatically expanded in the UI. */
	virtual bool GetAutoExpandAddActions() const = 0;

	/** Gets the add mode supported by these add group utilities. */
	virtual EAddMode GetAddMode() const = 0;

	/** Adds a new item directly. */
	virtual void AddItemDirectly() = 0;

	/** Populates an array with the valid add actions. */
	virtual void GenerateAddActions(TArray<TSharedRef<INiagaraStackItemGroupAddAction>>& OutAddActions, const FNiagaraStackItemGroupAddOptions& AddProperties = FNiagaraStackItemGroupAddOptions()) const = 0;

	/** Executes the specified add action. */
	virtual void ExecuteAddAction(TSharedRef<INiagaraStackItemGroupAddAction> AddAction, int32 TargetIndex) = 0;

	/** Should we add a library filter to the add menu? */
	virtual bool SupportsLibraryFilter() const = 0;

	/** Should we add a source filter to the add menu? */
	virtual bool SupportsSourceFilter() const = 0;

	virtual ~INiagaraStackItemGroupAddUtilities() = default;
};

class FNiagaraStackItemGroupAddUtilities : public INiagaraStackItemGroupAddUtilities
{
public:
	FNiagaraStackItemGroupAddUtilities(FText InAddItemName, EAddMode InAddMode, bool bInAutoExpandAddActions, bool bInShowLabel)
		: AddItemName(InAddItemName)
		, bShowLabel(bInShowLabel)
		, bAutoExpandAddActions(bInAutoExpandAddActions)
		, AddMode(InAddMode)
	{
	}

	virtual FText GetAddItemName() const override { return AddItemName; }
	virtual bool GetShowLabel() const override { return bShowLabel; }
	virtual bool GetAutoExpandAddActions() const override { return bAutoExpandAddActions; }
	virtual EAddMode GetAddMode() const override { return AddMode; }

	virtual bool SupportsLibraryFilter() const override { return false; }
	virtual bool SupportsSourceFilter() const override { return false; }

protected:
	FText AddItemName;
	bool bShowLabel;
	bool bAutoExpandAddActions;
	EAddMode AddMode;
};

template<typename AddedItemType>
class TNiagaraStackItemGroupAddUtilities : public FNiagaraStackItemGroupAddUtilities
{
public:
	DECLARE_DELEGATE_OneParam(FOnItemAdded, AddedItemType);

public:
	TNiagaraStackItemGroupAddUtilities(FText InAddItemName, EAddMode InAddMode, bool bInAutoExpandAddActions, bool bInShowLabel, FOnItemAdded InOnItemAdded)
		: FNiagaraStackItemGroupAddUtilities(InAddItemName, InAddMode, bInAutoExpandAddActions, bInShowLabel)
		, OnItemAdded(InOnItemAdded)
	{
	}

protected:
	FOnItemAdded OnItemAdded;
};
