// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

class UNiagaraNodeFunctionCall;
struct FNiagaraVariable;

class FNiagaraParameterHandle
{
public:
	NIAGARAEDITOR_API FNiagaraParameterHandle();

	NIAGARAEDITOR_API FNiagaraParameterHandle(FName InParameterHandleString);

	NIAGARAEDITOR_API FNiagaraParameterHandle(FName InNamespace, FName InName);

	NIAGARAEDITOR_API bool operator==(const FNiagaraParameterHandle& Other) const;

	bool operator!=(const FNiagaraParameterHandle& Other) const { return (*this == Other) == false; }

	static NIAGARAEDITOR_API FNiagaraParameterHandle CreateAliasedModuleParameterHandle(const FNiagaraParameterHandle& ModuleParameterHandle, const UNiagaraNodeFunctionCall* ModuleNode);
	static NIAGARAEDITOR_API FNiagaraParameterHandle CreateAliasedModuleParameterHandle(const FName FullInputName, const FName FunctionName);

	static NIAGARAEDITOR_API FNiagaraParameterHandle CreateEngineParameterHandle(const FNiagaraVariable& SystemVariable);

	static NIAGARAEDITOR_API FNiagaraParameterHandle CreateEmitterParameterHandle(const FNiagaraVariable& EmitterVariable);

	static NIAGARAEDITOR_API FNiagaraParameterHandle CreateParticleAttributeParameterHandle(const FName InName);

	static NIAGARAEDITOR_API FNiagaraParameterHandle CreateModuleParameterHandle(const FName InName);

	static NIAGARAEDITOR_API FNiagaraParameterHandle CreateInitialParameterHandle(const FNiagaraParameterHandle& Handle);

	NIAGARAEDITOR_API bool IsValid() const;

	NIAGARAEDITOR_API const FName GetParameterHandleString() const;

	NIAGARAEDITOR_API const FName GetName() const;

	NIAGARAEDITOR_API const FName GetNamespace() const;

	NIAGARAEDITOR_API const TArray<FName> GetHandleParts() const;

	NIAGARAEDITOR_API bool IsUserHandle() const;

	NIAGARAEDITOR_API bool IsEngineHandle() const;

	NIAGARAEDITOR_API bool IsSystemHandle() const;

	NIAGARAEDITOR_API bool IsEmitterHandle() const;

	NIAGARAEDITOR_API bool IsParticleAttributeHandle() const;

	NIAGARAEDITOR_API bool IsModuleHandle() const;

	NIAGARAEDITOR_API bool IsOutputHandle() const;

	NIAGARAEDITOR_API bool IsLocalHandle() const;

	NIAGARAEDITOR_API bool IsParameterCollectionHandle() const;

	NIAGARAEDITOR_API bool IsReadOnlyHandle() const;

	NIAGARAEDITOR_API bool IsTransientHandle() const;

	NIAGARAEDITOR_API bool IsDataInstanceHandle() const;

	NIAGARAEDITOR_API bool IsStackContextHandle() const;

private:
	FName ParameterHandleName;
	FName Name;
	FName Namespace;
	mutable TArray<FName> HandlePartsCache;
};

inline uint32 GetTypeHash(const FNiagaraParameterHandle& Var)
{
	return GetTypeHash(Var.GetParameterHandleString());
}
