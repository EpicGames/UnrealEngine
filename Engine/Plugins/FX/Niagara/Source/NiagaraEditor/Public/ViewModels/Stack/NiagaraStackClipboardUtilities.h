// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UNiagaraStackEntry;

namespace FNiagaraStackClipboardUtilities
{
	NIAGARAEDITOR_API bool TestCanCutSelectionWithMessage(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutCanCutMessage);

	NIAGARAEDITOR_API void CutSelection(const TArray<UNiagaraStackEntry*>& SelectedEntries);

	NIAGARAEDITOR_API bool TestCanCopySelectionWithMessage(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutCanCopyMessage);

	NIAGARAEDITOR_API void CopySelection(const TArray<UNiagaraStackEntry*>& SelectedEntries);

	NIAGARAEDITOR_API bool TestCanPasteSelectionWithMessage(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutCanPasteMessage);

	NIAGARAEDITOR_API void PasteSelection(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutPasteWarning);

	NIAGARAEDITOR_API bool TestCanDeleteSelectionWithMessage(const TArray<UNiagaraStackEntry*>& SelectedEntries, FText& OutMessage);

	NIAGARAEDITOR_API void CopyNote(const UNiagaraStackEntry* StackEntry);
	
	NIAGARAEDITOR_API void DeleteSelection(const TArray<UNiagaraStackEntry*>& SelectedEntries);
}