// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphNode_Comment.h"
#include "NiagaraStackObject.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "NiagaraStackCommentCollection.generated.h"

UCLASS(MinimalAPI)
class UNiagaraStackCommentCollection : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	UNiagaraStackCommentCollection() {}

	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData);

	virtual bool GetShouldShowInOverview() const override { return false; }
	virtual bool GetShouldShowInStack() const override { return false; }

	NIAGARAEDITOR_API UNiagaraStackObject* FindStackObjectForCommentNode(UEdGraphNode_Comment* CommentNode) const;
	
protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
};
