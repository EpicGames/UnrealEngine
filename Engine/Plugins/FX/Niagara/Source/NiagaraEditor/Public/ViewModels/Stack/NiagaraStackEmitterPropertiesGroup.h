// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "NiagaraStackEmitterPropertiesGroup.generated.h"

class FNiagaraEmitterViewModel;
class FNiagaraScriptViewModel;
class INiagaraStackItemGroupAddUtilities;
class UNiagaraSimulationStageBase;
class UNiagaraStackEmitterPropertiesItem;
class UNiagaraStackObject;


class FNiagaraStackEmitterStageAddUtilities : public FNiagaraStackItemGroupAddUtilities
{
private:
	enum class EStageAddMode
	{
		Event,
		SimulationStage,
	};

	class FAddEmitterStageAction : public INiagaraStackItemGroupAddAction
	{
	public:
		FAddEmitterStageAction(EStageAddMode InAddMode, UClass* InSimulationStageClass, const TArray<FString>& InCategories, FText InDisplayName, FText InDescription, FText InKeywords)
			: AddMode(InAddMode), SimulationStageClass(InSimulationStageClass), Categories(InCategories), DisplayName(InDisplayName), Description(InDescription), Keywords(InKeywords)
		{
		}

		virtual TArray<FString> GetCategories() const override { return Categories;	}
		virtual FText GetDisplayName() const override {	return DisplayName;	}
		virtual FText GetDescription() const override {	return Description;	}
		virtual FText GetKeywords() const override { return Keywords; }
		virtual TOptional<FNiagaraFavoritesActionData> GetFavoritesData() const override { return TOptional<FNiagaraFavoritesActionData>(); }
		virtual bool IsInLibrary() const override { return true; }
		virtual FNiagaraActionSourceData GetSourceData() const override { return FNiagaraActionSourceData(); }
		virtual TOptional<FSoftObjectPath> GetPreviewMoviePath() const override { return {}; }
		
		EStageAddMode AddMode;
		UClass* SimulationStageClass;
		TArray<FString> Categories;
		FText DisplayName;
		FText Description;
		FText Keywords;
	};

public:
	DECLARE_DELEGATE_TwoParams(FOnItemAdded, FGuid /* AddedEventHandlerId */, UNiagaraSimulationStageBase* /* AddedSimulationStage */);

public:
	FNiagaraStackEmitterStageAddUtilities(TSharedPtr<FNiagaraEmitterViewModel> InEmitterViewModel, FOnItemAdded InOnItemAdded, bool bInAllowEventHandlers = true, bool bInAllowSimulationStages = true);

	virtual void AddItemDirectly() override { unimplemented(); }
	virtual void GenerateAddActions(TArray<TSharedRef<INiagaraStackItemGroupAddAction>>& OutAddActions, const FNiagaraStackItemGroupAddOptions& AddProperties) const override;
	virtual void ExecuteAddAction(TSharedRef<INiagaraStackItemGroupAddAction> AddAction, int32 TargetIndex) override;

private:
	TWeakPtr<FNiagaraEmitterViewModel> EmitterViewModelWeak;
	FOnItemAdded OnItemAdded;
	bool bAllowEventHandlers = true;
	bool bAllowSimulationStages = true;
};

UCLASS(MinimalAPI)
class UNiagaraStackEmitterPropertiesGroup : public UNiagaraStackItemGroup
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API UNiagaraStackEmitterPropertiesGroup();

	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData);

	virtual EIconMode GetSupportedIconMode() const override { return EIconMode::Brush; }
	NIAGARAEDITOR_API virtual const FSlateBrush* GetIconBrush() const override;
	virtual bool SupportsSecondaryIcon() const override;
	NIAGARAEDITOR_API virtual const FSlateBrush* GetSecondaryIconBrush() const override;
	virtual bool GetCanExpandInOverview() const override { return false; }
	virtual bool GetShouldShowInStack() const override { return false; }

protected:
	NIAGARAEDITOR_API void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	NIAGARAEDITOR_API void ItemAddedFromUtilities(FGuid AddedEventHandlerId, UNiagaraSimulationStageBase* AddedSimulationStage);

private:
	UPROPERTY()
	TObjectPtr<UNiagaraStackEmitterPropertiesItem> PropertiesItem;

	TSharedPtr<INiagaraStackItemGroupAddUtilities> AddUtilities;
};
