// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEmitter.h"
#include "NiagaraStackSummaryViewInputCollection.h"
#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "NiagaraStackEmitterSettingsGroup.generated.h"

class FNiagaraEmitterViewModel;
class FNiagaraScriptViewModel;
class UNiagaraStackObject;
class IDetailTreeNode;

UCLASS(MinimalAPI)
class UNiagaraStackEmitterPropertiesItem : public UNiagaraStackItem
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData);

	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;

	NIAGARAEDITOR_API virtual FText GetTooltipText() const override;

	virtual bool SupportsResetToBase() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanResetToBaseWithMessage(FText& OutCanResetToBaseMessage) const override;
	NIAGARAEDITOR_API virtual void ResetToBase() override;

	virtual EIconMode GetSupportedIconMode() const { return EIconMode::Brush; }
	NIAGARAEDITOR_API virtual const FSlateBrush* GetIconBrush() const override;

	virtual bool GetShouldShowInOverview() const override { return false; }

protected:
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	NIAGARAEDITOR_API virtual void RefreshIssues(TArray<FStackIssue>& NewIssues);

private:
	NIAGARAEDITOR_API void EmitterPropertiesChanged();
	NIAGARAEDITOR_API FStackIssueFixDelegate GetUpgradeVersionFix();

	mutable TOptional<bool> bCanResetToBaseCache;

	FVersionedNiagaraEmitterWeakPtr EmitterWeakPtr;

	UPROPERTY()
	TObjectPtr<UNiagaraStackObject> EmitterObject;
};

UCLASS(MinimalAPI)
class UNiagaraStackEmitterSummaryItem : public UNiagaraStackItem
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData);

	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual FText GetTooltipText() const override;

	virtual bool SupportsResetToBase() const override { return false; }
	virtual EIconMode GetSupportedIconMode() const override { return EIconMode::Text; }
	NIAGARAEDITOR_API virtual FText GetIconText() const override;
	virtual bool GetShouldShowInOverview() const override { return false; }

	virtual bool SupportsEditMode() const override { return true; }
	NIAGARAEDITOR_API virtual void OnEditButtonClicked() override;
	NIAGARAEDITOR_API virtual TOptional<FText> GetEditModeButtonText() const override;
	NIAGARAEDITOR_API virtual TOptional<FText> GetEditModeButtonTooltip() const override;

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
	
	virtual void ToggleShowAdvancedInternal() override;

private:
	FVersionedNiagaraEmitterWeakPtr Emitter;

	UPROPERTY()
	TObjectPtr<UNiagaraStackSummaryViewCollection> SummaryViewCollection;
};

UCLASS(MinimalAPI)
class UNiagaraStackEmitterSummaryGroup : public UNiagaraStackItemGroup
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API UNiagaraStackEmitterSummaryGroup();

	virtual EIconMode GetSupportedIconMode() const override { return EIconMode::Text; }
	NIAGARAEDITOR_API virtual  FText GetIconText() const override;
	virtual bool GetCanExpandInOverview() const override { return false; }

protected:
	virtual bool GetShouldShowInStack() const override { return false; }
	virtual bool GetShouldShowInOverview() const override { return true; }
	
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	UPROPERTY()
	TObjectPtr<UNiagaraStackEmitterSummaryItem> SummaryItem;
};
