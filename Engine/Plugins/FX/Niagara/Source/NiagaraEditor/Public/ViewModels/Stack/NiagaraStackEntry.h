// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraValidationRule.h"
#include "Widgets/Views/STableRow.h"
#include "NiagaraStackEntry.generated.h"

class FNiagaraSystemViewModel;
class FNiagaraEmitterViewModel;
class FNiagaraScriptViewModel;
class UNiagaraStackEditorData;
class UNiagaraStackErrorItem;
class UNiagaraStackNote;
class UNiagaraClipboardContent;
class UNiagaraStackEntry;
struct FNiagaraStackNoteData;
class IToolTip;

UENUM()
enum class EStackIssueSeverity : uint8
{
	Error = 0,
	Warning, 
	Info,
	None
};

class FNiagaraStackEntryDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FNiagaraStackEntryDragDropOp, FDecoratedDragDropOp)

public:
	FNiagaraStackEntryDragDropOp(TArray<UNiagaraStackEntry*> InDraggedEntries)
	{
		DraggedEntries = InDraggedEntries;
	}

	const TArray<UNiagaraStackEntry*> GetDraggedEntries() const
	{
		return DraggedEntries;
	}

private:
	TArray<UNiagaraStackEntry*> DraggedEntries;
};

UCLASS(MinimalAPI)
class UNiagaraStackEntry : public UObject
{
	GENERATED_BODY()

public:
	enum class EDragOptions
	{
		Copy,
		None
	};

	enum class EDropOptions
	{
		Overview,
		None
	};

	enum class EIconMode
	{
		Brush,
		Text,
		None
	};

	struct FDropRequest
	{
		FDropRequest(TSharedRef<const FDragDropOperation> InDragDropOperation, EItemDropZone InDropZone, EDragOptions InDragOptions, EDropOptions InDropOptions)
			: DragDropOperation(InDragDropOperation)
			, DropZone(InDropZone)
			, DragOptions(InDragOptions)
			, DropOptions(InDropOptions)
		{
		}

		const TSharedRef<const FDragDropOperation> DragDropOperation;
		const EItemDropZone DropZone;
		const EDragOptions DragOptions;
		const EDropOptions DropOptions;
	};

	struct FDropRequestResponse
	{
		FDropRequestResponse(TOptional<EItemDropZone> InDropZone, FText InDropMessage = FText())
			: DropZone(InDropZone)
			, DropMessage(InDropMessage)
		{
		}

		const TOptional<EItemDropZone> DropZone;
		const FText DropMessage;
	};

	DECLARE_MULTICAST_DELEGATE(FOnExpansionChanged);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnStructureChanged, ENiagaraStructureChangedFlags);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDataObjectModified, TArray<UObject*>, ENiagaraDataObjectChange);
	DECLARE_MULTICAST_DELEGATE(FOnRequestFullRefresh);
	DECLARE_MULTICAST_DELEGATE(FOnRequestFullRefreshDeferred);
	DECLARE_DELEGATE_RetVal_TwoParams(TOptional<FDropRequestResponse>, FOnRequestDrop, const UNiagaraStackEntry& /*TargetEntry*/, const FDropRequest& /*DropRequest*/);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnFilterChild, const UNiagaraStackEntry&);
	DECLARE_DELEGATE(FStackIssueFixDelegate);
	DECLARE_MULTICAST_DELEGATE(FOnAlternateDisplayNameChanged);

public:
	struct FExecutionCategoryNames
	{
		static NIAGARAEDITOR_API const FName System;
		static NIAGARAEDITOR_API const FName Emitter;
		static NIAGARAEDITOR_API const FName StatelessEmitter;
		static NIAGARAEDITOR_API const FName Particle;
		static NIAGARAEDITOR_API const FName Render;
	};

	struct FExecutionSubcategoryNames
	{
		static NIAGARAEDITOR_API const FName Settings;
		static NIAGARAEDITOR_API const FName Spawn;
		static NIAGARAEDITOR_API const FName Update;
		static NIAGARAEDITOR_API const FName Event;
		static NIAGARAEDITOR_API const FName SimulationStage;
		static NIAGARAEDITOR_API const FName Render;
	};

	enum class EStackRowStyle
	{
		None,
		GroupHeader,
		GroupFooter,
		ItemHeader,
		ItemContent,
		ItemContentAdvanced,
		ItemContentNote,
		ItemFooter,
		ItemCategory,
		ItemSubCategory,
		StackIssue,
		Spacer
	};

	struct FRequiredEntryData
	{
		FRequiredEntryData(TSharedRef<FNiagaraSystemViewModel> InSystemViewModel, TSharedPtr<FNiagaraEmitterViewModel> InEmitterViewModel, FName InExecutionCategoryName, FName InExecutionSubcategoryName, UNiagaraStackEditorData& InStackEditorData)
			: SystemViewModel(InSystemViewModel)
			, EmitterViewModel(InEmitterViewModel)
			, ExecutionCategoryName(InExecutionCategoryName)
			, ExecutionSubcategoryName(InExecutionSubcategoryName)
			, StackEditorData(&InStackEditorData)
		{
		}

		const TSharedRef<FNiagaraSystemViewModel> SystemViewModel;
		const TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel;
		const FName ExecutionCategoryName;
		const FName ExecutionSubcategoryName;
		UNiagaraStackEditorData* const StackEditorData;
	};

	struct FStackSearchItem
	{
		FName Key;
		FText Value;
		
		inline bool operator==(FStackSearchItem Item) {
			return (Item.Key == Key 
				&& Item.Value.ToString() == Value.ToString());
		}
	};

	// stack issue stuff

	enum class EStackIssueFixStyle
	{
		Fix,
		Link
	};

	struct FStackIssueFix
	{
		NIAGARAEDITOR_API FStackIssueFix();

		NIAGARAEDITOR_API FStackIssueFix(FText InDescription, FStackIssueFixDelegate InFixDelegate, EStackIssueFixStyle FixStyle = EStackIssueFixStyle::Fix);

		NIAGARAEDITOR_API bool IsValid() const;

		NIAGARAEDITOR_API const FText& GetDescription() const;

		NIAGARAEDITOR_API void SetFixDelegate(const FStackIssueFixDelegate& InFixDelegate);

		NIAGARAEDITOR_API const FStackIssueFixDelegate& GetFixDelegate() const;

		NIAGARAEDITOR_API EStackIssueFixStyle GetStyle() const;

		NIAGARAEDITOR_API const FString& GetUniqueIdentifier() const;

	private:
		FText Description;
		FStackIssueFixDelegate FixDelegate;
		EStackIssueFixStyle Style;
		FString UniqueIdentifier;
	};

	struct FStackIssue
	{
		NIAGARAEDITOR_API FStackIssue();

		NIAGARAEDITOR_API FStackIssue(EStackIssueSeverity InSeverity, FText InShortDescription, FText InLongDescription, FString InStackEditorDataKey, bool bInCanBeDismissed, const TArray<FStackIssueFix>& InFixes, const FSimpleDelegate& InDismissHandler = FSimpleDelegate());

		NIAGARAEDITOR_API FStackIssue(EStackIssueSeverity InSeverity, FText InShortDescription, FText InLongDescription, FString InStackEditorDataKey, bool bInCanBeDismissed, FStackIssueFix InFix);

		NIAGARAEDITOR_API FStackIssue(EStackIssueSeverity InSeverity, FText InShortDescription, FText InLongDescription, FString InStackEditorDataKey, bool bInCanBeDismissed);

		NIAGARAEDITOR_API bool IsValid();

		NIAGARAEDITOR_API EStackIssueSeverity GetSeverity() const;

		NIAGARAEDITOR_API const FText& GetShortDescription() const;

		NIAGARAEDITOR_API const FText& GetLongDescription() const;

		NIAGARAEDITOR_API const FString& GetUniqueIdentifier() const;

		NIAGARAEDITOR_API bool GetCanBeDismissed() const;

		NIAGARAEDITOR_API const FSimpleDelegate& GetDismissHandler() const;

		NIAGARAEDITOR_API const TArray<FStackIssueFix>& GetFixes() const;

		NIAGARAEDITOR_API bool GetIsExpandedByDefault() const;

		NIAGARAEDITOR_API void SetIsExpandedByDefault(bool InExpanded);
		
		NIAGARAEDITOR_API void InsertFix(int32 InsertionIdx, const FStackIssueFix& Fix);

	private:
		EStackIssueSeverity Severity;
		FText ShortDescription;
		FText LongDescription;
		FString UniqueIdentifier;
		bool bCanBeDismissed = false;
		bool bIsExpandedByDefault = true;
		TArray<FStackIssueFix> Fixes;
		FSimpleDelegate DismissHandler;
	};

public:
	NIAGARAEDITOR_API UNiagaraStackEntry();

	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FString InStackEditorDataKey);

	NIAGARAEDITOR_API void Finalize();

	NIAGARAEDITOR_API bool IsFinalized() const;

	NIAGARAEDITOR_API virtual FText GetDisplayName() const;

	NIAGARAEDITOR_API TOptional<FText> GetAlternateDisplayName() const;

	NIAGARAEDITOR_API virtual UObject* GetDisplayedObject() const;

	NIAGARAEDITOR_API virtual FGuid GetSelectionId() const;

	NIAGARAEDITOR_API UNiagaraStackEditorData& GetStackEditorData() const;

	NIAGARAEDITOR_API FString GetStackEditorDataKey() const;
	
	NIAGARAEDITOR_API virtual FText GetTooltipText() const;
	NIAGARAEDITOR_API virtual bool UsesCustomTooltipWidget() const;
	NIAGARAEDITOR_API virtual TSharedPtr<IToolTip> GetCustomTooltipWidget() const;

	NIAGARAEDITOR_API virtual bool GetCanExpand() const;

	NIAGARAEDITOR_API virtual bool GetCanExpandInOverview() const;

	NIAGARAEDITOR_API virtual bool IsExpandedByDefault() const;

	/** By default, we hide the expander if an stack entry can't be expanded. */
	NIAGARAEDITOR_API virtual bool KeepExpanderIndentation() const;

	NIAGARAEDITOR_API bool GetIsExpanded() const;

	NIAGARAEDITOR_API void SetIsExpanded(bool bInExpanded);

	NIAGARAEDITOR_API void SetIsExpanded_Recursive(bool bInExpanded);

	NIAGARAEDITOR_API bool GetIsExpandedInOverview() const;

	NIAGARAEDITOR_API void SetIsExpandedInOverview(bool bInExpanded);

	NIAGARAEDITOR_API virtual bool GetIsEnabled() const;

	NIAGARAEDITOR_API bool GetOwnerIsEnabled() const;

	bool GetIsEnabledAndOwnerIsEnabled() const { return GetIsEnabled() && GetOwnerIsEnabled(); }

	NIAGARAEDITOR_API FName GetExecutionCategoryName() const;

	NIAGARAEDITOR_API FName GetExecutionSubcategoryName() const;

	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const;

	NIAGARAEDITOR_API int32 GetIndentLevel() const;

	/** Returns whether or not this entry should be treated as a child of a previous sibling for layout purposes. */
	virtual bool IsSemanticChild() const { return false; }

	NIAGARAEDITOR_API virtual bool GetShouldShowInStack() const;

	NIAGARAEDITOR_API virtual bool GetShouldShowInOverview() const;

	NIAGARAEDITOR_API void GetFilteredChildren(TArray<UNiagaraStackEntry*>& OutFilteredChildren) const;
	
	NIAGARAEDITOR_API void GetUnfilteredChildren(TArray<UNiagaraStackEntry*>& OutUnfilteredChildren) const;

	NIAGARAEDITOR_API void GetCustomFilteredChildren(TArray<UNiagaraStackEntry*>& OutFilteredChildren, const TArray<FOnFilterChild>& ChildFilters) const;

	void GetFilteredChildrenOfTypes(TArray<UNiagaraStackEntry*>& OutFilteredChildren, const TSet<UClass*>& AllowedClasses, bool bRecursive = false) const
	{
		TArray<UNiagaraStackEntry*> FilteredChildrenTmp;
		GetFilteredChildren(FilteredChildrenTmp);
		for (UNiagaraStackEntry* FilteredChild : FilteredChildrenTmp)
		{
			for(const UClass* Class : AllowedClasses)
			{
				if(FilteredChild->IsA(Class))
				{
					OutFilteredChildren.Add(FilteredChild);
					break;
				}
			}
			UClass* ChildClass = FilteredChild->GetClass();

			if(AllowedClasses.Contains(ChildClass))
			{
				OutFilteredChildren.Add(FilteredChild);

				if(bRecursive)
				{
					FilteredChild->GetFilteredChildrenOfTypes(OutFilteredChildren, AllowedClasses, bRecursive);
				}
			}
		}
	}
	
	template<typename T>
	void GetUnfilteredChildrenOfType(TArray<T*>& OutUnfilteredChildrenOfType, bool bRecursive = false) const
	{
		TArray<UNiagaraStackEntry*> UnfilteredChildren;
		GetUnfilteredChildren(UnfilteredChildren);
		for (UNiagaraStackEntry* UnfilteredChild : UnfilteredChildren)
		{
			T* UnfilteredChildOfType = Cast<T>(UnfilteredChild);
			if (UnfilteredChildOfType != nullptr)
			{
				OutUnfilteredChildrenOfType.Add(UnfilteredChildOfType);
			}
		}

		if(bRecursive)
		{
			for (UNiagaraStackEntry* UnfilteredChild : UnfilteredChildren)
			{
				UnfilteredChild->GetUnfilteredChildrenOfType(OutUnfilteredChildrenOfType, bRecursive);
			}
		}
	}

	template<typename T>
	void GetFilteredChildrenOfType(TArray<T*>& OutFilteredChildrenOfType, bool bRecursive = false) const
	{
		TArray<UNiagaraStackEntry*> OutFilteredChildren;
		GetFilteredChildren(OutFilteredChildren);
		for (UNiagaraStackEntry* FilteredChild : OutFilteredChildren)
		{
			T* FilteredChildOfType = Cast<T>(FilteredChild);
			if (FilteredChildOfType != nullptr)
			{
				OutFilteredChildrenOfType.Add(FilteredChildOfType);
			}
		}

		if(bRecursive)
		{
			for (UNiagaraStackEntry* FilteredChild : OutFilteredChildren)
			{
				FilteredChild->GetFilteredChildrenOfType(OutFilteredChildrenOfType, bRecursive);
			}
		}
	}

	template<typename T>
	void GetCustomFilteredChildrenOfType(TArray<T*>& OutFilteredChildrenOfType, const TArray<FOnFilterChild>& CustomChildFilters) const
	{
		for (UNiagaraStackEntry* Child : Children)
		{
			if(T* CastChild = Cast<T>(Child))
			{
				bool bPassesFilter = true;
				for (const FOnFilterChild& ChildFilter : CustomChildFilters)
				{
					if (ChildFilter.Execute(*CastChild) == false)
					{
						bPassesFilter = false;
						break;
					}
				}

				if (bPassesFilter)
				{
					OutFilteredChildrenOfType.Add(CastChild);
				}				
			}
		}
	}

	NIAGARAEDITOR_API UNiagaraStackNote* GetStackNote();
	
	NIAGARAEDITOR_API FOnExpansionChanged& OnExpansionChanged();

	NIAGARAEDITOR_API FOnExpansionChanged& OnExpansionInOverviewChanged();

	NIAGARAEDITOR_API FOnStructureChanged& OnStructureChanged();

	NIAGARAEDITOR_API FOnDataObjectModified& OnDataObjectModified();

	NIAGARAEDITOR_API FOnRequestFullRefresh& OnRequestFullRefresh();

	NIAGARAEDITOR_API const FOnRequestFullRefresh& OnRequestFullRefreshDeferred() const;

	NIAGARAEDITOR_API FOnRequestFullRefresh& OnRequestFullRefreshDeferred();

	NIAGARAEDITOR_API FOnAlternateDisplayNameChanged& OnAlternateDisplayNameChanged();

	/** Recursively refreshes the children for the current stack entry.  This may cause children to be added or removed and will automatically cause the filtered 
	children to be refreshed. This will also cause the structure changed delegate to be broadcast. */
	NIAGARAEDITOR_API void RefreshChildren();

	/** Invalidates the cached filtered children so that the filters will be run the next time that GetFilteredChildren is called.  This should be called any time
	a change to the data is made which will affect how children are filtered.  This will also cause the structure changed delegate to be broadcast. */
	NIAGARAEDITOR_API void RefreshFilteredChildren();

	NIAGARAEDITOR_API FDelegateHandle AddChildFilter(FOnFilterChild ChildFilter);
	NIAGARAEDITOR_API void RemoveChildFilter(FDelegateHandle FilterHandle);

	NIAGARAEDITOR_API TSharedRef<FNiagaraSystemViewModel> GetSystemViewModel() const;
	NIAGARAEDITOR_API TSharedPtr<FNiagaraEmitterViewModel> GetEmitterViewModel() const;

	/** Same as GetSystemViewModel but can return a nullptr. */
	NIAGARAEDITOR_API TSharedPtr<FNiagaraSystemViewModel> GetSystemViewModelPtr() const;

	template<typename ChildType, typename PredicateType>
	static ChildType* FindCurrentChildOfTypeByPredicate(const TArray<UNiagaraStackEntry*>& CurrentChildren, PredicateType Predicate)
	{
		for (UNiagaraStackEntry* CurrentChild : CurrentChildren)
		{
			ChildType* TypedCurrentChild = Cast<ChildType>(CurrentChild);
			if (TypedCurrentChild != nullptr && Predicate(TypedCurrentChild))
			{
				return TypedCurrentChild;
			}
		}
		return nullptr;
	}

	template<typename ChildType>
	static ChildType* FindCurrentChildOfType(const TArray<UNiagaraStackEntry*>& CurrentChildren)
	{
		for (UNiagaraStackEntry* CurrentChild : CurrentChildren)
		{
			ChildType* TypedCurrentChild = Cast<ChildType>(CurrentChild);
			if (TypedCurrentChild != nullptr)
			{
				return TypedCurrentChild;
			}
		}
		return nullptr;
	}

	NIAGARAEDITOR_API virtual void GetSearchItems(TArray<FStackSearchItem>& SearchItems) const;

	NIAGARAEDITOR_API virtual UObject* GetExternalAsset() const;

	NIAGARAEDITOR_API virtual bool CanDrag() const;

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> CanDrop(const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> Drop(const FDropRequest& DropRequest);

	NIAGARAEDITOR_API void SetOnRequestCanDrop(FOnRequestDrop InOnRequestCanDrop);

	NIAGARAEDITOR_API void SetOnRequestDrop(FOnRequestDrop InOnRequestCanDrop);

	NIAGARAEDITOR_API const bool GetIsSearchResult() const;

	NIAGARAEDITOR_API void SetIsSearchResult(bool bInIsSearchResult);

	NIAGARAEDITOR_API bool HasBaseEmitter() const;

	NIAGARAEDITOR_API bool HasIssuesOrAnyChildHasIssues() const;
	NIAGARAEDITOR_API bool HasUsagesOrAnyChildHasUsages() const;
	NIAGARAEDITOR_API void GetRecursiveUsages(bool& bRead, bool& bWrite) const;
	
	NIAGARAEDITOR_API int32 GetTotalNumberOfInfoIssues() const;

	NIAGARAEDITOR_API int32 GetTotalNumberOfWarningIssues() const;

	NIAGARAEDITOR_API int32 GetTotalNumberOfErrorIssues() const;

	virtual EStackIssueSeverity GetIssueSeverity() const { return EStackIssueSeverity::None; }

	NIAGARAEDITOR_API const TArray<FStackIssue>& GetIssues() const;

	NIAGARAEDITOR_API const TArray<UNiagaraStackEntry*>& GetAllChildrenWithIssues() const;
	
	NIAGARAEDITOR_API void AddValidationIssue(EStackIssueSeverity Severity, const FText& SummaryText, const FText& Description, bool bCanBeDismissed, const TArray<FNiagaraValidationFix>& Fixes, const TArray<FNiagaraValidationFix>& Links);
	NIAGARAEDITOR_API void AddExternalIssue(EStackIssueSeverity Severity, const FText& SummaryText, const FText& Description, bool bCanBeDismissed);
	NIAGARAEDITOR_API void ClearExternalIssues();

	virtual bool SupportsCut() const { return false; }

	virtual bool TestCanCutWithMessage(FText& OutMessage) const { return false; }

	virtual FText GetCutTransactionText() const { return FText(); }

	virtual void CopyForCut(UNiagaraClipboardContent* ClipboardContent) const { }
	virtual void RemoveForCut() { }

	virtual bool SupportsCopy() const { return false; }

	virtual bool TestCanCopyWithMessage(FText& OutMessage) const { return false; }

	virtual void Copy(UNiagaraClipboardContent* ClipboardContent) const { RequestPulse(); }

	virtual bool SupportsPaste() const { return false; }

	virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const { return false; }

	virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const { return FText(); }

	virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) { RequestPulse(); }

	void RequestPulse() const { OnPulseRequestedDelegate.ExecuteIfBound(); }
	FSimpleDelegate& OnPulseRequested() { return OnPulseRequestedDelegate; }
	
	virtual bool SupportsDelete() const { return false; }

	virtual bool TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const { return false; }

	virtual FText GetDeleteTransactionText() const { return FText(); }

	virtual void Delete() { }

	/** Returns whether or not this entry can be renamed. */
	virtual bool SupportsRename() const { return false; }

	/** Gets whether this entry has a rename pending. */
	NIAGARAEDITOR_API virtual bool GetIsRenamePending() const;

	/** Sets whether this entry has a rename pending. */
	NIAGARAEDITOR_API virtual void SetIsRenamePending(bool bIsRenamePending);

	/** Handler for when a rename is committed for this stack entry. */
	NIAGARAEDITOR_API virtual void OnRenamed(FText NewName);

	/** Generally, all stack items with a valid stack key would support the note feature. We only allow them on specific types of of entries. */
	virtual bool SupportsStackNotes() const { return false; }
	NIAGARAEDITOR_API bool HasStackNoteData() const;
	NIAGARAEDITOR_API FNiagaraStackNoteData GetStackNoteData() const;
	NIAGARAEDITOR_API void SetStackNoteData(FNiagaraStackNoteData InStackNoteData);
	NIAGARAEDITOR_API void DeleteStackNoteData();

	NIAGARAEDITOR_API void AddStackNote();
	NIAGARAEDITOR_API void ToggleStackNoteInlineDisplay();
	NIAGARAEDITOR_API void CopyStackNote();
	NIAGARAEDITOR_API void PasteStackNote();
	
	virtual bool SupportsSummaryView() const { return false; }
	NIAGARAEDITOR_API virtual struct FHierarchyElementIdentity DetermineSummaryIdentity() const;
	NIAGARAEDITOR_API bool IsInSummaryView() const;
	NIAGARAEDITOR_API bool IsAnyParentInSummaryView() const;
	NIAGARAEDITOR_API bool IsAnyChildInSummaryView(bool bRecursive = false) const;
	NIAGARAEDITOR_API bool ExistsInParentEmitterSummary() const;

	virtual EIconMode GetSupportedIconMode() const { return EIconMode::None; }

	virtual const FSlateBrush* GetIconBrush() const { return nullptr; }

	virtual FText GetIconText() const { return FText(); }

	virtual bool SupportsInheritance() const { return false; }

	virtual bool GetIsInherited() const { return false; }

	virtual FText GetInheritanceMessage() const { return FText(); }

	NIAGARAEDITOR_API virtual void InvalidateCollectedUsage();


	struct FCollectedUsageData
	{
	public:
		bool bHasReferencedParameterRead = false;
		bool bHasReferencedParameterWrite = false;
	};
	NIAGARAEDITOR_API virtual const FCollectedUsageData& GetCollectedUsageData() const;

protected:
	NIAGARAEDITOR_API virtual void BeginDestroy() override;

	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues);

	NIAGARAEDITOR_API virtual void PostRefreshChildrenInternal();

	NIAGARAEDITOR_API FRequiredEntryData CreateDefaultChildRequiredData() const;

	NIAGARAEDITOR_API virtual int32 GetChildIndentLevel() const;

	NIAGARAEDITOR_API void SetIndentLevelOverride(int32 IndentLevelOverride);

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> CanDropInternal(const FDropRequest& DropRequest);

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> DropInternal(const FDropRequest& DropRequest);

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> ChildRequestCanDropInternal(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> ChildRequestDropInternal(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API virtual void ChildStructureChangedInternal();

	NIAGARAEDITOR_API virtual void FinalizeInternal();


	mutable TOptional<FCollectedUsageData> CachedCollectedUsageData;

	bool IsSystemViewModelValid() const { return SystemViewModel.IsValid(); }

	FSimpleDelegate OnPulseRequestedDelegate;
private:
	NIAGARAEDITOR_API void ChildStructureChanged(ENiagaraStructureChangedFlags Info);

	NIAGARAEDITOR_API void ChildExpansionChanged();

	NIAGARAEDITOR_API void ChildExpansionInOverviewChanged();
	
	NIAGARAEDITOR_API void ChildDataObjectModified(TArray<UObject*> ChangedObjects, ENiagaraDataObjectChange ChangeType);

	NIAGARAEDITOR_API void ChildRequestFullRefresh();

	NIAGARAEDITOR_API void ChildRequestFullRefreshDeferred();

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> ChildRequestCanDrop(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> ChildRequestDrop(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API void RefreshStackErrorChildren();

	NIAGARAEDITOR_API void IssueModified();

	NIAGARAEDITOR_API void InvalidateFilteredChildren();

	struct FCollectedIssueData
	{
		FCollectedIssueData()
			: TotalNumberOfInfoIssues(0)
			, TotalNumberOfWarningIssues(0)
			, TotalNumberOfErrorIssues(0)
		{
		}

		bool HasAnyIssues() const { return TotalNumberOfInfoIssues > 0 || TotalNumberOfWarningIssues > 0 || TotalNumberOfErrorIssues > 0; }

		int32 TotalNumberOfInfoIssues;
		int32 TotalNumberOfWarningIssues;
		int32 TotalNumberOfErrorIssues;
		TArray<UNiagaraStackEntry*> ChildrenWithIssues;
	};


	NIAGARAEDITOR_API const FCollectedIssueData& GetCollectedIssueData() const;

	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;
	TWeakPtr<FNiagaraEmitterViewModel> EmitterViewModel;

	UPROPERTY()
	TObjectPtr<UNiagaraStackEditorData> StackEditorData;

	FString StackEditorDataKey;

	FOnExpansionChanged ExpansionChangedDelegate;

	FOnExpansionChanged ExpansionInOverviewChangedDelegate;
	
	FOnStructureChanged StructureChangedDelegate;

	FOnDataObjectModified DataObjectModifiedDelegate;

	FOnRequestFullRefresh RequestFullRefreshDelegate;

	FOnRequestFullRefresh RequestFullRefreshDeferredDelegate;

	FOnAlternateDisplayNameChanged AlternateDisplayNameChangedDelegate;

	TArray<FOnFilterChild> ChildFilters;

	UPROPERTY()
	TArray<TObjectPtr<UNiagaraStackEntry>> Children;

	mutable bool bFilterChildrenPending;

	mutable TArray<UNiagaraStackEntry*> FilteredChildren;

	UPROPERTY()
	TObjectPtr<UNiagaraStackNote> StackNote;
	
	UPROPERTY()
	TArray<TObjectPtr<UNiagaraStackErrorItem>> ErrorChildren;

	mutable TOptional<bool> bIsExpandedCache;
	mutable TOptional<bool> bIsExpandedInOverviewCache;

	int32 IndentLevel;

	FName ExecutionCategoryName;
	FName ExecutionSubcategoryName;

	FOnRequestDrop OnRequestCanDropDelegate;
	FOnRequestDrop OnRequestDropDelegate;
	
	TArray<FStackIssue> StackIssues;
	TArray<FStackIssue> ExternalStackIssues;

	bool bIsFinalized;

	bool bIsSearchResult;

	mutable TOptional<bool> bHasBaseEmitterCache;

	bool bOwnerIsEnabled;

	TOptional<FText> AlternateDisplayName;

	mutable TOptional<FCollectedIssueData> CachedCollectedIssueData;
};


UCLASS(MinimalAPI)
class UNiagaraStackSpacer : public UNiagaraStackEntry
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, float InSpacerHeight, TAttribute<bool> InShouldShowInStack, FString InOwningStackItemEditorDataKey);
	virtual EStackRowStyle GetStackRowStyle() const override { return UNiagaraStackEntry::EStackRowStyle::Spacer; }
	virtual bool GetCanExpand() const override { return false; }
	virtual bool GetShouldShowInStack() const override { return ShouldShowInStack.Get(); }

	float GetSpacerHeight() const { return SpacerHeight; }

private:
	float SpacerHeight;
	TAttribute<bool> ShouldShowInStack;
};
