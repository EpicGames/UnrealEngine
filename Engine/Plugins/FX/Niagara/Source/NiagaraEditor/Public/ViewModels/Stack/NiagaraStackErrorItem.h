// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "Input/Reply.h"
#include "NiagaraStackErrorItem.generated.h"

UCLASS(MinimalAPI)
class UNiagaraStackErrorItem : public UNiagaraStackEntry
{
	GENERATED_BODY()
public:
	DECLARE_MULTICAST_DELEGATE(FOnIssueNotify);
public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FStackIssue InStackIssue, FString InStackEditorDataKey);
	FStackIssue GetStackIssue() const { return StackIssue; }
	NIAGARAEDITOR_API void SetStackIssue(const FStackIssue& InStackIssue);
	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;
	NIAGARAEDITOR_API FOnIssueNotify& OnIssueModified();
	NIAGARAEDITOR_API virtual void GetSearchItems(TArray<FStackSearchItem>& SearchItems) const override;
	NIAGARAEDITOR_API virtual EStackIssueSeverity GetIssueSeverity() const override;
	NIAGARAEDITOR_API virtual bool IsExpandedByDefault() const override;
	NIAGARAEDITOR_API virtual void SetIsExpandedByDefault(bool bIsExpanded);

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

protected:
	FStackIssue StackIssue;
	FString EntryStackEditorDataKey;
	FOnIssueNotify IssueModifiedDelegate;
	bool bIsExpandedByDefault = true;

private:
	NIAGARAEDITOR_API void IssueFixed();
}; 

UCLASS(MinimalAPI)
class UNiagaraStackErrorItemLongDescription : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FStackIssue InStackIssue, FString InStackEditorDataKey);
	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;
	NIAGARAEDITOR_API virtual EStackIssueSeverity GetIssueSeverity() const override;

protected:
	FStackIssue StackIssue;
};

UCLASS(MinimalAPI)
class UNiagaraStackErrorItemFix : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FStackIssue InStackIssue, FStackIssueFix InIssueFix, FString InStackEditorDataKey);

	FStackIssueFix GetStackIssueFix() const { return IssueFix; };
	NIAGARAEDITOR_API FReply OnTryFixError();
	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;
	NIAGARAEDITOR_API virtual FText GetFixButtonText() const;
	NIAGARAEDITOR_API UNiagaraStackErrorItem::FOnIssueNotify& OnIssueFixed();
	NIAGARAEDITOR_API void SetFixDelegate(const FStackIssueFixDelegate& InFixDelegate);
	NIAGARAEDITOR_API virtual EStackIssueSeverity GetIssueSeverity() const override;

protected:
	FStackIssue StackIssue;
	FStackIssueFix IssueFix;
	UNiagaraStackErrorItem::FOnIssueNotify IssueFixedDelegate;
};

UCLASS(MinimalAPI)
class UNiagaraStackErrorItemDismiss : public UNiagaraStackErrorItemFix
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStackEntry::FStackIssue InStackIssue, FString InStackEditorDataKey);

	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;
	NIAGARAEDITOR_API virtual FText GetFixButtonText() const override;

private:
	void DismissIssue();

};

