// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraEmitter.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "NiagaraStackEventScriptItemGroup.generated.h"

class FNiagaraScriptViewModel;
class UNiagaraStackObject;
struct FNiagaraEventScriptProperties;
class IDetailTreeNode;

// This is a wrapper class used for the details customization in the stack, since the event script properties were moved from the emitter object into the version data struct
UCLASS(MinimalAPI)
class UNiagaraStackEventWrapper : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Events")
	TArray<FNiagaraEventScriptProperties> EventHandlerScriptProps;

	FVersionedNiagaraEmitterWeakPtr EmitterWeakPtr;

	NIAGARAEDITOR_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};

UCLASS(MinimalAPI)
class UNiagaraStackEventHandlerPropertiesItem : public UNiagaraStackItem
{
	GENERATED_BODY()
		 
public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FGuid InEventScriptUsageId);

	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;

	virtual bool SupportsResetToBase() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanResetToBaseWithMessage(FText& OutCanResetToBaseMessage) const override;
	NIAGARAEDITOR_API virtual void ResetToBase() override;

	virtual FGuid GetSelectionId() const override { return EventScriptUsageId; }

	FGuid GetEventScriptUsageId() const { return EventScriptUsageId; };

protected:
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	virtual bool SupportsSummaryView() const override { return true; }
	NIAGARAEDITOR_API virtual FHierarchyElementIdentity DetermineSummaryIdentity() const override;
private:
	NIAGARAEDITOR_API void EventHandlerPropertiesChanged();

	NIAGARAEDITOR_API void FilterEmitterStackObjectRootTreeNodes(const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes);

	NIAGARAEDITOR_API bool HasBaseEventHandler() const;

private:
	FGuid EventScriptUsageId;

	mutable TOptional<bool> bHasBaseEventHandlerCache;

	mutable TOptional<bool> bCanResetToBaseCache;

	FVersionedNiagaraEmitterWeakPtr EmitterWeakPtr;

	UPROPERTY()
	TObjectPtr<UNiagaraStackObject> EmitterObject;

	UPROPERTY()
	TObjectPtr<UNiagaraStackEventWrapper> EventWrapper;
};

UCLASS(MinimalAPI)
/** Meant to contain a single binding of a Emitter::EventScriptProperties to the stack.*/
class UNiagaraStackEventScriptItemGroup : public UNiagaraStackScriptItemGroup
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE(FOnModifiedEventHandlers);

public:
	NIAGARAEDITOR_API void Initialize(
		FRequiredEntryData InRequiredEntryData,
		TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
		ENiagaraScriptUsage InScriptUsage,
		FGuid InScriptUsageId,
		FGuid InEventSourceEmitterId);

	NIAGARAEDITOR_API void SetOnModifiedEventHandlers(FOnModifiedEventHandlers OnModifiedEventHandlers);

	virtual bool SupportsDelete() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const override;
	NIAGARAEDITOR_API virtual void Delete() override;

	virtual bool SupportsInheritance() const override { return true; }
	NIAGARAEDITOR_API virtual bool GetIsInherited() const override;
	NIAGARAEDITOR_API virtual FText GetInheritanceMessage() const override;

	const TObjectPtr<UNiagaraStackEventHandlerPropertiesItem>& GetEventHandlerPropertiesItem() const { return EventHandlerProperties; }
	FGuid GetEventSourceEmitterId() const {return EventSourceEmitterId; };

protected:
	FOnModifiedEventHandlers OnModifiedEventHandlersDelegate;

	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	virtual bool SupportsSummaryView() const override { return true; }
	NIAGARAEDITOR_API virtual FHierarchyElementIdentity DetermineSummaryIdentity() const override;
private:
	NIAGARAEDITOR_API bool HasBaseEventHandler() const;
	
private:
	mutable TOptional<bool> bHasBaseEventHandlerCache;

	FGuid EventSourceEmitterId;

	UPROPERTY()
	TObjectPtr<UNiagaraStackEventHandlerPropertiesItem> EventHandlerProperties;
};
