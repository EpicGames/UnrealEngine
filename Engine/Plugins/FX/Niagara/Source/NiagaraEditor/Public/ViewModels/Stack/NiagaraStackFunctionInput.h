// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItem.h"
#include "NiagaraTypes.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "ViewModels/Stack/NiagaraStackFunctionInputCondition.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraStackScriptHierarchyRoot.h"
#include "NiagaraStackFunctionInput.generated.h"

#define UE_API NIAGARAEDITOR_API

class UNiagaraHierarchyScriptParameter;
class UNiagaraNodeFunctionCall;
class UNiagaraNodeCustomHlsl;
class UNiagaraNodeAssignment;
class UNiagaraNodeInput;
class UNiagaraNodeParameterMapSet;
class FStructOnScope;
class UNiagaraStackObject;
class UNiagaraScript;
class UEdGraphPin;
class UNiagaraDataInterface;
enum class EStackParameterBehavior;
class UNiagaraClipboardFunctionInput;
class UNiagaraClipboardFunction;
class UNiagaraScriptVariable;
class FNiagaraPlaceholderDataInterfaceHandle;

struct FNiagaraStackFunctionInputActions
{
	FText			Label;
	FText			ToolTip;
	FExecuteAction	ExecuteAction;
};

/** Represents a single module input in the module stack view model. */
UCLASS(MinimalAPI)
class UNiagaraStackFunctionInput : public UNiagaraStackItemContent
{
	GENERATED_BODY()

public:
	/** Defines different modes which are used to provide the value for this function input. */
	enum class EValueMode
	{
		/** The value is set to a constant stored locally with this input. */
		Local,
		/** The value is linked to a parameter defined outside of this function. */
		Linked,
		/** The value is linked to a shared parameter defined in the top level script. */
		LinkedShared,
		/** The value is provided by a secondary dynamic input function. */
		Dynamic,
		/** The value is provided by a data interface object. */
		Data,
		/** The value is provided is an object asset. */
		ObjectAsset,
		/** The value is provided by an expression object. */
		Expression,
		/** The value is a default value provided by a function call. */
		DefaultFunction,
		/** This input is overridden in the stack graph, but the override is invalid. */
		InvalidOverride,
		/** This input has a default value set in it's graph which can't be displayed in the stack view. */
		UnsupportedDefault,
		/** This input has no value. */
		None
	};

	struct FNiagaraAvailableParameterInfo
	{
		UE_API FNiagaraAvailableParameterInfo(FNiagaraVariableBase InVariableBase);
		UE_API FNiagaraAvailableParameterInfo(FNiagaraVariableBase InVariableBase, UNiagaraScript* InConversionScript);

		/** The variable this info struct is about. */
		FNiagaraVariableBase Variable;
		/** If this is set, the implication is that the variable can only be applied to this stack function input through conversion. */
		UNiagaraScript* ConversionScript = nullptr;
		/** If this variable is defined in a parameter definition asset, the data is included here for further processing. */
		UNiagaraScriptVariable* MatchingParameterDefinition = nullptr;

		/** We use the variable as identifier so we can use array operations such as Contains or Find based on just the variable. */
		bool operator==(const FNiagaraAvailableParameterInfo& Other) const
		{
			return Variable == Other.Variable;
		}
	};
	
	DECLARE_MULTICAST_DELEGATE(FOnValueChanged);

public:
	NIAGARAEDITOR_API UNiagaraStackFunctionInput();

	/** 
	 * Sets the input data for this entry.
	 * @param InRequiredEntryData The required data for all stack entries.
	 * @param InStackEditorData The stack editor data for this input.
	 * @param InModuleNode The module function call which owns this input entry. NOTE: This input might not be an input to the module function call, it may be an input to a dynamic input function call which is owned by the module.
	 * @param InInputFunctionCallNode The function call which this entry is an input to. NOTE: This node can be a module function call node or a dynamic input node.
	 * @param InInputParameterHandle The input parameter handle for the function call.
	 * @param InInputType The type of this input.
	 * @param InParameterBehavior Determines how the parameter should behave in the stack
	 * @param InOwnerStackItemEditorDataKey The editor data key of the item that owns this input.
	 */
	void Initialize(
		FRequiredEntryData InRequiredEntryData,
		UNiagaraNodeFunctionCall& InModuleNode,
		UNiagaraNodeFunctionCall& InInputFunctionCallNode,
		FName InInputParameterHandle,
		FNiagaraTypeDefinition InInputType,
		EStackParameterBehavior InParameterBehavior,
		FString InOwnerStackItemEditorDataKey);

	/** Gets the function call node which owns this input. */
	NIAGARAEDITOR_API const UNiagaraNodeFunctionCall& GetInputFunctionCallNode() const;

	/** Gets the function call node which owns this input, or nullptr if the node has been garbage collected. */
	NIAGARAEDITOR_API UNiagaraNodeFunctionCall* GetInputFunctionCallNodePtr() const;

	/** Gets the script that the function call node was referencing when this input was initialized. */
	NIAGARAEDITOR_API UNiagaraScript* GetInputFunctionCallInitialScript() const;

	/** Gets the current value mode */
	NIAGARAEDITOR_API EValueMode GetValueMode() const;

	/** Gets the type of this input. */
	NIAGARAEDITOR_API const FNiagaraTypeDefinition& GetInputType() const;

	void SetScriptInstanceData(FNiagaraScriptInstanceData InScriptInstanceData) { ScriptInstanceData = InScriptInstanceData; }

	/** Gets the unit of this input. */
	NIAGARAEDITOR_API EUnit GetInputDisplayUnit() const;

	NIAGARAEDITOR_API FNiagaraInputParameterCustomization GetInputWidgetCustomization() const;

	//~ UNiagaraStackEntry interface
	virtual bool GetShouldShowInStack() const override;
	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual FText GetTooltipText() const override;
	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;
	NIAGARAEDITOR_API virtual UObject* GetExternalAsset() const override;
	virtual bool SupportsStackNotes() const override { return true; }
	virtual bool SupportsCut() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanCutWithMessage(FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual FText GetCutTransactionText() const override;
	NIAGARAEDITOR_API virtual void CopyForCut(UNiagaraClipboardContent* ClipboardContent) const override;
	NIAGARAEDITOR_API virtual void RemoveForCut() override;
	virtual bool SupportsCopy() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanCopyWithMessage(FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual void Copy(UNiagaraClipboardContent* ClipboardContent) const override;
	virtual bool SupportsPaste() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;
	void PasteFunctionInput(const UNiagaraClipboardFunctionInput* ClipboardInput);
	NIAGARAEDITOR_API virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;
	NIAGARAEDITOR_API virtual bool HasOverridenContent() const override;
	NIAGARAEDITOR_API virtual bool SupportsSummaryView() const override;
	NIAGARAEDITOR_API virtual FHierarchyElementIdentity DetermineSummaryIdentity() const override;
	virtual bool GetCanExpand() const override;
	virtual bool KeepExpanderIndentation() const override { return true; }

	/** Gets the tooltip that should be shown for the value of this input. */
	NIAGARAEDITOR_API FText GetValueToolTip() const;

	/** Get a list of custom actions that can be executed on the input from the right click context menu */
	NIAGARAEDITOR_API TArray<FNiagaraStackFunctionInputActions> GetCustomActions();

	/** Gets the tooltip that should be shown for the value of this input. */
	NIAGARAEDITOR_API FText GetCollapsedStateText() const;

	NIAGARAEDITOR_API void SetSummaryViewDisplayName(TAttribute<FText> InDisplayName);
	NIAGARAEDITOR_API void SetSummaryViewTooltip(TAttribute<FText> InTooltipOverride);

	/** Gets the path of parameter handles from the owning module to the function call which owns this input. */
	NIAGARAEDITOR_API const TArray<FNiagaraParameterHandle>& GetInputParameterHandlePath() const;

	/** Gets the parameter handle which defined this input in the module. */
	NIAGARAEDITOR_API const FNiagaraParameterHandle& GetInputParameterHandle() const;

	/** Gets the handle to the linked value for this input if there is one. */
	NIAGARAEDITOR_API const FNiagaraVariableBase& GetLinkedParameterValue() const;

	/** Sets the value of this input to a linked parameter handle. */
	NIAGARAEDITOR_API void SetLinkedParameterValue(const FNiagaraVariableBase& InParameter);

	/** Sets the value of this input to a linked parameter handle using a conversion script. */
	NIAGARAEDITOR_API void SetLinkedParameterValueViaConversionScript(const FNiagaraVariableBase& InParameter, UNiagaraScript& ConversionScript);

	/** Gets the current set of available parameter handles which can be assigned to this input. Optionally returns possible conversion scripts. */
	struct FGetAvailableParameterArgs
	{
		bool bIncludeConversionScripts = true;
		bool bIncludeParameterDefinitions = false;
	};
	
	NIAGARAEDITOR_API void GetAvailableParameters(TSet<FNiagaraAvailableParameterInfo>& OutAvailableParameters, FGetAvailableParameterArgs Arguments) const;

	/** Gets the function node form the script graph if the current value mode is DefaultFunction. */
	NIAGARAEDITOR_API UNiagaraNodeFunctionCall* GetDefaultFunctionNode() const;

	/** Gets the dynamic input node providing the value for this input, if one is available. */
	NIAGARAEDITOR_API UNiagaraNodeFunctionCall* GetDynamicInputNode() const;

	/** Gets the dynamic inputs available for this input. */
	NIAGARAEDITOR_API void GetAvailableDynamicInputs(TArray<UNiagaraScript*>& AvailableDynamicInputs, bool bIncludeNonLibraryInputs = false);

	/** Sets the dynamic input script for this input. */
	NIAGARAEDITOR_API void SetDynamicInput(UNiagaraScript* DynamicInput, FString SuggestedName = FString(), const FGuid& InScriptVersion = FGuid());

	/** Gets the expression providing the value for this input, if one is available. */
	NIAGARAEDITOR_API FText GetCustomExpressionText() const;

	/** Sets the dynamic custom expression script for this input. */
	NIAGARAEDITOR_API void SetCustomExpression(const FString& InCustomExpression);

	/** Create a new scratch pad dynamic inputs and set this input to use it. */
	NIAGARAEDITOR_API void SetScratch();

	/** Gets the current struct value of this input is there is one. */
	NIAGARAEDITOR_API TSharedPtr<const FStructOnScope> GetLocalValueStruct();

	/** Gets the current data object value of this input is there is one. */
	NIAGARAEDITOR_API UNiagaraDataInterface* GetDataValueObject();

	/** Gets the current object Asset value of this input is there is one. */
	NIAGARAEDITOR_API UObject* GetObjectAssetValue();

	/** Sets the current object Asset value of this input is there is one. */
	NIAGARAEDITOR_API void SetObjectAssetValue(UObject* NewValue);

	/** Called to notify the input that an ongoing change to it's value has begun. */
	NIAGARAEDITOR_API void NotifyBeginLocalValueChange();

	/** Called to notify the input that an ongoing change to it's value has ended. */
	NIAGARAEDITOR_API void NotifyEndLocalValueChange();

	/** Is this pin editable or should it show as disabled?*/
	NIAGARAEDITOR_API bool IsEnabled() const;

	/** Sets this input's local value. */
	NIAGARAEDITOR_API void SetLocalValue(TSharedRef<FStructOnScope> InLocalValue);
	
	/** Sets this input's data interface value via a caller-supplied function. The function receives
	 *  the placeholder DI; the graph-side override is synced via the manager's OnChanged handler. */
	NIAGARAEDITOR_API void SetDataInterfaceValueExternal(TSubclassOf<UNiagaraDataInterface> InDataInterfaceType, TFunction<void(UNiagaraDataInterface*)> ExternalSetFunction);

	/** Sets this input's data interface value by creating a new default Data Interface. */
	NIAGARAEDITOR_API void SetDataInterfaceValue(TSubclassOf<UNiagaraDataInterface> InDataInterfaceType);
	
	/** Returns whether or not the value or handle of this input has been overridden and can be reset. */
	NIAGARAEDITOR_API bool CanReset() const;

	/** Resets the value and handle of this input to the value and handle defined in the module. */
	NIAGARAEDITOR_API void Reset();

	/** Determine if this field is editable */
	NIAGARAEDITOR_API bool IsEditable() const;

	NIAGARAEDITOR_API bool HasAnyResettableChildrenInputs() const;
	
	/** If true the parameter can only be set to local constant values */
	NIAGARAEDITOR_API bool IsStaticParameter() const;

	/** Whether or not this input has a base value.  This is true for emitter instances in systems. */
	NIAGARAEDITOR_API bool EmitterHasBase() const;

	/** Whether or not this input can be reset to a base value. */
	NIAGARAEDITOR_API bool CanResetToBase() const;

	/** Resets this input to its base value. */
	NIAGARAEDITOR_API void ResetToBase();

	/** Returns whether or not this input can be renamed. */
	NIAGARAEDITOR_API virtual bool SupportsRename() const override;

	/** Renames this input to the name specified. */
	NIAGARAEDITOR_API virtual void OnRenamed(FText NewName) override;

	/** Returns whether or not this input can be deleted. */
	NIAGARAEDITOR_API bool CanDeleteInput() const;

	/** Deletes this input */
	NIAGARAEDITOR_API void DeleteInput();

	/** Gets the namespaces which new parameters for this input can be read from. */
	NIAGARAEDITOR_API void GetNamespacesForNewReadParameters(TArray<FName>& OutNamespacesForNewParameters) const;

	/** Gets the namespaces which new parameters for this input can write to. */
	NIAGARAEDITOR_API void GetNamespacesForNewWriteParameters(TArray<FName>& OutNamespacesForNewParameters) const;

	/** Gets a multicast delegate which is called whenever the value on this input changes. */
	NIAGARAEDITOR_API FOnValueChanged& OnValueChanged();

	/** Gets the variable that serves as an edit condition for this input. */
	NIAGARAEDITOR_API TOptional<FNiagaraVariable> GetEditConditionVariable() const;
	
	/** Gets whether or not this input has an associated edit condition input. */
	NIAGARAEDITOR_API bool GetHasEditCondition() const;

	/** Gets whether or not to show a control inline for the edit condition input associated with this input. */
	NIAGARAEDITOR_API bool GetShowEditConditionInline() const;

	/** Gets the enabled value of the edit condition input associated with this input. */
	NIAGARAEDITOR_API bool GetEditConditionEnabled() const;

	/** Sets the enabled value of the edit condition input associated with this input. */
	NIAGARAEDITOR_API void SetEditConditionEnabled(bool bIsEnabled);

	/** Gets whether or not this input has an associated visible condition input. */
	NIAGARAEDITOR_API bool GetHasVisibleCondition() const;
	
	/** Gets the variable that serves as a visible condition for this input. */
	NIAGARAEDITOR_API TOptional<FNiagaraVariable> GetVisibleConditionVariable() const;

	/** Gets the enabled value of the visible condition input associated with this input. */
	NIAGARAEDITOR_API bool GetVisibleConditionEnabled() const;

	/** Gets whether or not this input is used as an edit condition for another input and should be hidden. */
	NIAGARAEDITOR_API bool GetIsInlineEditConditionToggle() const;

	/** Gets whether or not a dynamic input script reassignment is pending.  This can happen when trying to fix dynamic inputs which are missing their scripts. */
	NIAGARAEDITOR_API bool GetIsDynamicInputScriptReassignmentPending() const;

	/** Gets whether or not a dynamic input script reassignment should be be pending. */
	NIAGARAEDITOR_API void SetIsDynamicInputScriptReassignmentPending(bool bIsPending);

	/** Reassigns the function script for the current dynamic input without resetting the sub-inputs. */
	NIAGARAEDITOR_API void ReassignDynamicInputScript(UNiagaraScript* DynamicInputScript);

	/** Gets whether or not this input is filtered from search results and appearing in stack due to visibility metadata*/
	NIAGARAEDITOR_API bool GetShouldPassFilterForVisibleCondition() const;
	
	NIAGARAEDITOR_API TArray<UNiagaraScript*> GetPossibleConversionScripts(const FNiagaraTypeDefinition& FromType) const;
	static NIAGARAEDITOR_API TArray<UNiagaraScript*> GetPossibleConversionScripts(const FNiagaraTypeDefinition& FromType, const FNiagaraTypeDefinition& ToType);

	NIAGARAEDITOR_API void SetClipboardContentViaConversionScript(const UNiagaraClipboardFunctionInput& ClipboardFunctionInput);

	NIAGARAEDITOR_API void ChangeScriptVersion(FGuid NewScriptVersion);

	NIAGARAEDITOR_API const UNiagaraClipboardFunctionInput* ToClipboardFunctionInput(UObject* InOuter) const;

	NIAGARAEDITOR_API void SetValueFromClipboardFunctionInput(const UNiagaraClipboardFunctionInput& ClipboardFunctionInput);

	NIAGARAEDITOR_API bool IsScratchDynamicInput() const;

	NIAGARAEDITOR_API bool ShouldDisplayInline() const;
	
	NIAGARAEDITOR_API TArray<UNiagaraStackFunctionInput*> GetChildInputs() const;

	NIAGARAEDITOR_API TOptional<FNiagaraVariableMetaData> GetInputMetaData() const;
	
	NIAGARAEDITOR_API void GetFilteredChildInputs(TArray<UNiagaraStackFunctionInput*>& OutFilteredChildInputs) const;

	NIAGARAEDITOR_API UNiagaraStackObject* GetChildDataObject() const;

	NIAGARAEDITOR_API virtual bool IsSemanticChild() const override;
	NIAGARAEDITOR_API void SetSemanticChild(bool IsSemanticChild);

	//~ UNiagaraStackEntry interface
	NIAGARAEDITOR_API virtual void GetSearchItems(TArray<FStackSearchItem>& SearchItems) const override;

	NIAGARAEDITOR_API virtual const FCollectedUsageData& GetCollectedUsageData() const override;

	NIAGARAEDITOR_API ENiagaraStackEntryInlineDisplayMode GetInlineDisplayMode() const;

	NIAGARAEDITOR_API void SetInlineDisplayMode(ENiagaraStackEntryInlineDisplayMode InlineDisplayMode);

	NIAGARAEDITOR_API bool OpenSourceAsset() const;

	NIAGARAEDITOR_API bool SupportsCustomExpressions() const;

	/** Gets available target script usages which can be used to create shared linked parameters. */
	NIAGARAEDITOR_API void GetAvailableScriptUsagesForSharedParameters(TArray<ENiagaraScriptUsage>& OutUsages) const;

	/** Gets a stack function input to display and edit a linked value, when this input is linked to a shared linked parameter, otherwise returns null */
	NIAGARAEDITOR_API UNiagaraStackFunctionInput* GetSharedLinkedValueInput();

	/** Gets whether or not this input was created by another input to display and edit a shared linked value. */
	bool GetIsSharedLinkedValueInput() const { return bIsSharedLinkedValueInput; }

	/** Moves the current value of this input, to a shared parameter and then changes the value of this input to link to that new parameter.  If a target usage is provided
		the shared input parameter will be added to that script, otherwise the shared parameter will be added to a transient parameter in the current script. */
	NIAGARAEDITOR_API void MoveValueToLinkedSharedParameter(TOptional<ENiagaraScriptUsage> LinkedSharedParameterTargetUsage);

	/** Breaks the link to a shared parameter and copies the value of the shared parameter this input. */
	NIAGARAEDITOR_API void UnlinkFromSharedParameter();

protected:
	//~ UNiagaraStackEntry interface
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	NIAGARAEDITOR_API bool UpdateRapidIterationParametersForAffectedScripts(const uint8* Data);
	NIAGARAEDITOR_API bool RemoveRapidIterationParametersForAffectedScripts(bool bUpdateGraphGuidsForAffected = false);
	NIAGARAEDITOR_API FString ResolveDisplayNameArgument(const FString& InArg) const;
	NIAGARAEDITOR_API TOptional<float> GetMaterialDynamicParamDefaultValue() const;
	NIAGARAEDITOR_API FStackIssueFixDelegate GetUpgradeDynamicInputVersionFix();

	/** Sets whether or not this input was created by another input to display and edit a shared linked value. */
	void SetIsSharedLinkedValueInput(bool bInIsSharedLinkedValueInput) { bIsSharedLinkedValueInput = bInIsSharedLinkedValueInput; }

private:
	struct FInputValues
	{
		FInputValues()
		{
		}

		bool HasEditableData() const
		{
			return Mode != EValueMode::None && Mode != EValueMode::InvalidOverride && Mode != EValueMode::UnsupportedDefault;
		}

		EValueMode Mode = EValueMode::None;
		TSharedPtr<FStructOnScope> LocalStruct;
		FNiagaraVariableBase LinkedParameter;
		TWeakObjectPtr<UNiagaraNodeAssignment> SharedSourceAssignmentNode;
		TWeakObjectPtr<UNiagaraNodeFunctionCall> DynamicNode;
		TWeakObjectPtr<UNiagaraNodeCustomHlsl> ExpressionNode;
		TWeakObjectPtr<UNiagaraDataInterface> DataObject;
		TWeakObjectPtr<UNiagaraNodeInput> ObjectAssetInputNode;
		TWeakObjectPtr<UNiagaraNodeFunctionCall> DefaultFunctionNode;
	};

private:
	/** Refreshes the current values for this input from the state of the graph. */
	NIAGARAEDITOR_API void RefreshValues();

	/** Refreshes additional state for this input which comes from input metadata. */
	NIAGARAEDITOR_API void RefreshFromMetaData(TArray<FStackIssue>& NewIssues);

	/** Called whenever the graph which generated this input changes. */
	NIAGARAEDITOR_API void OnGraphChanged(const struct FEdGraphEditAction& InAction);

	/** Called whenever rapid iteration parameters are changed for the script that owns the function that owns this input. */
	NIAGARAEDITOR_API void OnRapidIterationParametersChanged();

	/** Called whenever the script source that owns the function that owns this input changes. */
	NIAGARAEDITOR_API void OnScriptSourceChanged();

	/** Gets the graph node which owns the local overrides for the module that owns this input if it exists. */
	NIAGARAEDITOR_API UNiagaraNodeParameterMapSet* GetOverrideNode() const;

	/** Gets the graph node which owns the local overrides for the module that owns this input this input.  
	  * This will create the node and add it to the graph if it doesn't exist. */
	NIAGARAEDITOR_API UNiagaraNodeParameterMapSet& GetOrCreateOverrideNode();

	/** Gets the pin on the override node which is associated with this input if it exists. */
	NIAGARAEDITOR_API UEdGraphPin* GetOverridePin() const;

	/** Gets the pin on the override node which is associated with this input.  If either the override node or
	  * pin don't exist, they will be created. */
	NIAGARAEDITOR_API UEdGraphPin& GetOrCreateOverridePin();

	NIAGARAEDITOR_API void GetDefaultDataInterfaceValueFromDefaultPin(UEdGraphPin* DefaultPin, UNiagaraStackFunctionInput::FInputValues& InInputValues) const;

	NIAGARAEDITOR_API void GetDefaultObjectAssetValueFromDefaultPin(UEdGraphPin* DefaultPin, UNiagaraStackFunctionInput::FInputValues& InInputValues) const;

	NIAGARAEDITOR_API void GetDefaultLocalValueFromDefaultPin(UEdGraphPin* DefaultPin, UNiagaraStackFunctionInput::FInputValues& InInputValues) const;

	NIAGARAEDITOR_API void GetDefaultLinkedParameterOrLinkedFunctionFromDefaultPin(UEdGraphPin* DefaultPin, UNiagaraStackFunctionInput::FInputValues& InInputValues) const;

	NIAGARAEDITOR_API void UpdateValuesFromScriptDefaults(FInputValues& InInputValues) const;

	NIAGARAEDITOR_API void UpdateValuesFromOverridePin(const FInputValues& OldInputValues, FInputValues& NewInputValues, UEdGraphPin& InOverridePin) const;

	/** Removes all nodes connected to the override pin which provide it's value. */
	NIAGARAEDITOR_API void RemoveNodesForOverridePin(UEdGraphPin& OverridePin);

	/** Remove the override pin and all nodes connected to it. */
	NIAGARAEDITOR_API void RemoveOverridePin();

	/** Determine if the values in this input are possibly under the control of the rapid iteration array on the script.*/
	NIAGARAEDITOR_API bool IsRapidIterationCandidate() const;

	NIAGARAEDITOR_API FNiagaraVariable CreateRapidIterationVariable(const FName& InName);

	/** Handles the message manager refreshing messages. */
	NIAGARAEDITOR_API void OnMessageManagerRefresh(const TArray<TSharedRef<const INiagaraMessage>>& NewMessages);

	NIAGARAEDITOR_API void GetCurrentChangeIds(FGuid& OutOwningGraphChangeId, FGuid& OutFunctionGraphChangeId) const;

	NIAGARAEDITOR_API UNiagaraScript* FindConversionScript(const FNiagaraTypeDefinition& FromType, TMap<FNiagaraTypeDefinition, UNiagaraScript*>& ConversionScriptCache, bool bIncludeConversionScripts) const;

	NIAGARAEDITOR_API bool FilterInlineChildren(const UNiagaraStackEntry& Child) const;
	NIAGARAEDITOR_API bool FilterForVisibleCondition(const UNiagaraStackEntry& NiagaraStackEntry) const;
	bool FilterOnlyModified(const UNiagaraStackEntry& Child) const;
	bool FilterForIsInlineEditConditionToggle(const UNiagaraStackEntry& NiagaraStackEntry) const;

	void ReportScriptVersionChange() const;

private:
	/** The module function call which owns this input entry. NOTE: This input might not be an input to the module function
		call, it may be an input to a dynamic input function call which is owned by the module. */
	TWeakObjectPtr<UNiagaraNodeFunctionCall> OwningModuleNode;

	/** The function call which this entry is an input to. NOTE: This node can be a module function call node or a dynamic input node. */
	TWeakObjectPtr<UNiagaraNodeFunctionCall> OwningFunctionCallNode;

	/** The script which the owning function call is referencing. */
	TWeakObjectPtr<UNiagaraScript> OwningFunctionCallInitialScript;

	/** An optionally set hierarchy object that gives us information on child inputs etc. */
	TWeakObjectPtr<UNiagaraHierarchyScriptParameter> HierarchyScriptParameter;
	
	/** The assignment node which owns this input.  This is only valid for inputs of assignment modules. */
	TWeakObjectPtr<UNiagaraNodeAssignment> OwningAssignmentNode;

	/** The niagara type definition for this input. */
	FNiagaraTypeDefinition InputType;

	/** The meta data for this input, defined in the owning function's script. */
	TOptional<FNiagaraVariableMetaData> InputMetaData;

	// Stack issues generated when updating from metadata.
	TArray<FStackIssue> InputMetaDataIssues;

	/** A unique key for this input for looking up editor only UI data. */
	FString StackEditorDataKey;

	/** An array representing the path of Namespace.Name handles starting from the owning module to this function input. */
	TArray<FNiagaraParameterHandle> InputParameterHandlePath;

	/** The parameter handle which defined this input in the module graph. */
	FNiagaraParameterHandle InputParameterHandle;

	/** Defines the type of the stack parameter (e.g. 'static' if it should only allow constant compile-time values). */
	EStackParameterBehavior ParameterBehavior;

	/** The parameter handle which defined this input in the module graph, aliased for use in the current emitter
	  * graph.  This only affects parameter handles which are local module handles. */
	FNiagaraParameterHandle AliasedInputParameterHandle;

	/** The a rapid iteration variable that could potentially drive this entry..*/
	FNiagaraVariable RapidIterationParameter;

	/** The name of this input for display in the UI. */
	FText DisplayName;
	
	/** Optional override for the display name*/
	TOptional<FText> DisplayNameOverride;
	TOptional<TAttribute<FText>> SummaryViewDisplayNameOverride;
	TOptional<TAttribute<FText>> SummaryViewTooltipOverride;

	/** The default value for this input defined in the defining script. */
	FInputValues DefaultInputValues;

	/** Pointers and handles to the various values this input can have. */
	FInputValues InputValues;

	TSharedPtr<FNiagaraPlaceholderDataInterfaceHandle> PlaceholderDataInterfaceHandle;

	/** A cached pointer to the override node for this input if it exists.  This value is cached here since the
	  * UI reads this value every frame due to attribute updates. */
	mutable TOptional<UNiagaraNodeParameterMapSet*> OverrideNodeCache;

	/** A cached pointer to the override pin for this input if it exists.  This value is cached here since the
	  * UI reads this value every frame due to attribute updates. */
	mutable TOptional<UEdGraphPin*> OverridePinCache;

	/** Whether or not this input can be reset to its default value. */
	mutable TOptional<bool> bCanResetCache;

	/** Whether or not this input can be reset to a base value defined by a parent emitter. */
	mutable TOptional<bool> bCanResetToBaseCache;

	/** A tooltip to show for the value of this input. */
	mutable TOptional<FText> ValueToolTipCache;

	/** Text to display on a collapsed node. */
	mutable TOptional<FText> CollapsedTextCache;

	mutable TOptional<bool> bIsScratchDynamicInputCache;

	/** A flag to prevent handling graph changes when it's being updated directly by this object. */
	bool bUpdatingGraphDirectly;

	/** A flag to prevent handling changes to the local value when it's being set directly by this object. */
	bool bUpdatingLocalValueDirectly;

	/** A handle for removing the graph changed delegate. */
	FDelegateHandle GraphChangedHandle;
	FDelegateHandle OnRecompileHandle;

	/** A handle for removing the changed delegate for the rapid iteration parameters for the script
	   that owns the function that owns this input. */
	FDelegateHandle RapidIterationParametersChangedHandle;

	/** A multicast delegate which is called when the value of this input is changed. */
	FOnValueChanged ValueChangedDelegate;

	/** Script instance data passed in from the parent to determine additional instance data, such as whether inputs are currently visible or not. */
	FNiagaraScriptInstanceData ScriptInstanceData;
	
	/** The script which owns the function which owns this input.  This is also the authoritative version of the rapid iteration parameters. */
	TWeakObjectPtr<UNiagaraScript> SourceScript;

	/** An array of scripts which this input affects. */
	TArray<TWeakObjectPtr<UNiagaraScript>> AffectedScripts;

	/** An input condition handler for the edit condition. */
	FNiagaraStackFunctionInputCondition EditCondition;

	/** An input condition handler for the visible condition. */
	FNiagaraStackFunctionInputCondition VisibleCondition;
	
	/** Whether or not to show an inline control for the edit condition input. */
	bool bShowEditConditionInline;

	/** Whether or not this input is an edit condition toggle. */
	bool bIsInlineEditConditionToggle;

	/** Whether or not the dynamic input for this input has a function script reassignment pending due to a request to fix a missing script. */
	bool bIsDynamicInputScriptReassignmentPending;

	/** A key to the message manager registration and its delegate binding. */
	FGuid MessageManagerRegistrationKey;

	//** Issues created outside of the RefreshChildren call that will be committed the next time the UI state is refreshed. */
	TArray<FStackIssue> MessageManagerIssues;

	FGuid MessageLogGuid;

	// If true then this stack entry is the semantic child of another stack entry
	bool bIsSemanticChild = false;

	/** A stack function input to display and edit a linked value, when this input is linked to a shared linked parameter. */
	UPROPERTY()
	TObjectPtr<UNiagaraStackFunctionInput> SharedLinkedInput;
	/** Whether or not this input was created by another input to display and edit a shared linked value. */
	bool bIsSharedLinkedValueInput = false;

	// The value of the change id for the graph which owns the function call that owns this input as of the last refresh.
	TOptional<FGuid> LastOwningGraphChangeId;

	// The value of the change id for the called graph of the function call which owns this input as of the last refresh.
	TOptional<FGuid> LastFunctionGraphChangeId;
};

inline uint32 GetTypeHash(const UNiagaraStackFunctionInput::FNiagaraAvailableParameterInfo& ParameterInfo)
{
	return GetTypeHash(ParameterInfo.Variable);
}

#undef UE_API
