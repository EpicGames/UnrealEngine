// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraParameterMapHistory.h"
#include "NiagaraStackFunctionInputBinder.h"

class UNiagaraScript;
class UNiagaraNodeFunctionCall;
struct FNiagaraVariableMetaData;

/** Users a condition string to bind to an input on a stack function and determines if the value of that input matches a specific value. */
class FNiagaraStackFunctionInputCondition
{
public:
	void Initialize(UNiagaraScript* InScript,
		TArray<UNiagaraScript*> InDependentScripts,
		FCompileConstantResolver InConstantResolver,
		FString InOwningEmitterUniqueName,
		UNiagaraNodeFunctionCall* InFunctionCallNode);

	void Refresh(const FNiagaraInputConditionMetadata& InputCondition, FText& OutErrorMessage);

	bool IsValid() const;

	bool GetConditionIsEnabled() const;

	bool CanSetConditionIsEnabled() const;

	void SetConditionIsEnabled(bool bInIsEnabled);

	FName GetConditionInputName() const;

	FNiagaraTypeDefinition GetConditionInputType() const;

	TOptional<FNiagaraVariableMetaData> GetConditionInputMetaData() const;

private:
	UNiagaraScript* Script;

	TArray<UNiagaraScript*> DependentScripts;

	FCompileConstantResolver ConstantResolver;

	FString OwningEmitterUniqueName;

	UNiagaraNodeFunctionCall* FunctionCallNode;

	FNiagaraStackFunctionInputBinder InputBinder;

	TArray<TArray<uint8>> TargetValuesData;
};