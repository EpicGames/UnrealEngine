// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraTypes.h"
#include "NiagaraCommon.h"
#include "NiagaraParameterMapHistoryFwd.h"
#include "NiagaraVariableMetaData.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "AssetRegistry/AssetData.h"

class UEdGraph;
class UEdGraphPin;
class UNiagaraGraph;
class UNiagaraNode;
class UNiagaraNodeInput;
class UNiagaraNodeOutput;
class UNiagaraNodeParameterMapBase;
class UNiagaraNodeFunctionCall;
class UNiagaraNodeCustomHlsl;
class UNiagaraNodeAssignment;
class UNiagaraNodeParameterMapSet;
class FNiagaraSystemViewModel;
class UNiagaraEmitter;
class FNiagaraEmitterViewModel;
class UNiagaraScriptVariable;
class UNiagaraStackEditorData;
class UNiagaraStackErrorItem;
class FCompileConstantResolver;
class INiagaraMessage;
class UNiagaraHierarchyModuleInput;
class UNiagaraSimulationStageBase;
struct FNiagaraEventScriptProperties;
struct FNiagaraStackModuleData;
struct FNiagaraModuleDependency;
struct FNiagaraEmitterHandle;
class UNiagaraRendererProperties;

namespace FNiagaraStackGraphUtilities
{
	void MakeLinkTo(UEdGraphPin* PinA, UEdGraphPin* PinB);
	void BreakAllPinLinks(UEdGraphPin* PinA);

	void RelayoutGraph(UEdGraph& Graph);

	void ConnectPinToInputNode(UEdGraphPin& Pin, UNiagaraNodeInput& InputNode);

	UEdGraphPin* GetParameterMapInputPin(UNiagaraNode& Node);

	const UEdGraphPin* GetParameterMapInputPin(const UNiagaraNode& Node);

	UEdGraphPin* GetParameterMapOutputPin(UNiagaraNode& Node);

	void GetOrderedModuleNodes(UNiagaraNodeOutput& OutputNode, TArray<UNiagaraNodeFunctionCall*>& ModuleNodes);
	TArray<UNiagaraNodeFunctionCall*> GetAllModuleNodes(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	TArray<UNiagaraNodeFunctionCall*> GetAllModuleNodes(UNiagaraGraph* Graph);

	TArray<UNiagaraNodeFunctionCall*> GetAllSimStagesModuleNodes(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	TArray<UNiagaraNodeFunctionCall*> GetAllEventHandlerModuleNodes(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);

	UNiagaraNodeFunctionCall* GetPreviousModuleNode(UNiagaraNodeFunctionCall& CurrentNode);

	UNiagaraNodeFunctionCall* GetNextModuleNode(UNiagaraNodeFunctionCall& CurrentNode);

	UNiagaraNodeOutput* GetEmitterOutputNodeForStackNode(UNiagaraNode& StackNode);

	NIAGARAEDITOR_API ENiagaraScriptUsage GetOutputNodeUsage(const UNiagaraNode& StackNode);

	const UNiagaraNodeOutput* GetEmitterOutputNodeForStackNode(const UNiagaraNode& StackNode);

	UNiagaraNodeInput* GetEmitterInputNodeForStackNode(UNiagaraNode& StackNode);

	void CheckForDeprecatedScriptVersion(UNiagaraNodeFunctionCall* InputFunctionCallNode, const FString& StackEditorDataKey, UNiagaraStackEntry::FStackIssueFixDelegate VersionUpgradeFix, TArray<UNiagaraStackEntry::FStackIssue>& OutIssues);
	void CheckForDeprecatedEmitterVersion(TSharedPtr<FNiagaraEmitterViewModel> ViewModel, const FString& StackEditorDataKey, UNiagaraStackEntry::FStackIssueFixDelegate VersionUpgradeFix, TArray<UNiagaraStackEntry::FStackIssue>& OutIssues);

	struct FStackNodeGroup
	{
		TArray<UNiagaraNode*> StartNodes;
		UNiagaraNode* EndNode;
		void GetAllNodesInGroup(TArray<UNiagaraNode*>& OutAllNodes) const;
	};

	void GetStackNodeGroups(UNiagaraNode& StackNode, TArray<FStackNodeGroup>& OutStackNodeGroups);

	void DisconnectStackNodeGroup(const FStackNodeGroup& DisconnectGroup, const FStackNodeGroup& PreviousGroup, const FStackNodeGroup& NextGroup);

	void ConnectStackNodeGroup(const FStackNodeGroup& ConnectGroup, const FStackNodeGroup& NewPreviousGroup, const FStackNodeGroup& NewNextGroup);

	void InitializeStackFunctionInputs(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel, UNiagaraStackEditorData& StackEditorData, UNiagaraNodeFunctionCall& ModuleNode, UNiagaraNodeFunctionCall& InputFunctionCallNode);

	void InitializeStackFunctionInput(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel, UNiagaraStackEditorData& StackEditorData, UNiagaraNodeFunctionCall& ModuleNode, UNiagaraNodeFunctionCall& InputFunctionCallNode, FName InputName);

	namespace StackKeys
	{
		FString GenerateStackFunctionInputEditorDataKey(const UNiagaraNodeFunctionCall& FunctionCallNode, FNiagaraParameterHandle InputParameterHandle);
		FString GenerateStackModuleEditorDataKey(const UNiagaraNodeFunctionCall& ModuleNode);
		FString GenerateStackRendererEditorDataKey(const UNiagaraRendererProperties& Renderer);
	}


	TArray<FName> StackContextResolution(FVersionedNiagaraEmitter OwningEmitter, UNiagaraNodeOutput* OutputNodeInChain);
	void BuildParameterMapHistoryWithStackContextResolution(FVersionedNiagaraEmitter OwningEmitter, UNiagaraNodeOutput* OutputNodeInChain, UNiagaraNode* NodeToVisit, TArray<FNiagaraParameterMapHistory>& OutHistories, bool bRecursive = true, bool bFilterForCompilation = true);

	struct FInputDataCollection
	{
		TMap<FGuid, UNiagaraNodeFunctionCall*> NodeGuidToModuleNodeMap;
		TMap<UNiagaraHierarchyModuleInput*, FNiagaraVariable> HierarchyInputToAssignmentMap;
		TMap<UNiagaraHierarchyModuleInput*, TObjectPtr<UNiagaraScriptVariable>> HierarchyInputToScriptVariableMap;
		/** These two aren't currently in use as children inputs are handled manually by the author */
		TMap<UNiagaraHierarchyModuleInput*, TArray<FGuid>> HierarchyInputToChildrenGuidMap;
		TMap<FGuid, TObjectPtr<UNiagaraScriptVariable>> ChildrenGuidToScriptVariablesMap;
	};
	
	void GatherInputRelationsForStack(FInputDataCollection& State, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);

	enum class ENiagaraGetStackFunctionInputPinsOptions
	{
		AllInputs,
		ModuleInputsOnly
	};

	/* Returns the input pins for the given function call node. Try not to use this method if possible and use the version that accepts a FCompileConstantResolver parameter. This method should only be used if the current context is completely outside of any system or emitter (e.g. inside the graph editor itself) and constants cannot be resolved at all. */
	void GetStackFunctionInputsLegacy(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled = false);

	/* Returns the input pins for the given function call node. */
	NIAGARAEDITOR_API void GetStackFunctionInputsLegacy(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, TSet<FNiagaraVariable>& OutHiddenVariables, FCompileConstantResolver ConstantResolver, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled = false);
	NIAGARAEDITOR_API void GetStackFunctionInputsLegacy(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, FCompileConstantResolver ConstantResolver, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled = false);
	
	/* Returns the input pins for the given function call node. Try not to use this method if possible and use the version that accepts a FCompileConstantResolver parameter. This method should only be used if the current context is completely outside of any system or emitter (e.g. inside the graph editor itself) and constants cannot be resolved at all. */
	void GetStackFunctionInputs(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled = false);

	/* Returns the input pins for the given function call node. */
	NIAGARAEDITOR_API void GetStackFunctionInputs(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, TSet<FNiagaraVariable>& OutHiddenVariables, FCompileConstantResolver ConstantResolver, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled = false);
	NIAGARAEDITOR_API void GetStackFunctionInputs(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<FNiagaraVariable>& OutInputVariables, FCompileConstantResolver ConstantResolver, ENiagaraGetStackFunctionInputPinsOptions Options, bool bIgnoreDisabled = false);

	/* Returns the input pins for the given function call node.  Bypasses the module level caching of this data and generates it each call. */
	void GetStackFunctionInputPinsWithoutCache(
		const UNiagaraNodeFunctionCall& FunctionCallNode,
		TConstArrayView<FNiagaraVariable> StaticVars,
		TArray<const UEdGraphPin*>& OutInputPins,
		const FCompileConstantResolver& ConstantResolver,
		ENiagaraGetStackFunctionInputPinsOptions Options,
		bool bIgnoreDisabled,
		bool bFilterForCompilation);
		
	void ExtractInputPinsFromHistory(FNiagaraParameterMapHistory& History, UEdGraph* FunctionGraph, FNiagaraStackGraphUtilities::ENiagaraGetStackFunctionInputPinsOptions Options, TArray<const UEdGraphPin*>& OutPins);

	TArray<UNiagaraNodeOutput*> GetAllOutputNodes(TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	
	struct FMatchingFunctionInputData
	{
		FName InputName;
		FNiagaraTypeDefinition Type;
		FNiagaraVariableMetaData MetaData;
		bool bIsHidden;
		bool bIsStatic;
		UNiagaraNodeFunctionCall* FunctionCallNode = nullptr;
		TArray<FGuid> ChildrenInputGuids;
		TArray<FGuid> HiddenChildrenInputGuids;
	};

	struct FInputDataCacheKey
	{
		FInputDataCacheKey(FGuid InNodeGuid, FGuid InVariableGuid) : NodeGuid(InNodeGuid), VariableGuid(InVariableGuid)	{}
		
		FGuid NodeGuid;
		FGuid VariableGuid;
		
		bool operator==(const FInputDataCacheKey& OtherKey) const
		{
			return NodeGuid == OtherKey.NodeGuid && VariableGuid == OtherKey.VariableGuid;
		}

		bool operator!=(const FInputDataCacheKey& OtherKey) const
		{
			return !(*this == OtherKey);
		}
	};

	inline uint32 GetTypeHash(const FInputDataCacheKey& InputDataCacheKey)
	{
		uint32 Hash = 0;
		Hash = HashCombine(Hash, GetTypeHash(InputDataCacheKey.NodeGuid));
		Hash = HashCombine(Hash, GetTypeHash(InputDataCacheKey.VariableGuid));
		return Hash;
	}

	UNiagaraNodeFunctionCall* FindModuleNode(FGuid ModuleNodeGuid, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	UNiagaraNodeAssignment* FindAssignmentNode(FGuid AssignmentNodeGuid, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	UNiagaraNodeFunctionCall* FindDynamicInputNodeForInput(UNiagaraNodeFunctionCall& OwningFunctionNode, FName UnaliasedParameterName);
	/** When you have the guid of the node available, we don't need to traverse pins and can just look up the correct node directly. */
	UNiagaraNodeFunctionCall* FindFunctionCallNode(FGuid FunctionCallGuid, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	TArray<UNiagaraNodeFunctionCall*> FindModuleNodesForSimulationStage(UNiagaraSimulationStageBase& SimStage, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	TArray<UNiagaraNodeFunctionCall*> FindModuleNodesForEventHandler(FNiagaraEventScriptProperties& EventScriptProperties, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	TOptional<FMatchingFunctionInputData> FindAssignmentInputData(const UNiagaraNodeAssignment& AssignmentNode, FName VariableName, TSharedRef<FNiagaraEmitterViewModel> EmitterViewModel);
	
	/* Module script calls do not have direct inputs, but rely on the parameter map being initialized correctly. This utility function resolves which of the module's parameters are reachable during compilation and returns a list of pins on the parameter map node that do not have to be compiled. */
	TArray<UEdGraphPin*> GetUnusedFunctionInputPins(const UNiagaraNodeFunctionCall& FunctionCallNode, FCompileConstantResolver ConstantResolver);

	void GetStackFunctionStaticSwitchPinsLegacy(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<UEdGraphPin*>& OutInputPins, TSet<UEdGraphPin*>& OutHiddenPins, FCompileConstantResolver& ConstantResolver);

	NIAGARAEDITOR_API void GetStackFunctionStaticSwitchPins(const UNiagaraNodeFunctionCall& FunctionCallNode, TArray<UEdGraphPin*>& OutInputPins, TSet<UEdGraphPin*>& OutHiddenPins, FCompileConstantResolver& ConstantResolver);

	void GetStackFunctionOutputVariables(UNiagaraNodeFunctionCall& FunctionCallNode, FCompileConstantResolver ConstantResolver, TArray<FNiagaraVariable>& OutOutputVariables, TArray<FNiagaraVariable>& OutOutputVariablesWithOriginalAliasesIntact);

	/* Gather a stack function's input and output variables. Returns false if stack function does not have valid parameter map history build (e.g. no parameter map pin connected to output node of dynamic input script.) */
	bool GetStackFunctionInputAndOutputVariables(UNiagaraNodeFunctionCall& FunctionCallNode, FCompileConstantResolver ConstantResolver, TArray<FNiagaraVariable>& OutVariables, TArray<FNiagaraVariable>& OutVariablesWithOriginalAliasesIntact);

	UNiagaraNodeParameterMapSet* GetStackFunctionOverrideNode(UNiagaraNodeFunctionCall& FunctionCallNode);

	UNiagaraNodeParameterMapSet& GetOrCreateStackFunctionOverrideNode(UNiagaraNodeFunctionCall& FunctionCallNode, const FGuid& PreferredOverrideNodeGuid = FGuid());

	UEdGraphPin* GetStackFunctionInputOverridePin(UNiagaraNodeFunctionCall& StackFunctionCall, FNiagaraParameterHandle AliasedInputParameterHandle);

	NIAGARAEDITOR_API UEdGraphPin& GetOrCreateStackFunctionInputOverridePin(UNiagaraNodeFunctionCall& StackFunctionCall, FNiagaraParameterHandle AliasedInputParameterHandle, FNiagaraTypeDefinition InputType, const FGuid& InputScriptVariableId, const FGuid& PreferredOverrideNodeGuid);

	bool IsOverridePinForFunction(UEdGraphPin& OverridePin, UNiagaraNodeFunctionCall& FunctionCallNode);

	TArray<UEdGraphPin*> GetOverridePinsForFunction(UNiagaraNodeParameterMapSet& OverrideNode, UNiagaraNodeFunctionCall& FunctionCallNode);

	void RemoveNodesForStackFunctionInputOverridePin(UEdGraphPin& StackFunctionInputOverridePin);

	void RemoveNodesForStackFunctionInputOverridePin(UEdGraphPin& StackFunctinoInputOverridePin, TArray<TWeakObjectPtr<UNiagaraDataInterface>>& OutRemovedDataObjects);

	UEdGraphPin* GetLinkedValueHandleForFunctionInput(const UEdGraphPin& OverridePin);

	void GetParametersForContext(UEdGraph* Graph, UNiagaraSystem& System, TSet<FNiagaraVariableBase>& OutParameters);
	NIAGARAEDITOR_API void SetLinkedParameterValueForFunctionInput(UEdGraphPin& OverridePin, const FNiagaraVariableBase& LinkedParameter, const TSet<FNiagaraVariableBase>& KnownParameters, ENiagaraDefaultMode DesiredDefaultMode = ENiagaraDefaultMode::FailIfPreviouslyNotSet, const FGuid& NewNodePersistentId = FGuid());

	NIAGARAEDITOR_API void SetDataInterfaceValueForFunctionInput(UEdGraphPin& OverridePin, UClass* DataObjectType, FString InputNodeInputName, UNiagaraDataInterface*& OutDataObject, const FGuid& NewNodePersistentId = FGuid());

	NIAGARAEDITOR_API void SetObjectAssetValueForFunctionInput(UEdGraphPin& OverridePin, UClass* DataObjectType, FString InputNodeInputName, UObject* ObjectAsset, const FGuid& NewNodePersistentId = FGuid());

	NIAGARAEDITOR_API void SetDynamicInputForFunctionInput(UEdGraphPin& OverridePin, UNiagaraScript* DynamicInput, UNiagaraNodeFunctionCall*& OutDynamicInputFunctionCall, const FGuid& NewNodePersistentId = FGuid(), FString SuggestedName = FString(), const FGuid& InScriptVersion = FGuid());

	void SetCustomExpressionForFunctionInput(UEdGraphPin& OverridePin, const FString& CustomExpression, UNiagaraNodeCustomHlsl*& OutDynamicInputFunctionCall, const FGuid& NewNodePersistentId = FGuid());

	bool RemoveModuleFromStack(UNiagaraSystem& OwningSystem, FGuid OwningEmitterId, UNiagaraNodeFunctionCall& ModuleNode);

	bool RemoveModuleFromStack(UNiagaraSystem& OwningSystem, FGuid OwningEmitterId, UNiagaraNodeFunctionCall& ModuleNode, TArray<TWeakObjectPtr<UNiagaraNodeInput>>& OutRemovedInputNodes);

	bool RemoveModuleFromStack(UNiagaraScript& OwningScript, UNiagaraNodeFunctionCall& ModuleNode);

	bool RemoveModuleFromStack(UNiagaraScript& OwningScript, UNiagaraNodeFunctionCall& ModuleNode, TArray<TWeakObjectPtr<UNiagaraNodeInput>>& OutRemovedInputNodes);

	struct FAddScriptModuleToStackArgs
	{
	public:
		FAddScriptModuleToStackArgs(FAssetData InModuleScriptAsset, UNiagaraNodeOutput& InTargetOutputNode)
			: ModuleScriptAsset(InModuleScriptAsset)
			, TargetOutputNode(&InTargetOutputNode)
		{};

		FAddScriptModuleToStackArgs(UNiagaraScript* InModuleScript, UNiagaraNodeOutput& InTargetOutputNode)
			: ModuleScript(InModuleScript)
			, TargetOutputNode(&InTargetOutputNode)
		{};

	public:
		//~ Begin required args
		// ModuleScriptAsset OR ModuleScript is required. The script to add to the stack.
		const FAssetData ModuleScriptAsset = FAssetData();
		UNiagaraScript* const ModuleScript = nullptr;

		// The output node in the stack graph to add to.
		UNiagaraNodeOutput* const TargetOutputNode = nullptr;
		//~End required args

		// The index in the stack list in which to instert the script.
		int32 TargetIndex = INDEX_NONE;

		// The suggested name for the new script.
		FString SuggestedName = FString();

		// Whether or not to modify the final TargetIndex to be the closest value to TargetIndex which satisfies all script dependencies.
		bool bFixupTargetIndex = false;

		// The version to use for the script to add.
		FGuid VersionGuid = FGuid();

	private:
		FAddScriptModuleToStackArgs() = default;
	};

	UNiagaraNodeFunctionCall* AddScriptModuleToStack(const FAddScriptModuleToStackArgs& Args);

	UNiagaraNodeFunctionCall* AddScriptModuleToStack(FAssetData ModuleScriptAsset, UNiagaraNodeOutput& TargetOutputNode, int32 TargetIndex = INDEX_NONE, FString SuggestedName = FString());

	NIAGARAEDITOR_API UNiagaraNodeFunctionCall* AddScriptModuleToStack(UNiagaraScript* ModuleScript, UNiagaraNodeOutput& TargetOutputNode, int32 TargetIndex = INDEX_NONE, FString SuggestedName = FString(), const FGuid& VersionGuid = FGuid());
	
	bool FindScriptModulesInStack(FAssetData ModuleScriptAsset, UNiagaraNodeOutput& TargetOutputNode, TArray<UNiagaraNodeFunctionCall*>& OutFunctionCalls);

	NIAGARAEDITOR_API UNiagaraNodeAssignment* AddParameterModuleToStack(const TArray<FNiagaraVariable>& ParameterVariables, UNiagaraNodeOutput& TargetOutputNode, int32 TargetIndex, const TArray<FString>& InDefaultValues);
		
	TOptional<bool> GetModuleIsEnabled(UNiagaraNodeFunctionCall& FunctionCallNode);

	NIAGARAEDITOR_API void SetModuleIsEnabled(UNiagaraNodeFunctionCall& FunctionCallNode, bool bIsEnabled);

	bool ValidateGraphForOutput(UNiagaraGraph& NiagaraGraph, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, FText& ErrorMessage);

	UNiagaraNodeOutput* ResetGraphForOutput(UNiagaraGraph& NiagaraGraph, ENiagaraScriptUsage ScriptUsage, FGuid ScriptUsageId, const FGuid& PreferredOutputNodeGuid = FGuid(), const FGuid& PreferredInputNodeGuid = FGuid());

	bool IsRapidIterationType(const FNiagaraTypeDefinition& InputType);

	FNiagaraVariable CreateRapidIterationParameter(const FString& UniqueEmitterName, ENiagaraScriptUsage ScriptUsage, const FName& AliasedInputName, const FNiagaraTypeDefinition& InputType);

	void CleanUpStaleRapidIterationParameters(UNiagaraScript& Script, FVersionedNiagaraEmitter OwningEmitter);

	void CleanUpStaleRapidIterationParameters(FVersionedNiagaraEmitter Emitter);

	void GetNewParameterAvailableTypes(TArray<FNiagaraTypeDefinition>& OutAvailableTypes, FName Namespace);

	TOptional<FName> GetNamespaceForScriptUsage(ENiagaraScriptUsage ScriptUsage);
	TOptional<FName> GetNamespaceForOutputNode(const UNiagaraNodeOutput* OutputNode);
	
	bool IsValidDefaultDynamicInput(UNiagaraScript& OwningScript, UEdGraphPin& DefaultPin);

	bool CanWriteParameterFromUsage(FNiagaraVariable Parameter, ENiagaraScriptUsage Usage, const TOptional<FName>& StackContextOverride = TOptional<FName>(), const TArray<FName>& StackContextAllOverrides = TArray<FName>());
	bool CanWriteParameterFromUsageViaOutput(FNiagaraVariable Parameter, const UNiagaraNodeOutput* OutputNode);
	
	bool GetStackIssuesRecursively(const UNiagaraStackEntry* const Entry, TArray<UNiagaraStackErrorItem*>& OutIssues);

	void MoveModule(UNiagaraScript& SourceScript, UNiagaraNodeFunctionCall& ModuleToMove, UNiagaraSystem& TargetSystem, FGuid TargetEmitterHandleId, ENiagaraScriptUsage TargetUsage, FGuid TargetUsageId, int32 TargetModuleIndex, bool bForceCopy, UNiagaraNodeFunctionCall*& OutMovedModue);

	/** Whether a parameter is allowed to be used in a certain execution category. 
		Used to check if parameter can be dropped on a module or function stack entry. */
	NIAGARAEDITOR_API bool ParameterAllowedInExecutionCategory(const FName InParameterName, const FName ExecutionCategory);

	void RebuildEmitterNodes(UNiagaraSystem& System);

	void FindAffectedScripts(UNiagaraSystem* System, FVersionedNiagaraEmitter Emitter, UNiagaraNodeFunctionCall& ModuleNode, TArray<TWeakObjectPtr<UNiagaraScript>>& OutAffectedScripts);

	void RenameReferencingParameters(UNiagaraSystem* System, FVersionedNiagaraEmitter Emitter, UNiagaraNodeFunctionCall& FunctionCallNode, const FString& OldName, const FString& NewName);

	void GatherRenamedStackFunctionOutputVariableNames(FVersionedNiagaraEmitter Emitter, UNiagaraNodeFunctionCall& FunctionCallNode, const FString& OldFunctionName, const FString& NewFunctionName, TMap<FName, FName>& OutOldToNewNameMap);
	void GatherRenamedStackFunctionInputAndOutputVariableNames(FVersionedNiagaraEmitter Emitter, UNiagaraNodeFunctionCall& FunctionCallNode, const FString& OldFunctionName, const FString& NewFunctionName, TMap<FName, FName>& OutOldToNewNameMap);

	enum class EStackEditContext
	{
		System,
		Emitter
	};

	/** Gets the valid namespaces which new parameters for this usage can be read from. */
	void GetNamespacesForNewReadParameters(EStackEditContext EditContext, ENiagaraScriptUsage Usage, TArray<FName>& OutNamespacesForNewParameters);

	/** Gets the valid namespaces which new parameters for this usage can write to. */
	void GetNamespacesForNewWriteParameters(EStackEditContext EditContext, ENiagaraScriptUsage Usage, const TOptional<FName>& StackContextAlias, TArray<FName>& OutNamespacesForNewParameters);

	bool TryRenameAssignmentTarget(UNiagaraNodeAssignment& OwningAssignmentNode, FNiagaraVariable CurrentAssignmentTarget, FName NewAssignmentTargetName);

	void RenameAssignmentTarget(
		UNiagaraSystem& OwningSystem,
		FVersionedNiagaraEmitter OwningEmitter,
		UNiagaraScript& OwningScript,
		UNiagaraNodeAssignment& OwningAssignmentNode,
		FNiagaraVariable CurrentAssignmentTarget,
		FName NewAssignmentTargetName);

	/** Helper to add a new pin to a ParameterMapBaseNode, and conditionally add a new parameter to the underlying UNiagaraGraph if it does not yet exist. */
	NIAGARAEDITOR_API void AddNewVariableToParameterMapNode(UNiagaraNodeParameterMapBase* MapBaseNode, bool bCreateInputPin, const FNiagaraVariable& NewVariable);
	NIAGARAEDITOR_API void AddNewVariableToParameterMapNode(UNiagaraNodeParameterMapBase* MapBaseNode, bool bCreateInputPin, const UNiagaraScriptVariable* NewScriptVar);

	void SynchronizeVariableToLibraryAndApplyToGraph(UNiagaraScriptVariable* ScriptVarToSync);

	void PopulateFunctionCallNameBindings(UNiagaraNodeFunctionCall& InFunctionCallNode);

	NIAGARAEDITOR_API void SynchronizeReferencingMapPinsWithFunctionCall(UNiagaraNodeFunctionCall& InFunctionCallNode);

	FGuid GetScriptVariableIdForLinkedModuleParameter(const FNiagaraVariableBase& LinkedParameter, UNiagaraGraph& TargetGraph);

	/** Fixes orphaned and disconnected output pins on dynamic input nodes resulting from the underlying script changing and the pins being reallocated. */
	void FixDynamicInputNodeOutputPinsFromExternalChanges(UNiagaraNodeFunctionCall& InFunctionCallNode);

	void GetEmitterHandleAndCompiledScriptsForStackNode(const UNiagaraSystem& OwningSystem, UNiagaraNode& StackNode, const FNiagaraEmitterHandle*& OutEmitterHandle, TArray<const UNiagaraScript*>& OutCompiledScripts);

	namespace DependencyUtilities
	{
		bool DoesStackModuleProvideDependency(const FNiagaraStackModuleData& StackModuleData, const FNiagaraModuleDependency& SourceModuleRequiredDependency, const UNiagaraNodeOutput& SourceOutputNode);

		void GetModuleScriptAssetsByDependencyProvided(FName DependencyName, TOptional<ENiagaraScriptUsage> RequiredUsage, TArray<FAssetData>& OutAssets);

		int32 FindBestIndexForModuleInStack(UNiagaraNodeFunctionCall& ModuleNode, const UNiagaraNodeOutput& TargetOutputNode);
	}
}
