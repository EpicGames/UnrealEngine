// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraClipboard.h"
#include "NiagaraStackScriptHierarchyRoot.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "NiagaraTypes.h"
#include "DataHierarchyViewModelBase.h"
#include "ViewModels/HierarchyEditor/NiagaraSummaryViewViewModel.h"
#include "NiagaraStackInputCategory.generated.h"

class UNiagaraStackFunctionInput;
class UNiagaraNodeFunctionCall;
class UNiagaraClipboardFunctionInput;

UENUM()
enum class EStackParameterBehavior
{
	Dynamic, Static
};

UCLASS(MinimalAPI)
class UNiagaraStackCategory : public UNiagaraStackItemContent
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FString InOwningStackItemEditorDataKey, FString InStackEditorDataKey);

	//~ UNiagaraStackEntry interface
	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual bool GetShouldShowInStack() const override;
	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;
	NIAGARAEDITOR_API virtual void GetSearchItems(TArray<FStackSearchItem>& SearchItems) const override;

	virtual bool IsTopLevelCategory() const { return false; }
	NIAGARAEDITOR_API virtual int32 GetChildIndentLevel() const override;

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
private:
private:
	NIAGARAEDITOR_API bool FilterForVisibleCondition(const UNiagaraStackEntry& Child) const;
	bool FilterOnlyModified(const UNiagaraStackEntry& Child) const;
	NIAGARAEDITOR_API bool FilterForIsInlineEditConditionToggle(const UNiagaraStackEntry& Child) const;
protected:
	bool bShouldShowInStack;

	UPROPERTY()
	TObjectPtr<UNiagaraStackSpacer> CategorySpacer;
};

UCLASS(MinimalAPI)
class UNiagaraStackScriptHierarchyCategory : public UNiagaraStackCategory
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, const UHierarchyCategory& InHierarchyCategory, FString InOwningStackItemEditorDataKey, FString InStackEditorDataKey);

	/** UNiagaraStackEntry */
	virtual bool SupportsCopy() const override { return true; }
	virtual bool SupportsPaste() const override { return true; }
	virtual void Copy(UNiagaraClipboardContent* ClipboardContent) const override;
	virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;
	virtual bool TestCanCopyWithMessage(FText& OutMessage) const override;
	virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;

	void PasteFromClipboard(const UNiagaraClipboardContent* ClipboardContent);
	TArray<const UNiagaraClipboardFunctionInput*> ToClipboardFunctionInputs(UObject* InOuter) const;
	
	virtual FText GetDisplayName() const override;
	virtual FText GetTooltipText() const override;
	
	void SetOwningFunctionCallNode(UNiagaraNodeFunctionCall& InOnGetOwningFunctionCallNode) { OwningFunctionCallNode = &InOnGetOwningFunctionCallNode;}
	void SetOwningModuleNode(UNiagaraNodeFunctionCall& InOnGetOwningModuleNode) { OwningModuleNode = &InOnGetOwningModuleNode;}
	
	void SetScriptInstanceData(const FNiagaraScriptInstanceData& InScriptInstanceData) { ScriptInstanceData = InScriptInstanceData; }
	
	const UHierarchyCategory* GetHierarchyCategory() const { return HierarchyCategory.Get(); }
protected:
	virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	virtual bool IsTopLevelCategory() const override { return HierarchyCategory->GetOuter()->IsA<UHierarchyRoot>(); }
protected:
	TWeakObjectPtr<const UHierarchyCategory> HierarchyCategory;

	TWeakObjectPtr<UNiagaraNodeFunctionCall> OwningModuleNode;
	TWeakObjectPtr<UNiagaraNodeFunctionCall> OwningFunctionCallNode;
	
	FNiagaraScriptInstanceData ScriptInstanceData;
};

UCLASS(MinimalAPI)
class UNiagaraStackSummaryCategory : public UNiagaraStackCategory
{
	GENERATED_BODY()

public:
	UNiagaraStackSummaryCategory() {}

	NIAGARAEDITOR_API void Initialize(
		FRequiredEntryData InRequiredEntryData,
		TSharedPtr<FHierarchyCategoryViewModel> InCategory,
		FString InOwnerStackItemEditorDataKey);

	//~ UNiagaraStackEntry interface
	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	virtual bool GetIsEnabled() const override { return true; }
	
	TWeakPtr<FHierarchyCategoryViewModel> GetHierarchyCategory() const { return CategoryViewModelWeakPtr; }

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
	NIAGARAEDITOR_API virtual bool IsTopLevelCategory() const override;
	NIAGARAEDITOR_API virtual FText GetTooltipText() const override;
	NIAGARAEDITOR_API virtual int32 GetChildIndentLevel() const override;

private:
	TWeakPtr<FHierarchyCategoryViewModel> CategoryViewModelWeakPtr;
};
