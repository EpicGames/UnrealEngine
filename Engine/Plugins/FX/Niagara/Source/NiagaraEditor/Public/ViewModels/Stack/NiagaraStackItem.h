// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "NiagaraScriptHighlight.h"
#include "Layout/Visibility.h"
#include "NiagaraStackItem.generated.h"

struct FSlateBrush;
class UNiagaraStackItemFooter;
class UNiagaraNode;
class UNiagaraClipboardContent;

class INiagaraStackItemHeaderValueHandler
{
public: 
	enum class EValueMode
	{
		BoolToggle,
		EnumDropDown
	};

public:
	virtual ~INiagaraStackItemHeaderValueHandler() = default;
	virtual EValueMode GetMode() const = 0;
	virtual const UEnum* GetEnum() const = 0;
	virtual const FText& GetLabelText() const = 0;
	virtual const FSlateBrush* GetIconBrush() const = 0;
	virtual const EHorizontalAlignment GetHAlign() const = 0;

	virtual bool GetBoolValue() const = 0;
	virtual void NotifyBoolValueChanged(bool bInValue) = 0;

	virtual int32 GetEnumValue() const = 0;
	virtual void NotifyEnumValueChanged(int32 bInValue) = 0;
};

struct FNiagaraStackItemCustomIcon
{
	const FSlateBrush*		IconBrush = nullptr;
	TAttribute<FText>		ToolTipText;
	TAttribute<EVisibility>	VisibleFunction;
};

UCLASS(MinimalAPI)
class UNiagaraStackItem : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FOnModifiedGroupItems);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnRequestCanPaste, const UNiagaraClipboardContent* /* ClipboardContent */, FText& /* OutCanPasteMessage */);
	DECLARE_DELEGATE_ThreeParams(FOnRequestPaste, const UNiagaraClipboardContent* /* ClipboardContent */, int32 /* PasteIndex */, FText& /* OutPasteWarning */);

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FString InStackEditorDataKey);

	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;

	NIAGARAEDITOR_API FOnModifiedGroupItems& OnModifiedGroupItems();

	NIAGARAEDITOR_API void SetOnRequestCanPaste(FOnRequestCanPaste InOnRequestCanPaste);
	NIAGARAEDITOR_API void SetOnRequestPaste(FOnRequestPaste InOnRequestCanPaste);

	virtual bool SupportsChangeEnabled() const { return false; }
	NIAGARAEDITOR_API void SetIsEnabled(bool bInIsEnabled);

	virtual bool SupportsResetToBase() const { return false; }
	virtual bool TestCanResetToBaseWithMessage(FText& OutCanResetToBaseMessage) const { return false; }
	virtual void ResetToBase() { }

	virtual bool SupportsStackNotes() const override { return true; }
	
	virtual bool SupportsEditMode() const { return false; }
	virtual void OnEditButtonClicked() { }
	virtual TOptional<FText> GetEditModeButtonText() const { return TOptional<FText>(); }
	virtual TOptional<FText> GetEditModeButtonTooltip() const { return TOptional<FText>(); }
	virtual EVisibility IsEditButtonVisible() const { return SupportsEditMode() ? EVisibility::Visible : EVisibility::Collapsed; }

	virtual bool SupportsHeaderValues() const { return false; }
	virtual void GetHeaderValueHandlers(TArray<TSharedRef<INiagaraStackItemHeaderValueHandler>>& OutHeaderValueHandlers) const { };

	virtual bool GetIsInherited() const { return false; }

	virtual bool SupportsDebugDraw() const { return false; }
	virtual TOptional<FText> GetDebugDrawTooltip() const { return {}; }
	virtual bool IsDebugDrawEnabled() const { return false; }
	virtual void SetDebugDrawEnabled(bool bInEnabled) {}

	virtual TArray<FNiagaraStackItemCustomIcon> GetCustomIcons() const { return {}; }

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	NIAGARAEDITOR_API virtual void PostRefreshChildrenInternal() override;

	NIAGARAEDITOR_API virtual int32 GetChildIndentLevel() const override;

	virtual void SetIsEnabledInternal(bool bInIsEnabled) { }

private:
	NIAGARAEDITOR_API bool FilterAdvancedChildren(const UNiagaraStackEntry& Child) const;

	NIAGARAEDITOR_API bool FilterHiddenChildren(const UNiagaraStackEntry& Child) const;

	NIAGARAEDITOR_API void ToggleShowAdvanced();

	virtual void ToggleShowAdvancedInternal();

protected:
	FOnModifiedGroupItems ModifiedGroupItemsDelegate;
	FOnRequestCanPaste RequestCanPasteDelegete;
	FOnRequestPaste RequestPasteDelegate;

private:
	UPROPERTY()
	TObjectPtr<UNiagaraStackItemFooter> ItemFooter;
};

UCLASS(MinimalAPI)
class UNiagaraStackItemContent : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FString InOwningStackItemEditorDataKey, FString InStackEditorDataKey);

	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;

	NIAGARAEDITOR_API bool GetIsAdvanced() const;

	NIAGARAEDITOR_API bool GetIsHidden() const;

	NIAGARAEDITOR_API void SetIsHidden(bool bInIsHidden);

	// Returns true if this stack entry was changed by a user and differs from the default value
	NIAGARAEDITOR_API virtual bool HasOverridenContent() const;

	NIAGARAEDITOR_API bool FilterHiddenChildren(const UNiagaraStackEntry& Child) const;
protected:
	NIAGARAEDITOR_API FString GetOwnerStackItemEditorDataKey() const;

	NIAGARAEDITOR_API void SetIsAdvanced(bool bInIsAdvanced);

private:
	bool FilterAdvancedChildren(const UNiagaraStackEntry& Child) const;

private:
	FString OwningStackItemEditorDataKey;
	bool bIsAdvanced;
	bool bIsHidden;
};

UCLASS(MinimalAPI)
class UNiagaraStackItemTextContent : public UNiagaraStackItemContent
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FText InDisplayText, FString InOwningStackItemEditorDataKey);

	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;

private:
	FText DisplayText;
};
