// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "NiagaraStackItemFooter.generated.h"

class UNiagaraNodeFunctionCall;
class UNiagaraStackModuleItemOutputCollection;
class UNiagaraNode;

UCLASS(MinimalAPI)
class UNiagaraStackItemFooter : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE(FOnToggleShowAdvanced);

public:
	NIAGARAEDITOR_API void Initialize(
		FRequiredEntryData InRequiredEntryData,
		FString InOwnerStackItemEditorDataKey);

	NIAGARAEDITOR_API virtual bool GetCanExpand() const override;
	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;

	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;
	NIAGARAEDITOR_API void SetIsEnabled(bool bInIsEnabled);

	NIAGARAEDITOR_API bool GetHasAdvancedContent() const;
	NIAGARAEDITOR_API void SetHasAdvancedContent(bool bHInHasAdvancedRows, bool bHasChangedAdvancedContent);

	NIAGARAEDITOR_API void SetOnToggleShowAdvanced(FOnToggleShowAdvanced OnToggleShowAdvanced);

	NIAGARAEDITOR_API bool GetShowAdvanced() const;

	NIAGARAEDITOR_API void ToggleShowAdvanced();

	bool HasChangedAdvancedContent() const { return bHasChangedAdvancedContent; }

private:
	FString OwnerStackItemEditorDataKey;

	FOnToggleShowAdvanced ToggleShowAdvancedDelegate;

	bool bIsEnabled;

	bool bHasAdvancedContent;

	bool bHasChangedAdvancedContent;
};
