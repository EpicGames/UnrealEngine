// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/INiagaraStackItemGroupAddUtilities.h"
#include "Layout/Visibility.h"
#include "NiagaraStackItemGroup.generated.h"

class FNiagaraEmitterHandleViewModel;
class UNiagaraStackItemGroupFooter;
struct FSlateBrush;

UCLASS(MinimalAPI)
class UNiagaraStackItemGroup : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FText InDisplayName, FText InToolTip, INiagaraStackItemGroupAddUtilities* InAddUtilities);

	//~ UNiagaraStackEntry interface
	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;
	NIAGARAEDITOR_API virtual FText GetTooltipText() const override;

	virtual bool GetCanExpandInOverview() const override { return true; }
	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;
	virtual void SetIsEnabled(bool bEnabled) {}
	virtual bool SupportsChangeEnabled() const { return false; }

	virtual bool SupportsSecondaryIcon() const { return false; }
	virtual const FSlateBrush* GetSecondaryIconBrush() const { return nullptr; };

	NIAGARAEDITOR_API INiagaraStackItemGroupAddUtilities* GetAddUtilities() const;

	NIAGARAEDITOR_API uint32 GetRecursiveStackIssuesCount() const;
	NIAGARAEDITOR_API EStackIssueSeverity GetHighestStackIssueSeverity() const;
	
protected:
	NIAGARAEDITOR_API void SetDisplayName(FText InDisplayName);
	
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	NIAGARAEDITOR_API virtual int32 GetChildIndentLevel() const override;

	NIAGARAEDITOR_API virtual void ChildStructureChangedInternal() override;

private:
	NIAGARAEDITOR_API bool FilterChildrenWithIssues(const UNiagaraStackEntry& Child) const;

private:
	UPROPERTY()
	TObjectPtr<UNiagaraStackItemGroupFooter> GroupFooter;

	INiagaraStackItemGroupAddUtilities* AddUtilities;

	FText GroupDisplayName;
	FText GroupToolTip;

	/** How many errors this entry has along its tree. */
	mutable TOptional<uint32> RecursiveStackIssuesCount;
	/** The highest severity of issues along this entry's tree. */
	mutable TOptional<EStackIssueSeverity> HighestIssueSeverity;

	TWeakPtr<FNiagaraEmitterHandleViewModel> OwningEmitterHandleViewModelWeak;
};

UCLASS(MinimalAPI)
class UNiagaraStackItemGroupFooter : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData);

	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;

	NIAGARAEDITOR_API virtual bool GetCanExpand() const;
};
