// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraStackScriptHierarchyRoot.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "NiagaraTypes.h"
#include "NiagaraStackModuleItem.generated.h"

class UNiagaraNodeFunctionCall;
class UNiagaraStackModuleItemLinkedInputCollection;
class UNiagaraStackModuleItemOutputCollection;
class UNiagaraScript;
class INiagaraStackItemGroupAddUtilities;
struct FAssetData;
class UNiagaraClipboardFunctionInput;
class INiagaraMessage;

UCLASS(MinimalAPI)
class UNiagaraStackModuleItem : public UNiagaraStackItem
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE_OneParam(FOnRequestDeprecationRecommended, UNiagaraStackModuleItem*);
	DECLARE_DELEGATE_OneParam(FOnNoteModeSet, bool);

	NIAGARAEDITOR_API UNiagaraStackModuleItem();

	NIAGARAEDITOR_API UNiagaraNodeFunctionCall& GetModuleNode() const;

	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, INiagaraStackItemGroupAddUtilities* GroupAddUtilities, UNiagaraNodeFunctionCall& InFunctionCallNode);

	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual UObject* GetDisplayedObject() const override;
	NIAGARAEDITOR_API virtual FText GetTooltipText() const override;
	virtual bool UsesCustomTooltipWidget() const override;
	virtual TSharedPtr<IToolTip> GetCustomTooltipWidget() const override;

	NIAGARAEDITOR_API INiagaraStackItemGroupAddUtilities* GetGroupAddUtilities();

	NIAGARAEDITOR_API bool CanMoveAndDelete() const;
	NIAGARAEDITOR_API bool CanRefresh() const;
	NIAGARAEDITOR_API void Refresh();

	virtual bool SupportsRename() const override { return true; }

	virtual bool SupportsChangeEnabled() const override { return true; }
	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;

	NIAGARAEDITOR_API int32 GetModuleIndex() const;
	
	NIAGARAEDITOR_API UObject* GetExternalAsset() const override;

	NIAGARAEDITOR_API virtual bool CanDrag() const override;

	/** Gets the output node of this module. */
	NIAGARAEDITOR_API class UNiagaraNodeOutput* GetOutputNode() const;

	NIAGARAEDITOR_API bool CanAddInput(FNiagaraVariable InputParameter) const;

	NIAGARAEDITOR_API void AddInput(FNiagaraVariable InputParameter);

	/** Gets whether or not a module script reassignment is pending.  This can happen when trying to fix modules which are missing their scripts. */
	NIAGARAEDITOR_API bool GetIsModuleScriptReassignmentPending() const;

	/** Gets whether or not a module script reassignment should be be pending. */
	NIAGARAEDITOR_API void SetIsModuleScriptReassignmentPending(bool bIsPending);

	/** Reassigns the function script for the module without resetting the inputs. */
	NIAGARAEDITOR_API void ReassignModuleScript(UNiagaraScript* ModuleScript);
	
	NIAGARAEDITOR_API void ChangeScriptVersion(FGuid NewScriptVersion);

	NIAGARAEDITOR_API void SetInputValuesFromClipboardFunctionInputs(const TArray<const UNiagaraClipboardFunctionInput*>& ClipboardFunctionInputs);

	NIAGARAEDITOR_API void GetParameterInputs(TArray<class UNiagaraStackFunctionInput*>& OutResult) const;
	NIAGARAEDITOR_API TArray<class UNiagaraStackFunctionInput*> GetInlineParameterInputs() const;

	virtual bool SupportsCut() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanCutWithMessage(FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual FText GetCutTransactionText() const override;
	NIAGARAEDITOR_API virtual void CopyForCut(UNiagaraClipboardContent* ClipboardContent) const override;
	NIAGARAEDITOR_API virtual void RemoveForCut() override;

	virtual bool SupportsCopy() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanCopyWithMessage(FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual void Copy(UNiagaraClipboardContent* ClipboardContent) const override;

	virtual bool SupportsPaste() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;
	NIAGARAEDITOR_API virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

	virtual bool SupportsDelete() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const override;
	NIAGARAEDITOR_API virtual FText GetDeleteTransactionText() const override;
	NIAGARAEDITOR_API virtual void Delete() override;

	virtual bool SupportsInheritance() const override { return true; }
	NIAGARAEDITOR_API virtual bool GetIsInherited() const override;
	NIAGARAEDITOR_API virtual FText GetInheritanceMessage() const override;

	virtual bool SupportsSummaryView() const override { return true; }
	NIAGARAEDITOR_API virtual FHierarchyElementIdentity DetermineSummaryIdentity() const override;
	
	NIAGARAEDITOR_API bool IsScratchModule() const;

	void SetOnRequestDeprecationRecommended(FOnRequestDeprecationRecommended InOnRequest)
	{
		DeprecationDelegate = InOnRequest;
	}

	void SetEnabled(bool bEnabled)
	{
		SetIsEnabledInternal(bEnabled);
	}

	NIAGARAEDITOR_API virtual bool SupportsDebugDraw() const override;
	NIAGARAEDITOR_API virtual TOptional<FText> GetDebugDrawTooltip() const override;
	NIAGARAEDITOR_API virtual bool IsDebugDrawEnabled() const override;
	NIAGARAEDITOR_API virtual void SetDebugDrawEnabled(bool bInEnabled) override;

	NIAGARAEDITOR_API bool OpenSourceAsset() const;

protected:
	FOnRequestDeprecationRecommended DeprecationDelegate;

	NIAGARAEDITOR_API virtual void FinalizeInternal() override;
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
	NIAGARAEDITOR_API virtual void SetIsEnabledInternal(bool bInIsEnabled) override;

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> CanDropInternal(const FDropRequest& DropRequest) override;
	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> DropInternal(const FDropRequest& DropRequest) override;

	NIAGARAEDITOR_API virtual const FCollectedUsageData& GetCollectedUsageData() const override;

private:
	NIAGARAEDITOR_API bool FilterOutputCollection(const UNiagaraStackEntry& Child) const;
	NIAGARAEDITOR_API bool FilterOutputCollectionChild(const UNiagaraStackEntry& Child) const;
	NIAGARAEDITOR_API bool FilterLinkedInputCollection(const UNiagaraStackEntry& Child) const;
	NIAGARAEDITOR_API bool FilterLinkedInputCollectionChild(const UNiagaraStackEntry& Child) const;
	NIAGARAEDITOR_API void RefreshIssues(TArray<FStackIssue>& NewIssues);
	NIAGARAEDITOR_API void RefreshIsEnabled();
	NIAGARAEDITOR_API void OnMessageManagerRefresh(const TArray<TSharedRef<const INiagaraMessage>>& NewMessages);
	NIAGARAEDITOR_API FStackIssueFixDelegate GetUpgradeVersionFix();
	void ReportScriptVersionChange() const;

private:
	UNiagaraNodeOutput* OutputNode;
	UNiagaraNodeFunctionCall* FunctionCallNode;
	mutable TOptional<bool> bCanMoveAndDeleteCache;
	mutable TOptional<FText> DisplayNameCache;
	bool bIsEnabled;
	bool bCanRefresh;
	mutable TSharedPtr<IToolTip> TooltipCache;

	UPROPERTY()
	TObjectPtr<UNiagaraStackModuleItemLinkedInputCollection> LinkedInputCollection;
	
	UPROPERTY()
	TObjectPtr<UNiagaraStackScriptHierarchyRoot> InputRoot;
	
	UPROPERTY()
	TObjectPtr<UNiagaraStackModuleItemOutputCollection> OutputCollection;

	INiagaraStackItemGroupAddUtilities* GroupAddUtilities;

	mutable TOptional<bool> bIsScratchModuleCache;
	
	bool bIsModuleScriptReassignmentPending;

	FGuid MessageManagerRegistrationKey;

	//** Issues created outside of the RefreshChildren call that will be committed the next time the UI state is refreshed. */
	TArray<FStackIssue> MessageManagerIssues;

	FGuid MessageLogGuid;

	FGuid LastRefreshVersionCache;
};
