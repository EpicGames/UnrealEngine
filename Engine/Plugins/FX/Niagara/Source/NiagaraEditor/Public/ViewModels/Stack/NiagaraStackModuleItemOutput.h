// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraParameterHandle.h"
#include "NiagaraTypes.h"
#include "NiagaraStackModuleItemOutput.generated.h"

class UNiagaraNodeFunctionCall;

/** Represents a single module Output in the module stack view model. */
UCLASS(MinimalAPI)
class UNiagaraStackModuleItemOutput : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API UNiagaraStackModuleItemOutput();

	/** 
	 * Sets the Output data for this entry.
	 * @param InFunctionCallNode The function call node representing the module in the stack graph which owns this Output.
	 * @param InOutputParameterHandle The Namespace.Name handle for the Output to the owning module.
	 */
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraNodeFunctionCall& InFunctionCallNode, FName InOutputParameterHandle, FNiagaraTypeDefinition InOutputType);

	//~ UNiagaraStackEntry interface
	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual FText GetTooltipText() const override;
	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;
	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;
	NIAGARAEDITOR_API virtual void GetSearchItems(TArray<FStackSearchItem>& SearchItems) const override;

	/** Gets the parameter handle which defined this Output in the module. */
	NIAGARAEDITOR_API const FNiagaraParameterHandle& GetOutputParameterHandle() const;

	/** Gets the assigned parameter handle as displayable text. */
	NIAGARAEDITOR_API FText GetOutputParameterHandleText() const;

	NIAGARAEDITOR_API virtual const FCollectedUsageData& GetCollectedUsageData() const override;

private:
	/** The function call node which represents the emitter which owns this Output */
	TWeakObjectPtr<UNiagaraNodeFunctionCall> FunctionCallNode;

	/** The parameter handle which defined this Output in the module graph. */
	FNiagaraParameterHandle OutputParameterHandle;

	/** The name of this Output for display in the UI. */
	FText DisplayName;

	/** The niagara type definition for this input. */
	FNiagaraTypeDefinition OutputType;
};
