// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "NiagaraStackModuleItemOutputCollection.generated.h"

class UNiagaraNodeFunctionCall;

UCLASS(MinimalAPI)
class UNiagaraStackModuleItemOutputCollection : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API UNiagaraStackModuleItemOutputCollection();

	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraNodeFunctionCall& InFunctionCallNode);
	
	//~ UNiagaraStackEntry interface
	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual bool IsExpandedByDefault() const override;
	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;
	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	void GetCurrentChangeIds(FGuid& OutOwningGraphChangeId, FGuid& OutFunctionGraphChangeId) const;

private:
	UNiagaraNodeFunctionCall* FunctionCallNode;

	// The value of the change id for the graph which owns the function call as of the last refresh.
	TOptional<FGuid> LastOwningGraphChangeId;

	// The value of the change id for the called graph of the function call as of the last refresh.
	TOptional<FGuid> LastFunctionGraphChangeId;
};
