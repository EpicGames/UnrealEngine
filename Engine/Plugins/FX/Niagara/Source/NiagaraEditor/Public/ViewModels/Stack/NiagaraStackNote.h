// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraClipboard.h"
#include "NiagaraMessages.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraStackEntry.h"
#include "Styling/SlateTypes.h"
#include "NiagaraStackNote.generated.h"

UCLASS(MinimalAPI)
class UNiagaraStackNote : public UNiagaraStackEntry
{
	DECLARE_DELEGATE_OneParam(FOnNoteChanged, FNiagaraStackNoteData)
	
	GENERATED_BODY()
public:
	void Initialize(FRequiredEntryData InRequiredEntryData, FString InTargetStackEntryKey);

	NIAGARAEDITOR_API FString GetTargetStackEntryKey() const;
	NIAGARAEDITOR_API TOptional<FNiagaraStackNoteData> GetTargetStackNoteData() const;

	void RequestEditHeader() const { OnRequestEditHeaderDelegate.ExecuteIfBound(); }
	NIAGARAEDITOR_API void ToggleInlineDisplay();
	NIAGARAEDITOR_API void DeleteTargetStackNote();
	NIAGARAEDITOR_API void UpdateNoteColor(FLinearColor LinearColor);

	FSimpleDelegate& OnRequestEditHeader() { return OnRequestEditHeaderDelegate; }

	/** The delegate for the target stack entry to bind to. Forwards note changes which then need to be set via UNiagaraStackEntry::SetStackNoteData. */
	FOnNoteChanged& OnNoteChanged() { return OnNoteChangedDelegate; }

	virtual bool GetShouldShowInStack() const override;

	NIAGARAEDITOR_API FLinearColor GetColor() const;
protected:
	virtual bool GetShouldShowInOverview() const override { return false; }
	virtual EStackRowStyle GetStackRowStyle() const override;
	virtual bool GetCanExpand() const override { return false; }
	virtual bool KeepExpanderIndentation() const override { return false; }
	
	virtual bool SupportsRename() const override { return true; }
	virtual bool SupportsCopy() const override { return true; }
	virtual bool SupportsPaste() const override { return true; }
	
	virtual void Copy(UNiagaraClipboardContent* ClipboardContent) const override;
	virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

	virtual bool TestCanCopyWithMessage(FText& OutMessage) const override;
	virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	
	FSimpleDelegate OnRequestEditHeaderDelegate;
	FOnNoteChanged OnNoteChangedDelegate;

	FString TargetStackEntryKey;
};
