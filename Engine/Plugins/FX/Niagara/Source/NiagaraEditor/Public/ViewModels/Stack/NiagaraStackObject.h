// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackObjectShared.h"
#include "Misc/NotifyHook.h"
#include "PropertyEditorDelegates.h"
#include "NiagaraStackObject.generated.h"

class IPropertyRowGenerator;
class UNiagaraNode;
class IDetailTreeNode;
class INiagaraMessage;

class FStructOnScope;

UCLASS(MinimalAPI)
class UNiagaraStackObject : public UNiagaraStackItemContent, public FNotifyHook
{
	GENERATED_BODY()
		
public:
	NIAGARAEDITOR_API UNiagaraStackObject();

	/** Initialized an entry for displaying a UObject in the niagara stack. 
		@param InRequiredEntryData Struct with required data for all stack entries.
		@param InObject The object to be displayed in the stack.
		@param bInIsTopLevelObject Whether or not the object being displayed should be treated as a top level object shown in the root of the stack.  This can be used to make layout decisions such as category styling.
		@param InOwnerStackItemEditorDataKey The stack editor data key of the owning stack entry. 
		@param InOwningNiagaraNode An optional niagara node which owns this object. */
	NIAGARAEDITOR_API void Initialize(
		FRequiredEntryData InRequiredEntryData,
		UObject* InObject,
		bool bInIsTopLevelObject,
		bool bInHideTopLevelCategories,
		FString InOwnerStackItemEditorDataKey,
		UNiagaraNode* InOwningNiagaraNode = nullptr);

	/** Initializes an entry for displaying a UStruct in the niagara stack.
		@param InRequiredEntryData Struct with required data for all stack entries.
		@param InOwningObject The object which owns the struct being displayed in the stack.
		@param InStruct The struct which will be displayed in the stack.
		@param InStructName A unique name for the struct which is used for identification and for generating the stack editor data key.
		@param bInIsTopLevelStruct Whether or not the struct being displayed should be treated as a top level object shown in the root of the stack.  This can be used to make layout decisions such as category styling.
		@param InOwnerStackItemEditorDataKey The stack editor data key of the owning stack entry.
		@param InOwningNiagaraNode An optional niagara node which owns the displayed struct. */
	void Initialize(
		FRequiredEntryData InRequiredEntryData,
		UObject* InOwningObject,
		TSharedRef<FStructOnScope> InDisplayedStruct,
		const FString& InStructName,
		bool bInIsTopLevelStruct,
		bool bInHideTopLevelCategories,
		FString InOwnerStackItemEditorDataKey,
		UNiagaraNode* InOwningNiagaraNode = nullptr);

	enum class EDetailNodeFilterMode
	{
		FilterRootNodesOnly,
		FilterAllNodes
	};

	NIAGARAEDITOR_API void SetOnFilterDetailNodes(FNiagaraStackObjectShared::FOnFilterDetailNodes InOnFilterDetailNodes, EDetailNodeFilterMode FilterMode = EDetailNodeFilterMode::FilterRootNodesOnly);

	NIAGARAEDITOR_API void RegisterInstancedCustomPropertyLayout(UStruct* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate);
	NIAGARAEDITOR_API void RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr);

	UObject* GetObject() const { return WeakObject.Get(); }
	TSharedPtr<FStructOnScope> GetDisplayedStruct() const { return DisplayedStruct; }

	//~ FNotifyHook interface
	NIAGARAEDITOR_API virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	NIAGARAEDITOR_API virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

	//~ UNiagaraStackEntry interface+
	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;
	NIAGARAEDITOR_API virtual bool GetShouldShowInStack() const override;
	NIAGARAEDITOR_API virtual UObject* GetDisplayedObject() const override;

	void SetObjectGuid(FGuid InGuid) { ObjectGuid = InGuid; }
	virtual bool SupportsSummaryView() const override { return ObjectGuid.IsSet() && ObjectGuid->IsValid(); }
	NIAGARAEDITOR_API virtual FHierarchyElementIdentity DetermineSummaryIdentity() const override;
	
	/** Gets a custom name for indentifying this stack object. */
	FName GetCustomName() const { return CustomName; }

	/** Sets a custon name for identifying this stack object. */
	void SetCustomName(FName InCustomName) { CustomName = InCustomName; }

protected:
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
	NIAGARAEDITOR_API virtual void PostRefreshChildrenInternal() override;

	NIAGARAEDITOR_API bool FilterOnlyModified(const UNiagaraStackEntry& Child) const;
private:
	NIAGARAEDITOR_API void PropertyRowsRefreshed();
	NIAGARAEDITOR_API void OnMessageManagerRefresh(const TArray<TSharedRef<const INiagaraMessage>>& NewMessages);

private:
	struct FRegisteredClassCustomization
	{
		UStruct* Class;
		FOnGetDetailCustomizationInstance DetailLayoutDelegate;
	};

	struct FRegisteredPropertyCustomization
	{
		FName PropertyTypeName;
		FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate;
		TSharedPtr<IPropertyTypeIdentifier> Identifier;
	};

	TWeakObjectPtr<UObject> WeakObject;
	TSharedPtr<FStructOnScope> DisplayedStruct;

	// Whether or not the object being displayed should be treated as a top level object shown in the root of the stack.  This can be used to make layout decisions such as category styling.
	bool bIsTopLevel;

	bool bHideTopLevelCategories;

	UNiagaraNode* OwningNiagaraNode;
	FNiagaraStackObjectShared::FOnFilterDetailNodes OnFilterDetailNodesDelegate;
	EDetailNodeFilterMode FilterMode;
	TArray<FRegisteredClassCustomization> RegisteredClassCustomizations;
	TArray<FRegisteredPropertyCustomization> RegisteredPropertyCustomizations;
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	/** An optional object guid that can be provided to identify the object this stack entry represents. This can be used for summary view purposes and is also given to the child property rows. */
	TOptional<FGuid> ObjectGuid;

	FName CustomName = NAME_None;
	
	FGuid MessageManagerRegistrationKey;
	TArray<FStackIssue> MessageManagerIssues;
	FGuid MessageLogGuid;
};
