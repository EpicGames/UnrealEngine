// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackObject.h"

struct FNiagaraPlatformSet;

class INiagaraStackObjectIssueGenerator
{
public:
	virtual ~INiagaraStackObjectIssueGenerator()= default;
	virtual void GenerateIssues(void* Object, UNiagaraStackObject* StackObject, TArray<UNiagaraStackEntry::FStackIssue>& NewIssues) = 0;
};

class FNiagaraPlatformSetIssueGenerator : public INiagaraStackObjectIssueGenerator
{
public:
	virtual void GenerateIssues(void* Object, UNiagaraStackObject* StackObject, TArray<UNiagaraStackEntry::FStackIssue>& NewIssues)override;
private:
	static void FixIssue(FNiagaraPlatformSet* PlatformSet, TWeakObjectPtr<UObject> WeakObject, FName CVarName);
};
