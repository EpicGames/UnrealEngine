// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class IDetailTreeNode;

namespace FNiagaraStackObjectShared
{
	DECLARE_DELEGATE_TwoParams(FOnFilterDetailNodes, const TArray<TSharedRef<IDetailTreeNode>>& /* InSourceNodes */, TArray<TSharedRef<IDetailTreeNode>>& /* OutFilteredNodes */);
}