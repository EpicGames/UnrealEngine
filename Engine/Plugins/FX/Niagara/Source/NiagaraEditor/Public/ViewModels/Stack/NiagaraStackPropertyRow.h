// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackObjectShared.h"
#include "NiagaraStackPropertyRow.generated.h"

class FNiagaraStackObjectPropertyCustomization;
class IDetailTreeNode;
class UNiagaraNode;

UCLASS(MinimalAPI)
class UNiagaraStackPropertyRow : public UNiagaraStackItemContent
{
	GENERATED_BODY()
		
public:
	NIAGARAEDITOR_API void Initialize(
		FRequiredEntryData InRequiredEntryData,
		TSharedRef<IDetailTreeNode> InDetailTreeNode,
		bool bInIsTopLevelProperty,
		bool bInHideTopLevelCategories,
		FString InOwnerStackItemEditorDataKey,
		FString InOwnerStackEditorDataKey,
		UNiagaraNode* InOwningNiagaraNode);
	
	void SetOnFilterDetailNodes(FNiagaraStackObjectShared::FOnFilterDetailNodes InOnFilterDetailNodes) { OnFilterDetailNodes = InOnFilterDetailNodes; }

	void SetPropertyCustomization(TSharedPtr<const FNiagaraStackObjectPropertyCustomization> Customization);
	TSharedPtr<const FNiagaraStackObjectPropertyCustomization> GetCustomization() const { return PropertyCustomization; }
	
	NIAGARAEDITOR_API virtual EStackRowStyle GetStackRowStyle() const override;

	NIAGARAEDITOR_API virtual bool GetShouldShowInStack() const override;

	NIAGARAEDITOR_API TSharedRef<IDetailTreeNode> GetDetailTreeNode() const;

	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;

	NIAGARAEDITOR_API virtual bool HasOverridenContent() const override;

	NIAGARAEDITOR_API virtual bool IsExpandedByDefault() const override;

	virtual bool SupportsStackNotes() const override { return true; }
	
	NIAGARAEDITOR_API virtual bool CanDrag() const override;

	void SetOwnerGuid(TOptional<FGuid> InGuid) { OwnerGuid = InGuid; }

	virtual bool SupportsCopy() const override;

	virtual bool TestCanCopyWithMessage(FText& OutMessage) const override;

	virtual void Copy(UNiagaraClipboardContent* ClipboardContent) const override;

	virtual bool SupportsPaste() const override;

	virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;

	virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;

	virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

protected:
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
	void RefreshCustomization();

	NIAGARAEDITOR_API virtual int32 GetChildIndentLevel() const override;

	NIAGARAEDITOR_API virtual void GetSearchItems(TArray<FStackSearchItem>& SearchItems) const override;

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> CanDropInternal(const FDropRequest& DropRequest) override;

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> DropInternal(const FDropRequest& DropRequest) override;

	NIAGARAEDITOR_API virtual bool SupportsSummaryView() const override;
	NIAGARAEDITOR_API virtual FHierarchyElementIdentity DetermineSummaryIdentity() const override;

	NIAGARAEDITOR_API bool FilterOnlyModified(const UNiagaraStackEntry& Child) const;
private:
	TSharedPtr<IDetailTreeNode> DetailTreeNode;
	UNiagaraNode* OwningNiagaraNode;
	EStackRowStyle RowStyle;
	TSharedPtr<const FNiagaraStackObjectPropertyCustomization> PropertyCustomization;

	UPROPERTY()
	TObjectPtr<UNiagaraStackSpacer> CategorySpacer;

	/** An optional guid that can be used to identify the parent object. The parent object is responsible for setting this. It is used to create the summary view identity */
	TOptional<FGuid> OwnerGuid;

	bool bCannotEditInThisContext;
	bool bIsTopLevelProperty;
	bool bHideTopLevelCategories;
	bool bIsHiddenCategory;

	FNiagaraStackObjectShared::FOnFilterDetailNodes OnFilterDetailNodes;
};
