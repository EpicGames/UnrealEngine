// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEmitter.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "NiagaraStackRenderItemGroup.generated.h"

class INiagaraStackRenderersOwner;
class UNiagaraClipboardContent;
class UNiagaraRendererProperties;

UCLASS(MinimalAPI)
class UNiagaraStackRenderItemGroup : public UNiagaraStackItemGroup
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, TSharedPtr<INiagaraStackRenderersOwner> InRenderersOwner);

	virtual UNiagaraStackEntry::EIconMode GetSupportedIconMode() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual FText GetIconText() const override;

	TSharedPtr<INiagaraStackRenderersOwner> GetRenderersOwner() const { return RenderersOwner; }

	NIAGARAEDITOR_API virtual bool GetCanExpandInOverview() const;

	virtual bool SupportsPaste() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;
	NIAGARAEDITOR_API virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;
private:
	NIAGARAEDITOR_API void OwnerRenderersChanged();

	NIAGARAEDITOR_API bool ChildRequestCanPaste(const UNiagaraClipboardContent* ClipboardContent,FText& OutCanPasteMessage);
	NIAGARAEDITOR_API void ChildRequestPaste(const UNiagaraClipboardContent* ClipboardContent, int32 PasteIndex, FText& OutPasteWarning);

private:
	NIAGARAEDITOR_API void OnRendererAdded(UNiagaraRendererProperties* RendererProperties) const;
	TSharedPtr<INiagaraStackItemGroupAddUtilities> AddUtilities;
	TSharedPtr<INiagaraStackRenderersOwner> RenderersOwner;
};
