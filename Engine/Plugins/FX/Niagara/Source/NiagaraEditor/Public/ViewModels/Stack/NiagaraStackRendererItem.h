// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraRendererProperties.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "NiagaraTypes.h"
#include "NiagaraStackRendererItem.generated.h"

struct FVersionedNiagaraEmitterData;
class INiagaraStackRenderersOwner;
class UNiagaraEmitter;
class UNiagaraRendererProperties;
class UNiagaraStackObject;

UCLASS(MinimalAPI)
class UNiagaraStackRendererItem : public UNiagaraStackItem
{
	GENERATED_BODY()
		
public:
	NIAGARAEDITOR_API UNiagaraStackRendererItem();

	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, TSharedPtr<INiagaraStackRenderersOwner> InRenderersOwner, UNiagaraRendererProperties* InRendererProperties);

	NIAGARAEDITOR_API UNiagaraRendererProperties* GetRendererProperties();
	NIAGARAEDITOR_API const UNiagaraRendererProperties* GetRendererProperties() const;

	NIAGARAEDITOR_API void OpenSourceCodeLocation() const;

	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;

	virtual bool SupportsCut() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanCutWithMessage(FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual FText GetCutTransactionText() const override;
	NIAGARAEDITOR_API virtual void CopyForCut(UNiagaraClipboardContent* ClipboardContent) const override;
	NIAGARAEDITOR_API virtual void RemoveForCut() override;

	virtual bool SupportsCopy() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanCopyWithMessage(FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual void Copy(UNiagaraClipboardContent* ClipboardContent) const override;

	virtual bool SupportsPaste() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;
	NIAGARAEDITOR_API virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

	virtual bool SupportsDelete() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const override;
	NIAGARAEDITOR_API virtual FText GetDeleteTransactionText() const override;
	NIAGARAEDITOR_API virtual void Delete() override;

	virtual bool SupportsInheritance() const override { return true; }
	NIAGARAEDITOR_API virtual bool GetIsInherited() const override;
	NIAGARAEDITOR_API virtual FText GetInheritanceMessage() const override;

	virtual bool SupportsSummaryView() const override { return true; }
	NIAGARAEDITOR_API virtual FHierarchyElementIdentity DetermineSummaryIdentity() const override;
	
	virtual bool SupportsRename() const override { return true; }

	virtual bool SupportsStackNotes() const override { return true; }

	NIAGARAEDITOR_API bool HasBaseRenderer() const;

	virtual bool SupportsChangeEnabled() const override { return true; }
	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;

	virtual EIconMode GetSupportedIconMode() const override { return EIconMode::Brush; }
	NIAGARAEDITOR_API virtual const FSlateBrush* GetIconBrush() const override;

	virtual bool SupportsResetToBase() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanResetToBaseWithMessage(FText& OutCanResetToBaseMessage) const override;
	NIAGARAEDITOR_API virtual void ResetToBase() override;

	NIAGARAEDITOR_API virtual bool GetShouldShowInOverview() const;

	static NIAGARAEDITOR_API TArray<FNiagaraVariable> GetMissingVariables(UNiagaraRendererProperties* RendererProperties, const FVersionedNiagaraEmitterData* EmitterData);
	static NIAGARAEDITOR_API bool AddMissingVariable(const FVersionedNiagaraEmitterData* EmitterData, const FNiagaraVariable& Variable);

	NIAGARAEDITOR_API bool IsExcludedFromScalability() const;
	NIAGARAEDITOR_API bool IsOwningEmitterExcludedFromScalability() const;

	NIAGARAEDITOR_API virtual const FCollectedUsageData& GetCollectedUsageData() const override;

	NIAGARAEDITOR_API bool CanMoveRendererUp() const;
	NIAGARAEDITOR_API void MoveRendererUp() const;
	NIAGARAEDITOR_API bool CanMoveRendererDown() const;
	NIAGARAEDITOR_API void MoveRendererDown() const;
protected:
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	NIAGARAEDITOR_API virtual void SetIsEnabledInternal(bool bInIsEnabled) override;

private:
	NIAGARAEDITOR_API void RendererChanged();
	NIAGARAEDITOR_API void RefreshIssues(TArray<FStackIssue>& NewIssues);
	NIAGARAEDITOR_API void ProcessRendererIssues(const TArray<FNiagaraRendererFeedback>& InIssues, EStackIssueSeverity Severity, TArray<FStackIssue>& OutIssues);

	virtual bool SupportsDebugDraw() const override;
	virtual TOptional<FText> GetDebugDrawTooltip() const override;
	virtual bool IsDebugDrawEnabled() const override;
	virtual void SetDebugDrawEnabled(bool bInEnabled) override;

	virtual TArray<FNiagaraStackItemCustomIcon> GetCustomIcons() const override;

private:
	TSharedPtr<INiagaraStackRenderersOwner> RenderersOwner;

	TWeakObjectPtr<UNiagaraRendererProperties> RendererProperties;

	mutable TOptional<bool> bHasBaseRendererCache;

	mutable TOptional<bool> bCanResetToBaseCache;

	mutable TOptional<FText> DisplayNameCache;

	TArray<FNiagaraVariable> MissingAttributes;

	UPROPERTY()
	TObjectPtr<UNiagaraStackObject> RendererObject;
};
