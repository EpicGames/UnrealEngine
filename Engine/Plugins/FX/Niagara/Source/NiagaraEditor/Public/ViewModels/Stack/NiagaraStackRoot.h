// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraStackCommentCollection.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "NiagaraStackRoot.generated.h"

class FNiagaraEmitterHandleViewModel;
class FNiagaraEmitterViewModel;
class INiagaraStackRenderersOwner;
class UNiagaraSimulationStageBase;
class UNiagaraStackSystemPropertiesGroup;
class UNiagaraStackSystemUserParametersGroup;
class UNiagaraStackEmitterPropertiesGroup;
class UNiagaraStackScriptItemGroup;
class UNiagaraStackRenderItemGroup;
class UNiagaraStackEmitterSummaryGroup;
class UNiagaraStackEmitterStatelessGroup;

UCLASS(MinimalAPI)
class UNiagaraStackRoot : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API UNiagaraStackRoot();
	
	void Initialize(FRequiredEntryData InRequiredEntryData, bool bInIncludeSystemInformation, bool bInIncludeEmitterInformation);
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	NIAGARAEDITOR_API virtual bool GetCanExpand() const override;
	NIAGARAEDITOR_API virtual bool GetShouldShowInStack() const override;

	UNiagaraStackRenderItemGroup* GetRenderGroup() const;
	UNiagaraStackCommentCollection* GetCommentCollection() const;
	
protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	NIAGARAEDITOR_API void EmitterArraysChanged();
	NIAGARAEDITOR_API void OnSummaryViewStateChanged();

	void RefreshSystemChildren(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren);
	void RefreshEmitterFullChildren(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren);
	void RefreshEmitterSummaryChildren(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren);
	void RefreshEmitterStatelessChildren(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);

	UNiagaraStackEntry* GetOrCreateSystemPropertiesGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren);
	UNiagaraStackEntry* GetOrCreateSystemUserParametersGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren);
	UNiagaraStackEntry* GetOrCreateSystemCommentCollection(const TArray<UNiagaraStackEntry*>& CurrentChildren);

	UNiagaraStackEntry* GetOrCreateEmitterPropertiesGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren);
	UNiagaraStackEntry* GetOrCreateEmitterSummaryGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren);
	UNiagaraStackEntry* GetOrCreateEmitterRendererGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, TSharedPtr<INiagaraStackRenderersOwner> RenderersOwner);

	UNiagaraStackEntry* GetOrCreateStatelessEmitterGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);
	UNiagaraStackEntry* GetOrCreateStatelessEmitterSpawnGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);
	UNiagaraStackEntry* GetOrCreateStatelessEmitterSimulateGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel);

	UNiagaraStackEntry* GetCurrentScriptGroup(const TArray<UNiagaraStackEntry*>& CurrentChildren, ENiagaraScriptUsage InScriptUsage, FGuid InScriptUsageId) const;

	UNiagaraStackEntry* CreateScriptGroup(
		TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
		ENiagaraScriptUsage InScriptUsage,
		FGuid InScriptUsageId,
		UNiagaraStackEditorData& InStackEditorData,
		FName InExecutionCategoryName,
		FName InExecutionSubcategoryName,
		FText InDisplayName,
		FText InToolTip);

	UNiagaraStackEntry* CreateEventScriptGroup(
		FGuid SourceEmitterId,
		TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
		FGuid InScriptUsageId,
		UNiagaraStackEditorData& InStackEditorData);

	UNiagaraStackEntry* CreateSimulationStageScriptGroup(
		UNiagaraSimulationStageBase* InSimulationStage,
		TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
		UNiagaraStackEditorData& InStackEditorData);

private:
	bool bIncludeSystemInformation;
	bool bIncludeEmitterInformation;
};
