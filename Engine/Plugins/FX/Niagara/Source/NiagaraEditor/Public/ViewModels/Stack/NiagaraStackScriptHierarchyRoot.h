// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItem.h"
#include "NiagaraTypes.h"
#include "NiagaraStackSection.h"
#include "ViewModels/HierarchyEditor/NiagaraScriptParametersHierarchyViewModel.h"
#include "NiagaraStackScriptHierarchyRoot.generated.h"

class UNiagaraNodeFunctionCall;
class UNiagaraStackFunctionInput;
class UNiagaraClipboardFunctionInput;
class UEdGraphPin;

DECLARE_DELEGATE_RetVal(UNiagaraNodeFunctionCall&, FOnGetFunctionCallNode)


struct FNiagaraFunctionInputInstanceData
{
	bool bIsHidden;
};

struct FNiagaraScriptInstanceData
{
	void Reset();

	/** A list of all generally used inputs.
	 * This will exclude inputs that are found in the asset but are never actually used, or are edit inline conditions etc.
	 * Propagates to categories and inputs so they can determine how to handle children inputs. */
	TSet<FNiagaraVariable> UsedInputs;
	
	/** This includes per-input instance data, such as whether an input is hidden or not from static switch traversal. */
	TMap<FGuid, FNiagaraFunctionInputInstanceData> PerInputInstanceData;
};

using FNiagaraFunctionInputMap = TMap<FGuid, FNiagaraFunctionInputInstanceData>;
DECLARE_DELEGATE_RetVal(const FNiagaraScriptInstanceData&, FOnGetScriptInstanceData)
DECLARE_DELEGATE_RetVal(const FNiagaraFunctionInputMap&, FOnGetPerInputInstanceData)

/** A script's stack hierarchy root, representing a UNiagaraHierarchyRoot object in the stack.
 *  It creates categories and inputs equivalent to the children of the UNiagaraHierarchyRoot it represents.
 *  In addition, input parameters that have not been added to the hierarchy explicitly are added at the end as well.
 *  This avoids that hierarchy setup is _required_, and instead becomes optional
 */
UCLASS(MinimalAPI)
class UNiagaraStackScriptHierarchyRoot : public UNiagaraStackItemContent
{
	GENERATED_BODY()
public:
	void Initialize(
		FRequiredEntryData InRequiredEntryData,
		UNiagaraNodeFunctionCall& InModuleNode,
		UNiagaraNodeFunctionCall& InInputFunctionCallNode,
		FString InOwnerStackItemEditorDataKey);
	
	virtual void FinalizeInternal() override;

	UNiagaraNodeFunctionCall& GetOwningModuleNode() const { return *ModuleNode.Get(); }
	UNiagaraNodeFunctionCall& GetOwningFunctionCallNode() const { return *OwningFunctionCallNode.Get(); }
	UHierarchyRoot* GetScriptParameterHierarchyRoot() const;
	
	NIAGARAEDITOR_API TConstArrayView<const TObjectPtr<UHierarchySection>> GerHierarchySectionData() const;
	NIAGARAEDITOR_API const UHierarchySection* GetActiveHierarchySection() const;
	NIAGARAEDITOR_API void SetActiveHierarchySection(const UHierarchySection* InActiveSection);

	void SetShouldDisplayLabel(bool bInShouldDisplayLabel);
	NIAGARAEDITOR_API bool GetShouldDisplayLabel() const;

	void ToClipboardFunctionInputs(UObject* InOuter, TArray<const UNiagaraClipboardFunctionInput*>& OutClipboardFunctionInputs) const;
	NIAGARAEDITOR_API void SetValuesFromClipboardFunctionInputs(const TArray<const UNiagaraClipboardFunctionInput*>& ClipboardFunctionInputs);

	void GetChildInputs(TArray<UNiagaraStackFunctionInput*>& OutInputs) const;
	TArray<UNiagaraStackFunctionInput*> GetInlineParameters() const;
	
	static const FText AllSectionName;
protected:

	/** This will populate the data the active instance is using;
	 * a lot of information is static (categories, sorting etc.), whereas visibility is determined by the active configuration */
	void RefreshInstanceData();
	
	const UHierarchySection* FindSectionByName(FName SectionName);
	NIAGARAEDITOR_API virtual bool FilterByActiveSection(const UNiagaraStackEntry& Child) const;
	NIAGARAEDITOR_API bool FilterForVisibleCondition(const UNiagaraStackEntry& NiagaraStackEntry) const;
	bool FilterOnlyModified(const UNiagaraStackEntry& Child) const;
	bool FilterForIsInlineEditConditionToggle(const UNiagaraStackEntry& NiagaraStackEntry) const;
	
	const FNiagaraScriptInstanceData& GetScriptInstanceData() const { return ScriptInstanceData; }
	const TMap<FGuid, FNiagaraFunctionInputInstanceData>& GetPerInputInstanceData() const { return ScriptInstanceData.PerInputInstanceData; }

private:
	/** NiagaraStackEntry */
	virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override final;
	virtual bool GetCanExpand() const override;
	virtual bool GetShouldShowInStack() const override;
	virtual int32 GetChildIndentLevel() const override;
	
	void OnScriptApplied(UNiagaraScript* NiagaraScript, FGuid Guid);
protected:
	TWeakObjectPtr<const UHierarchySection> ActiveSection;

	TWeakObjectPtr<UNiagaraNodeFunctionCall> ModuleNode;
	TWeakObjectPtr<UNiagaraNodeFunctionCall> OwningFunctionCallNode;

	/** The script variable guid to input instance data map. Calculated once per hierarchy root, then forwarded to the children via delegates. */
	FNiagaraScriptInstanceData ScriptInstanceData;
	
	bool bShouldDisplayLabel = false;
};
