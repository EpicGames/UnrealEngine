// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "NiagaraCommon.h"
#include "AssetRegistry/AssetData.h"
#include "NiagaraStackScriptItemGroup.generated.h"

class FNiagaraScriptViewModel;
class UNiagaraNodeOutput;
class UNiagaraNodeFunctionCall;
class FScriptItemGroupAddUtilities;
class UEdGraph;

UCLASS(MinimalAPI)
class UNiagaraStackScriptItemGroup : public UNiagaraStackItemGroup
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(
		FRequiredEntryData InRequiredEntryData,
		FText InDisplayName,
		FText InToolTip,
		TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
		ENiagaraScriptUsage InScriptUsage,
		FGuid InScriptUsageId = FGuid());

	ENiagaraScriptUsage GetScriptUsage() const { return ScriptUsage; }
	FGuid GetScriptUsageId() const { return ScriptUsageId; }
	NIAGARAEDITOR_API UNiagaraNodeOutput* GetScriptOutputNode() const;

	NIAGARAEDITOR_API virtual UObject* GetDisplayedObject() const override;

	virtual bool SupportsPaste() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	NIAGARAEDITOR_API virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;
	NIAGARAEDITOR_API virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> CanDropInternal(const FDropRequest& DropRequest) override;

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> DropInternal(const FDropRequest& DropRequest) override;

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> ChildRequestCanDropInternal(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest) override;

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> ChildRequestDropInternal(const UNiagaraStackEntry& TargetChild, const FDropRequest& DropRequest) override;

private:
	NIAGARAEDITOR_API void ItemAdded(UNiagaraNodeFunctionCall* AddedModule);

	NIAGARAEDITOR_API void ChildModifiedGroupItems();

	NIAGARAEDITOR_API bool ChildRequestCanPaste(const UNiagaraClipboardContent* ClipboardContent, FText& OutCanPasteMessage);

	NIAGARAEDITOR_API void ChildRequestPaste(const UNiagaraClipboardContent* ClipboardContent, int32 PasteIndex, FText& OutPasteWarning);

	NIAGARAEDITOR_API void ChildRequestDeprecatedRecommendation(class UNiagaraStackModuleItem* TargetChild);

	NIAGARAEDITOR_API void OnScriptGraphChanged(const struct FEdGraphEditAction& InAction);

	NIAGARAEDITOR_API void OnSystemScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion);

	NIAGARAEDITOR_API void OnParticleScriptCompiled(UNiagaraScript* InScript, const FGuid& ScriptVersion);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> CanDropOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> CanDropEntriesOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> CanDropAssetsOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> CanDropScriptsOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> CanDropParameterOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> DropOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> DropEntriesOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> DropAssetsOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> DropScriptsOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

	NIAGARAEDITOR_API TOptional<FDropRequestResponse> DropParameterOnTarget(const UNiagaraStackEntry& TargetEntry, const FDropRequest& DropRequest);

protected:
	TWeakPtr<FNiagaraScriptViewModel> ScriptViewModel;
	NIAGARAEDITOR_API void RefreshIssues(TArray<FStackIssue>& NewIssues);

private:
	NIAGARAEDITOR_API void PasteModules(const UNiagaraClipboardContent* ClipboardContent, int32 PasteIndex, FText& OutPasteWarning);

private:
	TSharedPtr<FScriptItemGroupAddUtilities> AddUtilities;

	ENiagaraScriptUsage ScriptUsage;

	FGuid ScriptUsageId;
	bool bIsValidForOutput;

	TWeakObjectPtr<UEdGraph> ScriptGraph;

	TWeakObjectPtr<UNiagaraSystem> OwningSystemWeak;

	TWeakObjectPtr<UNiagaraScript> OwningParticleScriptWeak;

	FDelegateHandle OnGraphChangedHandle;
};
