// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraStackEntry.h"
#include "NiagaraStackSelection.generated.h"

UCLASS(MinimalAPI)
class UNiagaraStackSelection : public UNiagaraStackEntry
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData);

	NIAGARAEDITOR_API virtual bool GetCanExpand() const override;

	NIAGARAEDITOR_API virtual bool GetShouldShowInStack() const override;

	NIAGARAEDITOR_API void SetSelectedEntries(const TArray<UNiagaraStackEntry*>& InSelectedEntries);

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	TArray<TWeakObjectPtr<UNiagaraStackEntry>> SelectedEntries;
};
