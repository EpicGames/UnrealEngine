// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraCommon.h"
#include "NiagaraEmitter.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "NiagaraStackSimulationStageGroup.generated.h"

class FNiagaraScriptViewModel;
class UNiagaraStackObject;
struct FNiagaraEventScriptProperties;
class IDetailTreeNode;

UCLASS(MinimalAPI)
class UNiagaraStackSimulationStagePropertiesItem : public UNiagaraStackItem
{
	GENERATED_BODY()
		 
public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraSimulationStageBase* InSimulationStage);

	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;

	virtual bool SupportsResetToBase() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanResetToBaseWithMessage(FText& OutCanResetToBaseMessage) const override;
	NIAGARAEDITOR_API virtual void ResetToBase() override;

	NIAGARAEDITOR_API void SetSimulationStageEnabled(bool bIsEnabled);

	TWeakObjectPtr<UNiagaraSimulationStageBase> GetSimulationStage() const { return SimulationStage; }
protected:
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	virtual bool SupportsSummaryView() const override { return true; }
	NIAGARAEDITOR_API virtual FHierarchyElementIdentity DetermineSummaryIdentity() const override;
private:
	NIAGARAEDITOR_API void SimulationStagePropertiesChanged();

	NIAGARAEDITOR_API bool HasBaseSimulationStage() const;

private:
	TWeakObjectPtr<UNiagaraSimulationStageBase> SimulationStage;

	mutable TOptional<bool> bHasBaseSimulationStageCache;

	mutable TOptional<bool> bCanResetToBaseCache;

	UPROPERTY()
	TObjectPtr<UNiagaraStackObject> SimulationStageObject;
};

UCLASS(MinimalAPI)
/** Meant to contain a single binding of a Emitter::EventScriptProperties to the stack.*/
class UNiagaraStackSimulationStageGroup : public UNiagaraStackScriptItemGroup
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE(FOnModifiedSimulationStages);

public:
	NIAGARAEDITOR_API void Initialize(
		FRequiredEntryData InRequiredEntryData,
		TSharedRef<FNiagaraScriptViewModel> InScriptViewModel,
		UNiagaraSimulationStageBase* InSimulationStage);

	NIAGARAEDITOR_API UNiagaraSimulationStageBase* GetSimulationStage() const;
	const TObjectPtr<UNiagaraStackSimulationStagePropertiesItem>& GetSimulationStagePropertiesItem() const { return SimulationStageProperties; }
	NIAGARAEDITOR_API void SetOnModifiedSimulationStages(FOnModifiedSimulationStages OnModifiedSimulationStages);

	virtual bool SupportsDelete() const override { return true; }
	NIAGARAEDITOR_API virtual bool TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const override;
	NIAGARAEDITOR_API virtual void Delete() override;

	virtual bool SupportsInheritance() const override { return true; }
	NIAGARAEDITOR_API virtual bool GetIsInherited() const override;
	NIAGARAEDITOR_API virtual FText GetInheritanceMessage() const override;

	virtual bool SupportsStackNotes() const override { return true; }
	
	virtual bool CanDrag() const override { return true; }

	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;
	NIAGARAEDITOR_API virtual void SetIsEnabled(bool bEnabled) override;
	virtual bool SupportsChangeEnabled() const override { return true; }

	NIAGARAEDITOR_API INiagaraStackItemGroupAddUtilities* GetEmitterStageAddUtilities() const;

	NIAGARAEDITOR_API int32 GetStageIndex() const;

protected:
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> CanDropInternal(const FDropRequest& DropRequest) override;

	NIAGARAEDITOR_API virtual TOptional<FDropRequestResponse> DropInternal(const FDropRequest& DropRequest) override;

	virtual bool SupportsSummaryView() const override { return true; }
	NIAGARAEDITOR_API virtual FHierarchyElementIdentity DetermineSummaryIdentity() const override;
private:
	NIAGARAEDITOR_API void SimulationStagePropertiesChanged();

	NIAGARAEDITOR_API bool HasBaseSimulationStage() const;

	void StageAdded(FGuid AddedEventHandlerId, UNiagaraSimulationStageBase* AddedSimulationStage) const;
	
private:
	TWeakObjectPtr<UNiagaraSimulationStageBase> SimulationStage;

	mutable TOptional<bool> bHasBaseSimulationStageCache;

	FOnModifiedSimulationStages OnModifiedSimulationStagesDelegate;

	UPROPERTY()
	TObjectPtr<UNiagaraStackSimulationStagePropertiesItem> SimulationStageProperties;

	mutable TSharedPtr<INiagaraStackItemGroupAddUtilities> EmitterStageAddUtilities;
};
