// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackObjectShared.h"

#include "NiagaraStackStatelessEmitterGroup.generated.h"

class UNiagaraStatelessEmitter;
class UNiagaraStackObject;

UCLASS(MinimalAPI)
class UNiagaraStackStatelessEmitterGroup : public UNiagaraStackItemGroup
{
	GENERATED_BODY()

public:
	void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter);

	virtual EIconMode GetSupportedIconMode() const override { return EIconMode::Brush; }
	virtual const FSlateBrush* GetIconBrush() const override;

	virtual bool SupportsSecondaryIcon() const override;
	virtual const FSlateBrush* GetSecondaryIconBrush() const override;

	virtual bool GetCanExpandInOverview() const override { return false; }
	virtual bool GetShouldShowInStack() const override { return false; }

	UNiagaraStatelessEmitter* GetStatelessEmitter() const { return StatelessEmitterWeak.Get(); }

protected:
	virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	static void FilterDetailNodes(const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes);

private:
	TWeakObjectPtr<UNiagaraStatelessEmitter> StatelessEmitterWeak;
	TWeakObjectPtr<UNiagaraStackStatelessEmitterObjectItem> RawObjectItemWeak;
	TWeakObjectPtr<UNiagaraStackStatelessEmitterObjectItem> FilteredObjectItemWeak;
};

UCLASS(MinimalAPI)
class UNiagaraStackStatelessEmitterObjectItem : public UNiagaraStackItem
{
	GENERATED_BODY()

public:
	void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter, FText InDisplayName, bool bInIsExpandedByDefault, FNiagaraStackObjectShared::FOnFilterDetailNodes InOnFilterDetailNodes);

	virtual FText GetDisplayName() const override { return DisplayName; }
	virtual bool IsExpandedByDefault() const override { return bIsExpandedByDefault; }
	virtual bool GetShouldShowInOverview() const override { return false; }

	UNiagaraStatelessEmitter* GetStatelessEmitter() const { return StatelessEmitterWeak.Get(); }

protected:
	virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	TWeakObjectPtr<UNiagaraStatelessEmitter> StatelessEmitterWeak;
	FText DisplayName;
	bool bIsExpandedByDefault;
	FNiagaraStackObjectShared::FOnFilterDetailNodes OnFilterDetailNodes;

	TWeakObjectPtr<UNiagaraStackObject> StatelessEmitterStackObjectWeak;
};