// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackObjectShared.h"

#include "NiagaraStackStatelessEmitterSimulateGroup.generated.h"

class FNiagaraStackItemPropertyHeaderValue;
class FNiagaraStatelessEmitterSimulateGroupAddUtilities;
class UNiagaraStatelessEmitter;
class UNiagaraStatelessModule;
class UNiagaraStackObject;

UCLASS(MinimalAPI)
class UNiagaraStackStatelessEmitterSimulateGroup : public UNiagaraStackItemGroup
{
	GENERATED_BODY()

public:
	void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter);

	virtual EIconMode GetSupportedIconMode() const override { return EIconMode::Brush; }
	virtual const FSlateBrush* GetIconBrush() const override;

	virtual bool GetShouldShowInStack() const override { return false; }

	UNiagaraStatelessEmitter* GetStatelessEmitter() const { return StatelessEmitterWeak.Get(); }

	virtual bool SupportsPaste() const { return true; }
	virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;
	virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

protected:
	void OnTemplateChanged();
	virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	void ModuleAdded(UNiagaraStatelessModule* StatelessModule);
	void ModuleModifiedGroupItems();

private:
	TWeakObjectPtr<UNiagaraStatelessEmitter> StatelessEmitterWeak;
	TSharedPtr<FNiagaraStatelessEmitterSimulateGroupAddUtilities> AddUtilities;
};

UCLASS(MinimalAPI)
class UNiagaraStackStatelessModuleItem : public UNiagaraStackItem
{
	GENERATED_BODY()

public:
	static FString GenerateStackEditorDataKey(const UNiagaraStatelessModule* InStatelessModule);

	void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessModule* InStatelessModule);

	virtual FText GetDisplayName() const override { return DisplayName; }
	virtual FText GetTooltipText() const override;

	virtual bool SupportsCopy() const override { return true; }
	virtual bool TestCanCopyWithMessage(FText& OutMessage) const override;
	virtual void Copy(UNiagaraClipboardContent* ClipboardContent) const override;

	virtual bool SupportsPaste() const { return true; }
	virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;
	virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

	virtual bool SupportsDelete() const override { return true; }
	virtual bool TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const override;
	virtual FText GetDeleteTransactionText() const override;
	virtual void Delete() override;

	virtual bool SupportsChangeEnabled() const override;
	virtual bool GetIsEnabled() const override;

	virtual bool SupportsHeaderValues() const override { return true; }
	virtual void GetHeaderValueHandlers(TArray<TSharedRef<INiagaraStackItemHeaderValueHandler>>& OutHeaderValueHandlers) const override;

	virtual bool SupportsDebugDraw() const override;
	virtual TOptional<FText> GetDebugDrawTooltip() const override;
	virtual bool IsDebugDrawEnabled() const override;
	virtual void SetDebugDrawEnabled(bool bInEnabled) override;

	virtual const FCollectedUsageData& GetCollectedUsageData() const override;

	UNiagaraStatelessModule* GetStatelessModule() const { return StatelessModuleWeak.Get(); }
	UNiagaraStatelessEmitter* GetStatelessEmitter() const;

protected:
	virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	virtual void SetIsEnabledInternal(bool bInIsEnabled) override;

private:
	static void FilterDetailNodes(const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes);

	void OnHeaderValueChanged();

	FBoolProperty* GetDebugDrawProperty() const;

private:
	TWeakObjectPtr<UNiagaraStatelessModule> StatelessModuleWeak;
	FText DisplayName;
	TWeakObjectPtr<UNiagaraStackObject> ModuleObjectWeak;
	bool bGeneratedHeaderValueHandlers = false;
	TArray<TSharedRef<FNiagaraStackItemPropertyHeaderValue>> HeaderValueHandlers;
};

