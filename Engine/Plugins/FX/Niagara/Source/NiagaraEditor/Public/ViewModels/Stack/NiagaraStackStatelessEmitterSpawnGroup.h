// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "ViewModels/Stack/NiagaraStackObjectShared.h"

#include "NiagaraStackStatelessEmitterSpawnGroup.generated.h"

class FNiagaraStackItemPropertyHeaderValue;
struct FNiagaraStatelessSpawnInfo;
enum class ENiagaraStatelessSpawnInfoType;
class UNiagaraStatelessEmitter;
class UNiagaraStackObject;

class FStructOnScope;

UCLASS(MinimalAPI)
class UNiagaraStackStatelessEmitterSpawnGroup : public UNiagaraStackItemGroup
{
	GENERATED_BODY()

public:
	void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter);

	virtual EIconMode GetSupportedIconMode() const override { return EIconMode::Brush; }
	virtual const FSlateBrush* GetIconBrush() const override;

	virtual bool GetShouldShowInStack() const override { return false; }

	UNiagaraStatelessEmitter* GetStatelessEmitter() const { return StatelessEmitterWeak.Get(); }

	virtual bool SupportsPaste() const { return true; }
	virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;
	virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

protected:
	virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	void OnSpawnInfoAdded(FGuid AddedItemId);
	void OnChildRequestDelete(FGuid DeleteItemId);

private:
	TWeakObjectPtr<UNiagaraStatelessEmitter> StatelessEmitterWeak;
	TSharedPtr<INiagaraStackItemGroupAddUtilities> AddUtilities;
};

UCLASS(MinimalAPI)
class UNiagaraStackStatelessEmitterSpawnItem : public UNiagaraStackItem
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE_OneParam(FOnRequestDelete, FGuid /* AddedItemId */);

public:
	void Initialize(FRequiredEntryData InRequiredEntryData, UNiagaraStatelessEmitter* InStatelessEmitter, int32 InIndex);

	virtual FText GetDisplayName() const override;
	virtual FText GetTooltipText() const override;
	virtual FGuid GetSelectionId() const override;

	virtual bool SupportsCopy() const override { return true; }
	virtual bool TestCanCopyWithMessage(FText& OutMessage) const override;
	virtual void Copy(UNiagaraClipboardContent* ClipboardContent) const override;

	virtual bool SupportsPaste() const { return true; }
	virtual bool TestCanPasteWithMessage(const UNiagaraClipboardContent* ClipboardContent, FText& OutMessage) const override;
	virtual FText GetPasteTransactionText(const UNiagaraClipboardContent* ClipboardContent) const override;
	virtual void Paste(const UNiagaraClipboardContent* ClipboardContent, FText& OutPasteWarning) override;

	virtual bool SupportsDelete() const override { return true; }
	virtual bool TestCanDeleteWithMessage(FText& OutCanDeleteMessage) const override;
	virtual FText GetDeleteTransactionText() const override;
	virtual void Delete() override;

	virtual bool SupportsChangeEnabled() const override;
	virtual bool GetIsEnabled() const override;

	virtual bool SupportsHeaderValues() const override { return true; }
	virtual void GetHeaderValueHandlers(TArray<TSharedRef<INiagaraStackItemHeaderValueHandler>>& OutHeaderValueHandlers) const override;

	virtual const FCollectedUsageData& GetCollectedUsageData() const override;

	UNiagaraStatelessEmitter* GetStatelessEmitter() const { return StatelessEmitterWeak.Get(); }
	int32 GetIndex() const { return Index; }
	const FGuid& GetSourceId() const { return SourceId; }

	FOnRequestDelete& OnRequestDelete() { return OnRequestDeleteDelegate; }

	static FText GetDisplayName(ENiagaraStatelessSpawnInfoType SpawnInfoType);
	static FText GetTooltipText(ENiagaraStatelessSpawnInfoType SpawnInfoType);

protected:
	virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	virtual void SetIsEnabledInternal(bool bInIsEnabled) override;

private:
	FNiagaraStatelessSpawnInfo* GetSpawnInfo() const;

	static void FilterDetailNodes(const TArray<TSharedRef<IDetailTreeNode>>& InSourceNodes, TArray<TSharedRef<IDetailTreeNode>>& OutFilteredNodes);

	void OnHeaderValueChanged();
	void OnSpawnInfoModified(TArray<UObject*> Objects, ENiagaraDataObjectChange ChangeType);

private:
	TWeakObjectPtr<UNiagaraStatelessEmitter> StatelessEmitterWeak;
	int32 Index;
	FGuid SourceId;
	TSharedPtr<FStructOnScope> SpawnInfoStructOnScope;
	TWeakObjectPtr<UNiagaraStackObject> SpawnInfoObjectWeak;
	bool bGeneratedHeaderValueHandlers = false;
	TArray<TSharedRef<FNiagaraStackItemPropertyHeaderValue>> HeaderValueHandlers;
	FOnRequestDelete OnRequestDeleteDelegate;
};

