// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraEmitter.h"
#include "NiagaraStackValueCollection.h"
#include "DataHierarchyViewModelBase.h"
#include "NiagaraStackSummaryViewInputCollection.generated.h"

UCLASS(MinimalAPI)
class UNiagaraStackSummaryViewCollection : public UNiagaraStackValueCollection
{
	GENERATED_BODY()
		
public:
	DECLARE_DELEGATE_TwoParams(FOnSelectRootNodes, TArray<TSharedRef<IDetailTreeNode>>, TArray<TSharedRef<IDetailTreeNode>>*);

public:
	UNiagaraStackSummaryViewCollection() {}

	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FVersionedNiagaraEmitterWeakPtr InEmitter, FString InOwningStackItemEditorDataKey);
	NIAGARAEDITOR_API virtual void FinalizeInternal() override;

	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;
	NIAGARAEDITOR_API virtual bool GetIsEnabled() const override;

	void RefreshForAdvancedToggle();
protected:
	virtual bool GetShouldShowInStack() const override;
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

	NIAGARAEDITOR_API virtual void GetSectionsInternal(TArray<FNiagaraStackSection>& OutStackSections) const override;
	NIAGARAEDITOR_API TConstArrayView<const TObjectPtr<UHierarchySection>> GetHierarchySections() const;

	NIAGARAEDITOR_API virtual bool FilterByActiveSection(const UNiagaraStackEntry& Child) const override;
	bool FilterOnlyModified(const UNiagaraStackEntry& Child) const;
private:
	NIAGARAEDITOR_API void OnViewStateChanged();
	void OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload);
private:

	FVersionedNiagaraEmitterWeakPtr Emitter;
};

