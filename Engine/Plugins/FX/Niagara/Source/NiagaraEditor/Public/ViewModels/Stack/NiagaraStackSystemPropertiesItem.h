// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackScriptItemGroup.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "NiagaraStackSystemPropertiesItem.generated.h"

class UNiagaraStackObject;

UCLASS(MinimalAPI)
class UNiagaraStackSystemPropertiesItem : public UNiagaraStackItem
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData);

	NIAGARAEDITOR_API virtual FText GetDisplayName() const override;

	NIAGARAEDITOR_API virtual FText GetTooltipText() const override;

	virtual bool GetShouldShowInOverview() const override { return false; }

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;

private:
	NIAGARAEDITOR_API void SystemPropertiesChanged();

private:
	mutable TOptional<bool> bCanResetToBase;

	TWeakObjectPtr<UNiagaraSystem> System;

	UPROPERTY()
	TObjectPtr<UNiagaraStackObject> SystemObject;

};
