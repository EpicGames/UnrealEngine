// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItemGroup.h"
#include "NiagaraCommon.h"
#include "NiagaraStackItem.h"
#include "NiagaraParameterStore.h"
#include "Delegates/IDelegateInstance.h"
#include "NiagaraStackSystemSettingsGroup.generated.h"

class FNiagaraScriptViewModel;

UCLASS(MinimalAPI)
class UNiagaraStackSystemPropertiesGroup : public UNiagaraStackItemGroup
{
	GENERATED_BODY()
		
public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData);

	virtual EIconMode GetSupportedIconMode() const override { return UNiagaraStackEntry::EIconMode::Brush; }
	NIAGARAEDITOR_API virtual const FSlateBrush* GetIconBrush() const override;
	virtual bool GetCanExpandInOverview() const override { return false; }
	virtual bool GetShouldShowInStack() const override { return false; }

protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
};

UCLASS(MinimalAPI)
class UNiagaraStackSystemUserParametersGroup : public UNiagaraStackItemGroup
{
	GENERATED_BODY()

public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, UObject* InOwner, FNiagaraParameterStore* InParameterStore);

	virtual EIconMode GetSupportedIconMode() const override { return UNiagaraStackEntry::EIconMode::Text; }
	NIAGARAEDITOR_API virtual FText GetIconText() const override;
	virtual bool GetCanExpandInOverview() const override { return false; }
	virtual bool GetShouldShowInStack() const override { return false; }
private:
	TWeakObjectPtr<UObject> Owner;
	FDelegateHandle ParameterStoreChangedHandle;
};

