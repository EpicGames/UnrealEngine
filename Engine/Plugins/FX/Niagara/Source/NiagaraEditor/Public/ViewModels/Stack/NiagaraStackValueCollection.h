// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/Stack/NiagaraStackItem.h"
#include "NiagaraTypes.h"
#include "NiagaraStackSection.h"
#include "NiagaraStackValueCollection.generated.h"

class UNiagaraNodeFunctionCall;
class UNiagaraStackFunctionInput;
class UNiagaraClipboardFunctionInput;
class UEdGraphPin;

/** A base class for value collections. Values can be all kinds of input data, such as module inputs, object properties etc.
 * This has base functionality for sections.
 */
UCLASS(MinimalAPI)
class UNiagaraStackValueCollection : public UNiagaraStackItemContent
{
	GENERATED_BODY()
public:
	NIAGARAEDITOR_API void Initialize(FRequiredEntryData InRequiredEntryData, FString InOwningStackItemEditorDataKey, FString InStackEditorDataKey);
	
	bool GetShouldDisplayLabel() const { return bShouldDisplayLabel; }
	NIAGARAEDITOR_API void SetShouldDisplayLabel(bool bInShouldShowLabel);

	NIAGARAEDITOR_API const TArray<FText>& GetSections() const;

	NIAGARAEDITOR_API FText GetActiveSection() const;

	NIAGARAEDITOR_API void SetActiveSection(FText InActiveSection);

	NIAGARAEDITOR_API FText GetTooltipForSection(FString Section) const;

	void CacheLastActiveSection();
public:
	static NIAGARAEDITOR_API FText UncategorizedName;

	static NIAGARAEDITOR_API FText AllSectionName;
protected:
	NIAGARAEDITOR_API virtual void RefreshChildrenInternal(const TArray<UNiagaraStackEntry*>& CurrentChildren, TArray<UNiagaraStackEntry*>& NewChildren, TArray<FStackIssue>& NewIssues) override;
	virtual void GetSectionsInternal(TArray<FNiagaraStackSection>& OutStackSections) const { }
	NIAGARAEDITOR_API void UpdateCachedSectionData() const;
	NIAGARAEDITOR_API virtual bool FilterByActiveSection(const UNiagaraStackEntry& Child) const;
private:
	NIAGARAEDITOR_API virtual bool GetCanExpand() const override;
	NIAGARAEDITOR_API virtual bool GetShouldShowInStack() const override;
	NIAGARAEDITOR_API virtual int32 GetChildIndentLevel() const override;
private:
	mutable TOptional<TArray<FText>> SectionsCache;
	mutable TOptional<TMap<FString, TArray<FText>>> SectionToCategoryMapCache;
	mutable TOptional<TMap<FString, FText>> SectionToTooltipMapCache;
	mutable TOptional<FText> ActiveSectionCache;
	FText LastActiveSection;

	bool bShouldDisplayLabel;
};
