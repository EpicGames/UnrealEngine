// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "NiagaraMessageStore.h"
#include "UObject/ObjectKey.h"
#include "NiagaraStackViewModel.generated.h"

class FNiagaraSystemViewModel;
class FNiagaraEmitterHandleViewModel;
class UNiagaraStackEntry;
class UNiagaraStackRoot;

struct FNiagaraStackViewModelOptions
{
	FNiagaraStackViewModelOptions()
		: bIncludeSystemInformation(false)
		, bIncludeEmitterInformation(false)
	{
	}

	FNiagaraStackViewModelOptions(bool bInIncludeSystemInformation, bool bInIncludeEmitterInformation)
		: bIncludeSystemInformation(bInIncludeSystemInformation)
		, bIncludeEmitterInformation(bInIncludeEmitterInformation)
	{
	}

	bool GetIncludeSystemInformation() { return bIncludeSystemInformation; }
	bool GetIncludeEmitterInformation() { return bIncludeEmitterInformation; }

private:
	bool bIncludeSystemInformation;
	bool bIncludeEmitterInformation;
};

struct FNiagaraStackSearchConfig
{
	bool bScrollToResult = true;
};

struct FNiagaraStackEntrySelectionConfig
{
	bool bScrollToAndExpand = true;
	bool bSelectRow = false;
	bool bPulse = false;
};

UCLASS(MinimalAPI)
class UNiagaraStackViewModel : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE_OneParam(FOnChangeSearchTextExternal, FText)
	
	DECLARE_DELEGATE_TwoParams(FOnStackEntrySelectionRequested, UNiagaraStackEntry* StackEntry, FNiagaraStackEntrySelectionConfig SelectionConfig);
	DECLARE_MULTICAST_DELEGATE(FOnExpansionChanged);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnStructureChanged, ENiagaraStructureChangedFlags);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSearchCompleted, const FNiagaraStackSearchConfig& SearchConfig);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDataObjectChanged, TArray<UObject*> /** Changed objects */, ENiagaraDataObjectChange /** Change type */);
public:
	struct FSearchResult
	{
		TArray<UNiagaraStackEntry*> EntryPath;
		UNiagaraStackEntry::FStackSearchItem MatchingItem;
		UNiagaraStackEntry* GetEntry() const
		{
			return EntryPath.Num() > 0 ? 
				EntryPath[EntryPath.Num() - 1] : 
				nullptr;
		}
	};

	struct FTopLevelViewModel
	{
		NIAGARAEDITOR_API FTopLevelViewModel(TSharedPtr<FNiagaraSystemViewModel> InSystemViewModel);

		NIAGARAEDITOR_API FTopLevelViewModel(TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel);

		NIAGARAEDITOR_API bool IsValid() const;

		NIAGARAEDITOR_API UNiagaraStackEditorData* GetStackEditorData() const;

		NIAGARAEDITOR_API void GetMessageStores(TArray<FNiagaraMessageSourceAndStore>& OutMessageStores);

		NIAGARAEDITOR_API FText GetDisplayName() const;

		NIAGARAEDITOR_API bool operator==(const FTopLevelViewModel& Other)const;

		const TSharedPtr<FNiagaraSystemViewModel> SystemViewModel;
		const TSharedPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel;
		const TWeakObjectPtr<UNiagaraStackEntry> RootEntry;
	};

public:
	NIAGARAEDITOR_API void InitializeWithViewModels(TSharedPtr<FNiagaraSystemViewModel> InSystemViewModel, TSharedPtr<FNiagaraEmitterHandleViewModel> InEmitterHandleViewModel, FNiagaraStackViewModelOptions InOptions);
	NIAGARAEDITOR_API void InitializeWithRootEntry(UNiagaraStackEntry* Root);

	NIAGARAEDITOR_API void Finalize();

	NIAGARAEDITOR_API virtual void BeginDestroy() override;

	NIAGARAEDITOR_API UNiagaraStackEntry* GetRootEntry();

	NIAGARAEDITOR_API TArray<UNiagaraStackEntry*>& GetRootEntryAsArray();

	NIAGARAEDITOR_API FOnChangeSearchTextExternal& OnChangeSearchTextExternal();

	NIAGARAEDITOR_API FOnExpansionChanged& OnExpansionChanged();
	NIAGARAEDITOR_API FOnExpansionChanged& OnExpansionInOverviewChanged();
	NIAGARAEDITOR_API FOnStructureChanged& OnStructureChanged();
	NIAGARAEDITOR_API FOnSearchCompleted& OnSearchCompleted();
	NIAGARAEDITOR_API FOnDataObjectChanged& OnDataObjectChanged();
	NIAGARAEDITOR_API FOnStackEntrySelectionRequested& OnStackEntrySelectionRequested();

	NIAGARAEDITOR_API bool GetShowOnlyModified() const;
	NIAGARAEDITOR_API void SetShowOnlyModified(bool bInShowOnlyModified);
	
	NIAGARAEDITOR_API bool GetShowAllAdvanced() const;
	NIAGARAEDITOR_API void SetShowAllAdvanced(bool bInShowAllAdvanced);

	NIAGARAEDITOR_API bool GetShowOutputs() const;
	NIAGARAEDITOR_API void SetShowOutputs(bool bInShowOutputs);

	NIAGARAEDITOR_API bool GetShowLinkedInputs() const;
	NIAGARAEDITOR_API void SetShowLinkedInputs(bool bInShowLinkedInputs);

	NIAGARAEDITOR_API bool GetShowOnlyIssues() const;
	NIAGARAEDITOR_API void SetShowOnlyIssues(bool bInShowOnlyIssues);

	NIAGARAEDITOR_API double GetLastScrollPosition() const;
	NIAGARAEDITOR_API void SetLastScrollPosition(double InLastScrollPosition);

	/** Select a stack entry. This selection is distinct from the Selection from the FNiagaraSystemSelectionViewModel which changes the root stack selection. */
	NIAGARAEDITOR_API void SelectStackEntry(UNiagaraStackEntry* StackEntryToSelect, FNiagaraStackEntrySelectionConfig Config) const;

	NIAGARAEDITOR_API virtual void Tick();
	//~ stack search stuff
	NIAGARAEDITOR_API void ResetSearchText();
	FText GetCurrentSearchText() const { return CurrentSearchText; };
	NIAGARAEDITOR_API void OnSearchTextChanged(const FText& SearchText);
	NIAGARAEDITOR_API void SetSearchTextExternal(const FText& NewSearchText);
	NIAGARAEDITOR_API bool IsSearching();
	NIAGARAEDITOR_API const TArray<FSearchResult>& GetCurrentSearchResults();
	NIAGARAEDITOR_API UNiagaraStackEntry* GetCurrentFocusedEntry() const;
	NIAGARAEDITOR_API int32 GetCurrentFocusedEntryIndex() const;
	NIAGARAEDITOR_API void AddSearchScrollOffset(int NumberOfSteps);

	NIAGARAEDITOR_API void OnCycleThroughIssues(TSharedPtr<FTopLevelViewModel> TopLevelToCycle);
	NIAGARAEDITOR_API UNiagaraStackEntry* GetCurrentFocusedIssue() const;

	NIAGARAEDITOR_API void GetPathForEntry(UNiagaraStackEntry* Entry, TArray<UNiagaraStackEntry*>& EntryPath) const;

	/** Starts recursing through all entries to expand all groups and collapse all items. */
	NIAGARAEDITOR_API void CollapseToHeaders();
	
	NIAGARAEDITOR_API void UndismissAllIssues();
	NIAGARAEDITOR_API bool HasDismissedStackIssues();

	NIAGARAEDITOR_API const TArray<TSharedRef<FTopLevelViewModel>>& GetTopLevelViewModels() const;

	NIAGARAEDITOR_API TSharedPtr<FTopLevelViewModel> GetTopLevelViewModelForEntry(UNiagaraStackEntry& InEntry) const;

	NIAGARAEDITOR_API void Reset();
	NIAGARAEDITOR_API bool HasIssues() const;
	NIAGARAEDITOR_API void RequestRefreshDeferred();
	void RequestValidationUpdate() { bValidatorUpdatePending = true; }

	NIAGARAEDITOR_API bool ShouldHideDisabledModules() const;

	NIAGARAEDITOR_API void InvalidateCachedParameterUsage();

private:
	/** Recursively Expands all groups and collapses all items in the stack. */
	NIAGARAEDITOR_API void CollapseToHeadersRecursive(TArray<UNiagaraStackEntry*> Entries);

	struct FSearchWorkItem
	{
		TArray<UNiagaraStackEntry*> EntryPath;
		UNiagaraStackEntry* GetEntry()
		{
			return EntryPath.Num() > 0 ?
				EntryPath[EntryPath.Num() - 1] :
				nullptr;
		}
	};

	NIAGARAEDITOR_API void EntryExpansionChanged();
	NIAGARAEDITOR_API void EntryExpansionInOverviewChanged();
	NIAGARAEDITOR_API void EntryStructureChanged(ENiagaraStructureChangedFlags Flags);
	NIAGARAEDITOR_API void EntryDataObjectModified(TArray<UObject*> ChangedObjects, ENiagaraDataObjectChange ChangeType);
	NIAGARAEDITOR_API void EntryRequestFullRefresh();
	NIAGARAEDITOR_API void RefreshTopLevelViewModels();
	NIAGARAEDITOR_API void RefreshHasIssues();
	NIAGARAEDITOR_API void EmitterParentRemoved();
	/** Called by the tick function to perform partial search */
	NIAGARAEDITOR_API void SearchTick();
	NIAGARAEDITOR_API void GenerateTraversalEntries(UNiagaraStackEntry* Root, TArray<UNiagaraStackEntry*> ParentChain, 
		TArray<FSearchWorkItem>& TraversedArray);
	NIAGARAEDITOR_API bool ItemMatchesSearchCriteria(UNiagaraStackEntry::FStackSearchItem SearchItem);
	NIAGARAEDITOR_API void GeneratePathForEntry(UNiagaraStackEntry* Root, UNiagaraStackEntry* Entry, TArray<UNiagaraStackEntry*> CurrentPath, TArray<UNiagaraStackEntry*>& EntryPath) const;
	
	NIAGARAEDITOR_API void RestartSearch(const FNiagaraStackSearchConfig& InConfig);
	NIAGARAEDITOR_API void UpdateStackWithValidationResults();

private:
	TWeakPtr<FNiagaraEmitterHandleViewModel> EmitterHandleViewModel;
	TWeakPtr<FNiagaraSystemViewModel> SystemViewModel;
	
	TArray<UNiagaraStackEntry*> RootEntries;

	UPROPERTY()
	TObjectPtr<UNiagaraStackEntry> RootEntry;

	bool bExternalRootEntry;

	/** Used to forward a programmatic change in search text to a widget representing the search. The widget is responsible for binding to this delegate. */
	FOnChangeSearchTextExternal OnChangeSearchTextExternalDelegate;
	
	FOnExpansionChanged ExpansionChangedDelegate;
	FOnExpansionChanged ExpansionInOverviewChangedDelegate;
	FOnStructureChanged StructureChangedDelegate;
	FOnDataObjectChanged DataObjectChangedDelegate;
	FOnStackEntrySelectionRequested OnStackEntrySelectionRequestedDelegate;

	// ~Search stuff
	TOptional<FNiagaraStackSearchConfig> CurrentSearchConfig;
	FText CurrentSearchText;
	FOnSearchCompleted SearchCompletedDelegate;
	TArray<FSearchWorkItem> ItemsToSearch;
	TArray<FSearchResult> CurrentSearchResults;
	static NIAGARAEDITOR_API const double MaxSearchTime;
	TOptional<FSearchResult> FocusedSearchResultCache;
	bool bRestartSearch;
	bool bRefreshPending;
	bool bValidatorUpdatePending;
	bool bHasIssues;
	int32 CurrentIssueCycleIndex;
	TWeakPtr<FTopLevelViewModel> CyclingIssuesForTopLevel;

	bool bUsesTopLevelViewModels;
	TArray<TSharedRef<FTopLevelViewModel>> TopLevelViewModels;

	FNiagaraStackViewModelOptions Options;
};
