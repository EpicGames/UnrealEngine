// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"

/** 
 A helper class to allow us to hook up Niagara view model types with the source models that they are currently observing. The assumption is that 
 a ViewModel class will register and unregister the correct the model during its lifetime. Since it is possible that the ViewModel will hold onto the 
 Model with a weak object pointer, we keep track of the registered pair using a separate data structure (aka Handle). You can only unregister with a 
 valid handle. It is possible for a Model to be pointed to by multiple simultaneous ViewModels (like say event handler scripts). If this happens, you 
 can query the additional ViewModels in the GetExistingViewModelForObject method.
 */
template< class ObjectType, class ViewModelType>
class TNiagaraViewModelManager
{
public:
	class Handle
	{
	public:
		Handle() : Model(nullptr), ViewModel(nullptr)
		{

		}
		Handle(ObjectType* InModel, ViewModelType* InViewModel) : Model(InModel), ViewModel(InViewModel)
		{
		}
		ObjectType* Model;
		ViewModelType* ViewModel;
	};

protected:
	/** Called to register a specific Model/ViewModel pair.*/
	Handle RegisterViewModelWithMap(ObjectType* Model,  ViewModelType* ViewModel)
	{
		if (Model == nullptr)
		{
			return Handle(nullptr, nullptr);
		}

		TArray<ViewModelType*>* FoundItem = ObjectsToViewModels.Find(Model);
		Handle NewHandle(Model, ViewModel);
		if (FoundItem != nullptr)
		{
			check(FoundItem->Contains(ViewModel) == false);
			FoundItem->Add(ViewModel);
		}
		else
		{
			TArray<ViewModelType*> Values;
			Values.Add(ViewModel);
			ObjectsToViewModels.Add(Model, Values);
		}
		return NewHandle;
	}

	/** Called to forget about a specific Model/ViewModel pair.*/
	void UnregisterViewModelWithMap(Handle InHandle)
	{
		if (InHandle.Model != nullptr)
		{
			TArray<ViewModelType*>* FoundItem = ObjectsToViewModels.Find(InHandle.Model);
			if (FoundItem != nullptr && FoundItem->Remove(InHandle.ViewModel))
			{
				if (FoundItem->Num() == 0)
				{
					ObjectsToViewModels.Remove(InHandle.Model);
				}
			}
		}
	}

public:
	/** Query to determine if any ViewModel is currently pointing at this Model.*/
	static TSharedPtr<ViewModelType> GetExistingViewModelForObject(ObjectType* Object, int32 WhichIdx = 0)
	{
		TArray<ViewModelType*>* FoundItem = ObjectsToViewModels.Find(Object);
		if (FoundItem == nullptr)
		{
			return TSharedPtr<ViewModelType>();
		}

		if (FoundItem->Num() > WhichIdx && WhichIdx >= 0)
		{
			return StaticCastSharedRef<ViewModelType>((*FoundItem)[WhichIdx]->AsShared());
		}
		else
		{
			return TSharedPtr<ViewModelType>();
		}
	}

	static bool GetAllViewModelsForObject(ObjectType* Object, TArray<TSharedPtr<ViewModelType>>& OutViewModels)
	{
		TArray<ViewModelType*>* FoundItem = ObjectsToViewModels.Find(Object);
		if (FoundItem)
		{
			for (ViewModelType* ViewModel : *FoundItem)
			{
				OutViewModels.Add(ViewModel->AsShared());
			}
			return OutViewModels.Num() > 0;
		}
		return false;
	}

	static void GetAllViewModels(TArray<TSharedRef<ViewModelType>>& OutViewModels)
	{
		TArray<TArray<ViewModelType*>> ObjectsToViewModelsValues;
		ObjectsToViewModels.GenerateValueArray(ObjectsToViewModelsValues);
		for (const TArray<ViewModelType*>& ViewModelsByObject : ObjectsToViewModelsValues)
		{
			for (ViewModelType* ViewModel : ViewModelsByObject)
			{
				if (ViewModel != nullptr)
				{
					OutViewModels.Add(ViewModel->AsShared());
				}
			}
		}
	}

	/** Called by a module manager to ensure that all known references have been cleared out before module shutdown.*/
	static void CleanAll()
	{
		check(ObjectsToViewModels.Num() == 0);
		ObjectsToViewModels.Empty();
	}

private:
	static TMap<ObjectType*, TArray<ViewModelType*>> ObjectsToViewModels;
};
