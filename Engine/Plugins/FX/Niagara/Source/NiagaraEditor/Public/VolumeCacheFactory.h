// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "VolumeCacheFactory.generated.h"

/** Factory for creating volume texture */
#if WITH_EDITOR

UCLASS(hidecategories=Object, MinimalAPI)
class UVolumeCacheFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface
};
#endif


