// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/STaggedAssetBrowser.h"

#define UE_API NIAGARAEDITOR_API

class FNiagaraSystemViewModel;

class FFrontendFilter_NiagaraEmitterInheritance : public FFrontendFilter
{
public:
	FFrontendFilter_NiagaraEmitterInheritance(bool bInRequiredInheritanceState, TSharedPtr<FFrontendFilterCategory> Category) : FFrontendFilter(Category), bRequiredInheritanceState(bInRequiredInheritanceState)
	{}

	virtual FLinearColor GetColor() const override { return FLinearColor::Red; }
	virtual FString GetName() const override { return FString::Printf(TEXT("Inheritance: %s"), bRequiredInheritanceState ? TEXT("Yes") : TEXT("No")); }
	virtual FText GetDisplayName() const override;

	virtual FText GetToolTipText() const override;

	virtual bool PassesFilter(const FContentBrowserItem& InItem) const override;
private:
	bool bRequiredInheritanceState;
};

class FFrontendFilter_NiagaraSystemEffectType : public FFrontendFilter
{
public:
	FFrontendFilter_NiagaraSystemEffectType(bool bInRequiresEffectType, TSharedPtr<FFrontendFilterCategory> Category) : FFrontendFilter(Category), bRequiresEffectType(bInRequiresEffectType)
	{}

	virtual FLinearColor GetColor() const override { return FLinearColor::Red; }
	virtual FString GetName() const override { return FString::Printf(TEXT("Effect Type: %s"), bRequiresEffectType ? TEXT("Yes") : TEXT("No")); }
	virtual FText GetDisplayName() const override;

	virtual FText GetToolTipText() const override;

	virtual bool PassesFilter(const FContentBrowserItem& InItem) const override;
private:
	bool bRequiresEffectType;
};

class SNiagaraAddEmitterToSystemWindow : public STaggedAssetBrowserWindow
{
public:
	SLATE_BEGIN_ARGS(SNiagaraAddEmitterToSystemWindow)
	{
	}
		SLATE_ARGUMENT(STaggedAssetBrowserWindow::FArguments, AssetBrowserWindowArgs)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const UTaggedAssetBrowserConfiguration& Configuration, TSharedRef<FNiagaraSystemViewModel> SystemViewModel);
	
private:
	bool CheckValidArguments(const FArguments& InArgs) const;
	
	/** The function that will be called by our buttons or by the asset picker itself if double-clicking, hitting enter etc. */
	void OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type);

	FReply AddMinimalEmitter();
	FReply AddSelectedEmitters();
	FReply Cancel();
	
	FText GetAddButtonTooltip() const;
private:
	TWeakPtr<FNiagaraSystemViewModel> WeakSystemViewModel;
};

#undef UE_API
