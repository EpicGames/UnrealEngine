// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/SyntaxTokenizer.h"
#include "Text/HLSLSyntaxHighlighterMarshaller.h"

class FTextLayout;

class FNiagaraHLSLSyntaxHighlighter : public FHLSLSyntaxHighlighterMarshaller
{
public:

	static NIAGARAEDITOR_API TSharedRef<FNiagaraHLSLSyntaxHighlighter> Create(const FSyntaxTextStyle& InSyntaxTextStyle = GetSyntaxTextStyle());
	static NIAGARAEDITOR_API FSyntaxTextStyle GetSyntaxTextStyle();

protected:
	NIAGARAEDITOR_API FNiagaraHLSLSyntaxHighlighter(TSharedPtr< ISyntaxTokenizer > InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle);
};
