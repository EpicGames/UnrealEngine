// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SDynamicLayoutBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateNamedWidget, FName /* InWidgetName */);

	class FNamedWidgetProvider
	{
	public:
		NIAGARAEDITOR_API FNamedWidgetProvider();
		NIAGARAEDITOR_API FNamedWidgetProvider(FOnGenerateNamedWidget InGeneratedNamedWidget);

		NIAGARAEDITOR_API TSharedRef<SWidget> GetNamedWidget(FName InWidgetName) const;

	private:
		mutable TMap<FName, TSharedRef<SWidget>> NameToGeneratedWidgetCache;
		FOnGenerateNamedWidget GenerateNamedWidget;
	};

	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SWidget>, FOnGenerateNamedLayout, FName /* InLayoutName */, const FNamedWidgetProvider& /* InNamedWidgetProvider */);

	DECLARE_DELEGATE_RetVal(FName, FOnChooseLayout);

public:
	SLATE_BEGIN_ARGS(SDynamicLayoutBox)
	{}
		SLATE_EVENT(FOnGenerateNamedWidget, GenerateNamedWidget)
		SLATE_EVENT(FOnGenerateNamedLayout, GenerateNamedLayout)
		SLATE_EVENT(FOnChooseLayout, ChooseLayout);
	SLATE_END_ARGS();

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs);

	NIAGARAEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	FNamedWidgetProvider NamedWidgetProvider;
	FOnGenerateNamedLayout GenerateNamedLayout;
	FOnChooseLayout ChooseLayout;
	TOptional<FName> CurrentLayout;
};
