// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STreeView.h"
#include "Styling/AppStyle.h"
#include "NiagaraEditorStyle.h"
#include "Framework/Application/SlateApplication.h"

/** Called to force the item selector to refresh */
DECLARE_DELEGATE(FRefreshItemSelectorDelegate);

enum class EItemSelectorClickActivateMode
{
	SingleClick,
	DoubleClick
};

/** A generic widget for selecting an item from an array of items including optional filtering and categorization. */
template<typename CategoryType, typename ItemType, typename SectionType = CategoryType, typename CategoryKeyType = uint32, typename ItemKeyType = uint32, typename SectionKeyType = uint32>
class SItemSelector : public SCompoundWidget
{
public:
	struct FSectionData
	{
		enum ESectionType
		{
			Tree,
			List
		};

		FSectionData()
		{
			Type = ESectionType::Tree;
			bHideSectionIfEmpty = true;
		}

		FSectionData(ESectionType InType, bool bInHideSectionIfEmpty)
		: Type(InType)
		, bHideSectionIfEmpty(bInHideSectionIfEmpty)
		{
			
		}

		ESectionType Type;
		bool bHideSectionIfEmpty;
	};
	
	DECLARE_DELEGATE_RetVal_OneParam(TArray<CategoryType>, FOnGetCategoriesForItem, const ItemType& /* Item */);
	DECLARE_DELEGATE_RetVal_OneParam(TArray<SectionType>, FOnGetSectionsForItem, const ItemType& /* Item */);
	DECLARE_DELEGATE_RetVal_TwoParams(int32, FOnGetItemWeight, const ItemType& /* Item */, const TArray<FString>& /* FilterTerms */);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCompareSectionsForEquality, const SectionType& /* SectionA */, const SectionType& /* SectionB */);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCompareSectionsForSorting, const SectionType& /* SectionA */, const SectionType& /* SectionB */);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCompareCategoriesForEquality, const CategoryType& /* CategoryA */, const CategoryType& /* CategoryB */);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCompareCategoriesForSorting, const CategoryType& /* CategoryA */, const CategoryType& /* CategoryB */);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCompareItemsForEquality, const ItemType& /* ItemA */, const ItemType& /* ItemB */);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCompareItemsForSorting, const ItemType& /* ItemA */, const ItemType& /* ItemB */);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnDoesItemMatchFilterText, const FText& /* Filter text */, const ItemType& /* Item */);
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateWidgetForSection, const SectionType& /* Section */);
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateWidgetForCategory, const CategoryType& /* Category */);
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateWidgetForItem, const ItemType& /* Item */);
	DECLARE_DELEGATE(FOnSelectionChanged);
	DECLARE_DELEGATE_TwoParams(FOnCategoryExpansionChanged, const CategoryType& /* Category */, bool /* bIsExpanded */);
	DECLARE_DELEGATE_RetVal_OneParam(FSectionData, FOnGetSectionData, const SectionType& /* Section */);
	DECLARE_DELEGATE_OneParam(FOnItemActivated, const ItemType& /* Item */);
	DECLARE_DELEGATE_OneParam(FOnCategoryActivated, const CategoryType& /* Category */);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnDoesItemPassCustomFilter, const ItemType& /*Item */);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnDoesSectionPassCustomFilter, const SectionType& /* Section */);
	DECLARE_DELEGATE_RetVal_TwoParams(const FSlateBrush*, FOnGetCategoryBackgroundImage, bool /* bIsHovered */, bool /* bIsExpanded */)
	DECLARE_DELEGATE_TwoParams(FOnItemSelected, const ItemType& /* SelectedItem */, ESelectInfo::Type /* SelectInfo */);
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnItemsDragged, const TArray<ItemType>& /* DraggedItems */, const FPointerEvent& /* MouseEvent */);
	DECLARE_DELEGATE_RetVal_OneParam(const ItemKeyType&, FOnGetKeyForItem, const ItemType& /* Item */);
	DECLARE_DELEGATE_RetVal_OneParam(const CategoryKeyType&, FOnGetKeyForCategory, const CategoryType& /* Category */);
	DECLARE_DELEGATE_RetVal_OneParam(const SectionKeyType&, FOnGetKeyForSection, const SectionType& /* Section */);
	DECLARE_DELEGATE_OneParam(FOnSuggestionUpdated, int32 /* Suggestion Index*/)
	DECLARE_DELEGATE_TwoParams(FOnItemRowHoverEvent, const ItemType&, bool /* bHover */)

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCategoryPassesFilter, const CategoryType& /* CategoryA */);

public:
	SLATE_BEGIN_ARGS(SItemSelector)
		: _AllowMultiselect(false)
		, _ClickActivateMode(EItemSelectorClickActivateMode::DoubleClick)
		, _PreserveExpansionOnRefresh(false)
		, _PreserveSelectionOnRefresh(false)
		, _CategoryRowStyle(&FNiagaraEditorStyle::Get().GetWidgetStyle<FTableRowStyle>("ActionMenu.Row"))
		, _SectionRowStyle(&FNiagaraEditorStyle::Get().GetWidgetStyle<FTableRowStyle>("ActionMenu.Row"))
		, _ItemRowStyle(&FNiagaraEditorStyle::Get().GetWidgetStyle<FTableRowStyle>("ActionMenu.Row"))
		, _ClearSelectionOnClick(true)
		, _HideSingleSection(true)
		, _ExpandInitially(true)
	{}
		/** The items to display in the item selector. */
		SLATE_ARGUMENT(TArray<ItemType>, Items)

		/** The default categories to show in additional to ones generated from the items.
		NOTE: The OnCompareCategoriesForEquality, and OnGenerateWidgetForCategory delegates must be bound if this argument is supplied. */
		SLATE_ARGUMENT(TArray<CategoryType>, DefaultCategories)
		/** The default category paths to show in additional to ones generated from the items. 
		NOTE: The OnCompareCategoriesForEquality, and OnGenerateWidgetForCategory delegates must be bound if this argument is supplied. */
		SLATE_ARGUMENT(TArray<TArray<CategoryType>>, DefaultCategoryPaths)

		/** Whether or not this item selector should allow multiple items to be selected. */
		SLATE_ARGUMENT(bool, AllowMultiselect)

		/** Whether or not a single click activates an item. */
		SLATE_ARGUMENT(EItemSelectorClickActivateMode, ClickActivateMode)

		/** Whether or not to preserve the expansion state when refresh is called. If true, OnGetKeyForCategory and OnGetKeyForSection must be set if Categories or Sections are used, respectively. */
		SLATE_ARGUMENT(bool, PreserveExpansionOnRefresh)

		/** Whether or not to preserve the selection state when refresh is called. If true, OnGetKeyForItem must be set, and OnGetKeyForCategory and OnGetKeyForSection must be set if Categories or Sections are used, respectively. */
		SLATE_ARGUMENT(bool, PreserveSelectionOnRefresh)

		/** Optional style override to use for category rows. */
		SLATE_STYLE_ARGUMENT(FTableRowStyle, CategoryRowStyle)

		/** The style to use for section rows. */
        SLATE_STYLE_ARGUMENT(FTableRowStyle, SectionRowStyle)

		/** The style to use for category rows. */
        SLATE_STYLE_ARGUMENT(FTableRowStyle, ItemRowStyle)

		/** Optional border image override to use for category rows. */
		SLATE_EVENT(FOnGetCategoryBackgroundImage, OnGetCategoryBackgroundImage)

		/** Optional border background color override to use for category rows. */
		SLATE_ARGUMENT(FLinearColor, CategoryBorderBackgroundColor)

		/** Optional padding override to use for category child slots. */
		SLATE_ARGUMENT(FMargin, CategoryChildSlotPadding)

		/** Optional padding override to use for category backgrounds. */
		SLATE_ARGUMENT(FMargin, CategoryBorderBackgroundPadding)

		/** Whether or not the selection should be cleared when an empty area is clicked. */
		SLATE_ARGUMENT(bool, ClearSelectionOnClick)

		/** An optional delegate to get an array of categories for the specified item. Each category in the returned array represents one level of nested categories. 
		NOTE: The OnCompareCategoriesForEquality, and OnGenerateWidgetForCategory delegates must be bound if this delegate is bound. */
		SLATE_EVENT(FOnGetCategoriesForItem, OnGetCategoriesForItem)

		/** An optional delegate to get an array of sections for the specified item. Used to duplicate items across multiple sections. */
		SLATE_EVENT(FOnGetSectionsForItem, OnGetSectionsForItem)
	
		/** An optional delegate to compare two sections for equality which must be supplied when generating sections for items. This equality comparer will be used to collate items into matching categories. */
		SLATE_EVENT(FOnCompareSectionsForEquality, OnCompareSectionsForEquality)

		/** An optional delegate which determines the sorting for sections. If not bound categories will be ordered by the order they're encountered while processing items. */
		SLATE_EVENT(FOnCompareSectionsForSorting, OnCompareSectionsForSorting)
	
		/** An optional delegate to compare two categories for equality which must be supplied when generating categories for items.  This equality comparer will be used to collate items into matching categories. */
		SLATE_EVENT(FOnCompareCategoriesForEquality, OnCompareCategoriesForEquality)

		/** An optional delegate which determines the sorting for categories.  If not bound categories will be ordered by the order they're encountered while processing items. */
		SLATE_EVENT(FOnCompareCategoriesForSorting, OnCompareCategoriesForSorting)

		/** An optional delegate which compares items for equality.  This is required for using the SetSelectedItems api. */
		SLATE_EVENT(FOnCompareItemsForEquality, OnCompareItemsForEquality)

		/** An optional delegate which determines the sorting for items within each category. */
		SLATE_EVENT(FOnCompareItemsForSorting, OnCompareItemsForSorting)

		/** An optional delegate which can be used to filter items available for selection.  If not bound the search box will not be shown. */
		SLATE_EVENT(FOnDoesItemMatchFilterText, OnDoesItemMatchFilterText)

		/** An optional delegate which generates widgets for sections which can be bound to provide custom section widgets for items. */
		SLATE_EVENT(FOnGenerateWidgetForSection, OnGenerateWidgetForSection)
	
		/** An optional delegate which generates widgets for categories which must be bound when generating categories for items. */
		SLATE_EVENT(FOnGenerateWidgetForCategory, OnGenerateWidgetForCategory)

		/** The delegate which is used to generate widgets for the items to be selected. */
		SLATE_EVENT(FOnGenerateWidgetForItem, OnGenerateWidgetForItem)

		/** A delegate which is called when an item is activated by either double clicking on it or by pressing enter while it's selected. */
		SLATE_EVENT(FOnItemActivated, OnItemActivated)

		/** A delegate which is called when a category is activated by either double clicking on it or by pressing enter while it's selected. */
		SLATE_EVENT(FOnCategoryActivated, OnCategoryActivated)

		/** A delegate which is called when the selection changes. */
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)

		/** An optional delegate called when a categories expansion state changes. */
		SLATE_EVENT(FOnCategoryExpansionChanged, OnCategoryExpansionChanged)

		/** An optional delegate which is used to determine the suggested item of a search. The highest weighted item will be selected. */
		SLATE_EVENT(FOnGetItemWeight, OnGetItemWeight)
	
		/** An optional delegate which is called to check if an item should be filtered out by external code. Return false to exclude the item from the view. */
		SLATE_EVENT(FOnDoesItemPassCustomFilter, OnDoesItemPassCustomFilter)

		/** An optional delegate which is called to check if an entire section should be filtered out by external code. Return false to exclude the item from the view. */
		SLATE_EVENT(FOnDoesSectionPassCustomFilter, OnDoesSectionPassCustomFilter)

		/** An optional delegate used to determine additional options for sections, such as type (tree or list). */
		SLATE_EVENT(FOnGetSectionData, OnGetSectionData)

		/** An optional attribute to determine whether we should hide a single section. Used to reparent all children of our sole section to the root. */
		SLATE_ATTRIBUTE(bool, HideSingleSection)
	
		/** An optional delegate called when a context menu would open for an item. The user returns the menu content to display or null if a context menu should not be opened. */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)

		/** An optional delegate called when one or more items are selected. */
		SLATE_EVENT(FOnItemSelected, OnItemSelected)

		/** An optional delegate called when one or more items are dragged. */
		SLATE_EVENT(FOnItemsDragged, OnItemsDragged)

		/** An optional delegate called to get a stable key (supports GetTypeHash()) for an item. If set, this delegate is used to uniquely key items to preserve selection and expansion state when refresh is called. */
		SLATE_EVENT(FOnGetKeyForItem, OnGetKeyForItem)

		/** An optional delegate called to get a stable key (supports GetTypeHash()) for a category. If set, this delegate is used to uniquely key categories to preserve selection and expansion state when refresh is called. */
		SLATE_EVENT(FOnGetKeyForCategory, OnGetKeyForCategory)

		/** An optional delegate called to get a stable key (supports GetTypeHash()) for a section. If set, this delegate is used to uniquely key sections to preserve selection and expansion state when refresh is called. */
		SLATE_EVENT(FOnGetKeyForSection, OnGetKeyForSection)

		SLATE_EVENT(FOnCategoryPassesFilter, OnGetCategoryExpandedInitially)
	
		/** Whether we want to expand the tree initially or not. */
		SLATE_ARGUMENT(bool, ExpandInitially)

		SLATE_EVENT(FOnItemRowHoverEvent, OnItemRowHoverEvent)
		
		/** An optional delegate called when initially creating this and ExpandInitially is false. */

		/** Slot for additional widget content to go adjacent to right of the search box. */
		SLATE_NAMED_SLOT(FArguments, SearchBoxAdjacentContent)
	SLATE_END_ARGS();

private:
	enum class EItemSelectorItemViewModelType
	{
		Section,
		Category,
		Item,
		Root
	};

	class IItemSelectorItemViewModelUtilities
	{
	public:
		virtual ~IItemSelectorItemViewModelUtilities() = default;

		virtual bool IsFiltering() const = 0;
		virtual bool IsSearching() const = 0;
		virtual const FText& GetFilterText() const = 0;
		virtual bool DoesItemPassFilter(const ItemType& InItem) const = 0;
		virtual bool DoesSectionPassFilter(const SectionType& InSection) const = 0;
		virtual int32 GetItemWeight(const ItemType& Item, const TArray<FString>& FilterTerms) const = 0;
		
		virtual bool CompareSectionsForEquality(const SectionType& SectionA, const SectionType& SectionB) const = 0;
		virtual const FOnCompareSectionsForSorting& GetOnCompareSectionsForSorting() const = 0;
		virtual bool CompareCategoriesForEquality(const CategoryType& CategoryA, const CategoryType& CategoryB) const = 0;
		virtual const FOnCompareCategoriesForSorting& GetOnCompareCategoriesForSorting() const = 0;
		virtual bool CompareItemsForEquality(const ItemType& ItemA, const ItemType& ItemB) const = 0;
		virtual const FOnCompareItemsForSorting& GetOnCompareItemsForSorting() const = 0;

		virtual const ItemKeyType GetKeyForItem(const ItemType& InItem) const = 0;
		virtual bool ShouldHideSingleSection() const = 0;
	};

	class FItemSelectorItemViewModel
	{
	public:
		FItemSelectorItemViewModel(TSharedRef<IItemSelectorItemViewModelUtilities> InItemUtilities, EItemSelectorItemViewModelType InType)
			: ItemUtilities(InItemUtilities)
			, Type(InType)
		{
		}

		virtual ~FItemSelectorItemViewModel() = default;

		TSharedRef<IItemSelectorItemViewModelUtilities> GetItemUtilities() const
		{
			TSharedPtr<IItemSelectorItemViewModelUtilities> ItemUtilitiesPinned = ItemUtilities.Pin();
			checkf(ItemUtilities.IsValid(), TEXT("Item utilities deleted before item."));
			return ItemUtilitiesPinned.ToSharedRef();
		}

		EItemSelectorItemViewModelType GetType() const
		{
			return Type;
		}

		void GetChildren(TArray<TSharedRef<FItemSelectorItemViewModel>>& OutChildren) const
		{
			if (GetItemUtilities()->IsFiltering())
			{				
				TArray<TSharedRef<FItemSelectorItemViewModel>> Children;
				GetChildrenInternal(Children);
				for (TSharedRef<FItemSelectorItemViewModel> Child : Children)
				{
					if (Child->PassesFilter())
					{
						OutChildren.Add(Child);
					}
				}
			}
			else
			{
				GetChildrenInternal(OutChildren);
			}
		}

		virtual int32 GetItemWeight(const TArray<FString>& FilterTerms) const
		{
			return -1;
		}
		
		virtual bool PassesFilter() const
		{
			return true;
		}

	protected:
		virtual void GetChildrenInternal(TArray<TSharedRef<FItemSelectorItemViewModel>>& OutChildren) const
		{
		}

	private:
		TWeakPtr<IItemSelectorItemViewModelUtilities> ItemUtilities;
		EItemSelectorItemViewModelType Type;
	};

	class FItemSelectorItemContainerViewModel : public FItemSelectorItemViewModel
	{
	public:
		FItemSelectorItemContainerViewModel(TSharedRef<IItemSelectorItemViewModelUtilities> InItemUtilities, const ItemType& InItem, const ItemKeyType& InItemKey)
			: FItemSelectorItemViewModel(InItemUtilities, EItemSelectorItemViewModelType::Item)
			, Item(InItem)
			, ItemKey(InItemKey)
		{
		}

		const ItemType& GetItem() const
		{
			return Item;
		}

		const ItemKeyType& GetItemKey() const
		{
			return ItemKey;
		}

		virtual int32 GetItemWeight(const TArray<FString>& FilterTerms) const
		{
			return this->GetItemUtilities()->GetItemWeight(Item, FilterTerms);
		}
		
		virtual bool PassesFilter() const override
		{
			return this->GetItemUtilities()->DoesItemPassFilter(Item);
		}

	private:
		const ItemType& Item;
		const ItemKeyType ItemKey;
	};

	class FItemSelectorItemCategoryViewModel : public FItemSelectorItemViewModel
	{
	public:
		FItemSelectorItemCategoryViewModel(TSharedRef<IItemSelectorItemViewModelUtilities> InItemUtilities, const CategoryType& InCategory)
			: FItemSelectorItemViewModel(InItemUtilities, EItemSelectorItemViewModelType::Category)
			, Category(InCategory)
		{
		}

		const CategoryType& GetCategory() const
		{
			return Category;
		}

		void GetItemViewModelsForItems(const TArray<ItemType>& InItems, TArray<TSharedRef<FItemSelectorItemViewModel>>& OutItemViewModelsForItems)
		{
			for (TSharedRef<FItemSelectorItemCategoryViewModel> ChildCategoryViewModel : ChildCategoryViewModels)
			{
				ChildCategoryViewModel->GetItemViewModelsForItems(InItems, OutItemViewModelsForItems);
			}
			for (TSharedRef<FItemSelectorItemContainerViewModel> ChildItemViewModel : ChildItemViewModels)
			{
				const ItemType& ChildItem = ChildItemViewModel->GetItem();
				TSharedRef<IItemSelectorItemViewModelUtilities> MyItemUtilities = this->GetItemUtilities();
				if (InItems.ContainsByPredicate([ChildItem, MyItemUtilities](const ItemType& Item) { return MyItemUtilities->CompareItemsForEquality(Item, ChildItem); }))
				{
					OutItemViewModelsForItems.Add(ChildItemViewModel);
				}
			}
		}

		TSharedRef<FItemSelectorItemCategoryViewModel> AddCategory(const CategoryType& InCategory)
		{
			TSharedRef<FItemSelectorItemCategoryViewModel> NewCategoryViewModel = MakeShared<FItemSelectorItemCategoryViewModel>(this->GetItemUtilities(), InCategory);
			ChildCategoryViewModels.Add(NewCategoryViewModel);
			return NewCategoryViewModel;
		}

		TSharedPtr<FItemSelectorItemCategoryViewModel> FindCategory(const CategoryType& InCategory)
		{
			for (TSharedRef<FItemSelectorItemCategoryViewModel> ChildCategoryViewModel : ChildCategoryViewModels)
			{
				if (this->GetItemUtilities()->CompareCategoriesForEquality(ChildCategoryViewModel->GetCategory(), InCategory))
				{
					return ChildCategoryViewModel;
				}
			}
			return TSharedPtr<FItemSelectorItemCategoryViewModel>();
		}
		
		void AddItemDirect(const ItemType& InItem)
		{
			const ItemKeyType ItemKey = this->GetItemUtilities()->GetKeyForItem(InItem);
			ChildItemViewModels.Add(MakeShared<FItemSelectorItemContainerViewModel>(this->GetItemUtilities(), InItem, ItemKey));
		}

		void FlattenChildren(TArray<TSharedRef<FItemSelectorItemContainerViewModel>>& FilteredFlattenedItems)
		{
			for (TSharedRef<FItemSelectorItemCategoryViewModel> const& CategoryViewModel : ChildCategoryViewModels)
			{
				if(CategoryViewModel->PassesFilter())
				{
					CategoryViewModel->FlattenChildren(FilteredFlattenedItems);
				}
			}

			for (TSharedRef<FItemSelectorItemContainerViewModel> const& Item : ChildItemViewModels)
			{
				if(Item->PassesFilter())
				{
					FilteredFlattenedItems.Add(Item);					
				}
			}
		}
		
		void SortChildren()
		{
			TSharedRef<IItemSelectorItemViewModelUtilities> Utilities = this->GetItemUtilities();
			if (ChildCategoryViewModels.Num() > 0 && Utilities->GetOnCompareCategoriesForSorting().IsBound())
			{
				ChildCategoryViewModels.Sort([Utilities](const TSharedRef<FItemSelectorItemCategoryViewModel>& CategoryViewModelA, const TSharedRef<FItemSelectorItemCategoryViewModel>& CategoryViewModelB)
				{
					return Utilities->GetOnCompareCategoriesForSorting().Execute(CategoryViewModelA->GetCategory(), CategoryViewModelB->GetCategory());
				});

			}
			if (ChildItemViewModels.Num() > 0 && Utilities->GetOnCompareItemsForSorting().IsBound())
			{
				ChildItemViewModels.Sort([Utilities](const TSharedRef<FItemSelectorItemContainerViewModel>& ItemViewModelA, const TSharedRef<FItemSelectorItemContainerViewModel>& ItemViewModelB)
				{
					return Utilities->GetOnCompareItemsForSorting().Execute(ItemViewModelA->GetItem(), ItemViewModelB->GetItem());
				});
			}

			for (TSharedRef<FItemSelectorItemCategoryViewModel> ChildCategoryViewModel : ChildCategoryViewModels)
			{
				ChildCategoryViewModel->SortChildren();
			}
		}

		virtual int32 GetItemWeight(const TArray<FString>& FilterTerms) const override
		{
			return -1;	
		}
		
		virtual bool PassesFilter() const override
		{
			for (const TSharedRef<FItemSelectorItemContainerViewModel>& ChildItemViewModel : ChildItemViewModels)
			{
				if (ChildItemViewModel->PassesFilter())
				{
					return true;
				}
			}
			for (const TSharedRef<FItemSelectorItemCategoryViewModel>& ChildCategoryViewModel : ChildCategoryViewModels)
			{
				if (ChildCategoryViewModel->PassesFilter())
				{
					return true;
				}
			}
			return false;
		}

	protected:
		virtual void GetChildrenInternal(TArray<TSharedRef<FItemSelectorItemViewModel>>& OutChildren) const
		{
			OutChildren.Append(ChildCategoryViewModels);
			OutChildren.Append(ChildItemViewModels);
		}

	private:
		CategoryType Category;
		TArray<TSharedRef<FItemSelectorItemCategoryViewModel>> ChildCategoryViewModels;
		TArray<TSharedRef<FItemSelectorItemContainerViewModel>> ChildItemViewModels;
	};

	class FSectionViewModel : public FItemSelectorItemViewModel
	{
	public:
		FSectionViewModel(TSharedRef<IItemSelectorItemViewModelUtilities> InItemUtilities, const SectionType& InSection)
			: FItemSelectorItemViewModel(InItemUtilities, EItemSelectorItemViewModelType::Section)
			, Section(InSection)
		{
		}

		void AddItemDirect(const ItemType& InItem)
		{
			const ItemKeyType ItemKey = this->GetItemUtilities()->GetKeyForItem(InItem);
			ChildItemViewModels.Add(MakeShared<FItemSelectorItemContainerViewModel>(this->GetItemUtilities(), InItem, ItemKey));
		}

		TSharedRef<FItemSelectorItemCategoryViewModel> AddCategory(const CategoryType& InCategory)
		{
			TSharedRef<FItemSelectorItemCategoryViewModel> NewCategoryViewModel = MakeShared<FItemSelectorItemCategoryViewModel>(this->GetItemUtilities(), InCategory);
			ChildCategoryViewModels.Add(NewCategoryViewModel);
			return NewCategoryViewModel;
		}

		TSharedPtr<FItemSelectorItemCategoryViewModel> FindCategory(const CategoryType& InCategory)
		{
			for (TSharedRef<FItemSelectorItemCategoryViewModel> ChildCategoryViewModel : ChildCategoryViewModels)
			{
				if (this->GetItemUtilities()->CompareCategoriesForEquality(ChildCategoryViewModel->GetCategory(), InCategory))
				{
					return ChildCategoryViewModel;
				}
			}
			return TSharedPtr<FItemSelectorItemCategoryViewModel>();
		}

		TSharedRef<FItemSelectorItemCategoryViewModel> FindOrAddCategoryDirect(const CategoryType& InCategory)
		{
			TSharedPtr<FItemSelectorItemCategoryViewModel> Category = FindCategory(InCategory);

			if(!Category.IsValid())
			{
				Category = AddCategory(InCategory);
			}

			return Category.ToSharedRef();
		}

		void GetItemViewModelsForItems(const TArray<ItemType>& InItems, TArray<TSharedRef<FItemSelectorItemViewModel>>& OutItemViewModelsForItems)
		{
			for (TSharedRef<FItemSelectorItemCategoryViewModel> ChildCategoryViewModel : ChildCategoryViewModels)
			{
				ChildCategoryViewModel->GetItemViewModelsForItems(InItems, OutItemViewModelsForItems);
			}
			for (TSharedRef<FItemSelectorItemContainerViewModel> ChildItemViewModel : ChildItemViewModels)
			{
				const ItemType& ChildItem = ChildItemViewModel->GetItem();
				TSharedRef<IItemSelectorItemViewModelUtilities> MyItemUtilities = this->GetItemUtilities();
				if (InItems.ContainsByPredicate([ChildItem, MyItemUtilities](const ItemType& Item) { return MyItemUtilities->CompareItemsForEquality(Item, ChildItem); }))
				{
					OutItemViewModelsForItems.Add(ChildItemViewModel);
				}
			}
		}

		void FlattenChildren(TArray<TSharedRef<FItemSelectorItemContainerViewModel>>& FilteredFlattenedItems)
		{
			for (TSharedRef<FItemSelectorItemCategoryViewModel> const& Category : ChildCategoryViewModels)
			{
				if(Category->PassesFilter())
				{
					Category->FlattenChildren(FilteredFlattenedItems);
				}
			}

			for (TSharedRef<FItemSelectorItemContainerViewModel> const& Item : ChildItemViewModels)
			{
				if(Item->PassesFilter())
				{
					FilteredFlattenedItems.Add(Item);
				}
			}
		}
		
		void SortChildren()
		{
			TSharedRef<IItemSelectorItemViewModelUtilities> Utilities = this->GetItemUtilities();
			if (ChildCategoryViewModels.Num() > 0 && Utilities->GetOnCompareCategoriesForSorting().IsBound())
			{
				ChildCategoryViewModels.Sort([Utilities](const TSharedRef<FItemSelectorItemCategoryViewModel>& CategoryViewModelA, const TSharedRef<FItemSelectorItemCategoryViewModel>& CategoryViewModelB)
				{
					return Utilities->GetOnCompareCategoriesForSorting().Execute(CategoryViewModelA->GetCategory(), CategoryViewModelB->GetCategory());
				});
			}

			if (ChildItemViewModels.Num() > 0 && Utilities->GetOnCompareItemsForSorting().IsBound())
			{
				ChildItemViewModels.Sort([Utilities](const TSharedRef<FItemSelectorItemContainerViewModel>& ItemViewModelA, const TSharedRef<FItemSelectorItemContainerViewModel>& ItemViewModelB)
				{
					return Utilities->GetOnCompareItemsForSorting().Execute(ItemViewModelA->GetItem(), ItemViewModelB->GetItem());
				});
			}
			
			for (TSharedRef<FItemSelectorItemCategoryViewModel> ChildSectionViewModel : ChildCategoryViewModels)
			{
				ChildSectionViewModel->SortChildren();
			}
		}
		
		const SectionType& GetSection() const
		{
			return Section;
		}
		
		virtual int32 GetItemWeight(const TArray<FString>& FilterTerms) const override
		{
			return -1;
		}

		virtual bool PassesFilter() const override
		{
			return this->GetItemUtilities()->DoesSectionPassFilter(Section);
		}

		virtual void GetChildrenInternal(TArray<TSharedRef<FItemSelectorItemViewModel>>& OutChildren) const override
		{
			OutChildren.Append(ChildCategoryViewModels);
			OutChildren.Append(ChildItemViewModels);
		}

		TArray<TSharedRef<FItemSelectorItemCategoryViewModel>>& GetChildCategories()
		{
			return ChildCategoryViewModels;
		}
		
		TArray<TSharedRef<FItemSelectorItemContainerViewModel>>& GetChildItems()
		{
			return ChildItemViewModels;
		}
	
	private:
		SectionType Section;
		TArray<TSharedRef<FItemSelectorItemCategoryViewModel>> ChildCategoryViewModels;
		TArray<TSharedRef<FItemSelectorItemContainerViewModel>> ChildItemViewModels;
	};

	class FRootViewModel : public FItemSelectorItemViewModel
	{
	public:
		FRootViewModel(TSharedRef<IItemSelectorItemViewModelUtilities> InItemUtilities)
			: FItemSelectorItemViewModel(InItemUtilities, EItemSelectorItemViewModelType::Root)
		{
		}

		void GetItemViewModelsForItems(const TArray<ItemType>& InItems, TArray<TSharedRef<FItemSelectorItemViewModel>>& OutItemViewModelsForItems)
		{
			for (const TSharedRef<FSectionViewModel>& ChildSectionViewModel : ChildSectionViewModels)
			{
				ChildSectionViewModel->GetItemViewModelsForItems(InItems, OutItemViewModelsForItems);
			}
			for (const TSharedRef<FItemSelectorItemCategoryViewModel>& ChildCategoryViewModel : ChildCategoryViewModels)
			{
				ChildCategoryViewModel->GetItemViewModelsForItems(InItems, OutItemViewModelsForItems);
			}
		}

		TSharedRef<FSectionViewModel> AddSection(const SectionType& InSection)
		{
			TSharedRef<FSectionViewModel> SectionViewModel = MakeShared<FSectionViewModel>(this->GetItemUtilities(), InSection);
			ChildSectionViewModels.Add(SectionViewModel);
			return SectionViewModel;
		}

		TSharedRef<FItemSelectorItemCategoryViewModel> AddCategory(const CategoryType& InCategory)
		{
			TSharedRef<FItemSelectorItemCategoryViewModel> CategoryViewModel = MakeShared<FItemSelectorItemCategoryViewModel>(this->GetItemUtilities(), InCategory);
			ChildCategoryViewModels.Add(CategoryViewModel);
			return CategoryViewModel ;
		}
		
		TSharedPtr<FSectionViewModel> FindSection(const SectionType& InSection)
		{
			for (TSharedRef<FSectionViewModel> ChildCategoryViewModel : ChildSectionViewModels)
			{
				if (this->GetItemUtilities()->CompareSectionsForEquality(ChildCategoryViewModel->GetSection(), InSection))
				{
					return ChildCategoryViewModel;
				}
			}
			return TSharedPtr<FSectionViewModel>();
		}

		TSharedPtr<FItemSelectorItemCategoryViewModel> FindCategory(const CategoryType& InCategory)
		{
			for (TSharedRef<FItemSelectorItemCategoryViewModel> ChildCategoryViewModel : ChildCategoryViewModels)
			{
				if (this->GetItemUtilities()->CompareCategoriesForEquality(ChildCategoryViewModel->GetCategory(), InCategory))
				{
					return ChildCategoryViewModel;
				}
			}
			return TSharedPtr<FItemSelectorItemCategoryViewModel>();
		}

		TSharedRef<FItemSelectorItemCategoryViewModel> FindOrAddCategoryDirect(const CategoryType& InCategory)
		{
			TSharedPtr<FItemSelectorItemCategoryViewModel> Category = FindCategory(InCategory);

			if(!Category.IsValid())
			{
				Category = AddCategory(InCategory);
			}

			return Category.ToSharedRef();
		}

		void FlattenChildren(TArray<TSharedRef<FItemSelectorItemContainerViewModel>>& FilteredFlattenedItems)
		{
			for (TSharedRef<FSectionViewModel> const& Section : ChildSectionViewModels)
			{
				if(Section->PassesFilter())
				{
					Section->FlattenChildren(FilteredFlattenedItems);
				}
			}

			for (TSharedRef<FItemSelectorItemCategoryViewModel> const& Category : ChildCategoryViewModels)
			{
				if(Category->PassesFilter())
				{
					Category->FlattenChildren(FilteredFlattenedItems);
				}
			}

			for (TSharedRef<FItemSelectorItemContainerViewModel> const& Item : ChildItemViewModels)
			{
				if(Item->PassesFilter())
				{
					FilteredFlattenedItems.Add(Item);
				}				
			}
		}
		
		void SortChildren()
		{
			TSharedRef<IItemSelectorItemViewModelUtilities> Utilities = this->GetItemUtilities();
			if (ChildSectionViewModels.Num() > 0 && Utilities->GetOnCompareSectionsForSorting().IsBound())
			{
				ChildSectionViewModels.Sort([Utilities](const TSharedRef<FSectionViewModel>& SectionViewModelA, const TSharedRef<FSectionViewModel>& SectionViewModelB)
				{
					return Utilities->GetOnCompareSectionsForSorting().Execute(SectionViewModelA->GetSection(), SectionViewModelB->GetSection());
				});
			}

			if (ChildCategoryViewModels.Num() > 0 && Utilities->GetOnCompareCategoriesForSorting().IsBound())
			{
				ChildCategoryViewModels.Sort([Utilities](const TSharedRef<FItemSelectorItemCategoryViewModel>& CategoryViewModelA, const TSharedRef<FItemSelectorItemCategoryViewModel>& CategoryViewModelB)
				{
					return Utilities->GetOnCompareCategoriesForSorting().Execute(CategoryViewModelA->GetCategory(), CategoryViewModelB->GetCategory());
				});
			}

			if (ChildItemViewModels.Num() > 0 && Utilities->GetOnCompareItemsForSorting().IsBound())
			{
				ChildItemViewModels.Sort([Utilities](const TSharedRef<FItemSelectorItemContainerViewModel>& ItemViewModelA, const TSharedRef<FItemSelectorItemContainerViewModel>& ItemViewModelB)
				{
					return Utilities->GetOnCompareItemsForSorting().Execute(ItemViewModelA->GetItem(), ItemViewModelB->GetItem());
				});
			}

			for (TSharedRef<FSectionViewModel> ChildSectionViewModel : ChildSectionViewModels)
			{
				ChildSectionViewModel->SortChildren();
			}
			
			for (TSharedRef<FItemSelectorItemCategoryViewModel> ChildCategoryViewModel : ChildCategoryViewModels)
			{
				ChildCategoryViewModel->SortChildren();
			}
		}
		
		virtual int32 GetItemWeight(const TArray<FString>& FilterTerms) const override
		{
			return -1;
		}

		virtual bool PassesFilter() const override
		{
			return true;
		}

		const TArray<TSharedRef<FSectionViewModel>>& GetSections() const
		{
			return ChildSectionViewModels;
		}
		
		void AddItemToRootDirect(const ItemType& Item)
		{
			const ItemKeyType ItemKey = this->GetItemUtilities()->GetKeyForItem(Item);
			ChildItemViewModels.Add(MakeShared<FItemSelectorItemContainerViewModel>(this->GetItemUtilities(), Item, ItemKey));
		}
		
		void AddCategoryToRoot(const CategoryType& Category)
		{
			TSharedRef<FItemSelectorItemCategoryViewModel> NewCategoryViewModel = MakeShared<FItemSelectorItemCategoryViewModel>(this->GetItemUtilities(), Category);
			ChildCategoryViewModels.Add(NewCategoryViewModel);
		}

		void ReparentSectionItemsToRoot()
		{
			for(auto& ChildSectionViewModel : ChildSectionViewModels)
			{
				ChildCategoryViewModels.Append(ChildSectionViewModel->GetChildCategories());
				ChildItemViewModels.Append(ChildSectionViewModel->GetChildItems());
			}
			
			ChildSectionViewModels.Empty();
		}
	
	public:
	virtual void GetChildrenInternal(TArray<TSharedRef<FItemSelectorItemViewModel>>& OutChildren) const
		{
			// if we want to hide a single section, we determine if either we have exactly one section, or if we have only one that passes
			bool bAttachChildrenDirectly = false;
			if(this->GetItemUtilities()->ShouldHideSingleSection() && ChildSectionViewModels.Num() >= 1)
			{
				int32 PassingSectionsCount = 0;
				for(const TSharedRef<FSectionViewModel>& Section : ChildSectionViewModels)
				{
					if(Section->PassesFilter())
					{
						PassingSectionsCount++;
					}
				}

				bAttachChildrenDirectly = ChildSectionViewModels.Num() == 1 || PassingSectionsCount == 1;
			}

			// just add the sections directly in case we have multiple that should be displayed or we don't want to hide a single section
			if(!bAttachChildrenDirectly)
			{
				OutChildren.Append(ChildSectionViewModels);
			}
			else
			{
				// if we want to hide a single section, we need to perform an additional section pass test here
				// because sections that wouldn't pass would get their children displayed regardless otherwise
				for(const TSharedRef<FSectionViewModel>& Section : ChildSectionViewModels)
				{
					if(Section->PassesFilter())
					{
						Section->GetChildrenInternal(OutChildren);
					}
				}
			}
			OutChildren.Append(ChildCategoryViewModels);
			OutChildren.Append(ChildItemViewModels);
		}
	private:
		TArray<TSharedRef<FSectionViewModel>> ChildSectionViewModels;
		TArray<TSharedRef<FItemSelectorItemCategoryViewModel>> ChildCategoryViewModels;
		TArray<TSharedRef<FItemSelectorItemContainerViewModel>> ChildItemViewModels;
	};

	class FItemSelectorViewModel : public IItemSelectorItemViewModelUtilities, public TSharedFromThis<FItemSelectorViewModel>
	{
	public:
		FItemSelectorViewModel(TArray<ItemType> InItems,
			TArray<TArray<CategoryType>> InDefaultCategoryPaths,
			FOnGetCategoriesForItem InOnGetCategoriesForItem, FOnGetSectionsForItem InOnGetSectionsForItem,
			FOnCompareSectionsForEquality InOnCompareSectionsForEquality, FOnCompareSectionsForSorting InOnCompareSectionsForSorting,
			FOnCompareCategoriesForEquality InOnCompareCategoriesForEquality, FOnCompareCategoriesForSorting InOnCompareCategoriesForSorting,
			FOnCompareItemsForEquality InOnCompareItemsForEquality, FOnCompareItemsForSorting InOnCompareItemsForSorting,
			FOnDoesItemMatchFilterText InOnDoesItemMatchFilterText, FOnGetItemWeight InOnGetItemWeight, 
			FOnDoesItemPassCustomFilter InOnDoesItemPassCustomFilter, FOnDoesSectionPassCustomFilter InOnDoesSectionPassCustomFilter,
			FOnGetSectionData InOnGetSectionData, TAttribute<bool> InHideSingleSection,
			bool bInPreseveExpansionOnRefresh, bool bInPreserveSelectionOnRefresh,
			FOnGetKeyForItem& InOnGetKeyForItem, FOnGetKeyForCategory& InOnGetKeyForCategory, FOnGetKeyForSection& InOnGetKeyForSection,
			FOnSuggestionUpdated InOnSuggestionUpdated)
			: Items(InItems)
			, DefaultCategoryPaths(InDefaultCategoryPaths)
			, OnGetCategoriesForItem(InOnGetCategoriesForItem)
			, OnGetSectionsForItem(InOnGetSectionsForItem)
			, OnCompareSectionsForEquality(InOnCompareSectionsForEquality)
			, OnCompareSectionsForSorting(InOnCompareSectionsForSorting)
			, OnCompareCategoriesForEquality(InOnCompareCategoriesForEquality)
			, OnCompareCategoriesForSorting(InOnCompareCategoriesForSorting)
			, OnCompareItemsForEquality(InOnCompareItemsForEquality)
			, OnCompareItemsForSorting(InOnCompareItemsForSorting)
			, OnDoesItemMatchFilterText(InOnDoesItemMatchFilterText)
			, OnGetItemWeight(InOnGetItemWeight)
			, OnDoesItemPassCustomFilter(InOnDoesItemPassCustomFilter)
			, OnDoesSectionPassCustomFilter(InOnDoesSectionPassCustomFilter)
			, OnGetSectionData(InOnGetSectionData)
			, HideSingleSection(InHideSingleSection)
			, bPreserveExpansionOnRefresh(bInPreseveExpansionOnRefresh)
			, bPreserveSelectionOnRefresh(bInPreserveSelectionOnRefresh)
			, OnGetKeyForItem(InOnGetKeyForItem)
			, OnGetKeyForCategory(InOnGetKeyForCategory)
			, OnGetKeyForSection(InOnGetKeyForSection)
			, OnSuggestionUpdated(InOnSuggestionUpdated)
		{
		}
		
		void UpdateSuggestedItem()
		{
			CurrentMaxWeight = INDEX_NONE;
			CurrentSuggestionIndex = INDEX_NONE;
			
			FilteredFlattenedItems.Empty();
			RootViewModel->FlattenChildren(FilteredFlattenedItems);
					
			TArray<FString> FilterTerms;
			GetFilterText().ToString().ParseIntoArray(FilterTerms, TEXT(" "), true);
			
			if(FilterTerms.Num() > 0)
			{
				for(int32 ItemIndex = 0; ItemIndex < FilteredFlattenedItems.Num(); ItemIndex++)
				{
					int32 Weight = FilteredFlattenedItems[ItemIndex]->GetItemWeight(FilterTerms);
					if(GetCurrentMaxWeight() < Weight)
					{
						UpdateCurrentMaxWeight(Weight);
						UpdateCurrentSuggestionIndex(ItemIndex);
					}
				}
			}
			
			OnSuggestionUpdated.ExecuteIfBound(CurrentSuggestionIndex);
		}

		const TArray<TSharedRef<FItemSelectorItemViewModel>>* GetRootItems()
		{
			if (RootViewModel.IsValid() == false)
			{
				RootViewModel = MakeShared<FRootViewModel>(this->AsShared());

				// @todo feedback? Default categories but no section info
				for (const TArray<CategoryType>& DefaultCategoryPath : DefaultCategoryPaths)
				{
					FindOrAddNestedCategoryInRoot(DefaultCategoryPath);
				}

				for (const ItemType& Item : Items)
				{
					AddItemRecursive(Item);
				}
	
				RootViewModel->SortChildren();
				RootViewModel->GetChildren(RootTreeCategories);
			}
			
			return &RootTreeCategories;
		}
		
		void GetChildrenRecursive(TArray<TSharedRef<FItemSelectorItemViewModel>>& OutChildren)
        {
        	TArray<TSharedRef<FItemSelectorItemViewModel>> ItemsToProcess;
        	ItemsToProcess.Append(*GetRootItems());
        	while (ItemsToProcess.Num() > 0)
        	{
        		TSharedRef<FItemSelectorItemViewModel> ItemToProcess = ItemsToProcess[0];
        		OutChildren.Add(ItemToProcess);
        		ItemsToProcess.RemoveAtSwap(0);
        		ItemToProcess->GetChildren(ItemsToProcess);
        	}
        }

		void GetItemViewModelsForItems(const TArray<ItemType>& InItems, TArray<TSharedRef<FItemSelectorItemViewModel>>& OutItemViewModelsForItems)
		{
			RootViewModel->GetItemViewModelsForItems(InItems, OutItemViewModelsForItems);
		}

		TSharedRef<FSectionViewModel> FindOrAddSection(const SectionType& Section)
		{
			TSharedPtr<FSectionViewModel> SectionViewModel = RootViewModel->FindSection(Section);

			if(!SectionViewModel.IsValid())
			{
				SectionViewModel = RootViewModel->AddSection(Section);
			}

			return SectionViewModel.ToSharedRef();
		}

		TSharedPtr<FItemSelectorItemCategoryViewModel> FindOrAddNestedCategoryInRoot(const TArray<CategoryType>& CategoryPath)
		{
			TSharedPtr<FItemSelectorItemCategoryViewModel> NewCategoryModel = nullptr;

			if(CategoryPath.Num() > 0)
			{
				NewCategoryModel = RootViewModel->FindOrAddCategoryDirect(CategoryPath[0]);	
			}

			for (int32 CategoryIndex = 1; CategoryIndex < CategoryPath.Num(); CategoryIndex++)
			{
				TSharedPtr<FItemSelectorItemCategoryViewModel> ExistingCategoryViewModel = NewCategoryModel->FindCategory(CategoryPath[CategoryIndex]);
				if (ExistingCategoryViewModel.IsValid())
				{
					NewCategoryModel = ExistingCategoryViewModel;
				}
				else
				{
					TSharedRef<FItemSelectorItemCategoryViewModel> NewItemCategoryViewModel = NewCategoryModel->AddCategory(CategoryPath[CategoryIndex]);
					NewCategoryModel = NewItemCategoryViewModel;
				}
			}
			return NewCategoryModel;
		}

		TSharedPtr<FItemSelectorItemCategoryViewModel> FindOrAddNestedCategory(TSharedRef<FSectionViewModel>& SectionViewModel, const TArray<CategoryType>& CategoryPath)
		{
			TSharedPtr<FItemSelectorItemCategoryViewModel> NewCategoryModel = nullptr;

			if(CategoryPath.Num() > 0)
			{
				NewCategoryModel = SectionViewModel->FindOrAddCategoryDirect(CategoryPath[0]);	
			}

			// iterate over the remaining categories, if any, to find or create the nested category items
			for (int32 CategoryIndex = 1; CategoryIndex < CategoryPath.Num(); CategoryIndex++)
			{
				TSharedPtr<FItemSelectorItemCategoryViewModel> ExistingCategoryViewModel = NewCategoryModel->FindCategory(CategoryPath[CategoryIndex]);
				if (ExistingCategoryViewModel.IsValid())
				{
					NewCategoryModel = ExistingCategoryViewModel;
				}
				else
				{
					TSharedRef<FItemSelectorItemCategoryViewModel> NewItemCategoryViewModel = NewCategoryModel->AddCategory(CategoryPath[CategoryIndex]);
					NewCategoryModel = NewItemCategoryViewModel;
				}
			}
			return NewCategoryModel;
		}

		void AddItemToRoot(const ItemType& Item, const TSharedRef<TArray<CategoryType>> Categories)
		{
			TSharedPtr<FItemSelectorItemCategoryViewModel> CategoryViewModel = FindOrAddNestedCategoryInRoot(Categories.Get());

			if(CategoryViewModel.IsValid())
			{
				CategoryViewModel->AddItemDirect(Item);
			}
			else
			{
				RootViewModel->AddItemToRootDirect(Item);
			}
		}

		void AddCategoryToRoot(const CategoryType& Category)
		{
			RootViewModel->AddCategoryToRoot(Category);
		}
		
		void AddItemRecursive(const ItemType& Item)
		{
			TSharedRef<TArray<CategoryType>> ItemCategories = MakeShared<TArray<CategoryType>>();
			TSharedRef<TArray<SectionType>> Sections = MakeShared<TArray<SectionType>>();
		
			if (OnGetCategoriesForItem.IsBound())
			{
				ItemCategories->Append(OnGetCategoriesForItem.Execute(Item));
			}

			if(OnGetSectionsForItem.IsBound())
			{
				Sections->Append(OnGetSectionsForItem.Execute(Item));

				for(const SectionType& Section : Sections.Get())
				{
					FSectionData SectionData;

					// retrieve section data for every section. Use the default if it is not bound
					if(OnGetSectionData.IsBound())
					{
						SectionData = OnGetSectionData.Execute(Section);
					}
					
					TSharedRef<FSectionViewModel> SectionViewModel = FindOrAddSection(Section);

					// If we have a tree section, make use of the item's categories, if any
					if(SectionData.Type == FSectionData::ESectionType::Tree)
					{
						TSharedPtr<FItemSelectorItemCategoryViewModel> ItemCategory = nullptr;
						if(ItemCategories->Num() > 0)
						{
							ItemCategory = FindOrAddNestedCategory(SectionViewModel, ItemCategories.Get());						
						}

						if(!ItemCategory.IsValid())
						{
							SectionViewModel->AddItemDirect(Item);
						}
						else
						{
							ItemCategory->AddItemDirect(Item);
						}
					}
					// If we have a list section, we just directly add the item to the section
					else
					{
						SectionViewModel->AddItemDirect(Item);
					}					
				}
			}
			// If we have no sections for an item, we directly parent the item to the root
			else
			{
				AddItemToRoot(Item, ItemCategories);
			}
		}

		const FText& GetFilterText() const override
		{
			return FilterText;
		}

		void SetFilterText(FText InFilterText, const TSharedPtr<STreeView<TSharedRef<FItemSelectorItemViewModel>>>& ItemTreeRef)
		{
			FilterText = InFilterText;
			RootTreeCategories.Empty();
			RootViewModel->GetChildren(RootTreeCategories);
			UpdateSuggestedItem();
		}

		virtual bool IsFiltering() const override
		{
			if (OnDoesItemPassCustomFilter.IsBound())
			{
				return true;
			}
			
			return IsSearching();
		}

		virtual bool IsSearching() const override
		{
			return FilterText.IsEmptyOrWhitespace() == false;		
		}

		virtual int32 GetItemWeight(const ItemType& InItem, const TArray<FString>& FilterTerms) const override
		{
			if (OnGetItemWeight.IsBound())
			{
				return OnGetItemWeight.Execute(InItem, FilterTerms);
			}

			return 0;
		}
		
		virtual bool DoesItemPassFilter(const ItemType& InItem) const override
		{
			bool bPassesFilter = true;
			
			if (OnDoesItemPassCustomFilter.IsBound())
			{
				bPassesFilter &= OnDoesItemPassCustomFilter.Execute(InItem);
			}
			
			if ((FilterText.IsEmpty() == false) && OnDoesItemMatchFilterText.IsBound())
			{
				bPassesFilter &= OnDoesItemMatchFilterText.Execute(FilterText, InItem);
			}
			
			return bPassesFilter;
		}

		virtual bool DoesSectionPassFilter(const SectionType& InSection) const override
		{
			bool bPassesFilter = true;

			if (OnDoesSectionPassCustomFilter.IsBound())
			{
				bPassesFilter &= OnDoesSectionPassCustomFilter.Execute(InSection);
			}
			
			FSectionData Data;
			if(OnGetSectionData.IsBound())
			{
				Data = OnGetSectionData.Execute(InSection);
			}

			// we additionally check if bPassesFilter is true to avoid unnecessary filtering
			if(Data.bHideSectionIfEmpty && bPassesFilter == true)
			{
				const TArray<TSharedRef<FSectionViewModel>>& SectionViewModels = GetSections();
				for(const TSharedRef<FSectionViewModel>& SectionViewModel : SectionViewModels)
				{
					if(CompareSectionsForEquality(SectionViewModel->GetSection(), InSection))
					{
						TArray<TSharedRef<FItemSelectorItemViewModel>> Children;
						// we need to use GetChildren as it checks for passing children
						SectionViewModel->GetChildren(Children);
				
						bPassesFilter &= Children.Num() > 0;
					}
				}
			}
			
			return bPassesFilter;
		}

		virtual bool CompareSectionsForEquality(const SectionType& SectionA, const SectionType& SectionB) const override
		{
			return OnCompareSectionsForEquality.Execute(SectionA, SectionB);
		}

		virtual const FOnCompareSectionsForSorting& GetOnCompareSectionsForSorting() const override
		{
			return OnCompareSectionsForSorting;
		}
		
		virtual bool CompareCategoriesForEquality(const CategoryType& CategoryA, const CategoryType& CategoryB) const override
		{
			return OnCompareCategoriesForEquality.Execute(CategoryA, CategoryB);
		}

		virtual const FOnCompareCategoriesForSorting& GetOnCompareCategoriesForSorting() const override
		{
			return OnCompareCategoriesForSorting;
		}

		virtual bool CompareItemsForEquality(const ItemType& ItemA, const ItemType& ItemB) const override
		{
			return OnCompareItemsForEquality.Execute(ItemA, ItemB);
		}

		virtual const FOnCompareItemsForSorting& GetOnCompareItemsForSorting() const override
		{
			return OnCompareItemsForSorting;
		}

		virtual const ItemKeyType GetKeyForItem(const ItemType& InItem) const override
		{
			return OnGetKeyForItem.IsBound() ? OnGetKeyForItem.Execute(InItem) : ItemKeyType();
		}

		virtual bool ShouldHideSingleSection() const override
		{
			return HideSingleSection.Get();
		}

		bool CanCompareItems() const
		{
			return OnCompareItemsForSorting.IsBound();
		}

		int32 CompareItems(const ItemType& ItemA, const ItemType& ItemB)
		{
			return OnCompareItemsForSorting.Execute(ItemA, ItemB);
		}

		const TArray<TSharedRef<FItemSelectorItemContainerViewModel>>& GetFlattenedItems() const
		{
			return FilteredFlattenedItems;
		}

		virtual int32 GetCurrentSuggestionIndex() const
		{
			return CurrentSuggestionIndex;
		}
		
		virtual int32 GetCurrentMaxWeight() const
		{
			return CurrentMaxWeight;
		}
		
		virtual void UpdateCurrentSuggestionIndex(const int32& Index)
		{
			CurrentSuggestionIndex = Index;
		}
		
		virtual void UpdateCurrentMaxWeight(int32 InCurrentWeight)
		{
			CurrentMaxWeight = InCurrentWeight;
		}

		const TArray<TSharedRef<FSectionViewModel>>& GetSections() const
		{
			return RootViewModel->GetSections();
		}
		
		void Refresh(const TArray<ItemType>& InItems, const TArray<TArray<CategoryType>>& InDefaultCategoryPaths)
		{
			if (RootViewModel.IsValid())
			{
				RootViewModel.Reset();
				RootTreeCategories.Empty();
				FilteredFlattenedItems.Empty();
				CurrentSuggestionIndex = INDEX_NONE;
				CurrentMaxWeight = INDEX_NONE;
			}

			Items = InItems;
			DefaultCategoryPaths = InDefaultCategoryPaths;
			GetRootItems();

			// as the update to the suggested item will cause a selection, we want to avoid this when merely refreshing
			if(bPreserveSelectionOnRefresh == false)
			{
				UpdateSuggestedItem();
			}
		}

		int32 NumFlattenedItems()
		{
			return FilteredFlattenedItems.Num();
		}

		TSharedPtr<FRootViewModel> GetRootViewModel() { return RootViewModel; }
	
	private:
		TArray<ItemType> Items;
		TArray<TArray<CategoryType>> DefaultCategoryPaths;

		FOnGetCategoriesForItem OnGetCategoriesForItem;
		FOnGetSectionsForItem OnGetSectionsForItem;
		FOnCompareSectionsForEquality OnCompareSectionsForEquality;
		FOnCompareSectionsForSorting OnCompareSectionsForSorting;
		FOnCompareCategoriesForEquality OnCompareCategoriesForEquality;
		FOnCompareCategoriesForSorting OnCompareCategoriesForSorting;
		FOnCompareItemsForEquality OnCompareItemsForEquality;
		FOnCompareItemsForSorting OnCompareItemsForSorting;
		FOnDoesItemMatchFilterText OnDoesItemMatchFilterText;
		FOnGetItemWeight OnGetItemWeight;
		FOnDoesItemPassCustomFilter OnDoesItemPassCustomFilter;
		FOnDoesSectionPassCustomFilter OnDoesSectionPassCustomFilter;
		FOnGetSectionData OnGetSectionData;
		TAttribute<bool> HideSingleSection;
		bool bPreserveExpansionOnRefresh;
		bool bPreserveSelectionOnRefresh;
		FOnGetKeyForItem OnGetKeyForItem;
		FOnGetKeyForCategory OnGetKeyForCategory;
		FOnGetKeyForSection OnGetKeyForSection;
		
		int32 CurrentMaxWeight = INDEX_NONE;
		int32 CurrentSuggestionIndex = INDEX_NONE;
		TSharedPtr<FRootViewModel> RootViewModel;
		TArray<TSharedRef<FItemSelectorItemViewModel>> RootTreeCategories;
		TArray<TSharedRef<FItemSelectorItemContainerViewModel>> FilteredFlattenedItems;
		FText FilterText;

		FOnSuggestionUpdated OnSuggestionUpdated;
	};

	typedef STableRow<TSharedRef<FItemSelectorItemViewModel>> SItemSelectorTableRow;

	class SItemSelectorItemTableRow : public SItemSelectorTableRow
	{
	public:
		SLATE_BEGIN_ARGS(SItemSelectorItemTableRow)
			: _Style(&FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
			, _ShowSelection(true)
			{}
			SLATE_DEFAULT_SLOT(typename SItemSelectorItemTableRow::FArguments, Content)
			SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
			SLATE_ARGUMENT(bool, ShowSelection)
			SLATE_ARGUMENT(FMargin, Padding)
			SLATE_EVENT(FOnDragDetected, OnDragDetected)
			SLATE_EVENT(FOnItemRowHoverEvent, OnItemRowHoverEventDelegate)
			SLATE_ARGUMENT(TSharedPtr<FItemSelectorItemContainerViewModel>, ItemViewModel)
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTree)
		{
			ItemViewModelWeak = InArgs._ItemViewModel;
			
			typename SItemSelectorTableRow::FArguments Arguments;
			Arguments = Arguments[InArgs._Content.Widget];
			Arguments._Style = InArgs._Style;
			Arguments._ShowSelection = InArgs._ShowSelection;
			Arguments._Padding = InArgs._Padding;
			Arguments._OnDragDetected = InArgs._OnDragDetected;

			OnItemRowHoverEventDelegate = InArgs._OnItemRowHoverEventDelegate;
			SItemSelectorTableRow::Construct(Arguments, OwnerTree);
		}

		virtual void OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& PointerEvent) override
		{
			SItemSelectorTableRow::OnMouseEnter(InGeometry, PointerEvent);

			OnItemRowHoverEventDelegate.ExecuteIfBound(ItemViewModelWeak.Pin()->GetItem(), true);
		}

		virtual void OnMouseLeave(const FPointerEvent& PointerEvent) override
		{
			SItemSelectorTableRow::OnMouseLeave(PointerEvent);

			OnItemRowHoverEventDelegate.ExecuteIfBound(ItemViewModelWeak.Pin()->GetItem(), false);
		}

		TWeakPtr<FItemSelectorItemContainerViewModel> ItemViewModelWeak;
		FOnItemRowHoverEvent OnItemRowHoverEventDelegate;
	};

	class SItemSelectorItemCategoryTableRow : public SItemSelectorTableRow
	{
	public:
		SLATE_BEGIN_ARGS(SItemSelectorItemCategoryTableRow)
			: _Style(&FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
			, _ShowSelection(true)
			, _BorderBackgroundColor(FLinearColor::White)
			{}
			SLATE_DEFAULT_SLOT(typename SItemSelectorItemCategoryTableRow::FArguments, Content)
			SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
			SLATE_ARGUMENT(bool, ShowSelection)
			SLATE_ARGUMENT(FMargin, Padding)
			SLATE_ARGUMENT(FLinearColor, BorderBackgroundColor)
			SLATE_ARGUMENT(FMargin, ChildSlotPadding)
			SLATE_ARGUMENT(FMargin, BorderBackgroundPadding)
			SLATE_ARGUMENT(TSharedPtr<FItemSelectorItemCategoryViewModel>, CategoryViewModel)
			SLATE_EVENT(FOnGetCategoryBackgroundImage, OnGetCategoryBackgroundImage)
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTree)
		{
			CategoryViewModel = InArgs._CategoryViewModel;
			OnGetCategoryBackgroundImage = InArgs._OnGetCategoryBackgroundImage;

			typename SItemSelectorTableRow::FArguments Arguments;
			Arguments = Arguments[InArgs._Content.Widget];
			Arguments._Style = InArgs._Style;
			Arguments._ShowSelection = InArgs._ShowSelection;
			Arguments._Padding = InArgs._Padding;
			SItemSelectorTableRow::Construct(Arguments, OwnerTree);

			// If category background images are used, construct a border to display them.
			if (OnGetCategoryBackgroundImage.IsBound())
			{
				SItemSelectorTableRow::ChildSlot
				.Padding(InArgs._ChildSlotPadding)
				[
					SAssignNew(ContentBorder, SBorder)
					.BorderImage(this, &SItemSelectorItemCategoryTableRow::GetBackgroundImage)
					.Padding(InArgs._BorderBackgroundPadding)
					.BorderBackgroundColor(InArgs._BorderBackgroundColor)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(2.0f, 2.0f, 2.0f, 2.0f)
						.AutoWidth()
						[
							// Overwriting the ChildSlot of the STableRow removes the existing SExpanderArrow, so construct a new one.
							SNew(SExpanderArrow, SItemSelectorItemCategoryTableRow::SharedThis(this))
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						[
							InArgs._Content.Widget
						]
					]
				];
			}
		}

		const FSlateBrush* GetBackgroundImage() const
		{
			if (OnGetCategoryBackgroundImage.IsBound())
			{
				return OnGetCategoryBackgroundImage.Execute(SItemSelectorTableRow::IsHovered(), SItemSelectorTableRow::IsItemExpanded());
			}
			return nullptr;
		}

	private:
		FOnGetCategoryBackgroundImage OnGetCategoryBackgroundImage;
		TSharedPtr<SBorder> ContentBorder;
		TWeakPtr<FItemSelectorItemCategoryViewModel> CategoryViewModel;
	};

	class SItemSelectorItemSectionTableRow : public SItemSelectorTableRow
	{
		SLATE_BEGIN_ARGS(SItemSelectorItemSectionTableRow)
			: _Style(&FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
			, _ShowSelection(false)
			{}
			SLATE_DEFAULT_SLOT(typename SItemSelectorItemSectionTableRow::FArguments, Content)
			SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
			SLATE_ARGUMENT(bool, ShowSelection)
			SLATE_ARGUMENT(FMargin, Padding)
			SLATE_EVENT(FOnDragDetected, OnDragDetected)
			SLATE_EVENT(FOnItemRowHoverEvent, OnItemRowHoverEventDelegate)
			SLATE_ARGUMENT(TSharedPtr<FSectionViewModel>, SectionViewModel)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTree)
		{
			SectionViewModelWeak = InArgs._SectionViewModel;
			
			typename SItemSelectorTableRow::FArguments Arguments;
			Arguments = Arguments[InArgs._Content.Widget];
			Arguments._Style = InArgs._Style;
			Arguments._ShowSelection = InArgs._ShowSelection;
			Arguments._Padding = InArgs._Padding;
			Arguments._OnDragDetected = InArgs._OnDragDetected;
			
			SItemSelectorTableRow::Construct(Arguments, OwnerTree);
		}

		TWeakPtr<FSectionViewModel> SectionViewModelWeak;
	};
public:
	void Construct(const FArguments& InArgs)
	{
		Items = InArgs._Items;
		TArray<TArray<CategoryType>> DefaultCategoryPathsFromDefaultCategories;
		for (const CategoryType& DefaultCategory : InArgs._DefaultCategories)
		{
			DefaultCategoryPaths.AddDefaulted();
			DefaultCategoryPaths.Last().Add(DefaultCategory);
		}
		DefaultCategoryPaths.Append(InArgs._DefaultCategoryPaths);
		ClickActivateMode = InArgs._ClickActivateMode;
		bPreserveExpansionOnRefresh = InArgs._PreserveExpansionOnRefresh;
		bPreserveSelectionOnRefresh = InArgs._PreserveSelectionOnRefresh;
		SectionRowStyle = InArgs._SectionRowStyle;
		CategoryRowStyle = InArgs._CategoryRowStyle;
		ItemRowStyle = InArgs._ItemRowStyle;
		OnGetCategoryBackgroundImage = InArgs._OnGetCategoryBackgroundImage;
		CategoryBorderBackgroundColor = InArgs._CategoryBorderBackgroundColor;
		CategoryChildSlotPadding = InArgs._CategoryChildSlotPadding;
		CategoryBorderBackgroundPadding = InArgs._CategoryBorderBackgroundPadding;
		OnGetCategoriesForItem = InArgs._OnGetCategoriesForItem;
		OnGetSectionsForItem = InArgs._OnGetSectionsForItem;
		OnCompareSectionsForEquality = InArgs._OnCompareSectionsForEquality;
		OnCompareSectionsForSorting = InArgs._OnCompareSectionsForSorting;
		OnCompareCategoriesForEquality = InArgs._OnCompareCategoriesForEquality;
		OnCompareCategoriesForSorting = InArgs._OnCompareCategoriesForSorting;
		OnCompareItemsForEquality = InArgs._OnCompareItemsForEquality;
		OnCompareItemsForSorting = InArgs._OnCompareItemsForSorting;
		OnDoesItemMatchFilterText = InArgs._OnDoesItemMatchFilterText;
		OnGetItemWeight = InArgs._OnGetItemWeight;
		OnGenerateWidgetForSection = InArgs._OnGenerateWidgetForSection;
		OnGenerateWidgetForCategory = InArgs._OnGenerateWidgetForCategory;
		OnGenerateWidgetForItem = InArgs._OnGenerateWidgetForItem;
		OnItemActivated = InArgs._OnItemActivated;
		OnCategoryActivated = InArgs._OnCategoryActivated;
		OnSelectionChanged = InArgs._OnSelectionChanged;
		OnCategoryExpansionChanged = InArgs._OnCategoryExpansionChanged;
		OnDoesItemPassCustomFilter = InArgs._OnDoesItemPassCustomFilter;
		OnDoesSectionPassCustomFilter = InArgs._OnDoesSectionPassCustomFilter;
		OnGetSectionData = InArgs._OnGetSectionData;
		HideSingleSection = InArgs._HideSingleSection;
		OnItemSelected = InArgs._OnItemSelected;
		OnItemsDragged = InArgs._OnItemsDragged;
		OnGetKeyForItem = InArgs._OnGetKeyForItem;
		OnGetKeyForCategory = InArgs._OnGetKeyForCategory;
		OnGetKeyForSection = InArgs._OnGetKeyForSection;
		SearchBoxAdjacentContentWidget = InArgs._SearchBoxAdjacentContent.Widget;
		OnItemRowHoverEventDelegate = InArgs._OnItemRowHoverEvent;

		OnSuggestionUpdated = FOnSuggestionUpdated::CreateSP(this, &SItemSelector::OnSuggestionChanged);

		// Validate bindings for options set.
		checkf(DefaultCategoryPaths.Num() == 0 || OnCompareCategoriesForEquality.IsBound(), TEXT("OnCompareCategoriesForEquality must be bound if default categories are supplied."));
		checkf(DefaultCategoryPaths.Num() == 0 || OnGenerateWidgetForCategory.IsBound(), TEXT("OnGenerateWidgetForCategory must be bound if default categories are supplied."));
		checkf(OnGetCategoriesForItem.IsBound() == false || OnCompareCategoriesForEquality.IsBound(), TEXT("OnCompareCategoriesForEquality must be bound if OnGenerateCategoriesForItem is bound."));
		checkf(OnGetCategoriesForItem.IsBound() == false || OnGenerateWidgetForCategory.IsBound(), TEXT("OnGenerateWidgetForCategory must be bound if OnGenerateCategoriesForItem is bound."));
		checkf(OnGetSectionsForItem.IsBound() == false || OnCompareSectionsForEquality.IsBound(), TEXT("OnCompareSectionsForEquality must be bound if OnGetSectionsForItems is bound."))
		checkf(OnGetSectionData.IsBound() == false || OnGetSectionsForItem.IsBound(), TEXT("OnGetSectionsForItem must be bound if OnGetListSections is bound."))
		checkf(OnGenerateWidgetForItem.IsBound(), TEXT("OnGenerateWidgetForItem must be bound"));

		if (bPreserveExpansionOnRefresh || bPreserveSelectionOnRefresh)
		{
			if (bPreserveSelectionOnRefresh)
			{
				checkf(OnGetKeyForItem.IsBound(), TEXT("OnGetKeyForItem must be bound if PreserveSelectionOnRefresh is set."));
			}
			if (DefaultCategoryPaths.Num() != 0 || OnGetCategoriesForItem.IsBound())
			{
				checkf(OnGetKeyForCategory.IsBound(), TEXT("OnGetKeyForCategory must be bound if: default categories are supplied or OnGetCategoriesForItem is bound, and PreserveExpansionOnRefresh or PreserveSelectionOnRefresh are set."));
			}
			if (OnGetSectionsForItem.IsBound())
			{
				checkf(OnGetKeyForSection.IsBound(), TEXT("OnGetKeyForSection must be bound if OnGetSectionsForItem is bound and PreserveExpansionOnRefresh or PreserveSelectionOnRefresh are set."));
			}
		}

		ViewModelUtilities = MakeShared<FItemSelectorViewModel>(
            Items,
            DefaultCategoryPaths, 
            OnGetCategoriesForItem, OnGetSectionsForItem,
            OnCompareSectionsForEquality, OnCompareSectionsForSorting, 
            OnCompareCategoriesForEquality, OnCompareCategoriesForSorting, 
            OnCompareItemsForEquality, OnCompareItemsForSorting,
            OnDoesItemMatchFilterText, OnGetItemWeight, 
            OnDoesItemPassCustomFilter, OnDoesSectionPassCustomFilter,
            OnGetSectionData, HideSingleSection,
            bPreserveExpansionOnRefresh, bPreserveSelectionOnRefresh,
			OnGetKeyForItem, OnGetKeyForCategory,	OnGetKeyForSection,
			OnSuggestionUpdated);

		// Search Box
		SAssignNew(SearchBox, SSearchBox)
		.OnTextChanged(this, &SItemSelector::OnSearchTextChanged)
		.DelayChangeNotificationsWhileTyping(false);

		ChildSlot
		[	
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 5)
			[
				// Wrapped Search Box
				SNew(SHorizontalBox)
				.Visibility(this, &SItemSelector::GetSearchBoxVisibility)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SearchBox.ToSharedRef()
				]
				// Search Box Adjacent Content
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4, 0, 0, 0)
				[
					SearchBoxAdjacentContentWidget.ToSharedRef()
				]
			]
			+ SVerticalBox::Slot()
			.Padding(0)
			[
				SAssignNew(ItemTree, STreeView<TSharedRef<FItemSelectorItemViewModel>>)
				.SelectionMode(InArgs._AllowMultiselect ? ESelectionMode::Multi : ESelectionMode::SingleToggle)
				.OnGenerateRow(this, &SItemSelector::OnGenerateRow)
				.OnGetChildren(this, &SItemSelector::OnGetChildren)
				.OnMouseButtonClick(this, &SItemSelector::OnMouseClick)
				.OnMouseButtonDoubleClick(this, &SItemSelector::OnMouseDoubleClick)
				.OnContextMenuOpening(InArgs._OnContextMenuOpening)
				.OnSelectionChanged(this, &SItemSelector::OnTreeSelectionChanged)
				.OnExpansionChanged(this, &SItemSelector::OnTreeExpansionChanged)
				.TreeItemsSource(ViewModelUtilities->GetRootItems())
				.ClearSelectionOnClick(InArgs._ClearSelectionOnClick)
			]
		];
		
		if(InArgs._ExpandInitially)
		{
			ExpandTree();
		}
		else if (InArgs._OnGetCategoryExpandedInitially.IsBound())
		{
			ExpandTreeByFilter(InArgs._OnGetCategoryExpandedInitially);
		}

		ExpandSections();

		SearchBox->SetOnKeyDownHandler(FOnKeyDown::CreateSP(this, &SItemSelector::OnKeyDown));
	}

	TArray<ItemType> GetSelectedItems()
	{
		TArray<TSharedRef<FItemSelectorItemViewModel>> SelectedItemViewModels;
		ItemTree->GetSelectedItems(SelectedItemViewModels);

		TArray<ItemType> SelectedItems;
		for (TSharedRef<FItemSelectorItemViewModel> SelectedItemViewModel : SelectedItemViewModels)
		{
			if (SelectedItemViewModel->GetType() == EItemSelectorItemViewModelType::Item)
			{
				TSharedRef<FItemSelectorItemContainerViewModel> SelectedItemContainerViewModel = StaticCastSharedRef<FItemSelectorItemContainerViewModel>(SelectedItemViewModel);
				SelectedItems.Add(SelectedItemContainerViewModel->GetItem());
			}
		}
		return SelectedItems;
	}

	TArray<CategoryType> GetSelectedCategories()
	{
		TArray<TSharedRef<FItemSelectorItemViewModel>> SelectedItemViewModels;
		ItemTree->GetSelectedItems(SelectedItemViewModels);

		TArray<CategoryType> SelectedCategories;
		for (TSharedRef<FItemSelectorItemViewModel> SelectedItemViewModel : SelectedItemViewModels)
		{
			if (SelectedItemViewModel->GetType() == EItemSelectorItemViewModelType::Category)
			{
				TSharedRef<FItemSelectorItemCategoryViewModel> SelectedItemCategoryViewModel = StaticCastSharedRef<FItemSelectorItemCategoryViewModel>(SelectedItemViewModel);
				SelectedCategories.Add(SelectedItemCategoryViewModel->GetCategory());
			}
		}
		return SelectedCategories;
	}

	void SetSelectedItems(const TArray<ItemType>& NewSelectedItems, bool bExpandToShow = false)
	{
		checkf(OnCompareItemsForEquality.IsBound(), TEXT("OnCompareItemsForEquality event must be handled to use the SetSelectedItems function."));
		if (NewSelectedItems.Num() == 0)
		{
			ItemTree->ClearSelection();
		}
		else
		{
			TGuardValue<bool> SelectionGuard(bIsSettingSelection, true);

			TArray<TSharedRef<FItemSelectorItemViewModel>> SelectedItemViewModels;
			ViewModelUtilities->GetItemViewModelsForItems(NewSelectedItems, SelectedItemViewModels);
			for (TSharedRef<FItemSelectorItemViewModel> SelectedItemViewModel : SelectedItemViewModels)
			{
				ItemTree->SetSelection(SelectedItemViewModel);
				if(SelectedItemViewModel->GetType() == EItemSelectorItemViewModelType::Item)
				{
					TSharedRef<FItemSelectorItemContainerViewModel> ItemViewModel = StaticCastSharedRef<FItemSelectorItemContainerViewModel>(SelectedItemViewModel);
					OnItemSelected.ExecuteIfBound(ItemViewModel->GetItem(), ESelectInfo::Direct);
				}
				
				if (bExpandToShow)
				{
					TSharedPtr<FRootViewModel> RootViewModel = ViewModelUtilities->GetRootViewModel();
					if (RootViewModel.IsValid())
					{
						bool bFound = false;
						TSharedRef < FItemSelectorItemViewModel> AsBase = RootViewModel.ToSharedRef();
						FindAndExpandModelAndParents(bFound, AsBase, SelectedItemViewModel);
					}
				}
			}
			OnSelectionChanged.ExecuteIfBound();
		}
	}

	void RequestScrollIntoView(const ItemType& Item)
	{
		TArray<TSharedRef<FItemSelectorItemViewModel>> ItemViewModels;
		ViewModelUtilities->GetItemViewModelsForItems({Item}, ItemViewModels);
		if(ItemViewModels.Num() == 1)
		{
			ItemTree->RequestScrollIntoView(ItemViewModels[0]);
		}
	}

	void ClearSelectedItems()
	{
		ItemTree->ClearSelection();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		// @Todo Currently keys are hardcoded, change that in the future
		
		// Escape dismisses all menus.
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			FSlateApplication::Get().DismissAllMenus();
			return FReply::Handled();
		}
		else if (InKeyEvent.GetKey() == EKeys::Enter && (OnItemActivated.IsBound() || OnCategoryActivated.IsBound()) )
		{
			TArray<TSharedRef<FItemSelectorItemViewModel>> SelectedItemViewModels;
			ItemTree->GetSelectedItems(SelectedItemViewModels);
			if (SelectedItemViewModels.Num() == 1)
			{
				if(SelectedItemViewModels[0]->GetType() == EItemSelectorItemViewModelType::Item)
				{
					TSharedRef<FItemSelectorItemContainerViewModel> ItemContainer = StaticCastSharedRef<FItemSelectorItemContainerViewModel>(SelectedItemViewModels[0]);
					OnItemActivated.Execute(ItemContainer->GetItem());
					return FReply::Handled();
				}
				else if(SelectedItemViewModels[0]->GetType() == EItemSelectorItemViewModelType::Category)
				{
					TSharedRef<FItemSelectorItemCategoryViewModel> CategoryViewModel = StaticCastSharedRef<FItemSelectorItemCategoryViewModel>(SelectedItemViewModels[0]);
					OnCategoryActivated.Execute(CategoryViewModel->GetCategory());
					return FReply::Handled();
				}
			}
		}
		else if (!SearchBox->GetText().IsEmpty())
		{
			// Needs to be done here in order not to eat up the text navigation key events when list isn't populated
			if (Items.Num() == 0)
			{
				return FReply::Unhandled();
			}
		
			int32 SelectedSuggestion = ViewModelUtilities->GetCurrentSuggestionIndex();
			
			if (InKeyEvent.GetKey() == EKeys::Up)
			{
				ViewModelUtilities->UpdateCurrentSuggestionIndex(FMath::Max(0, SelectedSuggestion - 1));
			}
			else if (InKeyEvent.GetKey() == EKeys::Down)
			{
				ViewModelUtilities->UpdateCurrentSuggestionIndex(FMath::Min(ViewModelUtilities->NumFlattenedItems() - 1, SelectedSuggestion + 1));
			}
			else if (InKeyEvent.GetKey() == EKeys::PageUp)
			{
				const int32 NumItemsInAPage = 15; // arbitrary jump because we can't get at the visible item count from here
				ViewModelUtilities->UpdateCurrentSuggestionIndex(FMath::Max(0, SelectedSuggestion - NumItemsInAPage));
			}
			else if (InKeyEvent.GetKey() == EKeys::PageDown)
			{
				const int32 NumItemsInAPage = 15; // arbitrary jump because we can't get at the visible item count from here
				ViewModelUtilities->UpdateCurrentSuggestionIndex(FMath::Min(ViewModelUtilities->NumFlattenedItems() - 1, SelectedSuggestion + NumItemsInAPage));
			}
			else if (InKeyEvent.GetKey() == EKeys::Home && InKeyEvent.IsControlDown())
			{
				ViewModelUtilities->UpdateCurrentSuggestionIndex(0);
			}
			else if (InKeyEvent.GetKey() == EKeys::End && InKeyEvent.IsControlDown())
			{
				ViewModelUtilities->UpdateCurrentSuggestionIndex(ViewModelUtilities->NumFlattenedItems() - 1);
			}
			else
			{
				return FReply::Unhandled();
			}

			OnSuggestionChanged(ViewModelUtilities->GetCurrentSuggestionIndex());
			return FReply::Handled();
		}
		else
		{
			// When all else fails, it means we haven't filtered the list and we want to handle it as if we were just scrolling through a normal tree view
			return ItemTree->OnKeyDown(FindChildGeometry(MyGeometry, ItemTree.ToSharedRef()), InKeyEvent);
		}

		return FReply::Unhandled();
	}

	void RefreshItemsAndDefaultCategories(const TArray<ItemType>& InItems, const TArray<TArray<CategoryType>>& InDefaultCategoryPaths)
	{
		TSet<CategoryKeyType> ExpandedCategoryKeys;
		TSet<SectionKeyType> ExpandedSectionKeys;
		if (bPreserveExpansionOnRefresh)
		{
			GetExpandedKeys(ExpandedCategoryKeys, ExpandedSectionKeys);
		}

		TSet<ItemKeyType> SelectedItemKeys;
		TSet<CategoryKeyType> SelectedCategoryKeys;
		TSet<SectionKeyType> SelectedSectionKeys;
		if (bPreserveSelectionOnRefresh)
		{
			GetSelectedKeys(SelectedItemKeys, SelectedCategoryKeys, SelectedSectionKeys);
		}

		ViewModelUtilities->Refresh(InItems, InDefaultCategoryPaths);

		if (bPreserveExpansionOnRefresh)
		{
			RestoreExpansionState(ExpandedCategoryKeys, ExpandedSectionKeys);
		}
		else
		{
			ExpandTree();
		}

		if (bPreserveSelectionOnRefresh)
		{
			RestoreSelectionState(SelectedItemKeys, SelectedCategoryKeys, SelectedSectionKeys);
		}

		ItemTree->RequestTreeRefresh();
	}

	void RefreshItemsAndDefaultCategories(const TArray<ItemType>& InItems, const TArray<CategoryType> &InDefaultCategories)
	{
		TArray<TArray<CategoryType>> DefaultCategoryPathsFromDefaultCategories;
		for (const CategoryType& DefaultCategory : InDefaultCategories)
		{
			DefaultCategoryPathsFromDefaultCategories.AddDefaulted();
			DefaultCategoryPathsFromDefaultCategories.Last().Add(DefaultCategory);
		}
		RefreshItemsAndDefaultCategories(InItems, DefaultCategoryPathsFromDefaultCategories);
	}

	void RefreshItems(const TArray<ItemType>& InItems)
	{
		TArray<TArray<CategoryType>> UnusedDefaultCategoryPaths;
		RefreshItemsAndDefaultCategories(InItems, UnusedDefaultCategoryPaths);
	}

	void RefreshAllCurrentItems(bool bForceExpansion = false)
	{
		RefreshItemsAndDefaultCategories(Items, DefaultCategoryPaths);
		if(bForceExpansion)
		{
			ExpandTree();
		}
	}

	bool IsSearching() const
	{
		return ViewModelUtilities->IsSearching();
	}
	
	bool IsFiltering() const
	{
		return ViewModelUtilities->IsFiltering();
	}
	
	const FText& GetFilterText() const
	{
		return ViewModelUtilities->GetFilterText();
	}

	FText GetFilterTextNoRef() const
	{
		return ViewModelUtilities->GetFilterText();
	}

	TSharedRef<SWidget> GetSearchBox() const
	{
		return SearchBox.ToSharedRef();
	}

	void ExpandTree()
	{
		TArray<TSharedRef<FItemSelectorItemViewModel>> ItemsToProcess;
		ItemsToProcess.Append(*ViewModelUtilities->GetRootItems());
		while (ItemsToProcess.Num() > 0)
		{
			TSharedRef<FItemSelectorItemViewModel> ItemToProcess = ItemsToProcess[0];
			ItemsToProcess.RemoveAtSwap(0);
			ItemTree->SetItemExpansion(ItemToProcess, true);
			ItemToProcess->GetChildren(ItemsToProcess);
		}
	}

	void ExpandTreeByFilter(const FOnCategoryPassesFilter& InFilter)
	{
		TGuardValue<bool> SettingExpansionGuard(bIsSettingExpansion, true);

		TArray<TSharedRef<FItemSelectorItemViewModel>> ItemsToProcess;
		ItemsToProcess.Append(*ViewModelUtilities->GetRootItems());
		while (ItemsToProcess.Num() > 0)
		{
			TSharedRef<FItemSelectorItemViewModel> ItemToProcess = ItemsToProcess[0];
			ItemsToProcess.RemoveAtSwap(0);

			TOptional<bool> bForce;
			ExpandTreeByFilterRecursive(ItemToProcess, InFilter, bForce);
		}
	}

	bool IsItemSelected(const ItemType Item)
	{
		const TArray<ItemType> ItemArr = {Item};
		TArray<TSharedRef<FItemSelectorItemViewModel>> ItemViewModelArr;
		ViewModelUtilities->GetItemViewModelsForItems(ItemArr, ItemViewModelArr);
		if(ItemViewModelArr.Num() > 0)
		{
			return ItemTree->IsItemSelected(ItemViewModelArr[0]);
		}
		return false;
	}

	bool IsPendingRefresh() const
	{
		return ItemTree->IsPendingRefresh();
	}

private:
	void ExpandTreeByFilterRecursive(TSharedRef<FItemSelectorItemViewModel>& ItemToProcess, const FOnCategoryPassesFilter& InFilter, TOptional<bool> bForce)
	{
		if (!bForce.IsSet() && ItemToProcess->GetType() == EItemSelectorItemViewModelType::Category)
		{
			bool bExpand = InFilter.Execute(StaticCastSharedRef<FItemSelectorItemCategoryViewModel>(ItemToProcess)->GetCategory());
			ItemTree->SetItemExpansion(ItemToProcess, bExpand);
			bForce = bExpand;
		}
		else if (bForce.IsSet())
		{
			ItemTree->SetItemExpansion(ItemToProcess, bForce.GetValue());
		}

		TArray<TSharedRef<FItemSelectorItemViewModel>> ItemsToProcess;
		ItemToProcess->GetChildren(ItemsToProcess);

		for (TSharedRef<FItemSelectorItemViewModel>& Child : ItemsToProcess)
		{
			ExpandTreeByFilterRecursive(Child, InFilter, bForce);
		}
	}

	EVisibility GetSearchBoxVisibility() const
	{
		return OnDoesItemMatchFilterText.IsBound() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	void FindAndExpandModelAndParents(bool& bFound, const TSharedRef<FItemSelectorItemViewModel>& Current, const TSharedRef<FItemSelectorItemViewModel>& Target)
	{
		if (!bFound && Current == Target)
		{
			bFound = true;
		}

		if (!bFound)
		{
			TArray<TSharedRef<FItemSelectorItemViewModel>> Children;

			Current->GetChildren(Children);
			for (TSharedRef<FItemSelectorItemViewModel> Child : Children)
			{
				FindAndExpandModelAndParents(bFound, Child, Target);
				if (bFound)
				{
					break;
				}
			}
		}

		if (bFound)
		{
			ItemTree->SetItemExpansion(Current, true);
		}
	};

	void OnSearchTextChanged(const FText& SearchText)
	{
		if (ViewModelUtilities->GetFilterText().CompareTo(SearchText) != 0)
		{
			if(!SearchText.IsEmpty())
			{
				ViewModelUtilities->SetFilterText(SearchText, ItemTree);
				ItemTree->RequestTreeRefresh();
				
				ExpandTree();
			}
			else
			{
				// causes a refresh of items and therefore expansion as well
				// @todo cache expansion state. Not that easy since item data doesn't live across multiple refreshes.
				// Even caching off the items doesn't help since their category information gets invalidated as internal cache gets cleared
				ViewModelUtilities->SetFilterText(SearchText, ItemTree);
				ItemTree->RequestTreeRefresh();

				ItemTree->ClearExpandedItems();
				ExpandSections();			
			}
		}
	}

	TSharedRef<ITableRow> OnGenerateRow(TSharedRef<FItemSelectorItemViewModel> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		FMargin Margin(0.f, 2.f);
		
		switch (Item->GetType())
		{
		case EItemSelectorItemViewModelType::Category:
		{
			TSharedRef<FItemSelectorItemCategoryViewModel> ItemCategoryViewModel = StaticCastSharedRef<FItemSelectorItemCategoryViewModel>(Item);
			return SNew(SItemSelectorItemCategoryTableRow, OwnerTable)
				.Style(CategoryRowStyle)
				.ShowSelection(true)
				.Padding(Margin)
				.BorderBackgroundColor(CategoryBorderBackgroundColor)
				.ChildSlotPadding(CategoryChildSlotPadding)
				.BorderBackgroundPadding(CategoryBorderBackgroundPadding)
				.OnGetCategoryBackgroundImage(OnGetCategoryBackgroundImage)
				.CategoryViewModel(ItemCategoryViewModel)
				[
					OnGenerateWidgetForCategory.Execute(ItemCategoryViewModel->GetCategory())
				];
		}
		case EItemSelectorItemViewModelType::Item:
		{
			TSharedRef<FItemSelectorItemContainerViewModel> ItemViewModel = StaticCastSharedRef<FItemSelectorItemContainerViewModel>(Item);
				return SNew(SItemSelectorItemTableRow, OwnerTable)
				.Style(ItemRowStyle)
				.ShowSelection(true)
				.Padding(Margin)
				.OnDragDetected(this, &SItemSelector::OnItemDragDetected)
				.OnItemRowHoverEventDelegate(OnItemRowHoverEventDelegate)
				.ItemViewModel(ItemViewModel)
				[
					OnGenerateWidgetForItem.Execute(ItemViewModel->GetItem())
				];
		}
		case EItemSelectorItemViewModelType::Section:
		{
			TSharedRef<FSectionViewModel> SectionViewModel = StaticCastSharedRef<FSectionViewModel>(Item);
			return SNew(SItemSelectorItemSectionTableRow, OwnerTable)
				.Style(SectionRowStyle)
				.ShowSelection(false)
				.Padding(Margin)
				.SectionViewModel(SectionViewModel)
				[
					OnGenerateWidgetForSection.Execute(SectionViewModel->GetSection())
				];
		}
		default:
			checkf(false, TEXT("Unsupported type"));
			return SNew(STableRow<TSharedRef<FItemSelectorItemViewModel>>, OwnerTable);
		}
	}

	void OnGetChildren(TSharedRef<FItemSelectorItemViewModel> Item, TArray<TSharedRef<FItemSelectorItemViewModel>>& OutChildren)
	{
		Item->GetChildren(OutChildren);
	}

	void OnMouseClick(TSharedRef<FItemSelectorItemViewModel> ItemClicked)
	{
		if (ClickActivateMode == EItemSelectorClickActivateMode::SingleClick && OnItemActivated.IsBound() && ItemClicked->GetType() == EItemSelectorItemViewModelType::Item)
		{
			TSharedRef<FItemSelectorItemContainerViewModel> ItemContainer = StaticCastSharedRef<FItemSelectorItemContainerViewModel>(ItemClicked);
			OnItemActivated.Execute(ItemContainer->GetItem());
		}
	}

	void OnMouseDoubleClick(TSharedRef<FItemSelectorItemViewModel> ItemDoubleClicked)
	{
		if (ItemDoubleClicked->GetType() == EItemSelectorItemViewModelType::Item)
		{
			if (ClickActivateMode == EItemSelectorClickActivateMode::DoubleClick && OnItemActivated.IsBound())
			{
				TSharedRef<FItemSelectorItemContainerViewModel> ItemContainer = StaticCastSharedRef<FItemSelectorItemContainerViewModel>(ItemDoubleClicked);
				OnItemActivated.Execute(ItemContainer->GetItem());
			}
		}
		else if (ItemDoubleClicked->GetType() == EItemSelectorItemViewModelType::Category || ItemDoubleClicked->GetType() == EItemSelectorItemViewModelType::Section)
		{
			TArray<TSharedRef<FItemSelectorItemViewModel>> Children;
			ItemDoubleClicked->GetChildren(Children);

			if(Children.Num() > 0)
			{
				ItemTree->SetItemExpansion(ItemDoubleClicked, !ItemTree->IsItemExpanded(ItemDoubleClicked));
			}
		}
	}

	FReply OnItemDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (OnItemsDragged.IsBound() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			return OnItemsDragged.Execute(GetSelectedItems(), MouseEvent);
		}
		return FReply::Unhandled();
	}

	void OnTreeSelectionChanged(TSharedPtr<FItemSelectorItemViewModel> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		if (bIsSettingSelection == false)
		{
			OnSelectionChanged.ExecuteIfBound();
			if(OnItemSelected.IsBound() && SelectedItem.IsValid() && SelectedItem->GetType() == EItemSelectorItemViewModelType::Item)
			{
				TSharedPtr<FItemSelectorItemContainerViewModel> SelectedItemContainer = StaticCastSharedPtr<FItemSelectorItemContainerViewModel>(SelectedItem);
				OnItemSelected.Execute(SelectedItemContainer->GetItem(), SelectInfo);
			}
		}
	}

	void OnTreeExpansionChanged(TSharedRef<FItemSelectorItemViewModel> ChangedItem, bool bIsExpanded)
	{
		if (bIsSettingExpansion == false)
		{
			if (OnCategoryExpansionChanged.IsBound() && ChangedItem->GetType() == EItemSelectorItemViewModelType::Category)
			{
				TSharedRef<FItemSelectorItemCategoryViewModel> ChangedCategory = StaticCastSharedRef<FItemSelectorItemCategoryViewModel>(ChangedItem);
				OnCategoryExpansionChanged.Execute(ChangedCategory->GetCategory(), bIsExpanded);
			}
		}
	}
	
	void OnSuggestionChanged(int32 SuggestionIndex)
	{
		if(SuggestionIndex != INDEX_NONE && SuggestionIndex < ViewModelUtilities->NumFlattenedItems())
		{
			ItemTree->SetSelection(ViewModelUtilities->GetFlattenedItems()[SuggestionIndex], ESelectInfo::OnNavigation);
			ItemTree->RequestScrollIntoView(ViewModelUtilities->GetFlattenedItems()[SuggestionIndex]);
		}
		else
		{
			ItemTree->ClearSelection();
		}
	}
	
	void ExpandSections()
	{
		const TArray<TSharedRef<FSectionViewModel>>& ItemsToProcess = ViewModelUtilities->GetSections();
		for(const auto& Item : ItemsToProcess)
		{
			ItemTree->SetItemExpansion(Item, true);
		}
	}

	void GetExpandedKeys(TSet<CategoryKeyType>& OutExpandedCategoryKeys, TSet<SectionKeyType>& OutExpandedSectionKeys)
	{
		TSet<TSharedRef<FItemSelectorItemViewModel>> ExpandedItemViewModels;
		ItemTree->GetExpandedItems(ExpandedItemViewModels);
		for (const TSharedRef<FItemSelectorItemViewModel>& ExpandedItemViewModel : ExpandedItemViewModels)
		{
			if (ExpandedItemViewModel->GetType() == EItemSelectorItemViewModelType::Category)
			{
				OutExpandedCategoryKeys.Add(OnGetKeyForCategory.Execute(StaticCastSharedRef<FItemSelectorItemCategoryViewModel>(ExpandedItemViewModel)->GetCategory()));
			}
			else if (ExpandedItemViewModel->GetType() == EItemSelectorItemViewModelType::Section)
			{
				OutExpandedSectionKeys.Add(OnGetKeyForSection.Execute(StaticCastSharedRef<FSectionViewModel>(ExpandedItemViewModel)->GetSection()));
			}
		}
	}

	void GetSelectedKeys(TSet<ItemKeyType>& OutSelectedItemKeys, TSet<CategoryKeyType>& OutSelectedCategoryKeys, TSet<SectionKeyType>& OutSelectedSectionKeys)
	{
		TArray<TSharedRef<FItemSelectorItemViewModel>> SelectedItemViewModels;
		ItemTree->GetSelectedItems(SelectedItemViewModels);
		for (const TSharedRef<FItemSelectorItemViewModel>& SelectedItemViewModel : SelectedItemViewModels)
		{
			if (SelectedItemViewModel->GetType() == EItemSelectorItemViewModelType::Item)
			{
				OutSelectedItemKeys.Add(StaticCastSharedRef<FItemSelectorItemContainerViewModel>(SelectedItemViewModel)->GetItemKey());
			}
			else if (SelectedItemViewModel->GetType() == EItemSelectorItemViewModelType::Category)
			{
				OutSelectedCategoryKeys.Add(OnGetKeyForCategory.Execute(StaticCastSharedRef<FItemSelectorItemCategoryViewModel>(SelectedItemViewModel)->GetCategory()));
			}
			else if (SelectedItemViewModel->GetType() == EItemSelectorItemViewModelType::Section)
			{
				OutSelectedSectionKeys.Add(OnGetKeyForSection.Execute(StaticCastSharedRef<FSectionViewModel>(SelectedItemViewModel)->GetSection()));
			}
		}
	}

	void RestoreExpansionState(const TSet<CategoryKeyType>& ExpandedCategoryKeys, const TSet<SectionKeyType>& ExpandedSectionKeys)
	{
		if (ExpandedCategoryKeys.Num() == 0 && ExpandedSectionKeys.Num() == 0)
		{
			return;
		}	

		TGuardValue<bool> SettingExpansionGuard(bIsSettingExpansion, true);
		TArray<TSharedRef<FItemSelectorItemViewModel>> Children;
		ViewModelUtilities->GetChildrenRecursive(Children);

		for (const TSharedRef<FItemSelectorItemViewModel>& ChildItemViewModel : Children)
		{
			if (ChildItemViewModel->GetType() == EItemSelectorItemViewModelType::Category)
			{
				if (ExpandedCategoryKeys.Contains(OnGetKeyForCategory.Execute(StaticCastSharedRef<FItemSelectorItemCategoryViewModel>(ChildItemViewModel)->GetCategory())))
				{
					ItemTree->SetItemExpansion(ChildItemViewModel, true);
				}
			}
			else if (ChildItemViewModel->GetType() == EItemSelectorItemViewModelType::Section)
			{
				if (ExpandedSectionKeys.Contains(OnGetKeyForSection.Execute(StaticCastSharedRef<FSectionViewModel>(ChildItemViewModel)->GetSection())))
				{
					ItemTree->SetItemExpansion(ChildItemViewModel, true);
				}
			}
		}
	}

	void RestoreSelectionState(const TSet<ItemKeyType>& SelectedItemKeys, const TSet<CategoryKeyType>& SelectedCategoryKeys, const TSet<SectionKeyType>& SelectedSectionKeys)
	{
		if (SelectedItemKeys.Num() == 0 && SelectedCategoryKeys.Num() == 0 && SelectedSectionKeys.Num() == 0)
		{
			return;
		}

		TGuardValue<bool> SettingSelectionGuard(bIsSettingSelection, true);
		TArray<TSharedRef<FItemSelectorItemViewModel>> Children;
		ViewModelUtilities->GetChildrenRecursive(Children);

		ItemTree->ClearSelection();
		for (const TSharedRef<FItemSelectorItemViewModel>& ChildItemViewModel : Children)
		{
			if (ChildItemViewModel->GetType() == EItemSelectorItemViewModelType::Item)
			{
				if (SelectedItemKeys.Contains(OnGetKeyForItem.Execute(StaticCastSharedRef<FItemSelectorItemContainerViewModel>(ChildItemViewModel)->GetItem())))
				{
					ItemTree->SetItemSelection(ChildItemViewModel, true);
				}
			}
			else if (ChildItemViewModel->GetType() == EItemSelectorItemViewModelType::Category)
			{
				if (SelectedCategoryKeys.Contains(OnGetKeyForCategory.Execute(StaticCastSharedRef<FItemSelectorItemCategoryViewModel>(ChildItemViewModel)->GetCategory())))
				{
					ItemTree->SetItemSelection(ChildItemViewModel, true);
				}
			}
			else if (ChildItemViewModel->GetType() == EItemSelectorItemViewModelType::Section)
			{
				if (SelectedSectionKeys.Contains(OnGetKeyForSection.Execute(StaticCastSharedRef<FSectionViewModel>(ChildItemViewModel)->GetSection())))
				{
					ItemTree->SetItemSelection(ChildItemViewModel, true);
				}
			}
		}
	}

private:
	TArray<ItemType> Items;
	TArray<TArray<CategoryType>> DefaultCategoryPaths;
	
	EItemSelectorClickActivateMode ClickActivateMode;

	bool bPreserveExpansionOnRefresh;
	bool bPreserveSelectionOnRefresh;

	const FTableRowStyle* SectionRowStyle;
	const FTableRowStyle* CategoryRowStyle;
	const FTableRowStyle* ItemRowStyle;
	FOnGetCategoryBackgroundImage OnGetCategoryBackgroundImage;
	FLinearColor CategoryBorderBackgroundColor;
	FMargin CategoryChildSlotPadding;
	FMargin CategoryBorderBackgroundPadding;

	FOnGetCategoriesForItem OnGetCategoriesForItem;
	FOnGetSectionsForItem OnGetSectionsForItem;
	FOnCompareCategoriesForEquality OnCompareCategoriesForEquality;
	FOnCompareCategoriesForSorting OnCompareCategoriesForSorting;
	FOnCompareSectionsForEquality OnCompareSectionsForEquality;
	FOnCompareSectionsForSorting OnCompareSectionsForSorting;
	FOnCompareItemsForEquality OnCompareItemsForEquality;
	FOnCompareItemsForSorting OnCompareItemsForSorting;
	FOnDoesItemMatchFilterText OnDoesItemMatchFilterText;
	FOnGetItemWeight OnGetItemWeight;
	FOnGenerateWidgetForSection OnGenerateWidgetForSection;
	FOnGenerateWidgetForCategory OnGenerateWidgetForCategory;
	FOnGenerateWidgetForItem OnGenerateWidgetForItem;
	FOnItemActivated OnItemActivated;
	FOnCategoryActivated OnCategoryActivated;
	FOnSelectionChanged OnSelectionChanged;
	FOnDoesItemPassCustomFilter OnDoesItemPassCustomFilter;
	FOnDoesSectionPassCustomFilter OnDoesSectionPassCustomFilter;
	FOnCategoryExpansionChanged OnCategoryExpansionChanged;
	FOnGetSectionData OnGetSectionData;
	TAttribute<bool> HideSingleSection;
	FOnItemSelected OnItemSelected;
	FOnItemsDragged OnItemsDragged;
	FOnGetKeyForItem OnGetKeyForItem;
	FOnGetKeyForCategory OnGetKeyForCategory;
	FOnGetKeyForSection OnGetKeyForSection;
	FOnItemRowHoverEvent OnItemRowHoverEventDelegate;
	
	TSharedPtr<FItemSelectorViewModel> ViewModelUtilities;
	TSharedPtr<STreeView<TSharedRef<FItemSelectorItemViewModel>>> ItemTree;
	FOnSuggestionUpdated OnSuggestionUpdated;

	bool bIsSettingSelection = false;
	bool bIsSettingExpansion = false;

	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SWidget> SearchBoxAdjacentContentWidget;
};
