// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphActionMenu.h"
#include "Widgets/Views/SExpanderArrow.h"

class SNiagaraActionMenuExpander : public SExpanderArrow
{
	SLATE_BEGIN_ARGS(SNiagaraActionMenuExpander) {}
	SLATE_ATTRIBUTE(float, IndentAmount)
		SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const FCustomExpanderData& ActionMenuData)
	{
		OwnerRowPtr = ActionMenuData.TableRow;
		SetIndentAmount(InArgs._IndentAmount);
		if (!ActionMenuData.RowAction.IsValid())
		{
			SExpanderArrow::FArguments SuperArgs;
			SuperArgs._IndentAmount = InArgs._IndentAmount;

			SExpanderArrow::Construct(SuperArgs, ActionMenuData.TableRow);
		}
		else
		{
			ChildSlot
				.Padding(TAttribute<FMargin>(this, &SNiagaraActionMenuExpander::GetCustomIndentPadding))
				[
					SNew(SBox)
				];
		}
	}

private:
	FMargin GetCustomIndentPadding() const
	{
		return SExpanderArrow::GetExpanderPadding();
	}
};
