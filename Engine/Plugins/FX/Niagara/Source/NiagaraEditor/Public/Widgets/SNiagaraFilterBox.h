// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "NiagaraActions.h"
#include "Widgets/SNiagaraLibraryOnlyToggleHeader.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Views/SMultipleOptionTable.h"

/**
 * A single source filter checkbox, inherited from SCheckBox to add shift-click functionality
 */
class SNiagaraSourceFilterCheckBox : public SCheckBox
{
	DECLARE_DELEGATE_TwoParams(FOnSourceStateChanged, EScriptSource, bool);
	DECLARE_DELEGATE_TwoParams(FOnShiftClicked, EScriptSource, bool);
	
	SLATE_BEGIN_ARGS( SNiagaraSourceFilterCheckBox )
	{}
		SLATE_EVENT(FOnSourceStateChanged, OnSourceStateChanged)
		SLATE_EVENT(FOnShiftClicked, OnShiftClicked)
		SLATE_ATTRIBUTE(ECheckBoxState, IsChecked)
	SLATE_END_ARGS()

	NIAGARAEDITOR_API void Construct(const FArguments& Args, EScriptSource Source);

	NIAGARAEDITOR_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
private:	
	/** The script source entry associated with this checkbox. */
	EScriptSource Source;
	
	FOnSourceStateChanged OnSourceStateChanged;
	FOnShiftClicked OnShiftClicked;	
private:
	FSlateColor GetTextColor() const;
	FSlateColor GetScriptSourceColor() const;
	FMargin GetFilterNamePadding() const;
};

/**
 * The source filter box will create a source filter checkbox per enum entry and can be used to filter for different sources of assets, actions etc.
 */
class SNiagaraSourceFilterBox : public SCompoundWidget
{
public:
	typedef TMap<EScriptSource, bool> SourceMap;
	DECLARE_DELEGATE_OneParam(FOnFiltersChanged, const SourceMap&);
	
	SLATE_BEGIN_ARGS( SNiagaraSourceFilterBox )
	{}
		SLATE_EVENT(FOnFiltersChanged, OnFiltersChanged)
	SLATE_END_ARGS()

	NIAGARAEDITOR_API void Construct(const FArguments& Args);

	NIAGARAEDITOR_API bool IsFilterActive(EScriptSource Source) const;

private:
	TMap<EScriptSource, TSharedRef<SNiagaraSourceFilterCheckBox>> SourceButtons;
	static TMap<EScriptSource, bool> SourceState;
	
	void BroadcastFiltersChanged() const
	{
		OnFiltersChanged.Execute(SourceState);
	}

	ECheckBoxState OnIsFilterActive(EScriptSource Source) const;

private:
	FOnFiltersChanged OnFiltersChanged;
};

/**
 * The template tab box offers buttons (working like tabs or radio buttons) for filtering purposes.
 * It is used for emitters and systems to filter between standard assets, templates and behavior examples.
 */
class SNiagaraTemplateTabBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnTabActivated, ENiagaraScriptTemplateSpecification Tab);

	/**
	 * The template tab options will tell the tab box which tabs to offer in the form of buttons.
	 */
	struct FNiagaraTemplateTabOptions
	{
		FNiagaraTemplateTabOptions()
		{}

		void ChangeTabState(ENiagaraScriptTemplateSpecification AssetTab, bool bAvailable = true) { TabData.Add(AssetTab, bAvailable); }
		NIAGARAEDITOR_API bool IsTabAvailable(ENiagaraScriptTemplateSpecification AssetTab) const;

		NIAGARAEDITOR_API int32 GetNumAvailableTabs() const;
		NIAGARAEDITOR_API bool GetOnlyAvailableTab(ENiagaraScriptTemplateSpecification& OutTab) const;
		NIAGARAEDITOR_API bool GetOnlyShowTemplates() const;

		NIAGARAEDITOR_API const TMap<ENiagaraScriptTemplateSpecification, bool>& GetTabData() const;
	
	private:
		TMap<ENiagaraScriptTemplateSpecification, bool> TabData;
	};
	
	SLATE_BEGIN_ARGS(SNiagaraTemplateTabBox)
		: _Class(nullptr)
	{}
		SLATE_EVENT(FOnTabActivated, OnTabActivated)
		SLATE_ARGUMENT(UClass*, Class)
	SLATE_END_ARGS()

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, FNiagaraTemplateTabOptions TabOptions);

	NIAGARAEDITOR_API ENiagaraScriptTemplateSpecification GetActiveTab() const;
	/** If true, will set the parameter to the currently active tab. If false, the template filter wasn't initialized with any tabs. */
	NIAGARAEDITOR_API bool GetActiveTab(ENiagaraScriptTemplateSpecification& OutTemplateSpecification) const;
private:
	FOnTabActivated OnTabActivatedDelegate;
	void OnTabActivated(ENiagaraScriptTemplateSpecification AssetTab);
	void OnTabActivated(ECheckBoxState NewState, ENiagaraScriptTemplateSpecification AssetTab);
	FSlateColor GetBackgroundColor(ENiagaraScriptTemplateSpecification TemplateSpecification) const;
	FSlateColor GetTabForegroundColor(ENiagaraScriptTemplateSpecification TemplateSpecification) const;
	FText DetermineControlLabel(ENiagaraScriptTemplateSpecification TemplateSpecification) const;
	FText DetermineControlTooltip(ENiagaraScriptTemplateSpecification TemplateSpecification) const;
private:
	FNiagaraTemplateTabOptions TabOptions;
	TSharedPtr<SSegmentedControl<ENiagaraScriptTemplateSpecification>> TabContainer;
	
	/** bUseActiveTab is required due to tab options not specifically needing to have any tab state set to true. */
	bool bUseActiveTab = false;
	TOptional<ENiagaraScriptTemplateSpecification> ActiveTab;
	static TOptional<ENiagaraScriptTemplateSpecification> CachedActiveTab;
	
	TWeakObjectPtr<UClass> Class = nullptr;
};

/**
 * The primary access point to use different applicable filters.
 * Can be configured to add a source, library and template filter and offers callbacks via delegates to execute logic when the filter states change.
 */
class SNiagaraFilterBox : public SCompoundWidget
{
public:
	/**
	 * The filter options used to initialized the filter box.
	 * There needs to be at least two tab options set to true for the tab filter to show up, if added.
	 */
	struct FFilterOptions
	{
		FFilterOptions() : bAddSourceFilter(true), bAddLibraryFilter(true), bAddTemplateFilter(false)
		{}

		NIAGARAEDITOR_API bool IsAnyFilterActive();

		NIAGARAEDITOR_API void SetAddSourceFilter(bool bAddSourceFilter);
		NIAGARAEDITOR_API void SetAddLibraryFilter(bool bAddLibraryFilter);
		NIAGARAEDITOR_API void SetAddTemplateFilter(bool bAddTemplateFilter);
		NIAGARAEDITOR_API void SetTabOptions(const SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions& TabOptions);
		
		NIAGARAEDITOR_API bool GetAddSourceFilter() const;
		NIAGARAEDITOR_API bool GetAddLibraryFilter() const;
		NIAGARAEDITOR_API bool GetAddTemplateFilter() const;
		NIAGARAEDITOR_API SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions GetTabOptions() const;
	private:
		bool bAddSourceFilter;
		bool bAddLibraryFilter;
		bool bAddTemplateFilter;
		
		SNiagaraTemplateTabBox::FNiagaraTemplateTabOptions TabOptions;		
	};
	
	SLATE_BEGIN_ARGS(SNiagaraFilterBox)
		: _bLibraryOnly(true)
		, _Class(nullptr)
	{}
		SLATE_ATTRIBUTE(bool, bLibraryOnly)
		SLATE_EVENT(SNiagaraSourceFilterBox::FOnFiltersChanged, OnSourceFiltersChanged)
		SLATE_EVENT(SNiagaraLibraryOnlyToggleHeader::FOnLibraryOnlyChanged, OnLibraryOnlyChanged)
		SLATE_EVENT(SNiagaraTemplateTabBox::FOnTabActivated, OnTabActivated)
		SLATE_ARGUMENT(UClass*, Class)
	SLATE_END_ARGS()

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, FFilterOptions FilterOptions);
	
	NIAGARAEDITOR_API bool IsSourceFilterActive(EScriptSource Source) const;
	NIAGARAEDITOR_API bool GetActiveTemplateTab(ENiagaraScriptTemplateSpecification& OutTemplateSpecification) const;
	
private:
	TSharedPtr<SNiagaraSourceFilterBox> SourceFilterBox;
	TSharedPtr<SNiagaraLibraryOnlyToggleHeader> LibraryOnlyToggleHeader;
	TSharedPtr<SNiagaraTemplateTabBox> TemplateTabBox;
};
