// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GraphEditor.h"
#include "NiagaraActions.h"
#include "ViewModels/NiagaraScriptGraphViewModel.h"
#include "SItemSelector.h"
#include "SNiagaraFilterBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "EdGraph/EdGraphSchema.h"

class SGraphActionMenu;
class UEdGraph;

/////////////////////////////////////////////////////////////////////////////////////////////////

typedef SItemSelector<FString, TSharedPtr<FNiagaraAction_NewNode>, ENiagaraMenuSections> SNiagaraMenuActionSelector;

struct FNiagaraGraphActionMenuFilter
{
	FNiagaraGraphActionMenuFilter()
	{}

	virtual ~FNiagaraGraphActionMenuFilter() = default;

	virtual bool PassesFilter(const FNiagaraAction_NewNode& Item) const = 0;
	virtual TSharedRef<SWidget> GenerateWidget() = 0;
	TDelegate<void()> OnFilterChanged;
};

class SNiagaraGraphActionMenu : public SBorder
{
public:
	/** The name of the profile used to save favorite action data. Used with UNiagaraFavoriteActionsConfig.
	 * Each action gets an identifier, and the user-authored favorites data is saved per profile. */
	static const FName FavoriteActionsProfile;
	
	SLATE_BEGIN_ARGS( SNiagaraGraphActionMenu )
		: _GraphObj( static_cast<UEdGraph*>(NULL) )
		, _NewNodePosition( FVector2f::ZeroVector )
		, _AutoExpandActionMenu( false )
		{}

		SLATE_ARGUMENT( UEdGraph*, GraphObj )
		SLATE_ARGUMENT( FDeprecateSlateVector2D, NewNodePosition )
		SLATE_ARGUMENT( TArray<UEdGraphPin*>, DraggedFromPins )
		SLATE_ARGUMENT( SGraphEditor::FActionMenuClosed, OnClosedCallback )
		SLATE_ARGUMENT( bool, AutoExpandActionMenu )
	SLATE_END_ARGS()

	NIAGARAEDITOR_API void Construct( const FArguments& InArgs );

	NIAGARAEDITOR_API ~SNiagaraGraphActionMenu();

	NIAGARAEDITOR_API TSharedRef<SWidget> GetFilterTextBox();

protected:
	UEdGraph* GraphObj;
	FNiagaraScriptGraphViewModel* ViewModel;
	TArray<UEdGraphPin*> DraggedFromPins;
	FDeprecateSlateVector2D NewNodePosition;
	bool AutoExpandActionMenu;

	SGraphEditor::FActionMenuClosed OnClosedCallback;

	TSharedPtr<SNiagaraFilterBox> FilterBox;
	TSharedPtr<SNiagaraMenuActionSelector> ActionSelector;

	NIAGARAEDITOR_API TArray<TSharedPtr<FNiagaraAction_NewNode>> CollectAllActions();
	
	UEdGraph* OwnerOfTemporaries = nullptr;

	static NIAGARAEDITOR_API bool bLibraryOnly;
private:
	TArray<FString> OnGetCategoriesForItem(const TSharedPtr<FNiagaraAction_NewNode>& Item);
	TArray<ENiagaraMenuSections> OnGetSectionsForItem(const TSharedPtr<FNiagaraAction_NewNode>& Item);
	bool OnCompareSectionsForEquality(const ENiagaraMenuSections& SectionA, const ENiagaraMenuSections& SectionB);
	bool OnCompareSectionsForSorting(const ENiagaraMenuSections& SectionA, const ENiagaraMenuSections& SectionB);
	bool OnCompareCategoriesForEquality(const FString& CategoryA, const FString& CategoryB);
	bool OnCompareCategoriesForSorting(const FString& CategoryA, const FString& CategoryB);
	bool OnCompareItemsForEquality(const TSharedPtr<FNiagaraAction_NewNode>& ItemA, const TSharedPtr<FNiagaraAction_NewNode>& ItemB);
	bool OnCompareItemsForSorting(const TSharedPtr<FNiagaraAction_NewNode>& ItemA, const TSharedPtr<FNiagaraAction_NewNode>& ItemB);
	TSharedRef<SWidget> OnGenerateWidgetForSection(const ENiagaraMenuSections& Section);
	TSharedRef<SWidget> OnGenerateWidgetForCategory(const FString& Category);
	TSharedRef<SWidget> OnGenerateWidgetForItem(const TSharedPtr<FNiagaraAction_NewNode>& Item);
	bool DoesItemPassCustomFilter(const TSharedPtr<FNiagaraAction_NewNode>& Item);
	bool DoesSectionPassCustomFilter(const ENiagaraMenuSections& Section);
	void OnItemActivated(const TSharedPtr<FNiagaraAction_NewNode>& Item);
	void TriggerRefresh(const TMap<EScriptSource, bool>& SourceState);
private:
	FText GetFilterText() const { return ActionSelector->GetFilterText(); }
	bool GetLibraryOnly() const { return bLibraryOnly; }
	void SetLibraryOnly(bool bInIsLibraryOnly);	
};
