// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraNodeFunctionCall.h"
#include "Widgets/SCompoundWidget.h"

class SNiagaraHierarchyModuleInput : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraHierarchyModuleInput)
		{}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<struct FNiagaraModuleInputViewModel> InputViewModel);

private:
	FText GetDisplayNameOverride() const;
	EVisibility GetDisplayNameOverrideVisibility() const;
	FText GetDisplayNameOverrideTooltip() const;
	TWeakPtr<struct FNiagaraModuleInputViewModel> InputViewModelWeakPtr;
};
