// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"

class SGraphActionMenu;

class SNiagaraLibraryOnlyToggleHeader : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnLibraryOnlyChanged, bool);

public:
	SLATE_BEGIN_ARGS(SNiagaraLibraryOnlyToggleHeader)
		: _LibraryOnly(true)
		, _HeaderLabelText(NSLOCTEXT("LibraryOnlyToggle", "DefaultHeader", "Select Item"))
		, _bShowHeaderLabel(true)
		{}
		SLATE_ATTRIBUTE(bool, LibraryOnly)
		SLATE_EVENT(FOnLibraryOnlyChanged, LibraryOnlyChanged)
		SLATE_ARGUMENT(FText, HeaderLabelText)
		SLATE_ARGUMENT(bool, bShowHeaderLabel)
	SLATE_END_ARGS();

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs);

	// @Todo: Since new menus are making use of the SItemSelector instead of the graph menu, we should get rid of this once we clean up all dependencies
	NIAGARAEDITOR_API void SetActionMenu(TSharedRef<SGraphActionMenu> InActionMenu);

private:
	void OnCheckStateChanged(ECheckBoxState InCheckState);

	ECheckBoxState GetCheckState() const;

private:
	TAttribute<bool> LibraryOnly;
	FOnLibraryOnlyChanged LibraryOnlyChanged;
	TWeakPtr<SGraphActionMenu> ActionMenuWeak;
};