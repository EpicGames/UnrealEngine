// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraParameterMenu.h"

#include "EdGraphSchema_Niagara.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/StringOutputDevice.h"
#include "NiagaraActions.h"
#include "NiagaraConstants.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraGraph.h"
#include "NiagaraNode.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraNodeOutput.h"
#include "NiagaraNodeParameterMapBase.h"
#include "NiagaraNodeWithDynamicPins.h"
#include "NiagaraParameterDefinitions.h"
#include "NiagaraScriptVariable.h"
#include "NiagaraScriptSource.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemEditorData.h"
#include "SGraphActionMenu.h"
#include "SNiagaraGraphActionWidget.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/TNiagaraViewModelManager.h"
#include "Widgets/SNiagaraActionMenuExpander.h"

#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "NiagaraEditorStyle.h"
#include "Widgets/SNiagaraPinTypeSelector.h"


#include "NiagaraDataChannel.h"

#define LOCTEXT_NAMESPACE "SNiagaraParameterMenu"
namespace UE::Private
{
	template<typename... Strings>
	FText ConcatinateCategoriesIfNotEmpty(Strings... InCategories);
	FText ConcatinateCategoriesIfNotEmptyImpl(TConstArrayView<FStringView> Paths);
}

template<typename... Strings>
FText UE::Private::ConcatinateCategoriesIfNotEmpty(Strings... InCategories)
{
	return ConcatinateCategoriesIfNotEmptyImpl({ FStringView(InCategories)... });
}

FText UE::Private::ConcatinateCategoriesIfNotEmptyImpl(TConstArrayView<FStringView> Paths)
{
	FString Result;
	bool bIsFirst = true;
	for (FStringView String : Paths)
	{
		if (!String.IsEmpty())
		{
			if (!bIsFirst)
			{
				Result.Append(TEXT("|"));
			}
			Result.Append(String);
			bIsFirst = false;
		}
	}
	return FText::FromString(MoveTemp(Result));
}
///////////////////////////////////////////////////////////////////////////////
/// Base Parameter Menu														///
///////////////////////////////////////////////////////////////////////////////

void SNiagaraParameterMenu::Construct(const FArguments& InArgs)
{
	this->bAutoExpandMenu = InArgs._AutoExpandMenu;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			.MinDesiredWidth(300)
			.MaxDesiredHeight(700) // Set max desired height to prevent flickering bug for menu larger than screen
			[
				SAssignNew(GraphMenu, SGraphActionMenu)
				.OnActionSelected(this, &SNiagaraParameterMenu::OnActionSelected)
				.OnCollectAllActions(this, &SNiagaraParameterMenu::CollectAllActions)
				.SortItemsRecursively(false)
				.AlphaSortItems(false)
				.AutoExpandActionMenu(bAutoExpandMenu)
				.bAutomaticallySelectSingleAction(true)
				.ShowFilterTextBox(true)
				.OnGetSectionTitle(InArgs._OnGetSectionTitle)
				.OnCreateCustomRowExpander_Static(&SNiagaraParameterMenu::CreateCustomActionExpander)
				.OnCreateWidgetForAction_Lambda([](const FCreateWidgetForActionData* InData)
					{
						return SNew(SNiagaraGraphActionWidget, InData);
					})
				]
		]
	];
}

TSharedPtr<SEditableTextBox> SNiagaraParameterMenu::GetSearchBox()
{
	return GraphMenu->GetFilterTextBox();
}

void SNiagaraParameterMenu::OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions, ESelectInfo::Type InSelectionType)
{
	if (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::Direct || InSelectionType == ESelectInfo::OnKeyPress || SelectedActions.Num() == 0)
	{
		for (int32 ActionIndex = 0; ActionIndex < SelectedActions.Num(); ActionIndex++)
		{
			TSharedPtr<FNiagaraMenuAction> CurrentAction = StaticCastSharedPtr<FNiagaraMenuAction>(SelectedActions[ActionIndex]);

			if (CurrentAction.IsValid())
			{
				FSlateApplication::Get().DismissAllMenus();
				CurrentAction->ExecuteAction();
			}
		}
	}
}

TSharedRef<SExpanderArrow> SNiagaraParameterMenu::CreateCustomActionExpander(const struct FCustomExpanderData& ActionMenuData)
{
	return SNew(SNiagaraActionMenuExpander, ActionMenuData);
}

bool SNiagaraParameterMenu::IsStaticSwitchParameter(const FNiagaraVariable& Variable, const TArray<UNiagaraGraph*>& Graphs)
{
	for (const UNiagaraGraph* Graph : Graphs)
	{
		TArray<FNiagaraVariable> SwitchInputs = Graph->FindStaticSwitchInputs();
		if (SwitchInputs.Contains(Variable))
		{
			return true;
		}
	}
	return false;
}

FText SNiagaraParameterMenu::GetNamespaceCategoryText(const FNiagaraNamespaceMetadata& NamespaceMetaData)
{
	return NamespaceMetaData.DisplayNameLong.IsEmptyOrWhitespace() == false ? NamespaceMetaData.DisplayNameLong : NamespaceMetaData.DisplayName;
}

FText SNiagaraParameterMenu::GetNamespaceCategoryText(const FGuid& NamespaceId)
{
	const FNiagaraNamespaceMetadata NamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForId(NamespaceId);
	return GetNamespaceCategoryText(NamespaceMetaData);
}


///////////////////////////////////////////////////////////////////////////////
/// Add Parameter Menu														///
///////////////////////////////////////////////////////////////////////////////

void SNiagaraAddParameterFromPanelMenu::Construct(const FArguments& InArgs)
{
	this->OnSpecificParameterRequested = InArgs._OnSpecificParameterRequested;
	this->OnNewParameterRequested = InArgs._OnNewParameterRequested;
	this->OnAddScriptVar = InArgs._OnAddScriptVar;
	this->OnAllowMakeType = InArgs._OnAllowMakeType;
	this->OnAddParameterDefinitions = InArgs._OnAddParameterDefinitions;

	this->Graphs = InArgs._Graphs;
	this->AvailableParameterDefinitions = InArgs._AvailableParameterDefinitions;
	this->SubscribedParameterDefinitions = InArgs._SubscribedParameterDefinitions;
	this->NamespaceId = InArgs._NamespaceId;
	this->bAllowCreatingNew = InArgs._AllowCreatingNew;
	this->bShowNamespaceCategory = InArgs._ShowNamespaceCategory;
	this->bShowGraphParameters = InArgs._ShowGraphParameters;
	this->bIsParameterReadNode = InArgs._IsParameterRead;
	this->bForceCollectEngineNamespaceParameterActions = InArgs._ForceCollectEngineNamespaceParameterActions;
	this->bCullParameterActionsAlreadyInGraph = InArgs._CullParameterActionsAlreadyInGraph;
	this->AdditionalCulledParameterNames = InArgs._AdditionalCulledParameterNames;
	this->AssignmentNode = InArgs._AssignmentNode;
	this->bOnlyShowParametersInNamespaceId = NamespaceId.IsValid();

	SNiagaraParameterMenu::FArguments SuperArgs;
	SuperArgs._AutoExpandMenu = InArgs._AutoExpandMenu;
	SuperArgs._OnGetSectionTitle = FGetSectionTitle::CreateStatic(&SNiagaraAddParameterFromPanelMenu::GetSectionTitle);
	SNiagaraParameterMenu::Construct(SuperArgs);
}

void SNiagaraAddParameterFromPanelMenu::CollectMakeNew(FNiagaraMenuActionCollector& Collector, const FGuid& InNamespaceId)
{
	if (bAllowCreatingNew == false)
	{
		return;
	}
	
	TArray<FNiagaraTypeDefinition> SectionTypes;
	if(InNamespaceId == FNiagaraEditorGuids::UserNamespaceMetaDataGuid) 
	{
		FNiagaraEditorUtilities::GetAllowedUserVariableTypes(SectionTypes);
	}
	else if(InNamespaceId == FNiagaraEditorGuids::SystemNamespaceMetaDataGuid)
	{
		FNiagaraEditorUtilities::GetAllowedSystemVariableTypes(SectionTypes);
	}
	else if (InNamespaceId == FNiagaraEditorGuids::EmitterNamespaceMetaDataGuid)
	{
		FNiagaraEditorUtilities::GetAllowedEmitterVariableTypes(SectionTypes);
	}
	else if (InNamespaceId == FNiagaraEditorGuids::ParticleAttributeNamespaceMetaDataGuid)
	{
		FNiagaraEditorUtilities::GetAllowedParticleVariableTypes(SectionTypes);
	}
	else
	{
		FNiagaraEditorUtilities::GetAllowedParameterTypes(SectionTypes);
	}

	TArray<FNiagaraTypeDefinition> TypeDefinitions;
	for (const FNiagaraTypeDefinition& RegisteredType : SectionTypes)
	{
		bool bAllowType = true;
		if (OnAllowMakeType.IsBound())
		{
			bAllowType = OnAllowMakeType.Execute(RegisteredType);
		}

		if (bAllowType)
		{
			TypeDefinitions.Add(RegisteredType);
		}
	}

	AddMakeNewGroup(Collector, TypeDefinitions,  InNamespaceId,
		LOCTEXT("MakeNewCat", "Make New"), 1,
		bShowNamespaceCategory ? GetNamespaceCategoryText(InNamespaceId).ToString() : FString());
}

void SNiagaraAddParameterFromPanelMenu::AddParameterGroup(
	FNiagaraMenuActionCollector& Collector,
	TArray<FNiagaraVariable>& Variables,
	const FGuid& InNamespaceId /*= FGuid()*/,
	const FText& Category /*= FText::GetEmpty()*/,
	int32 SortOrder /*= 0*/)
{
	for (const FNiagaraVariable& Variable : Variables)
	{
		const FText DisplayName = FText::FromName(Variable.GetName());
		FText Tooltip = FText::GetEmpty();

		if (const UStruct* VariableStruct = Variable.GetType().GetStruct())
		{
			Tooltip = VariableStruct->GetToolTipText(true);
		}
		if (const FNiagaraVariableMetaData* VariableMetaData = FNiagaraConstants::GetConstantMetaData(Variable))
		{
			FText Text = VariableMetaData->Description;
			if (Text.IsEmptyOrWhitespace() == false)
			{
				Tooltip = Text;
			}
		}

		FText SubCategory = FNiagaraEditorUtilities::GetVariableTypeCategory(Variable);
		FText FullCategory = SubCategory.IsEmpty() ? Category : FText::Format(FText::FromString("{0}|{1}"), Category, SubCategory);
		TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(FullCategory, DisplayName, Tooltip, 0, FText(),
			FNiagaraMenuAction::FOnExecuteStackAction::CreateSP(this, &SNiagaraAddParameterFromPanelMenu::ParameterSelected, Variable, InNamespaceId)));
		Action->SetParameterVariable(Variable);

		if (Variable.IsDataInterface())
		{
			if (const UClass* DataInterfaceClass = Variable.GetType().GetClass())
			{
				Action->IsExperimental = DataInterfaceClass->GetMetaData("DevelopmentStatus") == TEXT("Experimental");
			}
		}

		Collector.AddAction(Action, SortOrder);
	}
}

void SNiagaraAddParameterFromPanelMenu::AddMakeNewGroup(
	FNiagaraMenuActionCollector& Collector,
	TArray<FNiagaraTypeDefinition>& TypeDefinitions, 
	const FGuid& InNamespaceId, 
	const FText& Category, 
	int32 SortOrder,
	const FString& RootCategory)
{
	for (const FNiagaraTypeDefinition& TypeDefinition : TypeDefinitions)
	{
		const FText DisplayName = TypeDefinition.GetNameText();
		FText Tooltip = FText::GetEmpty();

		if (const UStruct* VariableStruct = TypeDefinition.GetStruct())
		{
			Tooltip = VariableStruct->GetToolTipText(true);
		}

		FText SubCategory = FNiagaraEditorUtilities::GetTypeDefinitionCategory(TypeDefinition);
		FText FullCategory = UE::Private::ConcatinateCategoriesIfNotEmpty(RootCategory, Category.ToString(), SubCategory.ToString() );
		TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(FullCategory, DisplayName, Tooltip, 0, FText(),
			FNiagaraMenuAction::FOnExecuteStackAction::CreateSP(this, &SNiagaraAddParameterFromPanelMenu::NewParameterSelected, TypeDefinition, InNamespaceId)));

		if (TypeDefinition.IsDataInterface())
		{
			if (const UClass* DataInterfaceClass = TypeDefinition.GetClass())
			{
				Action->IsExperimental = DataInterfaceClass->GetMetaData("DevelopmentStatus") == TEXT("Experimental");
			}
		}

		Collector.AddAction(Action, SortOrder);
	}
}

void SNiagaraAddParameterFromPanelMenu::CollectParameterCollectionsActions(FNiagaraMenuActionCollector& Collector)
{
	//Create sub menus for parameter collections.
	const FText Category = GetNamespaceCategoryText(FNiagaraEditorGuids::ParameterCollectionNamespaceMetaDataGuid);
	TArray<UNiagaraParameterCollection*> AvailableParameterCollections;
	FNiagaraEditorUtilities::GetAvailableParameterCollections(AvailableParameterCollections);
	for (UNiagaraParameterCollection* Collection : AvailableParameterCollections)
	{
		AddParameterGroup(Collector, Collection->GetParameters(), FNiagaraEditorGuids::ParameterCollectionNamespaceMetaDataGuid, Category, 10);
	}
}

void SNiagaraAddParameterFromPanelMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FNiagaraMenuActionCollector Collector;
	auto CollectEngineNamespaceParameterActions = [this, &Collector]() {
		const FNiagaraNamespaceMetadata NamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForId(FNiagaraEditorGuids::EngineNamespaceMetaDataGuid);
		TArray<FNiagaraVariable> Variables = FNiagaraConstants::GetEngineConstants();
		const FText CategoryText = bShowNamespaceCategory ? GetNamespaceCategoryText(NamespaceMetaData) : LOCTEXT("EngineConstantNamespaceCategory", "Add Engine Constant");
		AddParameterGroup(
			Collector,
			Variables,
			FNiagaraEditorGuids::EngineNamespaceMetaDataGuid,
			CategoryText,
			4
		);
	};
	auto CollectEmitterNamespaceParameterActions = [this, &Collector]() {
		const FNiagaraNamespaceMetadata NamespaceMetaData = FNiagaraEditorUtilities::GetNamespaceMetaDataForId(FNiagaraEditorGuids::EmitterNamespaceMetaDataGuid);
		TArray<FNiagaraVariable> Variables = FNiagaraConstants::GetEngineConstants().FilterByPredicate([](const FNiagaraVariable& Var) { return Var.IsInNameSpace(FNiagaraConstants::EmitterNamespaceString); });
		const FText CategoryText = bShowNamespaceCategory ? GetNamespaceCategoryText(NamespaceMetaData) : LOCTEXT("EmitterConstantNamespaceCategory", "Add Emitter Constant");
		AddParameterGroup(
			Collector,
			Variables,
			FNiagaraEditorGuids::EngineNamespaceMetaDataGuid,
			CategoryText,
			4
		);
	};

	TSet<FGuid> ExistingGraphParameterIds;
	TSet<FName> VisitedParameterNames;
	// Append additional culled parameter names to visited parameter names so that we preemptively cull any parameters that are name matching.
	VisitedParameterNames.Append(AdditionalCulledParameterNames);
	TArray<FGuid> ExcludedNamespaceIds;
	// If this is a write node, exclude any read-only vars.
	if (!bIsParameterReadNode)
	{
		ExcludedNamespaceIds.Add(FNiagaraEditorGuids::UserNamespaceMetaDataGuid);
		ExcludedNamespaceIds.Add(FNiagaraEditorGuids::EngineNamespaceMetaDataGuid);
		ExcludedNamespaceIds.Add(FNiagaraEditorGuids::ParameterCollectionNamespaceMetaDataGuid);
	}

	// If this doesn't have particles in the script, exclude reading or writing them.
	// Also, collect the ids of all variables the graph owns to exclude them from parameters to add from libraries.
	for (const UNiagaraGraph* Graph : Graphs)
	{
		bool IsModule = Graph->FindOutputNode(ENiagaraScriptUsage::Module) != nullptr || Graph->FindOutputNode(ENiagaraScriptUsage::DynamicInput) != nullptr
			|| Graph->FindOutputNode(ENiagaraScriptUsage::Function) != nullptr;

		UNiagaraScriptSource* Source = Cast<UNiagaraScriptSource>(Graph->GetOuter());
		if (Source && IsModule)
		{
			UNiagaraScript* Script = Cast<UNiagaraScript>(Source->GetOuter());
			if (Script)
			{
				TArray<ENiagaraScriptUsage> Usages = Script->GetLatestScriptData()->GetSupportedUsageContexts();
				if (!Usages.Contains(ENiagaraScriptUsage::ParticleEventScript) &&
					!Usages.Contains(ENiagaraScriptUsage::ParticleSpawnScript) &&
					!Usages.Contains(ENiagaraScriptUsage::ParticleUpdateScript))
				{
					ExcludedNamespaceIds.Add(FNiagaraEditorGuids::ParticleAttributeNamespaceMetaDataGuid);
				}

				if (bIsParameterReadNode)
				{
					if (!Usages.Contains(ENiagaraScriptUsage::SystemSpawnScript) &&
						!Usages.Contains(ENiagaraScriptUsage::SystemUpdateScript))
					{
						ExcludedNamespaceIds.Add(FNiagaraEditorGuids::SystemNamespaceMetaDataGuid);
					}

					if (!Usages.Contains(ENiagaraScriptUsage::EmitterSpawnScript) &&
						!Usages.Contains(ENiagaraScriptUsage::EmitterUpdateScript))
					{
						ExcludedNamespaceIds.Add(FNiagaraEditorGuids::EmitterNamespaceMetaDataGuid);
					}
				}
			}
		}

		// If culling parameter actions that match existing parameters in the graph, collect all IDs for parameters visited in the graph.
		if(bCullParameterActionsAlreadyInGraph)
		{ 
			TArray<FGuid> VariableGuids;
			Graph->GetAllScriptVariableGuids(VariableGuids);
			ExistingGraphParameterIds.Append(VariableGuids);
		}
	}


	// Parameter collections
	if (NamespaceId == FNiagaraEditorGuids::ParameterCollectionNamespaceMetaDataGuid)
	{
		CollectParameterCollectionsActions(Collector);
	}
	// Engine intrinsic parameters
	else if (NamespaceId == FNiagaraEditorGuids::EngineNamespaceMetaDataGuid)
	{
		CollectEngineNamespaceParameterActions();
	}
	// Emitter intrinsic parameters
	else if (NamespaceId == FNiagaraEditorGuids::EmitterNamespaceMetaDataGuid)
	{
		CollectEmitterNamespaceParameterActions();
		CollectMakeNew(Collector, NamespaceId);
	}
	// DataInstance intrinsic parameters
	else if (NamespaceId == FNiagaraEditorGuids::DataInstanceNamespaceMetaDataGuid && (ExcludedNamespaceIds.Contains(FNiagaraEditorGuids::ParticleAttributeNamespaceMetaDataGuid) == false))
	{
		TArray<FNiagaraVariable> Variables;
		Variables.Add(SYS_PARAM_INSTANCE_ALIVE);
		AddParameterGroup(Collector, Variables, FNiagaraEditorGuids::DataInstanceNamespaceMetaDataGuid, FText(), 3);
	}
	// No NamespaceId set but still collecting engine namespace parameters (e.g. map get/set node menu.)
	else if (NamespaceId.IsValid() == false && bForceCollectEngineNamespaceParameterActions)
	{
		CollectEngineNamespaceParameterActions();
		CollectMakeNew(Collector, NamespaceId);

		// Special case; collect DataInstance.Alive so that it is an option if we are selecting a parameter from a map node in a script.
		TArray<FNiagaraVariable> Variables;
		Variables.Add(SYS_PARAM_INSTANCE_ALIVE);
		AddParameterGroup(Collector, Variables, FNiagaraEditorGuids::DataInstanceNamespaceMetaDataGuid, FText(), 3);
	}

	// Any other "unreserved" namespace
	else
	{
		CollectMakeNew(Collector, NamespaceId);
	}

	// Collect "add existing graph parameter" actions
	if (bShowGraphParameters)
	{
		for (const UNiagaraGraph* Graph : Graphs)
		{
			TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection> ParameterEntries = Graph->GetParameterReferenceMap();

			// Iterate the parameter reference map as this represents all parameters in the graph, including parameters the graph itself does not own.
			for (const auto& ParameterEntry : ParameterEntries)
			{
				const FNiagaraVariable& Parameter = ParameterEntry.Key;

				// Check if the graph owns the parameter (has a script variable for the parameter.)
				if (Graph->HasVariable(Parameter))
				{
					// The graph owns the parameter. Skip if it is a static switch.
					TOptional<bool> IsStaticSwitch = Graph->IsStaticSwitch(Parameter);
					if (IsStaticSwitch.IsSet() && *IsStaticSwitch)
					{
						continue;
					}
					// Check that we do not add a duplicate entry.
					const FName& ParameterName = Parameter.GetName();
					if (VisitedParameterNames.Contains(ParameterName) == false)
					{
						// The script variable is not a duplicate, add an entry for it.
						VisitedParameterNames.Add(ParameterName);
						FNiagaraVariable MutableParameter = FNiagaraVariable(Parameter);
						const FText Category = bShowNamespaceCategory ? GetNamespaceCategoryText(NamespaceId) : LOCTEXT("NiagaraAddExistingParameterMenu", "Add Existing Parameter");
						const FText DisplayName = FText::FromName(Parameter.GetName());
						const FText Tooltip = FText::GetEmpty();
						TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
							Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
							FNiagaraMenuAction::FOnExecuteStackAction::CreateSP(this, &SNiagaraAddParameterFromPanelMenu::ParameterSelected, MutableParameter)));
						Action->SetParameterVariable(Parameter);

						Collector.AddAction(Action, 3);
						continue;
					}
				}
			
				// The graph does not own the parameter, check if it is a reserved namespace parameter.
				const FNiagaraParameterHandle ParameterHandle = FNiagaraParameterHandle(Parameter.GetName());
				if (ParameterHandle.IsParameterCollectionHandle() || ParameterHandle.IsEngineHandle() || ParameterHandle.IsDataInstanceHandle())
				{
					// Check that we do not add a duplicate entry.
					const FName& ParameterName = Parameter.GetName();
					if (VisitedParameterNames.Contains(ParameterName) == false)
					{
						// The reserved namespace parameter is not a duplicate, add an entry for it.
						VisitedParameterNames.Add(ParameterName);
						FNiagaraVariable MutableParameter = FNiagaraVariable(Parameter);
						const FText Category = bShowNamespaceCategory ? GetNamespaceCategoryText(NamespaceId) : LOCTEXT("NiagaraAddExistingParameterMenu", "Add Existing Parameter");
						const FText DisplayName = FText::FromName(Parameter.GetName());
						const FText Tooltip = FText::GetEmpty();
						TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
							Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
							FNiagaraMenuAction::FOnExecuteStackAction::CreateSP(this, &SNiagaraAddParameterFromPanelMenu::ParameterSelected, MutableParameter)));
						Action->SetParameterVariable(Parameter);

						Collector.AddAction(Action, 3);
						continue;
					}
				}
			}
		}
	}

	// Collect "add parameter from parameter definition asset" actions
	for (UNiagaraParameterDefinitions* ParameterDefinitions : AvailableParameterDefinitions)
	{
		bool bTopLevelCategory = ParameterDefinitions->GetIsPromotedToTopInAddMenus();
		const FText Category = bTopLevelCategory ? FText::FromString(*ParameterDefinitions->GetName()) : FText();
		for (const UNiagaraScriptVariable* ScriptVar : ParameterDefinitions->GetParametersConst())
		{
			// Only add parameters in the same namespace as the target namespace id if bOnlyShowParametersInNamespaceId is set.
			if (bOnlyShowParametersInNamespaceId && FNiagaraEditorUtilities::GetNamespaceMetaDataForVariableName(ScriptVar->Variable.GetName()).GetGuid() != NamespaceId)
			{
				continue;
			}

			// Check that we do not add a duplicate entry.
			const FGuid& ScriptVarId = ScriptVar->Metadata.GetVariableGuid();
			if (bCullParameterActionsAlreadyInGraph && ExistingGraphParameterIds.Contains(ScriptVarId))
			{
				continue;
			}
			else if (VisitedParameterNames.Contains(ScriptVar->Variable.GetName()))
			{
				continue;
			}

			const FText DisplayName = FText::FromName(ScriptVar->Variable.GetName());
			const FText& Tooltip = ScriptVar->Metadata.Description;
			TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
				Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
				FNiagaraMenuAction::FOnExecuteStackAction::CreateSP(this, &SNiagaraAddParameterFromPanelMenu::ScriptVarFromParameterDefinitionsSelected, ScriptVar, ParameterDefinitions)));
			Action->SetParameterVariable(ScriptVar->Variable);
			
			if (bTopLevelCategory)
			{
				Collector.AddAction(Action, ParameterDefinitions->GetMenuSortOrder());
			}
			else
			{ 
				Action->SetSectionId(1); //Increment the default section id so parameter definitions actions are always categorized BELOW other actions.
				Collector.AddAction(Action, ParameterDefinitions->GetMenuSortOrder());
			}
		}
	}

	// Collect "add existing parameter" actions associated with Assignment nodes.
	if (AssignmentNode != nullptr)
	{
		// Gather required members for context from the AssignmentNode.
		UNiagaraNodeOutput* OutputNode = FNiagaraStackGraphUtilities::GetEmitterOutputNodeForStackNode(*AssignmentNode);
		UNiagaraSystem* OwningSystem = AssignmentNode->GetTypedOuter<UNiagaraSystem>();
		if (OwningSystem == nullptr)
		{
			return;
		}

		UNiagaraSystemEditorData* OwningSystemEditorData = Cast<UNiagaraSystemEditorData>(OwningSystem->GetEditorData());
		if (OwningSystemEditorData == nullptr)
		{
			return;
		}

		bool bOwningSystemIsPlaceholder = OwningSystemEditorData->GetOwningSystemIsPlaceholder();
		TOptional<FName> StackContextOverride = OutputNode->GetStackContextOverride();

		// Gather available parameters from the parameter map history.
		TArray<FNiagaraVariable> AvailableParameters;
		TArray<FName> CustomIterationNamespaces;
		TArray<FNiagaraParameterMapHistory> Histories = UNiagaraNodeParameterMapBase::GetParameterMaps(OutputNode->GetNiagaraGraph());
		for (FNiagaraParameterMapHistory& History : Histories)
		{
			for (int32 VarIdx = 0; VarIdx < History.Variables.Num(); VarIdx++)
			{
				FNiagaraVariable& Variable = History.Variables[VarIdx];
				if (StackContextOverride.IsSet() && StackContextOverride.GetValue() != NAME_None && Variable.IsInNameSpace(StackContextOverride.GetValue()))
				{
					AvailableParameters.AddUnique(Variable);
				}
				else if (History.IsPrimaryDataSetOutput(Variable, OutputNode->GetUsage()))
				{
					AvailableParameters.AddUnique(Variable);
				}
			}

			for (const FName& Namespace : History.IterationNamespaceOverridesEncountered)
			{
				CustomIterationNamespaces.AddUnique(Namespace);
			}
		}

		// Gather available parameters in the used namespace from the graph parameter to reference map and the system editor only parameters.
		auto VariableIsStrictlyInNamespace = [](const FNiagaraVariable& Var, const FName& Namespace)->bool {
			FName OutName;
			const TArray<FName> Namespaces = FNiagaraEditorUtilities::DecomposeVariableNamespace(Var.GetName(), OutName);
			if (Namespaces.Num() == 1 && Namespaces[0] == Namespace)
			{
				return true;
			}
			return false;
		};

		auto AddAvailableParametersForNamespace = [&AvailableParameters, &OutputNode, &OwningSystem, &VariableIsStrictlyInNamespace](const FName& Namespace) {
			for (const TPair<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection>& Entry : OutputNode->GetNiagaraGraph()->GetParameterReferenceMap())
			{
				// Pick up any params in the target namespace.
				if (VariableIsStrictlyInNamespace(Entry.Key, Namespace))
				{
					AvailableParameters.AddUnique(Entry.Key);
				}
			}

			TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = TNiagaraViewModelManager<UNiagaraSystem, FNiagaraSystemViewModel>::GetExistingViewModelForObject(OwningSystem);
			if (SystemViewModel.IsValid())
			{
				for (const UNiagaraScriptVariable* EditorOnlyScriptVar : SystemViewModel->GetEditorOnlyParametersAdapter()->GetParameters())
				{
					const FNiagaraVariable& EditorOnlyParameter = EditorOnlyScriptVar->Variable;
					if (VariableIsStrictlyInNamespace(EditorOnlyParameter, Namespace))
					{
						AvailableParameters.AddUnique(EditorOnlyParameter);
					}
				}
			}
		};

		TOptional<FName> UsageNamespace = FNiagaraStackGraphUtilities::GetNamespaceForOutputNode(OutputNode);
		if (UsageNamespace.IsSet())
		{
			AddAvailableParametersForNamespace(UsageNamespace.GetValue());
		}
		AddAvailableParametersForNamespace(FNiagaraConstants::TransientNamespace);

		// Now check to see if any of the available write namespaces have overlap with the iteration namespaces. If so, we need to exclude them if they aren't the active stack context.
		// This is for situations like Emitter.Grid2DCollection.TestValue which should only be written if in the sim stage scripts and not emitter scripts, which would normally be allowed.
		TArray<FName> AvailableWriteNamespaces;
		FNiagaraStackGraphUtilities::GetNamespacesForNewWriteParameters(
			bOwningSystemIsPlaceholder ? FNiagaraStackGraphUtilities::EStackEditContext::Emitter : FNiagaraStackGraphUtilities::EStackEditContext::System,
			OutputNode->GetUsage(), StackContextOverride, AvailableWriteNamespaces);

		TArray<FName> ExclusionList;
		for (const FName& IterationNamespace : CustomIterationNamespaces)
		{
			FNiagaraVariableBase TempVar(FNiagaraTypeDefinition::GetFloatDef(), IterationNamespace);
			for (const FName& AvailableWriteNamespace : AvailableWriteNamespaces)
			{
				if (TempVar.IsInNameSpace(AvailableWriteNamespace))
				{
					if (!StackContextOverride.IsSet() || (StackContextOverride.IsSet() && IterationNamespace != StackContextOverride.GetValue()))
						ExclusionList.AddUnique(IterationNamespace);
				}
			}
		}

		// Cull available parameters if they are outside the available namespaces.
		for (const FNiagaraVariable& AvailableParameter : AvailableParameters)
		{
			bool bFound = false;
			// Now check to see if the variable is possible to write to
			for (const FName& AvailableWriteNamespace : AvailableWriteNamespaces)
			{
				if (AvailableParameter.IsInNameSpace(AvailableWriteNamespace))
				{
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				continue;
			}
				
			// Now double-check that it doesn't overlap with a sub-namespace we're not allowed to write to
			bFound = false;
			for (const FName& ExcludedNamespace : ExclusionList)
			{
				if (AvailableParameter.IsInNameSpace(ExcludedNamespace))
				{
					bFound = true;
					break;
				}
			}

			if (bFound)
			{
				continue;
			}

			// Cull the available parameter if it has already been visited.
			const FName& ParameterName = AvailableParameter.GetName();
			if (VisitedParameterNames.Contains(ParameterName) == false)
			{
				// The parameter is not a duplicate, add an entry for it.
				VisitedParameterNames.Add(ParameterName);
				const FText Category = LOCTEXT("ModuleSetCategory", "Set Specific Parameters");
				const FText DisplayName = FText::FromName(AvailableParameter.GetName());
				const FText VarDesc = FNiagaraConstants::GetAttributeDescription(AvailableParameter);
				const FText Tooltip = FText::Format(LOCTEXT("SetFunctionPopupTooltip", "Description: Set the parameter {0}. {1}"), DisplayName, VarDesc);
				TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
					Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
					FNiagaraMenuAction::FOnExecuteStackAction::CreateSP(this, &SNiagaraAddParameterFromPanelMenu::ParameterSelected, AvailableParameter)));
				Action->SetParameterVariable(AvailableParameter);
				Collector.AddAction(Action, 3);
			}
		}
	}

	Collector.AddAllActionsTo(OutAllActions);
}

void SNiagaraAddParameterFromPanelMenu::ParameterSelected(FNiagaraVariable NewVariable, const FGuid InNamespaceId /*= FGuid()*/)
{	
	OnSpecificParameterRequested.ExecuteIfBound(NewVariable);
}

void SNiagaraAddParameterFromPanelMenu::NewParameterSelected(FNiagaraTypeDefinition NewParameterType, const FGuid InNamespaceId)
{
	auto GetNamespaceMetaData = [this, &InNamespaceId]()->const FNiagaraNamespaceMetadata
	{
		if (InNamespaceId.IsValid() == false)
		{
			if (bIsParameterReadNode) // Map Get
			{
				return FNiagaraEditorUtilities::GetNamespaceMetaDataForId(FNiagaraEditorGuids::ModuleNamespaceMetaDataGuid);
			}
			else // Map Set
			{
				return FNiagaraEditorUtilities::GetNamespaceMetaDataForId(FNiagaraEditorGuids::ModuleLocalNamespaceMetaDataGuid);
			}
		}
		return FNiagaraEditorUtilities::GetNamespaceMetaDataForId(InNamespaceId);
	};
	
	FString TypeDisplayName;
	if (NewParameterType.GetEnum() != nullptr)
	{
		TypeDisplayName = ((UField*)NewParameterType.GetEnum())->GetDisplayNameText().ToString();
	}
	else if (NewParameterType.GetStruct() != nullptr)
	{
		TypeDisplayName = NewParameterType.GetNameText().ToString();
	}
	else if (NewParameterType.GetClass() != nullptr)
	{
		TypeDisplayName = NewParameterType.GetClass()->GetDisplayNameText().ToString();
	}

	const FString NewVariableDefaultName = TypeDisplayName.IsEmpty()
		? TEXT("New Variable")
		: TypeDisplayName;

	TArray<FString> NameParts;

	const FNiagaraNamespaceMetadata NamespaceMetaData = GetNamespaceMetaData();
	checkf(NamespaceMetaData.IsValid(), TEXT("Failed to get valid namespace metadata when creating unique name for parameter menu add parameter action!"));
	for (const FName Namespace : NamespaceMetaData.Namespaces)
	{
		NameParts.Add(Namespace.ToString());
	}

	if (NamespaceMetaData.RequiredNamespaceModifier != NAME_None)
	{
		NameParts.Add(NamespaceMetaData.RequiredNamespaceModifier.ToString());
	}

	NameParts.Add(NewVariableDefaultName);
	const FString ResultName = FString::Join(NameParts, TEXT("."));

	FNiagaraVariable NewVariable(NewParameterType, FName(ResultName));
	OnNewParameterRequested.ExecuteIfBound(NewVariable);
}

void SNiagaraAddParameterFromPanelMenu::ParameterSelected(FNiagaraVariable NewVariable)
{
	ParameterSelected(NewVariable, FGuid());
}

void SNiagaraAddParameterFromPanelMenu::ScriptVarFromParameterDefinitionsSelected(const UNiagaraScriptVariable* NewScriptVar, UNiagaraParameterDefinitions* SourceParameterDefinitions)
{
	// If the parameter definitions the script var belongs to is not subscribed to, add it.
	const FGuid& SourceParameterDefinitionsId = SourceParameterDefinitions->GetDefinitionsUniqueId();
	if (SubscribedParameterDefinitions.ContainsByPredicate([SourceParameterDefinitionsId](const UNiagaraParameterDefinitions* ParameterDefinitions){ return ParameterDefinitions->GetDefinitionsUniqueId() == SourceParameterDefinitionsId; }) == false)
	{
		OnAddParameterDefinitions.ExecuteIfBound(SourceParameterDefinitions);
	}

	// Add the script var.
	OnAddScriptVar.ExecuteIfBound(NewScriptVar);
}

TSet<FName> SNiagaraAddParameterFromPanelMenu::GetAllGraphParameterNames() const
{
	TSet<FName> VisitedParameterNames;
	for (const UNiagaraGraph* Graph : Graphs)
	{
		const TMap<FNiagaraVariable, FNiagaraGraphParameterReferenceCollection> ParameterEntries = Graph->GetParameterReferenceMap();

		// Iterate the parameter reference map as this represents all parameters in the graph, including parameters the graph itself does not own.
		for (const auto& ParameterEntry : ParameterEntries)
		{
			VisitedParameterNames.Add(ParameterEntry.Key.GetName());
		}
	}
	return VisitedParameterNames;
}

FText SNiagaraAddParameterFromPanelMenu::GetSectionTitle(int32 SectionId)
{
	if (SectionId != 1)
	{
		ensureMsgf(false, TEXT("Encountered SectionId that was not \"1\"! Update formatting rules!"));
		return FText();
	}
	return LOCTEXT("ParameterDefinitionsSection", "Parameter Definitions");
}

///////////////////////////////////////////////////////////////////////////////
/// Add Parameter From Pin Menu												///
///////////////////////////////////////////////////////////////////////////////

void SNiagaraAddParameterFromPinMenu::Construct(const FArguments& InArgs)
{
	this->NiagaraNode = InArgs._NiagaraNode;
	this->AddPin = InArgs._AddPin;
	this->bIsParameterReadNode = AddPin->Direction == EEdGraphPinDirection::EGPD_Input ? false : true;

	SNiagaraParameterMenu::FArguments SuperArgs;
	SuperArgs._AutoExpandMenu = InArgs._AutoExpandMenu;
	SNiagaraParameterMenu::Construct(SuperArgs);
}


void SNiagaraAddParameterFromPinMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FNiagaraMenuActionCollector Collector;
	
	//First allow the node to add it's own actions.
	NiagaraNode->CollectAddPinActions(Collector, AddPin);	
	
	TArray<FNiagaraTypeDefinition> Types;
	FNiagaraEditorUtilities::GetAllowedParameterTypes(Types);
	Types.Sort([](const FNiagaraTypeDefinition& A, const FNiagaraTypeDefinition& B) { return (A.GetNameText().ToLower().ToString() < B.GetNameText().ToLower().ToString()); });

	for (const FNiagaraTypeDefinition& RegisteredType : Types)
	{
		bool bAllowType = false;
		bAllowType = NiagaraNode->AllowNiagaraTypeForAddPin(RegisteredType, bIsParameterReadNode ? EEdGraphPinDirection::EGPD_Output : EEdGraphPinDirection::EGPD_Input) ;

		if (bAllowType)
		{
			FNiagaraVariable Var(RegisteredType, FName(*RegisteredType.GetName()));
			FNiagaraEditorUtilities::ResetVariableToDefaultValue(Var);

			FText Category = FNiagaraEditorUtilities::GetVariableTypeCategory(Var);
			const FText DisplayName = RegisteredType.GetNameText();
			const FText Tooltip = FText::Format(LOCTEXT("AddButtonTypeEntryToolTipFormat", "Add a new {0} pin"), RegisteredType.GetNameText());
			const UEdGraphPin* ConstAddPin = AddPin;
			TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
				Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
				FNiagaraMenuAction::FOnExecuteStackAction::CreateUObject(NiagaraNode, &UNiagaraNodeWithDynamicPins::AddParameter, Var, ConstAddPin->Direction)));

			Collector.AddAction(Action, 0);
		}
	}

	Collector.AddAllActionsTo(OutAllActions);
}

///////////////////////////////////////////////////////////////////////////////
/// Change Pin Type Menu													///
///////////////////////////////////////////////////////////////////////////////

void SNiagaraChangePinTypeMenu::Construct(const FArguments& InArgs)
{
	checkf(InArgs._PinToModify != nullptr, TEXT("Tried to construct change pin type menu without valid pin ptr!"));
	this->PinToModify = InArgs._PinToModify;

	SNiagaraParameterMenu::FArguments SuperArgs;
	SuperArgs._AutoExpandMenu = InArgs._AutoExpandMenu;
	SNiagaraParameterMenu::Construct(SuperArgs);
}

void SNiagaraChangePinTypeMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FNiagaraMenuActionCollector Collector;
	UNiagaraNode* Node = Cast<UNiagaraNode>(PinToModify->GetOwningNode());
	checkf(Node, TEXT("Niagara node pin did not have a valid outer node!"));

	TArray<FNiagaraTypeDefinition> Types;
	FNiagaraEditorUtilities::GetAllowedParameterTypes(Types);
	Types.Sort([](const FNiagaraTypeDefinition& A, const FNiagaraTypeDefinition& B) { return (A.GetNameText().ToLower().ToString() < B.GetNameText().ToLower().ToString()); });

	for (const FNiagaraTypeDefinition& RegisteredType : Types)
	{
		const bool bAllowType = Node->AllowNiagaraTypeForPinTypeChange(RegisteredType, PinToModify);

		if (bAllowType)
		{
			FNiagaraVariable Var(RegisteredType, FName(*RegisteredType.GetName()));
			FNiagaraEditorUtilities::ResetVariableToDefaultValue(Var);

			FText Category = FNiagaraEditorUtilities::GetVariableTypeCategory(Var);
			const FText DisplayName = RegisteredType.GetNameText();
			const FText Tooltip = FText::Format(LOCTEXT("ChangeSelectorTypeEntryToolTipFormat", "Change to {0} pin"), RegisteredType.GetNameText());
			TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
				Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
				FNiagaraMenuAction::FOnExecuteStackAction::CreateUObject(Node, &UNiagaraNode::RequestNewPinType, PinToModify, RegisteredType)));

			Collector.AddAction(Action, 0);
		}
	}

	Collector.AddAllActionsTo(OutAllActions);
}

///////////////////////////////////////////////////////////////////////////////
/// Change Pin Type Menu													///
///////////////////////////////////////////////////////////////////////////////

void SNiagaraFunctionSpecifierNDCVariablesMenu::Construct(const FArguments& InArgs)
{
	checkf(InArgs._Owner != nullptr, TEXT("Tried to construct NDC Variable menu without an owner."));
	Owner = InArgs._Owner;
	AllowedTypes = InArgs._AllowedTypes;

	SNiagaraParameterMenu::FArguments SuperArgs;
	SuperArgs._AutoExpandMenu = InArgs._AutoExpandMenu;
	SNiagaraParameterMenu::Construct(SuperArgs);
}

void SNiagaraFunctionSpecifierNDCVariablesMenu::OnSelectionChanged(FNiagaraVariableBase Var)
{
	check(Owner);
	Owner->OnMenuSelectionChanged(Var);
}

void SNiagaraFunctionSpecifierNDCVariablesMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	FNiagaraMenuActionCollector Collector;

	UNiagaraDataChannel::ForEachDataChannel([&](UNiagaraDataChannel* DataChannel)
	{
		const UNiagaraDataChannelAsset* NDCAsset = DataChannel->GetAsset();
		if(NDCAsset)
		{
			for (const FNiagaraDataChannelVariable& NDCVar : DataChannel->GetVariables())
			{
				FText Category = FText::FromString(NDCAsset->GetName());
				const FText DisplayName = FText::FromName(NDCVar.GetName());
				const FText Tooltip = FText::Format(LOCTEXT("SNiagaraFunctionSpecifierNDCVariablesMenu_TooltipFmt", "Variable {0} from Data Channel {1}"), DisplayName, Category);

				if(AllowedTypes.Num() > 0 && !AllowedTypes.Contains(NDCVar.GetType()))
				{
					continue;//This type is not on the allowed list.
				}

				TSharedPtr<FNiagaraMenuAction> Action(new FNiagaraMenuAction(
					Category, DisplayName, Tooltip, 0, FText::GetEmpty(),
					FNiagaraMenuAction::FOnExecuteStackAction::CreateRaw(this, &SNiagaraFunctionSpecifierNDCVariablesMenu::OnSelectionChanged, FNiagaraVariableBase(NDCVar))));

				//TODO: The base niagara action widget will store this parameter and display this regardless of a change in the NDC variable.
				//Need to subclass this and have it refresh it's parameter on construction from the NDCVar that matches it's guid.
				Action->SetParameterVariable(NDCVar);
				Collector.AddAction(Action, 0);
			}
		}
	});

	Collector.AddAllActionsTo(OutAllActions);
}


//////////////////////////////////////////////////////////////////////////

const static FName VarNameSpecifierKey(TEXT("VarName"));
const static FName VarTypeSpecifierKey(TEXT("VarType"));

//Encode a Niagara Variable as a function specifier value.
void EncodeVariableAsSpecifiers(const FNiagaraVariableBase& Var, UNiagaraNodeFunctionCall* TargetNode)
{
	if(TargetNode)
	{
		FString TypeStr;
		UScriptStruct* TypeStruct = FNiagaraTypeDefinition::StaticStruct();
		FNiagaraTypeDefinition TypeDef = Var.GetType();
		TypeStruct->ExportText(TypeStr, &TypeDef, nullptr, nullptr, PPF_None, nullptr);

		TargetNode->SetFunctionSpecifier(VarNameSpecifierKey, Var.GetName());
		TargetNode->SetFunctionSpecifier(VarTypeSpecifierKey, *TypeStr);
	}
}

FNiagaraVariableBase DecodeVariableAsSpecifiers(UNiagaraNodeFunctionCall* Node)
{
	if(Node)
	{
		const FName* FuncSpecVarName = Node->FunctionSpecifiers.Find(VarNameSpecifierKey);
		const FName* FuncSpecVarType = Node->FunctionSpecifiers.Find(VarTypeSpecifierKey);
		if (FuncSpecVarName && FuncSpecVarType)
		{
			FStringOutputDevice ErrorOut;
			UScriptStruct* TypeStruct = FNiagaraTypeDefinition::StaticStruct();
			FNiagaraTypeDefinition TypeDef;
			if (TypeStruct->ImportText(*FuncSpecVarType->ToString(), &TypeDef, nullptr, PPF_None, &ErrorOut, TypeStruct->GetName(), true))
			{
				return FNiagaraVariableBase(TypeDef, *FuncSpecVarName);
			}
		}
	}
	return FNiagaraVariableBase();
}


void SNiagaraFunctionSpecifierNDCVariablesSelector::Construct(const FArguments& InArgs)
{
	checkf(InArgs._WeakNodeToModify != nullptr, TEXT("Tried to construct change pin type menu without valid pin ptr!"));
	this->WeakNodeToModify = InArgs._WeakNodeToModify;
	this->AllowedTypes = InArgs._AllowedTypes;

	GenerateButtonWidget();

	SelectorButton = SNew(SComboButton)
		.ContentPadding(3)
		.MenuPlacement(MenuPlacement_BelowAnchor)
		.HasDownArrow(true)
		.ButtonStyle(FAppStyle::Get(), "Button")
		.ToolTipText(GetTooltipText())
		.OnGetMenuContent(this, &SNiagaraFunctionSpecifierNDCVariablesSelector::GetMenuContent)
		.ButtonContent()
		[
			ButtonContent.ToSharedRef()
		];

	ChildSlot
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoWidth()
			.Padding(5)
			[
				SNew(STextBlock)
				.Text(FText::FromName(TEXT("NDC Variable")))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoWidth()
			.Padding(5)
			[
				SelectorButton.ToSharedRef()
			]
		];
}

void SNiagaraFunctionSpecifierNDCVariablesSelector::OnMenuSelectionChanged(const FNiagaraVariableBase& Var)
{
	if (UNiagaraNodeFunctionCall* NodeToModify = WeakNodeToModify.Get())
	{
		EncodeVariableAsSpecifiers(Var, NodeToModify);
		GenerateButtonWidget();
	}
}

void SNiagaraFunctionSpecifierNDCVariablesSelector::GenerateButtonWidget()
{
	if (UNiagaraNodeFunctionCall* NodeToModify = WeakNodeToModify.Get())
	{
		FNiagaraVariableBase FuncSpecVariable = DecodeVariableAsSpecifiers(NodeToModify);
		FText			   IconToolTip = FuncSpecVariable.GetType().GetNameText();
		FSlateBrush const* IconBrush = FuncSpecVariable.GetType().IsStatic() ? FNiagaraEditorStyle::Get().GetBrush(TEXT("NiagaraEditor.StaticIcon")) : FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
		const FLinearColor TypeColor = UEdGraphSchema_Niagara::GetTypeColor(FuncSpecVariable.GetType());
		FSlateColor        IconColor = FSlateColor(TypeColor);
		FString			   IconDocLink, IconDocExcerpt;
		FSlateBrush const* SecondaryIconBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
		FSlateColor        SecondaryIconColor = IconColor;
		TSharedRef<SWidget> IconWidget = SNew(SScaleBox)
			[
				SNew(SNiagaraIconWidget)
				.IconToolTip(IconToolTip)
			.IconBrush(IconBrush)
			.IconColor(IconColor)
			.DocLink(IconDocLink)
			.DocExcerpt(IconDocExcerpt)
			.SecondaryIconBrush(SecondaryIconBrush)
			.SecondaryIconColor(SecondaryIconColor)
			];

		ButtonContent = SNew(SBox)
			.MinDesiredWidth(150.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoWidth()
			.Padding(0)
			[
				IconWidget
			]
		+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoWidth()
			.Padding(0)
			[
				SNew(STextBlock).Text(FText::FromName(FuncSpecVariable.GetName()))
			]
			];

		return;
	}

	ButtonContent = SNew(STextBlock).Text(LOCTEXT("NDCVarButtonError","Error"));
}

TSharedRef<SWidget> SNiagaraFunctionSpecifierNDCVariablesSelector::GetMenuContent()
{
	TSharedRef<SNiagaraFunctionSpecifierNDCVariablesMenu> MenuWidget = SNew(SNiagaraFunctionSpecifierNDCVariablesMenu)
			.Owner(StaticCastSharedRef<SNiagaraFunctionSpecifierNDCVariablesSelector>(AsShared()).ToSharedPtr())
			.AutoExpandMenu(false)
			.AllowedTypes(AllowedTypes);

	SelectorButton->SetMenuContentWidgetToFocus(MenuWidget->GetSearchBox());
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(1)
		[
			MenuWidget
		];
}

FText SNiagaraFunctionSpecifierNDCVariablesSelector::GetTooltipText() const
{
	return LOCTEXT("FunctionSpecifierNDCVariableSelectorTooltip", "Select an NDC Variable to use.");
}

#undef LOCTEXT_NAMESPACE /*"SNiagaraParameterMenu"*/
