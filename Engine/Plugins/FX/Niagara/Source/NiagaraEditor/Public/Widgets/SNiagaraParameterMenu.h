// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "NiagaraActions.h"
#include "Misc/Guid.h"
#include "NiagaraTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;
struct FNiagaraNamespaceMetadata;
class SEditableTextBox;
class SExpanderArrow;
class SGraphActionMenu;
class UEdGraphPin;
class UNiagaraGraph;
class UNiagaraNodeAssignment;
class UNiagaraNodeWithDynamicPins;
class UNiagaraParameterDefinitions;
class UNiagaraScriptVariable;
class SComboButton;


class SNiagaraParameterMenu : public SCompoundWidget
{
public:
	/** Delegate to get the name of a section if the widget is a section separator. */
	DECLARE_DELEGATE_RetVal_OneParam(FText, FGetSectionTitle, int32);

	SLATE_BEGIN_ARGS(SNiagaraParameterMenu)
		: _AutoExpandMenu(false)
	{}
		SLATE_ARGUMENT(bool, AutoExpandMenu)
		SLATE_EVENT(FGetSectionTitle, OnGetSectionTitle)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

	TSharedPtr<SEditableTextBox> GetSearchBox();

protected:
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) = 0;
	void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedActions, ESelectInfo::Type InSelectionType);

	static TSharedRef<SExpanderArrow> CreateCustomActionExpander(const struct FCustomExpanderData& ActionMenuData);
	static bool IsStaticSwitchParameter(const FNiagaraVariable& Variable, const TArray<UNiagaraGraph*>& Graphs);
	static FText GetNamespaceCategoryText(const FNiagaraNamespaceMetadata& NamespaceMetaData);
	static FText GetNamespaceCategoryText(const FGuid& InNamespaceId);

private:
	bool bAutoExpandMenu;
	TSharedPtr<SGraphActionMenu> GraphMenu;
};

class SNiagaraAddParameterFromPanelMenu : public SNiagaraParameterMenu
{
public:
	DECLARE_DELEGATE_OneParam(FOnParameterRequested, FNiagaraVariable);
	DECLARE_DELEGATE_OneParam(FOnAddScriptVar, const UNiagaraScriptVariable*);
	DECLARE_DELEGATE_OneParam(FOnAddParameterDefinitions, UNiagaraParameterDefinitions*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnAllowMakeType, const FNiagaraTypeDefinition&);

	SLATE_BEGIN_ARGS(SNiagaraAddParameterFromPanelMenu)
		: _NamespaceId(FGuid())
		, _AllowCreatingNew(true)
		, _ShowNamespaceCategory(true)
		, _ShowGraphParameters(true)
		, _AutoExpandMenu(false)
		, _IsParameterRead(true)
		, _ForceCollectEngineNamespaceParameterActions(false)
		, _CullParameterActionsAlreadyInGraph(false)
		, _AdditionalCulledParameterNames(TSet<FName>())
		, _AssignmentNode(nullptr)
	{}
		//~ Begin Required Events
		SLATE_EVENT(FOnParameterRequested, OnSpecificParameterRequested)
		SLATE_EVENT(FOnParameterRequested, OnNewParameterRequested)
		SLATE_EVENT(FOnAddScriptVar, OnAddScriptVar)
		SLATE_EVENT(FOnAddParameterDefinitions, OnAddParameterDefinitions)
		//~ End Required Events
		SLATE_EVENT(FOnAllowMakeType, OnAllowMakeType)
		//~ Begin Required Args
		SLATE_ARGUMENT(TArray<UNiagaraGraph*>, Graphs)
		SLATE_ARGUMENT(TArray<UNiagaraParameterDefinitions*>, AvailableParameterDefinitions)
		SLATE_ARGUMENT(TArray<UNiagaraParameterDefinitions*>, SubscribedParameterDefinitions)
		//~ End Required Args
		SLATE_ARGUMENT(FGuid, NamespaceId)
		SLATE_ARGUMENT(bool, AllowCreatingNew)
		SLATE_ARGUMENT(bool, ShowNamespaceCategory)
		SLATE_ARGUMENT(bool, ShowGraphParameters)
		SLATE_ARGUMENT(bool, AutoExpandMenu)
		SLATE_ARGUMENT(bool, IsParameterRead)
		SLATE_ARGUMENT(bool, ForceCollectEngineNamespaceParameterActions)
		SLATE_ARGUMENT(bool, CullParameterActionsAlreadyInGraph)
		SLATE_ARGUMENT(TSet<FName>, AdditionalCulledParameterNames)
		SLATE_ARGUMENT(UNiagaraNodeAssignment*, AssignmentNode)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

	void AddParameterGroup(
		FNiagaraMenuActionCollector& Collector,
		TArray<FNiagaraVariable>& Variables,
		const FGuid& InNamespaceId = FGuid(),
		const FText& Category = FText::GetEmpty(),
		int32 SortOrder = 0);

	void AddMakeNewGroup(
		FNiagaraMenuActionCollector& Collector,
		TArray<FNiagaraTypeDefinition>& TypeDefinitions,
		const FGuid& InNamespaceId = FGuid(),
		const FText& Category = FText::GetEmpty(),
		int32 SortOrder = 0,
		const FString& RootCategory = FString());

	void CollectParameterCollectionsActions(FNiagaraMenuActionCollector& Collector);
	void CollectMakeNew(FNiagaraMenuActionCollector& Collector, const FGuid& InNamespaceId);

protected:
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;

private:
	void ParameterSelected(FNiagaraVariable NewVariable);
	void ParameterSelected(FNiagaraVariable NewVariable, const FGuid InNamespaceId);
	void NewParameterSelected(FNiagaraTypeDefinition NewParameterType, const FGuid InNamespaceId);
	
	void ScriptVarFromParameterDefinitionsSelected(const UNiagaraScriptVariable* NewScriptVar, UNiagaraParameterDefinitions* SourceParameterDefinitions);
	TSet<FName> GetAllGraphParameterNames() const;

	static FText GetSectionTitle(int32 SectionId);

private:
	/** Delegate to handle adding an existing parameter as an FNiagaraVariable. */
	FOnParameterRequested OnSpecificParameterRequested;

	/** Delegate to handle adding a parameter as an FNiagaraVariable. */
	FOnParameterRequested OnNewParameterRequested;

	/** Delegate to handle adding a parameter as a UNiagaraScriptVariable. */
	FOnAddScriptVar OnAddScriptVar;

	/** Delegate to handle adding a UNiagaraParameterDefinitions. */
	FOnAddParameterDefinitions OnAddParameterDefinitions;

	/** Delegate to filter FNiagaraTypeDefinitions allowed when creating new parameters. */
	FOnAllowMakeType OnAllowMakeType;

	TArray<UNiagaraGraph*> Graphs;
	TArray<UNiagaraParameterDefinitions*> AvailableParameterDefinitions;
	TArray<UNiagaraParameterDefinitions*> SubscribedParameterDefinitions;
	FGuid NamespaceId;
	bool bAllowCreatingNew;
	bool bShowNamespaceCategory;
	bool bShowGraphParameters;
	bool bIsParameterReadNode;
	bool bOnlyShowParametersInNamespaceId;
	bool bForceCollectEngineNamespaceParameterActions;
	bool bCullParameterActionsAlreadyInGraph;
	TSet<FName> AdditionalCulledParameterNames;
	UNiagaraNodeAssignment* AssignmentNode;
};

class SNiagaraAddParameterFromPinMenu : public SNiagaraParameterMenu
{
public:
	SLATE_BEGIN_ARGS(SNiagaraAddParameterFromPinMenu)
		: _AutoExpandMenu(false)
	{}
		//~ Begin Required Args
		SLATE_ARGUMENT(UNiagaraNodeWithDynamicPins*, NiagaraNode)
		SLATE_ARGUMENT(UEdGraphPin*, AddPin)
		//~ End Required Args
		SLATE_ARGUMENT(bool, AutoExpandMenu)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

protected:
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;

private:
	UNiagaraNodeWithDynamicPins* NiagaraNode;
	UEdGraphPin* AddPin;
	bool bIsParameterReadNode;
};

class SNiagaraChangePinTypeMenu : public SNiagaraParameterMenu
{
public:
	//DECLARE_DELEGATE_RetVal_OneParam(bool, FOnAllowMakeType, const FNiagaraTypeDefinition&);

	SLATE_BEGIN_ARGS(SNiagaraChangePinTypeMenu)
		: _AutoExpandMenu(false)
	{}
		//~ Begin Required Args
		SLATE_ARGUMENT(UEdGraphPin*, PinToModify)
		//~ End Required Args
		SLATE_ARGUMENT(bool, AutoExpandMenu)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

protected:
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;

private:
	UEdGraphPin* PinToModify;
};

class SNiagaraFunctionSpecifierNDCVariablesSelector;
class SNiagaraFunctionSpecifierNDCVariablesMenu : public SNiagaraParameterMenu
{
public:

	SLATE_BEGIN_ARGS(SNiagaraFunctionSpecifierNDCVariablesMenu)
		: _AutoExpandMenu(false)
	{}
		//~ Begin Required Args
		SLATE_ARGUMENT(TSharedPtr<SNiagaraFunctionSpecifierNDCVariablesSelector>, Owner)
		SLATE_ARGUMENT(TArray<FNiagaraTypeDefinition>, AllowedTypes)
		//~ End Required Args
		SLATE_ARGUMENT(bool, AutoExpandMenu)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);

	void OnSelectionChanged(FNiagaraVariableBase Var);

protected:
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;

	TSharedPtr<SNiagaraFunctionSpecifierNDCVariablesSelector> Owner = nullptr;
	TArray<FNiagaraTypeDefinition> AllowedTypes;
};

class SNiagaraFunctionSpecifierNDCVariablesSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraFunctionSpecifierNDCVariablesSelector)
		: _WeakNodeToModify(nullptr)		
	{}
	SLATE_ARGUMENT(TWeakObjectPtr<UNiagaraNodeFunctionCall>, WeakNodeToModify)
	SLATE_ARGUMENT(TArray<FNiagaraTypeDefinition>, AllowedTypes)
	//~ End Required Args
	SLATE_END_ARGS()
public:
	NIAGARAEDITOR_API void Construct(const FArguments& InArgs);

	void OnMenuSelectionChanged(const FNiagaraVariableBase& NewSelection);

protected:
	NIAGARAEDITOR_API virtual TSharedRef<SWidget>	GetMenuContent();
	NIAGARAEDITOR_API FText GetTooltipText() const;
	NIAGARAEDITOR_API FText GetSelectionText() const;

	NIAGARAEDITOR_API void GenerateButtonWidget();

private:
	TWeakObjectPtr<UNiagaraNodeFunctionCall> WeakNodeToModify;
	TArray<FNiagaraTypeDefinition> AllowedTypes;
	TSharedPtr<SComboButton> SelectorButton;
	TSharedPtr<SWidget> ButtonContent;
};