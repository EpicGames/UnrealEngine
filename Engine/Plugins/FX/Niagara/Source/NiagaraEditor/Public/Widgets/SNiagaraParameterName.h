// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Framework/SlateDelegates.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "NiagaraEditorStyle.h"

class SInlineEditableTextBlock;
class SBorder;
class UEdGraphPin;

enum class ENiagaraParameterDecoratorLocation
{
	BeforeName,
	AfterName,

	Default = AfterName
};

class SNiagaraParameterName : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnNameChanged, FName /* InNewName */);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnVerifyNameChange, FName /* InNewName */, FText& /* OutErrorMessage */)

public: 
	enum class ESingleNameDisplayMode
	{
		Namespace,
		Name
	};
public:
	SLATE_BEGIN_ARGS(SNiagaraParameterName) 
		: _EditableTextStyle(&FNiagaraEditorStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("NiagaraEditor.ParameterInlineEditableText"))
		, _ReadOnlyTextStyle(&FNiagaraEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("NiagaraEditor.ParameterText"))
		, _IsReadOnly(false)
		, _SingleNameDisplayMode(ESingleNameDisplayMode::Name)
		, _DecoratorLocation(ENiagaraParameterDecoratorLocation::Default)
		, _DecoratorHAlign(HAlign_Left)
		, _DecoratorPadding(5.0f, 0.0f, 0.0f, 0.0f)
	{
		_Clipping = EWidgetClipping::OnDemand;
	}
		SLATE_STYLE_ARGUMENT(FInlineEditableTextBlockStyle, EditableTextStyle)
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, ReadOnlyTextStyle)
		SLATE_ATTRIBUTE(FName, ParameterName)
		SLATE_ARGUMENT(bool, IsReadOnly)
		SLATE_ARGUMENT(ESingleNameDisplayMode, SingleNameDisplayMode)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_EVENT(FOnVerifyNameChange, OnVerifyNameChange)
		SLATE_EVENT(FOnNameChanged, OnNameChanged)
		SLATE_EVENT(FIsSelected, IsSelected)
		SLATE_EVENT(FPointerEventHandler, OnDoubleClicked)
		SLATE_ARGUMENT(ENiagaraParameterDecoratorLocation, DecoratorLocation)
		SLATE_ARGUMENT(EHorizontalAlignment, DecoratorHAlign)
		SLATE_ARGUMENT(FMargin, DecoratorPadding)
		SLATE_NAMED_SLOT(FArguments, Decorator)
	SLATE_END_ARGS()

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs);

	NIAGARAEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	NIAGARAEDITOR_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	NIAGARAEDITOR_API void EnterEditingMode();

	NIAGARAEDITOR_API void EnterNamespaceModifierEditingMode();

	NIAGARAEDITOR_API void UpdateDecorator(TSharedRef<SWidget> InDecorator);
	
private:
	TSharedRef<SBorder> CreateNamespaceWidget(FText NamespaceDisplayName, FText NamespaceDescription, FLinearColor NamespaceBorderColor, FName NamespaceForegroundStyle);

	void UpdateContent(FName InDisplayedParameterName, int32 InEditableNamespaceModifierIndex = INDEX_NONE);

	FName ReconstructNameFromEditText(const FText& InEditText);

	bool VerifyNameTextChange(const FText& InNewName, FText& OutErrorMessage, FName InOriginalName);

	void NameTextCommitted(const FText& InNewNameText, ETextCommit::Type CommitType);

	bool VerifyNamespaceModifierTextChange(const FText& InNewNamespaceModifier, FText& OutErrorMessage, FName InOriginalNamespaceModifier);

	void NamespaceModifierTextCommitted(const FText& InNewNameText, ETextCommit::Type CommitType);

private:
	const FInlineEditableTextBlockStyle* EditableTextStyle;
	const FTextBlockStyle* ReadOnlyTextStyle;

	TAttribute<FName> ParameterName;
	bool bIsReadOnly;
	ESingleNameDisplayMode SingleNameDisplayMode;
	TAttribute<FText> HighlightText;

	FOnVerifyNameChange OnVerifyNameChangeDelegate;
	FOnNameChanged OnNameChangedDelegate;
	FPointerEventHandler OnDoubleClickedDelegate;

	FIsSelected IsSelected;
	FName DisplayedParameterName;
	TSharedPtr<SInlineEditableTextBlock> EditableNameTextBlock;
	TSharedPtr<SInlineEditableTextBlock> EditableModifierTextBlock;
	bool bModifierIsPendingEdit;

	ENiagaraParameterDecoratorLocation DecoratorLocation = ENiagaraParameterDecoratorLocation::Default;
	EHorizontalAlignment DecoratorHAlign;
	FMargin DecoratorPadding;
	TSharedPtr<SWidget> Decorator;
};

class SNiagaraParameterNameTextBlock : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnDragDetectedHandler, const FGeometry&, const FPointerEvent&);
	
	SLATE_BEGIN_ARGS(SNiagaraParameterNameTextBlock)
		: _EditableTextStyle(&FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle"))
		, _ReadOnlyTextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
		, _IsReadOnly(false)
		, _DecoratorLocation(ENiagaraParameterDecoratorLocation::Default)
		, _DecoratorHAlign(HAlign_Left)
		, _DecoratorPadding(5.0f, 0.0f, 0.0f, 0.0f)
	{
		_Clipping = EWidgetClipping::OnDemand;
	}
		SLATE_STYLE_ARGUMENT(FInlineEditableTextBlockStyle, EditableTextStyle)
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, ReadOnlyTextStyle)
		SLATE_ATTRIBUTE(FText, ParameterText)
		SLATE_ARGUMENT(bool, IsReadOnly)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_EVENT(FOnVerifyTextChanged, OnVerifyTextChanged)
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)
		SLATE_EVENT(FOnDragDetectedHandler, OnDragDetected)
		SLATE_EVENT(FIsSelected, IsSelected)
		SLATE_ARGUMENT(ENiagaraParameterDecoratorLocation, DecoratorLocation)
		SLATE_ARGUMENT(EHorizontalAlignment, DecoratorHAlign)
		SLATE_ARGUMENT(FMargin, DecoratorPadding)
		SLATE_NAMED_SLOT(FArguments, Decorator)
	SLATE_END_ARGS()

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs);

	NIAGARAEDITOR_API void EnterEditingMode();

	NIAGARAEDITOR_API void EnterNamespaceModifierEditingMode();

private:
	FName GetParameterName() const;

	bool VerifyNameChange(FName InNewName, FText& OutErrorMessage);

	void NameChanged(FName InNewName);

	NIAGARAEDITOR_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	NIAGARAEDITOR_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

private:
	TAttribute<FText> ParameterText;
	FOnVerifyTextChanged OnVerifyNameTextChangedDelegate;
	FOnTextCommitted OnNameTextCommittedDelegate;
	FOnDragDetectedHandler OnDragDetectedHandlerDelegate;
	
	mutable FText DisplayedParameterTextCache;
	mutable FName ParameterNameCache;
	TSharedPtr<SNiagaraParameterName> ParameterName;
};

class SNiagaraParameterNamePinLabel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraParameterNamePinLabel)
		: _EditableTextStyle(&FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle"))
		, _IsReadOnly(false)
		, _DecoratorLocation(ENiagaraParameterDecoratorLocation::Default)
		, _DecoratorHAlign(HAlign_Left)
		, _DecoratorPadding(5.0f, 0.0f, 0.0f, 0.0f)
	{
		_Clipping = EWidgetClipping::OnDemand;
	}
		SLATE_STYLE_ARGUMENT(FInlineEditableTextBlockStyle, EditableTextStyle)
		SLATE_ATTRIBUTE(FText, ParameterText)
		SLATE_ARGUMENT(bool, IsReadOnly)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_EVENT(FOnVerifyTextChanged, OnVerifyTextChanged)
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)
		SLATE_EVENT(FIsSelected, IsSelected)
		SLATE_ARGUMENT(ENiagaraParameterDecoratorLocation, DecoratorLocation)
		SLATE_ARGUMENT(EHorizontalAlignment, DecoratorHAlign)
		SLATE_ARGUMENT(FMargin, DecoratorPadding)
		SLATE_NAMED_SLOT(FArguments, Decorator)
	SLATE_END_ARGS()

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, UEdGraphPin* InTargetPin);

	NIAGARAEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	NIAGARAEDITOR_API virtual FSlateColor GetForegroundColor() const override;

	NIAGARAEDITOR_API void EnterEditingMode();
private:
	UEdGraphPin* TargetPin;
	TSharedPtr<SNiagaraParameterNameTextBlock> ParameterNameTextBlock;
};
