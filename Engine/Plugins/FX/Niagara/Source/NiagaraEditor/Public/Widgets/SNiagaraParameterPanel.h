// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Internationalization/Text.h"
#include "NiagaraActions.h"
#include "NiagaraParameterPanelTypes.h"
#include "NiagaraTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SItemSelector.h"

class FUICommandList;
class INiagaraParameterPanelViewModel;
class SComboButton;
class SExpanderArrow;
class SGraphActionMenu;
class SSearchBox;
struct FGuid;

typedef SItemSelector<FNiagaraParameterPanelCategory /*CategoryType*/, FNiagaraParameterPanelItem /*ItemType*/, FNiagaraParameterPanelCategory /*SectionType*/, FGuid /*CategoryKeyType*/, FNiagaraVariableBase /*ItemKeyType*/> SNiagaraParameterPanelSelector;

class FNiagaraParameterPanelCommands : public TCommands<FNiagaraParameterPanelCommands>
{
public:
	/** Constructor */
	FNiagaraParameterPanelCommands()
		: TCommands<FNiagaraParameterPanelCommands>(TEXT("NiagaraParameterMapViewCommands"), NSLOCTEXT("Contexts", "NiagaraParameterPanel", "NiagaraParameterPanel"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	// Basic operations
	TSharedPtr<FUICommandInfo> DeleteItem;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

/** A widget for viewing and editing UNiagaraScriptVariables provided by an INiagaraParameterPanelViewModel */
class SNiagaraParameterPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraParameterPanel)
		: _ShowParameterSynchronizingWithLibraryIcon(true)
		, _ShowParameterSynchronizingWithLibraryIconExternallyReferenced(true)
		, _ShowParameterReferenceCounter(true)
	{}
		SLATE_ARGUMENT(bool, ShowParameterSynchronizingWithLibraryIcon)
		SLATE_ARGUMENT(bool, ShowParameterSynchronizingWithLibraryIconExternallyReferenced)
		SLATE_ARGUMENT(bool, ShowParameterReferenceCounter)
		SLATE_NAMED_SLOT(FArguments, SearchAdjacentWidget)
	SLATE_END_ARGS();

	NIAGARAEDITOR_API ~SNiagaraParameterPanel();

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, const TSharedPtr<INiagaraParameterPanelViewModel>& InParameterPanelViewModel, const TSharedPtr<FUICommandList>& InToolkitCommands);

	//~ Begin SItemSelector
	TArray<FNiagaraParameterPanelCategory> OnGetCategoriesForItem(const FNiagaraParameterPanelItem& Item);
	bool OnCompareCategoriesForEquality(const FNiagaraParameterPanelCategory& CategoryA, const FNiagaraParameterPanelCategory& CategoryB) const;
	bool OnCompareCategoriesForSorting(const FNiagaraParameterPanelCategory& CategoryA, const FNiagaraParameterPanelCategory& CategoryB) const;
	const FGuid& OnGetKeyForCategory(const FNiagaraParameterPanelCategory& Category) const;
	void OnCategoryExpansionChanged(const FNiagaraParameterPanelCategory& Category, bool bIsExpanded);
	bool OnCompareItemsForEquality(const FNiagaraParameterPanelItem& ItemA, const FNiagaraParameterPanelItem& ItemB) const;
	bool OnCompareItemsForSorting(const FNiagaraParameterPanelItem& ItemA, const FNiagaraParameterPanelItem& ItemB) const;
	const FNiagaraVariableBase& OnGetKeyForItem(const FNiagaraParameterPanelItem& Item) const;
	bool OnDoesItemMatchFilterText(const FText& FilterText, const FNiagaraParameterPanelItem& Item);
	TSharedRef<SWidget> OnGenerateWidgetForCategory(const FNiagaraParameterPanelCategory& Category);
	TSharedRef<SWidget> OnGenerateWidgetForItem(const FNiagaraParameterPanelItem& Item);
	TSharedPtr<SWidget> OnContextMenuOpening();
	const TArray<FNiagaraParameterPanelCategory>& GetDefaultCategories() const;
	//~ End SItemSelector

	NIAGARAEDITOR_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Refreshes the items for the item selector. */
	void Refresh(bool bRunCategoryExpansionFilter = false);
	void RefreshNextTick(bool bRunCategoryExpansionFilter = false);

	void OnParameterItemSelected(const FNiagaraParameterPanelItem& SelectedItem, ESelectInfo::Type SelectInfo) const;
	FReply OnParameterItemsDragged(const TArray<FNiagaraParameterPanelItem>& DraggedItems, const FPointerEvent& MouseEvent) const;
	void OnParameterItemActived(const FNiagaraParameterPanelItem& ActivatedItem) const;

	static TSharedRef<SExpanderArrow> CreateCustomActionExpander(const struct FCustomExpanderData& ActionMenuData);

private:
	void SelectParameterItemByName(const FName ParameterName) const;

	void AddParameterPendingRename(const FName ParameterName);

	void AddParameterPendingNamespaceModifierRename(const FName ParameterName);

	TSharedPtr<TArray<FName>> GetParametersWithNamespaceModifierRenamePending();

	/** Call per item delegates to enter rename mode. Implemented as a separate method to allow waiting for SItemSelector refresh to complete and delegates to be re-bound. */
	void ProcessParameterItemsPendingChange();

	TSharedRef<SWidget> CreateAddToCategoryButton(const FNiagaraParameterPanelCategory& Category, FText AddNewText, FName MetaDataTag);

	TSharedRef<SWidget> OnGetParameterMenu(FNiagaraParameterPanelCategory Category);

	bool GetCanAddParametersToCategory(FNiagaraParameterPanelCategory Category) const;

	void DeleteSelectedItems() const;
	bool CanDeleteSelectedItems() const;

	void RequestRenameSelectedItem() const;
	bool CanRequestRenameSelectedItem() const;

	void CopyParameterReference() const;
	bool CanCopyParameterReference() const;
	FText GetCopyParameterReferenceToolTip() const;

	void OnParameterNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, const FNiagaraParameterPanelItem ItemToBeRenamed) const;

	bool OnParameterNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage, const FNiagaraParameterPanelItem ItemToBeRenamed) const;

	void HandleExternalSelectionChanged(const UObject* Obj);

	const FSlateBrush* GetCategoryBackgroundImage(bool bIsCategoryHovered, bool bIsCategoryExpanded) const;
	bool GetCategoryExpandedInitially(const FNiagaraParameterPanelCategory& Category) const;

	TSharedRef<SWidget> GetViewOptionsMenu();

	FReply HandleDragDropOperation(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);
	
	bool GetCanHandleDragDropOperation(TSharedPtr<FDragDropOperation> DragDropOperation);

	static const FSlateBrush* GetViewOptionsBorderBrush();

	void ConstructSectionButtons();

	ECheckBoxState GetSectionCheckState(FText Section) const;
	bool GetSectionEnabled(FText Section) const;

	void OnSectionChecked(ECheckBoxState CheckState, FText Section);

	FText GetTooltipForSection(FText Section);
private:
	mutable bool bPendingRefresh;
	mutable bool bPendingSelectionRestore;
	mutable bool bParameterItemsPendingChange;
	mutable bool bRunCategoryExpansionFilter = false;

	/** Tracking list for parameters that are awaiting entering selection. */
	TArray<FName> ParametersWithSelectionPending;

	/** Tracking list for parameters that are awaiting entering rename mode. */
	TArray<FName> ParametersWithRenamePending;

	/** Tracking list for parameters that are awaiting entering namespace modifier edit mode. */
	TSharedPtr<TArray<FName>> ParametersWithNamespaceModifierRenamePending;

	/** Map of categories to buttons for setting selection on context menus when summoning them via the SComboButton. */
	TMap<FNiagaraNamespaceMetadata, TSharedPtr<SComboButton>> CategoryToButtonMap;

	TSharedPtr<FUICommandList> ToolkitCommands;

	TSharedPtr<INiagaraParameterPanelViewModel> ParameterPanelViewModel;
	TSharedPtr<SNiagaraParameterPanelSelector> ItemSelector;
	TSharedPtr<SSearchBox> ItemSelectorSearchBox;
	TSharedPtr<class SWrapBox> SectionSelectorBox;

	/** Whether or not to display icons signifying whether parameters in the panel are synchronizing with a subscribed parameter definition. */
	bool bShowParameterSynchronizingWithLibraryIcon;

	/** Whether or not to display icons signifying whether parameters in the panel are synchronizing with a subscribed parameter definition if that parameter is also externally referenced. */
	bool bShowParameterSynchronizingWithLibraryIconExternallyReferenced;

	/** Whether or not to display the reference counter for each parameter entry. */
	bool bShowParameterReferenceCounter;

	SNiagaraParameterPanelSelector::FOnCategoryPassesFilter FilterCategoryExpandedDelegate;

};
