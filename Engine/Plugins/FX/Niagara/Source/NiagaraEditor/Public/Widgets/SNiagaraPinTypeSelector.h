// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"
#include "Input/Reply.h"
#include "Misc/Attribute.h"
#include "NiagaraTypes.h"
#include "SGraphPalette.h"
#include "SlateFwd.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

class SNiagaraPinTypeSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraPinTypeSelector)
	{}
	SLATE_END_ARGS()
public:
	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, UEdGraphPin * InGraphPin);

protected:
	NIAGARAEDITOR_API virtual TSharedRef<SWidget>	GetMenuContent();
	NIAGARAEDITOR_API FText GetTooltipText() const;

private:
	UEdGraphPin* Pin = nullptr;
	TSharedPtr<SComboButton> SelectorButton;
};

class SNiagaraIconWidget : public SGraphPaletteItem
{
public:

	SLATE_BEGIN_ARGS(SNiagaraIconWidget)
	{}
		SLATE_ARGUMENT(FText, IconToolTip)
		SLATE_ARGUMENT(const FSlateBrush*, IconBrush)
		SLATE_ARGUMENT(FSlateColor, IconColor)
		SLATE_ARGUMENT(FString, DocLink)
		SLATE_ARGUMENT(FString, DocExcerpt)
		SLATE_ARGUMENT(const FSlateBrush*, SecondaryIconBrush)
		SLATE_ARGUMENT(FSlateColor, SecondaryIconColor)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			CreateIconWidget(
				InArgs._IconToolTip
				, InArgs._IconBrush
				, InArgs._IconColor
				, InArgs._DocLink
				, InArgs._DocExcerpt
				, InArgs._SecondaryIconBrush
				, InArgs._SecondaryIconColor
			)
		];
	}
};
