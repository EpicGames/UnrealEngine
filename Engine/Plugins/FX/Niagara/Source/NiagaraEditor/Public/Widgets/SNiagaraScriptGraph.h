// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GraphEditor.h"
#include "EdGraph/EdGraphNode.h"
#include "Widgets/Input/SSearchBox.h"
#include "NiagaraEditorCommon.h"
#include "Styling/AppStyle.h"

class FNiagaraScriptGraphViewModel;

/** A widget for editing a UNiagaraScript with a graph. */
class SNiagaraScriptGraph : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraScriptGraph)
		: _ForegroundColor(FAppStyle::GetColor("Graph.ForegroundColor"))
		, _IsEditable(true)
		, _DisplayAsReadOnly(false)
		, _ZoomToFitOnLoad(false)
		, _ShowHeader(true)
	{}
		/** The text displayed in the title bar of the graph. */
		SLATE_ATTRIBUTE(FText, GraphTitle)
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)
		SLATE_ATTRIBUTE(bool, IsEditable)
		SLATE_ATTRIBUTE(bool, DisplayAsReadOnly)
		SLATE_ARGUMENT(bool, ZoomToFitOnLoad)
		SLATE_ARGUMENT(bool, ShowHeader)
	SLATE_END_ARGS();

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, TSharedRef<FNiagaraScriptGraphViewModel> InViewModel, const FAssetData& InEditedAsset = FAssetData());

	NIAGARAEDITOR_API virtual ~SNiagaraScriptGraph() override;

	const TSharedPtr<SGraphEditor> GetGraphEditor() { return GraphEditor; };

	NIAGARAEDITOR_API void FocusGraphElement(const INiagaraScriptGraphFocusInfo* FocusInfo);

	void FocusGraphSearchBox();

	void OnCreateComment();

	void UpdateViewModel(TSharedRef<FNiagaraScriptGraphViewModel> InNewModel);

	void RecreateGraphWidget();

	TSharedPtr<FNiagaraScriptGraphViewModel> GetViewModel() { return ViewModel; };

private:
	/** Constructs the graph editor widget for the current graph. */
	TSharedRef<SGraphEditor> ConstructGraphEditor();

	/** Called whenever the selected nodes on the script view model changes. */
	void ViewModelSelectedNodesChanged();

	/** Called whenever the selected nodes in the graph editor changes. */
	void GraphEditorSelectedNodesChanged(const TSet<UObject*>& SelectedNodes);

	/** Called when a node is double clicked. */
	void OnNodeDoubleClicked(UEdGraphNode* ClickedNode);

	/** Called when nodes are based in the script view model. */
	void NodesPasted(const TSet<UEdGraphNode*>& PastedNodes);

	/** Sets the position on a group of newly pasted nodes. */
	void PositionPastedNodes(const TSet<UEdGraphNode*>& PastedNodes);

	/** Called whenever the view model's graph changes to a different graph. */
	void GraphChanged();

	/** Called whenever a user edits the name inline of a node.*/
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/** Called whenever a user left-clicks on the graph with a chord pressed.*/
	FReply OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition);

	FActionMenuContent OnCreateActionMenu(UEdGraph* Graph, const FVector2f& Position, const TArray<UEdGraphPin*>& DraggedPins, bool bAutoExpandActionMenu, SGraphEditor::FActionMenuClosed OnClosed);	
	
	/** Called whenever a user is trying to edit the name inline of a node and we want to make sure that it is valid.*/
	bool OnVerifyNodeTextCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage);

	void OnSearchTextChanged(const FText& SearchText);

	void OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	void OnSearchBoxSearch(SSearchBox::SearchDirection Direction);

	TOptional<SSearchBox::FSearchResultData> GetSearchResultData() const;

	int GetCurrentFocusedMatchIndex() const { return CurrentFocusedSearchMatchIndex; }

	FText GetCurrentSearchText() const { return CurrentSearchText; };

	EVisibility GetGraphSearchBoxVisibility() const { return bGraphSearchBoxActive ? EVisibility::Visible : EVisibility::Collapsed; };
	FText GetScriptSubheaderText() const;
	EVisibility GetScriptSubheaderVisibility() const;
	FLinearColor GetScriptSubheaderColor() const;
	
	int32 GetScriptAffectedAssets() const;
	void ResetAssetCount(const FAssetData& InAssetData);
	void AddAssetListeners();
	void ClearListeners();

	FReply CloseGraphSearchBoxPressed();

	FReply HandleGraphSearchBoxKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	void OnAlignTop();
	void OnAlignMiddle();
	void OnAlignBottom();
	void OnAlignLeft();
	void OnAlignCenter();
	void OnAlignRight();

	void OnStraightenConnections();

	void OnDistributeNodesH();
	void OnDistributeNodesV();

	void RebuildCommands();
private:
	/** The combined commands used by the graph editor. */
	TSharedPtr<FUICommandList> Commands;

	/** An attribute for the title text of the graph. */
	TAttribute<FText> GraphTitle;

	/** The view model which exposes the data used by the widget. */
	TSharedPtr<FNiagaraScriptGraphViewModel> ViewModel;

	/** The graph editor which is editing the script graph. */
	TSharedPtr<SGraphEditor> GraphEditor;

	/** The search box for searching the script graph. */
	TSharedPtr<SSearchBox> SearchBox;

	/** Flag to prevent modifying the view model selection when updating the graph
		editor selection due to a view model selection change. */
	bool bUpdatingGraphSelectionFromViewModel;

	// ~Search stuff
	FText CurrentSearchText;
	int CurrentFocusedSearchMatchIndex;
	TArray<TSharedPtr<INiagaraScriptGraphFocusInfo>> CurrentSearchResults;
	bool bGraphSearchBoxActive;
	bool bShowHeader;
	
	TAttribute<bool> IsEditable;
	TAttribute<bool> DisplayAsReadOnly;
	
	mutable TOptional<int32> ScriptAffectedAssets;
	FAssetData EditedAsset;

private:
	// action menu data
	UEdGraph* GraphObj;
	TArray<UEdGraphPin*> DraggedFromPins;
	FVector2D NewNodePosition;
	bool AutoExpandActionMenu;

	SGraphEditor::FActionMenuClosed OnClosedCallback;
};
