// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraGraph.h"
#include "NiagaraScriptVariable.h"
#include "DataHierarchyViewModelBase.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STreeView.h"

#define UE_API NIAGARAEDITOR_API

class UNiagaraHierarchyScriptParameter;
class FNiagaraObjectSelection;
class FNiagaraScriptToolkit;
class SComboButton;

class SNiagaraScriptInputPreviewPanel : public SCompoundWidget, public FGCObject, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SNiagaraScriptInputPreviewPanel)
		{
		}

	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs, TSharedRef<FNiagaraScriptToolkit> InScriptToolkit, TSharedRef<FNiagaraObjectSelection> InVariableObjectSelection);
	UE_API virtual ~SNiagaraScriptInputPreviewPanel() override;
	
	UE_API void Refresh();
	void RefreshOnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload) { Refresh(); }
	
	UE_API void SetupDelegates();
	UE_API void RemoveDelegates();
private:
	UE_API TSharedRef<ITableRow> OnGenerateRow(UHierarchyElement* Item, const TSharedRef<STableViewBase>& TableViewBase) const;
	UE_API void OnGetChildren(UHierarchyElement* Item, TArray<UHierarchyElement*>& OutChildren) const;
	UE_API void OnParametersChanged(TOptional<TInstancedStruct<FNiagaraParametersChangedData>> ParametersChangedData);

	UE_API FReply SummonHierarchyEditor() const;

	struct FSearchItem
	{
		TArray<UHierarchyElement*> Path;

		UHierarchyElement* GetEntry() const
		{
			return Path.Num() > 0 ? 
				Path[Path.Num() - 1] : 
				nullptr;
		}

		bool operator==(const FSearchItem& Item) const
		{
			return Path == Item.Path;
		}
	};

	UE_API void OnSearchTextChanged(const FText& Text);
	UE_API void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);
	UE_API void OnSearchButtonClicked(SSearchBox::SearchDirection SearchDirection);
	UE_API void GenerateSearchItems(UHierarchyElement* Root, TArray<UHierarchyElement*> ParentChain, TArray<FSearchItem>& OutSearchItems);
	UE_API void ExpandSourceSearchResults();
	UE_API void SelectNextSourceSearchResult();
	UE_API void SelectPreviousSourceSearchResult();
	UE_API TOptional<SSearchBox::FSearchResultData> GetSearchResultData() const;

	/** FGCObject */
	UE_API virtual FString GetReferencerName() const override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	/** FSelfRegisteringEditorUndoClient */
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;
private:
	TArray<UHierarchyElement*> RootArray;
	TWeakPtr<FNiagaraScriptToolkit> ScriptToolkit;
	TWeakPtr<FNiagaraObjectSelection> VariableObjectSelection;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<STreeView<UHierarchyElement*>> TreeView;

	/** We construct and maintain an array of parameters _not_ in the hierarchy to ensure we display all parameters without requiring hierarchy setup. */
	TArray<TObjectPtr<UNiagaraHierarchyScriptParameter>> TransientLeftoverParameters; 
	
	TArray<FSearchItem> SearchResults;
	TOptional<FSearchItem> FocusedSearchResult;
	TSharedPtr<SComboButton> AddParameterButton;
};

#undef UE_API
