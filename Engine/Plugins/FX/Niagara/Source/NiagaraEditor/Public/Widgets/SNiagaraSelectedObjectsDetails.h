// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Delegates/Delegate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "PropertyEditorDelegates.h"
#include "EditorUndoClient.h"

class FNiagaraObjectSelection;
class IDetailsView;
class UNiagaraScriptVariable;

/** A widget for viewing and editing a set of selected objects with a details panel. */
class SNiagaraSelectedObjectsDetails : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SNiagaraSelectedObjectsDetails)
		: _AllowEditingLibraryScriptVariables(false)
	{}
		SLATE_ARGUMENT(bool, AllowEditingLibraryScriptVariables)
	SLATE_END_ARGS();

	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, TSharedRef<FNiagaraObjectSelection> InSelectedObjects);
	NIAGARAEDITOR_API void Construct(const FArguments& InArgs, TSharedRef<FNiagaraObjectSelection> InSelectedObjects, TSharedRef<FNiagaraObjectSelection> InSelectedObjects2);

	//~ Begin FEditorUndoClient Interface 
	NIAGARAEDITOR_API virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); };
	//~ End FEditorUndoClient Interface 

	/** Delegate to know when one of the properties has been changed.*/
	FOnFinishedChangingProperties& OnFinishedChangingProperties() { return OnFinishedChangingPropertiesDelegate; }

	void SelectedObjectsChanged();

	// Fully regenerates the details view.
	void RefreshDetails();
private:
	/** Called whenever the object selection changes. */
	void SelectedObjectsChangedFirst();
	void SelectedObjectsChangedSecond();

	/** Internal delegate to route to third parties. */
	void OnDetailsPanelFinishedChangingProperties(const FPropertyChangedEvent& InEvent);

	/** Whether or not to enable editing for the entire details panel. */
	bool DetailsPanelIsEnabled() const;

	/** Whether or not to enable editing for a given property. */
	bool PropertyIsReadOnly(const FPropertyAndParent& PropertyAndParent) const;

	/** Whether or not to enable editing for a given custom row. */
	bool CustomRowIsReadOnly(const FName InRowName, const FName InParentName) const;

	/** Check whether SelectedObjects contains a UNiagaraScriptVariable that is synchronizing with and/or from a UNiagaraParameterDefinitions and set flags owned by this widget accordingly. */
	void UpdateSelectedObjectInfoFlags(const TSharedPtr<FNiagaraObjectSelection>& SelectedObjects);

	/** Convenience wrapper to get a single selected UNiagaraScriptVariable. Assumes there is only one UNiagaraScriptVariable selected. */
	const UNiagaraScriptVariable* GetSelectedScriptVar() const;
private:
	/** The selected objects being viewed and edited by this widget. */
	TArray<TSharedPtr<FNiagaraObjectSelection>> SelectedObjectsArray;

	/** The details view for the selected object. */
	TSharedPtr<IDetailsView> DetailsView;

	/** Delegate for third parties to be notified when properties have changed.*/
	FOnFinishedChangingProperties OnFinishedChangingPropertiesDelegate;

	/** Whether or not to allow editing a UNiagaraScriptVariable owned by a UNiagaraParameterDefinitions.*/
	bool bAllowEditingLibraryOwnedScriptVars;

	/** Flag to notify if a UNiagaraScriptVariable synchronizing with a UNiagaraParameterDefinitions is being viewed. Used to selectively disable editing. */
	bool bViewingLibrarySubscribedScriptVar;

	/** Flag to notify if a UNiagaraScriptVariable owned by a UNiagaraParameterDefinitions is being viewed. Used to selectively disable editing. */
	bool bViewingLibraryOwnedScriptVar;

	/** Index of the SelectedObjectsArray that was last set. Used to track which SelectedObjects to reset on refresh. */
	int32 LastSetSelectedObjectsArrayIdx;
};
