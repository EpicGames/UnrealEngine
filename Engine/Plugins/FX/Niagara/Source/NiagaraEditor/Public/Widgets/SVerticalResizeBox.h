// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.6, "Use SResizeBox.h instead of Widgets/SVerticalResizeBox.h, and add ToolWidgets as a module dependency")

#include "SResizeBox.h"
