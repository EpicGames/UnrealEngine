// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NiagaraCommon.h"
#include "NiagaraGraph.h"
#include "NiagaraNodeOutput.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Workflow/SWizard.h"

#define UE_API NIAGARAEDITOR_API

class UNiagaraClipboardContent;
class FNiagaraScratchPadScriptViewModel;
class IDetailsView;

namespace UE::Niagara::Wizard
{
	// a single page in the wizard
	struct FModuleWizardPage : TSharedFromThis<FModuleWizardPage>
	{
		FModuleWizardPage() = default;
		virtual ~FModuleWizardPage() = default;
		
		// If there is a follow-up page, this enables the button for it
		virtual bool CanGoToNextPage() const { return true; };

		// If true then the wizard as a whole can be finished, even if it's not the last page
		virtual bool CanCompleteWizard() const { return true; };

		UE_API virtual TSharedRef<SWidget> GetContent();

		// called when the page is activated
		UE_API virtual void RefreshContent();

		// name in the breadcrumb view
		FText Name;
	};

	// data model for the wizard holding all the pages in order. The model has no concept of branching, it follows the pages in linear order.
	struct FModuleWizardModel : TSharedFromThis<FModuleWizardModel>
	{
		FModuleWizardModel() = default;
		virtual ~FModuleWizardModel() = default;

		struct FModuleCreationEntry
		{
			UNiagaraNodeOutput* OutputNode = nullptr;
			int32 TargetIndex = INDEX_NONE;
		};

		// child classes can override this if they want to generate more than one module in the system or change the location of the generated module
		UE_API virtual TArray<FModuleCreationEntry> GetModulesToCreate(UNiagaraNodeOutput* ProvidedOutputNode, int32 ProvidedTargetIndex, TSharedPtr<FNiagaraSystemViewModel> SystemModel, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel); 

		// called when the scratch pad is generated and the graph can be modified, but before the scratch pad is added to the stack
		UE_API virtual void GenerateNewModuleContent(TSharedPtr<FNiagaraScratchPadScriptViewModel> ScratchPadScriptViewModel, const TArray<const UNiagaraNodeFunctionCall*>& PreviouslyCreatedModules);

		// called after the generated module is added to the stack - returning true will apply the modified clipboard content to the module inputs
		UE_API virtual bool UpdateModuleInputs(UNiagaraClipboardContent* NewModule, const TArray<const UNiagaraNodeFunctionCall*>& PreviouslyCreatedModules);

		// a unique name to identify this wizard with. Used in menus.
		virtual FName GetIdentifier() const { return "ModuleWizardModel"; }
		
		// target usage of the generated module
		ENiagaraScriptUsage TargetUsage;
		
		TArray<TSharedRef<FModuleWizardPage>> Pages;
	};

	// a wizard that is used to create new scratch pad modules
	class SNiagaraModuleWizard : public SWizard
	{
	public:

		SLATE_BEGIN_ARGS(SNiagaraModuleWizard) {}
			SLATE_EVENT(FSimpleDelegate, OnCreateModule)
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs, TSharedRef<FModuleWizardModel> Model);
		TArray<const UNiagaraNodeFunctionCall*> AddModulesToStack(TSharedPtr<FNiagaraSystemViewModel> SystemViewModel, UNiagaraNodeOutput* OutputNode, int32 TargetIndex, TSharedPtr<FNiagaraEmitterViewModel> EmitterViewModel, UNiagaraStackEditorData* StackEditorData);

		FSimpleDelegate OnCreateModule;

		TSharedPtr<FModuleWizardModel> Model;
		
	private:
		bool CanFinish() const;
		void OnFinished();
		void CloseContainingWindow();
	};

	// subclasses can be registered with FNiagaraEditorModule::RegisterModuleWizards() to automatically add wizards to the "add module" menu in the stack 
	class FModuleWizardGenerator
	{
	public:
		struct FAction
		{
			TSharedPtr<FModuleWizardModel> WizardModel;
			FText DisplayName;
			FText Description;
			FText Keywords;
			bool bSuggestedAction = false;
		};
		
		virtual TArray<FAction> CreateWizardActions(ENiagaraScriptUsage Usage) = 0;
		
		virtual ~FModuleWizardGenerator() = default;
	};

	namespace Utilities
	{	
		template<typename NodeType>
		NodeType* FindSingleNodeChecked(UNiagaraGraph* Graph)
		{
			check(Graph);
			TArray<NodeType*> NiagaraNodes;
			Graph->GetNodesOfClass(NiagaraNodes);
			check(NiagaraNodes.Num() == 1);
			return NiagaraNodes[0];
		}

		NIAGARAEDITOR_API UEdGraphPin* AddReadParameterPin(const FNiagaraTypeDefinition& Type, const FName& Name, UNiagaraNodeParameterMapGet* MapGetNode);
		NIAGARAEDITOR_API UEdGraphPin* AddWriteParameterPin(const FNiagaraTypeDefinition& Type, const FName& Name, UNiagaraNodeParameterMapSet* MapSetNode);
		NIAGARAEDITOR_API TSharedRef<IDetailsView> CreateDetailsView();
		NIAGARAEDITOR_API UNiagaraNodeFunctionCall* CreateDataInterfaceFunctionNode(const TSubclassOf<UNiagaraDataInterface>& DataInterfaceClass, const FName& FunctionName, UNiagaraGraph* Graph);
		NIAGARAEDITOR_API UNiagaraNodeFunctionCall* CreateFunctionCallNode(UNiagaraScript* FunctionScript, UNiagaraGraph* Graph);
		NIAGARAEDITOR_API UNiagaraNodeOp* CreateOpNode(const FName& OpName, UNiagaraGraph* Graph);

		template <typename T>
		NIAGARAEDITOR_API void SetDefaultValue(UNiagaraGraph* Graph, const FName& VarName, const FNiagaraTypeDefinition& TypeDef, T Value)
		{
			if (UNiagaraScriptVariable* ScriptVariable = Graph->GetScriptVariable(VarName))
			{
				// create temp var for data storage
				FNiagaraVariable Var(TypeDef, FName("Var"));
				Var.SetValue(Value);
				ScriptVariable->DefaultMode = ENiagaraDefaultMode::Value;
				ScriptVariable->SetDefaultValueData(Var.GetData());
				Graph->ScriptVariableChanged(ScriptVariable->Variable);
			}
		}
		NIAGARAEDITOR_API void SetDefaultBinding(UNiagaraGraph* Graph, const FName& VarName, const FName& DefaultBinding);
		NIAGARAEDITOR_API void SetTooltip(UNiagaraGraph* Graph, const FName& VarName, const FText& Tooltip);
	}
}

#undef UE_API
