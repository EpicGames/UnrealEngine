// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class NiagaraEditorWidgets : ModuleRules
{
	public NiagaraEditorWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePaths.AddRange(new string[] {
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
            "NiagaraCore",
			"Niagara",
			"NiagaraEditor",
			"NiagaraBlueprintNodes",
			"Engine",
			"Core",
			"CoreUObject",
			"Slate",
			"SlateCore",
			"InputCore",
			"EditorFramework",
			"UnrealEd",
			"GraphEditor",
			"EditorStyle",
			"PropertyEditor",
			"AppFramework",
			"Projects",
			"Sequencer",
            "EditorWidgets",
			"ApplicationCore",
			"CurveEditor",
			"DesktopPlatform",
			"ToolWidgets",
			"KismetWidgets",
			"ToolMenus",
			"DataHierarchyEditor",
			"UserAssetTagsEditor"
		});

		PrivateIncludePathModuleNames.AddRange(new string[] {
		});

		PublicDependencyModuleNames.AddRange(new string[] {
		});

		PublicIncludePathModuleNames.AddRange(new string[] {
		});
	}
}
