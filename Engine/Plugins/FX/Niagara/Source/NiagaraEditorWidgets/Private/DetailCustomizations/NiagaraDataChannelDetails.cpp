// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataChannelDetails.h"

#include "DetailLayoutBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"

#include "Types/SlateEnums.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "SListViewSelectorDropdownMenu.h"

#include "DataInterface/NiagaraDataInterfaceDataChannelRead.h"
#include "NiagaraDataChannel.h"
#include "NiagaraDataChannel_Islands.h"

#include "NiagaraSystem.h"
#include "NiagaraEditorUtilities.h"
#include "K2Node_DataChannelBase.h"

#include "NiagaraDataInterfaceDetails.h"

#define LOCTEXT_NAMESPACE "NiagaraDataChannelDetails"

class IPropertyHandle;
class SSearchBox;

//TODO: If more broadly useful, generalize and move to own files.
class SNiagaraVariableSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraVariableSelector) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InBaseProperty, FNiagaraVariableBase* InTarget, const TArray<TSharedPtr<FNiagaraVariableBase>>& InOptionsSource);

	void SetSourceArray(TArray<TSharedPtr<FNiagaraVariableBase>>& InOptionsSource);

	FSimpleMulticastDelegate& GetOnTargetChanged() { return OnTargetChanged; }

private:
	void OnComboOpening();

	void OnSelectionChanged(TSharedPtr<FNiagaraVariableBase> InNewSelection, ESelectInfo::Type);

	TSharedRef<ITableRow> GenerateAddElementRow(TSharedPtr<FNiagaraVariableBase> Entry, const TSharedRef<STableViewBase>& OwnerTable) const;


private:
	TArray<TSharedPtr<FNiagaraVariableBase>> OptionsSourceList;

	/** The current array property being edited */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** The component list control - part of the combo drop down */
	TSharedPtr< SListView<TSharedPtr<FNiagaraVariableBase>> > ElementsListView;

	TSharedPtr<SComboButton> ElementButton;

	FNiagaraVariableBase* Target = nullptr;

	TSharedPtr<SHorizontalBox> ComboButtonWidget;

	FSimpleMulticastDelegate OnTargetChanged;
};

void SNiagaraVariableSelector::Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InBaseProperty, FNiagaraVariableBase* InTarget, const TArray<TSharedPtr<FNiagaraVariableBase>>& InOptionsSource)
{
	PropertyHandle = InBaseProperty;
	OptionsSourceList = InOptionsSource;
	Target = InTarget;

	SAssignNew(ElementsListView, SListView<TSharedPtr<FNiagaraVariableBase>>)
		.ListItemsSource(&OptionsSourceList)
		.OnSelectionChanged(this, &SNiagaraVariableSelector::OnSelectionChanged)
		.OnGenerateRow(this, &SNiagaraVariableSelector::GenerateAddElementRow)
		.SelectionMode(ESelectionMode::Single);

	ElementsListView->RequestListRefresh();

	ComboButtonWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(1.f, 1.f)
		[
			FNiagaraParameterUtilities::GetParameterWidget(*Target, true, false)
		];

	// Create the Construct arguments for SComboButton

	SAssignNew(ElementButton, SComboButton)
		.ButtonContent()
		[
			ComboButtonWidget.ToSharedRef()
		]
	.MenuContent()
		[
			SNew(SListViewSelectorDropdownMenu<TSharedPtr<FNiagaraVariableBase>>, nullptr, ElementsListView)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(2)
				[
					SNew(SBox)
					.WidthOverride(175)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.MaxHeight(400)
						.Padding(8.f)
						[
							ElementsListView.ToSharedRef()
						]
					]
				]
			]
		]
	.IsFocusable(true)
		.ContentPadding(FMargin(5, 0))
		.OnComboBoxOpened(this, &SNiagaraVariableSelector::OnComboOpening);

	ElementsListView->EnableToolTipForceField(true);

	ChildSlot
		[
			ElementButton.ToSharedRef()
		];
};

void SNiagaraVariableSelector::SetSourceArray(TArray<TSharedPtr<FNiagaraVariableBase>>& InOptionsSource)
{
	OptionsSourceList = InOptionsSource;
	if (ElementsListView.IsValid())
	{
		ElementsListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SNiagaraVariableSelector::GenerateAddElementRow(TSharedPtr<FNiagaraVariableBase> Entry, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.NoHoverTableRow"))
		.ShowSelection(true)
		[
			SNew(SBox)
			.Padding(2.f)
			[
				FNiagaraParameterUtilities::GetParameterWidget(*Entry, true, false)
			]
		];
}

void SNiagaraVariableSelector::OnSelectionChanged(TSharedPtr<FNiagaraVariableBase> InNewSelection, ESelectInfo::Type SelectInfo)
{
	if (InNewSelection.IsValid() && (SelectInfo != ESelectInfo::OnNavigation))
	{
		*Target = *InNewSelection;
		ElementButton->SetIsOpen(false, false);

		ComboButtonWidget->GetSlot(0)
		[
			FNiagaraParameterUtilities::GetParameterWidget(*Target, true, false)
		];

		GetOnTargetChanged().Broadcast();
	}
}

void SNiagaraVariableSelector::OnComboOpening()
{
	
}

//////////////////////////////////////////////////////////////////////////

void FNiagaraDataChannelIslandsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

 	UObject* Obj = SelectedObjects[0].Get();
 	UNiagaraDataChannel_Islands* Islands = Cast<UNiagaraDataChannel_Islands>(Obj);

	if(Islands == nullptr)
	{
		return;
	}

	//Right now we're only interested in customizing the islands channel. In future we'll do others.
	auto AddProperties = [](const TArray<TSharedRef<IPropertyHandle>>& Properties, IDetailCategoryBuilder& CategoryBuilder)
	{
		for(auto& Prop : Properties)
		{
			CategoryBuilder.AddProperty(Prop);
		}
	};

	TArray<TSharedRef<IPropertyHandle>> Properties;

	static const FName ChannelCategoryName = TEXT("Data Channel");
	IDetailCategoryBuilder& ChannelCategoryBuilder = DetailBuilder.EditCategory(ChannelCategoryName);	
	ChannelCategoryBuilder.GetDefaultProperties(Properties);
	AddProperties(Properties, ChannelCategoryBuilder);

	static const FName IslandsCategoryName = TEXT("Islands");
	IDetailCategoryBuilder& IslandsCategoryBuilder = DetailBuilder.EditCategory(IslandsCategoryName);
	IslandsCategoryBuilder.GetDefaultProperties(Properties);
	AddProperties(Properties, IslandsCategoryBuilder);
}

TSharedRef<IDetailCustomization> FNiagaraDataChannelIslandsDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataChannelIslandsDetails>();
}

//////////////////////////////////////////////////////////////////////////


void FNiagaraDataInterfaceDataChannelReadDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FNiagaraDataInterfaceDetailsBase::CustomizeDetails(DetailBuilder);

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	if (SelectedObjects.Num() != 1 || SelectedObjects[0]->IsA<UNiagaraDataInterfaceDataChannelRead>() == false)
	{
		return;
	}

	UNiagaraDataInterfaceDataChannelRead* ReadDataInterface = CastChecked<UNiagaraDataInterfaceDataChannelRead>(SelectedObjects[0].Get());
	ReadDataInterfaceWeak = ReadDataInterface;
	NiagaraSystemWeak = ReadDataInterface->GetTypedOuter<UNiagaraSystem>();

	static const FName ReadCategoryName = TEXT("DataChannelRead");
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(ReadCategoryName);

	TArray<TSharedRef<IPropertyHandle>> Properties;
	CategoryBuilder.GetDefaultProperties(Properties, true, true);
	for (const TSharedRef<IPropertyHandle>& PropertyHandle : Properties)
	{
		PropertyHandle->GetProperty();
		CategoryBuilder.AddProperty(PropertyHandle);		
	}
}

TSharedRef<IDetailCustomization> FNiagaraDataInterfaceDataChannelReadDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceDataChannelReadDetails>();
}

void FNiagaraDataChannelBPNodeDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

	UObject* Obj = SelectedObjects[0].Get();
	UK2Node_DataChannelBase* Node = Cast<UK2Node_DataChannelBase>(Obj);
	if(Node == nullptr || Node->GetDataChannel() == nullptr)
	{
		return;
	}

	UNiagaraDataChannel* DataChannel = Node->GetDataChannel();
	IDetailCategoryBuilder& ChannelCategoryBuilder = DetailBuilder.EditCategory(TEXT("Data Channel"));
	for (const FNiagaraDataChannelVariable& Var : DataChannel->GetVariables())
	{
		TWeakObjectPtr<UK2Node_DataChannelBase> NodePtr(Node);
		FGuid VarGuid = Var.Version;
		auto CheckStateChanged = [NodePtr, VarGuid](const ECheckBoxState NewState)
		{
			if (UK2Node_DataChannelBase* Node = NodePtr.Get())
			{
				FScopedTransaction Transaction(LOCTEXT("ChangeAttributeAccess", "Change data channel attribute access"));
				Node->Modify();
				if (NewState == ECheckBoxState::Checked)
				{
					Node->IgnoredVariables.Remove(VarGuid);
					Node->ReconstructNode();
				}
				else if (NewState == ECheckBoxState::Unchecked)
				{
					Node->IgnoredVariables.Add(VarGuid);
					Node->ReconstructNode();
				}
			}
		};
		auto SPCheckState = [NodePtr, VarGuid]()
		{
			if (UK2Node_DataChannelBase* Node = NodePtr.Get())
			{
				return Node->IgnoredVariables.Contains(VarGuid) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			}
			return ECheckBoxState::Undetermined;
		};
		
		FDetailWidgetRow& Row = ChannelCategoryBuilder.AddCustomRow(FText::FromName(Var.GetName()));
		Row.NameWidget
		[
			SNew(STextBlock)
			.Text(FText::FromName(Var.GetName()))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
		Row.ValueWidget
		[
			SNew(SCheckBox)
			.OnCheckStateChanged_Lambda(CheckStateChanged)
			.IsChecked_Lambda(SPCheckState)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AsPinText", "(As pin)"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}
}

TSharedRef<IDetailCustomization> FNiagaraDataChannelBPNodeDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataChannelBPNodeDetails>();
}

#undef LOCTEXT_NAMESPACE

