// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "IDetailCustomization.h"
#include "NiagaraDataInterfaceDetails.h"

class UNiagaraDataInterfaceDataChannelRead;
class UNiagaraSystem;
class UNiagaraDataChannel;
class UNiagaraDataChannel_Islands;

/** Details customization for Niagara data channels. */
class FNiagaraDataChannelIslandsDetails : public IDetailCustomization
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:

	TWeakObjectPtr<UNiagaraDataChannel_Islands> DataChannel;
};


/** Details customization for Niagara data channel read data interface. */
class FNiagaraDataInterfaceDataChannelReadDetails : public FNiagaraDataInterfaceDetailsBase
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:

	TWeakObjectPtr<UNiagaraDataInterfaceDataChannelRead> ReadDataInterfaceWeak;
	TWeakObjectPtr<UNiagaraSystem> NiagaraSystemWeak;
};


/** Details customization for Blueprint nodes. */
class FNiagaraDataChannelBPNodeDetails : public IDetailCustomization
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();
};