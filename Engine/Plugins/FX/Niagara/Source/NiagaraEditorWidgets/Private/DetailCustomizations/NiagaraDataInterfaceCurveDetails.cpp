// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceCurveDetails.h"
#include "NiagaraDataInterfaceCurve.h"
#include "NiagaraDataInterfaceVector2DCurve.h"
#include "NiagaraDataInterfaceVectorCurve.h"
#include "NiagaraDataInterfaceVector4Curve.h"
#include "NiagaraDataInterfaceColorCurve.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorWidgetsModule.h"
#include "NiagaraSystem.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraEditorUtilities.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "ViewModels/NiagaraCurveSelectionViewModel.h"
#include "SNiagaraCurveKeySelector.h"
#include "SNiagaraCurveThumbnail.h"

#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Layout/SBox.h"
#include "Misc/Optional.h"
#include "Brushes/SlateColorBrush.h"
#include "Modules/ModuleManager.h"

#include "CurveEditor.h"
#include "Tree/SCurveEditorTree.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "Tree/SCurveEditorTreePin.h"
#include "RichCurveEditorModel.h"
#include "SCurveEditorPanel.h"
#include "SCurveKeyDetailPanel.h"
#include "CurveEditorCommands.h"

#include "SResizeBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveVector.h"
#include "Curves/CurveFloat.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/RichCurve.h"
#include "ScopedTransaction.h"
#include "SColorGradientEditor.h"
#include "Misc/AxisDisplayInfo.h"

#include "EditorFontGlyphs.h"
#include "Misc/CurveEditorIntegrationUtils.h"
#include "Widgets/Input/SSegmentedControl.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceCurveDetails"

struct FNiagaraCurveDetailsTreeItem : public ICurveEditorTreeItem, TSharedFromThis<FNiagaraCurveDetailsTreeItem>
{
	FNiagaraCurveDetailsTreeItem(UObject* InCurveOwnerObject, UNiagaraDataInterfaceCurveBase::FCurveData InCurveData, EAxisList::Type InCurveAxis)
		: CurveOwnerObject(InCurveOwnerObject)
		, CurveData(InCurveData)
		, CurveAxis(InCurveAxis)
	{
	}

	UObject* GetCurveOwnerObject() const
	{
		return CurveOwnerObject.Get();
	}

	FRichCurve* GetCurve() const
	{
		return CurveOwnerObject.IsValid() ? CurveData.Curve : nullptr;
	}

	virtual TSharedPtr<SWidget> GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& TableRow) override
	{
		if (InColumnName == ColumnNames.Label)
		{
			FText CurveDisplayName;
			FText CurveToolTipText;
			FLinearColor CurveDisplayColor;

			if (CurveAxis != EAxisList::None)
			{
				CurveDisplayName = AxisDisplayInfo::GetAxisDisplayNameShort(CurveAxis);
				CurveToolTipText = AxisDisplayInfo::GetAxisDisplayName(CurveAxis);
				CurveDisplayColor = AxisDisplayInfo::GetAxisColor(CurveAxis);
			}
			else
			{
				CurveDisplayName = FText::FromName(CurveData.Name);
				CurveDisplayColor = CurveData.Color;
			}

			TSharedRef<SHorizontalBox> LabelBox = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 5, 0)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.8"))
					.Text(FEditorFontGlyphs::Circle)
					.ColorAndOpacity(FSlateColor(CurveDisplayColor))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					.MinDesiredHeight(22)
					[
						SNew(STextBlock)
						.Text(CurveDisplayName)
						.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.CurveOverview.CurveComponentText")
					]
				];

			if (CurveToolTipText.IsEmptyOrWhitespace() == false)
			{
				LabelBox->SetToolTipText(CurveToolTipText);
			}

			return LabelBox;
		}
		else if (InColumnName == ColumnNames.PinHeader)
		{
			return SNew(SCurveEditorTreePin, InCurveEditor, InTreeItemID, TableRow);
		}

		return nullptr;
	}

	virtual void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override
	{
		if (CurveOwnerObject.IsValid() == false)
		{
			return;
		}

		TUniquePtr<FRichCurveEditorModelRaw> NewCurve = MakeUnique<FRichCurveEditorModelRaw>(CurveData.Curve, CurveOwnerObject.Get());
		if(CurveData.Name != NAME_None)
		{
			NewCurve->SetShortDisplayName(FText::FromName(CurveData.Name));
			NewCurve->SetChannelName(CurveData.Name);
		}
		NewCurve->SetColor(CurveData.Color);
		NewCurve->OnCurveModified().AddSP(this, &FNiagaraCurveDetailsTreeItem::CurveChanged);
		OutCurveModels.Add(MoveTemp(NewCurve));
	}

	virtual bool PassesFilter(const FCurveEditorTreeFilter* InFilter) const override { return true; }

	FSimpleMulticastDelegate& OnCurveChanged()
	{
		return CurveChangedDelegate;
	}

private:
	void CurveChanged()
	{
		CurveChangedDelegate.Broadcast();
	}

private:
	TWeakObjectPtr<UObject> CurveOwnerObject;
	UNiagaraDataInterfaceCurveBase::FCurveData CurveData;
	EAxisList::Type CurveAxis;
	FSimpleMulticastDelegate CurveChangedDelegate;
};

FRichCurve* GetCurveFromPropertyHandle(TSharedPtr<IPropertyHandle> Handle)
{
	TArray<void*> RawData;
	Handle->AccessRawData(RawData);
	return RawData.Num() == 1 ? static_cast<FRichCurve*>(RawData[0]) : nullptr;
}

class SNiagaraDataInterfaceCurveEditor : public SCompoundWidget
{
private:
	struct FDataInterfaceCurveEditorBounds : public ICurveEditorBounds
	{
		FDataInterfaceCurveEditorBounds(TSharedRef<FNiagaraStackCurveEditorOptions> InStackCurveEditorOptions)
			: StackCurveEditorOptions(InStackCurveEditorOptions)
		{
		}

		virtual void GetInputBounds(double& OutMin, double& OutMax) const
		{
			OutMin = StackCurveEditorOptions->GetViewMinInput();
			OutMax = StackCurveEditorOptions->GetViewMaxInput();
		}

		virtual void SetInputBounds(double InMin, double InMax)
		{
			StackCurveEditorOptions->SetInputViewRange((float)InMin, (float)InMax);
		}

	private:
		TSharedRef<FNiagaraStackCurveEditorOptions> StackCurveEditorOptions;
	};

public:
	SLATE_BEGIN_ARGS(SNiagaraDataInterfaceCurveEditor) {}
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		TArray<TSharedRef<IPropertyHandle>> InCurveProperties,
		TSharedRef<FNiagaraStackCurveEditorOptions> InStackCurveEditorOptions)
	{
		CurveProperties = InCurveProperties;
		StackCurveEditorOptions = InStackCurveEditorOptions;

		TArray<UObject*> OuterObjects;
		CurveProperties[0]->GetOuterObjects(OuterObjects);
		UNiagaraDataInterfaceCurveBase* CurveOwnerObject = Cast<UNiagaraDataInterfaceCurveBase>(OuterObjects[0]);
		if (CurveOwnerObject == nullptr)
		{
			return;
		}

		if (StackCurveEditorOptions->GetNeedsInitializeView())
		{
			InitializeView();
		}

		UNiagaraEditorSettings* EditorSettings = GetMutableDefault<UNiagaraEditorSettings>();
		CurveEditor = MakeShared<FCurveEditor>();
		FCurveEditorInitParams InitParams;
		InitParams.ResolveCurveModelDelegate.BindStatic(&UE::NiagaraEditorWidgets::ResolveNiagaraCurveModel); // Needed for undo / redo.
		CurveEditor->InitCurveEditor(InitParams);
		TAttribute<bool>::FGetter InputSnapEnabledGetter = TAttribute<bool>::FGetter::CreateUObject(EditorSettings, &UNiagaraEditorSettings::IsCurveInputSnapEnabled);
		TAttribute<bool>::FGetter OutputSnapEnabledGetter = TAttribute<bool>::FGetter::CreateUObject(EditorSettings, &UNiagaraEditorSettings::IsCurveOutputSnapEnabled);
		CurveEditor->InputSnapEnabledAttribute = TAttribute<bool>::Create(InputSnapEnabledGetter);
		CurveEditor->OutputSnapEnabledAttribute = TAttribute<bool>::Create(OutputSnapEnabledGetter);
		CurveEditor->OnInputSnapEnabledChanged.BindUObject(EditorSettings, &UNiagaraEditorSettings::SetCurveInputSnapEnabled);
		CurveEditor->OnOutputSnapEnabledChanged.BindUObject(EditorSettings, &UNiagaraEditorSettings::SetCurveOutputSnapEnabled);
		CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");

		// Initialize our bounds at slightly larger than default to avoid clipping the tabs on the color widget.
		TUniquePtr<ICurveEditorBounds> EditorBounds = MakeUnique<FDataInterfaceCurveEditorBounds>(InStackCurveEditorOptions);
		CurveEditor->SetBounds(MoveTemp(EditorBounds));

		TSharedPtr<SCurveEditorTree> CurveEditorTree;
		CurveEditorPanel = SNew(SCurveEditorPanel, CurveEditor.ToSharedRef())
			.MinimumViewPanelHeight(50.0f)
			.TreeContent()
			[
				CurveProperties.Num() == 1
					? SNullWidget::NullWidget
					: SAssignNew(CurveEditorTree, SCurveEditorTree, CurveEditor)
					.SelectColumnWidth(0)
			];

		TArray<UNiagaraDataInterfaceCurveBase::FCurveData> CurveDatas;
		CurveOwnerObject->GetCurveData(CurveDatas);
		TArray<FCurveEditorTreeItemID> TreeItemIds;
		int32 CurveComponentIndex = 0;
		for (const UNiagaraDataInterfaceCurveBase::FCurveData& CurveData : CurveDatas)
		{
			EAxisList::Type CurveAxis = FNiagaraEditorUtilities::VectorComponentToAxis(CurveDatas.Num(), CurveComponentIndex);
			TSharedRef<FNiagaraCurveDetailsTreeItem> TreeItem = MakeShared<FNiagaraCurveDetailsTreeItem>(CurveOwnerObject, CurveData, CurveAxis);
			TreeItem->OnCurveChanged().AddSP(this, &SNiagaraDataInterfaceCurveEditor::CurveChanged, TWeakPtr<FNiagaraCurveDetailsTreeItem>(TreeItem));
			FCurveEditorTreeItem* NewItem = CurveEditor->AddTreeItem(FCurveEditorTreeItemID::Invalid());
			NewItem->SetStrongItem(TreeItem);
			TreeItemIds.Add(NewItem->GetID());
			for (const FCurveModelID& CurveModel : NewItem->GetOrCreateCurves(CurveEditor.Get()))
			{
				CurveEditor->PinCurve(CurveModel);
			}
			CurveComponentIndex++;
		}

		float KeySelectorLeftPadding = CurveProperties.Num() == 1 ? 0 : 7;
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0, 0, 0, 5)
			[
				CurveEditorPanel.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(KeySelectorLeftPadding, 0, 8, 0)
				[
					SNew(SNiagaraCurveKeySelector, CurveEditor, TreeItemIds, CurveEditorTree)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0, 0, 3, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("KeyLabel", "Key Data"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				[
					CurveEditorPanel->GetKeyDetailsView().ToSharedRef()
				]
			]
		];
	}

	TSharedPtr<FUICommandList> GetCommands()
	{
		return CurveEditor->GetCommands();
	}

	TSharedPtr<FCurveEditor> GetEditor()
	{
		return CurveEditor;
	}

private:
	void InitializeView()
	{
		bool bHasKeys = false;
		float ViewMinInput = TNumericLimits<float>::Max();
		float ViewMaxInput = TNumericLimits<float>::Lowest();
		float ViewMinOutput = TNumericLimits<float>::Max();
		float ViewMaxOutput = TNumericLimits<float>::Lowest();

		for (TSharedRef<IPropertyHandle> CurveProperty : CurveProperties)
		{
			FRealCurve* Curve = GetCurveFromPropertyHandle(CurveProperty);
			for(auto KeyIterator = Curve->GetKeyHandleIterator(); KeyIterator; ++KeyIterator)
			{
				float KeyTime = Curve->GetKeyTime(*KeyIterator);
				float KeyValue = Curve->GetKeyValue(*KeyIterator);
				ViewMinInput = FMath::Min(ViewMinInput, KeyTime);
				ViewMaxInput = FMath::Max(ViewMaxInput, KeyTime);
				ViewMinOutput = FMath::Min(ViewMinOutput, KeyValue);
				ViewMaxOutput = FMath::Max(ViewMaxOutput, KeyValue);
				bHasKeys = true;
			}
		}

		if (bHasKeys == false)
		{
			ViewMinInput = 0;
			ViewMaxInput = 1;
			ViewMinOutput = 0;
			ViewMaxOutput = 1;
		}

		if (FMath::IsNearlyEqual(ViewMinInput, ViewMaxInput))
		{
			if (FMath::IsWithinInclusive(ViewMinInput, 0.0f, 1.0f))
			{
				ViewMinInput = 0;
				ViewMaxInput = 1;
			}
			else
			{
				ViewMinInput -= 0.5f;
				ViewMaxInput += 0.5f;
			}
		}

		if (FMath::IsNearlyEqual(ViewMinOutput, ViewMaxOutput))
		{
			if (FMath::IsWithinInclusive(ViewMinOutput, 0.0f, 1.0f))
			{
				ViewMinOutput = 0;
				ViewMaxOutput = 1;
			}
			else
			{
				ViewMinOutput -= 0.5f;
				ViewMaxOutput += 0.5f;
			}
		}

		float ViewInputRange = ViewMaxInput - ViewMinInput;
		float ViewOutputRange = ViewMaxOutput - ViewMinOutput;
		float ViewInputPadding = ViewInputRange * .05f;
		float ViewOutputPadding = ViewOutputRange * .05f;

		StackCurveEditorOptions->InitializeView(
			ViewMinInput - ViewInputPadding,
			ViewMaxInput + ViewInputPadding,
			ViewMinOutput - ViewOutputPadding,
			ViewMaxOutput + ViewOutputPadding);
	}

	void CurveChanged(TWeakPtr<FNiagaraCurveDetailsTreeItem> TreeItemWeak)
	{
		TSharedPtr<FNiagaraCurveDetailsTreeItem> TreeItem = TreeItemWeak.Pin();
		if(TreeItem.IsValid())
		{
			UNiagaraDataInterfaceCurveBase* EditedCurveOwner = Cast<UNiagaraDataInterfaceCurveBase>(TreeItem->GetCurveOwnerObject());
			if(EditedCurveOwner != nullptr)
			{
				EditedCurveOwner->UpdateLUT(); // we need this done before notify change because of the internal copy methods
				for (TSharedRef<IPropertyHandle> CurveProperty : CurveProperties)
				{
					if (GetCurveFromPropertyHandle(CurveProperty) == TreeItem->GetCurve())
					{
						CurveProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
						break;
					}
				}
			}
		}
	}

private:
	TArray<TSharedRef<IPropertyHandle>> CurveProperties;
	TSharedPtr<FNiagaraStackCurveEditorOptions> StackCurveEditorOptions;

	TSharedPtr<FCurveEditor> CurveEditor;
	TSharedPtr<SCurveEditorPanel> CurveEditorPanel;
};

class FNiagaraColorCurveDataInterfaceCurveOwner : public FCurveOwnerInterface
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnColorCurveChanged, TArray<FRichCurve*> /* ChangedCurves */);

public:
	FNiagaraColorCurveDataInterfaceCurveOwner(UNiagaraDataInterfaceColorCurve& InDataInterface)
	{
		DataInterfaceWeak = &InDataInterface;

		ConstCurves.Add(FRichCurveEditInfoConst(&InDataInterface.RedCurve, "Red"));
		ConstCurves.Add(FRichCurveEditInfoConst(&InDataInterface.GreenCurve, "Green"));
		ConstCurves.Add(FRichCurveEditInfoConst(&InDataInterface.BlueCurve, "Blue"));
		ConstCurves.Add(FRichCurveEditInfoConst(&InDataInterface.AlphaCurve, "Alpha"));

		Curves.Add(FRichCurveEditInfo(&InDataInterface.RedCurve, "Red"));
		Curves.Add(FRichCurveEditInfo(&InDataInterface.GreenCurve, "Green"));
		Curves.Add(FRichCurveEditInfo(&InDataInterface.BlueCurve, "Blue"));
		Curves.Add(FRichCurveEditInfo(&InDataInterface.AlphaCurve, "Alpha"));

		Owners.Add(&InDataInterface);
	}

	/** FCurveOwnerInterface */
	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const override
	{
		return ConstCurves;
	}

	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> OutCurves) const override
	{
		for (const FRichCurveEditInfoConst& ConstCurve: ConstCurves)
		{
			OutCurves.Add(ConstCurve);
		}
	}

	virtual TArray<FRichCurveEditInfo> GetCurves() override
	{
		return Curves;
	}

	virtual void ModifyOwner() override
	{
		if (DataInterfaceWeak.IsValid())
		{
			DataInterfaceWeak->Modify();
		}
	}

	virtual TArray<const UObject*> GetOwners() const override
	{
		return Owners;
	}

	virtual void MakeTransactional() override { }

	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override
	{
		TArray<FRichCurve*> ChangedCurves;
		for (const FRichCurveEditInfo& ChangedCurveEditInfo : ChangedCurveEditInfos)
		{
			ChangedCurves.Add((FRichCurve*)ChangedCurveEditInfo.CurveToEdit);
		}
		if(ChangedCurves.Num() > 0)
		{
			OnColorCurveChangedDelegate.Broadcast(ChangedCurves);
		}
	}

	virtual bool IsLinearColorCurve() const override 
	{
		return true;
	}

	virtual FLinearColor GetLinearColorValue(float InTime) const override
	{
		if (DataInterfaceWeak.IsValid())
		{
			return FLinearColor(
				DataInterfaceWeak->RedCurve.Eval(InTime),
				DataInterfaceWeak->GreenCurve.Eval(InTime),
				DataInterfaceWeak->BlueCurve.Eval(InTime),
				DataInterfaceWeak->AlphaCurve.Eval(InTime));
		}
		return FLinearColor::White;
	}

	virtual bool HasAnyAlphaKeys() const override 
	{ 
		return DataInterfaceWeak.IsValid() && DataInterfaceWeak->AlphaCurve.GetNumKeys() != 0;
	}

	virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override 
	{
		return Curves.Contains(CurveInfo);
	}

	virtual FLinearColor GetCurveColor(FRichCurveEditInfo CurveInfo) const override 
	{
		return FLinearColor::White;
	}

	FOnColorCurveChanged& OnColorCurveChanged()
	{
		return OnColorCurveChangedDelegate;
	}

private:
	TWeakObjectPtr<UNiagaraDataInterfaceColorCurve> DataInterfaceWeak;
	TArray<FRichCurveEditInfoConst> ConstCurves;
	TArray<FRichCurveEditInfo> Curves;
	TArray<const UObject*> Owners;
	FOnColorCurveChanged OnColorCurveChangedDelegate;
};

class SNiagaraDataInterfaceGradientEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraDataInterfaceGradientEditor) {}
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		UNiagaraDataInterfaceCurveBase* InDataInterface,
		TArray<TSharedRef<IPropertyHandle>> InCurveProperties)
	{
		UNiagaraDataInterfaceColorCurve* ColorCurveDataInterface = Cast<UNiagaraDataInterfaceColorCurve>(InDataInterface);
		if(ensureMsgf(ColorCurveDataInterface != nullptr, TEXT("Only UNiagaraDataInterfaceColorCurve is currently supported.")))
		{
			ColorCurveDataInterfaceWeak = ColorCurveDataInterface;
			CurveProperties = InCurveProperties;
			ColorCurveOwner = MakeShared<FNiagaraColorCurveDataInterfaceCurveOwner>(*ColorCurveDataInterface);
			ColorCurveOwner->OnColorCurveChanged().AddSP(this, &SNiagaraDataInterfaceGradientEditor::ColorCurveChanged);
			TSharedRef<SColorGradientEditor> GradientEditor = SNew(SColorGradientEditor)
				.ViewMinInput(0.0f)
				.ViewMaxInput(1.0f)
				.ClampStopsToViewRange(true);

			GradientEditor->SetCurveOwner(ColorCurveOwner.Get());
			ChildSlot
			[
				SNew(SBox)
				.Padding(0, 2, 5, 2)
				[
					GradientEditor
				]
			];
		}
	}

private:
	void ClampKeyTimes(FRichCurve& Curve)
	{
		for (auto It = Curve.GetKeyHandleIterator(); It; ++It)
		{
			if (Curve.GetKeyTime(*It) < 0)
			{
				Curve.SetKeyTime(*It, 0);
			}
			if (Curve.GetKeyTime(*It) > 1)
			{
				Curve.SetKeyTime(*It, 1);
			}
		}
	}

	void ColorCurveChanged(TArray<FRichCurve*> ChangedCurves)
	{
		UNiagaraDataInterfaceColorCurve* ColorCurveDataInterface = ColorCurveDataInterfaceWeak.Get();
		if(ColorCurveDataInterface != nullptr)
		{
			ClampKeyTimes(ColorCurveDataInterface->RedCurve);
			ClampKeyTimes(ColorCurveDataInterface->GreenCurve);
			ClampKeyTimes(ColorCurveDataInterface->BlueCurve);
			ClampKeyTimes(ColorCurveDataInterface->AlphaCurve);

			ColorCurveDataInterface->UpdateLUT();
			for (FRichCurve* ChangedCurve : ChangedCurves)
			{
				for (TSharedRef<IPropertyHandle> CurveProperty : CurveProperties)
				{
					if (GetCurveFromPropertyHandle(CurveProperty) == ChangedCurve)
					{
						CurveProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
					}
				}
			}
		}
	}

private:
	TWeakObjectPtr<UNiagaraDataInterfaceColorCurve> ColorCurveDataInterfaceWeak;
	TArray<TSharedRef<IPropertyHandle>> CurveProperties;
	TSharedPtr<FNiagaraColorCurveDataInterfaceCurveOwner> ColorCurveOwner;
};



// Curve Base
void FNiagaraDataInterfaceCurveDetailsBase::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	CustomDetailBuilder = &DetailBuilder;
	FNiagaraDataInterfaceDetailsBase::CustomizeDetails(DetailBuilder);

	// Only support single objects.
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}

	UNiagaraDataInterfaceCurveBase* CurveDataInterface = Cast<UNiagaraDataInterfaceCurveBase>(ObjectsBeingCustomized[0].Get());
	TSharedRef<FNiagaraStackCurveEditorOptions> StackCurveEditorOptions = FNiagaraEditorWidgetsModule::Get().GetOrCreateStackCurveEditorOptionsForObject(
		ObjectsBeingCustomized[0].Get(), GetDefaultHeight());

	CurveDataInterfaceWeak = CurveDataInterface;
	StackCurveEditorOptionsWeak = StackCurveEditorOptions;

	TArray<TSharedRef<IPropertyHandle>> CurveProperties;
	GetCurveProperties(DetailBuilder, CurveProperties);

	// Make sure all property handles are valid.
	for (TSharedRef<IPropertyHandle> CurveProperty : CurveProperties)
	{
		if (CurveProperty->IsValidHandle() == false)
		{
			return;
		}
	}

	for (TSharedRef<IPropertyHandle> CurveProperty : CurveProperties)
	{
		CurveProperty->MarkHiddenByCustomization();
	}
	TSharedRef<IPropertyHandle> AssetProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceCurveBase, CurveAsset), UNiagaraDataInterfaceCurveBase::StaticClass());
	IDetailPropertyRow* DefaultRow = DetailBuilder.EditDefaultProperty(AssetProperty);
	DefaultRow->Visibility(TAttribute<EVisibility>::CreateLambda([this]() -> EVisibility { return GetUsedCurveAsset(CurveDataInterfaceWeak.Get()) ? EVisibility::Visible : EVisibility::Collapsed; }));

	TSharedPtr<SNiagaraDataInterfaceCurveEditor> CurveEditor;
	TSharedRef<SWidget> CurveDataInterfaceCurveEditor = 
		SNew(SVerticalResizeBox)
		.ContentHeight(StackCurveEditorOptions, &FNiagaraStackCurveEditorOptions::GetHeight)
		.ContentHeightChanged(StackCurveEditorOptions, &FNiagaraStackCurveEditorOptions::SetHeight)
		.Content()
		[
			SAssignNew(CurveEditor, SNiagaraDataInterfaceCurveEditor, CurveProperties, StackCurveEditorOptions)
		];

	TSharedPtr<SWidget> CurveDataInterfaceEditor;
	if (GetIsColorCurve())
	{
		CurveDataInterfaceEditor = 
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &FNiagaraDataInterfaceCurveDetailsBase::GetGradientCurvesSwitcherIndex)
			+ SWidgetSwitcher::Slot()
			[
				SNew(SNiagaraDataInterfaceGradientEditor, CurveDataInterface, CurveProperties)
			]
			+ SWidgetSwitcher::Slot()
			[
				CurveDataInterfaceCurveEditor
			];
	}
	else
	{
		CurveDataInterfaceEditor = CurveDataInterfaceCurveEditor;
	}

	FToolBarBuilder ToolBarBuilder(CurveEditor->GetCommands(), FMultiBoxCustomization::None, nullptr, true);
	ToolBarBuilder.SetStyle(&FAppStyle::Get(), "CurveEditorToolBar");

	ToolBarBuilder.AddComboButton(
		FUIAction(), 
		FOnGetContent::CreateSP(this, &FNiagaraDataInterfaceCurveDetailsBase::GetCurveToCopyMenu),
		NSLOCTEXT("NiagaraDataInterfaceCurveDetails", "Import", "Import"),
		NSLOCTEXT("NiagaraDataInterfaceCurveDetails", "CopyCurveAsset", "Import data from another Curve asset"),
		FSlateIcon(FNiagaraEditorWidgetsStyle::Get().GetStyleSetName(), "NiagaraEditor.CurveDetails.Import"));

	ToolBarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &FNiagaraDataInterfaceCurveDetailsBase::OnShowInCurveEditor)),
		NAME_None,
		LOCTEXT("CurveOverviewLabel", "Curve Overview"),
		LOCTEXT("ShowInCurveOverviewToolTip", "Show this curve in the curve overview."),
		FSlateIcon(FNiagaraEditorWidgetsStyle::Get().GetStyleSetName(), "NiagaraEditor.CurveDetails.ShowInOverview"));

	ToolBarBuilder.AddSeparator();

	ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().ToggleInputSnapping);
	ToolBarBuilder.AddToolBarButton(FCurveEditorCommands::Get().ToggleOutputSnapping);

	if (GetIsColorCurve())
	{
		ToolBarBuilder.AddSeparator();

		ToolBarBuilder.AddWidget(
			SNew(SBox)
			.Padding(FMargin(0, 0, 2, 0))
			[
				SNew(SSegmentedControl<bool>)
				.OnValueChanged(this, &FNiagaraDataInterfaceCurveDetailsBase::SetGradientVisibility)
				.Value(this, &FNiagaraDataInterfaceCurveDetailsBase::IsGradientVisible)
				+ SSegmentedControl<bool>::Slot(false)
				.Text(LOCTEXT("CurvesButtonLabel", "Curves"))
				.ToolTip(LOCTEXT("ShowCurvesToolTip", "Show the curve editor."))
				+ SSegmentedControl<bool>::Slot(true)
				.Text(LOCTEXT("GradientButtonLabel", "Gradient"))
				.ToolTip(LOCTEXT("ShowGradientToolTip", "Show the gradient editor."))
			]);
	}

	//add flip curve buttons to tool bar
	ToolBarBuilder.AddSeparator();
	ToolBarBuilder.AddToolBarButton(
		FCurveEditorCommands::Get().FlipCurveHorizontal,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.FlipCurveHorizontal"));
	ToolBarBuilder.AddToolBarButton(
		FCurveEditorCommands::Get().FlipCurveVertical,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCurveEditor.FlipCurveVertical"));

	// add templates to toolbar
	ToolBarBuilder.AddSeparator();
	const UNiagaraEditorSettings* Settings = GetDefault<UNiagaraEditorSettings>();
	ToolBarBuilder.AddWidget(
		SNew(SBox)
		.VAlign(VAlign_Center)
		.Visibility_Lambda([this]() { return GetIsColorCurve() && IsGradientVisible() ? EVisibility::Collapsed : EVisibility::Visible; })
		.Padding(FMargin(3, 0, 5, 0))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CurveTemplateLabel", "Templates"))
		]);
	for (const FNiagaraCurveTemplate& CurveTemplate : Settings->GetCurveTemplates())
	{
		UCurveFloat* FloatCurveAsset = Cast<UCurveFloat>(CurveTemplate.CurveAsset.TryLoad());
		if(FloatCurveAsset != nullptr)
		{
			FText CurveDisplayName = CurveTemplate.DisplayNameOverride.IsEmpty()
				? FText::FromString(FName::NameToDisplayString(FloatCurveAsset->GetName(), false))
				: FText::FromString(CurveTemplate.DisplayNameOverride);

			TWeakObjectPtr<UCurveFloat> WeakCurveAsset(FloatCurveAsset);
			TSharedPtr<FCurveEditor> EditorPtr = CurveEditor->GetEditor();
			ToolBarBuilder.AddWidget(
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &FNiagaraDataInterfaceCurveDetailsBase::CurveTemplateSelected, WeakCurveAsset, EditorPtr)
				.ToolTipText(FText::Format(LOCTEXT("ApplyCurveTemplateFormat", "{0}\nClick to apply this template to the selected curves."), CurveDisplayName))
				.ContentPadding(FMargin(3, 10, 3, 0))
				.Visibility_Lambda([this]() { return GetIsColorCurve() && IsGradientVisible() ? EVisibility::Collapsed : EVisibility::Visible; })
				.Content()
				[
					SNew(SNiagaraCurveThumbnail, FloatCurveAsset->FloatCurve)
				]);
		}
	}

	IDetailCategoryBuilder& CurveCategory = DetailBuilder.EditCategory("Curve");
	CurveCategory.AddCustomRow(NSLOCTEXT("NiagaraDataInterfaceCurveDetails", "CurveOptions", "Options"))
		.Visibility(TAttribute<EVisibility>::CreateLambda([this]() -> EVisibility { return GetUsedCurveAsset(CurveDataInterfaceWeak.Get()) ? EVisibility::Collapsed : EVisibility::Visible; }))
		.WholeRowContent() [ ToolBarBuilder.MakeWidget() ];
	CurveCategory.AddCustomRow(NSLOCTEXT("NiagaraDataInterfaceCurveDetails", "CurveFilterText", "Curve"))
		.Visibility(TAttribute<EVisibility>::CreateLambda([this]() -> EVisibility { return GetUsedCurveAsset(CurveDataInterfaceWeak.Get()) ? EVisibility::Collapsed : EVisibility::Visible; }))
		.WholeRowContent() [ CurveDataInterfaceEditor.ToSharedRef() ];
}

void FNiagaraDataInterfaceCurveDetailsBase::SetGradientVisibility(bool bInShowGradient)
{
	TSharedPtr<FNiagaraStackCurveEditorOptions> CurveEditorOptions = StackCurveEditorOptionsWeak.Pin();
	if (CurveEditorOptions.IsValid())
	{
		CurveEditorOptions->SetIsGradientVisible(bInShowGradient);
	}
}

int32 FNiagaraDataInterfaceCurveDetailsBase::GetGradientCurvesSwitcherIndex() const
{
	return StackCurveEditorOptionsWeak.IsValid() && StackCurveEditorOptionsWeak.Pin()->GetIsGradientVisible() ? 0 : 1;
}

bool FNiagaraDataInterfaceCurveDetailsBase::IsGradientVisible() const
{
	return StackCurveEditorOptionsWeak.IsValid() ? StackCurveEditorOptionsWeak.Pin()->GetIsGradientVisible() : false;
}



void FNiagaraDataInterfaceCurveDetailsBase::OnShowInCurveEditor() const
{
	UNiagaraDataInterfaceCurveBase* CurveDataInterface = CurveDataInterfaceWeak.Get();
	if (CurveDataInterface != nullptr)
	{
		UNiagaraSystem* OwningSystem = CurveDataInterface->GetTypedOuter<UNiagaraSystem>();
		if (OwningSystem != nullptr)
		{
			TSharedPtr<FNiagaraSystemViewModel> SystemViewModel = FNiagaraEditorModule::Get().GetExistingViewModelForSystem(OwningSystem);
			if(SystemViewModel.IsValid())
			{
				SystemViewModel->GetCurveSelectionViewModel()->FocusAndSelectCurveDataInterface(*CurveDataInterface);
			}
		}
	}
}

UCurveBase* FNiagaraDataInterfaceCurveDetailsBase::GetUsedCurveAsset(UNiagaraDataInterfaceCurveBase* CurveDI) const
{
	return CurveDI ? CurveDI->CurveAsset : nullptr;
}

void FNiagaraDataInterfaceCurveDetailsBase::ImportSelectedAsset(UObject* SelectedAsset)
{
	if (CurveDataInterfaceWeak.IsValid() == false)
	{
		return;
	}

	TArray<FRichCurve> FloatCurves;
	GetFloatCurvesFromAsset(SelectedAsset, FloatCurves);
	TArray<TSharedRef<IPropertyHandle>> CurveProperties;
	GetCurveProperties(*CustomDetailBuilder, CurveProperties);
	if (FloatCurves.Num() == CurveProperties.Num())
	{
		FScopedTransaction ImportTransaction(LOCTEXT("ImportCurveTransaction", "Import curve"));
		CurveDataInterfaceWeak->Modify();

		if (!bCopyAssetData)
		{
			CurveDataInterfaceWeak->CurveAsset = Cast<UCurveBase>(SelectedAsset);
		}
		
		for (int i = 0; i < CurveProperties.Num(); i++)
		{
			if (CurveProperties[i]->IsValidHandle())
			{
				*GetCurveFromPropertyHandle(CurveProperties[i]) = FloatCurves[i];
			}
		}
		CurveDataInterfaceWeak->UpdateLUT(); // we need this done before notify change because of the internal copy methods
		for (auto CurveProperty : CurveProperties)
		{
			CurveProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}
}

TSharedRef<SWidget> FNiagaraDataInterfaceCurveDetailsBase::GetCurveToCopyMenu()
{
	FTopLevelAssetPath ClassName = GetSupportedAssetClassName();
	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &FNiagaraDataInterfaceCurveDetailsBase::CurveToCopySelected);
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.Filter.ClassPaths.Add(ClassName);
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(5)
		.AutoHeight()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "RadioButton")
			.IsChecked_Lambda([this]() { return bCopyAssetData ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.ToolTipText(LOCTEXT("CopyDataCheckBoxTooltip", "Copies the curve data from the selected asset into the stack - \nfurther updates to the asset will not affect the imported curves"))
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bCopyAssetData = NewState == ECheckBoxState::Checked ? true : false; })
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CopyDataCheckBoxText", "Copy curve data"))
			]
		]
		+ SVerticalBox::Slot()
		.Padding(5)
		.AutoHeight()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "RadioButton")
			.IsChecked_Lambda([this]() { return bCopyAssetData ? ECheckBoxState::Unchecked : ECheckBoxState::Checked; })
			.ToolTipText(LOCTEXT("RefAssetheckBoxTooltip", "References the selected data instead of just copying the data - \nwhen the curve asset is changed and the effect is recompiled or cooked it will reflect the changes from the asset."))
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bCopyAssetData = NewState == ECheckBoxState::Unchecked ? true : false; })
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RefAssetCheckBoxText", "Create asset reference"))
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(600)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SBox)
			.WidthOverride(400.0f)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		];
}

void FNiagaraDataInterfaceCurveDetailsBase::CurveToCopySelected(const FAssetData& AssetData)
{
	ImportSelectedAsset(AssetData.GetAsset());
	FSlateApplication::Get().DismissAllMenus();
}

FReply FNiagaraDataInterfaceCurveDetailsBase::CurveTemplateSelected(TWeakObjectPtr<UCurveFloat> WeakCurveAsset, TSharedPtr<FCurveEditor> CurveEditor)
{
	UCurveFloat* FloatCurveAsset = WeakCurveAsset.Get();
	if (FloatCurveAsset != nullptr)
	{
		TArray<FCurveModelID> CurveModelIdsToSet;
		if(CurveEditor->GetRootTreeItems().Num() == 1)
		{
			const FCurveEditorTreeItem& TreeItem = CurveEditor->GetTreeItem(CurveEditor->GetRootTreeItems()[0]);
			for(const FCurveModelID& CurveModelId : TreeItem.GetCurves())
			{
				CurveModelIdsToSet.Add(CurveModelId);
			}
		}
		else
		{
			for (const TPair<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& TreeItemSelectionState : CurveEditor->GetTreeSelection())
			{
				if (TreeItemSelectionState.Value != ECurveEditorTreeSelectionState::None)
				{
					const FCurveEditorTreeItem& TreeItem = CurveEditor->GetTreeItem(TreeItemSelectionState.Key);
					for (const FCurveModelID& CurveModelId : TreeItem.GetCurves())
					{
						CurveModelIdsToSet.Add(CurveModelId);
					}
				}
			}
		}

		if (CurveModelIdsToSet.Num() > 0)
		{
			FScopedTransaction ApplyTemplateTransaction(LOCTEXT("ApplyCurveTemplateTransaction", "Apply curve template"));
			for (const FCurveModelID& CurveModelId : CurveModelIdsToSet)
			{
				FCurveModel* CurveModel = CurveEditor->GetCurves()[CurveModelId].Get();
				if (CurveModel != nullptr)
				{
					const TArray<FKeyHandle> KeyHandles = CurveModel->GetAllKeys();
					CurveModel->RemoveKeys(KeyHandles,0.0);

					const FRichCurve& FloatCurve = FloatCurveAsset->FloatCurve;
					for (auto KeyIterator = FloatCurve.GetKeyHandleIterator(); KeyIterator; ++KeyIterator)
					{
						const FRichCurveKey& Key = FloatCurve.GetKey(*KeyIterator);
						FKeyPosition KeyPosition;
						KeyPosition.InputValue = Key.Time;
						KeyPosition.OutputValue = Key.Value;
						FKeyAttributes KeyAttributes;
						KeyAttributes.SetInterpMode(Key.InterpMode);
						KeyAttributes.SetTangentMode(Key.TangentMode);
						KeyAttributes.SetArriveTangent(Key.ArriveTangent);
						KeyAttributes.SetLeaveTangent(Key.LeaveTangent);
						CurveModel->AddKey(KeyPosition, KeyAttributes);
					}
				}
			}
			CurveEditor->ZoomToFit();
		}
	}
	return FReply::Handled();
}

// Curve
TSharedRef<IDetailCustomization> FNiagaraDataInterfaceCurveDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceCurveDetails>();
}

void FNiagaraDataInterfaceCurveDetails::GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& CurveProperties) const
{
	CurveProperties.Add(DetailBuilder.GetProperty(FName("Curve"), UNiagaraDataInterfaceCurve::StaticClass()));
}

FTopLevelAssetPath FNiagaraDataInterfaceCurveDetails::GetSupportedAssetClassName() const
{
	return UCurveFloat::StaticClass()->GetClassPathName();
}

void FNiagaraDataInterfaceCurveDetails::GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const
{
	UCurveFloat* CurveAsset = Cast<UCurveFloat>(SelectedAsset);
	FloatCurves.Add(CurveAsset->FloatCurve);
}

// Vector 2D Curve
TSharedRef<IDetailCustomization> FNiagaraDataInterfaceVector2DCurveDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceVector2DCurveDetails>();
}

void FNiagaraDataInterfaceVector2DCurveDetails::GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const
{
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("XCurve"), UNiagaraDataInterfaceVector2DCurve::StaticClass()));
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("YCurve"), UNiagaraDataInterfaceVector2DCurve::StaticClass()));
}

FTopLevelAssetPath FNiagaraDataInterfaceVector2DCurveDetails::GetSupportedAssetClassName() const
{
	return UCurveVector::StaticClass()->GetClassPathName();
}

void FNiagaraDataInterfaceVector2DCurveDetails::GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const
{
	UCurveVector* CurveAsset = Cast<UCurveVector>(SelectedAsset);
	for (int i = 0; i < 2; i++)
	{
		FloatCurves.Add(CurveAsset->FloatCurves[i]);
	}
}


// Vector Curve
TSharedRef<IDetailCustomization> FNiagaraDataInterfaceVectorCurveDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceVectorCurveDetails>();
}

void FNiagaraDataInterfaceVectorCurveDetails::GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const
{
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("XCurve"), UNiagaraDataInterfaceVectorCurve::StaticClass()));
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("YCurve"), UNiagaraDataInterfaceVectorCurve::StaticClass()));
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("ZCurve"), UNiagaraDataInterfaceVectorCurve::StaticClass()));
}

FTopLevelAssetPath FNiagaraDataInterfaceVectorCurveDetails::GetSupportedAssetClassName() const
{
	return UCurveVector::StaticClass()->GetClassPathName();
}

void FNiagaraDataInterfaceVectorCurveDetails::GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const
{
	UCurveVector* CurveAsset = Cast<UCurveVector>(SelectedAsset);
	for (int i = 0; i < 3; i++)
	{
		FloatCurves.Add(CurveAsset->FloatCurves[i]);
	}
}


// Vector 4 Curve
TSharedRef<IDetailCustomization> FNiagaraDataInterfaceVector4CurveDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceVector4CurveDetails>();
}

void FNiagaraDataInterfaceVector4CurveDetails::GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const
{
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("XCurve"), UNiagaraDataInterfaceVector4Curve::StaticClass()));
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("YCurve"), UNiagaraDataInterfaceVector4Curve::StaticClass()));
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("ZCurve"), UNiagaraDataInterfaceVector4Curve::StaticClass()));
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("WCurve"), UNiagaraDataInterfaceVector4Curve::StaticClass()));
}

FTopLevelAssetPath FNiagaraDataInterfaceVector4CurveDetails::GetSupportedAssetClassName() const
{
	return UCurveLinearColor::StaticClass()->GetClassPathName();
}

void FNiagaraDataInterfaceVector4CurveDetails::GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const
{
	UCurveLinearColor* CurveAsset = Cast<UCurveLinearColor>(SelectedAsset);
	for (int i = 0; i < 4; i++)
	{
		FloatCurves.Add(CurveAsset->FloatCurves[i]);
	}
}

// Color Curve
TSharedRef<IDetailCustomization> FNiagaraDataInterfaceColorCurveDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceColorCurveDetails>();
}

void FNiagaraDataInterfaceColorCurveDetails::GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const
{
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("RedCurve"), UNiagaraDataInterfaceColorCurve::StaticClass()));
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("GreenCurve"), UNiagaraDataInterfaceColorCurve::StaticClass()));
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("BlueCurve"), UNiagaraDataInterfaceColorCurve::StaticClass()));
	OutCurveProperties.Add(DetailBuilder.GetProperty(FName("AlphaCurve"), UNiagaraDataInterfaceColorCurve::StaticClass()));
}

FTopLevelAssetPath FNiagaraDataInterfaceColorCurveDetails::GetSupportedAssetClassName() const
{
	return UCurveLinearColor::StaticClass()->GetClassPathName();
}

void FNiagaraDataInterfaceColorCurveDetails::GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const
{
	UCurveLinearColor* CurveAsset = Cast<UCurveLinearColor>(SelectedAsset);
	for (int i = 0; i < 4; i++)
	{
		FloatCurves.Add(CurveAsset->FloatCurves[i]);
	}
}
#undef LOCTEXT_NAMESPACE