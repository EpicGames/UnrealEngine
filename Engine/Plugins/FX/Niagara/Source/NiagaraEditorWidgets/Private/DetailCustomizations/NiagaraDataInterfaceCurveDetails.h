// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "NiagaraDataInterfaceDetails.h"
#include "Styling/SlateTypes.h"
#include "Input/Reply.h"

class IPropertyHandle;
class UNiagaraDataInterfaceCurveBase;
class FNiagaraStackCurveEditorOptions;
class SWidget;
class UCurveBase;
class UCurveFloat;
struct FAssetData;
struct FRichCurve;

/** Base details customization for curve data interfaces. */
class FNiagaraDataInterfaceCurveDetailsBase : public FNiagaraDataInterfaceDetailsBase
{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	virtual void GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const = 0;
	virtual bool GetIsColorCurve() const { return false; }
	virtual float GetDefaultHeight() const { return 150; }
	virtual UCurveBase* GetUsedCurveAsset(UNiagaraDataInterfaceCurveBase* CurveDI) const;
	
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const = 0;
	virtual void ImportSelectedAsset(UObject* SelectedAsset);
	virtual void GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const = 0;
	
protected:
	IDetailLayoutBuilder* CustomDetailBuilder = nullptr;

private:
	virtual TSharedRef<SWidget> GetCurveToCopyMenu();
	void SetGradientVisibility(bool bInShowGradient);
	int32 GetGradientCurvesSwitcherIndex() const;
	bool IsGradientVisible() const;
	void OnShowInCurveEditor() const;
	void CurveToCopySelected(const FAssetData& AssetData);
	FReply CurveTemplateSelected(TWeakObjectPtr<UCurveFloat> WeakCurveAsset, TSharedPtr<class FCurveEditor> CurveEditor);
	TWeakObjectPtr<UNiagaraDataInterfaceCurveBase> CurveDataInterfaceWeak;
	TWeakPtr<FNiagaraStackCurveEditorOptions> StackCurveEditorOptionsWeak;
	bool bCopyAssetData = true;
};

/** Details customization for curve data interfaces. */
class FNiagaraDataInterfaceCurveDetails : public FNiagaraDataInterfaceCurveDetailsBase
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	virtual void GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const override;
	virtual float GetDefaultHeight() const override { return 130; }
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const override;
	virtual void GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const override;
};


/** Details customization for vector 2d curve data interfaces. */
class FNiagaraDataInterfaceVector2DCurveDetails : public FNiagaraDataInterfaceCurveDetailsBase
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	virtual void GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const override;
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const override;
	virtual void GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const override;
};

/** Details customization for vector curve data interfaces. */
class FNiagaraDataInterfaceVectorCurveDetails : public FNiagaraDataInterfaceCurveDetailsBase
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	virtual void GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const override;
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const override;
	virtual void GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const override;
};

/** Details customization for vector 4 curve data interfaces. */
class FNiagaraDataInterfaceVector4CurveDetails : public FNiagaraDataInterfaceCurveDetailsBase
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	virtual void GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const override;
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const override;
	virtual void GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const override;
};

/** Details customization for color curve data interfaces. */
class FNiagaraDataInterfaceColorCurveDetails : public FNiagaraDataInterfaceCurveDetailsBase
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	virtual void GetCurveProperties(IDetailLayoutBuilder& DetailBuilder, TArray<TSharedRef<IPropertyHandle>>& OutCurveProperties) const override;
	virtual bool GetIsColorCurve() const override { return true; }
	virtual FTopLevelAssetPath GetSupportedAssetClassName() const override;
	virtual void GetFloatCurvesFromAsset(UObject* SelectedAsset, TArray<FRichCurve>& FloatCurves) const override;
};