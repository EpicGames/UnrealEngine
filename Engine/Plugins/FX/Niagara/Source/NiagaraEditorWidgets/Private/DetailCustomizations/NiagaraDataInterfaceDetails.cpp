// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceDetails.h"
#include "IDetailCustomization.h"
#include "NiagaraDataInterface.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"
#include "NiagaraEditorStyle.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Images/SImage.h"
#include "PropertyHandle.h"
#include "IPropertyUtilities.h"
#include "Misc/NotifyHook.h"
#include "IDetailChildrenBuilder.h"
#include "NiagaraEditorModule.h"
#include "NiagaraVariableMetaData.h"

#define LOCTEXT_NAMESPACE "FNiagaraDataInterfaceDetailsBase"
#define ErrorsCategoryName  TEXT("Errors")

class SNiagaraDataInterfaceError : public SCompoundWidget
{
public:
	DECLARE_DELEGATE(FOnFixTriggered);

public:
	SLATE_BEGIN_ARGS(SNiagaraDataInterfaceError){}
		SLATE_EVENT(FOnFixTriggered, OnFixTriggered)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraDataInterface* InDataInterface, FNiagaraDataInterfaceError InError)
	{
		OnFixTriggered = InArgs._OnFixTriggered;
		
		Error = InError;
		DataInterface = InDataInterface;

		TSharedRef<SHorizontalBox> ErrorBox = SNew(SHorizontalBox);
		
		ErrorBox->AddSlot()
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				.ToolTipText(GetErrorTextTooltip())
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Error"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
					.Clipping(EWidgetClipping::ClipToBounds)
					.Text(GetErrorSummary())
				]
			];
		if (Error.GetErrorFixable())
		{
			ErrorBox->AddSlot()
			.VAlign(VAlign_Top)
			.Padding(5, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SNiagaraDataInterfaceError::OnFixNowClicked)
				.ToolTipText(NSLOCTEXT("NiagaraDataInterfaceError", "FixButtonLabelToolTip", "Fix the data linked to this interface."))
				.Content()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("NiagaraDataInterfaceError", "FixButtonLabel", "Fix Now"))
				]
			];
		}
		ChildSlot
		[
			ErrorBox
		];
	}

private:
	FText GetErrorSummary() const
	{
		return Error.GetErrorSummaryText();
	}

	FText GetErrorTextTooltip() const
	{
		return Error.GetErrorText();
	}

	FReply OnFixNowClicked()
	{
		FScopedTransaction ScopedTransaction(NSLOCTEXT("NiagaraDataInterfaceDetails", "FixDataInterfaceTransaction", "Fix asset for data interface"));
		OnFixTriggered.ExecuteIfBound();
		Error.TryFixError();
		DataInterface->PostEditChange();
		return FReply::Handled();
	}

private:
	FNiagaraDataInterfaceError Error;
	IDetailLayoutBuilder* DetailBuilder;
	UNiagaraDataInterface* DataInterface;
	FSimpleDelegate OnFixTriggered;
};

class SNiagaraDataInterfaceFeedback : public SCompoundWidget
{
public:
	DECLARE_DELEGATE(FOnFixTriggered);

public:
	SLATE_BEGIN_ARGS(SNiagaraDataInterfaceFeedback) {}
		SLATE_EVENT(FOnFixTriggered, OnFixTriggered)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UNiagaraDataInterface* InDataInterface, FNiagaraDataInterfaceFeedback InFeedback, bool bInWarning)
	{
		OnFixTriggered = InArgs._OnFixTriggered;

		bWarning = bInWarning;
		Feedback = InFeedback;
		DataInterface = InDataInterface;

		TSharedRef<SHorizontalBox> FeedbackBox = SNew(SHorizontalBox);

		FeedbackBox->AddSlot()
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				.ToolTipText(GetFeedbackTextTooltip())
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(bWarning ? "Icons.Warning" : "Icons.Info"))
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.TextStyle(FNiagaraEditorStyle::Get(), "NiagaraEditor.ParameterText")
			.Clipping(EWidgetClipping::ClipToBounds)
			.Text(GetFeedbackSummary())
			]
			];
		if (Feedback.GetFeedbackFixable())
		{
			FeedbackBox->AddSlot()
				.VAlign(VAlign_Top)
				.Padding(5, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(this, &SNiagaraDataInterfaceFeedback::OnFixNowClicked)
				.ToolTipText(NSLOCTEXT("NiagaraDataInterfaceFeedback", "FixButtonLabelToolTip", "Fix the data linked to this interface."))
				.Content()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("NiagaraDataInterfaceFeedback", "FixButtonLabel", "Fix Now"))
				]
				];
		}
		ChildSlot
			[
				FeedbackBox
			];
	}

private:
	FText GetFeedbackSummary() const
	{
		return Feedback.GetFeedbackSummaryText();
	}

	FText GetFeedbackTextTooltip() const
	{
		return Feedback.GetFeedbackText();
	}

	FReply OnFixNowClicked()
	{
		FScopedTransaction ScopedTransaction(NSLOCTEXT("NiagaraDataInterfaceDetails", "FixDataInterfaceTransaction", "Fix asset for data interface"));
		OnFixTriggered.ExecuteIfBound();
		Feedback.TryFixFeedback();
		DataInterface->PostEditChange();
		return FReply::Handled();
	}

private:
	FNiagaraDataInterfaceFeedback Feedback;
	IDetailLayoutBuilder* DetailBuilder;
	UNiagaraDataInterface* DataInterface;
	FSimpleDelegate OnFixTriggered;
	bool bWarning;
};

class FNiagaraDataInterfaceCustomNodeBuilder : public IDetailCustomNodeBuilder
											 , public TSharedFromThis<FNiagaraDataInterfaceCustomNodeBuilder>
{
public:
	FNiagaraDataInterfaceCustomNodeBuilder(TWeakPtr<IDetailLayoutBuilder> InDetailBuilder)
		: WeakDetailBuilder(MoveTemp(InDetailBuilder))
	{
	}

	void Initialize(UNiagaraDataInterface& InDataInterface)
	{
		DataInterface = &InDataInterface;
		DataInterface->OnChanged().AddSP(this, &FNiagaraDataInterfaceCustomNodeBuilder::OnDataInterfaceChanged);
		DataInterface->OnErrorsRefreshed().AddSP(this, &FNiagaraDataInterfaceCustomNodeBuilder::OnRefreshErrorsRequested);
	}

	virtual ~FNiagaraDataInterfaceCustomNodeBuilder() override
	{
		if (DataInterface.IsValid())
		{
			DataInterface->OnChanged().RemoveAll(this);
			DataInterface->OnErrorsRefreshed().RemoveAll(this);
		}
	}

	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override
	{
		OnRebuildChildren = InOnRegenerateChildren;
	}

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) {}
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual bool InitiallyCollapsed() const { return false; }

	virtual FName GetName() const  override
	{
		static const FName NiagaraDataInterfaceCustomNodeBuilder("NiagaraDataInterfaceCustomNodeBuilder");
		return NiagaraDataInterfaceCustomNodeBuilder;
	}

	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override
	{
		TArray<FNiagaraDataInterfaceError> Errors;
		TArray<FNiagaraDataInterfaceFeedback> Warnings;
		TArray<FNiagaraDataInterfaceFeedback> Infos;
		FNiagaraEditorModule::Get().GetDataInterfaceFeedbackSafe(DataInterface.Get(), Errors, Warnings, Infos);

		for (FNiagaraDataInterfaceError Error : Errors)
		{
			FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(NSLOCTEXT("NiagaraDataInterfaceDetails", "DataError", "Data Error"));
			Row.WholeRowContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SNiagaraDataInterfaceError, DataInterface.Get(), Error)
					.OnFixTriggered(this, &FNiagaraDataInterfaceCustomNodeBuilder::OnErrorFixTriggered)
				]
				];
		}
		for (FNiagaraDataInterfaceFeedback Warning : Warnings)
		{
			FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(NSLOCTEXT("NiagaraDataInterfaceDetails", "DataWarning", "Data Warning"));
			Row.WholeRowContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SNiagaraDataInterfaceFeedback, DataInterface.Get(), Warning, true)
					.OnFixTriggered(this, &FNiagaraDataInterfaceCustomNodeBuilder::OnErrorFixTriggered)
				]
				];
		}
		for (FNiagaraDataInterfaceFeedback Info : Infos)
		{
			FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(NSLOCTEXT("NiagaraDataInterfaceDetails", "DataInfo", "Data Info"));
			Row.WholeRowContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SNiagaraDataInterfaceFeedback, DataInterface.Get(), Info, false)
					.OnFixTriggered(this, &FNiagaraDataInterfaceCustomNodeBuilder::OnErrorFixTriggered)
				]
				];
		}
	}
private:
	void OnDataInterfaceChanged()
	{
		OnRebuildChildren.ExecuteIfBound();
	}

	void OnRefreshErrorsRequested()
	{
		OnRebuildChildren.ExecuteIfBound();
	}

	void OnErrorFixTriggered()
	{
		FProperty* PropertyPlaceholder = nullptr;  // we don't need to specify the property, all we need is to trigger the restart of the emitter
		FPropertyChangedEvent ChangeEvent(PropertyPlaceholder, EPropertyChangeType::Unspecified);
		if (TSharedPtr<IDetailLayoutBuilder> DetailBuilder = WeakDetailBuilder.Pin())
		{
			if (DetailBuilder->GetPropertyUtilities()->GetNotifyHook() != nullptr)
			{
				DetailBuilder->GetPropertyUtilities()->GetNotifyHook()->NotifyPostChange(ChangeEvent, PropertyPlaceholder);
			}
		}
	}

private:
	TWeakObjectPtr<UNiagaraDataInterface> DataInterface;
	TWeakPtr<IDetailLayoutBuilder> WeakDetailBuilder;
	FSimpleDelegate OnRebuildChildren;
};

void FNiagaraDataInterfaceDetailsBase::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{	
	PropertyUtilitiesWeak = DetailBuilder.GetPropertyUtilities();
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	check(SelectedObjects.Num() == 1);
	DataInterface = Cast<UNiagaraDataInterface>(SelectedObjects[0].Get());
	check(DataInterface.IsValid());
	DataInterface->OnErrorsRefreshed().AddSP(this, &FNiagaraDataInterfaceDetailsBase::OnErrorsRefreshed);
	IDetailCategoryBuilder& ErrorsBuilderRef = DetailBuilder.EditCategory(ErrorsCategoryName, LOCTEXT("ErrorsAndWarnings", "Errors And Warnings"), ECategoryPriority::Important);
	ErrorsCategoryBuilder = &ErrorsBuilderRef;
	CustomBuilder = MakeShared<FNiagaraDataInterfaceCustomNodeBuilder>(Builder);
	CustomBuilder->Initialize(*DataInterface);
	ErrorsCategoryBuilder->AddCustomBuilder(CustomBuilder.ToSharedRef());
	OnErrorsRefreshed();
}

void FNiagaraDataInterfaceDetailsBase::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	Builder = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

void FNiagaraDataInterfaceDetailsBase::OnErrorsRefreshed() // need to only refresh errors, and all will be good
{
	TSharedPtr<IPropertyUtilities> PropertyUtilities = PropertyUtilitiesWeak.Pin();
	bool bStillValid =
		DataInterface.IsValid() &&
		Builder.IsValid() &&
		PropertyUtilities.IsValid() &&
		PropertyUtilities->GetSelectedObjects().Num() == 1 &&
		PropertyUtilities->GetSelectedObjects()[0].IsValid() &&
		PropertyUtilities->GetSelectedObjects()[0].Get() == DataInterface.Get();

	if (bStillValid)
	{
		TArray<FNiagaraDataInterfaceError> Errors;
		TArray<FNiagaraDataInterfaceFeedback> Warnings;
		TArray<FNiagaraDataInterfaceFeedback> Info;
		FNiagaraEditorModule::Get().GetDataInterfaceFeedbackSafe(DataInterface.Get(), Errors, Warnings, Info);

		int CurrentErrorCount = Errors.Num() + Warnings.Num() + Info.Num();
		ErrorsCategoryBuilder->SetCategoryVisibility(CurrentErrorCount > 0);
	}
}

FNiagaraDataInterfaceDetailsBase::~FNiagaraDataInterfaceDetailsBase()
{
	if (DataInterface.IsValid())
	{
		DataInterface->OnChanged().RemoveAll(this);
	}
}

TSharedRef<IDetailCustomization> FNiagaraDataInterfaceDetailsBase::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceDetailsBase>();
}


#undef LOCTEXT_NAMESPACE