// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class UNiagaraDataInterface;
class IDetailCategoryBuilder;
class FNiagaraDataInterfaceCustomNodeBuilder;
class IPropertyUtilities;

/** Base details customization for Niagara data interfaces. */
class FNiagaraDataInterfaceDetailsBase : public IDetailCustomization
{
public:
	~FNiagaraDataInterfaceDetailsBase();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	void OnErrorsRefreshed();

private:
	TWeakObjectPtr<UNiagaraDataInterface> DataInterface;
	TSharedPtr<FNiagaraDataInterfaceCustomNodeBuilder> CustomBuilder;
	IDetailCategoryBuilder* ErrorsCategoryBuilder;
	TWeakPtr<IDetailLayoutBuilder> Builder;
	TWeakPtr<IPropertyUtilities> PropertyUtilitiesWeak;
};
