// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceGrid2DCollectionDetails.h"
#include "NiagaraDetailSourcedArrayBuilder.h"
#include "NiagaraDataInterfaceDetails.h"
#include "NiagaraDataInterfaceGrid2DCollection.h" 
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailGroup.h"
#include "NiagaraComponent.h"
#include "NiagaraConstants.h"
#include "NiagaraNodeInput.h"
#include "NiagaraSettings.h"
#include "SNiagaraNamePropertySelector.h"
#include "NiagaraEditorUtilities.h"

#define LOCTEXT_NAMESPACE "FNiagaraDataInterfaceGrid2DCollectionDetails"

FNiagaraDataInterfaceGrid2DCollectionDetails::~FNiagaraDataInterfaceGrid2DCollectionDetails()
{
}

void FNiagaraDataInterfaceGrid2DCollectionDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	static const FName GridCollectionCategoryName = TEXT("Grid");

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	if (SelectedObjects.Num() != 1 || SelectedObjects[0]->IsA<UNiagaraDataInterfaceGrid2DCollection>() == false)
	{
		return;
	}

	UNiagaraDataInterfaceGrid2DCollection* Grid2DInterface = CastChecked<UNiagaraDataInterfaceGrid2DCollection>(SelectedObjects[0].Get());
	Grid2DInterfacePtr = Grid2DInterface;

	Grid2DInterface->OnChanged().RemoveAll(this);
	Grid2DInterface->OnChanged().AddSP(this, &FNiagaraDataInterfaceGrid2DCollectionDetails::OnInterfaceChanged);

	IDetailCategoryBuilder* GridCollectionCategory = &DetailBuilder.EditCategory(GridCollectionCategoryName);
	{
		TArray<TSharedRef<IPropertyHandle>> Properties;
		GridCollectionCategory->GetDefaultProperties(Properties, true, true);

		for (TSharedPtr<IPropertyHandle> Property : Properties)
		{
			FProperty* PropertyPtr = Property->GetProperty();

			if (PropertyPtr->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceGrid2DCollection, PreviewAttribute))
			{
				TArray<TSharedPtr<FName>> PossibleNames;
				GeneratePreviewAttributes(PossibleNames);
				PreviewAttributesBuilder = SNew(SNiagaraNamePropertySelector, Property.ToSharedRef(), PossibleNames);

				IDetailPropertyRow& PropertyRow = GridCollectionCategory->AddProperty(Property);
				PropertyRow.CustomWidget(false)
					.NameContent()
					[
						Property->CreatePropertyNameWidget()
					]
					.ValueContent()
					.MaxDesiredWidth(TOptional<float>())
					[
						PreviewAttributesBuilder.ToSharedRef()
					];
			}
			else if (PropertyPtr->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceGrid2DCollection, OverrideBufferFormat))
			{
				IDetailPropertyRow& PropertyRow = GridCollectionCategory->AddProperty(Property);
				PropertyRow.CustomWidget(false)
				.NameContent()
				[
					Property->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Visibility(this, &FNiagaraDataInterfaceGrid2DCollectionDetails::IsOverideFormatInvisibile)
						.Text(this, &FNiagaraDataInterfaceGrid2DCollectionDetails::GetDefaultFormatText)
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SBox)
						.Visibility(this, &FNiagaraDataInterfaceGrid2DCollectionDetails::IsOverideFormatVisibile)
						[
							Property->CreatePropertyValueWidget()
						]
					]
				];
			}
			else
			{
				GridCollectionCategory->AddProperty(Property);
			}
		}
	}
}

TSharedRef<IDetailCustomization> FNiagaraDataInterfaceGrid2DCollectionDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceGrid2DCollectionDetails>();
}

void FNiagaraDataInterfaceGrid2DCollectionDetails::OnInterfaceChanged()
{
	OnDataChanged();
}

void FNiagaraDataInterfaceGrid2DCollectionDetails::OnDataChanged()
{
	if (PreviewAttributesBuilder)
	{
		TArray<TSharedPtr<FName>> PossibleNames;
		GeneratePreviewAttributes(PossibleNames);
		PreviewAttributesBuilder->SetSourceArray(PossibleNames);
	}
}

void FNiagaraDataInterfaceGrid2DCollectionDetails::GeneratePreviewAttributes(TArray<TSharedPtr<FName>>& SourceArray)
{
	SourceArray.Reset();
	SourceArray.Add(MakeShared<FName>(NAME_None));

	if ( UNiagaraDataInterfaceGrid2DCollection* Grid2DInterface = Grid2DInterfacePtr.Get() )
	{
		// Discover attributes by name
		FName VariableName;
		if (UNiagaraNodeInput* NodeInput = Cast<UNiagaraNodeInput>(Grid2DInterface->GetOuter()))
		{
			VariableName = NodeInput->Input.GetName();
		}

		if ( !VariableName.IsNone() )
		{
			// Resolve namespace
			if (UNiagaraEmitter* OwnerEmitter = Grid2DInterface->GetTypedOuter<UNiagaraEmitter>())
			{
				FNiagaraAliasContext ResolveAliasesContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript);
				ResolveAliasesContext.ChangeEmitterToEmitterName(OwnerEmitter->GetUniqueEmitterName());
				VariableName = FNiagaraUtilities::ResolveAliases(FNiagaraVariable(UNiagaraDataInterfaceGrid2DCollection::StaticClass(), VariableName), ResolveAliasesContext).GetName();
			}

			// Add named attributes
			TArray<FNiagaraVariableBase> FoundVariables;
			TArray<uint32> FoundVariableOffsets;
			int32 FoundNumAttribChannelsFound;

			// Finding attributes requires that we use the runtime instance of the data interface, so get that here.
			UNiagaraSystem* OwningSystem = Grid2DInterface->GetTypedOuter<UNiagaraSystem>();
			UNiagaraDataInterfaceGrid2DCollection* RuntimeGrid2DInterface = OwningSystem != nullptr 
				? Cast<UNiagaraDataInterfaceGrid2DCollection>(FNiagaraEditorUtilities::GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(*OwningSystem, *Grid2DInterface))
				: nullptr;
			if (RuntimeGrid2DInterface != nullptr)
			{
				RuntimeGrid2DInterface->FindAttributes(FoundVariables, FoundVariableOffsets, FoundNumAttribChannelsFound);
			}

			for (const FNiagaraVariableBase& Variable : FoundVariables)
			{
				SourceArray.Add(MakeShared<FName>(FName(Variable.GetName())));
			}
		}

		// Add anonymous attributes
		for (int32 i = 0; i < Grid2DInterface->NumAttributes; ++i)
		{
			FString AttributeName = FString::Printf(TEXT("%s %d"), *UNiagaraDataInterfaceGrid2DCollection::AnonymousAttributeString, i);
			SourceArray.Add(MakeShared<FName>(FName(*AttributeName)));
		}
	}
}

EVisibility FNiagaraDataInterfaceGrid2DCollectionDetails::IsOverideFormatVisibile() const
{
	UNiagaraDataInterfaceGrid2DCollection* Grid2D = Grid2DInterfacePtr.Get();
	return Grid2D && Grid2D->bOverrideFormat ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FNiagaraDataInterfaceGrid2DCollectionDetails::IsOverideFormatInvisibile() const
{
	UNiagaraDataInterfaceGrid2DCollection* Grid2D = Grid2DInterfacePtr.Get();
	return Grid2D && !Grid2D->bOverrideFormat ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FNiagaraDataInterfaceGrid2DCollectionDetails::GetDefaultFormatText() const
{
	return FText::Format(
		LOCTEXT("DefaultGridFormatFormat", "Using Project Default Format - {0}"),
		StaticEnum<ENiagaraGpuBufferFormat>()->GetDisplayValueAsText(GetDefault<UNiagaraSettings>()->DefaultGridFormat)
	);
}

#undef LOCTEXT_NAMESPACE
