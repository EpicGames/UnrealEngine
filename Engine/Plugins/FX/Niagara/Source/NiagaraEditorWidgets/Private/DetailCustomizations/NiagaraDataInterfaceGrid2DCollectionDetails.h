// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceDetails.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"

class IDetailLayoutBuilder;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SWidget;
class FName;
class FNiagaraDetailSourcedArrayBuilder;
class UNiagaraDataInterfaceGrid2DCollection;
class USkeletalMesh;
class SNiagaraNamePropertySelector;

/** Details customization for Niagara Grid2D Collection data interface. */
class FNiagaraDataInterfaceGrid2DCollectionDetails : public FNiagaraDataInterfaceDetailsBase
{
public:
	~FNiagaraDataInterfaceGrid2DCollectionDetails();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	void OnInterfaceChanged();
	void OnDataChanged();
	void GeneratePreviewAttributes(TArray<TSharedPtr<FName>>& SourceArray);

	EVisibility IsOverideFormatVisibile() const;
	EVisibility IsOverideFormatInvisibile() const;
	FText GetDefaultFormatText() const;

private:
	TWeakObjectPtr<UNiagaraDataInterfaceGrid2DCollection> Grid2DInterfacePtr;
	TSharedPtr<SNiagaraNamePropertySelector> PreviewAttributesBuilder;
};
