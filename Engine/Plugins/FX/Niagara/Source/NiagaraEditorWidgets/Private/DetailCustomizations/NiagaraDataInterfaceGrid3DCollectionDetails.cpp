// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceGrid3DCollectionDetails.h"
#include "NiagaraDetailSourcedArrayBuilder.h"
#include "NiagaraDataInterfaceDetails.h"
#include "NiagaraDataInterfaceGrid3DCollection.h" 
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailGroup.h"
#include "NiagaraComponent.h"
#include "NiagaraConstants.h"
#include "NiagaraNodeInput.h"
#include "NiagaraSettings.h"
#include "SNiagaraNamePropertySelector.h"
#include "NiagaraEditorUtilities.h"

#define LOCTEXT_NAMESPACE "FNiagaraDataInterfaceGrid3DCollectionDetails"

FNiagaraDataInterfaceGrid3DCollectionDetails::~FNiagaraDataInterfaceGrid3DCollectionDetails()
{
}

void FNiagaraDataInterfaceGrid3DCollectionDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	static const FName GridCollectionCategoryName = TEXT("Grid");

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	if (SelectedObjects.Num() != 1 || SelectedObjects[0]->IsA<UNiagaraDataInterfaceGrid3DCollection>() == false)
	{
		return;
	}

	UNiagaraDataInterfaceGrid3DCollection* Grid3DInterface = CastChecked<UNiagaraDataInterfaceGrid3DCollection>(SelectedObjects[0].Get());
	Grid3DInterfacePtr = Grid3DInterface;

	Grid3DInterface->OnChanged().RemoveAll(this);
	Grid3DInterface->OnChanged().AddSP(this, &FNiagaraDataInterfaceGrid3DCollectionDetails::OnInterfaceChanged);

	IDetailCategoryBuilder* GridCollectionCategory = &DetailBuilder.EditCategory(GridCollectionCategoryName);
	{
		TArray<TSharedRef<IPropertyHandle>> Properties;
		GridCollectionCategory->GetDefaultProperties(Properties, true, true);

		for (TSharedPtr<IPropertyHandle> Property : Properties)
		{
			FProperty* PropertyPtr = Property->GetProperty();

			if (PropertyPtr->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceGrid3DCollection, PreviewAttribute))
			{
				TArray<TSharedPtr<FName>> PossibleNames;
				GeneratePreviewAttributes(PossibleNames);
				PreviewAttributesBuilder = SNew(SNiagaraNamePropertySelector, Property.ToSharedRef(), PossibleNames);

				IDetailPropertyRow& PropertyRow = GridCollectionCategory->AddProperty(Property);
				PropertyRow.CustomWidget(false)
					.NameContent()
					[
						Property->CreatePropertyNameWidget()
					]
					.ValueContent()
					.MaxDesiredWidth(TOptional<float>())
					[
						PreviewAttributesBuilder.ToSharedRef()
					];
			}
			else if (PropertyPtr->GetFName() == GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceGrid3DCollection, OverrideBufferFormat))
			{
				IDetailPropertyRow& PropertyRow = GridCollectionCategory->AddProperty(Property);
				PropertyRow.CustomWidget(false)
				.NameContent()
				[
					Property->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Visibility(this, &FNiagaraDataInterfaceGrid3DCollectionDetails::IsOverideFormatInvisibile)
						.Text(this, &FNiagaraDataInterfaceGrid3DCollectionDetails::GetDefaultFormatText)
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SBox)
						.Visibility(this, &FNiagaraDataInterfaceGrid3DCollectionDetails::IsOverideFormatVisibile)
						[
							Property->CreatePropertyValueWidget()
						]
					]
				];
			}
			else
			{
				GridCollectionCategory->AddProperty(Property);
			}
		}
	}
}

TSharedRef<IDetailCustomization> FNiagaraDataInterfaceGrid3DCollectionDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceGrid3DCollectionDetails>();
}

void FNiagaraDataInterfaceGrid3DCollectionDetails::OnInterfaceChanged()
{
	OnDataChanged();
}

void FNiagaraDataInterfaceGrid3DCollectionDetails::OnDataChanged()
{
	if (PreviewAttributesBuilder)
	{
		TArray<TSharedPtr<FName>> PossibleNames;
		GeneratePreviewAttributes(PossibleNames);
		PreviewAttributesBuilder->SetSourceArray(PossibleNames);
	}
}

void FNiagaraDataInterfaceGrid3DCollectionDetails::GeneratePreviewAttributes(TArray<TSharedPtr<FName>>& SourceArray)
{
	SourceArray.Reset();
	SourceArray.Add(MakeShared<FName>(NAME_None));

	if ( UNiagaraDataInterfaceGrid3DCollection* Grid3DInterface = Grid3DInterfacePtr.Get() )
	{
		// Discover attributes by name
		FName VariableName;
		if (UNiagaraNodeInput* NodeInput = Cast<UNiagaraNodeInput>(Grid3DInterface->GetOuter()))
		{
			VariableName = NodeInput->Input.GetName();
		}

		if ( !VariableName.IsNone() )
		{
			// Resolve namespace
			if (UNiagaraEmitter* OwnerEmitter = Grid3DInterface->GetTypedOuter<UNiagaraEmitter>())
			{
				FNiagaraAliasContext ResolveAliasesContext(FNiagaraAliasContext::ERapidIterationParameterMode::EmitterOrParticleScript);
				ResolveAliasesContext.ChangeEmitterToEmitterName(OwnerEmitter->GetUniqueEmitterName());
				VariableName = FNiagaraUtilities::ResolveAliases(FNiagaraVariable(UNiagaraDataInterfaceGrid3DCollection::StaticClass(), VariableName), ResolveAliasesContext).GetName();
			}

			// Add named attributes
			TArray<FNiagaraVariableBase> FoundVariables;
			TArray<uint32> FoundVariableOffsets;
			int32 FoundNumAttribChannelsFound;

			// Finding attributes requires that we use the runtime instance of the data interface, so get that here.
			UNiagaraSystem* OwningSystem = Grid3DInterface->GetTypedOuter<UNiagaraSystem>();
			UNiagaraDataInterfaceGrid3DCollection* RuntimeGrid3DInterface = OwningSystem != nullptr
				? Cast<UNiagaraDataInterfaceGrid3DCollection>(FNiagaraEditorUtilities::GetResolvedRuntimeInstanceForEditorDataInterfaceInstance(*OwningSystem, *Grid3DInterface))
				: nullptr;
			if (RuntimeGrid3DInterface != nullptr)
			{
				RuntimeGrid3DInterface->FindAttributes(FoundVariables, FoundVariableOffsets, FoundNumAttribChannelsFound);
			}

			for (const FNiagaraVariableBase& Variable : FoundVariables)
			{
				SourceArray.Add(MakeShared<FName>(FName(Variable.GetName())));
			}
		}

		// Add anonymous attributes
		for (int32 i = 0; i < Grid3DInterface->NumAttributes; ++i)
		{
			FString AttributeName = FString::Printf(TEXT("%s %d"), *UNiagaraDataInterfaceGrid3DCollection::AnonymousAttributeString, i);
			SourceArray.Add(MakeShared<FName>(FName(*AttributeName)));
		}
	}
}

EVisibility FNiagaraDataInterfaceGrid3DCollectionDetails::IsOverideFormatVisibile() const
{
	UNiagaraDataInterfaceGrid3DCollection* Grid3D = Grid3DInterfacePtr.Get();
	return Grid3D && Grid3D->bOverrideFormat ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FNiagaraDataInterfaceGrid3DCollectionDetails::IsOverideFormatInvisibile() const
{
	UNiagaraDataInterfaceGrid3DCollection* Grid3D = Grid3DInterfacePtr.Get();
	return Grid3D && !Grid3D->bOverrideFormat ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FNiagaraDataInterfaceGrid3DCollectionDetails::GetDefaultFormatText() const
{
	return FText::Format(
		LOCTEXT("DefaultGridFormatFormat", "Using Project Default Format - {0}"),
		StaticEnum<ENiagaraGpuBufferFormat>()->GetDisplayValueAsText(GetDefault<UNiagaraSettings>()->DefaultGridFormat)
	);
}

#undef LOCTEXT_NAMESPACE
