// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceDetails.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"

class IDetailLayoutBuilder;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SWidget;
class FName;
class FNiagaraDetailSourcedArrayBuilder;
class UNiagaraDataInterfaceGrid3DCollection;
class USkeletalMesh;
class SNiagaraNamePropertySelector;

/** Details customization for Niagara Grid3D Collection data interface. */
class FNiagaraDataInterfaceGrid3DCollectionDetails : public FNiagaraDataInterfaceDetailsBase
{
public:
	~FNiagaraDataInterfaceGrid3DCollectionDetails();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	void OnInterfaceChanged();
	void OnDataChanged();
	void GeneratePreviewAttributes(TArray<TSharedPtr<FName>>& SourceArray);

	EVisibility IsOverideFormatVisibile() const;
	EVisibility IsOverideFormatInvisibile() const;
	FText GetDefaultFormatText() const;

private:
	TWeakObjectPtr<UNiagaraDataInterfaceGrid3DCollection> Grid3DInterfacePtr;
	TSharedPtr<SNiagaraNamePropertySelector> PreviewAttributesBuilder;
};
