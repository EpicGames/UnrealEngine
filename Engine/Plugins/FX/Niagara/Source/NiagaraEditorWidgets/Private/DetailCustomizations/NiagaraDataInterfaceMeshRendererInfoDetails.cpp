// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceMeshRendererInfoDetails.h"
#include "NiagaraDetailSourcedArrayBuilder.h"
#include "NiagaraDataInterfaceDetails.h"
#include "NiagaraDataInterfaceMeshRendererInfo.h"
#include "NiagaraMeshRendererProperties.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "NiagaraComponent.h"
#include "NiagaraEmitter.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraEditorModule.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FNiagaraDataInterfaceMeshRendererInfoDetails"

FNiagaraDataInterfaceMeshRendererInfoDetails::~FNiagaraDataInterfaceMeshRendererInfoDetails()
{
	if (DataInterface.IsValid())
	{
		DataInterface->OnChanged().RemoveAll(this);
	}
}

void FNiagaraDataInterfaceMeshRendererInfoDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	LayoutBuilder = &DetailBuilder;
	FNiagaraDataInterfaceDetailsBase::CustomizeDetails(DetailBuilder);

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	if (SelectedObjects.Num() != 1 || SelectedObjects[0]->IsA<UNiagaraDataInterfaceMeshRendererInfo>() == false)
	{
		return;
	}

	UNiagaraDataInterfaceMeshRendererInfo* Interface = CastChecked<UNiagaraDataInterfaceMeshRendererInfo>(SelectedObjects[0].Get());
	DataInterface = Interface;

	Interface->OnChanged().AddSP(this, &FNiagaraDataInterfaceMeshRendererInfoDetails::OnInterfaceChanged);

	GenerateRendererList();	

	static const FName SourceCategoryName = TEXT("Source");
	SourceCategory = &DetailBuilder.EditCategory(SourceCategoryName, LOCTEXT("SourceCategory", "Source"));
	{
		TArray<TSharedRef<IPropertyHandle>> Properties;
		SourceCategory->GetDefaultProperties(Properties, true, true);

		static const FName MeshRendererPropertyName = TEXT("MeshRenderer");
		MeshRendererProperty = DetailBuilder.GetProperty(MeshRendererPropertyName);
		for (TSharedPtr<IPropertyHandle> Property : Properties)
		{
			FProperty* PropertyPtr = Property->GetProperty();
			if (PropertyPtr == MeshRendererProperty->GetProperty())
			{				
				MeshRendererWidget = SNew(SMeshRendererComboBox)
					.OptionsSource(&RendererList)
					.InitiallySelectedItem(Interface->GetMeshRenderer())
					.OnComboBoxOpening(this, &FNiagaraDataInterfaceMeshRendererInfoDetails::GenerateRendererList)
					.OnSelectionChanged(this, &FNiagaraDataInterfaceMeshRendererInfoDetails::SetSelectedRenderer)
					.OnGenerateWidget(this, &FNiagaraDataInterfaceMeshRendererInfoDetails::CreateRendererItemWidget)
					.AddMetaData<FTagMetaData>(TEXT("SelectMeshRendererCobmo"))
					[
						SNew(STextBlock)
							.Text(this, &FNiagaraDataInterfaceMeshRendererInfoDetails::GetSelectedRendererTextLabel)
					];

				IDetailPropertyRow& RendererRow = SourceCategory->AddProperty(Property);
				RendererRow.CustomWidget(false)
					.NameContent()
					[
						Property->CreatePropertyNameWidget()
					]
					.ValueContent()
					.MaxDesiredWidth(TOptional<float>())
					[
						MeshRendererWidget.ToSharedRef()
					];
			}
			else
			{
				SourceCategory->AddProperty(Property);
			}
		}
	}
}

TSharedRef<IDetailCustomization> FNiagaraDataInterfaceMeshRendererInfoDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceMeshRendererInfoDetails>();
}

void FNiagaraDataInterfaceMeshRendererInfoDetails::OnInterfaceChanged()
{
	if (!bSettingSelection && MeshRendererWidget.IsValid())
	{
		MeshRendererWidget->SetSelectedItem(GetSelectedRenderer());
	}
}

void FNiagaraDataInterfaceMeshRendererInfoDetails::GenerateRendererList()
{
	RendererList.SetNum(0, EAllowShrinking::No);
	RendererLabels.SetNum(0, EAllowShrinking::No);

	// Get a list of all mesh renderers in the emitter
	if (auto Interface = DataInterface.Get())
	{
		UNiagaraSystem* System = nullptr;
		FVersionedNiagaraEmitter EmitterHandle;
		FNiagaraEditorModule::Get().GetTargetSystemAndEmitterForDataInterface(Interface, System, EmitterHandle);
		if (System && EmitterHandle.GetEmitterData())
		{
			for (auto RendererProps : EmitterHandle.GetEmitterData()->GetRenderers())
			{
				if (auto MeshProps = Cast<UNiagaraMeshRendererProperties>(RendererProps))
				{
					RendererList.Add(MeshProps);
					RendererLabels.Add(CreateRendererTextLabel(MeshProps));
				}
			}

			if(MeshRendererWidget.IsValid())
			{
				MeshRendererWidget->RefreshOptions();
			}
		}
	}
}

UNiagaraMeshRendererProperties* FNiagaraDataInterfaceMeshRendererInfoDetails::GetSelectedRenderer() const
{
	UObject* Value = nullptr;
	if (MeshRendererProperty.IsValid())
	{
		MeshRendererProperty->GetValue(Value);
	}
	return Cast<UNiagaraMeshRendererProperties>(Value);
}

void FNiagaraDataInterfaceMeshRendererInfoDetails::SetSelectedRenderer(TRendererPtr Selection, ESelectInfo::Type)
{
	if (MeshRendererProperty.IsValid() && DataInterface.IsValid())
	{
		bSettingSelection = true;
		UObject* Obj = nullptr;
		MeshRendererProperty->GetValue(Obj);
		if (Obj != Selection.Get())
		{
			const FScopedTransaction Transaction(NSLOCTEXT("FNiagaraDataInterfaceMeshRendererInfoDetails", "Change Mesh Renderer", "Change Mesh Renderer"));
			DataInterface->Modify();
			MeshRendererProperty->NotifyPreChange();
			MeshRendererProperty->SetValue(Selection.Get());
			MeshRendererProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
		bSettingSelection = false;
	}	
}

FText FNiagaraDataInterfaceMeshRendererInfoDetails::GetRendererTextLabel(TRendererPtr Renderer) const
{
	int32 RendererIndex = Renderer.IsValid() ? RendererList.Find(Renderer) : INDEX_NONE;
	if (!RendererLabels.IsValidIndex(RendererIndex))
	{
		return LOCTEXT("NoneOption", "None");
	}
	return RendererLabels[RendererIndex];
}

TSharedRef<SWidget> FNiagaraDataInterfaceMeshRendererInfoDetails::CreateRendererItemWidget(TRendererPtr Item)
{
	return SNew(STextBlock)
		.Text(this, &FNiagaraDataInterfaceMeshRendererInfoDetails::GetRendererTextLabel, Item);
}

FText FNiagaraDataInterfaceMeshRendererInfoDetails::CreateRendererTextLabel(const UNiagaraMeshRendererProperties* Properties)
{
	if (Properties == nullptr)
	{
		return LOCTEXT("NoneOption", "None");
	}

	UNiagaraEmitter* Emitter = Properties->GetTypedOuter<UNiagaraEmitter>();
	check(Emitter);

	UNiagaraEmitterEditorData* EmitterEditorData = Properties->GetEmitterData() ? static_cast<UNiagaraEmitterEditorData*>(Properties->GetEmitterData()->GetEditorData()) : nullptr;
	if (EmitterEditorData == nullptr)
	{
		return LOCTEXT("NoneOption", "None");
	}
	UNiagaraStackEditorData& EmitterStackEditorData = EmitterEditorData->GetStackEditorData();
	FString RendererStackEditorDataKey = FString::Printf(TEXT("Renderer-%s"), *Properties->GetName());
	const FText* RendererDisplayName = EmitterStackEditorData.GetStackEntryDisplayName(RendererStackEditorDataKey);

	return RendererDisplayName ? *RendererDisplayName : Properties->GetWidgetDisplayName();
}

#undef LOCTEXT_NAMESPACE