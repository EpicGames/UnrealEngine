// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceDetails.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class UNiagaraDataInterfaceMeshRendererInfo;
class UNiagaraMeshRendererProperties;

/** Details customization for Niagara mesh renderer info data interface. */
class FNiagaraDataInterfaceMeshRendererInfoDetails : public FNiagaraDataInterfaceDetailsBase
{
public:
	~FNiagaraDataInterfaceMeshRendererInfoDetails();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	using TRendererPtr = TWeakObjectPtr<UNiagaraMeshRendererProperties>;
	using SMeshRendererComboBox = SComboBox<TRendererPtr>;

	void OnInterfaceChanged();
	void GenerateRendererList();
	UNiagaraMeshRendererProperties* GetSelectedRenderer() const;
	void SetSelectedRenderer(TRendererPtr Selection, ESelectInfo::Type SelectInfo);
	FText GetRendererTextLabel(TRendererPtr Renderer) const;
	FText GetSelectedRendererTextLabel() const { return CreateRendererTextLabel(GetSelectedRenderer()); }
	TSharedRef<SWidget> CreateRendererItemWidget(TRendererPtr Item);
	
	static FText CreateRendererTextLabel(const UNiagaraMeshRendererProperties* Properties);

private:

	TSharedPtr<SMeshRendererComboBox> MeshRendererWidget;
	TArray<TRendererPtr> RendererList;
	TArray<FText> RendererLabels;
	IDetailLayoutBuilder* LayoutBuilder = nullptr;
	TWeakObjectPtr<UNiagaraDataInterfaceMeshRendererInfo>  DataInterface;
	IDetailCategoryBuilder* SourceCategory = nullptr;
	TSharedPtr<IPropertyHandle> MeshRendererProperty;
	bool bSettingSelection = false;
};
