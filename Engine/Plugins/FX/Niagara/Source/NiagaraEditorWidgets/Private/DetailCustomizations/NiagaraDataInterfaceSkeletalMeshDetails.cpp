// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSkeletalMeshDetails.h"
#include "NiagaraDetailSourcedArrayBuilder.h"
#include "NiagaraDataInterfaceDetails.h"
#include "NiagaraDataInterfaceSkeletalMesh.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailGroup.h"
#include "NiagaraComponent.h"
#include "Engine/SkeletalMeshSocket.h"
#include "SNiagaraNamePropertySelector.h"

#define LOCTEXT_NAMESPACE "FNiagaraDataInterfaceSkeletalMeshDetails"

void FNiagaraDataInterfaceSkeletalMeshDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	LayoutBuilder = &DetailBuilder;
	static const FName MeshCategoryName = TEXT("Mesh");
	static const FName SkelCategoryName = TEXT("Skeleton");

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	if(SelectedObjects.Num() != 1 || SelectedObjects[0]->IsA<UNiagaraDataInterfaceSkeletalMesh>() == false)
	{
		return;
	}

	UNiagaraDataInterfaceSkeletalMesh* Interface = CastChecked<UNiagaraDataInterfaceSkeletalMesh>(SelectedObjects[0].Get());
	MeshInterface = Interface;

	Interface->OnChanged().RemoveAll(this);
	Interface->OnChanged().AddSP(this, &FNiagaraDataInterfaceSkeletalMeshDetails::OnInterfaceChanged);

	SkinnedMeshObject = Interface->GetSkinnedMesh(Cast<UNiagaraComponent>(Interface->GetOuter()));
	SkeletalMeshObject = Cast<USkeletalMesh>(SkinnedMeshObject);
	if (SkeletalMeshObject.IsValid())
	{
		SkeletalMeshObject->GetOnMeshChanged().RemoveAll(this);
		SkeletalMeshObject->GetOnMeshChanged().AddSP(this, &FNiagaraDataInterfaceSkeletalMeshDetails::OnDataChanged);
	}

	MeshCategory = &DetailBuilder.EditCategory(MeshCategoryName, LOCTEXT("Mesh", "Mesh"));
	{
		TArray<TSharedRef<IPropertyHandle>> MeshProperties;
		MeshCategory->GetDefaultProperties(MeshProperties, true, true);

		TSharedPtr<IPropertyHandle> RegionsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSkeletalMesh, SamplingRegions));

		for (TSharedPtr<IPropertyHandle> Property : MeshProperties)
		{
			FProperty* PropertyPtr = Property->GetProperty();
			TArray<TSharedPtr<FName>> PossibleNames;
			if (PropertyPtr == RegionsProperty->GetProperty())
			{
				GenerateRegionsArray(PossibleNames);
				RegionsBuilder = TSharedPtr<FNiagaraDetailSourcedArrayBuilder>(new FNiagaraDetailSourcedArrayBuilder(Property.ToSharedRef(), PossibleNames));
				MeshCategory->AddCustomBuilder(RegionsBuilder.ToSharedRef());
			}
			else
			{
				MeshCategory->AddProperty(Property);
			}
		}
	}

	SkelCategory = &DetailBuilder.EditCategory(SkelCategoryName, LOCTEXT("SkeletonCat", "Skeleton"));
	{
		TArray<TSharedRef<IPropertyHandle>> SkelProperties;
		SkelCategory->GetDefaultProperties(SkelProperties, true, true);

		TSharedPtr<IPropertyHandle> BonesProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSkeletalMesh, FilteredBones));
		TSharedPtr<IPropertyHandle> SocketsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSkeletalMesh, FilteredSockets));
		TSharedPtr<IPropertyHandle> ExcludeBoneProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSkeletalMesh, ExcludeBoneName));

		for (TSharedPtr<IPropertyHandle> Property : SkelProperties)
		{
			FProperty* PropertyPtr = Property->GetProperty();
			TArray<TSharedPtr<FName>> PossibleNames;

			if (PropertyPtr == BonesProperty->GetProperty())
			{
				GenerateBonesArray(PossibleNames);
				BonesBuilder = TSharedPtr<FNiagaraDetailSourcedArrayBuilder>(new FNiagaraDetailSourcedArrayBuilder(Property.ToSharedRef(), PossibleNames));
				SkelCategory->AddCustomBuilder(BonesBuilder.ToSharedRef());
			}
			else if (PropertyPtr == SocketsProperty->GetProperty())
			{
				GenerateSocketsArray(PossibleNames);
				SocketsBuilder = TSharedPtr<FNiagaraDetailSourcedArrayBuilder>(new FNiagaraDetailSourcedArrayBuilder(Property.ToSharedRef(), PossibleNames));
				SkelCategory->AddCustomBuilder(SocketsBuilder.ToSharedRef());
			}
			else if (PropertyPtr == ExcludeBoneProperty->GetProperty())
			{
				GenerateBonesArray(PossibleNames);
				ExcludeBoneWidget = SNew(SNiagaraNamePropertySelector, Property.ToSharedRef(), PossibleNames);

				IDetailPropertyRow& ExcludeBoneRow = SkelCategory->AddProperty(Property);
				ExcludeBoneRow.CustomWidget(false)
					.NameContent()
					[
						Property->CreatePropertyNameWidget()
					]
					.ValueContent()
					.MaxDesiredWidth(TOptional<float>())
					[
						ExcludeBoneWidget.ToSharedRef()
					];
			}
			else
			{
				SkelCategory->AddProperty(Property);
			}
		}
	}
}

TSharedRef<IDetailCustomization> FNiagaraDataInterfaceSkeletalMeshDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceSkeletalMeshDetails>();
}

void FNiagaraDataInterfaceSkeletalMeshDetails::OnInterfaceChanged()
{
	// Rebuild the data changed listener
	if (SkeletalMeshObject.IsValid())
	{
		SkeletalMeshObject->GetOnMeshChanged().RemoveAll(this);
	}

	UNiagaraDataInterfaceSkeletalMesh* Interface = MeshInterface.Get();
	SkinnedMeshObject = Interface->GetSkinnedMesh(Cast<UNiagaraComponent>(Interface->GetOuter()));
	SkeletalMeshObject = Cast<USkeletalMesh>(SkinnedMeshObject);
	if (SkeletalMeshObject.IsValid())
	{
		SkeletalMeshObject->GetOnMeshChanged().AddSP(this, &FNiagaraDataInterfaceSkeletalMeshDetails::OnDataChanged);
	}

	OnDataChanged();
}

void FNiagaraDataInterfaceSkeletalMeshDetails::OnDataChanged()
{
	if (RegionsBuilder)
	{
		TArray<TSharedPtr<FName>> PossibleNames;
		GenerateRegionsArray(PossibleNames);
		RegionsBuilder->SetSourceArray(PossibleNames);
	}

	if (BonesBuilder)
	{
		TArray<TSharedPtr<FName>> PossibleNames;
		GenerateBonesArray(PossibleNames);
		BonesBuilder->SetSourceArray(PossibleNames);
	}

	if (SocketsBuilder)
	{
		TArray<TSharedPtr<FName>> PossibleNames;
		GenerateSocketsArray(PossibleNames);
		SocketsBuilder->SetSourceArray(PossibleNames);
	}

	if (ExcludeBoneWidget)
	{
		TArray<TSharedPtr<FName>> PossibleNames;
		GenerateBonesArray(PossibleNames);
		ExcludeBoneWidget->SetSourceArray(PossibleNames);
	}
}

void FNiagaraDataInterfaceSkeletalMeshDetails::GenerateRegionsArray(TArray<TSharedPtr<FName>>& SourceArray)
{
	SourceArray.Reset();
	if (UNiagaraDataInterfaceSkeletalMesh* Interface = MeshInterface.Get())
	{
		if (USkeletalMesh* Mesh = Cast<USkeletalMesh>(Interface->GetSkinnedMesh(Cast<UNiagaraComponent>(Interface->GetOuter()))))
		{
			for (FSkeletalMeshSamplingRegion Region : Mesh->GetSamplingInfo().Regions)
			{
				SourceArray.Add(MakeShared<FName>(Region.Name));
			}
		}
	}
}

 void FNiagaraDataInterfaceSkeletalMeshDetails::GenerateBonesArray(TArray<TSharedPtr<FName>>& SourceArray)
 {
	SourceArray.Reset();
	if (UNiagaraDataInterfaceSkeletalMesh* Interface = MeshInterface.Get())
	{
		if (USkinnedAsset* Mesh = Interface->GetSkinnedMesh(Cast<UNiagaraComponent>(Interface->GetOuter())))
		{
			for (const FMeshBoneInfo& Bone : Mesh->GetRefSkeleton().GetRefBoneInfo())
			{
				SourceArray.Add(MakeShared<FName>(Bone.Name));
			}
		}
	}
}

void FNiagaraDataInterfaceSkeletalMeshDetails::GenerateSocketsArray(TArray<TSharedPtr<FName>>& SourceArray)
{
	SourceArray.Reset();
	if (MeshInterface.IsValid())
	{
		UNiagaraDataInterfaceSkeletalMesh* Interface = MeshInterface.Get();
		if (USkeletalMesh* Mesh = Cast<USkeletalMesh>(Interface->GetSkinnedMesh(Cast<UNiagaraComponent>(Interface->GetOuter()))))
		{
			for (int32 SocketIdx = 0; SocketIdx < Mesh->NumSockets(); ++SocketIdx)
			{
				const USkeletalMeshSocket* SocketInfo = Mesh->GetSocketByIndex(SocketIdx);
				SourceArray.Add(MakeShared<FName>(SocketInfo->SocketName));
			}
		}
	}
 }

FNiagaraDataInterfaceSkeletalMeshDetails::~FNiagaraDataInterfaceSkeletalMeshDetails()
{
	if (MeshInterface.IsValid())
	{
		MeshInterface->OnChanged().RemoveAll(this);
	}
	if (SkeletalMeshObject.IsValid())
	{
		SkeletalMeshObject->GetOnMeshChanged().RemoveAll(this);
	}
}

#undef LOCTEXT_NAMESPACE