// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceDetails.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"

class IDetailLayoutBuilder;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SWidget;
class FName;
class FNiagaraDetailSourcedArrayBuilder;
class UNiagaraDataInterfaceSkeletalMesh;
class USkeletalMesh;
class USkinnedAsset;
class SNiagaraNamePropertySelector;

/** Details customization for Niagara skeletal mesh data interface. */
class FNiagaraDataInterfaceSkeletalMeshDetails : public FNiagaraDataInterfaceDetailsBase
{
public:
	~FNiagaraDataInterfaceSkeletalMeshDetails();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	void OnInterfaceChanged();
	void OnDataChanged();
	void GenerateRegionsArray(TArray<TSharedPtr<FName>>& SourceArray);
	void GenerateBonesArray(TArray<TSharedPtr<FName>>& SourceArray);
	void GenerateSocketsArray(TArray<TSharedPtr<FName>>& SourceArray);

private:
	TSharedPtr<FNiagaraDetailSourcedArrayBuilder> RegionsBuilder;
	TSharedPtr<FNiagaraDetailSourcedArrayBuilder> BonesBuilder;
	TSharedPtr<FNiagaraDetailSourcedArrayBuilder> SocketsBuilder;
	TSharedPtr<SNiagaraNamePropertySelector> ExcludeBoneWidget;
	IDetailLayoutBuilder* LayoutBuilder;
	TWeakObjectPtr<UNiagaraDataInterfaceSkeletalMesh>  MeshInterface;
	TWeakObjectPtr<USkeletalMesh> SkeletalMeshObject;
	TWeakObjectPtr<USkinnedAsset> SkinnedMeshObject;

	IDetailCategoryBuilder* MeshCategory;
	IDetailCategoryBuilder* SkelCategory;
};
