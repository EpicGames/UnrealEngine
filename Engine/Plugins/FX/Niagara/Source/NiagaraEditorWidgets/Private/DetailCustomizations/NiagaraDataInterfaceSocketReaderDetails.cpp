// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSocketReaderDetails.h"
#include "DataInterface/NiagaraDataInterfaceSocketReader.h" 
#include "NiagaraDetailSourcedArrayBuilder.h"

#include "Algo/Transform.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceSocketReaderDetails"

FNiagaraDataInterfaceSocketReaderDetails::~FNiagaraDataInterfaceSocketReaderDetails()
{
	if (UNiagaraDataInterfaceSocketReader* DataInterface = WeakDataInterface.Get())
	{
		DataInterface->OnChanged().RemoveAll(this);
	}
}

void FNiagaraDataInterfaceSocketReaderDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	if (SelectedObjects.Num() != 1 || SelectedObjects[0]->IsA<UNiagaraDataInterfaceSocketReader>() == false)
	{
		return;
	}

	// Get the data interface
	UNiagaraDataInterfaceSocketReader* DataInterface = CastChecked<UNiagaraDataInterfaceSocketReader>(SelectedObjects[0].Get());
	WeakDataInterface = DataInterface;

	DataInterface->OnChanged().RemoveAll(this);
	DataInterface->OnChanged().AddSP(this, &FNiagaraDataInterfaceSocketReaderDetails::OnDataChanged);

	// Customize
	{
		IDetailCategoryBuilder& DetailCategory = DetailBuilder.EditCategory("SocketReader");

		TArray<TSharedRef<IPropertyHandle>> Properties;
		DetailCategory.GetDefaultProperties(Properties, true, true);

		TSharedPtr<IPropertyHandle> FilteredSocketsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceSocketReader, FilteredSockets));
		for (TSharedPtr<IPropertyHandle> Property : Properties)
		{
			FProperty* PropertyPtr = Property->GetProperty();
			if (PropertyPtr == FilteredSocketsProperty->GetProperty())
			{
				SocketArrayBuilder = MakeShared<FNiagaraDetailSourcedArrayBuilder>(Property.ToSharedRef(), GetSocketNames());
				DetailCategory.AddCustomBuilder(SocketArrayBuilder.ToSharedRef());
			}
			else
			{
				DetailCategory.AddProperty(Property);
			}
		}
	}
}

 TSharedRef<IDetailCustomization> FNiagaraDataInterfaceSocketReaderDetails::MakeInstance()
 {
	 return MakeShared<FNiagaraDataInterfaceSocketReaderDetails>();
 }

void FNiagaraDataInterfaceSocketReaderDetails::OnDataChanged()
{
	if (SocketArrayBuilder.IsValid())
	{
		SocketArrayBuilder->SetSourceArray(GetSocketNames());
	}
 }

TArray<TSharedPtr<FName>> FNiagaraDataInterfaceSocketReaderDetails::GetSocketNames() const
{
	TArray<TSharedPtr<FName>> SocketNames;
	if (UNiagaraDataInterfaceSocketReader* DataInterface = WeakDataInterface.Get())
	{
		Algo::Transform(DataInterface->GetEditorSocketNames(), SocketNames, [](const FName& SocketName) { return MakeShared<FName>(SocketName); });
	}
	return SocketNames;
}

#undef LOCTEXT_NAMESPACE
