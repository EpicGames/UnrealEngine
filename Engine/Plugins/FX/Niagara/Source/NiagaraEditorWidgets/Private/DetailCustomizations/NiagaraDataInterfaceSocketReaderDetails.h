// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceDetails.h"

class IDetailLayoutBuilder;
class FNiagaraDetailSourcedArrayBuilder;
class UNiagaraDataInterfaceSocketReader;

class FNiagaraDataInterfaceSocketReaderDetails : public IDetailCustomization//FNiagaraDataInterfaceDetailsBase
{
public:
	~FNiagaraDataInterfaceSocketReaderDetails();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	void OnDataChanged();

	TArray<TSharedPtr<FName>> GetSocketNames() const;

private:
	TWeakObjectPtr<UNiagaraDataInterfaceSocketReader>	WeakDataInterface;
	TSharedPtr<FNiagaraDetailSourcedArrayBuilder>		SocketArrayBuilder;
};
