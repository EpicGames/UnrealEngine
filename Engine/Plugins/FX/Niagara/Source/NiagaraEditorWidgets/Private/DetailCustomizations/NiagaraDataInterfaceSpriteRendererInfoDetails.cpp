// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceSpriteRendererInfoDetails.h"
#include "NiagaraDetailSourcedArrayBuilder.h"
#include "NiagaraDataInterfaceDetails.h"
#include "NiagaraDataInterfaceSpriteRendererInfo.h"
#include "NiagaraSpriteRendererProperties.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "NiagaraComponent.h"
#include "NiagaraEmitter.h"
#include "NiagaraStackEditorData.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraEditorModule.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FNiagaraDataInterfaceSpriteRendererInfoDetails"

FNiagaraDataInterfaceSpriteRendererInfoDetails::~FNiagaraDataInterfaceSpriteRendererInfoDetails()
{
	if (DataInterface.IsValid())
	{
		DataInterface->OnChanged().RemoveAll(this);
	}
}

void FNiagaraDataInterfaceSpriteRendererInfoDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	LayoutBuilder = &DetailBuilder;
	FNiagaraDataInterfaceDetailsBase::CustomizeDetails(DetailBuilder);

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	if (SelectedObjects.Num() != 1 || SelectedObjects[0]->IsA<UNiagaraDataInterfaceSpriteRendererInfo>() == false)
	{
		return;
	}

	UNiagaraDataInterfaceSpriteRendererInfo* Interface = CastChecked<UNiagaraDataInterfaceSpriteRendererInfo>(SelectedObjects[0].Get());
	DataInterface = Interface;

	Interface->OnChanged().AddSP(this, &FNiagaraDataInterfaceSpriteRendererInfoDetails::OnInterfaceChanged);

	GenerateRendererList();	

	static const FName SourceCategoryName = TEXT("Source");
	SourceCategory = &DetailBuilder.EditCategory(SourceCategoryName, LOCTEXT("SourceCategory", "Source"));
	{
		TArray<TSharedRef<IPropertyHandle>> Properties;
		SourceCategory->GetDefaultProperties(Properties, true, true);

		static const FName SpriteRendererPropertyName = TEXT("SpriteRenderer");
		SpriteRendererProperty = DetailBuilder.GetProperty(SpriteRendererPropertyName);
		for (TSharedPtr<IPropertyHandle> Property : Properties)
		{
			FProperty* PropertyPtr = Property->GetProperty();
			if (PropertyPtr == SpriteRendererProperty->GetProperty())
			{				
				SpriteRendererWidget = SNew(SSpriteRendererComboBox)
					.OptionsSource(&RendererList)
					.InitiallySelectedItem(Interface->GetSpriteRenderer())
					.OnComboBoxOpening(this, &FNiagaraDataInterfaceSpriteRendererInfoDetails::GenerateRendererList)
					.OnSelectionChanged(this, &FNiagaraDataInterfaceSpriteRendererInfoDetails::SetSelectedRenderer)
					.OnGenerateWidget(this, &FNiagaraDataInterfaceSpriteRendererInfoDetails::CreateRendererItemWidget)
					.AddMetaData<FTagMetaData>(TEXT("SelectSpriteRendererCobmo"))
					[
						SNew(STextBlock)
							.Text(this, &FNiagaraDataInterfaceSpriteRendererInfoDetails::GetSelectedRendererTextLabel)
					];

				IDetailPropertyRow& RendererRow = SourceCategory->AddProperty(Property);
				RendererRow.CustomWidget(false)
					.NameContent()
					[
						Property->CreatePropertyNameWidget()
					]
					.ValueContent()
					.MaxDesiredWidth(TOptional<float>())
					[
						SpriteRendererWidget.ToSharedRef()
					];
			}
			else
			{
				SourceCategory->AddProperty(Property);
			}
		}
	}
}

TSharedRef<IDetailCustomization> FNiagaraDataInterfaceSpriteRendererInfoDetails::MakeInstance()
{
	return MakeShared<FNiagaraDataInterfaceSpriteRendererInfoDetails>();
}

void FNiagaraDataInterfaceSpriteRendererInfoDetails::OnInterfaceChanged()
{
	if (!bSettingSelection && SpriteRendererWidget.IsValid())
	{
		SpriteRendererWidget->SetSelectedItem(GetSelectedRenderer());
	}
}

void FNiagaraDataInterfaceSpriteRendererInfoDetails::GenerateRendererList()
{
	RendererList.SetNum(0, EAllowShrinking::No);
	RendererLabels.SetNum(0, EAllowShrinking::No);

	// Get a list of all Sprite renderers in the emitter
	if (auto Interface = DataInterface.Get())
	{
		UNiagaraSystem* System = nullptr;
		FVersionedNiagaraEmitter EmitterHandle;
		FNiagaraEditorModule::Get().GetTargetSystemAndEmitterForDataInterface(Interface, System, EmitterHandle);
		if (System && EmitterHandle.GetEmitterData())
		{
			for (auto RendererProps : EmitterHandle.GetEmitterData()->GetRenderers())
			{
				if (auto SpriteProps = Cast<UNiagaraSpriteRendererProperties>(RendererProps))
				{
					RendererList.Add(SpriteProps);
					RendererLabels.Add(CreateRendererTextLabel(SpriteProps));
				}
			}

			if(SpriteRendererWidget.IsValid())
			{
				SpriteRendererWidget->RefreshOptions();
			}
		}
	}
}

UNiagaraSpriteRendererProperties* FNiagaraDataInterfaceSpriteRendererInfoDetails::GetSelectedRenderer() const
{
	UObject* Value = nullptr;
	if (SpriteRendererProperty.IsValid())
	{
		SpriteRendererProperty->GetValue(Value);
	}
	return Cast<UNiagaraSpriteRendererProperties>(Value);
}

void FNiagaraDataInterfaceSpriteRendererInfoDetails::SetSelectedRenderer(TRendererPtr Selection, ESelectInfo::Type)
{
	if (SpriteRendererProperty.IsValid() && DataInterface.IsValid())
	{
		bSettingSelection = true;
		UObject* Obj = nullptr;
		SpriteRendererProperty->GetValue(Obj);
		if (Obj != Selection.Get())
		{
			const FScopedTransaction Transaction(NSLOCTEXT("FNiagaraDataInterfaceSpriteRendererInfoDetails", "Change Sprite Renderer", "Change Sprite Renderer"));
			DataInterface->Modify();
			SpriteRendererProperty->NotifyPreChange();
			SpriteRendererProperty->SetValue(Selection.Get());
			SpriteRendererProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
		bSettingSelection = false;
	}	
}

FText FNiagaraDataInterfaceSpriteRendererInfoDetails::GetRendererTextLabel(TRendererPtr Renderer) const
{
	int32 RendererIndex = Renderer.IsValid() ? RendererList.Find(Renderer) : INDEX_NONE;
	if (!RendererLabels.IsValidIndex(RendererIndex))
	{
		return LOCTEXT("NoneOption", "None");
	}
	return RendererLabels[RendererIndex];
}

TSharedRef<SWidget> FNiagaraDataInterfaceSpriteRendererInfoDetails::CreateRendererItemWidget(TRendererPtr Item)
{
	return SNew(STextBlock)
		.Text(this, &FNiagaraDataInterfaceSpriteRendererInfoDetails::GetRendererTextLabel, Item);
}

FText FNiagaraDataInterfaceSpriteRendererInfoDetails::CreateRendererTextLabel(const UNiagaraSpriteRendererProperties* Properties)
{
	if (Properties == nullptr)
	{
		return LOCTEXT("NoneOption", "None");
	}

	UNiagaraEmitter* Emitter = Properties->GetTypedOuter<UNiagaraEmitter>();
	check(Emitter);

	UNiagaraEmitterEditorData* EmitterEditorData = Properties->GetEmitterData() ? static_cast<UNiagaraEmitterEditorData*>(Properties->GetEmitterData()->GetEditorData()) : nullptr;
	if (EmitterEditorData == nullptr)
	{
		return LOCTEXT("NoneOption", "None");
	}
	UNiagaraStackEditorData& EmitterStackEditorData = EmitterEditorData->GetStackEditorData();
	FString RendererStackEditorDataKey = FString::Printf(TEXT("Renderer-%s"), *Properties->GetName());
	const FText* RendererDisplayName = EmitterStackEditorData.GetStackEntryDisplayName(RendererStackEditorDataKey);

	return RendererDisplayName ? *RendererDisplayName : Properties->GetWidgetDisplayName();
}

#undef LOCTEXT_NAMESPACE