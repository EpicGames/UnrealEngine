// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceDetails.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class UNiagaraDataInterfaceSpriteRendererInfo;
class UNiagaraSpriteRendererProperties;

/** Details customization for Niagara Sprite renderer info data interface. */
class FNiagaraDataInterfaceSpriteRendererInfoDetails : public FNiagaraDataInterfaceDetailsBase
{
public:
	virtual ~FNiagaraDataInterfaceSpriteRendererInfoDetails();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	using TRendererPtr = TWeakObjectPtr<UNiagaraSpriteRendererProperties>;
	using SSpriteRendererComboBox = SComboBox<TRendererPtr>;

	void OnInterfaceChanged();
	void GenerateRendererList();
	UNiagaraSpriteRendererProperties* GetSelectedRenderer() const;
	void SetSelectedRenderer(TRendererPtr Selection, ESelectInfo::Type SelectInfo);
	FText GetRendererTextLabel(TRendererPtr Renderer) const;
	FText GetSelectedRendererTextLabel() const { return CreateRendererTextLabel(GetSelectedRenderer()); }
	TSharedRef<SWidget> CreateRendererItemWidget(TRendererPtr Item);
	
	static FText CreateRendererTextLabel(const UNiagaraSpriteRendererProperties* Properties);

private:

	TSharedPtr<SSpriteRendererComboBox> SpriteRendererWidget;
	TArray<TRendererPtr> RendererList;
	TArray<FText> RendererLabels;
	IDetailLayoutBuilder* LayoutBuilder = nullptr;
	TWeakObjectPtr<UNiagaraDataInterfaceSpriteRendererInfo>  DataInterface;
	IDetailCategoryBuilder* SourceCategory = nullptr;
	TSharedPtr<IPropertyHandle> SpriteRendererProperty;
	bool bSettingSelection = false;
};
