// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceStaticMeshDetails.h"
#include "DataInterface/NiagaraDataInterfaceStaticMesh.h" 
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/StaticMeshSocket.h"
#include "IDetailGroup.h"
#include "NiagaraComponent.h"
#include "NiagaraDataInterfaceDetails.h"
#include "NiagaraDetailSourcedArrayBuilder.h"
#include "SNiagaraNamePropertySelector.h"

#define LOCTEXT_NAMESPACE "FNiagaraDataInterfaceStaticMeshDetails"

FNiagaraDataInterfaceStaticMeshDetails::~FNiagaraDataInterfaceStaticMeshDetails()
{
	if (MeshInterface.IsValid())
	{
		MeshInterface->OnChanged().RemoveAll(this);
	}
	if (MeshObject.IsValid())
	{
		MeshObject->GetOnMeshChanged().RemoveAll(this);
	}
}

void FNiagaraDataInterfaceStaticMeshDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	 LayoutBuilder = &DetailBuilder;
	 static const FName MeshCategoryName = TEXT("Mesh");

	 TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	 DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	 if(SelectedObjects.Num() != 1 || SelectedObjects[0]->IsA<UNiagaraDataInterfaceStaticMesh>() == false)
	 {
	 	return;
	 }

	 MeshInterface = CastChecked<UNiagaraDataInterfaceStaticMesh>(SelectedObjects[0].Get());
	 MeshInterface->OnChanged().RemoveAll(this);
	 MeshInterface->OnChanged().AddSP(this, &FNiagaraDataInterfaceStaticMeshDetails::OnInterfaceChanged);

	 UObject* Component = nullptr;
	 if (UNiagaraDataInterfaceStaticMesh* Interface = MeshInterface.Get())
	 {
		 MeshObject = Interface->ResolveStaticMesh(Component);
	 }
	 if (MeshObject.IsValid())
	 {
		 MeshObject->GetOnMeshChanged().RemoveAll(this);
		 MeshObject->GetOnMeshChanged().AddSP(this, &FNiagaraDataInterfaceStaticMeshDetails::OnDataChanged);
	 }

	 MeshCategory = &DetailBuilder.EditCategory(MeshCategoryName, LOCTEXT("StaticMeshCat", "StaticMesh"));
	 {
		 TArray<TSharedRef<IPropertyHandle>> Properties;
		 MeshCategory->GetDefaultProperties(Properties, true, true);

		 TSharedPtr<IPropertyHandle> SocketsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UNiagaraDataInterfaceStaticMesh, FilteredSockets));

		 for (TSharedPtr<IPropertyHandle> Property : Properties)
		 {
			 FProperty* PropertyPtr = Property->GetProperty();
			 TArray<TSharedPtr<FName>> PossibleNames;

			 if (PropertyPtr == SocketsProperty->GetProperty())
			 {
				 GenerateSocketsArray(PossibleNames);
				 SocketsBuilder = TSharedPtr<FNiagaraDetailSourcedArrayBuilder>(new FNiagaraDetailSourcedArrayBuilder(Property.ToSharedRef(), PossibleNames));
				 MeshCategory->AddCustomBuilder(SocketsBuilder.ToSharedRef());
			 }
			 else
			 {
				 MeshCategory->AddProperty(Property);
			 }
		 }
	 }
}

 TSharedRef<IDetailCustomization> FNiagaraDataInterfaceStaticMeshDetails::MakeInstance()
 {
	 return MakeShared<FNiagaraDataInterfaceStaticMeshDetails>();
 }

 void FNiagaraDataInterfaceStaticMeshDetails::OnInterfaceChanged()
 {
	 if (MeshObject.IsValid())
	 {
		 MeshObject->GetOnMeshChanged().RemoveAll(this);
		 MeshObject = nullptr;
	 }

	 UObject* Component = nullptr;
	 if (UNiagaraDataInterfaceStaticMesh* Interface = MeshInterface.Get())
	 {
		 MeshObject = Interface->ResolveStaticMesh(Component);
	 }

	 if (MeshObject.IsValid())
	 {
		 MeshObject->GetOnMeshChanged().AddSP(this, &FNiagaraDataInterfaceStaticMeshDetails::OnDataChanged);
	 }

	 OnDataChanged();
 }

 void FNiagaraDataInterfaceStaticMeshDetails::OnDataChanged()
 {
	if (SocketsBuilder)
	{
		TArray<TSharedPtr<FName>> PossibleNames;
		GenerateSocketsArray(PossibleNames);
		SocketsBuilder->SetSourceArray(PossibleNames);
	}
}

void FNiagaraDataInterfaceStaticMeshDetails::GenerateSocketsArray(TArray<TSharedPtr<FName>>& SourceArray)
{
	SourceArray.Reset();
	if (UStaticMesh* Mesh = MeshObject.Get())
	{
		for (const UStaticMeshSocket* Socket : Mesh->Sockets)
		{
			SourceArray.Add(MakeShared<FName>(Socket->SocketName));
		}
	}
}

#undef LOCTEXT_NAMESPACE
