// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraDataInterfaceDetails.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"

class IDetailLayoutBuilder;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SWidget;
class FName;
class FNiagaraDetailSourcedArrayBuilder;
class UNiagaraDataInterfaceStaticMesh;
class UStaticMesh;
class SNiagaraNamePropertySelector;

/** Details customization for Niagara static mesh data interface. */
class FNiagaraDataInterfaceStaticMeshDetails : public FNiagaraDataInterfaceDetailsBase
{
public:
	~FNiagaraDataInterfaceStaticMeshDetails();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	void OnInterfaceChanged();
	void OnDataChanged();

	void GenerateSocketsArray(TArray<TSharedPtr<FName>>& SourceArray);

private:
	TSharedPtr<FNiagaraDetailSourcedArrayBuilder> SocketsBuilder;
	IDetailLayoutBuilder* LayoutBuilder;
	TWeakObjectPtr<UNiagaraDataInterfaceStaticMesh>  MeshInterface;
	TWeakObjectPtr<UStaticMesh> MeshObject;

	IDetailCategoryBuilder* MeshCategory;
};
