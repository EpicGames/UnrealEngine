// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDecalRendererDetails.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Layout/SExpandableArea.h"

#include "NiagaraEditorSettings.h"

TSharedRef<IDetailCustomization> FNiagaraDecalRendererDetails::MakeInstance()
{
	return MakeShared<FNiagaraDecalRendererDetails>();
}

void FNiagaraDecalRendererDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	static const FName OrderedCategories[] =
	{
		FName("Decal Rendering"),
	};

	static const FName CollapsedCategories[] =
	{
		FName("Bindings"),
		FName("Rendering"),
		FName("Scalability"),
	};
	SetupCategories(DetailBuilder, MakeArrayView(OrderedCategories), MakeArrayView(CollapsedCategories));
}
