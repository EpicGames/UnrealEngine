// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDetailSourcedArrayBuilder.h"
#include "NiagaraDataInterfaceDetails.h"
#include "NiagaraDataInterfaceSkeletalMesh.h" 
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailGroup.h"
#include "PropertyCustomizationHelpers.h"
#include "IDetailChildrenBuilder.h"
#include "SNiagaraNamePropertySelector.h"

#define LOCTEXT_NAMESPACE "FNiagaraDetailSourcedArrayBuilder"

FNiagaraDetailSourcedArrayBuilder::FNiagaraDetailSourcedArrayBuilder(TSharedRef<IPropertyHandle> InBaseProperty, const TArray<TSharedPtr<FName>>& InOptionsSource, const FName InFNameSubproperty, bool InGenerateHeader, bool InDisplayResetToDefault, bool InDisplayElementNum, bool InAllowAutoFill)
	: FDetailArrayBuilder(InBaseProperty, InGenerateHeader, InDisplayResetToDefault, InDisplayElementNum)
	, OptionsSourceList(InOptionsSource)
	, ArrayProperty(InBaseProperty->AsArray())
	, FNameSubproperty(InFNameSubproperty)
	, bAllowAutoFill(InAllowAutoFill)
{
}

void FNiagaraDetailSourcedArrayBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	FDetailArrayBuilder::GenerateHeaderRowContent(NodeRow);

	if (bAllowAutoFill)
	{
		NodeRow.AddCustomContextMenuAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					ArrayProperty->EmptyArray();
					int32 NumItems = 0;
					for (TSharedPtr<FName> Name : OptionsSourceList)
					{
						ArrayProperty->AddItem();
						TSharedPtr<IPropertyHandle> Child = ArrayProperty->GetElement(NumItems++);
						Child->SetValue(*Name);
					}
				}
			),
			LOCTEXT("AutoFillActionText", "Auto-fill all options"),
			LOCTEXT("AutoFillActionTooltip", "Resets this list to a list of all possible options")
		);
	}
}

void FNiagaraDetailSourcedArrayBuilder::OnGenerateEntry(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	IDetailPropertyRow& RegionRow = ChildrenBuilder.AddProperty(PropertyHandle);

	FNumberFormattingOptions NoCommas;
	NoCommas.UseGrouping = false;
	const FText SlotDesc = FText::Format(LOCTEXT("ElementIndex", "Element #{0}"), FText::AsNumber(ArrayIndex, &NoCommas));

	RegionRow.DisplayName(SlotDesc);

	RegionRow.ShowPropertyButtons(true);

	TSharedPtr<IPropertyHandle> SubPropertyHandle = PropertyHandle;
	if (FNameSubproperty != NAME_None)
	{
		SubPropertyHandle = PropertyHandle->GetChildHandle(FNameSubproperty).ToSharedRef();
	}

	if (OptionsSourceList.Num() > 0)
	{
		RegionRow.CustomWidget(false)
			.NameContent()
			[
				SubPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(TOptional<float>())
			[
				SNew(SNiagaraNamePropertySelector, SubPropertyHandle.ToSharedRef(), OptionsSourceList)
			];
	}
	else
	{
		RegionRow.CustomWidget(false)
			.NameContent()
			[
				SubPropertyHandle->CreatePropertyNameWidget()
			]
		.ValueContent()
			.MaxDesiredWidth(TOptional<float>())
			[

				SubPropertyHandle->CreatePropertyValueWidget(false)
			];
	}
}

void FNiagaraDetailSourcedArrayBuilder::SetSourceArray(const TArray<TSharedPtr<FName>>& InOptionsSource)
{
	OptionsSourceList = InOptionsSource;
	RefreshChildren();
}

void FNiagaraDetailSourcedArrayBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	uint32 NumChildren = 0;
	ArrayProperty->GetNumElements(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ElementHandle = ArrayProperty->GetElement(ChildIndex);
		
		OnGenerateEntry(ElementHandle, ChildIndex, ChildrenBuilder);
	}
}


#undef LOCTEXT_NAMESPACE