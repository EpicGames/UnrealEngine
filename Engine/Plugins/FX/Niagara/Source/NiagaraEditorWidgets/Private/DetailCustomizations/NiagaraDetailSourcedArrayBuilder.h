// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"
#include "PropertyCustomizationHelpers.h"
class IDetailLayoutBuilder;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SWidget;
class SNiagaraNamePropertySelector;

/** Details customization for arrays composed of FName properties (or wrappers). The array contents are selected from a predetermined source list.*/
class FNiagaraDetailSourcedArrayBuilder : public FDetailArrayBuilder
										, public TSharedFromThis<FNiagaraDetailSourcedArrayBuilder>
{
public:
	FNiagaraDetailSourcedArrayBuilder(TSharedRef<IPropertyHandle> InBaseProperty, const TArray<TSharedPtr<FName>>& InOptionsSource, const FName InFNameSubproperty = NAME_None, bool InGenerateHeader = true, bool InDisplayResetToDefault = true, bool InDisplayElementNum = true, bool InAllowAutoFill = true);
	
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	
	void SetSourceArray(const TArray<TSharedPtr<FName>>& InOptionsSource);

private: 
	void OnGenerateEntry(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder);

private:
	TArray<TSharedPtr<FName>> OptionsSourceList;
	/** List of component class names, filtered by the current search string */

	TSharedPtr<IPropertyHandleArray> ArrayProperty;
	/** Subproperty of type FName that needs to be edited - in case of FName wrappers */
	FName FNameSubproperty;
	/** Whether or not to add a context menu option to reset the array to the list of all possible options */
	bool bAllowAutoFill;
};
