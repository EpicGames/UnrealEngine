// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraLightRendererDetails.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Layout/SExpandableArea.h"

TSharedRef<IDetailCustomization> FNiagaraLightRendererDetails::MakeInstance()
{
	return MakeShared<FNiagaraLightRendererDetails>();
}

void FNiagaraLightRendererDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	static const FName OrderedCategories[] =
	{
		FName("Light Rendering"),
		FName("Bindings"),
	};

	static const FName CollapsedCategories[] =
	{
		FName("Rendering"),
		FName("Scalability"),
	};

	SetupCategories(DetailBuilder, MakeArrayView(OrderedCategories), MakeArrayView(CollapsedCategories));
}
