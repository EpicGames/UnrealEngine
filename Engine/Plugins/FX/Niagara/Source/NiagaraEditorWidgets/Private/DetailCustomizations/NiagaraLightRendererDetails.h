// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraRendererDetails.h"

class IDetailLayoutBuilder;

class FNiagaraLightRendererDetails : public FNiagaraRendererDetails
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};