// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMeshRendererDetails.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Layout/SExpandableArea.h"

class FNiagaraMeshRendererMeshDetailBuilder : public IDetailCustomNodeBuilder
											, public TSharedFromThis<FNiagaraMeshRendererMeshDetailBuilder>
{
public:
	FNiagaraMeshRendererMeshDetailBuilder(TSharedPtr<IPropertyHandle> InParentProperty, TSharedPtr<IPropertyHandle> InMeshProperty, bool bInCanResetToDefault)
		: ParentProperty(InParentProperty)
		, MeshProperty(InMeshProperty)
		, bCanResetToDefault(bInCanResetToDefault)
	{}

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override
	{
		TSharedPtr<SWidget> NameWidget = ParentProperty->CreatePropertyNameWidget();
		TSharedPtr<SWidget> ValueWidget = MeshProperty->CreatePropertyValueWidget();

		if (bCanResetToDefault)
		{
			TSharedPtr<SHorizontalBox> Container;
			SAssignNew(Container, SHorizontalBox)
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ValueWidget.ToSharedRef()
					]
				+ SHorizontalBox::Slot()
					[
						ParentProperty->CreateDefaultPropertyButtonWidgets()
					];

			ValueWidget = Container;
		}

		FUIAction CopyAction;
		FUIAction PasteAction;
		ParentProperty->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);
		NodeRow
			.NameContent()
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			.MinDesiredWidth(300.0f)
			[
				ValueWidget.ToSharedRef()
			]
			.CopyAction(CopyAction)
			.PasteAction(PasteAction);
	}

	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override
	{
		uint32 NumChildren = 0;
		if (ParentProperty->GetNumChildren(NumChildren) != FPropertyAccess::Fail)
		{
			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = ParentProperty->GetChildHandle(ChildIndex);
				if (ChildProperty.IsValid() && ChildProperty->GetProperty()->GetFName() != MeshProperty->GetProperty()->GetFName())
				{
					ChildrenBuilder.AddProperty(ChildProperty.ToSharedRef())
						.ShouldAutoExpand(false);
				}
			}
		}
	}

	virtual FName GetName() const override
	{
		static const FName NiagaraMeshRendererMeshDetailBuilder("NiagaraMeshRendererMeshDetailBuilder");
		return NiagaraMeshRendererMeshDetailBuilder;
	}

	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override
	{
		OnRebuildChildren = InOnRegenerateChildren;
	}

	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual bool InitiallyCollapsed() const override { return true; }
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override { return ParentProperty; }

	void Rebuild()
	{
		OnRebuildChildren.ExecuteIfBound();
	}

private:
	TSharedPtr<IPropertyHandle> ParentProperty;
	TSharedPtr<IPropertyHandle> MeshProperty;
	FSimpleDelegate OnRebuildChildren;
	bool bCanResetToDefault = true;
};

///////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FNiagaraMeshRendererDetails::MakeInstance()
{
	return MakeShared<FNiagaraMeshRendererDetails>();
}

void FNiagaraMeshRendererDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	LayoutBuilder = &DetailBuilder;

	static const FName OrderedCategories[] =
	{
		FName("Mesh Rendering"),
		FName("Sorting"),
		FName("Visibility"),
	};

	static const FName CollapsedCategories[] =
	{
		FName("Bindings"),
		FName("Rendering"),
		FName("Scalability"),
		FName("SubUV"),
	};

	SetupCategories(DetailBuilder, MakeArrayView(OrderedCategories), MakeArrayView(CollapsedCategories));

	{
		static const FName MeshesName = TEXT("Meshes");
		static const FName EnableMeshFlipbookName = TEXT("bEnableMeshFlipbook");

		// Determine if mesh flipbook is enabled to disable allowing override resets
		bEnableMeshFlipbook = false;
		TSharedPtr<IPropertyHandle> EnableMeshFlipbookProperty = DetailBuilder.GetProperty(EnableMeshFlipbookName);
		if (EnableMeshFlipbookProperty.IsValid())
		{
			EnableMeshFlipbookProperty->GetValue(bEnableMeshFlipbook);
			EnableMeshFlipbookProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FNiagaraMeshRendererDetails::OnInvalidateDetails));
		}

		TSharedPtr<IPropertyHandle> MeshesProperty = DetailBuilder.GetProperty(MeshesName);
		check(MeshesProperty.IsValid());

		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(MeshesProperty->GetDefaultCategoryName(), MeshesProperty->GetDefaultCategoryText());

		TSharedRef<FDetailArrayBuilder> MeshesBuilder = MakeShared<FDetailArrayBuilder>(MeshesProperty.ToSharedRef(), true, !bEnableMeshFlipbook, true);
		MeshesBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FNiagaraMeshRendererDetails::OnGenerateMeshWidget));

		CategoryBuilder.AddCustomBuilder(MeshesBuilder);
	}

	{
		static const FName OverrideMaterialsName = TEXT("OverrideMaterials");
		static const FName EnableOverrideMaterialsName = TEXT("bOverrideMaterials");

		bEnableMaterialOverrides = false;
		TSharedPtr<IPropertyHandle> EnableOverrideMaterialsProperty = DetailBuilder.GetProperty(EnableOverrideMaterialsName);
		if (EnableOverrideMaterialsProperty.IsValid())
		{
			EnableOverrideMaterialsProperty->GetValue(bEnableMaterialOverrides);
			EnableOverrideMaterialsProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FNiagaraMeshRendererDetails::OnInvalidateDetails));
		}

		TSharedPtr<IPropertyHandle> OverrideMaterialsProperty = DetailBuilder.GetProperty(OverrideMaterialsName);
		check(OverrideMaterialsProperty.IsValid());

		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(OverrideMaterialsProperty->GetDefaultCategoryName(), OverrideMaterialsProperty->GetDefaultCategoryText());

		TSharedRef<FDetailArrayBuilder> OverrideMaterialsBuilder = MakeShared<FDetailArrayBuilder>(OverrideMaterialsProperty.ToSharedRef(), true, bEnableMaterialOverrides, true);
		OverrideMaterialsBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FNiagaraMeshRendererDetails::OnGenerateMaterialOverrideWidget));

		CategoryBuilder.AddProperty(EnableOverrideMaterialsProperty);
		CategoryBuilder.AddCustomBuilder(OverrideMaterialsBuilder);
	}
}

void FNiagaraMeshRendererDetails::OnInvalidateDetails()
{
	if (LayoutBuilder)
	{
		LayoutBuilder->ForceRefreshDetails();
	}
}

void FNiagaraMeshRendererDetails::OnGenerateMeshWidget(TSharedRef<IPropertyHandle> Property, int32 Index, IDetailChildrenBuilder& ChildrenBuilder)
{	
	static const FName MeshName = TEXT("Mesh");
	TSharedPtr<IPropertyHandle> MeshChildProperty = Property->GetChildHandle(MeshName, false);
	check(MeshChildProperty.IsValid());

	ChildrenBuilder.AddCustomBuilder(MakeShared<FNiagaraMeshRendererMeshDetailBuilder>(Property, MeshChildProperty, !bEnableMeshFlipbook));
}

void FNiagaraMeshRendererDetails::OnGenerateMaterialOverrideWidget(TSharedRef<IPropertyHandle> Property, int32 Index, IDetailChildrenBuilder& ChildrenBuilder)
{
	IDetailPropertyRow& Row = ChildrenBuilder.AddProperty(Property);
	Row.IsEnabled(bEnableMaterialOverrides);
}
