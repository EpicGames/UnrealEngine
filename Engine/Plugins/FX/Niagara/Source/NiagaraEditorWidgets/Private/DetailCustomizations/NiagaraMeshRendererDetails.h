// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NiagaraRendererDetails.h"

class IPropertyHandle;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;

class FNiagaraMeshRendererDetails : public FNiagaraRendererDetails
{
public:	
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	void OnInvalidateDetails();
	void OnGenerateMeshWidget(TSharedRef<IPropertyHandle> Property, int32 Index, IDetailChildrenBuilder& ChildrenBuilder);

	void OnGenerateMaterialOverrideWidget(TSharedRef<IPropertyHandle> Property, int32 Index, IDetailChildrenBuilder& ChildrenBuilder);

	IDetailLayoutBuilder* LayoutBuilder = nullptr;
	bool bEnableMeshFlipbook = false;
	bool bEnableMaterialOverrides = false;
};