// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRendererDetails.h"
#include "NiagaraEditorSettings.h"
#include "NiagaraParameterBinding.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Layout/SExpandableArea.h"

void FNiagaraRendererDetails::SetupCategories(IDetailLayoutBuilder& DetailBuilder, TConstArrayView<FName> OrderedCategories, TConstArrayView<FName> CollapsedCategories)
{
	// Force the order of catefories
	for (const FName& Category : OrderedCategories)
	{
		DetailBuilder.EditCategory(Category);
		//IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(Category);
	}

	// Iterate over each category setting the collapse state
	TArray<FName> Categories;
	DetailBuilder.GetCategoryNames(Categories);

	const ENiagaraCategoryExpandState CategoryExpandState = GetDefault<UNiagaraEditorSettings>()->RendererCategoryExpandState;
	const bool bForceCollapse = CategoryExpandState == ENiagaraCategoryExpandState::CollapseAll;
	const bool bForceExpand = CategoryExpandState == ENiagaraCategoryExpandState::ExpandAll;
	const bool bExpandMmodified = CategoryExpandState == ENiagaraCategoryExpandState::DefaultExpandModified;

	for (const FName& Category : Categories)
	{
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(Category);

		bool bShouldCollapse = !bForceExpand && (bForceCollapse || CollapsedCategories.Contains(Category));
		if (bShouldCollapse && bExpandMmodified)
		{
			TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailBuilder.GetSelectedObjects();

			TArray<TSharedRef<IPropertyHandle>> DefaultProperties;
			CategoryBuilder.GetDefaultProperties(DefaultProperties);

			for (const TSharedRef<IPropertyHandle>& PropertyHandle : DefaultProperties)
			{
				// Special compare because attribute bindings are special
				if ( FStructProperty* StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()) )
				{
					if (StructProperty->Struct == FNiagaraVariableAttributeBinding::StaticStruct())
					{
						for (TWeakObjectPtr<UObject> WeakSelectedObject : DetailBuilder.GetSelectedObjects())
						{
							UObject* SelectedObject = WeakSelectedObject.Get();
							if (SelectedObject == nullptr)
							{
								continue;
							}
							const FNiagaraVariableAttributeBinding* TargetVariableBinding = (FNiagaraVariableAttributeBinding*)PropertyHandle->GetValueBaseAddress((uint8*)SelectedObject);
							const FNiagaraVariableAttributeBinding* DefaultVariableBinding = (FNiagaraVariableAttributeBinding*)PropertyHandle->GetValueBaseAddress((uint8*)SelectedObject->GetClass()->GetDefaultObject());
							if (TargetVariableBinding->MatchesDefault(*DefaultVariableBinding, ENiagaraRendererSourceDataMode::Particles) == false)
							{
								bShouldCollapse = false;
								break;
							}
						}
						continue;
					}
					//-TODO: Implement this when the code makes it into here that fixes the reset to default, etc.
					//else if (StructProperty->Struct == FNiagaraParameterBinding::StaticStruct())
					//{
					//	for (TWeakObjectPtr<UObject> WeakSelectedObject : DetailBuilder.GetSelectedObjects())
					//	{
					//		UObject* SelectedObject = WeakSelectedObject.Get();
					//		if (SelectedObject == nullptr)
					//		{
					//			continue;
					//		}
					//		const FNiagaraParameterBinding* ParameterBinding = (FNiagaraParameterBinding*)PropertyHandle->GetValueBaseAddress((uint8*)SelectedObject);

					//		if ( ParameterBinding->IsSetoToDefault() == false )
					//		{
					//			bShouldCollapse = false;
					//			break;
					//		}
					//	}
					//}
				}

				if (PropertyHandle->DiffersFromDefault())
				{
					bShouldCollapse = false;
					break;
				}
			}

		}
		CategoryBuilder.InitiallyCollapsed(bShouldCollapse);
	}
}
