// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;

class FNiagaraRendererDetails : public IDetailCustomization
{
public:	
	void SetupCategories(IDetailLayoutBuilder& DetailBuilder, TConstArrayView<FName> CategoryOrder, TConstArrayView<FName> CollapsedCategories);
};
