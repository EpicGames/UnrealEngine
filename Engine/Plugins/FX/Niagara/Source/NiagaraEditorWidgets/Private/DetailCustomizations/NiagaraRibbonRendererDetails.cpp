// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraRibbonRendererDetails.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Layout/SExpandableArea.h"

TSharedRef<IDetailCustomization> FNiagaraRibbonRendererDetails::MakeInstance()
{
	return MakeShared<FNiagaraRibbonRendererDetails>();
}

void FNiagaraRibbonRendererDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	static const FName OrderedCategories[] =
	{
		FName("Ribbon Rendering"),
		FName("Ribbon Shape"),
		FName("Ribbon Tessellation"),
		FName("Sorting"),
		FName("Visibility"),
	};

	static const FName CollapsedCategories[] =
	{
		FName("Bindings"),
		FName("Rendering"),
		FName("Scalability"),
	};

	SetupCategories(DetailBuilder, MakeArrayView(OrderedCategories), MakeArrayView(CollapsedCategories));
}
