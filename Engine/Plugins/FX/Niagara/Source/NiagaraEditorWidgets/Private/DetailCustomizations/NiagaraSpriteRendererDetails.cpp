// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraSpriteRendererDetails.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Layout/SExpandableArea.h"

#include "NiagaraEditorSettings.h"

TSharedRef<IDetailCustomization> FNiagaraSpriteRendererDetails::MakeInstance()
{
	return MakeShared<FNiagaraSpriteRendererDetails>();
}

void FNiagaraSpriteRendererDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	static const FName OrderedCategories[] =
	{
		FName("Sprite Rendering"),
		FName("Sorting"),
		FName("Visibility"),
	};

	static const FName CollapsedCategories[] =
	{
		FName("SubUV"),
		FName("Cutout"),
		FName("Bindings"),
		FName("Rendering"),
		FName("Scalability"),
	};
	SetupCategories(DetailBuilder, MakeArrayView(OrderedCategories), MakeArrayView(CollapsedCategories));
}
