// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraNamePropertySelector.h"
#include "NiagaraDataInterfaceDetails.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SComboButton.h"
#include "SListViewSelectorDropdownMenu.h"

#define LOCTEXT_NAMESPACE "SNiagaraNamePropertySelector"

void SNiagaraNamePropertySelector::Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InBaseProperty, const TArray<TSharedPtr<FName>>& InOptionsSource)
{
	PropertyHandle = InBaseProperty;
	OptionsSourceList = InOptionsSource;
	GenerateFilteredElementList(CurrentSearchString.ToString());

	SAssignNew(ElementsListView, SListView<TSharedPtr<FName>>)
		.ListItemsSource(&FilteredSourceList)
		.OnSelectionChanged(this, &SNiagaraNamePropertySelector::OnSelectionChanged)
		.OnGenerateRow(this, &SNiagaraNamePropertySelector::GenerateAddElementRow)
		.SelectionMode(ESelectionMode::Single);

	ElementsListView->RequestListRefresh();

	SAssignNew(SearchBox, SSearchBox)
		.HintText(LOCTEXT("ArrayAddElementSearchBoxHint", "Search Elements"))
		.OnTextChanged(this, &SNiagaraNamePropertySelector::OnSearchBoxTextChanged)
		.OnTextCommitted(this, &SNiagaraNamePropertySelector::OnSearchBoxTextCommitted);

	// Create the Construct arguments for SComboButton
	SComboButton::FArguments Args;
	Args.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.f, 1.f)
			[
				SNew(STextBlock)
				.Text(this, &SNiagaraNamePropertySelector::GetComboText)
			]
		]
		.MenuContent()
		[
			SNew(SListViewSelectorDropdownMenu<TSharedPtr<FName>>, SearchBox, ElementsListView)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(2)
				[
					SNew(SBox)
					.WidthOverride(175)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
							.Padding(1.f)
							.AutoHeight()
							[
								SearchBox.ToSharedRef()
							]
						+ SVerticalBox::Slot()
							.MaxHeight(400)
							.Padding(8.f)
							[
								ElementsListView.ToSharedRef()
							]
					]
				]
			]
		]
		.IsFocusable(true)
		.ContentPadding(FMargin(5, 0))
		.OnComboBoxOpened(this, &SNiagaraNamePropertySelector::OnComboOpening);

	SAssignNew(ElementButton, SComboButton)
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1.f, 1.f)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor(1, 1, 1, 1))
				.Text(this, &SNiagaraNamePropertySelector::GetComboText)
			]
		]
		.MenuContent()
		[
			SNew(SListViewSelectorDropdownMenu<TSharedPtr<FName>>, SearchBox, ElementsListView)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(2)
				[
					SNew(SBox)
					.WidthOverride(175)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
							.Padding(1.f)
							.AutoHeight()
							[
								SearchBox.ToSharedRef()
							]
						+ SVerticalBox::Slot()
							.MaxHeight(400)
							.Padding(8.f)
							[
								ElementsListView.ToSharedRef()
							]
					]
				]
			]
		]
		.IsFocusable(true)
		.ContentPadding(FMargin(5, 0))
		.OnComboBoxOpened(this, &SNiagaraNamePropertySelector::OnComboOpening);

	ElementsListView->EnableToolTipForceField(true);
	// SComboButton can automatically handle setting focus to a specified control when the combo button is opened
	ElementButton->SetMenuContentWidgetToFocus(SearchBox);

	ChildSlot
	[
		ElementButton.ToSharedRef()
	];
};

void SNiagaraNamePropertySelector::OnSearchBoxTextChanged(const FText& InSearchText)
{
	CurrentSearchString = InSearchText;

	// Generate a filtered list
	ElementsListView->ClearSelection();
	GenerateFilteredElementList(CurrentSearchString.ToString());
	// Select first element, if any
	if (FilteredSourceList.Num() > 0)
	{
		ElementsListView->SetSelection(FilteredSourceList[0], ESelectInfo::OnNavigation);
	}
	// Ask the combo to update its contents on next tick
	ElementsListView->RequestListRefresh();
}

void SNiagaraNamePropertySelector::GenerateFilteredElementList(const FString& InSearchText)
{
	if (InSearchText.IsEmpty())
	{
		FilteredSourceList.Empty();
		FilteredSourceList.Append(OptionsSourceList);
	}
	else
	{
		FilteredSourceList.Empty();
		for (TSharedPtr<FName> Name : OptionsSourceList)
		{
			if (Name->ToString().Contains(InSearchText))
			{
				FilteredSourceList.AddUnique(Name);
			}
		}
	}
}

void SNiagaraNamePropertySelector::SetSourceArray(TArray<TSharedPtr<FName>>& InOptionsSource)
{
	OptionsSourceList = InOptionsSource;
	GenerateFilteredElementList(CurrentSearchString.ToString());
	if (ElementsListView.IsValid())
	{
		ElementsListView->RequestListRefresh();
	}
}

void SNiagaraNamePropertySelector::OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	// no need to handle this for now, due to the way SListViewSelectorDropdownMenu works (it eats up the Enter key and sends it to the list)
}

TSharedRef<ITableRow> SNiagaraNamePropertySelector::GenerateAddElementRow(TSharedPtr<FName> Entry, const TSharedRef<STableViewBase> &OwnerTable) const
{
	return
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.NoHoverTableRow"))
		.ShowSelection(true)
		[
			SNew(SBox)
			.Padding(2.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Entry->ToString()))
				.TextStyle(FNiagaraEditorWidgetsStyle::Get(), "NiagaraEditor.NamePropertySelectionEntry")
				.HighlightText(this, &SNiagaraNamePropertySelector::GetCurrentSearchString)
			]
		];
}

FText SNiagaraNamePropertySelector::GetComboText() const
{
	FText ValueText;
	PropertyHandle->GetValueAsDisplayText(ValueText);
	return ValueText;
}

void SNiagaraNamePropertySelector::OnSelectionChanged(TSharedPtr<FName> InNewSelection, ESelectInfo::Type SelectInfo)
{
	if (InNewSelection.IsValid() && (SelectInfo != ESelectInfo::OnNavigation))
	{
		PropertyHandle->SetValue(*InNewSelection);
		ElementButton->SetIsOpen(false, false);
	}
}

void SNiagaraNamePropertySelector::OnComboOpening()
{
	SearchBox->SetText(FText::GetEmpty());
}

#undef LOCTEXT_NAMESPACE