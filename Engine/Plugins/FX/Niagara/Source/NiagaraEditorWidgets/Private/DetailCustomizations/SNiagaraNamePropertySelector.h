// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"
#include "PropertyCustomizationHelpers.h"

class IPropertyHandle;
class SSearchBox;

/** Details customization for arrays composed of FName properties (or wrappers). The array contents are selected from a predetermined source list.*/
class SNiagaraNamePropertySelector : public SCompoundWidget 
{
public:
	SLATE_BEGIN_ARGS(SNiagaraNamePropertySelector) { }
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InBaseProperty, const TArray<TSharedPtr<FName>>& InOptionsSource);
	
	void SetSourceArray(TArray<TSharedPtr<FName>>& InOptionsSource);

private: 
	void OnComboOpening();
	
	void OnSelectionChanged(TSharedPtr<FName> InNewSelection, ESelectInfo::Type);
	
	TSharedRef<SWidget> MakeSelectionWidget(TSharedPtr<FName> InItem);
	
	FText GetComboText() const;
	
	void OnSearchBoxTextChanged(const FText& InSearchText);

	void OnSearchBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	TSharedRef<ITableRow> GenerateAddElementRow(TSharedPtr<FName> Entry, const TSharedRef<STableViewBase> &OwnerTable) const;

	void GenerateFilteredElementList(const FString& InSearchText);
	
	FText GetCurrentSearchString() const { return CurrentSearchString; };

private:
	TArray<TSharedPtr<FName>> OptionsSourceList;
	/** List of component class names, filtered by the current search string */
	TArray<TSharedPtr<FName>> FilteredSourceList;
	/** The current array property being edited */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** The search box control - part of the combo drop down */
	TSharedPtr<SSearchBox> SearchBox;

	/** The component list control - part of the combo drop down */
	TSharedPtr< SListView<TSharedPtr<FName>> > ElementsListView;
	
	/** The current search string */
	FText CurrentSearchString;

	TSharedPtr<SComboButton> ElementButton;
};
