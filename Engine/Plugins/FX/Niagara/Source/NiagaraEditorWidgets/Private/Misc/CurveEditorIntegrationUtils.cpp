// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveEditorIntegrationUtils.h"

#include "NiagaraDataInterfaceCurveBase.h"
#include "RichCurveEditorModel.h"
#include "Modification/Resolution/CurveMetaDataIdentifiers.h"

namespace UE::NiagaraEditorWidgets
{
TUniquePtr<FCurveModel> ResolveNiagaraCurveModel(const CurveEditor::FCurveModelLookUpArgs& InArgs)
{
	const CurveEditor::FCurveMetaDataIdentifiers* Metadata = InArgs.GetCurveMetaData();
	if (!Metadata)
	{
		return nullptr;
	}
	
	UNiagaraDataInterfaceCurveBase* CurveOwner = Cast<UNiagaraDataInterfaceCurveBase>(Metadata->Owner.Get());
	if (!CurveOwner)
	{
		return nullptr;
	}
	
	TArray<UNiagaraDataInterfaceCurveBase::FCurveData> CurveData;
	CurveOwner->GetCurveData(CurveData);
	if (CurveData.Num() == 1)
	{
		return MakeUnique<FRichCurveEditorModelRaw>(CurveData[0].Curve, CurveOwner);
	}

	const UNiagaraDataInterfaceCurveBase::FCurveData* Data = CurveData.FindByPredicate([Metadata](const UNiagaraDataInterfaceCurveBase::FCurveData& Data)
	{
		return Data.Name == Metadata->ChannelName;
	});
	return Data ? MakeUnique<FRichCurveEditorModelRaw>(Data->Curve, CurveOwner) : nullptr;
}
}
