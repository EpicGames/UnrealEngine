// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modification/Resolution/CurveModelLookUpInfo.h"
#include "Templates/UniquePtr.h"

class FCurveModel;

namespace UE::NiagaraEditorWidgets
{
/**
 * Tries to create a FCurveModel that was previously in the FCurveEditor but has since been removed.
 * This is needed for Curve Editor's command based undo system.
 * 
 * The user may make a change to curve X, then click on curve Y, and then undo; FCurveEditor::CurveData only contains the visible curves, so it will
 * ask us to temporarily recreate the curve for X, which we try to match based off of the metadata it had when it existed.
 */
TUniquePtr<FCurveModel> ResolveNiagaraCurveModel(const CurveEditor::FCurveModelLookUpArgs& InArgs);
}
