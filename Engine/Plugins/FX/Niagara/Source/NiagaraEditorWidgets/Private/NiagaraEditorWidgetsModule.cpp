// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorWidgetsModule.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "SNiagaraStack.h"
#include "DetailCustomizations/NiagaraDataInterfaceCurveDetails.h"
#include "DetailCustomizations/NiagaraDataInterfaceDetails.h"
#include "DetailCustomizations/NiagaraDataInterfaceGrid2DCollectionDetails.h"
#include "DetailCustomizations/NiagaraDataInterfaceGrid3DCollectionDetails.h"
#include "DetailCustomizations/NiagaraDataInterfaceSkeletalMeshDetails.h"
#include "DetailCustomizations/NiagaraDataInterfaceStaticMeshDetails.h"
#include "DetailCustomizations/NiagaraDataInterfaceSocketReaderDetails.h"
#include "DetailCustomizations/NiagaraDataInterfaceMeshRendererInfoDetails.h"
#include "DetailCustomizations/NiagaraDataInterfaceSpriteRendererInfoDetails.h"
#include "DetailCustomizations/NiagaraDecalRendererDetails.h"
#include "DetailCustomizations/NiagaraLightRendererDetails.h"
#include "DetailCustomizations/NiagaraMeshRendererDetails.h"
#include "DetailCustomizations/NiagaraRibbonRendererDetails.h"
#include "DetailCustomizations/NiagaraSpriteRendererDetails.h"
#include "DetailCustomizations/NiagaraDataChannelDetails.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "SNiagaraOverviewGraph.h"
#include "NiagaraEditorWidgetsUtilities.h"
#include "Stack/SNiagaraStackIssueIcon.h"
#include "SNiagaraScratchPadScriptManager.h"
#include "SNiagaraCurveOverview.h"
#include "SNiagaraCurveKeySelector.h"
#include "SNiagaraCurveThumbnail.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "GraphEditorActions.h"
#include "NiagaraEditorCommands.h"

#include "NiagaraDataChannelCommon.h"
#include "NiagaraDataChannel_Islands.h"

IMPLEMENT_MODULE(FNiagaraEditorWidgetsModule, NiagaraEditorWidgets);

FNiagaraStackCurveEditorOptions::FNiagaraStackCurveEditorOptions()
	: ViewMinInput(0)
	, ViewMaxInput(1)
	, ViewMinOutput(0)
	, ViewMaxOutput(1)
	, bIsGradientVisible(true)
	, bNeedsInitializeView(true)
	, Height(180)
{
}

bool FNiagaraStackCurveEditorOptions::GetNeedsInitializeView() const
{
	return bNeedsInitializeView;
}

void FNiagaraStackCurveEditorOptions::InitializeView(float InViewMinInput, float InViewMaxInput, float InViewMinOutput, float InViewMaxOutput)
{
	ViewMinInput = InViewMinInput;
	ViewMaxInput = InViewMaxInput;
	ViewMinOutput = InViewMinOutput;
	ViewMaxOutput = InViewMaxOutput;
	bNeedsInitializeView = false;
}

float FNiagaraStackCurveEditorOptions::GetViewMinInput() const
{
	return ViewMinInput;
}

float FNiagaraStackCurveEditorOptions::GetViewMaxInput() const
{
	return ViewMaxInput;
}

void FNiagaraStackCurveEditorOptions::SetInputViewRange(float InViewMinInput, float InViewMaxInput)
{
	ViewMinInput = InViewMinInput;
	ViewMaxInput = InViewMaxInput;
}

float FNiagaraStackCurveEditorOptions::GetViewMinOutput() const
{
	return ViewMinOutput;
}

float FNiagaraStackCurveEditorOptions::GetViewMaxOutput() const
{
	return ViewMaxOutput;
}

void FNiagaraStackCurveEditorOptions::SetOutputViewRange(float InViewMinOutput, float InViewMaxOutput)
{
	ViewMinOutput = InViewMinOutput;
	ViewMaxOutput = InViewMaxOutput;
}

float FNiagaraStackCurveEditorOptions::GetTimelineLength() const
{
	return ViewMaxInput - ViewMinInput;
}

float FNiagaraStackCurveEditorOptions::GetHeight() const
{
	return Height;
}

void FNiagaraStackCurveEditorOptions::SetHeight(float InHeight)
{
	Height = InHeight;
}

bool FNiagaraStackCurveEditorOptions::GetIsGradientVisible() const
{
	return bIsGradientVisible;
}

void FNiagaraStackCurveEditorOptions::SetIsGradientVisible(bool bInIsGradientVisible)
{
	bIsGradientVisible = bInIsGradientVisible;
}

void FNiagaraEditorWidgetsModule::StartupModule()
{
	FNiagaraEditorModule& NiagaraEditorModule = FModuleManager::LoadModuleChecked<FNiagaraEditorModule>("NiagaraEditor");
	WidgetProvider = MakeShared<FNiagaraEditorWidgetProvider>();
	NiagaraEditorModule.RegisterWidgetProvider(WidgetProvider.ToSharedRef());

	FNiagaraEditorWidgetsStyle::Register();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterface", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceDetailsBase::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceCurve", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceCurveDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceVector2DCurve", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceVector2DCurveDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceVectorCurve", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceVectorCurveDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceVector4Curve", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceVector4CurveDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceColorCurve", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceColorCurveDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceSkeletalMesh", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceSkeletalMeshDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceStaticMesh", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceStaticMeshDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceSocketReader", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceSocketReaderDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceGrid2DCollection", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceGrid2DCollectionDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceGrid3DCollection", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceGrid3DCollectionDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceMeshRendererInfo", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceMeshRendererInfoDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceSpriteRendererInfo", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceSpriteRendererInfoDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraLightRendererProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraLightRendererDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraMeshRendererProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraMeshRendererDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraRibbonRendererProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraRibbonRendererDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraSpriteRendererProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraSpriteRendererDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("NiagaraDecalRendererProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDecalRendererDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout("NiagaraDataInterfaceDataChannelRead", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataInterfaceDataChannelReadDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("K2Node_WriteDataChannel", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataChannelBPNodeDetails::MakeInstance));
	PropertyModule.RegisterCustomClassLayout("K2Node_ReadDataChannel", FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataChannelBPNodeDetails::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(
		UNiagaraDataChannel_Islands::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FNiagaraDataChannelIslandsDetails::MakeInstance));

	ReinitializeStyleCommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("fx.NiagaraEditorWidgets.ReinitializeStyle"),
		TEXT("Reinitializes the style for the niagara editor widgets module.  Used in conjuction with live coding for UI tweaks.  May crash the editor if style objects are in use."),
		FConsoleCommandDelegate::CreateRaw(this, &FNiagaraEditorWidgetsModule::ReinitializeStyle));

	FGraphEditorCommands::Register();
	FNiagaraEditorCommands::Register();
}

void FNiagaraEditorWidgetsModule::ShutdownModule()
{
	FNiagaraEditorModule* NiagaraEditorModule = FModuleManager::GetModulePtr<FNiagaraEditorModule>("NiagaraEditor");
	if (NiagaraEditorModule != nullptr)
	{
		NiagaraEditorModule->UnregisterWidgetProvider(WidgetProvider.ToSharedRef());
	}

	FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyModule != nullptr)
	{
		PropertyModule->UnregisterCustomClassLayout("NiagaraDataInterface");
		PropertyModule->UnregisterCustomClassLayout("NiagaraDataInterfaceCurve");
		PropertyModule->UnregisterCustomClassLayout("NiagaraDataInterfaceVector2DCurve");
		PropertyModule->UnregisterCustomClassLayout("NiagaraDataInterfaceVectorCurve");
		PropertyModule->UnregisterCustomClassLayout("NiagaraDataInterfaceVector4Curve");
		PropertyModule->UnregisterCustomClassLayout("NiagaraDataInterfaceColorCurve");
		PropertyModule->UnregisterCustomClassLayout("NiagaraDataInterfaceSkeletalMesh");
		PropertyModule->UnregisterCustomClassLayout("NiagaraDataInterfaceStaticMesh");
	}

	if (ReinitializeStyleCommand != nullptr)
	{
		IConsoleManager::Get().UnregisterConsoleObject(ReinitializeStyleCommand);
	}

	FNiagaraEditorWidgetsStyle::Shutdown();
}

FNiagaraEditorWidgetsModule& FNiagaraEditorWidgetsModule::Get()
{
	return FModuleManager::GetModuleChecked<FNiagaraEditorWidgetsModule>("NiagaraEditorWidgets");
}

void FNiagaraEditorWidgetsModule::ReinitializeStyle()
{
	FNiagaraEditorWidgetsStyle::ReinitializeStyle();
}

TSharedRef<FNiagaraStackCurveEditorOptions> FNiagaraEditorWidgetsModule::GetOrCreateStackCurveEditorOptionsForObject(UObject* Object, float DefaultHeight)
{
	TSharedRef<FNiagaraStackCurveEditorOptions>* StackCurveEditorOptions = ObjectToStackCurveEditorOptionsMap.Find(FObjectKey(Object));
	if (StackCurveEditorOptions == nullptr)
	{
		StackCurveEditorOptions = &ObjectToStackCurveEditorOptionsMap.Add(FObjectKey(Object), MakeShared<FNiagaraStackCurveEditorOptions>());
		(*StackCurveEditorOptions)->SetHeight(DefaultHeight);
	}
	return *StackCurveEditorOptions;
}

TSharedRef<SWidget> FNiagaraEditorWidgetsModule::FNiagaraEditorWidgetProvider::CreateStackView(UNiagaraStackViewModel& StackViewModel) const
{
	return SNew(SNiagaraStack, &StackViewModel);
}

TSharedRef<SWidget> FNiagaraEditorWidgetsModule::FNiagaraEditorWidgetProvider::CreateSystemOverview(TSharedRef<FNiagaraSystemViewModel> SystemViewModel, const FAssetData& EditedAsset) const
{
	return SNew(SNiagaraOverviewGraph, SystemViewModel->GetOverviewGraphViewModel().ToSharedRef(), EditedAsset);
}

TSharedRef<SWidget> FNiagaraEditorWidgetsModule::FNiagaraEditorWidgetProvider::CreateStackIssueIcon(UNiagaraStackViewModel& StackViewModel, UNiagaraStackEntry& StackEntry) const
{
	return SNew(SNiagaraStackIssueIcon, &StackViewModel, &StackEntry);
}

TSharedRef<SWidget> FNiagaraEditorWidgetsModule::FNiagaraEditorWidgetProvider::CreateScriptScratchPadManager(UNiagaraScratchPadViewModel& ScriptScratchPadViewModel) const
{
	return SNew(SNiagaraScratchPadScriptManager, &ScriptScratchPadViewModel);
}

TSharedRef<SWidget> FNiagaraEditorWidgetsModule::FNiagaraEditorWidgetProvider::CreateCurveOverview(TSharedRef<FNiagaraSystemViewModel> SystemViewModel) const
{
	return SNew(SNiagaraCurveOverview, SystemViewModel);
}

TSharedRef<SWidget> FNiagaraEditorWidgetsModule::FNiagaraEditorWidgetProvider::CreateCurveKeySelector(TSharedPtr<FCurveEditor> InCurveEditor, const TArray<FCurveEditorTreeItemID>& InCurveTreeItemIds, TSharedPtr<SCurveEditorTree> InCurveEditorTree) const
{
	return SNew(SNiagaraCurveKeySelector, InCurveEditor, InCurveTreeItemIds, InCurveEditorTree);
}

TSharedRef<SWidget> FNiagaraEditorWidgetsModule::FNiagaraEditorWidgetProvider::CreateCurveThumbnail(const FRichCurve& Curve) const
{
	return SNew(SNiagaraCurveThumbnail, Curve);
}

FLinearColor FNiagaraEditorWidgetsModule::FNiagaraEditorWidgetProvider::GetColorForExecutionCategory(FName ExecutionCategory) const
{
	return FNiagaraEditorWidgetsStyle::Get().GetColor(FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForExecutionCategory(ExecutionCategory));
}
