// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorWidgetsStyle.h"

#include "Styling/SlateStyleMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SlateGameResources.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Settings/EditorStyleSettings.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"

const FVector2D Icon6x6(6.0f, 6.0f);
const FVector2D Icon8x8(8.0f, 8.0f);
const FVector2D Icon8x16(8.0f, 16.0f);
const FVector2D Icon12x12(12.0f, 12.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon30x30(30.0f, 30.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TSharedPtr<FNiagaraEditorWidgetsStyle> FNiagaraEditorWidgetsStyle::NiagaraEditorWidgetsStyle = nullptr;

void FNiagaraEditorWidgetsStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FNiagaraEditorWidgetsStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

void FNiagaraEditorWidgetsStyle::Shutdown()
{
	Unregister();
	NiagaraEditorWidgetsStyle.Reset();
}

void FNiagaraEditorWidgetsStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const FNiagaraEditorWidgetsStyle& FNiagaraEditorWidgetsStyle::Get()
{
	if(!NiagaraEditorWidgetsStyle.IsValid())
	{
		NiagaraEditorWidgetsStyle = MakeShareable(new FNiagaraEditorWidgetsStyle());
	}
	
	return *NiagaraEditorWidgetsStyle;
}

void FNiagaraEditorWidgetsStyle::ReinitializeStyle()
{
	Unregister();
	NiagaraEditorWidgetsStyle.Reset();
	Register();
}

FNiagaraEditorWidgetsStyle::FNiagaraEditorWidgetsStyle() : FSlateStyleSet("NiagaraEditorWidgetsStyle")
{
	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("FX/Niagara/Content/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	
	InitColors();
	InitTextStyles();
	InitRowStyles();
	InitButtonStyles();
	InitInlineDynamicInputBrushes();
	InitSystemOverviewMisc();
	InitIcons();
	InitSizes();
	InitStackDropTarget();
	InitMisc();
}

void FNiagaraEditorWidgetsStyle::InitColors()
{
	Set("NiagaraEditor.Stack.UnknownColor", FLinearColor(1, 0, 1));

	Set("NiagaraEditor.Stack.ForegroundColor", FLinearColor(FColor(220, 220, 220)));
	Set("NiagaraEditor.Stack.GroupForegroundColor", FLinearColor(FColor(220, 220, 220)));
	Set("NiagaraEditor.Stack.FlatButtonColor", FLinearColor(FColor(205, 205, 205)));
	Set("NiagaraEditor.Stack.DividerColor", FLinearColor(FColor(92, 92, 92)));
	
	Set("NiagaraEditor.Stack.Stats.EvalTypeColor", FLinearColor(FColor(168, 168, 168)));
	Set("NiagaraEditor.Stack.Stats.RuntimePlaceholderColor", FLinearColor(FColor(86, 86, 86)));
	Set("NiagaraEditor.Stack.Stats.RuntimeUsageColorDefault", FLinearColor(FColor(200, 60, 60)));
	Set("NiagaraEditor.Stack.Stats.RuntimeUsageColorParticleUpdate", FLinearColor(FColor(246, 3, 142)));
	Set("NiagaraEditor.Stack.Stats.RuntimeUsageColorParticleSpawn", FLinearColor(FColor(255, 181, 0)));
	Set("NiagaraEditor.Stack.Stats.RuntimeUsageColorSystem", FLinearColor(FColor(20, 161, 255)));
	Set("NiagaraEditor.Stack.Stats.RuntimeUsageColorEmitter", FLinearColor(FColor(241, 99, 6)));
	Set("NiagaraEditor.Stack.Stats.LowCostColor", FLinearColor(FColor(143, 185, 130)));
	Set("NiagaraEditor.Stack.Stats.MediumCostColor", FLinearColor(FColor(220, 210, 86)));
	Set("NiagaraEditor.Stack.Stats.HighCostColor", FLinearColor(FColor(205, 114, 69)));
	Set("NiagaraEditor.Stack.Stats.MaxCostColor", FLinearColor(FColor(200, 60, 60)));

	Set("NiagaraEditor.Stack.AccentColor.System", FLinearColor(FColor(67, 105, 124)));
	Set("NiagaraEditor.Stack.AccentColor.Emitter", FLinearColor(FColor(126, 87, 67)));
	Set("NiagaraEditor.Stack.AccentColor.Particle", FLinearColor(FColor(87, 107, 61)));
	Set("NiagaraEditor.Stack.AccentColor.Render", FLinearColor(FColor(134, 80, 80)));
	Set("NiagaraEditor.Stack.AccentColor.None", FLinearColor::Transparent);

	Set("NiagaraEditor.Stack.IconColor.System", FLinearColor(FColor(1, 202, 252)));
	Set("NiagaraEditor.Stack.IconColor.Emitter", FLinearColor(FColor(241, 99, 6)));
	Set("NiagaraEditor.Stack.IconColor.StatelessEmitter", FLinearColor(FColor(255, 41, 2)));
	Set("NiagaraEditor.Stack.IconColor.Particle", FLinearColor(FColor(131, 218, 9)));
	Set("NiagaraEditor.Stack.IconColor.Render", FLinearColor(FColor(230, 102, 102)));
	Set("NiagaraEditor.Stack.IconColor.VersionUpgrade", FLinearColor(FColor(255, 181, 0)));

	Set("NiagaraEditor.Stack.InputValueIconColor.Linked", FLinearColor(FColor::Purple));
	Set("NiagaraEditor.Stack.InputValueIconColor.Data", FLinearColor(FColor::Yellow));
	Set("NiagaraEditor.Stack.InputValueIconColor.ObjectAsset", FLinearColor(FColor::Yellow));
	Set("NiagaraEditor.Stack.InputValueIconColor.Dynamic", FLinearColor(FColor::Cyan));
	Set("NiagaraEditor.Stack.InputValueIconColor.Expression", FLinearColor(FColor::Green));
	Set("NiagaraEditor.Stack.InputValueIconColor.Default", FLinearColor(FColor::White));

	Set("NiagaraEditor.Stack.SearchHighlightColor", FStyleColors::Highlight);

	InitSystemOverviewColors();
	InitScopeColors();
}

void FNiagaraEditorWidgetsStyle::InitTextStyles()
{
	const FTextBlockStyle NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FEditableTextBoxStyle NormalEditableTextBox = FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");

	const FTextBlockStyle CategoryText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DetailsView.CategoryTextStyle");
	FSlateFontInfo CategoryFont = FAppStyle::Get().GetFontStyle(TEXT("PropertyWindow.BoldFont"));
	FTextBlockStyle StackGroupText = FTextBlockStyle(CategoryText)
		.SetFont(CategoryFont);

	InitStackTextStyles();
	
	// Scalability
	FTextBlockStyle EffectTypeScalabilityPropertyTextStyle = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 8))
		.SetColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));

	FTextBlockStyle EffectTypeScalabilityValueTextStyle = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 9))
		.SetColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));

	Set("NiagaraEditor.Scalability.EffectType.Property", EffectTypeScalabilityPropertyTextStyle);
	Set("NiagaraEditor.Scalability.EffectType.Value", EffectTypeScalabilityPropertyTextStyle);

	FTextBlockStyle NamePropertySelectionEntryTextBlockStyle = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 10))
		.SetColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));
	Set("NiagaraEditor.NamePropertySelectionEntry", NamePropertySelectionEntryTextBlockStyle);
	
	// Curves
	FSlateFontInfo CurveOverviewTopLevelFont = DEFAULT_FONT("Bold", 10);
	FTextBlockStyle CurveOverviewTopLevelText = FTextBlockStyle(NormalText)
		.SetFont(CurveOverviewTopLevelFont);
	Set("NiagaraEditor.CurveOverview.TopLevelText", CurveOverviewTopLevelText);

	FSlateFontInfo CurveOverviewScriptFont = DEFAULT_FONT("Bold", 10);
	FTextBlockStyle CurveOverviewScriptText = FTextBlockStyle(NormalText)
		.SetFont(CurveOverviewScriptFont);
	Set("NiagaraEditor.CurveOverview.ScriptText", CurveOverviewScriptText);

	FSlateFontInfo CurveOverviewModuleFont = DEFAULT_FONT("Bold", 10);
	FTextBlockStyle CurveOverviewModuleText = FTextBlockStyle(NormalText)
		.SetFont(CurveOverviewModuleFont);
	Set("NiagaraEditor.CurveOverview.ModuleText", CurveOverviewModuleText);

	FSlateFontInfo CurveOverviewInputFont = DEFAULT_FONT("Bold", 9);
	FTextBlockStyle CurveOverviewInputText = FTextBlockStyle(NormalText)
		.SetFont(CurveOverviewInputFont);
	Set("NiagaraEditor.CurveOverview.InputText", CurveOverviewInputText);

	FSlateFontInfo CurveOverviewDataInterfaceFont = DEFAULT_FONT("Bold", 10);
	FTextBlockStyle CurveOverviewDataInterfaceText = FTextBlockStyle(NormalText)
		.SetFont(CurveOverviewDataInterfaceFont);
	Set("NiagaraEditor.CurveOverview.DataInterfaceText", CurveOverviewDataInterfaceText);

	FSlateFontInfo CurveOverviewSecondaryFont = DEFAULT_FONT("Italic", 9);
	FTextBlockStyle CurveOverviewSecondaryText = FTextBlockStyle(NormalText)
		.SetFont(CurveOverviewSecondaryFont);
	Set("NiagaraEditor.CurveOverview.SecondaryText", CurveOverviewSecondaryText);

	FSlateFontInfo CurveOverviewCurveComponentFont = DEFAULT_FONT("Regular", 9);
	FTextBlockStyle CurveOverviewCurveComponentText = FTextBlockStyle(NormalText)
		.SetFont(CurveOverviewCurveComponentFont);
	Set("NiagaraEditor.CurveOverview.CurveComponentText", CurveOverviewCurveComponentText);

	FSlateFontInfo CurveOverviewDefaultFont = DEFAULT_FONT("Bold", 10);
	FTextBlockStyle CurveOverviewDefaultText = FTextBlockStyle(NormalText)
		.SetFont(CurveOverviewDefaultFont);
	Set("NiagaraEditor.CurveOverview.DefaultText", CurveOverviewDefaultText);

	Set("NiagaraEditor.CurveDetails.TextButtonForeground", FLinearColor::White);

	// Scratch pad
	FSlateFontInfo ScratchPadEditorHeaderFont = DEFAULT_FONT("Bold", 11);
	FTextBlockStyle ScratchPadEditorHeaderText = FTextBlockStyle(NormalText)
		.SetFont(ScratchPadEditorHeaderFont);
	Set("NiagaraEditor.ScratchPad.EditorHeaderText", ScratchPadEditorHeaderText);

	FSlateFontInfo ScratchPadSubSectionHeaderFont = DEFAULT_FONT("Bold", 9);
	FTextBlockStyle ScratchPadSubSectionHeaderText = FTextBlockStyle(NormalText)
		.SetFont(ScratchPadSubSectionHeaderFont);
	Set("NiagaraEditor.ScratchPad.SubSectionHeaderText", ScratchPadSubSectionHeaderText);
	
	// System Overview
	FSlateFontInfo SystemOverviewListHeaderFont = DEFAULT_FONT("Bold", 12);
	FTextBlockStyle SystemOverviewListHeaderText = FTextBlockStyle(NormalText)
		.SetFont(SystemOverviewListHeaderFont);
	Set("NiagaraEditor.SystemOverview.ListHeaderText", SystemOverviewListHeaderText);

	FSlateFontInfo SystemOverviewItemFont = DEFAULT_FONT("Regular", 9);
	FTextBlockStyle SystemOverviewItemText = FTextBlockStyle(NormalText)
		.SetFont(SystemOverviewItemFont);
	Set("NiagaraEditor.SystemOverview.ItemText", SystemOverviewItemText);

	FSlateFontInfo SystemOverviewZoomedOutNodeFont = DEFAULT_FONT("Regular", 45);
	FTextBlockStyle SystemOverviewZoomedOutNodeText = FTextBlockStyle(NormalText)
		.SetFont(SystemOverviewZoomedOutNodeFont);
	Set("NiagaraEditor.SystemOverview.ZoomedOutNodeFont", SystemOverviewZoomedOutNodeText);

	FSlateFontInfo SystemOverviewAlternateItemFont = DEFAULT_FONT("Italic", 9);
	FTextBlockStyle SystemOverviewAlternateItemText = FTextBlockStyle(NormalText)
		.SetFont(SystemOverviewAlternateItemFont);

	FSlateFontInfo SystemOverviewInlineParametersFont = DEFAULT_FONT("Regular", 7);
	FTextBlockStyle SystemOverviewInlineParametersText = FTextBlockStyle(NormalText)
		.SetFont(SystemOverviewInlineParametersFont);

	Set("NiagaraEditor.SystemOverview.ItemText", SystemOverviewItemText);
	
	Set("NiagaraEditor.SystemOverview.AlternateItemText", SystemOverviewAlternateItemText);

	Set("NiagaraEditor.SystemOverview.InlineParameterText", SystemOverviewInlineParametersText);
}

void FNiagaraEditorWidgetsStyle::InitRowStyles()
{
	const FTableRowStyle& NormalTableRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");

	FLinearColor HoverAdd = FLinearColor(0.03f, 0.03f, 0.03f, 0.0f);
	FLinearColor PanelHoverColor = FStyleColors::Panel.GetSpecifiedColor() + HoverAdd;
	FLinearColor RecessedHoverColor = FStyleColors::Recessed.GetSpecifiedColor() + HoverAdd;
	FLinearColor HeaderHoverColor = FStyleColors::Header.GetSpecifiedColor() + HoverAdd;
	
	FSlateBoxBrush ScratchPadCategoryBrush = BOX_BRUSH("Icons/CategoryRow", FMargin(2.0f / 8.0f), FLinearColor(FColor(48, 48, 48)));
	FSlateBoxBrush ScratchPadHoveredCategoryBrush = BOX_BRUSH("Icons/CategoryRow", FMargin(2.0f / 8.0f), FLinearColor(FColor(38, 38, 38)));
	Set("NiagaraEditor.ScratchPad.CategoryRow", FTableRowStyle(NormalTableRowStyle)
		.SetEvenRowBackgroundBrush(ScratchPadCategoryBrush)
		.SetOddRowBackgroundBrush(ScratchPadCategoryBrush)
		.SetEvenRowBackgroundHoveredBrush(ScratchPadHoveredCategoryBrush)
		.SetOddRowBackgroundHoveredBrush(ScratchPadHoveredCategoryBrush));
	
	FTableRowStyle BaseStackTableRowStyle = FTableRowStyle(NormalTableRowStyle)
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(FSlateColorBrush(FStyleColors::Select))
		.SetActiveHoveredBrush(FSlateColorBrush(FStyleColors::Select))
		.SetInactiveBrush(FSlateColorBrush(FStyleColors::SelectInactive))
		.SetActiveHighlightedBrush(FSlateColorBrush(FStyleColors::PrimaryHover))
		.SetTextColor(FStyleColors::Foreground);

	Set("NiagaraEditor.Stack.TableViewRow.ItemContent", FTableRowStyle(BaseStackTableRowStyle)
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(PanelHoverColor))
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(PanelHoverColor))
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Panel))
		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Panel)));

	Set("NiagaraEditor.Stack.TableViewRow.ItemContentAdvanced", FTableRowStyle(BaseStackTableRowStyle)
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(RecessedHoverColor))
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(RecessedHoverColor))
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed)));

	Set("NiagaraEditor.Stack.TableViewRow.ItemHeader", FTableRowStyle(NormalTableRowStyle)
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(HeaderHoverColor))
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(HeaderHoverColor))
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Header))
		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Header)));

	Set("NiagaraEditor.Stack.TableViewRow.ItemContentNote", FTableRowStyle(BaseStackTableRowStyle));
	
	Set("NiagaraEditor.Stack.TableViewRow.Spacer", FTableRowStyle(NormalTableRowStyle)
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(FSlateNoResource())
		.SetActiveHoveredBrush(FSlateNoResource())
		.SetInactiveBrush(FSlateNoResource())
		.SetActiveHighlightedBrush(FSlateNoResource())
		.SetEvenRowBackgroundBrush(FSlateNoResource())
		.SetOddRowBackgroundBrush(FSlateNoResource())
		.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
		.SetOddRowBackgroundHoveredBrush(FSlateNoResource()));

	Set("NiagaraEditor.SystemOverview.TableViewRow.Item", FTableRowStyle(NormalTableRowStyle)
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(PanelHoverColor))
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(PanelHoverColor))
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Panel))
		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Panel)));

	Set("NiagaraEditor.SystemOverview.TableViewRow.Group", FTableRowStyle(NormalTableRowStyle)
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(HeaderHoverColor))
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(HeaderHoverColor))
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Header))
		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Header)));

	Set("Niagara.TableViewRowBorder", new BOX_BRUSH("Icons/Row", FMargin(3.0f / 8.0f), FStyleColors::Recessed));
}

void FNiagaraEditorWidgetsStyle::InitButtonStyles()
{
	FButtonStyle SimpleButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton");
	float NormalDesaturate = .25f;
	float DisabledDesaturate = .5;

	FLinearColor SystemIconColor = GetColor("NiagaraEditor.Stack.IconColor.System");
	FButtonStyle SystemAddItemButtonStyle = SimpleButtonStyle;
	SystemAddItemButtonStyle
		.SetNormalForeground(SystemIconColor.Desaturate(NormalDesaturate))
		.SetHoveredForeground(SystemIconColor)
		.SetPressedForeground(SystemIconColor.Desaturate(NormalDesaturate))
		.SetDisabledForeground(SystemIconColor.Desaturate(DisabledDesaturate));

	FLinearColor EmitterIconColor = GetColor("NiagaraEditor.Stack.IconColor.Emitter");
	FButtonStyle EmitterAddItemButtonStyle = SimpleButtonStyle;
	EmitterAddItemButtonStyle
		.SetNormalForeground(EmitterIconColor.Desaturate(NormalDesaturate))
		.SetHoveredForeground(EmitterIconColor)
		.SetPressedForeground(EmitterIconColor.Desaturate(NormalDesaturate))
		.SetDisabledForeground(EmitterIconColor.Desaturate(DisabledDesaturate));

	FLinearColor ParticleIconColor = GetColor("NiagaraEditor.Stack.IconColor.Particle");
	FButtonStyle ParticleAddItemButtonStyle = SimpleButtonStyle;
	ParticleAddItemButtonStyle
		.SetNormalForeground(ParticleIconColor.Desaturate(NormalDesaturate))
		.SetHoveredForeground(ParticleIconColor)
		.SetPressedForeground(ParticleIconColor.Desaturate(NormalDesaturate))
		.SetDisabledForeground(ParticleIconColor.Desaturate(DisabledDesaturate));

	FLinearColor RenderIconColor = GetColor("NiagaraEditor.Stack.IconColor.Render");
	FButtonStyle RenderAddItemButtonStyle = SimpleButtonStyle;
	RenderAddItemButtonStyle
		.SetNormalForeground(RenderIconColor.Desaturate(NormalDesaturate))
		.SetHoveredForeground(RenderIconColor)
		.SetPressedForeground(RenderIconColor.Desaturate(NormalDesaturate))
		.SetDisabledForeground(RenderIconColor.Desaturate(DisabledDesaturate));

	FButtonStyle LabeledAddItemButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
	LabeledAddItemButtonStyle
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Panel, 4))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Recessed, 4))
		.SetNormalPadding(FMargin(2.0f, 0.0f))
		.SetPressedPadding(FMargin(2.0f, 0.0f));

	FButtonStyle StackSimpleButtonStyle = SimpleButtonStyle;
	StackSimpleButtonStyle
		.SetNormalForeground(FStyleColors::Foreground)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::Foreground)
		.SetDisabledForeground(FStyleColors::Foreground.GetSpecifiedColor().Desaturate(DisabledDesaturate));

	Set("NiagaraEditor.Stack.AddItemButton.System", SystemAddItemButtonStyle);
	Set("NiagaraEditor.Stack.AddItemButton.Emitter", EmitterAddItemButtonStyle);
	Set("NiagaraEditor.Stack.AddItemButton.Particle", ParticleAddItemButtonStyle);
	Set("NiagaraEditor.Stack.AddItemButton.Render", RenderAddItemButtonStyle);
	Set("NiagaraEditor.Stack.LabeledAddItemButton", LabeledAddItemButtonStyle);
	Set("NiagaraEditor.Stack.SimpleButton", StackSimpleButtonStyle);

	FComboButtonStyle CompactComboButton = FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton");
	CompactComboButton.ButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("HoverHintOnly");
	CompactComboButton.DownArrowPadding = FMargin(1.0f, 0.0f);
	Set("NiagaraEditor.Stack.CompactComboButton", CompactComboButton);

	FComboButtonStyle CompactComboButtonWithBorder = FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton");
	CompactComboButtonWithBorder.DownArrowPadding = FMargin(1.0f, 0.0f);
	CompactComboButtonWithBorder.ContentPadding = FMargin(0.0f, 2.0f);
	CompactComboButtonWithBorder.ButtonStyle.NormalPadding = FMargin(6, 1, 2, 2);
	CompactComboButtonWithBorder.ButtonStyle.PressedPadding = FMargin(6, 2, 2, 1);
	Set("NiagaraEditor.Stack.CompactComboButtonWithBorder", CompactComboButtonWithBorder);

	FButtonStyle InlineParameterButtonStyle = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::AccentGray, 4.0f, FStyleColors::Input, 1.f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::White, 4.0f, FStyleColors::Input, 1.f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.f))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f, FStyleColors::Recessed, 1.f))
		.SetNormalForeground(FStyleColors::ForegroundHover)
		.SetHoveredForeground(FStyleColors::ForegroundInverted)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(2.f, 2.f, 2.f, 2.f))
		.SetPressedPadding(FMargin(2.f, 1.0f, 2.f, 0.0f));

	FButtonStyle InlineParameterButtonStyleEnum = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::AccentGray, 4.0f, FStyleColors::Input, 1.f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::White, 4.0f, FStyleColors::Input, 1.f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.f))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f, FStyleColors::Recessed, 1.f))
		.SetNormalForeground(FStyleColors::ForegroundHover)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(2.f, 2.f, 2.f, 2.f))
		.SetPressedPadding(FMargin(2.f, 1.0f, 2.f, 0.0f));

	// this style is used when the user has specified a custom color for an inline parameter. By default we'd tint it a bit more gray.
	FButtonStyle InlineParameterButtonStyleNoTint = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::White, 4.0f, FStyleColors::Input, 1.f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::White, 4.0f, FStyleColors::Input, 1.f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.f))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f, FStyleColors::Recessed, 1.f))
		.SetNormalForeground(FStyleColors::ForegroundHover)
		.SetHoveredForeground(FStyleColors::ForegroundInverted)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(2.f, 2.f, 2.f, 2.f))
		.SetPressedPadding(FMargin(2.f, 1.0f, 2.f, 0.0f));

	FButtonStyle InlineParameterButtonStyleIcon = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
		.SetDisabled(FSlateNoResource())
		.SetNormalForeground(FStyleColors::Foreground)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(1.f))
		.SetPressedPadding(FMargin(1.f, 2.f, 1.f, 0.f));
	
	Set("NiagaraEditor.SystemOverview.InlineParameterButton", InlineParameterButtonStyle);
	Set("NiagaraEditor.SystemOverview.InlineParameterButton.Enum", InlineParameterButtonStyleEnum);
	Set("NiagaraEditor.SystemOverview.InlineParameterButton.NoTint", InlineParameterButtonStyleNoTint);
	Set("NiagaraEditor.SystemOverview.InlineParameterButton.Icon", InlineParameterButtonStyleIcon);
}

void FNiagaraEditorWidgetsStyle::InitInlineDynamicInputBrushes()
{
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Row.Top",				new BOX_BRUSH("Icons/MultiPartBorderRowTop", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Row.Middle",			new BOX_BRUSH("Icons/MultiPartBorderRowMiddle", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Row.Bottom",			new BOX_BRUSH("Icons/MultiPartBorderRowBottom", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Column.Left",			new BOX_BRUSH("Icons/MultiPartBorderColumnLeft", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Column.Middle",		new BOX_BRUSH("Icons/MultiPartBorderColumnMiddle", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Column.Right",		new BOX_BRUSH("Icons/MultiPartBorderColumnRight", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Corner.BottomLeft",	new BOX_BRUSH("Icons/MultiPartBorderCornerBottomLeft", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Corner.BottomRight",	new BOX_BRUSH("Icons/MultiPartBorderCornerBottomRight", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Edge.Bottom",			new BOX_BRUSH("Icons/MultiPartBorderEdgeBottom", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Single",				new BOX_BRUSH("Icons/MultiPartBorderSingle", FMargin(1.0f / 4.0f)));
	Set("NiagaraEditor.Stack.InlineDynamicInput.GraphNode.Pointer",				new IMAGE_BRUSH("Icons/MultiPartBorderPointer", Icon16x16));
}

void FNiagaraEditorWidgetsStyle::InitSystemOverviewColors()
{
	Set("NiagaraEditor.SystemOverview.Item.BackgroundColor", FLinearColor(FColor(62, 62, 62)));
	Set("NiagaraEditor.SystemOverview.Group.BackgroundColor", FLinearColor::Transparent);
	Set("NiagaraEditor.SystemOverview.CheckBoxColor", FLinearColor(FColor(160, 160, 160)));
	Set("NiagaraEditor.SystemOverview.NodeBackgroundColor", FLinearColor(FColor(48, 48, 48)));
	Set("NiagaraEditor.SystemOverview.AffectedAssetsWarningColor", FLinearColor(FColor(255, 184, 0)));
}

void FNiagaraEditorWidgetsStyle::InitScopeColors()
{
	Set("NiagaraEditor.Scope.Engine", FLinearColor(FColor(230, 102, 102)));
	Set("NiagaraEditor.Scope.Owner", FLinearColor(FColor(210, 112, 112)));
	Set("NiagaraEditor.Scope.User", FLinearColor(FColor(114, 226, 254)));
	Set("NiagaraEditor.Scope.System", FLinearColor(FColor(1, 202, 252)));
	Set("NiagaraEditor.Scope.Emitter", FLinearColor(FColor(241, 99, 6)));
	Set("NiagaraEditor.Scope.Particles", FLinearColor(FColor(131, 218, 9)));
	Set("NiagaraEditor.Scope.ScriptPersistent", FLinearColor(FColor(255, 247, 77)));
	Set("NiagaraEditor.Scope.ScriptTransient", FLinearColor(FColor(255, 247, 77)));
}

void FNiagaraEditorWidgetsStyle::InitStackTextStyles()
{
	const FTextBlockStyle NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FEditableTextBoxStyle NormalEditableTextBox = FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");

	const FTextBlockStyle CategoryText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DetailsView.CategoryTextStyle");
	FSlateFontInfo CategoryFont = FAppStyle::Get().GetFontStyle(TEXT("PropertyWindow.BoldFont"));
	FTextBlockStyle StackGroupText = FTextBlockStyle(CategoryText)
		.SetFont(CategoryFont);
	
	Set("NiagaraEditor.Stack.GroupText", StackGroupText);

	FEditableTextBoxStyle StackEditableGroupText = FEditableTextBoxStyle(NormalEditableTextBox)
		.SetFont(CategoryFont);

	FInlineEditableTextBlockStyle StackInlineEditableGroupText = FInlineEditableTextBlockStyle()
		.SetEditableTextBoxStyle(StackEditableGroupText)
		.SetTextStyle(StackGroupText);

	Set("NiagaraEditor.Stack.EditableGroupText", StackInlineEditableGroupText);

	FSlateFontInfo NormalFont = FAppStyle::Get().GetFontStyle(TEXT("PropertyWindow.NormalFont"));
	FTextBlockStyle StackItemText = FTextBlockStyle(CategoryText)
		.SetFont(CategoryFont);

	Set("NiagaraEditor.Stack.ItemText", StackItemText);

	FSlateFontInfo StackItemHeaderValueLabelFont = FAppStyle::Get().GetFontStyle(TEXT("PropertyWindow.NormalFont"));
	FTextBlockStyle StackItemHeaderValueLabelText = FTextBlockStyle(NormalText)
		.SetFont(StackItemHeaderValueLabelFont);
	Set("NiagaraEditor.Stack.ItemHeaderValueLabelText", StackItemHeaderValueLabelText);

	FSlateFontInfo StackItemHeaderValueFont = FAppStyle::Get().GetFontStyle(TEXT("PropertyWindow.NormalFont"));
	FTextBlockStyle StackItemHeaderValueText = FTextBlockStyle(NormalText)
		.SetFont(StackItemHeaderValueFont)
		.SetColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));
	Set("NiagaraEditor.Stack.ItemHeaderValueText", StackItemHeaderValueText);
	
	FTextBlockStyle OverviewStackItemText = FTextBlockStyle(CategoryText)
		.SetFont(NormalFont);
	Set("NiagaraEditor.OverviewStack.ItemText", OverviewStackItemText);

	FEditableTextBoxStyle StackEditableItemText = FEditableTextBoxStyle(NormalEditableTextBox)
		.SetFont(NormalFont);

	FInlineEditableTextBlockStyle StackInlineEditableItemText = FInlineEditableTextBlockStyle()
		.SetEditableTextBoxStyle(StackEditableItemText)
		.SetTextStyle(StackItemText);
	Set("NiagaraEditor.Stack.EditableItemText", StackInlineEditableItemText);

	FTextBlockStyle StackDefaultText = FTextBlockStyle(NormalText)
		.SetFont(NormalFont);
	Set("NiagaraEditor.Stack.DefaultText", StackDefaultText);

	FSlateFontInfo StackSubduedItemFont = DEFAULT_FONT("Regular", 9);
	FTextBlockStyle StackSubduedItemText = FTextBlockStyle(NormalText)
		.SetFont(StackSubduedItemFont);
	Set("NiagaraEditor.Stack.SubduedItemText", StackSubduedItemText);
	
	FTextBlockStyle StackCategoryText = FTextBlockStyle(NormalText);
	Set("NiagaraEditor.Stack.CategoryText", StackCategoryText);
	Set("NiagaraEditor.SystemOverview.GroupHeaderText", StackCategoryText);

	FSlateFontInfo ParameterFont = DEFAULT_FONT("Regular", 8);
	FTextBlockStyle ParameterText = FTextBlockStyle(NormalText)
		.SetFont(ParameterFont);
	Set("NiagaraEditor.Stack.ParameterText", ParameterText);
	
	FSlateFontInfo TextContentFont = DEFAULT_FONT("Regular", 9);
	FTextBlockStyle TextContentText = FTextBlockStyle(NormalText)
		.SetFont(TextContentFont);
	Set("NiagaraEditor.Stack.TextContentText", TextContentText);

	FSlateFontInfo PerfWidgetDetailFont = DEFAULT_FONT("Regular", 7);
	Set("NiagaraEditor.Stack.Stats.DetailFont", PerfWidgetDetailFont);
	FSlateFontInfo PerfWidgetGroupFont = DEFAULT_FONT("Regular", 8);
	Set("NiagaraEditor.Stack.Stats.GroupFont", PerfWidgetGroupFont);
	FSlateFontInfo PerfWidgetEvalTypeFont = DEFAULT_FONT("Regular", 7);
	Set("NiagaraEditor.Stack.Stats.EvalTypeFont", PerfWidgetEvalTypeFont);

	// Inline Dynamic Input
	FSlateFontInfo StackInlineDynamicInputFormatFont = DEFAULT_FONT("Bold", 12);
	FTextBlockStyle StackInlineDynamicInputFormatText = FTextBlockStyle(NormalText)
		.SetFont(StackInlineDynamicInputFormatFont);
	Set("NiagaraEditor.Stack.InlineDynamicInputFormatText", StackInlineDynamicInputFormatText);

	FSlateFontInfo StackInlineDynamicInputFunctionFont = DEFAULT_FONT("Bold", 10);
	FTextBlockStyle StackInlineDynamicInputFunctionText = FTextBlockStyle(NormalText)
		.SetFont(StackInlineDynamicInputFunctionFont);
	Set("NiagaraEditor.Stack.InlineDynamicInputFunctionText", StackInlineDynamicInputFunctionText);
}

void FNiagaraEditorWidgetsStyle::InitSystemOverviewMisc()
{
	Set("NiagaraEditor.SystemOverview.CheckBoxBorder", new CORE_BOX_BRUSH("Common/GroupBorderLight", FMargin(4.0f / 16.0f)));
	Set("NiagaraEditor.SystemOverview.NodeBackgroundBorder", new BOX_BRUSH("Icons/SystemOverviewNodeBackground", FMargin(1.0f / 4.0f)));
}

void FNiagaraEditorWidgetsStyle::InitIcons()
{
	Set("NiagaraEditor.Scalability.EffectType.bAllowCullingForLocalPlayers", new IMAGE_BRUSH_SVG("Icons/Scalability/CullDistance_20", Icon20x20));
	Set("NiagaraEditor.Scalability.EffectType.UpdateFrequency", new IMAGE_BRUSH_SVG("Icons/Scalability/Update", Icon20x20));
	Set("NiagaraEditor.Scalability.EffectType.CullReaction",  new IMAGE_BRUSH_SVG("Icons/Scalability/CullReaction_20", Icon20x20));
	Set("NiagaraEditor.Scalability.EffectType.SignificanceHandler",  new IMAGE_BRUSH_SVG("Icons/Scalability/SignificanceHandler_20", Icon20x20));
	
	Set("NiagaraEditor.CurveDetails.Import.Small", new CORE_IMAGE_BRUSH("Icons/GeneralTools/Import_40x", Icon20x20));
	Set("NiagaraEditor.CurveDetails.ShowInOverview.Small", new CORE_IMAGE_BRUSH("Common/GoToSource", Icon12x12, FLinearColor(.9f, .9f, .9f, 1.0f)));

	Set("Niagara.CompactStackIssue.Error", new IMAGE_BRUSH("Icons/StackIssueError", Icon8x8));
	Set("Niagara.CompactStackIssue.Warning", new IMAGE_BRUSH("Icons/StackIssueWarning", Icon8x8));
	Set("Niagara.CompactStackIssue.Info", new IMAGE_BRUSH("Icons/StackIssueInfo", Icon8x8));
	Set("Niagara.CompactStackIssue.Message", new IMAGE_BRUSH("Icons/StackIssueMessage", Icon8x8));

	Set("NiagaraEditor.Stack.ReadIcon", new IMAGE_BRUSH("Icons/Read_Icon", Icon16x16));
	Set("NiagaraEditor.Stack.ReadWriteIcon", new IMAGE_BRUSH("Icons/Read_Write_Icon", Icon16x16));
	Set("NiagaraEditor.Stack.WriteIcon", new IMAGE_BRUSH_SVG("Icons/Write_Icon", Icon16x16));
	Set("NiagaraEditor.Stack.GoToSourceIcon", new CORE_IMAGE_BRUSH("Common/GoToSource", Icon30x30, FLinearColor::White));
	
	Set("NiagaraEditor.Stack.ParametersIcon", new IMAGE_BRUSH("Icons/SystemParams", Icon12x12, FLinearColor::White));
	Set("NiagaraEditor.Stack.ParametersIconHighlighted", new IMAGE_BRUSH("Icons/SystemParams", Icon16x16, FLinearColor::White));
	Set("NiagaraEditor.Stack.SpawnIcon", new IMAGE_BRUSH("Icons/Spawn", Icon12x12, FLinearColor::White));
	Set("NiagaraEditor.Stack.SpawnIconHighlighted", new IMAGE_BRUSH("Icons/Spawn", Icon16x16, FLinearColor::White));
	Set("NiagaraEditor.Stack.UpdateIcon", new IMAGE_BRUSH("Icons/Update", Icon12x12, FLinearColor::White));
	Set("NiagaraEditor.Stack.UpdateIconHighlighted", new IMAGE_BRUSH("Icons/Update", Icon16x16, FLinearColor::White));
	Set("NiagaraEditor.Stack.EventIcon", new IMAGE_BRUSH("Icons/Event", Icon12x12, FLinearColor::White));
	Set("NiagaraEditor.Stack.EventIconHighlighted", new IMAGE_BRUSH("Icons/Event", Icon16x16, FLinearColor::White));
	Set("NiagaraEditor.Stack.SimulationStageIcon", new IMAGE_BRUSH("Icons/SimulationStage", Icon12x12, FLinearColor::White));
	Set("NiagaraEditor.Stack.SimulationStageIconHighlighted", new IMAGE_BRUSH("Icons/SimulationStage", Icon16x16, FLinearColor::White));
	Set("NiagaraEditor.Stack.RenderIcon", new IMAGE_BRUSH("Icons/Render", Icon12x12, FLinearColor::White));
	Set("NiagaraEditor.Stack.RenderIconHighlighted", new IMAGE_BRUSH("Icons/Render", Icon16x16, FLinearColor::White));

	Set("NiagaraEditor.Stack.StaticInputValue", new IMAGE_BRUSH("Icons/StaticBoolStack", Icon16x16, FLinearColor::White));

	Set("NiagaraEditor.Stack.ModuleHighlight", new IMAGE_BRUSH("Icons/ModuleHighlight", Icon6x6, FLinearColor::White));
	Set("NiagaraEditor.Stack.ModuleHighlightMore", new IMAGE_BRUSH("Icons/ModuleHighlightMore", Icon6x6, FLinearColor::White));
	Set("NiagaraEditor.Stack.ModuleHighlightLarge", new IMAGE_BRUSH("Icons/ModuleHighlightLarge", Icon8x8, FLinearColor::White));
}

void FNiagaraEditorWidgetsStyle::InitSizes()
{
	// Stack
	Set("NiagaraEditor.Stack.IconSize", FVector2D(18.0f, 18.0f));
	Set("NiagaraEditor.Stack.IconHighlightedSize", 16.0f);

}

void FNiagaraEditorWidgetsStyle::InitStackDropTarget()
{
	Set("NiagaraEditor.Stack.DropTarget.BackgroundColor", FLinearColor(1.0f, 1.0f, 1.0f, 0.25f));
	Set("NiagaraEditor.Stack.DropTarget.BackgroundColorHover", FLinearColor(1.0f, 1.0f, 1.0f, 0.1f));
	Set("NiagaraEditor.Stack.DropTarget.BorderVertical", new IMAGE_BRUSH("Icons/StackDropTargetBorder_Vertical", FVector2D(2, 8), FLinearColor::White, ESlateBrushTileType::Vertical));
	Set("NiagaraEditor.Stack.DropTarget.BorderHorizontal", new IMAGE_BRUSH("Icons/StackDropTargetBorder_Horizontal", FVector2D(8, 2), FLinearColor::White, ESlateBrushTileType::Horizontal));
}

void FNiagaraEditorWidgetsStyle::InitMisc()
{
	Set("NiagaraEditor.Stack.ItemHeaderFooter.BackgroundBrush", new FSlateColorBrush(FLinearColor(FColor(20, 20, 20))));

	Set("NiagaraEditor.Stack.Splitter", FSplitterStyle()
		.SetHandleNormalBrush(CORE_IMAGE_BRUSH("Common/SplitterHandleHighlight", Icon8x8, FLinearColor(.1f, .1f, .1f, 1.0f)))
		.SetHandleHighlightBrush(CORE_IMAGE_BRUSH("Common/SplitterHandleHighlight", Icon8x8, FLinearColor::White))
	);

	Set("NiagaraEditor.Stack.SearchResult", new BOX_BRUSH("Icons/SearchResultBorder", FMargin(1.f/8.f)));

	Set("NiagaraEditor.Stack.ItemHeaderValue.BackgroundBrush", new FSlateRoundedBoxBrush(FStyleColors::Panel, 6));
}

