// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/** Manages the style which provides resources for niagara editor widgets. */
class FNiagaraEditorWidgetsStyle : public FSlateStyleSet
{
public:

	static void Register();
	static void Unregister();
	static void Shutdown();

	/** reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set for niagara editor widgets */
	static const FNiagaraEditorWidgetsStyle& Get();

	static void ReinitializeStyle();
private:
	FNiagaraEditorWidgetsStyle();
	void InitColors();
	void InitTextStyles();
	void InitRowStyles();
	void InitButtonStyles();
	void InitInlineDynamicInputBrushes();
	void InitSystemOverviewMisc();
	void InitIcons();
	void InitSizes();
	void InitStackDropTarget();
	void InitMisc();
	
	void InitSystemOverviewColors();
	void InitScopeColors();
	void InitStackTextStyles();

	static TSharedPtr<FNiagaraEditorWidgetsStyle> NiagaraEditorWidgetsStyle;
};
