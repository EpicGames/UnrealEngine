// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraEditorWidgetsUtilities.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackItem.h"
#include "ViewModels/Stack/NiagaraStackModuleItem.h"
#include "ViewModels/NiagaraSystemViewModel.h"
#include "Stack/SNiagaraStackItemGroupAddMenu.h"
#include "NiagaraNodeAssignment.h"
#include "NiagaraEditorWidgetsStyle.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraClipboard.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraNodeFunctionCall.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Modules/ModuleManager.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "NiagaraEditorCommands.h"
#include "NiagaraEmitterEditorData.h"
#include "NiagaraSystemEditorData.h"
#include "ViewModels/NiagaraEmitterViewModel.h"
#include "ViewModels/NiagaraSystemSelectionViewModel.h"
#include "ViewModels/Stack/NiagaraStackInputCategory.h"
#include "ViewModels/Stack/NiagaraStackRendererItem.h"
#include "ViewModels/Stack/NiagaraStackSimulationStageGroup.h"

#define LOCTEXT_NAMESPACE "NiagaraStackEditorWidgetsUtilities"

FName FNiagaraStackEditorWidgetsUtilities::GetColorNameForExecutionCategory(FName ExecutionCategoryName)
{
	if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::System)
	{
		return "NiagaraEditor.Stack.AccentColor.System";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::Emitter)
	{
		return "NiagaraEditor.Stack.AccentColor.Emitter";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::Particle)
	{
		return "NiagaraEditor.Stack.AccentColor.Particle";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::Render)
	{
		return "NiagaraEditor.Stack.AccentColor.Render";
	}
	else
	{
		return  "NiagaraEditor.Stack.AccentColor.None";
	}
}

FName FNiagaraStackEditorWidgetsUtilities::GetIconNameForExecutionSubcategory(FName ExecutionSubcategoryName, bool bIsHighlighted)
{
	if (bIsHighlighted)
	{
		if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Settings)
		{
			return "NiagaraEditor.Stack.ParametersIconHighlighted";
		}
		if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Spawn)
		{
			return "NiagaraEditor.Stack.SpawnIconHighlighted";
		}
		else if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Update)
		{
			return "NiagaraEditor.Stack.UpdateIconHighlighted";
		}
		else if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Event)
		{
			return "NiagaraEditor.Stack.EventIconHighlighted";
		}
		else if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::SimulationStage)
		{
			return "NiagaraEditor.Stack.SimulationStageIconHighlighted";
		}
		else if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Render)
		{
			return "NiagaraEditor.Stack.RenderIconHighlighted";
		}
	}
	else
	{
		if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Settings)
		{
			return "NiagaraEditor.Stack.ParametersIcon";
		}
		if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Spawn)
		{
			return "NiagaraEditor.Stack.SpawnIcon";
		}
		else if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Update)
		{
			return "NiagaraEditor.Stack.UpdateIcon";
		}
		else if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Event)
		{
			return "NiagaraEditor.Stack.EventIcon";
		}
		else if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::SimulationStage)
		{
			return "NiagaraEditor.Stack.SimulationStageIcon";
		}
		else if (ExecutionSubcategoryName == UNiagaraStackEntry::FExecutionSubcategoryNames::Render)
		{
			return "NiagaraEditor.Stack.RenderIcon";
		}
	}

	return NAME_None;
}

FName FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForExecutionCategory(FName ExecutionCategoryName)
{
	if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::System)
	{
		return "NiagaraEditor.Stack.IconColor.System";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::Emitter)
	{
		return "NiagaraEditor.Stack.IconColor.Emitter";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::StatelessEmitter)
	{
		return "NiagaraEditor.Stack.IconColor.StatelessEmitter";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::Particle)
	{
		return "NiagaraEditor.Stack.IconColor.Particle";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::Render)
	{
		return "NiagaraEditor.Stack.IconColor.Render";
	}
	else
	{
		return NAME_None;
	}
}

FName FNiagaraStackEditorWidgetsUtilities::GetAddItemButtonStyleNameForExecutionCategory(FName ExecutionCategoryName)
{
	if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::System)
	{
		return "NiagaraEditor.Stack.AddItemButton.System";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::Emitter)
	{
		return "NiagaraEditor.Stack.AddItemButton.Emitter";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::Particle)
	{
		return "NiagaraEditor.Stack.AddItemButton.Particle";
	}
	else if (ExecutionCategoryName == UNiagaraStackEntry::FExecutionCategoryNames::Render)
	{
		return "NiagaraEditor.Stack.AddItemButton.Render";
	}
	else
	{
		return NAME_None;
	}
}

FText FNiagaraStackEditorWidgetsUtilities::GetIconTextForInputMode(UNiagaraStackFunctionInput::EValueMode InputValueMode)
{
	switch (InputValueMode)
	{
	case UNiagaraStackFunctionInput::EValueMode::Linked:
	case UNiagaraStackFunctionInput::EValueMode::LinkedShared:
		return FEditorFontGlyphs::Link;
	case UNiagaraStackFunctionInput::EValueMode::Data:
		return FEditorFontGlyphs::Database;
	case UNiagaraStackFunctionInput::EValueMode::ObjectAsset:
		return FEditorFontGlyphs::Database;
	case UNiagaraStackFunctionInput::EValueMode::Dynamic:
		return FEditorFontGlyphs::Line_Chart;
	case UNiagaraStackFunctionInput::EValueMode::Expression:
		return FEditorFontGlyphs::Terminal;
	case UNiagaraStackFunctionInput::EValueMode::DefaultFunction:
		return FEditorFontGlyphs::Plug;
	case UNiagaraStackFunctionInput::EValueMode::InvalidOverride:
		return FEditorFontGlyphs::Question;
	case UNiagaraStackFunctionInput::EValueMode::UnsupportedDefault:
		return FEditorFontGlyphs::Star;
	default:
		return FEditorFontGlyphs::Question;
	}
}

FText FNiagaraStackEditorWidgetsUtilities::GetIconToolTipForInputMode(UNiagaraStackFunctionInput::EValueMode InputValueMode)
{
	static const FText InvalidText = LOCTEXT("InvalidInputIconToolTip", "Unsupported value.  Check the graph for issues.");
	switch (InputValueMode)
	{
	case UNiagaraStackFunctionInput::EValueMode::Linked:
		return LOCTEXT("LinkInputIconToolTip", "Linked Value");
	case UNiagaraStackFunctionInput::EValueMode::LinkedShared:
		return LOCTEXT("LinkSharedInputIconToolTip", "Linked Shared Value");
	case UNiagaraStackFunctionInput::EValueMode::Data:
		return LOCTEXT("DataInterfaceInputIconToolTip", "Data Value");
	case UNiagaraStackFunctionInput::EValueMode::ObjectAsset:
		return LOCTEXT("ObjectReferenceIconToolTip", "Object Asset");
	case UNiagaraStackFunctionInput::EValueMode::Dynamic:
		return LOCTEXT("DynamicInputIconToolTip", "Dynamic Value");
	case UNiagaraStackFunctionInput::EValueMode::Expression:
		return LOCTEXT("ExpressionInputIconToolTip", "Custom Expression");
	case UNiagaraStackFunctionInput::EValueMode::DefaultFunction:
		return LOCTEXT("DefaultFunctionIconToolTip", "Script Defined Default Function");
	case UNiagaraStackFunctionInput::EValueMode::InvalidOverride:
		return LOCTEXT("InvalidOverrideIconToolTip", "Invalid Script State");
	case UNiagaraStackFunctionInput::EValueMode::UnsupportedDefault:
		return LOCTEXT("UnsupportedDefaultIconToolTip", "Script Defined Custom Default");
	default:
		return InvalidText;
	}
}

FName FNiagaraStackEditorWidgetsUtilities::GetIconColorNameForInputMode(UNiagaraStackFunctionInput::EValueMode InputValueMode)
{
	switch (InputValueMode)
	{
	case UNiagaraStackFunctionInput::EValueMode::Linked:
	case UNiagaraStackFunctionInput::EValueMode::LinkedShared:
		return "NiagaraEditor.Stack.InputValueIconColor.Linked";
	case UNiagaraStackFunctionInput::EValueMode::Data:
		return "NiagaraEditor.Stack.InputValueIconColor.Data";
	case UNiagaraStackFunctionInput::EValueMode::ObjectAsset:
		return "NiagaraEditor.Stack.InputValueIconColor.ObjectAsset";
	case UNiagaraStackFunctionInput::EValueMode::Dynamic:
		return "NiagaraEditor.Stack.InputValueIconColor.Dynamic";
	case UNiagaraStackFunctionInput::EValueMode::Expression:
		return "NiagaraEditor.Stack.InputValueIconColor.Expression";
	case UNiagaraStackFunctionInput::EValueMode::InvalidOverride:
	case UNiagaraStackFunctionInput::EValueMode::UnsupportedDefault:
	case UNiagaraStackFunctionInput::EValueMode::DefaultFunction:
	default:
		return "NiagaraEditor.Stack.InputValueIconColor.Default";
	}
}

void OpenSourceAsset(TWeakObjectPtr<UNiagaraStackEntry> StackEntryWeak)
{
	UNiagaraStackEntry* StackEntry = StackEntryWeak.Get();
	if (StackEntry != nullptr)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(StackEntry->GetExternalAsset());
	}
}

void ShowAssetInContentBrowser(TWeakObjectPtr<UNiagaraStackEntry> StackEntryWeak)
{
	UNiagaraStackEntry* StackEntry = StackEntryWeak.Get();
	if (StackEntry != nullptr)
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		TArray<FAssetData> Assets;
		Assets.Add(FAssetData(StackEntry->GetExternalAsset()));
		ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
	}
}

bool FNiagaraStackEditorWidgetsUtilities::AddStackEntryAssetContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackEntry& StackEntry)
{
	if (StackEntry.GetExternalAsset() != nullptr)
	{
		MenuBuilder.BeginSection("AssetActions", LOCTEXT("AssetActions", "Asset Actions"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("OpenAndFocusAsset", "Open and Focus Asset"),
				FText::Format(LOCTEXT("OpenAndFocusAssetTooltip", "Open {0} in separate editor"), StackEntry.GetDisplayName()),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
				FUIAction(FExecuteAction::CreateStatic(&OpenSourceAsset, TWeakObjectPtr<UNiagaraStackEntry>(&StackEntry))));
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ShowAssetInContentBrowser", "Show in Content Browser"),
				FText::Format(LOCTEXT("ShowAssetInContentBrowserToolTip", "Navigate to {0} in the Content Browser window"), StackEntry.GetDisplayName()),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser.Small"),
				FUIAction(FExecuteAction::CreateStatic(&ShowAssetInContentBrowser, TWeakObjectPtr<UNiagaraStackEntry>(&StackEntry))));
		}
		MenuBuilder.EndSection();
		return true;
	}
	return false;
}

void DeleteItem(TWeakObjectPtr<UNiagaraStackItem> StackItemWeak)
{
	UNiagaraStackItem* StackItem = StackItemWeak.Get();
	FText Unused;
	if (StackItem != nullptr && StackItem->TestCanDeleteWithMessage(Unused))
	{
		StackItem->Delete();
	}
}

void ToggleEnabledState(TWeakObjectPtr<UNiagaraStackItem> StackItemWeak)
{
	TSet<UNiagaraStackItem*> ItemsToToggle;

	UNiagaraStackItem* StackItem = StackItemWeak.Get();
	if (StackItem != nullptr)
	{
		ItemsToToggle.Add(StackItem);
		bool bShouldBeEnabled = !StackItem->GetIsEnabled();
		
		TArray<UNiagaraStackEntry*> StackEntries;
		StackItem->GetSystemViewModel()->GetSelectionViewModel()->GetSelectedEntries(StackEntries);

		for(UNiagaraStackEntry* StackEntry : StackEntries)
		{
			UNiagaraStackItem* AdditionalStackItem = Cast<UNiagaraStackItem>(StackEntry);

			if (AdditionalStackItem != nullptr)
			{
				ItemsToToggle.Add(AdditionalStackItem);
			}
		}

		// we assume the same state for all of the selected entries rather than toggling item-by-item
		for(UNiagaraStackItem* CurrentStackItem : ItemsToToggle)
		{
			CurrentStackItem->SetIsEnabled(bShouldBeEnabled);		
		}
	}

}

void ToggleShouldDebugDraw(TWeakObjectPtr<UNiagaraStackItem> StackItemWeak)
{
	TSet<UNiagaraStackModuleItem*> ItemsToToggle;

	UNiagaraStackModuleItem* ModuleItem = Cast<UNiagaraStackModuleItem>(StackItemWeak.Get());
	if (ModuleItem != nullptr)
	{
		ItemsToToggle.Add(ModuleItem);
		bool bShouldBeEnabled = !ModuleItem->IsDebugDrawEnabled();

		TArray<UNiagaraStackEntry*> StackEntries;
		ModuleItem->GetSystemViewModel()->GetSelectionViewModel()->GetSelectedEntries(StackEntries);

		for(UNiagaraStackEntry* StackEntry : StackEntries)
		{
			UNiagaraStackModuleItem* AdditionalStackModuleItem = Cast<UNiagaraStackModuleItem>(StackEntry);

			if (AdditionalStackModuleItem != nullptr)
			{
				ItemsToToggle.Add(AdditionalStackModuleItem);
			}
		}

		// we assume the same state for all of the selected entries rather than toggling item-by-item
		for(UNiagaraStackModuleItem* CurrentStackItem : ItemsToToggle)
		{
			CurrentStackItem->SetDebugDrawEnabled(bShouldBeEnabled);
		}
	}
}

bool FNiagaraStackEditorWidgetsUtilities::AddStackItemContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackItem& StackItem)
{
	if (StackItem.SupportsChangeEnabled())
	{
		MenuBuilder.BeginSection("ItemActions", LOCTEXT("ItemActions", "Item Actions"));
		{
			if (StackItem.SupportsChangeEnabled())
			{
				FUIAction Action(FExecuteAction::CreateStatic(&ToggleEnabledState, TWeakObjectPtr<UNiagaraStackItem>(&StackItem)),
					FCanExecuteAction(),
					FIsActionChecked::CreateUObject(&StackItem, &UNiagaraStackItem::GetIsEnabled));
				MenuBuilder.AddMenuEntry(
					LOCTEXT("IsEnabled", "Is Enabled"),
					LOCTEXT("ToggleEnabledToolTip", "Toggle enabled/disabled state"),
					FSlateIcon(),
					Action,
					NAME_None,
					EUserInterfaceActionType::Check);
			}

			if (UNiagaraStackModuleItem* ModuleItem = Cast<UNiagaraStackModuleItem>(&StackItem))
			{
				if (ModuleItem->GetIsEnabled() == false)
				{
					MenuBuilder.AddMenuEntry(FNiagaraEditorCommands::Get().HideDisabledModules);
				}
				
				if (ModuleItem->GetModuleNode().ContainsDebugSwitch())
				{
					FText TooltipText = FNiagaraEditorSharedTexts::DebugDrawUIActionBaseText;

					if(FVersionedNiagaraScriptData* ScriptData = ModuleItem->GetModuleNode().GetScriptData())
					{
						if(ScriptData->DebugDrawMessage.IsEmpty() == false)
						{
							TooltipText = ScriptData->DebugDrawMessage;
						}
					}
					
					FUIAction Action(FExecuteAction::CreateStatic(&ToggleShouldDebugDraw, TWeakObjectPtr<UNiagaraStackItem>(&StackItem)),
						FCanExecuteAction(),
						FIsActionChecked::CreateUObject(ModuleItem, &UNiagaraStackModuleItem::IsDebugDrawEnabled));
					MenuBuilder.AddMenuEntry(
						LOCTEXT("ShouldDebugDraw", "Enable Debug Draw"),
						TooltipText,
						FSlateIcon(),
						Action,
						NAME_None,
						EUserInterfaceActionType::Check);
				}
			}

			if (UNiagaraStackRendererItem* RendererItem = Cast<UNiagaraStackRendererItem>(&StackItem))
			{
				FUIAction UpAction(FExecuteAction::CreateUObject(RendererItem, &UNiagaraStackRendererItem::MoveRendererUp),
					FCanExecuteAction::CreateUObject(RendererItem, &UNiagaraStackRendererItem::CanMoveRendererUp));
				MenuBuilder.AddMenuEntry(
					LOCTEXT("MoveRendererUp", "Move up"),
					LOCTEXT("MoveRendererUpToolTip", "Moves the renderer up, so it renders before other renderers with the same render order hint"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ArrowUp"),
					UpAction);

				FUIAction DownAction(FExecuteAction::CreateUObject(RendererItem, &UNiagaraStackRendererItem::MoveRendererDown),
					FCanExecuteAction::CreateUObject(RendererItem, &UNiagaraStackRendererItem::CanMoveRendererDown));
				MenuBuilder.AddMenuEntry(
					LOCTEXT("MoveRendererDown", "Move down"),
					LOCTEXT("MoveRendererDownToolTip", "Moves the renderer down, so it renders after other renderers with the same render order hint"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ArrowDown"),
					DownAction);
			}
		}
		MenuBuilder.EndSection();
		return true;
	}
	return false;
}

void ShowInsertModuleMenu(TWeakObjectPtr<UNiagaraStackModuleItem> StackModuleItemWeak, int32 InsertOffset, TWeakPtr<SWidget> TargetWidgetWeak)
{
	UNiagaraStackModuleItem* StackModuleItem = StackModuleItemWeak.Get();
	TSharedPtr<SWidget> TargetWidget = TargetWidgetWeak.Pin();
	if (StackModuleItem != nullptr && TargetWidget.IsValid())
	{
		TSharedRef<SNiagaraStackItemGroupAddMenu> MenuContent = SNew(SNiagaraStackItemGroupAddMenu, nullptr, StackModuleItem->GetGroupAddUtilities(), StackModuleItem->GetModuleIndex() + InsertOffset);
		FGeometry ThisGeometry = TargetWidget->GetCachedGeometry();
		bool bAutoAdjustForDpiScale = false; // Don't adjust for dpi scale because the push menu command is expecting an unscaled position.
		FVector2D MenuPosition = FSlateApplication::Get().CalculatePopupWindowPosition(ThisGeometry.GetLayoutBoundingRect(), MenuContent->GetDesiredSize(), bAutoAdjustForDpiScale);
		FSlateApplication::Get().PushMenu(TargetWidget.ToSharedRef(), FWidgetPath(), MenuContent, MenuPosition, FPopupTransitionEffect::ContextMenu);
		FSlateApplication::Get().SetKeyboardFocus(MenuContent->GetFilterTextBox());
	}
}

bool FNiagaraStackEditorWidgetsUtilities::AddStackModuleItemContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackModuleItem& StackModuleItem, TSharedRef<SWidget> TargetWidget)
{
	MenuBuilder.BeginSection("ModuleActions", LOCTEXT("ModuleActions", "Module Actions"));
	{
		// Insert Above/Below require valid group add utilities. These are not available when the module
		// item is displayed in the Emitter Summary context (utilities are null), so skip those entries.
		if (StackModuleItem.GetGroupAddUtilities() != nullptr)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("InsertModuleAbove", "Insert Above"),
				LOCTEXT("InsertModuleAboveToolTip", "Insert a new module above this module in the stack."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateStatic(&ShowInsertModuleMenu, TWeakObjectPtr<UNiagaraStackModuleItem>(&StackModuleItem), 0, TWeakPtr<SWidget>(TargetWidget))));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("InsertModuleBelow", "Insert Below"),
				LOCTEXT("InsertModuleBelowToolTip", "Insert a new module below this module in the stack."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateStatic(&ShowInsertModuleMenu, TWeakObjectPtr<UNiagaraStackModuleItem>(&StackModuleItem), 1, TWeakPtr<SWidget>(TargetWidget))));
		}
	}
	MenuBuilder.EndSection();
	return true;
}

void ShowInsertSimulationStageMenu(TWeakObjectPtr<UNiagaraStackSimulationStageGroup> StackSimulationStageGroupWeak, int32 InsertOffset, TWeakPtr<SWidget> TargetWidgetWeak)
{
	UNiagaraStackSimulationStageGroup* StackSimulationStageGroup = StackSimulationStageGroupWeak.Get();
	TSharedPtr<SWidget> TargetWidget = TargetWidgetWeak.Pin();
	if (StackSimulationStageGroup != nullptr && TargetWidget.IsValid())
	{
		TSharedRef<SNiagaraStackItemGroupAddMenu> MenuContent = SNew(SNiagaraStackItemGroupAddMenu, nullptr, StackSimulationStageGroup->GetEmitterStageAddUtilities(), StackSimulationStageGroup->GetStageIndex() + InsertOffset);
		FGeometry ThisGeometry = TargetWidget->GetCachedGeometry();
		bool bAutoAdjustForDpiScale = false; // Don't adjust for dpi scale because the push menu command is expecting an unscaled position.
		FVector2D MenuPosition = FSlateApplication::Get().CalculatePopupWindowPosition(ThisGeometry.GetLayoutBoundingRect(), MenuContent->GetDesiredSize(), bAutoAdjustForDpiScale);
		FSlateApplication::Get().PushMenu(TargetWidget.ToSharedRef(), FWidgetPath(), MenuContent, MenuPosition, FPopupTransitionEffect::ContextMenu);
		FSlateApplication::Get().SetKeyboardFocus(MenuContent->GetFilterTextBox());
	}
}

void FNiagaraStackEditorWidgetsUtilities::AddStackSimulationStageGroupContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackSimulationStageGroup* StackSimulationStageGroup, TSharedRef<SWidget> TargetWidget)
{
	MenuBuilder.BeginSection("SimulationStageActions", LOCTEXT("SimulationStageActions", "Simulation Stage Actions"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("InsertSimulationStageAbove", "Insert Above"),
			LOCTEXT("InsertSimulationStageAboveToolTip", "Insert a new simulation stage above this stage in the stack."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateStatic(&ShowInsertSimulationStageMenu, TWeakObjectPtr<UNiagaraStackSimulationStageGroup>(StackSimulationStageGroup), 0, TWeakPtr<SWidget>(TargetWidget))));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("InsertSimulationStageBelow", "Insert Below"),
			LOCTEXT("InsertSimulationStageBelowToolTip", "Insert a new simulation stage below this stage in the stack."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateStatic(&ShowInsertSimulationStageMenu, TWeakObjectPtr<UNiagaraStackSimulationStageGroup>(StackSimulationStageGroup), 1, TWeakPtr<SWidget>(TargetWidget))));
	}
	MenuBuilder.EndSection();
}

bool FNiagaraStackEditorWidgetsUtilities::AddStackNoteContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackEntry& StackEntry)
{
	if(StackEntry.SupportsStackNotes())
	{
		MenuBuilder.BeginSection("NoteActions", LOCTEXT("NoteActions", "Note Actions"));
		
		if(StackEntry.HasStackNoteData() == false)
		{
			MenuBuilder.AddMenuEntry(
			LOCTEXT("AddNote", "Add Note"),
			LOCTEXT("AddNoteTooltip", "Add a note to this row"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Comment"),
			FUIAction(FExecuteAction::CreateUObject(&StackEntry, &UNiagaraStackEntry::AddStackNote))
			);
		}
		else
		{				
			MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleInlineNote", "Toggle Inline Note Display"),
			LOCTEXT("ToggleInlineNoteTooltip", "Toggle the inline display for this row's note.\nAn inlined note will show up in the row itself, saving on space"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Comment"),
			FUIAction(FExecuteAction::CreateUObject(&StackEntry, &UNiagaraStackEntry::ToggleStackNoteInlineDisplay))
			);
		
			MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyNote", "Copy Note"),
			LOCTEXT("CopyNoteTooltip", "Copy the note of this row"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
			FUIAction(FExecuteAction::CreateUObject(&StackEntry, &UNiagaraStackEntry::CopyStackNote))
			);
		}
			
		if(const UNiagaraClipboardContent* ClipboardContent = FNiagaraEditorModule::Get().GetClipboard().GetClipboardContent())
		{
			if(ClipboardContent->StackNote.IsValid())
			{
				MenuBuilder.AddMenuEntry(
				LOCTEXT("PasteNote", "Paste Note"),
				LOCTEXT("TPasteNoteTooltip", "Paste the note copied in the clipboard"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"),
				FUIAction(FExecuteAction::CreateUObject(&StackEntry, &UNiagaraStackEntry::PasteStackNote))
				);
			}
		}

		if(StackEntry.HasStackNoteData())
		{
			MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteNote", "Delete Note"),
			LOCTEXT("DeleteNoteTooltip", "Delete this row's note"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
			FUIAction(FExecuteAction::CreateUObject(&StackEntry, &UNiagaraStackEntry::DeleteStackNoteData))
			);
		}

		MenuBuilder.EndSection();
		return true;
	}

	return false;
}

TSharedRef<FDragDropOperation> FNiagaraStackEditorWidgetsUtilities::ConstructDragDropOperationForStackEntries(const TArray<UNiagaraStackEntry*>& DraggedEntries)
{
	TSharedRef<FNiagaraStackEntryDragDropOp> DragDropOp = MakeShared<FNiagaraStackEntryDragDropOp>(DraggedEntries);
	DragDropOp->CurrentHoverText = DraggedEntries.Num() == 1 
		? DraggedEntries[0]->GetDisplayName()
		: FText::Format(LOCTEXT("MultipleEntryDragFormat", "{0} (and {1} others)"), DraggedEntries[0]->GetDisplayName(), FText::AsNumber(DraggedEntries.Num() - 1));
	DragDropOp->CurrentIconBrush = FNiagaraEditorWidgetsStyle::Get().GetBrush(
		GetIconNameForExecutionSubcategory(DraggedEntries[0]->GetExecutionSubcategoryName(), true));
	DragDropOp->CurrentIconColorAndOpacity = FNiagaraEditorWidgetsStyle::Get().GetColor(
		GetIconColorNameForExecutionCategory(DraggedEntries[0]->GetExecutionCategoryName()));
	DragDropOp->SetupDefaults();
	DragDropOp->Construct();
	return DragDropOp;
}

void FNiagaraStackEditorWidgetsUtilities::HandleDragLeave(const FDragDropEvent& InDragDropEvent)
{
	if (InDragDropEvent.GetOperation().IsValid())
	{
		TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = InDragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
		if (DecoratedDragDropOp.IsValid())
		{
			DecoratedDragDropOp->ResetToDefaultToolTip();
		}
	}
}

TOptional<EItemDropZone> FNiagaraStackEditorWidgetsUtilities::RequestDropForStackEntry(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry, UNiagaraStackEntry::EDropOptions DropOptions)
{
	TOptional<EItemDropZone> DropZone;
	if (InDragDropEvent.GetOperation().IsValid())
	{
		TSharedPtr<FDecoratedDragDropOp> DecoratedDragDropOp = InDragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
		if (DecoratedDragDropOp.IsValid())
		{
			DecoratedDragDropOp->ResetToDefaultToolTip();
		}

		UNiagaraStackEntry::EDragOptions DragOptions = UNiagaraStackEntry::EDragOptions::None;
		if (InDragDropEvent.IsAltDown() &&
			InDragDropEvent.IsShiftDown() == false &&
			InDragDropEvent.IsControlDown() == false &&
			InDragDropEvent.IsCommandDown() == false)
		{
			DragOptions = UNiagaraStackEntry::EDragOptions::Copy;
		}

		TOptional<UNiagaraStackEntry::FDropRequestResponse> Response = InTargetEntry->CanDrop(UNiagaraStackEntry::FDropRequest(InDragDropEvent.GetOperation().ToSharedRef(), InDropZone, DragOptions, DropOptions));
		if (Response.IsSet())
		{
			if (DecoratedDragDropOp.IsValid() && Response.GetValue().DropMessage.IsEmptyOrWhitespace() == false)
			{
				DecoratedDragDropOp->CurrentHoverText = Response.GetValue().DropMessage;
			}

			if (Response.GetValue().DropZone.IsSet())
			{
				DropZone = Response.GetValue().DropZone.GetValue();
			}
			else
			{
				if (DecoratedDragDropOp.IsValid())
				{
					DecoratedDragDropOp->CurrentIconBrush = FAppStyle::GetBrush("Icons.Error");
					DecoratedDragDropOp->CurrentIconColorAndOpacity = FLinearColor::White;
				}
			}
		}
	}
	return DropZone;
}

bool FNiagaraStackEditorWidgetsUtilities::HandleDropForStackEntry(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry, UNiagaraStackEntry::EDropOptions DropOptions)
{
	bool bHandled = false;
	if (InDragDropEvent.GetOperation().IsValid())
	{
		UNiagaraStackEntry::EDragOptions DragOptions = UNiagaraStackEntry::EDragOptions::None;
		if (InDragDropEvent.IsAltDown() &&
			InDragDropEvent.IsShiftDown() == false &&
			InDragDropEvent.IsControlDown() == false &&
			InDragDropEvent.IsCommandDown() == false)
		{
			DragOptions = UNiagaraStackEntry::EDragOptions::Copy;
		}

		UNiagaraStackEntry::FDropRequest DropRequest(InDragDropEvent.GetOperation().ToSharedRef(), InDropZone, DragOptions, DropOptions);
		bHandled = ensureMsgf(InTargetEntry->Drop(DropRequest).IsSet(),
			TEXT("Failed to drop stack entry when it was requested"));
	}
	return bHandled;
}

FString FNiagaraStackEditorWidgetsUtilities::StackEntryToStringForListDebug(UNiagaraStackEntry* StackEntry)
{
	return FString::Printf(TEXT("0x%08" UPTRINT_x_FMT " - %s - %s"), (UPTRINT)StackEntry, *StackEntry->GetClass()->GetName(), *StackEntry->GetDisplayName().ToString());
}

#undef LOCTEXT_NAMESPACE
