// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "ViewModels/Stack/NiagaraStackFunctionInput.h"
#include "NiagaraTypes.h"

class FMenuBuilder;
class SWidget;
class UNiagaraStackEntry;
class UNiagaraStackItem;
class UNiagaraStackModuleItem;
class UNiagaraStackSimulationStageGroup;

namespace FNiagaraStackEditorWidgetsUtilities
{
	FName GetColorNameForExecutionCategory(FName ExecutionCategoryName);

	FName GetIconNameForExecutionSubcategory(FName ExecutionSubcategoryName, bool bIsHighlighted);

	FName GetIconColorNameForExecutionCategory(FName ExecutionCategoryName);

	FName GetAddItemButtonStyleNameForExecutionCategory(FName ExecutionCategoryName);

	FText GetIconTextForInputMode(UNiagaraStackFunctionInput::EValueMode InputValueMode);

	FText GetIconToolTipForInputMode(UNiagaraStackFunctionInput::EValueMode InputValueMode);

	FName GetIconColorNameForInputMode(UNiagaraStackFunctionInput::EValueMode InputValueMode);

	bool AddStackEntryAssetContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackEntry& StackEntry);

	bool AddStackItemContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackItem& StackItem);

	bool AddStackModuleItemContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackModuleItem& StackItem, TSharedRef<SWidget> TargetWidget);

	void AddStackSimulationStageGroupContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackSimulationStageGroup* SimulationStageGroup, TSharedRef<SWidget> TagetWidget);

	bool AddStackNoteContextMenuActions(FMenuBuilder& MenuBuilder, UNiagaraStackEntry& StackEntry);
	
	TSharedRef<FDragDropOperation> ConstructDragDropOperationForStackEntries(const TArray<UNiagaraStackEntry*>& DraggedEntries);

	void HandleDragLeave(const FDragDropEvent& InDragDropEvent);

	TOptional<EItemDropZone> RequestDropForStackEntry(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry, UNiagaraStackEntry::EDropOptions DropOptions);

	bool HandleDropForStackEntry(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, UNiagaraStackEntry* InTargetEntry, UNiagaraStackEntry::EDropOptions DropOptions);

	FString StackEntryToStringForListDebug(UNiagaraStackEntry* StackEntry);
}
