// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraOverviewGraphNodeFactory.h"
#include "NiagaraOverviewNode.h"
#include "SNiagaraOverviewStackNode.h"

TSharedPtr<SGraphNode> FNiagaraOverviewGraphNodeFactory::CreateNodeWidget(UEdGraphNode* InNode)
{
	UNiagaraOverviewNode* OverviewNode = Cast<UNiagaraOverviewNode>(InNode);
	if (OverviewNode != nullptr)
	{
		return SNew(SNiagaraOverviewStackNode, OverviewNode);
	}
	else
	{
		return FGraphNodeFactory::CreateNodeWidget(InNode);
	}
}