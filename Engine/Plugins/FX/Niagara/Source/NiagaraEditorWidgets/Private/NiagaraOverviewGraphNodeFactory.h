// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeFactory.h"

class SGraphNode;
class UEdGraphNode;

class FNiagaraOverviewGraphNodeFactory : public FGraphNodeFactory
{
public:
	virtual TSharedPtr<SGraphNode> CreateNodeWidget(UEdGraphNode* InNode) override;
};