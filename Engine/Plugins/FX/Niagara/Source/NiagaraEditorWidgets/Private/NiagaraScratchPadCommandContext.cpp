// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraScratchPadCommandContext.h"
#include "ViewModels/NiagaraScratchPadViewModel.h"
#include "ViewModels/NiagaraScratchPadScriptViewModel.h"
#include "NiagaraEditorModule.h"
#include "NiagaraEditorCommands.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Logging/LogMacros.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "NiagaraScratchPadCommandContext"

FNiagaraScratchPadCommandContext::FNiagaraScratchPadCommandContext(UNiagaraScratchPadViewModel* InScratchPadViewModel)
	: Commands(MakeShared<FUICommandList>())
	, bCommandsAreSetup(false)
	, bProcessingCommandBindings(false)
	, ScratchPadViewModel(InScratchPadViewModel)
{
}

TSharedRef<FUICommandList> FNiagaraScratchPadCommandContext::GetCommands()
{
	if (bCommandsAreSetup == false)
	{
		SetupCommands();
		bCommandsAreSetup = true;
	}
	return Commands;
}

bool FNiagaraScratchPadCommandContext::ProcessCommandBindings(const FKeyEvent& InKeyEvent)
{
	TGuardValue<bool> ProcessingGuard(bProcessingCommandBindings, true);
	return GetCommands()->ProcessCommandBindings(InKeyEvent);
}

void FNiagaraScratchPadCommandContext::AddMenuItems(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("ScriptEdit", LOCTEXT("ScriptEditActions", "Edit"));
	{
		TAttribute<FText> CanRenameToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraScratchPadCommandContext::GetCanRenameSelectedScriptToolTip));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, TAttribute<FText>(), CanRenameToolTip);
		
		TAttribute<FText> CanCutToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraScratchPadCommandContext::GetCanCutSelectedScriptsToolTip));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut, NAME_None, TAttribute<FText>(), CanCutToolTip);

		TAttribute<FText> CanCopyToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraScratchPadCommandContext::GetCanCopySelectedScriptsToolTip));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy, NAME_None, TAttribute<FText>(), CanCopyToolTip);

		TAttribute<FText> CanPasteToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraScratchPadCommandContext::GetCanPasteSelectedScriptsToolTip));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste, NAME_None, TAttribute<FText>(), CanPasteToolTip);

		TAttribute<FText> CanDeleteToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraScratchPadCommandContext::GetCanDeleteSelectedScriptsToolTip));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, TAttribute<FText>(), CanDeleteToolTip);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Script", LOCTEXT("ScriptActions", "Script"));
	{
		MenuBuilder.AddMenuEntry(FNiagaraEditorModule::Get().Commands().Apply, NAME_None, LOCTEXT("ApplyChangesLabel", "Apply Changes"));
		MenuBuilder.AddMenuEntry(FNiagaraEditorModule::Get().Commands().Discard, NAME_None, LOCTEXT("DiscardChangesLabel", "Discard Changes"));
		MenuBuilder.AddMenuEntry(FNiagaraEditorModule::Get().Commands().SelectNextUsage, NAME_None, TAttribute<FText>(), LOCTEXT("SelectNextToolTip", "Select the next usage of this script in the selection stack."));
		MenuBuilder.AddMenuEntry(FNiagaraEditorModule::Get().Commands().CreateAssetFromSelection);
	}
	MenuBuilder.EndSection();
}

void FNiagaraScratchPadCommandContext::SetupCommands()
{
	Commands->MapAction(FGenericCommands::Get().Rename, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::RenameSelectedScript),
		FCanExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CanRenameSelectedScript)));
	Commands->MapAction(FGenericCommands::Get().Cut, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CutSelectedScripts),
		FCanExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CanCutSelectedScripts)));
	Commands->MapAction(FGenericCommands::Get().Copy, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CopySelectedScripts),
		FCanExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CanCopySelectedScripts)));
	Commands->MapAction(FGenericCommands::Get().Paste, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::PasteSelectedScripts),
		FCanExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CanPasteSelectedScripts)));
	Commands->MapAction(FGenericCommands::Get().Delete, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::DeleteSelectedScripts),
		FCanExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CanDeleteSelectedScripts)));
	Commands->MapAction(FNiagaraEditorModule::Get().Commands().Apply, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::ApplyChangesToSelectedScripts),
		FCanExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CanApplyChangesToSelectedScripts)));
	Commands->MapAction(FNiagaraEditorModule::Get().Commands().Discard, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::DiscardChangesFromSelectedScripts),
		FCanExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CanDiscardChangesFromSelectedScripts)));
	Commands->MapAction(FNiagaraEditorModule::Get().Commands().SelectNextUsage, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::SelectNextUsageForSelectedScript),
		FCanExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CanSelectNextUsageForSelectedScript)));
	Commands->MapAction(FNiagaraEditorModule::Get().Commands().CreateAssetFromSelection, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CreateAssetFromSelectedScript),
		FCanExecuteAction::CreateSP(this, &FNiagaraScratchPadCommandContext::CanCreateAssetFromSelectedScript)));
}

void FNiagaraScratchPadCommandContext::RenameSelectedScript() const
{
	ScratchPadViewModel->GetActiveScriptViewModel()->SetIsPendingRename(true);
}

bool FNiagaraScratchPadCommandContext::CanRenameSelectedScript() const
{
	return ScratchPadViewModel->GetActiveScriptViewModel().IsValid();
}

FText FNiagaraScratchPadCommandContext::GetCanRenameSelectedScriptToolTip() const
{
	return LOCTEXT("RenameToolTip", "Rename the selected scratch pad script");
}

bool FNiagaraScratchPadCommandContext::CanCutSelectedScripts() const
{
	return ScratchPadViewModel->GetActiveScriptViewModel().IsValid();
}

FText FNiagaraScratchPadCommandContext::GetCanCutSelectedScriptsToolTip() const
{
	return LOCTEXT("CutToolTip", "Cut the selected scratch pad script.");
}

void FNiagaraScratchPadCommandContext::CutSelectedScripts() const
{
	if (CanCutSelectedScripts())
	{
		FScopedTransaction Transaction(LOCTEXT("CutScratchPadScriptTransaction", "Cut the selected scratch pad script to the system clipboard."));
		ScratchPadViewModel->CopyActiveScript();
		ScratchPadViewModel->DeleteActiveScript();
	}
}

bool FNiagaraScratchPadCommandContext::CanCopySelectedScripts() const
{
	return ScratchPadViewModel->GetActiveScriptViewModel().IsValid();
}

FText FNiagaraScratchPadCommandContext::GetCanCopySelectedScriptsToolTip() const
{
	return LOCTEXT("CopyToolTip", "Copy the selected scratch pad script to the system clipboard.");
}

void FNiagaraScratchPadCommandContext::CopySelectedScripts() const
{
	ScratchPadViewModel->CopyActiveScript();
}

bool FNiagaraScratchPadCommandContext::CanPasteSelectedScripts() const
{
	return ScratchPadViewModel->CanPasteScript();
}

FText FNiagaraScratchPadCommandContext::GetCanPasteSelectedScriptsToolTip() const
{
	return LOCTEXT("PasteToolTip", "Paste a scratch pad script from the system clipboard.");;
}

void FNiagaraScratchPadCommandContext::PasteSelectedScripts() const
{
	ScratchPadViewModel->PasteScript();
}

bool FNiagaraScratchPadCommandContext::CanDeleteSelectedScripts() const
{
	return ScratchPadViewModel->GetActiveScriptViewModel().IsValid();
}

FText FNiagaraScratchPadCommandContext::GetCanDeleteSelectedScriptsToolTip() const
{
	return LOCTEXT("DeleteScriptMessage", "Delete this script and reset references to it.");
}

void FNiagaraScratchPadCommandContext::DeleteSelectedScripts() const
{
	ScratchPadViewModel->DeleteActiveScript();
}

bool FNiagaraScratchPadCommandContext::CanApplyChangesToSelectedScripts() const
{
	return ScratchPadViewModel->GetActiveScriptViewModel().IsValid() && ScratchPadViewModel->GetActiveScriptViewModel()->HasUnappliedChanges();
}

void FNiagaraScratchPadCommandContext::ApplyChangesToSelectedScripts() const
{
	if (CanApplyChangesToSelectedScripts())
	{
		ScratchPadViewModel->GetActiveScriptViewModel()->ApplyChanges();
	}
}

bool FNiagaraScratchPadCommandContext::CanDiscardChangesFromSelectedScripts() const
{
	return ScratchPadViewModel->GetActiveScriptViewModel().IsValid() && ScratchPadViewModel->GetActiveScriptViewModel()->HasUnappliedChanges();
}

void FNiagaraScratchPadCommandContext::DiscardChangesFromSelectedScripts() const
{
	if (CanDiscardChangesFromSelectedScripts())
	{
		ScratchPadViewModel->GetActiveScriptViewModel()->DiscardChanges();
	}
}

bool FNiagaraScratchPadCommandContext::CanSelectNextUsageForSelectedScript() const
{
	return ScratchPadViewModel->CanSelectNextUsageForActiveScript();
}

void FNiagaraScratchPadCommandContext::SelectNextUsageForSelectedScript() const
{
	ScratchPadViewModel->SelectNextUsageForActiveScript();
}

bool FNiagaraScratchPadCommandContext::CanCreateAssetFromSelectedScript() const
{
	return ScratchPadViewModel->GetActiveScriptViewModel().IsValid();
}

void FNiagaraScratchPadCommandContext::CreateAssetFromSelectedScript() const
{
	ScratchPadViewModel->CreateAssetFromActiveScript();
}

#undef LOCTEXT_NAMESPACE

