// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UNiagaraScratchPadViewModel;
class FUICommandList;
class FMenuBuilder;
struct FKeyEvent;

class FNiagaraScratchPadCommandContext : public TSharedFromThis<FNiagaraScratchPadCommandContext>
{
public:
	FNiagaraScratchPadCommandContext(UNiagaraScratchPadViewModel* InScratchPadViewModel);

	TSharedRef<FUICommandList> GetCommands();

	bool ProcessCommandBindings(const FKeyEvent& InKeyEvent);

	void AddMenuItems(FMenuBuilder& MenuBuilder);

private:
	void SetupCommands();

	void RenameSelectedScript() const;

	bool CanRenameSelectedScript() const;

	FText GetCanRenameSelectedScriptToolTip() const;

	bool CanCutSelectedScripts() const;

	FText GetCanCutSelectedScriptsToolTip() const;

	void CutSelectedScripts() const;

	bool CanCopySelectedScripts() const;

	FText GetCanCopySelectedScriptsToolTip() const;

	void CopySelectedScripts() const;

	bool CanPasteSelectedScripts() const;

	FText GetCanPasteSelectedScriptsToolTip() const;

	void PasteSelectedScripts() const;

	bool CanDeleteSelectedScripts() const;

	FText GetCanDeleteSelectedScriptsToolTip() const;

	void DeleteSelectedScripts() const;

	bool CanApplyChangesToSelectedScripts() const;

	void ApplyChangesToSelectedScripts() const;

	bool CanDiscardChangesFromSelectedScripts() const;

	void DiscardChangesFromSelectedScripts() const;

	bool CanSelectNextUsageForSelectedScript() const;

	void SelectNextUsageForSelectedScript() const;

	bool CanCreateAssetFromSelectedScript() const;

	void CreateAssetFromSelectedScript() const;

private:
	TSharedRef<FUICommandList> Commands;

	bool bCommandsAreSetup;

	bool bProcessingCommandBindings;

	UNiagaraScratchPadViewModel* ScratchPadViewModel;
};