// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraStackCommandContext.h"

#include "NiagaraEditorCommands.h"
#include "NiagaraEditorCommon.h"
#include "NiagaraEditorUtilities.h"
#include "NiagaraSystemEditorData.h"
#include "NiagaraStackEditorData.h"
#include "ViewModels/Stack/NiagaraStackClipboardUtilities.h"
#include "ViewModels/Stack/NiagaraStackEntry.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Logging/LogMacros.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ViewModels/NiagaraSystemViewModel.h"

#define LOCTEXT_NAMESPACE "NiagaraStackCommandContext"

FNiagaraStackCommandContext::FNiagaraStackCommandContext()
	: Commands(MakeShared<FUICommandList>())
	, bCommandsAreSetup(false)
	, bProcessingCommandBindings(false)
{
}

void FNiagaraStackCommandContext::SetSelectedEntries(const TArray<UNiagaraStackEntry*> InSelectedEntries)
{
	SelectedEntries = InSelectedEntries;
}

TSharedRef<FUICommandList> FNiagaraStackCommandContext::GetCommands()
{
	if (bCommandsAreSetup == false)
	{
		SetupCommands();
		bCommandsAreSetup = true;
	}
	return Commands;
}

bool FNiagaraStackCommandContext::ProcessCommandBindings(const FKeyEvent& InKeyEvent)
{
	TGuardValue<bool> ProcessingGuard(bProcessingCommandBindings, true);
	return GetCommands()->ProcessCommandBindings(InKeyEvent);
}

bool FNiagaraStackCommandContext::AddEditMenuItems(FMenuBuilder& MenuBuilder)
{
	bool bSupportsCut = SelectedEntries.ContainsByPredicate([](const UNiagaraStackEntry* SelectedEntry) { return SelectedEntry->SupportsCut(); });
	bool bSupportsCopy = SelectedEntries.ContainsByPredicate([](const UNiagaraStackEntry* SelectedEntry) { return SelectedEntry->SupportsCopy(); });
	bool bSupportsPaste = SelectedEntries.ContainsByPredicate([](const UNiagaraStackEntry* SelectedEntry) { return SelectedEntry->SupportsPaste(); });
	bool bSupportsDelete = SelectedEntries.ContainsByPredicate([](const UNiagaraStackEntry* SelectedEntry) { return SelectedEntry->SupportsDelete(); });
	bool bSupportsRename = SelectedEntries.ContainsByPredicate([](const UNiagaraStackEntry* SelectedEntry) { return SelectedEntry->SupportsRename(); });
	if (bSupportsCut || bSupportsCopy || bSupportsPaste || bSupportsDelete || bSupportsRename)
	{
		MenuBuilder.BeginSection("EntryEdit", LOCTEXT("EntryEditActions", "Edit"));
		{
			if (bSupportsCut)
			{
				TAttribute<FText> CanCutToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraStackCommandContext::GetCanCutSelectedEntriesToolTip));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut, NAME_None, TAttribute<FText>(), CanCutToolTip);
			}
			if (bSupportsCopy)
			{
				TAttribute<FText> CanCopyToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraStackCommandContext::GetCanCopySelectedEntriesToolTip));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy, NAME_None, TAttribute<FText>(), CanCopyToolTip);
			}
			if (bSupportsPaste)
			{
				TAttribute<FText> CanPasteToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraStackCommandContext::GetCanPasteSelectedEntriesToolTip));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste, NAME_None, TAttribute<FText>(), CanPasteToolTip);
			}
			if (bSupportsDelete)
			{
				TAttribute<FText> CanDeleteToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraStackCommandContext::GetCanDeleteSelectedEntriesToolTip));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, TAttribute<FText>(), CanDeleteToolTip);
			}
			if (bSupportsRename)
			{
				TAttribute<FText> CanRenameToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FNiagaraStackCommandContext::GetCanRenameSelectedEntriesToolTip));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, TAttribute<FText>(), CanRenameToolTip);
			}
		}
		MenuBuilder.EndSection();
		return true;
	}
	return false;
}

void FNiagaraStackCommandContext::SetupCommands()
{
	Commands->MapAction(FGenericCommands::Get().Cut, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::CutSelectedEntries),
		FCanExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::CanCutSelectedEntries)));
	Commands->MapAction(FGenericCommands::Get().Copy, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::CopySelectedEntries),
		FCanExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::CanCopySelectedEntries)));
	Commands->MapAction(FGenericCommands::Get().Paste, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::PasteSelectedEntries),
		FCanExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::CanPasteSelectedEntries)));
	Commands->MapAction(FGenericCommands::Get().Delete, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::DeleteSelectedEntries),
		FCanExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::CanDeleteSelectedEntries)));
	Commands->MapAction(FGenericCommands::Get().Rename, FUIAction(
		FExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::RenameSelectedEntries),
		FCanExecuteAction::CreateSP(this, &FNiagaraStackCommandContext::CanRenameSelectedEntries)));

	Commands->MapAction(
		FNiagaraEditorCommands::Get().HideDisabledModules,
		FExecuteAction::CreateLambda([this]()
		{
			if (SelectedEntries.Num() > 0)
			{
				UNiagaraStackEditorData& EditorData = SelectedEntries[0]->GetSystemViewModel()->GetEditorData().GetStackEditorData();
				EditorData.bHideDisabledModules = !EditorData.bHideDisabledModules;
			}
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			if (SelectedEntries.Num() > 0)
			{
				UNiagaraStackEditorData& EditorData = SelectedEntries[0]->GetSystemViewModel()->GetEditorData().GetStackEditorData();
				return EditorData.bHideDisabledModules;
			}
			return false;
		}));
}

bool FNiagaraStackCommandContext::CanCutSelectedEntries() const
{
	FText CanCutMessage;
	bool bCanCut = FNiagaraStackClipboardUtilities::TestCanCutSelectionWithMessage(SelectedEntries, CanCutMessage);
	if (bProcessingCommandBindings && bCanCut == false && CanCutMessage.IsEmptyOrWhitespace() == false)
	{
		FNiagaraEditorUtilities::WarnWithToastAndLog(CanCutMessage);
	}
	return bCanCut;
}

FText FNiagaraStackCommandContext::GetCanCutSelectedEntriesToolTip() const
{
	FText CanCutMessage;
	FNiagaraStackClipboardUtilities::TestCanCutSelectionWithMessage(SelectedEntries, CanCutMessage);
	return CanCutMessage;
}

void FNiagaraStackCommandContext::CutSelectedEntries() const
{
	FNiagaraStackClipboardUtilities::CutSelection(SelectedEntries);
}

bool FNiagaraStackCommandContext::CanCopySelectedEntries() const
{
	FText CanCopyMessage;
	bool bCanCopy = FNiagaraStackClipboardUtilities::TestCanCopySelectionWithMessage(SelectedEntries, CanCopyMessage);
	if (bProcessingCommandBindings && bCanCopy == false && CanCopyMessage.IsEmptyOrWhitespace() == false)
	{
		FNiagaraEditorUtilities::WarnWithToastAndLog(CanCopyMessage);
	}
	return bCanCopy;
}

FText FNiagaraStackCommandContext::GetCanCopySelectedEntriesToolTip() const
{
	FText CanCopyMessage;
	FNiagaraStackClipboardUtilities::TestCanCopySelectionWithMessage(SelectedEntries, CanCopyMessage);
	return CanCopyMessage;
}

void FNiagaraStackCommandContext::CopySelectedEntries() const
{
	FNiagaraStackClipboardUtilities::CopySelection(SelectedEntries);
}

bool FNiagaraStackCommandContext::CanPasteSelectedEntries() const
{
	FText CanPasteMessage;
	bool bCanPaste = FNiagaraStackClipboardUtilities::TestCanPasteSelectionWithMessage(SelectedEntries, CanPasteMessage);
	if (bProcessingCommandBindings && bCanPaste == false && CanPasteMessage.IsEmptyOrWhitespace() == false)
	{
		FNiagaraEditorUtilities::WarnWithToastAndLog(CanPasteMessage);
	}
	return bCanPaste;
}

FText FNiagaraStackCommandContext::GetCanPasteSelectedEntriesToolTip() const
{
	FText CanPasteMessage;
	FNiagaraStackClipboardUtilities::TestCanPasteSelectionWithMessage(SelectedEntries, CanPasteMessage);
	if (CanPasteMessage.IsEmptyOrWhitespace())
	{
		CanPasteMessage = LOCTEXT("NoValidDataToPaste", "Incompatible or no data to paste.");
	}
	return CanPasteMessage;
}

void FNiagaraStackCommandContext::PasteSelectedEntries() const
{
	FText PasteWarning;
	FNiagaraStackClipboardUtilities::PasteSelection(SelectedEntries, PasteWarning);
	if (PasteWarning.IsEmptyOrWhitespace() == false)
	{
		FNiagaraEditorUtilities::WarnWithToastAndLog(PasteWarning);
	}
}

bool FNiagaraStackCommandContext::CanDeleteSelectedEntries() const
{
	FText CanDeleteMessage;
	bool bCanDelete = FNiagaraStackClipboardUtilities::TestCanDeleteSelectionWithMessage(SelectedEntries, CanDeleteMessage);
	if (bProcessingCommandBindings && bCanDelete == false && CanDeleteMessage.IsEmptyOrWhitespace() == false)
	{
		FNiagaraEditorUtilities::WarnWithToastAndLog(CanDeleteMessage);
	}
	return bCanDelete;
}

FText FNiagaraStackCommandContext::GetCanDeleteSelectedEntriesToolTip() const
{
	FText CanDeleteMessage;
	FNiagaraStackClipboardUtilities::TestCanDeleteSelectionWithMessage(SelectedEntries, CanDeleteMessage);
	return CanDeleteMessage;
}

void FNiagaraStackCommandContext::DeleteSelectedEntries() const
{
	FNiagaraStackClipboardUtilities::DeleteSelection(SelectedEntries);
}

FText FNiagaraStackCommandContext::GetCanRenameSelectedEntriesToolTip() const
{
	if (SelectedEntries.Num() > 1)
	{
		return LOCTEXT("CantRenameMultipleToolTip", "Can't rename multiple items.");
	}

	if (!SelectedEntries[0]->SupportsRename())
	{
		return LOCTEXT("CantRenameToolTip", "The selected item cannot be renamed.");
	}

	return LOCTEXT("RenameToolTip", "Rename the selected item.");
}

bool FNiagaraStackCommandContext::CanRenameSelectedEntries() const
{
	if (SelectedEntries.Num() == 1)
	{
		return SelectedEntries[0]->SupportsRename();
	}

	return false;
}

void FNiagaraStackCommandContext::RenameSelectedEntries()
{
	if (SelectedEntries.Num() == 1)
	{
		SelectedEntries[0]->SetIsRenamePending(true);
	}
}

#undef LOCTEXT_NAMESPACE

