// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UNiagaraStackEntry;
class FUICommandList;
class FMenuBuilder;
struct FKeyEvent;

class FNiagaraStackCommandContext : public TSharedFromThis<FNiagaraStackCommandContext>
{
public:
	FNiagaraStackCommandContext();

	void SetSelectedEntries(const TArray<UNiagaraStackEntry*> InSelectedEntries);

	TSharedRef<FUICommandList> GetCommands();

	bool ProcessCommandBindings(const FKeyEvent& InKeyEvent);

	bool AddEditMenuItems(FMenuBuilder& MenuBuilder);

private:
	void SetupCommands();

	bool CanCutSelectedEntries() const;

	FText GetCanCutSelectedEntriesToolTip() const;

	void CutSelectedEntries() const;

	bool CanCopySelectedEntries() const;

	FText GetCanCopySelectedEntriesToolTip() const;

	void CopySelectedEntries() const;

	bool CanPasteSelectedEntries() const;

	FText GetCanPasteSelectedEntriesToolTip() const;

	void PasteSelectedEntries() const;

	bool CanDeleteSelectedEntries() const;

	FText GetCanDeleteSelectedEntriesToolTip() const;

	void DeleteSelectedEntries() const;

	FText GetCanRenameSelectedEntriesToolTip() const;

	bool CanRenameSelectedEntries() const;

	void RenameSelectedEntries();

private:
	TSharedRef<FUICommandList> Commands;

	bool bCommandsAreSetup;

	bool bProcessingCommandBindings;

	TArray<UNiagaraStackEntry*> SelectedEntries;
};