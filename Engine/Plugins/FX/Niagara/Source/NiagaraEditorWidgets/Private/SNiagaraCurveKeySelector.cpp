// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNiagaraCurveKeySelector.h"

#include "CurveEditor.h"
#include "Tree/SCurveEditorTree.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "EditorFontGlyphs.h"

#define LOCTEXT_NAMESPACE "SNiagaraCurveKeySelector"

void SNiagaraCurveKeySelector::Construct(const FArguments& InArgs, TSharedPtr<FCurveEditor> InCurveEditor, const TArray<FCurveEditorTreeItemID>& InCurveTreeItemIds, TSharedPtr<SCurveEditorTree> InCurveEditorTree)
{
	CurveEditor = InCurveEditor;
	OrderedCurveTreeItemIds = InCurveTreeItemIds;
	CurveEditorTree = InCurveEditorTree;
	ChildSlot
	[
		SNew(SHorizontalBox)

		// Zoom to fit button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, 4, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SNiagaraCurveKeySelector::ZoomToFitClicked)
			.ToolTipText(LOCTEXT("ZoomToFitToolTip", "Zoom to fit all keys"))
			.Content()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Text(FEditorFontGlyphs::Expand)
			]
		]

		// Previous curve button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, 1, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SNiagaraCurveKeySelector::PreviousCurveClicked)
			.ToolTipText(LOCTEXT("PreviousCurveToolTip", "Select the previous curve"))
			.Visibility(this, &SNiagaraCurveKeySelector::GetNextPreviousCurveButtonVisibility)
			.Content()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.ArrowUp"))
			]
		]

		// Next curve button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, 4, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SNiagaraCurveKeySelector::NextCurveClicked)
			.ToolTipText(LOCTEXT("NextCurveToolTip", "Select the next curve"))
			.Visibility(this, &SNiagaraCurveKeySelector::GetNextPreviousCurveButtonVisibility)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.ArrowDown"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		// Previous key button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, 1, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SNiagaraCurveKeySelector::PreviousKeyClicked)
			.ToolTipText(LOCTEXT("PreviousKeyToolTip", "Select the previous key for the selected curve."))
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.ArrowLeft"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		// Next key button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, 4, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SNiagaraCurveKeySelector::NextKeyClicked)
			.ToolTipText(LOCTEXT("NextKeyToolTip", "Select the next key for the selected curve."))
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.ArrowRight"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		// Add key button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, 1, 0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SNiagaraCurveKeySelector::AddKeyClicked)
			.ToolTipText(LOCTEXT("AddKeyToolTip", "Add a key to the selected curve."))
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		// Delete key button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SNiagaraCurveKeySelector::DeleteKeyClicked)
			.ToolTipText(LOCTEXT("DeleteKeyToolTip", "Delete the currently selected keys."))
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];
}

void SNiagaraCurveKeySelector::GetActiveCurveModelAndSelectedKeys(TOptional<FCurveModelID>& OutActiveCurveModelId, TArray<FKeyHandle>& OutSelectedKeyHandles)
{
	const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& CurveTreeSelection = CurveEditor->GetTree()->GetSelection();
	if (CurveTreeSelection.Num() != 0)
	{
		// If there are curves selected in the tree use those first.
		FCurveEditorTreeItemID FirstSelectedCurveTreeItemId;
		for (int32 i = OrderedCurveTreeItemIds.Num() - 1; i >= 0; i--)
		{
			const FCurveEditorTreeItemID& CurveTreeItemId = OrderedCurveTreeItemIds[i];
			if (CurveTreeSelection.Contains(CurveTreeItemId))
			{
				FirstSelectedCurveTreeItemId = CurveTreeItemId;
				break;
			}
		}

		TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(FirstSelectedCurveTreeItemId).GetOrCreateCurves(CurveEditor.Get());
		if (CurveModelIds.Num() == 1)
		{
			OutActiveCurveModelId = CurveModelIds[0];

			const FKeyHandleSet* SelectedKeyHandleSet = CurveEditor->GetSelection().GetAll().Find(OutActiveCurveModelId.GetValue());
			if (SelectedKeyHandleSet != nullptr)
			{
				OutSelectedKeyHandles = SelectedKeyHandleSet->AsArray();
			}
		}
	}
	else
	{
		// Otherwise check if there are keys selected and if so use first curve with selected keys.
		FCurveEditorSelection& CurveEditorSelection = CurveEditor->GetSelection();
		if (CurveEditorSelection.IsEmpty() == false)
		{
			for (int32 i = OrderedCurveTreeItemIds.Num() - 1; i >= 0; i--)
			{
				const FCurveEditorTreeItemID& CurveTreeItemId = OrderedCurveTreeItemIds[i];
				TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(CurveTreeItemId).GetCurves();
				if (CurveModelIds.Num() == 1)
				{
					const FKeyHandleSet* SelectedKeyHandleSet = CurveEditorSelection.GetAll().Find(CurveModelIds[0]);
					if (SelectedKeyHandleSet != nullptr)
					{
						OutActiveCurveModelId = CurveModelIds[0];
						OutSelectedKeyHandles = SelectedKeyHandleSet->AsArray();
						break;
					}
				}
			}
		}
		else
		{
			// Otherwise just use the first pinned curve.
			const TSet<FCurveModelID>& PinnedCurveIds = CurveEditor->GetPinnedCurves();
			if (PinnedCurveIds.Num() > 0)
			{
				for (FCurveEditorTreeItemID& CurveTreeItemId : OrderedCurveTreeItemIds)
				{
					TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(CurveTreeItemId).GetOrCreateCurves(CurveEditor.Get());
					if (CurveModelIds.Num() == 1 && PinnedCurveIds.Contains(CurveModelIds[0]))
					{
						OutActiveCurveModelId = CurveModelIds[0];
						break;
					}
				}
			}
		}
	}
}

void SNiagaraCurveKeySelector::GetSortedKeyHandlessAndPositionsForModel(FCurveModel& InCurveModel, TArray<FKeyHandlePositionPair>& OutSortedKeyHandlesAndPositions)
{
	const TArray<FKeyHandle> KeyHandles = InCurveModel.GetAllKeys();

	TArray<FKeyPosition> KeyPositions;
	KeyPositions.AddDefaulted(KeyHandles.Num());
	InCurveModel.GetKeyPositions(KeyHandles, KeyPositions);

	for (int32 i = 0; i < KeyHandles.Num(); i++)
	{
		FKeyHandlePositionPair& KeyHandlePositionPair = OutSortedKeyHandlesAndPositions.AddDefaulted_GetRef();
		KeyHandlePositionPair.Handle = KeyHandles[i];
		KeyHandlePositionPair.Position = KeyPositions[i];
	}

	OutSortedKeyHandlesAndPositions.Sort([](const FKeyHandlePositionPair& A, const FKeyHandlePositionPair& B) { return A.Position.InputValue < B.Position.InputValue; });
}

void SNiagaraCurveKeySelector::GetOrderedActiveCurveModelIds(TArray<FCurveModelID>& OutOrderedActiveCurveModelIds)
{
	if (CurveEditor->GetTreeSelection().Num() > 0)
	{
		// If there are curves selected in the tree then those are the only active ones.
		for (const FCurveEditorTreeItemID& OrderedCurveTreeItemId : OrderedCurveTreeItemIds)
		{
			if(CurveEditor->GetTreeSelectionState(OrderedCurveTreeItemId) != ECurveEditorTreeSelectionState::None)
			{
				TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(OrderedCurveTreeItemId).GetOrCreateCurves(CurveEditor.Get());
				if (CurveModelIds.Num() == 1)
				{
					OutOrderedActiveCurveModelIds.Add(CurveModelIds[0]);
				}
			}
		}
	}
	else
	{
		// Otherwise the active curves are the pinned curves.
		for (const FCurveEditorTreeItemID& OrderedCurveTreeItemId : OrderedCurveTreeItemIds)
		{
			TArrayView<const FCurveModelID> CurveModelIds = CurveEditor->GetTreeItem(OrderedCurveTreeItemId).GetCurves();
			if (CurveModelIds.Num() == 1 && CurveEditor->IsCurvePinned(CurveModelIds[0]))
			{
				OutOrderedActiveCurveModelIds.Add(CurveModelIds[0]);
			}
		}
	}
}

void SNiagaraCurveKeySelector::NavigateToAdjacentCurve(ENavigateDirection Direction)
{
	if (CurveEditorTree.IsValid())
	{
		if (CurveEditor->GetTreeSelection().Num() == 0)
		{
			CurveEditorTree->SetItemSelection(OrderedCurveTreeItemIds[0], true);
		}
		else
		{
			int32 TargetSelectedTreeItemIndex = INDEX_NONE;
			int32 StartIndex = Direction == ENavigateDirection::Previous ? 0 : OrderedCurveTreeItemIds.Num() - 1;
			int32 EndIndex = Direction == ENavigateDirection::Previous ? OrderedCurveTreeItemIds.Num() : -1;
			int32 IndexOffset = Direction == ENavigateDirection::Previous ? 1 : -1;
			for (int32 i = StartIndex; i != EndIndex; i += IndexOffset)
			{
				if (CurveEditor->GetTreeSelectionState(OrderedCurveTreeItemIds[i]) != ECurveEditorTreeSelectionState::None)
				{
					TargetSelectedTreeItemIndex = i;
					break;
				}
			}

			if (TargetSelectedTreeItemIndex != INDEX_NONE)
			{
				int32 IndexToSelect = Direction == ENavigateDirection::Previous 
					? TargetSelectedTreeItemIndex - 1
					: TargetSelectedTreeItemIndex + 1;

				if (IndexToSelect > OrderedCurveTreeItemIds.Num() - 1)
				{
					IndexToSelect = 0;
				}
				else if(IndexToSelect < 0)
				{
					IndexToSelect = OrderedCurveTreeItemIds.Num() - 1;
				}

				CurveEditorTree->ClearSelection();
				CurveEditorTree->SetItemSelection(OrderedCurveTreeItemIds[IndexToSelect], true);
			}
		}
	}
}

void SNiagaraCurveKeySelector::NavigateToAdjacentKey(ENavigateDirection Direction)
{
	TOptional<FCurveModelID> ActiveCurveModelId;
	TArray<FKeyHandle> SelectedKeyHandles;
	GetActiveCurveModelAndSelectedKeys(ActiveCurveModelId, SelectedKeyHandles);

	TOptional<FCurveModelID> CurveModelIdToSelect;
	TOptional<FKeyHandle> KeyHandleToSelect;
	if (ActiveCurveModelId.IsSet())
	{
		FCurveModel* ActiveCurveModel = CurveEditor->GetCurves()[ActiveCurveModelId.GetValue()].Get();
		TArray<FKeyHandlePositionPair> ActiveSortedKeyHandlePositionPairs;
		GetSortedKeyHandlessAndPositionsForModel(*ActiveCurveModel, ActiveSortedKeyHandlePositionPairs);

		if (SelectedKeyHandles.Num() != 0)
		{
			// If there's currently a selected key on the active curve then we want to use that as the target for navigating.
			int32 TargetSelectedKeyIndex = INDEX_NONE;
			int32 StartIndex = Direction == ENavigateDirection::Previous ? 0 : ActiveSortedKeyHandlePositionPairs.Num() - 1;
			int32 EndIndex = Direction == ENavigateDirection::Previous ? ActiveSortedKeyHandlePositionPairs.Num() : -1;
			int32 IndexOffset = Direction == ENavigateDirection::Previous ? 1 : -1;
			for (int32 i = StartIndex; i != EndIndex; i += IndexOffset)
			{
				if (SelectedKeyHandles.Contains(ActiveSortedKeyHandlePositionPairs[i].Handle))
				{
					TargetSelectedKeyIndex = i;
					break;
				}
			}

			if (TargetSelectedKeyIndex != INDEX_NONE)
			{
				if(Direction == ENavigateDirection::Previous && TargetSelectedKeyIndex > 0)
				{
					// If we're navigating previous and we're not at the first key we can just select the previous key on this curve.
					CurveModelIdToSelect = ActiveCurveModelId;
					KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs[TargetSelectedKeyIndex - 1].Handle;
				}
				else if (Direction == ENavigateDirection::Next && TargetSelectedKeyIndex < ActiveSortedKeyHandlePositionPairs.Num() - 1)
				{
					// If we're navigating next and we're not at the last key we can just select the next key on this curve.
					CurveModelIdToSelect = ActiveCurveModelId;
					KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs[TargetSelectedKeyIndex + 1].Handle;
				}
				else
				{
					// Otherwise we're going to need to navigate to another curve since we're at the start or end of the current curve.
					TArray<FCurveModelID> OrderedActiveCurveModelIds;
					GetOrderedActiveCurveModelIds(OrderedActiveCurveModelIds);

					// Find the adjacent curve with keys so we can select a key there.
					FCurveModel* CurveModelToSelect = nullptr;
					int32 CurrentIndex = OrderedActiveCurveModelIds.IndexOfByKey(ActiveCurveModelId);
					if (OrderedActiveCurveModelIds.Num() > 0)
					{
						int32 CurrentIndexOffset = Direction == ENavigateDirection::Previous ? -1 : 1;
						while (CurveModelIdToSelect.IsSet() == false)
						{
							CurrentIndex += CurrentIndexOffset;
							if (CurrentIndex < 0)
							{
								CurrentIndex = OrderedActiveCurveModelIds.Num() - 1;
							}
							else if (CurrentIndex > OrderedActiveCurveModelIds.Num() - 1)
							{
								CurrentIndex = 0;
							}

							FCurveModel* CurrentCurveModel = CurveEditor->GetCurves()[OrderedActiveCurveModelIds[CurrentIndex]].Get();
							if (CurrentCurveModel->GetNumKeys() > 0)
							{
								CurveModelIdToSelect = OrderedActiveCurveModelIds[CurrentIndex];
								CurveModelToSelect = CurrentCurveModel;
							}
						}
					}

					if (CurveModelIdToSelect == ActiveCurveModelId)
					{
						// There were no other active curves with keys to select from so just wrap around the currently active curve.
						if (Direction == ENavigateDirection::Previous)
						{
							KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs.Last().Handle;
						}
						else
						{
							KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs[0].Handle;
						}
					}
					else if(CurveModelToSelect != nullptr)
					{
						// We're selecting a key on a different curve so we need to sort the positions and select the first or last based
						// on the navigation direction.
						TArray<FKeyHandlePositionPair> SortedKeyHandlePositionPairs;
						GetSortedKeyHandlessAndPositionsForModel(*CurveModelToSelect, SortedKeyHandlePositionPairs);
						if (Direction == ENavigateDirection::Previous)
						{
							KeyHandleToSelect = SortedKeyHandlePositionPairs.Last().Handle;
						}
						else
						{
							KeyHandleToSelect = SortedKeyHandlePositionPairs[0].Handle;
						}
					}
				}
			}
		}
		else if(ActiveSortedKeyHandlePositionPairs.Num() > 0)
		{
			// There weren't any keys already selected so just select the first key on the active curve.
			CurveModelIdToSelect = ActiveCurveModelId;
			KeyHandleToSelect = ActiveSortedKeyHandlePositionPairs[0].Handle;
		}
	}

	if (CurveModelIdToSelect.IsSet() && KeyHandleToSelect.IsSet())
	{
		FCurveEditorSelection& CurveEditorSelection = CurveEditor->GetSelection();
		CurveEditorSelection.Clear();
		CurveEditorSelection.Add(CurveModelIdToSelect.GetValue(), ECurvePointType::Key, KeyHandleToSelect.GetValue());

		TArray<FCurveModelID> CurvesToFit;
		CurvesToFit.Add(CurveModelIdToSelect.GetValue());
		CurveEditor->ZoomToFitCurves(CurvesToFit);
	}
}

FReply SNiagaraCurveKeySelector::ZoomToFitClicked()
{
	TArray<FCurveModelID> ActiveCurveModelIds;
	GetOrderedActiveCurveModelIds(ActiveCurveModelIds);
	CurveEditor->ZoomToFitCurves(ActiveCurveModelIds);
	return FReply::Handled();
}

FReply SNiagaraCurveKeySelector::PreviousCurveClicked()
{
	NavigateToAdjacentCurve(ENavigateDirection::Previous);
	return FReply::Handled();
}

FReply SNiagaraCurveKeySelector::NextCurveClicked()
{
	NavigateToAdjacentCurve(ENavigateDirection::Next);
	return FReply::Handled();
}

FReply SNiagaraCurveKeySelector::PreviousKeyClicked()
{
	NavigateToAdjacentKey(ENavigateDirection::Previous);
	return FReply::Handled();
}

FReply SNiagaraCurveKeySelector::NextKeyClicked()
{
	NavigateToAdjacentKey(ENavigateDirection::Next);
	return FReply::Handled();
}

EVisibility SNiagaraCurveKeySelector::GetNextPreviousCurveButtonVisibility() const
{
	return OrderedCurveTreeItemIds.Num() == 1 ? EVisibility::Collapsed : EVisibility::Visible;
}

FReply SNiagaraCurveKeySelector::AddKeyClicked()
{
	TOptional<FCurveModelID> CurveModelIdForAdd;
	TArray<FKeyHandle> SelectedKeyHandles;
	GetActiveCurveModelAndSelectedKeys(CurveModelIdForAdd, SelectedKeyHandles);

	if(CurveModelIdForAdd.IsSet())
	{
		FCurveModel* CurveModelForAdd = CurveEditor->GetCurves()[CurveModelIdForAdd.GetValue()].Get();

		FKeyPosition NewKeyPosition;
		FKeyAttributes NewKeyAttributes = CurveEditor->GetDefaultKeyAttributes().Get();
		NewKeyAttributes.SetInterpMode(RCIM_Cubic);
		NewKeyAttributes.SetTangentMode(RCTM_Auto);
		if (CurveModelForAdd->GetNumKeys() == 0)
		{
			// If there are no keys, add one at 0, 0.
			NewKeyPosition.InputValue = 0.0f;
			NewKeyPosition.OutputValue = 0.0f;
		}
		else if (CurveModelForAdd->GetNumKeys() == 1)
		{
			// If there's a single key, add the new key at the same value, but time + 1.
			const TArray<FKeyHandle> KeyHandles = CurveModelForAdd->GetAllKeys();
				
			TArray<FKeyPosition> KeyPositions;
			KeyPositions.AddDefaulted();
			CurveModelForAdd->GetKeyPositions(KeyHandles, KeyPositions);
						
			NewKeyPosition.InputValue = KeyPositions[0].InputValue + 1;
			NewKeyPosition.OutputValue = KeyPositions[0].OutputValue;
		}
		else
		{
			TArray<FKeyHandlePositionPair> SortedKeyHandlePositionPairs;
			GetSortedKeyHandlessAndPositionsForModel(*CurveModelForAdd, SortedKeyHandlePositionPairs);

			int32 IndexToAddAfter = INDEX_NONE;
			if (SelectedKeyHandles.Num() > 0)
			{
				for (int32 i = SortedKeyHandlePositionPairs.Num() - 1; i >= 0; i--)
				{
					if (SelectedKeyHandles.Contains(SortedKeyHandlePositionPairs[i].Handle))
					{
						IndexToAddAfter = i;
						break;
					}
				}
			}

			if(IndexToAddAfter == INDEX_NONE)
			{
				IndexToAddAfter = SortedKeyHandlePositionPairs.Num() - 1;
			}

			if (IndexToAddAfter == SortedKeyHandlePositionPairs.Num() - 1)
			{
				const FKeyPosition& TargetKeyPosition = SortedKeyHandlePositionPairs[IndexToAddAfter].Position;
				const FKeyPosition& PreviousKeyPosition = SortedKeyHandlePositionPairs[IndexToAddAfter - 1].Position;
				NewKeyPosition.InputValue = TargetKeyPosition.InputValue + (TargetKeyPosition.InputValue - PreviousKeyPosition.InputValue);
				NewKeyPosition.OutputValue = TargetKeyPosition.OutputValue + (TargetKeyPosition.OutputValue - PreviousKeyPosition.OutputValue);
			}
			else
			{
				const FKeyPosition& TargetKeyPosition = SortedKeyHandlePositionPairs[IndexToAddAfter].Position;
				const FKeyPosition& NextKeyPosition = SortedKeyHandlePositionPairs[IndexToAddAfter + 1].Position;
				NewKeyPosition.InputValue = TargetKeyPosition.InputValue + ((NextKeyPosition.InputValue - TargetKeyPosition.InputValue) / 2);
				NewKeyPosition.OutputValue = TargetKeyPosition.OutputValue + ((NextKeyPosition.OutputValue - TargetKeyPosition.OutputValue) / 2);
			}
		}

		FScopedTransaction Transaction(LOCTEXT("AddKey", "Add Key"));
		TOptional<FKeyHandle> NewKeyHandle = CurveModelForAdd->AddKey(NewKeyPosition, NewKeyAttributes);
		if(NewKeyHandle.IsSet())
		{
			FCurveEditorSelection& CurveEditorSelection = CurveEditor->GetSelection();
			CurveEditorSelection.Clear();
			CurveEditorSelection.Add(CurveModelIdForAdd.GetValue(), ECurvePointType::Key, NewKeyHandle.GetValue());

			TArray<FCurveModelID> CurvesToFit;
			CurvesToFit.Add(CurveModelIdForAdd.GetValue());
			CurveEditor->ZoomToFitCurves(CurvesToFit);
		}
	}
	return FReply::Handled();
}

FReply SNiagaraCurveKeySelector::DeleteKeyClicked()
{
	CurveEditor->DeleteSelection();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
