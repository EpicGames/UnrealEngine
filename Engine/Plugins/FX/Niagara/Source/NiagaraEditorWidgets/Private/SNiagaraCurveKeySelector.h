// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveEditorTypes.h"
#include "Curves/KeyHandle.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "CurveDataAbstraction.h"

class FCurveEditor;
struct FCurveEditorTreeItemID;
class SCurveEditorTree;
class FCurveModel;

class SNiagaraCurveKeySelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNiagaraCurveKeySelector) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FCurveEditor> InCurveEditor, const TArray<FCurveEditorTreeItemID>& InCurveTreeItemIds, TSharedPtr<SCurveEditorTree> InCurveEditorTree);

private:
	void GetActiveCurveModelAndSelectedKeys(TOptional<FCurveModelID>& OutActiveCurveModelId, TArray<FKeyHandle>& OutSelectedKeyHandles);

	struct FKeyHandlePositionPair
	{
		FKeyHandle Handle;
		FKeyPosition Position;
	};

	void GetSortedKeyHandlessAndPositionsForModel(FCurveModel& InCurveModel, TArray<FKeyHandlePositionPair>& OutSortedKeyHandlesAndPositions);
	void GetOrderedActiveCurveModelIds(TArray<FCurveModelID>& OutOrderedActiveCurveModelIds);

	enum class ENavigateDirection
	{
		Previous,
		Next
	};

	void NavigateToAdjacentCurve(ENavigateDirection Direction);
	void NavigateToAdjacentKey(ENavigateDirection Direction);

	FReply ZoomToFitClicked();
	FReply PreviousCurveClicked();
	FReply NextCurveClicked();
	FReply PreviousKeyClicked();
	FReply NextKeyClicked();

	EVisibility GetNextPreviousCurveButtonVisibility() const;

	FReply AddKeyClicked();
	FReply DeleteKeyClicked();

private:
	TSharedPtr<FCurveEditor> CurveEditor;
	TArray<FCurveEditorTreeItemID> OrderedCurveTreeItemIds;
	TSharedPtr<SCurveEditorTree> CurveEditorTree;
};
